// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright (C) Microsoft. All rights reserved.

/*=============================================================================
GPUSkinCache.cpp: Performs skinning on a compute shader into a buffer to avoid vertex buffer skinning.
=============================================================================*/

#include "GPUSkinCache.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/RenderCommandPipes.h"
#include "SkeletalRenderGPUSkin.h"
#include "MeshDrawShaderBindings.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "RenderCaptureInterface.h"
#include "Engine/SkinnedAssetCommon.h"
#include "GPUSkinCacheVisualizationData.h"
#include "RHIContext.h"
#include "ShaderPlatformCachedIniValue.h"
#include "RenderUtils.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "RHIResourceUtils.h"
#include "Stats/StatsTrace.h"
#include "UObject/UObjectIterator.h"
#include "Algo/Sort.h"
#include "ComponentRecreateRenderStateContext.h"
#include "RenderGraphUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"
#include "SpanAllocator.h"
#include "UnifiedBuffer.h"

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Dispatches"), STAT_GPUSkinCache_NumDispatches, STATGROUP_GPUSkinCache);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Buffers"), STAT_GPUSkinCache_NumBuffers, STATGROUP_GPUSkinCache);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num RayTracing Dispatches"), STAT_GPUSkinCache_NumRayTracingDispatches, STATGROUP_GPUSkinCache);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num RayTracing Buffers"), STAT_GPUSkinCache_NumRayTracingBuffers, STATGROUP_GPUSkinCache);
DEFINE_STAT(STAT_GPUSkinCache_TotalNumChunks);
DEFINE_STAT(STAT_GPUSkinCache_TotalNumVertices);
DEFINE_STAT(STAT_GPUSkinCache_TotalMemUsed);
DEFINE_STAT(STAT_GPUSkinCache_TangentsIntermediateMemUsed);
DEFINE_STAT(STAT_GPUSkinCache_NumTrianglesForRecomputeTangents);
DEFINE_STAT(STAT_GPUSkinCache_NumSectionsProcessed);
DEFINE_LOG_CATEGORY_STATIC(LogSkinCache, Log, All);

/** Exec helper to handle GPU Skin Cache related commands. */
class FSkinCacheExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		/** Command to list all skeletal mesh lods which have the skin cache disabled. */
		if (FParse::Command(&Cmd, TEXT("list skincacheusage")))
		{
			UE_LOGF(LogTemp, Display, "Name, Lod Index, Skin Cache Usage");

			for (TObjectIterator<USkeletalMesh> It; It; ++It)
			{
				if (USkeletalMesh* SkeletalMesh = *It)
				{
					for (int32 LODIndex = 0; LODIndex < SkeletalMesh->GetLODNum(); ++LODIndex)
					{
						if (FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex))
						{
							UE_LOGF(LogTemp, Display, "%ls, %d, %d", *SkeletalMesh->GetFullName(), LODIndex, int(LODInfo->SkinCacheUsage));
						}
					}
				}
			}
			return true;
		}
		return false;
	}
};
static FSkinCacheExecHelper GSkelMeshExecHelper;

static TAutoConsoleVariable<bool> CVarAllowGPUSkinCache(
	TEXT("r.SkinCache.Allow"),
	true,
	TEXT("Whether or not to allow the GPU skin Cache system to be enabled.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static TAutoConsoleVariable<bool> CVarMobileAllowGPUSkinCache(
	TEXT("r.Mobile.SkinCache.Allow"),
	true,
	TEXT("Whether or not to allow the GPU skin Cache system to be enabled on the mobile renderer.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarEnableGPUSkinCacheShaders(
	TEXT("r.SkinCache.CompileShaders"),
	0,
	TEXT("Select which skinning paths to compile shaders for.\n")
	TEXT("Inline shaders apply skinning in the vertex shader.\n")
	TEXT("Skin cache shaders are used for skinning in compute jobs.\n")
	TEXT("0 (default) compiles inline only, 1 compiles both paths, 2 compiles only skin cache shaders."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 GEnableGPUSkinCache = 1;
static FAutoConsoleVariableRef CVarEnableGPUSkinCache(
	TEXT("r.SkinCache.Mode"),
	GEnableGPUSkinCache,
	TEXT("Whether or not to use the GPU compute skinning cache.\n")
	TEXT("This will perform skinning on a compute job and not skin on the vertex shader.\n")
	TEXT("Enabling requires r.SkinCache.CompileShaders!=0 and r.SkinCache.Allow=1\n")
	TEXT("Disabling requires r.SkinCache.CompileShaders!=2\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on(default)\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) { FGPUSkinCache::OnCVarUpdate(); }),
	ECVF_RenderThreadSafe
);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

static TAutoConsoleVariable<int32> CVarDefaultGPUSkinCacheBehavior(
	TEXT("r.SkinCache.DefaultBehavior"),
	(int32)ESkinCacheDefaultBehavior::Inclusive,
	TEXT("Default behavior if all skeletal meshes are included/excluded from the skin cache. If Support Ray Tracing is enabled on a mesh, will force inclusive behavior on that mesh.\n")
	TEXT(" Exclusive ( 0): All skeletal meshes are excluded from the skin cache. Each must opt in individually.\n")
	TEXT(" Inclusive ( 1): All skeletal meshes are included into the skin cache. Each must opt out individually. (default)"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) { FGPUSkinCache::OnCVarUpdate(); }),
	ECVF_RenderThreadSafe
);

int32 GSkinCacheRecomputeTangents = 2;
static FAutoConsoleVariableRef CVarGPUSkinCacheRecomputeTangents(
	TEXT("r.SkinCache.RecomputeTangents"),
	GSkinCacheRecomputeTangents,
	TEXT("This option enables recomputing the vertex tangents on the GPU.\n")
	TEXT("Can be changed at runtime, requires both r.SkinCache.CompileShaders!=0, r.SkinCache.Mode=1, r.SkinCache.Allow=1 and r.SkinCache.StoreDuplicatedVerticesForRecomputeTangents\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on, forces all skinned object to Recompute Tangents\n")
	TEXT(" 2: on, only recompute tangents on skinned objects who ticked the Recompute Tangents checkbox(default)\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) { FGPUSkinCache::OnCVarUpdate(); }),
	ECVF_RenderThreadSafe
);

static int32 GNumTangentIntermediateBuffers = 1;
static FAutoConsoleVariableRef CVarGPUSkinNumTangentIntermediateBuffers(
	TEXT("r.SkinCache.NumTangentIntermediateBuffers"),
	GNumTangentIntermediateBuffers,
	TEXT("How many intermediate buffers to use for intermediate results while\n")
	TEXT("doing Recompute Tangents; more may allow the GPU to overlap compute jobs."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) { FGPUSkinCache::OnCVarUpdate(); }),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarGPUSkinCacheDebug(
	TEXT("r.SkinCache.Debug"),
	1.0f,
	TEXT("A scaling constant passed to the SkinCache shader, useful for debugging"),
	ECVF_RenderThreadSafe
);

static int32 GStoreDuplicatedVerticesForRecomputeTangents = 0;
static FAutoConsoleVariableRef CVarGPUSkinCacheStoreDuplicatedVerticesForRecomputeTangents(
	TEXT("r.SkinCache.StoreDuplicatedVerticesForRecomputeTangents"),
	GStoreDuplicatedVerticesForRecomputeTangents,
	TEXT("0: Don't store duplicated vertices for all skeletal mesh render sections. It will still be stored if the render section has bRecomputeTangent set. (default)\n")
	TEXT("1: Store duplicated vertices for all skeletal mesh render sections.\n"),
	ECVF_ReadOnly
);

static int32 GUseDuplicatedVerticesForRecomputeTangents = 1;
static FAutoConsoleVariableRef CVarGPUSkinCacheAllowDupedVertesForRecomputeTangents(
	TEXT("r.SkinCache.UseDuplicatedVerticesForRecomputeTangents"),
	GUseDuplicatedVerticesForRecomputeTangents,
	TEXT("0: Disable usage of duplicated vertices for runtime tangent recomputation/\n")
	TEXT("1: Use stored duplicated vertices if they are available (default).\n"),
	ECVF_RenderThreadSafe
);

int32 GRecomputeTangentsParallelDispatch = 0;
static FAutoConsoleVariableRef CVarRecomputeTangentsParallelDispatch(
	TEXT("r.SkinCache.RecomputeTangentsParallelDispatch"),
	GRecomputeTangentsParallelDispatch,
	TEXT("This option enables parallel dispatches for recompute tangents.\n")
	TEXT(" 0: off (default), triangle pass is interleaved with vertex pass, requires resource barriers in between. \n")
	TEXT(" 1: on, batch triangle passes together, resource barrier, followed by vertex passes together, cost more memory. \n"),
	ECVF_RenderThreadSafe
);

static int32 GSkinCachePrintMemorySummary = 0;
static FAutoConsoleVariableRef CVarGPUSkinCachePrintMemorySummary(
	TEXT("r.SkinCache.PrintMemorySummary"),
	GSkinCachePrintMemorySummary,
	TEXT("Print break down of memory usage.")
	TEXT(" 0: off (default),")
	TEXT(" 1: print for N frames"),
	ECVF_RenderThreadSafe
);

static bool GSkinCacheAsyncCompute = false;
static FAutoConsoleVariableRef CVarSkinCacheAsyncCompute(
	TEXT("r.SkinCache.AsyncCompute"),
	GSkinCacheAsyncCompute,
	TEXT(" 0: off\n")
	TEXT(" 1: on\n"),
	ECVF_RenderThreadSafe
);

static int32 GSkinCacheDynamicMeshBounds = 2;
static FAutoConsoleVariable CVarSkinCacheDynamicMeshBounds(
	TEXT("r.SkinCache.DynamicMeshBounds"),
	GSkinCacheDynamicMeshBounds,
	TEXT("If enabled, skin cache will generate tight mesh bounds that are used during instance culling.\n")
	TEXT(" 0 - disabled support compiled out of all shaders.\n")
	TEXT(" 1 - enabled for all platforms that support atomic operations.\n")
	TEXT(" 2 - auto (default), enabled for higher end platforms (currently those supporting Nanite).\n"),
	ECVF_ReadOnly
);

static int32 GNumDispatchesToCapture = 0;
static FAutoConsoleVariableRef CVarGPUSkinCacheNumDispatchesToCapture(
	TEXT("r.SkinCache.Capture"),
	GNumDispatchesToCapture,
	TEXT("Trigger a render capture for the next skin cache dispatches."));

static int32 GGPUSkinCacheFlushCounter = 0;

const float MBSize = 1048576.f; // 1024 x 1024 bytes

ESkinCacheDefaultBehavior GetSkinCacheDefaultBehavior()
{
	return ESkinCacheDefaultBehavior(CVarDefaultGPUSkinCacheBehavior->GetInt()) == ESkinCacheDefaultBehavior::Inclusive
		? ESkinCacheDefaultBehavior::Inclusive
		: ESkinCacheDefaultBehavior::Exclusive;
}

ENGINE_API bool GPUSkinCacheNeedsDuplicatedVertices()
{
#if WITH_EDITOR // Duplicated vertices are used in the editor when merging meshes
	return true;
#else
	return GPUSkinCacheStoreDuplicatedVertices();
#endif
}

ENGINE_API bool GPUSkinCacheStoreDuplicatedVertices()
{
	return GStoreDuplicatedVerticesForRecomputeTangents > 0;
}

RDG_REGISTER_BLACKBOARD_STRUCT(FGPUSkinCache::FTaskData);

enum class EGPUSkinCacheDispatchFlags : uint8
{
	None                 = 0,
	Position             = 1 << 0,
	PositionPrevious     = 1 << 1,
	RecomputeTangents    = 1 << 2
};
ENUM_CLASS_FLAGS(EGPUSkinCacheDispatchFlags);

enum class EGPUSkinCacheBufferBits : uint8
{
	None                 = 0,
	IntermediateTangents = 1 << 1,
	PositionPrevious     = 1 << 2,
};
ENUM_CLASS_FLAGS(EGPUSkinCacheBufferBits);

struct FGPUSkinCache::FSkinCacheRWBuffer
{
	FRWBuffer	Buffer;
	ERHIAccess	AccessState = ERHIAccess::Unknown;	// Keep track of current access state
	mutable uint32	LastTransitionFence = 0;

	void Release()
	{
		Buffer.Release();
		AccessState = ERHIAccess::Unknown;
	}

	bool UpdateFence(uint32 NextTransitionFence)
	{
		const bool bUpdateRequired = LastTransitionFence != NextTransitionFence;
		LastTransitionFence = NextTransitionFence;
		return bUpdateRequired;
	}

	// Update the access state and return transition info
	FRHITransitionInfo UpdateAccessState(ERHIAccess NewState)
	{
		ERHIAccess OldState = AccessState;
		AccessState = NewState;
		return FRHITransitionInfo(Buffer.UAV.GetReference(), OldState, AccessState);
	}
};

struct FGPUSkinCache::FRWBuffersAllocationInitializer
{
	static const uint32 PositionStride = 4;

	EGPUSkinCacheBufferBits BufferBits = EGPUSkinCacheBufferBits::None;
	uint32 NumVertices = 0;
	uint32 IntermediateAccumulatedTangentsSize = 0;
	EPixelFormat TangentFormat = PF_Unknown;

	static uint32 GetPositionStride()
	{
		return PositionStride;
	}

	uint32 GetTangentStride() const
	{
		return GPixelFormats[TangentFormat].BlockBytes;
	}

	uint32 GetBufferSize() const
	{
		const bool bIntermediateTangents = EnumHasAnyFlags(BufferBits, EGPUSkinCacheBufferBits::IntermediateTangents);
		const bool bPositionPrevious = EnumHasAnyFlags(BufferBits, EGPUSkinCacheBufferBits::PositionPrevious);

		const uint32 TangentStride = GetTangentStride();

		const uint32 PositionBufferSize = PositionStride * NumVertices * 3 * (bPositionPrevious ? NUM_BUFFERS : 1);
		const uint32 TangentBufferSize =  TangentStride * NumVertices * 2;
		const uint32 IntermediateTangentBufferSize = bIntermediateTangents ? TangentStride * NumVertices * 2 : 0;
		const uint32 AccumulatedTangentBufferSize = IntermediateAccumulatedTangentsSize * FGPUSkinCache::IntermediateAccumBufferNumInts * sizeof(int32);

		return TangentBufferSize + IntermediateTangentBufferSize + PositionBufferSize + AccumulatedTangentBufferSize;
	}
};

struct FGPUSkinCache::FRWBuffersAllocation
{
	friend FRWBufferTracker;

	FRWBuffersAllocation(FRHICommandList& RHICmdList, const FRWBuffersAllocationInitializer& InInitializer, const FName& OwnerName)
		: Initializer(InInitializer)
	{
		const static FLazyName PositionsName(TEXT("SkinCachePositions"));
		const static FLazyName TangentsName(TEXT("SkinCacheTangents"));
		const static FLazyName IntermediateTangentsName(TEXT("SkinCacheIntermediateTangents"));
		const static FLazyName IntermediateAccumulatedTangentsName(TEXT("SkinCacheIntermediateAccumulatedTangents"));

		const int32 NumBuffers = EnumHasAnyFlags(Initializer.BufferBits, EGPUSkinCacheBufferBits::PositionPrevious) ? NUM_BUFFERS : 1;
		
		const uint32 PositionStride = InInitializer.GetPositionStride();
		const uint32 TangentStride = InInitializer.GetTangentStride();

		for (int32 Index = 0; Index < NumBuffers; ++Index)
		{
			PositionBuffers[Index].Buffer.ClassName = PositionsName;
			PositionBuffers[Index].Buffer.OwnerName = OwnerName;
			PositionBuffers[Index].Buffer.Initialize(RHICmdList, TEXT("SkinCachePositions"), PositionStride, Initializer.NumVertices * 3, PF_R32_FLOAT, ERHIAccess::SRVMask, BUF_Static);
			PositionBuffers[Index].Buffer.Buffer->SetOwnerName(OwnerName);
			PositionBuffers[Index].AccessState = ERHIAccess::Unknown;
		}

		// Tangents are skinned inside the main skinning compute shader and are always allocated, even if the recompute tangents pass doesn't run.
		Tangents.Buffer.ClassName = TangentsName;
		Tangents.Buffer.OwnerName = OwnerName;
		Tangents.Buffer.Initialize(RHICmdList, TEXT("SkinCacheTangents"), TangentStride, Initializer.NumVertices * 2, Initializer.TangentFormat, BUF_Static);
		Tangents.Buffer.Buffer->SetOwnerName(OwnerName);
		Tangents.AccessState = ERHIAccess::Unknown;

		if (EnumHasAnyFlags(Initializer.BufferBits, EGPUSkinCacheBufferBits::IntermediateTangents))
		{
			IntermediateTangents.Buffer.ClassName = IntermediateTangentsName;
			IntermediateTangents.Buffer.OwnerName = OwnerName;
			IntermediateTangents.Buffer.Initialize(RHICmdList, TEXT("SkinCacheIntermediateTangents"), TangentStride, Initializer.NumVertices * 2, Initializer.TangentFormat, BUF_Static);
			IntermediateTangents.Buffer.Buffer->SetOwnerName(OwnerName);
			IntermediateTangents.AccessState = ERHIAccess::Unknown;
		}

		if (Initializer.IntermediateAccumulatedTangentsSize > 0)
		{
			IntermediateAccumulatedTangents.Buffer.ClassName = IntermediateAccumulatedTangentsName;
			IntermediateAccumulatedTangents.Buffer.OwnerName = OwnerName;
			IntermediateAccumulatedTangents.Buffer.Initialize(RHICmdList, TEXT("SkinCacheIntermediateAccumulatedTangents"), sizeof(int32), Initializer.IntermediateAccumulatedTangentsSize * FGPUSkinCache::IntermediateAccumBufferNumInts, PF_R32_SINT, BUF_UnorderedAccess);
			IntermediateAccumulatedTangents.Buffer.Buffer->SetOwnerName(OwnerName);
			IntermediateAccumulatedTangents.AccessState = ERHIAccess::Unknown;

			// The UAV must be zero-filled. We leave it zeroed after each round (see RecomputeTangentsPerVertexPass.usf), so this is only needed on when the buffer is first created.
			RHICmdList.ClearUAVUint(IntermediateAccumulatedTangents.Buffer.UAV, FUintVector4(0, 0, 0, 0));
		}
	}

	~FRWBuffersAllocation()
	{
		for (int32 Index = 0; Index < NUM_BUFFERS; ++Index)
		{
			PositionBuffers[Index].Release();
		}

		Tangents.Release();
		IntermediateTangents.Release();
		IntermediateAccumulatedTangents.Release();
	}

	uint64 GetBufferSize() const
	{
		return Initializer.GetBufferSize();
	}

	FSkinCacheRWBuffer* GetTangentBuffer()
	{
		return &Tangents;
	}

	FSkinCacheRWBuffer* GetIntermediateTangentBuffer()
	{
		return EnumHasAllFlags(Initializer.BufferBits, EGPUSkinCacheBufferBits::IntermediateTangents) ? &IntermediateTangents : nullptr;
	}

	FSkinCacheRWBuffer* GetIntermediateAccumulatedTangentBuffer()
	{
		return Initializer.IntermediateAccumulatedTangentsSize > 0 ? &IntermediateAccumulatedTangents : nullptr;
	}

	bool HasPreviousBuffer() const
	{
		return EnumHasAllFlags(Initializer.BufferBits, EGPUSkinCacheBufferBits::PositionPrevious);
	}

	FSkinCacheRWBuffer& GetPositionBuffer()
	{
		return PositionBuffers[0];
	}

private:
	// Output of the GPU skinning (ie Pos, Normals)
	FSkinCacheRWBuffer PositionBuffers[NUM_BUFFERS];

	FSkinCacheRWBuffer Tangents;
	FSkinCacheRWBuffer IntermediateTangents;
	FSkinCacheRWBuffer IntermediateAccumulatedTangents;	// Intermediate buffer used to accumulate results of triangle pass to be passed onto vertex pass

	const FRWBuffersAllocationInitializer Initializer;
};

struct FGPUSkinCache::FRWBufferTracker
{
	FRWBuffersAllocation* Allocation;

	FRWBufferTracker()
		: Allocation(nullptr)
	{
		Reset();
	}

	void Reset()
	{
		for (int32 Index = 0; Index < NUM_BUFFERS; ++Index)
		{
			Revisions[Index] = 0;
			BoneBuffers[Index] = nullptr;
		}
	}

	uint32 GetBufferSize() const
	{
		return Allocation->GetBufferSize();
	}

	FSkinCacheRWBuffer* Find(const FVertexBufferAndSRV& BoneBuffer, uint32 Revision)
	{
		for (int32 Index = 0; Index < NUM_BUFFERS; ++Index)
		{
			if (Revisions[Index] == Revision && BoneBuffers[Index] == &BoneBuffer)
			{
				return &Allocation->PositionBuffers[Index];
			}
		}

		return nullptr;
	}

	FSkinCacheRWBuffer* GetTangentBuffer()
	{
		return Allocation ? Allocation->GetTangentBuffer() : nullptr;
	}

	FSkinCacheRWBuffer* GetIntermediateTangentBuffer()
	{
		return Allocation ? Allocation->GetIntermediateTangentBuffer() : nullptr;
	}

	FSkinCacheRWBuffer* GetIntermediateAccumulatedTangentBuffer()
	{
		return Allocation ? Allocation->GetIntermediateAccumulatedTangentBuffer() : nullptr;
	}

	// Allocates an element that's not the "Used" element passed in (or if Used is NULL, allocates any element).
	FSkinCacheRWBuffer* AllocateUnused(const FVertexBufferAndSRV& BoneBuffer, uint32 Revision, const FSkinCacheRWBuffer* Used)
	{
		int32 UnusedIndex = Used == &Allocation->PositionBuffers[0] ? 1 : 0;
		Revisions[UnusedIndex] = Revision;
		BoneBuffers[UnusedIndex] = &BoneBuffer;

		return &Allocation->PositionBuffers[UnusedIndex];
	}

	// On recreate of the render state where the GPU skin cache entry is preserved, the bone buffer will have been reallocated,
	// even though the transforms didn't change.  We need to force the Find() call above to treat the data as up-to-date, which
	// can be accomplished by updating the BoneBuffer pointer for the previous Revision, so it matches again.
	void UpdatePreviousBoneBuffer(const FVertexBufferAndSRV& PreviousBoneBuffer, uint32 PreviousRevision)
	{
		for (int32 Index = 0; Index < NUM_BUFFERS; ++Index)
		{
			if (Revisions[Index] == PreviousRevision)
			{
				BoneBuffers[Index] = &PreviousBoneBuffer;
				break;
			}
		}
	}

private:
	uint32 Revisions[NUM_BUFFERS];
	const FVertexBufferAndSRV* BoneBuffers[NUM_BUFFERS];
};

enum class EGPUSkinBoneInfluenceType : uint8
{
	Default,
	Extra,
	Unlimited
};

enum class EGPUSkinDeformationType : uint8
{
	Default,
	Morph,
	Cloth
};

class FGPUSkinCacheEntry
{
public:
	FGPUSkinCacheEntry(FGPUSkinCache* InSkinCache, FSkeletalMeshObject* InGPUSkin, FGPUSkinCache::FRWBuffersAllocation* InPositionAllocation, int32 InLOD, EGPUSkinCacheEntryMode InMode)
		: Mode(InMode)
		, PositionAllocation(InPositionAllocation)
		, SkinCache(InSkinCache)
		, GPUSkin(InGPUSkin)
		, LOD(InLOD)
	{
		const TArray<FSkelMeshRenderSection>& Sections = InGPUSkin->GetRenderSections(LOD);
		DispatchData.AddDefaulted(Sections.Num());

		UpdateSkinWeightBuffer();
	}

	~FGPUSkinCacheEntry()
	{
		check(!PositionAllocation);
	}

	struct FRecomputeTangentSection
	{
		// The section participates in the recompute-tangents passes (per-triangle scatter + per-vertex resolve).
		uint32 bEnable                  : 1  = 0;
		// The section blends the recomputed tangent with the originally-skinned tangent using a vertex-color mask channel.
		// When set, the skinned tangent is preserved in a separate IntermediateTangents buffer so the vertex pass can lerp.
		// When clear (channel == None), the recompute pass overwrites the tangent buffer directly.
		uint32 bBlendUsingVertexColor   : 1  = 0;
		// Offset into the shared IntermediateAccumulatedTangents (atomic int) scratch buffer for this section's vertices.
		uint32 IntermediateBufferOffset : 30 = 0;
	};

	struct FSectionDispatchData
	{
		FGPUSkinCache::FRWBufferTracker PositionTracker;

		const FGPUBaseSkinVertexFactory* SourceVertexFactory = nullptr;
		const FSkelMeshRenderSection* Section = nullptr;

		uint32 SectionIndex = INDEX_NONE;

		EGPUSkinDeformationType DeformationType = EGPUSkinDeformationType::Default;
		EGPUSkinCacheDispatchFlags DispatchFlags = EGPUSkinCacheDispatchFlags::None;

		uint32 UpdatedFrameNumber = 0;
		uint32 NumBoneInfluences = 0;

		uint32 InputStreamStart = 0;
		uint32 InputWeightStart = 0;
		uint32 OutputStreamStart = 0;
		uint32 NumVertices = 0;
		uint32 NumTexCoords = 1;

		FShaderResourceViewRHIRef TangentBufferSRV = nullptr;
		FShaderResourceViewRHIRef UVsBufferSRV = nullptr;
		FShaderResourceViewRHIRef ColorBufferSRV = nullptr;
		FShaderResourceViewRHIRef PositionBufferSRV = nullptr;
		FShaderResourceViewRHIRef ClothPositionsAndNormalsBuffer = nullptr;

		uint32 MorphBufferOffset = 0;

		uint32 ClothBufferOffset = 0;
		float ClothBlendWeight = 0.0f;
		uint32 ClothNumInfluencesPerVertex = 1;
		FMatrix44f ClothToLocal = FMatrix44f::Identity;
		FVector3f ClothWorldScale = FVector3f::OneVector;

		uint32 RevisionNumber = 0;
		FGPUSkinCache::FSkinCacheRWBuffer* TangentBuffer = nullptr;
		FGPUSkinCache::FSkinCacheRWBuffer* PositionBuffer = nullptr;
		FGPUSkinCache::FSkinCacheRWBuffer* PreviousPositionBuffer = nullptr;

		struct
		{
			FRecomputeTangentSection Section;
			uint32 IndexBufferOffsetValue = 0;
			uint32 NumTriangles = 0;
			FRHIShaderResourceView* IndexBuffer = nullptr;
			FGPUSkinCache::FSkinCacheRWBuffer* IntermediateTangentBuffer = nullptr;
			FGPUSkinCache::FSkinCacheRWBuffer* IntermediateAccumulatedTangentBuffer = nullptr;
			FShaderResourceViewRHIRef DuplicatedIndicesIndices = nullptr;
			FShaderResourceViewRHIRef DuplicatedIndices = nullptr;
		
		} RecomputeTangents;

		int32 DynamicBoundsOffset = -1;

		FSectionDispatchData() = default;

		inline FGPUSkinCache::FSkinCacheRWBuffer* GetPreviousPositionRWBuffer() const
		{
			check(PreviousPositionBuffer);
			return PreviousPositionBuffer;
		}

		inline FGPUSkinCache::FSkinCacheRWBuffer* GetPositionRWBuffer() const
		{
			check(PositionBuffer);
			return PositionBuffer;
		}

		inline FGPUSkinCache::FSkinCacheRWBuffer* GetTangentRWBuffer() const
		{
			check(TangentBuffer);
			return TangentBuffer;
		}

		FGPUSkinCache::FSkinCacheRWBuffer* GetActiveTangentRWBuffer() const
		{
			// This is the buffer containing tangent results from the skinning CS pass
			return (RecomputeTangents.IndexBuffer && RecomputeTangents.IntermediateTangentBuffer) ? RecomputeTangents.IntermediateTangentBuffer : TangentBuffer;
		}

		inline FGPUSkinCache::FSkinCacheRWBuffer* GetIntermediateAccumulatedTangentBuffer() const
		{
			check(RecomputeTangents.IntermediateAccumulatedTangentBuffer);
			return RecomputeTangents.IntermediateAccumulatedTangentBuffer;
		}

		void UpdateVertexFactoryDeclaration(FRHICommandListBase& RHICmdList, FGPUSkinPassthroughVertexFactory* InTargetVertexFactory, EGPUSkinCacheEntryMode InMode)
		{
			FRHIShaderResourceView* CurrentPositionSRV = PositionBuffer->Buffer.SRV;
			FRHIShaderResourceView* PreviousPositionSRV = PreviousPositionBuffer ? PreviousPositionBuffer->Buffer.SRV.GetReference() : CurrentPositionSRV;

			FGPUSkinPassthroughVertexFactory::FAddVertexAttributeDesc Desc;
			Desc.FrameNumber = InMode == EGPUSkinCacheEntryMode::Raster ? SourceVertexFactory->GetShaderData().UpdatedFrameNumber : 0;
			Desc.StreamBuffers[FGPUSkinPassthroughVertexFactory::EVertexAttribute::VertexPosition] = PositionBuffer->Buffer.Buffer;
			Desc.StreamBuffers[FGPUSkinPassthroughVertexFactory::EVertexAttribute::VertexTangent] = TangentBuffer->Buffer.Buffer;
			Desc.SRVs[FGPUSkinPassthroughVertexFactory::EShaderResource::Position] = CurrentPositionSRV;
			Desc.SRVs[FGPUSkinPassthroughVertexFactory::EShaderResource::PreviousPosition] = PreviousPositionSRV;
			Desc.SRVs[FGPUSkinPassthroughVertexFactory::EShaderResource::Tangent] = TangentBuffer->Buffer.SRV;
			InTargetVertexFactory->SetVertexAttributes(RHICmdList, SourceVertexFactory, Desc);
		}
	};

	void UpdateVertexFactoryDeclaration(FRHICommandListBase& RHICmdList, int32 Section)
	{
		DispatchData[Section].UpdateVertexFactoryDeclaration(RHICmdList, TargetVertexFactory, Mode);
	}

	inline FCachedGeometry::Section GetCachedGeometry(int32 SectionIndex) const
	{
		FCachedGeometry::Section MeshSection;
		if (SectionIndex >= 0 && SectionIndex < DispatchData.Num())
		{
			const FSkelMeshRenderSection& Section = *DispatchData[SectionIndex].Section;
			MeshSection.PositionBuffer = DispatchData[SectionIndex].PositionBuffer->Buffer.SRV;
			MeshSection.PreviousPositionBuffer = DispatchData[SectionIndex].PreviousPositionBuffer->Buffer.SRV;
			MeshSection.UVsBuffer = DispatchData[SectionIndex].UVsBufferSRV;
			MeshSection.TangentBuffer = DispatchData[SectionIndex].TangentBufferSRV;
			MeshSection.TotalVertexCount = DispatchData[SectionIndex].PositionBuffer->Buffer.NumBytes / (sizeof(float) * 3);
			MeshSection.NumPrimitives = Section.NumTriangles;
			MeshSection.NumVertices = Section.NumVertices;
			MeshSection.IndexBaseIndex = Section.BaseIndex;
			MeshSection.VertexBaseIndex = Section.BaseVertexIndex;
			MeshSection.IndexBuffer = nullptr;
			MeshSection.TotalIndexCount = 0;
			MeshSection.LODIndex = 0;
			MeshSection.SectionIndex = SectionIndex;
		}
		return MeshSection;
	}

	bool IsSectionValid(int32 Section) const
	{
		const FSectionDispatchData& SectionData = DispatchData[Section];
		return SectionData.SectionIndex == Section;
	}

	bool IsTargetVertexFactoryValid(const FGPUSkinPassthroughVertexFactory* InTargetVertexFactory) const
	{
		return TargetVertexFactory == InTargetVertexFactory;
	}

	bool IsValid(FSkeletalMeshObject* InSkin, int32 InLOD) const
	{
		return GPUSkin == InSkin && LOD == InLOD;
	}

	void UpdateSkinWeightBuffer()
	{
		FSkinWeightVertexBufferInfo Info = GPUSkin->GetSkinWeightVertexBuffer_RenderThread(LOD);
		bUse16BitBoneIndex         = Info.bUse16BitBoneIndex;
		bUse16BitBoneWeight        = Info.bUse16BitBoneWeight;
		InputWeightIndexSize       = Info.InputWeightIndexSize;
		InputWeightStride          = Info.InputWeightStride;
		InputWeightStreamSRV       = Info.InputWeightStreamSRV;
		InputWeightLookupStreamSRV = Info.InputWeightLookupStreamSRV;

		if (Info.BoneInfluenceType == GPUSkinBoneInfluenceType::DefaultBoneInfluence)
		{
			BoneInfluenceType = Info.MaxBoneInfluences > MAX_INFLUENCES_PER_STREAM ? EGPUSkinBoneInfluenceType::Extra : EGPUSkinBoneInfluenceType::Default;
		}
		else
		{
			BoneInfluenceType = EGPUSkinBoneInfluenceType::Unlimited;
		}
	}

	void SetupSection(
		int32 SectionIndex,
		const FSkelMeshRenderSection* Section,
		const FGPUBaseSkinVertexFactory* InSourceVertexFactory,
		FRecomputeTangentSection InRecomputeTangentSection,
		int32 DynamicBoundsOffset)
	{
		FSectionDispatchData& Data = DispatchData[SectionIndex];
		check(!Data.PositionTracker.Allocation || Data.PositionTracker.Allocation == PositionAllocation);

		FSkeletalMeshRenderData& SkelMeshRenderData = GPUSkin->GetSkeletalMeshRenderData();
		FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData.LODRenderData[LOD];

		Data.DynamicBoundsOffset = DynamicBoundsOffset;
		Data.PositionTracker.Allocation = PositionAllocation;
		Data.SectionIndex = SectionIndex;
		Data.Section = Section;
		Data.NumVertices = Section->GetNumVertices();
		Data.InputStreamStart = Section->BaseVertexIndex;
		Data.OutputStreamStart = Section->BaseVertexIndex;
		Data.TangentBufferSRV = InSourceVertexFactory->GetTangentsSRV();
		Data.UVsBufferSRV = InSourceVertexFactory->GetTextureCoordinatesSRV();
		Data.ColorBufferSRV = InSourceVertexFactory->GetColorComponentsSRV();
		Data.NumTexCoords = InSourceVertexFactory->GetNumTexCoords();
		Data.PositionBufferSRV = InSourceVertexFactory->GetPositionsSRV();
		Data.DeformationType = EGPUSkinDeformationType::Default;
		Data.NumBoneInfluences = InSourceVertexFactory->GetNumBoneInfluences();
		Data.InputWeightStart = (InputWeightStride * Section->BaseVertexIndex) / sizeof(float);
		Data.SourceVertexFactory = InSourceVertexFactory;

		if (InRecomputeTangentSection.bEnable)
		{
			FRawStaticIndexBuffer16or32Interface* IndexBuffer = LodData.MultiSizeIndexContainer.GetIndexBuffer();
			Data.RecomputeTangents.IndexBuffer = IndexBuffer->GetSRV();
			Data.RecomputeTangents.NumTriangles = Section->NumTriangles;
			Data.RecomputeTangents.IndexBufferOffsetValue = Section->BaseIndex;
			Data.RecomputeTangents.Section = InRecomputeTangentSection;

			check(Data.RecomputeTangents.IndexBuffer);
		}

		check(Data.SectionIndex == LodData.FindSectionIndex(*Section));
		check(Data.TangentBufferSRV && Data.PositionBufferSRV);
	}

#if RHI_RAYTRACING
	void GetRayTracingSegmentVertexBuffers(TArray<FBufferRHIRef>& OutVertexBuffers) const
	{
		OutVertexBuffers.SetNum(DispatchData.Num());
		for (int32 SectionIdx = 0; SectionIdx < DispatchData.Num(); SectionIdx++)
		{
			FGPUSkinCache::FSkinCacheRWBuffer* PositionBuffer = DispatchData[SectionIdx].PositionBuffer;
			OutVertexBuffers[SectionIdx] = PositionBuffer ? PositionBuffer->Buffer.Buffer : nullptr;
		}
	}
#endif // RHI_RAYTRACING

	TArray<FSectionDispatchData>& GetDispatchData() { return DispatchData; }
	TArray<FSectionDispatchData> const& GetDispatchData() const { return DispatchData; }

protected:
	EGPUSkinCacheEntryMode Mode;
	FGPUSkinCache::FRWBuffersAllocation* PositionAllocation;
	FGPUSkinCache* SkinCache;
	TArray<FSectionDispatchData> DispatchData;
	FSkeletalMeshObject* GPUSkin;
	FGPUSkinPassthroughVertexFactory* TargetVertexFactory = nullptr;
	EGPUSkinBoneInfluenceType BoneInfluenceType = EGPUSkinBoneInfluenceType::Default;
	bool bUse16BitBoneIndex;
	bool bUse16BitBoneWeight;
	bool bQueuedForDispatch = false;
	uint32 InputWeightIndexSize;
	uint32 InputWeightStride;
	FShaderResourceViewRHIRef InputWeightStreamSRV;
	FShaderResourceViewRHIRef InputWeightLookupStreamSRV;
	FRHIShaderResourceView* MorphBuffer = nullptr;
	FShaderResourceViewRHIRef ClothBuffer = nullptr;
	int32 LOD;

	friend class FGPUSkinCache;
	friend class FBaseRecomputeTangentsPerTriangleShader;
};

//////////////////////////////////////////////////////////////////////////

class FGPUSkinCacheCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGPUSkinCacheCS);
	SHADER_USE_PARAMETER_STRUCT(FGPUSkinCacheCS, FGlobalShader);

	class FBoneWeight16Dim            : SHADER_PERMUTATION_BOOL("GPUSKIN_BONE_WEIGHTS_UINT16");
	class FBoneIndex16Dim             : SHADER_PERMUTATION_BOOL("GPUSKIN_BONE_INDEX_UINT16");
	class FApexClothDim               : SHADER_PERMUTATION_BOOL("GPUSKIN_APEX_CLOTH");
	class FMorphBlendDim              : SHADER_PERMUTATION_BOOL("GPUSKIN_MORPH_BLEND");
	class FUnlimitedBoneInfluencesDim : SHADER_PERMUTATION_BOOL("GPUSKIN_UNLIMITED_BONE_INFLUENCE");
	class FExtraBoneInfluencesDim      : SHADER_PERMUTATION_BOOL("GPUSKIN_USE_EXTRA_INFLUENCES");

	using FPermutationDomain = TShaderPermutationDomain<FBoneWeight16Dim, FBoneIndex16Dim, FApexClothDim, FMorphBlendDim, FUnlimitedBoneInfluencesDim, FExtraBoneInfluencesDim>;

	static FPermutationDomain BuildPermutationVector(bool bUse16BitBoneWeight, bool bUse16BitBoneIndex, EGPUSkinBoneInfluenceType BoneInfluenceType, EGPUSkinDeformationType DeformationType)
	{
		FPermutationDomain PermutationVector;

		if (BoneInfluenceType == EGPUSkinBoneInfluenceType::Unlimited)
		{
			PermutationVector.Set<FUnlimitedBoneInfluencesDim>(true);
		}
		else
		{
			if (BoneInfluenceType == EGPUSkinBoneInfluenceType::Extra)
			{
				PermutationVector.Set<FExtraBoneInfluencesDim>(true);
			}

			PermutationVector.Set<FBoneWeight16Dim>(bUse16BitBoneWeight);
			PermutationVector.Set<FBoneIndex16Dim>(bUse16BitBoneIndex);
		}

		if (DeformationType == EGPUSkinDeformationType::Cloth)
		{
			PermutationVector.Set<FApexClothDim>(true);
		}
		else if (DeformationType == EGPUSkinDeformationType::Morph)
		{
			PermutationVector.Set<FMorphBlendDim>(true);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		const bool bBoneWeights16          = PermutationVector.Get<FBoneWeight16Dim>();
		const bool bBoneIndex16            = PermutationVector.Get<FBoneIndex16Dim>();
		const bool bUnlimitedBoneInfluence = PermutationVector.Get<FUnlimitedBoneInfluencesDim>();
		const bool bExtraBoneInfluences    = PermutationVector.Get<FExtraBoneInfluencesDim>();
		const bool bCloth                  = PermutationVector.Get<FApexClothDim>();
		const bool bMorph                  = PermutationVector.Get<FMorphBlendDim>();

		// Unlimited / Extra bone influences are mutually exclusive.
		if (bUnlimitedBoneInfluence && bExtraBoneInfluences)
		{
			return false;
		}

		// Unlimited bone influences are not compatible with 16 bit bones weights or indices.
		if (bUnlimitedBoneInfluence && (bBoneWeights16 || bBoneIndex16))
		{
			return false;
		}

		// Cloth and morph are mutually exclusive.
		if (bCloth && bMorph)
		{
			return false;
		}

		return IsGPUSkinCacheAvailable(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("GPUSKIN_RWBUFFER_OFFSET_TANGENT_X"), FGPUSkinCache::RWTangentXOffsetInFloats);
		OutEnvironment.SetDefine(TEXT("GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z"), FGPUSkinCache::RWTangentZOffsetInFloats);

		DynamicMeshBoundsModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumVertices)
		SHADER_PARAMETER(uint32, SkinCacheStart)
		SHADER_PARAMETER(uint32, InputStreamStart)
		SHADER_PARAMETER(uint32, NumBoneInfluences)
		SHADER_PARAMETER(uint32, InputWeightIndexSize)
		SHADER_PARAMETER(uint32, InputWeightStart)
		SHADER_PARAMETER(uint32, InputWeightStride)
		SHADER_PARAMETER(uint32, MorphBufferOffset)
		SHADER_PARAMETER(uint32, ClothBufferOffset)
		SHADER_PARAMETER(float,  ClothBlendWeight)
		SHADER_PARAMETER(FMatrix44f, ClothToLocal)
		SHADER_PARAMETER(uint32, ClothNumInfluencesPerVertex)
		SHADER_PARAMETER(FVector3f, WorldScale)
		SHADER_PARAMETER(int32,  DynamicBoundsOffset)

		SHADER_PARAMETER_SRV(Buffer<uint>,             InputWeightStream)
		SHADER_PARAMETER_SRV(Buffer<uint>,             InputWeightLookupStream)
		SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(BoneMatrices)
		SHADER_PARAMETER_SRV(Buffer<float4>,           TangentInputBuffer)
		SHADER_PARAMETER_SRV(Buffer<float>,            PositionInputBuffer)
		SHADER_PARAMETER_SRV(Buffer<float>,            MorphBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,           ClothBuffer)
		SHADER_PARAMETER_SRV(Buffer<float2>,           ClothPositionsAndNormalsBuffer)
		SHADER_PARAMETER_UAV(RWBuffer<float>,          PositionBufferUAV)
		SHADER_PARAMETER_UAV(RWBuffer<>,               TangentBufferUAV)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<int4>, OutBoundsBufferUAV)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FGPUSkinCacheCS, "/Engine/Private/GpuSkinCacheComputeShader.usf", "SkinCacheUpdateBatchCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FInitDynamicMeshBoundsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FInitDynamicMeshBoundsCS);
	SHADER_USE_PARAMETER_STRUCT(FInitDynamicMeshBoundsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32,									MaxNumToClear)
		SHADER_PARAMETER_SRV(StructuredBuffer<uint>,			SlotsToClearMask)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<FIntVector4>,	OutBoundsBufferUAV)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FInitDynamicMeshBoundsCS, "/Engine/Private/DynamicMeshBounds.usf", "InitDynamicMeshBoundsCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

/**
 * Manages a GPU buffer of bounds (AABBs) that are intended for use with mesh sections submitted through the rendering pipeline.
 * The returned slots can be piped through the FMeshBatch::DynamicMeshBoundsIndex and is then used in instance culling.
 * The allocated bounds are not persistent, in that if they are not updated in a given frame, they revert back to uninitialized.
 */
class FDynamicMeshBoundsBuffer
{
public:
	FDynamicMeshBoundsBuffer() = default;

	int32 AllocateOffset(int32 NumSlots)
	{
		int32 Offset = Allocator.Allocate(NumSlots);
		SlotsToClearMask.PadToNum(Allocator.GetMaxSize(), false);
		SlotsToClearMask.SetRange(Offset, NumSlots, true);
		return Offset;
	}

	void FreeOffset(int32 Offset, int32 NumSlots)
	{
		check(IsInParallelRenderingThread());
		return Allocator.Free(Offset, NumSlots);
	}

	int32 GetNumSlotsAllocated() const
	{
		return Allocator.GetMaxSize();
	}

	FDynamicMeshBoundsShaderParameters Update(FRHICommandList& RHICmdList, const TBitArray<>& SlotsToUpdateMask)
	{
		constexpr int32 MinSlotsToAllocate = 128;
		uint32 NumElementedToAllocate = uint32(FMath::Max(MinSlotsToAllocate, int32(FMath::RoundUpToPowerOfTwo(GetNumSlotsAllocated()))) * 2);
		ResizeResourceIfNeeded(RHICmdList, MeshBoundsBuffer, sizeof(FIntVector4) * NumElementedToAllocate, TEXT("DynamicMeshBoundsBuffer.MeshBoundsBuffer"));

		RHICmdList.Transition(FRHITransitionInfo(MeshBoundsBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute), ERHITransitionCreateFlags::AllowDecayPipelines);

		DispatchClearSlots(RHICmdList, SlotsToClearMask);

		RHICmdList.Transition(FRHITransitionInfo(MeshBoundsBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute));

		DispatchClearSlots(RHICmdList, SlotsToUpdateMask);

		SlotsToClearMask.Empty(GetNumSlotsAllocated());

		FDynamicMeshBoundsShaderParameters ShaderParameters;
		ShaderParameters.DynamicMeshBoundsBuffer = MeshBoundsBuffer.SRV;
		ShaderParameters.DynamicMeshBoundsMax    = GetNumSlotsAllocated();
		return ShaderParameters;
	}

	FRHIUnorderedAccessView* GetUAV() const
	{
		return MeshBoundsBuffer.UAV ? MeshBoundsBuffer.UAV : GBlackFloat4StructuredBufferWithSRV->UnorderedAccessViewRHI;
	}

private:
	void DispatchClearSlots(FRHICommandList& RHICmdList, const TBitArray<>& SlotsToUpdateMask)
	{
		if (!GetNumSlotsAllocated() || SlotsToUpdateMask.IsEmpty())
		{
			return;
		}

		FByteAddressBuffer SlotsToClearMaskBuffer;
		SlotsToClearMaskBuffer.Buffer = UE::RHIResourceUtils::CreateBufferFromArray(
			RHICmdList,
			TEXT("DynamicMeshBoundsBuffer.SlotsToClearMaskBuffer"),
			EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile, sizeof(uint32),
			SlotsToUpdateMask.GetData(),
			FBitSet::CalculateNumWords(SlotsToUpdateMask.Num()) * 4);
		SlotsToClearMaskBuffer.SRV = RHICmdList.CreateShaderResourceView(SlotsToClearMaskBuffer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(SlotsToClearMaskBuffer.Buffer));

		FInitDynamicMeshBoundsCS::FParameters PassParameters;
		PassParameters.SlotsToClearMask = SlotsToClearMaskBuffer.SRV;
		PassParameters.OutBoundsBufferUAV = MeshBoundsBuffer.UAV;
		PassParameters.MaxNumToClear = GetNumSlotsAllocated();

		auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FInitDynamicMeshBoundsCS>();
		FComputeShaderUtils::Dispatch(
			RHICmdList,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCountWrapped(PassParameters.MaxNumToClear, 64));
	}

	FSpanAllocator Allocator;
	FRWBufferStructured MeshBoundsBuffer;
	TBitArray<> SlotsToClearMask;
};

struct FDynamicMeshBoundsBlackboardStruct
{
	FDynamicMeshBoundsShaderParameters Parameters;
	bool bInitialized = false;
};

RDG_REGISTER_BLACKBOARD_STRUCT(FDynamicMeshBoundsBlackboardStruct);

FDynamicMeshBoundsShaderParameters GetDynamicMeshBoundsShaderParameters(FRDGBuilder& GraphBuilder)
{
	if (const FDynamicMeshBoundsBlackboardStruct* Struct = GraphBuilder.Blackboard.Get<FDynamicMeshBoundsBlackboardStruct>())
	{
		check(Struct->bInitialized);
		return Struct->Parameters;
	}

	FDynamicMeshBoundsShaderParameters ShaderParameters;
	ShaderParameters.DynamicMeshBoundsMax = 0;
	ShaderParameters.DynamicMeshBoundsBuffer = GBlackFloat4StructuredBufferWithSRV->ShaderResourceViewRHI;
	return ShaderParameters;
}

static bool DoesPlatformSupportDynamicMeshBounds(EShaderPlatform ShaderPlatform)
{
	return GSkinCacheDynamicMeshBounds == 1 
		|| (GSkinCacheDynamicMeshBounds == 2 && DoesPlatformSupportNanite(ShaderPlatform));
}

void DynamicMeshBoundsModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("ENABLE_DYNAMIC_MESH_BOUNDS"), DoesPlatformSupportDynamicMeshBounds(Parameters.Platform) ? 1 : 0);
}

int32 FGPUSkinCache::AllocateDynamicMeshBoundsSlot(int32 NumSlots)
{
	check(IsInParallelRenderingThread());
	return DynamicMeshBoundsBuffer.IsValid() ? DynamicMeshBoundsBuffer->AllocateOffset(NumSlots) : INDEX_NONE;
}

void FGPUSkinCache::ReleaseDynamicMeshBoundsSlot(int32 Offset, int32 NumSlots)
{
	check(IsInParallelRenderingThread());
	if (DynamicMeshBoundsBuffer.IsValid())
	{
		DynamicMeshBoundsBuffer->FreeOffset(Offset, NumSlots);
	}
}

//////////////////////////////////////////////////////////////////////////

FGPUSkinCache::FGPUSkinCache(ERHIFeatureLevel::Type InFeatureLevel, UWorld* InWorld)
	: DynamicMeshBoundsBuffer(DoesPlatformSupportDynamicMeshBounds(GetFeatureLevelShaderPlatform(InFeatureLevel)) ? MakePimpl<FDynamicMeshBoundsBuffer>() : nullptr)
	, FeatureLevel(InFeatureLevel)
	, World(InWorld)
{
	check(World);

	if (GetGPUSkinCacheRecomputeTangentsMode(FeatureLevel) == 1 && GStoreDuplicatedVerticesForRecomputeTangents == 0)
	{
		UE_LOGF(LogSkinCache, Warning, "r.SkinCache.RecomputeTangents is set to 1 to update all skinned objects but duplicated vertices are not are not always stored. Set r.SkinCache.RecomputeTangents to 2 or r.SkinCache.StoreDuplicatedVerticesForRecomputeTangents to 1.");
	}
}

FGPUSkinCache::~FGPUSkinCache()
{
	Cleanup();
}

void FGPUSkinCache::Cleanup()
{
	for (int32 Index = 0; Index < StagingBuffers.Num(); ++Index)
	{
		StagingBuffers[Index].Release();
	}

	while (Entries.Num() > 0)
	{
		Release(Entries.Last());
	}
	ensure(Allocations.Num() == 0);
}

class FRecomputeTangentsPerTriangleCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FRecomputeTangentsPerTriangleCS);
	SHADER_USE_PARAMETER_STRUCT(FRecomputeTangentsPerTriangleCS, FGlobalShader);

	class FMergeDuplicatedVerticesDim : SHADER_PERMUTATION_BOOL("MERGE_DUPLICATED_VERTICES");
	class FFullPrecisionUVDim         : SHADER_PERMUTATION_BOOL("FULL_PRECISION_UV");

	using FPermutationDomain = TShaderPermutationDomain<FMergeDuplicatedVerticesDim, FFullPrecisionUVDim>;

	static const uint32 ThreadGroupSizeX = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsGPUSkinCacheAvailable(Parameters.Platform);
	}
	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("INTERMEDIATE_ACCUM_BUFFER_NUM_INTS"), FGPUSkinCache::IntermediateAccumBufferNumInts);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumTriangles)
		SHADER_PARAMETER(uint32, SkinCacheStart)
		SHADER_PARAMETER(uint32, InputStreamStart)
		SHADER_PARAMETER(uint32, NumTexCoords)
		SHADER_PARAMETER(uint32, IndexBufferOffset)
		SHADER_PARAMETER(uint32, IntermediateAccumBufferOffset)

		SHADER_PARAMETER_SRV(Buffer<uint>,   IndexBuffer)
		SHADER_PARAMETER_SRV(Buffer<float2>, UVsInputBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>, GPUTangentCacheBuffer)
		SHADER_PARAMETER_SRV(Buffer<float>,  GPUPositionCacheBuffer)
		SHADER_PARAMETER_SRV(Buffer<uint>,   DuplicatedIndices)
		SHADER_PARAMETER_SRV(Buffer<uint>,   DuplicatedIndicesIndices)
		SHADER_PARAMETER_UAV(RWBuffer<int>,  IntermediateAccumBufferUAV)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FRecomputeTangentsPerTriangleCS, "/Engine/Private/RecomputeTangentsPerTrianglePass.usf", "MainCS", SF_Compute);

class FRecomputeTangentsPerVertexCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FRecomputeTangentsPerVertexCS);
	SHADER_USE_PARAMETER_STRUCT(FRecomputeTangentsPerVertexCS, FGlobalShader);

	class FBlendUsingVertexColorDim : SHADER_PERMUTATION_BOOL("BLEND_USING_VERTEX_COLOR");

	using FPermutationDomain = TShaderPermutationDomain<FBlendUsingVertexColorDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsGPUSkinCacheAvailable(Parameters.Platform);
	}

	static const uint32 ThreadGroupSizeX = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("GPUSKIN_RWBUFFER_OFFSET_TANGENT_X"), FGPUSkinCache::RWTangentXOffsetInFloats);
		OutEnvironment.SetDefine(TEXT("GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z"), FGPUSkinCache::RWTangentZOffsetInFloats);
		OutEnvironment.SetDefine(TEXT("INTERMEDIATE_ACCUM_BUFFER_NUM_INTS"), FGPUSkinCache::IntermediateAccumBufferNumInts);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumVertices)
		SHADER_PARAMETER(uint32, SkinCacheStart)
		SHADER_PARAMETER(uint32, InputStreamStart)
		SHADER_PARAMETER(uint32, VertexColorChannel)
		SHADER_PARAMETER(uint32, IntermediateAccumBufferOffset)
		
		SHADER_PARAMETER_SRV(Buffer<float4>, TangentInputBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>, ColorInputBuffer)
		SHADER_PARAMETER_UAV(RWBuffer<int>,  IntermediateAccumBufferUAV)
		SHADER_PARAMETER_UAV(RWBuffer<>,     TangentBufferUAV)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FRecomputeTangentsPerVertexCS, "/Engine/Private/RecomputeTangentsPerVertexPass.usf", "MainCS", SF_Compute);

void FGPUSkinCache::DispatchUpdateSkinTangentsVertexPass(FRHICommandList& RHICmdList, FGPUSkinCacheEntry* Entry, int32 SectionIndex, FSkinCacheRWBuffer*& StagingBuffer)
{
	FGPUSkinCacheEntry::FSectionDispatchData& DispatchData = Entry->DispatchData[SectionIndex];

	SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, SkinTangents_PerVertexPass, GetEmitDrawEvents(), TEXT("%sTangentsVertex Mesh=%s, LOD=%d, Chunk=%d, InputStreamStart=%d, OutputStreamStart=%d, Vert=%d")
		, RHI_BREADCRUMB_FORCE_STRING_LITERAL(Entry->Mode == EGPUSkinCacheEntryMode::RayTracing ? TEXT("[RT]") : TEXT(""))
		, GetSkeletalMeshObjectDebugName(Entry->GPUSkin)
		, Entry->LOD
		, SectionIndex
		, DispatchData.InputStreamStart
		, DispatchData.OutputStreamStart
		, DispatchData.NumVertices
	);

	if (!GRecomputeTangentsParallelDispatch)
	{
		// When triangle & vertex passes are interleaved, resource transition is needed in between.
		RHICmdList.Transition({
			DispatchData.GetTangentRWBuffer()->UpdateAccessState(ERHIAccess::UAVCompute),
			StagingBuffer->UpdateAccessState(ERHIAccess::UAVCompute)
		}, ERHITransitionCreateFlags::AllowDecayPipelines);
	}

	FRecomputeTangentsPerVertexCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRecomputeTangentsPerVertexCS::FBlendUsingVertexColorDim>(DispatchData.Section->RecomputeTangentsVertexMaskChannel != ESkinVertexColorChannel::None);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());
	TShaderMapRef<FRecomputeTangentsPerVertexCS> ComputeShader(GlobalShaderMap, PermutationVector);
	SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

	FRecomputeTangentsPerVertexCS::FParameters Parameters;
	Parameters.SkinCacheStart                = DispatchData.OutputStreamStart;
	Parameters.NumVertices                   = DispatchData.NumVertices;
	Parameters.InputStreamStart              = DispatchData.InputStreamStart;
	Parameters.VertexColorChannel            = uint32(DispatchData.Section->RecomputeTangentsVertexMaskChannel);
	Parameters.TangentInputBuffer            = DispatchData.RecomputeTangents.IntermediateTangentBuffer ? DispatchData.RecomputeTangents.IntermediateTangentBuffer->Buffer.SRV : nullptr;
	Parameters.ColorInputBuffer              = DispatchData.ColorBufferSRV;
	Parameters.IntermediateAccumBufferUAV    = GRecomputeTangentsParallelDispatch ? DispatchData.GetIntermediateAccumulatedTangentBuffer()->Buffer.UAV : StagingBuffer->Buffer.UAV;
	Parameters.IntermediateAccumBufferOffset = GRecomputeTangentsParallelDispatch * DispatchData.RecomputeTangents.Section.IntermediateBufferOffset;
	Parameters.TangentBufferUAV              = DispatchData.GetTangentRWBuffer()->Buffer.UAV;

	SetShaderParameters(RHICmdList, ComputeShader, ComputeShader.GetComputeShader(), Parameters);
	DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), FMath::DivideAndRoundUp(DispatchData.NumVertices, ComputeShader->ThreadGroupSizeX), 1, 1);
	UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
}

void FGPUSkinCache::DispatchUpdateSkinTangentsTrianglePass(FRHICommandList& RHICmdList, FGPUSkinCacheEntry* Entry, int32 SectionIndex, FSkinCacheRWBuffer*& StagingBuffer)
{
	FGPUSkinCacheEntry::FSectionDispatchData& DispatchData = Entry->DispatchData[SectionIndex];

	const int32 LODIndex = Entry->LOD;
	FSkeletalMeshRenderData& SkelMeshRenderData = Entry->GPUSkin->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData.LODRenderData[LODIndex];

	if (!GRecomputeTangentsParallelDispatch)
	{
		const int32 NumTangentIntermediateBuffers = FMath::Max(GNumTangentIntermediateBuffers, 1);
		if (StagingBuffers.Num() != NumTangentIntermediateBuffers)
		{
			// Release extra buffers if shrinking
			for (int32 Index = NumTangentIntermediateBuffers; Index < StagingBuffers.Num(); ++Index)
			{
				StagingBuffers[Index].Release();
			}
			StagingBuffers.SetNum(NumTangentIntermediateBuffers, EAllowShrinking::No);
		}

		// no need to clear the staging buffer because we create it cleared and clear it after each usage in the per vertex pass
		uint32 NumIntsPerBuffer = DispatchData.NumVertices * FGPUSkinCache::IntermediateAccumBufferNumInts;
		CurrentStagingBufferIndex = (CurrentStagingBufferIndex + 1) % StagingBuffers.Num();
		StagingBuffer = &StagingBuffers[CurrentStagingBufferIndex];
		if (StagingBuffer->Buffer.NumBytes < NumIntsPerBuffer * sizeof(uint32))
		{
			StagingBuffer->Release();
			StagingBuffer->Buffer.Initialize(RHICmdList, TEXT("SkinTangentIntermediate"), sizeof(int32), NumIntsPerBuffer, PF_R32_SINT, BUF_UnorderedAccess);
			RHICmdList.BindDebugLabelName(StagingBuffer->Buffer.UAV, TEXT("SkinTangentIntermediate"));

			const uint32 MemSize = NumIntsPerBuffer * sizeof(uint32);
			SET_MEMORY_STAT(STAT_GPUSkinCache_TangentsIntermediateMemUsed, MemSize);

			// The UAV must be zero-filled. We leave it zeroed after each round (see RecomputeTangentsPerVertexPass.usf), so this is only needed on when the buffer is first created.
			RHICmdList.ClearUAVUint(StagingBuffer->Buffer.UAV, FUintVector4(0, 0, 0, 0));
		}

		// When triangle & vertex passes are interleaved, resource transition is needed in between.
		RHICmdList.Transition({
			DispatchData.GetActiveTangentRWBuffer()->UpdateAccessState(ERHIAccess::SRVCompute),
			StagingBuffer->UpdateAccessState(ERHIAccess::UAVCompute)
		}, ERHITransitionCreateFlags::AllowDecayPipelines);
	}

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

	const bool bFullPrecisionUV = LodData.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs();
	const bool bUseDuplicatedVertices = GUseDuplicatedVerticesForRecomputeTangents && LodData.RenderSections[SectionIndex].DuplicatedVerticesBuffer.DuplicatedVerticesIndexBuffer.VertexBufferSRV;

	FRecomputeTangentsPerTriangleCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRecomputeTangentsPerTriangleCS::FMergeDuplicatedVerticesDim>(bUseDuplicatedVertices);
	PermutationVector.Set<FRecomputeTangentsPerTriangleCS::FFullPrecisionUVDim>(bFullPrecisionUV);

	TShaderMapRef<FRecomputeTangentsPerTriangleCS> ComputeShader(GlobalShaderMap, PermutationVector);
	check(ComputeShader.IsValid());

	SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, SkinTangents_PerTrianglePass, GetEmitDrawEvents(), TEXT("%sTangentsTri  Mesh=%s, LOD=%d, Chunk=%d, IndexStart=%d Tri=%d BoneInfluenceType=%d UVPrecision=%d")
		, RHI_BREADCRUMB_FORCE_STRING_LITERAL(Entry->Mode == EGPUSkinCacheEntryMode::RayTracing ? TEXT("[RT]") : TEXT(""))
		, GetSkeletalMeshObjectDebugName(Entry->GPUSkin)
		, LODIndex
		, SectionIndex
		, DispatchData.RecomputeTangents.IndexBufferOffsetValue
		, DispatchData.RecomputeTangents.NumTriangles
		, (int32)Entry->BoneInfluenceType
		, bFullPrecisionUV
	);

	if (bUseDuplicatedVertices)
	{
#if WITH_EDITOR
		check(LodData.RenderSections[SectionIndex].DuplicatedVerticesBuffer.DupVertData.Num() && LodData.RenderSections[SectionIndex].DuplicatedVerticesBuffer.DupVertIndexData.Num());
#endif
		DispatchData.RecomputeTangents.DuplicatedIndices        = LodData.RenderSections[SectionIndex].DuplicatedVerticesBuffer.DuplicatedVerticesIndexBuffer.VertexBufferSRV;
		DispatchData.RecomputeTangents.DuplicatedIndicesIndices = LodData.RenderSections[SectionIndex].DuplicatedVerticesBuffer.LengthAndIndexDuplicatedVerticesIndexBuffer.VertexBufferSRV;
	}

	INC_DWORD_STAT_BY(STAT_GPUSkinCache_NumTrianglesForRecomputeTangents, DispatchData.RecomputeTangents.NumTriangles);

	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

	FRecomputeTangentsPerTriangleCS::FParameters Parameters;
	Parameters.NumTriangles           = DispatchData.RecomputeTangents.NumTriangles;
	Parameters.SkinCacheStart         = DispatchData.OutputStreamStart;
	Parameters.IndexBuffer            = DispatchData.RecomputeTangents.IndexBuffer;
	Parameters.IndexBufferOffset      = DispatchData.RecomputeTangents.IndexBufferOffsetValue;
	Parameters.InputStreamStart       = DispatchData.InputStreamStart;
	Parameters.NumTexCoords           = DispatchData.NumTexCoords;
	Parameters.GPUPositionCacheBuffer = DispatchData.GetPositionRWBuffer()->Buffer.SRV;
	Parameters.GPUTangentCacheBuffer  = DispatchData.GetActiveTangentRWBuffer()->Buffer.SRV;
	Parameters.UVsInputBuffer         = DispatchData.UVsBufferSRV;
	Parameters.IntermediateAccumBufferUAV    = GRecomputeTangentsParallelDispatch ? DispatchData.GetIntermediateAccumulatedTangentBuffer()->Buffer.UAV : StagingBuffer->Buffer.UAV;
	Parameters.IntermediateAccumBufferOffset = GRecomputeTangentsParallelDispatch * DispatchData.RecomputeTangents.Section.IntermediateBufferOffset;

	if (DispatchData.RecomputeTangents.DuplicatedIndices)
	{
		Parameters.DuplicatedIndices        = DispatchData.RecomputeTangents.DuplicatedIndices;
		Parameters.DuplicatedIndicesIndices = DispatchData.RecomputeTangents.DuplicatedIndicesIndices;
	}

	SetShaderParameters(RHICmdList, ComputeShader, ComputeShader.GetComputeShader(), Parameters);
	DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), FMath::DivideAndRoundUp(DispatchData.RecomputeTangents.NumTriangles, FRecomputeTangentsPerTriangleCS::ThreadGroupSizeX), 1, 1);
	UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
}

DECLARE_GPU_STAT(GPUSkinCache);

void FGPUSkinCache::TransitionBuffers(FRHICommandList& RHICmdList, TArray<FSkinCacheRWBuffer*>& Buffers, ERHIAccess ToState)
{
	if (!Buffers.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TransitionBuffers);
		const uint32 NextTransitionFence = GetNextTransitionFence();

		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> UAVs;
		UAVs.Reserve(Buffers.Num());

		FSkinCacheRWBuffer* LastBuffer = nullptr;
		for (FSkinCacheRWBuffer* Buffer : Buffers)
		{
			if (!Buffer->UpdateFence(NextTransitionFence))
			{
				continue;
			}

			LastBuffer = Buffer;
			if (EnumHasAnyFlags(ToState, ERHIAccess::UAVMask) || Buffer->AccessState != ToState)
			{
				UAVs.Add(Buffer->UpdateAccessState(ToState));
			}
		}

		// The NoFence flag is necessary to silence the validator for transitioning from All pipes to Graphics.
		RHICmdList.Transition(MakeArrayView(UAVs.GetData(), UAVs.Num()), ERHITransitionCreateFlags::AllowDecayPipelines);
	}
}

void FGPUSkinCache::TransitionBufferUAVs(FRHICommandList& RHICmdList, TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator>& Transitions, const TArray<FSkinCacheRWBuffer*>& Buffers, TArray<FRHIUnorderedAccessView*>& OutUAVs)
{
	if (!Buffers.IsEmpty() || !Transitions.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TransitionBufferUAVs);
		const uint32 NextTransitionFence = GetNextTransitionFence();

		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> UAVs;
		Transitions.Reserve(Buffers.Num());
		OutUAVs.Reserve(Buffers.Num());

		for (FSkinCacheRWBuffer* Buffer : Buffers)
		{
			if (!Buffer->UpdateFence(NextTransitionFence))
			{
				continue;
			}

			Transitions.Add(Buffer->UpdateAccessState(ERHIAccess::UAVCompute));
			OutUAVs.Add(Buffer->Buffer.UAV);
		}

		// The NoFence flag is necessary to silence the validator for transitioning from All pipes to Graphics.
		RHICmdList.Transition(MakeArrayView(Transitions.GetData(), Transitions.Num()), ERHITransitionCreateFlags::AllowDecayPipelines);
	}
}

void FGPUSkinCache::TransitionBufferUAVs(FRHICommandList& RHICmdList, const TArray<FSkinCacheRWBuffer*>& Buffers, TArray<FRHIUnorderedAccessView*>& OutUAVs)
{
	TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> Transitions;
	TransitionBufferUAVs(RHICmdList, Transitions, Buffers, OutUAVs);
}

ERHIPipeline FGPUSkinCache::GetDispatchPipeline(FRDGBuilder& GraphBuilder)
{
	// Morph targets require the skeletal mesh updater to be able to support async compute.
	return GSkinCacheAsyncCompute && GraphBuilder.IsAsyncComputeEnabled() ? ERHIPipeline::AsyncCompute : ERHIPipeline::Graphics;
}

struct FGPUSkinCache::FTaskData
{
	FTaskData(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform, ERHIPipeline InPipeline)
		: Pipeline(InPipeline)
		, bAsyncCommandList(GraphBuilder.IsParallelSetupEnabled() && !GNumDispatchesToCapture)
	{
		if (DoesPlatformSupportDynamicMeshBounds(ShaderPlatform))
		{
			DynamicMeshBoundsBlackboardStruct = &GraphBuilder.Blackboard.Create<FDynamicMeshBoundsBlackboardStruct>();
		}

		if (bAsyncCommandList)
		{
			RHICmdList = new FRHICommandList;

			FRHICommandListScopedPipeline ScopedPipeline(GraphBuilder.RHICmdList, Pipeline);
			GraphBuilder.RHICmdList.QueueAsyncCommandListSubmit(RHICmdList);
		}
		else
		{
			RHICmdList = &GraphBuilder.RHICmdList;
		}
	}

	void Begin()
	{
		OriginalPipeline = RHICmdList->SwitchPipeline(Pipeline);

		RHICmdListScopedFence.Emplace(*RHICmdList);
	}

	void End()
	{
		RHICmdListScopedFence.Reset();

		if (bAsyncCommandList)
		{
			RHICmdList->FinishRecording();
		}
		else
		{
			RHICmdList->SwitchPipeline(OriginalPipeline);
		}
	}

	TArray<FDispatchEntry> TangentDispatches;
	TArray<FSortedDispatchEntry> SortedDispatches;
	TOptional<FRHICommandListScopedFence> RHICmdListScopedFence;
	FRHICommandList* RHICmdList = nullptr;
	FDynamicMeshBoundsBlackboardStruct* DynamicMeshBoundsBlackboardStruct = nullptr;

	struct
	{
		TArray<FSkinCacheRWBuffer*> FinalRead;

		struct
		{
			TArray<FSkinCacheRWBuffer*> Write;
			TArray<FRHIUnorderedAccessView*> Overlap;

		} Skinning;

		struct
		{
			TArray<FSkinCacheRWBuffer*> Write;
			TArray<FSkinCacheRWBuffer*> Read;

		} RecomputeTangents;

	} Transitions;

	UE::Tasks::FTask SetupTask;
	ERHIPipeline Pipeline = ERHIPipeline::Graphics;
	ERHIPipeline OriginalPipeline = ERHIPipeline::Graphics;
	const FRHITransition* AsyncComputeTransition = nullptr;
	bool bAsyncCommandList = false;
	bool bWaitPassAdded = false;
	bool bSignalPassAdded = false;
};

UE::Tasks::FTask FGPUSkinCache::Dispatch(FRDGBuilder& GraphBuilder, const UE::Tasks::FTask& PrerequisitesTask, ERHIPipeline InPipeline)
{
	FTaskData* TaskData = &GraphBuilder.Blackboard.Create<FTaskData>(GraphBuilder, GetFeatureLevelShaderPlatform(FeatureLevel), InPipeline);

	TaskData->SetupTask = GraphBuilder.AddSetupTask([this, TaskData]
	{
		TaskData->Begin();
		DispatchPassSetup(*TaskData);

	}, PrerequisitesTask, UE::Tasks::ETaskPriority::High, TaskData->bAsyncCommandList);

	GraphBuilder.AddSetupTask([this, TaskData]
	{
		FTaskTagScope TagScope(ETaskTag::EParallelRenderingThread);
		DispatchPassExecute(*TaskData);
		TaskData->End();

	}, TaskData->SetupTask, UE::Tasks::ETaskPriority::BackgroundHigh, TaskData->bAsyncCommandList);

	if (TaskData->Pipeline == ERHIPipeline::AsyncCompute)
	{
		// Tell the builder that we will manually sync async compute work back to graphics.
		GraphBuilder.SkipInitialAsyncComputeFence();

		GraphBuilder.AddPostExecuteCallback([TaskData, &RHICmdList = GraphBuilder.RHICmdList]
		{
			checkf(TaskData->bWaitPassAdded, TEXT("FGPUSkinCache::AddAsyncComputeWait was never called!"));
		});
	}

	return TaskData->SetupTask;
}

void FGPUSkinCache::AddAsyncComputeSignal(FRDGBuilder& GraphBuilder)
{
	FTaskData* TaskData = GraphBuilder.Blackboard.GetMutable<FTaskData>();

	if (TaskData && !TaskData->bSignalPassAdded && TaskData->Pipeline == ERHIPipeline::AsyncCompute)
	{
		AddPass(GraphBuilder, RDG_EVENT_NAME("GPUSkinCache_AsyncComputeSignal"), [] (FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.Transition({}, ERHIPipeline::Graphics, ERHIPipeline::AsyncCompute);
		});
		TaskData->bSignalPassAdded = true;
	}
}

void FGPUSkinCache::AddAsyncComputeWait(FRDGBuilder& GraphBuilder)
{
	FTaskData* TaskData = GraphBuilder.Blackboard.GetMutable<FTaskData>();

	if (!TaskData)
	{
		return;
	}

	if (TaskData->SetupTask.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGPUSkinCache::EndDispatch);
		TaskData->SetupTask.Wait();
		TaskData->SetupTask = {};
	}

	if (!TaskData->bWaitPassAdded && TaskData->Pipeline == ERHIPipeline::AsyncCompute)
	{
		AddPass(GraphBuilder, RDG_EVENT_NAME("GPUSkinCache_AsyncComputeWait"), [TaskData](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			check(TaskData->AsyncComputeTransition);
			RHICmdList.EndTransition(TaskData->AsyncComputeTransition);
		});
		TaskData->bWaitPassAdded = true;
	}
}

void FGPUSkinCache::DispatchPassSetup(FTaskData& TaskData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGPUSkinCache::DispatchPassSetup);

	const int32 NumDispatches = BatchDispatches.Num();

	TBitArray<> SlotsToUpdateMask;

	if (TaskData.DynamicMeshBoundsBlackboardStruct && DynamicMeshBoundsBuffer)
	{
		SlotsToUpdateMask.Init(false, DynamicMeshBoundsBuffer->GetNumSlotsAllocated());
	}

	TaskData.TangentDispatches.Reserve(NumDispatches);
	TaskData.Transitions.FinalRead.Reserve(NumDispatches * NUM_BUFFERS);
	TaskData.Transitions.Skinning.Write.Reserve(NumDispatches * NUM_BUFFERS);
	TaskData.Transitions.RecomputeTangents.Read.Reserve(NumDispatches * 2);

	if (GRecomputeTangentsParallelDispatch)
	{
		TaskData.Transitions.RecomputeTangents.Write.Reserve(NumDispatches);
	}

	struct
	{
		int32 NumRayTracingDispatches = 0;
		int32 NumRayTracingBuffers = 0;
		int32 NumBuffers = 0;

	} Stats;

	for (const FDispatchEntry& DispatchItem : BatchDispatches)
	{
		FGPUSkinCacheEntry* Entry = DispatchItem.SkinCacheEntry;
		Entry->bQueuedForDispatch = false;

		FGPUSkinCacheEntry::FSectionDispatchData& DispatchData = Entry->DispatchData[DispatchItem.Section];
		DispatchData.DispatchFlags = EGPUSkinCacheDispatchFlags::None;
		DispatchData.PreviousPositionBuffer = nullptr;
		DispatchData.RevisionNumber = 0;

		if (!SlotsToUpdateMask.IsEmpty() && DispatchData.DynamicBoundsOffset >= 0)
		{
			SlotsToUpdateMask[DispatchData.DynamicBoundsOffset] = true;
		}

		if (DispatchData.PositionTracker.Allocation->HasPreviousBuffer())
		{
			const FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = DispatchData.SourceVertexFactory->GetShaderData();

			const FVertexBufferAndSRV& BoneBuffer = ShaderData.GetBoneBufferForReading(false);
			const uint32 CurrentRevision = ShaderData.GetRevisionNumber(false);
			DispatchData.PositionBuffer = DispatchData.PositionTracker.Find(BoneBuffer, CurrentRevision);

			const FVertexBufferAndSRV& PreviousBoneBuffer = ShaderData.GetBoneBufferForReading(true);
			const uint32 PreviousRevision = ShaderData.GetRevisionNumber(true);
			DispatchData.PreviousPositionBuffer = DispatchData.PositionTracker.Find(PreviousBoneBuffer, PreviousRevision);

			// Allocate buffers if not found, excluding buffers already in use.  Or make the current buffer distinct if it happens to equal previous.
			if (!DispatchData.PositionBuffer || DispatchData.PositionBuffer == DispatchData.PreviousPositionBuffer)
			{
				DispatchData.PositionBuffer = DispatchData.PositionTracker.AllocateUnused(BoneBuffer, CurrentRevision, DispatchData.PreviousPositionBuffer);
				DispatchData.DispatchFlags |= EGPUSkinCacheDispatchFlags::Position;

				TaskData.Transitions.Skinning.Write.Emplace(DispatchData.PositionBuffer);
				TaskData.Transitions.FinalRead.Emplace(DispatchData.PositionBuffer);
			}

			if (!DispatchData.PreviousPositionBuffer)
			{
				DispatchData.PreviousPositionBuffer = DispatchData.PositionTracker.AllocateUnused(PreviousBoneBuffer, PreviousRevision, DispatchData.PositionBuffer);
				DispatchData.DispatchFlags |= EGPUSkinCacheDispatchFlags::PositionPrevious;

				TaskData.Transitions.Skinning.Write.Emplace(DispatchData.PreviousPositionBuffer);
				TaskData.Transitions.FinalRead.Emplace(DispatchData.PreviousPositionBuffer);
			}
		}
		else
		{
			DispatchData.PositionBuffer = &DispatchData.PositionTracker.Allocation->GetPositionBuffer();
			DispatchData.DispatchFlags |= EGPUSkinCacheDispatchFlags::Position;

			TaskData.Transitions.Skinning.Write.Emplace(DispatchData.PositionBuffer);
			TaskData.Transitions.FinalRead.Emplace(DispatchData.PositionBuffer);
		}

		check(DispatchData.PreviousPositionBuffer != DispatchData.PositionBuffer);

		DispatchData.TangentBuffer = DispatchData.PositionTracker.GetTangentBuffer();

		if (EnumHasAnyFlags(DispatchData.DispatchFlags, EGPUSkinCacheDispatchFlags::Position))
		{
			if (DispatchData.RecomputeTangents.Section.bEnable)
			{
				DispatchData.DispatchFlags |= EGPUSkinCacheDispatchFlags::RecomputeTangents;

				DispatchData.RecomputeTangents.IntermediateAccumulatedTangentBuffer = DispatchData.PositionTracker.GetIntermediateAccumulatedTangentBuffer();
				check(DispatchData.RecomputeTangents.IntermediateAccumulatedTangentBuffer);

				// The IntermediateTangents (skinned-tangent copy) buffer only exists when at least one section in the
				// entry uses vertex-color blending, but it's allocated entry-wide. Only wire it up for sections that
				// actually blend; sections without blending let skinning write straight to the final TangentBuffer.
				//
				// In both cases, the triangle pass reads the post-skinning tangents (via GetActiveTangentRWBuffer)
				// to obtain the original tangent-basis orientation sign. That buffer must be transitioned to SRV.
				if (DispatchData.RecomputeTangents.Section.bBlendUsingVertexColor)
				{
					DispatchData.RecomputeTangents.IntermediateTangentBuffer = DispatchData.PositionTracker.GetIntermediateTangentBuffer();
					check(DispatchData.RecomputeTangents.IntermediateTangentBuffer);

					TaskData.Transitions.Skinning.Write.Emplace(DispatchData.RecomputeTangents.IntermediateTangentBuffer);
					TaskData.Transitions.RecomputeTangents.Read.Emplace(DispatchData.RecomputeTangents.IntermediateTangentBuffer);
				}
				else
				{
					// Skinning wrote into the final TangentBuffer; the triangle pass reads it as SRV, then the parallel
					// dispatcher transitions it back to UAV (via TangentBuffers) for the vertex pass to overwrite.
					TaskData.Transitions.RecomputeTangents.Read.Emplace(DispatchData.TangentBuffer);
				}

				TaskData.Transitions.RecomputeTangents.Read.Emplace(DispatchData.PositionBuffer);

				if (GRecomputeTangentsParallelDispatch)
				{
					TaskData.Transitions.RecomputeTangents.Write.Add(DispatchData.GetIntermediateAccumulatedTangentBuffer());
				}

				TaskData.TangentDispatches.Emplace(DispatchItem);
			}

			TaskData.Transitions.Skinning.Write.Emplace(DispatchData.TangentBuffer);
			TaskData.Transitions.FinalRead.Emplace(DispatchData.TangentBuffer);
		}

		int32 NumBuffers = 0;
		NumBuffers += DispatchData.PositionBuffer ? 1 : 0;
		NumBuffers += DispatchData.PreviousPositionBuffer ? 1 : 0;
		NumBuffers += DispatchData.TangentBuffer ? 1 : 0;
		NumBuffers += DispatchData.RecomputeTangents.IntermediateTangentBuffer ? 1 : 0;
		NumBuffers += DispatchData.RecomputeTangents.IntermediateAccumulatedTangentBuffer ? 1 : 0;

		Stats.NumBuffers += NumBuffers;
		Stats.NumRayTracingBuffers    += Entry->Mode == EGPUSkinCacheEntryMode::RayTracing ? NumBuffers : 0;
		Stats.NumRayTracingDispatches += Entry->Mode == EGPUSkinCacheEntryMode::RayTracing ? 1 : 0;
	}

	INC_DWORD_STAT_BY(STAT_GPUSkinCache_NumBuffers, Stats.NumBuffers);
	INC_DWORD_STAT_BY(STAT_GPUSkinCache_NumRayTracingBuffers, Stats.NumRayTracingBuffers);
	INC_DWORD_STAT_BY(STAT_GPUSkinCache_NumRayTracingDispatches, Stats.NumRayTracingDispatches);
	INC_DWORD_STAT_BY(STAT_GPUSkinCache_NumDispatches, BatchDispatches.Num());

	if (GSkinCachePrintMemorySummary > 0)
	{
		GSkinCachePrintMemorySummary--;
		PrintMemorySummary();
	}

#if RHI_RAYTRACING
	if (IsGPUSkinCacheRayTracingSupported(FeatureLevel) && IsRayTracingEnabled())
	{
		for (FGPUSkinCacheEntry* SkinCacheEntry : PendingProcessRTGeometryEntries)
		{
			ProcessRayTracingGeometryToUpdate(*TaskData.RHICmdList, SkinCacheEntry);
		}
	}

	PendingProcessRTGeometryEntries.Reset();
#endif

	if (TaskData.DynamicMeshBoundsBlackboardStruct && DynamicMeshBoundsBuffer)
	{
		TaskData.DynamicMeshBoundsBlackboardStruct->Parameters = DynamicMeshBoundsBuffer->Update(*TaskData.RHICmdList, SlotsToUpdateMask);
		TaskData.DynamicMeshBoundsBlackboardStruct->bInitialized = true;
	}
}

void FGPUSkinCache::DispatchPassExecute(FTaskData& TaskData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGPUSkinCache::DispatchPassExecute);

	const int32 BatchCount = BatchDispatches.Num();
	INC_DWORD_STAT_BY(STAT_GPUSkinCache_TotalNumChunks, BatchCount);
	FRHICommandList& RHICmdList = *TaskData.RHICmdList;

	bool bCapture = BatchCount > 0 && GNumDispatchesToCapture > 0;
	RenderCaptureInterface::FScopedCapture RenderCapture(bCapture, &RHICmdList);
	GNumDispatchesToCapture -= bCapture ? 1 : 0;
	TaskData.SortedDispatches.Reserve(BatchCount);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildSortedDispatchList);

		for (int32 BatchIndex = 0; BatchIndex < BatchCount; ++BatchIndex)
		{
			FDispatchEntry& DispatchItem = BatchDispatches[BatchIndex];
			FGPUSkinCacheEntry* Entry = DispatchItem.SkinCacheEntry;
			FGPUSkinCacheEntry::FSectionDispatchData& DispatchData = Entry->DispatchData[DispatchItem.Section];

			if (EnumHasAnyFlags(DispatchData.DispatchFlags, EGPUSkinCacheDispatchFlags::PositionPrevious | EGPUSkinCacheDispatchFlags::Position))
			{
				FGPUSkinCacheCS::FPermutationDomain PermutationVector = FGPUSkinCacheCS::BuildPermutationVector(Entry->bUse16BitBoneWeight, Entry->bUse16BitBoneIndex, Entry->BoneInfluenceType, DispatchData.DeformationType);

				FSortedDispatchEntry SortedEntry;
				SortedEntry.ShaderIndex = PermutationVector.ToDimensionValueId();
				SortedEntry.BatchIndex  = BatchIndex;

				TaskData.SortedDispatches.Add(SortedEntry);
			}
		}

		Algo::Sort(TaskData.SortedDispatches, [](const FSortedDispatchEntry& A, const FSortedDispatchEntry& B)
		{
			if (A.ShaderIndex != B.ShaderIndex)
			{
				return A.ShaderIndex < B.ShaderIndex;
			}
			return A.BatchIndex < B.BatchIndex;
		});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateVertexFactoryDeclarations);

		for (FDispatchEntry& DispatchItem : BatchDispatches)
		{
			DispatchItem.SkinCacheEntry->UpdateVertexFactoryDeclaration(*TaskData.RHICmdList, DispatchItem.Section);
		}
	}

	FRHIUnorderedAccessView* BoundsBufferUAV = DynamicMeshBoundsBuffer ? DynamicMeshBoundsBuffer->GetUAV() : nullptr;

	{
		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> TransitionsInfos;
		const int32 NumToReserve = TaskData.Transitions.Skinning.Write.Num() + 1;

		TransitionsInfos.Reserve(NumToReserve);
		TaskData.Transitions.Skinning.Overlap.Reserve(NumToReserve);
		if (BoundsBufferUAV)
		{
			TransitionsInfos.Emplace(BoundsBufferUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute);
			TaskData.Transitions.Skinning.Overlap.Add(BoundsBufferUAV);
		}

		TransitionBufferUAVs(*TaskData.RHICmdList, TransitionsInfos, TaskData.Transitions.Skinning.Write, TaskData.Transitions.Skinning.Overlap);
		RHICmdList.BeginUAVOverlap(TaskData.Transitions.Skinning.Overlap);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GPUSkinCache_UpdateSkinningBatches);
		SCOPED_DRAW_EVENT(RHICmdList, GPUSkinCache_UpdateSkinningBatches);

		auto* GlobalShaderMap = GetGlobalShaderMap(GetFeatureLevel());

		int32 LastShaderIndex = -1;
		TShaderRef<FGPUSkinCacheCS> Shader;

		int32 SortedCount = TaskData.SortedDispatches.Num();
		for (const FSortedDispatchEntry& SortedEntry : TaskData.SortedDispatches)
		{
			if (SortedEntry.ShaderIndex != LastShaderIndex)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ChangeShader);

				LastShaderIndex = SortedEntry.ShaderIndex;
				Shader = TShaderMapRef<FGPUSkinCacheCS>(GlobalShaderMap, FGPUSkinCacheCS::FPermutationDomain(SortedEntry.ShaderIndex));

				check(Shader.IsValid());
				SetComputePipelineState(RHICmdList, Shader.GetComputeShader());
			}

			FDispatchEntry& DispatchEntry = BatchDispatches[SortedEntry.BatchIndex];
			FGPUSkinCacheEntry* Entry = DispatchEntry.SkinCacheEntry;
			FGPUSkinCacheEntry::FSectionDispatchData& DispatchData = Entry->DispatchData[DispatchEntry.Section];
			const FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = DispatchData.SourceVertexFactory->GetShaderData();

			SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, SkinCacheDispatch, GetEmitDrawEvents(), TEXT("%sSkinning%d%d%d%d Mesh=%s LOD=%d Chunk=%d InStreamStart=%d OutStart=%d Vert=%d Morph=%d/%d")
				, RHI_BREADCRUMB_FORCE_STRING_LITERAL(Entry->Mode == EGPUSkinCacheEntryMode::RayTracing ? TEXT("[RT]") : TEXT(""))
				, (int32)Entry->bUse16BitBoneIndex
				, (int32)Entry->bUse16BitBoneWeight
				, (int32)Entry->BoneInfluenceType
				, (int32)DispatchData.DeformationType
				, GetSkeletalMeshObjectDebugName(Entry->GPUSkin)
				, Entry->LOD
				, DispatchData.SectionIndex
				, DispatchData.InputStreamStart
				, DispatchData.OutputStreamStart
				, DispatchData.NumVertices
				, Entry->MorphBuffer != 0
				, DispatchData.MorphBufferOffset
			);

			uint32 VertexCountAlign64 = FMath::DivideAndRoundUp(DispatchData.NumVertices, (uint32)64);

			FSkinCacheRWBuffer* TangentBuffer = DispatchData.GetActiveTangentRWBuffer();
			check(TangentBuffer);

			FGPUSkinCacheCS::FParameters Parameters;
			Parameters.NumVertices = DispatchData.NumVertices;
			Parameters.SkinCacheStart = DispatchData.OutputStreamStart;
			Parameters.InputStreamStart = DispatchData.InputStreamStart;
			Parameters.NumBoneInfluences = DispatchData.NumBoneInfluences;
			Parameters.InputWeightIndexSize = Entry->InputWeightIndexSize;
			Parameters.InputWeightStart = DispatchData.InputWeightStart;
			Parameters.InputWeightStride = Entry->InputWeightStride;
			Parameters.InputWeightStream = Entry->InputWeightStreamSRV ? Entry->InputWeightStreamSRV : GNullVertexBuffer.VertexBufferSRV;
			Parameters.InputWeightLookupStream = Entry->InputWeightLookupStreamSRV;
			Parameters.PositionInputBuffer = DispatchData.PositionBufferSRV;
			Parameters.TangentInputBuffer = DispatchData.TangentBufferSRV;
			Parameters.DynamicBoundsOffset = DispatchData.DynamicBoundsOffset;
			Parameters.OutBoundsBufferUAV = BoundsBufferUAV;
			Parameters.TangentBufferUAV = TangentBuffer->Buffer.UAV;

			if (DispatchData.DeformationType == EGPUSkinDeformationType::Morph)
			{
				Parameters.MorphBuffer = Entry->MorphBuffer;
				Parameters.MorphBufferOffset = DispatchData.MorphBufferOffset;
			}
			else if (DispatchData.DeformationType == EGPUSkinDeformationType::Cloth)
			{
				Parameters.ClothBuffer = Entry->ClothBuffer;
				Parameters.ClothPositionsAndNormalsBuffer = DispatchData.ClothPositionsAndNormalsBuffer;
				Parameters.ClothBufferOffset = DispatchData.ClothBufferOffset;
				Parameters.ClothBlendWeight = DispatchData.ClothBlendWeight;
				Parameters.ClothToLocal = DispatchData.ClothToLocal;
				Parameters.ClothNumInfluencesPerVertex = DispatchData.ClothNumInfluencesPerVertex;
				Parameters.WorldScale = DispatchData.ClothWorldScale;
			}

			if (EnumHasAnyFlags(DispatchData.DispatchFlags, EGPUSkinCacheDispatchFlags::PositionPrevious))
			{
				const FVertexBufferAndSRV& PrevBoneBuffer = ShaderData.GetBoneBufferForReading(true);

				Parameters.BoneMatrices = PrevBoneBuffer.VertexBufferSRV;
				Parameters.PositionBufferUAV = DispatchData.GetPreviousPositionRWBuffer()->Buffer.UAV;

				INC_DWORD_STAT_BY(STAT_GPUSkinCache_TotalNumVertices, VertexCountAlign64 * 64);
				SetShaderParameters(RHICmdList, Shader, Shader.GetComputeShader(), Parameters);
				RHICmdList.DispatchComputeShader(VertexCountAlign64, 1, 1);
			}

			if (EnumHasAnyFlags(DispatchData.DispatchFlags, EGPUSkinCacheDispatchFlags::Position))
			{
				const FVertexBufferAndSRV& BoneBuffer = ShaderData.GetBoneBufferForReading(false);

				Parameters.BoneMatrices = BoneBuffer.VertexBufferSRV;
				Parameters.PositionBufferUAV = DispatchData.GetPositionRWBuffer()->Buffer.UAV;

				INC_DWORD_STAT_BY(STAT_GPUSkinCache_TotalNumVertices, VertexCountAlign64 * 64);
				SetShaderParameters(RHICmdList, Shader, Shader.GetComputeShader(), Parameters);
				RHICmdList.DispatchComputeShader(VertexCountAlign64, 1, 1);
			}

			check(DispatchData.PreviousPositionBuffer != DispatchData.PositionBuffer);
		}

		if (Shader.IsValid())
		{
			UnsetShaderUAVs(RHICmdList, Shader, Shader.GetComputeShader());
		}
	}

	RHICmdList.EndUAVOverlap(TaskData.Transitions.Skinning.Overlap);

	if (!TaskData.Transitions.RecomputeTangents.Read.IsEmpty())
	{
		TArray<FRHIUnorderedAccessView*> IntermediateAccumulatedTangentBuffersToOverlap;
		TransitionBuffers(RHICmdList, TaskData.Transitions.RecomputeTangents.Read, ERHIAccess::SRVCompute);
		TransitionBufferUAVs(RHICmdList, TaskData.Transitions.RecomputeTangents.Write, IntermediateAccumulatedTangentBuffersToOverlap);
		RHICmdList.BeginUAVOverlap(IntermediateAccumulatedTangentBuffersToOverlap);

		{
			SCOPED_DRAW_EVENT(RHICmdList, GPUSkinCache_RecomputeTangentsBatches);
			FSkinCacheRWBuffer* StagingBuffer = nullptr;
			TArray<FSkinCacheRWBuffer*> TangentBuffers;

			if (GRecomputeTangentsParallelDispatch)
			{
				TangentBuffers.Reserve(TaskData.TangentDispatches.Num());
			}

			for (const FDispatchEntry& DispatchItem : TaskData.TangentDispatches)
			{
				DispatchUpdateSkinTangentsTrianglePass(RHICmdList, DispatchItem.SkinCacheEntry, DispatchItem.Section, StagingBuffer);

				if (GRecomputeTangentsParallelDispatch)
				{
					TangentBuffers.Add(DispatchItem.SkinCacheEntry->DispatchData[DispatchItem.Section].GetTangentRWBuffer());
				}
				else
				{
					DispatchUpdateSkinTangentsVertexPass(RHICmdList, DispatchItem.SkinCacheEntry, DispatchItem.Section, StagingBuffer);
				}
			}

			if (GRecomputeTangentsParallelDispatch)
			{
				TArray<FRHIUnorderedAccessView*> TangentBuffersToOverlap;
				TransitionBuffers(RHICmdList, TaskData.Transitions.RecomputeTangents.Write, ERHIAccess::UAVCompute);
				TransitionBufferUAVs(RHICmdList, TangentBuffers, TangentBuffersToOverlap);
				RHICmdList.BeginUAVOverlap(TangentBuffersToOverlap);

				for (const FDispatchEntry& DispatchItem : TaskData.TangentDispatches)
				{
					DispatchUpdateSkinTangentsVertexPass(RHICmdList, DispatchItem.SkinCacheEntry, DispatchItem.Section, StagingBuffer);
				}

				RHICmdList.EndUAVOverlap(TangentBuffersToOverlap);
			}
		}

		RHICmdList.EndUAVOverlap(IntermediateAccumulatedTangentBuffersToOverlap);
	}

	{
		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> TransitionInfos;

		TRACE_CPUPROFILER_EVENT_SCOPE(TransitionAllToReadable);
		const uint32 NextTransitionFence = GetNextTransitionFence();
		const ERHIAccess ReadState = ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask;

		TransitionInfos.Reserve(TaskData.Transitions.FinalRead.Num() + 1);
		if (BoundsBufferUAV)
		{
			TransitionInfos.Emplace(BoundsBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask);
		}

		for (FSkinCacheRWBuffer* Buffer : TaskData.Transitions.FinalRead)
		{
			if (!Buffer->UpdateFence(NextTransitionFence))
			{
				continue;
			}

			if (Buffer->AccessState != ReadState)
			{
				TransitionInfos.Add(Buffer->UpdateAccessState(ReadState));
			}
		}

		if (TaskData.Pipeline == ERHIPipeline::Graphics)
		{
			RHICmdList.Transition(TransitionInfos, ERHIPipeline::Graphics, IsGPUSkinCacheRayTracingSupported(FeatureLevel) && !FRDGBuilder::IsImmediateMode() ? ERHIPipeline::All : ERHIPipeline::Graphics);
		}
		else
		{
			// When async compute is enabled the transition also acts as the fence back to the graphics pipe.
			check(TaskData.Pipeline == ERHIPipeline::AsyncCompute);
			TaskData.AsyncComputeTransition = RHICreateTransition({ ERHIPipeline::AsyncCompute, ERHIPipeline::All, ERHITransitionCreateFlags::None, TransitionInfos });
			RHICmdList.BeginTransition(TaskData.AsyncComputeTransition);
			RHICmdList.EndTransition(TaskData.AsyncComputeTransition);
			RHICmdList.SetTrackedAccess(TransitionInfos, ERHIPipeline::All);
		}
	}

	BatchDispatches.Reset();
}

void FGPUSkinCache::ProcessEntry(FRHICommandList& RHICmdList, const FProcessEntryInputs& Inputs, FGPUSkinCacheEntry*& InOutEntry)
{
	if (FlushCounter < GGPUSkinCacheFlushCounter)
	{
		FlushCounter = GGPUSkinCacheFlushCounter;
		InvalidateAllEntries();
	}

	INC_DWORD_STAT(STAT_GPUSkinCache_NumSectionsProcessed);

	FSkeletalMeshRenderData& SkelMeshRenderData = Inputs.Skin->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData& LodData = SkelMeshRenderData.LODRenderData[Inputs.LODIndex];

	const int32 DynamicBoundsStartOffset = Inputs.Skin->GetDynamicBoundsStartOffset(Inputs.LODIndex);

	// If the LOD changed, the entry has to be invalidated
	if (InOutEntry && !InOutEntry->IsValid(Inputs.Skin, Inputs.LODIndex))
	{
		Release(InOutEntry);
		InOutEntry = nullptr;
	}

	const bool bSetupSections = !InOutEntry || !InOutEntry->IsTargetVertexFactoryValid(Inputs.TargetVertexFactory);

	EGPUSkinCacheBufferBits BufferBits = EGPUSkinCacheBufferBits::None;

	// IntermediateAccumulatedTangents buffer is needed if mesh has at least one section needing recomputing tangents.
	TArray<FGPUSkinCacheEntry::FRecomputeTangentSection, TInlineAllocator<16>> RecomputeTangentSections;
	uint32 IntermediateAccumulatedTangentBufferSize = 0;

	if (bSetupSections)
	{
		if (Inputs.Mode == EGPUSkinCacheEntryMode::Raster)
		{
			const int32 SkinCacheRecomputeTangents = GetGPUSkinCacheRecomputeTangentsMode(FeatureLevel);
			if (SkinCacheRecomputeTangents > 0)
			{
				RecomputeTangentSections.Reserve(LodData.RenderSections.Num());

				for (int32 Index = 0; Index < LodData.RenderSections.Num(); ++Index)
				{
					FGPUSkinCacheEntry::FRecomputeTangentSection& RecomputeTangentSection = RecomputeTangentSections.Emplace_GetRef();

					const FSkelMeshRenderSection& RenderSection = LodData.RenderSections[Index];
					if (LodData.MultiSizeIndexContainer.GetIndexBuffer() && (SkinCacheRecomputeTangents == 1 || RenderSection.bRecomputeTangent))
					{
						RecomputeTangentSection.bEnable = true;
						RecomputeTangentSection.IntermediateBufferOffset = IntermediateAccumulatedTangentBufferSize;
						IntermediateAccumulatedTangentBufferSize += RenderSection.GetNumVertices();

						// The IntermediateTangents (skinned-tangent copy) buffer is only needed when the per-vertex pass
						// has to blend the recomputed tangent with the original skinned tangent via a vertex-color mask.
						if (RenderSection.RecomputeTangentsVertexMaskChannel < ESkinVertexColorChannel::None)
						{
							BufferBits |= EGPUSkinCacheBufferBits::IntermediateTangents;
							RecomputeTangentSection.bBlendUsingVertexColor = true;
						}
					}
				}
			}

			BufferBits |= EGPUSkinCacheBufferBits::PositionPrevious;
		}
	}

	// Recreate logic only matters when re-using an entry.
	const bool bRecreating = InOutEntry && Inputs.bRecreating;

	// Try to allocate a new entry
	if (!InOutEntry)
	{
		// If we're allocating the IntermediateTangents (skinned-tangent copy) buffer, we must also be allocating
		// the accumulator buffer — the blend path is a strict superset of the plain recompute path.
		check(!EnumHasAnyFlags(BufferBits, EGPUSkinCacheBufferBits::IntermediateTangents) || IntermediateAccumulatedTangentBufferSize > 0);

		FRWBuffersAllocationInitializer Initializer
		{
			  .BufferBits = BufferBits
			, .NumVertices = Inputs.TargetVertexFactory->GetNumVertices()
			, .IntermediateAccumulatedTangentsSize = IntermediateAccumulatedTangentBufferSize
		};

		// OpenGL ES does not support writing to RGBA16_SNORM images, so use the packed format instead.
		if (IsOpenGLPlatform(GMaxRHIShaderPlatform))
		{
			Initializer.TangentFormat = PF_R16G16B16A16_SINT;
		}
		else if (GPixelFormats[Inputs.TargetVertexFactory->GetTangentFormat()].BlockBytes == 4)
		{
			Initializer.TangentFormat = PF_R8G8B8A8_SNORM;
		}
		else
		{
			Initializer.TangentFormat = PF_R16G16B16A16_SNORM;
		}

		const uint32 BufferSize = Initializer.GetBufferSize();
		UsedMemoryInBytes += BufferSize;
		INC_MEMORY_STAT_BY(STAT_GPUSkinCache_TotalMemUsed, BufferSize);

		FRWBuffersAllocation* BufferAllocation = new FRWBuffersAllocation(RHICmdList, Initializer, Inputs.Skin->GetAssetPathName(Inputs.LODIndex));
		Allocations.Add(BufferAllocation);

		InOutEntry = new FGPUSkinCacheEntry(this, Inputs.Skin, BufferAllocation, Inputs.LODIndex, Inputs.Mode);
		InOutEntry->GPUSkin = Inputs.Skin;
		Entries.Add(InOutEntry);
	}

	for (const FProcessEntrySection& Section : Inputs.Sections)
	{
		const int32 SectionIndex = Section.SectionIndex;

		if (bSetupSections)
		{
			FGPUSkinCacheEntry::FRecomputeTangentSection RecomputeTangentSection;

			if (!RecomputeTangentSections.IsEmpty())
			{
				RecomputeTangentSection = RecomputeTangentSections[SectionIndex];
			}

			InOutEntry->SetupSection(
				SectionIndex,
				Section.Section,
				Section.SourceVertexFactory,
				RecomputeTangentSection,
				DynamicBoundsStartOffset >= 0 ? DynamicBoundsStartOffset + SectionIndex : -1);
		}

		FGPUSkinCacheEntry::FSectionDispatchData& SectionDispatchData = InOutEntry->DispatchData[SectionIndex];
		
		SectionDispatchData.DeformationType = EGPUSkinDeformationType::Default;
		
		if (Inputs.MorphVertexBuffer && Inputs.MorphVertexBuffer->SectionIds.Contains(SectionIndex))
		{
			InOutEntry->MorphBuffer = Inputs.MorphVertexBuffer->GetSRV();
			check(InOutEntry->MorphBuffer);

			SectionDispatchData.MorphBufferOffset = Section.Section->BaseVertexIndex;
			SectionDispatchData.DeformationType = EGPUSkinDeformationType::Morph;
		}

		if (Inputs.ClothVertexBuffer && Section.ClothSimulationData)
		{
			InOutEntry->ClothBuffer = Inputs.ClothVertexBuffer->GetSRV();
			check(InOutEntry->ClothBuffer);

			FVertexBufferAndSRV ClothPositionAndNormalsBuffer;
			TSkeletalMeshVertexData<FVector3f> VertexAndNormalData(true);
			const FClothSimulData* ClothSimulationData = Section.ClothSimulationData;

			if (!ClothSimulationData->Positions.IsEmpty())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SetupCloth);

				// Use the same cloth LOD mapping (= 0 bias) to get the number of Deformer weights.
				const uint32 NumWrapDeformerWeights = SectionDispatchData.Section->ClothMappingDataLODs.Num() ? SectionDispatchData.Section->ClothMappingDataLODs[0].Num() : 0;
				check(NumWrapDeformerWeights % SectionDispatchData.NumVertices == 0);
				SectionDispatchData.ClothNumInfluencesPerVertex = NumWrapDeformerWeights / SectionDispatchData.NumVertices;

				check(ClothSimulationData->Positions.Num() == ClothSimulationData->Normals.Num());
				VertexAndNormalData.ResizeBuffer(2 * ClothSimulationData->Positions.Num());

				if (SectionIndex < Inputs.ClothVertexBuffer->GetClothIndexMapping().Num())
				{
					check(ClothSimulationData->LODIndex != INDEX_NONE && ClothSimulationData->LODIndex <= Inputs.LODIndex);

					const FClothBufferIndexMapping& ClothBufferIndexMapping = Inputs.ClothVertexBuffer->GetClothIndexMapping()[SectionIndex];
					const uint32 ClothLODBias = static_cast<uint32>(Inputs.LODIndex - ClothSimulationData->LODIndex);
					const uint32 ClothBufferOffset = ClothBufferIndexMapping.MappingOffset + ClothBufferIndexMapping.LODBiasStride * ClothLODBias;

					// Set the buffer offset depending on whether enough deformer mapping data exists (RaytracingMinLOD/RaytracingLODBias/ClothLODBiasMode settings)
					const uint32 NumVertices = SectionDispatchData.NumVertices;
					const uint32 NumInfluences = NumVertices ? ClothBufferIndexMapping.LODBiasStride / NumVertices : 1;

					SectionDispatchData.ClothBufferOffset = ClothBufferOffset + (NumVertices * NumInfluences) <= Inputs.ClothVertexBuffer->GetNumVertices()
						// If the offset is valid, set the calculated LODBias offset
						? ClothBufferOffset
						// Otherwise fallback to a 0 ClothLODBias to prevent from reading pass the buffer (but still raytrace broken shadows/reflections/etc.)
						: ClothBufferIndexMapping.MappingOffset;
				}

				{
					const uint32 Stride = VertexAndNormalData.GetStride();
					FVector3f* Data = reinterpret_cast<FVector3f*>(VertexAndNormalData.GetDataPointer());

					check(Stride * VertexAndNormalData.GetNumVertices() == sizeof(FVector3f) * 2 * ClothSimulationData->Positions.Num());

					for (int32 Index = 0; Index < ClothSimulationData->Positions.Num(); ++Index)
					{
						*(Data + Index * 2)     = ClothSimulationData->Positions[Index];
						*(Data + Index * 2 + 1) = ClothSimulationData->Normals[Index];
					}
				}

				FResourceArrayInterface* ResourceArray = VertexAndNormalData.GetResourceArray();

				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex(TEXT("ClothPositionAndNormalsBuffer"), ResourceArray->GetResourceDataSize())
					.AddUsage(BUF_Static | BUF_ShaderResource)
					.SetInitActionResourceArray(ResourceArray)
					.DetermineInitialState();

				ClothPositionAndNormalsBuffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
				ClothPositionAndNormalsBuffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
					ClothPositionAndNormalsBuffer.VertexBufferRHI, 
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_G32R32F));

				SectionDispatchData.ClothPositionsAndNormalsBuffer = ClothPositionAndNormalsBuffer.VertexBufferSRV;
				SectionDispatchData.DeformationType = EGPUSkinDeformationType::Cloth;
			}
			else
			{
				UE_LOGF(LogSkinCache, Error, "Cloth sim data is missing on mesh %ls", *GetSkeletalMeshObjectName(Inputs.Skin));
			}

			SectionDispatchData.ClothToLocal = Section.ClothToLocal;
			SectionDispatchData.ClothBlendWeight = Inputs.ClothBlendWeight;
			SectionDispatchData.ClothWorldScale = Inputs.ClothWorldScale;
		}

		// Need to update the previous bone buffer pointer, so logic that checks if the bone buffers changed (FGPUSkinCache::FRWBufferTracker::Find)
		// doesn't invalidate the previous frame position data. Recreating the render state will have generated new bone buffers.
		if (bRecreating && Inputs.Mode == EGPUSkinCacheEntryMode::Raster)
		{
			const FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = Section.SourceVertexFactory->GetShaderData();

			if (ShaderData.HasBoneBufferForReading(true))
			{
				SectionDispatchData.PositionTracker.UpdatePreviousBoneBuffer(ShaderData.GetBoneBufferForReading(true), ShaderData.GetRevisionNumber(true));
			}
		}

		// Check if the combo of skin cache entry and section index already exists, if so use the entry and update to latest revision number.
		if (SectionDispatchData.RevisionNumber != 0)
		{
			SectionDispatchData.RevisionNumber = FMath::Max(InOutEntry->DispatchData[SectionIndex].RevisionNumber, Inputs.CurrentRevisionNumber);
		}
		else
		{
			SectionDispatchData.RevisionNumber = Inputs.CurrentRevisionNumber;
			BatchDispatches.Add({ InOutEntry, uint32(SectionIndex) });
		}
	}

	InOutEntry->TargetVertexFactory = Inputs.TargetVertexFactory;
	InOutEntry->bQueuedForDispatch = true;

#if RHI_RAYTRACING
	if (!Inputs.Skin->ShouldUseSeparateSkinCacheEntryForRayTracing() || Inputs.Mode == EGPUSkinCacheEntryMode::RayTracing)
	{
		// This is a RT skin cache entry
		PendingProcessRTGeometryEntries.Add(InOutEntry);
	}
#endif
}

bool FGPUSkinCache::IsGPUSkinCacheEnabled(ERHIFeatureLevel::Type FeatureLevel)
{
	// Override runtime mode for cases where either inline or skin cache are not available.
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return (GEnableGPUSkinCache > 0 || !AreInlineSkinningShadersEnabled(ShaderPlatform)) && IsGPUSkinCacheAvailable(ShaderPlatform);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 FGPUSkinCache::GetGPUSkinCacheRecomputeTangentsMode(ERHIFeatureLevel::Type FeatureLevel)
{
	return IsGPUSkinCacheEnabled(FeatureLevel) ? GSkinCacheRecomputeTangents : 0;
}

bool FGPUSkinCache::IsGPUSkinCacheRayTracingSupported(ERHIFeatureLevel::Type FeatureLevel)
{
#if RHI_RAYTRACING
	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Geometry.SupportSkeletalMeshes"));
	static const bool SupportSkeletalMeshes = CVar->GetInt() != 0;
	return IsRayTracingAllowed() && SupportSkeletalMeshes && IsGPUSkinCacheEnabled(FeatureLevel);
#else
	return false;
#endif
}

#if RHI_RAYTRACING

void FGPUSkinCache::ProcessRayTracingGeometryToUpdate(FRHICommandList& RHICmdList, FGPUSkinCacheEntry* SkinCacheEntry)
{
	check(IsGPUSkinCacheRayTracingSupported(FeatureLevel) && IsRayTracingEnabled());

	if (SkinCacheEntry && SkinCacheEntry->GPUSkin && SkinCacheEntry->GPUSkin->bSupportRayTracing)
	{
 		TArray<FBufferRHIRef> VertexBuffers;
 		SkinCacheEntry->GetRayTracingSegmentVertexBuffers(VertexBuffers);

		const int32 LODIndex = SkinCacheEntry->LOD;
		FSkeletalMeshRenderData& SkelMeshRenderData = SkinCacheEntry->GPUSkin->GetSkeletalMeshRenderData();
		check(LODIndex < SkelMeshRenderData.LODRenderData.Num());
		FSkeletalMeshLODRenderData& LODModel = SkelMeshRenderData.LODRenderData[LODIndex];

 		SkinCacheEntry->GPUSkin->UpdateRayTracingGeometry(RHICmdList, LODModel, LODIndex, VertexBuffers);
	}
}

#endif

void FGPUSkinCache::Dequeue(FGPUSkinCacheEntry* SkinCacheEntry)
{
	if (!SkinCacheEntry)
	{
		return;
	}
	
	FGPUSkinCache* SkinCache = SkinCacheEntry->SkinCache;
	checkf(SkinCache, TEXT("Attempting to dequeue a skin cache entry not linked to a parent skin cache"));
	
	SkinCache->PendingProcessRTGeometryEntries.Remove(SkinCacheEntry);

	if (SkinCacheEntry->bQueuedForDispatch)
	{
		for (int32 Index = 0; Index < SkinCache->BatchDispatches.Num(); )
		{
			FDispatchEntry& Dispatch = SkinCache->BatchDispatches[Index];
			
			if (Dispatch.SkinCacheEntry == SkinCacheEntry)
			{
				// Reset the revision, may not kick off the update otherwise
				SkinCacheEntry->DispatchData[Dispatch.Section].RevisionNumber = 0;
				
				SkinCache->BatchDispatches.RemoveAtSwap(Index);

				// Continue to search for other sections associated with this skin cache entry.
			}
			else
			{
				++Index;
			}
		}
		
		SkinCacheEntry->bQueuedForDispatch = false;
	}
}

void FGPUSkinCache::Release(FGPUSkinCacheEntry*& SkinCacheEntry)
{
	if (SkinCacheEntry)
	{
		Dequeue(SkinCacheEntry);

		ReleaseSkinCacheEntry(SkinCacheEntry);
		SkinCacheEntry = nullptr;
	}
}

void FGPUSkinCache::ReleaseSkinCacheEntry(FGPUSkinCacheEntry* SkinCacheEntry)
{
	FGPUSkinCache* SkinCache = SkinCacheEntry->SkinCache;

	FRWBuffersAllocation* PositionAllocation = SkinCacheEntry->PositionAllocation;
	if (PositionAllocation)
	{
		uint32 BufferSize = PositionAllocation->GetBufferSize();
		SkinCache->UsedMemoryInBytes -= BufferSize;
		DEC_MEMORY_STAT_BY(STAT_GPUSkinCache_TotalMemUsed, BufferSize);

		SkinCache->Allocations.Remove(PositionAllocation);

		delete PositionAllocation;

		SkinCacheEntry->PositionAllocation = nullptr;
	}

	SkinCache->Entries.RemoveSingleSwap(SkinCacheEntry, EAllowShrinking::No);
	delete SkinCacheEntry;
}

bool FGPUSkinCache::IsEntryValid(FGPUSkinCacheEntry* SkinCacheEntry, int32 Section)
{
	return SkinCacheEntry && SkinCacheEntry->IsSectionValid(Section);
}

void FGPUSkinCache::InvalidateAllEntries()
{
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		Entries[Index]->LOD = -1;
	}

	for (int32 Index = 0; Index < StagingBuffers.Num(); ++Index)
	{
		StagingBuffers[Index].Release();
	}
	StagingBuffers.SetNum(0, EAllowShrinking::No);
	SET_MEMORY_STAT(STAT_GPUSkinCache_TangentsIntermediateMemUsed, 0);
}

FRWBuffer* FGPUSkinCache::GetPositionBuffer(FRDGBuilder& GraphBuilder, FGPUSkinCacheEntry const* Entry, uint32 SectionIndex)
{
	if (Entry)
	{
		FGPUSkinCacheEntry::FSectionDispatchData const& DispatchData = Entry->GetDispatchData()[SectionIndex];
		FSkinCacheRWBuffer* SkinCacheRWBuffer = DispatchData.PositionBuffer;
		return SkinCacheRWBuffer != nullptr ? &SkinCacheRWBuffer->Buffer : nullptr;
	}
	return nullptr;
}

FRWBuffer* FGPUSkinCache::GetPreviousPositionBuffer(FRDGBuilder& GraphBuilder, FGPUSkinCacheEntry const* Entry, uint32 SectionIndex)
{
	if (Entry)
	{
		FGPUSkinCacheEntry::FSectionDispatchData const& DispatchData = Entry->GetDispatchData()[SectionIndex];
		FSkinCacheRWBuffer* SkinCacheRWBuffer = DispatchData.PreviousPositionBuffer;
		return SkinCacheRWBuffer != nullptr ? &SkinCacheRWBuffer->Buffer : nullptr;
	}
	return nullptr;
}

FRWBuffer* FGPUSkinCache::GetTangentBuffer(FRDGBuilder& GraphBuilder, FGPUSkinCacheEntry const* Entry, uint32 SectionIndex)
{
	if (Entry)
	{
		FGPUSkinCacheEntry::FSectionDispatchData const& DispatchData = Entry->GetDispatchData()[SectionIndex];
		FSkinCacheRWBuffer* SkinCacheRWBuffer = DispatchData.TangentBuffer;
		return SkinCacheRWBuffer != nullptr ? &SkinCacheRWBuffer->Buffer : nullptr;
	}
	return nullptr;
}

void FGPUSkinCache::UpdateSkinWeightBuffer(FGPUSkinCacheEntry* Entry)
{
	if (Entry)
	{
		// Dequeue any pending updates to the entries
		//   Skin weight updates reinitialize the vertex factories RHI state, which will in turn invalidate the bone data
		//   for any pending update in the dispatch list.
		FGPUSkinCache::Dequeue(Entry);
		
		Entry->UpdateSkinWeightBuffer();
	}
}

void FGPUSkinCache::SetEntryGPUSkin(FGPUSkinCacheEntry* Entry, FSkeletalMeshObject* Skin)
{
	if (Entry)
	{
		// Dequeue any pending updates to the entries
		//   When transferring owner there is a small window in which we may still reference the original vertex factory
		//   before the new owner has updated the entry. If the entry is pending an update in the dispatch list, we risk
		//   accessing invalid bone data if the original owner is released. The original owner *does* dequeue on release,
		//   however, the transfer nulls the old entry.
		FGPUSkinCache::Dequeue(Entry);

		// Reset target VF pointer to ensure IsTargetFactoryValid returns false when entry will get updated in next ProcessEntry call
		Entry->TargetVertexFactory = nullptr;
		Entry->GPUSkin = Skin;
	}
}

void FGPUSkinCache::OnCVarUpdate()
{
	if (GSkinCacheRecomputeTangents == 1 && GStoreDuplicatedVerticesForRecomputeTangents == 0)
	{
		UE_LOGF(LogSkinCache, Warning, "r.SkinCache.RecomputeTangents is set to 1 to update all skinned objects but duplicated vertices are not are not always stored. Set r.SkinCache.RecomputeTangents to 2 or r.SkinCache.StoreDuplicatedVerticesForRecomputeTangents to 1.");
	}

	ENQUEUE_RENDER_COMMAND(DoEnableSkinCaching)(UE::RenderCommandPipe::SkeletalMesh, [](FRHICommandList& RHICmdList)
	{
		++GGPUSkinCacheFlushCounter;
	});

	TArray<UActorComponent*> Components;
	for (USkinnedMeshComponent* Component : TObjectRange<USkinnedMeshComponent>())
	{
		if (Component->IsRegistered() && Component->IsRenderStateCreated())
		{
			Components.Emplace(Component);
		}
	}

	FGlobalComponentRecreateRenderStateContext Context(Components);
}

void FGPUSkinCache::PrintMemorySummary() const
{
	UE_LOGF(LogSkinCache, Display, "======= Skin Cache Memory Usage Summary =======");

	uint64 TotalMemInBytes = 0;
	for (int32 i = 0; i < Entries.Num(); ++i)
	{
		FGPUSkinCacheEntry* Entry = Entries[i];
		if (Entry)
		{
			FString RecomputeTangentSections;

			for (int32 DispatchIdx = 0; DispatchIdx < Entry->DispatchData.Num(); ++DispatchIdx)
			{
				const FGPUSkinCacheEntry::FSectionDispatchData& Data = Entry->DispatchData[DispatchIdx];
				if (Data.RecomputeTangents.IndexBuffer)
				{
					if (RecomputeTangentSections.IsEmpty())
					{
						RecomputeTangentSections = TEXT("[Section]") + FString::FromInt(Data.SectionIndex);
					}
					else
					{
						RecomputeTangentSections = RecomputeTangentSections + TEXT("/") + FString::FromInt(Data.SectionIndex);
					}
				}
			}

			if (RecomputeTangentSections.IsEmpty())
			{
				RecomputeTangentSections = TEXT("Off");
			}

			uint64 MemInBytes = Entry->PositionAllocation ? Entry->PositionAllocation->GetBufferSize() : 0;
			uint64 TangentsInBytes = (Entry->PositionAllocation && Entry->PositionAllocation->GetTangentBuffer()) ? Entry->PositionAllocation->GetTangentBuffer()->Buffer.NumBytes : 0;
			uint64 IntermediateTangentsInBytes = (Entry->PositionAllocation && Entry->PositionAllocation->GetIntermediateTangentBuffer()) ? Entry->PositionAllocation->GetIntermediateTangentBuffer()->Buffer.NumBytes : 0;
			uint64 IntermediateAccumulatedTangentsInBytes = (Entry->PositionAllocation && Entry->PositionAllocation->GetIntermediateAccumulatedTangentBuffer()) ? Entry->PositionAllocation->GetIntermediateAccumulatedTangentBuffer()->Buffer.NumBytes : 0;

			UE_LOGF(LogSkinCache, Display, "   SkinCacheEntry_%d: %lsMesh=%ls, LOD=%d, RecomputeTangent=%ls, Mem=%.3fKB (Tangents=%.3fKB, InterTangents=%.3fKB, InterAccumTangents=%.3fKB)"
				, i
				, Entry->Mode == EGPUSkinCacheEntryMode::RayTracing ? TEXT("[RT]") : TEXT("")
				, *GetSkeletalMeshObjectName(Entry->GPUSkin)
				, Entry->LOD
				, *RecomputeTangentSections
				, MemInBytes / 1024.f
				, TangentsInBytes / 1024.f
				, IntermediateTangentsInBytes / 1024.f
				, IntermediateAccumulatedTangentsInBytes / 1024.f
			);

			TotalMemInBytes += MemInBytes;
		}
	}
	ensure(TotalMemInBytes == UsedMemoryInBytes);

	UE_LOGF(LogSkinCache, Display, "Used: %.3fMB", UsedMemoryInBytes / MBSize);
	UE_LOGF(LogSkinCache, Display, "===============================================");
}

FString FGPUSkinCache::GetSkeletalMeshObjectName(const FSkeletalMeshObject* GPUSkin) const
{
	FString Name = "None";
	if (GPUSkin)
	{
#if !UE_BUILD_SHIPPING
		Name = GPUSkin->DebugName.ToString();
#endif // !UE_BUILD_SHIPPING
	}
	return Name;
}

FDebugName FGPUSkinCache::GetSkeletalMeshObjectDebugName(const FSkeletalMeshObject* GPUSkin) const
{
	if (!GPUSkin)
		return {};

	return GPUSkin->GetDebugName();
}

FColor FGPUSkinCache::GetVisualizationDebugColor(const FName& GPUSkinCacheVisualizationMode, FGPUSkinCacheEntry* Entry, FGPUSkinCacheEntry* RayTracingEntry, uint32 SectionIndex)
{
	const FGPUSkinCacheVisualizationData& VisualizationData = GetGPUSkinCacheVisualizationData();
	if (VisualizationData.IsActive())
	{
		// Color coding should match DrawVisualizationInfoText function
		FGPUSkinCacheVisualizationData::FModeType ModeType = VisualizationData.GetActiveModeType();

		if (ModeType == FGPUSkinCacheVisualizationData::FModeType::Overview)
		{
			bool bRecomputeTangent = Entry && Entry->DispatchData[SectionIndex].RecomputeTangents.IndexBuffer;
			return Entry ? 
				   (bRecomputeTangent ? GEngine->GPUSkinCacheVisualizationRecomputeTangentsColor.QuantizeRound() : GEngine->GPUSkinCacheVisualizationIncludedColor.QuantizeRound()) : 
				   GEngine->GPUSkinCacheVisualizationExcludedColor.QuantizeRound();
		}
		else if (ModeType == FGPUSkinCacheVisualizationData::FModeType::Memory)
		{
			uint64 MemoryInBytes = (Entry && Entry->PositionAllocation) ? Entry->PositionAllocation->GetBufferSize() : 0;
#if RHI_RAYTRACING
			if (RayTracingEntry && RayTracingEntry != Entry)
			{
				// Separate ray tracing entry
				MemoryInBytes += RayTracingEntry->PositionAllocation ? RayTracingEntry->PositionAllocation->GetBufferSize() : 0;
			}
#endif
			float MemoryInMB = MemoryInBytes / MBSize;

			return MemoryInMB < GEngine->GPUSkinCacheVisualizationLowMemoryThresholdInMB ? GEngine->GPUSkinCacheVisualizationLowMemoryColor.QuantizeRound() :
				  (MemoryInMB < GEngine->GPUSkinCacheVisualizationHighMemoryThresholdInMB ? GEngine->GPUSkinCacheVisualizationMidMemoryColor.QuantizeRound() : GEngine->GPUSkinCacheVisualizationHighMemoryColor.QuantizeRound());
		}
		else if (ModeType == FGPUSkinCacheVisualizationData::FModeType::RayTracingLODOffset)
		{
	#if RHI_RAYTRACING
			int32 LODOffset = (Entry && RayTracingEntry) ? (RayTracingEntry->LOD - Entry->LOD) : 0;
			check (LODOffset >= 0);
			const TArray<FLinearColor>& VisualizationColors = GEngine->GPUSkinCacheVisualizationRayTracingLODOffsetColors;
			if (VisualizationColors.Num() > 0)
			{
				int32 Index = VisualizationColors.IsValidIndex(LODOffset) ? LODOffset : (VisualizationColors.Num()-1);
				return VisualizationColors[Index].QuantizeRound();
			}
	#endif
		}
	}

	return FColor::White;
}

void FGPUSkinCache::DrawVisualizationInfoText(const FName& GPUSkinCacheVisualizationMode, FScreenMessageWriter& ScreenMessageWriter) const
{
	const FGPUSkinCacheVisualizationData& VisualizationData = GetGPUSkinCacheVisualizationData();
	if (VisualizationData.IsActive())
	{
		FGPUSkinCacheVisualizationData::FModeType ModeType = VisualizationData.GetActiveModeType();

		// Color coding should match GetVisualizationDebugColor function
		auto DrawText = [&ScreenMessageWriter](const FString& Message, const FColor& Color)
		{
			ScreenMessageWriter.DrawLine(FText::FromString(Message), 10, Color);
		};

		if (ModeType == FGPUSkinCacheVisualizationData::FModeType::Overview)
		{
			DrawText(TEXT("Skin Cache Visualization - Overview"), FColor::White);
			DrawText(TEXT("Non SK mesh"), FColor::White);
			DrawText(TEXT("SK Skin Cache Excluded"), GEngine->GPUSkinCacheVisualizationExcludedColor.QuantizeRound());
			DrawText(TEXT("SK Skin Cache Included"), GEngine->GPUSkinCacheVisualizationIncludedColor.QuantizeRound());
			DrawText(TEXT("SK Recompute Tangent ON"), GEngine->GPUSkinCacheVisualizationRecomputeTangentsColor.QuantizeRound());
		}
		else if (ModeType == FGPUSkinCacheVisualizationData::FModeType::Memory)
		{
			float UsedMemoryInMB = UsedMemoryInBytes / MBSize;

			FString LowMemoryText = FString::Printf(TEXT("0 - %fMB"), GEngine->GPUSkinCacheVisualizationLowMemoryThresholdInMB);
			DrawText(TEXT("Skin Cache Visualization - Memory"), FColor::White);
			DrawText(FString::Printf(TEXT("Total Used: %.2fMB"), UsedMemoryInMB), FColor::White);
			DrawText(FString::Printf(TEXT("Low: < %.2fMB"), GEngine->GPUSkinCacheVisualizationLowMemoryThresholdInMB), GEngine->GPUSkinCacheVisualizationLowMemoryColor.QuantizeRound());
			DrawText(FString::Printf(TEXT("Mid: %.2f - %.2fMB"), GEngine->GPUSkinCacheVisualizationLowMemoryThresholdInMB, GEngine->GPUSkinCacheVisualizationHighMemoryThresholdInMB), GEngine->GPUSkinCacheVisualizationMidMemoryColor.QuantizeRound());
			DrawText(FString::Printf(TEXT("High: > %.2fMB"), GEngine->GPUSkinCacheVisualizationHighMemoryThresholdInMB), GEngine->GPUSkinCacheVisualizationHighMemoryColor.QuantizeRound());
		}
		else if (ModeType == FGPUSkinCacheVisualizationData::FModeType::RayTracingLODOffset)
		{
	#if RHI_RAYTRACING
			DrawText(TEXT("Skin Cache Visualization - RayTracingLODOffset"), FColor::White);
			const TArray<FLinearColor>& VisualizationColors = GEngine->GPUSkinCacheVisualizationRayTracingLODOffsetColors;
			for (int32 i = 0; i < VisualizationColors.Num(); ++i)
			{
				DrawText(FString::Printf(TEXT("RT_LOD == Raster_LOD %s %d"), (i > 0 ? TEXT("+") : TEXT("")), i), VisualizationColors[i].QuantizeRound());
			}
	#endif
		}
	}
}

#undef IMPLEMENT_SKIN_CACHE_SHADER_CLOTH
#undef IMPLEMENT_SKIN_CACHE_SHADER_ALL_SKIN_TYPES
#undef IMPLEMENT_SKIN_CACHE_SHADER