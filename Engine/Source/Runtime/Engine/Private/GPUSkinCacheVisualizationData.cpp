// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUSkinCacheVisualizationData.h"
#include "HAL/IConsoleManager.h"
#include "RenderUtils.h"
#include "RHIDefinitions.h"
#include "Misc/ScopeLock.h"

#define LOCTEXT_NAMESPACE "FGPUSkinCacheVisualizationData"

DEFINE_LOG_CATEGORY_STATIC(LogGPUSkinCacheVisualization, Log, All);

static FGPUSkinCacheVisualizationData GGPUSkinCacheVisualizationData;

void FGPUSkinCacheVisualizationData::Initialize()
{
	UE::TScopeLock Lock(Mutex);
	if (!bIsInitialized)
	{
		// NOTE: The first parameter determines the console command parameter. "none", "off" and "list" are reserved
		AddVisualizationMode(
			TEXT("Overview"),
			LOCTEXT("Overview", "Overview"),
			LOCTEXT("OverviewDesc", "Skin cache on/off, recompute tangents on/off"),
			FModeType::Overview);

		AddVisualizationMode(
			TEXT("Memory"),
			LOCTEXT("Memory", "Memory"),
			LOCTEXT("MemoryDesc", "Memory usage"),
			FModeType::Memory);

		if (IsRayTracingEnabled())
		{
			AddVisualizationMode(
				TEXT("RayTracingLODOffset"),
				LOCTEXT("RayTracingLODOffset", "RayTracingLODOffset"),
				LOCTEXT("RayTracingLODOffsetDesc", "Ray Tracing LOD index offset from rasterization"),
				FModeType::RayTracingLODOffset);
		}

		ConfigureConsoleCommand();

		bIsInitialized = true;
	}
}

void FGPUSkinCacheVisualizationData::ConfigureConsoleCommand()
{
	FString AvailableVisualizationModes;
	for (TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FModeRecord& Record = It.Value();
		AvailableVisualizationModes += FString::Printf(TEXT(" '%s' "), *Record.ModeString);
	}

	ConsoleDocumentationVisualizationMode = TEXT("Specifies which visualization mode to display:");
	ConsoleDocumentationVisualizationMode += AvailableVisualizationModes;

	IConsoleManager::Get().RegisterConsoleVariable(
		GetVisualizeConsoleCommandName(),
		TEXT(""),
		*ConsoleDocumentationVisualizationMode,
		ECVF_Cheat);
}

void FGPUSkinCacheVisualizationData::SetActiveMode(FModeType ModeType, const FName& ModeName)
{
	ActiveVisualizationModeType = ModeType;
	ActiveVisualizationModeName = ModeName;
}

bool FGPUSkinCacheVisualizationData::IsActive() const
{
	if (!IsInitialized())
	{
		return false;
	}

	if (GetActiveModeType() == FModeType::Num)
	{
		return false;
	}

	return true;
}

bool FGPUSkinCacheVisualizationData::Update(const FName& InViewMode)
{
	bool bForceShowFlag = false;

	if (IsInitialized())
	{
		SetActiveMode(FModeType::Num, NAME_None);

		// Check if the console command is set (overrides the editor)
		static IConsoleVariable* ICVarVisualize = IConsoleManager::Get().FindConsoleVariable(GetVisualizeConsoleCommandName());
		if (ICVarVisualize)
		{
			const FString ConsoleVisualizationMode = ICVarVisualize->GetString();
			if (!ConsoleVisualizationMode.IsEmpty())
			{
				if (ConsoleVisualizationMode == TEXT("off") || ConsoleVisualizationMode == TEXT("none"))
				{
					// Disable visualization
				}
				else
				{
					const FName  ModeName = FName(*ConsoleVisualizationMode);
					const FModeType  ModeType = GetModeType(ModeName);
					if (ModeType == FModeType::Num)
					{
						UE_LOGF(LogGPUSkinCacheVisualization, Warning, "Unknown GPU Skin Cache visualization mode '%ls'", *ConsoleVisualizationMode);
					}
					else
					{
						SetActiveMode(ModeType, ModeName);
						bForceShowFlag = true;
					}
				}
			}
		}
		
		// Check the view mode state (set by editor).
		if (ActiveVisualizationModeType == FModeType::Num && InViewMode != NAME_None)
		{
			const FModeType ModeID = GetModeType(InViewMode);
			if (ModeID != FModeType::Num)
			{
				SetActiveMode(ModeID, InViewMode);
			}
		}
	}

	return bForceShowFlag;
}

void FGPUSkinCacheVisualizationData::AddVisualizationMode(
	const TCHAR* ModeString,
	const FText& ModeText,
	const FText& ModeDesc,
	const FModeType ModeType
)
{
	const FName ModeName = FName(ModeString);

	FModeRecord& Record	= ModeMap.Emplace(ModeName);
	Record.ModeString			= FString(ModeString);
	Record.ModeName				= ModeName;
	Record.ModeText				= ModeText;
	Record.ModeDesc				= ModeDesc;
	Record.ModeType				= ModeType;
}

FText FGPUSkinCacheVisualizationData::GetModeDisplayName(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeText;
	}
	else
	{
		return FText::GetEmpty();
	}
}

FGPUSkinCacheVisualizationData::FModeType FGPUSkinCacheVisualizationData::GetModeType(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeType;
	}
	else
	{
		return FModeType::Num;
	}
}

FGPUSkinCacheVisualizationData& GetGPUSkinCacheVisualizationData()
{
	if (!GGPUSkinCacheVisualizationData.IsInitialized())
	{
		GGPUSkinCacheVisualizationData.Initialize();
	}

	return GGPUSkinCacheVisualizationData;
}

#undef LOCTEXT_NAMESPACE
