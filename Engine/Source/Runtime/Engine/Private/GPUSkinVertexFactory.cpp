// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUVertexFactory.cpp: GPU skin vertex factory implementation
=============================================================================*/

#include "GPUSkinVertexFactory.h"
#include "Animation/MeshDeformerProvider.h"
#include "MeshBatch.h"
#include "GPUSkinCache.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "Misc/DelayedAutoRegister.h"
#include "SkeletalRenderGPUSkin.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "RenderGraphResources.h"
#include "RenderUtils.h"
#include "ShaderPlatformCachedIniValue.h"
#include "Engine/RendererSettings.h"
#include "Rendering/RenderCommandPipes.h"

#if INTEL_ISPC
#include "GPUSkinVertexFactory.ispc.generated.h"
#endif

// Deprecated 5.8: These CVars no longer affect section splitting or bone count limits.
// They are preserved only for DDC key backward compatibility via GetMaxGPUSkinBones().
static int32 GCVarMaxGPUSkinBones = FGPUBaseSkinVertexFactory::GHardwareMaxGPUSkinBones;
static FAutoConsoleVariableRef CVarMaxGPUSkinBones(
	TEXT("Compat.MAX_GPUSKIN_BONES"),
	GCVarMaxGPUSkinBones,
	TEXT("Deprecated. Preserved for DDC key backward compatibility only."),
	ECVF_ReadOnly);

static int32 GCVarSupport16BitBoneIndex = 0;
static FAutoConsoleVariableRef CVarSupport16BitBoneIndex(
	TEXT("r.GPUSkin.Support16BitBoneIndex"),
	GCVarSupport16BitBoneIndex,
	TEXT("Deprecated. All platforms now support 16-bit bone indices. Preserved for DDC key backward compatibility only."),
	ECVF_ReadOnly);

// Whether to use 2 bones influence instead of default 4 for GPU skinning
// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarGPUSkinLimit2BoneInfluences(
	TEXT("r.GPUSkin.Limit2BoneInfluences"),
	0,	
	TEXT("Whether to use 2 bones influence instead of default 4/8 for GPU skinning. Cannot be changed at runtime."),
	ECVF_ReadOnly);

static int32 GCVarUnlimitedBoneInfluences = 0;
static FAutoConsoleVariableRef CVarUnlimitedBoneInfluences(
	TEXT("r.GPUSkin.UnlimitedBoneInfluences"),
	GCVarUnlimitedBoneInfluences,
	TEXT("Whether to use unlimited bone influences instead of default 4/8 for GPU skinning. Cannot be changed at runtime."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static int32 GCVarUnlimitedBoneInfluencesThreshold = EXTRA_BONE_INFLUENCES;
static FAutoConsoleVariableRef CVarUnlimitedBoneInfluencesThreshold(
	TEXT("r.GPUSkin.UnlimitedBoneInfluencesThreshold"),
	GCVarUnlimitedBoneInfluencesThreshold,
	TEXT("Unlimited Bone Influences Threshold to use unlimited bone influences buffer if r.GPUSkin.UnlimitedBoneInfluences is enabled. Should be unsigned int. Cannot be changed at runtime."),
	ECVF_ReadOnly);

static bool GCVarAlwaysUseDeformerForUnlimitedBoneInfluences = false;
static FAutoConsoleVariableRef CVarAlwaysUseDeformerForUnlimitedBoneInfluences(
	TEXT("r.GPUSkin.AlwaysUseDeformerForUnlimitedBoneInfluences"),
	GCVarAlwaysUseDeformerForUnlimitedBoneInfluences,
	TEXT("Any meshes using Unlimited Bone Influences will always be rendered with a Mesh Deformer. This reduces the number of shader permutations needed for skeletal mesh materials, saving memory at the cost of performance. Has no effect if either Unlimited Bone Influences or Deformer Graph is disabled. Cannot be changed at runtime."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarMobileEnableCloth(
	TEXT("r.Mobile.EnableCloth"),
	true,
	TEXT("If enabled, compile cloth shader permutations and render simulated cloth on mobile platforms and Mobile mode on PC. Cannot be changed at runtime"),
	ECVF_ReadOnly);

#define IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_TYPE_INTERNAL(FactoryClass, ShaderFilename, Flags) \
	template <GPUSkinBoneInfluenceType BoneInfluenceType> FVertexFactoryType FactoryClass<BoneInfluenceType>::StaticType( \
	BoneInfluenceType == DefaultBoneInfluence ? TEXT(#FactoryClass) TEXT("Default") : TEXT(#FactoryClass) TEXT("Unlimited"), \
	TEXT(ShaderFilename), \
	Flags | EVertexFactoryFlags::SupportsPrimitiveIdStream, \
	IMPLEMENT_VERTEX_FACTORY_VTABLE(FactoryClass<BoneInfluenceType>) \
	); \
	template <GPUSkinBoneInfluenceType BoneInfluenceType> inline FVertexFactoryType* FactoryClass<BoneInfluenceType>::GetType() const { return &StaticType; }


#define IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_TYPE(FactoryClass, ShaderFilename, Flags) \
	IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_TYPE_INTERNAL(FactoryClass, ShaderFilename, Flags) \
	template class FactoryClass<DefaultBoneInfluence>;	\
	template class FactoryClass<UnlimitedBoneInfluence>;

#define IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_PARAMETER_TYPE(FactoryClass, Frequency, ParameterType) \
	IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FactoryClass<DefaultBoneInfluence>, Frequency, ParameterType); \
	IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FactoryClass<UnlimitedBoneInfluence>, Frequency, ParameterType)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarVelocityTest(
	TEXT("r.VelocityTest"),
	0,
	TEXT("Allows to enable some low level testing code for the velocity rendering (Affects object motion blur and TemporalAA).")
	TEXT(" 0: off (default)")
	TEXT(" 1: add random data to the buffer where we store skeletal mesh bone data to test if the code (good to test in PAUSED as well)."),
	ECVF_Cheat | ECVF_RenderThreadSafe);
#endif // if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

// Disable it by default as it seems to be up to 20% slower on current gen platforms
#if !defined(GPU_SKIN_COPY_BONES_ISPC_ENABLED_DEFAULT)
#define GPU_SKIN_COPY_BONES_ISPC_ENABLED_DEFAULT 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bGPUSkin_CopyBones_ISPC_Enabled = INTEL_ISPC && GPU_SKIN_COPY_BONES_ISPC_ENABLED_DEFAULT;
#else
static bool bGPUSkin_CopyBones_ISPC_Enabled = GPU_SKIN_COPY_BONES_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarGPUSkinCopyBonesISPCEnabled(TEXT("r.GPUSkin.CopyBones.ISPC"), bGPUSkin_CopyBones_ISPC_Enabled, TEXT("Whether to use ISPC optimizations when copying bones for GPU skinning"));
#endif

#if INTEL_ISPC
static_assert(sizeof(ispc::FMatrix44f) == sizeof(FMatrix44f), "sizeof(ispc::FMatrix44f) != sizeof(FMatrix44f)");
static_assert(sizeof(ispc::FMatrix3x4) == sizeof(FMatrix3x4), "sizeof(ispc::FMatrix3x4) != sizeof(FMatrix3x4)");
#endif

class FNullMorphVertexBuffer : public FVertexBuffer
{
public:
	FNullMorphVertexBuffer() = default;
	~FNullMorphVertexBuffer() = default;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Enough data for 64k vertices mesh
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FMorphGPUSkinVertex>(TEXT("FNullMorphVertexBuffer"), 65535)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionZeroData();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}
};

TGlobalResource<FNullMorphVertexBuffer, FRenderResource::EInitPhase::Pre> GNullMorphVertexBuffer;

/*-----------------------------------------------------------------------------
 FSharedPoolPolicyData
 -----------------------------------------------------------------------------*/
uint32 FSharedPoolPolicyData::GetPoolBucketIndex(uint32 Size)
{
	unsigned long Lower = 0;
	unsigned long Upper = NumPoolBucketSizes;
	unsigned long Middle;
	
	do
	{
		Middle = ( Upper + Lower ) >> 1;
		if( Size <= BucketSizes[Middle-1] )
		{
			Upper = Middle;
		}
		else
		{
			Lower = Middle;
		}
	}
	while( Upper - Lower > 1 );
	
	check( Size <= BucketSizes[Lower] );
	check( (Lower == 0 ) || ( Size > BucketSizes[Lower-1] ) );
	
	return Lower;
}

uint32 FSharedPoolPolicyData::GetPoolBucketSize(uint32 Bucket)
{
	check(Bucket < NumPoolBucketSizes);
	return BucketSizes[Bucket];
}

uint32 FSharedPoolPolicyData::BucketSizes[NumPoolBucketSizes] = {
	16, 48, 96, 192, 384, 768, 1536, 
	3072, 4608, 6144, 7680, 9216, 12288, 
	65536, 131072, 262144, 786432, 1572864 // these 5 numbers are added for large cloth simulation vertices, supports up to 65,536 verts
};

/*-----------------------------------------------------------------------------
 FBoneBufferPoolPolicy
 -----------------------------------------------------------------------------*/
FVertexBufferAndSRV FBoneBufferPoolPolicy::CreateResource(FRHICommandListBase& RHICmdList, CreationArguments Args)
{
	uint32 BufferSize = GetPoolBucketSize(GetPoolBucketIndex(Args));
	// in VisualStudio the copy constructor call on the return argument can be optimized out
	// see https://msdn.microsoft.com/en-us/library/ms364057.aspx#nrvo_cpp05_topic3
	FVertexBufferAndSRV Buffer;

#if USE_COMPRESSED_BONE_TRANSFORM
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateByteAddress(TEXT("FBoneBufferPoolPolicy"), BufferSize, sizeof(uint32))
		.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
		.DetermineInitialState();

	Buffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	Buffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		Buffer.VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Raw));
#else
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FBoneBufferPoolPolicy"), BufferSize)
		.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
		.DetermineInitialState();

	Buffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	Buffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		Buffer.VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_A32B32G32R32F));
#endif
	Buffer.Size = BufferSize;
	return Buffer;
}

FSharedPoolPolicyData::CreationArguments FBoneBufferPoolPolicy::GetCreationArguments(const FVertexBufferAndSRV& Resource)
{
	return Resource.Size;
}

void FBoneBufferPoolPolicy::FreeResource(const FVertexBufferAndSRV& Resource)
{
}

FVertexBufferAndSRV FClothBufferPoolPolicy::CreateResource(FRHICommandListBase& RHICmdList, CreationArguments Args)
{
	uint32 BufferSize = GetPoolBucketSize(GetPoolBucketIndex(Args));
	// in VisualStudio the copy constructor call on the return argument can be optimized out
	// see https://msdn.microsoft.com/en-us/library/ms364057.aspx#nrvo_cpp05_topic3
	FVertexBufferAndSRV Buffer;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FClothBufferPoolPolicy"), BufferSize)
		.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
		.DetermineInitialState();

	Buffer.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	Buffer.VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		Buffer.VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_G32R32F));
	Buffer.Size = BufferSize;
	return Buffer;
}

/*-----------------------------------------------------------------------------
 FBoneBufferPool
 -----------------------------------------------------------------------------*/
FBoneBufferPool::~FBoneBufferPool()
{
}

TStatId FBoneBufferPool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FBoneBufferPool, STATGROUP_Tickables);
}

uint32 FBoneBufferPool::GetBufferSize(uint32 NumBones)
{
	return PooledSizeForCreationArguments(NumBones * sizeof(FVector4f) * 3);
}

FClothBufferPool::~FClothBufferPool()
{
}

TStatId FClothBufferPool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FClothBufferPool, STATGROUP_Tickables);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
uint32 FGPUBaseSkinVertexFactory::FShaderDataType::MaxGPUSkinBones = FGPUBaseSkinVertexFactory::GHardwareMaxGPUSkinBones;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool SupportsNoOverwriteBuffers()
{
	static const TConsoleVariableData<int32>* const CVarGTSyncType =
		IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GTSyncType"));

	// We can only rely on the low-latency syncing approaches to respect NumSafeFrames properly.
	const int32 SyncType = CVarGTSyncType->GetValueOnRenderThread();

	return GRHIGlobals.SupportsMapWriteNoOverwrite && (SyncType == 1 || SyncType == 2);
}

TGlobalResource<FBoneBufferPool> GBoneBufferPool;

//////////////////////////////////////////////////////////////////////////

static void UpdateBoneData(
	FRHICommandList& RHICmdList,
	const FName& AssetPathName,
	TConstArrayView<FMatrix44f> ReferenceToLocalMatrices,
	TConstArrayView<FBoneIndexType> BoneMap,
	FRHIBuffer* VertexBufferRHI)
{
	const uint32 NumBones = BoneMap.Num();

	check(IsInParallelRenderingThread());
	check(NumBones > 0);
	check(VertexBufferRHI);

	VertexBufferRHI->SetOwnerName(AssetPathName);

	// RLM_WriteOnly_NoOverwrite is safe here because the buffer pool waits N frames before allowing new allocations.
	FCompressedBoneTransform* ChunkMatrices = (FCompressedBoneTransform*)RHICmdList.LockBuffer(VertexBufferRHI, 0, NumBones * sizeof(FCompressedBoneTransform), SupportsNoOverwriteBuffers() ? RLM_WriteOnly_NoOverwrite : RLM_WriteOnly);

	for (uint32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		StoreCompressedBoneTransform(ChunkMatrices[BoneIndex], ReferenceToLocalMatrices[BoneMap[BoneIndex]]);
	}

	RHICmdList.UnlockBuffer(VertexBufferRHI);
}

FUnifiedBoneBuffer::FUnifiedBoneBuffer(TConstArrayView<FBoneIndexType> InBoneMap)
	: BoneMap(InBoneMap)
	, BufferSize(GBoneBufferPool.GetBufferSize(BoneMap.Num()))
{}

FUnifiedBoneBuffer::~FUnifiedBoneBuffer()
{
	for (FVertexBufferAndSRV& Buffer : Buffers)
	{
		if (Buffer.VertexBufferRHI)
		{
			GBoneBufferPool.ReleasePooledResource(MoveTemp(Buffer));
		}
		Buffer.SafeRelease();
	}
}

void FUnifiedBoneBuffer::Allocate(FRHICommandList& RHICmdList, bool bPrevious)
{
	FVertexBufferAndSRV& Buffer = GetBuffer(bPrevious);

	if (!Buffer.VertexBufferRHI || SupportsNoOverwriteBuffers())
	{
		GBoneBufferPool.ReleasePooledResource(MoveTemp(Buffer));
		Buffer = GBoneBufferPool.CreatePooledResource(RHICmdList, BufferSize);
	}
}

void FUnifiedBoneBuffer::Update(FRHICommandList& RHICmdList, const FName& AssetPathName, TConstArrayView<FMatrix44f> ReferenceToLocalMatrices, bool bPrevious)
{
	UpdateBoneData(RHICmdList, AssetPathName, ReferenceToLocalMatrices, BoneMap, GetBuffer(bPrevious).VertexBufferRHI);
}

//////////////////////////////////////////////////////////////////////////

void FGPUBaseSkinVertexFactory::FShaderDataType::AllocateBoneBuffer(FRHICommandList& RHICmdList, uint32 BufferSize, FVertexBufferAndSRV& OutBoneBuffer)
{
	check(IsInParallelRenderingThread());
	check(BufferSize > 0);
	check(!bUnifiedBoneBuffer);

	if (!OutBoneBuffer.VertexBufferRHI || SupportsNoOverwriteBuffers())
	{
		GBoneBufferPool.ReleasePooledResource(MoveTemp(OutBoneBuffer));
		OutBoneBuffer = GBoneBufferPool.CreatePooledResource(RHICmdList, BufferSize);
	}
}

void FGPUBaseSkinVertexFactory::FShaderDataType::ReleaseBoneData()
{
	for (FVertexBufferAndSRV& Buffer : BoneBuffer)
	{
		// Unified bone buffers are assigned and the vertex factory doesn't own them.
		if (!bUnifiedBoneBuffer && IsValidRef(Buffer))
		{
			GBoneBufferPool.ReleasePooledResource(MoveTemp(Buffer));
		}
		Buffer.SafeRelease();
	}
}

void FGPUBaseSkinVertexFactory::FShaderDataType::UpdateBoneData(
	FRHICommandList& RHICmdList,
	const FName& AssetPathName,
	TConstArrayView<FMatrix44f> ReferenceToLocalMatrices,
	TConstArrayView<FBoneIndexType> BoneMap,
	FRHIBuffer* VertexBufferRHI)
{
	check(!bUnifiedBoneBuffer);
	::UpdateBoneData(RHICmdList, AssetPathName, ReferenceToLocalMatrices, BoneMap, VertexBufferRHI);
}

FGPUBaseSkinVertexFactory::FGPUBaseSkinVertexFactory(const FInitializer& Initializer)
	: FVertexFactory(Initializer.FeatureLevel)
	, NumBones(Initializer.NumBones)
	, BoneOffset(Initializer.BoneOffset)
	, BoneBufferSize(GBoneBufferPool.GetBufferSize(Initializer.NumBones))
	, NumVertices(Initializer.NumVertices)
	, BaseVertexIndex(Initializer.BaseVertexIndex)
	, bUsedForPassthroughVertexFactory(Initializer.bUsedForPassthroughVertexFactory)
{
	ShaderData.bUnifiedBoneBuffer = Initializer.bUnifiedBoneBuffer;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 FGPUBaseSkinVertexFactory::GetMinimumPerPlatformMaxGPUSkinBonesValue()
{
	const bool bUseGlobalMaxGPUSkinBones = (GCVarMaxGPUSkinBones != FGPUBaseSkinVertexFactory::GHardwareMaxGPUSkinBones);
	//Use the default value in case there is no valid target platform
	int32 MaxGPUSkinBones = GetDefault<URendererSettings>()->MaxSkinBones.GetValue();
#if WITH_EDITORONLY_DATA && WITH_EDITOR
	for (const TPair<FName, int32>& PlatformData : GetDefault<URendererSettings>()->MaxSkinBones.PerPlatform)
	{
		MaxGPUSkinBones = FMath::Min(MaxGPUSkinBones, PlatformData.Value);
	}
#endif
	if (bUseGlobalMaxGPUSkinBones)
	{
		MaxGPUSkinBones = FMath::Min(MaxGPUSkinBones, GCVarMaxGPUSkinBones);
	}
	return MaxGPUSkinBones;
}

int32 FGPUBaseSkinVertexFactory::GetMaxGPUSkinBones(const ITargetPlatform* TargetPlatform /*= nullptr*/)
{
	const bool bUseGlobalMaxGPUSkinBones = (GCVarMaxGPUSkinBones != FGPUBaseSkinVertexFactory::GHardwareMaxGPUSkinBones);
	if (bUseGlobalMaxGPUSkinBones)
	{
		static bool bIsLogged = false;
		if (!bIsLogged)
		{
			UE_LOGF(LogSkeletalMesh, Display, "The Engine config variable [SystemSettings] Compat.MAX_GPUSKIN_BONES (%d) is deprecated, please remove the variable from any engine .ini file. Instead use the per platform project settings - Engine - Rendering - Skinning - Maximum bones per sections. Until the variable is remove we will clamp the per platform value",
				   GCVarMaxGPUSkinBones);
			bIsLogged = true;
		}
	}
	//Use the default value in case there is no valid target platform
	int32 MaxGPUSkinBones = GetDefault<URendererSettings>()->MaxSkinBones.GetValue();
	
#if WITH_EDITOR
	const ITargetPlatform* TargetPlatformTmp = TargetPlatform;
	if (!TargetPlatformTmp)
	{
		//Get the running platform if the caller did not supply a platform
		ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
		TargetPlatformTmp = TargetPlatformManager.GetRunningTargetPlatform();
	}
	if (TargetPlatformTmp)
	{
		//Get the platform value
		MaxGPUSkinBones = GetDefault<URendererSettings>()->MaxSkinBones.GetValueForPlatform(*TargetPlatformTmp->IniPlatformName());
	}
#endif

	if (bUseGlobalMaxGPUSkinBones)
	{
		//Make sure we do not go over the global ini console variable GCVarMaxGPUSkinBones
		MaxGPUSkinBones = FMath::Min(MaxGPUSkinBones, GCVarMaxGPUSkinBones);
		
	}

	//We cannot go under MAX_TOTAL_INFLUENCES
	MaxGPUSkinBones = FMath::Max(MaxGPUSkinBones, MAX_TOTAL_INFLUENCES);

	if (GCVarSupport16BitBoneIndex > 0)
	{
		// 16-bit bone index is supported
		return MaxGPUSkinBones;
	}
	else
	{
		// 16-bit bone index is not supported, clamp the max bones to 8-bit
		return FMath::Min(MaxGPUSkinBones, 256);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(uint32 MaxBoneInfluences, const ITargetPlatform* TargetPlatform)
{
	if (!GetUnlimitedBoneInfluences(TargetPlatform))
	{
		return false;
	}

	uint32 UnlimitedBoneInfluencesThreshold = (uint32)GCVarUnlimitedBoneInfluencesThreshold;

#if ALLOW_OTHER_PLATFORM_CONFIG
	if (TargetPlatform)
	{
		const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		const bool bIsRunningPlatform = RunningPlatform == TargetPlatform;
		if (bIsRunningPlatform)
		{
			UnlimitedBoneInfluencesThreshold = CVarUnlimitedBoneInfluencesThreshold->GetInt();
		}
		else
		{
			TSharedPtr<IConsoleVariable> VariablePtr = CVarUnlimitedBoneInfluencesThreshold->GetPlatformValueVariable(*TargetPlatform->IniPlatformName());
			if (VariablePtr.IsValid())
			{
				UnlimitedBoneInfluencesThreshold = (uint32)VariablePtr->GetInt();
			}
		}
	}
#endif
	
	return MaxBoneInfluences > UnlimitedBoneInfluencesThreshold;
}

bool FGPUBaseSkinVertexFactory::GetUnlimitedBoneInfluences(const ITargetPlatform* TargetPlatform)
{
#if ALLOW_OTHER_PLATFORM_CONFIG
	if (TargetPlatform)
	{
		const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		const bool bIsRunningPlatform = RunningPlatform == TargetPlatform;
		if (bIsRunningPlatform)
		{
			return CVarUnlimitedBoneInfluences->GetBool();
		}
		else
		{
			TSharedPtr<IConsoleVariable> VariablePtr = CVarUnlimitedBoneInfluences->GetPlatformValueVariable(*TargetPlatform->IniPlatformName());
			if (VariablePtr.IsValid())
			{
				return VariablePtr->GetBool();
			}
		}
	}
#endif
	
	return (GCVarUnlimitedBoneInfluences!=0);
}

int32 FGPUBaseSkinVertexFactory::GetBoneInfluenceLimitForAsset(int32 AssetProvidedLimit, const ITargetPlatform* TargetPlatform /*= nullptr*/)
{
	if (AssetProvidedLimit > 0)
	{
		// The asset provided an explicit limit
		return AssetProvidedLimit;
	}

	int32 GlobalDefaultLimit = GetDefault<URendererSettings>()->DefaultBoneInfluenceLimit.GetValue();

#if WITH_EDITOR
	const ITargetPlatform* TargetPlatformTmp = TargetPlatform;
	if (!TargetPlatformTmp)
	{
		// Get the running platform if the caller did not supply a platform
		ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
		TargetPlatformTmp = TargetPlatformManager.GetRunningTargetPlatform();
	}

	if (TargetPlatformTmp)
	{
		// Get the platform value
		GlobalDefaultLimit = GetDefault<URendererSettings>()->DefaultBoneInfluenceLimit.GetValueForPlatform(*TargetPlatformTmp->IniPlatformName());
	}
#endif

	if (GlobalDefaultLimit > 0)
	{
		// A global default limit has been set for this platform
		return GlobalDefaultLimit;
	}

	// No limit has been set. Return the maximum possible value.
	return MAX_TOTAL_INFLUENCES;
}

bool FGPUBaseSkinVertexFactory::GetAlwaysUseDeformerForUnlimitedBoneInfluences(EShaderPlatform Platform)
{
	auto InnerFunc = [](EShaderPlatform Platform)
	{
		static FShaderPlatformCachedIniValue<bool> UseDeformerForUBICVar(TEXT("r.GPUSkin.AlwaysUseDeformerForUnlimitedBoneInfluences"));
		return AreMeshDeformersAvailable(Platform) && UseDeformerForUBICVar.Get(Platform);
	};

#if WITH_EDITOR
	return InnerFunc(Platform);
#else
	// This value can't change at runtime in a non-editor build, so it's safe to cache.
	static const bool bCachedResult = InnerFunc(Platform);
	return bCachedResult;
#endif
}

BEGIN_SHADER_PARAMETER_STRUCT(FGPUSkinVertexFactoryCommonShaderParameters, )
	// Bits 0-7 => Size of the bone weight index in bytes / bits 8-15 => Size of the bone weight weights value in bytes
	SHADER_PARAMETER(uint32, InputWeightIndexSize)
	// number of influences for this draw call, 4 or 8
	SHADER_PARAMETER(uint32, NumBoneInfluencesParam)
	SHADER_PARAMETER(uint32, bIsMorphTarget)
	SHADER_PARAMETER(uint32, bIsRefPose)
	SHADER_PARAMETER(uint32, TransformStorageMode)
	SHADER_PARAMETER(uint32, SectionBoneOffset)
	SHADER_PARAMETER(uint32, TransformBufferOffset)
	SHADER_PARAMETER(uint32, MaxTransformCount)
	SHADER_PARAMETER(uint32, CurrentTransformSlot)
	SHADER_PARAMETER(uint32, BoneUpdatedFrameNumber)
	SHADER_PARAMETER(uint32, MorphUpdatedFrameNumber)
	SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(BoneMatrices)
	SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(PreviousBoneMatrices)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightStream)
	SHADER_PARAMETER_SRV(Buffer<uint>, SectionBoneMap)
	SHADER_PARAMETER_SRV(Buffer<float>, PreviousMorphBuffer)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSkinVertexFactoryUniformShaderParameters, )
	SHADER_PARAMETER_STRUCT(FGPUSkinVertexFactoryCommonShaderParameters, Common)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSkinVertexFactoryUniformShaderParameters, "GPUSkinVFBase")

void GetGPUSkinVertexFactoryCommonShaderParameters(
	FGPUSkinVertexFactoryCommonShaderParameters& ShaderParameters,
	const FGPUBaseSkinVertexFactory* VertexFactory,
	const FSkinningHeader* SkinningHeader,
	FRHIShaderResourceView* TransformDataBufferSRV)
{
	const FGPUBaseSkinVertexFactory::FShaderDataType& ShaderData = VertexFactory->GetShaderData();
	const FMorphVertexBuffer* PreviousMorphVertexBuffer = VertexFactory->GetMorphVertexBuffer(true);

	if (ShaderData.HasBoneBufferForReading(false))
	{
		ShaderParameters.BoneMatrices = ShaderData.GetBoneBufferForReading(false).VertexBufferSRV;
		ShaderParameters.PreviousBoneMatrices = ShaderData.GetBoneBufferForReading(true).VertexBufferSRV;
		ShaderParameters.SectionBoneOffset = 0;
		ShaderParameters.TransformBufferOffset = 0;
		ShaderParameters.MaxTransformCount = 0;
		ShaderParameters.CurrentTransformSlot = 0;
		ShaderParameters.TransformStorageMode = GPUSKIN_TRANSFORM_STORAGE_BONE_MAP;
		ShaderParameters.bIsRefPose = false;
	}
	else
	{
		check(SkinningHeader);

		if (TransformDataBufferSRV)
		{
			ShaderParameters.BoneMatrices = TransformDataBufferSRV;
			ShaderParameters.PreviousBoneMatrices = TransformDataBufferSRV;
		}
		else
		{
			ShaderParameters.BoneMatrices = GNullVertexBuffer.VertexBufferSRV;
			ShaderParameters.PreviousBoneMatrices = GNullVertexBuffer.VertexBufferSRV;
		}
		ShaderParameters.TransformBufferOffset = SkinningHeader->TransformBufferOffset;
		ShaderParameters.MaxTransformCount = SkinningHeader->MaxTransformCount;
		ShaderParameters.CurrentTransformSlot = SkinningHeader->CurrentTransformSlot;
		ShaderParameters.SectionBoneOffset = VertexFactory->GetBoneOffset();
		ShaderParameters.TransformStorageMode = ShaderData.SectionBoneMapBuffer.IsValid() ? GPUSKIN_TRANSFORM_STORAGE_DIRECT : GPUSKIN_TRANSFORM_STORAGE_BONE_MAP;
		ShaderParameters.bIsRefPose = SkinningHeader->bIsRefPose;
	}
	ShaderParameters.BoneUpdatedFrameNumber = ShaderData.UpdatedFrameNumber;
	ShaderParameters.SectionBoneMap = ShaderData.SectionBoneMapBuffer ? ShaderData.SectionBoneMapBuffer : GNullVertexBuffer.VertexBufferSRV;
	ShaderParameters.bIsMorphTarget = VertexFactory->IsMorphTarget() ? 1 : 0;
	ShaderParameters.PreviousMorphBuffer = PreviousMorphVertexBuffer ? PreviousMorphVertexBuffer->GetSRV() : GNullVertexBuffer.VertexBufferSRV.GetReference();
	ShaderParameters.MorphUpdatedFrameNumber = VertexFactory->GetMorphVertexBufferUpdatedFrameNumber();
	ShaderParameters.InputWeightIndexSize = ShaderData.InputWeightIndexSize;
	ShaderParameters.InputWeightStream = ShaderData.InputWeightStream ? ShaderData.InputWeightStream : GNullVertexBuffer.VertexBufferSRV;
	ShaderParameters.NumBoneInfluencesParam = VertexFactory->GetNumBoneInfluences();
}

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSkinAPEXClothVertexFactoryUniformShaderParameters, )
	SHADER_PARAMETER_STRUCT(FGPUSkinVertexFactoryCommonShaderParameters, Common)
	/** Transform from cloth space (relative to cloth root bone) to local(component) space */
	SHADER_PARAMETER(FMatrix44f, ClothToLocal)
	SHADER_PARAMETER(FMatrix44f, PreviousClothToLocal)
	/** blend weight between simulated positions and original key-framed animation */
	SHADER_PARAMETER(float, ClothBlendWeight)
	/** Scale of the owner actor */
	SHADER_PARAMETER(FVector3f, WorldScale)
	// .x = Draw Index Buffer offset, .y = Offset into Cloth Vertex Buffer
	SHADER_PARAMETER(FUintVector2, GPUSkinApexClothStartIndexOffset)
	SHADER_PARAMETER(uint32, ClothNumInfluencesPerVertex)
	SHADER_PARAMETER(uint32, bEnabled)
	/** Vertex buffer from which to read simulated positions of clothing. */
	SHADER_PARAMETER_SRV(Buffer<float2>, ClothSimulVertsPositionsNormals)
	SHADER_PARAMETER_SRV(Buffer<float2>, PreviousClothSimulVertsPositionsNormals)
	SHADER_PARAMETER_SRV(Buffer<float4>, GPUSkinApexCloth)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSkinAPEXClothVertexFactoryUniformShaderParameters, "GPUSkinAPEXClothVF");

void GetGPUSkinAPEXClothVertexFactoryUniformShaderParameters(
	FGPUSkinAPEXClothVertexFactoryUniformShaderParameters& UniformParameters,
	const FGPUBaseSkinVertexFactory* VertexFactory,
	const FSkinningHeader* SkinningHeader,
	FRHIShaderResourceView* TransformDataBufferSRV)
{
	FGPUBaseSkinAPEXClothVertexFactory const* ClothVertexFactory = VertexFactory->GetClothVertexFactory();
	check(ClothVertexFactory != nullptr);

	const FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType& ClothShaderData = ClothVertexFactory->GetClothShaderData();
	const uint32 BaseVertexIndex = VertexFactory->GetBaseVertexIndex();

	FRHIShaderResourceView* ClothBufferSRV = ClothVertexFactory->GetClothBuffer();

	GetGPUSkinVertexFactoryCommonShaderParameters(UniformParameters.Common, VertexFactory, SkinningHeader, TransformDataBufferSRV);
	UniformParameters.ClothSimulVertsPositionsNormals = ClothShaderData.HasClothBufferForReading(false) ? ClothShaderData.GetClothBufferForReading(false).VertexBufferSRV : GNullVertexBuffer.VertexBufferSRV;
	UniformParameters.GPUSkinApexCloth = ClothBufferSRV ? ClothBufferSRV : GNullVertexBuffer.VertexBufferSRV.GetReference();
	UniformParameters.ClothToLocal = ClothShaderData.GetClothToLocalForReading(false);
	UniformParameters.ClothBlendWeight = ClothShaderData.ClothBlendWeight;
	UniformParameters.WorldScale = ClothShaderData.WorldScale;
	UniformParameters.GPUSkinApexClothStartIndexOffset = FUintVector2(BaseVertexIndex, ClothVertexFactory->GetClothIndexOffset(BaseVertexIndex));
	UniformParameters.ClothNumInfluencesPerVertex = ClothShaderData.NumInfluencesPerVertex;
	UniformParameters.bEnabled = ClothShaderData.bEnabled;
	UniformParameters.PreviousClothSimulVertsPositionsNormals = ClothShaderData.HasClothBufferForReading(true) ? ClothShaderData.GetClothBufferForReading(true).VertexBufferSRV : GNullVertexBuffer.VertexBufferSRV;
	UniformParameters.PreviousClothToLocal = ClothShaderData.GetClothToLocalForReading(true);
}

void FGPUBaseSkinVertexFactory::SetData(FRHICommandListBase& RHICmdList, const FGPUSkinDataType* InData)
{
	check(InData);

	if (!Data)
	{
		Data = MakeUnique<FGPUSkinDataType>();
	}

	*Data = *InData;
	UpdateRHI(RHICmdList);
}

void FGPUBaseSkinVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	// The primary vertex factory is used for cached mesh draw commands which needs a valid uniform buffer, so pre-create the uniform buffer with empty contents.
	if (!bUsedForPassthroughVertexFactory)
	{
		if (GetClothVertexFactory())
		{
			UniformBuffer = RHICreateUniformBuffer(nullptr, &FGPUSkinAPEXClothVertexFactoryUniformShaderParameters::GetStructMetadata()->GetLayout(), EUniformBufferUsage::UniformBuffer_MultiFrame);
		}
		else
		{
			UniformBuffer = RHICreateUniformBuffer(nullptr, &FGPUSkinVertexFactoryUniformShaderParameters::GetStructMetadata()->GetLayout(), EUniformBufferUsage::UniformBuffer_MultiFrame);
		}
	}

	MorphDeltaBufferSlot = FRHIStreamSourceSlot::Create(GNullMorphVertexBuffer.VertexBufferRHI.GetReference());
}

void FGPUBaseSkinVertexFactory::ReleaseRHI()
{
	FVertexFactory::ReleaseRHI();
	UniformBuffer.SafeRelease();
}

void FGPUBaseSkinVertexFactory::UpdateUniformBuffer(FRHICommandListBase& RHICmdList, const FSkinningHeader* SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV)
{
	if (!bUniformBufferDirty && !SkinningHeader)
	{
		return;
	}

	if (GetClothVertexFactory())
	{
		FGPUSkinAPEXClothVertexFactoryUniformShaderParameters UniformParameters;
		GetGPUSkinAPEXClothVertexFactoryUniformShaderParameters(UniformParameters, this, SkinningHeader, TransformDataBufferSRV);
		if (UniformBuffer)
		{
			RHICmdList.UpdateUniformBuffer(UniformBuffer, &UniformParameters);
		}
		else
		{
			// If this vertex factory is used for the passthrough one it's still possible to fall back to using this one, but we defer creation of the RHI uniform buffer.
			check(bUsedForPassthroughVertexFactory);
			UniformBuffer = RHICreateUniformBuffer(&UniformParameters, &FGPUSkinAPEXClothVertexFactoryUniformShaderParameters::GetStructMetadata()->GetLayout(), EUniformBufferUsage::UniformBuffer_MultiFrame);
		}
	}
	else
	{
		FGPUSkinVertexFactoryUniformShaderParameters UniformParameters;
		GetGPUSkinVertexFactoryCommonShaderParameters(UniformParameters.Common, this, SkinningHeader, TransformDataBufferSRV);
		if (UniformBuffer)
		{
			RHICmdList.UpdateUniformBuffer(UniformBuffer, &UniformParameters);
		}
		else
		{
			// If this vertex factory is used for the passthrough one it's still possible to fall back to using this one, but we defer creation of the RHI uniform buffer.
			check(bUsedForPassthroughVertexFactory);
			UniformBuffer = RHICreateUniformBuffer(&UniformParameters, &FGPUSkinVertexFactoryUniformShaderParameters::GetStructMetadata()->GetLayout(), EUniformBufferUsage::UniformBuffer_MultiFrame);
		}
	}

	bUniformBufferDirty = false;
}

void FGPUBaseSkinVertexFactory::UpdateMorphState(FRHICommandListBase& RHICmdList, bool bUseMorphTarget)
{
	check(Data);
	Data->bMorphTarget = bUseMorphTarget;

	if (bUseMorphTarget)
	{
		const FMorphVertexBuffer* MorphVertexBuffer = GetMorphVertexBuffer(false);
		RHICmdList.UpdateStreamSourceSlot(MorphDeltaBufferSlot, MorphVertexBuffer ? MorphVertexBuffer->VertexBufferRHI : GNullMorphVertexBuffer.VertexBufferRHI);
	}
}

void FGPUBaseSkinVertexFactory::CopyDataTypeForLocalVertexFactory(FLocalVertexFactory::FDataType& OutDestData) const
{
	check(Data.IsValid());

	OutDestData.PositionComponent = Data->PositionComponent;
	OutDestData.TangentBasisComponents[0] = Data->TangentBasisComponents[0];
	OutDestData.TangentBasisComponents[1] = Data->TangentBasisComponents[1];
	OutDestData.TextureCoordinates = Data->TextureCoordinates;
	OutDestData.ColorComponent = Data->ColorComponent;
	OutDestData.PreSkinPositionComponent = Data->PositionComponent;
	OutDestData.PositionComponentSRV = Data->PositionComponentSRV;
	OutDestData.PreSkinPositionComponentSRV = Data->PositionComponentSRV;
	OutDestData.TangentsSRV = Data->TangentsSRV;
	OutDestData.ColorComponentsSRV = Data->ColorComponentsSRV;
	OutDestData.ColorIndexMask = Data->ColorIndexMask;
	OutDestData.TextureCoordinatesSRV = Data->TextureCoordinatesSRV;
	OutDestData.LightMapCoordinateIndex = Data->LightMapCoordinateIndex;
	OutDestData.NumTexCoords = Data->NumTexCoords;
	OutDestData.LODLightmapDataIndex = Data->LODLightmapDataIndex;
}

const FMorphVertexBuffer* FGPUBaseSkinVertexFactory::GetMorphVertexBuffer(bool bPrevious) const
{
	check(Data.IsValid() && (!Data->bMorphTarget || Data->MorphVertexBufferPool));
	return Data->bMorphTarget ? &Data->MorphVertexBufferPool->GetMorphVertexBufferForReading(bPrevious) : nullptr;
}

uint32 FGPUBaseSkinVertexFactory::GetMorphVertexBufferUpdatedFrameNumber() const
{
	check(Data.IsValid() && (!Data->bMorphTarget || Data->MorphVertexBufferPool));
	return Data->bMorphTarget ? Data->MorphVertexBufferPool->GetUpdatedFrameNumber() : 0;
}

void FGPUBaseSkinVertexFactory::GetOverrideVertexStreams(FVertexInputStreamArray& VertexStreams) const
{
	if (MorphDeltaStreamIndex >= 0)
	{
		VertexStreams.Emplace(MorphDeltaStreamIndex, 0, MorphDeltaBufferSlot);
	}
}

/*-----------------------------------------------------------------------------
TGPUSkinVertexFactory
-----------------------------------------------------------------------------*/

template <GPUSkinBoneInfluenceType BoneInfluenceType>
bool TGPUSkinVertexFactory<BoneInfluenceType>::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	// Always compile for the default/special engine materials.
	if (Parameters.MaterialParameters.bIsSpecialEngineMaterial)
	{
		return true;
	}

	// Material must be flagged for skeletal mesh or morph target use.
	if (!Parameters.MaterialParameters.bIsUsedWithSkeletalMesh && !Parameters.MaterialParameters.bIsUsedWithMorphTargets)
	{
		return false;
	}

	// For the unlimited bone influence variant, only compile if UBI is actually enabled
	// and we're not forcing all UBI meshes through a mesh deformer.
	if (BoneInfluenceType == UnlimitedBoneInfluence)
	{
		static FShaderPlatformCachedIniValue<int32> UBICVar(TEXT("r.GPUSkin.UnlimitedBoneInfluences"));
		static FShaderPlatformCachedIniValue<bool> UseDeformerForUBICVar(TEXT("r.GPUSkin.AlwaysUseDeformerForUnlimitedBoneInfluences"));

		if (!UBICVar.Get(Parameters.Platform) || UseDeformerForUBICVar.Get(Parameters.Platform))
		{
			return false;
		}
	}

	// Inline skinning shaders are compiled when the skin cache doesn't fully replace them.
	if (AreInlineSkinningShadersEnabled(Parameters.Platform))
	{
		return true;
	}

	// Instanced skinned meshes use inline skinning even when the skin cache is active,
	// so compile these shaders regardless of the r.SkinCache.CompileShaders setting.
	if (Parameters.MaterialParameters.bIsUsedWithInstancedSkinnedMesh)
	{
		return true;
	}

	return false;
}

template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment )
{
	FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	const FStaticFeatureLevel MaxSupportedFeatureLevel = GetMaxSupportedFeatureLevel(Parameters.Platform);
	const bool bUseGPUScene = UseGPUScene(Parameters.Platform);
	const bool bSupportsPrimitiveIdStream = Parameters.VertexFactoryType->SupportsPrimitiveIdStream();
	{
		const bool bLimit2BoneInfluences = (CVarGPUSkinLimit2BoneInfluences.GetValueOnAnyThread() != 0);
		OutEnvironment.SetDefine(TEXT("GPUSKIN_LIMIT_2BONE_INFLUENCES"), (bLimit2BoneInfluences ? 1 : 0));
	}

	OutEnvironment.SetDefine(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"), BoneInfluenceType == UnlimitedBoneInfluence ? 1 : 0);

	OutEnvironment.SetDefine(TEXT("GPU_SKINNED_MESH_FACTORY"), 1);

	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bSupportsPrimitiveIdStream && bUseGPUScene);

	// Whether the material supports morph targets
	OutEnvironment.SetDefine(TEXT("GPUSKIN_MORPH_BLEND"), Parameters.MaterialParameters.bIsUsedWithMorphTargets || Parameters.MaterialParameters.bIsSpecialEngineMaterial);

	if (UseGPUScene(Parameters.Platform))
	{
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), TEXT("1"));
	}
}

/**
 * TGPUSkinVertexFactory does not support manual vertex fetch yet so worst case element set is returned to make sure the PSO can be compiled
 */
template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	check(VertexInputStreamType == EVertexInputStreamType::Default);
		
	uint32 StreamIndex = 0;

	// Position
	Elements.Add(FVertexElement(StreamIndex++, 0, VET_Float3, 0, 0, false));

	// Normals
	Elements.Add(FVertexElement(StreamIndex, 0, VET_PackedNormal, 1, 0, false));
	Elements.Add(FVertexElement(StreamIndex++, 4, VET_PackedNormal, 2, 0, false));
	
	// Texcoords
	Elements.Add(FVertexElement(StreamIndex, 0, VET_Half4, 5, 0, false));
	Elements.Add(FVertexElement(StreamIndex, 8, VET_Half4, 6, 0, false));	
	Elements.Add(FVertexElement(StreamIndex, 16, VET_Half4, 7, 0, false));
	Elements.Add(FVertexElement(StreamIndex++, 24, VET_Half4, 8, 0, false));
	
	// Color
	Elements.Add(FVertexElement(StreamIndex++, 0, VET_Color, 13, 0, false));

	// Bone data
	if (BoneInfluenceType == UnlimitedBoneInfluence)
	{
		// Blend offset count
		Elements.Add(FVertexElement(StreamIndex++, 0, VET_UInt, 3, 0, false));
	}
	else
	{
		// Blend indices & weight
		Elements.Add(FVertexElement(StreamIndex, 0, VET_UByte4, 3, 0, false));
		Elements.Add(FVertexElement(StreamIndex, 4, VET_UByte4N, 4, 0, false));
		
		// Extra blend indices & weights
		Elements.Add(FVertexElement(StreamIndex, 0, VET_UByte4, 14, 0, false));
		Elements.Add(FVertexElement(StreamIndex++, 4, VET_UByte4N, 15, 0, false));
	}

	// Morph blend data
	Elements.Add(FVertexElement(StreamIndex, 0, VET_Float3, 9, 0, false));
	Elements.Add(FVertexElement(StreamIndex++, 12, VET_Float3, 10, 0, false));
	
	// Primitive ID stream
	if (UseGPUScene(GMaxRHIShaderPlatform)
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		// Attribute ID
		Elements.Add(FVertexElement(StreamIndex++, 0, VET_UInt, 16, 0, true));
	}
}

template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FGPUSkinDataType& GPUSkinData, FVertexDeclarationElementList& OutElements, FVertexStreamList& InOutStreams, int32& OutMorphDeltaStreamIndex)
{
	check(InputStreamType == EVertexInputStreamType::Default);

	// Position
	OutElements.Add(AccessStreamComponent(GPUSkinData.PositionComponent, 0, InOutStreams));

	// Tangent basis vector
	OutElements.Add(AccessStreamComponent(GPUSkinData.TangentBasisComponents[0], 1, InOutStreams));
	OutElements.Add(AccessStreamComponent(GPUSkinData.TangentBasisComponents[1], 2, InOutStreams));

	// Texture coordinates
	if (GPUSkinData.TextureCoordinates.Num())
	{
		const uint8 BaseTexCoordAttribute = 5;				
		AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, GPUSkinData.TextureCoordinates, OutElements, InOutStreams);
	}

	if (GPUSkinData.ColorComponentsSRV == nullptr)
	{
		GPUSkinData.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
		GPUSkinData.ColorIndexMask = 0;
	}

	// Vertex color - account for the possibility that the mesh has no vertex colors
	if (GPUSkinData.ColorComponent.VertexBuffer)
	{
		OutElements.Add(AccessStreamComponent(GPUSkinData.ColorComponent, 13, InOutStreams));
	}
	else
	{
		// If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
		// This wastes 4 bytes of memory per vertex, but prevents having to compile out twice the number of vertex factories.
		FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
		OutElements.Add(AccessStreamComponent(NullColorComponent, 13, InOutStreams));
	}

	if (BoneInfluenceType == UnlimitedBoneInfluence)
	{
		// Blend offset count
		OutElements.Add(AccessStreamComponent(GPUSkinData.BlendOffsetCount, 3, InOutStreams));
	}
	else
	{
		// Bone indices
		OutElements.Add(AccessStreamComponent(GPUSkinData.BoneIndices, 3, InOutStreams));

		// Bone weights
		OutElements.Add(AccessStreamComponent(GPUSkinData.BoneWeights, 4, InOutStreams));

		// Extra bone indices & weights
		if (GPUSkinData.NumBoneInfluences > MAX_INFLUENCES_PER_STREAM)
		{
			OutElements.Add(AccessStreamComponent(GPUSkinData.ExtraBoneIndices, 14, InOutStreams));
			OutElements.Add(AccessStreamComponent(GPUSkinData.ExtraBoneWeights, 15, InOutStreams));
		}
		else
		{
			OutElements.Add(AccessStreamComponent(GPUSkinData.BoneIndices, 14, InOutStreams));
			OutElements.Add(AccessStreamComponent(GPUSkinData.BoneWeights, 15, InOutStreams));
		}
	}

	FVertexElement DeltaPositionElement = AccessStreamComponent(GPUSkinData.DeltaPositionComponent, 9, InOutStreams);
	OutElements.Add(DeltaPositionElement);
	OutElements.Add(AccessStreamComponent(GPUSkinData.DeltaTangentZComponent, 10, InOutStreams));

	// Cache delta stream index (position & tangentZ share the same stream)
	OutMorphDeltaStreamIndex = DeltaPositionElement.StreamIndex;
}

template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FGPUSkinDataType& GPUSkinData, FVertexDeclarationElementList& OutElements)
{
	FVertexStreamList VertexStreams;
	int32 MorphDeltaStreamIndex;
	GetVertexElements(FeatureLevel, InputStreamType, GPUSkinData, OutElements, VertexStreams, MorphDeltaStreamIndex);

	if (UseGPUScene(GMaxRHIShaderPlatform)
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		OutElements.Add(FVertexElement(VertexStreams.Num(), 0, VET_UInt, 16, 0, true));
	}
}

/**
* Add the vertex declaration elements for the streams.
* @param InData - Type with stream components.
* @param OutElements - Vertex declaration list to modify.
*/
template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::AddVertexElements(FVertexDeclarationElementList& OutElements)
{
	check(Data.IsValid());
	GetVertexElements(GetFeatureLevel(), EVertexInputStreamType::Default, *Data, OutElements, Streams, MorphDeltaStreamIndex);

	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, OutElements, 16, 16);
}

/**
* Creates declarations for each of the vertex stream components and
* initializes the device resource
*/
template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::InitRHI(FRHICommandListBase& RHICmdList)
{
	FGPUBaseSkinVertexFactory::InitRHI(RHICmdList);

	// list of declaration items
	FVertexDeclarationElementList Elements;
	AddVertexElements(Elements);

	// create the actual device decls
	InitDeclaration(Elements);
}

template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinVertexFactory<BoneInfluenceType>::ReleaseRHI()
{
	FGPUBaseSkinVertexFactory::ReleaseRHI();
	ShaderData.ReleaseBoneData();
}

/*-----------------------------------------------------------------------------
TGPUSkinAPEXClothVertexFactory
-----------------------------------------------------------------------------*/

template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinAPEXClothVertexFactory<BoneInfluenceType>::ReleaseRHI()
{
	Super::ReleaseRHI();
	ClothShaderData.ReleaseClothSimulData();

	// Release the RHIResource reference held in FGPUSkinAPEXClothDataType
	if (ClothDataPtr)
	{
		ClothDataPtr->ClothBuffer.SafeRelease();
	}
}

/*-----------------------------------------------------------------------------
TGPUSkinVertexFactoryShaderParameters
-----------------------------------------------------------------------------*/

class FGPUSkinVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FGPUSkinVertexFactoryShaderParameters, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		const FGPUBaseSkinVertexFactory* GPUSkinVertexFactory = static_cast<const FGPUBaseSkinVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGPUSkinVertexFactoryUniformShaderParameters>(), GPUSkinVertexFactory->GetUniformBuffer());
		GPUSkinVertexFactory->GetOverrideVertexStreams(VertexStreams);
	}
};

IMPLEMENT_TYPE_LAYOUT(FGPUSkinVertexFactoryShaderParameters);

IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_PARAMETER_TYPE(TGPUSkinVertexFactory, SF_Vertex, FGPUSkinVertexFactoryShaderParameters);

/** bind gpu skin vertex factory to its shader file and its shader parameters */
IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_TYPE(TGPUSkinVertexFactory, "/Engine/Private/GpuSkinVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials 
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| EVertexFactoryFlags::SupportsTriangleSorting
);

/*-----------------------------------------------------------------------------
	FGPUBaseSkinAPEXClothVertexFactory
-----------------------------------------------------------------------------*/
bool FGPUBaseSkinAPEXClothVertexFactory::IsClothEnabled(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileEnableClothIniValue(TEXT("r.Mobile.EnableCloth"));
	const bool bEnableClothOnMobile = (MobileEnableClothIniValue.Get(Platform) != 0);
	const bool bIsMobile = IsMobilePlatform(Platform);
	return !bIsMobile || bEnableClothOnMobile;
}

/*-----------------------------------------------------------------------------
	TGPUSkinAPEXClothVertexFactoryShaderParameters
-----------------------------------------------------------------------------*/

class TGPUSkinAPEXClothVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(TGPUSkinAPEXClothVertexFactoryShaderParameters, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		const FGPUBaseSkinVertexFactory* GPUSkinVertexFactory = static_cast<const FGPUBaseSkinVertexFactory*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGPUSkinAPEXClothVertexFactoryUniformShaderParameters>(), GPUSkinVertexFactory->GetUniformBuffer());
		GPUSkinVertexFactory->GetOverrideVertexStreams(VertexStreams);
	}
};

IMPLEMENT_TYPE_LAYOUT(TGPUSkinAPEXClothVertexFactoryShaderParameters);

/*-----------------------------------------------------------------------------
	TGPUSkinAPEXClothVertexFactory::ClothShaderType
-----------------------------------------------------------------------------*/

/** Pool of buffers for clothing simulation data */
TGlobalResource<FClothBufferPool> GClothBufferPool;

void FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::ReleaseClothSimulData()
{
	for (uint32 i = 0; i < 2; ++i)
	{
		if (IsValidRef(ClothSimulPositionNormalBuffer[i]))
		{
			GClothBufferPool.ReleasePooledResource(MoveTemp(ClothSimulPositionNormalBuffer[i]));
		}
	}
	Reset();
}

void FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::UpdateClothSimulationData(
	FRHICommandList& RHICmdList,
	TConstArrayView<FVector3f> InSimulPositions,
	TConstArrayView<FVector3f> InSimulNormals,
	uint32 RevisionNumber,
	const FName& AssetPathName)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FGPUBaseSkinAPEXClothVertexFactory_UpdateClothSimulationData);

	const bool bSupportNoOverwrite = SupportsNoOverwriteBuffers();

	uint32 NumSimulVerts = InSimulPositions.Num();

	check(IsInParallelRenderingThread());

	SetCurrentRevisionNumber(RevisionNumber);
	FVertexBufferAndSRV* CurrentClothBuffer = &GetClothBufferForWriting();

	NumSimulVerts = FMath::Min(NumSimulVerts, (uint32)MAX_APEXCLOTH_VERTICES_FOR_VB);

	uint32 VectorArraySize = NumSimulVerts * sizeof(float) * 6;
	if (bSupportNoOverwrite || CurrentClothBuffer->Size != GClothBufferPool.PooledSizeForCreationArguments(VectorArraySize))
	{
		GClothBufferPool.ReleasePooledResource(MoveTemp(*CurrentClothBuffer));
		*CurrentClothBuffer = GClothBufferPool.CreatePooledResource(RHICmdList, VectorArraySize);
	}
	check(IsValidRef(*CurrentClothBuffer));
	CurrentClothBuffer->VertexBufferRHI->SetOwnerName(AssetPathName);

	if(NumSimulVerts)
	{
		// RLM_WriteOnly_NoOverwrite is safe here because the buffer pool waits N frames before allowing new allocations.
		float* RESTRICT Data = (float* RESTRICT)RHICmdList.LockBuffer(CurrentClothBuffer->VertexBufferRHI, 0, VectorArraySize, bSupportNoOverwrite ? RLM_WriteOnly_NoOverwrite : RLM_WriteOnly);
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FGPUBaseSkinAPEXClothVertexFactory_UpdateClothSimulationData_CopyData);
			float* RESTRICT Pos = (float* RESTRICT) &InSimulPositions[0].X;
			float* RESTRICT Normal = (float* RESTRICT) &InSimulNormals[0].X;
			for (uint32 Index = 0; Index < NumSimulVerts; Index++)
			{
				FPlatformMisc::Prefetch(Pos + PLATFORM_CACHE_LINE_SIZE);
				FPlatformMisc::Prefetch(Normal + PLATFORM_CACHE_LINE_SIZE);

				FMemory::Memcpy(Data, Pos, sizeof(float) * 3);
				FMemory::Memcpy(Data + 3, Normal, sizeof(float) * 3);
				Data += 6;
				Pos += 3;
				Normal += 3;
			}
		}
		RHICmdList.UnlockBuffer(CurrentClothBuffer->VertexBufferRHI);
	}
}

void FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::SetCurrentRevisionNumber(uint32 RevisionNumber)
{
	if (bDoubleBuffer)
	{
		// Flip revision number to previous if this is new, otherwise keep current version.
		if (CurrentRevisionNumber != RevisionNumber)
		{
			PreviousRevisionNumber = CurrentRevisionNumber;
			CurrentRevisionNumber = RevisionNumber;
			CurrentBuffer = 1 - CurrentBuffer;
		}
	}
}

FVertexBufferAndSRV& FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothBufferForWriting()
{
	uint32 Index = GetClothBufferIndexForWriting();
	return ClothSimulPositionNormalBuffer[Index];
}

bool FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::HasClothBufferForReading(bool bPrevious) const
{
	uint32 Index = GetClothBufferIndexForReading(bPrevious);
	return bEnabled && ClothSimulPositionNormalBuffer[Index].VertexBufferRHI.IsValid();
}

const FVertexBufferAndSRV& FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothBufferForReading(bool bPrevious) const
{
	uint32 Index = GetClothBufferIndexForReading(bPrevious);
	checkf(ClothSimulPositionNormalBuffer[Index].VertexBufferRHI.IsValid(), TEXT("Index: %i Buffer0: %s Buffer1: %s"), Index, ClothSimulPositionNormalBuffer[0].VertexBufferRHI.IsValid() ? TEXT("true") : TEXT("false"), ClothSimulPositionNormalBuffer[1].VertexBufferRHI.IsValid() ? TEXT("true") : TEXT("false"));
	return ClothSimulPositionNormalBuffer[Index];
}

FMatrix44f& FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothToLocalForWriting()
{
	uint32 Index = GetClothBufferIndexForWriting();
	return ClothToLocal[Index];
}

const FMatrix44f& FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothToLocalForReading(bool bPrevious) const
{
	uint32 Index = GetClothBufferIndexForReading(bPrevious);
	return ClothToLocal[Index];
}

uint32 FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothBufferIndexInternal(bool bPrevious) const
{
	uint32 BufferIndex = 0;
	if (bDoubleBuffer)
	{
		if ((CurrentRevisionNumber - PreviousRevisionNumber) > 1)
		{
			// If the revision number has incremented too much, ignore the request and use the current buffer.
			// With ClearMotionVector calls, we intentionally increment revision number to retrieve current buffer for bPrevious true.
			bPrevious = false;
		}

		BufferIndex = CurrentBuffer ^ (uint32)bPrevious;
	}
	return BufferIndex;
}

uint32 FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothBufferIndexForWriting() const
{
	return bDoubleBuffer ? GetClothBufferIndexInternal(false) : 0;
}

uint32 FGPUBaseSkinAPEXClothVertexFactory::ClothShaderType::GetClothBufferIndexForReading(bool bPrevious) const
{
	uint32 BufferIndex = 0;
	if (bDoubleBuffer)
	{
		BufferIndex = GetClothBufferIndexInternal(bPrevious);
		if (!ClothSimulPositionNormalBuffer[BufferIndex].VertexBufferRHI.IsValid())
		{
			// This only could happen first time updating when the previous data is not available
			check(bPrevious);
			// If no previous data available, use the current one
			BufferIndex = GetClothBufferIndexInternal(false);
		}
	}
	return BufferIndex;
}

/*-----------------------------------------------------------------------------
	TGPUSkinAPEXClothVertexFactory
-----------------------------------------------------------------------------*/

/**
* Modify compile environment to enable the apex clothing path
* @param OutEnvironment - shader compile environment to modify
*/
template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinAPEXClothVertexFactory<BoneInfluenceType>::ModifyCompilationEnvironment( const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment )
{
	Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("GPUSKIN_APEX_CLOTH"),TEXT("1"));
}

template <GPUSkinBoneInfluenceType BoneInfluenceType>
bool TGPUSkinAPEXClothVertexFactory<BoneInfluenceType>::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	if (!IsClothEnabled(Parameters.Platform))
	{
		return false;
	}

	// Material must be flagged for cloth use (or be a special engine material).
	if (!Parameters.MaterialParameters.bIsUsedWithAPEXCloth && !Parameters.MaterialParameters.bIsSpecialEngineMaterial)
	{
		return false;
	}

	return Super::ShouldCompilePermutation(Parameters);
}

template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinAPEXClothVertexFactory<BoneInfluenceType>::SetData(FRHICommandListBase& RHICmdList, const FGPUSkinDataType* InData)
{
	const FGPUSkinAPEXClothDataType* InClothData = (const FGPUSkinAPEXClothDataType*)(InData);
	check(InClothData);

	if (!this->Data)
	{
		ClothDataPtr = new FGPUSkinAPEXClothDataType();
		this->Data = TUniquePtr<FGPUSkinDataType>(ClothDataPtr);
	}

	*ClothDataPtr = *InClothData;
	FGPUBaseSkinVertexFactory::UpdateRHI(RHICmdList);
}

/**
* Creates declarations for each of the vertex stream components and
* initializes the device resource
*/
template <GPUSkinBoneInfluenceType BoneInfluenceType>
void TGPUSkinAPEXClothVertexFactory<BoneInfluenceType>::InitRHI(FRHICommandListBase& RHICmdList)
{
	Super::InitRHI(RHICmdList);

	// list of declaration items
	FVertexDeclarationElementList Elements;
	Super::AddVertexElements(Elements);

	// create the actual device decls
	FVertexFactory::InitDeclaration(Elements);
}

IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_PARAMETER_TYPE(TGPUSkinAPEXClothVertexFactory, SF_Vertex, TGPUSkinAPEXClothVertexFactoryShaderParameters);

/** bind cloth gpu skin vertex factory to its shader file and its shader parameters */
IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_TYPE(TGPUSkinAPEXClothVertexFactory, "/Engine/Private/GpuSkinVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
);


IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSkinPassThroughFactoryLooseParameters, "GPUSkinPassThroughVFLooseParameters");

/*-----------------------------------------------------------------------------
FGPUSkinPassthroughVertexFactory
-----------------------------------------------------------------------------*/
FGPUSkinPassthroughVertexFactory::FGPUSkinPassthroughVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, EVertexAttributeFlags InVertexAttributeMask, uint32 InNumVertices)
	: FLocalVertexFactory(InFeatureLevel, "FGPUSkinPassthroughVertexFactory")
	, NumVertices(InNumVertices)
	, VertexAttributesRequested(InVertexAttributeMask)
{
	bGPUSkinPassThrough = true;
}

void FGPUSkinPassthroughVertexFactory::ResetVertexAttributes(FRHICommandListBase& RHICmdList)
{
	for (int32 Index = 0; Index < EVertexAttribute::NumAttributes; ++Index)
	{
		if (FRHIStreamSourceSlot* Slot = StreamSourceSlots[Index])
		{
			RHICmdList.UpdateStreamSourceSlot(Slot, SourceStreamBuffers[Index]);
		}
	}

	for (int32 Index = 0; Index < EShaderResource::NumShaderResources; ++Index)
	{
		SRVs[Index] = nullptr;
	}
	UpdatedFrameNumber = ~0U;
}

void FGPUSkinPassthroughVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	const bool bSupportsManualVertexFetch = SupportsManualVertexFetch(GetFeatureLevel());

	// Don't bother binding streams that are using manual vertex fetch.
	const auto IsManualVertexFetch = [bSupportsManualVertexFetch] (const FVertexStreamComponent& Component)
	{
		return bSupportsManualVertexFetch && EnumHasAnyFlags(Component.VertexStreamUsage, EVertexStreamUsage::ManualFetch);
	};

	const auto GetVertexBufferRHI = [] (const FVertexBuffer* VertexBuffer) -> FRHIBuffer*
	{
		return VertexBuffer ? VertexBuffer->GetRHI() : GNullVertexBuffer.GetRHI();
	};

	if (EnumHasAnyFlags(VertexAttributesRequested, EVertexAttributeFlags::Position))
	{
		FRHIBuffer* Buffer = GetVertexBufferRHI(Data.PositionComponent.VertexBuffer);
		SourceStreamBuffers[EVertexAttribute::VertexPosition] = Buffer;
		StreamSourceSlots[EVertexAttribute::VertexPosition] = FRHIStreamSourceSlot::Create(Buffer);
		Data.PositionComponent.Offset = 0;
		Data.PositionComponent.VertexStreamUsage |= EVertexStreamUsage::Overridden;
		Data.PositionComponent.Stride = 3 * sizeof(float);
		VertexAttributesToBind |= EVertexAttributeFlags::Position;
	}

	if (EnumHasAnyFlags(VertexAttributesRequested, EVertexAttributeFlags::Color))
	{
		if (!IsManualVertexFetch(Data.ColorComponent))
		{
			FRHIBuffer* Buffer = GetVertexBufferRHI(Data.ColorComponent.VertexBuffer);
			SourceStreamBuffers[EVertexAttribute::VertexColor] = Buffer;
			StreamSourceSlots[EVertexAttribute::VertexColor] = FRHIStreamSourceSlot::Create(Buffer);
			Data.ColorComponent.Offset = 0;
			Data.ColorComponent.Type = VET_Color;
			Data.ColorComponent.VertexStreamUsage |= EVertexStreamUsage::Overridden;
			Data.ColorComponent.Stride = sizeof(uint32);

			VertexAttributesToBind |= EVertexAttributeFlags::Color;
		}
		
		// Set mask to allow full vertex indexing in vertex shader.
		Data.ColorIndexMask = ~0u;
	}

	if (Data.TangentsSRV)
	{
		PreSkinnedTangentSRV = Data.TangentsSRV;
	}

	// TODO: SkinCache can work with PF_R8G8B8A8_SNORM, however Deformers expect PF_R16G16B16A16_SNORM
	// User need to pass an expected format into VF
	TangentFormat = PF_R16G16B16A16_SNORM;
	if (IsOpenGLPlatform(GMaxRHIShaderPlatform))
	{
		// See SkinCache TangentFormat
		TangentFormat = PF_R16G16B16A16_SINT;
	}

	if (EnumHasAnyFlags(VertexAttributesRequested, EVertexAttributeFlags::Tangent) && !IsManualVertexFetch(Data.TangentBasisComponents[0]))
	{
		EVertexElementType VtxType = (TangentFormat == PF_R8G8B8A8_SNORM) ? VET_PackedNormal : VET_Short4N;
		uint32 Stride = (TangentFormat == PF_R8G8B8A8_SNORM) ? 8 : 16;
		uint32 Offset = (TangentFormat == PF_R8G8B8A8_SNORM) ? 4 : 8;
		
		FRHIBuffer* Buffer = GetVertexBufferRHI(Data.TangentBasisComponents[0].VertexBuffer);
		SourceStreamBuffers[EVertexAttribute::VertexTangent] = Buffer;
		StreamSourceSlots[EVertexAttribute::VertexTangent] = FRHIStreamSourceSlot::Create(Buffer);
		Data.TangentBasisComponents[0].VertexStreamUsage |= EVertexStreamUsage::Overridden;
		Data.TangentBasisComponents[0].Offset = 0;
		Data.TangentBasisComponents[0].Type = VtxType;
		Data.TangentBasisComponents[0].Stride = Stride;
		Data.TangentBasisComponents[1].VertexStreamUsage |= EVertexStreamUsage::Overridden;
		Data.TangentBasisComponents[1].Offset = Offset;
		Data.TangentBasisComponents[1].Type = VtxType;
		Data.TangentBasisComponents[1].Stride = Stride;
		VertexAttributesToBind |= EVertexAttributeFlags::Tangent;
	}

	FGPUSkinPassThroughFactoryLooseParameters LooseParameters;
	LooseParameters.FrameNumber = -1;
	LooseParameters.PositionBuffer = GNullVertexBuffer.VertexBufferSRV;
	LooseParameters.PreviousPositionBuffer = GNullVertexBuffer.VertexBufferSRV;
	LooseParameters.PreSkinnedTangentBuffer = GNullVertexBuffer.VertexBufferSRV;
	LooseParametersUniformBuffer = TUniformBufferRef<FGPUSkinPassThroughFactoryLooseParameters>::CreateUniformBufferImmediate(LooseParameters, UniformBuffer_MultiFrame);

	FLocalVertexFactory::InitRHI(RHICmdList);
}

void FGPUSkinPassthroughVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, bool bSupportsManualVertexFetch, FLocalVertexFactory::FDataType& Data, FVertexDeclarationElementList& Elements)
{
	// This is used only for PSO pre-caching 
	// See the tangents setup in a FGPUSkinPassthroughVertexFactory::InitRHI
	if (!bSupportsManualVertexFetch)
	{
		Data.TangentBasisComponents[0].Type = VET_Short4N;
		Data.TangentBasisComponents[0].Stride = 16;
		Data.TangentBasisComponents[0].Offset = 0;
		Data.TangentBasisComponents[1].Type = VET_Short4N;
		Data.TangentBasisComponents[1].Stride = 16;
		Data.TangentBasisComponents[1].Offset = 8;
	}
		
	return FLocalVertexFactory::GetVertexElements(FeatureLevel, InputStreamType, bSupportsManualVertexFetch, Data, Elements);
}

void FGPUSkinPassthroughVertexFactory::UpdateUniformBuffer(FRHICommandListBase& RHICmdList, const FGPUBaseSkinVertexFactory* InSourceVertexFactory)
{
	if (RHISupportsManualVertexFetch(GetFeatureLevelShaderPlatform(GetFeatureLevel())))
	{
		Data.TangentsSRV = SRVs[EShaderResource::Tangent] ? SRVs[EShaderResource::Tangent] : (FRHIShaderResourceView*)InSourceVertexFactory->GetTangentsSRV();
		Data.ColorComponentsSRV = SRVs[EShaderResource::Color] ? SRVs[EShaderResource::Color] : (FRHIShaderResourceView*)InSourceVertexFactory->GetColorComponentsSRV();
		Data.ColorIndexMask = SRVs[EShaderResource::Color] ? Data.ColorIndexMask : InSourceVertexFactory->GetColorIndexMask();
		Data.TextureCoordinatesSRV = SRVs[EShaderResource::TexCoord] ? SRVs[EShaderResource::TexCoord] : (FRHIShaderResourceView*)InSourceVertexFactory->GetTextureCoordinatesSRV();

		const int32 DefaultBaseVertexIndex = 0;
		const int32 DefaultPreSkinBaseVertexIndex = 0;
		FLocalVertexFactoryUniformShaderParameters Parameters;
		GetLocalVFUniformShaderParameters(Parameters, this, Data.LODLightmapDataIndex, nullptr, DefaultBaseVertexIndex, DefaultPreSkinBaseVertexIndex);
		UniformBuffer.UpdateUniformBufferImmediate(RHICmdList, Parameters);
	}
}

void FGPUSkinPassthroughVertexFactory::UpdateLooseUniformBuffer(FRHICommandListBase& RHICmdList, FGPUBaseSkinVertexFactory const* InSourceVertexFactory, uint32 InFrameNumber)
{
	FRHIShaderResourceView* PositionSRV = SRVs[EShaderResource::Position] != nullptr ? SRVs[EShaderResource::Position] : (FRHIShaderResourceView*)InSourceVertexFactory->GetPositionsSRV();
	FRHIShaderResourceView* PrevPositionSRV = SRVs[EShaderResource::PreviousPosition] != nullptr ? SRVs[EShaderResource::PreviousPosition] : PositionSRV;

	FGPUSkinPassThroughFactoryLooseParameters Parameters;
	Parameters.FrameNumber = InFrameNumber;
	Parameters.PositionBuffer = PositionSRV;
	Parameters.PreviousPositionBuffer = PrevPositionSRV;
	if (PreSkinnedTangentSRV)
	{
		Parameters.PreSkinnedTangentBuffer = PreSkinnedTangentSRV;
	}
	else
	{
		Parameters.PreSkinnedTangentBuffer = GNullVertexBuffer.VertexBufferSRV;
	}
	LooseParametersUniformBuffer.UpdateUniformBufferImmediate(RHICmdList, Parameters);
}

void FGPUSkinPassthroughVertexFactory::SetVertexAttributes(FRHICommandListBase& RHICmdList, FGPUBaseSkinVertexFactory const* InSourceVertexFactory, FAddVertexAttributeDesc const& InDesc)
{
	// Check for modified SRVs.
	bool bNeedUniformBufferUpdate = false;
	bool bNeedLooseUniformBufferUpdate = false;
	for (int32 Index = 0; Index < EShaderResource::NumShaderResources; ++Index)
	{
		if (SRVs[Index] != InDesc.SRVs[Index])
		{
			SRVs[Index] = InDesc.SRVs[Index];

			if (Index == EShaderResource::Position || Index == EShaderResource::PreviousPosition)
			{
				// Position SRVs are stored in the special "loose" uniform buffer used only by the passthrough vertex factory.
				bNeedLooseUniformBufferUpdate = true;
			}
			else
			{
				// All other SRVs are stored in the main vertex factory uniform buffer.
				bNeedUniformBufferUpdate = true;
			}
		}
	}

	if (bNeedUniformBufferUpdate || bNeedLooseUniformBufferUpdate)
	{
		for (int32 Index = 0; Index < EVertexAttribute::NumAttributes; ++Index)
		{
			if (FRHIStreamSourceSlot* Slot = StreamSourceSlots[Index])
			{
				RHICmdList.UpdateStreamSourceSlot(Slot, InDesc.StreamBuffers[Index] ? InDesc.StreamBuffers[Index] : SourceStreamBuffers[Index]);
			}
		}
	}

	if (UpdatedFrameNumber != InDesc.FrameNumber)
	{
		// Loose uniform buffer include the latest frame number.
		UpdatedFrameNumber = InDesc.FrameNumber;
		bNeedLooseUniformBufferUpdate = true;
	}

	if (bNeedUniformBufferUpdate)
	{
		// Only need to recreate the vertex factory uniform buffer.
		UpdateUniformBuffer(RHICmdList, InSourceVertexFactory);
	}

	if (bNeedLooseUniformBufferUpdate)
	{
		// Update the loose uniform buffer.
		UpdateLooseUniformBuffer(RHICmdList, InSourceVertexFactory, InDesc.FrameNumber);
	}
}

void FGPUSkinPassthroughVertexFactory::GetOverrideVertexStreams(FVertexInputStreamArray& VertexStreams) const
{
	for (int32 Index = 0; Index < EVertexAttribute::NumAttributes; ++Index)
	{
		if (EnumHasAnyFlags(VertexAttributesToBind, static_cast<EVertexAttributeFlags>(1 << Index)))
		{
			VertexStreams.Emplace(Index, 0, StreamSourceSlots[Index]);
		}
	}
}

FGPUBaseSkinVertexFactory::FUpdateScope::FUpdateScope()
	: Cloth(GClothBufferPool)
	, Bone(GBoneBufferPool)
{}

#undef IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_PARAMETER_TYPE
#undef IMPLEMENT_GPUSKINNING_VERTEX_FACTORY_TYPE
