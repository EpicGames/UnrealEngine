// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Matrix.isph"

export void UpdateBoneData_CopyBones(uniform FMatrix3x4 ChunkMatrices[],
										const uniform FMatrix44f ReferenceToLocalMatrices[],
										const uniform unsigned int16 BoneMap[],
										const uniform int NumBones)
{
	uniform FMatrix3x4 *uniform ChunkPtr = (uniform FMatrix3x4 * uniform) ChunkMatrices;
	uniform FMatrix44f *uniform RefToLocalPtr = (uniform FMatrix44f * uniform) ReferenceToLocalMatrices;

	for(uniform int BoneIdx = 0; BoneIdx < NumBones; BoneIdx++)
	{
		const uniform unsigned int16 RefToLocalIdx = BoneMap[BoneIdx];

		uniform float *uniform SrcPtr = (uniform float *uniform)(RefToLocalPtr + RefToLocalIdx);
		uniform float *uniform DstPtr = (uniform float *uniform)(ChunkPtr + BoneIdx);

#if TARGET_WIDTH == 4
		static const varying int vPermxy = {0, 4, 1, 5};
		static const varying int vPermzw = {2, 6, 3, 7};

		static const varying int vPermxx = {0, 1, 4, 5};
		static const varying int vPermyy = {2, 3, 6, 7};

		float S1 = SrcPtr[programIndex];
		float S2 = SrcPtr[programIndex+programCount];
		float S3 = SrcPtr[programIndex+2*programCount];
		float S4 = SrcPtr[programIndex+3*programCount];

		float xy1 = shuffle(S1, S2, vPermxy);
		float zw1 = shuffle(S1, S2, vPermzw);
		float xy2 = shuffle(S3, S4, vPermxy);
		float zw2 = shuffle(S3, S4, vPermzw);

		DstPtr[programIndex] = shuffle(xy1, xy2, vPermxx);
		DstPtr[programIndex+programCount] = shuffle(xy1, xy2, vPermyy);
		DstPtr[programIndex+2*programCount] = shuffle(zw1, zw2, vPermxx);

#elif TARGET_WIDTH == 8
		static const varying int vPermxy = {0, 4, 8, 12, 1, 5, 9, 13};
		static const varying int vPermzw = {2, 6, 10, 14, 3, 7, 11, 15};

		float S1 = SrcPtr[programIndex];
		float S2 = SrcPtr[programIndex+programCount];

		DstPtr[programIndex] = shuffle(S1, S2, vPermxy);
		if(programIndex < 4)
		{
			DstPtr[programIndex+programCount] = shuffle(S1, S2, vPermzw);
		}

#elif TARGET_WIDTH == 16
		static const varying int vPerm = {0, 4, 8, 12, 1, 5, 9, 13,
										2, 6, 10, 14, 3, 7, 11, 15};

		float S1 = SrcPtr[programIndex];

		if(programIndex < 12)
		{
			DstPtr[programIndex] = shuffle(S1, vPerm);
		}

#else
		#error "Unknown ISPC_TARGET_* value"
#endif
	}
}
