// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSort.cpp: Implementation for sorting buffers on the GPU.
=============================================================================*/

#include "GPUSort.h"
#include "Math/RandomStream.h"
#include "RHIBreadcrumbs.h"
#include "RenderingThread.h"
#include "RHIContext.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterUtils.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DEFINE_LOG_CATEGORY_STATIC(LogGPUSort, Log, All);

/*------------------------------------------------------------------------------
	Global settings.
------------------------------------------------------------------------------*/

static bool GbSortDebugOffsets = false;
static bool GbSortDebug = false;
static FAutoConsoleVariableRef CVarDebugOffsets(TEXT("GPUSort.DebugOffsets"), GbSortDebugOffsets, TEXT("Debug GPU sort offsets."));
static FAutoConsoleVariableRef CVarDebugSort(TEXT("GPUSort.DebugSort"), GbSortDebug, TEXT("Debug GPU sorting."));

#define GPUSORT_BITCOUNT 32
#define RADIX_BITS 4
#define DIGIT_COUNT (1 << RADIX_BITS)
#define KEYS_PER_LOOP 8
#define THREAD_COUNT 128
#define TILE_SIZE (THREAD_COUNT * KEYS_PER_LOOP)
#define MAX_GROUP_COUNT 64
#define MAX_PASS_COUNT (32 / RADIX_BITS)

/**
 * Setup radix sort shader compiler environment.
 * @param OutEnvironment - The environment to set.
 */
void SetRadixSortShaderCompilerEnvironment( FShaderCompilerEnvironment& OutEnvironment )
{
	OutEnvironment.SetDefine( TEXT("RADIX_BITS"), RADIX_BITS );
	OutEnvironment.SetDefine( TEXT("THREAD_COUNT"), THREAD_COUNT );
	OutEnvironment.SetDefine( TEXT("KEYS_PER_LOOP"), KEYS_PER_LOOP );
	OutEnvironment.SetDefine( TEXT("MAX_GROUP_COUNT"), MAX_GROUP_COUNT );
	OutEnvironment.CompilerFlags.Add( CFLAG_StandardOptimization );
}

/*------------------------------------------------------------------------------
	Uniform buffer for passing in radix sort parameters.
------------------------------------------------------------------------------*/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FRadixSortParameters, )
	SHADER_PARAMETER( uint32, RadixShift )
	SHADER_PARAMETER( uint32, TilesPerGroup )
	SHADER_PARAMETER( uint32, ExtraTileCount )
	SHADER_PARAMETER( uint32, ExtraKeyCount )
	SHADER_PARAMETER( uint32, GroupCount )
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FRadixSortParameters,"RadixSortUB");

typedef TUniformBufferRef<FRadixSortParameters> FRadixSortUniformBufferRef;

/*------------------------------------------------------------------------------
	Global resources.
------------------------------------------------------------------------------*/

/**
 * Global sort offset buffer resources.
 */
class FSortOffsetBuffers : public FRenderResource
{
public:

	/** Vertex buffer storage for the actual offsets. */
	FBufferRHIRef Buffers[2];
	/** Shader resource views for offset buffers. */
	FShaderResourceViewRHIRef BufferSRVs[2];
	/** Unordered access views for offset buffers. */
	FUnorderedAccessViewRHIRef BufferUAVs[2];

	/**
	 * Initialize RHI resources.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const int32 OffsetsCount = DIGIT_COUNT * MAX_GROUP_COUNT;
		const int32 OffsetsBufferSize = OffsetsCount * sizeof(uint32);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("SortOffset"), OffsetsBufferSize)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::UnorderedAccess)
			.DetermineInitialState();

		for (int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex)
		{
			Buffers[BufferIndex] = RHICmdList.CreateBuffer(CreateDesc);
			BufferSRVs[BufferIndex] = RHICmdList.CreateShaderResourceView(
				Buffers[BufferIndex],
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
			BufferUAVs[BufferIndex] = RHICmdList.CreateUnorderedAccessView(
				Buffers[BufferIndex],
				FRHIViewDesc::CreateBufferUAV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
		}
	}

	/**
	 * Release RHI resources.
	 */
	virtual void ReleaseRHI() override
	{
		for ( int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex )
		{
			BufferUAVs[BufferIndex].SafeRelease();
			BufferSRVs[BufferIndex].SafeRelease();
			Buffers[BufferIndex].SafeRelease();
		}
	}

	/**
	 * Retrieve offsets from the buffer.
	 * @param OutOffsets - Array to hold the offsets.
	 * @param BufferIndex - Which buffer to retrieve.
	 */
	void GetOffsets(FRHICommandListBase& RHICmdList, TArray<uint32>& OutOffsets, int32 BufferIndex )
	{
		const int32 OffsetsCount = DIGIT_COUNT * MAX_GROUP_COUNT;
		const int32 OffsetsBufferSize = OffsetsCount * sizeof(uint32);

		OutOffsets.Empty( OffsetsCount );
		OutOffsets.AddUninitialized( OffsetsCount );
		uint32* MappedOffsets = (uint32*)RHICmdList.LockBuffer( Buffers[BufferIndex], 0, OffsetsBufferSize, RLM_ReadOnly );
		FMemory::Memcpy( OutOffsets.GetData(), MappedOffsets, OffsetsBufferSize );
		RHICmdList.UnlockBuffer( Buffers[BufferIndex] );
	}

	/**
	 * Dumps the contents of the offsets buffer via debugf.
	 * @param BufferIndex - Which buffer to dump.
	 */
	void DumpOffsets(FRHICommandListBase& RHICmdList, int32 BufferIndex)
	{
		TArray<uint32> Offsets;
		uint32 GrandTotal = 0;

		GetOffsets(RHICmdList, Offsets, BufferIndex);
		for (int32 GroupIndex = 0; GroupIndex < MAX_GROUP_COUNT; ++GroupIndex)
		{
			uint32 DigitTotal = 0;
			FString GroupOffsets;
			for (int32 DigitIndex = 0; DigitIndex < DIGIT_COUNT; ++DigitIndex)
			{
				const uint32 Value = Offsets[GroupIndex * DIGIT_COUNT + DigitIndex];
				GroupOffsets += FString::Printf(TEXT(" %04d"), Value);
				DigitTotal += Value;
				GrandTotal += Value;
			}
			UE_LOGF(LogGPUSort, Log, "%ls = %u", *GroupOffsets, DigitTotal);
		}
		UE_LOGF(LogGPUSort, Log, "Total: %u", GrandTotal);
	}
};

/** The global sort offset buffer resources. */
TGlobalResource<FSortOffsetBuffers> GSortOffsetBuffers;

/**
 * This buffer is used to workaround a constant buffer bug that appears to
 * manifest itself on NVIDIA GPUs.
 */
class FRadixSortParametersBuffer : public FRenderResource
{
public:

	/** The vertex buffer used for storage. */
	FBufferRHIRef SortParametersBufferRHI;
	/** Shader resource view in to the vertex buffer. */
	FShaderResourceViewRHIRef SortParametersBufferSRV;
	
	/**
	 * Initialize RHI resources.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FRadixSortParametersBuffer"), sizeof(FRadixSortParameters))
			.AddUsage(EBufferUsageFlags::Volatile | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState();

		SortParametersBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		SortParametersBufferSRV = RHICmdList.CreateShaderResourceView(
				SortParametersBufferRHI,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
	}

	/**
	 * Release RHI resources.
	 */
	virtual void ReleaseRHI() override
	{
		SortParametersBufferSRV.SafeRelease();
		SortParametersBufferRHI.SafeRelease();
	}
};

/** The global resource for the radix sort parameters buffer. */
TGlobalResource<FRadixSortParametersBuffer> GRadixSortParametersBuffer;

/*------------------------------------------------------------------------------
	The offset clearing kernel. This kernel just zeroes out the offsets buffer.

	Note that MAX_GROUP_COUNT * DIGIT_COUNT must be a multiple of THREAD_COUNT.
------------------------------------------------------------------------------*/

class FRadixSortClearOffsetsCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FRadixSortClearOffsetsCS,Global);
public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("RADIX_SORT_CLEAR_OFFSETS"), 1);
		SetRadixSortShaderCompilerEnvironment(OutEnvironment);
	}

	/** Default constructor. */
	FRadixSortClearOffsetsCS() = default;

	/** Initialization constructor. */
	explicit FRadixSortClearOffsetsCS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FGlobalShader(Initializer)
	{
		OutOffsets.Bind( Initializer.ParameterMap, TEXT("OutOffsets") );
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHIUnorderedAccessView* OutOffsetsUAV)
	{
		SetUAVParameter(BatchedParameters, OutOffsets, OutOffsetsUAV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetUAVParameter(BatchedUnbinds, OutOffsets);
	}

private:

	/** The buffer to which offsets will be written. */
	LAYOUT_FIELD(FShaderResourceParameter, OutOffsets);
};
IMPLEMENT_SHADER_TYPE(,FRadixSortClearOffsetsCS,TEXT("/Engine/Private/RadixSortShaders.usf"),TEXT("RadixSort_ClearOffsets"),SF_Compute);

/*------------------------------------------------------------------------------
	The upsweep sorting kernel. This kernel performs an upsweep scan on all
	tiles allocated to this group and computes per-digit totals. These totals
	are output to the offsets buffer.
------------------------------------------------------------------------------*/

class FRadixSortUpsweepCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FRadixSortUpsweepCS,Global);

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetDefine( TEXT("RADIX_SORT_UPSWEEP"), 1 );
		SetRadixSortShaderCompilerEnvironment( OutEnvironment );
	}

	/** Default constructor. */
	FRadixSortUpsweepCS()
	{
	}

	/** Initialization constructor. */
	explicit FRadixSortUpsweepCS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FGlobalShader(Initializer)
	{
		RadixSortParameterBuffer.Bind( Initializer.ParameterMap, TEXT("RadixSortParameterBuffer") );
		InKeys.Bind( Initializer.ParameterMap, TEXT("InKeys") );
		OutOffsets.Bind( Initializer.ParameterMap, TEXT("OutOffsets") );
	}

	/**
	 * Returns true if this shader was compiled to require the constant buffer
	 * workaround.
	 */
	bool RequiresConstantBufferWorkaround() const
	{
		return RadixSortParameterBuffer.IsBound();
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FRadixSortUniformBufferRef& SortUniformBufferRef, FRHIUnorderedAccessView* OutOffsetsUAV, FRHIShaderResourceView* InKeysSRV, const FRadixSortUniformBufferRef& RadixSortUniformBuffer, FRHIShaderResourceView* RadixSortParameterBufferSRV)
	{
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FRadixSortParameters>(), SortUniformBufferRef);

		SetUAVParameter(BatchedParameters, OutOffsets, OutOffsetsUAV);
		SetSRVParameter(BatchedParameters, InKeys, InKeysSRV);
		SetSRVParameter(BatchedParameters, RadixSortParameterBuffer, RadixSortParameterBufferSRV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetSRVParameter(BatchedUnbinds, RadixSortParameterBuffer);
		UnsetSRVParameter(BatchedUnbinds, InKeys);
		UnsetUAVParameter(BatchedUnbinds, OutOffsets);
	}

private:
	/** Uniform parameters stored in a vertex buffer, used to workaround an NVIDIA driver bug. */
	LAYOUT_FIELD(FShaderResourceParameter, RadixSortParameterBuffer);
	/** The buffer containing input keys. */
	LAYOUT_FIELD(FShaderResourceParameter, InKeys);
	/** The buffer to which offsets will be written. */
	LAYOUT_FIELD(FShaderResourceParameter, OutOffsets);
};
IMPLEMENT_SHADER_TYPE(,FRadixSortUpsweepCS,TEXT("/Engine/Private/RadixSortShaders.usf"),TEXT("RadixSort_Upsweep"),SF_Compute);

/*------------------------------------------------------------------------------
	The spine sorting kernel. This kernel performs a parallel prefix sum on
	the offsets computed by each work group in upsweep. The outputs will be used
	by individual work groups in downsweep to compute the final location of keys.
------------------------------------------------------------------------------*/

class FRadixSortSpineCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FRadixSortSpineCS,Global);

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetDefine( TEXT("RADIX_SORT_SPINE"), 1 );
		SetRadixSortShaderCompilerEnvironment( OutEnvironment );
	}

	/** Default constructor. */
	FRadixSortSpineCS()
	{
	}

	/** Initialization constructor. */
	explicit FRadixSortSpineCS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FGlobalShader(Initializer)
	{
		InOffsets.Bind( Initializer.ParameterMap, TEXT("InOffsets") );
		OutOffsets.Bind( Initializer.ParameterMap, TEXT("OutOffsets") );
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHIUnorderedAccessView* OutOffsetsUAV, FRHIShaderResourceView* InOffsetsSRV)
	{
		SetUAVParameter(BatchedParameters, OutOffsets, OutOffsetsUAV);
		SetSRVParameter(BatchedParameters, InOffsets, InOffsetsSRV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetSRVParameter(BatchedUnbinds, InOffsets);
		UnsetUAVParameter(BatchedUnbinds, OutOffsets);
	}

private:

	/** The buffer containing input offsets. */
	LAYOUT_FIELD(FShaderResourceParameter, InOffsets);
	/** The buffer to which offsets will be written. */
	LAYOUT_FIELD(FShaderResourceParameter, OutOffsets);
};
IMPLEMENT_SHADER_TYPE(,FRadixSortSpineCS,TEXT("/Engine/Private/RadixSortShaders.usf"),TEXT("RadixSort_Spine"),SF_Compute);

/*------------------------------------------------------------------------------
	The downsweep sorting kernel. This kernel reads the per-work group partial
	sums in to LocalTotals. The kernel then recomputes much of the work done
	upsweep, this time computing a full set of prefix sums so that keys can be
	scattered in to global memory.
------------------------------------------------------------------------------*/

class FRadixSortDownsweepCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FRadixSortDownsweepCS,Global);

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetDefine( TEXT("RADIX_SORT_DOWNSWEEP"), 1 );
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
		SetRadixSortShaderCompilerEnvironment( OutEnvironment );
	}

	/** Default constructor. */
	FRadixSortDownsweepCS()
	{
	}

	/** Initialization constructor. */
	explicit FRadixSortDownsweepCS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FGlobalShader(Initializer)
	{
		RadixSortParameterBuffer.Bind( Initializer.ParameterMap, TEXT("RadixSortParameterBuffer") );
		InKeys.Bind( Initializer.ParameterMap, TEXT("InKeys") );
		InValues.Bind( Initializer.ParameterMap, TEXT("InValues") );
		InOffsets.Bind( Initializer.ParameterMap, TEXT("InOffsets") );
		OutKeys.Bind( Initializer.ParameterMap, TEXT("OutKeys") );
		OutValues.Bind( Initializer.ParameterMap, TEXT("OutValues") );
	}

	/**
	 * Returns true if this shader was compiled to require the constant buffer
	 * workaround.
	 */
	bool RequiresConstantBufferWorkaround() const
	{
		return RadixSortParameterBuffer.IsBound();
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FRadixSortUniformBufferRef& SortUniformBufferRef,
		FRHIUnorderedAccessView* OutKeysUAV,
		FRHIUnorderedAccessView* OutValuesUAV,
		FRHIShaderResourceView* InKeysSRV,
		FRHIShaderResourceView* InValuesSRV,
		FRHIShaderResourceView* InOffsetsSRV,
		FRHIShaderResourceView* RadixSortParameterBufferSRV )
	{
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FRadixSortParameters>(), SortUniformBufferRef);

		SetUAVParameter(BatchedParameters, OutKeys, OutKeysUAV);
		SetUAVParameter(BatchedParameters, OutValues, OutValuesUAV);

		SetSRVParameter(BatchedParameters, RadixSortParameterBuffer, RadixSortParameterBufferSRV);
		SetSRVParameter(BatchedParameters, InKeys, InKeysSRV);
		SetSRVParameter(BatchedParameters, InValues, InValuesSRV);
		SetSRVParameter(BatchedParameters, InOffsets, InOffsetsSRV);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetSRVParameter(BatchedUnbinds, RadixSortParameterBuffer);
		UnsetSRVParameter(BatchedUnbinds, InKeys);
		UnsetSRVParameter(BatchedUnbinds, InValues);
		UnsetSRVParameter(BatchedUnbinds, InOffsets);
		UnsetUAVParameter(BatchedUnbinds, OutKeys);
		UnsetUAVParameter(BatchedUnbinds, OutValues);
	}

private:
	/** Uniform parameters stored in a vertex buffer, used to workaround an NVIDIA driver bug. */
	LAYOUT_FIELD(FShaderResourceParameter, RadixSortParameterBuffer);
	/** The buffer containing input keys. */
	LAYOUT_FIELD(FShaderResourceParameter, InKeys);
	/** The buffer containing input values. */
	LAYOUT_FIELD(FShaderResourceParameter, InValues);
	/** The buffer from which offsets will be read. */
	LAYOUT_FIELD(FShaderResourceParameter, InOffsets);
	/** The buffer to which sorted keys will be written. */
	LAYOUT_FIELD(FShaderResourceParameter, OutKeys);
	/** The buffer to which sorted values will be written. */
	LAYOUT_FIELD(FShaderResourceParameter, OutValues);
};
IMPLEMENT_SHADER_TYPE(,FRadixSortDownsweepCS,TEXT("/Engine/Private/RadixSortShaders.usf"),TEXT("RadixSort_Downsweep"),SF_Compute);

/*------------------------------------------------------------------------------
	Public interface.
------------------------------------------------------------------------------*/

/**
 * Get the number of passes we will need to make in order to sort
 */
int32 GetGPUSortPassCount(uint32 KeyMask)
{
	const int32 BitCount = GPUSORT_BITCOUNT;
	const int32 PassCount = BitCount / RADIX_BITS;

	int32 PassesRequired = 0;

	uint32 PassBits = DIGIT_COUNT - 1;
	for (int32 PassIndex = 0; PassIndex < PassCount; ++PassIndex)
	{
		// Check to see if these key bits matter.
		if ((PassBits & KeyMask) != 0)
		{
			++PassesRequired;
		}
		PassBits <<= RADIX_BITS;
	}
	return PassesRequired;
}

/**
 * Sort a buffer on the GPU.
 * @param SortBuffers - The buffer to sort including required views and a ping-
 *			pong location of appropriate size.
 * @param BufferIndex - Index of the buffer containing keys.
 * @param KeyMask - Bitmask indicating which key bits contain useful information.
 * @param Count - How many items in the buffer need to be sorted.
 * @returns The index of the buffer containing sorted results.
 */
int32 SortGPUBuffers(FRHICommandList& RHICmdList, FGPUSortBuffers SortBuffers, int32 BufferIndex, uint32 KeyMask, int32 Count, ERHIFeatureLevel::Type FeatureLevel)
{
	FRadixSortParameters SortParameters;
	FRadixSortUniformBufferRef SortUniformBufferRef;
	const bool bDebugOffsets = GbSortDebugOffsets;
	const bool bDebugSort = GbSortDebug;

	SCOPED_DRAW_EVENTF(RHICmdList, SortGPU, TEXT("Sort(%d)"), Count);

	// Determine how many tiles need to be sorted.
	const int32 TileCount = Count / TILE_SIZE;

	// Determine how many groups will be needed.
	int32 GroupCount = TileCount;
	if ( GroupCount > MAX_GROUP_COUNT )
	{
		GroupCount = MAX_GROUP_COUNT;
	}
	else if ( GroupCount == 0 )
	{
		GroupCount = 1;
	}

	// The number of tiles processed by each group.
	const int32 TilesPerGroup = TileCount / GroupCount;

	// The number of additional tiles that need to be processed.
	const int32 ExtraTileCount = TileCount % GroupCount;

	// The number of additional keys that need to be processed.
	const int32 ExtraKeyCount = Count % TILE_SIZE;

	// Determine how many passes are required.
	const int32 BitCount = GPUSORT_BITCOUNT;
	const int32 PassCount = BitCount / RADIX_BITS;

	// Setup sort parameters.
	SortParameters.RadixShift = 0;
	SortParameters.TilesPerGroup = TilesPerGroup;
	SortParameters.ExtraTileCount = ExtraTileCount;
	SortParameters.ExtraKeyCount = ExtraKeyCount;
	SortParameters.GroupCount = GroupCount;

	// Grab shaders.
	auto ShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FRadixSortClearOffsetsCS> ClearOffsetsCS(ShaderMap);
	TShaderMapRef<FRadixSortUpsweepCS> UpsweepCS(ShaderMap);
	TShaderMapRef<FRadixSortSpineCS> SpineCS(ShaderMap);
	TShaderMapRef<FRadixSortDownsweepCS> DownsweepCS(ShaderMap);

	// Constant buffer workaround. Both shaders must use either the constant buffer or vertex buffer.
	check( UpsweepCS->RequiresConstantBufferWorkaround() == DownsweepCS->RequiresConstantBufferWorkaround() );
	const bool bUseConstantBufferWorkaround = UpsweepCS->RequiresConstantBufferWorkaround();


	
	// Execute each pass as needed.
	uint32 PassBits = DIGIT_COUNT - 1;
	for ( int32 PassIndex = 0; PassIndex < PassCount; ++PassIndex )
	{
		// Check to see if these key bits matter.
		if ( (PassBits & KeyMask) != 0 )
		{
			// Update uniform buffer.
			if ( bUseConstantBufferWorkaround )
			{
				void* ParameterBuffer = RHICmdList.LockBuffer( GRadixSortParametersBuffer.SortParametersBufferRHI, 0, sizeof(FRadixSortParameters), RLM_WriteOnly );
				FMemory::Memcpy( ParameterBuffer, &SortParameters, sizeof(FRadixSortParameters) );
				RHICmdList.UnlockBuffer( GRadixSortParametersBuffer.SortParametersBufferRHI );
			}
			else
			{
				SortUniformBufferRef = FRadixSortUniformBufferRef::CreateUniformBufferImmediate( SortParameters, UniformBuffer_SingleDraw );
			}

			//make UAV safe for clear
			RHICmdList.Transition({
				FRHITransitionInfo(GSortOffsetBuffers.BufferUAVs[0], ERHIAccess::Unknown, ERHIAccess::UAVCompute)
			});
			
			// Clear the offsets buffer.
			SetComputePipelineState(RHICmdList, ClearOffsetsCS.GetComputeShader());
			SetShaderParametersLegacyCS(RHICmdList, ClearOffsetsCS, GSortOffsetBuffers.BufferUAVs[0]);

			DispatchComputeShader(RHICmdList, ClearOffsetsCS.GetShader(), 1, 1 ,1 );

			UnsetShaderParametersLegacyCS(RHICmdList, ClearOffsetsCS);

			//make UAV safe for readback
			RHICmdList.Transition({
				FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[BufferIndex], ERHIAccess::Unknown, ERHIAccess::SRVCompute),
				FRHITransitionInfo(GSortOffsetBuffers.BufferUAVs[0], ERHIAccess::Unknown, ERHIAccess::UAVCompute)
			});

			// Phase 1: Scan upsweep to compute per-digit totals.
			SetComputePipelineState(RHICmdList, UpsweepCS.GetComputeShader());

			SetShaderParametersLegacyCS(RHICmdList, UpsweepCS, SortUniformBufferRef, GSortOffsetBuffers.BufferUAVs[0], SortBuffers.RemoteKeySRVs[BufferIndex], SortUniformBufferRef, GRadixSortParametersBuffer.SortParametersBufferSRV);

			DispatchComputeShader(RHICmdList, UpsweepCS.GetShader(), GroupCount, 1, 1);

			UnsetShaderParametersLegacyCS(RHICmdList, UpsweepCS);

			//barrier both UAVS since for next step.
			RHICmdList.Transition({
				FRHITransitionInfo(GSortOffsetBuffers.BufferUAVs[0], ERHIAccess::UAVCompute, ERHIAccess::SRVCompute),
				FRHITransitionInfo(GSortOffsetBuffers.BufferUAVs[1], ERHIAccess::Unknown, ERHIAccess::UAVCompute)
			});

			if (bDebugOffsets)
			{
				UE_LOGF(LogGPUSort, Log, "\n========== UPSWEEP ==========");
				GSortOffsetBuffers.DumpOffsets(RHICmdList, 0);
			}

			// Phase 2: Parallel prefix scan on the offsets buffer.
			SetComputePipelineState(RHICmdList, SpineCS.GetComputeShader());
			SetShaderParametersLegacyCS(RHICmdList, SpineCS, GSortOffsetBuffers.BufferUAVs[1], GSortOffsetBuffers.BufferSRVs[0]);

			DispatchComputeShader(RHICmdList, SpineCS.GetShader(), 1, 1, 1 );

			UnsetShaderParametersLegacyCS(RHICmdList, SpineCS);

			if (bDebugOffsets)
			{
				UE_LOGF(LogGPUSort, Log, "\n========== SPINE ==========");
				GSortOffsetBuffers.DumpOffsets(RHICmdList, 1);
			}

			RHICmdList.Transition({
				FRHITransitionInfo(GSortOffsetBuffers.BufferUAVs[1], ERHIAccess::UAVCompute, ERHIAccess::SRVCompute),
				FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[BufferIndex ^ 0x1], ERHIAccess::Unknown, ERHIAccess::UAVCompute),
				FRHITransitionInfo(SortBuffers.RemoteValueUAVs[BufferIndex ^ 0x1], ERHIAccess::Unknown, ERHIAccess::UAVCompute)
			});

			const bool bIsLastPass = ((PassBits << RADIX_BITS) & KeyMask) == 0;
			// Phase 3: Downsweep to compute final offsets and scatter keys.
			SetComputePipelineState(RHICmdList, DownsweepCS.GetComputeShader());

			{
				FRHIUnorderedAccessView* ValuesUAV = nullptr;
				if (bIsLastPass && SortBuffers.FinalValuesUAV)
				{
					ValuesUAV = SortBuffers.FinalValuesUAV;
					// Transition resource since FinalValuesUAV can also be SortBuffers.FirstValuesSRV.
					RHICmdList.Transition(FRHITransitionInfo(ValuesUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
				}
				else
				{
					ValuesUAV = SortBuffers.RemoteValueUAVs[BufferIndex ^ 0x1];
				}

				FRHIShaderResourceView* ValuesSRV = (PassIndex == 0 && SortBuffers.FirstValuesSRV) ? SortBuffers.FirstValuesSRV : SortBuffers.RemoteValueSRVs[BufferIndex];

				SetShaderParametersLegacyCS(RHICmdList, DownsweepCS, SortUniformBufferRef, SortBuffers.RemoteKeyUAVs[BufferIndex ^ 0x1], ValuesUAV, SortBuffers.RemoteKeySRVs[BufferIndex], ValuesSRV, GSortOffsetBuffers.BufferSRVs[1], GRadixSortParametersBuffer.SortParametersBufferSRV);
			}
			DispatchComputeShader(RHICmdList, DownsweepCS.GetShader(), GroupCount, 1, 1 );
			UnsetShaderParametersLegacyCS(RHICmdList, DownsweepCS);


			RHICmdList.Transition({
				FRHITransitionInfo(SortBuffers.RemoteKeyUAVs[BufferIndex ^ 0x1], ERHIAccess::UAVCompute, ERHIAccess::SRVCompute),
				FRHITransitionInfo(SortBuffers.RemoteValueUAVs[BufferIndex ^ 0x1], ERHIAccess::UAVCompute, ERHIAccess::SRVCompute)
			});

			// Flip buffers.
			BufferIndex ^= 0x1;

			if (bDebugSort || bDebugOffsets)
			{
				return BufferIndex;
			}
		}

		// Update the radix shift for the next pass and flip buffers.
		SortParameters.RadixShift += RADIX_BITS;
		PassBits <<= RADIX_BITS;
	}

	return BufferIndex;
}

/*------------------------------------------------------------------------------
	Testing.
------------------------------------------------------------------------------*/

enum
{
	GPU_SORT_TEST_SIZE_SMALL = (1 << 9),
	GPU_SORT_TEST_SIZE_LARGE = (1 << 20),
	GPU_SORT_TEST_SIZE_MIN = (1 << 4),
	GPU_SORT_TEST_SIZE_MAX = (1 << 20)
};

/**
 * Execute a GPU sort test.
 * @param TestSize - The number of elements to sort.
 * @returns true if the sort succeeded.
 */
static bool RunGPUSortTest(FRHICommandListImmediate& RHICmdList, int32 TestSize, ERHIFeatureLevel::Type FeatureLevel)
{
	FRandomStream RandomStream(0x3819FFE4);
	FGPUSortBuffers SortBuffers;
	TArray<uint32> Keys;
	TArray<uint32> RefSortedKeys;
	TArray<uint32> SortedKeys;
	TArray<uint32> SortedValues;
	FBufferRHIRef KeysBufferRHI[2], ValuesBufferRHI[2];
	FShaderResourceViewRHIRef KeysBufferSRV[2], ValuesBufferSRV[2];
	FUnorderedAccessViewRHIRef KeysBufferUAV[2], ValuesBufferUAV[2];
	int32 ResultBufferIndex;
	int32 IncorrectKeyIndex = 0;
	const int32 BufferSize = TestSize * sizeof(uint32);
	const bool bDebugOffsets = GbSortDebugOffsets;
	const bool bDebugSort = GbSortDebug;

	// Generate the test keys.
	Keys.Reserve(TestSize);
	Keys.AddUninitialized(TestSize);
	for (int32 KeyIndex = 0; KeyIndex < TestSize; ++KeyIndex)
	{
		Keys[KeyIndex] = RandomStream.GetUnsignedInt();
	}

	// Perform a reference sort on the CPU.
	RefSortedKeys = Keys;
	RefSortedKeys.Sort();

	// Upload initial keys and values to the GPU.
	for ( int32 i=0; i < UE_ARRAY_COUNT(KeysBufferRHI); ++i )
	{
		const FRHIBufferCreateDesc KeysCreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("KeysBuffer"), BufferSize)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::UnorderedAccess)
			.SetInitActionInitializer()
			.DetermineInitialState();

		FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(KeysCreateDesc);
		Initializer.WriteData(Keys.GetData(), BufferSize);
		KeysBufferRHI[i] = Initializer.Finalize();
	}

	for (int32 i = 0; i < UE_ARRAY_COUNT(ValuesBufferRHI); ++i)
	{
		const FRHIBufferCreateDesc ValuesCreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("ValuesBuffer"), BufferSize)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::UnorderedAccess)
			.SetInitActionInitializer()
			.DetermineInitialState();

		FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(ValuesCreateDesc);
		Initializer.WriteData(Keys.GetData(), BufferSize);
		ValuesBufferRHI[i] = Initializer.Finalize();
	}

	// Allocate GPU resources.
	for (int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex)
	{
		KeysBufferSRV[BufferIndex] = RHICmdList.CreateShaderResourceView(
			KeysBufferRHI[BufferIndex],
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
		KeysBufferUAV[BufferIndex] = RHICmdList.CreateUnorderedAccessView(
			KeysBufferRHI[BufferIndex], 
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));

		ValuesBufferSRV[BufferIndex] = RHICmdList.CreateShaderResourceView(
			ValuesBufferRHI[BufferIndex],
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
		ValuesBufferUAV[BufferIndex] = RHICmdList.CreateUnorderedAccessView(
			ValuesBufferRHI[BufferIndex], 
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}

	// Execute the GPU sort.
	for (int32 BufferIndex = 0; BufferIndex < 2; ++BufferIndex)
	{
		SortBuffers.RemoteKeySRVs[BufferIndex] = KeysBufferSRV[BufferIndex];
		SortBuffers.RemoteKeyUAVs[BufferIndex] = KeysBufferUAV[BufferIndex];
		SortBuffers.RemoteValueSRVs[BufferIndex] = ValuesBufferSRV[BufferIndex];
		SortBuffers.RemoteValueUAVs[BufferIndex] = ValuesBufferUAV[BufferIndex];
	}
	ResultBufferIndex = SortGPUBuffers(RHICmdList, SortBuffers, 0, 0xFFFFFFFF, TestSize, FeatureLevel);

	// Download results from the GPU.
	{
		uint32* Buffer;

		SortedKeys.Reserve(TestSize);
		SortedKeys.AddUninitialized(TestSize);
		SortedValues.Reserve(TestSize);
		SortedValues.AddUninitialized(TestSize);

		Buffer = (uint32*)RHICmdList.LockBuffer(KeysBufferRHI[ResultBufferIndex], /*Offset=*/ 0, BufferSize, RLM_ReadOnly);
		FMemory::Memcpy(SortedKeys.GetData(), Buffer, BufferSize);
		RHICmdList.UnlockBuffer(KeysBufferRHI[ResultBufferIndex]);
		Buffer = (uint32*)RHICmdList.LockBuffer(ValuesBufferRHI[ResultBufferIndex], /*Offset=*/ 0, BufferSize, RLM_ReadOnly);
		FMemory::Memcpy(SortedValues.GetData(), Buffer, BufferSize);
		RHICmdList.UnlockBuffer(ValuesBufferRHI[ResultBufferIndex]);
	}

	// Verify results.
	bool bSucceeded = true;
	for (int32 KeyIndex = 0; KeyIndex < TestSize; ++KeyIndex)
	{
		if (SortedKeys[KeyIndex] != RefSortedKeys[KeyIndex] || SortedValues[KeyIndex] != RefSortedKeys[KeyIndex])
		{
			IncorrectKeyIndex = KeyIndex;
			bSucceeded = false;
			break;
		}
	}

	if (bSucceeded)
	{
		UE_LOGF(LogGPUSort, Log, "GPU Sort Test (%d keys+values) succeeded.", TestSize);
	}
	else
	{
		UE_LOGF(LogGPUSort, Log, "GPU Sort Test (%d keys+values) FAILED.", TestSize);

		if (bDebugSort || !bDebugOffsets)
		{
			const int32 FirstKeyIndex = FMath::Max<int32>(IncorrectKeyIndex - 8, 0);
			const int32 LastKeyIndex = FMath::Min<int32>(FirstKeyIndex + 1024, TestSize - 1);
			UE_LOGF(LogGPUSort, Log, "       Input    : S.Keys   : S.Values : Ref Sorted Keys");
			for (int32 KeyIndex = FirstKeyIndex; KeyIndex <= LastKeyIndex; ++KeyIndex)
			{
				UE_LOGF(LogGPUSort, Log, "%04u : %08X : %08X : %08X : %08X%ls",
					KeyIndex,
					Keys[KeyIndex],
					SortedKeys[KeyIndex],
					SortedValues[KeyIndex],
					RefSortedKeys[KeyIndex],
					(KeyIndex == IncorrectKeyIndex) ? TEXT(" <----") : TEXT("")
					);
			}
		}
	}

	return bSucceeded;
}

/**
 * Executes a sort test with debug information enabled.
 * @param TestSize - The number of elements to sort.
 */
static void RunGPUSortTestWithDebug(FRHICommandListImmediate& RHICmdList, int32 TestSize, ERHIFeatureLevel::Type FeatureLevel)
{
	TGuardValue<bool> SortDebugOffsets(GbSortDebugOffsets, true);
	TGuardValue<bool> SortDebug(GbSortDebug, true);
	RunGPUSortTest(RHICmdList, TestSize, FeatureLevel);
}

/**
 * Executes a sort test. If the sort fails, it reruns the sort with debug
 * information enabled.
 * @param TestSize - The number of elements to sort.
 */
static bool TestGPUSortForSize(FRHICommandListImmediate& RHICmdList, int32 TestSize, ERHIFeatureLevel::Type FeatureLevel)
{
	check(IsInRenderingThread());
	const bool bResult = RunGPUSortTest(RHICmdList, TestSize, FeatureLevel);
	if (bResult == false)
	{
		RunGPUSortTestWithDebug(RHICmdList, TestSize, FeatureLevel);
	}
	return bResult;
}

/**
 * Test that GPU sorting works.
 * @param TestToRun - The test to run.
 */
static bool TestGPUSort_RenderThread(FRHICommandListImmediate& RHICmdList, EGPUSortTest TestToRun, ERHIFeatureLevel::Type FeatureLevel)
{
	check(IsInRenderingThread());

	switch (TestToRun)
	{
	case GPU_SORT_TEST_SMALL:
		return TestGPUSortForSize(RHICmdList, GPU_SORT_TEST_SIZE_SMALL, FeatureLevel);

	case GPU_SORT_TEST_LARGE:
		return TestGPUSortForSize(RHICmdList, GPU_SORT_TEST_SIZE_LARGE, FeatureLevel);

	case GPU_SORT_TEST_EXHAUSTIVE:
		{
			// First test all power-of-two sizes within the range.
			int32 TestSize = GPU_SORT_TEST_SIZE_MIN;
			while (TestSize <= GPU_SORT_TEST_SIZE_MAX)
			{
				if (TestGPUSortForSize(RHICmdList, TestSize, FeatureLevel) == false)
				{
					return false;
				}
				TestSize <<= 1;
			}

			// Offset the size by one to test non-power-of-two.
			TestSize = GPU_SORT_TEST_SIZE_MIN;
			while (TestSize <= GPU_SORT_TEST_SIZE_MAX)
			{
				if (TestGPUSortForSize(RHICmdList, TestSize - 1, FeatureLevel) == false)
				{
					return false;
				}
				TestSize <<= 1;
			}
		}
		return true;

	case GPU_SORT_TEST_RANDOM:
		for ( int32 i = 0; i < 1000; ++i )
		{
			int32 TestSize = FMath::TruncToInt(FMath::SRand() * (float)(GPU_SORT_TEST_SIZE_MAX - GPU_SORT_TEST_SIZE_MIN)) + GPU_SORT_TEST_SIZE_MIN;
			if (TestGPUSortForSize(RHICmdList, (TestSize + 0xF) & 0xFFFFFFF0, FeatureLevel) == false)
			{
				return false;
			}
		}
		return true;
	}

	return true;
}

/**
 * Test that GPU sorting works.
 * @param TestToRun - The test to run.
 */
void TestGPUSort(EGPUSortTest TestToRun, ERHIFeatureLevel::Type FeatureLevel)
{
	ENQUEUE_RENDER_COMMAND(FTestGPUSortCommand)(
		[TestToRun, FeatureLevel](FRHICommandListImmediate& RHICmdList)
		{
			TestGPUSort_RenderThread(RHICmdList, TestToRun, FeatureLevel);
		});
}
