// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSortManager.h: Interface manage sorting between different FXSystemInterface
=============================================================================*/

#include "GPUSortManager.h"
#include "GPUSort.h"
#include "GPUBitonicSort.h"
#include "GlobalShader.h"
#include "Math/Float16.h"
#include "ShaderParameterUtils.h"
#include "FXSystem.h" // FXConsoleVariables::bAllowGPUSorting

#include "RenderGraphUtils.h"

//*****************************************************************************

int32 GGPUSortFrameCountBeforeBufferShrinking = 100;
static FAutoConsoleVariableRef CVarGPUSortFrameCountBeforeBufferShrinking(
	TEXT("fx.GPUSort.FrameCountBeforeShrinking"),
	GGPUSortFrameCountBeforeBufferShrinking,
	TEXT("Number of consecutive frames where the GPU sort buffer is considered oversized before allowing shrinking. (default=100)"),
	ECVF_Default
);

int32 GGPUSortMinBufferSize = 8192;
static FAutoConsoleVariableRef CVarGPUSortMinBufferSize(
	TEXT("fx.GPUSort.MinBufferSize"),
	GGPUSortMinBufferSize,
	TEXT("Minimum GPU sort buffer size, in particles (default=8192)"),
	ECVF_Default
);

float GGPUSortBufferSlack = 2.f;
static FAutoConsoleVariableRef CVarGPUSortBufferSlack(
	TEXT("fx.GPUSort.BufferSlack"),
	GGPUSortBufferSlack,
	TEXT("Slack ratio when resizing GPU sort buffers. Must be bigger than 1 (default=2)"),
	ECVF_Default
);

int32 GGPUSortStressTest = 0;
static FAutoConsoleVariableRef CVarGPUSortStressTest(
	TEXT("fx.GPUSort.StressTest"),
	GGPUSortStressTest,
	TEXT("Force a stress test on the GPU sort by release persistent data every frame (default=0)"),
	ECVF_Cheat
);

enum class EGPUSortAlgorithm : int32
{
	Auto    = -1,
	Radix   = 0,
	Bitonic = 1,
};

// Bitonic wins at low counts (low dispatch overhead); Radix wins at high counts (better asymptotic scaling).
int32 GGPUSortAlgorithm = (int32)EGPUSortAlgorithm::Auto;
FAutoConsoleVariableRef CVarGPUSortAlgorithm(
	TEXT("fx.GPUSort.Algorithm"),
	GGPUSortAlgorithm,
	TEXT("GPU sort: 0=Radix (scales to high counts), 1=Bitonic (low overhead at low counts), -1=Auto (default; choose by fx.GPUSort.Bitonic.AutoThresholdLog2)."),
	ECVF_Default
);

int32 GGPUSortBitonicAutoThresholdLog2 = 17;
FAutoConsoleVariableRef CVarGPUSortBitonicAutoThresholdLog2(
	TEXT("fx.GPUSort.Bitonic.AutoThresholdLog2"),
	GGPUSortBitonicAutoThresholdLog2,
	TEXT("Auto-mode threshold (log2): use Bitonic when Count <= (1 << value) (default=17, 131072)."),
	ECVF_Default
);

EGPUSortAlgorithm GetPreferredGPUSortAlgorithm(int32 Count)
{
	switch (static_cast<EGPUSortAlgorithm>(GGPUSortAlgorithm))
	{
	case EGPUSortAlgorithm::Bitonic: return EGPUSortAlgorithm::Bitonic;
	case EGPUSortAlgorithm::Auto:
	{
		const int32 ClampedLog2 = FMath::Clamp(GGPUSortBitonicAutoThresholdLog2, 0, 30);
		return Count <= (1 << ClampedLog2) ? EGPUSortAlgorithm::Bitonic : EGPUSortAlgorithm::Radix;
	}
	case EGPUSortAlgorithm::Radix:
	default:                         return EGPUSortAlgorithm::Radix;
	}
}


//*****************************************************************************

DECLARE_GPU_STAT_NAMED(GPUKeyGenAndSort, TEXT("GPU KeyGen & Sort"));

//*****************************************************************************
//****************************** FGPUSortDummyUAV *****************************
//*****************************************************************************

class FGPUSortDummyUAV : public FRenderResource
{
public:
	FRWBuffer Buffer;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		Buffer.Initialize(RHICmdList, TEXT("FGPUSortDummyUAV"), sizeof(int32), 1, EPixelFormat::PF_R32_UINT, BUF_Static);
	}

	virtual void ReleaseRHI() override
	{
		Buffer.Release();
	}
}; 

//*****************************************************************************
//****************************** FCopyUIntBufferCS *****************************
//*****************************************************************************

#define COPYUINTCS_BUFFER_COUNT 3
#define COPYUINTCS_THREAD_COUNT 64

/**
 * Compute shader used to copy a reference buffer split in several buffers.
 * Each buffer contains a segment of the buffer, and each buffer is increasingly bigger to hold the next part.
 * Could alternatively use DMA copy, if the RHI provided a way to copy buffer regions.
 */
class FCopyUIntBufferCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FCopyUIntBufferCS,Global);

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), COPYUINTCS_THREAD_COUNT);
		OutEnvironment.SetDefine(TEXT("BUFFER_COUNT"), COPYUINTCS_BUFFER_COUNT);
	}

	/** Default constructor. */
	FCopyUIntBufferCS() {}

	/** Initialization constructor. */
	explicit FCopyUIntBufferCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	/**
	 * Set parameters.
	 */
	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		FRHIShaderResourceView* InSourceData,
		FRHIUnorderedAccessView* const* InDestDatas,
		const int32* InUsedIndexCounts, 
		int32 StartingIndex,
		int32 DestCount);

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds);

	void Begin(FRHICommandList& RHICmdList);
	void End(FRHICommandList& RHICmdList);

private:
	static TGlobalResource<FGPUSortDummyUAV> NiagaraSortingDummyUAV[COPYUINTCS_BUFFER_COUNT];

	LAYOUT_FIELD(FShaderParameter, CopyParams);
	LAYOUT_FIELD(FShaderResourceParameter, SourceData);
	LAYOUT_ARRAY(FShaderResourceParameter, DestData, COPYUINTCS_BUFFER_COUNT);
};

TGlobalResource<FGPUSortDummyUAV> FCopyUIntBufferCS::NiagaraSortingDummyUAV[COPYUINTCS_BUFFER_COUNT];

//*****************************************************************************

IMPLEMENT_SHADER_TYPE(, FCopyUIntBufferCS, TEXT("/Engine/Private/CopyUIntBuffer.usf"), TEXT("MainCS"), SF_Compute);

FCopyUIntBufferCS::FCopyUIntBufferCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	CopyParams.Bind(Initializer.ParameterMap, TEXT("CopyParams"));
	SourceData.Bind(Initializer.ParameterMap, TEXT("SourceData"));
	DestData[0].Bind(Initializer.ParameterMap, TEXT("DestData0"));
	DestData[1].Bind(Initializer.ParameterMap, TEXT("DestData1"));
	DestData[2].Bind(Initializer.ParameterMap, TEXT("DestData2"));
}

void FCopyUIntBufferCS::SetParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	FRHIShaderResourceView* InSourceData,
	FRHIUnorderedAccessView* const* InDestDatas,
	const int32* InUsedIndexCounts,
	int32 StartingIndex,
	int32 DestCount)
{
	check(DestCount > 0 && DestCount <= COPYUINTCS_BUFFER_COUNT);

	SetSRVParameter(BatchedParameters, SourceData, InSourceData);

	FUintVector4 CopyParamsValue(StartingIndex, 0, 0, 0);
	for (int32 Index = 0; Index < DestCount; ++Index)
	{
		SetUAVParameter(BatchedParameters, DestData[Index], InDestDatas[Index]);
		CopyParamsValue[Index + 1] = InUsedIndexCounts[Index];
	}

	for (int32 Index = DestCount; Index < COPYUINTCS_BUFFER_COUNT; ++Index)
	{
		// TR-DummyUAVs : those buffers are only ever used here, but there content is never accessed.
		SetUAVParameter(BatchedParameters, DestData[Index], NiagaraSortingDummyUAV[Index].Buffer.UAV);
	}

	SetShaderValue(BatchedParameters, CopyParams, CopyParamsValue);
}

void FCopyUIntBufferCS::UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
{
	UnsetSRVParameter(BatchedUnbinds, SourceData);
	for (int32 Index = 0; Index < COPYUINTCS_BUFFER_COUNT; ++Index)
	{
		UnsetUAVParameter(BatchedUnbinds, DestData[Index]);
	}
}

void FCopyUIntBufferCS::Begin(FRHICommandList& RHICmdList)
{
	FRHIUnorderedAccessView* Views[COPYUINTCS_BUFFER_COUNT];
	for (int32 Index = 0; Index < COPYUINTCS_BUFFER_COUNT; ++Index)
	{
		Views[Index] = NiagaraSortingDummyUAV[Index].Buffer.UAV;
	}
	RHICmdList.BeginUAVOverlap(Views);
}

void FCopyUIntBufferCS::End(FRHICommandList& RHICmdList)
{
	FRHIUnorderedAccessView* Views[COPYUINTCS_BUFFER_COUNT];
	for (int32 Index = 0; Index < COPYUINTCS_BUFFER_COUNT; ++Index)
	{
		Views[Index] = NiagaraSortingDummyUAV[Index].Buffer.UAV;
	}
	RHICmdList.EndUAVOverlap(Views);
}

//*****************************************************************************

void CopyUIntBufferToTargets(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel,  FRHIShaderResourceView* SourceSRV, FRHIUnorderedAccessView*const* TargetUAVs, int32* TargetSizes, int32 StartingOffset, int32 NumTargets)
{
	// No that resource transition must be made outside this call as we don't know how the content of those have been generated, and will be used.

	TShaderMapRef<FCopyUIntBufferCS> CopyBufferCS(GetGlobalShaderMap(FeatureLevel));
	SetComputePipelineState(RHICmdList, CopyBufferCS.GetComputeShader());
	
	CopyBufferCS->Begin(RHICmdList);

	int32 Index0InPass = 0;
	while (Index0InPass < NumTargets)
	{
		const int32 NumTargetsInPass = FMath::Min<int32>(NumTargets - Index0InPass, COPYUINTCS_BUFFER_COUNT);
		const int32 NumElementsInPass = TargetSizes[Index0InPass + NumTargetsInPass - 1] - StartingOffset;

		SetShaderParametersLegacyCS(RHICmdList, CopyBufferCS, SourceSRV, TargetUAVs + Index0InPass, TargetSizes + Index0InPass, StartingOffset, NumTargetsInPass);

		DispatchComputeShader(RHICmdList, CopyBufferCS, FMath::DivideAndRoundUp(NumElementsInPass, COPYUINTCS_THREAD_COUNT), 1, 1);

		StartingOffset += NumElementsInPass;
		Index0InPass += COPYUINTCS_BUFFER_COUNT;
	};

	UnsetShaderParametersLegacyCS(RHICmdList, CopyBufferCS);

	CopyBufferCS->End(RHICmdList);
}

//*****************************************************************************
//*********************** FGPUSortManager::FKeyGenInfo ************************
//*****************************************************************************

FGPUSortManager::FKeyGenInfo::FKeyGenInfo(uint32 NumElements, bool bHighPrecisionKeys)
{
	ElementKeyMask = (1 << FMath::CeilLogTwo(NumElements)) - 1;
	ElementKeyShift = 16;
	SortKeyMask = 0xFFFF;

	SortKeyParams.X = SortKeyMask;
	SortKeyParams.Y = 0;
	SortKeyParams.Z = 0x8000;
	SortKeyParams.W = 0; 

	if (bHighPrecisionKeys)
	{
		ElementKeyMask = FMath::Max<uint32>(ElementKeyMask , 1); // Need at list 1 bit for the above logic
		uint32 UnusedBits = FPlatformMath::CountLeadingZeros(ElementKeyMask << ElementKeyShift);
		ElementKeyShift += UnusedBits;
		SortKeyMask = ~(ElementKeyMask << ElementKeyShift);

		SortKeyParams.X = SortKeyMask;
		SortKeyParams.Y = 16 - UnusedBits;
		SortKeyParams.Z <<= UnusedBits;
	}
}

//*****************************************************************************
//*********************** FGPUSortManager::FValueBuffer ***********************
//*****************************************************************************

FGPUSortManager::FValueBuffer::FValueBuffer(FRHICommandListBase& RHICmdList, int32 InAllocatedCount, int32 InUsedCount, const FGPUSortManager::FSettings& InSettings)
	: AllocatedCount(InAllocatedCount)
	, UsedCount(InUsedCount)
{
	check(InUsedCount >= 0 && InUsedCount <= InAllocatedCount);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("ValueBuffer"), InAllocatedCount)
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::UnorderedAccess)
		.SetInitialState(ERHIAccess::SRVGraphics);

	VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

	UInt32SRV = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT));

	UInt32UAV = RHICmdList.CreateUnorderedAccessView(
		VertexBufferRHI, 
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT));

	if (EnumHasAnyFlags(InSettings.AllowedFlags, EGPUSortFlags::ValuesAsInt32))
	{
		Int32SRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_SINT));

		Int32UAV = RHICmdList.CreateUnorderedAccessView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_SINT));
	}
	if (EnumHasAnyFlags(InSettings.AllowedFlags, EGPUSortFlags::ValuesAsG16R16F))
	{
		G16R16SRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_G16R16F));

		G16R16UAV = RHICmdList.CreateUnorderedAccessView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_G16R16F));
	}
}

void FGPUSortManager::FValueBuffer::Allocate(FAllocationInfo& OutInfo, int32 ValueCount, EGPUSortFlags Flags)
{
	OutInfo.BufferOffset = UsedCount;
	UsedCount += ValueCount;
	check(UsedCount <= AllocatedCount);
	if (EnumHasAnyFlags(Flags, EGPUSortFlags::ValuesAsInt32))
	{
		// Note that the sorting shader reads the indices as UInt32 (see FParticleSortBuffers::InitRHI()).
		OutInfo.BufferSRV = UInt32SRV;
	}
	else
	{
		checkSlow(EnumHasAnyFlags(Flags, EGPUSortFlags::ValuesAsG16R16F));
		OutInfo.BufferSRV = G16R16SRV;
	}
}

void FGPUSortManager::FValueBuffer::ReleaseRHI()
{
	UInt32SRV.SafeRelease();
	UInt32UAV.SafeRelease();
	Int32SRV.SafeRelease();
	Int32UAV.SafeRelease();
	G16R16SRV.SafeRelease();
	G16R16UAV.SafeRelease();
	FVertexBuffer::ReleaseRHI();

	AllocatedCount=0;
	UsedCount=0;
}

//*****************************************************************************
//******************** FGPUSortManager::FDynamicValueBuffer *******************
//*****************************************************************************

void FGPUSortManager::FDynamicValueBuffer::Allocate(FRHICommandListBase& RHICmdList, FAllocationInfo& OutInfo, const FGPUSortManager::FSettings& InSettings, int32 ValueCount, EGPUSortFlags Flags)
{
	if (ValueBuffers.Num())
	{
		FValueBuffer& LastValueBuffer = ValueBuffers.Last();
		const int32 RequiredCount = LastValueBuffer.UsedCount + ValueCount;

		if (RequiredCount <= LastValueBuffer.AllocatedCount)
		{
			LastValueBuffer.Allocate(OutInfo, ValueCount, Flags);
		}
		else
		{
			// Bitonic sort addresses [0, RoundUpPow2(UsedCount)) of this buffer. AllocatedCount must be at least that. 
			const int32 RawNewBufferSize = (int32)(RequiredCount * InSettings.BufferSlack);
			const int32 NewBufferSize = (int32)FMath::RoundUpToPowerOfTwo((uint32)FMath::Max(RawNewBufferSize, RequiredCount));
			FValueBuffer* NewValueBuffer = new FValueBuffer(RHICmdList, NewBufferSize, LastValueBuffer.UsedCount, InSettings);
			NewValueBuffer->Allocate(OutInfo, ValueCount, Flags);
			ValueBuffers.Add(NewValueBuffer);
		}
	}
	else
	{
		const int32 RawNewBufferSize = FMath::Max(InSettings.MinBufferSize, (int32)(ValueCount * InSettings.BufferSlack));
		const int32 NewBufferSize = (int32)FMath::RoundUpToPowerOfTwo((uint32)FMath::Max(RawNewBufferSize, ValueCount));
		FValueBuffer* NewValueBuffer = new FValueBuffer(RHICmdList, NewBufferSize, 0, InSettings);
		NewValueBuffer->Allocate(OutInfo, ValueCount, Flags);
		ValueBuffers.Add(NewValueBuffer);
	}
}

void FGPUSortManager::FDynamicValueBuffer::SkrinkAndReset(FRHICommandListBase& RHICmdList, const FGPUSortManager::FSettings& InSettings)
{
	const int32 UsedCount = GetUsedCount();
	if (UsedCount > 0)
	{
		const int32 AllocatedCount = GetAllocatedCount();
		// Only keep the last buffer which is the one of the right size.
		ValueBuffers.RemoveAt(0, ValueBuffers.Num() - 1);

		const float RecommendedSize = FMath::Max<float>(InSettings.MinBufferSize, UsedCount * InSettings.BufferSlack);
		if (RecommendedSize * InSettings.BufferSlack < AllocatedCount)
		{
			if (++NumFramesRequiringShrinking > InSettings.FrameCountBeforeShrinking)
			{
				ValueBuffers.Empty();
				// Power-of-two so bitonic's addressing fits
				const int32 ShrunkSize = (int32)FMath::RoundUpToPowerOfTwo((uint32)FMath::Max((int32)RecommendedSize, UsedCount));
				ValueBuffers.Add(new FValueBuffer(RHICmdList, ShrunkSize, 0, InSettings));
			}
		}
		else
		{
			NumFramesRequiringShrinking = 0;
		}

		// Reset UsedCount for next frame.
		ValueBuffers.Last().UsedCount = 0;
	}
	else
	{
		if (++NumFramesRequiringShrinking > InSettings.FrameCountBeforeShrinking)
		{
			ValueBuffers.Empty();
		}
	}

	CurrentSortBatchId = INDEX_NONE;
}

void FGPUSortManager::FDynamicValueBuffer::ReleaseRHI()
{
	for (FValueBuffer& ValueBuffer : ValueBuffers)
	{
		ValueBuffer.ReleaseRHI();
	}
}

//*****************************************************************************
//************************ FGPUSortManager::FSortBatch ************************
//*****************************************************************************

void FGPUSortManager::FSortBatch::UpdateProcessingOrder()
{
	if (EnumHasAnyFlags(Flags, EGPUSortFlags::SortAfterPreRender))
	{
		check(!EnumHasAnyFlags(Flags, EGPUSortFlags::KeyGenAfterPostRenderOpaque));
		ProcessingOrder = ESortBatchProcessingOrder::KeyGenAndSortAfterPreRender;
	}
	else // SortAfterPostRenderOpaque
	{
		check(EnumHasAnyFlags(Flags, EGPUSortFlags::SortAfterPostRenderOpaque));
		if (EnumHasAnyFlags(Flags, EGPUSortFlags::KeyGenAfterPreRender))
		{
			ProcessingOrder = ESortBatchProcessingOrder::KeyGenAfterPreRenderAndSortAfterPostRenderOpaque;
		}
		else
		{
			ProcessingOrder = ESortBatchProcessingOrder::KeyGenAndSortAfterPostRenderOpaque;
		}
	}
}

void FGPUSortManager::FSortBatch::GenerateKeys(FRHICommandListImmediate& RHICmdList, const FGPUSortManager::FCallbackArray& InCallbacks, EGPUSortFlags KeyGenLocation)
{
	const int32 InitialIndex = 0;
	FRHIUnorderedAccessView* KeyValueUAVs[] = { SortBuffers->GetKeyBufferUAV(InitialIndex), DynamicValueBuffer->ValueBuffers.Last().UInt32UAV };
	FRHITransitionInfo KeyValueUAVTransitions[] = {
		FRHITransitionInfo(SortBuffers->GetKeyBufferUAV(InitialIndex), ERHIAccess::Unknown, ERHIAccess::UAVCompute), 
		FRHITransitionInfo(DynamicValueBuffer->ValueBuffers.Last().UInt32UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute),
	};

	// TR-KeyGen : Sync the keys with the last GPU sort task.
	RHICmdList.Transition(MakeArrayView(KeyValueUAVTransitions, UE_ARRAY_COUNT(KeyValueUAVTransitions)));

	for (const FCallbackInfo& Callback : InCallbacks)
	{
		if (EnumHasAnyFlags(KeyGenLocation & Callback.Flags, EGPUSortFlags::AnyKeyGenLocation))
		{
			SCOPED_DRAW_EVENTF(RHICmdList, GPUSortBatch, TEXT("KeyGen_%s"), Callback.Name);
			const bool bAsInt32 = EnumHasAnyFlags(Callback.Flags, EGPUSortFlags::ValuesAsInt32);
			FRHIUnorderedAccessView* TypedValueUAV = bAsInt32 ? DynamicValueBuffer->ValueBuffers.Last().Int32UAV : DynamicValueBuffer->ValueBuffers.Last().G16R16UAV;
			// TR-KeyGen : TypedValueUAV is the same as ValueUAVs[1] but with a different type. The callback needs to do an BeginUAVOverlap / EndUAVOverlap between each dispatch updating partially the content.
			Callback.Delegate.Execute(RHICmdList, Id, NumElements, (Flags & EGPUSortFlags::AnyKeyPrecision) | KeyGenLocation, KeyValueUAVs[0], TypedValueUAV);
		}
	}

	// TR-KeyGen : Those buffers will now be red as input for the compute GPU sort.
	RHICmdList.Transition(MakeArrayView(KeyValueUAVTransitions, UE_ARRAY_COUNT(KeyValueUAVTransitions)));
}

void FGPUSortManager::FSortBatch::SortAndResolve(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel)
{
	check(SortBuffers && DynamicValueBuffer);
	const FValueBufferArray& ValueBuffers = DynamicValueBuffer->ValueBuffers;
	if (ValueBuffers.Num())
	{
		FGPUSortBuffers SortBuffer = SortBuffers->GetSortBuffers();

		// TR-SortedValues : at this point the first sorted values buffer was updated by the GenerateKeys and are ready for compute.

		SortBuffer.FirstValuesSRV = ValueBuffers.Last().UInt32SRV;
		SortBuffer.FinalValuesUAV = ValueBuffers.Last().UInt32UAV;

		RHICmdList.Transition(FRHITransitionInfo(SortBuffer.FinalValuesUAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute));

		const int32 InitialIndex = 0;
		const FGPUSortManager::FKeyGenInfo KeyGenInfo((uint32)NumElements, EnumHasAnyFlags(Flags, EGPUSortFlags::HighPrecisionKeys) != 0);
		const uint32 KeyMask = (KeyGenInfo.ElementKeyMask << KeyGenInfo.ElementKeyShift) | KeyGenInfo.SortKeyMask;
		const int32 SortCount = DynamicValueBuffer->GetUsedCount();
		const EGPUSortAlgorithm Algorithm = GetPreferredGPUSortAlgorithm(SortCount);

		int32 ResultIndex;
		if (Algorithm == EGPUSortAlgorithm::Bitonic)
		{
			ResultIndex = BitonicSortGPUBuffers(RHICmdList, SortBuffer, InitialIndex, SortCount, InFeatureLevel);
		}
		else
		{
			ResultIndex = SortGPUBuffers(RHICmdList, SortBuffer, InitialIndex, KeyMask, SortCount, InFeatureLevel);
		}

		if (ValueBuffers.Num() > 1)
		{
			SCOPED_DRAW_EVENT(RHICmdList, CopyResults);

			TArray<FRHIUnorderedAccessView*, TInlineAllocator<(COPYUINTCS_BUFFER_COUNT+1)>> TargetUAVs;
			TArray<int32, TInlineAllocator<COPYUINTCS_BUFFER_COUNT>> TargetSizes;
			TArray<FRHITransitionInfo, TInlineAllocator<(COPYUINTCS_BUFFER_COUNT + 1)>> UAVTransitions;
			TArray<FRHITransitionInfo, TInlineAllocator<(COPYUINTCS_BUFFER_COUNT + 1)>> SRVTransitions;
			for (FValueBuffer& ValueBuffer : DynamicValueBuffer->ValueBuffers)
			{
				// The biggest buffer contains 
				if (ValueBuffer.UInt32UAV != SortBuffer.FinalValuesUAV)
				{
					TargetUAVs.Add(ValueBuffer.UInt32UAV);
					TargetSizes.Add(ValueBuffer.UsedCount);

					UAVTransitions.Add(FRHITransitionInfo(ValueBuffer.UInt32UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
					SRVTransitions.Add(FRHITransitionInfo(ValueBuffer.UInt32UAV, ERHIAccess::Unknown, ERHIAccess::SRVMask));
				}
			}

			// TR-SortedValues : The sort value buffers (from the GPU sort dispatches) will now be updated by the FCopyUIntBufferCS dispatch.
			RHICmdList.Transition(FRHITransitionInfo(SortBuffer.FinalValuesUAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute));
			RHICmdList.Transition(MakeArrayView(UAVTransitions.GetData(), UAVTransitions.Num()));

			CopyUIntBufferToTargets(RHICmdList, InFeatureLevel, SortBuffer.FirstValuesSRV, TargetUAVs.GetData(), TargetSizes.GetData(), 0, TargetUAVs.Num());

			// TR-SortedValues : The other (smaller) buffers containing sorted values have been updated by the FCopyUIntBufferCS dispatch, and can now be red by the Gfx pipeline.
			RHICmdList.Transition(MakeArrayView(SRVTransitions.GetData(), SRVTransitions.Num()));
		}

		// TR-SortedValues : Make this readable by the Gfx pipeline.
		RHICmdList.Transition(FRHITransitionInfo(SortBuffer.FinalValuesUAV, ERHIAccess::Unknown, ERHIAccess::SRVMask));
	}
}

//*****************************************************************************
//****************************** FGPUSortManager ******************************
//*****************************************************************************

bool FGPUSortManager::HasEither(EGPUSortFlags Flags, EGPUSortFlags A, EGPUSortFlags B)
{
	const bool bHasA = (Flags & A) != EGPUSortFlags::None;
	const bool bHasB = (Flags & B) != EGPUSortFlags::None;
	return bHasA != bHasB;
}

EGPUSortFlags FGPUSortManager::GetBatchFlags(EGPUSortFlags TaskFlags)
{
	// The batch only contains information about prevision and sort location. 
	// Value formats are globally defined and all formats are available in all batches.
	// KeyGen location callbacks (either PreRender or PostRenderOpaque) must be handled by 
	// the system pushing the task to avoid overcomplexity in the sort manager.
	// The batch accumulates all the location where key gen must be called.
	return TaskFlags & (EGPUSortFlags::AnyKeyPrecision | EGPUSortFlags::AnyKeyGenLocation | EGPUSortFlags::AnySortLocation);
}

EGPUSortFlags FGPUSortManager::CombineBatchFlags(EGPUSortFlags BatchFlags, EGPUSortFlags TaskFlags)
{
	// Key precision and Sort location gets restricted as tasks are added to the batch.
	return ((BatchFlags & TaskFlags) & (EGPUSortFlags::AnyKeyPrecision | EGPUSortFlags::AnySortLocation)) | 
	// KeyGen location increases and since each task only has one KeyGen location, compatible with the sort location, this is safe.
	   	   ((BatchFlags | TaskFlags) & EGPUSortFlags::AnyKeyGenLocation);
}

bool FGPUSortManager::TestBatchFlags(EGPUSortFlags BatchFlags, EGPUSortFlags TaskFlags)
{
	// Check that the batch supports this precision and this location. Note that only one of the precision and sort location needs to be in.
	return EnumHasAnyFlags(BatchFlags, TaskFlags & EGPUSortFlags::AnyKeyPrecision) && EnumHasAnyFlags(BatchFlags, TaskFlags & EGPUSortFlags::AnySortLocation);
}

auto FGPUSortManager::GetPrecisionString(EGPUSortFlags BatchFlags) -> TCHAR const(&)[1]
{
	return RHI_BREADCRUMB_FORCE_STRING_LITERAL(EnumHasAnyFlags(BatchFlags, EGPUSortFlags::LowPrecisionKeys) ? TEXT("LowPrecision") : TEXT("HighPrecision"));
}

FGPUSortManager::FGPUSortManager(ERHIFeatureLevel::Type InFeatureLevel) 
	: FeatureLevel(InFeatureLevel) 
{
	if (GGPUSortBufferSlack > 1.f)
	{
		Settings.BufferSlack = GGPUSortBufferSlack;
	}
	Settings.FrameCountBeforeShrinking = GGPUSortFrameCountBeforeBufferShrinking;
	Settings.MinBufferSize = GGPUSortMinBufferSize;
}

FGPUSortManager::~FGPUSortManager()
{
	for (FSortBatch& SortBatch : SortBatches)
	{
		if (SortBatch.SortBuffers)
		{
			SortBuffersPool.Push(SortBatch.SortBuffers);
			SortBatch.SortBuffers = nullptr;
		}
	}

	ReleaseSortBuffers();
}

void FGPUSortManager::Register(const FGPUSortKeyGenDelegate& CallbackDelegate, EGPUSortFlags UsedFlags, const FName& InName)
{
	// Callbacks must only have a single value format otherwise we can't tell which UAV to pass to it.
	// A system could use two callbacks to generate the initial values if ultimately required.
	check(HasEither(UsedFlags, EGPUSortFlags::ValuesAsInt32, EGPUSortFlags::ValuesAsG16R16F));

	Settings.AllowedFlags |= UsedFlags;

	FCallbackInfo& Callback = Callbacks.AddDefaulted_GetRef();
	Callback.Delegate = CallbackDelegate;
	Callback.Flags = UsedFlags;
	Callback.Name = InName;
}

bool FGPUSortManager::AddTask(FRHICommandListBase& RHICmdList, FAllocationInfo& OutInfo, int32 ValueCount, EGPUSortFlags TaskFlags)
{
	if (!FXConsoleVariables::bAllowGPUSorting)
	{
		return false;
	}

	LLM_SCOPE(ELLMTag::GPUSort);

	// Can only use flags that were registered initially.
	check(EnumHasAllFlags(Settings.AllowedFlags, TaskFlags));
	
	// The must be at least one key precision.
	checkSlow(EnumHasAnyFlags(TaskFlags, EGPUSortFlags::AnyKeyPrecision));
	// Key gen must be set at only one place, and as early as possible to maximize batch grouping.
	checkSlow(HasEither(TaskFlags, EGPUSortFlags::KeyGenAfterPreRender, EGPUSortFlags::KeyGenAfterPostRenderOpaque));
	// The must be at least one sort location.
	checkSlow(EnumHasAnyFlags(TaskFlags, EGPUSortFlags::AnySortLocation));
	// Value format must be known since we set the SRV in OutInfo.
	checkSlow(HasEither(TaskFlags, EGPUSortFlags::ValuesAsInt32, EGPUSortFlags::ValuesAsG16R16F));
	// Invalid if it needs to be sorted before the key are to be generated. Ex : Cascade opaque materials using depth buffer in the simulation.
	checkSlow(!EnumHasAllFlags(TaskFlags, EGPUSortFlags::KeyGenAfterPostRenderOpaque | EGPUSortFlags::SortAfterPreRender));

	UE::TScopeLock Lock(AddTaskMutex);

	// Search into the current batch for a compatible entries
	for (FSortBatch& SortBatch : SortBatches)
	{
		// Check that the batch supports this precision and this location. Note that only one of the precision and sort location needs to be in.
		if (TestBatchFlags(SortBatch.Flags, TaskFlags))
		{
			OutInfo.SortBatchId = SortBatch.Id;
			OutInfo.ElementIndex = SortBatch.NumElements;

			SortBatch.NumElements += 1;
			SortBatch.Flags = CombineBatchFlags(SortBatch.Flags, TaskFlags);
			SortBatch.DynamicValueBuffer->Allocate(RHICmdList, OutInfo, Settings, ValueCount, TaskFlags);
			return true;
		}
	}

	// Otherwise, create a new sort batch.
	const int32 BatchId = SortBatches.Num();
	OutInfo.SortBatchId = BatchId;
	OutInfo.ElementIndex = 0;

	FSortBatch& SortBatch = SortBatches.AddDefaulted_GetRef();
	SortBatch.Id = BatchId;
	SortBatch.NumElements = 1;
	SortBatch.Flags = GetBatchFlags(TaskFlags);
	// Bind a sort buffer to the batch. Use the Flags for the task and not from the current batch
	// since the task flags are more restrictive at this point than the batch flags that subsets as task get grouped.
	SortBatch.DynamicValueBuffer = GetDynamicValueBufferFromPool(TaskFlags, BatchId);
	checkSlow(SortBatch.DynamicValueBuffer);
	SortBatch.DynamicValueBuffer->Allocate(RHICmdList, OutInfo, Settings, ValueCount, TaskFlags);

	return true;
}

void FGPUSortManager::FinalizeSortBatches()
{
	// At this point the batches are complete as no more tasks will be added to them.
	// Update FDynamicValueBuffer::LastSortBatchFlags before the batch can potentially be destroyed.
	// This will helps rebind the same DynamicValueBuffer to the same batches.
	for (FSortBatch& SortBatch : SortBatches)
	{
		// Resolve the sort location if there is still ambiguity about where the batch needs to be sorted.
		if (EnumHasAllFlags(SortBatch.Flags, EGPUSortFlags::AnySortLocation))
		{
			// If it needs KeyGen after PostRenderOpaque(), then remove the possibility of sorting after PreRender().
			if (EnumHasAnyFlags(SortBatch.Flags, EGPUSortFlags::KeyGenAfterPostRenderOpaque))
			{
				SortBatch.Flags ^= EGPUSortFlags::SortAfterPreRender;
			}
			// Otherwise it must have the possibility to KeyGen after PreRender(), and so remove the possibility of sorting after PostRenderOpaque().
			else
			{
				checkSlow(EnumHasAnyFlags(SortBatch.Flags, EGPUSortFlags::KeyGenAfterPreRender))
				SortBatch.Flags ^= EGPUSortFlags::SortAfterPostRenderOpaque;
			}
		}
		checkSlow(HasEither(SortBatch.Flags, EGPUSortFlags::SortAfterPreRender, EGPUSortFlags::SortAfterPostRenderOpaque));

		// Update the sort batch flags now that they are final. This will help rebind the same dynamic value sort batches to the s
		checkSlow(SortBatch.DynamicValueBuffer);
		SortBatch.DynamicValueBuffer->LastSortBatchFlags = SortBatch.Flags;

		// Update the sort batch processing order.
		SortBatch.UpdateProcessingOrder();
	}
}

void FGPUSortManager::UpdateSortBuffersPool(FRHICommandListBase& RHICmdList)
{
	int32 NumSortBuffersRequired = DynamicValueBufferPool.Num() ? 1 : 0;
	int32 MaxSortBuffersSizes = 0;

	int32 NumDedicatedSortBuffersRequired = 0;
	for (const FDynamicValueBuffer& DynamicValueBuffer : DynamicValueBufferPool)
	{
		MaxSortBuffersSizes = FMath::Max<int32>(MaxSortBuffersSizes, DynamicValueBuffer.GetAllocatedCount());

		// If some keys need to be generated at PreRender() but only be used in PostRenderOpaque(), then this buffer won't be sharable in between that timeframe.
		if (EnumHasAllFlags(DynamicValueBuffer.LastSortBatchFlags, EGPUSortFlags::KeyGenAfterPreRender | EGPUSortFlags::SortAfterPostRenderOpaque))
		{
			++NumDedicatedSortBuffersRequired;
		}
	}
	// There are currently no usecase where there can be more than one of those buffers so the SortBuffers pooling strategy is simple.
	ensure(NumDedicatedSortBuffersRequired <= 1);

	// The dedicated sort buffers can be reused.
	NumSortBuffersRequired = FMath::Max<int32>(NumSortBuffersRequired, NumDedicatedSortBuffersRequired);

	// Allocate all the buffers at the right size.
	for (int32 SortBufferIndex = 0; SortBufferIndex < NumSortBuffersRequired; ++SortBufferIndex)
	{
		if (SortBuffersPool.IsValidIndex(SortBufferIndex))
		{
			FParticleSortBuffers* SortBuffers = SortBuffersPool[SortBufferIndex];
			check(SortBuffers);
			if (SortBuffers->GetSize() != MaxSortBuffersSizes)
			{
				SortBuffers->ReleaseRHI();
				SortBuffers->SetBufferSize(MaxSortBuffersSizes);
				SortBuffers->InitRHI(RHICmdList);
			}
		}
		else
		{
			FParticleSortBuffers* SortBuffers = new FParticleSortBuffers;
			SortBuffers->SetBufferSize(MaxSortBuffersSizes);
			SortBuffers->InitRHI(RHICmdList);
			SortBuffersPool.Add(SortBuffers);
		}
	}

	// Free unrequired buffers.
	while (SortBuffersPool.Num() > NumSortBuffersRequired)
	{
		FParticleSortBuffers* SortBuffers = SortBuffersPool.Pop();
		check(SortBuffers);
		SortBuffers->ReleaseRHI();
		SortBuffers->SetBufferSize(0);
		delete SortBuffers;
	}
}

void FGPUSortManager::OnPreRender(FRDGBuilder& GraphBuilder)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, GPUSort);
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FGPUSortManager::OnPreRender"),
		[this](FRHICommandListImmediate& RHICmdList)
		{
			LLM_SCOPE(ELLMTag::GPUSort);
			FinalizeSortBatches();
			UpdateSortBuffersPool(RHICmdList);

			if (SortBatches.Num())
			{
				// Sort batches so that the next batch to handle is at the end of the array.
				SortBatches.Sort([](const FSortBatch& A, const FSortBatch& B) { return (uint32)A.ProcessingOrder > (uint32)B.ProcessingOrder; });

				RHI_BREADCRUMB_EVENT_STAT(RHICmdList, GPUKeyGenAndSort, "GPU KeyGen & Sort");

				while (SortBatches.Num() && SortBatches.Last().ProcessingOrder == ESortBatchProcessingOrder::KeyGenAndSortAfterPreRender)
				{
					// Remove the SortBatch but don't remove the SortBuffers from the pool since it can be reused immediately.
					FSortBatch SortBatch = SortBatches.Pop();
					FParticleSortBuffers* SortBuffers = SortBuffersPool.Last();
					SortBatch.SortBuffers = SortBuffers;
					checkSlow(SortBuffers && SortBatch.GetUsedValueCount() <= SortBuffers->GetSize());

					SCOPED_DRAW_EVENTF(RHICmdList, GPUSortBatch, TEXT("GPUSort_Batch%d(%s)"), SortBatch.Id, GetPrecisionString(SortBatch.Flags));

					SortBatch.GenerateKeys(RHICmdList, Callbacks, EGPUSortFlags::KeyGenAfterPreRender);
					SortBatch.SortAndResolve(RHICmdList, FeatureLevel);

					// Release the sort batch.
					checkSlow(SortBatch.DynamicValueBuffer && SortBatch.DynamicValueBuffer->CurrentSortBatchId == SortBatch.Id);
					SortBatch.DynamicValueBuffer->CurrentSortBatchId = INDEX_NONE;
					SortBatch.SortBuffers = nullptr;
				}

				for (int32 BatchIndex = SortBatches.Num() - 1; BatchIndex >= 0 && SortBatches[BatchIndex].ProcessingOrder == ESortBatchProcessingOrder::KeyGenAfterPreRenderAndSortAfterPostRenderOpaque; --BatchIndex)
				{
					// Those sort batches will be processed again after PostRenderOpaque(). Because of this, the particle sort buffers can not be reused.
					FSortBatch& SortBatch = SortBatches[BatchIndex];
					FParticleSortBuffers* SortBuffers = SortBuffersPool.Pop();
					SortBatch.SortBuffers = SortBuffers;
					checkSlow(SortBuffers && SortBatch.GetUsedValueCount() <= SortBuffers->GetSize());

					SCOPED_DRAW_EVENTF(RHICmdList, GPUSortBatchPreStep, TEXT("GPUSort_Batch%d(PreStep,%s)"), SortBatch.Id, GetPrecisionString(SortBatch.Flags));

					SortBatch.GenerateKeys(RHICmdList, Callbacks, EGPUSortFlags::KeyGenAfterPreRender);
				}
			}
			PostPreRenderEvent.Broadcast(RHICmdList);
		}
	);
}

void FGPUSortManager::OnPostRenderOpaque(FRDGBuilder& GraphBuilder)
{
	LLM_SCOPE(ELLMTag::GPUSort);
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, GPUSort);
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FGPUSortManager::OnPostRenderOpaque"),
		[this](FRHICommandListImmediate& RHICmdList)
		{
			if (SortBatches.Num())
			{
				RHI_BREADCRUMB_EVENT_STAT(RHICmdList, GPUKeyGenAndSort, "GPU KeyGen & Sort");

				while (SortBatches.Num())
				{
					// Remove the SortBatch but don't remove the SortBuffers from the pool since it can be reused immediately.
					FSortBatch SortBatch = SortBatches.Pop();
					checkSlow((SortBatch.SortBuffers != nullptr) == (SortBatch.ProcessingOrder == ESortBatchProcessingOrder::KeyGenAfterPreRenderAndSortAfterPostRenderOpaque));

					SCOPED_DRAW_EVENTF(RHICmdList, GPUSortBatch, TEXT("GPUSort_Batch%d(%s)"), SortBatch.Id, GetPrecisionString(SortBatch.Flags));

					if (!SortBatch.SortBuffers)
					{
						FParticleSortBuffers* SortBuffers = SortBuffersPool.Pop();
						SortBatch.SortBuffers = SortBuffers;
						checkSlow(SortBuffers && SortBatch.GetUsedValueCount() <= SortBuffers->GetSize());
					}

					SortBatch.GenerateKeys(RHICmdList, Callbacks, EGPUSortFlags::KeyGenAfterPostRenderOpaque);
					SortBatch.SortAndResolve(RHICmdList, FeatureLevel);

					// Release the sort batch.
					checkSlow(SortBatch.DynamicValueBuffer && SortBatch.DynamicValueBuffer->CurrentSortBatchId == SortBatch.Id);
					SortBatch.DynamicValueBuffer->CurrentSortBatchId = INDEX_NONE;
					SortBuffersPool.Push(SortBatch.SortBuffers);
					SortBatch.SortBuffers = nullptr;
				}
			}
			PostPostRenderEvent.Broadcast(RHICmdList);

			ResetDynamicValuesBuffers(RHICmdList);
		}
	);
}

FGPUSortManager::FDynamicValueBuffer* FGPUSortManager::GetDynamicValueBufferFromPool(EGPUSortFlags TaskFlags, int32 SortBatchId)
{
	for (FDynamicValueBuffer& DynamicValueBuffer : DynamicValueBufferPool)
	{
		// If the buffer is unused and compatible.
		if (DynamicValueBuffer.CurrentSortBatchId == INDEX_NONE && TestBatchFlags(DynamicValueBuffer.LastSortBatchFlags, TaskFlags))
		{
			DynamicValueBuffer.CurrentSortBatchId = SortBatchId;
			return &DynamicValueBuffer; // Safe since we are using IndirectArray.
		}
	}

	FDynamicValueBuffer* DynamicValueBuffer = new FDynamicValueBuffer;
	DynamicValueBuffer->CurrentSortBatchId = SortBatchId;
	DynamicValueBufferPool.Add(DynamicValueBuffer);
	return DynamicValueBuffer;
}

void FGPUSortManager::ResetDynamicValuesBuffers(FRHICommandListBase& RHICmdList)
{
	check(!SortBatches.Num()) ;

	if (!GGPUSortStressTest)
	{
		for (int32 Index = 0; Index < DynamicValueBufferPool.Num(); ++Index)
		{
			FDynamicValueBuffer& DynamicValueBuffer = DynamicValueBufferPool[Index];
			DynamicValueBuffer.SkrinkAndReset(RHICmdList, Settings);
			if (DynamicValueBuffer.GetAllocatedCount() == 0)
			{
				DynamicValueBufferPool.RemoveAtSwap(Index);
				--Index;
			}
		}
	}
	else
	{
		DynamicValueBufferPool.Empty();
		ReleaseSortBuffers();
	}
}

void FGPUSortManager::ReleaseSortBuffers()
{
	for (FParticleSortBuffers* SortBuffers : SortBuffersPool)
	{
		check(SortBuffers);
		SortBuffers->ReleaseRHI();
		delete SortBuffers;
	}

	SortBuffersPool.Empty();
}
