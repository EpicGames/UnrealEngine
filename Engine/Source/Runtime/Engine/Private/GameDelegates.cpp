// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameDelegates.h"
#include "Delegates/DelegateBase.h"

FGameDelegates& FGameDelegates::Get()
{
	// return the singleton object
	static FGameDelegates Singleton;
	return Singleton;
}
