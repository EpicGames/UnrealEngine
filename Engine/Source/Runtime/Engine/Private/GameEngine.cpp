// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GameEngine.cpp: Unreal game engine.
=============================================================================*/

#include "Engine/GameEngine.h"
#include "Brushes/SlateNoResource.h"
#include "Engine/GameInstance.h"
#include "Engine/NetConnection.h"
#include "Framework/Docking/TabManager.h"
#include "GenericPlatform/GenericPlatformSurvey.h"
#include "Misc/TimeGuard.h"
#include "Misc/App.h"
#include "GameMapsSettings.h"
#include "EngineStats.h"
#include "Rendering/SlateRenderer.h"
#include "RenderingThread.h"
#include "Engine/EngineConsoleCommandExecutor.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LevelStreaming.h"
#include "Engine/PlatformInterfaceBase.h"
#include "ContentStreaming.h"
#include "Subsystems/EngineSubsystem.h"
#include "UnrealEngine.h"
#include "HAL/PlatformSplash.h"
#include "UObject/Package.h"
#include "GameFramework/GameModeBase.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "AudioDeviceManager.h"
#include "Net/NetworkProfiler.h"
#include "EngineModule.h"
#include "Misc/PackageName.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "ShaderPipelineCache.h"

#include "Misc/ConfigCacheIni.h"

#include "Slate/SceneViewport.h"

#include "IMovieSceneCapture.h"
#include "MovieSceneCaptureModule.h"

#include "SynthBenchmark.h"

#include "SceneViewExtension.h"
#include "Engine/LocalPlayer.h"
#include "Slate/SGameLayerManager.h"
#include "Components/SkyLightComponent.h"
#include "Components/ReflectionCaptureComponent.h"
#include "GameFramework/GameUserSettings.h"
#include "GameDelegates.h"
#include "ImageUtils.h"
#include "Misc/FileHelper.h"
#include "Misc/EmbeddedCommunication.h"
#include "Engine/CoreSettings.h"
#include "EngineAnalytics.h"
#include "Engine/DemoNetDriver.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DynamicResolutionProxy.h"
#include "DynamicResolutionState.h"
#include "MoviePlayerProxy.h"
#include "RenderTargetPool.h"
#include "RenderGraphBuilder.h"
#include "CustomResourcePool.h"
#include "ComponentRecreateRenderStateContext.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsWindow.h"
#include "Windows/WindowsPlatformCrashContext.h"
#endif

#if !UE_SERVER
	#include "IMediaModule.h"
#endif



CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

ENGINE_API bool GDisallowNetworkTravel = false;

// How slow must a frame be (in seconds) to be logged out (<= 0 to disable)
ENGINE_API float GSlowFrameLoggingThreshold = 0.0f;

static bool bGameWindowSettingsOverrideEnabled = true;

static FAutoConsoleVariableRef CvarSlowFrameLoggingThreshold(
	TEXT("t.SlowFrameLoggingThreshold"),
	GSlowFrameLoggingThreshold,
	TEXT("How slow must a frame be (in seconds) to be logged out (<= 0 to disable)."),
	ECVF_Default
	);

static int32 GDoAsyncEndOfFrameTasks = 0;
static FAutoConsoleVariableRef CVarDoAsyncEndOfFrameTasks(
	TEXT("tick.DoAsyncEndOfFrameTasks"),
	GDoAsyncEndOfFrameTasks,
	TEXT("Experimental option to run various things concurrently with the HUD render.")
	);

static int32 GMinimizedSyncDrawToGPU = 1;
static FAutoConsoleVariableRef CVarMinimizedSyncDrawToGPU(
	TEXT("tick.MinimizedSyncDrawToGPU"),
	GMinimizedSyncDrawToGPU,
	TEXT("True means we will wait for GPU idle when minimized. Prevents mem leaks due to CPU issuing draws faster than GPU processes when minimized.")
);

bool ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY, int32& WindowMode);

/** Benchmark results to the log */
static void RunSynthBenchmark(const TArray<FString>& Args)
{
	float WorkScale = 10.0f;

	if ( Args.Num() > 0 )
	{
		WorkScale = FCString::Atof(*Args[0]);
		WorkScale = FMath::Clamp(WorkScale, 1.0f, 1000.0f);
	}

	FSynthBenchmarkResults Result;
	ISynthBenchmark::Get().Run(Result, true, WorkScale);
}

/** Helper function to generate a set of windowed resolutions which are convenient for the current primary display size */
static void GenerateConvenientWindowedResolutions(int32 WorkAreaWidth, int32 WorkAreaHeight, TArray<FIntPoint>& OutResolutions)
{
	bool bInPortraitMode = WorkAreaWidth < WorkAreaHeight;

	// Generate windowed resolutions as scaled versions of primary monitor size
	static const float Scales[] = { 3.0f / 6.0f, 4.0f / 6.0f, 4.5f / 6.0f, 5.0f / 6.0f };
	static const float Ratios[] = { 9.0f, 10.0f, 12.0f };
	static const float MinWidth = 1280.0f;
	static const float MinHeight = 720.0f; // UI layout doesn't work well below this, as the accept/cancel buttons go off the bottom of the screen

	static const uint32 NumScales = sizeof(Scales) / sizeof(float);
	static const uint32 NumRatios = sizeof(Ratios) / sizeof(float);

	for (uint32 ScaleIndex = 0; ScaleIndex < NumScales; ++ScaleIndex)
	{
		for (uint32 AspectIndex = 0; AspectIndex < NumRatios; ++AspectIndex)
		{
			float TargetWidth, TargetHeight;
			float Aspect = Ratios[AspectIndex] / 16.0f;

			if (bInPortraitMode)
			{
				TargetHeight = FMath::RoundToFloat(WorkAreaHeight * Scales[ScaleIndex]);
				TargetWidth = TargetHeight * Aspect;
			}
			else
			{
				TargetWidth = FMath::RoundToFloat(WorkAreaWidth * Scales[ScaleIndex]);
				TargetHeight = TargetWidth * Aspect;
			}

			if (TargetWidth < WorkAreaWidth && TargetHeight < WorkAreaHeight && TargetWidth >= MinWidth && TargetHeight >= MinHeight)
			{
				OutResolutions.Add(FIntPoint(TargetWidth, TargetHeight));
			}
		}
	}
	
	// if no convenient resolutions have been found, add a minimum one
	if (OutResolutions.Num() == 0)
	{
		if (WorkAreaHeight > MinHeight && WorkAreaWidth > MinWidth)
		{
			//Add the minimum size if it fit
			OutResolutions.Add(FIntPoint(MinWidth, MinHeight));
		}
		else
		{
			//Force a resolution even if its smaller then the minimum height and width to avoid a bigger window then the desktop
			// LWC_TODO: revisit. Seems like the round should be done after the multiply, otherwise the int was going to float and no rounding at all occurred here.
			float TargetWidth = FMath::RoundToFloat((float)WorkAreaWidth) * Scales[NumScales - 1];
			float TargetHeight = FMath::RoundToFloat((float)WorkAreaHeight) * Scales[NumScales - 1];
			OutResolutions.Add(FIntPoint(TargetWidth, TargetHeight));
		}
	}
}

void GenerateConvenientWindowedResolutions(const struct FDisplayMetrics& InDisplayMetrics, TArray<FIntPoint>& OutResolutions)
{
	GenerateConvenientWindowedResolutions(InDisplayMetrics.PrimaryDisplayWidth, InDisplayMetrics.PrimaryDisplayHeight, OutResolutions);
}

static FAutoConsoleCommand GSynthBenchmarkCmd(
	TEXT("SynthBenchmark"),
	TEXT("Run simple benchmark to get some metrics to find reasonable game settings automatically\n")
	TEXT("Optional (float) parameter allows to scale with work amount to trade time or precision (default: 10)."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&RunSynthBenchmark)
	);

EWindowMode::Type GetWindowModeType(EWindowMode::Type WindowMode)
{
	return FPlatformProperties::SupportsWindowedMode() ? WindowMode : EWindowMode::Fullscreen;
}

UGameEngine::UGameEngine(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	cleanup!!
-----------------------------------------------------------------------------*/

TSharedRef<SViewport> UGameEngine::CreateGameViewportWidget()
{
	bool bRenderDirectlyToWindow = (!StartupMovieCaptureHandle.IsValid() || IMovieSceneCaptureModule::Get().IsStereoAllowed()) && GIsDumpingMovie == 0;

	const bool bStereoAllowed = bRenderDirectlyToWindow;

	return SNew(SViewport)
			// Render directly to the window backbuffer unless capturing a movie or getting screenshots
			// @todo TEMP
			.RenderDirectlyToWindow(bRenderDirectlyToWindow)
			//gamma handled by the scene renderer
			.EnableGammaCorrection(false)
			.EnableStereoRendering(bStereoAllowed);
}

void UGameEngine::CreateGameViewportWidget( UGameViewportClient* GameViewportClient )
{
	GameViewportWidget = CreateGameViewportWidget();
}

void UGameEngine::CreateGameViewport( UGameViewportClient* GameViewportClient )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateGameViewport);
	check(GameViewportWindow.IsValid());
	check(!SceneViewport.IsValid());

	if( !GameViewportWidget.IsValid() )
	{
		GameViewportWidget = CreateGameViewportWidget();
	}
	SceneViewport = GameViewportClient->CreateViewport(GameViewportWidget);

	auto Window = GameViewportWindow.Pin();

	Window->SetOnWindowClosed( FOnWindowClosed::CreateUObject( this, &UGameEngine::OnGameWindowClosed ) );

	// SAVEWINPOS tells us to load/save window positions to user settings (this is disabled by default)
	int32 SaveWinPos;
	if (FParse::Value(FCommandLine::Get(), TEXT("SAVEWINPOS="), SaveWinPos) && SaveWinPos > 0 )
	{
		// Convert to 0-based slot index
		const int32 SaveWinPosIndex = SaveWinPos - 1;
		const int32 MaxSlots = GetGameUserSettings()->GetMaxNumWindowPositionSaveSlots();

		if (SaveWinPosIndex < MaxSlots)
		{
			Window->SetOnWindowMoved(FOnWindowMoved::CreateWeakLambda(this, [SaveWinPosIndex, this](const TSharedRef<SWindow>& WindowBeingMoved)
				{
					this->OnGameWindowMoved_SavePosition(WindowBeingMoved, SaveWinPosIndex);
				}));
		}
		else
		{
			UE_LOGF(LogEngine, Warning, "SaveWinPos index %d exceeds the maximum of %d. See cvar GameUserSettings.WindowPositionsMaxSize. Window position won't be saved.", SaveWinPosIndex, MaxSlots - 1);
		}
	}

	RegisterViewport(SceneViewport.ToSharedRef());
	//GameViewportClient->CreateHighresScreenshotCaptureRegionWidget(); //  Disabled until mouse based input system can be made to work correctly.

	FViewport::ViewportResizedEvent.AddUObject(this, &UGameEngine::OnViewportResized);
}

FSceneViewport* UGameEngine::GetGameSceneViewport(UGameViewportClient* ViewportClient) const
{
	return ViewportClient->GetGameViewport();
}

void UGameEngine::EnableGameWindowSettingsOverride(bool bEnabled)
{
	check(IsInGameThread());
	bGameWindowSettingsOverrideEnabled = bEnabled;
}

void UGameEngine::ConditionallyOverrideSettings(int32& ResolutionX, int32& ResolutionY, EWindowMode::Type& WindowMode, int32 MonitorIndex)
{
	if (bGameWindowSettingsOverrideEnabled)
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("Windowed")) || FParse::Param(FCommandLine::Get(), TEXT("SimMobile")))
		{
			// -Windowed or -SimMobile
			WindowMode = EWindowMode::Windowed;
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("FullScreen")))
		{
			// -FullScreen
			static auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FullScreenMode"));
			check(CVar);
			WindowMode = CVar->GetValueOnGameThread() == 0 ? EWindowMode::Fullscreen : EWindowMode::WindowedFullscreen;
		}
	}

	DetermineGameWindowResolution(ResolutionX, ResolutionY, WindowMode, false, MonitorIndex);
}

void UGameEngine::DetermineGameWindowResolution( int32& ResolutionX, int32& ResolutionY, EWindowMode::Type& WindowMode, bool bUseWorkAreaForWindowed, int32 MonitorIndex )
{
	FString ResolutionStr;;
	if (bGameWindowSettingsOverrideEnabled && FParse::Value(FCommandLine::Get(), TEXT("Res="), ResolutionStr))
	{
		uint32 ResX = 0;
		uint32 ResY = 0;
		int32 WinMode = EWindowMode::Windowed;

		if (ParseResolution(*ResolutionStr, ResX, ResY, WinMode))
		{
			ResolutionX = ResX;
			ResolutionY = ResY;
			WindowMode = EWindowMode::ConvertIntToWindowMode(WinMode);
		}
	}
	else
	{
		bool UserSpecifiedWidth = bGameWindowSettingsOverrideEnabled && FParse::Value(FCommandLine::Get(), TEXT("ResX="), ResolutionX);
		bool UserSpecifiedHeight = bGameWindowSettingsOverrideEnabled && FParse::Value(FCommandLine::Get(), TEXT("ResY="), ResolutionY);

		const float AspectRatio = 16.0f / 9.0f;

		if (UserSpecifiedWidth && !UserSpecifiedHeight)
		{
			ResolutionY = int32(ResolutionX / AspectRatio);
		}
		else if (UserSpecifiedHeight && !UserSpecifiedWidth)
		{
			ResolutionX = int32(ResolutionY * AspectRatio);
		}
	}

	//fullscreen is always supported, but don't allow windowed mode on platforms that dont' support it.
	WindowMode = (!FPlatformProperties::SupportsWindowedMode() && (WindowMode == EWindowMode::Windowed || WindowMode == EWindowMode::WindowedFullscreen)) ? EWindowMode::Fullscreen : WindowMode;

	// consume available desktop area
	FDisplayMetrics DisplayMetrics;
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);
	}
	else
	{
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
	}

	// Find the maximum allowed resolution
	// Use PrimaryDisplayWidth/Height in windowed mode
	int32 MaxResolutionX = bUseWorkAreaForWindowed && WindowMode == EWindowMode::Windowed ? DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left : DisplayMetrics.PrimaryDisplayWidth;
	int32 MaxResolutionY = bUseWorkAreaForWindowed && WindowMode == EWindowMode::Windowed ? DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top : DisplayMetrics.PrimaryDisplayHeight;
	float DisplayAspect = (float)DisplayMetrics.PrimaryDisplayWidth / (float)DisplayMetrics.PrimaryDisplayHeight;
	if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndex))
	{
		// Fall back to the monitor's native width/height if there was no max width/height found.
		const FMonitorInfo& Info = DisplayMetrics.MonitorInfo[MonitorIndex];
		int32 MonitorWidth = Info.MaxResolution.X;
		int32 MonitorHeight = Info.MaxResolution.Y;
		if (MonitorWidth == 0 || MonitorHeight == 0)
		{
			MonitorWidth = Info.NativeWidth;
			MonitorHeight = Info.NativeHeight;
		}

		DisplayAspect = (float)MonitorWidth / (float)MonitorHeight;

		if (bUseWorkAreaForWindowed && WindowMode == EWindowMode::Windowed)
		{
			MaxResolutionX = Info.WorkArea.Right - Info.WorkArea.Left;
			MaxResolutionY = Info.WorkArea.Bottom - Info.WorkArea.Top;
		}
		else
		{
			MaxResolutionX = MonitorWidth;
			MaxResolutionY = MonitorHeight;
		}
	}
	else if (WindowMode == EWindowMode::Fullscreen && !DisplayMetrics.MonitorInfo.IsEmpty())
	{
		// In fullscreen, PrimaryDisplayWidth/Height is equal to your current resolution, so we will use your max native resolution instead
		// Since we have info for at least one monitor, default to that if the primary can not be found
		MaxResolutionX = DisplayMetrics.MonitorInfo[0].NativeWidth;
		MaxResolutionY = DisplayMetrics.MonitorInfo[0].NativeHeight;

		// Now try to find the primary monitor
		for (const FMonitorInfo& MonitorInfo : DisplayMetrics.MonitorInfo)
		{
			if (MonitorInfo.bIsPrimary)
			{
				// This is the primary monitor. Use this monitor's max width/height.
				MaxResolutionX = MonitorInfo.MaxResolution.X;
				MaxResolutionY = MonitorInfo.MaxResolution.Y;

				// Fall back to the monitor's native width/height if there was no max width/height found.
				if (MaxResolutionX == 0 || MaxResolutionY == 0)
				{
					MaxResolutionX = MonitorInfo.NativeWidth;
					MaxResolutionY = MonitorInfo.NativeHeight;
				}

				break;
			}
		}
	}

	// Optionally force the resolution by passing -ForceRes
	const bool bForceRes = bGameWindowSettingsOverrideEnabled && FParse::Param(FCommandLine::Get(), TEXT("ForceRes"));

	//Don't allow a resolution bigger then the desktop found a convenient one
	if (!bForceRes && !IsRunningDedicatedServer() && ((ResolutionX <= 0 || ResolutionX > MaxResolutionX) || (ResolutionY <= 0 || ResolutionY > MaxResolutionY)))
	{
		ResolutionX = MaxResolutionX;
		ResolutionY = MaxResolutionY;

		// If we're in windowed mode, attempt to choose a suitable starting resolution that is smaller than the desktop, with a matching aspect ratio
		if (WindowMode == EWindowMode::Windowed)
		{
			TArray<FIntPoint> WindowedResolutions;
			GenerateConvenientWindowedResolutions(MaxResolutionX, MaxResolutionY, WindowedResolutions);

			if (WindowedResolutions.Num() > 0)
			{
				// We'll default to the largest one we have
				ResolutionX = WindowedResolutions[WindowedResolutions.Num() - 1].X;
				ResolutionY = WindowedResolutions[WindowedResolutions.Num() - 1].Y;

				// Attempt to find the largest one with the same aspect ratio
				for (int32 i = WindowedResolutions.Num() - 1; i >= 0; --i)
				{
					float Aspect = (float)WindowedResolutions[i].X / (float)WindowedResolutions[i].Y;
					if (FMath::Abs(Aspect - DisplayAspect) < UE_KINDA_SMALL_NUMBER)
					{
						ResolutionX = WindowedResolutions[i].X;
						ResolutionY = WindowedResolutions[i].Y;
						break;
					}
				}
			}
		}
	}

	// Check the platform to see if we should override the user settings.
	if (FPlatformProperties::HasFixedResolution())
	{
		// We need to pass the resolution back out to GameUserSettings, or it will just override it again
		ResolutionX = DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left;
		ResolutionY = DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top;
		FSystemResolution::RequestResolutionChange(ResolutionX, ResolutionY, EWindowMode::Fullscreen);
	}

	if (bGameWindowSettingsOverrideEnabled && FParse::Param(FCommandLine::Get(), TEXT("Portrait")))
	{
		Swap(ResolutionX, ResolutionY);
	}
}

TSharedRef<SWindow> UGameEngine::CreateGameWindow()
{
	FString DeviceLocalizedName;
	int32 ResX = GSystemResolution.ResX;
	int32 ResY = GSystemResolution.ResY;
	EWindowMode::Type WindowMode = GSystemResolution.WindowMode;
	const int32 MonitorIndex = GSystemResolution.MonitorIndex;
	{
		ConditionallyOverrideSettings(ResX, ResY, WindowMode, MonitorIndex);

		// If the current settings have been overridden, apply them back into the system
		if (ResX != GSystemResolution.ResX || ResY != GSystemResolution.ResY || WindowMode != GSystemResolution.WindowMode)
		{
			FSystemResolution::RequestResolutionChange(ResX, ResY, WindowMode);
			IConsoleManager::Get().CallAllConsoleVariableSinks();
		}
	}

	// bool bHaveProjectSettings = IsClassLoaded<UGeneralProjectSettings>();

	/*const FText WindowTitleOverride = bHaveProjectSettings ? GetDefault<UGeneralProjectSettings>()->ProjectDisplayedTitle : FText();
	const FText WindowTitleComponent = WindowTitleOverride.IsEmpty() ? NSLOCTEXT("UnrealEd", "GameWindowTitle", "{GameName}") : WindowTitleOverride;*/

	FText WindowTitleOverride = FText();
	GConfig->GetText(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("ProjectDisplayedTitle"), WindowTitleOverride, GGameIni);
	const FText WindowTitleComponent = WindowTitleOverride.IsEmpty() ? NSLOCTEXT("UnrealEd", "GameWindowTitle", "{GameName}") : WindowTitleOverride;

	FText WindowDebugInfoComponent = FText();
#if !UE_BUILD_SHIPPING
	//const FText WindowDebugInfoOverride = bHaveProjectSettings ? GetDefault<UGeneralProjectSettings>()->ProjectDebugTitleInfo : FText();
	FText WindowDebugInfoOverride = FText();
	GConfig->GetText(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("ProjectDebugTitleInfo"), WindowDebugInfoOverride, GGameIni);
	WindowDebugInfoComponent = WindowDebugInfoOverride.IsEmpty() ? NSLOCTEXT("UnrealEd", "GameWindowTitleDebugInfo", "({PlatformArchitecture}-bit {BuildConfiguration} {RHIName})") : WindowDebugInfoOverride;
#endif

	//These are invariant strings so they don't need to be localized
	const FText PlatformBits = FText::FromString( TEXT( "64" ) );

	// Note: If these parameters are updated or renamed, please update the tooltip on the ProjectDisplayedTitle and ProjectDebugTitleInfo properties
	FFormatNamedArguments Args;
	Args.Add( TEXT("GameName"), FText::FromString( FApp::GetProjectName() ) );
	Args.Add( TEXT("PlatformArchitecture"), PlatformBits );

	FText RHIName = FText::FromName(LegacyShaderPlatformToShaderFormat( GMaxRHIShaderPlatform ));
#if WITH_EDITOR
	FString ShaderPlatformName;
	// If we are overriding the ShaderPlatform, update the RHIName
	if (FParse::Value(FCommandLine::Get(), TEXT("OverrideSP="), ShaderPlatformName) && !ShaderPlatformName.IsEmpty())
	{
		EShaderPlatform OverrideShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(*ShaderPlatformName);
		if (OverrideShaderPlatform != SP_NumPlatforms)
		{
			if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(OverrideShaderPlatform))
			{
				FString DeviceProfileName;
				FString FriendlyDeviceProfilePreviewName = FDataDrivenShaderPlatformInfo::GetFriendlyName(OverrideShaderPlatform).ToString();
				// If we are overriding the DeviceProfile, append the DP name to the RHIName.
				if(FParse::Value(FCommandLine::Get(), TEXT("dp.Override="), DeviceProfileName))
				{
					FriendlyDeviceProfilePreviewName += TEXT(" DeviceProfile: ") + DeviceProfileName;
				}
				RHIName = FText::FromName(FName(FriendlyDeviceProfilePreviewName));
			}
		}
	}
#endif
	Args.Add( TEXT("RHIName"), RHIName);
	Args.Add( TEXT("BuildConfiguration"), FText::FromString(LexToString(FApp::GetBuildConfiguration()) ) );

	/************************************************************************/
	/************************ Add device name to window title****************/
	/************************************************************************/
	const FText WindowTitleVar = FText::Format( FText::FromString(TEXT("{0} {1} {2}")), WindowTitleComponent, WindowDebugInfoComponent, FGlobalTabmanager::Get()->GetApplicationTitle() );
	const FText WindowTitle = FText::Format(WindowTitleVar, Args);

	auto GetProjectSettingBool = [](const FString& ParamName, bool Default) -> bool
		{
			bool Temp = Default;
			GConfig->GetBool(TEXT("/Script/EngineSettings.GeneralProjectSettings"), *ParamName, Temp, GGameIni);
			return Temp;
		};

	auto GetProjectSettingInt = [](const FString& ParamName, int Default) -> int32
	{
		int32 Temp = Default;
		GConfig->GetInt(TEXT("/Script/EngineSettings.GeneralProjectSettings"), *ParamName, Temp, GGameIni);
		return Temp;
	};

	const bool bShouldPreserveAspectRatio = GetProjectSettingBool(TEXT("bShouldWindowPreserveAspectRatio"), true);
	const bool bUseBorderlessWindow = GetProjectSettingBool(TEXT("bUseBorderlessWindow"), false) && PLATFORM_SUPPORTS_BORDERLESS_WINDOW;
	const bool bAllowWindowResize = GetProjectSettingBool(TEXT("bAllowWindowResize"), true);
	const bool bAllowClose = GetProjectSettingBool(TEXT("bAllowClose"), true);
	const bool bAllowMaximize = GetProjectSettingBool(TEXT("bAllowMaximize"), true);
	const bool bAllowMinimize = GetProjectSettingBool(TEXT("bAllowMinimize"), true);

	const int32 MinWindowWidth = GetProjectSettingInt(TEXT("MinWindowWidth"), 640);
	const int32 MinWindowHeight = GetProjectSettingInt(TEXT("MinWindowHeight"), 480);

	// Allow optional winX/winY parameters to set initial window position
	EAutoCenter AutoCenterType = EAutoCenter::PrimaryWorkArea;
	int32 WinX=0;
	int32 WinY=0;
	if (FParse::Value(FCommandLine::Get(), TEXT("WinX="), WinX) && FParse::Value(FCommandLine::Get(), TEXT("WinY="), WinY))
	{
		AutoCenterType = EAutoCenter::None;
	}

	// SAVEWINPOS tells us to load/save window positions to user settings (this is disabled by default)
	int32 SaveWinPos;
	if (FParse::Value(FCommandLine::Get(), TEXT("SAVEWINPOS="), SaveWinPos) && SaveWinPos > 0)
	{
		// Note GameUserSettings is not instantiated here yet, so we need to read directly from the configs
		FString ScriptEngineCategory = "/Script/Engine.Engine";
		FString GameUserSettingsCategory = "/Script/Engine.GameUserSettings";
		GConfig->GetString(*ScriptEngineCategory, TEXT("GameUserSettingsClassName"), GameUserSettingsCategory, GEngineIni);

		// Backwards compatibility for the move from a single saved window position to an array to handle multiple instances.
		// Old behavior: If SaveWinPos > 0 and WindowPosX and WindowPosY config values exist, use those values for the window position.
		// New behavior: If SaveWinPos > 0, use this value as a slot number to save multiple positions in the WindowPositions array.
		// For compatibility, if WindowPosX and WindowPosY exist in the config, use the old behavior.
		// Upon re-saving the config, those values will be moved into the new WindowPositions array.
		if (GConfig->GetInt(*GameUserSettingsCategory, TEXT("WindowPosX"), WinX, GGameUserSettingsIni) &&
			GConfig->GetInt(*GameUserSettingsCategory, TEXT("WindowPosY"), WinY, GGameUserSettingsIni))
		{
			AutoCenterType = EAutoCenter::None;
		}
		else
		{
			TArray<FString> PositionStrings;
			GConfig->GetArray(*GameUserSettingsCategory, TEXT("WindowPositions"), PositionStrings, GGameUserSettingsIni);

			// SaveWinPos is 1-based, the WindowPositions array is 0-based.
			int32 Index = SaveWinPos - 1;
			if (PositionStrings.IsValidIndex(Index))
			{
				FIntPoint Position;
				if (Position.InitFromString(PositionStrings[Index]))
				{
					WinX = Position.X;
					WinY = Position.Y;
					AutoCenterType = EAutoCenter::None;
				}
				else
				{
					UE_LOGF(LogEngine, Warning, "Saved position for SaveWinPos=%d invalid, ignoring.", SaveWinPos);
				}
			}
		}
	}

	// Give the window the max width/height of either the requested resolution, or your available desktop resolution
	// We need to do this as we request some 4K windows when rendering sequences, and the OS may try and clamp that
	// window to your available desktop resolution
	TOptional<float> MaxWindowWidth;
	TOptional<float> MaxWindowHeight;
	if (WindowMode == EWindowMode::Windowed)
	{
		// Get available desktop area
		FDisplayMetrics DisplayMetrics;
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().GetInitialDisplayMetrics(DisplayMetrics);
		}
		else
		{
			FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		}

		MaxWindowWidth = FMath::Max(DisplayMetrics.VirtualDisplayRect.Right - DisplayMetrics.VirtualDisplayRect.Left, ResX);
		MaxWindowHeight = FMath::Max(DisplayMetrics.VirtualDisplayRect.Bottom - DisplayMetrics.VirtualDisplayRect.Top, ResY);
	}

	if ((MonitorIndex != -1) && (AutoCenterType == EAutoCenter::PrimaryWorkArea))
	{
		FDisplayMetrics DisplayMetrics;
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().GetInitialDisplayMetrics(DisplayMetrics);
		}
		else
		{
			FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		}

		if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndex))
		{
			AutoCenterType = EAutoCenter::None;
			WinX = DisplayMetrics.MonitorInfo[MonitorIndex].WorkArea.Left;
			WinY = DisplayMetrics.MonitorInfo[MonitorIndex].WorkArea.Top;

			if (WindowMode == EWindowMode::Windowed)
			{
				const FPlatformRect& WorkArea = DisplayMetrics.MonitorInfo[MonitorIndex].WorkArea;
				const int32 WorkWidth = WorkArea.Right - WorkArea.Left;
				const int32 WorkHeight = WorkArea.Bottom - WorkArea.Top;

				if (ResX < WorkWidth)
				{
					WinX += (WorkWidth - ResX) / 2;
				}

				if (ResY < WorkHeight)
				{
					WinY += (WorkHeight - ResY) / 2;
				}
			}
		}
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
	.Style(bUseBorderlessWindow ? &FWindowStyle::GetBorderless() : &FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window"))
	.ClientSize(FVector2D(ResX, ResY))
	.AdjustInitialSizeAndPositionForDPIScale(false)
	.Title(WindowTitle)
	.AutoCenter(AutoCenterType)
	.ScreenPosition(FVector2D(WinX, WinY))
	.MinWidth(MinWindowWidth)
	.MinHeight(MinWindowHeight)
	.MaxWidth(MaxWindowWidth)
	.MaxHeight(MaxWindowHeight)
	.FocusWhenFirstShown(true)
	.SaneWindowPlacement(AutoCenterType == EAutoCenter::None)
	.UseOSWindowBorder(!bUseBorderlessWindow)
	.CreateTitleBar(!bUseBorderlessWindow)
	.ShouldPreserveAspectRatio(bShouldPreserveAspectRatio)
	.LayoutBorder(bUseBorderlessWindow ? FMargin(0) : FMargin(5, 5, 5, 5))
	.SizingRule(bAllowWindowResize ? ESizingRule::UserSized : ESizingRule::FixedSize)
	.HasCloseButton(bAllowClose)
	.SupportsMinimize(bAllowMinimize)
	.SupportsMaximize(bAllowMaximize);
	Window->SetAllowFastUpdate(true);

	const bool bShowImmediately = false;

	FSlateApplication::Get().AddWindow( Window, bShowImmediately );
	
	// Do not set fullscreen mode here, since it doesn't take 
	// HMDDevice into account. The window mode will be set properly later
	// from SwitchGameWindowToUseGameViewport() method (see ResizeWindow call).
	if (WindowMode == EWindowMode::Fullscreen)
	{
		Window->SetWindowMode(EWindowMode::WindowedFullscreen);
	}
	else
	{
		Window->SetWindowMode(WindowMode);
	}

	// No need to show window if rendering off-screen without the null platform as it does not render to screen
	if (FSlateApplication::Get().IsRenderingOffScreen() && !FNullPlatformApplicationMisc::IsUsingNullApplication())
	{
		FSlateApplicationBase::Get().GetRenderer()->CreateViewport(Window);
	}
	else
	{
		Window->ShowWindow();
	}

	// Tick now to force a redraw of the window and ensure correct fullscreen application
	FSlateApplication::Get().Tick();

	return Window;
}

void UGameEngine::SwitchGameWindowToUseGameViewport()
{
	if (TSharedPtr<SWindow> GameViewportWindowPtr = GameViewportWindow.Pin())
	{
		if (GameViewportWindowPtr->GetContent() != GameViewportWidget)
		{
			if (!GameViewportWidget.IsValid())
			{
				CreateGameViewport(GameViewport);
			}

			if (GameViewportWidget.IsValid() && FSlateApplication::IsInitialized())
			{
				GameViewportWindowPtr->SetContent(GameViewportWidget.ToSharedRef());
				GameViewportWindowPtr->SlatePrepass(FSlateApplication::Get().GetApplicationScale() * GameViewportWindowPtr->GetNativeWindow()->GetDPIScaleFactor());
			}
			else
			{
				UE_LOGF(LogEngine, Error, "The Game Viewport Widget is invalid.");
			}

			// If Scene Viewport is not valid, the window was closed.
			if (SceneViewport.IsValid() && FSlateApplication::IsInitialized())
			{
				SceneViewport->SetMonitorIndexForResize(GSystemResolution.MonitorIndex);
				SceneViewport->ResizeFrame((uint32)GSystemResolution.ResX, (uint32)GSystemResolution.ResY, GSystemResolution.WindowMode);

				// Registration of the game viewport to that messages are correctly received.
				// Could be a re-register, however it's necessary after the window is set.
				if (GameViewportWidget.IsValid())
				{
					FSlateApplication::Get().RegisterGameViewport(GameViewportWidget.ToSharedRef());
				}
				FSlateApplication::Get().SetAllUserFocusToGameViewport(EFocusCause::SetDirectly);
			}
		}
	}
}

void UGameEngine::OnGameWindowClosed(const TSharedRef<SWindow>& WindowBeingClosed)
{
	FSlateApplication::Get().UnregisterGameViewport();
#if UE_BUILD_SHIPPING || WITH_EDITOR
	// Hide crash for users but not developers.
	if (ensure(SceneViewport.IsValid()))
#endif
	{
		// This will shutdown the game
		GameViewport->CloseRequested(SceneViewport.Get());
		UnregisterViewport(SceneViewport.ToSharedRef());
		SceneViewport.Reset();
	}
}

void UGameEngine::OnGameWindowMoved_SavePosition(const TSharedRef<SWindow>& WindowBeingMoved, int32 SaveWinPosIndex)
{
	const FSlateRect WindowRect = WindowBeingMoved->GetRectInScreen();

	// Reload config in case another running instance has modified it since the last load.
	// Prevents this instance from clobbering other SaveWinPos slots.
	GetGameUserSettings()->ReloadWindowPositions();

	// The -SaveWinPos=X command-line option is 1-based, but the GameUserSettings array is 0-based.
	GetGameUserSettings()->SetWindowPosition(WindowRect.Left, WindowRect.Top, SaveWinPosIndex);
	GetGameUserSettings()->SaveConfig();
}

void UGameEngine::OnGameWindowMoved(const TSharedRef<SWindow>& WindowBeingMoved)
{
	OnGameWindowMoved_SavePosition(WindowBeingMoved, 0);
}

void UGameEngine::RedrawViewports( bool bShouldPresent /*= true*/ )
{
	if ( GameViewport != NULL )
	{
		GameViewport->LayoutPlayers();
	}
	Super::RedrawViewports(bShouldPresent);
}

void UGameEngine::OnViewportResized(FViewport* Viewport, uint32 Unused)
{
	if (Viewport && Viewport == SceneViewport.Get() && GameViewportWindow.IsValid() && GameViewportWindow.Pin()->GetWindowMode() == EWindowMode::Windowed)
	{
		const FIntPoint ViewportSize = Viewport->GetSizeXY();
		if (ViewportSize.X > 0 && ViewportSize.Y > 0)
		{
			GSystemResolution.ResX = ViewportSize.X;
			GSystemResolution.ResY = ViewportSize.Y;
			FSystemResolution::RequestResolutionChange(GSystemResolution.ResX, GSystemResolution.ResY, EWindowMode::Windowed);

			UGameUserSettings* Settings = GetGameUserSettings();
			Settings->SetScreenResolution(ViewportSize);
			Settings->ConfirmVideoMode();
			Settings->RequestUIUpdate();
		}
	}
}

void UGameEngine::OnGPUCrash()
{
#if PLATFORM_WINDOWS
	static IConsoleVariable* ScreenshotOnCrash = IConsoleManager::Get().FindConsoleVariable(TEXT("WindowsPlatformCrashContext.ScreenshotOnGPUCrash"));
	if (GIsEditor || !ScreenshotOnCrash || !ScreenshotOnCrash->GetBool())
	{
		return;
	}

	// Find the game window and attempt to retrieve its pixels.
	uint32 Width = 0, Height = 0;
	TArray<FColor> Pixels;

	TSharedPtr<SWindow> Window = GameViewportWindow.Pin();
	if (Window)
	{
		TSharedPtr<FGenericWindow> NativeWindow = Window->GetNativeWindow();
		if (NativeWindow && NativeWindow->IsDefinitionValid())
		{
			const TSharedPtr<FWindowsWindow> WindowsWindow = StaticCastSharedPtr<FWindowsWindow>(NativeWindow);
			Pixels = WindowsWindow->GetWindowPixels(Width, Height);
		}
	}

	if (!Pixels.IsEmpty())
	{
		check(Width > 0 && Height > 0);
		TArray64<uint8> JPGData;
		FImageView Image((void*)Pixels.GetData(), Width, Height, ERawImageFormat::BGRA8);
		FImageUtils::CompressImage(JPGData, TEXT("jpg"), Image, 70);

		FString ScreenshotPath = FPaths::Combine(FPaths::ProjectLogDir(), TEXT("CrashCapture.jpg"));
		if (FFileHelper::SaveArrayToFile(JPGData, *ScreenshotPath))
		{
			FWindowsPlatformCrashContext::SetEngineData(TEXT("CrashScreenshotWidth"), FString::FromInt(Width));
			FWindowsPlatformCrashContext::SetEngineData(TEXT("CrashScreenshotHeight"), FString::FromInt(Height));
			FWindowsPlatformCrashContext::AddOptionalAttachment(ScreenshotPath);
		}
		else
		{
			UE_LOGF(LogEngine, Warning, "Error saving crash screenshot to file: %ls", *ScreenshotPath);
		}
	}
#endif
}

/*-----------------------------------------------------------------------------
	Game init and exit.
-----------------------------------------------------------------------------*/
UEngine::UEngine(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ViewExtensions( new FSceneViewExtensions() )
{
	C_WorldBox = FColor(0, 0, 40, 255);
	C_BrushWire = FColor(192, 0, 0, 255);
	C_AddWire = FColor(127, 127, 255, 255);
	C_SubtractWire = FColor(255, 127, 127, 255);
	C_SemiSolidWire = FColor(127, 255, 0, 255);
	C_NonSolidWire = FColor(63, 192, 32, 255);
	C_WireBackground = FColor(0, 0, 0, 255);
	C_ScaleBoxHi = FColor(223, 149, 157, 255);
	C_VolumeCollision = FColor(149, 223, 157, 255);
	C_BSPCollision = FColor(149, 157, 223, 255);
	C_OrthoBackground = FColor(30, 30, 30, 255);
	C_Volume = FColor(255, 196, 255, 255);
	C_BrushShape = FColor(128, 255, 128, 255);

	SelectionHighlightIntensity = 0.0f;
	BSPSelectionHighlightIntensity = 0.0f;

	SelectionHighlightIntensityBillboards = 0.25f;

	bIsInitialized = false;

	BeginStreamingPauseDelegate = NULL;
	EndStreamingPauseDelegate = NULL;

	bCanBlueprintsTickByDefault = true;
	bOptimizeAnimBlueprintMemberVariableAccess = true;
	bAllowMultiThreadedAnimationUpdate = true;

	bUseFixedFrameRate = false;
	FixedFrameRate = 30.f;

	bIsVanillaProduct = false;

	bGenerateDefaultTimecode = true;
	GenerateDefaultTimecodeFrameRate = FFrameRate(24, 1);
	GenerateDefaultTimecodeFrameDelay = 0.f;

	GameScreenshotSaveDirectory.Path = FPaths::ScreenShotDir();

	LastGCFrame = TNumericLimits<uint64>::Max();

	#if !UE_SERVER
	{
		bIsDynamicResolutionPaused = false;
		bDynamicResolutionEnableUserSetting = false;
		LastDynamicResolutionEvent = EDynamicResolutionStateEvent::EndFrame;

		if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
		{
			DynamicResolutionState = NextDynamicResolutionState = FDynamicResolutionHeuristicProxy::CreateDefaultState();
		}
	}
	#endif

	// Relay to deprecated delegates
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PreRenderDelegateEx.AddLambda([this](FRDGBuilder&) { PreRenderDelegate.Broadcast(); });
	PostRenderDelegateEx.AddLambda([this](FRDGBuilder&){ PostRenderDelegate.Broadcast(); });
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}



//@todo kairos: Move this and maybe the above engine handling code to somewhere else. I can't put this into Core
// with Embedded because of the Json dependency that I don't want/can't? add to Core. Maybe ApplicationCore?

#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameEngine)


class FEmbeddedCommunicationExec : public FSelfRegisteringExec
{
public:
	FEmbeddedCommunicationExec()
		: FSelfRegisteringExec()
	{

		FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(TEXT("engine")).AddLambda([](const FEmbeddedCallParamsHelper& Message)
		{
			if (Message.Command == TEXT("StartUELive"))
			{
				FName Requester = *Message.Parameters.FindRef(TEXT("requester"));
				bool bTickOnly = Message.Parameters.FindRef(TEXT("tickonly")) == TEXT("true");
				
				FEmbeddedCommunication::KeepAwake(Requester, !bTickOnly);
				Message.OnCompleteDelegate({}, TEXT(""));
			}
			else if (Message.Command == TEXT("StopUELive"))
			{
				FName Requester = *Message.Parameters.FindRef(TEXT("requester"));
				
				FEmbeddedCommunication::AllowSleep(Requester);
				Message.OnCompleteDelegate({}, TEXT(""));
			}
			else if (Message.Command == TEXT("devicesleep"))
			{
				bool bEnabled = Message.Parameters.FindRef(TEXT("enabled")).ToBool();
				FPlatformApplicationMisc::ControlScreensaver(bEnabled ? FGenericPlatformApplicationMisc::Enable : FGenericPlatformApplicationMisc::Disable);
				Message.OnCompleteDelegate({}, TEXT(""));
			}
			else if (Message.Command == TEXT("shaderresumebatching"))
			{
				FShaderPipelineCache::ResumeBatching();
				Message.OnCompleteDelegate({}, TEXT(""));
			}
			else if (Message.Command == TEXT("shaderpausebatching"))
			{
				FShaderPipelineCache::PauseBatching();
				Message.OnCompleteDelegate({}, TEXT(""));
			}
			else if (Message.Command == TEXT("getmemorybucket"))
			{
				Message.OnCompleteDelegate({ { TEXT("bucket"), LexToString((int)FPlatformMemory::GetMemorySizeBucket()) } }, TEXT(""));
			}

// disable things that are too scary to expose to a shipping external app/webview
#if !UE_BUILD_SHIPPING
			// execute any console commands
			else if (Message.Command == TEXT("console"))
			{
				// executed too early, must fail
				if (GEngine == nullptr)
				{
					// call the completion delegate with all text output
					TArray<FString> Keys;
					Message.Parameters.GetKeys(Keys);
					Message.OnCompleteDelegate({ }, FString::Printf(TEXT("GEngine does not exist yet, unable to execute console command (%s)"), Keys.Num() ? *Keys[0] : TEXT("None")));
					return;
				}
				
				// gather all of the output
				FStringOutputDevice Output;
				ULocalPlayer* LocalPlayer = GEngine->GetDebugLocalPlayer();
				
				for (auto Pair : Message.Parameters)
				{
					if( LocalPlayer )
					{
						LocalPlayer->Exec( LocalPlayer->GetWorld(), *Pair.Key, Output );
					}
					// and fall back to UEngine otherwise.
					else
					{
						GEngine->Exec( GWorld, *Pair.Key, Output );
					}
				}
				
				// call the completion delegate with all text output
				Message.OnCompleteDelegate({ { TEXT("output"), Output } }, TEXT(""));
			}
			else if (Message.Command == TEXT("setconfig"))
			{
				FString File = Message.Parameters.FindRef(TEXT("file"));
				FString Section = Message.Parameters.FindRef(TEXT("section"));
				FString Key = Message.Parameters.FindRef(TEXT("key"));
				FString Value = Message.Parameters.FindRef(TEXT("value"));
				bool bSkipSave = Message.Parameters.FindRef(TEXT("skipsave")) == TEXT("true");
				
				FString& ConfigFile = GetConfigFromName(File);
				
				GConfig->SetString(*Section, *Key, *Value, ConfigFile);
				if (!bSkipSave)
				{
					GConfig->Flush(false, ConfigFile);
				}
				
				// send back empty reply, nothing to report
				Message.OnCompleteDelegate({ }, TEXT(""));
			}
#endif
			else if (Message.Command == TEXT("getconfig"))
			{
				FString File = Message.Parameters.FindRef(TEXT("file"));
				FString Section = Message.Parameters.FindRef(TEXT("section"));
				FString Key = Message.Parameters.FindRef(TEXT("key"));
				
				FString ConfigFile = GetConfigFromName(File);
				
				FString Value;
				if (GConfig->GetString(*Section, *Key, Value, ConfigFile))
				{
					// send back the value
					Message.OnCompleteDelegate({ {TEXT("value"), Value} }, TEXT(""));
				}
				else
				{
					Message.OnCompleteDelegate({ }, FString::Printf(TEXT("Config key [%s] : %s in %s was not found"), *Section, *Key, *File));
				}
			}
			else if (Message.Command == TEXT("cvar"))
			{
				FString Name = Message.Parameters.FindRef(TEXT("name"));
				IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*Name);
				if (CVar)
				{
					// send back the value
					Message.OnCompleteDelegate({ {TEXT("value"), CVar->GetString()} }, TEXT(""));
				}
				else
				{
					Message.OnCompleteDelegate({ }, FString::Printf(TEXT("CVar %s not found"), *Name));
				}
			}
            else if (Message.Command == TEXT("shareurl"))
            {
                FString URL = Message.Parameters.FindRef(TEXT("url"));
                FString Description = Message.Parameters.FindRef(TEXT("description"));
                FString Subject = Message.Parameters.FindRef(TEXT("title"));
                
                FString ParsedX = Message.Parameters.FindRef(TEXT("x"));
                FString ParsedY = Message.Parameters.FindRef(TEXT("y"));
                
                int ConvertedX = 0;
                int ConvertedY = 0;
                if ((!ParsedX.IsEmpty()
                     && !ParsedY.IsEmpty()))
                {
                    LexFromString(ConvertedX, *ParsedX);
                    LexFromString(ConvertedY, *ParsedY);
                }
                
                FPlatformMisc::ShareURL(URL, FText::FromString(Description), ConvertedX, ConvertedY);
            }
			else
			{
				Message.OnCompleteDelegate({}, TEXT("Unknown command"));
			}
		});
//	#endif

	}
	
	static FString& GetConfigFromName(const FString& Name)
	{
		if (Name == TEXT("Game"))
		{
			return GGameIni;
		}
		else if (Name == TEXT("Input"))
		{
			return GInputIni;
		}
		else if (Name == TEXT("GameUserSettings"))
		{
			return GGameUserSettingsIni;
		}
		else if (Name == TEXT("Scalability"))
		{
			return GScalabilityIni;
		}
		else if (Name == TEXT("Hardware"))
		{
			return GHardwareIni;
		}
		return GEngineIni;
	}

#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("exitembedded")))
		{
			FEmbeddedCallParamsHelper Helper;
			Helper.Command = TEXT("exitembedded");
			if (FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("native")).IsBound())
			{
				FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("native")).Broadcast(Helper);
			}
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("DumpEmbedded")))
		{
			UE_LOGF(LogEngine, Display, "Embedded state: %ls", *FEmbeddedCommunication::GetDebugInfo());
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("webcall")))
		{
			int CommandIndex = FCString::Atoi(Cmd);
			FString Key = FString::Printf(TEXT("Calls[%d]"), CommandIndex);
			FString Command;
			if (GConfig->GetString(TEXT("WebCalls"), *Key, Command, GEngineIni))
			{
				const TSharedRef< TJsonReader<> >& Reader = TJsonReaderFactory<>::Create(Command.Replace(TEXT("'"), TEXT("\"")));
				TSharedPtr<FJsonObject> CommandObject;
				if (FJsonSerializer::Deserialize(Reader, CommandObject))
				{
					FEmbeddedCallParamsHelper Helper;
					FString SubsystemString;
					CommandObject->TryGetStringField(TEXT("Subsystem"), SubsystemString);
					CommandObject->TryGetStringField(TEXT("Command"), Helper.Command);

					const TSharedPtr<FJsonObject>* Args = nullptr;
					if (CommandObject->TryGetObjectField(TEXT("Args"), Args))
					{
						for (auto It : (*Args)->Values)
						{
							FString ValueString;
							if (!It.Value->TryGetString(ValueString))
							{
								// if casual string conversion failed, then encode it as a json string
								const TArray< TSharedPtr<FJsonValue> >* ValueArray = nullptr;
								if (It.Value->TryGetArray(ValueArray))
								{
									TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&ValueString);
									FJsonSerializer::Serialize(*ValueArray, Writer);
								}
								else
								{
									const TSharedPtr<FJsonObject>* ValueObject = nullptr;
									if (It.Value->TryGetObject(ValueObject))
									{
										TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&ValueString);
										FJsonSerializer::Serialize(ValueObject->ToSharedRef(), Writer);
									}
								}
							}

							// now put whatever string we made into the map
							Helper.Parameters.Add(FString(It.Key), ValueString);
						}
					}

					Helper.OnCompleteDelegate = [](const FEmbeddedCommunicationMap& InReturnValues, FString InError) {};
					FName Subsystem(*SubsystemString);
					if (FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(Subsystem).IsBound())
					{
						FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(Subsystem).Broadcast(Helper);
					}
				}
			}
			return true;
		}
		return false;
	}
#endif // UE_ALLOW_EXEC_COMMANDS
} GEmbeddedCommunicationExec;


void UGameEngine::Init(IEngineLoop* InEngineLoop)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameEngine::Init);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UGameEngine Init"), STAT_GameEngineStartup, STATGROUP_LoadTime);

	if (!GIsEditor)
	{
		CmdExec = MakePimpl<FEngineConsoleCommandExecutor>(this);
	}

	// Call base.
	UEngine::Init(InEngineLoop);

#if USE_NETWORK_PROFILER
	FString NetworkProfilerTag;
	if( FParse::Value(FCommandLine::Get(), TEXT("NETWORKPROFILER="), NetworkProfilerTag ) )
	{
		GNetworkProfiler.EnableTracking(true);
	}
#endif

	// Load and apply user game settings
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitGameUserSettings);

		// Push recreate render state context to force single recreate instead of multiple recreates for each changed cvar
		FGlobalComponentRecreateRenderStateContext Context;

		GetGameUserSettings()->LoadSettings();
		GetGameUserSettings()->ApplyNonResolutionSettings();
	}

	// Create game instance.  For GameEngine, this should be the only GameInstance that ever gets created.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitGameInstance);
		FSoftClassPath GameInstanceClassName = GetDefault<UGameMapsSettings>()->GameInstanceClass;
		UClass* GameInstanceClass = (GameInstanceClassName.IsValid() ? LoadObject<UClass>(NULL, *GameInstanceClassName.ToString()) : UGameInstance::StaticClass());
		
		if (GameInstanceClass == nullptr)
		{
			UE_LOGF(LogEngine, Error, "Unable to load GameInstance Class '%ls'. Falling back to generic UGameInstance.", *GameInstanceClassName.ToString());
			GameInstanceClass = UGameInstance::StaticClass();
		}

		GameInstance = NewObject<UGameInstance>(this, GameInstanceClass);

		GameInstance->InitializeStandalone();
	}
 
//  	// Creates the initial world context. For GameEngine, this should be the only WorldContext that ever gets created.
//  	FWorldContext& InitialWorldContext = CreateNewWorldContext(EWorldType::Game);

	IMovieSceneCaptureInterface* MovieSceneCaptureImpl = nullptr;
#if WITH_EDITOR
	if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
	{
		MovieSceneCaptureImpl = IMovieSceneCaptureModule::Get().InitializeFromCommandLine();
		if (MovieSceneCaptureImpl)
		{
			StartupMovieCaptureHandle = MovieSceneCaptureImpl->GetHandle();
		}
	}
#endif

	// Initialize the viewport client.
	UGameViewportClient* ViewportClient = NULL;
	if(GIsClient)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitGameViewPortClient);
		ViewportClient = NewObject<UGameViewportClient>(this, GameViewportClientClass);
		ViewportClient->Init(*GameInstance->GetWorldContext(), GameInstance);
		GameViewport = ViewportClient;
		GameInstance->GetWorldContext()->GameViewport = ViewportClient;
	}

	LastTimeLogsFlushed = FPlatformTime::Seconds();

	// Attach the viewport client to a new viewport.
	if(ViewportClient)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AttachGameViewport);
		// This must be created before any gameplay code adds widgets
		bool bWindowAlreadyExists = GameViewportWindow.IsValid();
		if (!bWindowAlreadyExists)
		{
			UE_LOGF(LogEngine, Log, "GameWindow did not exist.  Was created");
			GameViewportWindow = CreateGameWindow();
		}

		CreateGameViewport( ViewportClient );

		if( !bWindowAlreadyExists )
		{
			SwitchGameWindowToUseGameViewport();
		}

		FString Error;
		if(ViewportClient->SetupInitialLocalPlayer(Error) == NULL)
		{
			UE_LOGF(LogEngine, Fatal,"%ls",*Error);
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(BroadcastOnViewPortCreated);
		UGameViewportClient::OnViewportCreated().Broadcast();
	}

	UE_LOGF(LogInit, Display, "Game Engine Initialized." );

	// for IsInitialized()
	bIsInitialized = true;
}

void UGameEngine::Start()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameEngine::Start);
	UE_LOGF(LogInit, Display, "Starting Game.");

	GameInstance->StartGameInstance();
}

void UGameEngine::PreExit()
{
	UE_LOGF(LogInit, Display, "PreExit Game.");

	GetGameUserSettings()->SaveSettings();

	// Stop tracking, automatically flushes.
	NETWORK_PROFILER(GNetworkProfiler.EnableTracking(false));

	CancelAllPending();

	// Clean up all worlds
	for (int32 WorldIndex = 0; WorldIndex < WorldList.Num(); ++WorldIndex)
	{
		UWorld* const World = WorldList[WorldIndex].World();
		if ( World != NULL )
		{
			World->BeginTearingDown();

			// Cancel any pending connection to a server
			CancelPending(World);

			// Shut down any existing game connections
			ShutdownWorldNetDriver(World);

			// Force mark all streaming levels for stream out
			World->bIsLevelStreamingFrozen = false;
			World->SetShouldForceUnloadStreamingLevels(true);

			// Make sure there are no pending visibility requests.
			World->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);
			
			World->EndPlay(EEndPlayReason::Quit);

			if (World->GetGameInstance() != nullptr)
			{
				World->GetGameInstance()->Shutdown();
			}

			World->CleanupWorld();
		}
		else 
		{
			// A world may not be bound to the context in an early shutdown case such as the world failing to load. 
			// Preserve the call to Shutdown to avoid introducing a different destruction order in an exceptional error 
			// case.
			CancelPending(WorldList[WorldIndex]);
			
			if (WorldList[WorldIndex].OwningGameInstance != nullptr)
			{
				WorldList[WorldIndex].OwningGameInstance->Shutdown();
			}
		}
	}

	Super::PreExit();
}

void UGameEngine::FinishDestroy()
{
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		// Game exit.
		UE_LOGF(LogExit, Log, "Game engine shut down" );
	}

	Super::FinishDestroy();

	CmdExec.Reset();
}

bool UGameEngine::NetworkRemapPath(UNetConnection* Connection, FString& Str, bool bReading /*= true*/)
{
	if (Connection == nullptr)
	{
		return false;
	}

	UWorld* const World = Connection->GetWorld();

	if (World == nullptr)
	{
		return false;
	}

	if (!bReading)
	{
		return false;
	}

	// If the driver is using a duplicate level ID, find the level collection using the driver
	// and see if any of its levels match the prefixed name. If so, remap Str to that level's
	// prefixed name.
	if (UNetDriver* Driver = Connection->GetDriver())
	{
		if (Driver->GetDuplicateLevelID() != INDEX_NONE)
		{
			const FName PrefixedName = *UWorld::ConvertToPIEPackageName(Str, Driver->GetDuplicateLevelID());

			for (const FLevelCollection& Collection : World->GetLevelCollections())
			{
				if (Collection.GetNetDriver() == Driver || Collection.GetDemoNetDriver() == Driver)
				{
					for (const ULevel* Level : Collection.GetLevels())
					{
						const UPackage* const CachedOutermost = Level ? Level->GetOutermost() : nullptr;
						if (CachedOutermost && CachedOutermost->GetFName() == PrefixedName)
						{
							Str = PrefixedName.ToString();
							return true;
						}
					}
				}
			}
		}
	}

	// Try to find the level script objects and remap them for when demos are being replayed.
	if (Connection->IsInternalAck() && World->RemapCompiledScriptActor(Str))
	{
		return true;
	}

	// If the game has created multiple worlds, some of them may have prefixed package names,
	// so we need to remap the world package and streaming levels for replay playback to work correctly.
	FWorldContext& Context = GetWorldContextFromWorldChecked(World);
	if (Context.PIEInstance == INDEX_NONE)
	{
		if (WorldList.Num() > 1)
		{
			// If this is not a PIE instance but sender is PIE, we need to strip the PIE prefix
			const FString Stripped = UWorld::RemovePIEPrefix(Str);
			if (!Stripped.Equals(Str, ESearchCase::CaseSensitive))
			{
				Str = Stripped;
				return true;
			}
		}
		return false;
	}

	// If the prefixed path matches the world package name or the name of a streaming level,
	// return the prefixed name.
	FString PackageNameOnly = Str;
	FPackageName::TryConvertFilenameToLongPackageName(PackageNameOnly, PackageNameOnly);

	const FString PrefixedFullName = UWorld::ConvertToPIEPackageName(Str, Context.PIEInstance);
	const FString PrefixedPackageName = UWorld::ConvertToPIEPackageName(PackageNameOnly, Context.PIEInstance);
	const FString WorldPackageName = World->GetOutermost()->GetName();

	if (WorldPackageName == PrefixedPackageName)
	{
		Str = PrefixedFullName;
		return true;
	}

	for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
	{
		if (StreamingLevel != nullptr)
		{
			const FString StreamingLevelName = StreamingLevel->GetWorldAsset().GetLongPackageName();
			if (StreamingLevelName == PrefixedPackageName)
			{
				Str = PrefixedFullName;
				return true;
			}
		}
	}

	return false;
}

bool UGameEngine::ShouldDoAsyncEndOfFrameTasks() const
{
	return FApp::ShouldUseThreadingForPerformance() && ENamedThreads::GetRenderThread() != ENamedThreads::GameThread && !!GDoAsyncEndOfFrameTasks;
}

/*-----------------------------------------------------------------------------
	Command line executor.
-----------------------------------------------------------------------------*/

#if UE_ALLOW_EXEC_COMMANDS
bool UGameEngine::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	if( FParse::Command( &Cmd,TEXT("REATTACHCOMPONENTS")) || FParse::Command( &Cmd,TEXT("REREGISTERCOMPONENTS")))
	{
		UE_LOGF(LogConsoleResponse, Warning, "Deprectated command! Please use 'Reattach.Components' instead.");
		return true;
	}
	else if( FParse::Command( &Cmd,TEXT("EXIT")) || FParse::Command(&Cmd,TEXT("QUIT")))
	{
		FString CmdName = FParse::Token(Cmd, 0);
		bool Background = false;
		bool Forced = false;

		if (!CmdName.IsEmpty())
		{
			if (!FCString::Stricmp(*CmdName, TEXT("background")))
			{
				Background = true;
			}

#if  !UE_BUILD_SHIPPING
			// in non-shipping let things force an exit on all platforms for automation
			if (!FCString::Stricmp(*CmdName, TEXT("force")))
			{
				Forced = true;
			}
#endif
		}

		if ( Background && FPlatformProperties::SupportsMinimize() )
		{
			return HandleMinimizeCommand( Cmd, Ar );
		}
		else if ( FPlatformProperties::SupportsQuit() || Forced )
		{
			return HandleExitCommand( Cmd, Ar );
		}
		else
		{
			// ignore command on remaining platforms as it will cause a crash
			return true;
		}
	}
	else if( FParse::Command( &Cmd, TEXT("GETMAXTICKRATE") ) )
	{
		return HandleGetMaxTickRateCommand( Cmd, Ar );
	}
	else if( FParse::Command( &Cmd, TEXT("CANCEL") ) )
	{
		return HandleCancelCommand( Cmd, Ar, InWorld );	
	}
	else if ( FParse::Command( &Cmd, TEXT("TOGGLECVAR") ) )
	{
		FString CVarName;
		FParse::Token(Cmd, CVarName, false);

		bool bEnoughParamsSupplied = false;
		IConsoleVariable * CVar = nullptr;

		if (CVarName.Len() > 0)
		{
			CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
		}

		if (CVar)
		{
			// values to toggle between
			FString StringVal1, StringVal2;
			
			if (FParse::Token(Cmd, StringVal1, false))
			{
				if (FParse::Token(Cmd, StringVal2, false))
				{
					bEnoughParamsSupplied = true;
					FString CurrentValue = CVar->GetString();

					FString Command(FString::Printf(TEXT("%s %s"), *CVarName, (CurrentValue == StringVal1) ? *StringVal2 : *StringVal1));
					GEngine->Exec(InWorld, *Command);
				}
			}
		}
		else
		{
			Ar.Log(*FString::Printf(TEXT("TOGGLECVAR: cvar '%s' was not found"), *CVarName));
			bEnoughParamsSupplied = true;	// cannot say anything about the rest of parameters
		}
		
		if (!bEnoughParamsSupplied)
		{
			Ar.Log(TEXT("Usage: TOGGLECVAR CVarName Value1 Value2"));
		}

		return true;
	}
#if !UE_BUILD_SHIPPING
	else if( FParse::Command( &Cmd, TEXT("ApplyUserSettings") ) )
	{
		return HandleApplyUserSettingsCommand( Cmd, Ar );
	}
#endif // !UE_BUILD_SHIPPING
#if WITH_EDITOR
	else if( FParse::Command(&Cmd,TEXT("STARTMOVIECAPTURE")) && GIsEditor )
	{
		IMovieSceneCaptureInterface* CaptureInterface = IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture();
		if (CaptureInterface)
		{
			CaptureInterface->StartCapturing();
			return true;
		}
		else if (SceneViewport.IsValid())
		{
			if (IMovieSceneCaptureModule::Get().CreateMovieSceneCapture(SceneViewport))
			{
				return true;
			}
		}
		return false;
	}
#endif
	else if( InWorld && InWorld->Exec( InWorld, Cmd, Ar ) )
	{
		return true;
	}
	else if( InWorld && InWorld->GetAuthGameMode() && InWorld->GetAuthGameMode()->ProcessConsoleExec(Cmd,Ar,NULL) )
	{
		return true;
	}
	else
	{
#if UE_BUILD_SHIPPING
		// disallow set of actor properties if network game
		if ((FParse::Command( &Cmd, TEXT("SET")) || FParse::Command( &Cmd, TEXT("SETNOPEC"))))
		{
			FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);
			if( Context.PendingNetGame != NULL || InWorld->GetNetMode() != NM_Standalone)
			{
				return true;
			}
			// the effects of this cannot be easily reversed, so prevent the user from playing network games without restarting to avoid potential exploits
			GDisallowNetworkTravel = true;
		}
#endif // UE_BUILD_SHIPPING
		if (UEngine::Exec(InWorld, Cmd, Ar))
		{
			return true;
		}
		else if (UPlatformInterfaceBase::StaticExec(Cmd, Ar))
		{
			return true;
		}
	
		return false;
	}
}
#endif // UE_ALLOW_EXEC_COMMANDS

bool UGameEngine::HandleExitCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	Ar.Log( TEXT("Closing by request") );

	FGameDelegates::Get().GetExitCommandDelegate().Broadcast();

	FPlatformMisc::RequestExit(false, TEXT("UGameEngine::HandleExitCommand"));
	return true;
}

bool UGameEngine::HandleMinimizeCommand( const TCHAR *Cmd, FOutputDevice &Ar )
{
	Ar.Log( TEXT("Minimize by request") );
	FPlatformApplicationMisc::RequestMinimize();

	return true;
}

bool UGameEngine::HandleGetMaxTickRateCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	Ar.Logf( TEXT("%f"), GetMaxTickRate(0,false) );
	return true;
}

bool UGameEngine::HandleCancelCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	CancelPending(GetWorldContextFromWorldChecked(InWorld));
	return true;
}

#if !UE_BUILD_SHIPPING
bool UGameEngine::HandleApplyUserSettingsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GetGameUserSettings()->ApplySettings(false);
	return true;
}
#endif // !UE_BUILD_SHIPPING

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

float UGameEngine::GetMaxTickRate(float DeltaTime, bool bAllowFrameRateSmoothing) const
{
	float MaxTickRate = 0.f;

	if (FPlatformProperties::SupportsWindowedMode() || IsRunningDedicatedServer())
	{
		// This applies for "non-console" platforms...

		UWorld* World = NULL;

		for (int32 WorldIndex = 0; WorldIndex < WorldList.Num(); ++WorldIndex)
		{
			if (WorldList[WorldIndex].WorldType == EWorldType::Game)
			{
				World = WorldList[WorldIndex].World();
				break;
			}
		}

		if( World )
		{
			UNetDriver* NetDriver = World->GetNetDriver();
			// In network games, limit framerate to not saturate bandwidth.
			if( NetDriver && (NetDriver->GetNetMode() == NM_DedicatedServer || (NetDriver->GetNetMode() == NM_ListenServer && NetDriver->bClampListenServerTickRate)))
			{
				// We're a dedicated server, use the LAN or Net tick rate.
				MaxTickRate = FMath::Clamp( NetDriver->GetNetServerMaxTickRate(), 1, 1000 );
			}
			/*else if( NetDriver && NetDriver->ServerConnection )
			{
				if( NetDriver->ServerConnection->CurrentNetSpeed <= 10000 )
				{
					MaxTickRate = FMath::Clamp( MaxTickRate, 10.f, 90.f );
				}
			}*/
		}
	}

	// See if the code in the base class wants to replace this
	float SuperTickRate = Super::GetMaxTickRate(DeltaTime, bAllowFrameRateSmoothing);
	if(SuperTickRate != 0.0)
	{
		MaxTickRate = SuperTickRate;
	}

	return MaxTickRate;
}


void UGameEngine::Tick( float DeltaSeconds, bool bIdleMode )
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EngineTickMisc);
	SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick"));
	SCOPE_CYCLE_COUNTER(STAT_GameEngineTick);
	NETWORK_PROFILER(GNetworkProfiler.TrackFrameBegin());
	
	// -----------------------------------------------------
	// Non-World related stuff
	// -----------------------------------------------------

	if( DeltaSeconds < 0.0f )
	{
#if (UE_BUILD_SHIPPING && WITH_EDITOR)
		// End users don't have access to the secure parts of UDN.  Regardless, they won't
		// need the warning because the game ships with AMD drivers that address the issue.
		UE_LOGF(LogEngine, Fatal,"Negative delta time!");
#else
		// Send developers to the support list thread.
		UE_LOGF(LogEngine, Fatal,"Negative delta time! Please see https://udn.epicgames.com/lists/showpost.php?list=ue3bugs&id=4364");
#endif
	}

	if ((GSlowFrameLoggingThreshold > 0.0f) && (DeltaSeconds > GSlowFrameLoggingThreshold))
	{
		UE_LOGF(LogEngine, Log, "Slow GT frame detected (GT frame %llu, delta time %f s)", GFrameCounter - 1, DeltaSeconds);
	}

	if (IsRunningDedicatedServer())
	{
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - LastTimeLogsFlushed > static_cast<double>(ServerFlushLogInterval))
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(LogFlush);
			GLog->Flush();

			LastTimeLogsFlushed = FPlatformTime::Seconds();
		}
	}
	else if (!IsRunningCommandlet() && FApp::CanEverRender())	// skip in case of commandlets, dedicated servers and headless games
	{
		// Clean up the game viewports that have been closed.
		CleanupGameViewport();
	}

	// If all viewports closed, time to exit - unless we're running headless
	if (GIsClient && (GameViewport == nullptr) && FApp::CanEverRender())
	{
		UE_LOGF(LogEngine, Log,  "All Windows Closed" );
		FPlatformMisc::RequestExit(false, TEXT("UGameEngine::Tick.ViewportClosed"));
		return;
	}

#if !UE_SERVER
	// Media module present?
	static const FName MediaModuleName("Media");
	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
	if (MediaModule != nullptr)
	{
		// Yes. Will a world trigger the MediaFramework tick due to an active Sequencer?
		bool bWorldWillTickMediaFramework = false;
		if (!bIdleMode)
		{
			for (int32 i = 0; i < WorldList.Num(); ++i)
			{
				FWorldContext& Context = WorldList[i];
				if (Context.World() != nullptr && Context.World()->ShouldTick() && Context.World()->IsMovieSceneSequenceTickHandlerBound())
				{
					bWorldWillTickMediaFramework = true;
					break;
				}
			}
		}
		if (!bWorldWillTickMediaFramework)
		{
			// tick media framework if no world would do it later on
			// (so we can normally - no Sequencer active - assume that the media state changes are all done early)
			MediaModule->TickPreEngine();
		}
	}
#endif

	if (!GUseUnifiedTimeBudgetForStreaming)
	{
		// This assumes that UObject::StaticTick only calls ProcessAsyncLoading.
		SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick - StaticTick"));
		StaticTick(DeltaSeconds, !!GAsyncLoadingUseFullTimeLimit, GAsyncLoadingTimeLimit / 1000.f);
	}

	{
		SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick - Analytics"));
		FEngineAnalytics::Tick(DeltaSeconds);
	}

	GConfig->Tick(DeltaSeconds);

	// -----------------------------------------------------
	// Begin ticking worlds
	// -----------------------------------------------------

	bool bIsAnyNonPreviewWorldUnpaused = false;

	FName OriginalGWorldContext = NAME_None;
	for (int32 i=0; i < WorldList.Num(); ++i)
	{
		if (WorldList[i].World() == GWorld)
		{
			OriginalGWorldContext = WorldList[i].ContextHandle;
			break;
		}
	}

	for (int32 WorldIdx = 0; WorldIdx < WorldList.Num(); ++WorldIdx)
	{
		FWorldContext &Context = WorldList[WorldIdx];
		if (Context.World() == NULL || !Context.World()->ShouldTick())
		{
			continue;
		}

		GWorld = Context.World();

		// Tick all travel and Pending NetGames (Seamless, server, client)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_UGameEngine_Tick_TickWorldTravel);
			TickWorldTravel(Context, DeltaSeconds);
		}

		if (!bIdleMode)
		{
			SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick - WorldTick"));

			// Tick the world.
			Context.World()->Tick( LEVELTICK_All, DeltaSeconds );
		}

		if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_UGameEngine_Tick_CheckCaptures);

			// Checking AreAlwaysLoadedLevelsLoaded() is quite expensive. Check that we have captures to update first.
			if (USkyLightComponent::HasSkyCapturesToUpdate() || UReflectionCaptureComponent::HasReflectionCapturesToUpdate())
			{
				// Only update reflection captures in game once all 'always loaded' levels have been loaded
				// This won't work with actual level streaming though
				if (Context.World()->AreAlwaysLoadedLevelsLoaded())
				{
					const bool bInsideTick = true;

					// Update sky light first because it's considered direct lighting, sky diffuse will be visible in reflection capture indirect specular
					USkyLightComponent::UpdateSkyCaptureContents(Context.World());
					UReflectionCaptureComponent::UpdateReflectionCaptureContents(Context.World(), nullptr, false, false, bInsideTick);
				}
			}
		}



		// Issue cause event after first tick to provide a chance for the game to spawn the player and such.
		if( Context.World()->bWorldWasLoadedThisTick )
		{
			Context.World()->bWorldWasLoadedThisTick = false;
			
			const TCHAR* InitialExec = Context.LastURL.GetOption(TEXT("causeevent="),NULL);
			ULocalPlayer* GamePlayer = Context.OwningGameInstance ? Context.OwningGameInstance->GetFirstGamePlayer() : NULL;
			if( InitialExec && GamePlayer )
			{
				UE_LOGF(LogEngine, Log, "Issuing initial cause event passed from URL: %ls", InitialExec);
				GamePlayer->Exec( GamePlayer->GetWorld(), *(FString("CAUSEEVENT ") + InitialExec), *GLog );
			}

			Context.World()->bTriggerPostLoadMap = true;
		}
	
		UpdateTransitionType(Context.World());

		// Block on async loading if requested.
		if (Context.World()->bRequestedBlockOnAsyncLoading)
		{
			BlockTillLevelStreamingCompleted(Context.World());
			Context.World()->bRequestedBlockOnAsyncLoading = false;
		}

		// streamingServer
		if( GIsServer == true && !GUseUnifiedTimeBudgetForStreaming)
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateLevelStreaming);
			Context.World()->UpdateLevelStreaming();
		}

		// See whether any map changes are pending and we requested them to be committed.
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UGameEngine_Tick_ConditionalCommitMapChange);
		ConditionalCommitMapChange(Context);

		if (Context.WorldType != EWorldType::EditorPreview && !Context.World()->IsPaused())
		{
			bIsAnyNonPreviewWorldUnpaused = true;
		}
	}

	// ----------------------------
	//	End per-world ticking
	// ----------------------------
	{
		SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick - TickObjects"));
		FTickableGameObject::TickObjects(nullptr, LEVELTICK_All, false, DeltaSeconds);
	}

	// Restore original GWorld*. This will go away one day.
	if (OriginalGWorldContext != NAME_None)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UGameEngine_Tick_GetWorldContextFromHandleChecked);
		GWorld = GetWorldContextFromHandleChecked(OriginalGWorldContext).World();
	}

#if !UE_SERVER
	if (MediaModule != nullptr)
	{
		MediaModule->TickPostEngine();
	}
#endif

	for (FWorldContext& Context : WorldList)
	{
		if (Context.World())
		{
			Context.World()->StartAsyncSendAllEndOfFrameUpdates();
		}
	}

	// Tick the viewport
	if ( GameViewport != NULL && !bIdleMode )
	{
		SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick - TickViewport"));
		SCOPE_CYCLE_COUNTER(STAT_GameViewportTick);
		GameViewport->Tick(DeltaSeconds);
	}

	FMoviePlayerProxy::BlockingForceFinished();
	if (FPlatformProperties::SupportsWindowedMode())
	{
		// Hide the splashscreen and show the game window
		static bool bFirstTime = true;
		if ( bFirstTime )
		{
			bFirstTime = false;
			FPlatformSplash::Hide();
			if ( GameViewportWindow.IsValid() )
			{
				// Don't show window in off-screen rendering mode as it doesn't render to screen
				if (!FSlateApplication::Get().IsRenderingOffScreen())
				{
					GameViewportWindow.Pin()->ShowWindow();
				}
				FSlateApplication::Get().RegisterGameViewport( GameViewportWidget.ToSharedRef() );
			}
		}
	}

	const bool bRenderingSuspended = IsRenderingSuspended();
	const bool bIsRenderingScene = !bIdleMode && !IsRunningDedicatedServer() && !IsRunningCommandlet() && FEmbeddedCommunication::IsAwakeForRendering();

	if (bIsRenderingScene && !bRenderingSuspended)
	{
		SCOPE_CYCLE_COUNTER(STAT_RedrawViewports);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ViewportMisc);
		RedrawViewports();
	}

	for (FWorldContext& Context : WorldList)
	{
		if (Context.World())
		{
			Context.World()->FinishAsyncSendAllEndOfFrameUpdates();
		}
	}

	if (GUseUnifiedTimeBudgetForStreaming)
	{
		HandleUnifiedStreaming(DeltaSeconds);
	}

	if (bIsRenderingScene)
	{
		if (!bRenderingSuspended)
		{
			// CPU/GPU synchronization is achieved by calling EndDrawingViewport. If no viewports are updated (because the game is hidden),
			// we need to explicitly wait for the GPU to finish here, to prevent the CPU from submitting work faster than the GPU
			// can process it, which leads to an unbounded accumulation of resources.
			if (GMinimizedSyncDrawToGPU && AreAllWindowsHidden())
			{
				ENQUEUE_RENDER_COMMAND(SubmitAndBlockUntilGPUIdle_MinimizedRealtime)([](FRHICommandListImmediate& RHICmdList)
				{
					RHICmdList.SubmitAndBlockUntilGPUIdle();
				});
			}
			else
			{
				// Some tasks can only be done once we finish all scenes/viewports
				GetRendererModule().PostRenderAllViewports();
			}
		}

		if (!GUseUnifiedTimeBudgetForStreaming && !GIsServer && WorldList.Num() > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateLevelStreaming);
			for (FWorldContext& Context : WorldList)
			{
				if (Context.World())
				{
					Context.World()->UpdateLevelStreaming();
				}
			}
		}
	}

	if( GIsClient )
	{
		// Update resource streaming after viewports have had a chance to update view information. Normal update.
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UGameEngine_Tick_IStreamingManager);
		IStreamingManager::Get().Tick( DeltaSeconds );
	}

	// Update Audio. This needs to occur after rendering as the rendering code updates the listener position.
	FAudioDeviceManager* GameAudioDeviceManager = GEngine->GetAudioDeviceManager();
	if (GameAudioDeviceManager)
	{
		SCOPE_TIME_GUARD(TEXT("UGameEngine::Tick - Update Audio"));
		GameAudioDeviceManager->UpdateActiveAudioDevices(bIsAnyNonPreviewWorldUnpaused);
	}

	ENQUEUE_RENDER_COMMAND(TickPoolElements, LoopTick)([](FRHICommandListImmediate& RHICmdList)
	{			
		GRenderTargetPool.TickPoolElements();
		FRDGBuilder::TickPoolElements();
		ICustomResourcePool::TickPoolElements(RHICmdList);
	});

	if (bRenderingSuspended)
	{
		GetRendererModule().PerFrameCleanupIfSkipRenderer();
	}

#if WITH_EDITOR
	BroadcastPostEditorTick(DeltaSeconds);

	// Tick the asset registry
	FAssetRegistryModule::TickAssetRegistry(DeltaSeconds);
#endif
}


void UGameEngine::ProcessToggleFreezeCommand( UWorld* InWorld )
{
	if (GameViewport)
	{
		GameViewport->Viewport->ProcessToggleFreezeCommand();
	}
}


void UGameEngine::ProcessToggleFreezeStreamingCommand(UWorld* InWorld)
{
	// if not already frozen, then flush async loading before we freeze so that we don't mess up any in-process streaming
	if (!InWorld->bIsLevelStreamingFrozen)
	{
		FlushAsyncLoading();
	}

	// toggle the frozen state
	InWorld->bIsLevelStreamingFrozen = !InWorld->bIsLevelStreamingFrozen;
}


UWorld* UGameEngine::GetGameWorld()
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		const FWorldContext& Context = *It;
		// Explicitly not checking for PIE worlds here, this should only 
		// be called outside of editor (and thus is in UGameEngine
		if (Context.WorldType == EWorldType::Game && Context.World())
		{
			return Context.World();
		}
	}

	return NULL;
}

void UGameEngine::HandleNetworkFailure_NotifyGameInstance(UWorld* World, UNetDriver* NetDriver, ENetworkFailure::Type FailureType)
{
	if (GameInstance != nullptr)
	{
		bool bIsServer = true;
		if (NetDriver != nullptr)
		{
			bIsServer = NetDriver->GetNetMode() != NM_Client;
		}
		GameInstance->HandleNetworkError(FailureType, bIsServer);
	}
}

void UGameEngine::HandleTravelFailure_NotifyGameInstance(UWorld* World, ETravelFailure::Type FailureType)
{
	if (GameInstance != nullptr)
	{
		GameInstance->HandleTravelError(FailureType);
	}
}

void UGameEngine::HandleBrowseToDefaultMapFailure(FWorldContext& Context, const FString& TextURL, const FString& Error)
{
	Super::HandleBrowseToDefaultMapFailure(Context, TextURL, Error);
	FPlatformMisc::RequestExit(false, TEXT("UGameEngine::HandleBrowseToDefaultMapFailure"));
}

