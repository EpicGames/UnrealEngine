// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/AsyncActionHandleSaveGame.h"
#include "Kismet/GameplayStatics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncActionHandleSaveGame)

UAsyncActionHandleSaveGame* UAsyncActionHandleSaveGame::AsyncSaveGameToSlot(UObject* WorldContextObject, USaveGame* SaveGameObject, const FString& SlotName, const int32 UserIndex)
{
	UAsyncActionHandleSaveGame* Action = NewObject<UAsyncActionHandleSaveGame>();
	Action->Operation = ESaveGameOperation::Save;
	Action->SaveGameObject = SaveGameObject;
	Action->SlotName = SlotName;
	Action->UserIndex = UserIndex;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

UAsyncActionHandleSaveGame* UAsyncActionHandleSaveGame::AsyncLoadGameFromSlot(UObject* WorldContextObject, const FString& SlotName, const int32 UserIndex)
{
	UAsyncActionHandleSaveGame* Action = NewObject<UAsyncActionHandleSaveGame>();
	Action->Operation = ESaveGameOperation::Load;
	Action->SaveGameObject = nullptr;
	Action->SlotName = SlotName;
	Action->UserIndex = UserIndex;
	Action->RegisterWithGameInstance(WorldContextObject);

	return Action;
}

void UAsyncActionHandleSaveGame::Activate()
{
	switch (Operation)
	{
	case ESaveGameOperation::Save:
		UGameplayStatics::AsyncSaveGameToSlot(SaveGameObject, SlotName, UserIndex, FAsyncSaveGameToSlotDelegate::CreateUObject(this, &UAsyncActionHandleSaveGame::HandleAsyncSave));
		return;
	case ESaveGameOperation::Load:
		UGameplayStatics::AsyncLoadGameFromSlot(SlotName, UserIndex, FAsyncLoadGameFromSlotDelegate::CreateUObject(this, &UAsyncActionHandleSaveGame::HandleAsyncLoad));
		return;
	}

	UE_LOGF(LogScript, Error, "UAsyncActionHandleSaveGame Created with invalid operation!");

	ExecuteCompleted(false);
}

void UAsyncActionHandleSaveGame::HandleAsyncSave(const FString& InSlotName, const int32 InUserIndex, bool bSuccess)
{
	ExecuteCompleted(bSuccess);
}

void UAsyncActionHandleSaveGame::HandleAsyncLoad(const FString& InSlotName, const int32 InUserIndex, USaveGame* LoadedSave)
{
	SaveGameObject = LoadedSave;
	ExecuteCompleted(SaveGameObject != nullptr);
}

void UAsyncActionHandleSaveGame::ExecuteCompleted(bool bSuccess)
{
	Completed.Broadcast(SaveGameObject, bSuccess);

	SaveGameObject = nullptr;
	SetReadyToDestroy();
}

