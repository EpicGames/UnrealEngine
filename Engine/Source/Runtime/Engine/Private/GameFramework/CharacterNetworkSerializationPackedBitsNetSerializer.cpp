// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/CharacterNetworkSerializationPackedBitsNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CharacterNetworkSerializationPackedBitsNetSerializer)


#include "GameFramework/CharacterNetworkSerializationPackedBitsNetSerializer.h"
#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "GameFramework/CharacterMovementReplication.h"
#include "Iris/Serialization/NetReferenceCollector.h"
#include "Iris/Serialization/NetSerializerArrayStorage.h"
#include "Iris/Serialization/IrisPackageMapExportUtil.h"
#include "Containers/ArrayView.h"

namespace UE::Net::Private
{
	static constexpr inline uint32 CalculateRequiredWordCount(uint32 NumBits) { return (NumBits + NumBitsPerDWORD - 1U) / NumBitsPerDWORD; }
}

namespace UE::Net
{

struct FCharacterNetworkSerializationPackedBitsNetSerializerQuantizedType
{
	typedef uint32 WordType;
	static constexpr uint32 MaxInlinedObjectRefs = 4;
	static constexpr uint32 InlinedWordCount = Private::CalculateRequiredWordCount(CHARACTER_SERIALIZATION_PACKEDBITS_RESERVED_SIZE);

	typedef FNetSerializerArrayStorage<WordType, AllocationPolicies::TInlinedElementAllocationPolicy<InlinedWordCount>> FDataBitsStorage;

	FIrisPackageMapExportsQuantizedType QuantizedExports;
	FDataBitsStorage DataBitsStorage;
	uint32 NumDataBits;
};

}

template <> struct TIsPODType<UE::Net::FCharacterNetworkSerializationPackedBitsNetSerializerQuantizedType> { enum { Value = true }; };

namespace UE::Net
{

struct FCharacterNetworkSerializationPackedBitsNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	typedef uint32 WordType;

	typedef FCharacterNetworkSerializationPackedBits SourceType;
	typedef FCharacterNetworkSerializationPackedBitsNetSerializerQuantizedType QuantizedType;
	typedef struct FCharacterNetworkSerializationPackedBitsNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
	};

	static void FreeDynamicState(FNetSerializationContext&, QuantizedType& Value);

	static FCharacterNetworkSerializationPackedBitsNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	inline static IConsoleVariable* CVarNetPackedMovementMaxBits = nullptr;
};

UE_NET_IMPLEMENT_SERIALIZER(FCharacterNetworkSerializationPackedBitsNetSerializer);

const FCharacterNetworkSerializationPackedBitsNetSerializer::ConfigType FCharacterNetworkSerializationPackedBitsNetSerializer::DefaultConfig;
FCharacterNetworkSerializationPackedBitsNetSerializer::FNetSerializerRegistryDelegates FCharacterNetworkSerializationPackedBitsNetSerializer::NetSerializerRegistryDelegates;

void FCharacterNetworkSerializationPackedBitsNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	// For now we ignore this in default state hash due to complications with asymmetrically serialized state.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Serialize captured references and exports
	FIrisPackageMapExportsUtil::Serialize(Context, Value.QuantizedExports);

	// Write data bits
	const uint32 NumDataBits = Value.NumDataBits;
	if (Writer->WriteBool(NumDataBits > 0))
	{
		UE::Net::WritePackedUint32(Writer, NumDataBits);
		Writer->WriteBitStream(Value.DataBitsStorage.GetData(), 0, NumDataBits);
	}
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::FreeDynamicState(FNetSerializationContext& Context, QuantizedType& Value)
{
	// Free quantized state for captured reference and exports
	FIrisPackageMapExportsUtil::FreeDynamicState(Context, Value.QuantizedExports);

	Value.DataBitsStorage.Free(Context);
	Value.NumDataBits = 0;
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	// For consistency, we should never get here. For now we ignore this in default state hash due to complications with asymmetrically serialized state.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Deserialize captured references and exports
	FIrisPackageMapExportsUtil::Deserialize(Context, TargetValue.QuantizedExports);

	const bool bHasDataBits = Reader->ReadBool();
	if (bHasDataBits)
	{
		const uint32 NumDataBits = UE::Net::ReadPackedUint32(Reader);

		const uint32 MaxNumDataBits = CVarNetPackedMovementMaxBits ? static_cast<uint32>(CVarNetPackedMovementMaxBits->GetInt()) : Config->MaxAllowedDataBits;
		if (NumDataBits > MaxNumDataBits)
		{
			Context.SetError(GNetError_ArraySizeTooLarge);
			UE_LOGF(LogNetPlayerMovement, Error, "FCharacterNetworkSerializationPackedBits::Deserialize: Invalidating move due to NumBits (%u) exceeding allowable limit (%u). See NetPackedMovementMaxBits.", NumDataBits, MaxNumDataBits);
			ensureMsgf(false, TEXT("Invalidating move due to NumBits exceeding allowable limit"));
			return;
		}
		
		const uint32 RequiredWordCount = Private::CalculateRequiredWordCount(NumDataBits);
		TargetValue.DataBitsStorage.AdjustSize(Context, RequiredWordCount);

		Reader->ReadBitStream(TargetValue.DataBitsStorage.GetData(), NumDataBits);
		TargetValue.NumDataBits = NumDataBits;
	}
	else
	{
		TargetValue.DataBitsStorage.Free(Context);
		TargetValue.NumDataBits = 0U;
	}
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	// Quantize captured references and exports
	FIrisPackageMapExportsUtil::Quantize(Context, SourceValue.PackageMapExports, MakeArrayView(SourceValue.NetTokensPendingExport), TargetValue.QuantizedExports);

	uint32 NumDataBits = SourceValue.DataBits.Num();

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const uint32 MaxNumDataBits = CVarNetPackedMovementMaxBits ? static_cast<uint32>(CVarNetPackedMovementMaxBits->GetInt()) : Config->MaxAllowedDataBits;

	if (NumDataBits > MaxNumDataBits)
	{
		// This is just to avoid disconnect and instead warn and invalidate the data on the sending side.
		UE_LOGF(LogNetPlayerMovement, Error, "FCharacterNetworkSerializationPackedBits::Quantize: Invalidating move due to NumBits (%u) exceeding allowable limit (%u). See NetPackedMovementMaxBits.", NumDataBits, MaxNumDataBits);
		NumDataBits = 0U;
		ensureMsgf(false, TEXT("Invalidating move due to NumBits exceeding allowable limit"));
	}

	TargetValue.DataBitsStorage.AdjustSize(Context, Private::CalculateRequiredWordCount(NumDataBits));
	if (NumDataBits > 0)
	{
		FMemory::Memcpy(TargetValue.DataBitsStorage.GetData(), SourceValue.DataBits.GetData(), (NumDataBits + 7U) / 8U);
	}
	TargetValue.NumDataBits = NumDataBits;
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	// Dequantize captured references and exports and inject into target
	FIrisPackageMapExportsUtil::Dequantize(Context, Source.QuantizedExports, Target.PackageMapExports);

	// DataBits
	Target.DataBits.SetNumUninitialized(Source.NumDataBits);
	Target.DataBits.SetRangeFromRange(0, Source.NumDataBits, Source.DataBitsStorage.GetData());
}

bool FCharacterNetworkSerializationPackedBitsNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.NumDataBits != Value1.NumDataBits)
		{
			return false;
		}

		// Compare references and exports
		if (!FIrisPackageMapExportsUtil::IsEqual(Context, Value0.QuantizedExports, Value1.QuantizedExports))
		{
			return false;
		}

		const uint32 RequiredWords = Private::CalculateRequiredWordCount(Value0.NumDataBits);
		if (RequiredWords > 0 && FMemory::Memcmp(Value0.DataBitsStorage.GetData(), Value1.DataBitsStorage.GetData(), sizeof(WordType) * RequiredWords) != 0)
		{
			return false;
		}
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);
		return Value0.DataBits == Value1.DataBits;
	}

	return true;
}

bool FCharacterNetworkSerializationPackedBitsNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);

	const uint32 MaxNumDataBits = CVarNetPackedMovementMaxBits ? static_cast<uint32>(CVarNetPackedMovementMaxBits->GetInt()) : Config->MaxAllowedDataBits;

	if (!FIrisPackageMapExportsUtil::Validate(Context, SourceValue.QuantizedExports))
	{
		return false;
	}
	return true;
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	FIrisPackageMapExportsUtil::CloneDynamicState(Context, TargetValue.QuantizedExports, SourceValue.QuantizedExports);

	TargetValue.DataBitsStorage.Clone(Context, SourceValue.DataBitsStorage);
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	FreeDynamicState(Context, Value);
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<UE::Net::FNetReferenceCollector*>(Args.Collector);

	FIrisPackageMapExportsUtil::CollectNetReferences(Context, Value.QuantizedExports, Args.ChangeMaskInfo, Collector);
}

static const FName PropertyNetSerializerRegistry_NAME_CharacterMoveResponsePackedBits("CharacterMoveResponsePackedBits");
static const FName PropertyNetSerializerRegistry_NAME_CharacterServerMovePackedBitsPackedBits("CharacterServerMovePackedBits");
static const FName PropertyNetSerializerRegistry_NAME_CharacterNetworkSerializationPackedBits("CharacterNetworkSerializationPackedBits");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterMoveResponsePackedBits, FCharacterNetworkSerializationPackedBitsNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterServerMovePackedBitsPackedBits, FCharacterNetworkSerializationPackedBitsNetSerializer);
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterNetworkSerializationPackedBits, FCharacterNetworkSerializationPackedBitsNetSerializer);

FCharacterNetworkSerializationPackedBitsNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterMoveResponsePackedBits);
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterServerMovePackedBitsPackedBits);
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterNetworkSerializationPackedBits);
}

void FCharacterNetworkSerializationPackedBitsNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	FCharacterNetworkSerializationPackedBitsNetSerializer::CVarNetPackedMovementMaxBits = IConsoleManager::Get().FindConsoleVariable(TEXT("p.NetPackedMovementMaxBits"), false);
#if WITH_SERVER_CODE
	ensureMsgf(CVarNetPackedMovementMaxBits != nullptr, TEXT("%s"), TEXT("Unable to find cvar p.NetPackedMovementMaxBits"));
#endif

	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterMoveResponsePackedBits);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterServerMovePackedBitsPackedBits);
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_CharacterNetworkSerializationPackedBits);
}

}

