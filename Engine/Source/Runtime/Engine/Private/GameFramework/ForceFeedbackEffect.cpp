// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/ForceFeedbackEffect.h"
#include "GameFramework/ForceFeedbackParameters.h"
#include "Misc/App.h"
#include "GameFramework/InputDeviceProperties.h"
#include "HAL/IConsoleManager.h"	// For FAutoConsoleVariableRef

#include UE_INLINE_GENERATED_CPP_BY_NAME(ForceFeedbackEffect)

namespace UE::Input::Private
{
	// This is a flag that will force the re-evaluation of a force feedback effect's duration at runtime.
	// This fixes an issue where if you specify an input device type override and the duration of that override
	// curve was longer then the default one, it wouldn't play the whole effect.
	// see UE-178719
	static bool bShouldAlwaysEvaluateForceFeedbackDuration = true;
	static FAutoConsoleVariableRef CVarShouldAlwaysEvaluateForceFeedbackDuration(TEXT("Input.ShouldAlwaysEvaluateForceFeedbackDuration"),
		bShouldAlwaysEvaluateForceFeedbackDuration,
		TEXT("Should the duration of a force feedback effect be evaluated every time it is called?"));
}


UForceFeedbackEffect::UForceFeedbackEffect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Make sure that by default the force feedback effect has an entry
	FForceFeedbackChannelDetails ChannelDetail;
	ChannelDetails.Add(ChannelDetail);
}

FForceFeedbackEffectOverridenChannelDetails::FForceFeedbackEffectOverridenChannelDetails()
{
	// Add one default channel details by default
	ChannelDetails.AddDefaulted(1);
}

#if WITH_EDITOR
void UForceFeedbackEffect::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// After any edit (really we only care about the curve, but easier this way) update the cached duration value
	GetDuration();
	GetTotalDevicePropertyDuration();
}
#endif

float UForceFeedbackEffect::GetDuration()
{
	// Always recalc the duration when in the editor as it could change
	if (GIsEditor || ( Duration < UE_SMALL_NUMBER ) || UE::Input::Private::bShouldAlwaysEvaluateForceFeedbackDuration)
	{
		Duration = 0.f;

		// Just use the primary platform user when calculating duration, this won't be affected by which player the effect is for
		const TArray<FForceFeedbackChannelDetails>& CurrentDetails = GetCurrentChannelDetails(IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser());

		float MinTime, MaxTime;
		for (int32 Index = 0; Index < CurrentDetails.Num(); ++Index)
		{
			CurrentDetails[Index].Curve.GetRichCurveConst()->GetTimeRange(MinTime, MaxTime);

			if (MaxTime > Duration)
			{
				Duration = MaxTime;
			}
		}
	}

	return Duration;
}

float UForceFeedbackEffect::GetTotalDevicePropertyDuration()
{
	float LongestDuration = 0.0f;

	// Check the device properties for any longer durations
	for (const TObjectPtr<UInputDeviceProperty>& DeviceProperty : DeviceProperties)
	{
		if (DeviceProperty)
		{
			const float PropertyDuration = DeviceProperty->RecalculateDuration();
			if (PropertyDuration > LongestDuration)
			{
				LongestDuration = PropertyDuration;
			}
		}
	}

	return LongestDuration;
}

void UForceFeedbackEffect::GetValues(const float EvalTime, FForceFeedbackValues& Values, const FPlatformUserId PlatformUser, const float ValueMultiplier) const
{
	const TArray<FForceFeedbackChannelDetails>& CurrentDetails = GetCurrentChannelDetails(PlatformUser);

	for (int32 Index = 0; Index < CurrentDetails.Num(); ++Index)
	{
		const FForceFeedbackChannelDetails& Details = CurrentDetails[Index];
		const float Value = Details.Curve.GetRichCurveConst()->Eval(EvalTime) * ValueMultiplier;

		if (Details.bAffectsLeftLarge)
		{
			Values.LeftLarge = FMath::Clamp(Value, Values.LeftLarge, 1.f);
		}
		if (Details.bAffectsLeftSmall)
		{
			Values.LeftSmall = FMath::Clamp(Value, Values.LeftSmall, 1.f);
		}
		if (Details.bAffectsRightLarge)
		{
			Values.RightLarge = FMath::Clamp(Value, Values.RightLarge, 1.f);
		}
		if (Details.bAffectsRightSmall)
		{
			Values.RightSmall = FMath::Clamp(Value, Values.RightSmall, 1.f);
		}
	}
}

const TArray<FForceFeedbackChannelDetails>& UForceFeedbackEffect::GetCurrentChannelDetails(const FPlatformUserId PlatformUser) const
{	
	if (const UInputDeviceSubsystem* SubSystem = UInputDeviceSubsystem::Get())
	{
		FHardwareDeviceIdentifier Hardware = SubSystem->GetMostRecentlyUsedHardwareDevice(PlatformUser);
		// Check if there are any per-input device overrides available
		if (const FForceFeedbackEffectOverridenChannelDetails* Details = PerDeviceOverrides.Find(Hardware.HardwareDeviceIdentifier))
		{
			return Details->ChannelDetails;
		}
	}

	return ChannelDetails;
}

FActiveForceFeedbackEffect::~FActiveForceFeedbackEffect()
{
	ResetDeviceProperties();
}

void FActiveForceFeedbackEffect::GetValues(FForceFeedbackValues& Values) const
{
	if (ForceFeedbackEffect)
	{
		const float Duration = ForceFeedbackEffect->GetDuration();
		const float EvalTime = PlayTime - Duration * FMath::FloorToFloat(PlayTime / Duration);
		ForceFeedbackEffect->GetValues(EvalTime, Values, PlatformUser);
	}
	else
	{
		Values = FForceFeedbackValues();
	}
}

bool FActiveForceFeedbackEffect::Update(const float DeltaTime, FForceFeedbackValues& Values)
{
	if (ForceFeedbackEffect == nullptr)
	{
		return false;
	}

	const float EffectDuration = ForceFeedbackEffect->GetDuration();

	// If this is the first time that we are playing the effect
	if (!bActivatedDeviceProperties)
	{
		ActivateDeviceProperties();
	}

	PlayTime += (Parameters.bIgnoreTimeDilation ? FApp::GetDeltaTime() : DeltaTime);

	// If the play time is longer then the force feedback effect curve's last key value, 
	// or if there are still device properties that need to be evaluated
	if (PlayTime > EffectDuration && (!Parameters.bLooping || (EffectDuration == 0.0f)))
	{
		return false;
	}
	// Update the effect values if we can. Always get the values for a looping effect.
	if (PlayTime <= EffectDuration || Parameters.bLooping)
	{
		GetValues(Values);
	}

	return true;
}

void FActiveForceFeedbackEffect::ActivateDeviceProperties()
{
	if (ensure(ForceFeedbackEffect))
	{
		if (UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get())
		{
			for (const TObjectPtr<UInputDeviceProperty>& DeviceProp : ForceFeedbackEffect->DeviceProperties)
			{
				if (DeviceProp)
				{
					FActivateDevicePropertyParams Params = {};
					Params.bIgnoreTimeDilation = Parameters.bIgnoreTimeDilation;
					Params.UserId = PlatformUser;
					Params.bPlayWhilePaused = Parameters.bPlayWhilePaused;
					Params.bLooping = Parameters.bLooping;
					ActiveDeviceProperties.Emplace(System->ActivateDeviceProperty(DeviceProp, Params));
				}
			}
		}

		bActivatedDeviceProperties = true;
	}	
}

void FActiveForceFeedbackEffect::ResetDeviceProperties()
{
	if (!ActiveDeviceProperties.IsEmpty())
	{
		if (UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get())
		{
			System->RemoveDevicePropertyHandles(ActiveDeviceProperties);
			ActiveDeviceProperties.Empty();
		}
	}
}
