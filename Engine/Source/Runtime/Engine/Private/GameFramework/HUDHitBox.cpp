// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/HUDHitBox.h"
#include "Engine/Engine.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"

FHUDHitBox::FHUDHitBox(FVector2D InCoords, FVector2D InSize, const FName& InName, bool bInConsumesInput, int32 InPriority)
	: Coords(InCoords)
	, Size(InSize)
	, Name(InName)
	, bConsumesInput(bInConsumesInput)
	, Priority(InPriority)
{
}

bool FHUDHitBox::Contains(FVector2D InCoords) const
{
	bool bResult = false;
	if ((InCoords.X >= Coords.X) && (InCoords.X <= (Coords.X + Size.X)))
	{
		if ((InCoords.Y >= Coords.Y) && (InCoords.Y <= (Coords.Y + Size.Y)))
		{
			bResult = true;
		}
	}
	return bResult;
}

void FHUDHitBox::Draw(FCanvas* InCanvas, const FLinearColor& InColor) const
{
	FCanvasBoxItem	BoxItem(Coords, Size);
	BoxItem.SetColor(InColor);
	InCanvas->DrawItem(BoxItem);
	FCanvasTextItem	TextItem(Coords, FText::FromName(Name), GEngine->GetSmallFont(), InColor);
	InCanvas->DrawItem(TextItem);
}
