// Copyright Epic Games, Inc. All Rights Reserved.
//

#include "AudioDevice.h"
#include "Engine/Engine.h"
#include "GenericPlatform/IInputInterface.h"
#include "Haptics/HapticFeedbackEffect_Base.h"
#include "Haptics/HapticFeedbackEffect_Curve.h"
#include "Haptics/HapticFeedbackEffect_SoundWave.h"
#include "Haptics/HapticFeedbackEffect_Buffer.h"
DEFINE_LOG_CATEGORY_STATIC(LogHaptics, Display, All);

bool FActiveHapticFeedbackEffect::Update(const float DeltaTime, FHapticFeedbackValues& Values)
{
	if (HapticEffect == nullptr)
	{
		return false;
	}

	const float Duration = HapticEffect->GetDuration();
	PlayTime += DeltaTime;

	if ((PlayTime > Duration) || (Duration == 0.f))
	{
		return false;
	}

	HapticBuffer.RawData = nullptr;
	Values.HapticBuffer = &HapticBuffer;
	HapticEffect->GetValues(PlayTime, Values);
	// Don't return a HapticBuffer if the effect didn't fill in RawData.
	// Previously this buffer was owned by the HapticEffect itself, but that prevents
	// playing the same effect on multiple controllers simultaneously.
	if (HapticBuffer.RawData == nullptr)
	{
		Values.HapticBuffer = nullptr;
	}
	Values.Amplitude *= Scale;
	if (Values.HapticBuffer)
	{
		Values.HapticBuffer->ScaleFactor = Scale;
		if (Values.HapticBuffer->bFinishedPlaying)
		{
			Values.HapticBuffer = nullptr;
			return false;
		}
	}
	
	return true;
}

//==========================================================================
// UHapticFeedbackEffect_Base
//==========================================================================

UHapticFeedbackEffect_Base::UHapticFeedbackEffect_Base(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

void UHapticFeedbackEffect_Base::GetValues(const float EvalTime, FHapticFeedbackValues& Values)
{
}

float UHapticFeedbackEffect_Base::GetDuration() const
{
	return 0.f;
}

//==========================================================================
// UHapticFeedbackEffect_Curve
//==========================================================================

UHapticFeedbackEffect_Curve::UHapticFeedbackEffect_Curve(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UHapticFeedbackEffect_Curve::GetValues(const float EvalTime, FHapticFeedbackValues& Values)
{
	Values.Amplitude = HapticDetails.Amplitude.GetRichCurveConst()->Eval(EvalTime);
	Values.Frequency = HapticDetails.Frequency.GetRichCurveConst()->Eval(EvalTime);
}

float UHapticFeedbackEffect_Curve::GetDuration() const
{
	float AmplitudeMinTime, AmplitudeMaxTime;
	float FrequencyMinTime, FrequencyMaxTime;

	HapticDetails.Amplitude.GetRichCurveConst()->GetTimeRange(AmplitudeMinTime, AmplitudeMaxTime);
	HapticDetails.Frequency.GetRichCurveConst()->GetTimeRange(FrequencyMinTime, FrequencyMaxTime);

	return FMath::Max(AmplitudeMaxTime, FrequencyMaxTime);
}

//==========================================================================
// UHapticFeedbackEffect_Buffer
//==========================================================================

UHapticFeedbackEffect_Buffer::UHapticFeedbackEffect_Buffer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHapticFeedbackEffect_Buffer::~UHapticFeedbackEffect_Buffer()
{
}


void UHapticFeedbackEffect_Buffer::Initialize(FHapticFeedbackBuffer& HapticBuffer)
{
	HapticBuffer.CurrentPtr = 0;
	HapticBuffer.SamplesSent = 0;
	HapticBuffer.bFinishedPlaying = false;
	HapticBuffer.RawData = nullptr;
	HapticBuffer.CurrentSampleIndex[0] = 0;
	HapticBuffer.CurrentSampleIndex[1] = 0;
}

void UHapticFeedbackEffect_Buffer::GetValues(const float EvalTime, FHapticFeedbackValues& Values)
{
	int ampidx = EvalTime * SampleRate;

	Values.Frequency = 1.0;
	Values.Amplitude = ampidx < Amplitudes.Num() ? (float)Amplitudes[ampidx] / 255.f : 0.f;
}

float UHapticFeedbackEffect_Buffer::GetDuration() const
{
	return (float)Amplitudes.Num() / SampleRate;
}

//==========================================================================
// UHapticFeedbackEffect_SoundWave
//==========================================================================

UHapticFeedbackEffect_SoundWave::UHapticFeedbackEffect_SoundWave(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bPrepared = false;
}

UHapticFeedbackEffect_SoundWave::~UHapticFeedbackEffect_SoundWave()
{
	RawData.Empty();
}

void UHapticFeedbackEffect_SoundWave::Initialize(FHapticFeedbackBuffer& HapticBuffer)
{
	if (!bPrepared)
	{
		PrepareSoundWaveBuffer();
	}
	HapticBuffer.BufferLength = RawData.Num();
	HapticBuffer.CurrentPtr = 0;
	HapticBuffer.SamplesSent = 0;
	HapticBuffer.bFinishedPlaying = false;
	HapticBuffer.SamplingRate = SoundWave->GetSampleRateForCurrentPlatform();
	HapticBuffer.bUseStereo = bUseStereo;
	HapticBuffer.CurrentSampleIndex[0] = 0;
	HapticBuffer.CurrentSampleIndex[1] = 0;
}

void UHapticFeedbackEffect_SoundWave::GetValues(const float EvalTime, FHapticFeedbackValues& Values)
{
	int ampidx = EvalTime * RawData.Num() / SoundWave->GetDuration();
	Values.Frequency = 1.0;
	Values.Amplitude = ampidx < RawData.Num() ? (float)RawData[ampidx] / 255.f : 0.f;
	Values.HapticBuffer->RawData = RawData.GetData();
}

float UHapticFeedbackEffect_SoundWave::GetDuration() const
{
	return SoundWave ? SoundWave->GetDuration() : 0.f;
}

void UHapticFeedbackEffect_SoundWave::PrepareSoundWaveBuffer()
{
	FAudioDeviceHandle AD = GEngine->GetMainAudioDevice();
	if (!AD || !SoundWave)
	{
		return;
	}

	if (SoundWave->LoadingBehavior != ESoundWaveLoadingBehavior::ForceInline)
	{
		UE_LOGF(LogHaptics, Error, "The LoadingBehavior of a SoundWave needs to be 'ESoundWaveLoadingBehavior::ForceInline' for use in Haptic Feedback. ('%ls' requires a change)", *GetNameSafe(SoundWave));
		return;
	}

	AD->Precache(SoundWave, true, false, true);
	// Remove call to InitAudioResource because AD->Precache calls the InitAudioResource code and calling it twice causes issues with Bulk data loading. 
	uint8* PCMData = SoundWave->RawPCMData;
	int32 RawPCMDataSize = SoundWave->RawPCMDataSize;
	check((PCMData != nullptr) || (RawPCMDataSize == 0));
	if (bUseStereo)
	{
		PrepareSoundWaveStereoBuffer(PCMData, RawPCMDataSize);
	}
	else
	{
		PrepareSoundWaveMonoBuffer(PCMData, RawPCMDataSize);
	}
	bPrepared = true;
}

void UHapticFeedbackEffect_SoundWave::PrepareSoundWaveMonoBuffer(uint8* PCMData, int32 RawPCMDataSize)
{
	// Some platforms may need to resample the PCM data.  Such resampling should be performed at the platform specific plugin level
	int32 NumChannels = SoundWave->NumChannels;
	if (NumChannels > 1)
	{
		UE_LOGF(LogHaptics, Warning, "%ls used for mono vibration has more than 1 channel. Only the first channel will be used.", *SoundWave->GetPathName());
		check(RawPCMDataSize % (sizeof(int16) * NumChannels) == 0);
		int32 NumSamples = RawPCMDataSize / sizeof(int16) / NumChannels;
		RawData.AddUninitialized(NumSamples * sizeof(int16));
		int16* SourceData = reinterpret_cast<int16*>(PCMData);
		int16* DestData = reinterpret_cast<int16*>(RawData.GetData());
		for (int32 i = 0; i < NumSamples; i++)
		{
			DestData[i] = SourceData[i * NumChannels];
		}
	}
	else
	{
		RawData.Append(PCMData, RawPCMDataSize);
	}
}

void UHapticFeedbackEffect_SoundWave::PrepareSoundWaveStereoBuffer(uint8* PCMData, int32 RawPCMDataSize)
{
	// Some platforms may need to resample the PCM data.  Such resampling should be performed at the platform specific plugin level
	int32 NumChannels = SoundWave->NumChannels;
	if (NumChannels < 2)
	{
		constexpr static int StereoChannels = 2;
		UE_LOGF(LogHaptics, Warning, "%ls used for stereo vibration has only 1 channel. The first channel will be copied into the second channel.", *SoundWave->GetPathName());
		check(RawPCMDataSize % (sizeof(int16)) == 0);
		int32 NumSamples = RawPCMDataSize / sizeof(int16);
		RawData.AddUninitialized(NumSamples * StereoChannels * sizeof(int16));
		int16* SourceData = reinterpret_cast<int16*>(PCMData);
		int16* DestData = reinterpret_cast<int16*>(RawData.GetData());

		for (int32 i = 0; i < NumSamples; i++)
		{
			DestData[i * StereoChannels] = SourceData[i];
			DestData[i * StereoChannels + 1] = SourceData[i];
		}
	}
	else
	{
		RawData.Append(PCMData, RawPCMDataSize);
	}
}
