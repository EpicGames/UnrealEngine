// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/InputDeviceProperties.h"
#include "Framework/Application/SlateApplication.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveFloat.h"
#include "GameFramework/InputDeviceLibrary.h"
#include "GameFramework/InputSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputDeviceProperties)

///////////////////////////////////////////////////////////////////////
// UInputDeviceProperty

UInputDeviceProperty::UInputDeviceProperty(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RecalculateDuration();
}

void UInputDeviceProperty::ApplyDeviceProperty(const FPlatformUserId UserId, const FInputDeviceId DeviceId)
{
	UInputDeviceProperty::ApplyDeviceProperty_Internal(UserId, DeviceId, GetInternalDeviceProperty());
}

void UInputDeviceProperty::ApplyDeviceProperty_Internal(const FPlatformUserId UserId, const FInputDeviceId DeviceId, FInputDeviceProperty* RawProperty)
{
	if (ensure(RawProperty))
	{
		IInputInterface* InputInterface = FSlateApplication::Get().IsInitialized() ? FSlateApplication::Get().GetInputInterface() : nullptr;
		if (InputInterface)
		{
			int32 ControllerId = INDEX_NONE;
			IPlatformInputDeviceMapper::Get().RemapUserAndDeviceToControllerId(UserId, ControllerId, DeviceId);

			// TODO_BH: Refactor input interface to take an FPlatformUserId directly (UE-158881)
			InputInterface->SetDeviceProperty(ControllerId, RawProperty);
		}
	}
}

float UInputDeviceProperty::GetDuration() const
{
	return PropertyDuration;
}

float UInputDeviceProperty::RecalculateDuration()
{
	return PropertyDuration;
}

#if WITH_EDITOR
void UInputDeviceProperty::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	RecalculateDuration();
}
#endif	// WITH_EDITOR

void UInputDeviceProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{

}

void UInputDeviceProperty::ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset /*= false*/)
{

}

///////////////////////////////////////////////////////////////////////
// UColorInputDeviceProperty

void UColorInputDeviceProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{
	// Check for an override on the current input device
	if (const FDeviceColorData* Data = GetDeviceSpecificData<FDeviceColorData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		InternalProperty.bEnable = Data->bEnable;
		InternalProperty.Color = Data->LightColor;
	}
	// Otherwise use the default color data
	else
	{
		InternalProperty.bEnable = ColorData.bEnable;
		InternalProperty.Color = ColorData.LightColor;
	}
}

void UColorInputDeviceProperty::ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset /*= false*/)
{
	bool bReset = ColorData.bResetAfterCompletion;
	if (const FDeviceColorData* Data = GetDeviceSpecificData<FDeviceColorData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		bReset = Data->bResetAfterCompletion;
	}

	bReset |= bForceReset;

	if (bReset)
	{
		// Disabling the light will reset the color
		InternalProperty.bEnable = false;
		ApplyDeviceProperty(PlatformUser, DeviceId);
	}
}

FInputDeviceProperty* UColorInputDeviceProperty::GetInternalDeviceProperty()
{
	return &InternalProperty;
}

///////////////////////////////////////////////////////////////////////
// UColorInputDeviceCurveProperty

void UColorInputDeviceCurveProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{
	// Check for an override on the current input device
	if (const FDeviceColorCurveData* Data = GetDeviceSpecificData<FDeviceColorCurveData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		InternalProperty.bEnable = Data->bEnable;

		if (ensure(Data->DeviceColorCurve))
		{
			FLinearColor CurveColor = Data->DeviceColorCurve->GetLinearColorValue(Duration);
			InternalProperty.Color = CurveColor.ToFColorSRGB();
		}
	}
	// Otherwise use the default color data
	else
	{
		InternalProperty.bEnable = ColorData.bEnable;

		if (ensure(ColorData.DeviceColorCurve))
		{
			FLinearColor CurveColor = ColorData.DeviceColorCurve->GetLinearColorValue(Duration);
			InternalProperty.Color = CurveColor.ToFColorSRGB();
		}
	}
}

void UColorInputDeviceCurveProperty::ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset /*= false*/)
{
	bool bReset = ColorData.bResetAfterCompletion;
	if (const FDeviceColorCurveData* Data = GetDeviceSpecificData<FDeviceColorCurveData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		bReset = Data->bResetAfterCompletion;
	}

	bReset |= bForceReset;
	
	if (bReset)
	{
		// Disabling the light will reset the color
    	InternalProperty.bEnable = false;
    	ApplyDeviceProperty(PlatformUser, DeviceId);
	}	
}

FInputDeviceProperty* UColorInputDeviceCurveProperty::GetInternalDeviceProperty()
{
	return &InternalProperty;
}

float UColorInputDeviceCurveProperty::RecalculateDuration()
{
	float MinTime = 0.f;
	float MaxTime = 0.f;

	if (ColorData.DeviceColorCurve)
	{
		ColorData.DeviceColorCurve->GetTimeRange(MinTime, MaxTime);
	}

	// Find the max time of any device specific data
	for (const TPair<FName, FDeviceColorCurveData>& Pair : DeviceOverrideData)
	{
		if (Pair.Value.DeviceColorCurve)
		{
			float DeviceSpecificMinTime = 0.f;
			float DeviceSpecificMaxTime = 0.f;
			Pair.Value.DeviceColorCurve->GetTimeRange(DeviceSpecificMinTime, DeviceSpecificMaxTime);
			if (DeviceSpecificMaxTime > MaxTime)
			{
				MaxTime = DeviceSpecificMaxTime;
			}
		}		
	}

	PropertyDuration = MaxTime;
	
	return PropertyDuration;
}

///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerEffect

FInputDeviceProperty* UInputDeviceTriggerEffect::GetInternalDeviceProperty()
{
	return &ResetProperty;
}

void UInputDeviceTriggerEffect::ResetDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, bool bForceReset /*= false*/)
{
	if (bForceReset || (BaseTriggerData.bResetUponCompletion && BaseTriggerData.AffectedTriggers != EInputDeviceTriggerMask::None))
	{
		// Pass in our reset property
		ResetProperty.AffectedTriggers = BaseTriggerData.AffectedTriggers;
		ApplyDeviceProperty_Internal(PlatformUser, DeviceId, &ResetProperty);
	}	
}

void UInputDeviceTriggerEffect::ApplyDeviceProperty(const FPlatformUserId UserId, const FInputDeviceId DeviceId)
{
	// If the affected triggers is set to none then nothing will happen and the platform may throw an ensure, so just skip it and log here
	if (BaseTriggerData.AffectedTriggers != EInputDeviceTriggerMask::None)
	{
		Super::ApplyDeviceProperty(UserId, DeviceId);
	}
	else
	{
		UE_LOGF(LogInputDevices, Warning, "The Affected Triggers is set to none on '%ls', nothing will happen!", *GetFName().ToString());
	}
}

///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerFeedbackProperty

UInputDeviceTriggerFeedbackProperty::UInputDeviceTriggerFeedbackProperty()
	: UInputDeviceTriggerEffect()
{
	InternalProperty.AffectedTriggers = BaseTriggerData.AffectedTriggers;
}

int32 UInputDeviceTriggerFeedbackProperty::GetPositionValue(const FDeviceTriggerFeedbackData* Data, const float Duration) const
{
	if (ensure(Data->FeedbackPositionCurve))
	{
		int32 Pos = Data->FeedbackPositionCurve->GetFloatValue(Duration);
		return FMath::Clamp(Pos, 0, UInputPlatformSettings::Get()->MaxTriggerFeedbackPosition);
	}

	return 0;
}

int32 UInputDeviceTriggerFeedbackProperty::GetStrengthValue(const FDeviceTriggerFeedbackData* Data, const float Duration) const
{
	if (ensure(Data->FeedbackStrenghCurve))
	{
		int32 Strength = Data->FeedbackStrenghCurve->GetFloatValue(Duration);
		return FMath::Clamp(Strength, 0, UInputPlatformSettings::Get()->MaxTriggerFeedbackStrength);
	}

	return 0;
}

void UInputDeviceTriggerFeedbackProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{		
	InternalProperty.AffectedTriggers = BaseTriggerData.AffectedTriggers;

	const FDeviceTriggerFeedbackData* DataToUse = &TriggerData;

	if (const FDeviceTriggerFeedbackData* OverrideData = GetDeviceSpecificData<FDeviceTriggerFeedbackData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		DataToUse = OverrideData;
	}

	InternalProperty.Position = GetPositionValue(DataToUse, Duration);
	InternalProperty.Strengh = GetStrengthValue(DataToUse, Duration);
}

FInputDeviceProperty* UInputDeviceTriggerFeedbackProperty::GetInternalDeviceProperty()
{
	return &InternalProperty;
}

float UInputDeviceTriggerFeedbackProperty::RecalculateDuration()
{
	// Get the max time from the two curves on this property
	float MinTime, MaxTime = 0.0f;
	if (TriggerData.FeedbackPositionCurve)
	{
		TriggerData.FeedbackPositionCurve->GetTimeRange(MinTime, MaxTime);
	}
	
	if (TriggerData.FeedbackStrenghCurve)
	{
		TriggerData.FeedbackStrenghCurve->GetTimeRange(MinTime, MaxTime);
	}

	// Find the max time of any device specific data
	for (const TPair<FName, FDeviceTriggerFeedbackData>& Pair : DeviceOverrideData)
	{
		if (Pair.Value.FeedbackPositionCurve)
		{
			Pair.Value.FeedbackPositionCurve->GetTimeRange(MinTime, MaxTime);
		}

		if (Pair.Value.FeedbackStrenghCurve)
		{
			Pair.Value.FeedbackStrenghCurve->GetTimeRange(MinTime, MaxTime);
		}
	}
	
	PropertyDuration = MaxTime;
	return PropertyDuration;
}

///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerResistanceProperty

UInputDeviceTriggerResistanceProperty::UInputDeviceTriggerResistanceProperty()
	: UInputDeviceTriggerEffect()
{
	PropertyDuration = 1.0f;
}

void UInputDeviceTriggerResistanceProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{
	InternalProperty.AffectedTriggers = BaseTriggerData.AffectedTriggers;

	if (const FDeviceTriggerTriggerResistanceData* Data = GetDeviceSpecificData<FDeviceTriggerTriggerResistanceData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		InternalProperty.StartPosition = Data->StartPosition;
		InternalProperty.StartStrengh = Data->StartStrengh;
		InternalProperty.EndPosition = Data->EndPosition;
		InternalProperty.EndStrengh = Data->EndStrengh;
	}
	else
	{
		InternalProperty.StartPosition = TriggerData.StartPosition;
		InternalProperty.StartStrengh = TriggerData.StartStrengh;
		InternalProperty.EndPosition = TriggerData.EndPosition;
		InternalProperty.EndStrengh = TriggerData.EndStrengh;
	}
}

FInputDeviceProperty* UInputDeviceTriggerResistanceProperty::GetInternalDeviceProperty()
{
	return &InternalProperty;
}


///////////////////////////////////////////////////////////////////////
// UInputDeviceTriggerVibrationProperty

UInputDeviceTriggerVibrationProperty::UInputDeviceTriggerVibrationProperty()
	: UInputDeviceTriggerEffect()
{
	PropertyDuration = 1.0f;
}

void UInputDeviceTriggerVibrationProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{
	const FDeviceTriggerTriggerVibrationData* DataToUse = &TriggerData;

	if (const FDeviceTriggerTriggerVibrationData* OverrideData = GetDeviceSpecificData<FDeviceTriggerTriggerVibrationData>(PlatformUser, DeviceId, DeviceOverrideData))
	{
		DataToUse = OverrideData;
	}

	InternalProperty.AffectedTriggers = BaseTriggerData.AffectedTriggers;
	InternalProperty.TriggerPosition = GetTriggerPositionValue(DataToUse, Duration);
	InternalProperty.VibrationFrequency = GetVibrationFrequencyValue(DataToUse, Duration);
	InternalProperty.VibrationAmplitude = GetVibrationAmplitudeValue(DataToUse, Duration);
}

FInputDeviceProperty* UInputDeviceTriggerVibrationProperty::GetInternalDeviceProperty()
{
	return &InternalProperty;
}

float UInputDeviceTriggerVibrationProperty::RecalculateDuration()
{
	// Get the max time from the curves on this property
	float MaxTime = 0.0f;

	auto EvaluateMaxTime = [&MaxTime](TObjectPtr<UCurveFloat> InCurve)
	{
		float MinCurveTime, MaxCurveTime = 0.0f;
		if (InCurve)
		{
			InCurve->GetTimeRange(MinCurveTime, MaxCurveTime);
			if (MaxCurveTime > MaxTime)
			{
				MaxTime = MaxCurveTime;
			}
		}
	};

	EvaluateMaxTime(TriggerData.TriggerPositionCurve);
	EvaluateMaxTime(TriggerData.VibrationFrequencyCurve);
	EvaluateMaxTime(TriggerData.VibrationAmplitudeCurve);

	for (const TPair<FName, FDeviceTriggerTriggerVibrationData>& Pair : DeviceOverrideData)
	{
		EvaluateMaxTime(Pair.Value.TriggerPositionCurve);
		EvaluateMaxTime(Pair.Value.VibrationFrequencyCurve);
		EvaluateMaxTime(Pair.Value.VibrationAmplitudeCurve);
	}

	PropertyDuration = MaxTime;
	return PropertyDuration;
}

int32 UInputDeviceTriggerVibrationProperty::GetTriggerPositionValue(const FDeviceTriggerTriggerVibrationData* Data, const float Duration) const
{
	if (ensure(Data->TriggerPositionCurve))
	{
		int32 Strength = Data->TriggerPositionCurve->GetFloatValue(Duration);
		return FMath::Clamp(Strength, 0, UInputPlatformSettings::Get()->MaxTriggerVibrationTriggerPosition);
	}

	return 0;
}

int32 UInputDeviceTriggerVibrationProperty::GetVibrationFrequencyValue(const FDeviceTriggerTriggerVibrationData* Data, const float Duration) const
{
	if (ensure(Data->VibrationFrequencyCurve))
	{
		int32 Strength = Data->VibrationFrequencyCurve->GetFloatValue(Duration);
		return FMath::Clamp(Strength, 0, UInputPlatformSettings::Get()->MaxTriggerVibrationFrequency);
	}
	
	return 0;
}

int32 UInputDeviceTriggerVibrationProperty::GetVibrationAmplitudeValue(const FDeviceTriggerTriggerVibrationData* Data, const float Duration) const
{
	if (ensure(Data->VibrationAmplitudeCurve))
	{
		int32 Strength = Data->VibrationAmplitudeCurve->GetFloatValue(Duration);
		return FMath::Clamp(Strength, 0, UInputPlatformSettings::Get()->MaxTriggerVibrationAmplitude);
	}
	
	return 0;
}

///////////////////////////////////////////////////////////////////////
// UInputDeviceSoundBasedVibrationProperty

#include "Sound/SoundBase.h"
#include "Sound/SoundSubmix.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/InputDeviceSubsystem.h"

FAudioBasedVibrationData::FAudioBasedVibrationData()
	: Sound(nullptr)
{
}

void UInputDeviceAudioBasedVibrationProperty::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{
	// There is nothing to do here, the endpoints are set on the sound asset.
}

void UInputDeviceAudioBasedVibrationProperty::ApplyDeviceProperty(const FPlatformUserId UserId, const FInputDeviceId DeviceId)
{
	// This sound should have the submix sends set that it wants, all we have to do is play it.
	if (const FAudioBasedVibrationData* DataToUse = GetRelevantData(UserId, DeviceId))
	{
		if (DataToUse->Sound)
		{
			// Get the player controller and play the sound
			if (APlayerController* PC = UInputDeviceLibrary::GetPlayerControllerFromPlatformUser(UserId))
			{
				// The sound endpoints will have been populated above in the Evaluate function
				PC->ClientPlaySound(DataToUse->Sound);
			}
		}		
	}
}

FInputDeviceProperty* UInputDeviceAudioBasedVibrationProperty::GetInternalDeviceProperty()
{
	return nullptr;
}

float UInputDeviceAudioBasedVibrationProperty::RecalculateDuration()
{
	// Returning a durtion of 0 means that it will be played as a "one shot" effect
	return 0.0f;
}

const FAudioBasedVibrationData* UInputDeviceAudioBasedVibrationProperty::GetRelevantData(const FPlatformUserId UserId, const FInputDeviceId DeviceId) const
{
	const FAudioBasedVibrationData* DataToUse = &Data;
	
	if (const FAudioBasedVibrationData* OverrideData = GetDeviceSpecificData<FAudioBasedVibrationData>(UserId, DeviceId, DeviceOverrideData))
	{
		DataToUse = OverrideData;
	}
	return DataToUse;
}
