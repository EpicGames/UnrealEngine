// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/InputDevicePropertyHandle.h"

////////////////////////////////////////////////////////
// FInputDevicePropertyHandle

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputDevicePropertyHandle)

FInputDevicePropertyHandle FInputDevicePropertyHandle::InvalidHandle(0);

FInputDevicePropertyHandle::FInputDevicePropertyHandle()
	: InternalId(0)
{

}

FInputDevicePropertyHandle::FInputDevicePropertyHandle(uint32 InInternalID)
	: InternalId(InInternalID)
{

}

bool FInputDevicePropertyHandle::operator==(const FInputDevicePropertyHandle& Other) const
{
	return InternalId == Other.InternalId;
}

bool FInputDevicePropertyHandle::operator!=(const FInputDevicePropertyHandle& Other) const
{
	return InternalId != Other.InternalId;
}

uint32 FInputDevicePropertyHandle::GetTypeHash() const
{
	return ::GetTypeHash(InternalId);
}

uint32 GetTypeHash(const FInputDevicePropertyHandle& InHandle)
{
	return InHandle.GetTypeHash();
}

FString FInputDevicePropertyHandle::ToString() const
{
	return IsValid() ? FString::FromInt(InternalId) : TEXT("Invalid");
}

bool FInputDevicePropertyHandle::IsValid() const
{
	return InternalId != FInputDevicePropertyHandle::InvalidHandle.InternalId;
}

FInputDevicePropertyHandle FInputDevicePropertyHandle::AcquireValidHandle()
{
	// 0 is the "Invalid" index for these handles. Start them at 1
	static uint32 GHandleIndex = 1;

	return FInputDevicePropertyHandle(++GHandleIndex);
}
