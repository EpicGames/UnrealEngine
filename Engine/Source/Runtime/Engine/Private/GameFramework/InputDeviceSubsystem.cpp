// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/InputDeviceSubsystem.h"
#include "GameFramework/InputDeviceProperties.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"			// For RegisterInputPreProcessor
#include "GenericPlatform/InputDeviceRegistry.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"	// For FApp::GetDeltaTime()

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputDeviceSubsystem)

#if WITH_EDITOR
#include "Editor.h"		// For PIE delegates
#else
#include "Engine/Engine.h"
#endif

DEFINE_LOG_CATEGORY(LogInputDeviceProperties);

namespace UE
{
	static bool bShouldUpdatedDeviceSubsystemOnConnection = true;
	static FAutoConsoleVariableRef CVarShouldUpdatedDeviceSubsystemOnConnection(TEXT("input.bShouldUpdatedDeviceSubsystemOnConnection"),
		bShouldUpdatedDeviceSubsystemOnConnection,
		TEXT("If true, then upon receiving a device connection event, the UInputDeviceSubsystem will attempt to update the latest known metadata about a given device."));

	static bool bShouldUpdateDeviceSubsystemOnDescriptorChanged = true;
	static FAutoConsoleVariableRef CVarShouldUpdateDeviceSubsystemOnDescriptorChanged(TEXT("input.bShouldUpdateDeviceSubsystemOnDescriptorChanged"),
		bShouldUpdateDeviceSubsystemOnDescriptorChanged,
		TEXT("If true, then UInputDeviceSubsystem will route FInputDeviceRegistry::OnInputDeviceDescriptorChanged broadcasts into SetMostRecentlyUsedHardwareDevice ")
		TEXT("(which fires OnInputHardwareDeviceChanged). Set to false to disable the descriptor-change-driven path on a specific platform if it causes issues; ")
		TEXT("the device-connection path (input.bShouldUpdatedDeviceSubsystemOnConnection) is unaffected."));
}

////////////////////////////////////////////////////////
// FInputDeviceSubsystemProcessor

/** An input processor for detecting changes to input devices based on the current FInputDeviceScope stack */
class FInputDeviceSubsystemProcessor : public IInputProcessor
{
	friend class UInputDeviceSubsystem;
	
	void UpdateLatestDevice(const FKey Key, const FInputDeviceId InDeviceId)
	{
#if WITH_EDITOR
		// If we're stopped at a breakpoint we need for this input preprocessor to just ignore all incoming input
		// because we're now doing stuff outside the game loop in the editor and it needs to not block all that.
		// This can happen if you suspend input while spawning a dialog and then hit another breakpoint and then
		// try and use the editor, you can suddenly be unable to do anything.
		if (GIntraFrameDebuggingGameThread)
		{
			return;
		}
#endif
		
		if (UInputDeviceSubsystem* SubSystem = UInputDeviceSubsystem::Get())
		{
			if (const TOptional<FInputDeviceDescriptor> Desc = FInputDeviceRegistry::FindDescriptor(InDeviceId))
			{
				if (const FHardwareDeviceIdentifier* Hardware = UInputPlatformSettings::Get()->GetHardwareDeviceForClassName(Desc->HardwareDeviceIdentifier))
				{
					SubSystem->SetMostRecentlyUsedHardwareDevice(InDeviceId, *Hardware);
				}
				// If there isn't one specified in the config file, we can just use the device name and scope name to make one.
				else
				{
					SubSystem->SetMostRecentlyUsedHardwareDevice(InDeviceId, { Desc->InputDeviceName, Desc->HardwareDeviceIdentifier });
				}
			}
			// If there isn't a recent input device scope, then we can check if the key was from a keyboard and mouse
			else if (!Key.IsGamepadKey())
			{
				// We only want to set our "fallback" of DefaultKeyboardAndMouse if there is no known existing data for the device id
				const FHardwareDeviceIdentifier ExistingDevice = SubSystem->GetInputDeviceHardwareIdentifier(InDeviceId);
				if (ExistingDevice == FHardwareDeviceIdentifier::Invalid)
				{
					SubSystem->SetMostRecentlyUsedHardwareDevice(InDeviceId, FHardwareDeviceIdentifier::DefaultKeyboardAndMouse);	
				}
			}
		}
	}
	
public:

	// Required by IInputProcessor
	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override { }
	
	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InEvent) override
	{
		UpdateLatestDevice(InEvent.GetKey(), InEvent.GetInputDeviceId());
		return false;
	}

	virtual bool HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InEvent) override
	{
		if (!FMath::IsNearlyZero(InEvent.GetAnalogValue(), UE_KINDA_SMALL_NUMBER))
		{
			UpdateLatestDevice(InEvent.GetKey(), InEvent.GetInputDeviceId());
		}
		
		return false;
	}

	virtual bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& InEvent) override
	{
		// We only want input events that have made changes
		if (!InEvent.GetCursorDelta().IsNearlyZero())
		{
			UpdateLatestDevice(InEvent.GetEffectingButton(), InEvent.GetInputDeviceId());	
		}
		
		return false;
	}

	virtual bool HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& InEvent) override
	{
		UpdateLatestDevice(InEvent.GetEffectingButton(), InEvent.GetInputDeviceId());
		return false;
	}

	virtual bool HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& InEvent) override
	{
		UpdateLatestDevice(InEvent.GetEffectingButton(), InEvent.GetInputDeviceId());
		return false;
	}
	
	virtual bool HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& InEvent, const FPointerEvent* InGestureEvent) override
	{
		UpdateLatestDevice(InEvent.GetEffectingButton(), InEvent.GetInputDeviceId());
		return false;
	}
};

////////////////////////////////////////////////////////
// FActiveDeviceProperty

FActiveDeviceProperty::FActiveDeviceProperty()
	: Property(nullptr)
	, EvaluatedDuration(0.0)
	, PlatformUser(PLATFORMUSERID_NONE)
	, DeviceId(INPUTDEVICEID_NONE)
	, PropertyHandle(FInputDevicePropertyHandle::InvalidHandle)
	, bLooping(false)
	, bIgnoreTimeDilation(false)
	, bPlayWhilePaused(false)
	, bHasBeenAppliedAtLeastOnce(false)
{ }

uint32 GetTypeHash(const FActiveDeviceProperty& InProp)
{
	return InProp.PropertyHandle.GetTypeHash();
}

bool operator==(const FActiveDeviceProperty& ActiveProp, const FInputDevicePropertyHandle& Handle)
{
	return ActiveProp.PropertyHandle == Handle;
}

bool operator!=(const FActiveDeviceProperty& ActiveProp, const FInputDevicePropertyHandle& Handle)
{
	return ActiveProp.PropertyHandle != Handle;
}

// The property handles are the only things that matter when comparing, they will always be unique.
bool FActiveDeviceProperty::operator==(const FActiveDeviceProperty& Other) const
{
	return PropertyHandle == Other.PropertyHandle;
}

bool FActiveDeviceProperty::operator!=(const FActiveDeviceProperty& Other) const
{
	return PropertyHandle != Other.PropertyHandle;
}

////////////////////////////////////////////////////////
// FActivateDevicePropertyParams

FActivateDevicePropertyParams::FActivateDevicePropertyParams()
	: UserId(FSlateApplicationBase::SlateAppPrimaryPlatformUser)
	// Set this Device Id to NONE by default. The subsystem will detect this being invalid and fall back 
	// to the default device for the given platform user. This will let the default behavior of the system be 
	// consistent, while still allowing for setting specific input devices if desired.
	, DeviceId(INPUTDEVICEID_NONE)
	, bLooping(false)
	, bIgnoreTimeDilation(false)
	, bPlayWhilePaused(false)
{
}

////////////////////////////////////////////////////////
// UInputDeviceSubsystem

UInputDeviceSubsystem* UInputDeviceSubsystem::Get()
{
	return GEngine ? GEngine->GetEngineSubsystem<UInputDeviceSubsystem>() : nullptr;
}

void UInputDeviceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// We have to have a valid slate app to run this subsystem
	check(FSlateApplication::IsInitialized());
	
	InputPreprocessor = MakeShared<FInputDeviceSubsystemProcessor>();
	FSlateApplication::Get().RegisterInputPreProcessor(InputPreprocessor, EInputPreProcessorType::Engine);

	IPlatformInputDeviceMapper::Get().GetOnInputDeviceHardwareInfoAvailable().AddUObject(this, &UInputDeviceSubsystem::HandleDeviceConnectionChange);

	// Listen for descriptor changes on the registry so we can republish the hardware identifier into
	// our own latest-device tracking. RegisterDevice already filters to only fire this delegate when
	// a descriptor's contents actually change, so by routing through SetMostRecentlyUsedHardwareDevice
	// (which fires OnInputHardwareDeviceChanged) we collapse "device connection" and "device descriptor
	// refresh" into a single delegate that gameplay code can subscribe to.
	FInputDeviceRegistry::OnInputDeviceDescriptorChanged().AddUObject(this, &UInputDeviceSubsystem::HandleInputDeviceDescriptorChanged);

#if WITH_EDITOR
	FEditorDelegates::PreBeginPIE.AddUObject(this, &UInputDeviceSubsystem::OnPrePIEStarted);
	FEditorDelegates::PausePIE.AddUObject(this, &UInputDeviceSubsystem::OnPIEPaused);
	FEditorDelegates::ResumePIE.AddUObject(this, &UInputDeviceSubsystem::OnPIEResumed);
	FEditorDelegates::EndPIE.AddUObject(this, &UInputDeviceSubsystem::OnPIEStopped);
#endif	// WITH_EDITOR
}

void UInputDeviceSubsystem::Deinitialize()
{
	Super::Deinitialize();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputPreprocessor);
	}
	InputPreprocessor.Reset();

	IPlatformInputDeviceMapper::Get().GetOnInputDeviceHardwareInfoAvailable().RemoveAll(this);
	FInputDeviceRegistry::OnInputDeviceDescriptorChanged().RemoveAll(this);
}

bool UInputDeviceSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	// No slate app means we can't process any input
	if (!FSlateApplication::IsInitialized() ||
		// Commandlets and servers have no use for this subsystem
		IsRunningCommandlet() ||
		IsRunningDedicatedServer())
	{
		return false;
	}
	
	return Super::ShouldCreateSubsystem(Outer);
}

UWorld* UInputDeviceSubsystem::GetTickableGameObjectWorld() const
{
	// Use the default world by default...
	UWorld* World = GetWorld();

	// ...but if we don't have one (i.e. we are in the editor and not PIE'ing)
	// then we need to get the editor world. This will let us preview
	// device properties without needing to actually PIE every time
	if (!World && GEngine)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* ThisWorld = Context.World();
			if (!ThisWorld)
			{
				continue;
			}
			// Prefer new PIE window worlds
			else if (Context.WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Game)
			{
				World = ThisWorld;
				break;
			}
			// Fallback to the editor world, which is still valid for previewing device properties
			else if (Context.WorldType == EWorldType::Editor)
			{
				World = ThisWorld;
			}
		}
	}

	return World;
}

ETickableTickType UInputDeviceSubsystem::GetTickableTickType() const
{
	return (IsTemplate() ? ETickableTickType::Never : ETickableTickType::Conditional);
}

bool UInputDeviceSubsystem::IsTickable() const
{
	// Only tick when there are active device properties or ones we want to remove
	const bool bWantsTick = !ActiveProperties.IsEmpty() || !PropertiesPendingRemoval.IsEmpty();
#if WITH_EDITOR
	// If we are PIE'ing, then check if PIE is paused
	if (GEditor && (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld))
	{
		return bIsPIEPlaying && bWantsTick;
	}	
#endif
	
	return bWantsTick;
}

bool UInputDeviceSubsystem::IsTickableInEditor() const
{
	// We want to tick in editor to allow previewing of device properties
	return true;
}

TStatId UInputDeviceSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UInputDeviceSubsystem, STATGROUP_Tickables);
}

void UInputDeviceSubsystem::Tick(float InDeltaTime)
{
	// If a property doesn't want to be affected by time dilation then we can use this instead
	const double NonDialatedDeltaTime = FApp::GetDeltaTime();

	const UWorld* World = GetTickableGameObjectWorld();
	const bool bIsGamePaused = World ? World->IsPaused() : false;

	for (auto It = ActiveProperties.CreateIterator(); It; ++It)
	{
		FActiveDeviceProperty& ActiveProp = *It;
		if (!ActiveProp.Property.IsValid())
		{
			// The property may have been GC'd if the world has changed and it was an instanced property
			// on an actor in that world
			It.RemoveCurrent();
			PropertiesPendingRemoval.Remove(ActiveProp.PropertyHandle);
			continue;
		}
		
		if (PropertiesPendingRemoval.Contains(ActiveProp.PropertyHandle))
		{
			ActiveProp.Property->ResetDeviceProperty(ActiveProp.PlatformUser, ActiveProp.DeviceId);
			It.RemoveCurrent();
			PropertiesPendingRemoval.Remove(ActiveProp.PropertyHandle);
			continue;
		}

		// If the game is paused, only play effects that have explicitly specified that they should be
		// played while paused.
		if (bIsGamePaused && !ActiveProp.bPlayWhilePaused)
		{
			continue;
		}

		const double DeltaTime = ActiveProp.bIgnoreTimeDilation ? NonDialatedDeltaTime : static_cast<double>(InDeltaTime);

		// Increase the evaluated time of this property
		ActiveProp.EvaluatedDuration += DeltaTime;

		// If the property has run past it's duration, reset it and remove it from our active properties
		// Don't remove properties that are looping, because the user desires them to keep playing
		if (!ActiveProp.bLooping && ActiveProp.EvaluatedDuration > ActiveProp.Property->GetDuration() && ActiveProp.bHasBeenAppliedAtLeastOnce)
		{
			ActiveProp.Property->ResetDeviceProperty(ActiveProp.PlatformUser, ActiveProp.DeviceId);
			PropertiesPendingRemoval.Remove(ActiveProp.PropertyHandle);
			It.RemoveCurrent();
			continue;
		}
		// Otherwise, we can evaluate and apply it as normal
		else
		{
			ActiveProp.Property->EvaluateDeviceProperty(ActiveProp.PlatformUser, ActiveProp.DeviceId, DeltaTime, ActiveProp.EvaluatedDuration);
			ActiveProp.Property->ApplyDeviceProperty(ActiveProp.PlatformUser, ActiveProp.DeviceId);

			// Track that this property has now successfully been applied at least once
			ActiveProp.bHasBeenAppliedAtLeastOnce = true;
		}
	}

	// If there are still properties pending removal, then they have already been removed by the loop
	// above. Empty out the pending removal set to start fresh next frame.
	PropertiesPendingRemoval.Empty();
}

FInputDevicePropertyHandle UInputDeviceSubsystem::ActivateDeviceProperty(UInputDeviceProperty* Property, const FActivateDevicePropertyParams& Params)
{
	if (!Property)
	{
		UE_LOGF(LogInputDeviceProperties, Error, "Invalid Property passed into ActivateDeviceProperty! Nothing will happen.");
		return FInputDevicePropertyHandle::InvalidHandle;
	}

	if (!Params.UserId.IsValid())
	{
		UE_LOGF(LogInputDeviceProperties, Error, "Invalid Platform User Id '%d' given to ActivateDeviceProperty! Nothing will happen.", Params.UserId.GetInternalId());
		return FInputDevicePropertyHandle::InvalidHandle;
	}

	FInputDevicePropertyHandle OutHandle = FInputDevicePropertyHandle::AcquireValidHandle();

	if (ensureMsgf(OutHandle.IsValid(), TEXT("Unable to acquire a valid input device property handle! The input device property cannot be activated")))
	{
		FActiveDeviceProperty ActiveProp = {};
		ActiveProp.Property = Property;
		ActiveProp.PlatformUser = Params.UserId;

		// If a valid input device was not given, then set this property on the default device for the user.
		ActiveProp.DeviceId = 
			Params.DeviceId.IsValid() ? Params.DeviceId : IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(Params.UserId);		

		ActiveProp.PropertyHandle = OutHandle;
		ActiveProp.bLooping = Params.bLooping;
		ActiveProp.bIgnoreTimeDilation = Params.bIgnoreTimeDilation;
		ActiveProp.bPlayWhilePaused = Params.bPlayWhilePaused;

		// Keep track of this new active property. Placing it in this set will have it updated on this subsystem's Tick
		ActiveProperties.Add(ActiveProp);

#if WITH_EDITORONLY_DATA
		// If we are currently PIE'ing, then track this property if it's class isn't already known. This will
		// ensure that we reset device properties of unique types
		if (bIsPIEPlaying)
		{
			const bool bAlreadyHasResetData = 
				 PropertyClassesRequiringReset.ContainsByPredicate([&ActiveProp](const FResetPIEData& Data)
				 {
				 	return Data.DeviceId == ActiveProp.DeviceId && Data.UserId == ActiveProp.PlatformUser && Data.ClassToReset == ActiveProp.Property->GetClass();
				 });
			
			if (!bAlreadyHasResetData)
			{
			 	FResetPIEData ResetData = {};
             	ResetData.ClassToReset = ActiveProp.Property->GetClass();
             	ResetData.DeviceId = ActiveProp.DeviceId;
             	ResetData.UserId = ActiveProp.PlatformUser;
			 	PropertyClassesRequiringReset.Emplace(ResetData);	
			}
		}
#endif	// WITH_EDITOR
	}

	return OutHandle;
}

FInputDevicePropertyHandle UInputDeviceSubsystem::ActivateDevicePropertyOfClass(TSubclassOf<UInputDeviceProperty> PropertyClass, const FActivateDevicePropertyParams& Params)
{
	if (!PropertyClass)
	{
		UE_LOGF(LogInputDeviceProperties, Error, "Invalid PropertyClass passed into ActivateDeviceProperty! Nothing will happen.");
		return FInputDevicePropertyHandle::InvalidHandle;
	}

	// Spawn a new instance of the given device property and activate it as normal
	return ActivateDeviceProperty(NewObject<UInputDeviceProperty>(/* Outer = */ this, /* Class */ PropertyClass), Params);
}

UInputDeviceProperty* UInputDeviceSubsystem::GetActiveDeviceProperty(const FInputDevicePropertyHandle Handle) const
{
	// Don't include any properties that are pending removal
	if (!PropertiesPendingRemoval.Contains(Handle))
	{
		// We can find the active property based on the handle's hash because FActiveDeviceProperty::GetTypeHash
		// just returns its FInputDevicePropertyHandle's GetTypeHash
		if (const FActiveDeviceProperty* ExistingProp = ActiveProperties.FindByHash(Handle.GetTypeHash(), Handle))
		{
			return ExistingProp->Property.Get();
		}
	}
	
	return nullptr;
}

bool UInputDeviceSubsystem::IsPropertyActive(const FInputDevicePropertyHandle Handle) const
{
	return ActiveProperties.ContainsByHash(Handle.GetTypeHash(), Handle) && !PropertiesPendingRemoval.Contains(Handle);
}

void UInputDeviceSubsystem::RemoveDevicePropertyByHandle(const FInputDevicePropertyHandle HandleToRemove)
{
	PropertiesPendingRemoval.Add(HandleToRemove);
}

void UInputDeviceSubsystem::RemoveDevicePropertyHandles(const TSet<FInputDevicePropertyHandle>& HandlesToRemove)
{
	if (!HandlesToRemove.IsEmpty())
	{
		PropertiesPendingRemoval.Append(HandlesToRemove);	
	}
	else
	{
		UE_LOGF(LogInputDeviceProperties, Warning, "Provided an empty set of handles to remove. Nothing will happen.");
	}
}

void UInputDeviceSubsystem::RemoveAllDeviceProperties()
{
	for (FActiveDeviceProperty& ActiveProperty : ActiveProperties)
	{
		if (ActiveProperty.Property.IsValid())
		{
			ActiveProperty.Property->ResetDeviceProperty(ActiveProperty.PlatformUser, ActiveProperty.DeviceId);
		}		
	}
	
	ActiveProperties.Empty();
	PropertiesPendingRemoval.Empty();
}

FHardwareDeviceIdentifier UInputDeviceSubsystem::GetMostRecentlyUsedHardwareDevice(const FPlatformUserId InUserId) const
{
	if (const FLatestInputDeviceData* FoundDevice = LatestUserDeviceIdentifiers.Find(InUserId))
	{
		if (const FHardwareDeviceIdentifier* ExistingDeviceId = LatestInputDeviceIdentifiers.Find(FoundDevice->LatestDeviceId))
		{
			return *ExistingDeviceId;
		}
	}
		
	return FHardwareDeviceIdentifier::Invalid;
}

FInputDeviceId UInputDeviceSubsystem::GetMostRecentlyUsedInputDeviceId(const FPlatformUserId InUserId, const EHardwareDevicePrimaryType OfType /*= EHardwareDevicePrimaryType::Unspecified*/) const
{
	if (const FLatestInputDeviceData* FoundDevice = LatestUserDeviceIdentifiers.Find(InUserId))
	{
		// If we didn't specific a hardware device type, then just return the latest one that has been used
		if (OfType == EHardwareDevicePrimaryType::Unspecified)
		{
			return FoundDevice->LatestDeviceId;	
		}

		// Attempt to find a device of that specific type
		if (const FInputDeviceId* FoundDeviceOfType = FoundDevice->LatestDevices.Find(OfType))
		{
			return *FoundDeviceOfType;
		}
	}
		
	return INPUTDEVICEID_NONE;
}

FInputDeviceId UInputDeviceSubsystem::GetLatestDeviceOfType(const FPlatformUserId InUserId, const EHardwareDevicePrimaryType OfType) const
{
	// TODO: In the future, we may provide an interface to specify a specify a specific
	// input device to play force feedback on. We would implement that here. For example, it could
	// be useful to be able to play different force feedback events on multiple devices for a given user
	// or otherwise allow a game to specify how a force feedback effect behaves when played.
	// For example, you may want to play a force feedback event on multiple input devices if you have a racing sim
	// or HOTAS setup, with multiple HID's connected at once. 
	
	// Query what the most recently used devices are for the given user...
	if (const FLatestInputDeviceData* FoundDevice = LatestUserDeviceIdentifiers.Find(InUserId))
	{
		// And then return the most recently used device of this type for them.
		if (const FInputDeviceId* FoundDeviceOfType = FoundDevice->LatestDevices.Find(OfType))
		{
			return *FoundDeviceOfType;
		}
	}
	
	return INPUTDEVICEID_NONE;
}

FHardwareDeviceIdentifier UInputDeviceSubsystem::GetInputDeviceHardwareIdentifier(const FInputDeviceId InputDevice) const
{
	if (const FHardwareDeviceIdentifier* FoundDevice = LatestInputDeviceIdentifiers.Find(InputDevice))
	{
		return *FoundDevice;
	}

	return FHardwareDeviceIdentifier::Invalid;
}

void UInputDeviceSubsystem::HandleDeviceConnectionChange(EInputDeviceConnectionState NewState, FPlatformUserId User, FInputDeviceId DeviceId)
{
	if (!UE::bShouldUpdatedDeviceSubsystemOnConnection)
	{
		return;
	}

	// Not all systems will have a device descriptor available right away upon connection yet
	const TOptional<FInputDeviceDescriptor> Desc = FInputDeviceRegistry::FindDescriptor(DeviceId);

	UE_LOGF(LogInputDeviceProperties, Log, "Input device connection change: NewState=%ls, DeviceId=%d, PlatformUser=%d, Device Info=%ls::%ls",
		*UEnum::GetValueAsString(NewState), DeviceId.GetId(), User.GetInternalId(),
		Desc.IsSet() ? *Desc->InputDeviceName.ToString() : TEXT("(unknown)"),
		Desc.IsSet() ? *Desc->HardwareDeviceIdentifier.ToString() : TEXT("(unknown)"));

	// When a new device is connected, we can attempt to populate the subsystem with its connection info.
	// This way it is easier to build menus or respond to specific devices on their connection states
	if (NewState != EInputDeviceConnectionState::Connected)
	{
		return;
	}

	// We need a valid descriptor to proceed past this point
	if (!Desc)
	{
		return;
	}

	if (const FHardwareDeviceIdentifier* Hardware = UInputPlatformSettings::Get()->GetHardwareDeviceForClassName(Desc->HardwareDeviceIdentifier))
	{
		SetMostRecentlyUsedHardwareDevice(DeviceId, *Hardware);
	}
	else
	{
		SetMostRecentlyUsedHardwareDevice(DeviceId, { Desc->InputDeviceName, Desc->HardwareDeviceIdentifier });
	}
}

void UInputDeviceSubsystem::HandleInputDeviceDescriptorChanged(const FInputDeviceDescriptor& Descriptor)
{
	// Kill switch — defaults true. Lets a platform / project disable the descriptor-change-driven
	// SetMostRecentlyUsedHardwareDevice path without rebuilding, in case it causes a live issue.
	// The other update paths (HandleDeviceConnectionChange, the input-preprocessor's UpdateLatestDevice)
	// are independent and continue to function regardless.
	if (!UE::bShouldUpdateDeviceSubsystemOnDescriptorChanged)
	{
		return;
	}

	// Convert the registry descriptor into the FHardwareDeviceIdentifier shape the rest of the
	// subsystem speaks. This is the same translation HandleDeviceConnectionChange does — it prefers
	// the project-config-mapped FHardwareDeviceIdentifier when one is registered for the descriptor's
	// HardwareDeviceIdentifier name, otherwise synthesizes a minimal one from the descriptor's fields.
	if (const FHardwareDeviceIdentifier* Hardware = UInputPlatformSettings::Get()->GetHardwareDeviceForClassName(Descriptor.HardwareDeviceIdentifier))
	{
		SetMostRecentlyUsedHardwareDevice(Descriptor.HardwareDeviceHandle, *Hardware);
	}
	else
	{
		SetMostRecentlyUsedHardwareDevice(Descriptor.HardwareDeviceHandle, { Descriptor.InputDeviceName, Descriptor.HardwareDeviceIdentifier });
	}
}

void UInputDeviceSubsystem::SetMostRecentlyUsedHardwareDevice(const FInputDeviceId InDeviceId, const FHardwareDeviceIdentifier& InHardwareId)
{
	UE_LOGF(LogInputDeviceProperties, Verbose, "SetMostRecentlyUsedHardwareDevice: DeviceId=%d, Hardware=%ls",
		InDeviceId.GetId(), *InHardwareId.ToString());

	// Capture the previous hardware identifier for this device BEFORE we overwrite it so we can
	// detect "same DeviceId, different hardware" 
	// This case must also fire OnInputHardwareDeviceChanged so downstream consumers re-query
	// device capabilities, even though the user's "most recent device id" hasn't changed.
	const FHardwareDeviceIdentifier* PreviousHardwareForDevice = LatestInputDeviceIdentifiers.Find(InDeviceId);
	const bool bHardwareIdentifierChanged = (PreviousHardwareForDevice == nullptr) || (*PreviousHardwareForDevice != InHardwareId);

	// Store or update the most recent hardware information about this FInputDeviceId.
	// We always want this to be up to date no matter what. This data doesn't care about the owning platform user,
	// it is only about the actual Input Device itself.
	LatestInputDeviceIdentifiers.Add(InDeviceId, InHardwareId);

	// Next, we will track information per platform user about which input devices that user has been
	// We want to broadcast the OnInputHardwareDeviceChanged delegate if the user's most recent input device
	// has changed to something else.
	const FPlatformUserId OwningUserId = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(InDeviceId);
	
	// Keep a map to platform users so that we can easily get their most recent hardware
	FLatestInputDeviceData& LatestUserData = LatestUserDeviceIdentifiers.FindOrAdd(OwningUserId);

	const bool bUserLatestDeviceChanged = (LatestUserData.LatestDeviceId != InDeviceId);

	// Keep track of the latest hardware device and the latest of this type for the given user
	LatestUserData.LatestDevices.FindOrAdd(InHardwareId.PrimaryDeviceType, InDeviceId);
	LatestUserData.LatestDeviceId = InDeviceId;

	// Fire the delegate when EITHER (a) the user just switched to a different device, or (b) the
	// device itself reports new hardware. Both are device-identity changes that gameplay code may
	// care about.
	//
	// The (b) "hardware identifier changed for the same DeviceId" branch is gated by the same
	// kill switch as HandleInputDeviceDescriptorChanged. When that CVar is false the behavior of
	// this function reduces to the original (fire only when LatestDeviceId for the user changes),
	// so disabling input.bShouldUpdateDeviceSubsystemOnDescriptorChanged fully restores the
	// pre-change semantics — both at the registry-routing entry point AND inside this setter.
	const bool bRequiresDelegateUpdate = bUserLatestDeviceChanged
		|| (UE::bShouldUpdateDeviceSubsystemOnDescriptorChanged && bHardwareIdentifierChanged);

	if (bRequiresDelegateUpdate)
	{
		OnInputHardwareDeviceChangedNative.Broadcast(OwningUserId, InDeviceId);
		if (OnInputHardwareDeviceChanged.IsBound())
		{
			OnInputHardwareDeviceChanged.Broadcast(OwningUserId, InDeviceId);
		}	
	}
}

#if WITH_EDITOR
void UInputDeviceSubsystem::OnPrePIEStarted(bool bSimulating)
{
	// Remove all active properties, just in case someone was previewing something in the editor that are still going
	RemoveAllDeviceProperties();
#if WITH_EDITORONLY_DATA
	PropertyClassesRequiringReset.Reset();
#endif	// WITH_EDITORONLY_DATA
	bIsPIEPlaying = true;
}

void UInputDeviceSubsystem::OnPIEPaused(bool bSimulating)
{
	bIsPIEPlaying = false;
}

void UInputDeviceSubsystem::OnPIEResumed(bool bSimulating)
{
	bIsPIEPlaying = true;
}

void UInputDeviceSubsystem::OnPIEStopped(bool bSimulating)
{
	// Remove all active properties when PIE stops
	RemoveAllDeviceProperties();

#if WITH_EDITORONLY_DATA
	// Reset any Device Properties that have been added so that their input devices are in a clean state when PIE ends
	for (const FResetPIEData& ResetData : PropertyClassesRequiringReset)
	{
		UInputDeviceProperty* ToReset = NewObject<UInputDeviceProperty>(/* Outer = */ GetTransientPackage(), /* Class */ ResetData.ClassToReset);
		ToReset->ResetDeviceProperty(ResetData.UserId, ResetData.DeviceId, /* bForceReset= */ true);
	}
	
	PropertyClassesRequiringReset.Reset();
#endif	// WITH_EDITORONLY_DATA
	
	bIsPIEPlaying = false;
}
#endif	// WITH_EDITOR
