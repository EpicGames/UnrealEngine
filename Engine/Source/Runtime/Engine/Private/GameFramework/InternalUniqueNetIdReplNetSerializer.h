// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "InternalUniqueNetIdReplNetSerializer.generated.h"

USTRUCT()
struct FUniqueNetIdReplNetSerializerStringStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FString String;
};

USTRUCT()
struct FUniqueNetIdReplNetSerializerNameStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name;
};
