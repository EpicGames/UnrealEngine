// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerStateCountLimiterConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerStateCountLimiterConfig)

UPlayerStateCountLimiterConfig::UPlayerStateCountLimiterConfig()
: Super()
{
	Mode = ENetObjectCountLimiterMode::Fill;
	MaxObjectCount = 2;
	Priority = 1.0f;
	// We want the owning connection's player state to have high priority. 
	OwningConnectionPriority = 2.0f;
	bEnableOwnedObjectsFastLane = true;
}
