// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PrimitiveComponentUtilities.h"

#include "Components/PrimitiveComponent.h"
#include "GameFramework/UObjectAnnotationSparseActorList.h"
#include "Misc/ScopeRWLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrimitiveComponentUtilities)

// TInlineAllocator<4> because split screen is usually 2-4 players. 
static FUObjectAnnotationSparseActorList<TInlineAllocator<4>> VisibilityOwners;
UPrimitiveComponentUtilities::UPrimitiveComponentUtilities(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPrimitiveComponentUtilities::AddVisibilityOwner(UPrimitiveComponent* Component, AActor* Owner)
{
	if (!Component || !Owner)
	{
		return;
	}

	VisibilityOwners.AddOwner(Component, Owner);
	Component->MarkRenderStateDirty();
}

void UPrimitiveComponentUtilities::AddVisibilityOwnerActor(AActor* Actor, AActor* Owner, bool bRecursivelyIncludeAttachedActors)
{
	if (!Owner || !Actor)
	{
		return;
	}

	ForEachActorPrimitiveComponent(Actor, [Owner](TObjectPtr<UPrimitiveComponent> Component)
	{
		AddVisibilityOwner(Component, Owner);
	}, bRecursivelyIncludeAttachedActors);
}

void UPrimitiveComponentUtilities::RemoveVisibilityOwner(UPrimitiveComponent* Component, const AActor* Owner)
{
	if (!Component || !Owner)
	{
		return;
	}

	VisibilityOwners.RemoveOwner(Component, Owner);
	Component->MarkRenderStateDirty();
}

void UPrimitiveComponentUtilities::RemoveVisibilityOwnerActor(AActor* Actor, AActor* Owner, bool bRecursivelyIncludeAttachedActors)
{
	if (!Actor || !Owner)
	{
		return;
	}

	ForEachActorPrimitiveComponent(Actor, [Owner](TObjectPtr<UPrimitiveComponent> Component)
	{
		RemoveVisibilityOwner(Component, Owner);
	}, bRecursivelyIncludeAttachedActors);
}

void UPrimitiveComponentUtilities::ClearVisibilityOwners(UPrimitiveComponent* Component)
{
	if (!Component)
	{
		return;
	}

	TArray<const AActor*> Owners = GetVisibilityOwners(Component);
	for (const AActor* Owner : Owners)
	{
		RemoveVisibilityOwner(Component, Owner);
	}
}

TArray<const AActor*> UPrimitiveComponentUtilities::GetVisibilityOwners(const UPrimitiveComponent* Component)
{
	TArray<const AActor*> Result;
	if (!Component)
	{
		return Result;
	}
		
	FAnnotationSparseActorList Annotation = VisibilityOwners.GetAnnotation(Component);
	Result.Reserve(Annotation.OwnerList.Num());

	for (TWeakObjectPtr<const AActor> WeakOwner : Annotation.OwnerList)
	{
		if (const AActor* Owner = WeakOwner.Get())
		{
			Result.Add(Owner);
		}
	}

	return Result;
}

void UPrimitiveComponentUtilities::SetOnlyVisibleToOwner(AActor* Actor, bool bNewOnlyOwnerSee, bool bRecursivelyIncludeAttachedActors)
{
	if(!Actor)
	{
		return;
	}

	ForEachActorPrimitiveComponent(Actor, [bNewOnlyOwnerSee](TObjectPtr<UPrimitiveComponent> Component)
	{
		if (Component)
		{
			Component->SetOnlyOwnerSee(bNewOnlyOwnerSee);
			Component->MarkRenderStateDirty();
		}
	}, bRecursivelyIncludeAttachedActors);
}

void UPrimitiveComponentUtilities::ForEachVisibilityOwner(const UPrimitiveComponent* Component, TFunctionRef<void(TObjectPtr<const AActor>)> Func)
{
	if (!Component)
	{
		return;
	}

	FAnnotationSparseActorList Annotation = VisibilityOwners.GetAnnotation(Component);
	for (TWeakObjectPtr<const AActor> WeakOwner : Annotation.OwnerList)
	{
		if (const AActor* Owner = WeakOwner.Get())
		{
			Func(Owner);
		}
	}
}

void UPrimitiveComponentUtilities::ForEachActorPrimitiveComponent(AActor* Actor, TFunctionRef<void(TObjectPtr<UPrimitiveComponent>)> Func, bool bRecursivelyIncludeAttachedActors)
{
	if (Actor != nullptr)
	{
		TArray<AActor*> AttachedActors;
		AttachedActors.Add(Actor);

		if (bRecursivelyIncludeAttachedActors)
		{
			Actor->GetAttachedActors(AttachedActors, false, bRecursivelyIncludeAttachedActors);
		}

		for (AActor* AttachedActor : AttachedActors)
		{
			TInlineComponentArray<UPrimitiveComponent*> Components;
			AttachedActor->GetComponents(Components);

			for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
			{
				if (UPrimitiveComponent* PrimitiveComponent = Components[ComponentIndex])
				{
					Func(PrimitiveComponent);
				}
			}
		}
	}
}