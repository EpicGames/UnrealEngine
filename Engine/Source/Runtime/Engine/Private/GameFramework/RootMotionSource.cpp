// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/RootMotionSource.h"
#include "DrawDebugHelpers.h"
#include "Engine/World.h"
#include "GameFramework/Character.h"
#include "EngineLogs.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveFloat.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionSource)

#if ROOT_MOTION_DEBUG
TAutoConsoleVariable<int32> RootMotionSourceDebug::CVarDebugRootMotionSources(
	TEXT("p.RootMotion.Debug"),
	0,
	TEXT("Whether to draw root motion source debug information.\n")
	TEXT("0: Disable, 1: Enable"),
	ECVF_Cheat);

static TAutoConsoleVariable<float> CVarDebugRootMotionSourcesLifetime(
	TEXT("p.RootMotion.DebugSourceLifeTime"),
	6.f,
	TEXT("How long a visualized root motion source persists.\n")
	TEXT("Time in seconds each visualized root motion source persists."),
	ECVF_Cheat);

void RootMotionSourceDebug::PrintOnScreen(const ACharacter& InCharacter, const FString& InString)
{
	// Skip bots, debug player networking.
	if (InCharacter.IsPlayerControlled())
	{
		const FString AdjustedDebugString = FString::Printf(TEXT("[%" UINT64_FMT "] [%s] %s"), (uint64)GFrameCounter, *InCharacter.GetName(), *InString);

		// If on the server, replicate this message to everyone.
		if (!InCharacter.IsLocallyControlled() && (InCharacter.GetLocalRole() == ROLE_Authority))
		{
			for (FConstPlayerControllerIterator Iterator = InCharacter.GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				if (const APlayerController* const PlayerController = Iterator->Get())
				{
					if (ACharacter* const Character = PlayerController->GetCharacter())
					{
						Character->RootMotionDebugClientPrintOnScreen(AdjustedDebugString);
					}
				}
			}
		}
		else
		{
			const FColor DebugColor = (InCharacter.IsLocallyControlled()) ? FColor::Green : FColor::Purple;
			GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, DebugColor, AdjustedDebugString, false, FVector2D::UnitVector * 1.5f);

			UE_LOGF(LogRootMotion, Verbose, "%ls", *AdjustedDebugString);
		}
	}
}

void RootMotionSourceDebug::PrintOnScreenServerMsg(const FString& InString)
{
	const FColor DebugColor = FColor::Red;
	GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, DebugColor, InString, false, FVector2D::UnitVector * 1.5f);

	UE_LOGF(LogRootMotion, Verbose, "%ls", *InString);
}

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)


const float RootMotionSource_InvalidStartTime = -UE_BIG_NUMBER;


static float EvaluateFloatCurveAtFraction(const UCurveFloat& Curve, const float Fraction)
{
	float MinCurveTime(0.f);
	float MaxCurveTime(1.f);

	Curve.GetTimeRange(MinCurveTime, MaxCurveTime);
	return Curve.GetFloatValue(FMath::GetRangeValue(FVector2f(MinCurveTime, MaxCurveTime), Fraction));
}

static FVector EvaluateVectorCurveAtFraction(const UCurveVector& Curve, const float Fraction)
{
	float MinCurveTime(0.f);
	float MaxCurveTime(1.f);

	Curve.GetTimeRange(MinCurveTime, MaxCurveTime);
	return Curve.GetVectorValue(FMath::GetRangeValue(FVector2f(MinCurveTime, MaxCurveTime), Fraction));
}


//
// FRootMotionServerToLocalIDMapping
//

FRootMotionServerToLocalIDMapping::FRootMotionServerToLocalIDMapping()
	: ServerID(0)
	, LocalID(0)
	, TimeStamp(0.0f)
{
}

bool FRootMotionServerToLocalIDMapping::IsStillValid(float CurrentTimeStamp)
{
	const float MappingValidityDuration = 3.0f; // Mappings updated within this many seconds are still valid
	return TimeStamp >= (CurrentTimeStamp - MappingValidityDuration);
}

//
// FRootMotionSourceStatus
//

FRootMotionSourceStatus::FRootMotionSourceStatus()
	: Flags(0)
{
}

void FRootMotionSourceStatus::Clear()
{
	Flags = 0;
}

void FRootMotionSourceStatus::SetFlag(ERootMotionSourceStatusFlags Flag)
{
	Flags |= (uint8)Flag;
}

void FRootMotionSourceStatus::UnSetFlag(ERootMotionSourceStatusFlags Flag)
{
	Flags &= ~((uint8)Flag);
}

bool FRootMotionSourceStatus::HasFlag(ERootMotionSourceStatusFlags Flag) const
{
	return (Flags & (uint8)Flag) != 0;
}

//
// FRootMotionSourceSettings
//

FRootMotionSourceSettings::FRootMotionSourceSettings()
	: Flags(0)
{
}

void FRootMotionSourceSettings::Clear()
{
	Flags = 0;
}

void FRootMotionSourceSettings::SetFlag(ERootMotionSourceSettingsFlags Flag)
{
	Flags |= (uint8)Flag;
}

void FRootMotionSourceSettings::UnSetFlag(ERootMotionSourceSettingsFlags Flag)
{
	Flags &= ~((uint8)Flag);
}

bool FRootMotionSourceSettings::HasFlag(ERootMotionSourceSettingsFlags Flag) const
{
	return (Flags & (uint8)Flag) != 0;
}

FRootMotionSourceSettings& FRootMotionSourceSettings::operator+=(const FRootMotionSourceSettings& Other)
{
	Flags |= Other.Flags;
	return *this;
}

//
// FRootMotionSource
//

FRootMotionSource::FRootMotionSource()
	: Priority(0)
	, LocalID((uint16)ERootMotionSourceID::Invalid)
	, StartTime(RootMotionSource_InvalidStartTime)
	, CurrentTime(0.0f)
	, PreviousTime(0.0f)
	, Duration(-1.0f)
	, AccumulateMode(ERootMotionAccumulateMode::Override)
	, bInLocalSpace(false)
	, bNeedsSimulatedCatchup(false)
	, bSimulatedNeedsSmoothing(false)
{
}

float FRootMotionSource::GetTime() const
{
	return CurrentTime;
}

float FRootMotionSource::GetStartTime() const
{
	return StartTime;
}

bool FRootMotionSource::IsStartTimeValid() const
{
	return (StartTime != RootMotionSource_InvalidStartTime);
}

float FRootMotionSource::GetDuration() const
{
	return Duration;
}

bool FRootMotionSource::IsTimeOutEnabled() const
{
	return Duration >= 0.f;
}

FRootMotionSource* FRootMotionSource::Clone() const
{
	// If child classes don't override this, savedmoves will not work
	checkf(false, TEXT("FRootMotionSource::Clone() being called erroneously. This should always be overridden in child classes!"));
	return nullptr;
}

bool FRootMotionSource::IsActive() const
{
	return true;
}

bool FRootMotionSource::Matches(const FRootMotionSource* Other) const
{
	return Other != nullptr && 
		GetScriptStruct() == Other->GetScriptStruct() && 
		Priority == Other->Priority &&
		AccumulateMode == Other->AccumulateMode &&
		bInLocalSpace == Other->bInLocalSpace &&
		InstanceName == Other->InstanceName &&
		FMath::IsNearlyEqual(Duration, Other->Duration, UE_SMALL_NUMBER);
}

bool FRootMotionSource::MatchesAndHasSameState(const FRootMotionSource* Other) const
{
	// Check that it matches
	if (!Matches(Other))
	{
		return false;
	}

	// Check state
	return Status.Flags == Other->Status.Flags && GetTime() == Other->GetTime();
}

bool FRootMotionSource::UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	if (SourceToTakeStateFrom != nullptr)
	{
		if (GetScriptStruct() == SourceToTakeStateFrom->GetScriptStruct())
		{
			bNeedsSimulatedCatchup = bMarkForSimulatedCatchup;

			const bool bWasMarkedForRemoval = Status.HasFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
			Status = SourceToTakeStateFrom->Status;
			// Never undo removal when updating state from another source, should always be guaranteed
			if (bWasMarkedForRemoval)
			{
				Status.SetFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
			}

			SetTime(SourceToTakeStateFrom->GetTime());
			return true;
		}
		else
		{
			// UpdateStateFrom() should only be called on matching Sources. If we hit this case,
			// we have an issue with Matches() and/or LocalIDs being mapped to invalid "partners"
			checkf(false, TEXT("FRootMotionSource::UpdateStateFrom() is being updated from non-matching Source!"));
		}
	}

	return false;
}

void FRootMotionSource::SetTime(float NewTime)
{
	PreviousTime = CurrentTime;
	CurrentTime = NewTime;

	CheckTimeOut();
}

void FRootMotionSource::CheckTimeOut()
{
	// If I'm beyond my duration, I'm finished and can be removed
	if (IsTimeOutEnabled())
	{
		const bool bTimedOut = CurrentTime >= Duration;
		if (bTimedOut)
		{
			Status.SetFlag(ERootMotionSourceStatusFlags::Finished);
		}
		else
		{
			Status.UnSetFlag(ERootMotionSourceStatusFlags::Finished);
		}
	}
}

void FRootMotionSource::PrepareRootMotion
	(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
	)
{
	RootMotionParams.Clear();
}

bool FRootMotionSource::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Ar << Priority;
	Ar << LocalID;

	uint8 AccumulateModeSerialize = (uint8) AccumulateMode;
	Ar << AccumulateModeSerialize;
	AccumulateMode = (ERootMotionAccumulateMode) AccumulateModeSerialize;

	Ar << InstanceName;
	Ar << CurrentTime;
	Ar << Duration;
	Ar << Status.Flags;
	Ar << bInLocalSpace;
	//Ar << RootMotionParams; // Do we need this for simulated proxies?

	bOutSuccess = true;
	return true;
}

UScriptStruct* FRootMotionSource::GetScriptStruct() const
{
	return FRootMotionSource::StaticStruct();
}

FString FRootMotionSource::ToSimpleString() const
{
	return FString::Printf(TEXT("[ID:%u] FRootMotionSource %s"), LocalID, *InstanceName.GetPlainNameString());
}

//
// FRootMotionSource_ConstantForce
//

FRootMotionSource_ConstantForce::FRootMotionSource_ConstantForce()
	: Force(ForceInitToZero)
	, StrengthOverTime(nullptr)
{
	// Disable Partial End Tick for Constant Forces.
	// Otherwise we end up with very inconsistent velocities on the last frame.
	// This ensures that the ending velocity is maintained and consistent.
	Settings.SetFlag(ERootMotionSourceSettingsFlags::DisablePartialEndTick);
}

FRootMotionSource* FRootMotionSource_ConstantForce::Clone() const
{
	FRootMotionSource_ConstantForce* CopyPtr = new FRootMotionSource_ConstantForce(*this);
	return CopyPtr;
}

bool FRootMotionSource_ConstantForce::Matches(const FRootMotionSource* Other) const
{
	if (!FRootMotionSource::Matches(Other))
	{
		return false;
	}

	// We can cast safely here since in FRootMotionSource::Matches() we ensured ScriptStruct equality
	const FRootMotionSource_ConstantForce* OtherCast = static_cast<const FRootMotionSource_ConstantForce*>(Other);

	return FVector::PointsAreNear(Force, OtherCast->Force, 0.1f) &&
		StrengthOverTime == OtherCast->StrengthOverTime;
}

bool FRootMotionSource_ConstantForce::MatchesAndHasSameState(const FRootMotionSource* Other) const
{
	// Check that it matches
	if (!FRootMotionSource::MatchesAndHasSameState(Other))
	{
		return false;
	}

	return true; // ConstantForce has no unique state
}

bool FRootMotionSource_ConstantForce::UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	if (!FRootMotionSource::UpdateStateFrom(SourceToTakeStateFrom, bMarkForSimulatedCatchup))
	{
		return false;
	}

	return true; // ConstantForce has no unique state other than Time which is handled by FRootMotionSource
}

void FRootMotionSource_ConstantForce::PrepareRootMotion
	(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
	)
{
	RootMotionParams.Clear();

	FTransform NewTransform(Force);

	// Scale strength of force over time
	if (StrengthOverTime)
	{
		const float TimeValue = Duration > 0.f ? FMath::Clamp(GetTime() / Duration, 0.f, 1.f) : GetTime();
		const float TimeFactor = StrengthOverTime->GetFloatValue(TimeValue);
		NewTransform.ScaleTranslation(TimeFactor);
	}

	// Scale force based on Simulation/MovementTime differences
	// Ex: Force is to go 200 cm per second forward.
	//     To catch up with server state we need to apply
	//     3 seconds of this root motion in 1 second of
	//     movement tick time -> we apply 600 cm for this frame
	const float Multiplier = (MovementTickTime > UE_SMALL_NUMBER) ? (SimulationTime / MovementTickTime) : 1.f;
	NewTransform.ScaleTranslation(Multiplier);

#if ROOT_MOTION_DEBUG
	if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnAnyThread() == 1)
	{
		FString AdjustedDebugString = FString::Printf(TEXT("FRootMotionSource_ConstantForce::PrepareRootMotion NewTransform(%s) Multiplier(%f)"),
			*NewTransform.GetTranslation().ToCompactString(), Multiplier);
		RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
	}
#endif

	RootMotionParams.Set(NewTransform);

	SetTime(GetTime() + SimulationTime);
}

bool FRootMotionSource_ConstantForce::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	if (!FRootMotionSource::NetSerialize(Ar, Map, bOutSuccess))
	{
		return false;
	}

	Ar << Force; // TODO-RootMotionSource: Quantization
	Ar << StrengthOverTime;

	bOutSuccess = true;
	return true;
}

UScriptStruct* FRootMotionSource_ConstantForce::GetScriptStruct() const
{
	return FRootMotionSource_ConstantForce::StaticStruct();
}

FString FRootMotionSource_ConstantForce::ToSimpleString() const
{
	return FString::Printf(TEXT("[ID:%u]FRootMotionSource_ConstantForce %s"), LocalID, *InstanceName.GetPlainNameString());
}

void FRootMotionSource_ConstantForce::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(StrengthOverTime);

	FRootMotionSource::AddReferencedObjects(Collector);
}

//
// FRootMotionSource_RadialForce
//

FRootMotionSource_RadialForce::FRootMotionSource_RadialForce()
	: Location(ForceInitToZero)
	, LocationActor(nullptr)
	, Radius(1.f)
	, Strength(0.f)
	, bIsPush(true)
	, bNoZForce(false)
	, StrengthDistanceFalloff(nullptr)
	, StrengthOverTime(nullptr)
	, bUseFixedWorldDirection(false)
	, FixedWorldDirection(ForceInitToZero)
{
}

FRootMotionSource* FRootMotionSource_RadialForce::Clone() const
{
	FRootMotionSource_RadialForce* CopyPtr = new FRootMotionSource_RadialForce(*this);
	return CopyPtr;
}

bool FRootMotionSource_RadialForce::Matches(const FRootMotionSource* Other) const
{
	if (!FRootMotionSource::Matches(Other))
	{
		return false;
	}

	// We can cast safely here since in FRootMotionSource::Matches() we ensured ScriptStruct equality
	const FRootMotionSource_RadialForce* OtherCast = static_cast<const FRootMotionSource_RadialForce*>(Other);

	return bIsPush == OtherCast->bIsPush &&
		bNoZForce == OtherCast->bNoZForce &&
		bUseFixedWorldDirection == OtherCast->bUseFixedWorldDirection &&
		StrengthDistanceFalloff == OtherCast->StrengthDistanceFalloff &&
		StrengthOverTime == OtherCast->StrengthOverTime &&
		(LocationActor == OtherCast->LocationActor ||
		FVector::PointsAreNear(Location, OtherCast->Location, 1.0f)) &&
		FMath::IsNearlyEqual(Radius, OtherCast->Radius, UE_SMALL_NUMBER) &&
		FMath::IsNearlyEqual(Strength, OtherCast->Strength, UE_SMALL_NUMBER) &&
		FixedWorldDirection.Equals(OtherCast->FixedWorldDirection, 3.0f);
}

bool FRootMotionSource_RadialForce::MatchesAndHasSameState(const FRootMotionSource* Other) const
{
	// Check that it matches
	if (!FRootMotionSource::MatchesAndHasSameState(Other))
	{
		return false;
	}

	return true; // RadialForce has no unique state
}

bool FRootMotionSource_RadialForce::UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	if (!FRootMotionSource::UpdateStateFrom(SourceToTakeStateFrom, bMarkForSimulatedCatchup))
	{
		return false;
	}

	return true; // RadialForce has no unique state other than Time which is handled by FRootMotionSource
}

void FRootMotionSource_RadialForce::PrepareRootMotion
	(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
	)
{
	RootMotionParams.Clear();

	const FVector CharacterLocation = Character.GetActorLocation();
	FVector Force = FVector::ZeroVector;
	const FVector ForceLocation = LocationActor ? LocationActor->GetActorLocation() : Location;
	float Distance = FVector::Dist(ForceLocation, CharacterLocation);
	if (Distance < Radius)
	{
		// Calculate strength
		float CurrentStrength = Strength;
		{
			float AdditiveStrengthFactor = 1.f;
			if (StrengthDistanceFalloff)
			{
				const float DistanceFactor = StrengthDistanceFalloff->GetFloatValue(FMath::Clamp(Distance / Radius, 0.f, 1.f));
				AdditiveStrengthFactor -= (1.f - DistanceFactor);
			}

			if (StrengthOverTime)
			{
				const float TimeValue = Duration > 0.f ? FMath::Clamp(GetTime() / Duration, 0.f, 1.f) : GetTime();
				const float TimeFactor = StrengthOverTime->GetFloatValue(TimeValue);
				AdditiveStrengthFactor -= (1.f - TimeFactor);
			}

			CurrentStrength = Strength * FMath::Clamp(AdditiveStrengthFactor, 0.f, 1.f);
		}

		if (bUseFixedWorldDirection)
		{
			Force = FixedWorldDirection.Vector() * CurrentStrength;
		}
		else
		{
			Force = (ForceLocation - CharacterLocation).GetSafeNormal() * CurrentStrength;
			
			if (bIsPush)
			{
				Force *= -1.f;
			}
		}
	}

	if (bNoZForce)
	{
		Force.Z = 0.f;
	}

	FTransform NewTransform(Force);

	// Scale force based on Simulation/MovementTime differences
	// Ex: Force is to go 200 cm per second forward.
	//     To catch up with server state we need to apply
	//     3 seconds of this root motion in 1 second of
	//     movement tick time -> we apply 600 cm for this frame
	if (SimulationTime != MovementTickTime && MovementTickTime > UE_SMALL_NUMBER)
	{
		const float Multiplier = SimulationTime / MovementTickTime;
		NewTransform.ScaleTranslation(Multiplier);
	}

	RootMotionParams.Set(NewTransform);

	SetTime(GetTime() + SimulationTime);
}

bool FRootMotionSource_RadialForce::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	if (!FRootMotionSource::NetSerialize(Ar, Map, bOutSuccess))
	{
		return false;
	}

	Ar << Location; // TODO-RootMotionSource: Quantization
	Ar << LocationActor;
	Ar << Radius;
	Ar << Strength;
	Ar << bIsPush;
	Ar << bNoZForce;
	Ar << StrengthDistanceFalloff;
	Ar << StrengthOverTime;
	Ar << bUseFixedWorldDirection;
	Ar << FixedWorldDirection;

	bOutSuccess = true;
	return true;
}

UScriptStruct* FRootMotionSource_RadialForce::GetScriptStruct() const
{
	return FRootMotionSource_RadialForce::StaticStruct();
}

FString FRootMotionSource_RadialForce::ToSimpleString() const
{
	return FString::Printf(TEXT("[ID:%u]FRootMotionSource_RadialForce %s"), LocalID, *InstanceName.GetPlainNameString());
}

void FRootMotionSource_RadialForce::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(LocationActor);
	Collector.AddReferencedObject(StrengthDistanceFalloff);
	Collector.AddReferencedObject(StrengthOverTime);

	FRootMotionSource::AddReferencedObjects(Collector);
}

//
// FRootMotionSource_MoveToForce
//

FRootMotionSource_MoveToForce::FRootMotionSource_MoveToForce()
	: StartLocation(ForceInitToZero)
	, TargetLocation(ForceInitToZero)
	, bRestrictSpeedToExpected(false)
	, PathOffsetCurve(nullptr)
{
}

FRootMotionSource* FRootMotionSource_MoveToForce::Clone() const
{
	FRootMotionSource_MoveToForce* CopyPtr = new FRootMotionSource_MoveToForce(*this);
	return CopyPtr;
}

bool FRootMotionSource_MoveToForce::Matches(const FRootMotionSource* Other) const
{
	if (!FRootMotionSource::Matches(Other))
	{
		return false;
	}

	// We can cast safely here since in FRootMotionSource::Matches() we ensured ScriptStruct equality
	const FRootMotionSource_MoveToForce* OtherCast = static_cast<const FRootMotionSource_MoveToForce*>(Other);

	return bRestrictSpeedToExpected == OtherCast->bRestrictSpeedToExpected &&
		PathOffsetCurve == OtherCast->PathOffsetCurve &&
		FVector::PointsAreNear(TargetLocation, OtherCast->TargetLocation, 0.1f);
}

bool FRootMotionSource_MoveToForce::MatchesAndHasSameState(const FRootMotionSource* Other) const
{
	// Check that it matches
	if (!FRootMotionSource::MatchesAndHasSameState(Other))
	{
		return false;
	}

	return true; // MoveToForce has no unique state
}

bool FRootMotionSource_MoveToForce::UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	if (!FRootMotionSource::UpdateStateFrom(SourceToTakeStateFrom, bMarkForSimulatedCatchup))
	{
		return false;
	}

	return true; // MoveToForce has no unique state other than Time which is handled by FRootMotionSource
}

void FRootMotionSource_MoveToForce::SetTime(float NewTime)
{
	FRootMotionSource::SetTime(NewTime);

	// TODO-RootMotionSource: Check if reached destination?
}

FVector FRootMotionSource_MoveToForce::GetPathOffsetInWorldSpace(const float MoveFraction) const
{
	if (PathOffsetCurve)
	{
		// Calculate path offset
		const FVector PathOffsetInFacingSpace = EvaluateVectorCurveAtFraction(*PathOffsetCurve, MoveFraction);
		FRotator FacingRotation((TargetLocation-StartLocation).Rotation());
		FacingRotation.Pitch = 0.f; // By default we don't include pitch in the offset, but an option could be added if necessary
		return FacingRotation.RotateVector(PathOffsetInFacingSpace);
	}

	return FVector::ZeroVector;
}

void FRootMotionSource_MoveToForce::PrepareRootMotion
	(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
	)
{
	RootMotionParams.Clear();

	if (Duration > UE_SMALL_NUMBER && MovementTickTime > UE_SMALL_NUMBER)
	{
		const float MoveFraction = (GetTime() + SimulationTime) / Duration;

		FVector CurrentTargetLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, MoveFraction);
		CurrentTargetLocation += GetPathOffsetInWorldSpace(MoveFraction);

		const FVector CurrentLocation = Character.GetActorLocation();

		FVector Force = (CurrentTargetLocation - CurrentLocation) / MovementTickTime;

		if (bRestrictSpeedToExpected && !Force.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
		{
			// Calculate expected current location (if we didn't have collision and moved exactly where our velocity should have taken us)
			const float PreviousMoveFraction = GetTime() / Duration;
			FVector CurrentExpectedLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, PreviousMoveFraction);
			CurrentExpectedLocation += GetPathOffsetInWorldSpace(PreviousMoveFraction);

			// Restrict speed to the expected speed, allowing some small amount of error
			const FVector ExpectedForce = (CurrentTargetLocation - CurrentExpectedLocation) / MovementTickTime;
			const float ExpectedSpeed = ExpectedForce.Size();
			const float CurrentSpeedSqr = Force.SizeSquared();

			const float ErrorAllowance = 0.5f; // in cm/s
			if (CurrentSpeedSqr > FMath::Square(ExpectedSpeed + ErrorAllowance))
			{
				Force.Normalize();
				Force *= ExpectedSpeed;
			}
		}

		// Debug
#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() != 0)
		{
			const FVector LocDiff = MoveComponent.UpdatedComponent->GetComponentLocation() - CurrentLocation;
			const float DebugLifetime = CVarDebugRootMotionSourcesLifetime.GetValueOnGameThread();

			// Current
			DrawDebugCapsule(Character.GetWorld(), MoveComponent.UpdatedComponent->GetComponentLocation(), Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, true, DebugLifetime);

			// Current Target
			DrawDebugCapsule(Character.GetWorld(), CurrentTargetLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Green, true, DebugLifetime);

			// Target
			DrawDebugCapsule(Character.GetWorld(), TargetLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Blue, true, DebugLifetime);

			// Force
			DrawDebugLine(Character.GetWorld(), CurrentLocation, CurrentLocation+Force, FColor::Blue, true, DebugLifetime);
		}
#endif

		FTransform NewTransform(Force);
		RootMotionParams.Set(NewTransform);
	}
	else
	{
		checkf(Duration > UE_SMALL_NUMBER, TEXT("FRootMotionSource_MoveToForce prepared with invalid duration."));
	}

	SetTime(GetTime() + SimulationTime);
}

bool FRootMotionSource_MoveToForce::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	if (!FRootMotionSource::NetSerialize(Ar, Map, bOutSuccess))
	{
		return false;
	}

	Ar << StartLocation; // TODO-RootMotionSource: Quantization
	Ar << TargetLocation; // TODO-RootMotionSource: Quantization
	Ar << bRestrictSpeedToExpected;
	Ar << PathOffsetCurve;

	bOutSuccess = true;
	return true;
}

UScriptStruct* FRootMotionSource_MoveToForce::GetScriptStruct() const
{
	return FRootMotionSource_MoveToForce::StaticStruct();
}

FString FRootMotionSource_MoveToForce::ToSimpleString() const
{
	return FString::Printf(TEXT("[ID:%u]FRootMotionSource_MoveToForce %s"), LocalID, *InstanceName.GetPlainNameString());
}

void FRootMotionSource_MoveToForce::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PathOffsetCurve);

	FRootMotionSource::AddReferencedObjects(Collector);
}

//
// FRootMotionSource_MoveToDynamicForce
//

FRootMotionSource_MoveToDynamicForce::FRootMotionSource_MoveToDynamicForce()
	: StartLocation(ForceInitToZero)
	, InitialTargetLocation(ForceInitToZero)
	, TargetLocation(ForceInitToZero)
	, bRestrictSpeedToExpected(false)
	, PathOffsetCurve(nullptr)
	, TimeMappingCurve(nullptr)
{
}

void FRootMotionSource_MoveToDynamicForce::SetTargetLocation(FVector NewTargetLocation)
{
	TargetLocation = NewTargetLocation;
}

FRootMotionSource* FRootMotionSource_MoveToDynamicForce::Clone() const
{
	FRootMotionSource_MoveToDynamicForce* CopyPtr = new FRootMotionSource_MoveToDynamicForce(*this);
	return CopyPtr;
}

bool FRootMotionSource_MoveToDynamicForce::Matches(const FRootMotionSource* Other) const
{
	if (!FRootMotionSource::Matches(Other))
	{
		return false;
	}

	// We can cast safely here since in FRootMotionSource::Matches() we ensured ScriptStruct equality
	const FRootMotionSource_MoveToDynamicForce* OtherCast = static_cast<const FRootMotionSource_MoveToDynamicForce*>(Other);

	return bRestrictSpeedToExpected == OtherCast->bRestrictSpeedToExpected &&
		PathOffsetCurve == OtherCast->PathOffsetCurve &&
		TimeMappingCurve == OtherCast->TimeMappingCurve;
}

bool FRootMotionSource_MoveToDynamicForce::MatchesAndHasSameState(const FRootMotionSource* Other) const
{
	// Check that it matches
	if (!FRootMotionSource::MatchesAndHasSameState(Other))
	{
		return false;
	}
	
	// We can cast safely here since in FRootMotionSource::Matches() we ensured ScriptStruct equality
	const FRootMotionSource_MoveToDynamicForce* OtherCast = static_cast<const FRootMotionSource_MoveToDynamicForce*>(Other);

	return (StartLocation.Equals(OtherCast->StartLocation) &&
			TargetLocation.Equals(OtherCast->TargetLocation));
}

bool FRootMotionSource_MoveToDynamicForce::UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	if (!FRootMotionSource::UpdateStateFrom(SourceToTakeStateFrom, bMarkForSimulatedCatchup))
	{
		return false;
	}

	// We can cast safely here since in FRootMotionSource::UpdateStateFrom() we ensured ScriptStruct equality
	const FRootMotionSource_MoveToDynamicForce* OtherCast = static_cast<const FRootMotionSource_MoveToDynamicForce*>(SourceToTakeStateFrom);

	StartLocation = OtherCast->StartLocation;
	TargetLocation = OtherCast->TargetLocation;

	return true;
}

void FRootMotionSource_MoveToDynamicForce::SetTime(float NewTime)
{
	FRootMotionSource::SetTime(NewTime);

	// TODO-RootMotionSource: Check if reached destination?
}

FVector FRootMotionSource_MoveToDynamicForce::GetPathOffsetInWorldSpace(const float MoveFraction) const
{
	if (PathOffsetCurve)
	{
		// Calculate path offset
		const FVector PathOffsetInFacingSpace = EvaluateVectorCurveAtFraction(*PathOffsetCurve, MoveFraction);
		FRotator FacingRotation((TargetLocation-StartLocation).Rotation());
		FacingRotation.Pitch = 0.f; // By default we don't include pitch in the offset, but an option could be added if necessary
		return FacingRotation.RotateVector(PathOffsetInFacingSpace);
	}

	return FVector::ZeroVector;
}

void FRootMotionSource_MoveToDynamicForce::PrepareRootMotion
	(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
	)
{
	RootMotionParams.Clear();

	if (Duration > UE_SMALL_NUMBER && MovementTickTime > UE_SMALL_NUMBER)
	{
		float MoveFraction = (GetTime() + SimulationTime) / Duration;
		
		if (TimeMappingCurve)
		{
			MoveFraction = EvaluateFloatCurveAtFraction(*TimeMappingCurve, MoveFraction);
		}

		FVector CurrentTargetLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, MoveFraction);
		CurrentTargetLocation += GetPathOffsetInWorldSpace(MoveFraction);

		const FVector CurrentLocation = Character.GetActorLocation();

		FVector Force = (CurrentTargetLocation - CurrentLocation) / MovementTickTime;

		if (bRestrictSpeedToExpected && !Force.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
		{
			// Calculate expected current location (if we didn't have collision and moved exactly where our velocity should have taken us)
			float PreviousMoveFraction = GetTime() / Duration;
			if (TimeMappingCurve)
			{
				PreviousMoveFraction = EvaluateFloatCurveAtFraction(*TimeMappingCurve, PreviousMoveFraction);
			}

			FVector CurrentExpectedLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, PreviousMoveFraction);
			CurrentExpectedLocation += GetPathOffsetInWorldSpace(PreviousMoveFraction);

			// Restrict speed to the expected speed, allowing some small amount of error
			const FVector ExpectedForce = (CurrentTargetLocation - CurrentExpectedLocation) / MovementTickTime;
			const float ExpectedSpeed = ExpectedForce.Size();
			const float CurrentSpeedSqr = Force.SizeSquared();

			const float ErrorAllowance = 0.5f; // in cm/s
			if (CurrentSpeedSqr > FMath::Square(ExpectedSpeed + ErrorAllowance))
			{
				Force.Normalize();
				Force *= ExpectedSpeed;
			}
		}

		// Debug
#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() != 0)
		{
			const FVector LocDiff = MoveComponent.UpdatedComponent->GetComponentLocation() - CurrentLocation;
			const float DebugLifetime = CVarDebugRootMotionSourcesLifetime.GetValueOnGameThread();

			// Current
			DrawDebugCapsule(Character.GetWorld(), MoveComponent.UpdatedComponent->GetComponentLocation(), Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, true, DebugLifetime);

			// Current Target
			DrawDebugCapsule(Character.GetWorld(), CurrentTargetLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Green, true, DebugLifetime);

			// Target
			DrawDebugCapsule(Character.GetWorld(), TargetLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Blue, true, DebugLifetime);

			// Force
			DrawDebugLine(Character.GetWorld(), CurrentLocation, CurrentLocation+Force, FColor::Blue, true, DebugLifetime);
		}
#endif

		FTransform NewTransform(Force);
		RootMotionParams.Set(NewTransform);
	}
	else
	{
		checkf(Duration > UE_SMALL_NUMBER, TEXT("FRootMotionSource_MoveToDynamicForce prepared with invalid duration."));
	}

	SetTime(GetTime() + SimulationTime);
}

bool FRootMotionSource_MoveToDynamicForce::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	if (!FRootMotionSource::NetSerialize(Ar, Map, bOutSuccess))
	{
		return false;
	}

	Ar << StartLocation; // TODO-RootMotionSource: Quantization
	Ar << InitialTargetLocation; // TODO-RootMotionSource: Quantization
	Ar << TargetLocation; // TODO-RootMotionSource: Quantization
	Ar << bRestrictSpeedToExpected;
	Ar << PathOffsetCurve;
	Ar << TimeMappingCurve;

	bOutSuccess = true;
	return true;
}

UScriptStruct* FRootMotionSource_MoveToDynamicForce::GetScriptStruct() const
{
	return FRootMotionSource_MoveToDynamicForce::StaticStruct();
}

FString FRootMotionSource_MoveToDynamicForce::ToSimpleString() const
{
	return FString::Printf(TEXT("[ID:%u]FRootMotionSource_MoveToDynamicForce %s"), LocalID, *InstanceName.GetPlainNameString());
}

void FRootMotionSource_MoveToDynamicForce::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PathOffsetCurve);
	Collector.AddReferencedObject(TimeMappingCurve);

	FRootMotionSource::AddReferencedObjects(Collector);
}


//
// FRootMotionSource_JumpForce
//

FRootMotionSource_JumpForce::FRootMotionSource_JumpForce()
	: Rotation(ForceInitToZero)
	, Distance(-1.0f)
	, Height(-1.0f)
	, bDisableTimeout(false)
	, PathOffsetCurve(nullptr)
	, TimeMappingCurve(nullptr)
	, SavedHalfwayLocation(FVector::ZeroVector)
{
	// Don't allow partial end ticks. Jump forces are meant to provide velocity that
	// carries through to the end of the jump, and if we do partial ticks at the very end,
	// it means the provided velocity can be significantly reduced on the very last tick,
	// resulting in lost momentum. This is not desirable for jumps.
	Settings.SetFlag(ERootMotionSourceSettingsFlags::DisablePartialEndTick);
}

bool FRootMotionSource_JumpForce::IsTimeOutEnabled() const
{
	if (bDisableTimeout)
	{
		return false;
	}
	return FRootMotionSource::IsTimeOutEnabled();
}

FRootMotionSource* FRootMotionSource_JumpForce::Clone() const
{
	FRootMotionSource_JumpForce* CopyPtr = new FRootMotionSource_JumpForce(*this);
	return CopyPtr;
}

bool FRootMotionSource_JumpForce::Matches(const FRootMotionSource* Other) const
{
	if (!FRootMotionSource::Matches(Other))
	{
		return false;
	}

	// We can cast safely here since in FRootMotionSource::Matches() we ensured ScriptStruct equality
	const FRootMotionSource_JumpForce* OtherCast = static_cast<const FRootMotionSource_JumpForce*>(Other);

	return bDisableTimeout == OtherCast->bDisableTimeout &&
		PathOffsetCurve == OtherCast->PathOffsetCurve &&
		TimeMappingCurve == OtherCast->TimeMappingCurve &&
		FMath::IsNearlyEqual(Distance, OtherCast->Distance, UE_SMALL_NUMBER) &&
		FMath::IsNearlyEqual(Height, OtherCast->Height, UE_SMALL_NUMBER) &&
		Rotation.Equals(OtherCast->Rotation, 1.0f);
}

bool FRootMotionSource_JumpForce::MatchesAndHasSameState(const FRootMotionSource* Other) const
{
	// Check that it matches
	if (!FRootMotionSource::MatchesAndHasSameState(Other))
	{
		return false;
	}

	return true; // JumpForce has no unique state
}

bool FRootMotionSource_JumpForce::UpdateStateFrom(const FRootMotionSource* SourceToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	if (!FRootMotionSource::UpdateStateFrom(SourceToTakeStateFrom, bMarkForSimulatedCatchup))
	{
		return false;
	}

	return true; // JumpForce has no unique state other than Time which is handled by FRootMotionSource
}

FVector FRootMotionSource_JumpForce::GetPathOffset(const float MoveFraction) const
{
	FVector PathOffset(FVector::ZeroVector);
	if (PathOffsetCurve)
	{
		// Calculate path offset
		PathOffset = EvaluateVectorCurveAtFraction(*PathOffsetCurve, MoveFraction);
	}
	else
	{
		// Default to "jump parabola", a simple x^2 shifted to be upside-down and shifted
		// to get [0,1] X (MoveFraction/Distance) mapping to [0,1] Y (height)
		// Height = -(2x-1)^2 + 1
		const float Phi = 2.f*MoveFraction - 1;
		const float Z = -(Phi*Phi) + 1;
		PathOffset.Z = Z;
	}

	// Scale Z offset to height. If Height < 0, we use direct path offset values
	if (Height >= 0.f)
	{
		PathOffset.Z *= Height;
	}

	return PathOffset;
}

FVector FRootMotionSource_JumpForce::GetRelativeLocation(float MoveFraction) const
{
	// Given MoveFraction, what relative location should a character be at?
	FRotator FacingRotation(Rotation);
	FacingRotation.Pitch = 0.f; // By default we don't include pitch, but an option could be added if necessary

	FVector RelativeLocationFacingSpace = FVector(MoveFraction * Distance, 0.f, 0.f) + GetPathOffset(MoveFraction);

	return FacingRotation.RotateVector(RelativeLocationFacingSpace);
}

void FRootMotionSource_JumpForce::PrepareRootMotion
	(
		float SimulationTime, 
		float MovementTickTime,
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent
	)
{
	RootMotionParams.Clear();

	if (Duration > UE_SMALL_NUMBER && MovementTickTime > UE_SMALL_NUMBER && SimulationTime > UE_SMALL_NUMBER)
	{
		float CurrentTimeFraction = GetTime() / Duration;
		float TargetTimeFraction = (GetTime() + SimulationTime) / Duration;

		// If we're beyond specified duration, we need to re-map times so that
		// we continue our desired ending velocity
		if (TargetTimeFraction > 1.f)
		{
			float TimeFractionPastAllowable = TargetTimeFraction - 1.0f;
			TargetTimeFraction -= TimeFractionPastAllowable;
			CurrentTimeFraction -= TimeFractionPastAllowable;
		}

		float CurrentMoveFraction = CurrentTimeFraction;
		float TargetMoveFraction = TargetTimeFraction;

		if (TimeMappingCurve)
		{
			CurrentMoveFraction = EvaluateFloatCurveAtFraction(*TimeMappingCurve, CurrentMoveFraction);
			TargetMoveFraction  = EvaluateFloatCurveAtFraction(*TimeMappingCurve, TargetMoveFraction);
		}

		const FVector CurrentRelativeLocation = GetRelativeLocation(CurrentMoveFraction);
		const FVector TargetRelativeLocation = GetRelativeLocation(TargetMoveFraction);

		const FVector Force = (TargetRelativeLocation - CurrentRelativeLocation) / MovementTickTime;

		// Debug
#if ROOT_MOTION_DEBUG
		if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnGameThread() != 0)
		{
			const FVector CurrentLocation = Character.GetActorLocation();
			const FVector CurrentTargetLocation = CurrentLocation + (TargetRelativeLocation - CurrentRelativeLocation);
			const FVector LocDiff = MoveComponent.UpdatedComponent->GetComponentLocation() - CurrentLocation;
			const float DebugLifetime = CVarDebugRootMotionSourcesLifetime.GetValueOnGameThread();

			// Current
			DrawDebugCapsule(Character.GetWorld(), MoveComponent.UpdatedComponent->GetComponentLocation(), Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, true, DebugLifetime);

			// Current Target
			DrawDebugCapsule(Character.GetWorld(), CurrentTargetLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Green, true, DebugLifetime);

			// Target
			DrawDebugCapsule(Character.GetWorld(), CurrentTargetLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::Blue, true, DebugLifetime);

			// Force
			DrawDebugLine(Character.GetWorld(), CurrentLocation, CurrentLocation+Force, FColor::Blue, true, DebugLifetime);

			// Halfway point
			const FVector HalfwayLocation = CurrentLocation + (GetRelativeLocation(0.5f) - CurrentRelativeLocation);
			if (SavedHalfwayLocation.IsNearlyZero())
			{
				SavedHalfwayLocation = HalfwayLocation;
			}
			if (FVector::DistSquared(SavedHalfwayLocation, HalfwayLocation) > 50.f*50.f)
			{
				UE_LOGF(LogRootMotion, Verbose, "RootMotion JumpForce drifted from saved halfway calculation!");
				SavedHalfwayLocation = HalfwayLocation;
			}
			DrawDebugCapsule(Character.GetWorld(), HalfwayLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::White, true, DebugLifetime);

			// Destination point
			const FVector DestinationLocation = CurrentLocation + (GetRelativeLocation(1.0f) - CurrentRelativeLocation);
			DrawDebugCapsule(Character.GetWorld(), DestinationLocation + LocDiff, Character.GetSimpleCollisionHalfHeight(), Character.GetSimpleCollisionRadius(), FQuat::Identity, FColor::White, true, DebugLifetime);

			UE_LOG(LogRootMotion, VeryVerbose, TEXT("RootMotionJumpForce %s %s preparing from %f to %f from (%s) to (%s) resulting force %s"), 
				Character.GetLocalRole() == ROLE_AutonomousProxy ? TEXT("AUTONOMOUS") : TEXT("AUTHORITY"),
				Character.bClientUpdating ? TEXT("UPD") : TEXT("NOR"),
				GetTime(), GetTime() + SimulationTime, 
				*CurrentLocation.ToString(), *CurrentTargetLocation.ToString(), 
				*Force.ToString());

			{
				FString AdjustedDebugString = FString::Printf(TEXT("    FRootMotionSource_JumpForce::Prep Force(%s) SimTime(%.3f) MoveTime(%.3f) StartP(%.3f) EndP(%.3f)"),
					*Force.ToCompactString(), SimulationTime, MovementTickTime, CurrentMoveFraction, TargetMoveFraction);
				RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
			}
		}
#endif

		const FTransform NewTransform(Force);
		RootMotionParams.Set(NewTransform);
	}
	else
	{
		checkf(Duration > UE_SMALL_NUMBER, TEXT("FRootMotionSource_JumpForce prepared with invalid duration."));
	}

	SetTime(GetTime() + SimulationTime);
}

bool FRootMotionSource_JumpForce::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	if (!FRootMotionSource::NetSerialize(Ar, Map, bOutSuccess))
	{
		return false;
	}

	Ar << Rotation; // TODO-RootMotionSource: Quantization
	Ar << Distance;
	Ar << Height;
	Ar << bDisableTimeout;
	Ar << PathOffsetCurve;
	Ar << TimeMappingCurve;

	bOutSuccess = true;
	return true;
}

UScriptStruct* FRootMotionSource_JumpForce::GetScriptStruct() const
{
	return FRootMotionSource_JumpForce::StaticStruct();
}

FString FRootMotionSource_JumpForce::ToSimpleString() const
{
	return FString::Printf(TEXT("[ID:%u]FRootMotionSource_JumpForce %s"), LocalID, *InstanceName.GetPlainNameString());
}

void FRootMotionSource_JumpForce::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PathOffsetCurve);
	Collector.AddReferencedObject(TimeMappingCurve);

	FRootMotionSource::AddReferencedObjects(Collector);
}

//
// FRootMotionSourceGroup
//

FRootMotionSourceGroup::FRootMotionSourceGroup()
	: bHasAdditiveSources(false)
	, bHasOverrideSources(false)
	, bHasOverrideSourcesWithIgnoreZAccumulate(false)
	, bIsAdditiveVelocityApplied(false)
	, LastPreAdditiveVelocity(ForceInitToZero)
{
}

bool FRootMotionSourceGroup::HasActiveRootMotionSources() const
{
	return RootMotionSources.Num() > 0 || PendingAddRootMotionSources.Num() > 0;
}

bool FRootMotionSourceGroup::HasOverrideVelocity() const
{
	return bHasOverrideSources;
}

bool FRootMotionSourceGroup::HasOverrideVelocityWithIgnoreZAccumulate() const
{
	return bHasOverrideSourcesWithIgnoreZAccumulate;
}

bool FRootMotionSourceGroup::HasAdditiveVelocity() const
{
	return bHasAdditiveSources;
}

bool FRootMotionSourceGroup::HasVelocity() const
{
	return HasOverrideVelocity() || HasAdditiveVelocity();
}

bool FRootMotionSourceGroup::HasRootMotionToApply() const
{
	return HasActiveRootMotionSources();
}

void FRootMotionSourceGroup::CleanUpInvalidRootMotion(float DeltaTime, const ACharacter& Character, UCharacterMovementComponent& MoveComponent)
{
	// Remove active sources marked for removal or that are invalid
	RootMotionSources.RemoveAll([this, DeltaTime, &Character, &MoveComponent](const TSharedPtr<FRootMotionSource>& RootSource)
	{
		if (RootSource.IsValid())
		{
			if (!RootSource->Status.HasFlag(ERootMotionSourceStatusFlags::MarkedForRemoval) &&
				!RootSource->Status.HasFlag(ERootMotionSourceStatusFlags::Finished))
			{
				return false;
			}

			// When additive root motion sources are removed we add their effects back to Velocity
			// so that any maintained momentum/velocity that they were contributing affects character
			// velocity and it's not a sudden stop
			if (RootSource->AccumulateMode == ERootMotionAccumulateMode::Additive)
			{
				if (bIsAdditiveVelocityApplied)
				{
					const FVector PreviousLastPreAdditiveVelocity = LastPreAdditiveVelocity;
					AccumulateRootMotionVelocityFromSource(*RootSource, DeltaTime, Character, MoveComponent, LastPreAdditiveVelocity);

#if ROOT_MOTION_DEBUG
					if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnAnyThread() == 1)
					{
						FString AdjustedDebugString = FString::Printf(TEXT("PrepareRootMotion RemovingAdditiveSource LastPreAdditiveVelocity(%s) Old(%s)"),
							*LastPreAdditiveVelocity.ToCompactString(), *PreviousLastPreAdditiveVelocity.ToCompactString());
						RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
					}
#endif
				}
			}

			// Process FinishVelocity options when RootMotionSource is removed.
			if (RootSource->FinishVelocityParams.Mode == ERootMotionFinishVelocityMode::ClampVelocity)
			{
				// For Z, only clamp positive values to prevent shooting off, we don't want to slow down a fall.
				MoveComponent.Velocity = MoveComponent.Velocity.GetClampedToMaxSize2D(RootSource->FinishVelocityParams.ClampVelocity);
				MoveComponent.Velocity.Z = FMath::Min<FVector::FReal>(MoveComponent.Velocity.Z, RootSource->FinishVelocityParams.ClampVelocity);

				// if we have additive velocity applied, LastPreAdditiveVelocity will stomp velocity, so make sure it gets clamped too.
				if (bIsAdditiveVelocityApplied)
				{
					// For Z, only clamp positive values to prevent shooting off, we don't want to slow down a fall.
					LastPreAdditiveVelocity = LastPreAdditiveVelocity.GetClampedToMaxSize2D(RootSource->FinishVelocityParams.ClampVelocity);
					LastPreAdditiveVelocity.Z = FMath::Min<FVector::FReal>(LastPreAdditiveVelocity.Z, RootSource->FinishVelocityParams.ClampVelocity);
				}
			}
			else if (RootSource->FinishVelocityParams.Mode == ERootMotionFinishVelocityMode::SetVelocity)
			{
				MoveComponent.Velocity = RootSource->FinishVelocityParams.SetVelocity;
				// if we have additive velocity applied, LastPreAdditiveVelocity will stomp velocity, so make sure this gets set too.
				if (bIsAdditiveVelocityApplied)
				{
					LastPreAdditiveVelocity = RootSource->FinishVelocityParams.SetVelocity;
				}
			}

			UE_LOGF(LogRootMotion, VeryVerbose, "RootMotionSource being removed: %ls", *RootSource->ToSimpleString());

#if ROOT_MOTION_DEBUG
			if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnAnyThread() == 1)
			{
				FString AdjustedDebugString = FString::Printf(TEXT("PrepareRootMotion Removing RootMotionSource(%s)"),
					*RootSource->ToSimpleString());
				RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
			}
#endif
		}
		return true;
	});

	// Remove pending sources that could have been marked for removal before they were made active
	PendingAddRootMotionSources.RemoveAll([&Character](const TSharedPtr<FRootMotionSource>& RootSource)
	{
		if (RootSource.IsValid())
		{
			if (!RootSource->Status.HasFlag(ERootMotionSourceStatusFlags::MarkedForRemoval) &&
				!RootSource->Status.HasFlag(ERootMotionSourceStatusFlags::Finished))
			{
				return false;
			}

			UE_LOGF(LogRootMotion, VeryVerbose, "Pending RootMotionSource being removed: %ls", *RootSource->ToSimpleString());

#if ROOT_MOTION_DEBUG
			if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnAnyThread() == 1)
			{
				FString AdjustedDebugString = FString::Printf(TEXT("PrepareRootMotion Removing PendingAddRootMotionSource(%s)"),
					*RootSource->ToSimpleString());
				RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
			}
#endif
		}
		return true;
	});
}

void FRootMotionSourceGroup::PrepareRootMotion(float DeltaTime, const ACharacter& Character, const UCharacterMovementComponent& MoveComponent, bool bForcePrepareAll)
{
	// Add pending sources
	{
		RootMotionSources.Append(PendingAddRootMotionSources);
		PendingAddRootMotionSources.Empty();
	}

	// Sort by priority
	if (RootMotionSources.Num() > 1)
	{
		RootMotionSources.StableSort([](const TSharedPtr<FRootMotionSource>& SourceL, const TSharedPtr<FRootMotionSource>& SourceR)
			{
				if (SourceL.IsValid() && SourceR.IsValid())
				{
					return SourceL->Priority > SourceR->Priority;
				}
				checkf(false, TEXT("RootMotionSources being sorted are invalid pointers"));
				return true;
			});
	}

	// Prepare active sources
	{
		bHasOverrideSources = false;
		bHasOverrideSourcesWithIgnoreZAccumulate = false;
		bHasAdditiveSources = false;
		LastAccumulatedSettings.Clear();

		// Go through all sources, prepare them so that they each save off how much they're going to contribute this frame
		for (TSharedPtr<FRootMotionSource>& RootMotionSource : RootMotionSources)
		{
			if (RootMotionSource.IsValid())
			{
				if (!RootMotionSource->Status.HasFlag(ERootMotionSourceStatusFlags::Prepared) || bForcePrepareAll)
				{
					float SimulationTime = DeltaTime;

					// If we've received authoritative correction to root motion state, we need to
					// increase simulation time to catch up to where we were
					if (RootMotionSource->bNeedsSimulatedCatchup)
					{
						float CorrectionDelta = RootMotionSource->PreviousTime - RootMotionSource->CurrentTime;
						if (CorrectionDelta > 0.f)
						{
							// In the simulated catch-up case, we are a simulated proxy receiving authoritative state
							// from the server version of the root motion. When receiving authoritative state we could:
							// 1) Always snap precisely to authoritative time
							//     - But with latency, this could just result in unnecessary jerkiness for correction.
							//       We're completely reliant on mesh smoothing for fix-up.
							// 2) Always maintain Time as "authoritative" on simulated
							//     - But if the application of the root motion source was off, we're just maintaining
							//       that inconsistency indefinitely, and any additional error/hitches on the server's
							//       or latency's part only push the authoritative and simulated versions further apart
							//       and we have no mechanism for reconciling them over time.
							// 3) Split it down the middle - move towards authoritative state while not doing full snaps
							//    so that we use both internal "simulated catchup" smoothing along with mesh smoothing
							//    and over time we correct towards authoritative time
							// Below is #3
							const float MaxTimeDeltaCorrectionPercent = 0.5f; // Max percent of time mismatch to make up per authoritative update
							const float MaxTimeDeltaCorrectionAbsolute = 0.5f; // Amount of time in seconds we can erase on simulated
							
							CorrectionDelta = FMath::Min(CorrectionDelta * MaxTimeDeltaCorrectionPercent, MaxTimeDeltaCorrectionAbsolute);

							const float PreviousSimulationTime = SimulationTime;

							SimulationTime += CorrectionDelta;

							UE_LOGF(LogRootMotion, VeryVerbose, "Adjusting SimulationTime due to bNeedsSimulatedCatchup before Preparing RootMotionSource %ls from %f to %f", 
								*RootMotionSource->ToSimpleString(), PreviousSimulationTime, SimulationTime);
						}
					}

					// Handle partial ticks
					{
						// Start of root motion (Root motion StartTime vs. character movement time)
						{
							const bool bRootMotionHasNotStarted = RootMotionSource->GetTime() == 0.f;
							const bool bRootMotionHasValidStartTime = RootMotionSource->IsStartTimeValid();
							if (bRootMotionHasNotStarted && bRootMotionHasValidStartTime)
							{
								float CharacterMovementTime = -1.f;
								if (Character.GetLocalRole() == ROLE_AutonomousProxy)
								{
									const FNetworkPredictionData_Client_Character* ClientData = MoveComponent.HasPredictionData_Client() ? static_cast<FNetworkPredictionData_Client_Character*>(MoveComponent.GetPredictionData_Client()) : nullptr;
									if (ClientData)
									{
										if (!Character.bClientUpdating) 
										{
											CharacterMovementTime = ClientData->CurrentTimeStamp;
										}
										else
										{
											// TODO: To enable this during bClientUpdating case, we need to have access
											// to the CurrentTimeStamp that happened during the original move.
											// We could add this in saved move replay PrepMoveFor() logic.
											// Right now we don't have this, but it should only affect first server move
											// of root motion corrections which shouldn't have corrections in the common case
											// (we don't have cases where we set StartTime to be in the distant future yet)
										}
									}
								}
								else if (Character.GetLocalRole() == ROLE_Authority)
								{
									const FNetworkPredictionData_Server_Character* ServerData = MoveComponent.HasPredictionData_Server() ? static_cast<FNetworkPredictionData_Server_Character*>(MoveComponent.GetPredictionData_Server()) : nullptr;
									if (ServerData)
									{
										CharacterMovementTime = ServerData->CurrentClientTimeStamp - DeltaTime; // CurrentClientTimeStamp is the client time AFTER this DeltaTime move
									}
								}

								const bool bHasValidMovementTime = CharacterMovementTime >= 0.f;
								const bool bHasSourceNotStarted = RootMotionSource->GetStartTime() > CharacterMovementTime;
								if (bHasValidMovementTime && bHasSourceNotStarted)
								{
									const float PreviousSimulationTime = SimulationTime;

									// Our StartTime hasn't yet hit, we'll need to adjust SimulationTime
									const float EndCharacterMovementTime = CharacterMovementTime + SimulationTime;
									if (EndCharacterMovementTime <= RootMotionSource->GetStartTime())
									{
										// We won't reach the StartTime this frame at all, so we don't need any SimulationTime done
										SimulationTime = 0.f;
										UE_LOGF(LogRootMotion, VeryVerbose, "Adjusting SimulationTime due to StartTime not reachable this tick before Preparing RootMotionSource %ls from %f to %f", 
											*RootMotionSource->ToSimpleString(), PreviousSimulationTime, SimulationTime);
									}
									else
									{
										// Root motion will kick in partway through this tick, adjust SimulationTime
										// so that the amount of root motion applied matches what length of time it
										// should have been active (need to do this because root motions are either
										// on for an entire movement tick or not at all)
										SimulationTime = EndCharacterMovementTime - RootMotionSource->GetStartTime();
										UE_LOGF(LogRootMotion, VeryVerbose, "Adjusting SimulationTime due to StartTime reachable partway through tick before Preparing RootMotionSource %ls from %f to %f", 
											*RootMotionSource->ToSimpleString(), PreviousSimulationTime, SimulationTime);
									}
								}
							}
						}

						// End of root motion
						if (RootMotionSource->IsTimeOutEnabled() && !RootMotionSource->Settings.HasFlag(ERootMotionSourceSettingsFlags::DisablePartialEndTick))
						{
							const float Duration = RootMotionSource->GetDuration();
							if (RootMotionSource->GetTime() + SimulationTime >= Duration)
							{
								const float PreviousSimulationTime = SimulationTime;

								// Upcoming tick will go beyond the intended duration, if we kept
								// SimulationTime unchanged we would get more movement than was
								// intended so we clamp it to duration
								SimulationTime = Duration - RootMotionSource->GetTime() + UE_KINDA_SMALL_NUMBER; // Plus a little to make sure we push it over Duration
								UE_LOGF(LogRootMotion, VeryVerbose, "Adjusting SimulationTime due to Duration reachable partway through tick before Preparing RootMotionSource %ls from %f to %f", 
									*RootMotionSource->ToSimpleString(), PreviousSimulationTime, SimulationTime);
							}
						}
					}

					// Sanity check resulting SimulationTime
					SimulationTime = FMath::Max(SimulationTime, 0.f);

					// Do the Preparation (calculates root motion transforms to be applied)
					RootMotionSource->bSimulatedNeedsSmoothing = false;
					RootMotionSource->PrepareRootMotion(SimulationTime, DeltaTime, Character, MoveComponent);
					LastAccumulatedSettings += RootMotionSource->Settings;
					RootMotionSource->Status.SetFlag(ERootMotionSourceStatusFlags::Prepared);

#if ROOT_MOTION_DEBUG
					if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnAnyThread() == 1)
					{
						FString AdjustedDebugString = FString::Printf(TEXT("PrepareRootMotion Prepared RootMotionSource(%s)"),
							*RootMotionSource->ToSimpleString());
						RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
					}
#endif

					RootMotionSource->bNeedsSimulatedCatchup = false;
				}
				else // if (!RootMotionSource->Status.HasFlag(ERootMotionSourceStatusFlags::Prepared) || bForcePrepareAll)
				{
#if ROOT_MOTION_DEBUG
					if (RootMotionSourceDebug::CVarDebugRootMotionSources.GetValueOnAnyThread() == 1)
					{
						FString AdjustedDebugString = FString::Printf(TEXT("PrepareRootMotion AlreadyPrepared RootMotionSource(%s)"),
							*RootMotionSource->ToSimpleString());
						RootMotionSourceDebug::PrintOnScreen(Character, AdjustedDebugString);
					}
#endif
				}

				if (RootMotionSource->AccumulateMode == ERootMotionAccumulateMode::Additive)
				{
					bHasAdditiveSources = true;
				}
				else if (RootMotionSource->AccumulateMode == ERootMotionAccumulateMode::Override)
				{
					bHasOverrideSources = true;

					if (RootMotionSource->Settings.HasFlag(ERootMotionSourceSettingsFlags::IgnoreZAccumulate))
					{
						bHasOverrideSourcesWithIgnoreZAccumulate = true;
					}
				}
			}
		}
	}
}

void FRootMotionSourceGroup::AccumulateOverrideRootMotionVelocity
	(
		float DeltaTime, 
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent, 
		FVector& InOutVelocity
	) const
{
	AccumulateRootMotionVelocity(ERootMotionAccumulateMode::Override, DeltaTime, Character, MoveComponent, InOutVelocity);
}

void FRootMotionSourceGroup::AccumulateAdditiveRootMotionVelocity
	(
		float DeltaTime, 
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent, 
		FVector& InOutVelocity
	) const
{
	AccumulateRootMotionVelocity(ERootMotionAccumulateMode::Additive, DeltaTime, Character, MoveComponent, InOutVelocity);
}

void FRootMotionSourceGroup::AccumulateRootMotionVelocity
	(
		ERootMotionAccumulateMode RootMotionType,
		float DeltaTime, 
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent, 
		FVector& InOutVelocity
	) const
{
	check(RootMotionType == ERootMotionAccumulateMode::Additive || RootMotionType == ERootMotionAccumulateMode::Override);

	// Go through all sources, accumulate their contribution to root motion
	for (const auto& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->AccumulateMode == RootMotionType)
		{
			AccumulateRootMotionVelocityFromSource(*RootMotionSource, DeltaTime, Character, MoveComponent, InOutVelocity);

			// For Override root motion, we apply the highest priority override and ignore the rest
			if (RootMotionSource->AccumulateMode == ERootMotionAccumulateMode::Override)
			{
				break;
			}
		}
	}
}

void FRootMotionSourceGroup::AccumulateRootMotionVelocityFromSource
	(
		const FRootMotionSource& RootMotionSource,
		float DeltaTime, 
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent, 
		FVector& InOutVelocity
	) const
{
	FRootMotionMovementParams RootMotionParams = RootMotionSource.RootMotionParams;

	// Transform RootMotion if needed (world vs local space)
	if (RootMotionSource.bInLocalSpace && MoveComponent.UpdatedComponent)
	{
		RootMotionParams.Set( RootMotionParams.GetRootMotionTransform() * MoveComponent.UpdatedComponent->GetComponentToWorld().GetRotation() );
	}

	const FVector RootMotionVelocity = RootMotionParams.GetRootMotionTransform().GetTranslation();

	const FVector InputVelocity = InOutVelocity;
	if (RootMotionSource.AccumulateMode == ERootMotionAccumulateMode::Override)
	{
		InOutVelocity = RootMotionVelocity;
	}
	else if (RootMotionSource.AccumulateMode == ERootMotionAccumulateMode::Additive)
	{
		InOutVelocity += RootMotionVelocity;
	}
	if (RootMotionSource.Settings.HasFlag(ERootMotionSourceSettingsFlags::IgnoreZAccumulate))
	{
		InOutVelocity.Z = InputVelocity.Z;
	}
}

bool FRootMotionSourceGroup::GetOverrideRootMotionRotation
	(
		float DeltaTime, 
		const ACharacter& Character, 
		const UCharacterMovementComponent& MoveComponent, 
		FQuat& OutRotation
	) const
{
	for (const auto& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->AccumulateMode == ERootMotionAccumulateMode::Override)
		{
			OutRotation = RootMotionSource->RootMotionParams.GetRootMotionTransform().GetRotation();
			return !OutRotation.IsIdentity();
		}
	}
	return false;
}

bool FRootMotionSourceGroup::NeedsSimulatedSmoothing() const
{
	for (const auto& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->bSimulatedNeedsSmoothing)
		{
			return true;
		}
	}
	return false;
}

void FRootMotionSourceGroup::SetPendingRootMotionSourceMinStartTimes(float NewStartTime)
{
	for (auto& RootMotionSource : PendingAddRootMotionSources)
	{
		if (RootMotionSource.IsValid())
		{
			const float PreviousStartTime = RootMotionSource->StartTime;
			const float MinStartTime = NewStartTime;
			RootMotionSource->StartTime = FMath::Max(PreviousStartTime, NewStartTime);
			if (PreviousStartTime != RootMotionSource->StartTime)
			{
				UE_LOGF(LogRootMotion, VeryVerbose, "Pending RootMotionSource %ls starting time modification: previous: %f new: %f", *RootMotionSource->ToSimpleString(), PreviousStartTime, RootMotionSource->StartTime);
			}
		}
	}
}

void FRootMotionSourceGroup::ApplyTimeStampReset(float DeltaTime)
{
	check(-DeltaTime > RootMotionSource_InvalidStartTime);

	for (auto& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->IsStartTimeValid())
		{
			const float PreviousStartTime = RootMotionSource->StartTime;
			RootMotionSource->StartTime -= DeltaTime;
			UE_LOGF(LogRootMotion, VeryVerbose, "Applying time stamp reset to RootMotionSource %ls StartTime: previous(%f), new(%f)", *RootMotionSource->ToSimpleString(), PreviousStartTime, RootMotionSource->StartTime);
		}
	}

	for (auto& RootMotionSource : PendingAddRootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->IsStartTimeValid())
		{
			const float PreviousStartTime = RootMotionSource->StartTime;
			RootMotionSource->StartTime -= DeltaTime;
			UE_LOGF(LogRootMotion, VeryVerbose, "Applying time stamp reset to PendingAddRootMotionSource %ls StartTime: previous(%f), new(%f)", *RootMotionSource->ToSimpleString(), PreviousStartTime, RootMotionSource->StartTime);
		}
	}
}

uint16 FRootMotionSourceGroup::ApplyRootMotionSource(TSharedPtr<FRootMotionSource> SourcePtr)
{
	if (ensure(SourcePtr.IsValid()))
	{
		// Get valid localID
		// Note: Current ID method could produce duplicate IDs "in flight" at one time
		// if you have one root motion source applied while 2^16-1 other root motion sources
		// get applied and it's still applied and it happens that the 2^16-1th root motion
		// source is applied on this character movement component. 
		// This was preferred over the complexity of ensuring unique IDs.
		static uint16 LocalIDGenerator = 0;
		uint16 LocalID = ++LocalIDGenerator;
		if (LocalID == (uint16)ERootMotionSourceID::Invalid)
		{
			LocalID = ++LocalIDGenerator;
		}
		SourcePtr->LocalID = LocalID;

		// Apply to pending so that next Prepare it gets added to "active"
		PendingAddRootMotionSources.Add(SourcePtr);
		UE_LOGF(LogRootMotion, VeryVerbose, "RootMotionSource added to Pending: [%u] %ls", LocalID, *SourcePtr->ToSimpleString());

		return LocalID;
	}

	return (uint16)ERootMotionSourceID::Invalid;
}

TSharedPtr<FRootMotionSource> FRootMotionSourceGroup::GetRootMotionSource(FName InstanceName)
{
	for (const auto& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->InstanceName == InstanceName)
		{
			return TSharedPtr<FRootMotionSource>(RootMotionSource);
		}
	}

	for (const auto& RootMotionSource : PendingAddRootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->InstanceName == InstanceName)
		{
			return TSharedPtr<FRootMotionSource>(RootMotionSource);
		}
	}

	return TSharedPtr<FRootMotionSource>(nullptr);
}

TSharedPtr<FRootMotionSource> FRootMotionSourceGroup::GetRootMotionSourceByID(uint16 RootMotionSourceID)
{
	for (const auto& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->LocalID == RootMotionSourceID)
		{
			return TSharedPtr<FRootMotionSource>(RootMotionSource);
		}
	}

	for (const auto& RootMotionSource : PendingAddRootMotionSources)
	{
		if (RootMotionSource.IsValid() && RootMotionSource->LocalID == RootMotionSourceID)
		{
			return TSharedPtr<FRootMotionSource>(RootMotionSource);
		}
	}

	return TSharedPtr<FRootMotionSource>(nullptr);
}

void FRootMotionSourceGroup::RemoveRootMotionSource(FName InstanceName)
{
	if (!InstanceName.IsNone()) // Don't allow removing None since that's the default
	{
		for (const auto& RootMotionSource : RootMotionSources)
		{
			if (RootMotionSource.IsValid() && RootMotionSource->InstanceName == InstanceName)
			{
				RootMotionSource->Status.SetFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
			}
		}

		for (const auto& RootMotionSource : PendingAddRootMotionSources)
		{
			if (RootMotionSource.IsValid() && RootMotionSource->InstanceName == InstanceName)
			{
				RootMotionSource->Status.SetFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
			}
		}
	}
}

void FRootMotionSourceGroup::RemoveRootMotionSourceByID(uint16 RootMotionSourceID)
{
	if (RootMotionSourceID != (uint16)ERootMotionSourceID::Invalid)
	{
		for (const auto& RootMotionSource : RootMotionSources)
		{
			if (RootMotionSource.IsValid() && RootMotionSource->LocalID == RootMotionSourceID)
			{
				RootMotionSource->Status.SetFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
			}
		}

		for (const auto& RootMotionSource : PendingAddRootMotionSources)
		{
			if (RootMotionSource.IsValid() && RootMotionSource->LocalID == RootMotionSourceID)
			{
				RootMotionSource->Status.SetFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
			}
		}
	}
}

void FRootMotionSourceGroup::UpdateStateFrom(const FRootMotionSourceGroup& GroupToTakeStateFrom, bool bMarkForSimulatedCatchup)
{
	bIsAdditiveVelocityApplied = GroupToTakeStateFrom.bIsAdditiveVelocityApplied;
	LastPreAdditiveVelocity = GroupToTakeStateFrom.LastPreAdditiveVelocity;

	// If we have a PendingAdd root motion source that is already active in GroupToTakeStateFrom, make it active
	PendingAddRootMotionSources.RemoveAll([this, &GroupToTakeStateFrom](const TSharedPtr<FRootMotionSource>& RootSource) 
		{ 
			if (RootSource.IsValid())
			{
				if (RootSource->LocalID != (uint16)ERootMotionSourceID::Invalid)
				{
					for (const TSharedPtr<FRootMotionSource>& TakeFromRootMotionSource : GroupToTakeStateFrom.RootMotionSources)
					{
						if (TakeFromRootMotionSource.IsValid() && (RootSource->LocalID == TakeFromRootMotionSource->LocalID))
						{
							// Matches, move to active and remove from pending
							UE_LOGF(LogRootMotion, VeryVerbose, "UpdateStateFream moving PendingAdd RMS to active: %ls", *RootSource->ToSimpleString());
							RootMotionSources.Add(RootSource);
							return true;
						}
					}
				}
			}
			return false;
		});

	// For each matching Source in GroupToTakeStateFrom, move state over to this group's Sources
	// Can do all matching with LocalID only, since anything passed into this function should have
	// already been "matched" to LocalIDs
	for (const TSharedPtr<FRootMotionSource>& TakeFromRootMotionSource : GroupToTakeStateFrom.RootMotionSources)
	{
		if (TakeFromRootMotionSource.IsValid() && (TakeFromRootMotionSource->LocalID != (uint16)ERootMotionSourceID::Invalid))
		{
			for (const TSharedPtr<FRootMotionSource>& RootMotionSource : RootMotionSources)
			{
				if (RootMotionSource.IsValid() && (RootMotionSource->LocalID == TakeFromRootMotionSource->LocalID))
				{
					// We rely on the 'Matches' rule to be exact, verify that it is still correct here.
					// If not, we're matching different root motion sources, or we're using properties that change over time for matching.
					if (!RootMotionSource->Matches(TakeFromRootMotionSource.Get()))
					{
						ensureMsgf(false, TEXT("UpdateStateFrom RootMotionSource(%s) has the same LocalID(%d) as a non-matching TakeFromRootMotionSource(%s)!"),
							*RootMotionSource->ToSimpleString(), RootMotionSource->LocalID, *TakeFromRootMotionSource->ToSimpleString());
						
						// See if multiple local sources match this ServerRootMotionSource by rules
						UE_LOGF(LogRootMotion, Warning, "Finding Matches by rules for TakeFromRootMotionSource(%ls)", *TakeFromRootMotionSource->ToSimpleString());
						for (int32 Index=0; Index<RootMotionSources.Num(); Index++)
						{
							const TSharedPtr<FRootMotionSource>& TestRootMotionSource = RootMotionSources[Index];
							if (TestRootMotionSource.IsValid())
							{
								UE_LOGF(LogRootMotion, Warning, "[%d/%d] Matches(%ls) ? (%d)",
									Index + 1, RootMotionSources.Num(), *TestRootMotionSource->ToSimpleString(), TestRootMotionSource->Matches(TakeFromRootMotionSource.Get()));
							}
						}

						// See if multiple local sources match this ServerRootMotionSource by ID
						UE_LOGF(LogRootMotion, Warning, "Finding Matches by ID for TakeFromRootMotionSource(%ls)", *TakeFromRootMotionSource->ToSimpleString());
						for (int32 Index = 0; Index < RootMotionSources.Num(); Index++)
						{
							const TSharedPtr<FRootMotionSource>& TestRootMotionSource = RootMotionSources[Index];
							if (TestRootMotionSource.IsValid())
							{
								UE_LOGF(LogRootMotion, Warning, "[%d/%d] Matches(%ls) ? (%d)",
									Index + 1, RootMotionSources.Num(), *TestRootMotionSource->ToSimpleString(), TestRootMotionSource->LocalID == TakeFromRootMotionSource->LocalID);
							}
						}

						continue;
					}

					const bool bSuccess = RootMotionSource->UpdateStateFrom(TakeFromRootMotionSource.Get(), bMarkForSimulatedCatchup);
					if (bSuccess)
					{
						// If we've updated state, we'll need prepared before being able to contribute
						RootMotionSource->Status.UnSetFlag(ERootMotionSourceStatusFlags::Prepared);

						UE_LOGF(LogRootMotion, VeryVerbose, "RootMotionSource UpdatedState: %ls", *RootMotionSource->ToSimpleString());
					}
					else
					{
						RootMotionSource->Status.SetFlag(ERootMotionSourceStatusFlags::MarkedForRemoval);
						UE_LOGF(LogRootMotion, Warning,  "RootMotionSource failed to be updated from matching Source, marking for removal");
					}
				}
			}
		}
	}
}

struct FRootMotionSourceDeleter
{
	FORCEINLINE void operator()(FRootMotionSource* Object) const
	{
		check(Object);
		UScriptStruct* ScriptStruct = Object->GetScriptStruct();
		check(ScriptStruct);
		ScriptStruct->DestroyStruct(Object);
		FMemory::Free(Object);
	}
};

void FRootMotionSourceGroup::NetSerializeRMSArray(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess, TArray< TSharedPtr<FRootMotionSource> >& RootMotionSourceArray, uint8 MaxNumRootMotionSourcesToSerialize/* = MAX_uint8*/)
{
	uint8 SourcesNum;
	if (Ar.IsSaving())
	{
		UE_CLOGF(RootMotionSourceArray.Num() > MaxNumRootMotionSourcesToSerialize, LogRootMotion, Warning, "Too many root motion sources (%d!) to net serialize. Clamping to %d",
			RootMotionSourceArray.Num(), MaxNumRootMotionSourcesToSerialize);
		SourcesNum = FMath::Min<int32>(RootMotionSourceArray.Num(), MaxNumRootMotionSourcesToSerialize);
	}
	Ar << SourcesNum;
	if (Ar.IsLoading())
	{
		RootMotionSourceArray.SetNumZeroed(SourcesNum);
	}

	for (int32 i = 0; i < SourcesNum && !Ar.IsError(); ++i)
	{
		TCheckedObjPtr<UScriptStruct> ScriptStruct = RootMotionSourceArray[i].IsValid() ? RootMotionSourceArray[i]->GetScriptStruct() : nullptr;
		UScriptStruct* ScriptStructLocal = ScriptStruct.Get();
		Ar << ScriptStruct;

		if (ScriptStruct.IsValid())
		{
			// Restrict replication to derived classes of FRootMotionSource for security reasons:
			// If FRootMotionSourceGroup is replicated through a Server RPC, we need to prevent clients from sending us
			// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
			// for invalid structs. All provided sources are direct children of RMS and we never expect to have deep hierarchies
			// so this should not be too costly
			bool bIsDerivedFromBaseRMS = false;
			UStruct* CurrentSuperStruct = ScriptStruct->GetSuperStruct();
			while (CurrentSuperStruct)
			{
				if (CurrentSuperStruct == FRootMotionSource::StaticStruct())
				{
					bIsDerivedFromBaseRMS = true;
					break;
				}
				CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
			}

			if (bIsDerivedFromBaseRMS)
			{
				if (Ar.IsLoading())
				{
					if (RootMotionSourceArray[i].IsValid() && ScriptStructLocal == ScriptStruct.Get())
					{
						// What we have locally is the same type as we're being serialized into, so we don't need to
						// reallocate - just use existing structure
					}
					else
					{
						// For now, just reset/reallocate the data when loading.
						// Longer term if we want to generalize this and use it for property replication, we should support
						// only reallocating when necessary
						FRootMotionSource* NewSource = (FRootMotionSource*)FMemory::Malloc(ScriptStruct->GetCppStructOps()->GetSize());
						ScriptStruct->InitializeStruct(NewSource);

						RootMotionSourceArray[i] = TSharedPtr<FRootMotionSource>(NewSource, FRootMotionSourceDeleter());
					}
				}

				void* ContainerPtr = RootMotionSourceArray[i].Get();

				if (ScriptStruct->StructFlags & STRUCT_NetSerializeNative)
				{
					ScriptStruct->GetCppStructOps()->NetSerialize(Ar, Map, bOutSuccess, RootMotionSourceArray[i].Get());
				}
				else
				{
					checkf(false, TEXT("Serializing RootMotionSource without NetSerializeNative - not supported!"));
				}
			}
			else
			{
				UE_LOGF(LogRootMotion, Error, "FRootMotionSourceGroup::NetSerialize: ScriptStruct not derived from FRootMotionSource attempted to serialize.");
				Ar.SetError();
				break;
			}
		}
		else if (ScriptStruct.IsError())
		{
			UE_LOGF(LogRootMotion, Error, "FRootMotionSourceGroup::NetSerialize: Invalid ScriptStruct serialized.");
			Ar.SetError();
			break;
		}
	}

}

bool FRootMotionSourceGroup::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess, uint8 MaxNumRootMotionSourcesToSerialize/* = MAX_uint8*/)
{
	FArchive_Serialize_BitfieldBool(Ar, bHasAdditiveSources);
	FArchive_Serialize_BitfieldBool(Ar, bHasOverrideSources);
	FArchive_Serialize_BitfieldBool(Ar, bHasOverrideSourcesWithIgnoreZAccumulate);
	LastPreAdditiveVelocity.NetSerialize(Ar, Map, bOutSuccess);
	FArchive_Serialize_BitfieldBool(Ar, bIsAdditiveVelocityApplied);
	Ar << LastAccumulatedSettings.Flags;

	uint8 NumRootMotionSourcesToSerialize = FMath::Min<int32>(RootMotionSources.Num(), MaxNumRootMotionSourcesToSerialize);
	uint8 NumPendingAddRootMotionSourcesToSerialize = NumRootMotionSourcesToSerialize < MaxNumRootMotionSourcesToSerialize ? MaxNumRootMotionSourcesToSerialize - NumRootMotionSourcesToSerialize : 0;
	NetSerializeRMSArray(Ar, Map, bOutSuccess, RootMotionSources, NumRootMotionSourcesToSerialize);
	NetSerializeRMSArray(Ar, Map, bOutSuccess, PendingAddRootMotionSources, NumPendingAddRootMotionSourcesToSerialize);

	if (Ar.IsError())
	{
		// Something bad happened, make sure to not return invalid shared ptrs
		for (int32 i = RootMotionSources.Num() - 1; i >= 0; --i)
		{
			if (RootMotionSources[i].IsValid() == false)
			{
				RootMotionSources.RemoveAt(i);
			}
		}
		for (int32 i = PendingAddRootMotionSources.Num() - 1; i >= 0; --i)
		{
			if (PendingAddRootMotionSources[i].IsValid() == false)
			{
				PendingAddRootMotionSources.RemoveAt(i);
			}
		}
		bOutSuccess = false;
		return false;
	}

	bOutSuccess = true;
	return true;
}

void FRootMotionSourceGroup::CullInvalidSources()
{
	RootMotionSources.RemoveAll([](const TSharedPtr<FRootMotionSource>& RootSource) 
		{ 
			if (RootSource.IsValid())
			{
				if (RootSource->LocalID != (uint16)ERootMotionSourceID::Invalid)
				{
					return false;
				}
				UE_LOGF(LogRootMotion, VeryVerbose, "RootMotionSource being culled as invalid: %ls", *RootSource->ToSimpleString());
			}
			return true;
		});
}

void FRootMotionSourceGroup::Clear()
{
	RootMotionSources.Empty();
	PendingAddRootMotionSources.Empty();
	bIsAdditiveVelocityApplied = false;
	bHasAdditiveSources = false;
	bHasOverrideSources = false;
	bHasOverrideSourcesWithIgnoreZAccumulate = false;
	LastAccumulatedSettings.Clear();
}

FRootMotionSourceGroup& FRootMotionSourceGroup::operator=(const FRootMotionSourceGroup& Other)
{
	// Perform deep copy of this Group
	if (this != &Other)
	{
		// Deep copy Sources
		RootMotionSources.Empty(Other.RootMotionSources.Num());
		for (int i = 0; i < Other.RootMotionSources.Num(); ++i)
		{
			if (Other.RootMotionSources[i].IsValid())
			{
				FRootMotionSource* CopyOfSourcePtr = Other.RootMotionSources[i]->Clone();
				RootMotionSources.Add(TSharedPtr<FRootMotionSource>(CopyOfSourcePtr));
			}
			else
			{
				UE_LOGF(LogRootMotion, Warning, "RootMotionSourceGroup::operator= trying to copy bad Other RMS");
			}
		}

		// Deep copy PendingAdd sources
		PendingAddRootMotionSources.Empty(Other.PendingAddRootMotionSources.Num());
		for (int i = 0; i < Other.PendingAddRootMotionSources.Num(); ++i)
		{
			if (Other.PendingAddRootMotionSources[i].IsValid())
			{
				FRootMotionSource* CopyOfSourcePtr = Other.PendingAddRootMotionSources[i]->Clone();
				PendingAddRootMotionSources.Add(TSharedPtr<FRootMotionSource>(CopyOfSourcePtr));
			}
			else
			{
				UE_LOGF(LogRootMotion, Warning, "RootMotionSourceGroup::operator= trying to copy bad Other PendingAdd");
			}
		}

		bHasAdditiveSources = Other.bHasAdditiveSources;
		bHasOverrideSources = Other.bHasOverrideSources;
		bHasOverrideSourcesWithIgnoreZAccumulate = Other.bHasOverrideSourcesWithIgnoreZAccumulate;
		LastPreAdditiveVelocity = Other.LastPreAdditiveVelocity;
		bIsAdditiveVelocityApplied = Other.bIsAdditiveVelocityApplied;
		LastAccumulatedSettings = Other.LastAccumulatedSettings;
	}
	return *this;
}

bool FRootMotionSourceGroup::operator==(const FRootMotionSourceGroup& Other) const
{
	if (bHasAdditiveSources != Other.bHasAdditiveSources || 
		bHasOverrideSources != Other.bHasOverrideSources ||
		bHasOverrideSourcesWithIgnoreZAccumulate != Other.bHasOverrideSourcesWithIgnoreZAccumulate ||
		bIsAdditiveVelocityApplied != Other.bIsAdditiveVelocityApplied ||
		!LastPreAdditiveVelocity.Equals(Other.LastPreAdditiveVelocity, 1.f))
	{
		return false;
	}

	// Both invalid structs or both valid and Pointer compare (???) // deep comparison equality
	if (RootMotionSources.Num() != Other.RootMotionSources.Num())
	{
		return false;
	}
	if (PendingAddRootMotionSources.Num() != Other.PendingAddRootMotionSources.Num())
	{
		return false;
	}
	for (int32 i = 0; i < RootMotionSources.Num(); ++i)
	{
		if (RootMotionSources[i].IsValid() == Other.RootMotionSources[i].IsValid())
		{
			if (RootMotionSources[i].IsValid())
			{
				if (!RootMotionSources[i]->MatchesAndHasSameState(Other.RootMotionSources[i].Get()))
				{
					return false; // They're valid and don't match/have same state
				}
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}
	for (int32 i = 0; i < PendingAddRootMotionSources.Num(); ++i)
	{
		if (PendingAddRootMotionSources[i].IsValid() == Other.PendingAddRootMotionSources[i].IsValid())
		{
			if (PendingAddRootMotionSources[i].IsValid())
			{
				if (!PendingAddRootMotionSources[i]->MatchesAndHasSameState(Other.PendingAddRootMotionSources[i].Get()))
				{
					return false; // They're valid and don't match/have same state
				}
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}
	return true;
}

bool FRootMotionSourceGroup::operator!=(const FRootMotionSourceGroup& Other) const
{
	return !(FRootMotionSourceGroup::operator==(Other));
}

void FRootMotionSourceGroup::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	for (const TSharedPtr<FRootMotionSource>& RootMotionSource : RootMotionSources)
	{
		if (RootMotionSource.IsValid())
		{
			RootMotionSource->AddReferencedObjects(Collector);
		}
	}

	for (const TSharedPtr<FRootMotionSource>& RootMotionSource : PendingAddRootMotionSources)
	{
		if (RootMotionSource.IsValid())
		{
			RootMotionSource->AddReferencedObjects(Collector);
		}
	}
}

