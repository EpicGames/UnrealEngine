// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/RootMotionSourceGroupNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionSourceGroupNetSerializer)


#include "GameFramework/RootMotionSource.h"
#include "HAL/PlatformMemory.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/PackedVectorNetSerializers.h"
#include "Iris/Serialization/PolymorphicNetSerializerImpl.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

namespace UE::Net
{

/*
 * FRootMotionSourceGroup supports capping the number of root motion sources to be replicated via an added parameter to NetSerialize. It's used for movement adjustment RPCs.
 * Such RPCs will end up using the FCharacterNetworkSerializationPackedBitsNetSerializer instead so it shouldn't be a problem at this time.
 * If needed a private member with the desired max number of root motion sources to replicate can be added to the FRootMotionSourceGroup and accessed here in the quantization.
 * The quantize method may have to construct a new temporary instance and populating it with a capped amount of root motion sources and forward that to the polymorphic net serializer.
 */

struct FRootMotionSourceGroupNetSerializer
{
private:
	static TArrayView<TSharedPtr<FRootMotionSource>> GetRootMotionSources(FRootMotionSourceGroup& ArrayContainer);
	static void SetRootMotionSourcesNum(FRootMotionSourceGroup& ArrayContainer, SIZE_T Num);

	static TArrayView<TSharedPtr<FRootMotionSource>> GetPendingRootMotionSources(FRootMotionSourceGroup& ArrayContainer);
	static void SetPendingRootMotionSourcesNum(FRootMotionSourceGroup& ArrayContainer, SIZE_T Num);

public:	
	typedef FRootMotionSourceGroup SourceType;
	typedef FRootMotionSource SourceArrayItemType;
	typedef FRootMotionSourceGroupNetSerializerConfig ConfigType;
	using RootMotionSourcesSerializer = TPolymorphicArrayStructNetSerializerImpl<SourceType, SourceArrayItemType, GetRootMotionSources, SetRootMotionSourcesNum>;
	using PendingRootMotionSourcesSerializer = TPolymorphicArrayStructNetSerializerImpl<SourceType, SourceArrayItemType, GetPendingRootMotionSources, SetPendingRootMotionSourcesNum>;

	enum EPropertyFlags : uint32
	{
		HasAdditiveSources = 1U,
		HasOverrideSources = HasAdditiveSources << 1U,
		HasOverrideSourcesWithIgnoreZAccumulate = HasOverrideSources << 1U,
		IsAdditiveVelocityApplied = HasOverrideSourcesWithIgnoreZAccumulate << 1U,
		// Add flags above this comment and update BitCount.

		BitCount = 4
	};

	struct FFlags
	{
		uint32 RootMotionSourceSettings;
		uint32 PropertyFlags;
	};

	struct FQuantizedVelocity
	{
		alignas(16) uint64 Buffer[4];
	};

	struct FQuantizedData
	{
		RootMotionSourcesSerializer::QuantizedType RootMotionSources;
		PendingRootMotionSourcesSerializer::QuantizedType PendingRootMotionSources;
		FQuantizedVelocity LastPreAdditiveVelocity;
		FFlags Flags;
	};

	typedef FQuantizedData QuantizedType;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasCustomNetReference = true; // We don't know. Assume we need it.

	static const uint32 Version = 0;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

	static void InitTypeCache();

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		FNetSerializerRegistryDelegates();
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
		virtual void OnLoadedModulesUpdated() override;

		inline static const FName PropertyNetSerializerRegistry_NAME_RootMotionSourceGroup = FName("RootMotionSourceGroup");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RootMotionSourceGroup, FRootMotionSourceGroupNetSerializer);
	};

	static constexpr uint32 MaxReplicatedRootMotionSources = 255U;

	inline static bool bIsPostFreezeCalled = false;
	inline static FRootMotionSourceGroupNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;

	static const FNetSerializer* QuantizedVectorNetSerializer;
};

const FNetSerializer* FRootMotionSourceGroupNetSerializer::QuantizedVectorNetSerializer = &UE_NET_GET_SERIALIZER(FVectorNetQuantize10NetSerializer);

UE_NET_IMPLEMENT_SERIALIZER(FRootMotionSourceGroupNetSerializer);

void FRootMotionSourceGroupNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	Writer.WriteBits(Source.Flags.RootMotionSourceSettings, 8U);
	Writer.WriteBits(Source.Flags.PropertyFlags, EPropertyFlags::BitCount);

	{
		FNetSerializeArgs SerializeArgs = Args;
		SerializeArgs.Source = NetSerializerValuePointer(&Source.LastPreAdditiveVelocity);
		SerializeArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		QuantizedVectorNetSerializer->Serialize(Context, SerializeArgs);
	}

	{
		FNetSerializeArgs SerializeArgs = Args;
		SerializeArgs.Source = NetSerializerValuePointer(&Source.RootMotionSources);
		RootMotionSourcesSerializer::Serialize(Context, SerializeArgs);
	}

	{
		FNetSerializeArgs SerializeArgs = Args;
		SerializeArgs.Source = NetSerializerValuePointer(&Source.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::Serialize(Context, SerializeArgs);
	}
}

void FRootMotionSourceGroupNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	{
		Target.Flags.RootMotionSourceSettings = Reader.ReadBits(8U);
		Target.Flags.PropertyFlags = Reader.ReadBits(EPropertyFlags::BitCount);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	{
		FNetDeserializeArgs DeserializeArgs = Args;
		DeserializeArgs.Target = NetSerializerValuePointer(&Target.LastPreAdditiveVelocity);
		DeserializeArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		QuantizedVectorNetSerializer->Deserialize(Context, DeserializeArgs);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	{
		FNetDeserializeArgs DeserializeArgs = Args;
		DeserializeArgs.Target = NetSerializerValuePointer(&Target.RootMotionSources);
		RootMotionSourcesSerializer::Deserialize(Context, DeserializeArgs);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	{
		FNetDeserializeArgs DeserializeArgs = Args;
		DeserializeArgs.Target = NetSerializerValuePointer(&Target.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::Deserialize(Context, DeserializeArgs);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}
}

void FRootMotionSourceGroupNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamWriter& Writer = *Context.GetBitStreamWriter();

	const bool bFlagsAreDifferent = (Value.Flags.RootMotionSourceSettings != PrevValue.Flags.RootMotionSourceSettings) || (Value.Flags.PropertyFlags != PrevValue.Flags.PropertyFlags);
	if (Writer.WriteBool(bFlagsAreDifferent))
	{
		Writer.WriteBits(Value.Flags.RootMotionSourceSettings, 8U);
		Writer.WriteBits(Value.Flags.PropertyFlags, EPropertyFlags::BitCount);
	}

	{
		FNetSerializeDeltaArgs SerializeDeltaArgs = Args;
		SerializeDeltaArgs.Source = NetSerializerValuePointer(&Value.LastPreAdditiveVelocity);
		SerializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.LastPreAdditiveVelocity);
		SerializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		QuantizedVectorNetSerializer->SerializeDelta(Context, SerializeDeltaArgs);
	}

	{
		FNetSerializeDeltaArgs SerializeDeltaArgs = Args;
		SerializeDeltaArgs.Source = NetSerializerValuePointer(&Value.RootMotionSources);
		SerializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.RootMotionSources);
		RootMotionSourcesSerializer::SerializeDelta(Context, SerializeDeltaArgs);
	}

	{
		FNetSerializeDeltaArgs SerializeDeltaArgs = Args;
		SerializeDeltaArgs.Source = NetSerializerValuePointer(&Value.PendingRootMotionSources);
		SerializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::SerializeDelta(Context, SerializeDeltaArgs);
	}
}

void FRootMotionSourceGroupNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);
	const QuantizedType& PrevValue = *reinterpret_cast<const QuantizedType*>(Args.Prev);

	FNetBitStreamReader& Reader = *Context.GetBitStreamReader();

	Target.Flags = PrevValue.Flags;
	if (Reader.ReadBool())
	{
		Target.Flags.RootMotionSourceSettings = Reader.ReadBits(8U);
		Target.Flags.PropertyFlags = Reader.ReadBits(EPropertyFlags::BitCount);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	{
		FNetDeserializeDeltaArgs DeserializeDeltaArgs = Args;
		DeserializeDeltaArgs.Target = NetSerializerValuePointer(&Target.LastPreAdditiveVelocity);
		DeserializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.LastPreAdditiveVelocity);
		DeserializeDeltaArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		QuantizedVectorNetSerializer->DeserializeDelta(Context, DeserializeDeltaArgs);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	{
		FNetDeserializeDeltaArgs DeserializeDeltaArgs = Args;
		DeserializeDeltaArgs.Target = NetSerializerValuePointer(&Target.RootMotionSources);
		DeserializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.RootMotionSources);
		RootMotionSourcesSerializer::DeserializeDelta(Context, DeserializeDeltaArgs);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}

	{
		FNetDeserializeDeltaArgs DeserializeDeltaArgs = Args;
		DeserializeDeltaArgs.Target = NetSerializerValuePointer(&Target.PendingRootMotionSources);
		DeserializeDeltaArgs.Prev = NetSerializerValuePointer(&PrevValue.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::DeserializeDelta(Context, DeserializeDeltaArgs);
		if (Context.HasErrorOrOverflow())
		{
			return;
		}
	}
}

void FRootMotionSourceGroupNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	{
		Target.Flags.RootMotionSourceSettings = Source.LastAccumulatedSettings.Flags;
		Target.Flags.PropertyFlags = 0U;
		Target.Flags.PropertyFlags |= (Source.bHasAdditiveSources ?  EPropertyFlags::HasAdditiveSources : 0U);
		Target.Flags.PropertyFlags |= (Source.bHasOverrideSources ? EPropertyFlags::HasOverrideSources : 0U);
		Target.Flags.PropertyFlags |= (Source.bHasOverrideSourcesWithIgnoreZAccumulate ? EPropertyFlags::HasOverrideSourcesWithIgnoreZAccumulate : 0U);
		Target.Flags.PropertyFlags |= (Source.bIsAdditiveVelocityApplied ? EPropertyFlags::IsAdditiveVelocityApplied : 0U);
	}

	{
		FNetQuantizeArgs QuantizeArgs = Args;
		QuantizeArgs.Source = NetSerializerValuePointer(&Source.LastPreAdditiveVelocity);
		QuantizeArgs.Target = NetSerializerValuePointer(&Target.LastPreAdditiveVelocity);
		QuantizeArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		QuantizedVectorNetSerializer->Quantize(Context, QuantizeArgs);
	}

	{
		FNetQuantizeArgs QuantizeArgs = Args;
		// Note that the serializer operates directly on the source type so the Source should not be changed.
		QuantizeArgs.Target = NetSerializerValuePointer(&Target.RootMotionSources);
		RootMotionSourcesSerializer::Quantize(Context, QuantizeArgs);
		if (Context.HasError())
		{
			return;
		}
	}

	{
		FNetQuantizeArgs QuantizeArgs = Args;
		// Note that the serializer operates directly on the source type so the Source should not be changed.
		QuantizeArgs.Target = NetSerializerValuePointer(&Target.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::Quantize(Context, QuantizeArgs);
		if (Context.HasError())
		{
			return;
		}
	}
}

void FRootMotionSourceGroupNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	{
		Target.LastAccumulatedSettings.Flags = Source.Flags.RootMotionSourceSettings;
		Target.bHasAdditiveSources = (Source.Flags.PropertyFlags & EPropertyFlags::HasAdditiveSources) != 0U;
		Target.bHasOverrideSources = (Source.Flags.PropertyFlags & EPropertyFlags::HasOverrideSources) != 0U;
		Target.bHasOverrideSourcesWithIgnoreZAccumulate = (Source.Flags.PropertyFlags & EPropertyFlags::HasOverrideSourcesWithIgnoreZAccumulate) != 0U;
		Target.bIsAdditiveVelocityApplied = (Source.Flags.PropertyFlags & EPropertyFlags::IsAdditiveVelocityApplied) != 0U;
	}

	{
		FNetDequantizeArgs DequantizeArgs = Args;
		DequantizeArgs.Source = NetSerializerValuePointer(&Source.LastPreAdditiveVelocity);
		DequantizeArgs.Target = NetSerializerValuePointer(&Target.LastPreAdditiveVelocity);
		DequantizeArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		QuantizedVectorNetSerializer->Dequantize(Context, DequantizeArgs);
	}

	{
		FNetDequantizeArgs DequantizeArgs = Args;
		DequantizeArgs.Source = NetSerializerValuePointer(&Source.RootMotionSources);
		RootMotionSourcesSerializer::Dequantize(Context, DequantizeArgs);
	}

	{
		FNetDequantizeArgs DequantizeArgs = Args;
		DequantizeArgs.Source = NetSerializerValuePointer(&Source.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::Dequantize(Context, DequantizeArgs);
	}
}

bool FRootMotionSourceGroupNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
		const QuantizedType& ValueA = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& ValueB = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		bool bFlagsAreDifferent = false;
		bFlagsAreDifferent |= ValueA.Flags.RootMotionSourceSettings != ValueB.Flags.RootMotionSourceSettings;
		bFlagsAreDifferent |= ValueA.Flags.PropertyFlags != ValueB.Flags.PropertyFlags;
		if (bFlagsAreDifferent)
		{
			return false;
		}

		if (0 != FPlatformMemory::Memcmp(&ValueA.LastPreAdditiveVelocity, &ValueB.LastPreAdditiveVelocity, sizeof(FQuantizedVelocity)))
		{
			return false;
		}

		{
			FNetIsEqualArgs IsEqualArgs = Args;
			IsEqualArgs.Source0 = NetSerializerValuePointer(&ValueA.RootMotionSources);
			IsEqualArgs.Source1 = NetSerializerValuePointer(&ValueB.RootMotionSources);
			if (!RootMotionSourcesSerializer::IsEqual(Context, IsEqualArgs))
			{
				return false;
			}
		}

		{
			FNetIsEqualArgs IsEqualArgs = Args;
			IsEqualArgs.Source0 = NetSerializerValuePointer(&ValueA.PendingRootMotionSources);
			IsEqualArgs.Source1 = NetSerializerValuePointer(&ValueB.PendingRootMotionSources);
			if (!PendingRootMotionSourcesSerializer::IsEqual(Context, IsEqualArgs))
			{
				return false;
			}
		}
	}
	else
	{
		const SourceType& ValueA = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& ValueB = *reinterpret_cast<const SourceType*>(Args.Source1);
		return ValueA == ValueB;
	}

	return true;
}

bool FRootMotionSourceGroupNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);

	{
		FNetValidateArgs ValidateArgs = Args;
		ValidateArgs.Source = NetSerializerValuePointer(&Source.LastPreAdditiveVelocity);
		ValidateArgs.NetSerializerConfig = NetSerializerConfigParam(QuantizedVectorNetSerializer->DefaultConfig);
		if (!QuantizedVectorNetSerializer->Validate(Context, ValidateArgs))
		{
			return false;
		}
	}

	// The polymorphic serializer will extract the appropriate array and validate it. There's no need to make up new args until there's proper versioning support.
	if (!RootMotionSourcesSerializer::Validate(Context, Args))
	{
		return false;
	}

	if (!PendingRootMotionSourcesSerializer::Validate(Context, Args))
	{
		return false;
	}

	return true;
}

void FRootMotionSourceGroupNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	{
		FNetCloneDynamicStateArgs CloneArgs = Args;
		CloneArgs.Source = NetSerializerValuePointer(&Source.RootMotionSources);
		CloneArgs.Target = NetSerializerValuePointer(&Target.RootMotionSources);
		RootMotionSourcesSerializer::CloneDynamicState(Context, CloneArgs);
	}

	{
		FNetCloneDynamicStateArgs CloneArgs = Args;
		CloneArgs.Source = NetSerializerValuePointer(&Source.PendingRootMotionSources);
		CloneArgs.Target = NetSerializerValuePointer(&Target.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::CloneDynamicState(Context, CloneArgs);
	}
}

void FRootMotionSourceGroupNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Source = *reinterpret_cast<QuantizedType*>(Args.Source);

	{
		FNetFreeDynamicStateArgs FreeArgs = Args;
		FreeArgs.Source = NetSerializerValuePointer(&Source.RootMotionSources);
		RootMotionSourcesSerializer::FreeDynamicState(Context, FreeArgs);
	}

	{
		FNetFreeDynamicStateArgs FreeArgs = Args;
		FreeArgs.Source = NetSerializerValuePointer(&Source.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::FreeDynamicState(Context, FreeArgs);
	}
}

void FRootMotionSourceGroupNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);

	{
		FNetCollectReferencesArgs CollectNetReferencesArgs = Args;
		CollectNetReferencesArgs.Source = NetSerializerValuePointer(&Source.RootMotionSources);
		RootMotionSourcesSerializer::CollectNetReferences(Context, CollectNetReferencesArgs);
	}

	{
		FNetCollectReferencesArgs CollectNetReferencesArgs = Args;
		CollectNetReferencesArgs.Source = NetSerializerValuePointer(&Source.PendingRootMotionSources);
		PendingRootMotionSourcesSerializer::CollectNetReferences(Context, CollectNetReferencesArgs);
	}
}

void FRootMotionSourceGroupNetSerializer::InitTypeCache()
{
	// When post freeze is called we expect all custom serializers to have been registered so that the type cache will get the appropriate serializer when creating the ReplicationStateDescriptor.
	if (bIsPostFreezeCalled)
	{
		// We don't need to call InitTypeCache for RootMotionPendingSourcesSerializer as it's the FRootMotionSourceGroupNetSerializer DefaultConfig that is populated with relevant info.
		RootMotionSourcesSerializer::InitTypeCache<FRootMotionSourceGroupNetSerializer>();
	}
}

TArrayView<TSharedPtr<FRootMotionSource>> FRootMotionSourceGroupNetSerializer::GetRootMotionSources(FRootMotionSourceGroup& ArrayContainer)
{
	return MakeArrayView(ArrayContainer.RootMotionSources);
}

void FRootMotionSourceGroupNetSerializer::SetRootMotionSourcesNum(FRootMotionSourceGroup& ArrayContainer, SIZE_T Num)
{
	ArrayContainer.RootMotionSources.SetNum(static_cast<SSIZE_T>(Num));
}

TArrayView<TSharedPtr<FRootMotionSource>> FRootMotionSourceGroupNetSerializer::GetPendingRootMotionSources(FRootMotionSourceGroup& ArrayContainer)
{
	return MakeArrayView(ArrayContainer.PendingAddRootMotionSources);
}

void FRootMotionSourceGroupNetSerializer::SetPendingRootMotionSourcesNum(FRootMotionSourceGroup& ArrayContainer, SIZE_T Num)
{
	ArrayContainer.PendingAddRootMotionSources.SetNum(static_cast<SSIZE_T>(Num));
}

FRootMotionSourceGroupNetSerializer::FNetSerializerRegistryDelegates::FNetSerializerRegistryDelegates()
: UE::Net::FNetSerializerRegistryDelegates(EFlags::ShouldBindLoadedModulesUpdatedDelegate)
{
}

FRootMotionSourceGroupNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RootMotionSourceGroup);
}

void FRootMotionSourceGroupNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_RootMotionSourceGroup);
}

void FRootMotionSourceGroupNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	bIsPostFreezeCalled = true;
}

void FRootMotionSourceGroupNetSerializer::FNetSerializerRegistryDelegates::OnLoadedModulesUpdated()
{
	UE::Net::FRootMotionSourceGroupNetSerializer::InitTypeCache();
}

}

