// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/SaveGame.h"
#include "Engine/LocalPlayer.h"
#include "Kismet/GameplayStatics.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SaveGame)

ULocalPlayerSaveGame* ULocalPlayerSaveGame::LoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, APlayerController* LocalPlayerController, const FString& SlotName)
{
	const ULocalPlayer* LocalPlayer = LocalPlayerController ? LocalPlayerController->GetLocalPlayer() : nullptr;

	if (!ensure(LocalPlayer))
	{
		UE_LOGF(LogPlayerManagement, Error, "LoadOrCreateSaveGameForLocalPlayer called with null or remote player controller!");
		return nullptr;
	}

	return LoadOrCreateSaveGameForLocalPlayer(SaveGameClass, LocalPlayer, SlotName);
}

ULocalPlayerSaveGame* ULocalPlayerSaveGame::LoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer, const FString& SlotName)
{
	if (!ensure(LocalPlayer))
	{
		UE_LOGF(LogPlayerManagement, Error, "LoadOrCreateSaveGame called with null local player!");
		return nullptr;
	}

	if (!ensure(SaveGameClass))
	{
		UE_LOGF(LogPlayerManagement, Error, "LoadOrCreateSaveGame called with null SaveGameClass!");
		return nullptr;
	}

	if (!ensure(SlotName.Len() > 0))
	{
		UE_LOGF(LogPlayerManagement, Error, "LoadOrCreateSaveGame called with empty slot name!");
		return nullptr;
	}

	int32 UserIndex = LocalPlayer->GetPlatformUserIndex();

	// If the save game exists, load it.
	ULocalPlayerSaveGame* LoadedSave = nullptr;
	USaveGame* BaseSave = nullptr;
	if (UGameplayStatics::DoesSaveGameExist(SlotName, UserIndex))
	{
		BaseSave = UGameplayStatics::LoadGameFromSlot(SlotName, UserIndex);
	}

	LoadedSave = ProcessLoadedSave(BaseSave, SlotName, UserIndex, SaveGameClass, LocalPlayer);

	return LoadedSave;
}

bool ULocalPlayerSaveGame::AsyncLoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, APlayerController* LocalPlayerController, const FString& SlotName, FOnLocalPlayerSaveGameLoaded Delegate)
{
	const ULocalPlayer* LocalPlayer = LocalPlayerController ? LocalPlayerController->GetLocalPlayer() : nullptr;

	if (!ensure(LocalPlayer))
	{
		UE_LOGF(LogPlayerManagement, Error, "AsyncLoadOrCreateSaveGameForLocalPlayer called with null or remote player controller!");
		return false;
	}

	// Simple lambda that calls the dynamic delegate
	FOnLocalPlayerSaveGameLoadedNative Lambda = FOnLocalPlayerSaveGameLoadedNative::CreateLambda([Delegate]
		(ULocalPlayerSaveGame* LoadedSave)
		{
			Delegate.ExecuteIfBound(LoadedSave);
		});

	return AsyncLoadOrCreateSaveGameForLocalPlayer(SaveGameClass, LocalPlayer, SlotName, Lambda);
}

bool ULocalPlayerSaveGame::AsyncLoadOrCreateSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer, const FString& SlotName, FOnLocalPlayerSaveGameLoadedNative Delegate)
{
	if (!ensure(LocalPlayer))
	{
		UE_LOGF(LogPlayerManagement, Error, "AsyncLoadOrCreateSaveGame called with null local player!");
		return false;
	}

	if (!ensure(SaveGameClass))
	{
		UE_LOGF(LogPlayerManagement, Error, "AsyncLoadOrCreateSaveGame called with null SaveGameClass!");
		return false;
	}

	if (!ensure(SlotName.Len() > 0))
	{
		UE_LOGF(LogPlayerManagement, Error, "AsyncLoadOrCreateSaveGame called with empty slot name!");
		return false;
	}
	
	FAsyncLoadGameFromSlotDelegate Lambda = FAsyncLoadGameFromSlotDelegate::CreateWeakLambda(LocalPlayer, [LocalPlayer, Delegate, SaveGameClass]
		(const FString& SlotName, const int32 UserIndex, USaveGame* BaseSave) 
		{
			ULocalPlayerSaveGame* LoadedSave = ProcessLoadedSave(BaseSave, SlotName, UserIndex, SaveGameClass, LocalPlayer);
			
			Delegate.ExecuteIfBound(LoadedSave);
		});

	UGameplayStatics::AsyncLoadGameFromSlot(SlotName, LocalPlayer->GetPlatformUserIndex(), Lambda);

	return true;
}

ULocalPlayerSaveGame* ULocalPlayerSaveGame::ProcessLoadedSave(USaveGame* BaseSave, const FString& SlotName, const int32 UserIndex, TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer)
{
	ULocalPlayerSaveGame* LoadedSave = Cast<ULocalPlayerSaveGame>(BaseSave);
	if (BaseSave && (!LoadedSave || !LoadedSave->IsA(SaveGameClass.Get())))
	{
		UE_LOGF(LogPlayerManagement, Warning, "ProcessLoadedSave found invalid save game object %ls in slot %ls for player %d!", *GetNameSafe(LoadedSave), *SlotName, UserIndex);
		LoadedSave = nullptr;
	}

	if (LoadedSave == nullptr)
	{
		LoadedSave = CreateNewSaveGameForLocalPlayer(SaveGameClass, LocalPlayer, SlotName);
	}
	else
	{
		LoadedSave->InitializeSaveGame(LocalPlayer, SlotName, true);
	}

	return LoadedSave;
}

ULocalPlayerSaveGame* ULocalPlayerSaveGame::CreateNewSaveGameForLocalPlayer(TSubclassOf<ULocalPlayerSaveGame> SaveGameClass, const ULocalPlayer* LocalPlayer, const FString& SlotName)
{
	ULocalPlayerSaveGame* LoadedSave = Cast<ULocalPlayerSaveGame>(UGameplayStatics::CreateSaveGameObject(SaveGameClass));
	if (ensure(LoadedSave))
	{
		LoadedSave->InitializeSaveGame(LocalPlayer, SlotName, false);
	}
	return LoadedSave;
}

bool ULocalPlayerSaveGame::SaveGameToSlotForLocalPlayer()
{
	// Subclasses can override this to add additional checks like if one is in progress
	if (!ensure(GetLocalPlayer()))
	{
		UE_LOGF(LogPlayerManagement, Error, "SaveGameToSlotForLocalPlayer called with null local player!");
		return false;
	}

	int32 RequestUserIndex = GetPlatformUserIndex();
	FString RequestSlotName = GetSaveSlotName();
	if (!ensure(RequestSlotName.Len() > 0))
	{
		UE_LOGF(LogPlayerManagement, Error, "SaveGameToSlotForLocalPlayer called with empty slot name!");
		return false;
	}

	HandlePreSave();

	bool bSuccess = UGameplayStatics::SaveGameToSlot(this, RequestSlotName, RequestUserIndex);

	ProcessSaveComplete(RequestSlotName, RequestUserIndex, bSuccess, CurrentSaveRequest);

	return true;
}

bool ULocalPlayerSaveGame::AsyncSaveGameToSlotForLocalPlayer()
{
	// Subclasses can override this to add additional checks like if one is in progress
	if (!ensure(GetLocalPlayer()))
	{
		UE_LOGF(LogPlayerManagement, Error, "AsyncSaveGameToSlotForLocalPlayer called with null local player!");
		return false;
	}
	
	int32 RequestUserIndex = GetPlatformUserIndex();
	FString RequestSlotName = GetSaveSlotName();
	if (!ensure(RequestSlotName.Len() > 0))
	{
		UE_LOGF(LogPlayerManagement, Error, "AsyncSaveGameToSlotForLocalPlayer called with empty slot name!");
		return false;
	}

	HandlePreSave();

	// Start an async save and pass through the current save request count
	FAsyncSaveGameToSlotDelegate SavedDelegate;
	SavedDelegate.BindUObject(this, &ULocalPlayerSaveGame::ProcessSaveComplete, CurrentSaveRequest);
	UGameplayStatics::AsyncSaveGameToSlot(this, RequestSlotName, RequestUserIndex, SavedDelegate);

	return true;
}

void ULocalPlayerSaveGame::ProcessSaveComplete(const FString& SlotName, const int32 UserIndex, bool bSuccess, int32 SaveRequest)
{
	// Now that a save completed, update the success/failure and set based on the request that just completed
	// Every time CurrentSaveRequest is incremented it will result in setting either success or error, but there could be multiple in flight that are processed in order
	if (bSuccess)
	{
		ensure(CurrentSaveRequest > LastSuccessfulSaveRequest);
		LastSuccessfulSaveRequest = SaveRequest;
	}
	else
	{
		ensure(CurrentSaveRequest > LastErrorSaveRequest);
		LastErrorSaveRequest = SaveRequest;
	}

	HandlePostSave(bSuccess);
}

APlayerController* ULocalPlayerSaveGame::GetLocalPlayerController() const
{
	if (OwningPlayer)
	{
		return OwningPlayer->GetPlayerController(nullptr);
	}

	return nullptr;
}

const ULocalPlayer* ULocalPlayerSaveGame::GetLocalPlayer() const
{
	return OwningPlayer;
}

void ULocalPlayerSaveGame::SetLocalPlayer(const ULocalPlayer* LocalPlayer)
{
	OwningPlayer = LocalPlayer;
}

FPlatformUserId ULocalPlayerSaveGame::GetPlatformUserId() const
{
	if (OwningPlayer)
	{
		return OwningPlayer->GetPlatformUserId();
	}

	return FPlatformUserId();
}

int32 ULocalPlayerSaveGame::GetPlatformUserIndex() const
{
	if (OwningPlayer)
	{
		return OwningPlayer->GetPlatformUserIndex();
	}

	return INDEX_NONE;
}

FString ULocalPlayerSaveGame::GetSaveSlotName() const
{
	return SaveSlotName;
}

void ULocalPlayerSaveGame::SetSaveSlotName(const FString& SlotName)
{
	SaveSlotName = SlotName;
}

int32 ULocalPlayerSaveGame::GetSavedDataVersion() const
{
	return SavedDataVersion;
}

int32 ULocalPlayerSaveGame::GetInvalidDataVersion() const
{
	return -1;
}

int32 ULocalPlayerSaveGame::GetLatestDataVersion() const
{
	// Override with game-specific logic
	return 0;
}

bool ULocalPlayerSaveGame::WasLoaded() const
{
	return LoadedDataVersion != GetInvalidDataVersion();
}

bool ULocalPlayerSaveGame::IsSaveInProgress() const
{
	// If the current request is greater than both the last error and last success, that must mean it is in progress
	return (CurrentSaveRequest > LastErrorSaveRequest) && (CurrentSaveRequest > LastSuccessfulSaveRequest);
}

bool ULocalPlayerSaveGame::WasLastSaveSuccessful() const
{
	return (WasSaveRequested() && LastSuccessfulSaveRequest > LastErrorSaveRequest);
}

bool ULocalPlayerSaveGame::WasSaveRequested() const
{
	return CurrentSaveRequest > 0;
}

void ULocalPlayerSaveGame::InitializeSaveGame(const ULocalPlayer* LocalPlayer, FString InSlotName, bool bWasLoaded)
{
	SetLocalPlayer(LocalPlayer);
	SetSaveSlotName(InSlotName);

	if (bWasLoaded)
	{
		LoadedDataVersion = SavedDataVersion;
		HandlePostLoad();
	}
	else
	{
		ResetToDefault();
	}
}

void ULocalPlayerSaveGame::ResetToDefault()
{
	SavedDataVersion = GetInvalidDataVersion();
	LoadedDataVersion = SavedDataVersion;

	OnResetToDefault();
}

void ULocalPlayerSaveGame::HandlePostLoad()
{
	// Override this to handle SavedDataVersion fixups

	OnPostLoad();
}

void ULocalPlayerSaveGame::HandlePreSave()
{
	OnPreSave();

	// Set the save data version and increment the requested count

	SavedDataVersion = GetLatestDataVersion();
	CurrentSaveRequest++;

	UE_LOGF(LogPlayerManagement, Verbose, "Starting to save game %ls request %d to slot %ls for user %d", *GetName(), CurrentSaveRequest, *GetSaveSlotName(), GetPlatformUserIndex());
}

void ULocalPlayerSaveGame::HandlePostSave(bool bSuccess)
{
	// Override for error handling
	if (bSuccess)
	{
		UE_LOGF(LogPlayerManagement, Verbose, "Successfully saved game %ls request %d to slot %ls for user %d", *GetName(), LastSuccessfulSaveRequest, *GetSaveSlotName(), GetPlatformUserIndex());
	}
	else
	{
		UE_LOGF(LogPlayerManagement, Error, "Failed to save game %ls request %d to slot %ls for user %d!", *GetName(), LastErrorSaveRequest, *GetSaveSlotName(), GetPlatformUserIndex());
	}

	OnPostSave(bSuccess);
}
