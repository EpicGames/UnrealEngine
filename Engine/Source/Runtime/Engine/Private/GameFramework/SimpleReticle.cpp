// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/SimpleReticle.h"
#include "Engine/Canvas.h"

void FSimpleReticle::Draw( UCanvas* InCanvas, FLinearColor InColor )
{
	FVector2D CanvasCenter( InCanvas->OrgX + ((InCanvas->ClipX - InCanvas->OrgX) / 2.0f), InCanvas->OrgX + ((InCanvas->ClipY - InCanvas->OrgY) / 2.0f) );
	FCanvasLineItem LineItem( CanvasCenter, FVector2D(0.0f, 0.0f) );
	LineItem.SetColor( InColor );
	LineItem.Draw( InCanvas->Canvas, CanvasCenter - HorizontalOffsetMin, CanvasCenter - HorizontalOffsetMax );
	LineItem.Draw( InCanvas->Canvas, CanvasCenter + HorizontalOffsetMin, CanvasCenter + HorizontalOffsetMax );
	LineItem.Draw( InCanvas->Canvas, CanvasCenter - VerticalOffsetMin, CanvasCenter - VerticalOffsetMax );
	LineItem.Draw( InCanvas->Canvas, CanvasCenter + VerticalOffsetMin, CanvasCenter + VerticalOffsetMax );
}
