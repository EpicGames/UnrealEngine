// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/SpawnActorTimer.h"
#include "HAL/IConsoleManager.h"

#if ENABLE_SPAWNACTORTIMER

DECLARE_LOG_CATEGORY_EXTERN(LogSpawnActorTimer, Log, All);
DEFINE_LOG_CATEGORY(LogSpawnActorTimer);

void FSpawnActorTimer::SpawnActorTimerCmdFunc(const TArray<FString>& Args)
{
	if(Args.Num() == 1)
	{
		if (Args[0].Compare(FString("start"), ESearchCase::IgnoreCase) == 0)
		{
			Get().Start();
		}
		else if(Args[0].Compare(FString("stop"), ESearchCase::IgnoreCase) == 0)
		{
			Get().Stop();
		}
	}
}

void FSpawnActorTimer::Start()
{
	if(!bIsRecordingSpawnActorTimes)
	{
		bIsRecordingSpawnActorTimes = true;
		UE_LOGF(LogSpawnActorTimer, Log, "Starting SpawnActorTimer");
	}
}

void FSpawnActorTimer::Stop()
{
	if (bIsRecordingSpawnActorTimes)
	{
		OutputSpawnTimings();

		// Throw away recorded info
		SpawnActorInfos.Empty();
		IncompleteSpawnActorMap.Empty();
		FrameCount = 0;

		// Clear recording bool
		bIsRecordingSpawnActorTimes = false;
	}
}

void FSpawnActorTimer::IncrementFrameCount()
{
	if(bIsRecordingSpawnActorTimes)
	{
		FrameCount++;
	}
}

void FSpawnActorTimer::OutputSpawnTimings()
{
	UE_LOGF(LogSpawnActorTimer, Log, "===============");

	double SpawnActorTotalTime = 0.f;

	// Struct for storing stats aggregated by class
	struct FSpawnActorAggregateStats
	{
		FName ClassName;
		int32 NumSpawned;
		double TotalSpawnTime;
		double AverageSpawnTime;
	};

	// Array of all aggregated stats
	TArray<FSpawnActorAggregateStats> AggregateStats;
	// Map from class name to index of aggregated stats for that class in AggregateStats
	TMap<FName, int32> ClassNameToStatIndexMap;

	// For each spawn completed...
	for (const FSpawnActorTimingInfo& Info : SpawnActorInfos)
	{
		// If we already have an entry for this class, add to that entry
		int32* StatIndexPtr = ClassNameToStatIndexMap.Find(Info.ClassName);
		if(StatIndexPtr != nullptr)
		{
			FSpawnActorAggregateStats& AggStat = AggregateStats[*StatIndexPtr];
			check(AggStat.ClassName == Info.ClassName);
			AggStat.NumSpawned++;
			AggStat.TotalSpawnTime += Info.SpawnTime;
			AggStat.AverageSpawnTime = AggStat.TotalSpawnTime/(double)AggStat.NumSpawned;
		}
		// First entry for this class, create a new entry
		else
		{
			FSpawnActorAggregateStats NewAggStat;
			NewAggStat.ClassName = Info.ClassName;
			NewAggStat.NumSpawned = 1;
			NewAggStat.TotalSpawnTime = Info.SpawnTime;
			NewAggStat.AverageSpawnTime = Info.SpawnTime;

			// Add to aggregate stats array
			int32 StatIndex = AggregateStats.Add(NewAggStat);
			// Add to map to make lookup faster
			ClassNameToStatIndexMap.Add(Info.ClassName, StatIndex);
		}

		SpawnActorTotalTime += Info.SpawnTime; // Accumulate total spawn time
	}

	// Now print out each entry
	for(FSpawnActorAggregateStats& AggStat : AggregateStats)
	{
		UE_LOGF(LogSpawnActorTimer, Log, "%ls,%f,%f,%d", *AggStat.ClassName.ToString(), AggStat.AverageSpawnTime, AggStat.TotalSpawnTime, AggStat.NumSpawned);
	}

	UE_LOGF(LogSpawnActorTimer, Log, "%d Spawns over %d frames", SpawnActorInfos.Num(), FrameCount);
	UE_LOGF(LogSpawnActorTimer, Log, "Average spawn time per frame: %f", (SpawnActorTotalTime/(double)FrameCount) * 1000.f);
	UE_LOGF(LogSpawnActorTimer, Log, "===============");
}

FSpawnActorTimer& FSpawnActorTimer::Get()
{
	static FSpawnActorTimer Singleton;
	return Singleton;
}

void FSpawnActorTimer::ReportSpawnActor(FName ClassName, FName ActorName, double SpawnTime, ESpawnActorTimingType SpawnActorType)
{
	if(bIsRecordingSpawnActorTimes && ActorName != NAME_None)
	{
		// If it's a SpawnActor timing..
		if(SpawnActorType != ESpawnActorTimingType::FinishSpawning)
		{
			// Fill in info struct
			FSpawnActorTimingInfo NewInfo;
			NewInfo.ClassName = ClassName;
			NewInfo.SpawnTime = SpawnTime;

			// If deferring, add to 'incomplete' map
			if(SpawnActorType == ESpawnActorTimingType::SpawnActorDeferred)
			{
				FSpawnActorTimingInfo* ExistingInfo = IncompleteSpawnActorMap.Find(ActorName);
				if (ExistingInfo != nullptr)
				{
					UE_LOGF(LogSpawnActorTimer, Warning, "Trying to start deferred spawn for '%ls', but already an incomplete entry!", *ActorName.ToString());
				}
				else
				{
					IncompleteSpawnActorMap.Add(ActorName, NewInfo);
				}
			}
			// If not deferring, add straight to 'completed' map
			else
			{
				SpawnActorInfos.Add(NewInfo);
			}
		}
		// If it's a FinishSpawning timing 
		else
		{
			// See if this is an actor we haven't finished spawning yet
			// If there isn't, that isn't an error condition, it just means that FinishSpawning was called non-deferred (ie before SpawnActor timing has been logged),
			// so we just ignore it here
			FSpawnActorTimingInfo* ExistingInfo = IncompleteSpawnActorMap.Find(ActorName);
			if (ExistingInfo != nullptr)
			{
				FSpawnActorTimingInfo NewInfo = *ExistingInfo;
				// Add finish time to current time
				NewInfo.SpawnTime += SpawnTime;
				// Add to completed array
				SpawnActorInfos.Add(NewInfo);
				// Remove from incomplete map
				IncompleteSpawnActorMap.Remove(ActorName);
			}
		}
	}
}

static FAutoConsoleCommand SpawnActorTimerCmd(
	TEXT("spawnactortimer"),
	TEXT("Allows recording of spawn actor times."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FSpawnActorTimer::SpawnActorTimerCmdFunc)
	);

#endif // ENABLE_SPAWNACTORTIMER
