// Copyright Epic Games, Inc. All Rights Reserved.

/** Util for recording SpawnActor times */

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"

// Define that controls compilation of this feature
#if (!UE_BUILD_SHIPPING && 1)
#define ENABLE_SPAWNACTORTIMER 1
#else
#define ENABLE_SPAWNACTORTIMER 0
#endif

#if ENABLE_SPAWNACTORTIMER

/** Info about one spawn actor event */
struct FSpawnActorTimingInfo
{
	/** Name of class we spawned */
	FName ClassName;
	/** Time it took to spawn this instance */
	double SpawnTime;
};

/** What kind of spawn event we are timing */
enum class ESpawnActorTimingType
{
	SpawnActorNonDeferred,
	SpawnActorDeferred,
	FinishSpawning
};

/** Class for saving times about spawn actor events */
class FSpawnActorTimer
{
public:
	FSpawnActorTimer()
		: bIsRecordingSpawnActorTimes(false)
		, FrameCount(0)
	{}

	/** Tell the timer about a spawn event */
	void ReportSpawnActor(FName ClassName, FName ActorName, double SpawnTime, ESpawnActorTimingType SpawnActorType);

	/** Increment the current frame count while timing spawns */
	void IncrementFrameCount();

	/** Static accessor for timer singleton */
	static FSpawnActorTimer& Get();

	/** Handler for 'spawnactortimer' console command */
	static void SpawnActorTimerCmdFunc(const TArray<FString>& Args);

private:

	/** Start recording */
	void Start();
	/** Stop recording */
	void Stop();
	/** Output current spawn info to log */
	void OutputSpawnTimings();

	/** If we are currently recording spawn actor times */
	bool bIsRecordingSpawnActorTimes;

	/** Store of all spawn actor timings */
	TArray<FSpawnActorTimingInfo> SpawnActorInfos;

	/** Map of currently incomplete spawn actor timings */
	TMap<FName, FSpawnActorTimingInfo> IncompleteSpawnActorMap;

	/** Number of frames that we have been recording spawn actor times for. */
	int32 FrameCount;
};

/** Scoped timer for actor spawning */
class FScopedSpawnActorTimer
{
public:
	FScopedSpawnActorTimer(FName InClassName, ESpawnActorTimingType InSpawnActorType)
		: StartTime(FPlatformTime::Seconds())
		, ClassName(InClassName)
		, ActorName(NAME_None)
		, SpawnActorType(InSpawnActorType)
	{
	}

	~FScopedSpawnActorTimer()
	{
		double StopTime = FPlatformTime::Seconds();
		double ElapsedTime = StopTime - StartTime;
		FSpawnActorTimer::Get().ReportSpawnActor(ClassName, ActorName, ElapsedTime, SpawnActorType);
	}

	void SetActorName(FName InActorName)
	{
		ActorName = InActorName;
	}

private:
	/** Time we started the scoped timing */
	double StartTime;
	/** Name of class we are spawning */
	FName ClassName;
	/** Name of actor we are spawning */
	FName ActorName;
	/** Type of spawn event we are recording */
	ESpawnActorTimingType SpawnActorType;
};

#endif // ENABLE_SPAWNACTORTIMER
