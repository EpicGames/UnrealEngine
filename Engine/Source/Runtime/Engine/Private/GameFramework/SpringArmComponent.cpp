// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/SpringArmComponent.h"
#include "GameFramework/Pawn.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "DrawDebugHelpers.h"
#include "Math/RotationMatrix.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpringArmComponent)

//////////////////////////////////////////////////////////////////////////
// USpringArmComponent

const FName USpringArmComponent::SocketName(TEXT("SpringEndpoint"));

USpringArmComponent::USpringArmComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
	
	bAutoActivate = true;
	bTickInEditor = true;
	bUsePawnControlRotation = false;
	bDoCollisionTest = true;

	bInheritPitch = true;
	bInheritYaw = true;
	bInheritRoll = true;

	TargetArmLength = 300.0f;
	ProbeSize = 12.0f;
	ProbeChannel = ECC_Camera;

	RelativeSocketRotation = FQuat::Identity;

	bUseCameraLagSubstepping = true;
	CameraLagSpeed = 10.f;
	CameraRotationLagSpeed = 10.f;
	CameraLagMaxTimeStep = 1.f / 60.f;
	CameraLagMaxDistance = 0.f;
	bClampToMaxPhysicsDeltaTime = false;

 	UnfixedCameraPosition = FVector::ZeroVector;
}

FRotator USpringArmComponent::GetDesiredRotation() const
{
	return GetComponentRotation();
}

FRotator USpringArmComponent::GetTargetRotation() const
{
	FRotator DesiredRot = GetDesiredRotation();

	if (bUsePawnControlRotation)
	{
		if (APawn* OwningPawn = Cast<APawn>(GetOwner()))
		{
			const FRotator PawnViewRotation = OwningPawn->GetViewRotation();
			if (DesiredRot != PawnViewRotation)
			{
				DesiredRot = PawnViewRotation;
			}
		}
	}

	// If inheriting rotation, check options for which components to inherit
	if (!IsUsingAbsoluteRotation())
	{
		const FRotator LocalRelativeRotation = GetRelativeRotation();
		if (!bInheritPitch)
		{
			DesiredRot.Pitch = LocalRelativeRotation.Pitch;
		}

		if (!bInheritYaw)
		{
			DesiredRot.Yaw = LocalRelativeRotation.Yaw;
		}

		if (!bInheritRoll)
		{
			DesiredRot.Roll = LocalRelativeRotation.Roll;
		}
	}

	return DesiredRot;
}

void USpringArmComponent::UpdateDesiredArmLocation(bool bDoTrace, bool bDoLocationLag, bool bDoRotationLag, float DeltaTime)
{
	FRotator DesiredRot = GetTargetRotation();

	// If our viewtarget is simulating using physics, we may need to clamp deltatime
	if (bClampToMaxPhysicsDeltaTime)
	{
		// Use the same max timestep cap as the physics system to avoid camera jitter when the viewtarget simulates less time than the camera
		DeltaTime = FMath::Min(DeltaTime, UPhysicsSettings::Get()->MaxPhysicsDeltaTime);
	}

	// Apply 'lag' to rotation if desired
	if(bDoRotationLag)
	{
		if (bUseCameraLagSubstepping && DeltaTime > CameraLagMaxTimeStep && CameraRotationLagSpeed > 0.f)
		{
			const FRotator ArmRotStep = (DesiredRot - PreviousDesiredRot).GetNormalized() * (1.f / DeltaTime);
			FRotator LerpTarget = PreviousDesiredRot;
			float RemainingTime = DeltaTime;
			while (RemainingTime > UE_KINDA_SMALL_NUMBER)
			{
				const float LerpAmount = FMath::Min(CameraLagMaxTimeStep, RemainingTime);
				LerpTarget += ArmRotStep * LerpAmount;
				RemainingTime -= LerpAmount;

				DesiredRot = FRotator(FMath::QInterpTo(FQuat(PreviousDesiredRot), FQuat(LerpTarget), LerpAmount, CameraRotationLagSpeed));
				PreviousDesiredRot = DesiredRot;
			}
		}
		else
		{
			DesiredRot = FRotator(FMath::QInterpTo(FQuat(PreviousDesiredRot), FQuat(DesiredRot), DeltaTime, CameraRotationLagSpeed));
		}
	}
	PreviousDesiredRot = DesiredRot;

	// Get the spring arm 'origin', the target we want to look at
	FVector ArmOrigin = GetComponentLocation() + TargetOffset;
	// We lag the target, not the actual camera position, so rotating the camera around does not have lag
	FVector DesiredLoc = ArmOrigin;
	if (bDoLocationLag)
	{
		if (bUseCameraLagSubstepping && DeltaTime > CameraLagMaxTimeStep && CameraLagSpeed > 0.f)
		{
			const FVector ArmMovementStep = (DesiredLoc - PreviousDesiredLoc) * (1.f / DeltaTime);
			FVector LerpTarget = PreviousDesiredLoc;

			float RemainingTime = DeltaTime;
			while (RemainingTime > UE_KINDA_SMALL_NUMBER)
			{
				const float LerpAmount = FMath::Min(CameraLagMaxTimeStep, RemainingTime);
				LerpTarget += ArmMovementStep * LerpAmount;
				RemainingTime -= LerpAmount;

				DesiredLoc = FMath::VInterpTo(PreviousDesiredLoc, LerpTarget, LerpAmount, CameraLagSpeed);
				PreviousDesiredLoc = DesiredLoc;
			}
		}
		else
		{
			DesiredLoc = FMath::VInterpTo(PreviousDesiredLoc, DesiredLoc, DeltaTime, CameraLagSpeed);
		}

		// Clamp distance if requested
		bool bClampedDist = false;
		if (CameraLagMaxDistance > 0.f)
		{
			const FVector FromOrigin = DesiredLoc - ArmOrigin;
			if (FromOrigin.SizeSquared() > FMath::Square(CameraLagMaxDistance))
			{
				DesiredLoc = ArmOrigin + FromOrigin.GetClampedToMaxSize(CameraLagMaxDistance);
				bClampedDist = true;
			}
		}		

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (bDrawDebugLagMarkers)
		{
			DrawDebugSphere(GetWorld(), ArmOrigin, 5.f, 8, FColor::Green);
			DrawDebugSphere(GetWorld(), DesiredLoc, 5.f, 8, FColor::Yellow);

			const FVector ToOrigin = ArmOrigin - DesiredLoc;
			DrawDebugDirectionalArrow(GetWorld(), DesiredLoc, DesiredLoc + ToOrigin * 0.5f, 7.5f, bClampedDist ? FColor::Red : FColor::Green);
			DrawDebugDirectionalArrow(GetWorld(), DesiredLoc + ToOrigin * 0.5f, ArmOrigin,  7.5f, bClampedDist ? FColor::Red : FColor::Green);
		}
#endif
	}

	PreviousArmOrigin = ArmOrigin;
	PreviousDesiredLoc = DesiredLoc;

	// Now offset camera position back along our rotation
	DesiredLoc -= DesiredRot.Vector() * TargetArmLength;
	// Add socket offset in local space
	DesiredLoc += FRotationMatrix(DesiredRot).TransformVector(SocketOffset);

	// Do a sweep to ensure we are not penetrating the world
	FVector ResultLoc;
	if (bDoTrace && (TargetArmLength != 0.0f))
	{
		bIsCameraFixed = true;
		FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(SpringArm), false, GetOwner());

		FHitResult Result;
		GetWorld()->SweepSingleByChannel(Result, ArmOrigin, DesiredLoc, FQuat::Identity, ProbeChannel, FCollisionShape::MakeSphere(ProbeSize), QueryParams);
		
		UnfixedCameraPosition = DesiredLoc;

		ResultLoc = BlendLocations(DesiredLoc, Result.Location, Result.bBlockingHit, DeltaTime);

		if (ResultLoc == DesiredLoc) 
		{	
			bIsCameraFixed = false;
		}
	}
	else
	{
		ResultLoc = DesiredLoc;
		bIsCameraFixed = false;
		UnfixedCameraPosition = ResultLoc;
	}

	// Form a transform for new world transform for camera
	FTransform WorldCamTM(DesiredRot, ResultLoc);
	// Convert to relative to component
	FTransform RelCamTM = WorldCamTM.GetRelativeTransform(GetComponentTransform());

	// Update socket location/rotation
	RelativeSocketLocation = RelCamTM.GetLocation();
	RelativeSocketRotation = RelCamTM.GetRotation();

	UpdateChildTransforms();
}

FVector USpringArmComponent::BlendLocations(const FVector& DesiredArmLocation, const FVector& TraceHitLocation, bool bHitSomething, float DeltaTime)
{
	return bHitSomething ? TraceHitLocation : DesiredArmLocation;
}

void USpringArmComponent::ApplyWorldOffset(const FVector & InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);
	PreviousDesiredLoc += InOffset;
	PreviousArmOrigin += InOffset;
}

void USpringArmComponent::OnRegister()
{
	Super::OnRegister();

	// enforce reasonable limits to avoid potential div-by-zero
	CameraLagMaxTimeStep = FMath::Max(CameraLagMaxTimeStep, 1.f / 200.f);
	CameraLagSpeed = FMath::Max(CameraLagSpeed, 0.f);

	// Set initial location (without lag).
	UpdateDesiredArmLocation(false, false, false, 0.f);
}

void USpringArmComponent::PostLoad()
{
	Super::PostLoad();
}

void USpringArmComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	UpdateDesiredArmLocation(bDoCollisionTest, bEnableCameraLag, bEnableCameraRotationLag, DeltaTime);
}

FTransform USpringArmComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	FTransform RelativeTransform(RelativeSocketRotation, RelativeSocketLocation);

	switch(TransformSpace)
	{
		case RTS_World:
		{
			return RelativeTransform * GetComponentTransform();
		}
		case RTS_Actor:
		{
			if( const AActor* Actor = GetOwner() )
			{
				FTransform SocketTransform = RelativeTransform * GetComponentTransform();
				return SocketTransform.GetRelativeTransform(Actor->GetTransform());
			}
			break;
		}
		case RTS_Component:
		{
			return RelativeTransform;
		}
	}
	return RelativeTransform;
}

bool USpringArmComponent::HasAnySockets() const
{
	return true;
}

void USpringArmComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	new (OutSockets) FComponentSocketDescription(SocketName, EComponentSocketType::Socket);
}

FVector USpringArmComponent::GetUnfixedCameraPosition() const
{
	return UnfixedCameraPosition;
}

bool USpringArmComponent::IsCollisionFixApplied() const
{
	return bIsCameraFixed;
}

