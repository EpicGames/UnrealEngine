// Copyright Epic Games, Inc. All Rights Reserved.
//

#include "Tests/AutomationCommon.h"
#include "GenericPlatform/IInputInterface.h"
#include "Framework/Application/SlateApplication.h"

DEFINE_LOG_CATEGORY_STATIC(LogHapticTest, Display, All);

#if WITH_DEV_AUTOMATION_TESTS

/**
* Play Low Level Haptic Effect by Amp/Freq
*/
DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FPlayAmplitudeFrequencyHapticLatentCommand, EControllerHand, Hand, float, Amplitude, float, Frequency);
bool FPlayAmplitudeFrequencyHapticLatentCommand::Update()
{
	const int32 ControllerId = 0;

	IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
	if (InputInterface)
	{
		FHapticFeedbackValues HapticValues;
		HapticValues.Amplitude = Amplitude;
		HapticValues.Frequency = Frequency;

		InputInterface->SetHapticFeedbackValues(ControllerId, (int32)Hand, HapticValues);
	}

	return true;
}


//@TODO - remove editor
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAmplitudeFrequencyHapticTest, "System.VR.All.Haptics.AmplitudeAndFrequency", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAmplitudeFrequencyHapticTest::RunTest(const FString&)
{
	float ActiveDuration = 1.0f;
	uint32 MaxHand = ((uint32)EControllerHand::Right) + 1;
	for (uint32 HandIndex = 0; HandIndex < MaxHand; ++HandIndex)
	{
		//Amplitude Checks
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, .25f, 1.0f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, .5f,  1.0f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, .75f, 1.0f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, 1.f, 1.0f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));

		//Frequency Checks
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, 1.0f, .25f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, 1.0f, .5f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, 1.0f, .75f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, 1.0f, 1.f));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(ActiveDuration));

		//turn back off!
		ADD_LATENT_AUTOMATION_COMMAND(FPlayAmplitudeFrequencyHapticLatentCommand((EControllerHand)HandIndex, 0.f, 0.0f));
	}


	return true;
}

#endif
