// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/TouchInterface.h"
#include "Engine/Texture2D.h"
#include "Widgets/Input/SVirtualJoystick.h"
#include "Slate/DeferredCleanupSlateBrush.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TouchInterface)

UTouchInterface::UTouchInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// defaults
	ActiveOpacity = 1.0f;
	InactiveOpacity = 0.1f;
	TimeUntilDeactive = 0.5f;
	TimeUntilReset = 2.0f;
	ActivationDelay = 0.f;
	StartupDelay = 0.f;

	bPreventRecenter = false;
}

void UTouchInterface::Activate(TSharedPtr<SVirtualJoystick> VirtualJoystick)
{
	if (VirtualJoystick.IsValid())
	{
		VirtualJoystick->SetGlobalParameters(ActiveOpacity, InactiveOpacity, TimeUntilDeactive, TimeUntilReset, ActivationDelay, bPreventRecenter, StartupDelay);

		// convert from the UStructs to the slate structs
		TArray<SVirtualJoystick::FControlInfo> SlateControls;

		for (int32 ControlIndex = 0; ControlIndex < Controls.Num(); ControlIndex++)
		{
			FTouchInputControl Control = Controls[ControlIndex];
			SVirtualJoystick::FControlInfo* SlateControl = new(SlateControls)SVirtualJoystick::FControlInfo;

			SlateControl->bTreatAsButton = Control.bTreatAsButton;
			SlateControl->Image1 = Control.Image1 ? StaticCastSharedRef<ISlateBrushSource>(FDeferredCleanupSlateBrush::CreateBrush(Control.Image1)) : TSharedPtr<ISlateBrushSource>();
			SlateControl->Image2 = Control.Image2 ? StaticCastSharedRef<ISlateBrushSource>(FDeferredCleanupSlateBrush::CreateBrush(Control.Image2)) : TSharedPtr<ISlateBrushSource>();
			SlateControl->Center = Control.Center;
			SlateControl->VisualSize = Control.VisualSize;
			SlateControl->ThumbSize = Control.ThumbSize;
			if (Control.InputScale.SizeSquared() > FMath::Square(UE_DELTA))
			{
				SlateControl->InputScale = Control.InputScale;
			}
			SlateControl->InteractionSize = Control.InteractionSize;
			SlateControl->MainInputKey = Control.MainInputKey;
			SlateControl->AltInputKey = Control.AltInputKey;
		}

		// set them as active!
		VirtualJoystick->SetControls(SlateControls);
	}
}

