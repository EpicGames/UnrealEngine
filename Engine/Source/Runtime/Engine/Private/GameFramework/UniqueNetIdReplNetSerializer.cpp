// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/UniqueNetIdReplNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniqueNetIdReplNetSerializer)


#include "GameFramework/InternalUniqueNetIdReplNetSerializer.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"
#include "Net/OnlineEngineInterface.h"

namespace UE::Net
{

struct FUniqueNetIdReplNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true;
	static constexpr bool bHasDynamicState = true;

	// Types
	enum : uint32
	{
		TypeHash_Default = 0U,
		TypeHash_Other = 31U,

		TypeHashBitCount = 5U,
	};

	struct FQuantizedType
	{
		// If it's not valid it's empty.
		uint32 bIsValid : 1U;
		// Whether the ID is represented by a string or an integer.
		uint32 bIsString : 1U;
		// The type of the online ID.
		uint32 TypeHash : TypeHashBitCount;

		uint64 Number;

		// Used when not encoded as a number.
		alignas(8) uint8 QuantizedStringID[16];
		// Used when TypeHash is TypeHash_Other.
		alignas(8) uint8 QuantizedTypeName[32];
	};

	typedef FUniqueNetIdRepl SourceType;
	typedef FQuantizedType QuantizedType;
	typedef struct FUniqueNetIdReplNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs&);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static void FreeDynamicState(FNetSerializationContext&, QuantizedType& Value);

	static FUniqueNetIdReplNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig StructNetSerializerConfigForString;
	static FStructNetSerializerConfig StructNetSerializerConfigForName;
	static const FNetSerializer* StructNetSerializer;
};
UE_NET_IMPLEMENT_SERIALIZER(FUniqueNetIdReplNetSerializer);

const FUniqueNetIdReplNetSerializer::ConfigType FUniqueNetIdReplNetSerializer::DefaultConfig;
FUniqueNetIdReplNetSerializer::FNetSerializerRegistryDelegates FUniqueNetIdReplNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FUniqueNetIdReplNetSerializer::StructNetSerializerConfigForString;
FStructNetSerializerConfig FUniqueNetIdReplNetSerializer::StructNetSerializerConfigForName;
const FNetSerializer* FUniqueNetIdReplNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FUniqueNetIdReplNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	if (!Writer->WriteBool(Value.bIsValid))
	{
		return;
	}

	Writer->WriteBits(Value.bIsString, 1U);
	Writer->WriteBits(Value.TypeHash, TypeHashBitCount);

	if (Value.bIsString)
	{
		FNetSerializeArgs SerializeArgs = {};
		SerializeArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		SerializeArgs.Source = NetSerializerValuePointer(&Value.QuantizedStringID);
		StructNetSerializer->Serialize(Context, SerializeArgs);
	}
	else
	{
		// $IRIS TODO : Consider WritePackedUint64.
		WriteUint64(Writer, Value.Number);
	}

	if (Value.TypeHash == TypeHash_Other)
	{
		FNetSerializeArgs SerializeArgs = {};
		SerializeArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		SerializeArgs.Source = NetSerializerValuePointer(&Value.QuantizedTypeName);
		StructNetSerializer->Serialize(Context, SerializeArgs);
	}
}

void FUniqueNetIdReplNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	QuantizedType TempValue = {};
	
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();
	TempValue.bIsValid = Reader->ReadBool();
	if (!TempValue.bIsValid)
	{
		// Free any dynamic state
		FreeDynamicState(Context, TargetValue);
		TargetValue = TempValue;
		return;
	}

	TempValue.bIsString = Reader->ReadBits(1);
	TempValue.TypeHash = Reader->ReadBits(TypeHashBitCount);

	if (TempValue.bIsString)
	{
		FNetDeserializeArgs DeserializeArgs = {};
		DeserializeArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		DeserializeArgs.Target = NetSerializerValuePointer(&TempValue.QuantizedStringID);
		StructNetSerializer->Deserialize(Context, DeserializeArgs);
	}
	else
	{
		TempValue.Number = ReadUint64(Reader);
	}

	if (TempValue.TypeHash == TypeHash_Other)
	{
		FNetDeserializeArgs DeserializeArgs = {};
		DeserializeArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		DeserializeArgs.Target = NetSerializerValuePointer(&TempValue.QuantizedTypeName);
		StructNetSerializer->Deserialize(Context, DeserializeArgs);
	}

	FreeDynamicState(Context, TargetValue);
	TargetValue = TempValue;
}

void FUniqueNetIdReplNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	return NetSerializeDeltaDefault<FUniqueNetIdReplNetSerializer::Serialize>(Context, Args);
}

void FUniqueNetIdReplNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	return NetDeserializeDeltaDefault<FUniqueNetIdReplNetSerializer::Deserialize>(Context, Args);
}

void FUniqueNetIdReplNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	// Avoiding complex logic. Free any memory allocations.
	if (TargetValue.bIsString)
	{
		FNetFreeDynamicStateArgs FreeArgs = {};
		FreeArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		FreeArgs.Source = NetSerializerValuePointer(&TargetValue.QuantizedStringID);
		StructNetSerializer->FreeDynamicState(Context, FreeArgs);
	}

	if (TargetValue.TypeHash == TypeHash_Other)
	{
		FNetFreeDynamicStateArgs FreeArgs = {};
		FreeArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		FreeArgs.Source = NetSerializerValuePointer(&TargetValue.QuantizedTypeName);
		StructNetSerializer->FreeDynamicState(Context, FreeArgs);
	}

	// Reset state.
	TargetValue.bIsValid = 0;
	TargetValue.bIsString = 0;
	TargetValue.TypeHash = 0;
	TargetValue.Number = 0;

	if (!SourceValue.IsValid())
	{
		return;
	}

	const FString& StringID = SourceValue.ToString();
	// This is compatible with FUniqueNetIdRepl::MakeReplicationData() which treats all empty strings this way, regardless of the ID type.
	if (StringID.IsEmpty())
	{
		return;
	}

	bool bIsNumber = StringID.Len() > 0;
	uint64 StringAsNumber = 0;
	// Check whether the ID can be encoded as a number.
	// $IRIS TODO: Turns out the known common case will be a hex-encoded string and with a length greater than what could fit in a 64-bit number.
	for (const TCHAR Char : StringID)
	{
		bIsNumber = FChar::IsDigit(Char);
		if (!bIsNumber)
		{
			break;
		}

		const uint64 PrevStringAsNumber = StringAsNumber;
		StringAsNumber = (StringAsNumber*10U) + (Char - TEXT('0'));
		// Detect leading zeros and integer overflow which would cause the inability to restore the original string from the number.
		if (StringAsNumber <= PrevStringAsNumber)
		{
			bIsNumber = false;
			break;
		}
	}

	// Check the type.
	const FName TypeName = SourceValue->GetType();
	uint8 TypeHash = UOnlineEngineInterface::Get()->GetReplicationHashForSubsystem(TypeName);
	if (TypeHash >= TypeHash_Other)
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	if (TypeHash == 0 && TypeName != NAME_None)
	{
		TypeHash = TypeHash_Other;
	}

	// Fill in the information
	TargetValue.bIsValid = 1U,
	TargetValue.bIsString = !bIsNumber;
	if (bIsNumber)
	{
		TargetValue.Number = StringAsNumber;
	}
	else
	{
		// Store ID as string
		FNetQuantizeArgs QuantizeArgs = {};
		QuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		QuantizeArgs.Source = NetSerializerValuePointer(&StringID);
		QuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.QuantizedStringID);
		StructNetSerializer->Quantize(Context, QuantizeArgs);
	}

	TargetValue.TypeHash = TypeHash;
	if (TargetValue.TypeHash == TypeHash_Other)
	{
		// Store type name
		FNetQuantizeArgs QuantizeArgs = {};
		QuantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		QuantizeArgs.Source = NetSerializerValuePointer(&TypeName);
		QuantizeArgs.Target = NetSerializerValuePointer(&TargetValue.QuantizedTypeName);
		StructNetSerializer->Quantize(Context, QuantizeArgs);
	}
}

void FUniqueNetIdReplNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	if (!Source.bIsValid)
	{
		SourceType Empty;
		Target = MoveTemp(Empty);
		return;
	}

	const uint32 TypeHash = Source.TypeHash;
	FName TypeName;
	if (TypeHash == TypeHash_Default)
	{
		// If no type was encoded, assume default
		TypeName = UOnlineEngineInterface::Get()->GetDefaultOnlineSubsystemName();
	}
	else if (TypeHash == TypeHash_Other)
	{
		FNetDequantizeArgs DequantizeArgs = {};
		DequantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		DequantizeArgs.Source = NetSerializerValuePointer(&Source.QuantizedTypeName);
		DequantizeArgs.Target = NetSerializerValuePointer(&TypeName);
		StructNetSerializer->Dequantize(Context, DequantizeArgs);
		if (Context.HasError())
		{
			return;
		}
	}
	else
	{
		TypeName = UOnlineEngineInterface::Get()->GetSubsystemFromReplicationHash(TypeHash);
	}

	// We always expect the type name to be valid.
	if (TypeName.IsNone())
	{
		Context.SetError(GNetError_InvalidValue);
		return;
	}

	FString StringID;
	if (Source.bIsString)
	{
		FNetDequantizeArgs DequantizeArgs = {};
		DequantizeArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		DequantizeArgs.Source = NetSerializerValuePointer(&Source.QuantizedStringID);
		DequantizeArgs.Target = NetSerializerValuePointer(&StringID);
		StructNetSerializer->Dequantize(Context, DequantizeArgs);
		if (Context.HasError())
		{
			return;
		}
	}
	else
	{
		FCString::CharType NumberAsStringBuffer[32];
		const int32 CharCount = FCString::Sprintf(NumberAsStringBuffer, TEXT("%" UINT64_FMT), Source.Number);
		// No point in validating CharCount as only a bad Sprintf implementation could fail. We expect Sprintf to be well-tested.
		StringID.AppendChars(NumberAsStringBuffer, CharCount);
	}

	// Code from FNetUniqueIdRepl::UniqueIdFromString
	{
		FUniqueNetIdWrapper UniqueNetIdWrapper = UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(StringID, TypeName);
		Target.SetUniqueNetId(UniqueNetIdWrapper.GetUniqueNetId());
	}
}

bool FUniqueNetIdReplNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		// Assume everything is set, even when unnecessary, during quantization as it allows for faster testing of differences.
		if ((Value0.bIsValid != Value1.bIsValid) | (Value0.bIsString != Value1.bIsString) | (Value0.TypeHash != Value1.TypeHash) | (Value0.Number != Value1.Number))
		{
			return false;
		}

		if (Value0.bIsString)
		{
			FNetIsEqualArgs IsEqualArgs = Args;
			IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
			IsEqualArgs.Source0 = NetSerializerValuePointer(&Value0.QuantizedStringID);
			IsEqualArgs.Source1 = NetSerializerValuePointer(&Value1.QuantizedStringID);
			const bool bIDIsEqual = StructNetSerializer->IsEqual(Context, IsEqualArgs);
			if (!bIDIsEqual)
			{
				return false;
			}
		}

		if (Value0.TypeHash == TypeHash_Other)
		{
			FNetIsEqualArgs IsEqualArgs = Args;
			IsEqualArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
			IsEqualArgs.Source0 = NetSerializerValuePointer(&Value0.QuantizedTypeName);
			IsEqualArgs.Source1 = NetSerializerValuePointer(&Value1.QuantizedTypeName);
			const bool bTypeNameIsEqual = StructNetSerializer->IsEqual(Context, IsEqualArgs);
			if (!bTypeNameIsEqual)
			{
				return false;
			}
		}
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);
		return Value0 == Value1;
	}

	return true;
}

bool FUniqueNetIdReplNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	if (!SourceValue.IsValid())
	{
		return true;
	}

	const FString& StringID = SourceValue->ToString();
	if (StringID.IsEmpty())
	{
		return true;
	}

	// The type hash ID needs to fit in the reserved number of bits.
	const FName TypeName = SourceValue->GetType();
	uint8 TypeHash = UOnlineEngineInterface::Get()->GetReplicationHashForSubsystem(TypeName);
	if (TypeHash >= TypeHash_Other)
	{
		return false;
	}

	// We assume the type name is valid.

	// Check if the string ID is valid.
	// Note that we do not try to convert the string to a 64-bit number and avoid the string validation.
	// If the string can be represented as a 64-bit number we expect that to not fail the validation anyway.
	{
		FNetValidateArgs ValidateArgs = {};
		ValidateArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		ValidateArgs.Source = NetSerializerValuePointer(&StringID);
		const bool bIsValid = StructNetSerializer->Validate(Context, ValidateArgs);
		if (!bIsValid)
		{
			return false;
		}
	}

	return true;
}

void FUniqueNetIdReplNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const FQuantizedType& SourceValue = *reinterpret_cast<const FQuantizedType*>(Args.Source);
	FQuantizedType& TargetValue = *reinterpret_cast<FQuantizedType*>(Args.Target);

	if (!SourceValue.bIsValid)
	{
		return;
	}

	if (SourceValue.bIsString)
	{
		FNetCloneDynamicStateArgs CloneArgs = {};
		CloneArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		CloneArgs.Source = NetSerializerValuePointer(&SourceValue.QuantizedStringID);
		CloneArgs.Target = NetSerializerValuePointer(&TargetValue.QuantizedStringID);
		StructNetSerializer->CloneDynamicState(Context, CloneArgs);
	}

	if (SourceValue.TypeHash == TypeHash_Other)
	{
		FNetCloneDynamicStateArgs CloneArgs = {};
		CloneArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		CloneArgs.Source = NetSerializerValuePointer(&SourceValue.QuantizedTypeName);
		CloneArgs.Target = NetSerializerValuePointer(&TargetValue.QuantizedTypeName);
		StructNetSerializer->CloneDynamicState(Context, CloneArgs);
	}
}

void FUniqueNetIdReplNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	FreeDynamicState(Context, Value);
}

void FUniqueNetIdReplNetSerializer::CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&)
{
	// There are no references.
}

void FUniqueNetIdReplNetSerializer::FreeDynamicState(FNetSerializationContext& Context, QuantizedType& Value)
{
	if (!Value.bIsValid)
	{
		return;
	}

	if (Value.bIsString)
	{
		FNetFreeDynamicStateArgs FreeArgs = {};
		FreeArgs.NetSerializerConfig = &StructNetSerializerConfigForString;
		FreeArgs.Source = NetSerializerValuePointer(&Value.QuantizedStringID);
		StructNetSerializer->FreeDynamicState(Context, FreeArgs);
	}

	if (Value.TypeHash == TypeHash_Other)
	{
		FNetFreeDynamicStateArgs FreeArgs = {};
		FreeArgs.NetSerializerConfig = &StructNetSerializerConfigForName;
		FreeArgs.Source = NetSerializerValuePointer(&Value.QuantizedTypeName);
		StructNetSerializer->FreeDynamicState(Context, FreeArgs);
	}
}

static const FName PropertyNetSerializerRegistry_NAME_UniqueNetIdRepl("UniqueNetIdRepl");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_UniqueNetIdRepl, FUniqueNetIdReplNetSerializer);

FUniqueNetIdReplNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_UniqueNetIdRepl);
}

void FUniqueNetIdReplNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_UniqueNetIdRepl);
}

void FUniqueNetIdReplNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	using namespace UE::Net;

	{
		const UStruct* StringStruct = FUniqueNetIdReplNetSerializerStringStruct::StaticStruct();
		StructNetSerializerConfigForString.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(StringStruct);
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForString.StateDescriptor.GetReference();
		check(Descriptor != nullptr);

		// Validate our assumptions regarding quantized state size and alignment.
		constexpr SIZE_T OffsetOfQuantizedStringID = offsetof(FQuantizedType, QuantizedStringID);
		if ((sizeof(FQuantizedType::QuantizedTypeName) < Descriptor->InternalSize) || (((OffsetOfQuantizedStringID/Descriptor->InternalAlignment)*Descriptor->InternalAlignment) != OffsetOfQuantizedStringID))
		{
			LowLevelFatalError(TEXT("FQuantizedType::QuantizedStringID has size %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::QuantizedStringID)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
		}
	}

	{
		const UStruct* NameStruct = FUniqueNetIdReplNetSerializerNameStruct::StaticStruct();
		StructNetSerializerConfigForName.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(NameStruct);
		const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfigForName.StateDescriptor.GetReference();
		check(Descriptor != nullptr);

		// Validate our assumptions regarding quantized state size and alignment.
		constexpr SIZE_T OffsetOfQuantizedTypeName = offsetof(FQuantizedType, QuantizedTypeName);
		if ((sizeof(FQuantizedType::QuantizedTypeName) < Descriptor->InternalSize) || (((OffsetOfQuantizedTypeName/Descriptor->InternalAlignment)*Descriptor->InternalAlignment) != OffsetOfQuantizedTypeName))
		{
			LowLevelFatalError(TEXT("FQuantizedType::QuantizedTypeName has size %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::QuantizedTypeName)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment));
		}
	}
}

}

