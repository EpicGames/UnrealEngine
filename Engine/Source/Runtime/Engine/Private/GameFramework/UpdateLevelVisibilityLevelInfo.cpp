// Copyright Epic Games, Inc. All Rights Reserved.
#include "GameFramework/UpdateLevelVisibilityLevelInfo.h"
#include "Engine/Level.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UpdateLevelVisibilityLevelInfo)

// CVars
namespace PlayerControllerCVars
{
	static bool LevelVisibilityDontSerializeFileName = false;
	FAutoConsoleVariableRef CVarLevelVisibilityDontSerializeFileName(
		TEXT("PlayerController.LevelVisibilityDontSerializeFileName"),
		LevelVisibilityDontSerializeFileName,
		TEXT("When true, we'll always skip serializing FileName with FUpdateLevelVisibilityLevelInfo's. This will save bandwidth when games don't need both.")
	);
}

FUpdateLevelVisibilityLevelInfo::FUpdateLevelVisibilityLevelInfo(const ULevel* const Level, const bool bInIsVisible, const bool bInTryMakeVisible)
	: CookPackageName(NAME_None)
	, bIsVisible(bInIsVisible)
	, bTryMakeVisible(bInTryMakeVisible)
	, bSkipCloseOnError(false)
{
	// For backward compatibility, bTryMakeVisible was added instead of converting bIsVisible to an enum.
	// Make sure we don't receive the invalid state (bIsVisible == true) && (bTryMakeVisible == true)
	check(!bTryMakeVisible || (bIsVisible != bTryMakeVisible));
	const UPackage* const LevelPackage = Level->GetOutermost();
	PackageName = LevelPackage->GetFName();
#if WITH_EDITOR
	CookPackageName = Level->GetCookPackageName();
#endif

	// When packages are duplicated for PIE, they may not have a FileName.
	// For now, just revert to the old behavior.
	FName LoadedPathName = LevelPackage->GetLoadedPath().GetPackageFName();
	FileName = (LoadedPathName.IsNone()) ? PackageName : LoadedPathName;
}

bool FUpdateLevelVisibilityLevelInfo::NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess)
{
	bool bArePackageAndFileTheSame = !!((PlayerControllerCVars::LevelVisibilityDontSerializeFileName) || (FileName == PackageName) || (FileName == NAME_None));
	bool bLocalIsVisible = !!bIsVisible;
	bool bLocalTryMakeVisible = !!bTryMakeVisible;
	bool bArePackageAndCookPackageTheSame = (CookPackageName == PackageName);
	bool bIsCookPackageNameNone = CookPackageName.IsNone();

	Ar.SerializeBits(&bArePackageAndFileTheSame, 1);
	Ar.SerializeBits(&bLocalIsVisible, 1);
	Ar.SerializeBits(&bLocalTryMakeVisible, 1);
	Ar.SerializeBits(&bArePackageAndCookPackageTheSame, 1);
	Ar.SerializeBits(&bIsCookPackageNameNone, 1);
	Ar << PackageName;

	if (!bArePackageAndFileTheSame)
	{
		Ar << FileName;
	}
	else if (Ar.IsLoading())
	{
		FileName = PackageName;
	}

	if (!bIsCookPackageNameNone)
	{
		if (!bArePackageAndCookPackageTheSame)
		{
			Ar << CookPackageName;
		}
		else if (Ar.IsLoading())
		{
			CookPackageName = PackageName;
		}
	}
	else if (Ar.IsLoading())
	{
		CookPackageName = NAME_None;
	}

	VisibilityRequestId.NetSerialize(Ar, PackageMap, bOutSuccess);

	bIsVisible = bLocalIsVisible;
	bTryMakeVisible = bLocalTryMakeVisible;

	bOutSuccess = !Ar.IsError();
	return true;
}

bool FNetLevelVisibilityTransactionId::NetSerialize(FArchive& Ar, UPackageMap* PackageMap, bool& bOutSuccess)
{
	if (Ar.IsLoading())
	{
		bool bIsClientInstigator = false;
		uint32 Value = 0U;

		Ar.SerializeBits(&bIsClientInstigator, 1);
		Ar.SerializeIntPacked(Value);
		
		*this = FNetLevelVisibilityTransactionId(Value, bIsClientInstigator);
	}
	else
	{
		bool bIsClientInstigator = IsClientTransaction();
		uint32 Value = GetTransactionIndex();

		Ar.SerializeBits(&bIsClientInstigator, 1);
		Ar.SerializeIntPacked(Value);		
	}

	bOutSuccess = !Ar.IsError();
	return true;
}



