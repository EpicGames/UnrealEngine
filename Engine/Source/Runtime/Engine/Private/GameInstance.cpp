// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/GameInstance.h"
#include "AnalyticsEventAttribute.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/MessageDialog.h"
#include "Misc/CommandLine.h"
#include "GameMapsSettings.h"
#include "AI/NavigationSystemBase.h"
#include "Misc/Paths.h"
#include "Engine/Console.h"
#include "Engine/GameViewportClient.h"
#include "SceneInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/GameEngine.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/OnlineSession.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/GameSession.h"
#include "Net/OnlineEngineInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/PackageName.h"
#include "Net/ReplayPlaylistTracker.h"
#include "Net/Core/Connection/NetEnums.h"
#include "ReplaySubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameInstance)

#if WITH_EDITOR
#include "Settings/LevelEditorPlayNetworkEmulationSettings.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Editor/EditorEngine.h"
#else
#include "TimerManager.h"
#include "UObject/Package.h"
#endif

#if WITH_EDITOR
FGameInstancePIEParameters::FGameInstancePIEParameters()
	: bSimulateInEditor(false)
	, bAnyBlueprintErrors(false)
	, bStartInSpectatorMode(false)
	, bRunAsDedicated(false)
	, bIsPrimaryPIEClient(false)
	, WorldFeatureLevel(ERHIFeatureLevel::Num)
	, EditorPlaySettings(nullptr)
	, NetMode(EPlayNetMode::PIE_Standalone)
{}
#endif

UGameInstance::UGameInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TimerManager(new FTimerManager(this))
	, LatentActionManager(new FLatentActionManager())
{
}

void UGameInstance::FinishDestroy()
{
	if (TimerManager)
	{
		delete TimerManager;
		TimerManager = nullptr;
	}

	// delete operator should handle null, but maintaining pattern of TimerManager:
	if (LatentActionManager)
	{
		delete LatentActionManager;
		LatentActionManager = nullptr;
	}

	Super::FinishDestroy();
}

void UGameInstance::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UGameInstance* This = CastChecked<UGameInstance>(InThis);
		
	This->SubsystemCollection.AddReferencedObjects(This, Collector);

	UObject::AddReferencedObjects(This, Collector);
}

UWorld* UGameInstance::GetWorld() const
{
	return WorldContext ? WorldContext->World() : NULL;
}

UEngine* UGameInstance::GetEngine() const
{
	return CastChecked<UEngine>(GetOuter());
}

void UGameInstance::Init()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameInstance::Init);
	ReceiveInit();

	if (!IsRunningCommandlet())
	{
		UClass* SpawnClass = GetOnlineSessionClass();
		OnlineSession = NewObject<UOnlineSession>(this, SpawnClass);
		if (OnlineSession)
		{
			OnlineSession->RegisterOnlineDelegates();
		}

		if (!IsDedicatedServerInstance())
		{
			TSharedPtr<GenericApplication> App = FSlateApplication::Get().GetPlatformApplication();
			if (App.IsValid())
			{
				App->RegisterConsoleCommandListener(GenericApplication::FOnConsoleCommandListener::CreateUObject(this, &ThisClass::OnConsoleInput));
			}
		}

		FNetDelegates::OnReceivedNetworkEncryptionToken.BindUObject(this, &ThisClass::ReceivedNetworkEncryptionToken);
		FNetDelegates::OnReceivedNetworkEncryptionAck.BindUObject(this, &ThisClass::ReceivedNetworkEncryptionAck);
		FNetDelegates::OnReceivedNetworkEncryptionFailure.BindUObject(this, &ThisClass::ReceivedNetworkEncryptionFailure);

		IPlatformInputDeviceMapper& PlatformInputMapper = IPlatformInputDeviceMapper::Get();
		PlatformInputMapper.GetOnInputDeviceConnectionChange().AddUObject(this, &UGameInstance::HandleInputDeviceConnectionChange);
		PlatformInputMapper.GetOnInputDevicePairingChange().AddUObject(this, &UGameInstance::HandleInputDevicePairingChange);
	}

	SubsystemCollection.Initialize(this);
}

void UGameInstance::OnConsoleInput(const FString& Command)
{
#if !UE_BUILD_SHIPPING
	UConsole* ViewportConsole = (GEngine->GameViewport != nullptr) ? GEngine->GameViewport->ViewportConsole : nullptr;
	if (ViewportConsole) 
	{
		ViewportConsole->ConsoleCommand(Command);
	}
	else
	{
		GEngine->Exec(GetWorld(), *Command);
	}
#endif
}

void UGameInstance::Shutdown()
{
	ReceiveShutdown();

	if (OnlineSession)
	{
		OnlineSession->ClearOnlineDelegates();
		OnlineSession = nullptr;
	}

	for (int32 PlayerIdx = LocalPlayers.Num() - 1; PlayerIdx >= 0; --PlayerIdx)
	{
		ULocalPlayer* Player = LocalPlayers[PlayerIdx];

		if (Player)
		{
			RemoveLocalPlayer(Player);
		}
	}

	SubsystemCollection.Deinitialize();

	FNetDelegates::OnReceivedNetworkEncryptionToken.Unbind();
	FNetDelegates::OnReceivedNetworkEncryptionAck.Unbind();

	IPlatformInputDeviceMapper& PlatformInputMapper = IPlatformInputDeviceMapper::Get();
	PlatformInputMapper.GetOnInputDeviceConnectionChange().RemoveAll(this);
	PlatformInputMapper.GetOnInputDevicePairingChange().RemoveAll(this);

	// Clear the world context pointer to prevent further access.
	WorldContext = nullptr;
}

void UGameInstance::HandleInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId)
{
	OnInputDeviceConnectionChange.Broadcast(NewConnectionState, PlatformUserId, InputDeviceId);
}

void UGameInstance::HandleInputDevicePairingChange(FInputDeviceId InputDeviceId, FPlatformUserId NewUserPlatformId, FPlatformUserId OldUserPlatformId)
{
	OnUserInputDevicePairingChange.Broadcast(InputDeviceId, NewUserPlatformId, OldUserPlatformId);
}

void UGameInstance::InitializeStandalone(const FName InPackageName, UPackage* InWorldPackage)
{
	// Creates the world context. This should be the only WorldContext that ever gets created for this GameInstance.
	WorldContext = &GetEngine()->CreateNewWorldContext(EWorldType::Game);
	WorldContext->OwningGameInstance = this;

	// In standalone create a dummy world from the beginning to avoid issues of not having a world until LoadMap gets us our real world
	UWorld* DummyWorld = UWorld::CreateWorld(EWorldType::Game, false, InPackageName, InWorldPackage);
	DummyWorld->SetGameInstance(this);
	WorldContext->SetCurrentWorld(DummyWorld);

	Init();
}

void UGameInstance::InitializeForMinimalNetRPC(const FName InPackageName)
{
	check(!InPackageName.IsNone());

	// Creates the world context. This should be the only WorldContext that ever gets created for this GameInstance.
	WorldContext = &GetEngine()->CreateNewWorldContext(EWorldType::GameRPC);
	WorldContext->OwningGameInstance = this;

	UPackage* NetWorldPackage = nullptr;
	UWorld* NetWorld = nullptr;
	CreateMinimalNetRPCWorld(InPackageName, NetWorldPackage, NetWorld);

	NetWorld->SetGameInstance(this);
	WorldContext->SetCurrentWorld(NetWorld);

	Init();
}

void UGameInstance::CreateMinimalNetRPCWorld(const FName InPackageName, UPackage*& OutWorldPackage, UWorld*& OutWorld)
{
	check(!InPackageName.IsNone());

	const FName WorldName = FPackageName::GetShortFName(InPackageName);

	// Create the empty named world within the given package. This will be the world name used with "Browse" to initialize the RPC server and connect the client(s).
	OutWorldPackage = NewObject<UPackage>(nullptr, InPackageName, RF_Transient);
	OutWorldPackage->ThisContainsMap();

	OutWorld = NewObject<UWorld>(OutWorldPackage, WorldName);
	OutWorld->WorldType = EWorldType::GameRPC;
	OutWorld->InitializeNewWorld(UWorld::InitializationValues()
		.InitializeScenes(false)
		.AllowAudioPlayback(false)
		.RequiresHitProxies(false)
		.CreatePhysicsScene(false)
		.CreateNavigation(false)
		.CreateAISystem(false)
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(false)
		.SetTransactional(false)
		.CreateFXSystem(false)
		);
}

#if WITH_EDITOR
static ENetMode GetNetModeFromPlayNetMode(const EPlayNetMode InPlayNetMode, const bool bInDedicatedServer)
{
	switch (InPlayNetMode)
	{
		case EPlayNetMode::PIE_Client:
		{
			return NM_Client;
		}
		case EPlayNetMode::PIE_ListenServer:
		{
			if (bInDedicatedServer)
			{
				return NM_DedicatedServer;
			}

			return NM_ListenServer;
		}
		case EPlayNetMode::PIE_Standalone:
		{
			return NM_Standalone;
		}
		default:
		{
			break;
		}
	}

	return NM_Standalone;
}

FGameInstancePIEResult UGameInstance::InitializeForPlayInEditor(int32 PIEInstanceIndex, const FGameInstancePIEParameters& Params)
{
	FWorldDelegates::OnPIEStarted.Broadcast(this);
	FWorldDelegates::OnPIEMapCreated.Broadcast(this);

	UEditorEngine* const EditorEngine = CastChecked<UEditorEngine>(GetEngine());

	// Look for an existing pie world context, may have been created before
	WorldContext = EditorEngine->GetWorldContextFromPIEInstance(PIEInstanceIndex);

	if (!WorldContext)
	{
		// If not, create a new one
		WorldContext = &EditorEngine->CreateNewWorldContext(EWorldType::PIE);
		WorldContext->PIEInstance = PIEInstanceIndex;
	}

	WorldContext->PIEWorldFeatureLevel = Params.WorldFeatureLevel;

	WorldContext->RunAsDedicated = Params.bRunAsDedicated;

	WorldContext->bIsPrimaryPIEInstance = Params.bIsPrimaryPIEClient;

	WorldContext->OwningGameInstance = this;
	
	UWorld* EditorWorld = EditorEngine->GetEditorWorldContext().World();
	const FString WorldPackageName = EditorWorld->GetOutermost()->GetName();

	// Establish World Context for PIE World
	WorldContext->LastURL.Map = WorldPackageName;
	WorldContext->PIEPrefix = WorldContext->PIEInstance != INDEX_NONE ? UWorld::BuildPIEPackagePrefix(WorldContext->PIEInstance) : FString();

	
	// We always need to create a new PIE world unless we're using the editor world for SIE
	UWorld* NewWorld = nullptr;

	bool bNeedsGarbageCollection = false;

	if (Params.NetMode == EPlayNetMode::PIE_Client)
	{
		// We are going to connect, so just load an empty world
		NewWorld = EditorEngine->CreatePIEWorldFromEntry(*WorldContext, EditorWorld, PIEMapName);
	}
	else
	{
		if (Params.OverrideMapURL.Len() > 0)
		{
			// Attempt to load the target world asset
			FSoftObjectPath TargetWorld = FSoftObjectPath(Params.OverrideMapURL);
			UWorld* WorldToDuplicate = Cast<UWorld>(TargetWorld.TryLoad());
			if (WorldToDuplicate && EditorWorld->Scene != nullptr)
			{
				WorldToDuplicate->ShaderPlatformChanged(EditorWorld->Scene->GetShaderPlatform(), false);
				NewWorld = EditorEngine->CreatePIEWorldByDuplication(*WorldContext, WorldToDuplicate, PIEMapName);
			}
		}
		else
		{
			// Standard PIE path: just duplicate the EditorWorld
			NewWorld = EditorEngine->CreatePIEWorldByDuplication(*WorldContext, EditorWorld, PIEMapName);
		}

		// Duplication can result in unreferenced objects, so indicate that we should do a GC pass after initializing the world context
		bNeedsGarbageCollection = true;
	}

	// failed to create the world!
	if (NewWorld == nullptr)
	{
		return FGameInstancePIEResult::Failure(NSLOCTEXT("UnrealEd", "Error_FailedCreateEditorPreviewWorld", "Failed to create editor preview world."));
	}

	NewWorld->SetPlayInEditorInitialNetMode(GetNetModeFromPlayNetMode(Params.NetMode, Params.bRunAsDedicated));
	NewWorld->SetGameInstance(this);
	WorldContext->SetCurrentWorld(NewWorld);
	WorldContext->AddRef(static_cast<UWorld*&>(EditorEngine->PlayWorld));	// Tie this context to this UEngine::PlayWorld*		// @fixme, needed still?
	NewWorld->bKismetScriptError = Params.bAnyBlueprintErrors;

	// Do a GC pass if necessary to remove any potentially unreferenced objects
	if(bNeedsGarbageCollection)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	// This creates the game instance subsystems
	Init();
	
	// Initialize the world after setting world context and initializing the game instance to be consistent with normal loads.
	// This creates the world subsystems and prepares to begin play
	EditorEngine->PostCreatePIEWorld(NewWorld);

	FWorldDelegates::OnPIEMapReady.Broadcast(this);

	// Games can override this to return failure if PIE is not allowed for some reason
	return FGameInstancePIEResult::Success();
}

#if WITH_EDITOR
void UGameInstance::ReportPIEStartupTime()
{
	if (!bReportedPIEStartupTime)
	{
		static bool bHasRunPIEThisSession = false;

		bool bReportFirstTime = false;
		
		if (!bHasRunPIEThisSession)
		{
			bReportFirstTime = true;
			bHasRunPIEThisSession = true;
		}

		bReportedPIEStartupTime = true;
	}

	FWorldDelegates::OnPIEReady.Broadcast(this);
}
#endif

FGameInstancePIEResult UGameInstance::StartPlayInEditorGameInstance(ULocalPlayer* LocalPlayer, const FGameInstancePIEParameters& Params)
{
	if (!Params.EditorPlaySettings)
	{
		return FGameInstancePIEResult::Failure(NSLOCTEXT("UnrealEd", "Error_InvalidEditorPlaySettings", "Invalid Editor Play Settings!"));
	}

	if (PIEStartTime == 0)
	{
		PIEStartTime = Params.PIEStartTime;
	}

	BroadcastOnStart();

	UEditorEngine* const EditorEngine = CastChecked<UEditorEngine>(GetEngine());

	// for clients, just connect to the server
	if (Params.NetMode == PIE_Client)
	{
		FString Error;
		FURL BaseURL = WorldContext->LastURL;

		FString URLString("127.0.0.1");
		uint16 ServerPort = 0;
		if (Params.EditorPlaySettings->GetServerPort(ServerPort))
		{
			URLString += FString::Printf(TEXT(":%hu"), ServerPort);
		}

		if (Params.EditorPlaySettings->IsNetworkEmulationEnabled())
		{
			if (Params.EditorPlaySettings->NetworkEmulationSettings.IsEmulationEnabledForTarget(NetworkEmulationTarget::Client))
			{
				URLString += Params.EditorPlaySettings->NetworkEmulationSettings.BuildPacketSettingsForURL();
			}
		}

		if (EditorEngine->Browse(*WorldContext, FURL(&BaseURL, *URLString, (ETravelType)TRAVEL_Absolute), Error) == EBrowseReturnVal::Pending)
		{
			EditorEngine->TransitionType = ETransitionType::WaitingToConnect;
		}
		else
		{
			return FGameInstancePIEResult::Failure(FText::Format(NSLOCTEXT("UnrealEd", "Error_CouldntLaunchPIEClient", "Couldn't Launch PIE Client: {0}"), FText::FromString(Error)));
		}
	}
	else
	{
		FScopedSlowTask SlowTask(100, NSLOCTEXT("UnrealEd", "StartPlayInEditor", "Starting PIE..."));
		// Disabled for now in PIE until a proper fix is found to give the focus to the blueprint debugger
		// during beginplay if a breakpoint is hit. The focus is currently held by the slowtask and prevents the debugger from working.
		// Related JIRA UE-159973
		SlowTask.MakeDialogDelayed(1.0f, false, false);

		// we're going to be playing in the current world, get it ready for play
		UWorld* const PlayWorld = GetWorld();

		FString ExtraURLOptions;
		if (Params.EditorPlaySettings->IsNetworkEmulationEnabled())
		{
			NetworkEmulationTarget CurrentTarget = Params.NetMode == PIE_ListenServer ? NetworkEmulationTarget::Server : NetworkEmulationTarget::Client;
			if (Params.EditorPlaySettings->NetworkEmulationSettings.IsEmulationEnabledForTarget(CurrentTarget))
			{
				ExtraURLOptions += Params.EditorPlaySettings->NetworkEmulationSettings.BuildPacketSettingsForURL();
			}
		}

		// make a URL
		FURL URL;
		// If the user wants to start in spectator mode, do not use the custom play world for now
		if (EditorEngine->UserEditedPlayWorldURL.Len() > 0 || Params.OverrideMapURL.Len() > 0)
		{
			FString UserURL = EditorEngine->UserEditedPlayWorldURL.Len() > 0 ? EditorEngine->UserEditedPlayWorldURL : Params.OverrideMapURL;
			UserURL += ExtraURLOptions;

			// If the user edited the play world url. Verify that the map name is the same as the currently loaded map.
			URL = FURL(NULL, *UserURL, TRAVEL_Absolute);
			if (URL.Map != PIEMapName)
			{
				// Ensure the URL map name is the same as the generated play world map name.
				URL.Map = PIEMapName;
			}
		}
		else
		{
			// The user did not edit the url, just build one from scratch.
			URL = FURL(NULL, *EditorEngine->BuildPlayWorldURL(*PIEMapName, Params.bStartInSpectatorMode, ExtraURLOptions), TRAVEL_Absolute);
		}

		// Save our URL for later map travels
		SetPersistentTravelURL(URL);

		// If a start location is specified, spawn a temporary PlayerStartPIE actor at the start location and use it as the portal.
		AActor* PlayerStart = NULL;
		if (!EditorEngine->SpawnPlayFromHereStart(PlayWorld, PlayerStart))
		{
			// failed to create "play from here" playerstart
			return FGameInstancePIEResult::Failure(NSLOCTEXT("UnrealEd", "Error_FailedCreatePlayFromHerePlayerStart", "Failed to create PlayerStart at desired starting location."));
		}

		if (!PlayWorld->SetGameMode(URL))
		{
			// Setting the game mode failed so bail 
			return FGameInstancePIEResult::Failure(NSLOCTEXT("UnrealEd", "Error_FailedCreateEditorPreviewWorld", "Failed to create editor preview world."));
		}

		FGameInstancePIEResult PostCreateGameModeResult = PostCreateGameModeForPIE(Params, PlayWorld->GetAuthGameMode<AGameModeBase>());
		if (!PostCreateGameModeResult.IsSuccess())
		{
			return PostCreateGameModeResult;
		}

		SlowTask.EnterProgressFrame(10, NSLOCTEXT("UnrealEd", "PIEFlushingLevelStreaming", "Starting PIE (Loading always loaded objects)..."));

		// Make sure "always loaded" sub-levels are fully loaded
		PlayWorld->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);

		UGameViewportClient* const GameViewport = GetGameViewportClient();
		if (GameViewport != NULL && GameViewport->Viewport != NULL)
		{
			SlowTask.EnterProgressFrame(25, NSLOCTEXT("UnrealEd", "PIEWaitingForInitialLevelStreaming", "Starting PIE (Waiting for initial level streaming)..."));
			// Stream any always loaded levels now that need to be loaded before the game starts
			GEngine->BlockTillLevelStreamingCompleted(PlayWorld);
		}

		SlowTask.EnterProgressFrame(10, NSLOCTEXT("UnrealEd", "PIECreatingAISystem", "Starting PIE (Creating AI System)..."));
		PlayWorld->CreateAISystem();

		SlowTask.EnterProgressFrame(10, NSLOCTEXT("UnrealEd", "PIEInitializingActors", "Starting PIE (Initializing Actors)..."));
		{
			FRegisterComponentContext Context(PlayWorld);
			PlayWorld->InitializeActorsForPlay(URL, true, &Context);
		}
		// calling it after InitializeActorsForPlay has been called to have all potential bounding boxed initialized
		FNavigationSystem::AddNavigationSystemToWorld(*PlayWorld, FNavigationSystemRunMode::PIEMode);

		// @todo, just use WorldContext.GamePlayer[0]?
		if (LocalPlayer)
		{
			FString Error;
			if (!LocalPlayer->SpawnPlayActor(URL.ToString(1), Error, PlayWorld))
			{
				return FGameInstancePIEResult::Failure(FText::Format(NSLOCTEXT("UnrealEd", "Error_CouldntSpawnPlayer", "Couldn't spawn player: {0}"), FText::FromString(Error)));
			}

			if (GameViewport != NULL && GameViewport->Viewport != NULL)
			{
				SlowTask.EnterProgressFrame(25, NSLOCTEXT("UnrealEd", "PIEWaitingForLevelStreaming", "Starting PIE (Waiting for level streaming)..."));
				// Stream any levels now that need to be loaded before the game starts as a result of spawning the local player
				GEngine->BlockTillLevelStreamingCompleted(PlayWorld);
			}
		}

		if (Params.NetMode == PIE_ListenServer)
		{
			// Add port
			uint32 ListenPort = 0;
			uint16 ServerPort = 0;
			if (Params.EditorPlaySettings->GetServerPort(ServerPort))
			{
				ListenPort = ServerPort;
			}

			// Start a listen server
			ensureMsgf(EnableListenServer(true, ListenPort), TEXT("Starting Listen Server for Play in Editor failed!"));
		}

		SlowTask.EnterProgressFrame(10, NSLOCTEXT("UnrealEd", "PIEBeginPlay", "Starting PIE (Begin play)..."));
		PlayWorld->BeginPlay();

		SlowTask.EnterProgressFrame(10);
#if WITH_EDITOR
		if (PlayWorld->WorldType == EWorldType::PIE)
		{
			ReportPIEStartupTime();
		}
#endif
	}

	// Games can override this to return failure if PIE is not allowed for some reason
	return FGameInstancePIEResult::Success();
}

FGameInstancePIEResult UGameInstance::PostCreateGameModeForPIE(const FGameInstancePIEParameters& Params, AGameModeBase* GameMode)
{
	return FGameInstancePIEResult::Success();
}
#endif


UGameViewportClient* UGameInstance::GetGameViewportClient() const
{
	FWorldContext* const WC = GetWorldContext();
	return WC ? WC->GameViewport : nullptr;
}

void UGameInstance::OnWorldChanged(UWorld* OldWorld, UWorld* NewWorld)
{
	if (NewWorld)
	{
		// Fix up our world context if it is incorrect and we have a valid new world
		FWorldContext* NewWorldContext = const_cast<FWorldContext*>(GEngine->GetWorldContextFromWorld(NewWorld));

		if (WorldContext == nullptr)
		{
			// This may be a test/preview world that did not have a world context set before
			WorldContext = NewWorldContext;
		}
		else if (WorldContext != NewWorldContext)
		{
			UE_LOGF(LogLoad, Warning, "GameInstance %ls changed from world %ls to world %ls with a different world context!", *GetName(), *GetPathNameSafe(OldWorld), *GetPathNameSafe(NewWorld));
			WorldContext = NewWorldContext;
		}
	}
	else
	{
		// Clear our world context if our world is cleared
		WorldContext = nullptr;
	}

	FWorldDelegates::OnGameInstanceWorldChanged.Broadcast(this, OldWorld, NewWorld);
}

// This can be defined in the target.cs file to allow map overrides in shipping builds
#ifndef UE_ALLOW_MAP_OVERRIDE_IN_SHIPPING
#define UE_ALLOW_MAP_OVERRIDE_IN_SHIPPING 0
#endif

void UGameInstance::StartGameInstance()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameInstance::StartGameInstance);
	UEngine* const Engine = GetEngine();

	// Create default URL.
	// @note: if we change how we determine the valid start up map update LaunchEngineLoop's GetStartupMap()
	FURL DefaultURL;
	DefaultURL.LoadURLConfig(TEXT("DefaultPlayer"), GGameIni);

	// Enter initial world.
	EBrowseReturnVal::Type BrowseRet = EBrowseReturnVal::Failure;
	FString Error;
	
	const TCHAR* Tmp = FCommandLine::Get();

#if UE_BUILD_SHIPPING && !UE_SERVER && !UE_ALLOW_MAP_OVERRIDE_IN_SHIPPING && !ENABLE_PGO_PROFILE
	// In shipping don't allow a map override unless on server, or running PGO profiling
	Tmp = TEXT("");
#endif // UE_BUILD_SHIPPING && !UE_SERVER && !UE_ALLOW_MAP_OVERRIDE_IN_SHIPPING && !ENABLE_PGO_PROFILE

#if !UE_SERVER
	// Parse replay name if specified on cmdline
	FString ReplayCommand;
	if (FParse::Value(Tmp, TEXT("-REPLAY="), ReplayCommand))
	{
		if (PlayReplay(ReplayCommand))
		{
			return;
		}
	}
	else if (FParse::Value(Tmp, TEXT("-REPLAYPLAYLIST="), ReplayCommand, false))
	{
		FReplayPlaylistParams Params;
		if (ReplayCommand.ParseIntoArray(Params.Playlist, TEXT(",")))
		{
			if (PlayReplayPlaylist(Params))
			{
				return;
			}
		}
	}
#endif // !UE_SERVER

	const UGameMapsSettings* GameMapsSettings = GetDefault<UGameMapsSettings>();
	const FString& DefaultMap = GameMapsSettings->GetGameDefaultMap();

	FString PackageName;
	if (!GetMapOverrideName(Tmp, PackageName))
	{
		PackageName = DefaultMap + GameMapsSettings->LocalMapOptions;
	}

	// Cache a stable identifier to the WorldContext to enable WorldContext to be restored if the initial call to Browse fails.
	// This may be browsing to a user provided map which may not exist. On failure, this enable the user to decide to load to 
	// the default map.
	FName InitialWorldContextHandle = WorldContext->ContextHandle;

	FURL URL(&DefaultURL, *PackageName, TRAVEL_Partial);
	if (URL.Valid)
	{
		BrowseRet = Engine->Browse(*WorldContext, URL, Error);
	}

	// If waiting for a network connection, go into the starting level.
	if (BrowseRet == EBrowseReturnVal::Failure)
	{
		UE_LOGF(LogLoad, Error, "%ls", *FString::Printf(TEXT("Failed to enter %s: %s. Please check the log for errors."), *URL.Map, *Error));

		// the map specified on the command-line couldn't be loaded.  ask the user if we should load the default map instead
		if (FCString::Stricmp(*PackageName, *DefaultMap) != 0)
		{
			// Restore the WorldContext used in the initial call to Browse or create a new WorldContext if the WorldContext was destroyed. 
			WorldContext = &Engine->GetWorldContextFromHandleChecked(InitialWorldContextHandle);
			
			const FText Message = FText::Format(NSLOCTEXT("Engine", "MapNotFound", "The map specified on the commandline '{0}' could not be found. Would you like to load the default map instead?"), FText::FromString(URL.Map));
			if (   FCString::Stricmp(*URL.Map, *DefaultMap) != 0  
				&& FMessageDialog::Open(EAppMsgType::OkCancel, Message) != EAppReturnType::Ok)
			{
				// user canceled (maybe a typo while attempting to run a commandlet)
				FPlatformMisc::RequestExit(false, TEXT("UGameInstance::StartGameInstance.Cancelled"));
				return;
			}
			else
			{
				BrowseRet = Engine->Browse(*WorldContext, FURL(&DefaultURL, *(DefaultMap + GameMapsSettings->LocalMapOptions), TRAVEL_Partial), Error);
			}
		}
		else
		{
			const FText Message = FText::Format(NSLOCTEXT("Engine", "MapNotFoundNoFallback", "The map specified on the commandline '{0}' could not be found. Exiting."), FText::FromString(URL.Map));
			FMessageDialog::Open(EAppMsgType::Ok, Message);
			FPlatformMisc::RequestExit(false, TEXT("UGameInstance::StartGameInstance.MapNotFound"));
			return;
		}
	}

	// Handle failure.
	if (BrowseRet == EBrowseReturnVal::Failure)
	{
		UE_LOGF(LogLoad, Error, "%ls", *FString::Printf(TEXT("Failed to enter %s: %s. Please check the log for errors."), *DefaultMap, *Error));
		const FText Message = FText::Format(NSLOCTEXT("Engine", "DefaultMapNotFound", "The default map '{0}' could not be found. Exiting."), FText::FromString(DefaultMap));
		FMessageDialog::Open(EAppMsgType::Ok, Message);
		FPlatformMisc::RequestExit(false, TEXT("UGameInstance::StartGameInstance.BrowseFailure"));
		return;
	}

	BroadcastOnStart();
}

void UGameInstance::BroadcastOnStart()
{
	FWorldDelegates::OnStartGameInstance.Broadcast(this);
	OnStart();
}

void UGameInstance::OnStart()
{

}

bool UGameInstance::GetMapOverrideName(const TCHAR* CmdLine, FString& OverrideMapName)
{
	const TCHAR* ParsedCmdLine = CmdLine;

	while (*ParsedCmdLine)
	{
		FString Token = FParse::Token(ParsedCmdLine, 0);

		if (Token.IsEmpty())
		{
			continue;
		}

		if (Token[0] != TCHAR('-'))
		{
			OverrideMapName = Token;

			return true;
		}
		else if (FParse::Value(*Token, TEXT("-map="), OverrideMapName))
		{
			return true;
		}
	}

	return false;
}

bool UGameInstance::HandleOpenCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	check(WorldContext && WorldContext->World() == InWorld);

	UEngine* const Engine = GetEngine();
	return Engine->HandleOpenCommand(Cmd, Ar, InWorld);
}

bool UGameInstance::HandleDisconnectCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	check(WorldContext && WorldContext->World() == InWorld);

	UEngine* const Engine = GetEngine();
	return Engine->HandleDisconnectCommand(Cmd, Ar, InWorld);
}

bool UGameInstance::HandleReconnectCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	check(WorldContext && WorldContext->World() == InWorld);

	UEngine* const Engine = GetEngine();
	return Engine->HandleReconnectCommand(Cmd, Ar, InWorld);
}

bool UGameInstance::HandleTravelCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	check(WorldContext && WorldContext->World() == InWorld);

	UEngine* const Engine = GetEngine();
	return Engine->HandleTravelCommand(Cmd, Ar, InWorld);
}

#if UE_ALLOW_EXEC_COMMANDS
bool UGameInstance::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// @todo a bunch of stuff in UEngine probably belongs here as well
	if (FParse::Command(&Cmd, TEXT("OPEN")))
	{
		return HandleOpenCommand(Cmd, Ar, InWorld);
	}
	else if (FParse::Command(&Cmd, TEXT("DISCONNECT")))
	{
		return HandleDisconnectCommand(Cmd, Ar, InWorld);
	}
	else if (FParse::Command(&Cmd, TEXT("RECONNECT")))
	{
		return HandleReconnectCommand(Cmd, Ar, InWorld);
	}
	else if (FParse::Command(&Cmd, TEXT("TRAVEL")))
	{
		return HandleTravelCommand(Cmd, Ar, InWorld);
	}

	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

ULocalPlayer* UGameInstance::CreateInitialPlayer(FString& OutError)
{
	return CreateLocalPlayer(IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser(), OutError, false);
}

ULocalPlayer* UGameInstance::CreateLocalPlayer(int32 ControllerId, FString& OutError, bool bSpawnPlayerController)
{
	// A compatibility call that will map the old int32 ControllerId to the new platform user
	FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
	FInputDeviceId DummyInputDevice = INPUTDEVICEID_NONE;
	IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DummyInputDevice);
	return CreateLocalPlayer(UserId, OutError, bSpawnPlayerController);
}

ULocalPlayer* UGameInstance::CreateLocalPlayer(FPlatformUserId UserId, FString& OutError, bool bSpawnPlayerController)
{
	check(GetEngine()->LocalPlayerClass != NULL);

	ULocalPlayer* NewPlayer = NULL;
	int32 InsertIndex = INDEX_NONE;
	UGameViewportClient* GameViewport = GetGameViewportClient();

	if (GameViewport == nullptr)
	{
		if (ensure(IsDedicatedServerInstance()))
		{
			OutError = FString::Printf(TEXT("Dedicated servers cannot have local players"));
			return nullptr;
		}
	}

	const int32 MaxSplitscreenPlayers = GameViewport ? GameViewport->MaxSplitscreenPlayers : 1;

	if (FindLocalPlayerFromPlatformUserId(UserId) != NULL)
	{
		OutError = FString::Printf(TEXT("A local player already exists for PlatformUserId %d,"), UserId.GetInternalId());
	}
	else if (LocalPlayers.Num() < MaxSplitscreenPlayers)
	{
		// If the controller ID is not specified then find the first available
		if (!UserId.IsValid())
		{
			for (int32 Id = 0; Id < MaxSplitscreenPlayers; ++Id)
			{
				// Iterate until we find a null player. We want the next available platform user ID
				FPlatformUserId DummyId = IPlatformInputDeviceMapper::Get().GetPlatformUserForUserIndex(Id);

				if (DummyId.IsValid())
				{
					UserId = DummyId;
				}
				
				if (FindLocalPlayerFromControllerId(Id) == nullptr)
				{
					break;
				}
			}
			check(UserId.GetInternalId() < MaxSplitscreenPlayers);
		}
		else if (UserId.GetInternalId() >= MaxSplitscreenPlayers)
		{
			UE_LOGF(LogPlayerManagement, Warning, "Controller ID (%d) is unlikely to map to any physical device, so this player will not receive input", UserId.GetInternalId());
		}

		NewPlayer = NewObject<ULocalPlayer>(GetEngine(), GetEngine()->LocalPlayerClass);
		
		static int32 NextLocalPlayerIdentifier = 0;
		NewPlayer->SetConnectionIdentifier(NextLocalPlayerIdentifier);
		NextLocalPlayerIdentifier++;
		
		InsertIndex = AddLocalPlayer(NewPlayer, UserId);
		UWorld* CurrentWorld = GetWorld();
		if (bSpawnPlayerController && InsertIndex != INDEX_NONE && CurrentWorld != nullptr)
		{
			if (CurrentWorld->GetNetMode() != NM_Client)
			{
				// server; spawn a new PlayerController immediately
				if (!NewPlayer->SpawnPlayActor("", OutError, CurrentWorld))
				{
					RemoveLocalPlayer(NewPlayer);
					NewPlayer = nullptr;
				}
			}
			else if (CurrentWorld->IsPlayingReplay())
			{
				if (UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver())
				{
					// demo playback; ask the replay driver to spawn a splitscreen client
					if (!DemoNetDriver->SpawnSplitscreenViewer(NewPlayer, CurrentWorld))
					{
						RemoveLocalPlayer(NewPlayer);
						NewPlayer = nullptr;
					}
				}
			}
			else
			{
				// client; ask the server to let the new player join
				TArray<FString> Options;
				NewPlayer->SendSplitJoin(Options);
			}
		}
	}
	else
	{
		OutError = FString::Printf(TEXT( "Maximum number of players (%d) already created.  Unable to create more."), MaxSplitscreenPlayers);
	}

	if (OutError != TEXT(""))
	{
		UE_LOGF(LogPlayerManagement, Log, "UPlayer* creation failed with error: %ls", *OutError);
	}

	return NewPlayer;
}

int32 UGameInstance::AddLocalPlayer(ULocalPlayer* NewLocalPlayer, FPlatformUserId UserId)
{
	if (NewLocalPlayer == nullptr)
	{
		return INDEX_NONE;
	}

	// Add to list
	const int32 InsertIndex = LocalPlayers.AddUnique(NewLocalPlayer);

	// Notify the player they were added
	NewLocalPlayer->PlayerAdded(GetGameViewportClient(), UserId);

	// Notify the viewport that we added a player (so it can update splitscreen settings, etc)
	if ( GetGameViewportClient() != nullptr)
	{
		GetGameViewportClient()->NotifyPlayerAdded(InsertIndex, NewLocalPlayer);
	}

	UE_LOGF(LogPlayerManagement, Log, "UGameInstance::AddLocalPlayer: Added player %ls with PlatformUserId %d at index %d (%d remaining players)", *NewLocalPlayer->GetName(), NewLocalPlayer->GetPlatformUserId().GetInternalId(), InsertIndex, LocalPlayers.Num());

	OnLocalPlayerAddedEvent.Broadcast(NewLocalPlayer);

	return InsertIndex;
}

bool UGameInstance::RemoveLocalPlayer(ULocalPlayer* ExistingPlayer)
{
	// FIXME: Notify server we want to leave the game if this is an online game
	if (ExistingPlayer->PlayerController != nullptr)
	{
		bool bShouldRemovePlayer = (ExistingPlayer->PlayerController->GetLocalRole() == ROLE_Authority);

		// FIXME: Do this all inside PlayerRemoved?
		ExistingPlayer->PlayerController->CleanupGameViewport();

		UWorld* CurrentWorld = GetWorld();
		if (CurrentWorld && CurrentWorld->IsPlayingReplay())
		{
			if (UDemoNetDriver* DemoNetDriver = CurrentWorld->GetDemoNetDriver())
			{
				if (!DemoNetDriver->RemoveSplitscreenViewer(ExistingPlayer->PlayerController))
				{
					UE_LOGF(LogPlayerManagement, Warning, "UGameInstance::RemovePlayer: Did not remove player %ls with ControllerId %i as it was unable to be removed from the demo",
						*ExistingPlayer->GetName(), ExistingPlayer->GetControllerId());
				}
			}

			bShouldRemovePlayer = true;
		}

		// Destroy the player's actors.
		if (bShouldRemovePlayer)
		{
			// This is fine to forceremove as we have to be in a special case for demos or the authority
			ExistingPlayer->PlayerController->Destroy(true);
		}
	}

	// Remove the player from the context list
	const int32 OldIndex = LocalPlayers.Find(ExistingPlayer);

	if (ensure(OldIndex != INDEX_NONE))
	{
		ExistingPlayer->PlayerRemoved();
		LocalPlayers.RemoveAt(OldIndex);

		// Notify the viewport so the viewport can do the fixups, resize, etc
		if (GetGameViewportClient() != nullptr)
		{
			GetGameViewportClient()->NotifyPlayerRemoved(OldIndex, ExistingPlayer);
		}
	}

	// Disassociate this viewport client from the player.
	// Do this after notifications, as some of them require the ViewportClient.
	ExistingPlayer->ViewportClient = nullptr;

	UE_LOGF(LogPlayerManagement, Log, "UGameInstance::RemovePlayer: Removed player %ls with ControllerId %i at index %i (%i remaining players)", *ExistingPlayer->GetName(), ExistingPlayer->GetControllerId(), OldIndex, LocalPlayers.Num());

	OnLocalPlayerRemovedEvent.Broadcast(ExistingPlayer);

	// Marked as garbage here to detect outstanding references
	if (!UObjectBaseUtility::IsGarbageEliminationEnabled())
	{
		ExistingPlayer->MarkAsGarbage();
		GEngine->ForceGarbageCollection();
	}

	return true;
}

void UGameInstance::DebugCreatePlayer(int32 ControllerId)
{
#if !UE_BUILD_SHIPPING
	FString Error;
	const ULocalPlayer* LP = CreateLocalPlayer(ControllerId, Error, true);
	if (Error.Len() > 0 || !LP)
	{
		UE_LOGF(LogPlayerManagement, Error, "Failed to DebugCreatePlayer: %ls", *Error);
	}
	else
	{
		FPlatformUserId UserId = LP->GetPlatformUserId();
		FInputDeviceId InputDevice = INPUTDEVICEID_NONE;
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		DeviceMapper.RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, InputDevice);
	
		// If the input device that was created hasn't been mapped yet, we shuold map a dummy input device to it
		if (!DeviceMapper.GetUserForInputDevice(InputDevice).IsValid())
		{
			DeviceMapper.Internal_MapInputDeviceToUser(InputDevice, UserId, EInputDeviceConnectionState::Connected);
		}
	}
#endif
}

void UGameInstance::DebugRemovePlayer(int32 ControllerId)
{
#if !UE_BUILD_SHIPPING

	ULocalPlayer* const ExistingPlayer = FindLocalPlayerFromControllerId(ControllerId);
	if (ExistingPlayer != NULL)
	{
		RemoveLocalPlayer(ExistingPlayer);
	}
#endif
}

int32 UGameInstance::GetNumLocalPlayers() const
{
	return LocalPlayers.Num();
}

ULocalPlayer* UGameInstance::GetLocalPlayerByIndex(const int32 Index) const
{
	if (LocalPlayers.IsValidIndex(Index))
	{
		return LocalPlayers[Index];
	}

	return nullptr;
}

APlayerController* UGameInstance::GetFirstLocalPlayerController(const UWorld* World) const
{
	// Use the consistent local players order if possible
	if (World == nullptr || World == GetWorld())
	{
		for (ULocalPlayer* Player : LocalPlayers)
		{
			// Returns the first non-null UPlayer::PlayerController without filtering by UWorld.
			if (Player && Player->PlayerController)
			{
				// return first non-null entry
				return Player->PlayerController;
			}
		}
	}
	else
	{
		// Only return a local PlayerController from the given World.
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PC = Iterator->Get();
			if (PC && PC->IsLocalController())
			{
				return PC;
			}
		}
	}

	// didn't find one
	return nullptr;
}

APlayerController* UGameInstance::GetPrimaryPlayerController(bool bRequiresValidUniqueId) const
{
	UWorld* World = GetWorld();

	APlayerController* PrimaryController = nullptr;
	if (ensure(World))
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* NextPlayer = Iterator->Get();
			if (NextPlayer && NextPlayer->PlayerState && NextPlayer->IsPrimaryPlayer() && (!bRequiresValidUniqueId || NextPlayer->PlayerState->GetUniqueId().IsValid()))
			{
				PrimaryController = NextPlayer;
				break;
			}
		}
	}

	return PrimaryController;
}

FUniqueNetIdRepl UGameInstance::GetPrimaryPlayerUniqueIdRepl() const
{
	ULocalPlayer* PrimaryLP = nullptr;

	TArray<ULocalPlayer*>::TConstIterator LocalPlayerIt = GetLocalPlayerIterator();
	for (; LocalPlayerIt && *LocalPlayerIt; ++LocalPlayerIt)
	{
		PrimaryLP = *LocalPlayerIt;
		if (PrimaryLP && PrimaryLP->PlayerController && PrimaryLP->PlayerController->IsPrimaryPlayer())
		{
			break;
		}
	}

	FUniqueNetIdRepl LocalUserId;
	if (PrimaryLP)
	{
		LocalUserId = PrimaryLP->GetPreferredUniqueNetId();
	}

	return LocalUserId;
}

ULocalPlayer* UGameInstance::FindLocalPlayerFromControllerId(const int32 ControllerId) const
{
	for (ULocalPlayer * LP : LocalPlayers)
	{
		if (LP && (LP->GetControllerId() == ControllerId))
		{
			return LP;
		}
	}

	return nullptr;
}

ULocalPlayer* UGameInstance::FindLocalPlayerFromPlatformUserId(const FPlatformUserId UserId) const
{
	for (ULocalPlayer* LP : LocalPlayers)
	{
		if (LP && (LP->GetPlatformUserId() == UserId))
		{
			return LP;
		}
	}

	return nullptr;
}

ULocalPlayer* UGameInstance::FindLocalPlayerFromDeviceId(const FInputDeviceId DeviceID) const
{
	const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceID);
	return FindLocalPlayerFromPlatformUserId(UserId);
}

ULocalPlayer* UGameInstance::FindLocalPlayerFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	return FindLocalPlayerFromUniqueNetId(FUniqueNetIdRepl(UniqueNetId.AsShared()));
}

ULocalPlayer* UGameInstance::FindLocalPlayerFromUniqueNetId(FUniqueNetIdPtr UniqueNetId) const
{
	if (!UniqueNetId.IsValid())
	{
		return nullptr;
	}

	return FindLocalPlayerFromUniqueNetId(*UniqueNetId);
}

ULocalPlayer* UGameInstance::FindLocalPlayerFromUniqueNetId(const FUniqueNetIdRepl& UniqueNetId) const
{
	for (ULocalPlayer* Player : LocalPlayers)
	{
		if (Player == nullptr)
		{
			continue;
		}

		FUniqueNetIdRepl OtherUniqueNetId = Player->GetPreferredUniqueNetId();
		if (OtherUniqueNetId.IsValid() &&
			*OtherUniqueNetId == UniqueNetId)
		{
			// Match
			return Player;
		}
	}

	// didn't find one
	return nullptr;
}

ULocalPlayer* UGameInstance::GetFirstGamePlayer() const
{
	return (LocalPlayers.Num() > 0) ? LocalPlayers[0] : nullptr;
}

void UGameInstance::CleanupGameViewport()
{
	// Clean up the viewports that have been closed.
	for(int32 idx = LocalPlayers.Num()-1; idx >= 0; --idx)
	{
		ULocalPlayer *Player = LocalPlayers[idx];

		if(Player && Player->ViewportClient && !Player->ViewportClient->Viewport)
		{
			RemoveLocalPlayer( Player );
		}
	}
}

TArray<class ULocalPlayer*>::TConstIterator	UGameInstance::GetLocalPlayerIterator() const
{
	return ToRawPtrTArrayUnsafe(LocalPlayers).CreateConstIterator();
}

const TArray<class ULocalPlayer*>& UGameInstance::GetLocalPlayers() const
{
	return LocalPlayers;
}

void UGameInstance::StartRecordingReplay(const FString& Name, const FString& FriendlyName, const TArray<FString>& AdditionalOptions, TSharedPtr<IAnalyticsProvider> AnalyticsProvider)
{
	if (UReplaySubsystem* ReplaySubsystem = GetSubsystem<UReplaySubsystem>())
	{
		ReplaySubsystem->RecordReplay(Name, FriendlyName, AdditionalOptions, AnalyticsProvider);
	}
}

void UGameInstance::StopRecordingReplay()
{
	if (UReplaySubsystem* ReplaySubsystem = GetSubsystem<UReplaySubsystem>())
	{
		return ReplaySubsystem->StopReplay();
	}
}

bool UGameInstance::PlayReplay(const FString& Name, UWorld* WorldOverride, const TArray<FString>& AdditionalOptions)
{
	if (UReplaySubsystem* ReplaySubsystem = GetSubsystem<UReplaySubsystem>())
	{
		return ReplaySubsystem->PlayReplay(Name, WorldOverride, AdditionalOptions);
	}
	
	return false;
}

bool UGameInstance::IsReplayDeferred(const FString& ReplayName) const
{
	if (UReplaySubsystem* ReplaySubsystem = GetSubsystem<UReplaySubsystem>())
	{
		return ReplaySubsystem->IsReplayDeferred(ReplayName);
	}

	return false;
}

class FGameInstanceReplayPlaylistHelper
{
private:

	friend class UGameInstance;

	static const bool StartReplay(const FReplayPlaylistParams& PlaylistParams, UGameInstance* GameInstance)
	{
		// Can't use MakeShared directly since the PlaylistTracker constructor is private.
		// Also, the Playlist Tracker will manage holding onto references to itself.
		return TSharedRef<FReplayPlaylistTracker>(new FReplayPlaylistTracker(PlaylistParams, GameInstance))->Start();
	}
};

bool UGameInstance::PlayReplayPlaylist(const FReplayPlaylistParams& PlaylistParams)
{
	LLM_SCOPE(ELLMTag::Replays);

	return FGameInstanceReplayPlaylistHelper::StartReplay(PlaylistParams, this);
}

void UGameInstance::AddUserToReplay(const FString& UserString)
{
	if (UReplaySubsystem* ReplaySubsystem = GetSubsystem<UReplaySubsystem>())
	{
		ReplaySubsystem->AddUserToReplay(UserString);
	}
}

void UGameInstance::SetPersistentTravelURL(FURL InURL)
{
	check(WorldContext);
	WorldContext->LastURL = InURL;

#if WITH_EDITOR
	// Strip any PIE prefix from the map name
	WorldContext->LastURL.Map = UWorld::StripPIEPrefixFromPackageName(WorldContext->LastURL.Map, WorldContext->PIEPrefix);
#endif
}

bool UGameInstance::EnableListenServer(bool bEnable, int32 PortOverride /*= 0*/)
{
	UWorld* World = GetWorld();

	if (!World || !World->IsGameWorld())
	{
		return false;
	}

	ENetMode ExistingMode = World->GetNetMode();

	if (ExistingMode == NM_Client)
	{
		// Clients cannot change to listen!
		return false;
	}

	int32 DefaultListenPort = FURL::UrlConfig.DefaultPort;
	if (bEnable)
	{
		// Modify the persistent url
		if (PortOverride != 0)
		{
			WorldContext->LastURL.Port = PortOverride;
		}
		WorldContext->LastURL.AddOption(TEXT("Listen"));

		if (!World->GetNetDriver())
		{
			// This actually opens the port
			FURL ListenURL = WorldContext->LastURL;
			return World->Listen(ListenURL);
		}
		else
		{
			// Already listening
			return true;
		}
	}
	else
	{
		if (ExistingMode == NM_DedicatedServer)
		{
			UE_LOGF(LogGameSession, Warning, "EnableListenServer: Dedicated servers always listen for connections");
			return false;
		}

		WorldContext->LastURL.RemoveOption(TEXT("Listen"));
		WorldContext->LastURL.Port = FURL::UrlConfig.DefaultPort;

		if (ExistingMode == NM_ListenServer)
		{
			// What to do in this case is very game-specific
			UE_LOGF(LogGameSession, Warning, "EnableListenServer: Disabling a listen server with active connections does not disconnect existing players by default");
		}

		return true;
	}
}

void UGameInstance::ReceivedNetworkEncryptionToken(const FString& EncryptionToken, const FOnEncryptionKeyResponse& Delegate)
{
	FEncryptionKeyResponse Response(EEncryptionResponse::Failure, TEXT("ReceivedNetworkEncryptionToken not implemented"));
	Delegate.ExecuteIfBound(Response);
}

void UGameInstance::ReceivedNetworkEncryptionAck(const FOnEncryptionKeyResponse& Delegate)
{
	FEncryptionKeyResponse Response(EEncryptionResponse::Failure, TEXT("ReceivedNetworkEncryptionAck not implemented"));
	Delegate.ExecuteIfBound(Response);
}

EEncryptionFailureAction UGameInstance::ReceivedNetworkEncryptionFailure(UNetConnection* Connection)
{
	return EEncryptionFailureAction::Default;
}

TSubclassOf<UOnlineSession> UGameInstance::GetOnlineSessionClass()
{
	return UOnlineSession::StaticClass();
}

bool UGameInstance::IsDedicatedServerInstance() const
{
	if (IsRunningDedicatedServer())
	{
		return true;
	}
	else
	{
		return WorldContext ? WorldContext->RunAsDedicated : false;
	}
}

FName UGameInstance::GetOnlinePlatformName() const
{
	return UOnlineEngineInterface::Get()->GetDefaultOnlineSubsystemName();
}

bool UGameInstance::ClientTravelToSession(int32 ControllerId, FName InSessionName)
{
	UWorld* World = GetWorld();

	FString URL;
	if (UOnlineEngineInterface::Get()->GetResolvedConnectString(World, InSessionName, URL))
	{
		ULocalPlayer* LP = GEngine->GetLocalPlayerFromControllerId(World, ControllerId);
		APlayerController* PC = LP ? ToRawPtr(LP->PlayerController) : nullptr;
		if (PC)
		{
			PC->ClientTravel(URL, TRAVEL_Absolute);
			return true;
		}
		else
		{
			UE_LOGF(LogGameSession, Warning, "Failed to find local player for controller id %d", ControllerId);
		}
	}
	else
	{
		UE_LOGF(LogGameSession, Warning, "Failed to resolve session connect string for %ls", *InSessionName.ToString());
	}

	return false;
}

void UGameInstance::NotifyPreClientTravel(const FString& PendingURL, ETravelType TravelType, bool bIsSeamlessTravel)
{
	OnNotifyPreClientTravel().Broadcast(PendingURL, TravelType, bIsSeamlessTravel);
}

EReplicationSystem UGameInstance::GetDesiredReplicationSystem(FName InNetDriverDefinition) const
{
	EReplicationSystem DesiredRepSystem = EReplicationSystem::Default;

	if (InNetDriverDefinition == NAME_GameNetDriver)
	{
		if (UWorld* World = GetWorld())
		{
			// If we are the server, return the game mode's desired replication system
			if (AGameModeBase* ServerGameMode = World->GetAuthGameMode())
			{
				return ServerGameMode->GetGameNetDriverReplicationSystem();
			}
		}
	}

	return DesiredRepSystem;
}

void UGameInstance::ReturnToMainMenu()
{
	UWorld* const World = GetWorld();
	
	if (ensureMsgf(World != nullptr, TEXT("UGameInstance::ReturnToMainMenu requires a valid world.")))
	{
		if (UOnlineSession* const LocalOnlineSession = GetOnlineSession())
		{
			LocalOnlineSession->HandleDisconnect(World, World->GetNetDriver());
		}
		else
		{
			GetEngine()->HandleDisconnect(World, World->GetNetDriver());
		}
	}
}

void UGameInstance::PreloadContentForURL(FURL InURL)
{
	// Preload game mode and other content if needed here
}

AGameModeBase* UGameInstance::CreateGameModeForURL(FURL InURL, UWorld* InWorld)
{
	// Init the game info.
	FString Options;
	FString GameParam;
	for (int32 i = 0; i < InURL.Op.Num(); i++)
	{
		Options += TEXT("?");
		Options += InURL.Op[i];
		FParse::Value(*InURL.Op[i], TEXT("GAME="), GameParam);
	}

	UWorld* World = InWorld ? InWorld : GetWorld();
	AWorldSettings* Settings = World->GetWorldSettings();
	UGameEngine* const GameEngine = Cast<UGameEngine>(GEngine);

	// Get the GameMode class. Start by using the default game type specified in the map's worldsettings.  It may be overridden by settings below.
	TSubclassOf<AGameModeBase> GameClass = Settings->DefaultGameMode;

	// If there is a GameMode parameter in the URL, allow it to override the default game type
	if (!GameParam.IsEmpty())
	{
		FString const GameClassName = UGameMapsSettings::GetGameModeForName(GameParam);

		// If the gamename was specified, we can use it to fully load the pergame PreLoadClass packages
		if (GameEngine)
		{
			GameEngine->LoadPackagesFully(World, FULLYLOAD_Game_PreLoadClass, *GameClassName);
		}

		// Don't overwrite the map's world settings if we failed to load the value off the command line parameter
		TSubclassOf<AGameModeBase> GameModeParamClass = LoadClass<AGameModeBase>(nullptr, *GameClassName);
		if (GameModeParamClass)
		{
			GameClass = GameModeParamClass;
		}
		else
		{
			UE_LOGF(LogLoad, Warning, "Failed to load game mode '%ls' specified by URL options.", *GameClassName);
		}
	}

	// Next try to parse the map prefix
	if (!GameClass)
	{
		FString MapName = InURL.Map;
		FString MapNameNoPath = FPaths::GetBaseFilename(MapName);
		if (MapNameNoPath.StartsWith(PLAYWORLD_PACKAGE_PREFIX))
		{
			const int32 PrefixLen = UWorld::BuildPIEPackagePrefix(WorldContext->PIEInstance).Len();
			MapNameNoPath.MidInline(PrefixLen, MAX_int32, EAllowShrinking::No);
		}

		FString const GameClassName = UGameMapsSettings::GetGameModeForMapName(FString(MapNameNoPath));

		if (!GameClassName.IsEmpty())
		{
			if (GameEngine)
			{
				GameEngine->LoadPackagesFully(World, FULLYLOAD_Game_PreLoadClass, *GameClassName);
			}

			TSubclassOf<AGameModeBase> GameModeParamClass = LoadClass<AGameModeBase>(nullptr, *GameClassName);
			if (GameModeParamClass)
			{
				GameClass = GameModeParamClass;
			}
			else
			{
				UE_LOGF(LogLoad, Warning, "Failed to load game mode '%ls' specified by prefixed map name %ls.", *GameClassName, *MapNameNoPath);
			}
		}
	}

	// Fall back to game default
	if (!GameClass)
	{
		GameClass = LoadClass<AGameModeBase>(nullptr, *UGameMapsSettings::GetGlobalDefaultGameMode());
	}

	if (!GameClass)
	{
		// Fall back to raw GameMode
		GameClass = AGameModeBase::StaticClass();
	}
	else
	{
		// See if game instance wants to override it
		GameClass = OverrideGameModeClass(GameClass, FPaths::GetBaseFilename(InURL.Map), Options, *InURL.Portal);
	}

	// no matter how the game was specified, we can use it to load the PostLoadClass packages
	if (GameEngine)
	{
		GameEngine->LoadPackagesFully(World, FULLYLOAD_Game_PostLoadClass, GameClass->GetPathName());
		GameEngine->LoadPackagesFully(World, FULLYLOAD_Game_PostLoadClass, TEXT("LoadForAllGameModes"));
	}

	// Spawn the GameMode.
	UE_LOGF(LogLoad, Log, "Game class is '%ls'", *GameClass->GetName());
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save game modes into a map

	return World->SpawnActor<AGameModeBase>(GameClass, SpawnInfo);
}

TSubclassOf<AGameModeBase> UGameInstance::OverrideGameModeClass(TSubclassOf<AGameModeBase> GameModeClass, const FString& MapName, const FString& Options, const FString& Portal) const
{
	 return GameModeClass;
}

void UGameInstance::RegisterReferencedObject(UObject* ObjectToReference)
{
	ReferencedObjects.AddUnique(ObjectToReference);
}

void UGameInstance::UnregisterReferencedObject(UObject* ObjectToReference)
{
	ReferencedObjects.RemoveSingleSwap(ObjectToReference);
}