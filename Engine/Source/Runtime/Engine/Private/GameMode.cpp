// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GameMode.cpp: AGameMode c++ code.
=============================================================================*/

#include "GameFramework/GameMode.h"
#include "Engine/GameInstance.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/NetDriver.h"
#include "GameFramework/EngineMessage.h"
#include "GameFramework/GameState.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/GameSession.h"
#include "GameFramework/CheatManager.h"
#include "GameDelegates.h"
#include "MoviePlayerProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameMode)

namespace MatchState
{
	const FName EnteringMap = FName("EnteringMap");
	const FName WaitingToStart = FName("WaitingToStart");
	const FName InProgress = FName("InProgress");
	const FName WaitingPostMatch = FName("WaitingPostMatch");
	const FName LeavingMap = FName("LeavingMap");
	const FName Aborted = FName("Aborted");
}

AGameMode::AGameMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bDelayedStart = false;

	// One-time initialization
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_PrePhysics;
	MatchState = MatchState::EnteringMap;
	EngineMessageClass = UEngineMessage::StaticClass();
	GameStateClass = AGameState::StaticClass();
	MinRespawnDelay = 1.0f;
	InactivePlayerStateLifeSpan = 300.f;
	MaxInactivePlayers = 16;
}

FString AGameMode::GetNetworkNumber()
{
	UNetDriver* NetDriver = GetWorld()->GetNetDriver();
	return NetDriver ? NetDriver->LowLevelGetNetworkNumber() : FString();
}

void AGameMode::InitGame(const FString& MapName, const FString& Options, FString& ErrorMessage)
{
	Super::InitGame(MapName, Options, ErrorMessage);
	SetMatchState(MatchState::EnteringMap);

	if (GameStateClass == nullptr)
	{
		UE_LOGF(LogGameMode, Error, "GameStateClass is not set, falling back to AGameState.");
		GameStateClass = AGameState::StaticClass();
	}
	else if (!GameStateClass->IsChildOf<AGameState>())
	{
		UE_LOGF(LogGameMode, Error, "Mixing AGameStateBase with AGameMode is not compatible. Change AGameStateBase subclass (%ls) to derive from AGameState, or make both derive from Base", *GameStateClass->GetName());
	}

	// Bind to delegates
	FGameDelegates::Get().GetPendingConnectionLostDelegate().AddUObject(this, &AGameMode::NotifyPendingConnectionLost);
	FGameDelegates::Get().GetPreCommitMapChangeDelegate().AddUObject(this, &AGameMode::PreCommitMapChange);
	FGameDelegates::Get().GetPostCommitMapChangeDelegate().AddUObject(this, &AGameMode::PostCommitMapChange);
	FGameDelegates::Get().GetHandleDisconnectDelegate().AddUObject(this, &AGameMode::HandleDisconnect);
}

void AGameMode::RestartGame()
{
	if ( GameSession->CanRestartGame() )
	{
		if (GetMatchState() == MatchState::LeavingMap)
		{
			return;
		}

		GetWorld()->ServerTravel("?Restart",GetTravelType());
	}
}

void AGameMode::PostLogin( APlayerController* NewPlayer )
{
	UWorld* World = GetWorld();

	// update player count
	if (MustSpectate(NewPlayer))
	{
		NumSpectators++;
	}
	else if (World->IsInSeamlessTravel() || NewPlayer->HasClientLoadedCurrentWorld())
	{
		NumPlayers++;
	}
	else
	{
		NumTravellingPlayers++;
	}

	// save network address for re-associating with reconnecting player, after stripping out port number
	FString Address = NewPlayer->GetPlayerNetworkAddress();
	int32 pos = Address.Find(TEXT(":"), ESearchCase::CaseSensitive);
	NewPlayer->PlayerState->SavedNetworkAddress = (pos > 0) ? Address.Left(pos) : Address;

	// check if this player is reconnecting and already has PlayerState
	FindInactivePlayer(NewPlayer);

	Super::PostLogin(NewPlayer);
}

void AGameMode::Logout( AController* Exiting )
{
	APlayerController* PC = Cast<APlayerController>(Exiting);
	if ( PC != nullptr )
	{
		RemovePlayerControllerFromPlayerCount(PC);
		AddInactivePlayer(PC->PlayerState, PC);
	}

	Super::Logout(Exiting);
}

void AGameMode::StartPlay()
{
	// Don't call super, this class handles begin play/match start itself

	if (MatchState == MatchState::EnteringMap)
	{
		SetMatchState(MatchState::WaitingToStart);
	}

	// Check to see if we should immediately transfer to match start
	if (MatchState == MatchState::WaitingToStart && ReadyToStartMatch())
	{
		StartMatch();
	}
}

void AGameMode::HandleMatchIsWaitingToStart()
{
	if (GameSession != nullptr)
	{
		GameSession->HandleMatchIsWaitingToStart();
	}

	// Calls begin play on actors, unless we're about to transition to match start

	if (!ReadyToStartMatch())
	{
		GetWorldSettings()->NotifyBeginPlay();
	}
}

/// @cond DOXYGEN_WARNINGS

bool AGameMode::ReadyToStartMatch_Implementation()
{
	// If bDelayed Start is set, wait for a manual match start
	if (bDelayedStart)
	{
		return false;
	}

	// By default start when we have > 0 players
	if (GetMatchState() == MatchState::WaitingToStart)
	{
		if (NumPlayers + NumBots > 0)
		{
			return true;
		}
	}
	return false;
}

/// @endcond

void AGameMode::StartMatch()
{
	if (HasMatchStarted())
	{
		// Already started
		return;
	}

	//Let the game session override the StartMatch function, in case it wants to wait for arbitration
	if (GameSession->HandleStartMatchRequest())
	{
		return;
	}

	SetMatchState(MatchState::InProgress);
}

void AGameMode::HandleMatchHasStarted()
{
	GameSession->HandleMatchHasStarted();

	// start human players first
	for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		APlayerController* PlayerController = Iterator->Get();
		if (PlayerController && (PlayerController->GetPawn() == nullptr) && PlayerCanRestart(PlayerController))
		{
			RestartPlayer(PlayerController);
		}
	}

	// Make sure level streaming is up to date before triggering NotifyMatchStarted
	GEngine->BlockTillLevelStreamingCompleted(GetWorld());

	// First fire BeginPlay, if we haven't already in waiting to start match
	GetWorldSettings()->NotifyBeginPlay();

	// Then fire off match started
	GetWorldSettings()->NotifyMatchStarted();

	// if passed in bug info, send player to right location
	const FString BugLocString = UGameplayStatics::ParseOption(OptionsString, TEXT("BugLoc"));
	const FString BugRotString = UGameplayStatics::ParseOption(OptionsString, TEXT("BugRot"));
	if( !BugLocString.IsEmpty() || !BugRotString.IsEmpty() )
	{
		for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController &&  PlayerController->CheatManager != nullptr)
			{
				PlayerController->CheatManager->BugItGoString( BugLocString, BugRotString );
			}
		}
	}

	if (IsHandlingReplays() && GetGameInstance() != nullptr)
	{
		GetGameInstance()->StartRecordingReplay(GetWorld()->GetMapName(), GetWorld()->GetMapName());
	}
}

/// @cond DOXYGEN_WARNINGS

bool AGameMode::ReadyToEndMatch_Implementation()
{
	// By default don't explicitly end match
	return false;
}

/// @endcond

void AGameMode::EndMatch()
{
	if (!IsMatchInProgress())
	{
		return;
	}

	SetMatchState(MatchState::WaitingPostMatch);
}

void AGameMode::HandleMatchHasEnded()
{
	GameSession->HandleMatchHasEnded();

	if (IsHandlingReplays() && GetGameInstance() != nullptr)
	{
		GetGameInstance()->StopRecordingReplay();
	}
}

void AGameMode::StartToLeaveMap()
{
	SetMatchState(MatchState::LeavingMap);
}

void AGameMode::HandleLeavingMap()
{

}

void AGameMode::AbortMatch()
{
	SetMatchState(MatchState::Aborted);
}

void AGameMode::HandleMatchAborted()
{

}

bool AGameMode::HasMatchStarted() const
{
	if (GetMatchState() == MatchState::EnteringMap || GetMatchState() == MatchState::WaitingToStart)
	{
		return false;
	}

	return true;
}

bool AGameMode::IsMatchInProgress() const
{
	if (GetMatchState() == MatchState::InProgress)
	{
		return true;
	}

	return false;
}

bool AGameMode::HasMatchEnded() const
{
	if (GetMatchState() == MatchState::WaitingPostMatch || GetMatchState() == MatchState::LeavingMap)
	{
		return true;
	}

	return false;
}

void AGameMode::SetMatchState(FName NewState)
{
	if (MatchState == NewState)
	{
		return;
	}

	UE_LOGF(LogGameMode, Display, "Match State Changed from %ls to %ls", *MatchState.ToString(), *NewState.ToString());
	FMoviePlayerProxyBlock MoviePlayerProxyBlock;

	MatchState = NewState;

	OnMatchStateSet();

	AGameState* FullGameState = GetGameState<AGameState>();
	if (FullGameState)
	{
		FullGameState->SetMatchState(NewState);
	}

	K2_OnSetMatchState(NewState);
}

void AGameMode::OnMatchStateSet()
{
	FGameModeEvents::OnGameModeMatchStateSetEvent().Broadcast(MatchState);
	// Call change callbacks
	if (MatchState == MatchState::WaitingToStart)
	{
		HandleMatchIsWaitingToStart();
	}
	else if (MatchState == MatchState::InProgress)
	{
		HandleMatchHasStarted();
	}
	else if (MatchState == MatchState::WaitingPostMatch)
	{
		HandleMatchHasEnded();
	}
	else if (MatchState == MatchState::LeavingMap)
	{
		HandleLeavingMap();
	}
	else if (MatchState == MatchState::Aborted)
	{
		HandleMatchAborted();
	}
}

void AGameMode::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	if (GetMatchState() == MatchState::WaitingToStart)
	{
		// Check to see if we should start the match
		if (ReadyToStartMatch())
		{
			UE_LOGF(LogGameMode, Log, "GameMode returned ReadyToStartMatch");
			StartMatch();
		}
	}
	if (GetMatchState() == MatchState::InProgress)
	{
		// Check to see if we should end the match
		if (ReadyToEndMatch())
		{
			UE_LOGF(LogGameMode, Log, "GameMode returned ReadyToEndMatch");
			EndMatch();
		}
	}
}

void AGameMode::HandleSeamlessTravelPlayer(AController*& C)
{
	UE_LOGF(LogGameMode, Log, ">> GameMode::HandleSeamlessTravelPlayer: %ls ", *C->GetName());

	APlayerController* PC = Cast<APlayerController>(C);

	UClass* PCClassToSpawn = GetPlayerControllerClassToSpawnForSeamlessTravel(PC);

	if (PC && PC->GetClass() != PCClassToSpawn)
	{
		if (PC->Player != nullptr)
		{
			// We need to spawn a new PlayerController to replace the old one
			APlayerController* const NewPC = SpawnPlayerControllerCommon(PC->IsLocalPlayerController() ? ROLE_SimulatedProxy : ROLE_AutonomousProxy, PC->GetFocalLocation(), PC->GetControlRotation(), PCClassToSpawn);
			if (NewPC == nullptr)
			{
				UE_LOGF(LogGameMode, Warning, "Failed to spawn new PlayerController for %ls (old class %ls)", *PC->GetHumanReadableName(), *PC->GetClass()->GetName());
				PC->Destroy();
				return;
			}
			else
			{
				PC->SeamlessTravelTo(NewPC);
				NewPC->SeamlessTravelFrom(PC);
				SwapPlayerControllers(PC, NewPC);
				PC = NewPC;
				C = NewPC;
			}
		}
		else
		{
			PC->Destroy();
		}
	}
	else
	{
		// clear out data that was only for the previous game
		C->PlayerState->Reset();
		// create a new PlayerState and copy over info; this is necessary because the old GameMode may have used a different PlayerState class
		APlayerState* OldPlayerState = C->PlayerState;
		C->InitPlayerState();
		OldPlayerState->SeamlessTravelTo(C->PlayerState);
		// we don"t need the old PlayerState anymore
		//@fixme: need a way to replace PlayerStates that doesn't cause incorrect "player left the game"/"player entered the game" messages
		OldPlayerState->Destroy();
	}

	InitSeamlessTravelPlayer(C);

	// Initialize hud and other player details, shared with PostLogin
	GenericPlayerInitialization(C);

	if (PC)
	{
		// This may spawn the player pawn if the game is in progress
		HandleStartingNewPlayer(PC);
	}

	UE_LOGF(LogGameMode, Log, "<< GameMode::HandleSeamlessTravelPlayer: %ls", *C->GetName());
}

void AGameMode::InitSeamlessTravelPlayer(AController* NewController)
{
	Super::InitSeamlessTravelPlayer(NewController);

	APlayerController* NewPC = Cast<APlayerController>(NewController);

	if (NewPC != nullptr)
	{
		SetSeamlessTravelViewTarget(NewPC);

		if (!MustSpectate(NewPC))
		{
			NumPlayers++;
			NumTravellingPlayers--;
		}
	}
	else
	{
		NumBots++;
	}
}

void AGameMode::SetSeamlessTravelViewTarget(APlayerController* PC)
{
	PC->SetViewTarget(PC); //-V678
}

void AGameMode::PlayerSwitchedToSpectatorOnly(APlayerController* PC)
{
	RemovePlayerControllerFromPlayerCount(PC);
	NumSpectators++;
}

void AGameMode::RemovePlayerControllerFromPlayerCount(APlayerController* PC)
{
	if ( PC )
	{
		if ( MustSpectate(PC) )
		{
			NumSpectators--;
		}
		else
		{
			if ( GetWorld()->IsInSeamlessTravel() || PC->HasClientLoadedCurrentWorld() )
			{
				NumPlayers--;
			}
			else
			{
				NumTravellingPlayers--;
			}
		}
	}
}

int32 AGameMode::GetNumPlayers()
{
	return NumPlayers + NumTravellingPlayers;
}

int32 AGameMode::GetNumSpectators()
{
	return NumSpectators;
}

void AGameMode::HandleStartingNewPlayer_Implementation(APlayerController* NewPlayer)
{
	// If players should start as spectators, leave them in the spectator state
	if (!bStartPlayersAsSpectators && !MustSpectate(NewPlayer))
	{
		// If match is in progress, start the player
		if (IsMatchInProgress() && PlayerCanRestart(NewPlayer))
		{
			RestartPlayer(NewPlayer);
		}
		// Check to see if we should start right away, avoids a one frame lag in single player games
		else if (GetMatchState() == MatchState::WaitingToStart)
		{
			// Check to see if we should start the match
			if (ReadyToStartMatch())
			{
				StartMatch();
			}
		}
	}
}

bool AGameMode::PlayerCanRestart_Implementation(APlayerController* Player)
{
	if (!IsMatchInProgress())
	{
		return false;
	}

	return Super::PlayerCanRestart_Implementation(Player);
}

bool AGameMode::GetTravelType()
{
	return false;
}

void AGameMode::Say(const FString& Msg)
{
	Broadcast(nullptr, Msg, NAME_None);
}

void AGameMode::Broadcast( AActor* Sender, const FString& Msg, FName Type )
{
	APlayerState* SenderPlayerState = nullptr;
	if ( Cast<APawn>(Sender) != nullptr)
	{
		SenderPlayerState = Cast<APawn>(Sender)->GetPlayerState();
	}
	else if ( Cast<AController>(Sender) != nullptr)
	{
		SenderPlayerState = Cast<AController>(Sender)->PlayerState;
	}

	for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		if (APlayerController* PC = Iterator->Get())
		{
			PC->ClientTeamMessage(SenderPlayerState, Msg, Type);
		}
	}
}


void AGameMode::BroadcastLocalized( AActor* Sender, TSubclassOf<ULocalMessage> Message, int32 Switch, APlayerState* RelatedPlayerState_1, APlayerState* RelatedPlayerState_2, UObject* OptionalObject )
{
	for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		if (APlayerController* PC = Iterator->Get())
		{
			PC->ClientReceiveLocalizedMessage(Message, Switch, RelatedPlayerState_1, RelatedPlayerState_2, OptionalObject);
		}
	}
}

void AGameMode::AddInactivePlayer(APlayerState* PlayerState, APlayerController* PC)
{
	check(PlayerState)
	UWorld* LocalWorld = GetWorld();
	// don't store if it's an old PlayerState from the previous level or if it's a spectator... or if we are shutting down
	if (!PlayerState->IsFromPreviousLevel() && !MustSpectate(PC) && !LocalWorld->bIsTearingDown)
	{
		APlayerState* const NewPlayerState = PlayerState->Duplicate();
		if (NewPlayerState)
		{
			// Side effect of Duplicate() adding PlayerState to PlayerArray (see APlayerState::PostInitializeComponents)
			GameState->RemovePlayerState(NewPlayerState);

			// make PlayerState inactive
			NewPlayerState->SetReplicates(false);

			// delete after some time
			NewPlayerState->SetLifeSpan(InactivePlayerStateLifeSpan);

			// On console, we have to check the unique net id as network address isn't valid
			const bool bIsConsole = !PLATFORM_DESKTOP;
			// Assume valid unique ids means comparison should be via this method
			const bool bHasValidUniqueId = NewPlayerState->GetUniqueId().IsValid();
			// Don't accidentally compare empty network addresses (already issue with two clients on same machine during development)
			const bool bHasValidNetworkAddress = !NewPlayerState->SavedNetworkAddress.IsEmpty();
			const bool bUseUniqueIdCheck = bIsConsole || bHasValidUniqueId;
			
			// make sure no duplicates
			for (int32 Idx = 0; Idx < InactivePlayerArray.Num(); ++Idx)
			{
				APlayerState* const CurrentPlayerState = InactivePlayerArray[Idx];
				if (!IsValid(CurrentPlayerState))
				{
					// already destroyed, just remove it
					InactivePlayerArray.RemoveAt(Idx, 1);
					Idx--;
				}
				else 
				{
					bool bDestroyPlayerState = false;
					if (!bUseUniqueIdCheck)
					{
						if (bHasValidNetworkAddress && CurrentPlayerState->SavedNetworkAddress == NewPlayerState->SavedNetworkAddress)
						{
							bDestroyPlayerState = true;
						}
					}
					else if (CurrentPlayerState->GetUniqueId() == NewPlayerState->GetUniqueId())
					{
						bDestroyPlayerState = true;
					}

					if (bDestroyPlayerState)
					{
						// destroy the playerstate, then remove it from the tracking
						CurrentPlayerState->Destroy();
						InactivePlayerArray.RemoveAt(Idx, 1);
						Idx--;
					}
					
				}
			}
			InactivePlayerArray.Add(NewPlayerState);

			// make sure we dont go over the maximum number of inactive players allowed
			if ( InactivePlayerArray.Num() > MaxInactivePlayers )
			{
				int32 const NumToRemove = InactivePlayerArray.Num() - MaxInactivePlayers;

				// destroy the extra inactive players
				for (int Idx = 0; Idx < NumToRemove; ++Idx)
				{
					APlayerState* const PS = InactivePlayerArray[Idx];
					if (PS != nullptr)
					{
						PS->Destroy();
					}
				}

				// and then remove them from the tracking array
				InactivePlayerArray.RemoveAt(0, NumToRemove);
			}
		}
	}
}

bool AGameMode::FindInactivePlayer(APlayerController* PC)
{
	check(PC && PC->PlayerState);

	if (InactivePlayerArray.Num() == 0)
	{
		return false;
	}

	// don't bother for spectators
	if (MustSpectate(PC))
	{
		return false;
	}

	#if PLATFORM_DESKTOP
	const FString NewName = PC->PlayerState->GetPlayerName();
	auto EvaluatePlayerState = [PC, &NewName](APlayerState* CurrentPlayerState)
	#else
	auto EvaluatePlayerState = [PC](APlayerState* CurrentPlayerState)
	#endif
	{
		// On console, we have to check the unique net id as network address isn't valid
		#if PLATFORM_DESKTOP
		// Assume valid unique ids means comparison should be via this method
		if (PC->PlayerState->GetUniqueId().IsValid())
		#endif
		{
			return CurrentPlayerState->GetUniqueId() == PC->PlayerState->GetUniqueId();
		}

		#if PLATFORM_DESKTOP
		const FString& NewNetworkAddress = PC->PlayerState->SavedNetworkAddress;
		const bool bHasValidNetworkAddress = !PC->PlayerState->SavedNetworkAddress.IsEmpty();
		return bHasValidNetworkAddress && (FCString::Stricmp(*CurrentPlayerState->SavedNetworkAddress, *NewNetworkAddress) == 0) && (FCString::Stricmp(*CurrentPlayerState->GetPlayerName(), *NewName) == 0);
		#endif
	};

	for (int32 i=0; i < InactivePlayerArray.Num(); i++)
	{
		APlayerState* CurrentPlayerState = InactivePlayerArray[i];
		if ( !IsValid(CurrentPlayerState) )
		{
			InactivePlayerArray.RemoveAt(i,1);
			i--;
		}
		else if (EvaluatePlayerState(CurrentPlayerState))
		{
			// found it!
			APlayerState* OldPlayerState = PC->PlayerState;
			PC->SetPlayerState(CurrentPlayerState);
			PC->PlayerState->SetOwner(PC);
			PC->PlayerState->SetReplicates(true);
			PC->PlayerState->SetLifeSpan(0.0f);
			OverridePlayerState(PC, OldPlayerState);
			GameState->AddPlayerState(PC->PlayerState);
			InactivePlayerArray.RemoveAt(i, 1);
			OldPlayerState->SetIsInactive(true);
			// Set the uniqueId to nullptr so it will not kill the player's registration 
			// in UnregisterPlayerWithSession()
			OldPlayerState->SetUniqueId(FUniqueNetIdRepl());
			OldPlayerState->Destroy();
			PC->PlayerState->OnReactivated();
			return true;
		}
		
	}
	return false;
}

void AGameMode::OverridePlayerState(APlayerController* PC, APlayerState* OldPlayerState)
{
	PC->PlayerState->DispatchOverrideWith(OldPlayerState);
}


bool AGameMode::CanServerTravel(const FString& URL, bool bAbsolute)
{
	if (!Super::CanServerTravel(URL, bAbsolute))
	{
		return false;
	}

	// Check for an error in the server's connection
	if (GetMatchState() == MatchState::Aborted)
	{
		UE_LOGF(LogGameMode, Log, "Not traveling because of network error");
		return false;
	}

	return true;
}

void AGameMode::PostSeamlessTravel()
{
	if (GameSession)
	{
		GameSession->PostSeamlessTravel();
	}

	// We have to make a copy of the controller list, since the code after this will destroy
	// and create new controllers in the world's list
	TArray<AController*> OldControllerList;
	for (auto It = GetWorld()->GetControllerIterator(); It; ++It)
	{
		OldControllerList.Add(It->Get());
	}

	// handle players that are already loaded
	for( AController* Controller : OldControllerList )
	{
		if (Controller->PlayerState)
		{
			APlayerController* PlayerController = Cast<APlayerController>(Controller);
			if (PlayerController == nullptr)
			{
				HandleSeamlessTravelPlayer(Controller);
			}
			else
			{
				if (MustSpectate(PlayerController))
				{
					// The spectator count must be incremented here, instead of in HandleSeamlessTravelPlayer,
					// as otherwise spectators can 'hide' from player counters, by making HasClientLoadedCurrentWorld return false
					NumSpectators++;
				}
				else
				{
					NumTravellingPlayers++;
				}
				if (PlayerController->HasClientLoadedCurrentWorld())
				{
					HandleSeamlessTravelPlayer(Controller);
				}
			}
		}
	}
}

bool AGameMode::IsHandlingReplays()
{
	// If we're running in PIE, don't record demos
	if (GetWorld() != nullptr && GetWorld()->IsPlayInEditor())
	{
		return false;
	}

	return bHandleDedicatedServerReplays && GetNetMode() == ENetMode::NM_DedicatedServer;
}

void AGameMode::PreCommitMapChange(const FString& PreviousMapName, const FString& NextMapName) {}

void AGameMode::PostCommitMapChange() {}

void AGameMode::NotifyPendingConnectionLost(const FUniqueNetIdRepl& ConnectionUniqueId) {}

void AGameMode::HandleDisconnect(UWorld* InWorld, UNetDriver* NetDriver)
{
	AbortMatch();
}

bool AGameMode::SpawnPlayerFromSimulate(const FVector& NewLocation, const FRotator& NewRotation)
{
#if WITH_EDITOR
	APlayerController* PC = GetGameInstance()->GetFirstLocalPlayerController();
	if (PC != nullptr)
	{
		RemovePlayerControllerFromPlayerCount(PC);
		NumPlayers++;
	}
#endif

	return Super::SpawnPlayerFromSimulate(NewLocation, NewRotation);
}

