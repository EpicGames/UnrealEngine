// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameModeBase.h"
#include "Blueprint/BlueprintSupport.h"
#include "Engine/GameInstance.h"
#include "Engine/ServerStatReplicator.h"
#include "GameFramework/GameNetworkManager.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LevelScriptActor.h"
#include "Misc/CommandLine.h"
#include "Misc/PackageName.h"
#include "Net/OnlineEngineInterface.h"
#include "GameFramework/GameStateBase.h"
#include "GameFramework/SpectatorPawn.h"
#include "GameFramework/HUD.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/GameSession.h"
#include "Engine/NetConnection.h"
#include "Engine/ChildConnection.h"
#include "Engine/PlayerStartPIE.h"
#include "Engine/LocalPlayer.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/LevelStreaming.h"
#include "UObject/RemoteObjectTransfer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameModeBase)

#if WITH_EDITOR
	#include "IMovieSceneCapture.h"
	#include "MovieSceneCaptureModule.h"
	#include "MovieSceneCaptureSettings.h"
#endif

DEFINE_LOG_CATEGORY(LogGameMode);

// Statically declared events for plugins to use
FGameModeEvents::FGameModeInitializedEvent FGameModeEvents::GameModeInitializedEvent;
FGameModeEvents::FGameModePreLoginEvent FGameModeEvents::GameModePreLoginEvent;
FGameModeEvents::FGameModePostLoginEvent FGameModeEvents::GameModePostLoginEvent;
FGameModeEvents::FGameModeLogoutEvent FGameModeEvents::GameModeLogoutEvent;
FGameModeEvents::FGameModeMatchStateSetEvent FGameModeEvents::GameModeMatchStateSetEvent;

namespace UE::GameModeBase::Private
{
	static bool bAllowPIESeamlessTravel = false;
	static FAutoConsoleVariableRef CVarAllowPIESeamlessTravel(
		TEXT("net.AllowPIESeamlessTravel"),
		bAllowPIESeamlessTravel,
		TEXT("When true, allow seamless travels in single process PIE.")
	);

	// Fixed in UE5.4
	static bool bAllowListenParamSentToClient = false;
	static FAutoConsoleVariableRef CVarAllowListenParamSentToClient(
		TEXT("net.fix.AllowListenParamSentToClient"),
		bAllowListenParamSentToClient,
		TEXT("When true, allow '?listen' sent to the client during Travel. This would confuse the client and cause crashes with RPCs.")
	);
}


AGameModeBase::AGameModeBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
{
	bNetLoadOnClient = false;
	bPauseable = true;
	bStartPlayersAsSpectators = false;

	DefaultPawnClass = ADefaultPawn::StaticClass();
	PlayerControllerClass = APlayerController::StaticClass();
	PlayerStateClass = APlayerState::StaticClass();
	GameStateClass = AGameStateBase::StaticClass();
	HUDClass = AHUD::StaticClass();
	GameSessionClass = AGameSession::StaticClass();
	SpectatorClass = ASpectatorPawn::StaticClass();
	ReplaySpectatorPlayerControllerClass = APlayerController::StaticClass();
	ServerStatReplicatorClass = AServerStatReplicator::StaticClass();
}

void AGameModeBase::InitGame(const FString& MapName, const FString& Options, FString& ErrorMessage)
{
	UWorld* World = GetWorld();

	// Save Options for future use
	OptionsString = Options;

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save game sessions into a map
	GameSession = World->SpawnActor<AGameSession>(GetGameSessionClass(), SpawnInfo);
	GameSession->InitOptions(Options);

	FGameModeEvents::GameModeInitializedEvent.Broadcast(this);
	if (GetNetMode() != NM_Standalone)
	{
		// Attempt to login, returning true means an async login is in flight
		if (!UOnlineEngineInterface::Get()->DoesSessionExist(World, GameSession->SessionName) && 
			!GameSession->ProcessAutoLogin())
		{
			GameSession->RegisterServer();
		}
	}
}

void AGameModeBase::InitGameState()
{
	GameState->GameModeClass = GetClass();
	GameState->ReceivedGameModeClass();

	GameState->SpectatorClass = SpectatorClass;
	GameState->ReceivedSpectatorClass();
}

void AGameModeBase::PreInitializeComponents()
{
	Super::PreInitializeComponents();

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save game states or network managers into a map									
											
	// Fallback to default GameState if none was specified.
	if (GameStateClass == nullptr)
	{
		UE_LOGF(LogGameMode, Warning, "No GameStateClass was specified in %ls (%ls)", *GetName(), *GetClass()->GetName());
		GameStateClass = AGameStateBase::StaticClass();
	}

	UWorld* World = GetWorld();
	GameState = World->SpawnActor<AGameStateBase>(GameStateClass, SpawnInfo);
	World->SetGameState(GameState);
	if (GameState)
	{
		GameState->AuthorityGameMode = this;
	}

	// Only need NetworkManager for servers in net games
	AWorldSettings* WorldSettings = World->GetWorldSettings();
	World->NetworkManager = WorldSettings->GameNetworkManagerClass ? World->SpawnActor<AGameNetworkManager>(WorldSettings->GameNetworkManagerClass, SpawnInfo) : nullptr;

	InitGameState();
}

TSubclassOf<AGameSession> AGameModeBase::GetGameSessionClass() const
{
	if (UClass* Class = GameSessionClass.Get())
	{
		return Class;
	}

	return AGameSession::StaticClass();
}

/// @cond DOXYGEN_WARNINGS

UClass* AGameModeBase::GetDefaultPawnClassForController_Implementation(AController* InController)
{
#if WITH_EDITOR && DO_CHECK
	UClass* DefaultClass = DefaultPawnClass.DebugAccessRawClassPtr();
	if (DefaultClass)
	{
		if (FBlueprintSupport::IsClassPlaceholder(DefaultClass))
		{
			ensureMsgf(false, TEXT("Trying to spawn class that is, directly or indirectly, a placeholder"));
			return ADefaultPawn::StaticClass();
		}
	}
#endif
	return DefaultPawnClass;
}

/// @endcond

int32 AGameModeBase::GetNumPlayers()
{
	int32 PlayerCount = 0;
	for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		APlayerController* PlayerActor = Iterator->Get();
		if (PlayerActor && PlayerActor->PlayerState && !MustSpectate(PlayerActor))
		{
			PlayerCount++;
		}
	}
	return PlayerCount;
}

int32 AGameModeBase::GetNumSpectators()
{
	int32 PlayerCount = 0;
	for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		APlayerController* PlayerActor = Iterator->Get();
		if (PlayerActor && PlayerActor->PlayerState && MustSpectate(PlayerActor))
		{
			PlayerCount++;
		}
	}
	return PlayerCount;
}

void AGameModeBase::StartPlay()
{
	GameState->HandleBeginPlay();
}

bool AGameModeBase::HasMatchStarted() const
{
	return GameState && GameState->HasMatchStarted();
}

bool AGameModeBase::HasMatchEnded() const
{
	return GameState && GameState->HasMatchEnded();
}

bool AGameModeBase::SetPause(APlayerController* PC, FCanUnpause CanUnpauseDelegate /*= FCanUnpause()*/)
{
	if (AllowPausing(PC))
	{
		// Add it for querying
		Pausers.Add(CanUnpauseDelegate);

		// Let the first one in "own" the pause state
		AWorldSettings * WorldSettings = GetWorldSettings();
		if (WorldSettings->GetPauserPlayerState() == nullptr)
		{
			WorldSettings->SetPauserPlayerState(PC->PlayerState);
		}
		return true;
	}
	return false;
}

bool AGameModeBase::ClearPause()
{
	bool bPauseCleared = false;

	if (!AllowPausing() && Pausers.Num() > 0)
	{
		UE_LOGF(LogGameMode, Log, "Clearing list of UnPause delegates for %ls because game type is not pauseable", *GetFName().ToString());
		Pausers.Reset();
		bPauseCleared = true;
	}

	for (int32 Index = Pausers.Num() - 1; Index >= 0; --Index)
	{
		FCanUnpause CanUnpauseCriteriaMet = Pausers[Index];
		if (CanUnpauseCriteriaMet.IsBound())
		{
			const bool bResult = CanUnpauseCriteriaMet.Execute();
			if (bResult)
			{
				Pausers.RemoveAtSwap(Index, EAllowShrinking::No);
				bPauseCleared = true;
			}
		}
		else
		{
			Pausers.RemoveAtSwap(Index, EAllowShrinking::No);
			bPauseCleared = true;
		}
	}

	// Clear the pause state if the list is empty
	if (Pausers.Num() == 0)
	{
		GetWorldSettings()->SetPauserPlayerState(nullptr);
	}

	return bPauseCleared;
}

void AGameModeBase::ForceClearUnpauseDelegates(AActor* PauseActor)
{
	if (PauseActor != nullptr)
	{
		bool bUpdatePausedState = false;
		for (int32 PauserIdx = Pausers.Num() - 1; PauserIdx >= 0; PauserIdx--)
		{
			FCanUnpause& CanUnpauseDelegate = Pausers[PauserIdx];
			if (CanUnpauseDelegate.GetUObject() == PauseActor)
			{
				Pausers.RemoveAt(PauserIdx);
				bUpdatePausedState = true;
			}
		}

		// If we removed some CanUnpause delegates, we may be able to unpause the game now
		if (bUpdatePausedState)
		{
			ClearPause();
		}

		APlayerController* PC = Cast<APlayerController>(PauseActor);
		AWorldSettings * WorldSettings = GetWorldSettings();
		if (PC != nullptr && PC->PlayerState != nullptr && WorldSettings != nullptr && WorldSettings->GetPauserPlayerState() == PC->PlayerState)
		{
			// Try to find another player to be the worldsettings's PauserPlayerState
			for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* Player = Iterator->Get();
				if (Player && Player->PlayerState != nullptr
					&&	Player->PlayerState != PC->PlayerState
					&& !Player->IsPendingKillPending() && !Player->PlayerState->IsPendingKillPending())
				{
					WorldSettings->SetPauserPlayerState(Player->PlayerState);
					break;
				}
			}

			// If it's still pointing to the original player's PlayerState, clear it completely
			if (WorldSettings->GetPauserPlayerState() == PC->PlayerState)
			{
				WorldSettings->SetPauserPlayerState(nullptr);
			}
		}
	}
}

bool AGameModeBase::AllowPausing(APlayerController* PC /*= nullptr*/)
{
	return bPauseable || GetNetMode() == NM_Standalone;
}

bool AGameModeBase::IsPaused() const
{
	return Pausers.Num() > 0;
}

void AGameModeBase::Reset()
{
	Super::Reset();
	InitGameState();
}

/// @cond DOXYGEN_WARNINGS

bool AGameModeBase::ShouldReset_Implementation(AActor* ActorToReset)
{
	return true;
}

/// @endcond

void AGameModeBase::ResetLevel()
{
	UE_LOGF(LogGameMode, Verbose, "Reset %ls", *GetName());

	// Reset ALL controllers first
	for (FConstControllerIterator Iterator = GetWorld()->GetControllerIterator(); Iterator; ++Iterator)
	{
		AController* Controller = Iterator->Get();
		APlayerController* PlayerController = Cast<APlayerController>(Controller);
		if (PlayerController)
		{
			PlayerController->ClientReset();
		}
		Controller->Reset();
	}

	// Reset all actors (except controllers, the GameMode, and any other actors specified by ShouldReset())
	for (FActorIterator It(GetWorld()); It; ++It)
	{
		AActor* A = *It;
		if (IsValid(A) && A != this && !A->IsA<AController>() && ShouldReset(A))
		{
			A->Reset();
		}
	}

	// Reset the GameMode
	Reset();

	// Notify the level script that the level has been reset
	ALevelScriptActor* LevelScript = GetWorld()->GetLevelScriptActor();
	if (LevelScript)
	{
		LevelScript->LevelReset();
	}
}

void AGameModeBase::ReturnToMainMenuHost()
{
	if (GameSession)
	{
		GameSession->ReturnToMainMenuHost();
	}
}

APlayerController* AGameModeBase::ProcessClientTravel(FString& FURL, bool bSeamless, bool bAbsolute)
{
	// We call PreClientTravel directly on any local PlayerPawns (ie listen server)
	APlayerController* LocalPlayerController = nullptr;
	for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		if (APlayerController* PlayerController = Iterator->Get())
		{
			if (Cast<UNetConnection>(PlayerController->Player) != nullptr)
			{
				FString ClientUrl = FURL;
				if (!UE::GameModeBase::Private::bAllowListenParamSentToClient)
				{
					// We are going to lop off the listen option when sending to the client, so they don't think they're the server.
					// That sounds silly, but in GetNetMode, we will eventually arrive at AttemptDeriveFromURL in certain circumstances (such as during travel!)
					// The client would then think that they're supposed to execute things locally rather than RPC them based on the listen param.
					static FString ListenString = "?listen";
					int32 ListenIndex = ClientUrl.Find(ListenString, ESearchCase::IgnoreCase);
					if (ListenIndex >= 0)
					{
						// Only remove if the next character isn't part of a longer option string
						int32 NextIndex = ListenIndex + ListenString.Len();						
						if (NextIndex >= ClientUrl.Len() || ClientUrl[NextIndex] == TEXT('#') || ClientUrl[NextIndex] == TEXT('?'))
						{
							ClientUrl.RemoveAt(ListenIndex, ListenString.Len());
						}
					}
				}

				// Remote player
				PlayerController->ClientTravel(ClientUrl, TRAVEL_Relative, bSeamless);
			}
			else
			{
				// Local player
				LocalPlayerController = PlayerController;
				PlayerController->PreClientTravel(FURL, bAbsolute ? TRAVEL_Absolute : TRAVEL_Relative, bSeamless);
			}
		}
	}

	return LocalPlayerController;
}

bool AGameModeBase::CanServerTravel(const FString& FURL, bool bAbsolute)
{
	UWorld* World = GetWorld();

	check(World);

	if (FURL.Contains(TEXT("%")))
	{
		UE_LOGF(LogGameMode, Error, "CanServerTravel: FURL %ls Contains illegal character '%%'.", *FURL);
		return false;
	}

	if (FURL.Contains(TEXT(":")) || FURL.Contains(TEXT("\\")))
	{
		UE_LOGF(LogGameMode, Error, "CanServerTravel: FURL %ls blocked, contains : or \\", *FURL);
		return false;
	}

	FString MapName;
	int32 OptionStart = FURL.Find(TEXT("?"));
	if (OptionStart == INDEX_NONE)
	{
		MapName = FURL;
	}
	else
	{
		MapName = FURL.Left(OptionStart);
	}

	// Check for invalid package names.
	FText InvalidPackageError;
	if (MapName.StartsWith(TEXT("/")) && !FPackageName::IsValidLongPackageName(MapName, true, &InvalidPackageError))
	{
		UE_LOGF(LogGameMode, Log, "CanServerTravel: FURL %ls blocked (%ls)", *FURL, *InvalidPackageError.ToString());
		return false;
	}
	
	return true;
}

void AGameModeBase::ProcessServerTravel(const FString& URL, bool bAbsolute)
{
#if WITH_SERVER_CODE
	StartToLeaveMap();

	UE_LOGF(LogGameMode, Log, "ProcessServerTravel: %ls", *URL);
	UWorld* World = GetWorld();
	check(World);
	FWorldContext& WorldContext = GEngine->GetWorldContextFromWorldChecked(World);

	// Use game mode setting but default to full load screen if the server has been up for a long time so that TimeSeconds doesn't overflow and break everything
	bool bSeamless = (bUseSeamlessTravel && GetWorld()->TimeSeconds < 172800.0f); // 172800 seconds == 48 hours

	// Compute the next URL, and pull the map out of it. This handles short->long package name conversion
	FURL NextURL = FURL(&WorldContext.LastURL, *URL, bAbsolute ? TRAVEL_Absolute : TRAVEL_Relative);

	// Override based on URL parameters
	if (NextURL.HasOption(TEXT("SeamlessTravel")))
	{
		bSeamless = true;
	}
	else if (NextURL.HasOption(TEXT("NoSeamlessTravel")))
	{
		bSeamless = false;
	}

	// There are some issues with seamless travel in PIE, so fall back to hard travel unless it is supported
	if (World->WorldType == EWorldType::PIE && bSeamless && !FParse::Param(FCommandLine::Get(), TEXT("MultiprocessOSS")))
	{
		if (!UE::GameModeBase::Private::bAllowPIESeamlessTravel)
		{
			UE_LOGF(LogGameMode, Warning, "ProcessServerTravel: Seamless travel is disabled in PIE, set net.AllowPIESeamlessTravel=1 to enable.");
			bSeamless = false;
		}
	}

	// Notify clients we're switching level and give them time to receive.
	FString URLMod = NextURL.ToString();
	APlayerController* LocalPlayer = ProcessClientTravel(URLMod, bSeamless, bAbsolute);

	World->NextURL = URLMod;
	ENetMode NetMode = GetNetMode();

	if (bSeamless)
	{
		World->SeamlessTravel(World->NextURL, bAbsolute);
		World->NextURL = TEXT("");
	}
	else
	{
		// Switch immediately if not networking.
		if (NetMode != NM_DedicatedServer && NetMode != NM_ListenServer)
		{
			World->NextSwitchCountdown = 0.0f;
		}

		GEngine->IncrementGlobalNetTravelCount();
		GEngine->SaveConfig();
	}
#endif // WITH_SERVER_CODE
}

void AGameModeBase::GetSeamlessTravelActorList(bool bToTransition, TArray<AActor*>& ActorList)
{
	// Get allocations for the elements we're going to add handled in one go
	const int32 ActorsToAddCount = GameState->PlayerArray.Num() + (bToTransition ? 3 : 0);
	ActorList.Reserve(ActorsToAddCount);

	// Always keep PlayerStates, so that after we restart we can keep players on the same team, etc
	ActorList.Append(GameState->PlayerArray);

	if (bToTransition)
	{
		// Keep ourselves until we transition to the final destination
		ActorList.Add(this);
		// Keep general game state until we transition to the final destination
		ActorList.Add(GameState);
		// Keep the game session state until we transition to the final destination
		ActorList.Add(GameSession);

		// If adding in this section best to increase the literal above for the ActorsToAddCount
	}
}

void AGameModeBase::SwapPlayerControllers(APlayerController* OldPC, APlayerController* NewPC)
{
	if (IsValid(OldPC) && IsValid(NewPC) && OldPC->Player != nullptr)
	{
		// move the Player to the new PC
		UPlayer* Player = OldPC->Player;
		NewPC->NetPlayerIndex = OldPC->NetPlayerIndex; //@warning: critical that this is first as SetPlayer() may trigger RPCs
		NewPC->NetConnection = OldPC->NetConnection;
		NewPC->SetReplicates(OldPC->GetIsReplicated());
		NewPC->SetPlayer(Player);
		NewPC->CopyRemoteRoleFrom(OldPC);

		K2_OnSwapPlayerControllers(OldPC, NewPC);

		// send destroy event to old PC immediately if it's local
		if (Cast<ULocalPlayer>(Player))
		{
			GetWorld()->DestroyActor(OldPC);
		}
		else
		{
			OldPC->PendingSwapConnection = Cast<UNetConnection>(Player);
			//@note: at this point, any remaining RPCs sent by the client on the old PC will be discarded
			// this is consistent with general owned Actor destruction,
			// however in this particular case it could easily be changed
			// by modifying UActorChannel::ReceivedBunch() to account for PendingSwapConnection when it is setting bNetOwner
		}
	}
	else
	{
		UE_LOGF(LogGameMode, Warning, "SwapPlayerControllers: Invalid OldPC, invalid NewPC, or OldPC has no Player!");
	}
}

TSubclassOf<APlayerController> AGameModeBase::GetPlayerControllerClassToSpawnForSeamlessTravel(APlayerController* PreviousPC)
{
	UClass* PCClassToSpawn = PlayerControllerClass;

	if (PreviousPC && ReplaySpectatorPlayerControllerClass && PreviousPC->PlayerState && PreviousPC->PlayerState->IsOnlyASpectator())
	{
		PCClassToSpawn = ReplaySpectatorPlayerControllerClass;
	}

	return PCClassToSpawn;
}

void AGameModeBase::HandleSeamlessTravelPlayer(AController*& C)
{
	// Default behavior is to spawn new controllers and copy data
	APlayerController* PC = Cast<APlayerController>(C);
	if (PC && PC->Player)
	{
		// We need to spawn a new PlayerController to replace the old one
		UClass* PCClassToSpawn = GetPlayerControllerClassToSpawnForSeamlessTravel(PC);
		APlayerController* const NewPC = SpawnPlayerControllerCommon(PC->IsLocalPlayerController() ? ROLE_SimulatedProxy : ROLE_AutonomousProxy, PC->GetFocalLocation(), PC->GetControlRotation(), PCClassToSpawn);
		if (NewPC)
		{
			PC->SeamlessTravelTo(NewPC);
			NewPC->SeamlessTravelFrom(PC);
			SwapPlayerControllers(PC, NewPC);
			PC = NewPC;
			C = NewPC;
		}
		else
		{
			UE_LOGF(LogGameMode, Warning, "HandleSeamlessTravelPlayer: Failed to spawn new PlayerController for %ls (old class %ls)", *PC->GetHumanReadableName(), *PC->GetClass()->GetName());
			PC->Destroy();
			return;
		}
	}

	InitSeamlessTravelPlayer(C);

	// Initialize hud and other player details, shared with PostLogin
	GenericPlayerInitialization(C);

	if (PC)
	{
		// This may spawn the player pawn if the game is in progress
		HandleStartingNewPlayer(PC);
	}
}

void AGameModeBase::PostSeamlessTravel()
{
	if (GameSession)
	{
		GameSession->PostSeamlessTravel();
	}

	// We have to make a copy of the controller list, since the code after this will destroy
	// and create new controllers in the world's list
	TArray<AController*> OldControllerList;
	for (auto It = GetWorld()->GetControllerIterator(); It; ++It)
	{
		OldControllerList.Add(It->Get());
	}

	// Handle players that are already loaded
	for (AController* Controller : OldControllerList)
	{
		if (Controller->PlayerState)
		{
			APlayerController* PlayerController = Cast<APlayerController>(Controller);
			if (!PlayerController || PlayerController->HasClientLoadedCurrentWorld())
			{
				// Don't handle if player is still loading world, that gets called in ServerNotifyLoadedWorld
				HandleSeamlessTravelPlayer(Controller);
			}
		}
	}
}

void AGameModeBase::StartToLeaveMap()
{

}

void AGameModeBase::GameWelcomePlayer(UNetConnection* Connection, FString& RedirectURL) 
{

}

void AGameModeBase::PreLogin(const FString& Options, const FString& Address, const FUniqueNetIdRepl& UniqueId, FString& ErrorMessage)
{
	// Login unique id must match server expected unique id type OR No unique id could mean game doesn't use them
	const bool bUniqueIdCheckOk = (!UniqueId.IsValid() || UOnlineEngineInterface::Get()->IsCompatibleUniqueNetId(UniqueId));
	if (bUniqueIdCheckOk)
	{
		ErrorMessage = GameSession->ApproveLogin(Options);
	}
	else
	{
		ErrorMessage = TEXT("incompatible_unique_net_id");
	}

	FGameModeEvents::GameModePreLoginEvent.Broadcast(this, UniqueId, ErrorMessage);
}

void AGameModeBase::PreLoginAsync(const FString& Options, const FString& Address, const FUniqueNetIdRepl& UniqueId, const FOnPreLoginCompleteDelegate& OnComplete)
{
	FString ErrorMessage;
	PreLogin(Options, Address, UniqueId, ErrorMessage);
	OnComplete.ExecuteIfBound(ErrorMessage);
}

APlayerController* AGameModeBase::Login(UPlayer* NewPlayer, ENetRole InRemoteRole, const FString& Portal, const FString& Options, const FUniqueNetIdRepl& UniqueId, FString& ErrorMessage)
{
	if (GameSession == nullptr)
	{
		ErrorMessage = TEXT("Failed to spawn player controller, GameSession is null");
		return nullptr;
	}

	ErrorMessage = GameSession->ApproveLogin(Options);
	if (!ErrorMessage.IsEmpty())
	{
		return nullptr;
	}

	APlayerController* const NewPlayerController = SpawnPlayerController(InRemoteRole, Options);
	if (NewPlayerController == nullptr)
	{
		// Handle spawn failure.
		UE_LOGF(LogGameMode, Log, "Login: Couldn't spawn player controller of class %ls", PlayerControllerClass ? *PlayerControllerClass->GetName() : TEXT("NULL"));
		ErrorMessage = FString::Printf(TEXT("Failed to spawn player controller"));
		return nullptr;
	}

	// Customize incoming player based on URL options
	ErrorMessage = InitNewPlayer(NewPlayerController, UniqueId, Options, Portal);
	if (!ErrorMessage.IsEmpty())
	{
		NewPlayerController->Destroy();
		return nullptr;
	}

	return NewPlayerController;
}

APlayerController* AGameModeBase::SpawnPlayerController(ENetRole InRemoteRole, const FString& Options)
{
	if (Options.Contains(FString("SpectatorOnly=1")) && ReplaySpectatorPlayerControllerClass != nullptr)
	{
		return SpawnPlayerControllerCommon(InRemoteRole, FVector::ZeroVector, FRotator::ZeroRotator, ReplaySpectatorPlayerControllerClass);
	}
	
	return SpawnPlayerControllerCommon(InRemoteRole, FVector::ZeroVector, FRotator::ZeroRotator, PlayerControllerClass);
}

APlayerController* AGameModeBase::SpawnPlayerControllerCommon(ENetRole InRemoteRole, FVector const& SpawnLocation, FRotator const& SpawnRotation, TSubclassOf<APlayerController> InPlayerControllerClass)
{
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save player controllers into a map
	SpawnInfo.bDeferConstruction = true;
	APlayerController* NewPC = GetWorld()->SpawnActor<APlayerController>(InPlayerControllerClass, SpawnLocation, SpawnRotation, SpawnInfo);
	if (NewPC)
	{
		if (InRemoteRole == ROLE_SimulatedProxy)
		{
			// This is a local player because it has no authority/autonomous remote role
			NewPC->SetAsLocalPlayerController();
		}

		UGameplayStatics::FinishSpawningActor(NewPC, FTransform(SpawnRotation, SpawnLocation));
	}

	return NewPC;
}

FString AGameModeBase::InitNewPlayer(APlayerController* NewPlayerController, const FUniqueNetIdRepl& UniqueId, const FString& Options, const FString& Portal)
{
	check(NewPlayerController);

	// The player needs a PlayerState to register successfully
	if (NewPlayerController->PlayerState == nullptr)
	{
		return FString("PlayerState is null");
	}

	// Register the player with the session
	GameSession->RegisterPlayer(NewPlayerController, UniqueId, UGameplayStatics::HasOption(Options, TEXT("bIsFromInvite")));

	// Find a starting spot
	FString ErrorMessage;
	if (!UpdatePlayerStartSpot(NewPlayerController, Portal, ErrorMessage))
	{
		UE_LOGF(LogGameMode, Warning, "InitNewPlayer: %ls", *ErrorMessage);
	}

	// Set up spectating
	bool bSpectator = FCString::Stricmp(*UGameplayStatics::ParseOption(Options, TEXT("SpectatorOnly")), TEXT("1")) == 0;
	if (bSpectator || MustSpectate(NewPlayerController))
	{
		NewPlayerController->StartSpectatingOnly();
	}

	// Init player's name
	FString InName = UGameplayStatics::ParseOption(Options, TEXT("Name")).Left(20);
	if (InName.IsEmpty())
	{
		InName = FString::Printf(TEXT("%s%i"), *DefaultPlayerName.ToString(), NewPlayerController->PlayerState->GetPlayerId());
	}

	ChangeName(NewPlayerController, InName, false);

	return ErrorMessage;
}

void AGameModeBase::InitSeamlessTravelPlayer(AController* NewController)
{
	APlayerController* NewPC = Cast<APlayerController>(NewController);

	FString ErrorMessage;
	if (!UpdatePlayerStartSpot(NewController, TEXT(""), ErrorMessage))
	{
		UE_LOGF(LogGameMode, Warning, "InitSeamlessTravelPlayer: %ls", *ErrorMessage);
	}

	if (NewPC != nullptr)
	{
		NewPC->PostSeamlessTravel();

		if (MustSpectate(NewPC))
		{
			NewPC->StartSpectatingOnly();
		}
		else
		{
			NewPC->bPlayerIsWaiting = true;
			NewPC->ChangeState(NAME_Spectating);
			NewPC->ClientGotoState(NAME_Spectating);
		}
	}
}

bool AGameModeBase::UpdatePlayerStartSpot(AController* Player, const FString& Portal, FString& OutErrorMessage)
{
	OutErrorMessage.Reset();

	AActor* const StartSpot = FindPlayerStart(Player, Portal);
	if (StartSpot != nullptr)
	{
		FRotator StartRotation(0, StartSpot->GetActorRotation().Yaw, 0);
		Player->SetInitialLocationAndRotation(StartSpot->GetActorLocation(), StartRotation);

		Player->StartSpot = StartSpot;

		return true;
	}

	OutErrorMessage = FString::Printf(TEXT("Could not find a starting spot"));
	return false;
}

bool AGameModeBase::ShouldStartInCinematicMode(APlayerController* Player, bool& OutHidePlayer, bool& OutHideHUD, bool& OutDisableMovement, bool& OutDisableTurning)
{
	ULocalPlayer* LocPlayer = Player->GetLocalPlayer();
	if (!LocPlayer)
	{
		return false;
	}

#if WITH_EDITOR
	// If we have an active movie scene capture, we can take the settings from that
	if (LocPlayer->ViewportClient && LocPlayer->ViewportClient->Viewport)
	{
		if (auto* MovieSceneCapture = IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture())
		{
			const FMovieSceneCaptureSettings& Settings = MovieSceneCapture->GetSettings();
			if (Settings.bCinematicMode)
			{
				OutDisableMovement = !Settings.bAllowMovement;
				OutDisableTurning = !Settings.bAllowTurning;
				OutHidePlayer = !Settings.bShowPlayer;
				OutHideHUD = !Settings.bShowHUD;
				return true;
			}
		}
	}
#endif

	return false;
}

/// @cond DOXYGEN_WARNINGS

void AGameModeBase::InitializeHUDForPlayer_Implementation(APlayerController* NewPlayer)
{
	// Tell client what HUD class to use
	NewPlayer->ClientSetHUD(HUDClass);
}

/// @endcond

void AGameModeBase::UpdateGameplayMuteList(APlayerController* aPlayer)
{
	if (aPlayer)
	{
		aPlayer->MuteList.bHasVoiceHandshakeCompleted = true;
		aPlayer->ClientVoiceHandshakeComplete();
	}
}

void AGameModeBase::ReplicateStreamingStatus(APlayerController* PC)
{
	UWorld* MyWorld = GetWorld();

	if (MyWorld->GetWorldSettings()->bUseClientSideLevelStreamingVolumes)
	{
		// Client will itself decide what to stream
		return;
	}

	// Don't do this for local players or players after the first on a splitscreen client
	if (Cast<ULocalPlayer>(PC->Player) == nullptr && Cast<UChildConnection>(PC->Player) == nullptr)
	{
		// If we've loaded levels via CommitMapChange() that aren't normally in the StreamingLevels array, tell the client about that
		if (MyWorld->CommittedPersistentLevelName != NAME_None)
		{
			PC->ClientPrepareMapChange(MyWorld->CommittedPersistentLevelName, true, true);
			// Tell the client to commit the level immediately
			PC->ClientCommitMapChange();
		}

		if (MyWorld->GetStreamingLevels().Num() > 0)
		{
			// Tell the player controller the current streaming level status
			TArray<FUpdateLevelStreamingLevelStatus> LevelStatuses;
			for (ULevelStreaming* TheLevel : MyWorld->GetStreamingLevels())
			{
				if (TheLevel && TheLevel->CanReplicateStreamingStatus())
				{
					const ULevel* LoadedLevel = TheLevel->GetLoadedLevel();

					const bool bTheLevelShouldBeVisible = TheLevel->ShouldBeVisible();
					const bool bTheLevelShouldBeLoaded = TheLevel->ShouldBeLoaded();

					UE_LOGF(LogGameMode, Verbose, "ReplicateStreamingStatus: %ls %i %i %i %ls %i",
						*TheLevel->GetWorldAssetPackageName(),
						bTheLevelShouldBeVisible,
						LoadedLevel && LoadedLevel->bIsVisible,
						bTheLevelShouldBeLoaded,
						*GetNameSafe(LoadedLevel),
						TheLevel->HasLoadRequestPending());

					FUpdateLevelStreamingLevelStatus& LevelStatus = *new( LevelStatuses ) FUpdateLevelStreamingLevelStatus();
					LevelStatus.PackageName = PC->NetworkRemapPath(TheLevel->GetWorldAssetPackageFName(), false);
					LevelStatus.bNewShouldBeLoaded = bTheLevelShouldBeLoaded;
					LevelStatus.bNewShouldBeVisible = bTheLevelShouldBeVisible;
					LevelStatus.bNewShouldBlockOnLoad = TheLevel->bShouldBlockOnLoad;
					LevelStatus.bNewShouldBlockOnUnload = TheLevel->bShouldBlockOnUnload;
					LevelStatus.LODIndex = TheLevel->GetLevelLODIndex();
				}
			}
			PC->ClientUpdateMultipleLevelsStreamingStatus( LevelStatuses );
			PC->ClientFlushLevelStreaming();
		}

		// If we're preparing to load different levels using PrepareMapChange() inform the client about that now
		if (MyWorld->PreparingLevelNames.Num() > 0)
		{
			for (int32 LevelIndex = 0; LevelIndex < MyWorld->PreparingLevelNames.Num(); LevelIndex++)
			{
				PC->ClientPrepareMapChange(MyWorld->PreparingLevelNames[LevelIndex], LevelIndex == 0, LevelIndex == MyWorld->PreparingLevelNames.Num() - 1);
			}
			// DO NOT commit these changes yet - we'll send that when we're done preparing them
		}
	}
}

void AGameModeBase::GenericPlayerInitialization(AController* C)
{
	APlayerController* PC = Cast<APlayerController>(C);
	if (PC != nullptr)
	{
		InitializeHUDForPlayer(PC);

		// Notify the game that we can now be muted and mute others
		UpdateGameplayMuteList(PC);

		if (GameSession != nullptr)
		{
			// Tell the player to enable voice by default or use the push to talk method
			PC->ClientEnableNetworkVoice(!GameSession->RequiresPushToTalk());
		}

		ReplicateStreamingStatus(PC);

		bool HidePlayer = false, HideHUD = false, DisableMovement = false, DisableTurning = false;

		// Check to see if we should start in cinematic mode
		if (ShouldStartInCinematicMode(PC, HidePlayer, HideHUD, DisableMovement, DisableTurning))
		{
			PC->SetCinematicMode(true, HidePlayer, HideHUD, DisableMovement, DisableTurning);
		}
	}
}

void AGameModeBase::PostLogin(APlayerController* NewPlayer)
{
	// Runs shared initialization that can happen during seamless travel as well

	GenericPlayerInitialization(NewPlayer);

	// Perform initialization that only happens on initially joining a server

	UWorld* World = GetWorld();

	NewPlayer->ClientCapBandwidth(NewPlayer->Player->CurrentNetSpeed);

	if (MustSpectate(NewPlayer))
	{
		NewPlayer->ClientGotoState(NAME_Spectating);
	}
	else
	{
		// If NewPlayer is not only a spectator and has a valid ID, add it as a user to the replay.
		const FUniqueNetIdRepl& NewPlayerStateUniqueId = NewPlayer->PlayerState->GetUniqueId();
		if (NewPlayerStateUniqueId.IsValid() && NewPlayerStateUniqueId.IsV1())
		{
			GetGameInstance()->AddUserToReplay(NewPlayerStateUniqueId.ToString());
		}
	}

	if (GameSession)
	{
		GameSession->PostLogin(NewPlayer);
	}

	OnPostLogin(NewPlayer);

	// Now that initialization is done, try to spawn the player's pawn and start match
	HandleStartingNewPlayer(NewPlayer);
}

void AGameModeBase::DispatchPostLogin(AController* NewPlayer)
{
}

void AGameModeBase::OnPostLogin(AController* NewPlayer)
{
	if (APlayerController* NewPC = Cast<APlayerController>(NewPlayer))
	{
		K2_PostLogin(NewPC);
		FGameModeEvents::GameModePostLoginEvent.Broadcast(this, NewPC);
	}
}

void AGameModeBase::Logout(AController* Exiting)
{
	APlayerController* PC = Cast<APlayerController>(Exiting);
	if (PC != nullptr)
	{
		FGameModeEvents::GameModeLogoutEvent.Broadcast(this, Exiting);
		K2_OnLogout(Exiting);

		if (GameSession)
		{
			GameSession->NotifyLogout(PC);
		}
	}
}

/// @cond DOXYGEN_WARNINGS

void AGameModeBase::HandleStartingNewPlayer_Implementation(APlayerController* NewPlayer)
{
	// If players should start as spectators, leave them in the spectator state
	if (!bStartPlayersAsSpectators && !MustSpectate(NewPlayer) && PlayerCanRestart(NewPlayer))
	{
		// Otherwise spawn their pawn immediately
		RestartPlayer(NewPlayer);
	}
}

bool AGameModeBase::MustSpectate_Implementation(APlayerController* NewPlayerController) const
{
	if (!NewPlayerController || !NewPlayerController->PlayerState)
	{
		return false;
	}

	return NewPlayerController->PlayerState->IsOnlyASpectator();
}

bool AGameModeBase::CanSpectate_Implementation(APlayerController* Viewer, APlayerState* ViewTarget)
{
	return true;
}

AActor* AGameModeBase::ChoosePlayerStart_Implementation(AController* Player)
{
	// Choose a player start
	APlayerStart* FoundPlayerStart = nullptr;
	UClass* PawnClass = GetDefaultPawnClassForController(Player);
	APawn* PawnToFit = PawnClass ? PawnClass->GetDefaultObject<APawn>() : nullptr;
	TArray<APlayerStart*> UnOccupiedStartPoints;
	TArray<APlayerStart*> OccupiedStartPoints;
	UWorld* World = GetWorld();
	for (TActorIterator<APlayerStart> It(World); It; ++It)
	{
		APlayerStart* PlayerStart = *It;

		if (PlayerStart->IsA<APlayerStartPIE>())
		{
			// Always prefer the first "Play from Here" PlayerStart, if we find one while in PIE mode
			FoundPlayerStart = PlayerStart;
			break;
		}
		else
		{
			FVector ActorLocation = PlayerStart->GetActorLocation();
			const FRotator ActorRotation = PlayerStart->GetActorRotation();
			if (!World->EncroachingBlockingGeometry(PawnToFit, ActorLocation, ActorRotation))
			{
				UnOccupiedStartPoints.Add(PlayerStart);
			}
			else if (World->FindTeleportSpot(PawnToFit, ActorLocation, ActorRotation))
			{
				OccupiedStartPoints.Add(PlayerStart);
			}
		}
	}
	if (FoundPlayerStart == nullptr)
	{
		if (UnOccupiedStartPoints.Num() > 0)
		{
			FoundPlayerStart = UnOccupiedStartPoints[FMath::RandRange(0, UnOccupiedStartPoints.Num() - 1)];
		}
		else if (OccupiedStartPoints.Num() > 0)
		{
			FoundPlayerStart = OccupiedStartPoints[FMath::RandRange(0, OccupiedStartPoints.Num() - 1)];
		}
	}
	return FoundPlayerStart;
}

/// @endcond

bool AGameModeBase::ShouldSpawnAtStartSpot(AController* Player)
{
	return (Player != nullptr && Player->StartSpot != nullptr);
}

/// @cond DOXYGEN_WARNINGS

AActor* AGameModeBase::FindPlayerStart_Implementation(AController* Player, const FString& IncomingName)
{
	UWorld* World = GetWorld();

	// If incoming start is specified, then just use it
	if (!IncomingName.IsEmpty())
	{
		const FName IncomingPlayerStartTag = FName(*IncomingName);
		for (TActorIterator<APlayerStart> It(World); It; ++It)
		{
			APlayerStart* Start = *It;
			if (Start && Start->PlayerStartTag == IncomingPlayerStartTag)
			{
				return Start;
			}
		}
	}

	// Always pick StartSpot at start of match
	if (ShouldSpawnAtStartSpot(Player))
	{
		if (AActor* PlayerStartSpot = Player->StartSpot.Get())
		{
			return PlayerStartSpot;
		}
		else
		{
			UE_LOGF(LogGameMode, Error, "FindPlayerStart: ShouldSpawnAtStartSpot returned true but the Player StartSpot was null.");
		}
	}

	AActor* BestStart = ChoosePlayerStart(Player);
	if (BestStart == nullptr)
	{
		// No player start found
		UE_LOGF(LogGameMode, Log, "FindPlayerStart: PATHS NOT DEFINED or NO PLAYERSTART with positive rating");

		// This is a bit odd, but there was a complex chunk of code that in the end always resulted in this, so we may as well just 
		// short cut it down to this.  Basically we are saying spawn at 0,0,0 if we didn't find a proper player start
		BestStart = World->GetWorldSettings();
	}

	return BestStart;
}

/// @endcond

AActor* AGameModeBase::K2_FindPlayerStart(AController* Player, const FString& IncomingName)
{
	return FindPlayerStart(Player, IncomingName);
}

/// @cond DOXYGEN_WARNINGS

bool AGameModeBase::PlayerCanRestart_Implementation(APlayerController* Player)
{
	if (Player == nullptr || Player->IsPendingKillPending())
	{
		return false;
	}

	// Ask the player controller if it's ready to restart as well
	return Player->CanRestartPlayer();
}

APawn* AGameModeBase::SpawnDefaultPawnFor_Implementation(AController* NewPlayer, AActor* StartSpot)
{
	// Don't allow pawn to be spawned with any pitch or roll
	FRotator StartRotation(ForceInit);
	StartRotation.Yaw = StartSpot->GetActorRotation().Yaw;
	FVector StartLocation = StartSpot->GetActorLocation();

	FTransform Transform = FTransform(StartRotation, StartLocation);
	return SpawnDefaultPawnAtTransform(NewPlayer, Transform);
}

APawn* AGameModeBase::SpawnDefaultPawnAtTransform_Implementation(AController* NewPlayer, const FTransform& SpawnTransform)
{
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Instigator = GetInstigator();
	SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save default player pawns into a map
	UClass* PawnClass = GetDefaultPawnClassForController(NewPlayer);
	APawn* ResultPawn = GetWorld()->SpawnActor<APawn>(PawnClass, SpawnTransform, SpawnInfo);
	if (!ResultPawn)
	{
		UE_LOGF(LogGameMode, Warning, "SpawnDefaultPawnAtTransform: Couldn't spawn Pawn of type %ls at %ls", *GetNameSafe(PawnClass), *SpawnTransform.ToHumanReadableString());
	}
	return ResultPawn;
}

/// @endcond

void AGameModeBase::RestartPlayer(AController* NewPlayer)
{
	if (NewPlayer == nullptr || NewPlayer->IsPendingKillPending())
	{
		return;
	}

	AActor* StartSpot = FindPlayerStart(NewPlayer);

	// If a start spot wasn't found,
	if (StartSpot == nullptr)
	{
		// Check for a previously assigned spot
		if (NewPlayer->StartSpot != nullptr)
		{
			StartSpot = NewPlayer->StartSpot.Get();
			UE_LOGF(LogGameMode, Warning, "RestartPlayer: Player start not found, using last start spot");
		}	
	}

	RestartPlayerAtPlayerStart(NewPlayer, StartSpot);
}

void AGameModeBase::RestartPlayerAtPlayerStart(AController* NewPlayer, AActor* StartSpot)
{
	if (NewPlayer == nullptr || NewPlayer->IsPendingKillPending())
	{
		return;
	}

	if (!StartSpot)
	{
		UE_LOGF(LogGameMode, Warning, "RestartPlayerAtPlayerStart: Player start not found");
		return;
	}

	FRotator SpawnRotation = StartSpot->GetActorRotation();

	UE_LOGF(LogGameMode, Verbose, "RestartPlayerAtPlayerStart %ls", (NewPlayer && NewPlayer->PlayerState) ? *NewPlayer->PlayerState->GetPlayerName() : TEXT("Unknown"));

	if (MustSpectate(Cast<APlayerController>(NewPlayer)))
	{
		UE_LOGF(LogGameMode, Verbose, "RestartPlayerAtPlayerStart: Tried to restart a spectator-only player!");
		return;
	}

	if (NewPlayer->GetPawn() != nullptr)
	{
		// If we have an existing pawn, just use it's rotation
		SpawnRotation = NewPlayer->GetPawn()->GetActorRotation();
	}
	else if (GetDefaultPawnClassForController(NewPlayer) != nullptr)
	{
		// Try to create a pawn to use of the default class for this player
		APawn* NewPawn = SpawnDefaultPawnFor(NewPlayer, StartSpot);
		if (IsValid(NewPawn))
		{
			NewPlayer->SetPawn(NewPawn);
		}
	}
	
	if (!IsValid(NewPlayer->GetPawn()))
	{
		FailedToRestartPlayer(NewPlayer);
	}
	else
	{
		// Tell the start spot it was used
		InitStartSpot(StartSpot, NewPlayer);

		FinishRestartPlayer(NewPlayer, SpawnRotation);
	}
}

void AGameModeBase::RestartPlayerAtTransform(AController* NewPlayer, const FTransform& SpawnTransform)
{
	if (NewPlayer == nullptr || NewPlayer->IsPendingKillPending())
	{
		return;
	}

	UE_LOGF(LogGameMode, Verbose, "RestartPlayerAtTransform %ls", (NewPlayer && NewPlayer->PlayerState) ? *NewPlayer->PlayerState->GetPlayerName() : TEXT("Unknown"));

	if (MustSpectate(Cast<APlayerController>(NewPlayer)))
	{
		UE_LOGF(LogGameMode, Verbose, "RestartPlayerAtTransform: Tried to restart a spectator-only player!");
		return;
	}

	FRotator SpawnRotation = SpawnTransform.GetRotation().Rotator();

	if (NewPlayer->GetPawn() != nullptr)
	{
		// If we have an existing pawn, just use it's rotation
		SpawnRotation = NewPlayer->GetPawn()->GetActorRotation();
	}
	else if (GetDefaultPawnClassForController(NewPlayer) != nullptr)
	{
		// Try to create a pawn to use of the default class for this player
		APawn* NewPawn = SpawnDefaultPawnAtTransform(NewPlayer, SpawnTransform);
		if (IsValid(NewPawn))
		{
			NewPlayer->SetPawn(NewPawn);
		}
	}

	if (!IsValid(NewPlayer->GetPawn()))
	{
		FailedToRestartPlayer(NewPlayer);
	}
	else
	{
		FinishRestartPlayer(NewPlayer, SpawnRotation);
	}
}

void AGameModeBase::FailedToRestartPlayer(AController* NewPlayer)
{
	NewPlayer->FailedToSpawnPawn();
}

void AGameModeBase::FinishRestartPlayer(AController* NewPlayer, const FRotator& StartRotation)
{
	NewPlayer->Possess(NewPlayer->GetPawn());

	// If the Pawn is destroyed as part of possession we have to abort
	if (!IsValid(NewPlayer->GetPawn()))
	{
		FailedToRestartPlayer(NewPlayer);
	}
	else
	{
		// Set initial control rotation to starting rotation rotation
		NewPlayer->ClientSetRotation(NewPlayer->GetPawn()->GetActorRotation(), true);

		FRotator NewControllerRot = StartRotation;
		NewControllerRot.Roll = 0.f;
		NewPlayer->SetControlRotation(NewControllerRot);

		SetPlayerDefaults(NewPlayer->GetPawn());

		K2_OnRestartPlayer(NewPlayer);
	}
}

/// @cond DOXYGEN_WARNINGS

void AGameModeBase::InitStartSpot_Implementation(AActor* StartSpot, AController* NewPlayer)
{

}

/// @endcond


void AGameModeBase::SetPlayerDefaults(APawn* PlayerPawn)
{
	PlayerPawn->SetPlayerDefaults();
}

void AGameModeBase::ChangeName(AController* Other, const FString& S, bool bNameChange)
{
	if (Other && !S.IsEmpty())
	{
		Other->PlayerState->SetPlayerName(S);

		K2_OnChangeName(Other, S, bNameChange);
	}
}

bool AGameModeBase::AllowCheats(APlayerController* P)
{
	return (GetNetMode() == NM_Standalone || GIsEditor); // Always allow cheats in editor (PIE now supports networking)
}

bool AGameModeBase::IsHandlingReplays()
{
	return false;
}

bool AGameModeBase::SpawnPlayerFromSimulate(const FVector& NewLocation, const FRotator& NewRotation)
{
#if WITH_EDITOR
	APlayerController* PC = GetGameInstance()->GetFirstLocalPlayerController();
	if (PC != nullptr)
	{
		PC->PlayerState->SetIsOnlyASpectator(false);

		bool bNeedsRestart = true;
		if (PC->GetPawn() == nullptr)
		{
			// Use the "auto-possess" pawn in the world, if there is one.
			for (APawn* Pawn : TActorRange<APawn>(GetWorld()))
			{
				if (Pawn->AutoPossessPlayer == EAutoReceiveInput::Player0)
				{
					if (Pawn->GetController() == nullptr)
					{
						PC->Possess(Pawn);
						bNeedsRestart = false;
					}
					break;
				}
			}
		}

		if (bNeedsRestart)
		{
			RestartPlayer(PC);

			if (PC->GetPawn())
			{
				// If there was no player start, then try to place the pawn where the camera was.						
				if (PC->StartSpot == nullptr || Cast<AWorldSettings>(PC->StartSpot.Get()))
				{
					const FVector Location = NewLocation;
					const FRotator Rotation = NewRotation;
					PC->SetControlRotation(Rotation);
					PC->GetPawn()->TeleportTo(Location, Rotation);
				}
			}
		}
	}
#endif
	return true;
}

