// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GameNetworkManager.cpp: AGameNetworkMAnager C++ code.
=============================================================================*/

#include "GameFramework/GameNetworkManager.h"
#include "EngineLogs.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/NetDriver.h"
#include "Engine/Player.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameNetworkManager)

DEFINE_LOG_CATEGORY_STATIC(LogGameNetworkManager, Log, All);


AGameNetworkManager::AGameNetworkManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BadPacketLossThreshold = 0.05f;
	SeverePacketLossThreshold = 0.15f;
	BadPingThreshold = 200;
	SeverePingThreshold = 500;

	MoveRepSize = 42.0f;
	MAXPOSITIONERRORSQUARED = 3.0f;
	MAXNEARZEROVELOCITYSQUARED = 9.0f;
	CLIENTADJUSTUPDATECOST = 180.0f;
	MAXCLIENTUPDATEINTERVAL = 0.25f;
	MaxClientForcedUpdateDuration=1.0f;
	ServerForcedUpdateHitchThreshold = 0.150f;
	ServerForcedUpdateHitchCooldown = 0.100f;
	MaxMoveDeltaTime = 0.125f;
	MaxClientSmoothingDeltaTime = 0.50f;
	ClientNetSendMoveDeltaTime = 0.0166f;
	ClientNetSendMoveDeltaTimeThrottled = 0.0222f;
	ClientNetSendMoveDeltaTimeStationary = 0.0166f;
	ClientNetSendMoveThrottleAtNetSpeed = 10000;
	ClientNetSendMoveThrottleOverPlayerCount = 10;
	ClientAuthorativePosition = false;
	ClientErrorUpdateRateLimit = 0.0f;
	ClientNetCamUpdateDeltaTime = 0.0166f;
	ClientNetCamUpdatePositionLimit = 1000.0f;
	bMovementTimeDiscrepancyDetection = false;
	bMovementTimeDiscrepancyResolution = false;
	MovementTimeDiscrepancyMaxTimeMargin = 0.25f;
	MovementTimeDiscrepancyMinTimeMargin = -0.25f;
	MovementTimeDiscrepancyResolutionRate = 1.0f;
	MovementTimeDiscrepancyDriftAllowance = 0.0f;
	bMovementTimeDiscrepancyForceCorrectionsDuringResolution = false;
	bUseDistanceBasedRelevancy = true;
}

void AGameNetworkManager::EnableStandbyCheatDetection(bool bIsEnabled)
{
	UNetDriver* Driver = GetWorld()->GetNetDriver();
	if (Driver)
	{
		// If it's being enabled set all of the vars
		if (bIsEnabled)
		{
			Driver->bHasStandbyCheatTriggered = false;
			Driver->StandbyRxCheatTime = StandbyRxCheatTime;
			Driver->StandbyTxCheatTime = StandbyTxCheatTime;
			Driver->BadPingThreshold = BadPingThreshold;
			Driver->PercentMissingForRxStandby = PercentMissingForRxStandby;
			Driver->PercentMissingForTxStandby = PercentMissingForTxStandby;
			Driver->PercentForBadPing = PercentForBadPing;
			Driver->JoinInProgressStandbyWaitTime = JoinInProgressStandbyWaitTime;
		}
		// Enable/disable based upon the passed in value and make sure the cheat time vars are valid
		Driver->bIsStandbyCheckingEnabled = bIsEnabled && StandbyRxCheatTime > 0.f;
		UE_LOGF(LogGameNetworkManager, Log, "Standby check is %ls with RxTime (%f), TxTime (%f), PingThreshold (%d), JoinInProgressStandbyWaitTime (%f)",
			Driver->bIsStandbyCheckingEnabled ? TEXT("enabled") : TEXT("disabled"),
			StandbyRxCheatTime,
			StandbyTxCheatTime,
			BadPingThreshold,
			JoinInProgressStandbyWaitTime);
	}
}

void AGameNetworkManager::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	AdjustedNetSpeed = MaxDynamicBandwidth;
}

void AGameNetworkManager::UpdateNetSpeedsTimer()
{
	UpdateNetSpeeds(false);
}

bool AGameNetworkManager::IsInLowBandwidthMode()
{
	return false;
}

void AGameNetworkManager::UpdateNetSpeeds(bool bIsLanMatch)
{
	// Don't adjust net speeds for LAN matches or dedicated servers
	ENetMode NetMode = GetNetMode();
	if ( (NetMode == NM_DedicatedServer) || (NetMode == NM_Standalone) || bIsLanMatch )
	{
		return;
	}

	if ( GetWorld()->TimeSeconds - LastNetSpeedUpdateTime < 1.0f )
	{
		GetWorldTimerManager().SetTimer(TimerHandle_UpdateNetSpeedsTimer, this, &AGameNetworkManager::UpdateNetSpeedsTimer, 1.0f);
		return;
	}

	LastNetSpeedUpdateTime = GetWorld()->TimeSeconds;

	int32 NewNetSpeed = CalculatedNetSpeed();
	UE_LOGF(LogNet, Log, "New Dynamic NetSpeed %i vs old %i", NewNetSpeed, AdjustedNetSpeed);

	if ( AdjustedNetSpeed != NewNetSpeed )
	{
		AdjustedNetSpeed = NewNetSpeed;
		for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			if (APlayerController* PC = Iterator->Get())
			{
				PC->SetNetSpeed(AdjustedNetSpeed);
			}
		}
	}
}

int32 AGameNetworkManager::CalculatedNetSpeed()
{
	int32 NumPlayers = 1;
	AGameModeBase* GameMode = GetWorld()->GetAuthGameMode();

	if (GameMode)
	{
		NumPlayers = FMath::Max(GameMode->GetNumPlayers(), 1);
	}

	return FMath::Clamp(TotalNetBandwidth/NumPlayers, MinDynamicBandwidth, MaxDynamicBandwidth);
}

void AGameNetworkManager::StandbyCheatDetected(EStandbyType StandbyType) {}

bool AGameNetworkManager::WithinUpdateDelayBounds(APlayerController* PC, float LastUpdateTime) const
{
	if (!PC || !PC->Player)
		return false;

	const float TimeSinceUpdate = PC->GetWorld()->GetTimeSeconds() - LastUpdateTime;
	if (ClientErrorUpdateRateLimit > 0.f && TimeSinceUpdate < ClientErrorUpdateRateLimit)
	{
		return true;
	}
	else if (TimeSinceUpdate < GetDefault<AGameNetworkManager>(GetClass())->CLIENTADJUSTUPDATECOST/PC->Player->CurrentNetSpeed)
	{
		return true;
	}
	return false;
}

bool AGameNetworkManager::ExceedsAllowablePositionError(FVector LocDiff) const
{
	return (LocDiff | LocDiff) > GetDefault<AGameNetworkManager>(GetClass())->MAXPOSITIONERRORSQUARED;
}

bool AGameNetworkManager::NetworkVelocityNearZero(FVector InVelocity) const
{
	return InVelocity.SizeSquared() < GetDefault<AGameNetworkManager>(GetClass())->MAXNEARZEROVELOCITYSQUARED;
}
