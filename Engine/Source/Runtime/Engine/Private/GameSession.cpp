// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GameSession.cpp: GameSession code.
=============================================================================*/

#include "GameFramework/GameSession.h"
#include "Engine/World.h"
#include "Misc/CommandLine.h"
#include "Engine/Engine.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/NetConnection.h"
#include "Net/OnlineEngineInterface.h"
#include "GameFramework/PlayerState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameSession)

DEFINE_LOG_CATEGORY(LogGameSession);

static TAutoConsoleVariable<int32> CVarMaxPlayersOverride( TEXT( "net.MaxPlayersOverride" ), 0, TEXT( "If greater than 0, will override the standard max players count. Useful for testing full servers." ) );

APlayerController* GetPlayerControllerFromNetId(UWorld* World, const FUniqueNetId& PlayerNetId)
{
	return GetPlayerControllerFromNetId(World, FUniqueNetIdRepl(PlayerNetId.AsShared()));
}

APlayerController* GetPlayerControllerFromNetId(UWorld* World, const FUniqueNetIdRepl& PlayerNetId)
{
	if (PlayerNetId.IsValid())
	{
		// Iterate through the controller list looking for the net id
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			// Determine if this is a player with replication
			if (PlayerController && PlayerController->PlayerState && PlayerController->PlayerState->GetUniqueId().IsValid())
			{
				// If the ids match, then this is the right player.
				if (PlayerController->PlayerState->GetUniqueId() == PlayerNetId)
				{
					return PlayerController;
				}
			}
		}
	}

	return nullptr;
}

AGameSession::AGameSession(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	MaxPartySize(INDEX_NONE)
{
}

void AGameSession::HandleMatchIsWaitingToStart()
{
}

void AGameSession::HandleMatchHasStarted()
{
	UWorld* World = GetWorld();
	if (UOnlineEngineInterface::Get()->DoesSessionExist(World, SessionName))
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && !PlayerController->IsLocalController())
			{
				PlayerController->ClientStartOnlineSession();
			}
		}

		FOnlineSessionStartComplete CompletionDelegate = FOnlineSessionStartComplete::CreateUObject(this, &AGameSession::OnStartSessionComplete);
		UOnlineEngineInterface::Get()->StartSession(World, SessionName, CompletionDelegate);
	}

	if (STATS && !UE_BUILD_SHIPPING)
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("MatchAutoStatCapture")))
		{
			UE_LOGF(LogGameSession, Log, "Match has started - begin automatic stat capture");
			GEngine->Exec(GetWorld(), TEXT("stat startfile"));
		}
	}
}

void AGameSession::OnStartSessionComplete(FName InSessionName, bool bWasSuccessful)
{
	UE_LOGF(LogGameSession, Verbose, "OnStartSessionComplete %ls bSuccess: %d", *InSessionName.ToString(), bWasSuccessful);
}

void AGameSession::HandleMatchHasEnded()
{
	if (STATS && !UE_BUILD_SHIPPING)
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("MatchAutoStatCapture")))
		{
			UE_LOGF(LogGameSession, Log, "Match has ended - end automatic stat capture");
			GEngine->Exec(GetWorld(), TEXT("stat stopfile"));
		}
	}

	UWorld* World = GetWorld();
	if (UOnlineEngineInterface::Get()->DoesSessionExist(World, SessionName))
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && !PlayerController->IsLocalController())
			{
				PlayerController->ClientEndOnlineSession();
			}
		}

		FOnlineSessionStartComplete CompletionDelegate = FOnlineSessionEndComplete::CreateUObject(this, &AGameSession::OnEndSessionComplete);
		UOnlineEngineInterface::Get()->EndSession(World, SessionName, CompletionDelegate);
	}
}

void AGameSession::OnEndSessionComplete(FName InSessionName, bool bWasSuccessful)
{
	UE_LOGF(LogGameSession, Verbose, "OnEndSessionComplete %ls bSuccess: %d", *InSessionName.ToString(), bWasSuccessful);
}

void AGameSession::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	if (!IsTemplate())
	{
		if (MaxPlayersOptionOverride.IsSet())
		{
			MaxPlayers = *MaxPlayersOptionOverride;
		}

		if (MaxSpectatorsOptionOverride.IsSet())
		{
			MaxSpectators = *MaxSpectatorsOptionOverride;
		}
	}
}

bool AGameSession::HandleStartMatchRequest()
{
	return false;
}

void AGameSession::InitOptions( const FString& Options )
{
	UWorld* const World = GetWorld();
	check(World);
	AGameModeBase* const GameMode = World ? World->GetAuthGameMode() : nullptr;

	if (UGameplayStatics::HasOption(Options, TEXT("MaxPlayers")))
	{
		MaxPlayers = UGameplayStatics::GetIntOption(Options, TEXT("MaxPlayers"), MaxPlayers);
		MaxPlayersOptionOverride = MaxPlayers;
	}

	if (UGameplayStatics::HasOption(Options, TEXT("MaxSpectators")))
	{
		MaxSpectators = UGameplayStatics::GetIntOption(Options, TEXT("MaxSpectators"), MaxSpectators);
		MaxSpectatorsOptionOverride = MaxSpectators;
	}
	
	if (GameMode)
	{
		UClass* PlayerStateClass = GameMode->PlayerStateClass;
		APlayerState const* const DefaultPlayerState = (PlayerStateClass ? GetDefault<APlayerState>(PlayerStateClass) : nullptr);
		if (DefaultPlayerState)
		{
			SessionName = DefaultPlayerState->SessionName;
		}
		else
		{
			UE_LOGF(LogGameSession, Error, "Player State class is invalid for game mode: %ls!", *GameMode->GetName());
		}
	}
}

bool AGameSession::ProcessAutoLogin()
{
	UWorld* World = GetWorld();

	FOnlineAutoLoginComplete CompletionDelegate = FOnlineAutoLoginComplete::CreateUObject(this, &ThisClass::OnAutoLoginComplete);
	if (UOnlineEngineInterface::Get()->AutoLogin(World, 0, CompletionDelegate))
	{
		// Async login started
		return true;
	}

	// Not waiting for async login
	return false;
}

void AGameSession::OnAutoLoginComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& Error)
{
	UWorld* World = GetWorld();
	if (UOnlineEngineInterface::Get()->IsLoggedIn(World, LocalUserNum))
	{
		RegisterServer();
	}
	else
	{
		RegisterServerFailed();
	}
}

void AGameSession::RegisterServer()
{
}

void AGameSession::RegisterServerFailed()
{
	UE_LOGF(LogGameSession, Warning, "Autologin attempt failed, unable to register server!");
}

FString AGameSession::ApproveLogin(const FString& Options)
{
	UWorld* const World = GetWorld();
	check(World);

	AGameModeBase* const GameMode = World->GetAuthGameMode();
	check(GameMode);

	int32 SpectatorOnly = 0;
	SpectatorOnly = UGameplayStatics::GetIntOption(Options, TEXT("SpectatorOnly"), SpectatorOnly);

	if (AtCapacity(SpectatorOnly == 1))
	{
		return TEXT( "Server full." );
	}

	int32 SplitscreenCount = 0;
	SplitscreenCount = UGameplayStatics::GetIntOption(Options, TEXT("SplitscreenCount"), SplitscreenCount);

	if (SplitscreenCount > MaxSplitscreensPerConnection)
	{
		UE_LOGF(LogGameSession, Warning, "ApproveLogin: A maximum of %i splitscreen players are allowed", MaxSplitscreensPerConnection);
		return TEXT("Maximum splitscreen players");
	}

	return TEXT("");
}

void AGameSession::PostLogin(APlayerController* NewPlayer)
{
}

int32 AGameSession::GetNextPlayerID()
{
	// Start at 256, because 255 is special (means all team for some UT Emote stuff)
	static constexpr int32 MinPlayerId = 256;
	static constexpr int32 MaxPlayerId = TNumericLimits<int32>::Max() - 1;
	
	static int32 NextPlayerID = MinPlayerId;
	
	// Prevent possible integer overflow by wrapping the value to the max player ID
	if (NextPlayerID >= MaxPlayerId)
	{
		UE_LOGF(LogGameSession, Warning, "AGameSession::GetNextPlayerID had to wrap the Player ID, this probably shouldn't have happened. PlayerID collisions may occur! Is this function being called incorrectly in a loop?");
		NextPlayerID = MinPlayerId;
	}
	
	return NextPlayerID++;
}

void AGameSession::RegisterPlayer(APlayerController* NewPlayer, const FUniqueNetIdRepl& UniqueId, bool bWasFromInvite)
{
	if (NewPlayer != NULL)
	{
		// Set the player's ID.
		check(NewPlayer->PlayerState);
		NewPlayer->PlayerState->SetPlayerId(GetNextPlayerID());
		NewPlayer->PlayerState->SetUniqueId(UniqueId);
		NewPlayer->PlayerState->RegisterPlayerWithSession(bWasFromInvite);
	}
}

void AGameSession::UnregisterPlayer(FName InSessionName, const FUniqueNetIdRepl& UniqueId)
{
	UWorld* World = GetWorld();
	if (GetNetMode() != NM_Standalone &&
		UniqueId.IsValid() &&
		UOnlineEngineInterface::Get()->DoesSessionExist(World, InSessionName))
	{
		// Remove the player from the session
		UOnlineEngineInterface::Get()->UnregisterPlayer(World, InSessionName, UniqueId);
	}
}

void AGameSession::UnregisterPlayers(FName InSessionName, const TArray<FUniqueNetIdRepl>& Players)
{
	UWorld * World = GetWorld();
	if (GetNetMode() != NM_Standalone &&
		Players.Num() > 0 &&
		UOnlineEngineInterface::Get()->DoesSessionExist(World, InSessionName))
	{
		// Remove the player from the session
		TArray<FUniqueNetIdWrapper> PlayerIdsAsWrappers;
		for (const FUniqueNetIdRepl& PlayerId : Players)
		{
			PlayerIdsAsWrappers.Emplace(PlayerId);
		}
		UOnlineEngineInterface::Get()->UnregisterPlayers(World, InSessionName, PlayerIdsAsWrappers);
	}
}

void AGameSession::UnregisterPlayer(const APlayerController* ExitingPlayer)
{
	if (GetNetMode() != NM_Standalone &&
		ExitingPlayer != NULL &&
		ExitingPlayer->PlayerState &&
		ExitingPlayer->PlayerState->GetUniqueId().IsValid())
	{
		UnregisterPlayer(ExitingPlayer->PlayerState->SessionName, ExitingPlayer->PlayerState->GetUniqueId());
	}
}

bool AGameSession::AtCapacity(bool bSpectator)
{
	if ( GetNetMode() == NM_Standalone )
	{
		return false;
	}

	AGameModeBase* GameMode = GetWorld()->GetAuthGameMode();

	if ( bSpectator )
	{
		return ( (GameMode->GetNumSpectators() >= MaxSpectators)
		&& ((GetNetMode() != NM_ListenServer) || (GameMode->GetNumPlayers() > 0)) );
	}
	else
	{
		const int32 MaxPlayersToUse = CVarMaxPlayersOverride.GetValueOnGameThread() > 0 ? CVarMaxPlayersOverride.GetValueOnGameThread() : MaxPlayers;

		return ( (MaxPlayersToUse>0) && (GameMode->GetNumPlayers() >= MaxPlayersToUse) );
	}
}

void AGameSession::NotifyLogout(FName InSessionName, const FUniqueNetIdRepl& UniqueId)
{
	// Unregister the player from the online layer
	UnregisterPlayer(InSessionName, UniqueId);
}

void AGameSession::NotifyLogout(const APlayerController* PC)
{
	// Unregister the player from the online layer
	UnregisterPlayer(PC);
}

void AGameSession::AddAdmin(APlayerController* AdminPlayer)
{
}

void AGameSession::RemoveAdmin(APlayerController* AdminPlayer)
{
}

bool AGameSession::KickPlayer(APlayerController* KickedPlayer, const FText& KickReason)
{
	// Do not kick logged admins
	if (KickedPlayer != NULL && Cast<UNetConnection>(KickedPlayer->Player) != NULL)
	{
		if (KickedPlayer->GetPawn() != NULL)
		{
			KickedPlayer->GetPawn()->Destroy();
		}

		KickedPlayer->ClientWasKicked(KickReason);

		if (KickedPlayer != NULL)
		{
			KickedPlayer->Destroy();
		}

		return true;
	}
	return false;
}

bool AGameSession::BanPlayer(class APlayerController* BannedPlayer, const FText& BanReason)
{
	return KickPlayer(BannedPlayer, BanReason);
}

void AGameSession::ReturnToMainMenuHost()
{
	const FText RemoteReturnReason = NSLOCTEXT("NetworkErrors", "HostHasLeft", "Host has left the game.");
	const FText LocalReturnReason = FText::GetEmpty();

	APlayerController* Controller = NULL;
	FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator();
	for(; Iterator; ++Iterator)
	{
		Controller = Iterator->Get();
		if (Controller && !Controller->IsLocalPlayerController() && Controller->IsPrimaryPlayer())
		{
			// Clients
			Controller->ClientReturnToMainMenuWithTextReason(RemoteReturnReason);
		}
	}

	Iterator.Reset();
	for(; Iterator; ++Iterator)
	{
		Controller = Iterator->Get();
		if (Controller && Controller->IsLocalPlayerController() && Controller->IsPrimaryPlayer())
		{
			Controller->ClientReturnToMainMenuWithTextReason(LocalReturnReason);
			break;
		}
	}
}

void AGameSession::PostSeamlessTravel()
{
}

void AGameSession::DumpSessionState()
{
	UE_LOGF(LogGameSession, Log, "  MaxPlayers: %i", MaxPlayers);
	UE_LOGF(LogGameSession, Log, "  MaxSpectators: %i", MaxSpectators);

	UOnlineEngineInterface::Get()->DumpSessionState(GetWorld());
}

bool AGameSession::CanRestartGame()
{
	return true;
}

bool AGameSession::GetSessionJoinability(FName InSessionName, FJoinabilitySettings& OutSettings)
{
	UWorld* const World = GetWorld();
	check(World);

	OutSettings.MaxPlayers = MaxPlayers;
	OutSettings.MaxPartySize = MaxPartySize;
	return UOnlineEngineInterface::Get()->GetSessionJoinability(World, InSessionName, OutSettings);
}

void AGameSession::UpdateSessionJoinability(FName InSessionName, bool bPublicSearchable, bool bAllowInvites, bool bJoinViaPresence, bool bJoinViaPresenceFriendsOnly)
{
	if (GetNetMode() != NM_Standalone)
	{
		UOnlineEngineInterface::Get()->UpdateSessionJoinability(GetWorld(), InSessionName, bPublicSearchable, bAllowInvites, bJoinViaPresence, bJoinViaPresenceFriendsOnly);
	}
}

