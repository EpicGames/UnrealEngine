// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GameState.cpp: GameState C++ code.
=============================================================================*/

#include "GameFramework/GameState.h"
#include "Async/TaskGraphInterfaces.h"
#include "TimerManager.h"
#include "GameFramework/GameMode.h"
#include "GameFramework/WorldSettings.h"
#include "Net/UnrealNetwork.h"
#include "Misc/CoreDelegates.h"
#include "MoviePlayerProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameState)

AGameState::AGameState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MatchState = MatchState::EnteringMap;
	PreviousMatchState = MatchState::EnteringMap;
}

void AGameState::ReceivedGameModeClass()
{
	Super::ReceivedGameModeClass();

	if (!GameModeClass->IsChildOf<AGameMode>())
	{
		UE_LOGF(LogGameState, Error, "Mixing AGameState with AGameModeBase is not compatible. Change AGameModeBase subclass (%ls) to derive from AGameMode, or make both derive from Base", *GameModeClass->GetName());
	}
}

void AGameState::DefaultTimer()
{
	if (IsMatchInProgress())
	{
		++ElapsedTime;
		if (GetNetMode() != NM_DedicatedServer)
		{
			OnRep_ElapsedTime();
		}
	}

	GetWorldTimerManager().SetTimer(TimerHandle_DefaultTimer, this, &AGameState::DefaultTimer, GetWorldSettings()->GetEffectiveTimeDilation() / GetWorldSettings()->DemoPlayTimeDilation, true);
}

void AGameState::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	FTimerManager& TimerManager = GetWorldTimerManager();
	TimerManager.SetTimer(TimerHandle_DefaultTimer, this, &AGameState::DefaultTimer, GetWorldSettings()->GetEffectiveTimeDilation() / GetWorldSettings()->DemoPlayTimeDilation, true);
}

void AGameState::HandleMatchIsWaitingToStart()
{
	if (GetLocalRole() != ROLE_Authority)
	{
		FMoviePlayerProxyBlock MoviePlayerBlock;
		// Server handles this in AGameMode::HandleMatchIsWaitingToStart
		GetWorldSettings()->NotifyBeginPlay();
	}

	FCoreDelegates::GameStateClassChanged.Broadcast(GetClass()->GetName());
}

void AGameState::HandleMatchHasStarted()
{
	if (GetLocalRole() != ROLE_Authority)
	{
		// Server handles this in AGameMode::HandleMatchHasStarted
		GetWorldSettings()->NotifyMatchStarted();
	}
	else
	{
		// Now that match has started, act like the base class and set replicated flag
		bReplicatedHasBegunPlay = true;
	}
}

void AGameState::HandleMatchHasEnded()
{

}

void AGameState::HandleLeavingMap()
{

}

bool AGameState::HasMatchStarted() const
{
	if (GetMatchState() == MatchState::EnteringMap || GetMatchState() == MatchState::WaitingToStart)
	{
		return false;
	}

	return true;
}

void AGameState::HandleBeginPlay()
{
	// Overridden to not do anything, the state machine tells world when to start
}

bool AGameState::IsMatchInProgress() const
{
	if (GetMatchState() == MatchState::InProgress)
	{
		return true;
	}

	return false;
}

bool AGameState::HasMatchEnded() const
{
	if (GetMatchState() == MatchState::WaitingPostMatch || GetMatchState() == MatchState::LeavingMap)
	{
		return true;
	}

	return false;
}

void AGameState::SetMatchState(FName NewState)
{
	if (GetLocalRole() == ROLE_Authority)
	{
		UE_LOGF(LogGameState, Log, "Match State Changed from %ls to %ls", *MatchState.ToString(), *NewState.ToString());

		MatchState = NewState;

		// Call the onrep to make sure the callbacks happen
		OnRep_MatchState();
	}
}

void AGameState::OnRep_MatchState()
{
	if (MatchState == MatchState::WaitingToStart || PreviousMatchState == MatchState::EnteringMap)
	{
		// Call MatchIsWaiting to start even if you join in progress at a later state
		HandleMatchIsWaitingToStart();
	}
	
	if (MatchState == MatchState::InProgress)
	{
		HandleMatchHasStarted();
	}
	else if (MatchState == MatchState::WaitingPostMatch)
	{
		HandleMatchHasEnded();
	}
	else if (MatchState == MatchState::LeavingMap)
	{
		HandleLeavingMap();
	}

	PreviousMatchState = MatchState;
}

void AGameState::OnRep_ElapsedTime()
{
	// Blank on purpose
}

float AGameState::GetPlayerStartTime(class AController* Controller) const
{
	return ElapsedTime;
}

float AGameState::GetPlayerRespawnDelay(class AController* Controller) const
{
	const AGameMode* GameMode = GetDefaultGameMode<AGameMode>();

	if (GameMode)
	{
		return GameMode->MinRespawnDelay;
	}

	return Super::GetPlayerRespawnDelay(Controller);
}

/// @cond DOXYGEN_WARNINGS

void AGameState::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DOREPLIFETIME( AGameState, MatchState );
	DOREPLIFETIME_CONDITION( AGameState, ElapsedTime,	COND_InitialOnly );
}

/// @endcond

