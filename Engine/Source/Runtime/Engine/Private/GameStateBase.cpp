// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GameState.cpp: GameState C++ code.
=============================================================================*/

#include "GameFramework/GameStateBase.h"
#include "TimerManager.h"
#include "Net/UnrealNetwork.h"
#include "GameFramework/PlayerState.h"
#include "EngineUtils.h"
#include "Engine/DemoNetDriver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameStateBase)

DEFINE_LOG_CATEGORY(LogGameState);

DEFINE_STAT(STAT_GetPlayerStateFromUniqueId);

AGameStateBase::AGameStateBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
		.DoNotCreateDefaultSubobject(TEXT("Sprite")) )
{
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	bAlwaysRelevant = true;
	SetReplicatingMovement(false);

	// Note: this is very important to set to false. Though all replication infos are spawned at run time, during seamless travel
	// they are held on to and brought over into the new world. In ULevel::InitializeNetworkActors, these PlayerStates may be treated as map/startup actors
	// and given static NetGUIDs. This also causes their deletions to be recorded and sent to new clients, which if unlucky due to name conflicts,
	// may end up deleting the new PlayerStates they had just spawned.
	bNetLoadOnClient = false;

	// Default to every 100 ms
	ServerWorldTimeSecondsUpdateFrequency = 0.1f;

	SumServerWorldTimeSecondsDelta = 0.0;
	NumServerWorldTimeSecondsDeltas = 0;

	NetPriority = 10.f;	// We need to prioritize updates to ensure that game state transitions reliably occur on the clients.
}

const AGameModeBase* AGameStateBase::GetDefaultGameMode() const
{
	if ( GameModeClass )
	{
		AGameModeBase* const DefaultGameActor = GameModeClass->GetDefaultObject<AGameModeBase>();
		return DefaultGameActor;
	}
	return nullptr;
}

void AGameStateBase::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	UWorld* World = GetWorld();
	World->SetGameState(this);

	FTimerManager& TimerManager = GetWorldTimerManager();
	if (World->IsGameWorld() && GetLocalRole() == ROLE_Authority)
	{
		UpdateServerTimeSeconds();
		if (ServerWorldTimeSecondsUpdateFrequency > 0.f)
		{
			TimerManager.SetTimer(TimerHandle_UpdateServerTimeSeconds, this, &AGameStateBase::UpdateServerTimeSeconds, ServerWorldTimeSecondsUpdateFrequency, true);
		}
	}

	for (TActorIterator<APlayerState> It(World); It; ++It)
	{
		AddPlayerState(*It);
	}
}

void AGameStateBase::OnRep_GameModeClass()
{
	ReceivedGameModeClass();
}

void AGameStateBase::OnRep_SpectatorClass()
{
	ReceivedSpectatorClass();
}

void AGameStateBase::ReceivedGameModeClass()
{
	// Tell each PlayerController that the Game class is here
	for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		APlayerController* const PlayerController = Iterator->Get();
		if (PlayerController)
		{
			PlayerController->ReceivedGameModeClass(GameModeClass);
		}
	}
}

void AGameStateBase::ReceivedSpectatorClass()
{
	// Tell each PlayerController that the Spectator class is here
	for (FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		APlayerController* const PlayerController = Iterator->Get();
		if (PlayerController && PlayerController->IsLocalController())
		{
			PlayerController->ReceivedSpectatorClass(SpectatorClass);
		}
	}
}

void AGameStateBase::SeamlessTravelTransitionCheckpoint(bool bToTransitionMap)
{
	// mark all existing player states as from previous level for various bookkeeping
	for (int32 i = 0; i < PlayerArray.Num(); i++)
	{
		PlayerArray[i]->SetIsFromPreviousLevel(true);
	}
}

void AGameStateBase::AddPlayerState(APlayerState* PlayerState)
{
	// Determine whether it should go in the active or inactive list
	if (!PlayerState->IsInactive())
	{
		// make sure no duplicates
		PlayerArray.AddUnique(PlayerState);
	}
}

void AGameStateBase::RemovePlayerState(APlayerState* PlayerState)
{
	for (int32 i=0; i<PlayerArray.Num(); i++)
	{
		if (PlayerArray[i] == PlayerState)
		{
			PlayerArray.RemoveAt(i,1);
			return;
		}
	}
}

double AGameStateBase::GetServerWorldTimeSeconds() const
{
	UWorld* World = GetWorld();
	if (World)
	{
		return World->GetTimeSeconds() + ServerWorldTimeSecondsDelta;
	}

	return 0.;
}

void AGameStateBase::UpdateServerTimeSeconds()
{
	UWorld* World = GetWorld();
	if (World)
	{
		ReplicatedWorldTimeSecondsDouble = World->GetTimeSeconds();
	}
}

void AGameStateBase::OnRep_ReplicatedWorldTimeSecondsDouble()
{
	UWorld* World = GetWorld();
	if (World)
	{
		const double ServerWorldTimeDelta = ReplicatedWorldTimeSecondsDouble - World->GetTimeSeconds();

		// Accumulate the computed server world delta
		SumServerWorldTimeSecondsDelta += ServerWorldTimeDelta;
		NumServerWorldTimeSecondsDeltas += 1;

		// Reset the accumulated values to ensure that we remain representative of the current delta
		if (NumServerWorldTimeSecondsDeltas > 250)
		{
			SumServerWorldTimeSecondsDelta /= NumServerWorldTimeSecondsDeltas;
			NumServerWorldTimeSecondsDeltas = 1;
		}

		double TargetWorldTimeSecondsDelta = SumServerWorldTimeSecondsDelta / NumServerWorldTimeSecondsDeltas;

		// Smoothly interpolate towards the new delta if we've already got one to avoid significant spikes
		if (ServerWorldTimeSecondsDelta == 0.0)
		{
			ServerWorldTimeSecondsDelta = TargetWorldTimeSecondsDelta;
		}
		else
		{
			ServerWorldTimeSecondsDelta += (TargetWorldTimeSecondsDelta - ServerWorldTimeSecondsDelta) * 0.5;
		}
	}
}

void AGameStateBase::OnRep_ReplicatedHasBegunPlay()
{
	if (bReplicatedHasBegunPlay && GetLocalRole() != ROLE_Authority)
	{
		GetWorldSettings()->NotifyBeginPlay();
		GetWorldSettings()->NotifyMatchStarted();
	}
}

void AGameStateBase::HandleBeginPlay()
{
	bReplicatedHasBegunPlay = true;

	GetWorldSettings()->NotifyBeginPlay();
	GetWorldSettings()->NotifyMatchStarted();
}

bool AGameStateBase::HasBegunPlay() const
{
	UWorld* World = GetWorld();
	if (World)
	{
		return World->GetBegunPlay();
	}

	return false;
}

bool AGameStateBase::HasMatchStarted() const
{
	UWorld* World = GetWorld();
	if (World)
	{
		return World->bMatchStarted;
	}

	return false;
}

bool AGameStateBase::HasMatchEnded() const
{
	return false;
}

float AGameStateBase::GetPlayerStartTime(AController* Controller) const
{
	return GetServerWorldTimeSeconds();
}

float AGameStateBase::GetPlayerRespawnDelay(AController* Controller) const
{
	return 1.0f;
}

void AGameStateBase::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DOREPLIFETIME( AGameStateBase, SpectatorClass );

	DOREPLIFETIME_CONDITION( AGameStateBase, GameModeClass,	COND_InitialOnly );
	DOREPLIFETIME( AGameStateBase, ReplicatedWorldTimeSecondsDouble );

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;

	DOREPLIFETIME( AGameStateBase, bReplicatedHasBegunPlay );
}

APlayerState* AGameStateBase::GetPlayerStateFromUniqueNetId(const FUniqueNetIdWrapper& InPlayerId) const
{
	SCOPE_CYCLE_COUNTER(STAT_GetPlayerStateFromUniqueId);
	const TArray<APlayerState*>& Players = PlayerArray;
	for (APlayerState* Player : Players)
	{
		if (Player && Player->GetUniqueId() == InPlayerId)
		{
			return Player;
		}
	}

	return nullptr;
}

