// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/GameUserSettings.h"
#include "GenericPlatform/GenericPlatformFramePacer.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/App.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/GameViewportClient.h"
#include "UnrealEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/GameEngine.h"
#include "Sound/SoundCue.h"
#include "AudioDevice.h"
#include "HAL/PlatformFramePacer.h"
#include "HDRHelper.h"
#include "UnrealClient.h"
#include "ComponentRecreateRenderStateContext.h"
#include "RenderUtils.h"

// For sandboxing migration of UserSettings from AppData\Local to AppData\LocalLow.
#if PLATFORM_WINDOWS
#include "Windows/WindowsPlatformProcess.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameUserSettings)

DEFINE_LOG_CATEGORY_STATIC(LogGameUserSettings, Log, All);

extern EWindowMode::Type GetWindowModeType(EWindowMode::Type WindowMode);

static TAutoConsoleVariable<int32> CVar_GameUserSettings_WindowPositionsMaxSize(
	TEXT("GameUserSettings.WindowPositionsMaxSize"),
	32,
	TEXT("Maximum number of saved window positions for the -SaveWinPos command-line option."));

enum EGameUserSettingsVersion
{
	/** Version for user game settings. All settings will be wiped if the serialized version differs. */
	UE_GAMEUSERSETTINGS_VERSION = 5
};


UGameUserSettings::UGameUserSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// this will only call the base version of SetToDefaults but some constructors may rely on it being set
	SetToDefaults();
}

FIntPoint UGameUserSettings::GetScreenResolution() const
{
	return FIntPoint(ResolutionSizeX, ResolutionSizeY);
}

FIntPoint UGameUserSettings::GetLastConfirmedScreenResolution() const
{
	return FIntPoint(LastUserConfirmedResolutionSizeX, LastUserConfirmedResolutionSizeY);
}

FIntPoint UGameUserSettings::GetDesktopResolution() const
{
	FDisplayMetrics DisplayMetrics;
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);
	}
	else
	{
		if (FApp::CanEverRender())
		{
			FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		}
		else
		{
			// assume headless
			DisplayMetrics.PrimaryDisplayWidth = 0;
			DisplayMetrics.PrimaryDisplayHeight = 0;
		}
	}

	FIntPoint Resolution = FIntPoint(DisplayMetrics.PrimaryDisplayWidth, DisplayMetrics.PrimaryDisplayHeight);
	const int32 MonitorIndex = DisplayMetrics.GetClosestMonitorFromIDAndIndex(DisplayID, DisplayIndex);
	if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndex))
	{
		const FPlatformRect& DisplayRect = DisplayMetrics.MonitorInfo[MonitorIndex].DisplayRect;
		Resolution.X = DisplayRect.Right - DisplayRect.Left;
		Resolution.Y = DisplayRect.Bottom - DisplayRect.Top;
	}
	return Resolution;
}

void UGameUserSettings::SetScreenResolution(FIntPoint Resolution)
{
	if (ResolutionSizeX != Resolution.X || ResolutionSizeY != Resolution.Y)
	{
		ResolutionSizeX = Resolution.X;
		ResolutionSizeY = Resolution.Y;
		UpdateResolutionQuality();
	}
}

static EWindowMode::Type GetPlatformFullscreenMode(int InFullscreenMode)
{
	EWindowMode::Type Mode = EWindowMode::ConvertIntToWindowMode(InFullscreenMode);

	return (!FPlatformProperties::SupportsWindowedMode()) ? EWindowMode::Fullscreen : Mode;
}

EWindowMode::Type UGameUserSettings::GetFullscreenMode() const
{
	return GetPlatformFullscreenMode(FullscreenMode);
}

EWindowMode::Type UGameUserSettings::GetLastConfirmedFullscreenMode() const
{
	return GetPlatformFullscreenMode(LastConfirmedFullscreenMode);
}

void UGameUserSettings::SetFullscreenMode(EWindowMode::Type InFullscreenMode)
{
	if (FPlatformProperties::HasFixedResolution())
	{
		return;
	}

	if (FullscreenMode != InFullscreenMode)
	{
		switch (InFullscreenMode)
		{
		case EWindowMode::Fullscreen:
			FullscreenMode = 0;
			break;
		case EWindowMode::WindowedFullscreen:
			FullscreenMode = 1;
			break;
		case EWindowMode::Windowed:
		default:
			FullscreenMode = 2;
			break;
		}

		UpdateResolutionQuality();
	}
}

EWindowMode::Type UGameUserSettings::GetPreferredFullscreenMode() const
{
	return PreferredFullscreenMode == 0 ? EWindowMode::Fullscreen : EWindowMode::WindowedFullscreen;
}

FString UGameUserSettings::GetDisplayID() const
{
	return DisplayID;
}

FString UGameUserSettings::GetLastConfirmedDisplayID() const
{
	return LastUserConfirmedDisplayID;
}

int32 UGameUserSettings::GetDisplayIndex() const
{
	return DisplayIndex;
}

int32 UGameUserSettings::GetLastConfirmedDisplayIndex() const
{
	return LastUserConfirmedDisplayIndex;
}

static int32 GetClosestMonitorFromIDAndIndex(const FString& DisplayID, int32 DisplayIndex)
{
	int32 MonitorIndex = -1;

	if (!DisplayID.IsEmpty() || (DisplayIndex != -1))
	{
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		MonitorIndex = DisplayMetrics.GetClosestMonitorFromIDAndIndex(DisplayID, DisplayIndex);
	}

	return MonitorIndex;
}

int32 UGameUserSettings::GetClosestMonitorIndex() const
{
	return GetClosestMonitorFromIDAndIndex(DisplayID, DisplayIndex);
}

void UGameUserSettings::SetDisplayProperties(const FString& ID, int32 Index)
{
	DisplayID = ID;
	DisplayIndex = Index;
}

void UGameUserSettings::SetVSyncEnabled(bool bEnable)
{
	bUseVSync = bEnable;
}

bool UGameUserSettings::IsVSyncEnabled() const
{
	return bUseVSync;
}

void UGameUserSettings::SetDynamicResolutionEnabled(bool bEnable)
{
	bUseDynamicResolution = bEnable;
}

bool UGameUserSettings::IsDynamicResolutionEnabled() const
{
	return bUseDynamicResolution;
}

bool UGameUserSettings::IsScreenResolutionDirty() const
{
	bool bIsDirty = false;
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		bIsDirty = (ResolutionSizeX != GSystemResolution.ResX || ResolutionSizeY != GSystemResolution.ResY) ? true : false;
	}
	return bIsDirty;
}

bool UGameUserSettings::IsFullscreenModeDirty() const
{
	bool bIsDirty = false;
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		EWindowMode::Type CurrentFullscreenMode = GEngine->GameViewport->Viewport->GetWindowMode();
		EWindowMode::Type NewFullscreenMode = GetFullscreenMode();
		bIsDirty = (CurrentFullscreenMode != NewFullscreenMode);
	}
	return bIsDirty;
}

bool UGameUserSettings::IsVSyncDirty() const
{
	bool bIsDirty = false;
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
		bIsDirty = (bUseVSync != (CVar->GetValueOnAnyThread() != 0));
	}
	return bIsDirty;
}

bool UGameUserSettings::IsDynamicResolutionDirty() const
{
	bool bIsDirty = false;
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		bIsDirty = (bUseDynamicResolution != GEngine->GetDynamicResolutionUserSetting());
	}
	return bIsDirty;
}

bool UGameUserSettings::IsDirty() const
{
	return IsScreenResolutionDirty() || IsFullscreenModeDirty() || IsVSyncDirty() || IsDynamicResolutionDirty();
}

void UGameUserSettings::ConfirmVideoMode()
{
	LastConfirmedFullscreenMode = FullscreenMode;
	LastUserConfirmedResolutionSizeX = ResolutionSizeX;
	LastUserConfirmedResolutionSizeY = ResolutionSizeY;
	LastUserConfirmedDisplayID = DisplayID;
	LastUserConfirmedDisplayIndex = DisplayIndex;
}

void UGameUserSettings::RevertVideoMode()
{
	FullscreenMode = LastConfirmedFullscreenMode;
	ResolutionSizeX = LastUserConfirmedResolutionSizeX;
	ResolutionSizeY = LastUserConfirmedResolutionSizeY;
	DisplayID = LastUserConfirmedDisplayID;
	DisplayIndex = LastUserConfirmedDisplayIndex;

	OnGameUserSettingsVideoRevert.Broadcast();
}

void UGameUserSettings::SetToDefaults()
{
	ResolutionSizeX = LastUserConfirmedResolutionSizeX = GetDefaultResolution().X;
	ResolutionSizeY = LastUserConfirmedResolutionSizeY = GetDefaultResolution().Y;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WindowPosX_DEPRECATED = GetDefaultWindowPosition().X;
	WindowPosY_DEPRECATED = GetDefaultWindowPosition().Y;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	WindowPositions.Reset();
	FullscreenMode = GetDefaultWindowMode();
	FrameRateLimit = 0.0f;
	DynamicResolutionFrameTarget = 0.0f;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MinResolutionScale = Scalability::MinResolutionScale;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	DesiredScreenWidth = 1280;
	DesiredScreenHeight = 720;
	LastUserConfirmedDesiredScreenWidth = DesiredScreenWidth;
	LastUserConfirmedDesiredScreenHeight = DesiredScreenHeight;
	LastCPUBenchmarkResult = -1.0f;
	LastGPUBenchmarkResult = -1.0f;
	LastCPUBenchmarkSteps.Empty();
	LastGPUBenchmarkSteps.Empty();
	LastGPUBenchmarkMultiplier = 1.0f;
	LastRecommendedScreenWidth = -1.0f;
	LastRecommendedScreenHeight = -1.0f;

	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FullScreenMode"));
	PreferredFullscreenMode = CVar->GetValueOnGameThread();

	DisplayID.Empty();
	DisplayIndex = -1;
	LastUserConfirmedDisplayID = DisplayID;
	LastUserConfirmedDisplayIndex = DisplayIndex;

	ScalabilityQuality.SetDefaults();

	if (FApp::CanEverRender())
	{
		UpdateResolutionQuality();
	}

	bUseDynamicResolution = false;
	bUseHDRDisplayOutput = FPlatformMisc::UseHDRByDefault();
	HDRDisplayOutputNits = 1000;
	HDRPaperWhiteNits = 203.f; // HDR Reference White in BT.2408
	HDRUILuminanceNits = 203.f; // Graphics White in BT.2408
	bIsHDRUILuminanceSeparate = false;
}

bool UGameUserSettings::IsVersionValid()
{
	return (Version == UE_GAMEUSERSETTINGS_VERSION);
}

void UGameUserSettings::UpdateVersion()
{
	Version = UE_GAMEUSERSETTINGS_VERSION;
}

void UGameUserSettings::UpdateResolutionQuality()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MinResolutionScale = Scalability::MinResolutionScale;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (bUseDesiredScreenHeight)
	{
		ScalabilityQuality.ResolutionQuality = GetDefaultResolutionScale();
	}
	else
	{
		ScalabilityQuality.ResolutionQuality = FMath::Max(ScalabilityQuality.ResolutionQuality, Scalability::MinResolutionScale);
	}
}

float UGameUserSettings::GetDefaultResolutionScale()
{
	const int32 ScreenWidth = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution().X : ResolutionSizeX;
	const int32 ScreenHeight = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution().Y : ResolutionSizeY;
	const int32 ClampedWidth = (ScreenWidth > 0 && DesiredScreenWidth > ScreenWidth) ? ScreenWidth : DesiredScreenWidth;
	const int32 ClampedHeight = (ScreenHeight > 0 && DesiredScreenHeight > ScreenHeight) ? ScreenHeight : DesiredScreenHeight;

	const float DesiredResQuality = FindResolutionQualityForScreenSize(ClampedWidth, ClampedHeight);
	return FMath::Max(DesiredResQuality, Scalability::MinResolutionScale);
}

float UGameUserSettings::GetRecommendedResolutionScale()
{
	const float RecommendedResQuality = FindResolutionQualityForScreenSize(LastRecommendedScreenWidth, LastRecommendedScreenHeight);

	return FMath::Max(RecommendedResQuality, Scalability::MinResolutionScale);
}

float UGameUserSettings::FindResolutionQualityForScreenSize(float Width, float Height)
{
	float ResolutionQuality = 100.0f;

	const FIntPoint ScreenSize = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution() : FIntPoint(ResolutionSizeX, ResolutionSizeY);
	const float ScreenAspectRatio = (float)ScreenSize.X / (float)ScreenSize.Y;
	const float AspectRatio = Width / Height;

	// If the screen aspect ratio is different than the target resolution's aspect ratio, we need to adjust the target width and height
	if (!FMath::IsNearlyEqual(ScreenAspectRatio, AspectRatio))
	{
		if (ScreenAspectRatio < AspectRatio)
		{
			// For smaller aspect ratios we allow more vertical space so that the screen width matches the width of the original mode
			Height = (Height * AspectRatio) / ScreenAspectRatio;
		}
		else
		{
			// For wider screens we try to choose a screen size that'd have similar total number of pixels as the original mode
			Height = FMath::Sqrt((Height * Height * AspectRatio) / ScreenAspectRatio);
		}
		Width = Height * AspectRatio;
	}

	if (Height < ScreenSize.Y)
	{
		ResolutionQuality = ((float)Height / (float)ScreenSize.Y) * 100.0f;
	}

	return ResolutionQuality;
}

void UGameUserSettings::SetFrameRateLimitCVar(float InLimit)
{
	GEngine->SetMaxFPS(FMath::Max(InLimit, 0.0f));
}

void UGameUserSettings::SetSyncTypeCVar(int32 InType)
{
	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GTSyncType"));
	if (ensure(SyncIntervalCVar))
	{
		SyncIntervalCVar->Set(InType, ECVF_SetByCode);
	}
}

float UGameUserSettings::GetEffectiveFrameRateLimit()
{
	return FrameRateLimit;
}

void UGameUserSettings::SetPreferredFullscreenMode(int32 Mode)
{
	PreferredFullscreenMode = Mode;

	auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.FullScreenMode"));
	if (CVar)
	{
		CVar->Set(Mode, ECVF_SetByGameSetting);
	}
}

void UGameUserSettings::ValidateSettings()
{
	// Should we wipe all user settings?
	if (!IsVersionValid())
	{
		// First try loading the settings, if they haven't been loaded before.
		LoadSettings(true);

		// If it still an old version, delete the user settings file and reload defaults.
		if (!IsVersionValid())
		{
			// Force reset if there aren't any default .ini settings.
			SetToDefaults();
			static const auto CVarVSync = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
			SetVSyncEnabled(CVarVSync->GetValueOnGameThread() != 0);

			if (GEngine)
			{
				SetDynamicResolutionEnabled(GEngine->GetDynamicResolutionUserSetting());
			}

			FConfigBranch* Branch = GConfig->FindBranch(FName("GameUserSettings"), FString());
			Branch->Delete();

			LoadSettings(true);
		}
	}

	if (ResolutionSizeX <= 0 || ResolutionSizeY <= 0)
	{
		SetScreenResolution(FIntPoint(GSystemResolution.ResX, GSystemResolution.ResY));

		// Set last confirmed video settings
		LastConfirmedFullscreenMode = FullscreenMode;
		LastUserConfirmedResolutionSizeX = ResolutionSizeX;
		LastUserConfirmedResolutionSizeY = ResolutionSizeY;
		LastUserConfirmedDisplayID = DisplayID;
		LastUserConfirmedDisplayIndex = DisplayIndex;
	}

	const int32 ScreenWidth = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution().X : ResolutionSizeX;
	const int32 ScreenHeight = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution().Y : ResolutionSizeY;
	const int32 ClampedWidth = (ScreenWidth > 0 && DesiredScreenWidth > ScreenWidth) ? ScreenWidth : DesiredScreenWidth;
	const int32 ClampedHeight = (ScreenHeight > 0 && DesiredScreenHeight > ScreenHeight) ? ScreenHeight : DesiredScreenHeight;

	LastUserConfirmedDesiredScreenWidth = DesiredScreenWidth;
	LastUserConfirmedDesiredScreenHeight = DesiredScreenHeight;

	// We do not modify the user setting on console if HDR is not supported
	if (!FPlatformMisc::UseHDRByDefault())
	{
		if (bUseHDRDisplayOutput && !SupportsHDRDisplayOutput())
		{
			bUseHDRDisplayOutput = false;
		}
	}

	LastConfirmedAudioQualityLevel = AudioQualityLevel;

	// The user settings have now been validated for the current version.
	UpdateVersion();
}

void UGameUserSettings::ApplyNonResolutionSettings()
{
	QUICK_SCOPE_CYCLE_COUNTER(GameUserSettings_ApplyNonResolutionSettings);

	ValidateSettings();

	// Update vsync cvar
	{
		FString ConfigSection = "SystemSettings";
#if WITH_EDITOR
		if (GIsEditor)
		{
			ConfigSection = TEXT("SystemSettingsEditor");
		}
#endif
		int32 VSyncValue = 0;
		if (GConfig->GetInt(*ConfigSection, TEXT("r.Vsync"), VSyncValue, GEngineIni))
		{
			// VSync was already set by system settings. We are not capable of setting it here.
		}
		else
		{
			static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
			CVar->Set(IsVSyncEnabled(), ECVF_SetByGameSetting);
		}
	}

	GEngine->SetDynamicResolutionUserSetting(IsDynamicResolutionEnabled());

	if (!IsRunningDedicatedServer())
	{
		SetFrameRateLimitCVar(GetEffectiveFrameRateLimit());
	}

	// in init those are loaded earlier, after that we apply consolevariables.ini
	if (GEngine->IsInitialized())
	{
		if (bEnableScalabilitySettings)
		{
			Scalability::SetQualityLevels(ScalabilityQuality);
		}
	}

	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		FAudioQualitySettings AudioSettings = AudioDevice->GetQualityLevelSettings();
		AudioDevice->SetMaxChannels(AudioSettings.MaxChannels);
	}

	IConsoleManager::Get().CallAllConsoleVariableSinks();

	bool bEnableHDR = (IsHDRAllowed() && bUseHDRDisplayOutput);

	EnableHDRDisplayOutputInternal(bEnableHDR, HDRDisplayOutputNits, true);
}

void UGameUserSettings::ApplyResolutionSettings(bool bCheckForCommandLineOverrides)
{
#if !UE_SERVER
	QUICK_SCOPE_CYCLE_COUNTER(GameUserSettings_ApplyResolutionSettings);

	if (FPlatformProperties::HasFixedResolution())
	{
		return;
	}

	ValidateSettings();

	EWindowMode::Type NewFullscreenMode = GetFullscreenMode();

	// Request a resolution change
	const int32 MonitorIndex = GetClosestMonitorIndex();
	RequestResolutionChange(ResolutionSizeX, ResolutionSizeY, NewFullscreenMode, bCheckForCommandLineOverrides, MonitorIndex);

	if (NewFullscreenMode == EWindowMode::Fullscreen || NewFullscreenMode == EWindowMode::WindowedFullscreen)
	{
		SetPreferredFullscreenMode(NewFullscreenMode == EWindowMode::Fullscreen ? 0 : 1);
	}

	IConsoleManager::Get().CallAllConsoleVariableSinks();
#endif
}

void UGameUserSettings::ApplySettings(bool bCheckForCommandLineOverrides)
{
	{
		// Push recreate render state context to force single recreate instead of multiple recreates for each changed cvar
		FGlobalComponentRecreateRenderStateContext Context;
		ApplyResolutionSettings(bCheckForCommandLineOverrides);
		ApplyNonResolutionSettings();
	}
	RequestUIUpdate();

	SaveSettings();
}

void UGameUserSettings::LoadSettings(bool bForceReload/*=false*/)
{
	QUICK_SCOPE_CYCLE_COUNTER(GameUserSettings_LoadSettings);

	if (bForceReload)
	{
        if (OnUpdateGameUserSettingsFileFromCloud.IsBound())
        {
            FString IniFileLocation = FPaths::GeneratedConfigDir() + UGameplayStatics::GetPlatformName() + "/" +  GGameUserSettingsIni + ".ini";
            UE_LOGF(LogGameUserSettings, Verbose, "%ls", *IniFileLocation);

            if (!OnUpdateGameUserSettingsFileFromCloud.Execute(FString(*IniFileLocation)))
            {
                UE_LOGF(LogGameUserSettings, Warning, "Failed to read the ini file from the Cloud interface %ls", *IniFileLocation);
            }
        }
        
		LoadConfigIni(bForceReload);
	}
	LoadConfig(GetClass(), *GGameUserSettingsIni);

	// Note: Scalability::LoadState() should not be needed as we already loaded the settings earlier (needed so the engine can startup with that before the game is initialized)
	ScalabilityQuality = Scalability::GetQualityLevels();

	// Allow override using command-line settings
	bool bDetectingResolution = ResolutionSizeX == 0 || ResolutionSizeY == 0;

	if (bDetectingResolution)
	{
		ConfirmVideoMode();
	}

	// Update r.FullScreenMode CVar
	SetPreferredFullscreenMode(PreferredFullscreenMode);

	// Window position backwards compatability
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FIntPoint OldWindowPosition(WindowPosX_DEPRECATED, WindowPosY_DEPRECATED);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (OldWindowPosition != GetDefaultWindowPosition())
	{
		if (WindowPositions.IsEmpty())
		{
			WindowPositions.SetNum(1);
		}
		WindowPositions[0] = OldWindowPosition;

		// Write out the non-deprecated value
		SaveConfig(CPF_Config, *GGameUserSettingsIni);
	}
}

void UGameUserSettings::RequestResolutionChange(int32 InResolutionX, int32 InResolutionY, EWindowMode::Type InWindowMode, bool bInDoOverrides /* = true */, int32 MonitorIndex /* = -1 */)
{
	if (FPlatformProperties::HasFixedResolution())
	{
		return;
	}

	if (bInDoOverrides)
	{
		UGameEngine::ConditionallyOverrideSettings(InResolutionX, InResolutionY, InWindowMode, MonitorIndex);
	}

	FSystemResolution::SetMonitorIndex(MonitorIndex);
	FSystemResolution::RequestResolutionChange(InResolutionX, InResolutionY, InWindowMode);
}

void UGameUserSettings::SaveSettings()
{
	QUICK_SCOPE_CYCLE_COUNTER(GameUserSettings_SaveSettings);

	// Save the Scalability state to the same ini file as it was loaded from in FEngineLoop::Preinit
	if (bEnableScalabilitySettings)
	{
		Scalability::SaveState(GIsEditor ? GEditorSettingsIni : GGameUserSettingsIni);
	}

	ReloadWindowPositions();
	SaveConfig(CPF_Config, *GGameUserSettingsIni);
    
    if (OnUpdateCloudDataFromGameUserSettings.IsBound())
    {
        FString IniFileLocation = FPaths::GeneratedConfigDir() + UGameplayStatics::GetPlatformName() + "/" +  GGameUserSettingsIni + ".ini";
        UE_LOGF(LogGameUserSettings, Verbose, "%ls", *IniFileLocation);

        bool bDidSucceed = false;
        bDidSucceed = OnUpdateCloudDataFromGameUserSettings.Execute(FString(*IniFileLocation));
        
        if (!bDidSucceed)
        {
            UE_LOGF(LogGameUserSettings, Warning, "Failed to load the ini file from the Cloud interface %ls", *IniFileLocation);
        }
    }
}

void UGameUserSettings::LoadConfigIni(bool bForceReload/*=false*/)
{
#if PLATFORM_WINDOWS
	// If configured, try to migrate the user ini file by copying it from AppData\Local to AppData\LocalLow.
	static const bool bShouldAutomigrateUserData = FPaths::ShouldSaveToUserDir() &&
		FWindowsPlatformProcess::ShouldExpectLowIntegrityLevel() &&
		(WINDOWS_LOWINTEGRITYLEVEL_AUTOMIGRATE_USERDATA || FParse::Param(FCommandLine::Get(), TEXT("AutomigrateUserDataToLowIntegrity")));
	if (bShouldAutomigrateUserData)
	{
		static bool bAttemptedSettingsMigration = false;
		if (!bAttemptedSettingsMigration)
		{
			bAttemptedSettingsMigration = true;

			// The low integrity ini file path e.g. AppData\LocalLow\...
			const FString LowIntegrityPath = FConfigCacheIni::GetDestIniFilename(*GGameUserSettingsIni, nullptr, *FPaths::GeneratedConfigDir());
			if (LowIntegrityPath.StartsWith(FWindowsPlatformProcess::UserSettingsDir())) // Confirm that assumptions are valid.
			{
				// The medium integrity ini file path e.g. AppData\Local\...
				const FString MediumIntegrityPath = LowIntegrityPath.Replace(FWindowsPlatformProcess::UserSettingsDir(), FWindowsPlatformProcess::UserSettingsDirMediumIntegrity());

				// This file must exist for migration to proceed. We only migrate from medium integrity to low integrity, not the reverse.
				if (FPaths::FileExists(MediumIntegrityPath))
				{
					FDateTime MediumIntegrityTimestamp, LowIntegrityTimestamp;
					IFileManager::Get().GetTimeStampPair(*MediumIntegrityPath, *LowIntegrityPath, MediumIntegrityTimestamp, LowIntegrityTimestamp);
					if (MediumIntegrityTimestamp > LowIntegrityTimestamp)
					{
						UE_LOGF(LogGameUserSettings, Log, "Migrating config file to low integrity. %ls to %ls", *MediumIntegrityPath, *LowIntegrityPath);
						IFileManager::Get().Copy(*LowIntegrityPath, *MediumIntegrityPath, true);
					}
					else
					{
						UE_LOGF(LogGameUserSettings, Log, "Skipping config file migration to low integrity. %ls is newer than %ls", *LowIntegrityPath, *MediumIntegrityPath);
					}
				}
			}
		}
	}
#endif

	FConfigContext Context = FConfigContext::ReadIntoGConfig();
	Context.bForceReload = bForceReload;
	Context.Load(TEXT("GameUserSettings"), GGameUserSettingsIni);
}

void UGameUserSettings::PreloadResolutionSettings(bool bAllowCmdLineOverrides /*= true*/)
{
	// Note: This preloads resolution settings without loading the user settings object.  
	// When changing this code care must be taken to ensure the window starts at the same resolution as the in game resolution
	LoadConfigIni();

	FString ScriptEngineCategory = "/Script/Engine.Engine";
	FString GameUserSettingsCategory = "/Script/Engine.GameUserSettings";

	GConfig->GetString(*ScriptEngineCategory, TEXT("GameUserSettingsClassName"), GameUserSettingsCategory, GEngineIni);

	int32 ResolutionX = GetDefaultResolution().X;
	int32 ResolutionY = GetDefaultResolution().Y;
	EWindowMode::Type WindowMode = GetDefaultWindowMode();
	bool bUseDesktopResolution = false;
	bool bUseHDR = FPlatformMisc::UseHDRByDefault();
	FString DisplayID;
	int32 DisplayIndex = -1;

	int32 Version = 0;
	if (GConfig->GetInt(*GameUserSettingsCategory, TEXT("Version"), Version, GGameUserSettingsIni) && Version == UE_GAMEUSERSETTINGS_VERSION)
	{
		GConfig->GetBool(*GameUserSettingsCategory, TEXT("bUseDesktopResolution"), bUseDesktopResolution, GGameUserSettingsIni);

		int32 WindowModeInt = (int32)WindowMode;
		GConfig->GetInt(*GameUserSettingsCategory, TEXT("FullscreenMode"), WindowModeInt, GGameUserSettingsIni);
		WindowMode = EWindowMode::ConvertIntToWindowMode(WindowModeInt);

		GConfig->GetInt(*GameUserSettingsCategory, TEXT("ResolutionSizeX"), ResolutionX, GGameUserSettingsIni);
		GConfig->GetInt(*GameUserSettingsCategory, TEXT("ResolutionSizeY"), ResolutionY, GGameUserSettingsIni);

		GConfig->GetString(*GameUserSettingsCategory, TEXT("DisplayID"), DisplayID, GGameUserSettingsIni);
		GConfig->GetInt(*GameUserSettingsCategory, TEXT("DisplayIndex"), DisplayIndex, GGameUserSettingsIni);

#if PLATFORM_DESKTOP
		if (bUseDesktopResolution && ResolutionX == 0 && ResolutionY == 0 && WindowMode != EWindowMode::Windowed)
		{
			// Grab display metrics so we can get the display output size.
			FDisplayMetrics DisplayMetrics;
			FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
			const int32 MonitorIndex = DisplayMetrics.GetClosestMonitorFromIDAndIndex(DisplayID, DisplayIndex);
			if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndex))
			{
				ResolutionX = DisplayMetrics.MonitorInfo[MonitorIndex].NativeWidth;
				ResolutionY = DisplayMetrics.MonitorInfo[MonitorIndex].NativeHeight;
			}
			else
			{
				ResolutionX = DisplayMetrics.PrimaryDisplayWidth;
				ResolutionY = DisplayMetrics.PrimaryDisplayHeight;
			}
		}
#endif

		GConfig->GetBool(*GameUserSettingsCategory, TEXT("bUseHDRDisplayOutput"), bUseHDR, GGameUserSettingsIni);
	}

#if !PLATFORM_MANAGES_HDR_SETTING
	if (IsHDRAllowed())
	{
		// Set the user-preference HDR switch
		static auto CVarHDROutputEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.EnableHDROutput"));
		if (CVarHDROutputEnabled)
		{
			CVarHDROutputEnabled->Set(bUseHDR ? 1 : 0, ECVF_SetByGameSetting);
		}
	}
#endif

	const int32 MonitorIndex = GetClosestMonitorFromIDAndIndex(DisplayID, DisplayIndex);
	RequestResolutionChange(ResolutionX, ResolutionY, WindowMode, bAllowCmdLineOverrides, MonitorIndex);

	IConsoleManager::Get().CallAllConsoleVariableSinks();
}

FIntPoint UGameUserSettings::GetDefaultResolution()
{
	return FIntPoint::ZeroValue;
}

FIntPoint UGameUserSettings::GetDefaultWindowPosition()
{
	return FIntPoint(-1, -1);
}

EWindowMode::Type UGameUserSettings::GetDefaultWindowMode()
{
	// WindowedFullscreen should be the general default for games
	return EWindowMode::WindowedFullscreen;
}

int32 UGameUserSettings::GetSyncInterval()
{
	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("rhi.syncinterval"));
	if (ensure(SyncIntervalCVar))
	{
		return SyncIntervalCVar->GetInt();
	}
	else
	{
		return 0;
	}
}

int32 UGameUserSettings::GetFramePace()
{
	return FPlatformRHIFramePacer::GetFramePace();
}

void UGameUserSettings::ResetToCurrentSettings()
{
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->GetWindow().IsValid())
	{
		//handle the fullscreen setting
		SetFullscreenMode(GetWindowModeType(GEngine->GameViewport->GetWindow()->GetWindowMode()));

		//set the current resolution
		SetScreenResolution(FIntPoint(GSystemResolution.ResX, GSystemResolution.ResY));

		// Set the current VSync state
		static const auto CVarVSync = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
		SetVSyncEnabled(CVarVSync->GetValueOnGameThread() != 0);

		// Set the current dynamic resolution state
		SetDynamicResolutionEnabled(GEngine->GetDynamicResolutionUserSetting());

		// Reset confirmed settings
		ConfirmVideoMode();
		LastUserConfirmedDesiredScreenWidth = DesiredScreenWidth;
		LastUserConfirmedDesiredScreenHeight = DesiredScreenHeight;

		// Reset the quality settings to the current levels
		ScalabilityQuality = Scalability::GetQualityLevels();

		// Reset the audio quality level
		AudioQualityLevel = LastConfirmedAudioQualityLevel;

		UpdateResolutionQuality();
	}
}

void UGameUserSettings::SetWindowPosition(int32 WinX, int32 WinY, int32 SaveSlotIndex)
{
	if (SaveSlotIndex >= 0 && SaveSlotIndex < CVar_GameUserSettings_WindowPositionsMaxSize.GetValueOnAnyThread())
	{
		const int32 NumToAdd = FMath::Max(0, SaveSlotIndex - WindowPositions.Num() + 1);
		for (int32 i = 0; i < NumToAdd; ++i)
		{
			WindowPositions.Add(GetDefaultWindowPosition());
		}
		WindowPositions[SaveSlotIndex].X = WinX;
		WindowPositions[SaveSlotIndex].Y = WinY;
	}
	else
	{
		UE_LOGF(LogGameUserSettings, Warning, "UGameUserSettings::SetWindowPosition attempted to save window position in invalid slot %d. Maximum slot index is %d.",
			SaveSlotIndex, CVar_GameUserSettings_WindowPositionsMaxSize.GetValueOnAnyThread() - 1);
	}
}

void UGameUserSettings::SetWindowPosition(int32 WinX, int32 WinY)
{
	SetWindowPosition(WinX, WinY, 0);
}

FIntPoint UGameUserSettings::GetWindowPosition(int32 SaveSlotIndex)
{
	if (WindowPositions.IsValidIndex(SaveSlotIndex))
	{
		return FIntPoint(WindowPositions[SaveSlotIndex].X, WindowPositions[SaveSlotIndex].Y);
	}

	return GetDefaultWindowPosition();
}

FIntPoint UGameUserSettings::GetWindowPosition()
{
	return GetWindowPosition(0);
}

void UGameUserSettings::ReloadWindowPositions()
{
	FProperty* WindowPositionsProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UGameUserSettings, WindowPositions));
	if (WindowPositionsProperty != nullptr)
	{
		LoadConfigIni(false);
		ReloadConfig(GetClass(), *GGameUserSettingsIni, UE::LCPF_None, WindowPositionsProperty);
	}
}

int32 UGameUserSettings::GetMaxNumWindowPositionSaveSlots() const
{
	return CVar_GameUserSettings_WindowPositionsMaxSize.GetValueOnAnyThread();
}

void UGameUserSettings::SetBenchmarkFallbackValues()
{
	ScalabilityQuality.SetBenchmarkFallback();
}

void UGameUserSettings::SetAudioQualityLevel(int32 QualityLevel)
{
	if (AudioQualityLevel != QualityLevel)
	{
		AudioQualityLevel = QualityLevel;

		USoundCue::StaticAudioQualityChanged(QualityLevel);
	}
}

void UGameUserSettings::SetFrameRateLimit(float NewLimit)
{
	FrameRateLimit = NewLimit;
}

float UGameUserSettings::GetFrameRateLimit() const
{
	return FrameRateLimit;
}

void UGameUserSettings::SetOverallScalabilityLevel(int32 Value)
{
	ScalabilityQuality.SetFromSingleQualityLevel(Value);
}

int32 UGameUserSettings::GetOverallScalabilityLevel() const
{
	return ScalabilityQuality.GetSingleQualityLevel();
}

void UGameUserSettings::GetResolutionScaleInformationEx(float& CurrentScaleNormalized, float& CurrentScaleValue, float& MinScaleValue, float& MaxScaleValue) const
{
	CurrentScaleValue = ScalabilityQuality.ResolutionQuality;
	MinScaleValue = Scalability::MinResolutionScale;
	MaxScaleValue = Scalability::MaxResolutionScale;
	CurrentScaleNormalized = ((float)CurrentScaleValue - (float)MinScaleValue) / (float)(MaxScaleValue - MinScaleValue);
}

float UGameUserSettings::GetResolutionScaleNormalized() const
{
	float CurrentScaleNormalized, CurrentScaleValue, MinScaleValue, MaxScaleValue;
	GetResolutionScaleInformationEx(CurrentScaleNormalized, CurrentScaleValue, MinScaleValue, MaxScaleValue);

	return CurrentScaleNormalized;
}

void UGameUserSettings::SetResolutionScaleValueEx(float NewScaleValue)
{
	ScalabilityQuality.ResolutionQuality = FMath::Clamp(NewScaleValue, Scalability::MinResolutionScale, Scalability::MaxResolutionScale);
	const int32 ScreenWidth = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution().X : ResolutionSizeX;
	const int32 ScreenHeight = (FullscreenMode == EWindowMode::WindowedFullscreen) ? GetDesktopResolution().Y : ResolutionSizeY;
	DesiredScreenWidth = ScreenWidth * ScalabilityQuality.ResolutionQuality / 100.0f;
	DesiredScreenHeight = ScreenHeight * ScalabilityQuality.ResolutionQuality / 100.0f;
}

void UGameUserSettings::SetResolutionScaleNormalized(float NewScaleNormalized)
{
	const float RemappedValue = FMath::Lerp((float)Scalability::MinResolutionScale, (float)Scalability::MaxResolutionScale, NewScaleNormalized);
	SetResolutionScaleValueEx(RemappedValue);
}

void UGameUserSettings::SetViewDistanceQuality(int32 Value)
{
	ScalabilityQuality.SetViewDistanceQuality(Value);
}

int32 UGameUserSettings::GetViewDistanceQuality() const
{
	return ScalabilityQuality.ViewDistanceQuality;
}

void UGameUserSettings::SetShadowQuality(int32 Value)
{
	ScalabilityQuality.SetShadowQuality(Value);
}

int32 UGameUserSettings::GetShadowQuality() const
{
	return ScalabilityQuality.ShadowQuality;
}

void UGameUserSettings::SetGlobalIlluminationQuality(int32 Value)
{
	ScalabilityQuality.SetGlobalIlluminationQuality(Value);
}

int32 UGameUserSettings::GetGlobalIlluminationQuality() const
{
	return ScalabilityQuality.GlobalIlluminationQuality;
}

void UGameUserSettings::SetReflectionQuality(int32 Value)
{
	ScalabilityQuality.SetReflectionQuality(Value);
}

int32 UGameUserSettings::GetReflectionQuality() const
{
	return ScalabilityQuality.ReflectionQuality;
}

void UGameUserSettings::SetAntiAliasingQuality(int32 Value)
{
	ScalabilityQuality.SetAntiAliasingQuality(Value);
}

int32 UGameUserSettings::GetAntiAliasingQuality() const
{
	return ScalabilityQuality.AntiAliasingQuality;
}

void UGameUserSettings::SetTextureQuality(int32 Value)
{
	ScalabilityQuality.SetTextureQuality(Value);
}

int32 UGameUserSettings::GetTextureQuality() const
{
	return ScalabilityQuality.TextureQuality;
}

void UGameUserSettings::SetVisualEffectQuality(int32 Value)
{
	ScalabilityQuality.SetEffectsQuality(Value);
}

int32 UGameUserSettings::GetVisualEffectQuality() const
{
	return ScalabilityQuality.EffectsQuality;
}

void UGameUserSettings::SetPostProcessingQuality(int32 Value)
{
	ScalabilityQuality.SetPostProcessQuality(Value);
}

int32 UGameUserSettings::GetPostProcessingQuality() const
{
	return ScalabilityQuality.PostProcessQuality;
}

void UGameUserSettings::SetFoliageQuality(int32 Value)
{
	ScalabilityQuality.SetFoliageQuality(Value);
}

int32 UGameUserSettings::GetFoliageQuality() const
{
	return ScalabilityQuality.FoliageQuality;
}

void UGameUserSettings::SetShadingQuality(int32 Value)
{
	ScalabilityQuality.SetShadingQuality(Value);
}

int32 UGameUserSettings::GetShadingQuality() const
{
	return ScalabilityQuality.ShadingQuality;
}

void UGameUserSettings::SetLandscapeQuality(int32 Value)
{
	ScalabilityQuality.SetLandscapeQuality(Value);
}

int32 UGameUserSettings::GetLandscapeQuality() const
{
	return ScalabilityQuality.LandscapeQuality;
}

UGameUserSettings* UGameUserSettings::GetGameUserSettings()
{
	return GEngine->GetGameUserSettings();
}

void UGameUserSettings::RunHardwareBenchmark(int32 WorkScale, float CPUMultiplier, float GPUMultiplier)
{
	ScalabilityQuality = Scalability::BenchmarkQualityLevels(WorkScale, CPUMultiplier, GPUMultiplier);
	LastCPUBenchmarkResult = ScalabilityQuality.CPUBenchmarkResults;
	LastGPUBenchmarkResult = ScalabilityQuality.GPUBenchmarkResults;
	LastCPUBenchmarkSteps = ScalabilityQuality.CPUBenchmarkSteps;
	LastGPUBenchmarkSteps = ScalabilityQuality.GPUBenchmarkSteps;
	LastGPUBenchmarkMultiplier = GPUMultiplier;
}

void UGameUserSettings::ApplyHardwareBenchmarkResults()
{
	// Apply the new settings and save them
	Scalability::SetQualityLevels(ScalabilityQuality);

	if (bEnableScalabilitySettings)
	{
		Scalability::SaveState(GGameUserSettingsIni);
	}

	SaveSettings();
}

bool UGameUserSettings::SupportsHDRDisplayOutput() const
{
#if PLATFORM_USES_DYNAMIC_HDR_SETTING
	return true;
#else
	return GRHISupportsHDROutput;
#endif
}

void UGameUserSettings::EnableHDRDisplayOutput(bool bEnable, int32 DisplayNits /*= 1000*/)
{
	EnableHDRDisplayOutputInternal(bEnable, DisplayNits, false);
}

void UGameUserSettings::EnableHDRDisplayOutputInternal(bool bEnable, int32 DisplayNits, bool bFromUserSettings)
{
	static IConsoleVariable* CVarHDROutputEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.EnableHDROutput"));

	if (CVarHDROutputEnabled)
	{
		if (bEnable && (!SupportsHDRDisplayOutput() || !IsHDRAllowed()))
		{
			UE_LOGF(LogGameUserSettings, Display, "Tried to enable HDR display output but unsupported or disallowed, forcing off.");
			bEnable = false;
		}

		// Apply device-specific output encoding
		if (bEnable)
		{
			CVarHDROutputEnabled->Set(1, bFromUserSettings ? ECVF_SetByGameSetting : ECVF_SetByCode);
		}

		// Always test this branch as can be used to flush errors
		if (!bEnable)
		{
			CVarHDROutputEnabled->Set(0, bFromUserSettings ? ECVF_SetByGameSetting : ECVF_SetByCode);
		}

		// Update final requested state for saved config
#if !PLATFORM_USES_FIXED_HDR_SETTING
		// Do not override the user setting on console (we rely on the OS setting)
		bUseHDRDisplayOutput = bEnable;
#endif
		HDRDisplayOutputNits = DisplayNits;
	}
}

int32 UGameUserSettings::GetCurrentHDRDisplayNits() const
{
	return bUseHDRDisplayOutput ? HDRDisplayOutputNits : 0;
}

void UGameUserSettings::SetMaximumHDRDisplayNits(int32 DisplayNits)
{
	HDRDisplayOutputNits = DisplayNits;
}

int32 UGameUserSettings::GetMaximumHDRDisplayNits() const
{
	return HDRDisplayOutputNits;
}

void UGameUserSettings::SetHDRCalibrationUsed(bool bUseCalibration)
{
	bUseHDRDisplayCalibration = bUseCalibration;
}

bool UGameUserSettings::IsHDRCalibrationUsed() const
{
	return bUseHDRDisplayCalibration;
}

void UGameUserSettings::SetHDRPaperWhiteNits(float Nits)
{
	HDRPaperWhiteNits = Nits;
}

float UGameUserSettings::GetHDRPaperWhiteNits() const
{
	return HDRPaperWhiteNits;
}

void UGameUserSettings::SetHDRUILuminanceNits(float Nits)
{
	HDRUILuminanceNits = Nits;
}

float UGameUserSettings::GetHDRUILuminanceNits() const
{
	return HDRUILuminanceNits;
}

void UGameUserSettings::SetHDRUILuminanceSeparate(bool bIndependent)
{
	bIsHDRUILuminanceSeparate = bIndependent;
}

bool UGameUserSettings::IsHDRUILuminanceSeparate() const
{
	return bIsHDRUILuminanceSeparate;
}

void UGameUserSettings::SetHDREnabled(bool bEnableHDR)
{
	EnableHDRDisplayOutputInternal(bEnableHDR, HDRDisplayOutputNits, true);
}

bool UGameUserSettings::IsHDREnabled() const
{
	return bUseHDRDisplayOutput;
}

bool UGameUserSettings::IsGlobalIlluminationAllowed(UObject* WorldContextObject) const
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);

	const EShaderPlatform ShaderPlatform = World && World->Scene ? World->Scene->GetShaderPlatform() : GMaxRHIShaderPlatform;

	static const auto CVarLumenAllowed = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Lumen.DiffuseIndirect.Allow"));

	const bool bLumenAllowed = CVarLumenAllowed->GetValueOnGameThread()
		&& DoesPlatformSupportLumenGI(ShaderPlatform);

	return bLumenAllowed;
}