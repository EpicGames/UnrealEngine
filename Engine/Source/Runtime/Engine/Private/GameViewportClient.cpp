// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/GameViewportClient.h"
#include "Engine/GameInstance.h"
#include "EngineGlobals.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "GameFramework/Pawn.h"
#include "ImageCore.h"
#include "Misc/FileHelper.h"
#include "Input/CursorReply.h"
#include "Misc/Paths.h"
#include "InputKeyEventArgs.h"
#include "Misc/CoreDelegates.h"
#include "GameMapsSettings.h"
#include "EngineStats.h"
#include "Net/Core/Connection/NetEnums.h"
#include "RenderingThread.h"
#include "LegacyScreenPercentageDriver.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/Canvas.h"
#include "Components/SkeletalMeshComponent.h"
#include "StateStreamManager.h"
#include "StateStreamDebugRenderer.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Particles/ParticleSystemComponent.h"
#include "Engine/NetDriver.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Console.h"
#include "GameFramework/HUD.h"
#include "FXSystem.h"
#include "SubtitleManager.h"
#include "ImageUtils.h"
#include "SceneViewExtension.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "EngineModule.h"
#include "AudioDevice.h"
#include "Audio/AudioDebug.h"
#include "HighResScreenshot.h"
#include "BufferVisualizationData.h"
#include "GameFramework/InputSettings.h"
#include "Components/LineBatchComponent.h"
#include "Debug/DebugDrawService.h"
#include "Components/BrushComponent.h"
#include "Engine/GameEngine.h"
#include "Logging/MessageLog.h"
#include "Blueprint/UserWidget.h"
#include "GameFramework/GameUserSettings.h"
#include "Engine/CoreSettings.h"
#include "Engine/UserInterfaceSettings.h"
#include "Slate/SceneViewport.h"
#include "Slate/SGameLayerManager.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"
#include "ActorEditorUtils.h"
#include "ComponentRecreateRenderStateContext.h"
#include "DynamicResolutionState.h"
#include "ProfilingDebugging/TraceScreenshot.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ObjectTrace.h"
#include "DynamicResolutionState.h"
#include "HDRHelper.h"
#include "GlobalRenderResources.h"
#include "ShaderCore.h"
#include "StereoRenderUtils.h"

#if WITH_EDITOR
#include "Settings/LevelEditorPlaySettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameViewportClient)

#define LOCTEXT_NAMESPACE "GameViewport"

/** This variable allows forcing full screen of the first player controller viewport, even if there are multiple controllers plugged in and no cinematic playing. */
bool GForceFullscreen = false;

/** Delegate called at the end of the frame when a non-HDR screenshot is captured */
FOnScreenshotCaptured UGameViewportClient::ScreenshotCapturedDelegate;

/** Delegate called at the end of the frame when an HDR screenshot is captured */
FOnHDRScreenshotCaptured UGameViewportClient::HDRScreenshotCapturedDelegate;

/** Delegate called right after the viewport is rendered */
FOnViewportRendered UGameViewportClient::ViewportRenderedDelegate;

/** Delegate called when the game viewport is created. */
FSimpleMulticastDelegate UGameViewportClient::CreatedDelegate;

/** A list of all the stat names which are enabled for this viewport (static so they persist between runs) */
TArray<FString> UGameViewportClient::EnabledStats;

/**
 * UI Stats
 */
DECLARE_CYCLE_STAT(TEXT("UI Drawing Time"),STAT_UIDrawingTime,STATGROUP_UI);

static TAutoConsoleVariable<int32> CVarSetBlackBordersEnabled(
	TEXT("r.BlackBorders"),
	0,
	TEXT("To draw black borders around the rendered image\n")
	TEXT("(prevents artifacts from post processing passes that read outside of the image e.g. PostProcessAA)\n")
	TEXT("in pixels, 0:off"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarScreenshotDelegate(
	TEXT("r.ScreenshotDelegate"),
	1,
	TEXT("ScreenshotDelegates prevent processing of incoming screenshot request and break some features. This allows to disable them.\n")
	TEXT("Ideally we rework the delegate code to not make that needed.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: delegates are on (default)"),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSecondaryScreenPercentage( // TODO: make it a user settings instead?
	TEXT("r.SecondaryScreenPercentage.GameViewport"),
	0,
	TEXT("Override secondary screen percentage for game viewport.\n")
	TEXT(" 0: Compute secondary screen percentage = 100 / DPIScalefactor automaticaly (default);\n")
	TEXT(" 1: override secondary screen percentage."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarTransparentViewportClear(
	TEXT("r.TransparentViewportClear"),
	false,
	TEXT("If true, then the viewport will clear to transparent instead of black.\n")
	TEXT("This can be useful with r.PostProcessing.PropagateAlpha if the game view\n")
	TEXT("does not cover the whole window."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarRemapDeviceIdForOffsetPlayerGamepadIds(
	TEXT("input.bRemapDeviceIdForOffsetPlayerGamepadIds"),
	true,
	TEXT("If true, then when bOffsetPlayerGamepadIds is true we will create a new Input Device Id\n")
	TEXT("as needed for the next local player. This fixes the behavior in split screen.\n")
	TEXT("Note: This CVar will be removed in a future release, this is a temporary wrapper for bug fix behavior."),
	ECVF_Default);

static FAutoConsoleCommandWithWorld SplitScreenSwapControllersCommand
(
	TEXT("SplitScreen.RotateControllerIds"),
	TEXT("Rotates controller ids among gameplayers, useful for testing splitscreen with only one controller. Alias to 'SSSwapController'"),
	FConsoleCommandWithWorldDelegate::CreateLambda([](const UWorld* InWorld)
		{
			UGameViewportClient* Client = InWorld->GetGameViewport();
			if (!Client)
			{
				return;
			}
			Client->SSSwapControllers();
		})
);

static TAutoConsoleVariable<int32> CVarScreenshotTextureArrayMode(
	TEXT("r.Screenshot.TextureArrayMode"),
	0,
	TEXT("Choose how screenshots should be handled when the viewport is using a texture array. Collation is supported only for non-HDR, non-UI screenshots.")
	TEXT(" 0: Include only a single array slice (default, controlled by r.Screenshot.TextureArrayIndex)\n")
	TEXT(" 1: Collate all array slices horizontally (left-to-right)\n")
	TEXT(" 2: Collate all array slices vertically (top-to-bottom)\n")
	TEXT("When collating, the screenshot resolution is used for each individual array slice, so requesting a 400x400 screenshot from\n")
	TEXT("a viewport with two slices and then collating horizontally will yield a final screenshot that's 800x400."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarScreenshotTextureArrayIndex(
	TEXT("r.Screenshot.TextureArrayIndex"),
	0,
	TEXT("Texture array slice to use for screenshots when r.Screenshot.TextureArrayMode = 0"),
	ECVF_Default);

#if CSV_PROFILER_STATS
struct FCsvLocalPlayer
{
	FCsvLocalPlayer()
	{
		CategoryIndex = INDEX_NONE;
		PrevViewOrigin = FVector::ZeroVector;
		LastFrame = 0;
		PrevTime = 0.0;
	}

	uint32 CategoryIndex;
	uint32 LastFrame;
	float PrevTime;
	FVector PrevViewOrigin;
};
static TMap<uint32, FCsvLocalPlayer> GCsvLocalPlayers;
#endif

void UGameViewportClient::EnableCsvPlayerStats(int32 LocalPlayerCount)
{
#if CSV_PROFILER_STATS
	if (GCsvLocalPlayers.Num() < LocalPlayerCount)
	{
		for (int PlayerIndex = GCsvLocalPlayers.Num(); PlayerIndex < LocalPlayerCount; PlayerIndex++)
		{
			FCsvLocalPlayer& CsvData = GCsvLocalPlayers.Add(PlayerIndex);
			uint32 index = GCsvLocalPlayers.Num() - 1;
			FString CategoryName = (PlayerIndex == 0) ? TEXT("View") : FString::Printf(TEXT("View%d"), index);
			CsvData.CategoryIndex = FCsvProfiler::Get()->RegisterCategory(CategoryName, (PlayerIndex == 0) ? true : false, false);
		}
	}

	int32 PlayerIndex = 0;
	for (auto& KV : GCsvLocalPlayers)
	{
		FCsvLocalPlayer& value = KV.Value;
		FCsvProfiler::Get()->EnableCategoryByIndex(value.CategoryIndex, PlayerIndex < LocalPlayerCount);
		PlayerIndex++;
	}
#endif
}

void UGameViewportClient::UpdateCsvCameraStats(const TMap<ULocalPlayer*, FSceneView*>& PlayerViewMap)
{
#if CSV_PROFILER_STATS
	UWorld* MyWorld = GetWorld();
	if (!ensure(World))
	{
		return;
	}

	for (TMap<ULocalPlayer*, FSceneView*>::TConstIterator It(PlayerViewMap); It; ++It)
	{
		ULocalPlayer* LocalPlayer = It.Key();
		FSceneView* SceneView = It.Value();

		uint32 PlayerIndex = ConvertLocalPlayerToGamePlayerIndex(LocalPlayer);
		if (PlayerIndex != INDEX_NONE)
		{
			FCsvLocalPlayer& CsvData = GCsvLocalPlayers.FindOrAdd(PlayerIndex);
			if (CsvData.CategoryIndex == INDEX_NONE)
			{
				uint32 index = GCsvLocalPlayers.Num() - 1;
				FString CategoryName = LocalPlayer->IsPrimaryPlayer() ? TEXT("View") : FString::Printf(TEXT("View%d"), index);
				CsvData.CategoryIndex = FCsvProfiler::Get()->RegisterCategory(CategoryName, (index == 0) ? true : false, false);
			}

			FVector ViewOrigin = SceneView->ViewMatrices.GetViewOrigin();
			FVector Diff = ViewOrigin - CsvData.PrevViewOrigin;
			float CurrentTime = MyWorld->GetRealTimeSeconds();
			float DeltaT = CurrentTime - CsvData.PrevTime;
			CsvData.PrevViewOrigin = ViewOrigin;
			CsvData.LastFrame = GFrameNumber;
			CsvData.PrevTime = CurrentTime;

			FCsvProfiler::RecordCustomStat("PosX", CsvData.CategoryIndex, ViewOrigin.X, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("PosY", CsvData.CategoryIndex, ViewOrigin.Y, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("PosZ", CsvData.CategoryIndex, ViewOrigin.Z, ECsvCustomStatOp::Set);

			if (!FMath::IsNearlyZero(DeltaT))
			{
				FVector Velocity = Diff / float(DeltaT);
				float CameraSpeed = Velocity.Size();
				float CameraSpeed2D = Velocity.Size2D();

				FCsvProfiler::RecordCustomStat("Speed", CsvData.CategoryIndex, CameraSpeed, ECsvCustomStatOp::Set);
				FCsvProfiler::RecordCustomStat("Speed2D", CsvData.CategoryIndex, CameraSpeed2D, ECsvCustomStatOp::Set);
			}

#if !UE_BUILD_SHIPPING
			FVector ForwardVec = SceneView->ViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(2);
			FVector UpVec = SceneView->ViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(1);
			FCsvProfiler::RecordCustomStat("ForwardX", CsvData.CategoryIndex, ForwardVec.X, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("ForwardY", CsvData.CategoryIndex, ForwardVec.Y, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("ForwardZ", CsvData.CategoryIndex, ForwardVec.Z, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("UpX", CsvData.CategoryIndex, UpVec.X, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("UpY", CsvData.CategoryIndex, UpVec.Y, ECsvCustomStatOp::Set);
			FCsvProfiler::RecordCustomStat("UpZ", CsvData.CategoryIndex, UpVec.Z, ECsvCustomStatOp::Set);
#endif // !UE_BUILD_SHIPPING
		}
	}
#endif
}


UGameViewportClient::UGameViewportClient(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EngineShowFlags(ESFIM_Game)
	, Viewport(nullptr)
	, CurrentBufferVisualizationMode(NAME_None)
	, CurrentNaniteVisualizationMode(NAME_None)
	, CurrentLumenVisualizationMode(NAME_None)
	, CurrentMegaLightsVisualizationMode(NAME_None)
	, CurrentSubstrateVisualizationMode(NAME_None)
	, CurrentGroomVisualizationMode(NAME_None)
	, CurrentVirtualShadowMapVisualizationMode(NAME_None)
	, HighResScreenshotDialog(nullptr)
	, bUseSoftwareCursorWidgets(true)
	, bIgnoreInput(false)
	, MouseCaptureMode(EMouseCaptureMode::CapturePermanently)
	, bHideCursorDuringCapture(false)
	, MouseLockMode(EMouseLockMode::LockOnCapture)
	, bIsMouseOverClient(false)
#if WITH_EDITOR
	, bUseMouseForTouchInEditor(false)
#endif
{

	bIsPlayInEditorViewport = false;
	ViewModeIndex = VMI_Lit;

	SplitscreenInfo.Init(FSplitscreenData(), ESplitScreenType::SplitTypeCount);

	static float OneOverThree = 1.0f / 3.0f;
	static float TwoOverThree = 2.0f / 3.0f;

	SplitscreenInfo[ESplitScreenType::None].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, 1.0f, 0.0f, 0.0f));

	SplitscreenInfo[ESplitScreenType::TwoPlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, 0.5f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::TwoPlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, 0.5f, 0.0f, 0.5f));

	SplitscreenInfo[ESplitScreenType::TwoPlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 1.0f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::TwoPlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 1.0f, 0.5f, 0.0f));

	SplitscreenInfo[ESplitScreenType::ThreePlayer_FavorTop].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, 0.5f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_FavorTop].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.0f, 0.5f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_FavorTop].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.5f, 0.5f));

	SplitscreenInfo[ESplitScreenType::ThreePlayer_FavorBottom].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_FavorBottom].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.5f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_FavorBottom].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, 0.5f, 0.0f, 0.5f));

	SplitscreenInfo[ESplitScreenType::ThreePlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(OneOverThree, 1.0f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(OneOverThree, 1.0f, OneOverThree, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(OneOverThree, 1.0f, TwoOverThree, 0.0f));

	SplitscreenInfo[ESplitScreenType::ThreePlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, OneOverThree, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, OneOverThree, 0.0f, OneOverThree));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.0f, OneOverThree, 0.0f, TwoOverThree));

	SplitscreenInfo[ESplitScreenType::ThreePlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.5f, 0.0f));
	SplitscreenInfo[ESplitScreenType::ThreePlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.0f, 0.5f));


	SplitscreenInfo[ESplitScreenType::FourPlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.5f, 0.0f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.0f, 0.5f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Grid].PlayerData.Add(FPerPlayerSplitscreenData(0.5f, 0.5f, 0.5f, 0.5f));

	SplitscreenInfo[ESplitScreenType::FourPlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(0.25f, 1.0f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(0.25f, 1.0f, 0.25f, 0.0f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(0.25f, 1.0f, 0.5f, 0.0f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Vertical].PlayerData.Add(FPerPlayerSplitscreenData(0.25f, 1.0f, 0.75f, 0.0f));

	SplitscreenInfo[ESplitScreenType::FourPlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.f, 0.25f, 0.0f, 0.0f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.f, 0.25f, 0.0f, 0.25f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.f, 0.25f, 0.0f, 0.5f));
	SplitscreenInfo[ESplitScreenType::FourPlayer_Horizontal].PlayerData.Add(FPerPlayerSplitscreenData(1.f, 0.25f, 0.0f, 0.75f));

	bSuppressTransitionMessage = true;

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		StatUnitData = new FStatUnitData();
		StatThermalsData = new FStatThermalsData();
		StatHitchesData = new FStatHitchesData();
		FCoreDelegates::StatCheckEnabled.AddUObject(this, &UGameViewportClient::HandleViewportStatCheckEnabled);
		FCoreDelegates::StatEnabled.AddUObject(this, &UGameViewportClient::HandleViewportStatEnabled);
		FCoreDelegates::StatDisabled.AddUObject(this, &UGameViewportClient::HandleViewportStatDisabled);
		FCoreDelegates::StatDisableAll.AddUObject(this, &UGameViewportClient::HandleViewportStatDisableAll);

		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().OnRegularWindowMovedOrResized().AddUObject(this, &UGameViewportClient::HandleRegularWindowMovedOrResized);
#if WITH_EDITOR
			if (GIsEditor)
			{
				FSlateApplication::Get().OnWindowDPIScaleChanged().AddUObject(this, &UGameViewportClient::HandleWindowDPIScaleChanged);
			}
#endif
		}

		AudioDeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddLambda([this](const Audio::FDeviceId InDeviceId)
		{
			if (InDeviceId == AudioDevice.GetDeviceID())
			{
				AudioDevice.Reset();
			}
		});
	}
}

UGameViewportClient::UGameViewportClient(FVTableHelper& Helper)
	: Super(Helper)
	, bEnablePlayersSplitRT(false)
	, EngineShowFlags(ESFIM_Game)
	, Viewport(nullptr)
	, CurrentBufferVisualizationMode(NAME_None)
	, CurrentNaniteVisualizationMode(NAME_None)
	, CurrentLumenVisualizationMode(NAME_None)
	, CurrentMegaLightsVisualizationMode(NAME_None)
	, CurrentSubstrateVisualizationMode(NAME_None)
	, CurrentGroomVisualizationMode(NAME_None)
	, CurrentVirtualShadowMapVisualizationMode(NAME_None)
	, HighResScreenshotDialog(nullptr)
	, bIgnoreInput(false)
	, MouseCaptureMode(EMouseCaptureMode::CapturePermanently)
	, bHideCursorDuringCapture(false)
	, MouseLockMode(EMouseLockMode::LockOnCapture)
{

}

UGameViewportClient::~UGameViewportClient()
{
	if (EngineShowFlags.Collision)
	{
		// Clear ref to world as it may be GC'd & we don't want to use it in toggle below.
		World = nullptr;

		EngineShowFlags.SetCollision(false);
		ToggleShowCollision();
	}

	FCoreDelegates::StatCheckEnabled.RemoveAll(this);
	FCoreDelegates::StatEnabled.RemoveAll(this);
	FCoreDelegates::StatDisabled.RemoveAll(this);
	FCoreDelegates::StatDisableAll.RemoveAll(this);

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnRegularWindowMovedOrResized().RemoveAll(this);
#if WITH_EDITOR
		if (GIsEditor)
		{
			FSlateApplication::Get().OnWindowDPIScaleChanged().RemoveAll(this);
		}
#endif
	}

	if (StatHitchesData)
	{
		delete StatHitchesData;
		StatHitchesData = NULL;
	}

	if (StatUnitData)
	{
		delete StatUnitData;
		StatUnitData = NULL;
	}
	
	if (StatThermalsData)
	{
		delete StatThermalsData;
		StatThermalsData = NULL;
	}
}

void UGameViewportClient::PostInitProperties()
{
	Super::PostInitProperties();
	EngineShowFlags = FEngineShowFlags(ESFIM_Game);
}

void UGameViewportClient::BeginDestroy()
{
	if (AudioDeviceDestroyedHandle.IsValid())
	{
		FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(AudioDeviceDestroyedHandle);
		AudioDeviceDestroyedHandle.Reset();
	}
	AudioDevice.Reset();

	RemoveAllViewportWidgets();
	Super::BeginDestroy();
}

void UGameViewportClient::DetachViewportClient()
{
	ViewportConsole = NULL;
	ResetHardwareCursorStates();
	RemoveAllViewportWidgets();
	RemoveFromRoot();
}

TSharedRef<FSceneViewport> UGameViewportClient::CreateViewport(TSharedPtr<SViewport> InViewportWidget)
{
	return FSceneViewport::Create(this, InViewportWidget);
}

FSceneViewport* UGameViewportClient::CreateGameViewport(TSharedPtr<SViewport> InViewportWidget)
{
	FSceneViewport* Result = new FSceneViewport(InViewportWidget);
	Result->SetViewportClient(this);
	return Result;
}

FSceneViewport* UGameViewportClient::GetGameViewport()
{
	return Viewport;
}

const FSceneViewport* UGameViewportClient::GetGameViewport() const
{
	return Viewport;
}


TSharedPtr<class SViewport> UGameViewportClient::GetGameViewportWidget() const
{
	const FSceneViewport* SceneViewport = GetGameViewport();
	if (SceneViewport != nullptr)
	{
		TWeakPtr<SViewport> WeakViewportWidget = SceneViewport->GetViewportWidget();
		TSharedPtr<SViewport> ViewportWidget = WeakViewportWidget.Pin();
		return ViewportWidget;
	}
	return nullptr;
}

void UGameViewportClient::Tick( float DeltaTime )
{
	TickDelegate.Broadcast(DeltaTime);
}

FString UGameViewportClient::ConsoleCommand( const FString& Command)
{
	FString TruncatedCommand = Command.Left(1000);
	FConsoleOutputDevice ConsoleOut(ViewportConsole);
	Exec( GetWorld(), *TruncatedCommand,ConsoleOut);
	return *ConsoleOut;
}

void UGameViewportClient::SetEnabledStats(const TArray<FString>& InEnabledStats)
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		EnabledStats = InEnabledStats;
	}
	else
	{
		UE_LOGF(LogPlayerManagement, Warning, "WARNING: Stats disabled for non multi-threading platforms");
	}

#if ENABLE_AUDIO_DEBUG
	if (UWorld* MyWorld = GetWorld())
	{
		if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
		{
			Audio::FAudioDebugger::ResolveDesiredStats(this);
		}
	}
#endif // ENABLE_AUDIO_DEBUG
}


void UGameViewportClient::Init(struct FWorldContext& WorldContext, UGameInstance* OwningGameInstance, bool bCreateNewAudioDevice)
{
	// set reference to world context
	WorldContext.AddRef(static_cast<UWorld*&>(World));

	// remember our game instance
	GameInstance = OwningGameInstance;

	// Set the projects default viewport mouse capture mode
	MouseCaptureMode = GetDefault<UInputSettings>()->DefaultViewportMouseCaptureMode;
	FString DefaultViewportMouseCaptureMode;
	if (FParse::Value(FCommandLine::Get(), TEXT("DefaultViewportMouseCaptureMode="), DefaultViewportMouseCaptureMode))
	{
		const UEnum* EnumPtr = StaticEnum<EMouseCaptureMode>();
		checkf(EnumPtr, TEXT("Unable to find EMouseCaptureMode enum"));
		if (EnumPtr)
		{
			int64 EnumValue = EnumPtr->GetValueByName(FName(*DefaultViewportMouseCaptureMode));
			if (EnumValue != INDEX_NONE)
			{
				MouseCaptureMode = static_cast<EMouseCaptureMode>(EnumValue);
			}
			else
			{
				UE_LOGF(LogInit, Warning, "Unknown DefaultViewportMouseCaptureMode %ls. Command line setting will be ignored.", *DefaultViewportMouseCaptureMode);
			}
		}
	}
	MouseLockMode = GetDefault<UInputSettings>()->DefaultViewportMouseLockMode;

	// Don't capture mouse when headless
	if(!FApp::CanEverRender())
	{
		MouseCaptureMode = EMouseCaptureMode::NoCapture;
		MouseLockMode = EMouseLockMode::DoNotLock;
	}

	// In off-screen rendering mode don't lock mouse to the viewport, as we don't want mouse to lock to an invisible window
	if (FSlateApplication::Get().IsRenderingOffScreen()) {
		MouseLockMode = EMouseLockMode::DoNotLock;
	}

	// Create the cursor Widgets
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>(UUserInterfaceSettings::StaticClass());

	if (GEngine)
	{
		FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
		if (AudioDeviceManager)
		{
			// Get a new audio device for this world.
			FAudioDeviceParams DeviceParams = AudioDeviceManager->GetDefaultParamsForNewWorld();
			DeviceParams.AssociatedWorld = World;

			AudioDevice = AudioDeviceManager->RequestAudioDevice(DeviceParams);

			if (AudioDevice.IsValid())
			{
#if ENABLE_AUDIO_DEBUG
				Audio::FAudioDebugger::ResolveDesiredStats(this);
#endif // ENABLE_AUDIO_DEBUG

				// Set the base mix of the new device based on the world settings of the world
				if (World)
				{
					AudioDevice.GetAudioDevice()->SetDefaultBaseSoundMix(World->GetWorldSettings()->DefaultBaseSoundMix);

					// Set the world's audio device handle to use so that sounds which play in that world will use the correct audio device
					World->SetAudioDevice(AudioDevice);
				}

				// Set this audio device handle on the world context so future world's set onto the world context
				// will pass the audio device handle to them and audio will play on the correct audio device
				WorldContext.AudioDeviceID = AudioDevice.GetDeviceID();
			}
		}
	}

	// Set all the software cursors.
	for ( auto& Entry : UISettings->SoftwareCursors )
	{
		SetSoftwareCursorFromClassPath(Entry.Key, Entry.Value);
	}

	// Set all the hardware cursors.
	for ( auto& Entry : UISettings->HardwareCursors )
	{
		SetHardwareCursor(Entry.Key, Entry.Value.CursorPath, Entry.Value.HotSpot);
	}
}

void UGameViewportClient::RebuildCursors()
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>(UUserInterfaceSettings::StaticClass());
	// Set all the software cursors.
	for (auto& Entry : UISettings->SoftwareCursors)
	{
		SetSoftwareCursorFromClassPath(Entry.Key, Entry.Value);
	}

	// Set all the hardware cursors.
	for (auto& Entry : UISettings->HardwareCursors)
	{
		SetHardwareCursor(Entry.Key, Entry.Value.CursorPath, Entry.Value.HotSpot);
	}
}

UWorld* UGameViewportClient::GetWorld() const
{
	return World;
}

UGameInstance* UGameViewportClient::GetGameInstance() const
{
	return GameInstance;
}

bool UGameViewportClient::TryToggleFullscreenOnInputKey(FKey Key, EInputEvent EventType)
{
	if ((Key == EKeys::Enter && EventType == EInputEvent::IE_Pressed && FSlateApplication::Get().GetModifierKeys().IsAltDown() && GetDefault<UInputSettings>()->bAltEnterTogglesFullscreen)
		|| (IsRunningGame() && Key == EKeys::F11 && EventType == EInputEvent::IE_Pressed && GetDefault<UInputSettings>()->bF11TogglesFullscreen && !FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Control | EModifierKey::Alt)))
	{
		HandleToggleFullscreenCommand();
		return true;
	}

	return false;
}

void UGameViewportClient::RemapControllerInput(FInputKeyEventArgs& InOutEventArgs)
{
	const int32 NumLocalPlayers = World ? World->GetGameInstance()->GetNumLocalPlayers() : 0;

	if (NumLocalPlayers > 1 && InOutEventArgs.Key.IsGamepadKey() && GetDefault<UGameMapsSettings>()->bOffsetPlayerGamepadIds)
	{
		// Temp cvar in case this change somehow breaks input for any split screen games.
		if (CVarRemapDeviceIdForOffsetPlayerGamepadIds.GetValueOnAnyThread())
		{
			const TArray<ULocalPlayer*>& CurrentLocalPlayers = World->GetGameInstance()->GetLocalPlayers();
			if (CurrentLocalPlayers.IsValidIndex(InOutEventArgs.ControllerId) && CurrentLocalPlayers.IsValidIndex(InOutEventArgs.ControllerId + 1))
			{
				const FPlatformUserId DesiredPlatformUser = CurrentLocalPlayers[InOutEventArgs.ControllerId + 1]->GetPlatformUserId();

				IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

				// Check for if this FPlatformUserID already has a primary input device ID. If it does, we can use that
				FInputDeviceId DesiredInputDeviceId = DeviceMapper.GetPrimaryInputDeviceForUser(DesiredPlatformUser);
				if (!DesiredInputDeviceId.IsValid())
				{
					// Otherwise we need to create a new "Fake" input device ID...
					DesiredInputDeviceId = DeviceMapper.AllocateNewInputDeviceId();

					// ...  and map it to our desired platform user so that the PlayerController knows it that this is associated with the local player
					DeviceMapper.Internal_MapInputDeviceToUser(DesiredInputDeviceId, DesiredPlatformUser, EInputDeviceConnectionState::Connected);
				}

				// Say that this input event is from the other local player's input device!
				InOutEventArgs.InputDevice = DesiredInputDeviceId;
			}
		}

		// We still want to increment the controller ID in case there is any legacy code listening for it
		InOutEventArgs.ControllerId++;
	}
	else if (InOutEventArgs.Viewport->IsPlayInEditorViewport() && InOutEventArgs.Key.IsGamepadKey())
	{
		GEngine->RemapGamepadControllerIdForPIE(this, InOutEventArgs.ControllerId);
	}
}

bool UGameViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	FInputKeyEventArgs EventArgs = InEventArgs;

	if (TryToggleFullscreenOnInputKey(EventArgs.Key, EventArgs.Event))
	{
		return true;
	}

	if (EventArgs.Key == EKeys::LeftMouseButton && EventArgs.Event == EInputEvent::IE_Pressed)
	{
		GEngine->SetFlashIndicatorLatencyMarker(GFrameCounter);
	}

	RemapControllerInput(EventArgs);


#if WITH_EDITOR
	if (EventArgs.Key.IsGamepadKey())
	{
		/** For PIE, since this is gamepad, check if we want to route gamepad to second window.
		 * Let the next PIE window handle the input (this allows people to use a controller for one window and kbm for the other).
		 */
		const FViewportClient* InViewportClient = InEventArgs.Viewport != nullptr ? InEventArgs.Viewport->GetClient() : nullptr;
		if (InViewportClient == this)
		{
			const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
			const bool CanRouteGamepadToSecondWindow = [&PlayInSettings] { bool RouteGamepadToSecondWindow(false); return (PlayInSettings->GetRouteGamepadToSecondWindow(RouteGamepadToSecondWindow) && RouteGamepadToSecondWindow); }();
			const bool CanRunUnderOneProcess = [&PlayInSettings] { bool RunUnderOneProcess(false); return (PlayInSettings->GetRunUnderOneProcess(RunUnderOneProcess) && RunUnderOneProcess); }();
			if (CanRouteGamepadToSecondWindow && CanRunUnderOneProcess && InEventArgs.Viewport->IsPlayInEditorViewport())
			{
				if (UGameViewportClient* NextViewport = GEngine->GetNextPIEViewport(this))
				{
					const bool bResult = NextViewport->InputKey(InEventArgs);
					return false;
				}
			}
		}	
	}
#endif
	

	if (IgnoreInput())
	{
		return ViewportConsole ? ViewportConsole->InputKey(EventArgs.InputDevice, EventArgs.Key, EventArgs.Event, EventArgs.AmountDepressed, EventArgs.IsGamepad()) : false;
	}

	OnInputKeyEvent.Broadcast(EventArgs);

#if WITH_EDITOR
	// Give debugger commands a chance to process key binding
	if (GameViewportInputKeyDelegate.IsBound())
	{
		if ( GameViewportInputKeyDelegate.Execute(EventArgs.Key, FSlateApplication::Get().GetModifierKeys(), EventArgs.Event) )
		{
			return true;
		}
	}
#endif

	// route to subsystems that care
	bool bResult = ( ViewportConsole ? ViewportConsole->InputKey(EventArgs.InputDevice, EventArgs.Key, EventArgs.Event, EventArgs.AmountDepressed, EventArgs.IsGamepad()) : false );

	// Try the override callback, this may modify event args
	if (!bResult && OnOverrideInputKeyEvent.IsBound())
	{
		bResult = OnOverrideInputKeyEvent.Execute(EventArgs);
	}

	if (!bResult)
	{
		ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromInputDevice(this, EventArgs.InputDevice);
		if (TargetPlayer && TargetPlayer->PlayerController)
		{
			bResult = TargetPlayer->PlayerController->InputKey(EventArgs);
		}

		// A gameviewport is always considered to have responded to a mouse buttons to avoid throttling
		if (!bResult && EventArgs.Key.IsMouseButton())
		{
			bResult = true;
		}
	}


	return bResult;
}

bool UGameViewportClient::InputAxis(const FInputKeyEventArgs& InArgs)
{
	if (IgnoreInput())
	{
		return false;
	}

	// Handle mapping controller id and key if needed, which may need to mutate the args
	FInputKeyEventArgs MutableEventArgs = InArgs;

	FViewport* InViewport = MutableEventArgs.Viewport;
	const bool bGamepad = MutableEventArgs.IsGamepad();
	float Delta = MutableEventArgs.AmountDepressed;

	RemapControllerInput(MutableEventArgs);

#if WITH_EDITOR
	if (bGamepad && InViewport)
	{
		/** For PIE, since this is gamepad, check if we want to route gamepad to second window.
		 * Let the next PIE window handle the input (this allows people to use a controller for one window and kbm for the other).
		 */
		const FViewportClient* InViewportClient = InViewport->GetClient();
		if (InViewportClient == this)
		{
			const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();
			const bool CanRouteGamepadToSecondWindow = [&PlayInSettings] { bool RouteGamepadToSecondWindow(false); return (PlayInSettings->GetRouteGamepadToSecondWindow(RouteGamepadToSecondWindow) && RouteGamepadToSecondWindow); }();
			const bool CanRunUnderOneProcess = [&PlayInSettings] { bool RunUnderOneProcess(false); return (PlayInSettings->GetRunUnderOneProcess(RunUnderOneProcess) && RunUnderOneProcess); }();
			if (CanRouteGamepadToSecondWindow && CanRunUnderOneProcess && InViewport->IsPlayInEditorViewport())
			{
				if (UGameViewportClient* NextViewport = GEngine->GetNextPIEViewport(this))
				{
					const bool bResult = NextViewport->InputAxis(MutableEventArgs);
					return false;
				}
			}
		}
	}
#endif
	
	OnInputAxisEvent.Broadcast(InViewport, MutableEventArgs.ControllerId, MutableEventArgs.Key, Delta, MutableEventArgs.DeltaTime, MutableEventArgs.NumSamples, MutableEventArgs.IsGamepad());
	
	bool bResult = false;

	// Don't allow mouse/joystick input axes while in PIE and the console has forced the cursor to be visible.  It's
	// just distracting when moving the mouse causes mouse look while you are trying to move the cursor over a button
	// in the editor!
	if (InViewport)
	{
		if( !( InViewport->IsSlateViewport() && InViewport->IsPlayInEditorViewport() ) || ViewportConsole == nullptr || !ViewportConsole->ConsoleActive() )
		{
			// route to subsystems that care
			if (ViewportConsole != nullptr)
			{
				bResult = ViewportConsole->InputAxis(MutableEventArgs.InputDevice, MutableEventArgs.Key, Delta, MutableEventArgs.DeltaTime, MutableEventArgs.NumSamples, MutableEventArgs.IsGamepad());
			}
		
			// Try the override callback, this may modify event args
			if (!bResult && OnOverrideInputAxisEvent.IsBound())
			{
				bResult = OnOverrideInputAxisEvent.Execute(MutableEventArgs, Delta, MutableEventArgs.DeltaTime, MutableEventArgs.NumSamples);
			}

			if (!bResult)
			{
				ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromInputDevice(this, MutableEventArgs.InputDevice);
				if (TargetPlayer && TargetPlayer->PlayerController)
				{
					bResult = TargetPlayer->PlayerController->InputKey(MutableEventArgs);
				}
			}

			if( InViewport->IsSlateViewport() && InViewport->IsPlayInEditorViewport() )
			{
				// Absorb all keys so game input events are not routed to the Slate editor frame
				bResult = true;
			}
		}
	}

	return bResult;
}


bool UGameViewportClient::InputChar(FViewport* InViewport, int32 ControllerId, TCHAR Character)
{
	// should probably just add a ctor to FString that takes a TCHAR
	FString CharacterString;
	CharacterString += Character;

	//Always route to the console
	bool bResult = (ViewportConsole ? ViewportConsole->InputChar(FInputDeviceId::CreateFromInternalId(ControllerId), CharacterString) : false);

	if (IgnoreInput())
	{
		return bResult;
	}

	// fall back to default behavior
	if (!bResult)
	{
		bResult = Super::InputChar(InViewport, ControllerId, Character);
	}

	return bResult;
}

bool UGameViewportClient::InputTouch(
	FViewport* const InViewport,
	const FTouchId TouchId,
	const ETouchType::Type Type,
	const FVector2D& TouchLocation,
	const float Force,
	const uint64 Timestamp)
{
	if (IgnoreInput())
	{
		return false;
	}

	// route to subsystems that care
	bool bResult = (ViewportConsole ? ViewportConsole->InputTouch(TouchId, Type, TouchLocation, Force, Timestamp) : false);
	if (!bResult)
	{
		ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromInputDevice(this, TouchId.GetDeviceId());
		if (TargetPlayer && TargetPlayer->PlayerController)
		{
			bResult = TargetPlayer->PlayerController->InputTouch(TouchId, Type, TouchLocation, Force, Timestamp);
		}
	}

	return bResult;
}

bool UGameViewportClient::InputMotion(FViewport* InViewport, const FInputDeviceId DeviceId, const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, const uint64 Timestamp)
{
	if (IgnoreInput() || !GetDefault<UInputSettings>()->bEnableMotionControls)
	{
		return false;
	}

	// route to subsystems that care
	bool bResult = false;

	ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromInputDevice(this, DeviceId);
	if (TargetPlayer && TargetPlayer->PlayerController)
	{
		bResult = TargetPlayer->PlayerController->InputMotion(DeviceId, Tilt, RotationRate, Gravity, Acceleration, Timestamp);
	}

	return bResult;
}

bool UGameViewportClient::InputGesture(FViewport* InViewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp)
{
	if (IgnoreInput())
	{
		return false;
	}

	bool bResult = false;

	ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromInputDevice(this, DeviceId);
	if (TargetPlayer && TargetPlayer->PlayerController)
	{
		bResult = TargetPlayer->PlayerController->InputGesture(DeviceId, GestureType, GesturePhase, GestureDelta, bIsDirectionInvertedFromDevice, Timestamp);
	}

	return bResult;
}

void UGameViewportClient::SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)
{
	if (FPlatformMisc::DesktopTouchScreen() && GetUseMouseForTouch())
	{
		FSlateApplication::Get().SetGameIsFakingTouchEvents(!bInIsSimulateInEditorViewport);
	}

	for (ULocalPlayer* LocalPlayer : GetOuterUEngine()->GetGamePlayers(this))
	{
		if (LocalPlayer->PlayerController)
		{
			if (bInIsSimulateInEditorViewport)
			{
				LocalPlayer->PlayerController->CleanupGameViewport();
			}
			else
			{
				LocalPlayer->PlayerController->CreateTouchInterface();
			}
		}
	}
}

float UGameViewportClient::UpdateViewportClientWindowDPIScale() const
{
	TSharedPtr<SWindow> PinnedWindow = GetWindow();

	float DPIScale = 1.0f;

	if(PinnedWindow.IsValid() && PinnedWindow->GetNativeWindow().IsValid())
	{
		DPIScale = PinnedWindow->GetNativeWindow()->GetDPIScaleFactor();
	}

	return DPIScale;
}

void UGameViewportClient::MouseEnter(FViewport* InViewport, int32 x, int32 y)
{
	Super::MouseEnter(InViewport, x, y);

	if (FPlatformMisc::DesktopTouchScreen() && InViewport && GetUseMouseForTouch() && GetGameViewport() && !GetGameViewport()->GetPlayInEditorIsSimulate())
	{
		FSlateApplication::Get().SetGameIsFakingTouchEvents(true);
	}

	// Replace all the cursors.
	TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();
	if ( ICursor* Cursor = PlatformCursor.Get() )
	{
		for ( auto& Entry : HardwareCursors )
		{
			Cursor->SetTypeShape(Entry.Key, Entry.Value);
		}
	}

	bIsMouseOverClient = true;
}

void UGameViewportClient::MouseLeave(FViewport* InViewport)
{
	Super::MouseLeave(InViewport);

	if (InViewport && GetUseMouseForTouch())
	{
		// Only send the touch end event if we're not drag/dropping, as that will end the drag/drop operation.
		if ( !FSlateApplication::Get().IsDragDropping() )
		{
			FIntPoint LastViewportCursorPos;
			InViewport->GetMousePos(LastViewportCursorPos, false);

			if (FPlatformMisc::DesktopTouchScreen())
			{
				TSharedPtr<class SViewport> ViewportWidget = GetGameViewportWidget();
				if (ViewportWidget.IsValid() && !ViewportWidget->HasFocusedDescendants())
				{
					FVector2D CursorPos(LastViewportCursorPos.X, LastViewportCursorPos.Y);
					FSlateApplication::Get().SetGameIsFakingTouchEvents(false, &CursorPos);
				}
			}
		}
	}

#if WITH_EDITOR

	// NOTE: Only do this in editor builds where the editor is running.
	// We don't care about bothering to clear them otherwise, and it may negatively impact
	// things like drag/drop, since those would 'leave' the viewport.
	if ( !FSlateApplication::Get().IsDragDropping() )
	{
		bIsMouseOverClient = false;
		ResetHardwareCursorStates();
	}

#endif
}

void UGameViewportClient::ResetHardwareCursorStates()
{
	// clear all the overridden hardware cursors
	TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();
	if (ICursor* Cursor = PlatformCursor.Get())
	{
		for (auto& Entry : HardwareCursors)
		{
			Cursor->SetTypeShape(Entry.Key, nullptr);
		}
	}
}

bool UGameViewportClient::GetMousePosition(FVector2D& MousePosition) const
{
	bool bGotMousePosition = false;

	if (Viewport && FSlateApplication::Get().IsMouseAttached())
	{
		FIntPoint MousePos;
		Viewport->GetMousePos(MousePos);
		if (MousePos.X >= 0 && MousePos.Y >= 0)
		{
			MousePosition = FVector2D(MousePos);
			bGotMousePosition = true;
		}
	}

	return bGotMousePosition;
}

bool UGameViewportClient::RequiresUncapturedAxisInput() const
{
	bool bRequired = false;
	if (Viewport != NULL && Viewport->HasFocus())
	{
		if (ViewportConsole && ViewportConsole->ConsoleActive())
		{
			bRequired = true;
		}
		else if (GameInstance && GameInstance->GetFirstLocalPlayerController())
		{
			bRequired = GameInstance->GetFirstLocalPlayerController()->ShouldShowMouseCursor();
		}
	}

	return bRequired;
}


EMouseCursor::Type UGameViewportClient::GetCursor(FViewport* InViewport, int32 X, int32 Y)
{
	// If the viewport isn't active or the console is active we don't want to override the cursor
	if (!FSlateApplication::Get().IsActive())
	{
		return EMouseCursor::Default;
	}
	else if (!InViewport->HasMouseCapture() && !InViewport->HasFocus())
	{
		return EMouseCursor::Default;
	}
	else if (ViewportConsole && ViewportConsole->ConsoleActive())
	{
		return EMouseCursor::Default;
	}
	else if (GameInstance && GameInstance->GetFirstLocalPlayerController())
	{
		return GameInstance->GetFirstLocalPlayerController()->GetMouseCursor();
	}

	return FViewportClient::GetCursor(InViewport, X, Y);
}

void UGameViewportClient::SetVirtualCursorWidget(EMouseCursor::Type Cursor, UUserWidget* UserWidget)
{
	if (UserWidget)
	{
		SetSoftwareCursorWidget(Cursor, UserWidget);
	}
	else
	{
		CursorWidgets.Remove(Cursor);
	}
}

void UGameViewportClient::AddSoftwareCursorFromSlateWidget(EMouseCursor::Type InCursorType, TSharedPtr<SWidget> CursorWidgetPtr)
{
	// We set it only when it's not null to be on parity with the behavior we had before deprecation.
	if (CursorWidgetPtr.IsValid())
	{
		SetSoftwareCursorWidget(InCursorType, CursorWidgetPtr);
	}
}

void UGameViewportClient::AddSoftwareCursor(EMouseCursor::Type Cursor, const FSoftClassPath& CursorClass)
{
	SetSoftwareCursorFromClassPath(Cursor, CursorClass);
}

void UGameViewportClient::SetSoftwareCursorFromClassPath(EMouseCursor::Type Cursor, const FSoftClassPath & CursorClass)
{
	if (CursorClass.IsValid())
	{
		if (UClass* Class = CursorClass.TryLoadClass<UUserWidget>())
		{
			UUserWidget* UserWidget = CreateWidget(GetGameInstance(), Class);
			SetSoftwareCursorWidget(Cursor, UserWidget);
		}
		else
		{
			FMessageLog("PIE").Warning(FText::Format(LOCTEXT("SetSoftwareCursorFromClassPath:LoadFailed", "UGameViewportClient::SetSoftwareCursorFromClassPath: Could not load cursor class '{0}'."), FText::FromString(CursorClass.GetAssetName())));
		}
	}
	else
	{
		FMessageLog("PIE").Warning(LOCTEXT("SetSoftwareCursorFromClassPath:InvalidClass", "UGameViewportClient::SetSoftwareCursorFromClassPath: Invalid class specified."));
	}
}

void UGameViewportClient::SetSoftwareCursorWidget(EMouseCursor::Type InCursorType, TSharedPtr<SWidget> CursorWidgetPtr)
{
	if (CursorWidgetPtr.IsValid())
	{
		CursorWidgets.Emplace(InCursorType, CursorWidgetPtr);
	}
	else
	{
		CursorWidgets.Remove(InCursorType);
	}
}

void UGameViewportClient::SetSoftwareCursorWidget(EMouseCursor::Type InCursorType, class UUserWidget* UserWidget)
{
	if (UserWidget)
	{
		SetSoftwareCursorWidget(InCursorType, UserWidget->TakeWidget());
	}
	else
	{
		CursorWidgets.Remove(InCursorType);
	}
}

TSharedPtr<SWidget> UGameViewportClient::GetSoftwareCursorWidget(EMouseCursor::Type Cursor) const
{
	if (CursorWidgets.Contains(Cursor))
	{
		const TSharedPtr<SWidget> CursorWidgetPtr = CursorWidgets.FindRef(Cursor);
		if (CursorWidgetPtr.IsValid())
		{	
			return CursorWidgetPtr;
		}
	}
	return nullptr;
}

bool UGameViewportClient::HasSoftwareCursor(EMouseCursor::Type Cursor) const
{
	return CursorWidgets.Contains(Cursor);
}

void UGameViewportClient::AddCursorWidget(EMouseCursor::Type Cursor, class UUserWidget* CursorWidget)
{
	// We set it only when it's not null to be on parity with the behavior we had before deprecation.
	if (CursorWidget)
	{
		SetSoftwareCursorWidget(Cursor, CursorWidget->TakeWidget());
	}
}

TOptional<TSharedRef<SWidget>> UGameViewportClient::MapCursor(FViewport* InViewport, const FCursorReply& CursorReply)
{
	if (bUseSoftwareCursorWidgets)
	{
		if (CursorReply.GetCursorType() != EMouseCursor::None)
		{
			const TSharedPtr<SWidget> CursorWidgetPtr = CursorWidgets.FindRef(CursorReply.GetCursorType());

			if (CursorWidgetPtr.IsValid())
			{
				return CursorWidgetPtr.ToSharedRef();
			}
			else
			{
				UE_LOGF(LogPlayerManagement, Warning, "UGameViewportClient::MapCursor: Could not find cursor to map to %d.", int32(CursorReply.GetCursorType()));
			}
		}
	}

	return TOptional<TSharedRef<SWidget>>();
}

void UGameViewportClient::SetViewport( FSceneViewport* InViewport )
{
	FSceneViewport* PreviousViewport = Viewport;
	Viewport = InViewport;

	if ( PreviousViewport == NULL && Viewport != NULL )
	{
		// ensure that the player's Origin and Size members are initialized the moment we get a viewport
		LayoutPlayers();
	}
}

void UGameViewportClient::AddAssociation(FViewport& InViewport)
{
	FSceneViewport* SceneViewport = InViewport.AsSceneViewport();
	if (!SceneViewport || Viewport == SceneViewport)
	{
		return;
	}
 
	if (Viewport)
	{
		// "Primary" viewport is first-come first-served
		return;
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetViewport(SceneViewport); // To be inlined here after deprecation period
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (!GameLayerManager.IsValid())
	{
		SAssignNew(GameLayerManager, SGameLayerManager)
			.SceneViewport(SceneViewport)
			[
				SAssignNew(ViewportOverlayWidget, SOverlay)
			];
	}
	GameLayerManager->SetSceneViewport(SceneViewport);

	if (TSharedPtr<SViewport> ViewportWidget = SceneViewport->GetViewportWidget().Pin())
	{
		// If the viewport we're drawing to has an overlay as its child, insert instead of replacing (licensee code should not assume this behavior)
		if (TSharedPtr<SWidget> Child = ViewportWidget->GetContent())
		{
			if (Child->GetWidgetClass().GetWidgetType() == SOverlay::StaticWidgetClass().GetWidgetType())
			{
				StaticCastSharedPtr<SOverlay>(Child)->AddSlot(0)
					[
						GameLayerManager->AsWidget()
					];
				return;
			}
		}
		ViewportWidget->SetContent(GameLayerManager->AsWidget());
	}
}

void UGameViewportClient::RemoveAssociation(FViewport& InViewport)
{
	if (Viewport != &InViewport)
	{
		return;
	}

	FSceneViewport* SceneViewport = InViewport.AsSceneViewport();
	if (SceneViewport && GameLayerManager.IsValid())
	{
		TSharedPtr<SViewport> SlateViewport = SceneViewport->GetViewportWidget().Pin();
		if (SlateViewport.IsValid())
		{
			if (SlateViewport->GetContent() == GameLayerManager->AsWidget())
			{
				SlateViewport->SetContent(SNullWidget::NullWidget);
			}
			else if (SlateViewport->GetContent().IsValid() && SlateViewport->GetContent()->GetWidgetClass().GetWidgetType() == SOverlay::StaticWidgetClass().GetWidgetType())
			{
				TSharedPtr<SOverlay> Overlay = StaticCastSharedPtr<SOverlay>(SlateViewport->GetContent());
				Overlay->RemoveSlot(GameLayerManager->AsWidget());
			}
		}

		GameLayerManager->SetSceneViewport(nullptr);
	}

	// Without viewport cannot receive MouseLeave to undo the set-to-true from MouseEnter 
	if (FGenericPlatformMisc::DesktopTouchScreen())
	{
		FSlateApplication::Get().SetGameIsFakingTouchEvents(false);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetViewport(nullptr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedPtr<SWindow> UGameViewportClient::GetWindow() const
{
	if (TSharedPtr<SWindow> Pinned = Window.Pin())
	{
		return Pinned;
	}
	if (Viewport)
	{
		if (TSharedPtr<SViewport> ViewportWidget = Viewport->GetViewportWidget().Pin())
		{
			TSharedPtr<SWindow> Found = FSlateApplication::Get().FindWidgetWindow(ViewportWidget.ToSharedRef());
			Window = Found.ToWeakPtr();
			return Found;
		}
	}
	return nullptr;
}

void UGameViewportClient::SetViewportOverlayWidget(TSharedPtr< SWindow > InWindow, TSharedRef<SOverlay> InViewportOverlayWidget)
{
	Window = InWindow;
	ViewportOverlayWidget = InViewportOverlayWidget;
}

void UGameViewportClient::SetGameLayerManager(TSharedPtr< IGameLayerManager > LayerManager)
{
	GameLayerManager = LayerManager;
}

void UGameViewportClient::GetViewportSize( FVector2D& out_ViewportSize ) const
{
	if ( Viewport != NULL )
	{
		out_ViewportSize.X = Viewport->GetSizeXY().X;
		out_ViewportSize.Y = Viewport->GetSizeXY().Y;
	}
}

bool UGameViewportClient::IsFullScreenViewport() const
{
	if (Viewport != nullptr)
	{
		return Viewport->IsFullscreen();
	}

	return false;
}

bool UGameViewportClient::IsExclusiveFullscreenViewport() const
{
	if (Viewport != nullptr)
	{
		return Viewport->IsExclusiveFullscreen();
	}

	return false;
}

bool UGameViewportClient::ShouldForceFullscreenViewport() const
{
	bool bResult = false;
	if ( GForceFullscreen )
	{
		bResult = true;
	}
	else if ( GetOuterUEngine()->GetNumGamePlayers(this) == 0 )
	{
		bResult = true;
	}
	else if ( UWorld* MyWorld = GetWorld() )
	{
		if ( MyWorld->bIsDefaultLevel )
		{
			bResult = true;
		}
		else if ( GameInstance )
		{
			APlayerController* PlayerController = GameInstance->GetFirstLocalPlayerController();
			if( ( PlayerController ) && ( PlayerController->bCinematicMode ) )
			{
				bResult = true;
			}
		}
	}
	return bResult;
}

/** Util to find named canvas in transient package, and create if not found */
static UCanvas* GetCanvasByName(FName CanvasName)
{
	// Cache to avoid FString/FName conversions/compares
	static TMap<FName, UCanvas*> CanvasMap;
	UCanvas** FoundCanvas = CanvasMap.Find(CanvasName);
	if (!FoundCanvas)
	{
		UCanvas* CanvasObject = FindObject<UCanvas>(GetTransientPackage(),*CanvasName.ToString());
		if( !CanvasObject )
		{
			CanvasObject = NewObject<UCanvas>(GetTransientPackage(), CanvasName);
			CanvasObject->AddToRoot();
		}

		CanvasMap.Add(CanvasName, CanvasObject);
		return CanvasObject;
	}

	return *FoundCanvas;
}

EViewStatusForScreenPercentage UGameViewportClient::GetViewStatusForScreenPercentage() const
{
	if (EngineShowFlags.PathTracing)
	{
		return EViewStatusForScreenPercentage::PathTracer;
	}
	else if (EngineShowFlags.StereoRendering)
	{
		return EViewStatusForScreenPercentage::VR;
	}
	else if (World && World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
	{
		return EViewStatusForScreenPercentage::Mobile;
	}
	else
	{
		return EViewStatusForScreenPercentage::Desktop;
	}
}

void UGameViewportClient::Draw(FViewport* InViewport, FCanvas* SceneCanvas)
{
	//Valid SceneCanvas is required.  Make this explicit.
	check(SceneCanvas);
	check(GEngine);

	if (Viewport)
	{
		if (TSharedPtr<SViewport> ViewportWidget = Viewport->GetViewportWidget().Pin())
		{
			Window = FSlateApplication::Get().FindWidgetWindow(ViewportWidget.ToSharedRef());
		}
	}

	BeginDrawDelegate.Broadcast();

	const bool bStereoRendering = GEngine->IsStereoscopic3D(InViewport);
	FCanvas* DebugCanvas = InViewport->GetDebugCanvas();

	// Create a temporary canvas if there isn't already one.
	static FName CanvasObjectName("CanvasObject");
	UCanvas* CanvasObject = GetCanvasByName(CanvasObjectName);
	CanvasObject->Canvas = SceneCanvas;

	// Create temp debug canvas object
	FIntPoint DebugCanvasSize = InViewport->GetSizeXY();
	if (bStereoRendering && GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice())
	{
		DebugCanvasSize = GEngine->XRSystem->GetHMDDevice()->GetIdealDebugCanvasRenderTargetSize();
	}

	static FName DebugCanvasObjectName("DebugCanvasObject");
	UCanvas* DebugCanvasObject = GetCanvasByName(DebugCanvasObjectName);
	DebugCanvasObject->Init(DebugCanvasSize.X, DebugCanvasSize.Y, NULL, DebugCanvas);

	if (DebugCanvas)
	{
		DebugCanvas->SetScaledToRenderTarget(bStereoRendering);
		DebugCanvas->SetStereoRendering(bStereoRendering);
	}
	if (SceneCanvas)
	{
		SceneCanvas->SetScaledToRenderTarget(bStereoRendering);
		SceneCanvas->SetStereoRendering(bStereoRendering);
	}

	UWorld* MyWorld = GetWorld();
	if (MyWorld == nullptr)
	{
		return;
	}

	// create the view family for rendering the world scene to the viewport's render target
	bool bRequireMultiView = false;
	if (GEngine->IsStereoscopic3D())
	{
		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(MyWorld->Scene->GetShaderPlatform());
		bRequireMultiView = Aspects.IsMobileMultiViewEnabled();
	}

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport,
		MyWorld->Scene,
		EngineShowFlags)
		.SetRealtimeUpdate(true)
		.SetRequireMobileMultiView(bRequireMultiView));

	ViewFamily.DebugDPIScale = GetDPIScale();

	// Setup final resolution multiplier used by system relying on the final pixel size. Only used by the high resolution screenshot
	if (GIsHighResScreenshot)
	{
		ViewFamily.FinalResolutionScale = GetHighResScreenshotConfig().ResolutionMultiplier;
	}

#if WITH_EDITOR
	if (GIsEditor && ViewFamily.SupportsScreenPercentage())
	{
		// Force enable view family show flag for HighDPI derived's screen percentage.
		ViewFamily.EngineShowFlags.ScreenPercentage = true;
	}
#endif

	FSceneViewExtensionContext ViewExtensionContext(InViewport);
	ViewExtensionContext.bStereoEnabled = true;
	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);

	for (auto ViewExt : ViewFamily.ViewExtensions)
	{
		ViewExt->SetupViewFamily(ViewFamily);
	}

	uint32 StereoMobilePrimaryScalingMode = 0;
	if (bStereoRendering)
	{
		// Allow HMD to modify screen settings
		if (GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice())
		{
			GEngine->XRSystem->GetHMDDevice()->UpdateScreenSettings(Viewport);
		}
		
		// Update stereo flag in viewport client so we can accurately run GetViewStatusForScreenPercentage()
		static const bool bEmulateStereo = FParse::Param(FCommandLine::Get(), TEXT("emulatestereo"));
		EngineShowFlags.StereoRendering = bEmulateStereo ? true : ViewFamily.EngineShowFlags.StereoRendering;

		// Get the stereo mobile primary scaling mode, which determines if primary scaling (e.g. dynres) is redirected to the HMD runtime when otherwise unsupported
		if (World && World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
		{
			static const IConsoleVariable* StereoMobilePrimaryScalingModeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("xr.MobilePrimaryScalingMode"));
			if (StereoMobilePrimaryScalingModeCVar)
			{
				StereoMobilePrimaryScalingMode = StereoMobilePrimaryScalingModeCVar->GetInt();
				if (StereoMobilePrimaryScalingMode < 0 || StereoMobilePrimaryScalingMode > 2)
				{
					ensureMsgf(false, TEXT("Supported values for xr.MobilePrimaryScalingMode are 0-2, %d is unsupported. Using value 0 instead."), StereoMobilePrimaryScalingMode);
					StereoMobilePrimaryScalingMode = 0;
				}
			}
		}
	}

	ESplitScreenType::Type SplitScreenConfig = GetCurrentSplitscreenConfiguration();
	ViewFamily.ViewMode = EViewModeIndex(ViewModeIndex);
	EngineShowFlagOverride(ESFIM_Game, ViewFamily.ViewMode, ViewFamily.EngineShowFlags, false);

	if (ViewFamily.EngineShowFlags.VisualizeBuffer && AllowDebugViewmodes())
	{
		// Process the buffer visualization console command
		FName NewBufferVisualizationMode = NAME_None;
		static IConsoleVariable* ICVar = IConsoleManager::Get().FindConsoleVariable(FBufferVisualizationData::GetVisualizationTargetConsoleCommandName());
		if (ICVar)
		{
			static const FName OverviewName = "Overview";
			FString ModeNameString = ICVar->GetString();
			FName ModeName = *ModeNameString;
			if (ModeNameString.IsEmpty() || ModeName == OverviewName || ModeName == NAME_None)
			{
				NewBufferVisualizationMode = NAME_None;
			}
			else
			{
				if (GetBufferVisualizationData().GetMaterial(ModeName) == nullptr)
				{
					// Mode is out of range, so display a message to the user, and reset the mode back to the previous valid one
					UE_LOGF(LogConsoleResponse, Warning, "Buffer visualization mode '%ls' does not exist", *ModeNameString);
					NewBufferVisualizationMode = CurrentBufferVisualizationMode;
					// todo: cvars are user settings, here the cvar state is used to avoid log spam and to auto correct for the user (likely not what the user wants)
					ICVar->Set(*NewBufferVisualizationMode.GetPlainNameString(), ECVF_SetByCode);
				}
				else
				{
					NewBufferVisualizationMode = ModeName;
				}
			}
		}

		if (NewBufferVisualizationMode != CurrentBufferVisualizationMode)
		{
			CurrentBufferVisualizationMode = NewBufferVisualizationMode;
		}
	}

	// Setup the screen percentage and upscaling method for the view family.
	bool bFinalScreenPercentageShowFlag;
	bool bUsesDynamicResolution = false;
	{
		checkf(ViewFamily.GetScreenPercentageInterface() == nullptr,
			TEXT("Some code has tried to set up an alien screen percentage driver, that could be wrong if not supported very well by the RHI."));

		// Set up secondary resolution fraction for the view family (r.SecondaryScreenPercentage.GameViewport).
		// Skip if we're using stereo rendering and redirecting all secondary scaling to the HMD runtime (xr.SecondaryScreenPercentage.HMDRenderTarget).
		if (!bStereoRendering && ViewFamily.SupportsScreenPercentage())
		{
			float CustomSecondaryScreenPercentage = CVarSecondaryScreenPercentage.GetValueOnGameThread();

			if (CustomSecondaryScreenPercentage > 0.0)
			{
				// Override secondary resolution fraction with CVar.
				ViewFamily.SecondaryViewFraction = FMath::Min(CustomSecondaryScreenPercentage / 100.0f, 1.0f);
			}
			else
			{
				// Automatically compute secondary resolution fraction from DPI.
				ViewFamily.SecondaryViewFraction = GetDPIDerivedResolutionFraction();
			}

			check(ViewFamily.SecondaryViewFraction > 0.0f);
		}

#if WITH_DYNAMIC_RESOLUTION
		// Setup main view family with screen percentage interface by dynamic resolution if screen percentage is enabled.
		// Force screen percentage support if we're using stereo rendering and redirecting primary scaling to the HMD runtime.
		bool bDynamicResolutionScalingViaHMDRuntime = false;
		bool bAllResolutionScalingViaHMDRuntime = false;
		switch (StereoMobilePrimaryScalingMode)
		{
		case 0:		// Never
			bDynamicResolutionScalingViaHMDRuntime = false;
			bAllResolutionScalingViaHMDRuntime = false;
			break;
		case 1:		// Only if otherwise unsupported and dynamic resolution driven
			bDynamicResolutionScalingViaHMDRuntime = true;
			bAllResolutionScalingViaHMDRuntime = false;
			break;
		case 2:		// Only if otherwise unsupported
			bDynamicResolutionScalingViaHMDRuntime = true;
			bAllResolutionScalingViaHMDRuntime = true;
			break;
		default:
			checkNoEntry();
			break;
		}
		
		if (ViewFamily.EngineShowFlags.ScreenPercentage || bDynamicResolutionScalingViaHMDRuntime)
		{
			FDynamicResolutionStateInfos DynamicResolutionStateInfos;
			GEngine->GetDynamicResolutionCurrentStateInfos(/* out */ DynamicResolutionStateInfos);

			// Do not allow dynamic resolution to touch the view family if not supported to ensure there is no possibility to ruin
			// game play experience on platforms that does not support it, but have it enabled by mistake.
			if (DynamicResolutionStateInfos.Status == EDynamicResolutionStatus::Enabled)
			{
				GEngine->GetDynamicResolutionState()->SetupMainViewFamily(ViewFamily);

				bUsesDynamicResolution = ViewFamily.GetScreenPercentageInterface() != nullptr;
			}
			else if (DynamicResolutionStateInfos.Status == EDynamicResolutionStatus::DebugForceEnabled)
			{
				ViewFamily.EngineShowFlags.ScreenPercentage = true; // If this is false and we're scaling via HMD, need to set it to true now to avoid an assert in FLegacyScreenPercentageDriver()
				ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
					ViewFamily,
					DynamicResolutionStateInfos.ResolutionFractionApproximations[GDynamicPrimaryResolutionFraction],
					DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction]));

				bUsesDynamicResolution = true;
			}

			// Feed approximated resolution fraction to CSV
#if CSV_PROFILER
			if (DynamicResolutionStateInfos.ResolutionFractionApproximations[GDynamicPrimaryResolutionFraction] >= 0.0f)
			{
				// Keep same name as before for primary screen percentage
				CSV_CUSTOM_STAT_MINIMAL_GLOBAL(DynamicResolutionPercentage, DynamicResolutionStateInfos.ResolutionFractionApproximations[GDynamicPrimaryResolutionFraction] * 100.0f, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_GLOBAL(DynamicResolutionPercentageMax, DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction] * 100.0f, ECsvCustomStatOp::Set);
			}
			for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
			{
				const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
				if (Budget == GDynamicPrimaryResolutionFraction)
				{
					continue;
				}

				float Value = DynamicResolutionStateInfos.ResolutionFractionApproximations[Budget] * 100.0f;
				const char* NameChar = Budget.GetAnsiName();

				TRACE_CSV_PROFILER_INLINE_STAT(NameChar, CSV_CATEGORY_INDEX_GLOBAL);
				FCsvProfiler::RecordCustomStat(NameChar, CSV_CATEGORY_INDEX_GLOBAL, Value, ECsvCustomStatOp::Set);
			}
#endif // CSV_PROFILER
		}

		ViewFamily.EngineShowFlags.ScreenPercentage = ViewFamily.EngineShowFlags.ScreenPercentage || bUsesDynamicResolution || bAllResolutionScalingViaHMDRuntime;
#endif // WITH_DYNAMIC_RESOLUTION
		
		bFinalScreenPercentageShowFlag = ViewFamily.EngineShowFlags.ScreenPercentage;
	}

	TMap<ULocalPlayer*,FSceneView*> PlayerViewMap;
	TArray<FSceneView*> Views;
	bool bAnyPlayerCameraCut = false;

	for (FLocalPlayerIterator Iterator(GEngine, MyWorld); Iterator; ++Iterator)
	{
		ULocalPlayer* LocalPlayer = *Iterator;
		if (LocalPlayer && LocalPlayer->GetViewportClient() == this)
		{
			APlayerController* PlayerController = LocalPlayer->PlayerController;

			const int32 NumViews = bStereoRendering ? 
				GEngine->StereoRenderingDevice->GetDesiredNumberOfViews(bStereoRendering) 
				: (LocalPlayer->bEmulateSplitscreen ? 2 : 1);

			checkf(!bStereoRendering || !LocalPlayer->bEmulateSplitscreen, TEXT("r.ScreenshotTest.EmulateSplitscreenMode is unsupported when using stereo rendering."));

			for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
			{
				// Calculate the player's view information.
				FVector		ViewLocation;
				FRotator	ViewRotation;

				FSceneView* View = LocalPlayer->CalcSceneView(&ViewFamily, ViewLocation, ViewRotation, InViewport, nullptr, (bStereoRendering || LocalPlayer->bEmulateSplitscreen) ? ViewIndex : INDEX_NONE);

				if (View)
				{
					Views.Add(View);

					if (View->Family->EngineShowFlags.Wireframe)
					{
						// Wireframe color is emissive-only, and mesh-modifying materials do not use material substitution, hence...
						View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
						View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}
					else if (View->Family->EngineShowFlags.OverrideDiffuseAndSpecular)
					{
						View->DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
						View->SpecularOverrideParameter = FVector4f(.1f, .1f, .1f, 0.0f);
					}
					else if (View->Family->EngineShowFlags.LightingOnlyOverride)
					{
						View->DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
						View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}
					else if (View->Family->EngineShowFlags.ReflectionOverride)
					{
						View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
						View->SpecularOverrideParameter = FVector4f(1, 1, 1, 0.0f);
						View->NormalOverrideParameter = FVector4f(0, 0, 1, 0.0f);
						View->RoughnessOverrideParameter = FVector2f(0.0f, 0.0f);
					}

					if (!View->Family->EngineShowFlags.Diffuse)
					{
						View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}

					if (!View->Family->EngineShowFlags.Specular)
					{
						View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}

					if (!ViewFamily.EngineShowFlags.MaterialNormal)
					{
						View->NormalOverrideParameter = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
					}

					if (!ViewFamily.EngineShowFlags.MaterialAmbientOcclusion)
					{
						View->AmbientOcclusionOverrideParameter = FVector2f(1.0f, 0.0f);
					}

					View->CurrentBufferVisualizationMode = CurrentBufferVisualizationMode;
					View->CurrentNaniteVisualizationMode = CurrentNaniteVisualizationMode;
					View->CurrentLumenVisualizationMode = CurrentLumenVisualizationMode;
					View->CurrentMegaLightsVisualizationMode = CurrentMegaLightsVisualizationMode;
					View->CurrentSubstrateVisualizationMode = CurrentSubstrateVisualizationMode;
					View->CurrentGroomVisualizationMode = CurrentGroomVisualizationMode;
					View->CurrentVirtualShadowMapVisualizationMode = CurrentVirtualShadowMapVisualizationMode;

					View->CameraConstrainedViewRect = View->UnscaledViewRect;

					// If this is the primary drawing pass, update things that depend on the view location
					if (ViewIndex == 0)
					{
						// Save the location of the view.
						LocalPlayer->LastViewLocation = ViewLocation;

						PlayerViewMap.Add(LocalPlayer, View);

						// Update the listener.
						if (AudioDevice && PlayerController != NULL)
						{
							bool bUpdateListenerPosition = true;

							// If the main audio device is used for multiple PIE viewport clients, we only
							// want to update the main audio device listener position if it is in focus
							if (GEngine)
							{
								FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();

								// If there is more than one world referencing the main audio device
								if (AudioDeviceManager->GetNumMainAudioDeviceWorlds() > 1)
								{
									Audio::FDeviceId MainAudioDeviceID = GEngine->GetMainAudioDeviceID();
									if (AudioDevice->DeviceID == MainAudioDeviceID && !bHasAudioFocus)
									{
										bUpdateListenerPosition = false;
									}
								}
							}

							if (bUpdateListenerPosition)
							{
								FVector Location;
								FVector ProjFront;
								FVector ProjRight;
								PlayerController->GetAudioListenerPosition(/*out*/ Location, /*out*/ ProjFront, /*out*/ ProjRight);

								FTransform ListenerTransform(FRotationMatrix::MakeFromXY(ProjFront, ProjRight));

								// Allow the HMD to adjust based on the head position of the player, as opposed to the view location
								if (GEngine->XRSystem.IsValid() && GEngine->StereoRenderingDevice.IsValid() && GEngine->StereoRenderingDevice->IsStereoEnabled())
								{
									const FVector Offset = GEngine->XRSystem->GetAudioListenerOffset();
									Location += ListenerTransform.TransformPositionNoScale(Offset);
								}

								ListenerTransform.SetTranslation(Location);
								ListenerTransform.NormalizeRotation();

								uint32 ViewportIndex = PlayerViewMap.Num() - 1;
								AudioDevice->SetListener(MyWorld, ViewportIndex, ListenerTransform, (View->bCameraCut ? 0.f : MyWorld->GetDeltaSeconds()));

								FVector OverrideAttenuation;
								if (PlayerController->GetAudioListenerAttenuationOverridePosition(OverrideAttenuation))
								{
									AudioDevice->SetListenerAttenuationOverride(ViewportIndex, OverrideAttenuation);
								}
								else
								{
									AudioDevice->ClearListenerAttenuationOverride(ViewportIndex);
								}
							}
						}
					}

					AddStreamingViewInfo(*World, *View);
					World->LastRenderTime = World->GetTimeSeconds();
				}
			}

			if (PlayerController)
			{
				if (APlayerCameraManager* const CameraManager = PlayerController->PlayerCameraManager)
				{
					bAnyPlayerCameraCut = bAnyPlayerCameraCut || CameraManager->bGameCameraCutThisFrame;
					// Assumes any other systems that call ULocalPlayer::CalcSceneView(...) and care about camera cuts will do so before this happens.
					CameraManager->bGameCameraCutThisFrame = false;
				}
			}
		}
	}

	InViewport->SetCameraCut(bAnyPlayerCameraCut);

#if CSV_PROFILER_STATS
	UpdateCsvCameraStats(PlayerViewMap);
#endif

#if OBJECT_TRACE_ENABLED 
	for (TMap<ULocalPlayer*, FSceneView*>::TConstIterator It(PlayerViewMap); It; ++It)
	{
		ULocalPlayer* LocalPlayer = It.Key();
		FSceneView* SceneView = It.Value();
		TRACE_VIEW(LocalPlayer, SceneView);	
	}
#endif

	FinalizeViews(&ViewFamily, PlayerViewMap);

	// Find largest rectangle bounded by all rendered views.
	uint32 MinX=InViewport->GetSizeXY().X, MinY=InViewport->GetSizeXY().Y, MaxX=0, MaxY=0;
	uint32 TotalArea = 0;
	{
		for( int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ++ViewIndex )
		{
			const FSceneView* View = ViewFamily.Views[ViewIndex];

			FIntRect UpscaledViewRect = View->UnscaledViewRect;

			MinX = FMath::Min<uint32>(UpscaledViewRect.Min.X, MinX);
			MinY = FMath::Min<uint32>(UpscaledViewRect.Min.Y, MinY);
			MaxX = FMath::Max<uint32>(UpscaledViewRect.Max.X, MaxX);
			MaxY = FMath::Max<uint32>(UpscaledViewRect.Max.Y, MaxY);
			TotalArea += UpscaledViewRect.Width() * UpscaledViewRect.Height();
		}

		// To draw black borders around the rendered image (prevents artifacts from post processing passes that read outside of the image e.g. PostProcessAA)
		{
			int32 BlackBorders = FMath::Clamp(CVarSetBlackBordersEnabled.GetValueOnGameThread(), 0, 10);

			if(ViewFamily.Views.Num() == 1 && BlackBorders)
			{
				MinX += BlackBorders;
				MinY += BlackBorders;
				MaxX -= BlackBorders;
				MaxY -= BlackBorders;
				TotalArea = (MaxX - MinX) * (MaxY - MinY);
			}
		}
	}

	// If the views don't cover the entire bounding rectangle, clear the entire buffer.
	bool bBufferCleared = false;
	bool bStereoscopicPass = (ViewFamily.Views.Num() != 0 && IStereoRendering::IsStereoEyeView(*ViewFamily.Views[0]));
	bool bTransparentViewportClear = CVarTransparentViewportClear.GetValueOnAnyThread();
	if (ViewFamily.Views.Num() == 0 || TotalArea != (MaxX-MinX)*(MaxY-MinY) || bDisableWorldRendering || bStereoscopicPass || bTransparentViewportClear)
	{
		if (bDisableWorldRendering || !bStereoscopicPass || bTransparentViewportClear) // TotalArea computation does not work correctly for stereoscopic views
		{
			SceneCanvas->Clear(FLinearColor::Transparent);
		}

		bBufferCleared = true;
	}

	{
		// If a screen percentage interface was not set by dynamic resolution, then create one matching legacy behavior.
		if (ViewFamily.GetScreenPercentageInterface() == nullptr)
		{
			float GlobalResolutionFraction = 1.0f;

			if (ViewFamily.EngineShowFlags.ScreenPercentage && !bDisableWorldRendering && ViewFamily.Views.Num() > 0)
			{
				// Get global view fraction.
				FStaticResolutionFractionHeuristic StaticHeuristic;
				StaticHeuristic.Settings.PullRunTimeRenderingSettings(GetViewStatusForScreenPercentage());
				StaticHeuristic.PullViewFamilyRenderingSettings(ViewFamily);
				StaticHeuristic.DPIScale = GetDPIScale();

				GlobalResolutionFraction = StaticHeuristic.ResolveResolutionFraction();
			}

			ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
				ViewFamily, GlobalResolutionFraction));
		}

		check(ViewFamily.GetScreenPercentageInterface() != nullptr);

		// Change screen percentage method to raw output when using stereo rendering and redirecting primary scaling to the HMD runtime.
		if (bStereoRendering)
		{
			bool bUsingRawOutput = false;
			switch (StereoMobilePrimaryScalingMode)
			{
			case 0:		// Never
				bUsingRawOutput = false;
				break;
			case 1:		// Only if otherwise unsupported and dynamic-resolution driven
				bUsingRawOutput = !ViewFamily.SupportsScreenPercentage() && bUsesDynamicResolution;
				break;
			case 2:		// Only if otherwise unsupported
				bUsingRawOutput = !ViewFamily.SupportsScreenPercentage() && ViewFamily.EngineShowFlags.ScreenPercentage;
				break;
			default:
				checkNoEntry();
				break;
			}

			if (bUsingRawOutput)
			{
				for (FSceneView* View : Views)
				{
					static bool bWarnOnce = true;

					if (bWarnOnce && View->PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale)
					{
						UE_LOGF(LogViewport, Warning, "Temporal upscaling is unsupported when redirecting primary scaling to the HMD runtime. The HMD runtime's spatial scaling will be used instead.");
						bWarnOnce = false;
					}

					View->PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::RawOutput;
				}
			}
		}

		// Make sure the engine show flag for screen percentage is still what it was when setting up the screen percentage interface
		ViewFamily.EngineShowFlags.ScreenPercentage = bFinalScreenPercentageShowFlag;

		// Make sure the screen percentage show flag is not set to true if screen percentage is unsupported.
		// If we're using RawOutput due to the setup above, SupportsScreenPercentage() should now return true.
		checkf(!(ViewFamily.EngineShowFlags.ScreenPercentage && !ViewFamily.SupportsScreenPercentage()), TEXT("Screen percentage is not supported, but show flag was incorectly set to true."));
	}

	ViewFamily.bIsHDR = GetWindow().IsValid() ? GetWindow().Get()->GetIsHDR() : false;

	ViewFamily.bSplitScreenDebugAllowed = true;
	ViewFamily.bIsMainViewFamily = true;

	// Draw the player views.
	if (!bDisableWorldRendering && PlayerViewMap.Num() > 0 && FSlateApplication::Get().GetPlatformApplication()->IsAllowedToRender())
	{
		// Scene view extension SetupView calls already done in LocalPlayer->CalcSceneView above.

		GetRendererModule().BeginRenderingViewFamily(SceneCanvas, &ViewFamily);
	}
	else
	{
		GetRendererModule().PerFrameCleanupIfSkipRenderer();
	}

	// Clear areas of the rendertarget (backbuffer) that aren't drawn over by the views.
	if (!bBufferCleared)
	{
		const FLinearColor ClearColor = bTransparentViewportClear ? FLinearColor::Transparent : FLinearColor::Black;

		// clear left
		if( MinX > 0 )
		{
			SceneCanvas->DrawTile(0,0,MinX,InViewport->GetSizeXY().Y,0.0f,0.0f,1.0f,1.f,ClearColor,NULL,false);
		}
		// clear right
		if( MaxX < (uint32)InViewport->GetSizeXY().X )
		{
			SceneCanvas->DrawTile(MaxX,0,InViewport->GetSizeXY().X,InViewport->GetSizeXY().Y,0.0f,0.0f,1.0f,1.f,ClearColor,NULL,false);
		}
		// clear top
		if( MinY > 0 )
		{
			SceneCanvas->DrawTile(MinX,0,MaxX,MinY,0.0f,0.0f,1.0f,1.f,ClearColor,NULL,false);
		}
		// clear bottom
		if( MaxY < (uint32)InViewport->GetSizeXY().Y )
		{
			SceneCanvas->DrawTile(MinX,MaxY,MaxX,InViewport->GetSizeXY().Y,0.0f,0.0f,1.0f,1.f,ClearColor,NULL,false);
		}
	}

	// Remove temporary debug lines.
	constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
	MyWorld->FlushLineBatchers(LineBatchersToFlush);

	// Draw FX debug information.
	if (MyWorld->FXSystem)
	{
		MyWorld->FXSystem->DrawDebug(SceneCanvas);
	}

#if WITH_STATE_STREAM
	if (IStateStreamManager* Manager = MyWorld->GetStateStreamManager())
	{
		class FStateStreamDebugRenderer : public IStateStreamDebugRenderer
		{
		public:
			FStateStreamDebugRenderer(FViewport* V, UCanvas* C) : Viewport(V), Canvas(C) {}

			virtual void DrawText(const FStringView& Text, const FColor& Color) override
			{
				Canvas->SetDrawColor(Color);
				Canvas->DrawText(GEngine->GetSmallFont(), Text, 100, Y);
				Y += 20;
			}

			virtual void DrawText(uint32 InX, uint32 InY, const FStringView& Text, const FColor& Color) override
			{
				Canvas->SetDrawColor(Color);
				Canvas->DrawText(GEngine->GetSmallFont(), Text, InX, InY);
			}

			uint32 Y = 200;
			FViewport* Viewport;
			UCanvas* Canvas;
		} StateStreamDebugRenderer(InViewport, DebugCanvasObject);
		Manager->Game_DebugRender(StateStreamDebugRenderer, 0);
	}
#endif

	// Render the UI.
	if (FSlateApplication::Get().GetPlatformApplication()->IsAllowedToRender())
	{
		SCOPE_CYCLE_COUNTER(STAT_UIDrawingTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UI);

		// render HUD
		bool bDisplayedSubtitles = false;
		for( FConstPlayerControllerIterator Iterator = MyWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController)
			{
				ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player);
				if( LocalPlayer )
				{
					FSceneView* View = PlayerViewMap.FindRef(LocalPlayer);
					if (View != NULL)
					{
						// rendering to directly to viewport target
						FVector CanvasOrigin(FMath::TruncToFloat((float)View->UnscaledViewRect.Min.X), FMath::TruncToFloat((float)View->UnscaledViewRect.Min.Y), 0.f);

						CanvasObject->Init(View->UnscaledViewRect.Width(), View->UnscaledViewRect.Height(), View, SceneCanvas);

						// Set the canvas transform for the player's view rectangle.
						check(SceneCanvas);
						SceneCanvas->PushAbsoluteTransform(FTranslationMatrix(CanvasOrigin));
						CanvasObject->ApplySafeZoneTransform();

						// Render the player's HUD.
						if( PlayerController->MyHUD )
						{
							SCOPE_CYCLE_COUNTER(STAT_HudTime);

							DebugCanvasObject->SceneView = View;
							PlayerController->MyHUD->SetCanvas(CanvasObject, DebugCanvasObject);

							PlayerController->MyHUD->PostRender();

							// Put these pointers back as if a blueprint breakpoint hits during HUD PostRender they can
							// have been changed
							CanvasObject->Canvas = SceneCanvas;
							DebugCanvasObject->Canvas = DebugCanvas;

							// A side effect of PostRender is that the playercontroller could be destroyed
							if (IsValid(PlayerController))
							{
								PlayerController->MyHUD->SetCanvas(NULL, NULL);
							}
						}

						if (DebugCanvas != nullptr)
						{
							DebugCanvas->PushAbsoluteTransform(FTranslationMatrix(CanvasOrigin));
							UDebugDrawService::Draw(ViewFamily.EngineShowFlags, InViewport, View, DebugCanvas, DebugCanvasObject, PlayerController);
							DebugCanvas->PopTransform();
						}

						CanvasObject->PopSafeZoneTransform();
						SceneCanvas->PopTransform();

						// draw subtitles
						if (!bDisplayedSubtitles)
						{
							FVector2D MinPos(0.f, 0.f);
							FVector2D MaxPos(1.f, 1.f);
							GetSubtitleRegion(MinPos, MaxPos);

							const uint32 SizeX = SceneCanvas->GetRenderTarget()->GetSizeXY().X;
							const uint32 SizeY = SceneCanvas->GetRenderTarget()->GetSizeXY().Y;
							FIntRect SubtitleRegion(FMath::TruncToInt(SizeX * MinPos.X), FMath::TruncToInt(SizeY * MinPos.Y), FMath::TruncToInt(SizeX * MaxPos.X), FMath::TruncToInt(SizeY * MaxPos.Y));
							FSubtitleManager::GetSubtitleManager()->DisplaySubtitles( SceneCanvas, SubtitleRegion, MyWorld->GetAudioTimeSeconds() );
							bDisplayedSubtitles = true;
						}
					}
				}
			}
		}

		//ensure canvas has been flushed before rendering UI
		SceneCanvas->Flush_GameThread();

		DrawnDelegate.Broadcast();

		// Allow the viewport to render additional stuff
		PostRender(DebugCanvasObject);
	}


	// Grab the player camera location and orientation so we can pass that along to the stats drawing code.
	FVector PlayerCameraLocation = FVector::ZeroVector;
	FRotator PlayerCameraRotation = FRotator::ZeroRotator;
	{
		for( FConstPlayerControllerIterator Iterator = MyWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			if (APlayerController* PC = Iterator->Get())
			{
				PC->GetPlayerViewPoint(PlayerCameraLocation, PlayerCameraRotation);
			}
		}
	}

	if (DebugCanvas)
	{
		// Reset the debug canvas to be full-screen before drawing the console
		// (the debug draw service above has messed with the viewport size to fit it to a single player's subregion)
		DebugCanvasObject->Init(DebugCanvasSize.X, DebugCanvasSize.Y, NULL, DebugCanvas);

		DrawStatsHUD( MyWorld, InViewport, DebugCanvas, DebugCanvasObject, DebugProperties, PlayerCameraLocation, PlayerCameraRotation );

		if (GEngine->IsStereoscopic3D(InViewport))
		{
#if 0 //!UE_BUILD_SHIPPING
			// TODO: replace implementation in OculusHMD with a debug renderer
			if (GEngine->XRSystem.IsValid())
			{
				GEngine->XRSystem->DrawDebug(DebugCanvasObject);
			}
#endif
		}

		// Render the console absolutely last because developer input is was matter the most.
		if (ViewportConsole)
		{
			ViewportConsole->PostRender_Console(DebugCanvasObject);
		}
	}

	EndDrawDelegate.Broadcast();
}

template<class FColorType, typename TChannelType>
bool ProcessScreenshotData(TArray<FColorType>& Bitmap, FIntVector Size, TChannelType OpaqueAlphaValue, bool bHdrEnabled, bool bIsUI, const TCHAR* ToExtension)
{
	bool bIsScreenshotSaved = false;
	{
		FString ScreenShotName = FScreenshotRequest::GetFilename();
		if (GIsDumpingMovie && ScreenShotName.IsEmpty())
		{
			// Request a new screenshot with a formatted name
			const bool bShowUI = false;
			const bool bAddFilenameSuffix = true;
			FScreenshotRequest::RequestScreenshot(FString(), bShowUI, bAddFilenameSuffix, bHdrEnabled);
			ScreenShotName = FScreenshotRequest::GetFilename();
		}

		// If a screenshot is requested during PIE (via F9), it does a screenshot of the entire editor window, including UI.
		// We need to ignore the high resolution screenshot alpha mask in that case, as the mask isn't relevant when taking a
		// screenshot of the entire window (and it will trigger an assert).  We don't want to solve this by modifying the
		// global variables associated with the screenshot feature, as the application may be taking its own screenshots.
		if (GIsHighResScreenshot && !bIsUI)
		{
			GetHighResScreenshotConfig().MergeMaskIntoAlpha(Bitmap, FIntRect(0, 0, 0, 0));
		}
		else
		{
			// Ensure that all pixels' alpha is set to opaque for a regular screenshot regardless of UI settings
			// (Regular screenshots return with 0 in the alpha channels)
			for (auto& Color : Bitmap)
			{
				Color.A = OpaqueAlphaValue;
			}
		}

		FIntRect SourceRect(0, 0, GScreenshotResolutionX, GScreenshotResolutionY);
		if (GIsHighResScreenshot)
		{
			SourceRect = GetHighResScreenshotConfig().CaptureRegion;
		}

		// Clip the bitmap to just the capture region if valid
		if (!SourceRect.IsEmpty())
		{
			const int32 OldWidth = Size.X;
			const int32 OldHeight = Size.Y;

			//clamp in bounds:
			int CaptureMinX = FMath::Clamp(SourceRect.Min.X, 0, OldWidth);
			int CaptureMinY = FMath::Clamp(SourceRect.Min.Y, 0, OldHeight);

			int CaptureMaxX = FMath::Clamp(SourceRect.Max.X, 0, OldWidth);
			int CaptureMaxY = FMath::Clamp(SourceRect.Max.Y, 0, OldHeight);

			int32 NewWidth = CaptureMaxX - CaptureMinX;
			int32 NewHeight = CaptureMaxY - CaptureMinY;

			if (NewWidth > 0 && NewHeight > 0 && ((NewWidth != OldWidth) || (NewHeight != OldHeight)))
			{
				FColorType* const Data = Bitmap.GetData();

				for (int32 Row = 0; Row < NewHeight; Row++)
				{
					FMemory::Memmove(Data + Row * NewWidth, Data + (Row + CaptureMinY) * OldWidth + CaptureMinX, NewWidth * sizeof(*Data));
				}

				Bitmap.RemoveAt(NewWidth * NewHeight, OldWidth * OldHeight - NewWidth * NewHeight, EAllowShrinking::No);
				Size = FIntVector(NewWidth, NewHeight, 0);
			}
		}

		if (FPaths::GetExtension(ScreenShotName).IsEmpty())
		{
			ScreenShotName += ToExtension;
		}

		bool bSuppressWritingToFile = false;
#if UE_SCREENSHOT_TRACE_ENABLED
		if (SHOULD_TRACE_SCREENSHOT())
		{
			bSuppressWritingToFile = FTraceScreenshot::ShouldSuppressWritingToFile();
			FTraceScreenshot::TraceScreenshot(Size.X, Size.Y, Bitmap, ScreenShotName);
		}
#endif

		// Save the contents of the array to a png file.
		if (!bSuppressWritingToFile)
		{
			FImageView Image((const FColorType*)Bitmap.GetData(), Size.X, Size.Y);
			bIsScreenshotSaved = FImageUtils::SaveImageByExtension(*ScreenShotName, Image);
		}
	}

	return bIsScreenshotSaved;
}

bool UGameViewportClient::ProcessScreenShots(FViewport* InViewport)
{
	bool bIsScreenshotSaved = false;

	if (GIsDumpingMovie || FScreenshotRequest::IsScreenshotRequested() || GIsHighResScreenshot)
	{
		TArray<FColor> Bitmap;
		TArray<FLinearColor> BitmapHDR;

		bool bShowUI = false;
		TSharedPtr<SWindow> WindowPtr = GetWindow();
		if (!GIsDumpingMovie && (FScreenshotRequest::ShouldShowUI() && WindowPtr.IsValid()))
		{
			bShowUI = true;
		}

		bool bScreenshotSuccessful = false;
		bool bIsUI = false;
		FIntVector Size(InViewport->GetRenderTargetTextureSizeXY().X, InViewport->GetRenderTargetTextureSizeXY().Y, 0);

		EDisplayOutputFormat ViewportOutputFormat = InViewport->GetDisplayOutputFormat();
		bool bHdrEnabled = InViewport->GetSceneHDREnabled();

		if( bShowUI && FSlateApplication::IsInitialized() )
		{
			ensureMsgf(FScreenshotRequest::GetViewRect().IsEmpty(), TEXT("Specifying a screenshot view rect is not supported for slate widget screenshots (bShowUI=true). Using the full window size."));

			TSharedRef<SWidget> WindowRef = WindowPtr.ToSharedRef();
			if (FScreenshotRequest::ShouldRestrictToGameViewport())
			{
				TSharedPtr<SViewport> GameViewportWidgetPtr = GetGameViewportWidget();
				if (GameViewportWidgetPtr.IsValid())
				{
					WindowRef = GameViewportWidgetPtr.ToSharedRef();
				}
			}

			if (bHdrEnabled)
			{
				bScreenshotSuccessful = FSlateApplication::Get().TakeHDRScreenshot(WindowRef, BitmapHDR, Size);
				ConvertPixelDataToSCRGB(BitmapHDR, ViewportOutputFormat);
			}
			else
			{
				bScreenshotSuccessful = FSlateApplication::Get().TakeScreenshot(WindowRef, Bitmap, Size);
			}
			GScreenshotResolutionX = Size.X;
			GScreenshotResolutionY = Size.Y;
			bIsUI = true;
		}
		else
		{
			// Set up view rect
			FIntRect ViewRect = FScreenshotRequest::GetViewRect();
			if (ViewRect.IsEmpty())
			{
				// If an empty view rect was provided (default), screenshot the entire render target
				ViewRect = FIntRect(0, 0, Size.X, Size.Y);
			}
			else if (!FIntRect(0, 0, Size.X, Size.Y).Contains(ViewRect))
			{
				ensureMsgf(false, TEXT("Requested a screenshot with view rect %s, which does not fit within the viewport size of %dx%d. Using the full viewport size."), *ViewRect.ToString(), Size.X, Size.Y);
				ViewRect = FIntRect(0, 0, Size.X, Size.Y);
			}
			else
			{
				// If a valid view rect was provided, update the output size to match the view rect size
				Size.X = ViewRect.Width();
				Size.Y = ViewRect.Height();
			}

			check(FIntPoint(Size.X, Size.Y) == ViewRect.Size());

			// Take screenshot
			if (bHdrEnabled)
			{
				bScreenshotSuccessful = GetViewportScreenShotHDR(InViewport, BitmapHDR, ViewRect);
				ConvertPixelDataToSCRGB(BitmapHDR, ViewportOutputFormat);
			}
			else if (InViewport->GetShaderResourceTexture() && InViewport->GetShaderResourceTexture()->GetDesc().Dimension == ETextureDimension::Texture2DArray)
			{
				// Handle Texture2DArray, we need to determine which array slices to include
				FReadSurfaceDataFlags ReadSurfaceFlags = FReadSurfaceDataFlags();
				const int32 ScreenshotTextureArrayMode = CVarScreenshotTextureArrayMode.GetValueOnAnyThread();
				const int32 ArraySize = InViewport->GetShaderResourceTexture()->GetDesc().ArraySize;

				ensureAlwaysMsgf(ScreenshotTextureArrayMode >= 0 && ScreenshotTextureArrayMode < 3, TEXT("r.Screenshot.TextureArrayMode %d is not a valid mode. Defaulting to mode 0."), ScreenshotTextureArrayMode);

				// Screenshot all slices, then collate horizontally or vertically
				if (ScreenshotTextureArrayMode == 1 || ScreenshotTextureArrayMode == 2)
				{
					TArray<TArray<FColor>> BitmapArray;
					BitmapArray.SetNum(ArraySize);

					for (int CurrentArrayIndex = 0; CurrentArrayIndex < ArraySize; CurrentArrayIndex++)
					{
						ReadSurfaceFlags.SetArrayIndex(CurrentArrayIndex);
						bScreenshotSuccessful = GetViewportScreenShot(InViewport, BitmapArray[CurrentArrayIndex], ViewRect, ReadSurfaceFlags);

						if (!bScreenshotSuccessful)
						{
							break;
						}
					}

					if (bScreenshotSuccessful)
					{
						const int32 NumPixelsPerSlice = Size.X * Size.Y;
						const int32 NumPixelsTotal = NumPixelsPerSlice * ArraySize;

						Bitmap.Reset();
						Bitmap.SetNumUninitialized(NumPixelsTotal);

						// Horizontal collation
						if (ScreenshotTextureArrayMode == 1)
						{
							for (int CurrentPixel = 0; CurrentPixel < NumPixelsTotal; CurrentPixel++)
							{
								// Complex case, need to alternate between array entries for each horizontal line
								const int32 CurrentRow = CurrentPixel / (ArraySize * Size.X);
								const int32 CurrentColumn = CurrentPixel % Size.X;
								 
								const int32 ArrayIndex = (CurrentPixel / Size.X) % ArraySize;
								const int32 PixelIndex = CurrentColumn + CurrentRow * Size.X;
								Bitmap[CurrentPixel] = BitmapArray[ArrayIndex][PixelIndex];
							}

							Size.X *= ArraySize;
							GScreenshotResolutionX = Size.X;
						}

						// Vertical collation
						else
						{
							for (int CurrentPixel = 0; CurrentPixel < NumPixelsTotal; CurrentPixel++)
							{
								// Simple case, just concatenate all bitmaps
								const int32 ArrayIndex = CurrentPixel / NumPixelsPerSlice;
								const int32 PixelIndex = CurrentPixel % NumPixelsPerSlice;
								Bitmap[CurrentPixel] = BitmapArray[ArrayIndex][PixelIndex];
							}

							Size.Y *= ArraySize;
							GScreenshotResolutionY = Size.Y;
						}
					}
				}

				// Screenshot a specific array slice (ScreenshotTextureArrayMode == 0 or other invalid values)
				else
				{
					int32 ArrayIndex = CVarScreenshotTextureArrayIndex.GetValueOnAnyThread();
					const int32 ClampedArrayIndex = FMath::Clamp(ArrayIndex, 0, ArraySize - 1);

					if (ArrayIndex != ClampedArrayIndex)
					{
						ensureAlwaysMsgf(false, TEXT("r.Screenshot.TextureArrayIndex %d is out of range for the current viewport, which has %d texture array slices. Defaulting to slice %d."), ArrayIndex, ArraySize, ClampedArrayIndex);
						ArrayIndex = ClampedArrayIndex;
					}

					ReadSurfaceFlags.SetArrayIndex(ArrayIndex);
					bScreenshotSuccessful = GetViewportScreenShot(InViewport, Bitmap, ViewRect, ReadSurfaceFlags);
				}
			}
			else
			{
				bScreenshotSuccessful = GetViewportScreenShot(InViewport, Bitmap, ViewRect);
			}
		}

		if (bScreenshotSuccessful)
		{
			if (Bitmap.Num() > 0)
			{
				if (ScreenshotCapturedDelegate.IsBound() && CVarScreenshotDelegate.GetValueOnGameThread())
				{
					// Ensure that all pixels' alpha is set to 255
					for (FColor& Color : Bitmap)
					{
						Color.A = 255;
					}

					// If delegate subscribed, fire it instead of writing out a file to disk
					ScreenshotCapturedDelegate.Broadcast(Size.X, Size.Y, Bitmap);
				}
				else
				{
					bIsScreenshotSaved = ProcessScreenshotData(Bitmap, Size, 255, bHdrEnabled, bIsUI, TEXT(".png"));
				}
			}
			else
			{
				if (HDRScreenshotCapturedDelegate.IsBound() && CVarScreenshotDelegate.GetValueOnGameThread())
				{
					// Ensure that all pixels' alpha is set to 1.0f
					for (FLinearColor& LinearColor : BitmapHDR)
					{
						LinearColor.A = 1.0f;
					}

					// If delegate subscribed, fire it instead of writing out a file to disk
					HDRScreenshotCapturedDelegate.Broadcast(Size.X, Size.Y, BitmapHDR);
				}
				else
				{
					bIsScreenshotSaved = ProcessScreenshotData(BitmapHDR, Size, 1.0f, bHdrEnabled, bIsUI, TEXT(".exr"));
				}
			}
		}

		FScreenshotRequest::Reset();
#if UE_SCREENSHOT_TRACE_ENABLED
		FTraceScreenshot::Reset();
#endif
		FScreenshotRequest::OnScreenshotRequestProcessed().Broadcast();

		// Reeanble screen messages - if we are NOT capturing a movie
		GAreScreenMessagesEnabled = GScreenMessagesRestoreState;
	}

	return bIsScreenshotSaved;
}

void UGameViewportClient::Precache()
{
	if(!GIsEditor)
	{
		// Precache sounds...
		if (AudioDevice)
		{
			UE_LOGF(LogPlayerManagement, Log, "Precaching sounds...");
			for(TObjectIterator<USoundWave> It;It;++It)
			{
				USoundWave* SoundWave = *It;
				AudioDevice->Precache( SoundWave, false, false, false );
			}
			UE_LOGF(LogPlayerManagement, Log, "Precaching sounds completed...");
		}
	}

	// Log time till first precache is finished.
	static bool bIsFirstCallOfFunction = true;
	if( bIsFirstCallOfFunction )
	{
		UE_LOGF(LogPlayerManagement, Log, "%5.2f seconds passed since startup.",FPlatformTime::Seconds()-GStartTime);
		bIsFirstCallOfFunction = false;
	}
}

TOptional<bool> UGameViewportClient::QueryShowFocus(const EFocusCause InFocusCause) const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>(UUserInterfaceSettings::StaticClass());

	if ( UISettings->RenderFocusRule == ERenderFocusRule::Never ||
		(UISettings->RenderFocusRule == ERenderFocusRule::NonPointer && InFocusCause == EFocusCause::Mouse) ||
		(UISettings->RenderFocusRule == ERenderFocusRule::NavigationOnly && InFocusCause != EFocusCause::Navigation))
	{
		return false;
	}

	return true;
}

void UGameViewportClient::LostFocus(FViewport* InViewport)
{
	// We need to reset some key inputs, since keyup events will sometimes not be processed (such as going into immersive/maximized mode).
	// Resetting them will prevent them from "sticking"
	UWorld* const ViewportWorld = GetWorld();
	const bool bShouldFlush = GetDefault<UInputSettings>()->bShouldFlushPressedKeysOnViewportFocusLost;
	
	if (ViewportWorld && !ViewportWorld->bIsTearingDown)
	{
		for (FConstPlayerControllerIterator Iterator = ViewportWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* const PlayerController = Iterator->Get();
			if (PlayerController && (bShouldFlush || PlayerController->ShouldFlushKeysWhenViewportFocusChanges()))
			{
				PlayerController->FlushPressedKeys();
			}
		}
	}

	if (GEngine && GEngine->GetAudioDeviceManager())
	{
		bHasAudioFocus = false;
	}
}

void UGameViewportClient::ReceivedFocus(FViewport* InViewport)
{
	if (FPlatformMisc::DesktopTouchScreen() && GetUseMouseForTouch() && GetGameViewport() && !GetGameViewport()->GetPlayInEditorIsSimulate())
	{
		FSlateApplication::Get().SetGameIsFakingTouchEvents(true);
	}

	if (GEngine && GEngine->GetAudioDeviceManager())
	{
		GEngine->GetAudioDeviceManager()->SetActiveDevice(AudioDevice.GetDeviceID());
		bHasAudioFocus = true;
	}
}

void UGameViewportClient::Activated(FViewport* InViewport, const FWindowActivateEvent& InActivateEvent)
{
	ReceivedFocus(InViewport);
}

void UGameViewportClient::Deactivated(FViewport* InViewport, const FWindowActivateEvent& InActivateEvent)
{
	LostFocus(InViewport);
}

bool UGameViewportClient::IsInPermanentCapture()
{
	bool bIsInPermanentCapture = FViewportClient::IsInPermanentCapture();
	if (ViewportConsole)
	{
		bIsInPermanentCapture = !ViewportConsole->ConsoleActive() && bIsInPermanentCapture;
	}
	return bIsInPermanentCapture;
}

bool UGameViewportClient::WindowCloseRequested()
{
	return !WindowCloseRequestedDelegate.IsBound() || WindowCloseRequestedDelegate.Execute();
}

void UGameViewportClient::CloseRequested(FViewport* InViewport)
{
	check(InViewport == Viewport);

	// broadcast close request to anyone that registered an interest
	CloseRequestedDelegate.Broadcast(InViewport);

	// Historically, "CloseRequested" means "reset the Viewport reference to null as a signal for the game to shut down"
	if (InViewport)
	{
		RemoveAssociation(*InViewport);
	}

	// If this viewport has a high res screenshot window attached to it, close it
	if (HighResScreenshotDialog.IsValid())
	{
		HighResScreenshotDialog.Pin()->RequestDestroyWindow();
		HighResScreenshotDialog = NULL;
	}
}

bool UGameViewportClient::IsOrtho() const
{
	return false;
}

void UGameViewportClient::PostRender(UCanvas* Canvas)
{
#if WITH_EDITOR
	DrawTitleSafeArea(Canvas);
#endif
	// Draw the transition screen.
	DrawTransition(Canvas);
}

void UGameViewportClient::PeekTravelFailureMessages(UWorld* InWorld, ETravelFailure::Type FailureType, const FString& ErrorString)
{
	UE_LOGF(LogNet, Warning, "Travel Failure: [%ls]: %ls", ETravelFailure::ToString(FailureType), *ErrorString);
}

void UGameViewportClient::PeekNetworkFailureMessages(UWorld *InWorld, UNetDriver *NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString)
{
	static double LastTimePrinted = 0.0f;
	if (FPlatformTime::Seconds() - LastTimePrinted > GEngine->NetErrorLogInterval)
	{
		UE_LOGF(LogNet, Warning, "Network Failure: %ls[%ls]: %ls", NetDriver ? *NetDriver->NetDriverName.ToString() : TEXT("NULL"), ENetworkFailure::ToString(FailureType), *ErrorString);
		LastTimePrinted = FPlatformTime::Seconds();
	}
}

void UGameViewportClient::SSSwapControllers()
{
#if !UE_BUILD_SHIPPING
	UEngine* const Engine = GetOuterUEngine();

	int32 const NumPlayers = Engine ? Engine->GetNumGamePlayers(this) : 0;
	if (NumPlayers > 1)
	{
		ULocalPlayer* const LP = Engine ? Engine->GetFirstGamePlayer(this) : nullptr;
		const int32 TmpControllerID = LP ? LP->GetControllerId() : 0;

		for (int32 Idx = 0; Idx<NumPlayers-1; ++Idx)
		{
			Engine->GetGamePlayer(this, Idx)->SetControllerId(Engine->GetGamePlayer(this, Idx + 1)->GetControllerId());
		}
		Engine->GetGamePlayer(this, NumPlayers-1)->SetControllerId(TmpControllerID);
	}
#endif
}

void UGameViewportClient::ShowTitleSafeArea()
{
	static IConsoleVariable* DebugSafeZoneModeCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DebugSafeZone.Mode"));
	if (DebugSafeZoneModeCvar)
	{
		const int32 DebugSafeZoneMode = DebugSafeZoneModeCvar->GetInt();
		if (DebugSafeZoneMode != 1)
		{
			DebugSafeZoneModeCvar->Set(1);
		}
		else
		{
			DebugSafeZoneModeCvar->Set(0);
		}
	}
}

void UGameViewportClient::SetConsoleTarget(int32 PlayerIndex)
{
#if !UE_BUILD_SHIPPING
	if (ViewportConsole)
	{
		if(PlayerIndex >= 0 && PlayerIndex < GetOuterUEngine()->GetNumGamePlayers(this))
		{
			ViewportConsole->ConsoleTargetPlayer = GetOuterUEngine()->GetGamePlayer(this, PlayerIndex);
		}
		else
		{
			ViewportConsole->ConsoleTargetPlayer = NULL;
		}
	}
#endif
}


ULocalPlayer* UGameViewportClient::SetupInitialLocalPlayer(FString& OutError)
{
	check(GetOuterUEngine()->ConsoleClass != NULL);

	ActiveSplitscreenType = ESplitScreenType::None;

#if ALLOW_CONSOLE
	// Create the viewport's console.
	ViewportConsole = NewObject<UConsole>(this, GetOuterUEngine()->ConsoleClass);
	// register console to get all log messages
	GLog->AddOutputDevice(ViewportConsole);
#endif // !UE_BUILD_SHIPPING

	// Keep an eye on any network or server travel failures
	GEngine->OnTravelFailure().AddUObject(this, &UGameViewportClient::PeekTravelFailureMessages);
	GEngine->OnNetworkFailure().AddUObject(this, &UGameViewportClient::PeekNetworkFailureMessages);

	UGameInstance * ViewportGameInstance = GEngine->GetWorldContextFromGameViewportChecked(this).OwningGameInstance;

	if ( !ensure( ViewportGameInstance != NULL ) )
	{
		return NULL;
	}

	// Create the initial player - this is necessary or we can't render anything in-game.
	return ViewportGameInstance->CreateInitialPlayer(OutError);
}

void UGameViewportClient::UpdateActiveSplitscreenType()
{
	ESplitScreenType::Type SplitType = ESplitScreenType::None;

	int32 NumPlayers = 0;
	for (ULocalPlayer* LocalPlayer : GEngine->GetGamePlayers(this))
	{
		if (LocalPlayer && LocalPlayer->GetViewportClient() == this)
		{
			++NumPlayers;
		}
	}
	const UGameMapsSettings* Settings = GetDefault<UGameMapsSettings>();

	const bool bAllowSplitScreen = !bDisableSplitScreenOverride;

	if (Settings->bUseSplitscreen && bAllowSplitScreen)
	{
		switch (NumPlayers)
		{
		case 0:
		case 1:
			SplitType = ESplitScreenType::None;
			break;

		case 2:
			if (bEnablePlayersSplitRT)
			{
				SplitType = ESplitScreenType::TwoPlayer_Horizontal;
			}
			else
			{
				switch (Settings->TwoPlayerSplitscreenLayout)
				{
				case ETwoPlayerSplitScreenType::Horizontal:
					SplitType = ESplitScreenType::TwoPlayer_Horizontal;
					break;

				case ETwoPlayerSplitScreenType::Vertical:
					SplitType = ESplitScreenType::TwoPlayer_Vertical;
					break;

				default:
					check(0);
				}
			}
			break;

		case 3:
			if (bEnablePlayersSplitRT)
			{
				SplitType = ESplitScreenType::ThreePlayer_Grid;
			}
			else
			{
				switch (Settings->ThreePlayerSplitscreenLayout)
				{
				case EThreePlayerSplitScreenType::FavorTop:
					SplitType = ESplitScreenType::ThreePlayer_FavorTop;
					break;

				case EThreePlayerSplitScreenType::FavorBottom:
					SplitType = ESplitScreenType::ThreePlayer_FavorBottom;
					break;

				case EThreePlayerSplitScreenType::Vertical:
					SplitType = ESplitScreenType::ThreePlayer_Vertical;
					break;

				case EThreePlayerSplitScreenType::Horizontal:
					SplitType = ESplitScreenType::ThreePlayer_Horizontal;
					break;

				default:
					check(0);
				}
			}
			break;

		default:
			ensure(NumPlayers == 4);
			if (bEnablePlayersSplitRT)
			{
				SplitType = ESplitScreenType::FourPlayer_Grid;
			}
			else
			{
				switch (Settings->FourPlayerSplitscreenLayout)
				{
				case EFourPlayerSplitScreenType::Grid:
					SplitType = ESplitScreenType::FourPlayer_Grid;
					break;

				case EFourPlayerSplitScreenType::Vertical:
					SplitType = ESplitScreenType::FourPlayer_Vertical;
					break;

				case EFourPlayerSplitScreenType::Horizontal:
					SplitType = ESplitScreenType::FourPlayer_Horizontal;
					break;

				default:
					check(0);
				}
			}
			break;
		}
	}
	else
	{
		SplitType = ESplitScreenType::None;
	}

	ActiveSplitscreenType = SplitType;
}

void UGameViewportClient::LayoutPlayers()
{
	UpdateActiveSplitscreenType();
	const ESplitScreenType::Type SplitType = GetCurrentSplitscreenConfiguration();

	// Initialize the players
	const TArray<ULocalPlayer*>& PlayerList = GetOuterUEngine()->GetGamePlayers(this);

	for ( int32 PlayerIdx = 0; PlayerIdx < PlayerList.Num(); PlayerIdx++ )
	{
		if (!PlayerList[PlayerIdx] || PlayerList[PlayerIdx]->GetViewportClient() != this)
		{
			continue;
		}

		if ( SplitType < SplitscreenInfo.Num() && PlayerIdx < SplitscreenInfo[SplitType].PlayerData.Num() )
		{
			PlayerList[PlayerIdx]->Size.X =	SplitscreenInfo[SplitType].PlayerData[PlayerIdx].SizeX;
			PlayerList[PlayerIdx]->Size.Y =	SplitscreenInfo[SplitType].PlayerData[PlayerIdx].SizeY;
			PlayerList[PlayerIdx]->Origin.X =	SplitscreenInfo[SplitType].PlayerData[PlayerIdx].OriginX;
			PlayerList[PlayerIdx]->Origin.Y =	SplitscreenInfo[SplitType].PlayerData[PlayerIdx].OriginY;
		}
		else
		{
			PlayerList[PlayerIdx]->Size.X =	0.f;
			PlayerList[PlayerIdx]->Size.Y =	0.f;
			PlayerList[PlayerIdx]->Origin.X =	0.f;
			PlayerList[PlayerIdx]->Origin.Y =	0.f;
		}
	}
}

void UGameViewportClient::SetForceDisableSplitscreen(const bool bDisabled)
{
	bDisableSplitScreenOverride = bDisabled;
	LayoutPlayers();
}

void UGameViewportClient::GetSubtitleRegion(FVector2D& MinPos, FVector2D& MaxPos)
{
	MaxPos.X = 1.0f;
	MaxPos.Y = (GetOuterUEngine()->GetNumGamePlayers(this) == 1) ? 0.9f : 0.5f;
}


int32 UGameViewportClient::ConvertLocalPlayerToGamePlayerIndex( ULocalPlayer* LPlayer )
{
	return GetOuterUEngine()->GetGamePlayers(this).Find( LPlayer );
}

bool UGameViewportClient::HasTopSafeZone( int32 LocalPlayerIndex )
{
	switch ( GetCurrentSplitscreenConfiguration() )
	{
	case ESplitScreenType::None:
	case ESplitScreenType::TwoPlayer_Vertical:
	case ESplitScreenType::ThreePlayer_Vertical:
	case ESplitScreenType::FourPlayer_Vertical:
		return true;

	case ESplitScreenType::TwoPlayer_Horizontal:
	case ESplitScreenType::ThreePlayer_FavorTop:
	case ESplitScreenType::ThreePlayer_Horizontal:
	case ESplitScreenType::FourPlayer_Horizontal:
		return (LocalPlayerIndex == 0);

	case ESplitScreenType::ThreePlayer_FavorBottom:
	case ESplitScreenType::FourPlayer_Grid:
		return (LocalPlayerIndex < 2);
	}

	return false;
}

bool UGameViewportClient::HasBottomSafeZone( int32 LocalPlayerIndex )
{
	switch ( GetCurrentSplitscreenConfiguration() )
	{
	case ESplitScreenType::None:
	case ESplitScreenType::TwoPlayer_Vertical:
	case ESplitScreenType::ThreePlayer_Vertical:
	case ESplitScreenType::FourPlayer_Vertical:
		return true;

	case ESplitScreenType::TwoPlayer_Horizontal:
	case ESplitScreenType::ThreePlayer_FavorTop:
		return (LocalPlayerIndex > 0);

	case ESplitScreenType::ThreePlayer_FavorBottom:
	case ESplitScreenType::FourPlayer_Grid:
	case ESplitScreenType::ThreePlayer_Horizontal:
		return (LocalPlayerIndex > 1);

	case ESplitScreenType::FourPlayer_Horizontal:
		return (LocalPlayerIndex > 2);
	}

	return false;
}

bool UGameViewportClient::HasLeftSafeZone( int32 LocalPlayerIndex )
{
	switch ( GetCurrentSplitscreenConfiguration() )
	{
	case ESplitScreenType::None:
	case ESplitScreenType::TwoPlayer_Horizontal:
	case ESplitScreenType::ThreePlayer_Horizontal:
	case ESplitScreenType::FourPlayer_Horizontal:
		return true;

	case ESplitScreenType::TwoPlayer_Vertical:
	case ESplitScreenType::ThreePlayer_Vertical:
	case ESplitScreenType::FourPlayer_Vertical:
		return (LocalPlayerIndex == 0);

	case ESplitScreenType::ThreePlayer_FavorTop:
		return (LocalPlayerIndex < 2) ? true : false;

	case ESplitScreenType::ThreePlayer_FavorBottom:
	case ESplitScreenType::FourPlayer_Grid:
		return (LocalPlayerIndex == 0 || LocalPlayerIndex == 2);
	}

	return false;
}

bool UGameViewportClient::HasRightSafeZone( int32 LocalPlayerIndex )
{
	switch ( GetCurrentSplitscreenConfiguration() )
	{
	case ESplitScreenType::None:
	case ESplitScreenType::TwoPlayer_Horizontal:
	case ESplitScreenType::ThreePlayer_Horizontal:
	case ESplitScreenType::FourPlayer_Horizontal:
		return true;

	case ESplitScreenType::TwoPlayer_Vertical:
	case ESplitScreenType::ThreePlayer_FavorBottom:
		return (LocalPlayerIndex > 0);

	case ESplitScreenType::ThreePlayer_FavorTop:
		return (LocalPlayerIndex != 1);

	case ESplitScreenType::ThreePlayer_Vertical:
		return (LocalPlayerIndex == 2);

	case ESplitScreenType::FourPlayer_Vertical:
		return (LocalPlayerIndex == 3);

	case ESplitScreenType::FourPlayer_Grid:
		return (LocalPlayerIndex == 1 || LocalPlayerIndex == 3);
	}

	return false;
}


void UGameViewportClient::GetPixelSizeOfScreen( float& Width, float& Height, UCanvas* Canvas, int32 LocalPlayerIndex )
{
	switch ( GetCurrentSplitscreenConfiguration() )
	{
	case ESplitScreenType::None:
		Width = Canvas->ClipX;
		Height = Canvas->ClipY;
		return;
	case ESplitScreenType::TwoPlayer_Horizontal:
		Width = Canvas->ClipX;
		Height = Canvas->ClipY * 2;
		return;
	case ESplitScreenType::TwoPlayer_Vertical:
		Width = Canvas->ClipX * 2;
		Height = Canvas->ClipY;
		return;
	case ESplitScreenType::ThreePlayer_FavorTop:
		if (LocalPlayerIndex == 0)
		{
			Width = Canvas->ClipX;
		}
		else
		{
			Width = Canvas->ClipX * 2;
		}
		Height = Canvas->ClipY * 2;
		return;
	case ESplitScreenType::ThreePlayer_FavorBottom:
		if (LocalPlayerIndex == 2)
		{
			Width = Canvas->ClipX;
		}
		else
		{
			Width = Canvas->ClipX * 2;
		}
		Height = Canvas->ClipY * 2;
		return;
	case ESplitScreenType::ThreePlayer_Vertical:
		Width = Canvas->ClipX * 3;
		Height = Canvas->ClipY;
		return;
	case ESplitScreenType::ThreePlayer_Horizontal:
		Width = Canvas->ClipX;
		Height = Canvas->ClipY * 3;
		return;
	case ESplitScreenType::FourPlayer_Grid:
		Width = Canvas->ClipX * 2;
		Height = Canvas->ClipY * 2;
		return;
	case ESplitScreenType::FourPlayer_Vertical:
		Width = Canvas->ClipX * 4;
		Height = Canvas->ClipY;
		return;
	case ESplitScreenType::FourPlayer_Horizontal:
		Width = Canvas->ClipX;
		Height = Canvas->ClipY * 4;
		return;
	}
}

void UGameViewportClient::CalculateSafeZoneValues( FMargin& InSafeZone, UCanvas* Canvas, int32 LocalPlayerIndex, bool bUseMaxPercent )
{
	float ScreenWidth, ScreenHeight;
	GetPixelSizeOfScreen(ScreenWidth, ScreenHeight, Canvas, LocalPlayerIndex);

	FVector2D ScreenSize(ScreenWidth, ScreenHeight);
	FSlateApplication::Get().GetSafeZoneSize(InSafeZone, ScreenSize);
}


bool UGameViewportClient::CalculateDeadZoneForAllSides( ULocalPlayer* LPlayer, UCanvas* Canvas, float& fTopSafeZone, float& fBottomSafeZone, float& fLeftSafeZone, float& fRightSafeZone, bool bUseMaxPercent )
{
	// save separate - if the split screen is in bottom right, then
	FMargin SafeZone;
	if ( LPlayer != NULL )
	{
		int32 LocalPlayerIndex = ConvertLocalPlayerToGamePlayerIndex( LPlayer );

		if ( LocalPlayerIndex != -1 )
		{
			// see if this player should have a safe zone for any particular zonetype
			bool bHasTopSafeZone = HasTopSafeZone( LocalPlayerIndex );
			bool bHasBottomSafeZone = HasBottomSafeZone( LocalPlayerIndex );
			bool bHasLeftSafeZone = HasLeftSafeZone( LocalPlayerIndex );
			bool bHasRightSafeZone = HasRightSafeZone( LocalPlayerIndex );

			// if they need a safezone, then calculate it and save it
			if ( bHasTopSafeZone || bHasBottomSafeZone || bHasLeftSafeZone || bHasRightSafeZone)
			{
				// calculate the safezones
				CalculateSafeZoneValues(SafeZone, Canvas, LocalPlayerIndex, bUseMaxPercent );

				if (bHasTopSafeZone)
				{
					fTopSafeZone = SafeZone.Top;
				}
				else
				{
					fTopSafeZone = 0.f;
				}

				if (bHasBottomSafeZone)
				{
					fBottomSafeZone = SafeZone.Bottom;
				}
				else
				{
					fBottomSafeZone = 0.f;
				}

				if (bHasLeftSafeZone)
				{
					fLeftSafeZone = SafeZone.Left;
				}
				else
				{
					fLeftSafeZone = 0.f;
				}

				if (bHasRightSafeZone)
				{
					fRightSafeZone = SafeZone.Right;
				}
				else
				{
					fRightSafeZone = 0.f;
				}

				return true;
			}
		}
	}
	return false;
}

void UGameViewportClient::DrawTitleSafeArea( UCanvas* Canvas )
{
#if WITH_EDITOR
	// If we have a valid player hud, then the title area has already rendered.
	APlayerController* FirstPlayerController = GetWorld()->GetFirstPlayerController();
	if (FirstPlayerController && FirstPlayerController->GetHUD())
	{
		return;
	}

	// If r.DebugSafeZone.Mode isn't set to draw title area, don't draw it.
	static IConsoleVariable* SafeZoneModeCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DebugSafeZone.Mode"));
	if (SafeZoneModeCvar && (SafeZoneModeCvar->GetInt() != 1))
	{
		return;
	}

	FMargin SafeZone;
	const ULevelEditorPlaySettings* PlayInSettings = GetDefault<ULevelEditorPlaySettings>();

	float Width, Height;
	GetPixelSizeOfScreen(Width, Height, Canvas, 0);

	FLinearColor UnsafeZoneColor(1.0f, 0.0f, 0.0f, 0.25f);
	static IConsoleVariable* AlphaCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DebugSafeZone.OverlayAlpha"));
	if (AlphaCvar)
	{
		UnsafeZoneColor.A = AlphaCvar->GetFloat();
	}

	FCanvasTileItem TileItem(FVector2D::ZeroVector, GWhiteTexture, UnsafeZoneColor);
	TileItem.BlendMode = SE_BLEND_Translucent;

	// CalculateSafeZoneValues() can be slow, so we only want to run it if we have boundaries to draw
	if (FDisplayMetrics::GetDebugTitleSafeZoneRatio() < 1.f)
	{
		CalculateSafeZoneValues(SafeZone, Canvas, 0, false);
		const float HeightOfSides = Height - SafeZone.GetTotalSpaceAlong<Orient_Vertical>();
		// Top bar
		TileItem.Position = FVector2D::ZeroVector;
		TileItem.Size = FVector2D(Width, SafeZone.Top);
		Canvas->DrawItem(TileItem);

		// Bottom bar
		TileItem.Position = FVector2D(0.0f, Height - SafeZone.Bottom);
		TileItem.Size = FVector2D(Width, SafeZone.Bottom);
		Canvas->DrawItem(TileItem);

		// Left bar
		TileItem.Position = FVector2D(0.0f, SafeZone.Top);
		TileItem.Size = FVector2D(SafeZone.Left, HeightOfSides);
		Canvas->DrawItem(TileItem);

		// Right bar
		TileItem.Position = FVector2D(Width - SafeZone.Right, SafeZone.Top);
		TileItem.Size = FVector2D(SafeZone.Right, HeightOfSides);
		Canvas->DrawItem(TileItem);
	}
	else if (FSlateApplication::Get().IsCustomSafeZoneSet())
	{
		ULevelEditorPlaySettings* PlaySettings = GetMutableDefault<ULevelEditorPlaySettings>();
		PlaySettings->CalculateCustomUnsafeZones(PlaySettings->CustomUnsafeZoneStarts, PlaySettings->CustomUnsafeZoneDimensions, PlaySettings->DeviceToEmulate, FVector2D(Width, Height));

		for (int ZoneIndex = 0; ZoneIndex < PlaySettings->CustomUnsafeZoneStarts.Num(); ZoneIndex++)
		{
			TileItem.Position = PlaySettings->CustomUnsafeZoneStarts[ZoneIndex];
			TileItem.Size = PlaySettings->CustomUnsafeZoneDimensions[ZoneIndex];
			Canvas->DrawItem(TileItem);
		}
	}
#endif
}

void UGameViewportClient::DrawTransition(UCanvas* Canvas)
{
	if (bSuppressTransitionMessage == false)
	{
		switch (GetOuterUEngine()->TransitionType)
		{
		case ETransitionType::Loading:
			DrawTransitionMessage(Canvas, NSLOCTEXT("GameViewportClient", "LoadingMessage", "LOADING").ToString());
			break;
		case ETransitionType::Saving:
			DrawTransitionMessage(Canvas, NSLOCTEXT("GameViewportClient", "SavingMessage", "SAVING").ToString());
			break;
		case ETransitionType::Connecting:
			DrawTransitionMessage(Canvas, NSLOCTEXT("GameViewportClient", "ConnectingMessage", "CONNECTING").ToString());
			break;
		case ETransitionType::Precaching:
			DrawTransitionMessage(Canvas, NSLOCTEXT("GameViewportClient", "PrecachingMessage", "PRECACHING").ToString());
			break;
		case ETransitionType::Paused:
			DrawTransitionMessage(Canvas, NSLOCTEXT("GameViewportClient", "PausedMessage", "PAUSED").ToString());
			break;
		case ETransitionType::WaitingToConnect:
			DrawTransitionMessage(Canvas, TEXT("Waiting to connect...")); // Temp - localization of the FString messages is broke atm. Loc this when its fixed.
			break;
		}
	}
}

void UGameViewportClient::DrawTransitionMessage(UCanvas* Canvas,const FString& Message)
{
	UFont* Font = GEngine->GetLargeFont();
	FCanvasTextItem TextItem( FVector2D::ZeroVector, FText::GetEmpty(), Font, FLinearColor::Blue);
	TextItem.EnableShadow( FLinearColor::Black );
	TextItem.Text = FText::FromString(Message);
	float XL, YL;
	Canvas->StrLen( Font , Message, XL, YL );
	Canvas->DrawItem( TextItem, 0.5f * (Canvas->ClipX - XL), 0.66f * Canvas->ClipY - YL * 0.5f );
}

void UGameViewportClient::NotifyPlayerAdded( int32 PlayerIndex, ULocalPlayer* AddedPlayer )
{
	LayoutPlayers();

	FSlateApplication::Get().SetUserFocusToGameViewport(PlayerIndex);

	if ( GameLayerManager.IsValid() )
	{
		GameLayerManager->NotifyPlayerAdded(PlayerIndex, AddedPlayer);
	}

	PlayerAddedDelegate.Broadcast( PlayerIndex );
}

void UGameViewportClient::NotifyPlayerRemoved( int32 PlayerIndex, ULocalPlayer* RemovedPlayer )
{
	LayoutPlayers();

	if ( GameLayerManager.IsValid() )
	{
		GameLayerManager->NotifyPlayerRemoved(PlayerIndex, RemovedPlayer);
	}

	PlayerRemovedDelegate.Broadcast( PlayerIndex );
}

void UGameViewportClient::AddViewportWidgetContent( TSharedRef<SWidget> ViewportContent, const int32 ZOrder )
{
	if (ensure(ViewportOverlayWidget.IsValid()))
	{
		ViewportOverlayWidget->AddSlot( ZOrder )
			[
				ViewportContent
			];
	}
}

void UGameViewportClient::RemoveViewportWidgetContent( TSharedRef<SWidget> ViewportContent )
{
	if (ViewportOverlayWidget.IsValid())
	{
		ViewportOverlayWidget->RemoveSlot(ViewportContent);
	}
}

void UGameViewportClient::AddViewportWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent, const int32 ZOrder)
{
	if (ensure(Player))
	{
		if (GameLayerManager.IsValid())
		{
			GameLayerManager->AddWidgetForPlayer(Player, ViewportContent, ZOrder);
		}
	}
	//TODO - If this fails what should we do?
}

void UGameViewportClient::RemoveViewportWidgetForPlayer(ULocalPlayer* Player, TSharedRef<SWidget> ViewportContent)
{
	if ( GameLayerManager.IsValid() )
	{
		GameLayerManager->RemoveWidgetForPlayer(Player, ViewportContent);
	}
}

void UGameViewportClient::RemoveAllViewportWidgets()
{
	CursorWidgets.Empty();

	if (ViewportOverlayWidget.IsValid())
	{
		ViewportOverlayWidget->ClearChildren();
	}

	if ( GameLayerManager.IsValid() )
	{
		GameLayerManager->ClearWidgets();
	}
}

void UGameViewportClient::AddGameLayerWidget(TSharedRef<SWidget> ViewportContent, const int32 ZOrder)
{
	if (GameLayerManager.IsValid())
	{
		GameLayerManager->AddGameLayer(ViewportContent, ZOrder);
	}
}

void UGameViewportClient::RemoveGameLayerWidget(TSharedRef<SWidget> ViewportContent)
{
	if (GameLayerManager.IsValid())
	{
		GameLayerManager->RemoveGameLayer(ViewportContent);
	}
}

void UGameViewportClient::VerifyPathRenderingComponents()
{
	const bool bShowPaths = !!EngineShowFlags.Navigation;

	UWorld* const ViewportWorld = GetWorld();

	if (ViewportWorld)
	{
		FNavigationSystem::VerifyNavigationRenderingComponents(*ViewportWorld, bShowPaths);
	}
}

void UGameViewportClient::SetMouseCaptureMode(EMouseCaptureMode Mode)
{
	if (MouseCaptureMode != Mode)
	{
		UE_LOGF(LogViewport, Display, "Viewport MouseCaptureMode Changed, %ls -> %ls",
			*StaticEnum<EMouseCaptureMode>()->GetNameStringByValue((uint64)MouseCaptureMode),
			*StaticEnum<EMouseCaptureMode>()->GetNameStringByValue((uint64)Mode)
		);

		MouseCaptureMode = Mode;
	}
}

EMouseCaptureMode UGameViewportClient::GetMouseCaptureMode() const
{
	return MouseCaptureMode;
}

bool UGameViewportClient::CaptureMouseOnLaunch()
{
	// Capture mouse unless headless
	return !FApp::CanEverRender() ? false : GetDefault<UInputSettings>()->bCaptureMouseOnLaunch;
}

void UGameViewportClient::SetMouseLockMode(EMouseLockMode InMouseLockMode)
{
	if (MouseLockMode != InMouseLockMode)
	{
		UE_LOGF(LogViewport, Display, "Viewport MouseLockMode Changed, %ls -> %ls",
			*StaticEnum<EMouseLockMode>()->GetNameStringByValue((uint64)MouseLockMode),
			*StaticEnum<EMouseLockMode>()->GetNameStringByValue((uint64)InMouseLockMode)
		);

		MouseLockMode = InMouseLockMode;
	}
}

EMouseLockMode UGameViewportClient::GetMouseLockMode() const
{
	return MouseLockMode;
}

void UGameViewportClient::SetHideCursorDuringCapture(bool InHideCursorDuringCapture)
{
	if (bHideCursorDuringCapture != InHideCursorDuringCapture)
	{
		UE_LOGF(LogViewport, Display, "Viewport HideCursorDuringCapture Changed, %ls -> %ls",
			bHideCursorDuringCapture ? TEXT("True") : TEXT("False"),
			InHideCursorDuringCapture ? TEXT("True") : TEXT("False")
		);

		bHideCursorDuringCapture = InHideCursorDuringCapture;
	}
}

#if UE_ALLOW_EXEC_COMMANDS
bool UGameViewportClient::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FExec::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	else if (ProcessConsoleExec(Cmd, Ar, NULL))
	{
		return true;
	}
	else if (GameInstance && (GameInstance->Exec(InWorld, Cmd, Ar) || GameInstance->ProcessConsoleExec(Cmd, Ar, nullptr)))
	{
		return true;
	}
	else if (GEngine->Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	else
	{
		return false;
	}
}
#endif // UE_ALLOW_EXEC_COMMANDS

bool UGameViewportClient::Exec_Runtime( UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar)
{
	if ( FParse::Command(&Cmd,TEXT("FORCEFULLSCREEN")) )
	{
		return HandleForceFullscreenCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("SHOW")) )
	{
		return HandleShowCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command(&Cmd,TEXT("SHOWLAYER")) )
	{
		return HandleShowLayerCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd,TEXT("VIEWMODE")))
	{
		return HandleViewModeCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("NEXTVIEWMODE")))
	{
		return HandleNextViewModeCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("PREVVIEWMODE")))
	{
		return HandlePrevViewModeCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command(&Cmd,TEXT("PRECACHE")) )
	{
		return HandlePreCacheCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("TOGGLE_FULLSCREEN")) || FParse::Command(&Cmd,TEXT("FULLSCREEN")) )
	{
		return HandleToggleFullscreenCommand();
	}
	else if( FParse::Command(&Cmd,TEXT("SETRES")) )
	{
		return HandleSetResCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("HighResShot")) )
	{
		return HandleHighresScreenshotCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("HighResShotUI")) )
	{
		return HandleHighresScreenshotUICommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("SHOT")) || FParse::Command(&Cmd,TEXT("SCREENSHOT")) )
	{
		return HandleScreenshotCommand( Cmd, Ar );
	}
	else if ( FParse::Command(&Cmd, TEXT("BUGSCREENSHOTWITHHUDINFO")) )
	{
		return HandleBugScreenshotwithHUDInfoCommand( Cmd, Ar );
	}
	else if ( FParse::Command(&Cmd,TEXT("BUGSCREENSHOT")) )
	{
		return HandleBugScreenshotCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("KILLPARTICLES")) )
	{
		return HandleKillParticlesCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("FORCESKELLOD")) )
	{
		return HandleForceSkelLODCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("DISPLAY")))
	{
		return HandleDisplayCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("DISPLAYALL")))
	{
		return HandleDisplayAllCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("DISPLAYALLLOCATION")))
	{
		return HandleDisplayAllLocationCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("DISPLAYALLROTATION")))
	{
		return HandleDisplayAllRotationCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("DISPLAYCLEAR")))
	{
		return HandleDisplayClearCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("GETALLLOCATION")))
	{
		return HandleGetAllLocationCommand(Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("GETALLROTATION")))
	{
		return HandleGetAllRotationCommand(Cmd, Ar);
	}
	else if(FParse::Command(&Cmd, TEXT("TEXTUREDEFRAG")))
	{
		return HandleTextureDefragCommand( Cmd, Ar );
	}
	else
	{
		return false;
	}
}

bool UGameViewportClient::HandleForceFullscreenCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GForceFullscreen = !GForceFullscreen;
	return true;
}

bool UGameViewportClient::HandleShowCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
#if UE_BUILD_SHIPPING
	// don't allow show flags in net games, but on con
	if ( InWorld->GetNetMode() != NM_Standalone || (GEngine->GetWorldContextFromWorldChecked(InWorld).PendingNetGame != NULL) )
	{
		return true;
	}
	// the effects of this cannot be easily reversed, so prevent the user from playing network games without restarting to avoid potential exploits
	GDisallowNetworkTravel = true;
#endif // UE_BUILD_SHIPPING

	// First, look for skeletal mesh show commands

	bool bUpdateSkelMeshCompDebugFlags = false;
	static bool bShowPrePhysSkelBones = false;

	if(FParse::Command(&Cmd,TEXT("PREPHYSBONES")))
	{
		bShowPrePhysSkelBones = !bShowPrePhysSkelBones;
		bUpdateSkelMeshCompDebugFlags = true;
	}

	// If we changed one of the skel mesh debug show flags, set it on each of the components in the World.
	if(bUpdateSkelMeshCompDebugFlags)
	{
		for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			USkeletalMeshComponent* SkelComp = *It;
			if( SkelComp->GetScene() == InWorld->Scene )
			{
				SkelComp->bShowPrePhysBones = bShowPrePhysSkelBones;
				SkelComp->MarkRenderStateDirty();
			}
		}

		// Now we are done.
		return true;
	}

	// EngineShowFlags
	{
		TArray<FString> ShowFlagsArgs;
		if (!FString(Cmd).ParseIntoArray(ShowFlagsArgs, TEXT(" ")))
		{
			ShowFlagsArgs.Add(Cmd);
		}

		int32 FlagIndex = FEngineShowFlags::FindIndexByName(*ShowFlagsArgs[0]);

		if(FlagIndex != -1)
		{
			bool bCanBeToggled = true;

			if(GIsEditor)
			{
				if(!FEngineShowFlags::CanBeToggledInEditor(*ShowFlagsArgs[0]))
				{
					bCanBeToggled = false;
				}
			}

			if(bCanBeToggled)
			{
				bool bOldState = EngineShowFlags.GetSingleFlag(FlagIndex);

				if (FEngineShowFlags::IsNameThere(*ShowFlagsArgs[0], TEXT("ActorColoration")))
				{
					if (ShowFlagsArgs.Num() > 1)
					{
						bOldState &= !FActorPrimitiveColorHandler::Get().SetActivePrimitiveColorHandler(*ShowFlagsArgs[1], InWorld);
					}
				}

				EngineShowFlags.SetSingleFlag(FlagIndex, !bOldState);

				if(FEngineShowFlags::IsNameThere(*ShowFlagsArgs[0], TEXT("Navigation,Cover")))
				{
					VerifyPathRenderingComponents();
				}

				if(FEngineShowFlags::IsNameThere(*ShowFlagsArgs[0], TEXT("Volumes")))
				{
					// TODO: Investigate why this is doesn't appear to work
					if (AllowDebugViewmodes())
					{
						ToggleShowVolumes();
					}
					else
					{
						Ar.Logf(TEXT("Debug viewmodes not allowed on consoles by default.  See AllowDebugViewmodes()."));
					}
				}
			}

			if(FEngineShowFlags::IsNameThere(*ShowFlagsArgs[0], TEXT("Collision")))
			{
				ToggleShowCollision();
			}

			return true;
		}
	}

	// create a sorted list of showflags
	TSet<FString> LinesToSort;
	{
		struct FIterSink
		{
			FIterSink(TSet<FString>& InLinesToSort, const FEngineShowFlags InEngineShowFlags) : LinesToSort(InLinesToSort), EngineShowFlags(InEngineShowFlags)
			{
			}

			bool HandleShowFlag(uint32 InIndex, const FString& InName)
			{
				FString Value = FString::Printf(TEXT("%s=%d"), *InName, EngineShowFlags.GetSingleFlag(InIndex) ? 1 : 0);
				LinesToSort.Add(Value);
				return true;
			}

			bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
			{
				return HandleShowFlag(InIndex, InName);
			}

			bool OnCustomShowFlag(uint32 InIndex, const FString& InName)
			{
				return HandleShowFlag(InIndex, InName);
			}

			TSet<FString>& LinesToSort;
			const FEngineShowFlags EngineShowFlags;
		};

		FIterSink Sink(LinesToSort, EngineShowFlags);

		FEngineShowFlags::IterateAllFlags(Sink);
	}

	LinesToSort.Sort( TLess<FString>() );

	for(TSet<FString>::TConstIterator It(LinesToSort); It; ++It)
	{
		const FString Value = *It;

		Ar.Logf(TEXT("%s"), *Value);
	}

	return true;
}

FPopupMethodReply UGameViewportClient::OnQueryPopupMethod() const
{
	return FPopupMethodReply::UseMethod(EPopupMethod::UseCurrentWindow)
		.SetShouldThrottle(EShouldThrottle::No);
}

bool UGameViewportClient::HandleNavigation(const uint32 InUserIndex, TSharedPtr<SWidget> InDestination)
{
	if (CustomNavigationEvent.IsBound())
	{
		return CustomNavigationEvent.Execute(InUserIndex, InDestination);
	}
	return false;
}

void UGameViewportClient::ToggleShowVolumes()
{
	// Don't allow 'show collision' and 'show volumes' at the same time, so turn collision off
	if (EngineShowFlags.Volumes && EngineShowFlags.Collision)
	{
		EngineShowFlags.SetCollision(false);
		ToggleShowCollision();
	}

	// Iterate over all brushes
	for (TObjectIterator<UBrushComponent> It; It; ++It)
	{
		UBrushComponent* BrushComponent = *It;
		AVolume* Owner = Cast<AVolume>(BrushComponent->GetOwner());

		// Only bother with volume brushes that belong to the world's scene
		if (Owner && BrushComponent->GetScene() == GetWorld()->Scene && !FActorEditorUtils::IsABuilderBrush(Owner))
		{
			// We're expecting this to be in the game at this point
			check(Owner->GetWorld()->IsGameWorld());

			// Toggle visibility of this volume
			if (BrushComponent->IsVisible())
			{
				BrushComponent->SetVisibility(false);
				BrushComponent->SetHiddenInGame(true);
			}
			else
			{
				BrushComponent->SetVisibility(true);
				BrushComponent->SetHiddenInGame(false);
			}
		}
	}
}

void UGameViewportClient::ToggleShowCollision()
{
	// special case: for the Engine.Collision flag, we need to un-hide any primitive components that collide so their collision geometry gets rendered
	const bool bIsShowingCollision = EngineShowFlags.Collision;

	if (bIsShowingCollision)
	{
		// Don't allow 'show collision' and 'show volumes' at the same time, so turn collision off
		if (EngineShowFlags.Volumes)
		{
			EngineShowFlags.SetVolumes(false);
			ToggleShowVolumes();
		}
	}

#if !UE_BUILD_SHIPPING
	if (World != nullptr)
	{
		// Tell engine to create proxies for hidden components, so we can still draw collision
		World->bCreateRenderStateForHiddenComponentsWithCollsion = bIsShowingCollision;

		// Need to recreate scene proxies when this flag changes.
		FGlobalComponentRecreateRenderStateContext Recreate;
	}
#endif // !UE_BUILD_SHIPPING


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (EngineShowFlags.Collision)
	{
		for (FLocalPlayerIterator It((UEngine*)GetOuter(), World); It; ++It)
		{
			APlayerController* PC = It->PlayerController;
			if (PC != NULL && PC->GetPawn() != NULL)
			{
				PC->ClientMessage(FString::Printf(TEXT("!!!! Player Pawn %s Collision Info !!!!"), *PC->GetPawn()->GetName()));
				if (const UObject* MovementBaseObject = PC->GetPawn()->GetMovementBaseObject())
				{
					PC->ClientMessage(FString::Printf(TEXT("Base %s"), *MovementBaseObject->GetName()));
				}
				TSet<AActor*> TouchingActors;
				PC->GetPawn()->GetOverlappingActors(TouchingActors);
				int32 i = 0;
				for (AActor* TouchingActor : TouchingActors)
				{
					PC->ClientMessage(FString::Printf(TEXT("Touching %d: %s"), i++, *TouchingActor->GetName()));
				}
			}
		}
	}
#endif
}

bool UGameViewportClient::HandleShowLayerCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	FString LayerName = FParse::Token(Cmd, 0);
	// optional 0/1 for setting vis, instead of toggling
	FString SetModeParam = FParse::Token(Cmd, 0);

	int32 SetMode = -1;
	if (SetModeParam != TEXT(""))
	{
		SetMode = FCString::Atoi(*SetModeParam);
	}

	bool bPrintValidEntries = false;

	if (LayerName.IsEmpty())
	{
		Ar.Logf(TEXT("Missing layer name."));
		bPrintValidEntries = true;
	}
	else
	{
		int32 NumActorsToggled = 0;
		FName LayerFName = FName(*LayerName);

		for (FActorIterator It(InWorld); It; ++It)
		{
			AActor* Actor = *It;

			if (Actor->Layers.Contains(LayerFName))
			{
				const bool bHiddenLocal = Actor->IsHidden();

				// look for always toggle, or a set when it's unset, etc
				if ((SetMode == -1) || (SetMode == 0 && !bHiddenLocal) || (SetMode != 0 && bHiddenLocal))
				{
					NumActorsToggled++;
					// Note: overriding existing hidden property, ideally this would be something orthogonal
					Actor->SetHidden(!bHiddenLocal);

					Actor->MarkComponentsRenderStateDirty();
				}
			}
		}

		Ar.Logf(TEXT("Toggled visibility of %u actors"), NumActorsToggled);
		bPrintValidEntries = NumActorsToggled == 0;
	}

	if (bPrintValidEntries)
	{
		TArray<FName> LayerNames;

		for (FActorIterator It(InWorld); It; ++It)
		{
			AActor* Actor = *It;

			for (int32 LayerIndex = 0; LayerIndex < Actor->Layers.Num(); LayerIndex++)
			{
				LayerNames.AddUnique(Actor->Layers[LayerIndex]);
			}
		}

		Ar.Logf(TEXT("Valid layer names:"));

		for (int32 LayerIndex = 0; LayerIndex < LayerNames.Num(); LayerIndex++)
		{
			Ar.Logf(TEXT("   %s"), *LayerNames[LayerIndex].ToString());
		}
	}

	return true;
}

bool UGameViewportClient::HandleViewModeCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
#if !UE_BUILD_DEBUG
	// If there isn't a cheat manager, exit out
	bool bCheatsEnabled = false;
	for (FLocalPlayerIterator It((UEngine*)GetOuter(), InWorld); It; ++It)
	{
		if (It->PlayerController != NULL && It->PlayerController->CheatManager != NULL)
		{
			bCheatsEnabled = true;
			break;
		}
	}
	if (!bCheatsEnabled)
	{
		return true;
	}
#endif
	FString ViewModeName = FParse::Token(Cmd, 0);

	if(!ViewModeName.IsEmpty())
	{
		uint32 i = 0;
		for(; i < VMI_Max; ++i)
		{
			if(ViewModeName == GetViewModeName((EViewModeIndex)i))
			{
				ViewModeIndex = i;
				Ar.Logf(TEXT("Set new viewmode: %s"), GetViewModeName((EViewModeIndex)ViewModeIndex));
				break;
			}
		}
		if(i == VMI_Max)
		{
			Ar.Logf(TEXT("Error: view mode not recognized: %s"), *ViewModeName);
		}
	}
	else
	{
		Ar.Logf(TEXT("Current view mode: %s"), GetViewModeName((EViewModeIndex)ViewModeIndex));

		FString ViewModes;
		for(uint32 i = 0; i < VMI_Max; ++i)
		{
			if(i != 0)
			{
				if ((i % 5) == 0)
				{
					ViewModes += TEXT("\n     ");
				}
				else
				{
					ViewModes += TEXT(", ");
				}
			}
			ViewModes += GetViewModeName((EViewModeIndex)i);
		}
		Ar.Logf(TEXT("Available view modes: %s"), *ViewModes);
	}

	SetViewMode((EViewModeIndex)ViewModeIndex);
	
	return true;
}

bool UGameViewportClient::HandleNextViewModeCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
#if !UE_BUILD_DEBUG
	// If there isn't a cheat manager, exit out
	bool bCheatsEnabled = false;
	for (FLocalPlayerIterator It((UEngine*)GetOuter(), InWorld); It; ++It)
	{
		if (It->PlayerController != NULL && It->PlayerController->CheatManager != NULL)
		{
			bCheatsEnabled = true;
			break;
		}
	}
	if (!bCheatsEnabled)
	{
		return true;
	}
#endif
	ViewModeIndex = ViewModeIndex + 1;

	// wrap around
	if(ViewModeIndex == VMI_Max)
	{
		ViewModeIndex = 0;
	}

	Ar.Logf(TEXT("New view mode: %s"), GetViewModeName((EViewModeIndex)ViewModeIndex));
	ApplyViewMode((EViewModeIndex)ViewModeIndex, true, EngineShowFlags);
	return true;
}

bool UGameViewportClient::HandlePrevViewModeCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
#if !UE_BUILD_DEBUG
	// If there isn't a cheat manager, exit out
	bool bCheatsEnabled = false;
	for (FLocalPlayerIterator It((UEngine*)GetOuter(), InWorld); It; ++It)
	{
		if (It->PlayerController != NULL && It->PlayerController->CheatManager != NULL)
		{
			bCheatsEnabled = true;
			break;
		}
	}
	if (!bCheatsEnabled)
	{
		return true;
	}
#endif
	ViewModeIndex = ViewModeIndex - 1;

	// wrap around
	if(ViewModeIndex < 0)
	{
		ViewModeIndex = VMI_Max - 1;
	}

	Ar.Logf(TEXT("New view mode: %s"), GetViewModeName((EViewModeIndex)ViewModeIndex));
	ApplyViewMode((EViewModeIndex)ViewModeIndex, true, EngineShowFlags);
	return true;
}

bool UGameViewportClient::HandlePreCacheCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	Precache();
	return true;
}

bool UGameViewportClient::SetDisplayConfiguration(const FIntPoint* Dimensions, EWindowMode::Type WindowMode)
{
	if (Viewport == nullptr)
	{
		return true;
	}

	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);

	if (GameEngine)
	{
		UGameUserSettings* UserSettings = GameEngine->GetGameUserSettings();

		UserSettings->SetFullscreenMode(WindowMode);

		if (Dimensions)
		{
			UserSettings->SetScreenResolution(*Dimensions);
		}

		UserSettings->ApplySettings(false);
	}
	else
	{
		int32 NewX = GSystemResolution.ResX;
		int32 NewY = GSystemResolution.ResY;

		if (Dimensions)
		{
			NewX = Dimensions->X;
			NewY = Dimensions->Y;
		}

		FSystemResolution::RequestResolutionChange(NewX, NewY, WindowMode);
	}

	return true;
}

bool UGameViewportClient::HandleToggleFullscreenCommand()
{
	if constexpr (FPlatformProperties::SupportsWindowedMode())
	{
		if (!Viewport)
		{
			return true;
		}

		static auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FullScreenMode"));
		check(CVar);
		EWindowMode::Type FullScreenMode = Viewport->IsFullscreen()
			? EWindowMode::Windowed
			: (((CVar->GetValueOnGameThread() == 0) && Viewport->IsExclusiveFullscreenSupported())
				? EWindowMode::Fullscreen
				: EWindowMode::WindowedFullscreen);

		int32 ResolutionX = GSystemResolution.ResX;
		int32 ResolutionY = GSystemResolution.ResY;
		const int32 MonitorIndex = Viewport->GetCurrentMonitorIndex();
		bool bNewModeApplied = false;

		// Make sure the user's settings are updated after pressing Alt+Enter to toggle fullscreen.  Note
		// that we don't need to "apply" the setting change, as we already did that above directly.
		UGameEngine* const GameEngine = Cast<UGameEngine>( GEngine );
		if (GameEngine != nullptr)
		{
			UGameUserSettings* const UserSettings = GameEngine->GetGameUserSettings();
			if (UserSettings != nullptr)
			{
				ResolutionX = UserSettings->GetScreenResolution().X;
				ResolutionY = UserSettings->GetScreenResolution().Y;

				const FDisplayMetrics& DisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
				if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndex))
				{
					const FMonitorInfo& Monitor = DisplayMetrics.MonitorInfo[MonitorIndex];
					if (FullScreenMode == EWindowMode::Fullscreen)
					{
						ResolutionX = Monitor.MaxResolution.X;
						ResolutionY = Monitor.MaxResolution.Y;
						if ((ResolutionX == 0) || (ResolutionY == 0))
						{
							ResolutionX = Monitor.NativeWidth;
							ResolutionY = Monitor.NativeHeight;
						}
					}
					else
					{
						const FPlatformRect& DisplayRect = Monitor.DisplayRect;
						ResolutionX = DisplayRect.Right - DisplayRect.Left;
						ResolutionY = DisplayRect.Bottom - DisplayRect.Top;
					}

					UserSettings->SetDisplayProperties(Monitor.ID, MonitorIndex);
				}

				// Ensure that our desired screen size will fit on the display
				UGameEngine::DetermineGameWindowResolution(ResolutionX, ResolutionY, FullScreenMode, true, MonitorIndex);

				UserSettings->SetScreenResolution(FIntPoint(ResolutionX, ResolutionY));
				UserSettings->SetFullscreenMode(FullScreenMode);
				UserSettings->ConfirmVideoMode();
				UserSettings->ApplySettings(false);
				bNewModeApplied = true;
			}
		}

		if (!bNewModeApplied)
		{
			FSystemResolution::SetMonitorIndex(MonitorIndex);
			FSystemResolution::RequestResolutionChange(ResolutionX, ResolutionY, FullScreenMode);
		}

		ToggleFullscreenDelegate.Broadcast(FullScreenMode != EWindowMode::Windowed);
	}

	return true;
}

bool UGameViewportClient::HandleSetResCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (Viewport)
	{
		int32 X=FCString::Atoi(Cmd);
		const TCHAR* CmdTemp = FCString::Strchr(Cmd,'x') ? FCString::Strchr(Cmd,'x')+1 : FCString::Strchr(Cmd,'X') ? FCString::Strchr(Cmd,'X')+1 : TEXT("");
		int32 Y=FCString::Atoi(CmdTemp);
		Cmd = CmdTemp;
		EWindowMode::Type WindowMode = Viewport->GetWindowMode();

		if(FCString::Strchr(Cmd,'w') || FCString::Strchr(Cmd,'W'))
		{
			if(FCString::Strchr(Cmd, 'f') || FCString::Strchr(Cmd, 'F'))
			{
				WindowMode = EWindowMode::WindowedFullscreen;
			}
			else
			{
				WindowMode = EWindowMode::Windowed;
			}

		}
		else if(FCString::Strchr(Cmd,'f') || FCString::Strchr(Cmd,'F'))
		{
			WindowMode = EWindowMode::Fullscreen;
		}
		if( X && Y )
		{
			FSystemResolution::RequestResolutionChange(X, Y, WindowMode);
		}
	}
	return true;
}

bool UGameViewportClient::HandleHighresScreenshotCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if(Viewport)
	{
		if (GetHighResScreenshotConfig().ParseConsoleCommand(Cmd, Ar))
		{
			Viewport->TakeHighResScreenShot();
		}
	}
	return true;
}

bool UGameViewportClient::HandleHighresScreenshotUICommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Open the highres screenshot UI. When the capture region editing works properly, we can pass CaptureRegionWidget through
	// HighResScreenshotDialog = SHighResScreenshotDialog::OpenDialog(GetWorld(), Viewport, NULL /*CaptureRegionWidget*/);
	// Disabled until mouse specification UI can be used correctly
	return true;
}


bool UGameViewportClient::HandleScreenshotCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if(Viewport)
	{
		bool bShowUI = FParse::Command(&Cmd, TEXT("SHOWUI"));
		bool bAddFilenameSuffix = true;

		// support arguments
		FString FileName;
		bShowUI = FParse::Param(Cmd, TEXT("showui")) || bShowUI;
		FParse::Value(Cmd, TEXT("filename="), FileName);

		if (FParse::Param(Cmd, TEXT("nosuffix")))
		{
			bAddFilenameSuffix = false;
		}

		FScreenshotRequest::RequestScreenshot(FileName, bShowUI, bAddFilenameSuffix, Viewport->GetSceneHDREnabled());

		GScreenshotResolutionX = Viewport->GetRenderTargetTextureSizeXY().X;
		GScreenshotResolutionY = Viewport->GetRenderTargetTextureSizeXY().Y;
	}
	return true;
}

bool UGameViewportClient::HandleBugScreenshotwithHUDInfoCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	return RequestBugScreenShot(Cmd, true);
}

bool UGameViewportClient::HandleBugScreenshotCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	return RequestBugScreenShot(Cmd, false);
}

bool UGameViewportClient::HandleKillParticlesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Don't kill in the Editor to avoid potential content clobbering.
	if( !GIsEditor )
	{
		extern ENGINE_API bool GIsAllowingParticles;
		// Deactivate system and kill existing particles.
		for( TObjectIterator<UParticleSystemComponent> It; It; ++It )
		{
			UParticleSystemComponent* ParticleSystemComponent = *It;
			ParticleSystemComponent->DeactivateSystem();
			ParticleSystemComponent->KillParticlesForced();
		}
		// No longer initialize particles from here on out.
		GIsAllowingParticles = false;
	}
	return true;
}

bool UGameViewportClient::HandleForceSkelLODCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	int32 ForceLod = 0;
	if(FParse::Value(Cmd,TEXT("LOD="),ForceLod))
	{
		ForceLod++;
	}

	for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
	{
		USkeletalMeshComponent* SkelComp = *It;
		if( SkelComp->GetScene() == InWorld->Scene && !SkelComp->IsTemplate())
		{
			SkelComp->SetForcedLOD(ForceLod);
		}
	}
	return true;
}

bool UGameViewportClient::HandleDisplayCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TCHAR ObjectName[256];
	TCHAR PropStr[256];
	if ( FParse::Token(Cmd, ObjectName, UE_ARRAY_COUNT(ObjectName), true) &&
		FParse::Token(Cmd, PropStr, UE_ARRAY_COUNT(PropStr), true) )
	{
		UObject* Obj = FindFirstObject<UObject>(ObjectName, EFindFirstObjectOptions::NativeFirst, ELogVerbosity::Warning, TEXT("HandleDisplayCommand"));
		if (Obj != nullptr)
		{
			FName PropertyName(PropStr, FNAME_Find);
			if (PropertyName != NAME_None && FindFProperty<FProperty>(Obj->GetClass(), PropertyName) != nullptr)
			{
				AddDebugDisplayProperty(Obj, nullptr, PropertyName);
			}
			else
			{
				Ar.Logf(TEXT("Property '%s' not found on object '%s'"), PropStr, *Obj->GetName());
			}
		}
		else
		{
			Ar.Logf(TEXT("Object not found"));
		}
	}

	return true;
}

bool UGameViewportClient::HandleDisplayAllCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TCHAR ClassName[256];
	TCHAR PropStr[256];
	if (FParse::Token(Cmd, ClassName, UE_ARRAY_COUNT(ClassName), true))
	{
		bool bValidClassToken = true;
		UClass* WithinClass = nullptr;
		{
			FString ClassStr(ClassName);
			int32 DotIndex = ClassStr.Find(TEXT("."));
			if (DotIndex != INDEX_NONE)
			{
				// first part is within class
				WithinClass = FindFirstObject<UClass>(*ClassStr.Left(DotIndex), EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleDisplayAllCommand"));
				if (WithinClass == nullptr)
				{
					Ar.Logf(TEXT("Within class not found"));
					bValidClassToken = false;
				}
				else
				{
					FCString::Strncpy(ClassName, *ClassStr.Right(ClassStr.Len() - DotIndex - 1), 256);
					bValidClassToken = FCString::Strlen(ClassName) > 0;
				}
			}
		}
		if (bValidClassToken)
		{
			FParse::Token(Cmd, PropStr, UE_ARRAY_COUNT(PropStr), true);
			UClass* Cls = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleDisplayAllCommand"));
			if (Cls != nullptr)
			{
				FName PropertyName(PropStr, FNAME_Find);
				FProperty* Prop = PropertyName != NAME_None ? FindFProperty<FProperty>(Cls, PropertyName) : nullptr;
				{
					// add all un-GCable things immediately as that list is static
					// so then we only have to iterate over dynamic things each frame
					for (TObjectIterator<UObject> It; It; ++It)
					{
						if (!GUObjectArray.IsDisregardForGC(*It))
						{
							break;
						}
						else if (It->IsA(Cls) && !It->IsTemplate() && (WithinClass == nullptr || (It->GetOuter() != nullptr && It->GetOuter()->GetClass()->IsChildOf(WithinClass))))
						{
							AddDebugDisplayProperty(*It, nullptr, PropertyName, !Prop);
						}
					}
					AddDebugDisplayProperty(Cls, WithinClass, PropertyName, !Prop);
				}
			}
			else
			{
				Ar.Logf(TEXT("Object not found"));
			}
		}
	}

	return true;
}

bool UGameViewportClient::HandleDisplayAllLocationCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TCHAR ClassName[256];
	if (FParse::Token(Cmd, ClassName, UE_ARRAY_COUNT(ClassName), true))
	{
		UClass* Cls = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleDisplayAllLocationCommand"));
		if (Cls != nullptr)
		{
			// add all un-GCable things immediately as that list is static
			// so then we only have to iterate over dynamic things each frame
			for (TObjectIterator<UObject> It(true); It; ++It)
			{
				if (!GUObjectArray.IsDisregardForGC(*It))
				{
					break;
				}
				else if (It->IsA(Cls))
				{
					AddDebugDisplayProperty(*It, nullptr, NAME_Location, true);
				}
			}
			AddDebugDisplayProperty(Cls, nullptr, NAME_Location, true);
		}
		else
		{
			Ar.Logf(TEXT("Object not found"));
		}
	}

	return true;
}

bool UGameViewportClient::HandleDisplayAllRotationCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TCHAR ClassName[256];
	if (FParse::Token(Cmd, ClassName, UE_ARRAY_COUNT(ClassName), true))
	{
		UClass* Cls = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleDisplayAllRotationCommand"));
		if (Cls != nullptr)
		{
			// add all un-GCable things immediately as that list is static
			// so then we only have to iterate over dynamic things each frame
			for (TObjectIterator<UObject> It(true); It; ++It)
			{
				if (!GUObjectArray.IsDisregardForGC(*It))
				{
					break;
				}
				else if (It->IsA(Cls))
				{
					AddDebugDisplayProperty(*It, nullptr, NAME_Rotation, true);
				}
			}
			AddDebugDisplayProperty(Cls, nullptr, NAME_Rotation, true);
		}
		else
		{
			Ar.Logf(TEXT("Object not found"));
		}
	}

	return true;
}

bool UGameViewportClient::HandleDisplayClearCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	DebugProperties.Empty();

	return true;
}

bool UGameViewportClient::HandleGetAllLocationCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// iterate through all actors of the specified class and log their location
	TCHAR ClassName[256];
	UClass* Class;

	if (FParse::Token(Cmd, ClassName, UE_ARRAY_COUNT(ClassName), 1) &&
		(Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleGetAllLocationCommand"))) != nullptr)
	{
		bool bShowPendingKills = FParse::Command(&Cmd, TEXT("SHOWPENDINGKILLS"));
		int32 cnt = 0;
		for (TObjectIterator<AActor> It; It; ++It)
		{
			if ((bShowPendingKills || IsValid(*It)) && It->IsA(Class))
			{
				FVector ActorLocation = It->GetActorLocation();
				Ar.Logf(TEXT("%i) %s (%f, %f, %f)"), cnt++, *It->GetFullName(), ActorLocation.X, ActorLocation.Y, ActorLocation.Z);
			}
		}
	}
	else
	{
		Ar.Logf(TEXT("Unrecognized class %s"), ClassName);
	}

	return true;
}

bool UGameViewportClient::HandleGetAllRotationCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// iterate through all actors of the specified class and log their rotation
	TCHAR ClassName[256];
	UClass* Class;

	if (FParse::Token(Cmd, ClassName, UE_ARRAY_COUNT(ClassName), 1) &&
		(Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleGetAllRotationCommand"))) != nullptr)
	{
		bool bShowPendingKills = FParse::Command(&Cmd, TEXT("SHOWPENDINGKILLS"));
		int32 cnt = 0;
		for (TObjectIterator<AActor> It; It; ++It)
		{
			if ((bShowPendingKills || IsValid(*It)) && It->IsA(Class))
			{
				FRotator ActorRotation = It->GetActorRotation();
				Ar.Logf(TEXT("%i) %s (%f, %f, %f)"), cnt++, *It->GetFullName(), ActorRotation.Yaw, ActorRotation.Pitch, ActorRotation.Roll);
			}
		}
	}
	else
	{
		Ar.Logf(TEXT("Unrecognized class %s"), ClassName);
	}

	return true;
}

bool UGameViewportClient::HandleTextureDefragCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	extern void appDefragmentTexturePool();
	appDefragmentTexturePool();
	return true;
}

bool UGameViewportClient::RequestBugScreenShot(const TCHAR* Cmd, bool bDisplayHUDInfo)
{
	// Path/name is the first (and only supported) argument
	FString FileName = Cmd;

	// Handle just a plain console command (e.g. "BUGSCREENSHOT").
	bool bHDREnabled = (Viewport != NULL && Viewport->GetSceneHDREnabled());
	const TCHAR* ScreenshotExtension = bHDREnabled ? TEXT("exr") : TEXT("png");
	if (FileName.Len() == 0)
	{
		FileName = bHDREnabled ? TEXT("BugScreenShot.exr") : TEXT("BugScreenShot.png");
	}

	// Handle a console command and name (e.g. BUGSCREENSHOT FOO)
	if (FileName.Contains(TEXT("/")) == false)
	{
		// Path will be <gamename>/bugit/<platform>/desc_
		const FString BaseFile = FString::Printf(TEXT("%s%s_"), *FPaths::BugItDir(), *FPaths::GetBaseFilename(FileName));

		// find the next filename in the sequence, e.g <gamename>/bugit/<platform>/desc_00000.png
		FFileHelper::GenerateNextBitmapFilename(BaseFile, ScreenshotExtension, FileName);
	}

	if (Viewport != NULL)
	{
		UWorld* const ViewportWorld = GetWorld();
		if (bDisplayHUDInfo && (ViewportWorld != nullptr))
		{
			for (FConstPlayerControllerIterator Iterator = ViewportWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				APlayerController* PlayerController = Iterator->Get();
				if (PlayerController && PlayerController->GetHUD())
				{
					PlayerController->GetHUD()->HandleBugScreenShot();
				}
			}
		}

		const bool bShowUI = true;
		const bool bAddFilenameSuffix = false;
		FScreenshotRequest::RequestScreenshot(FileName, true, bAddFilenameSuffix, bHDREnabled);
	}

	return true;
}

void UGameViewportClient::HandleViewportStatCheckEnabled(const TCHAR* InName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled)
{
	// Check to see which viewports have this enabled (current, non-current)
	const bool bEnabled = IsStatEnabled(InName);
	if (GStatProcessingViewportClient == this && GEngine->GameViewport == this)
	{
		bOutCurrentEnabled = bEnabled;
	}
	else
	{
		bOutOthersEnabled |= bEnabled;
	}
}

void UGameViewportClient::HandleViewportStatEnabled(const TCHAR* InName)
{
	// Just enable this on the active viewport
	if (GStatProcessingViewportClient == this && GEngine->GameViewport == this)
	{
		SetStatEnabled(InName, true);
	}
}

void UGameViewportClient::HandleViewportStatDisabled(const TCHAR* InName)
{
	// Just disable this on the active viewport
	if (GStatProcessingViewportClient == this && GEngine->GameViewport == this)
	{
		SetStatEnabled(InName, false);
	}
}

void UGameViewportClient::HandleViewportStatDisableAll(const bool bInAnyViewport)
{
	// Disable all on either all or the current viewport (depending on the flag)
	if (bInAnyViewport || (GStatProcessingViewportClient == this && GEngine->GameViewport == this))
	{
		SetStatEnabled(NULL, false, true);
	}
}

void UGameViewportClient::HandleRegularWindowMovedOrResized(TSharedRef<SWindow> InWindow)
{
	if ((InWindow == GetWindow()) && Viewport)
	{
		UGameEngine* const GameEngine = Cast<UGameEngine>(GEngine);
		if (GameEngine)
		{
			UGameUserSettings* const UserSettings = GameEngine->GetGameUserSettings();
			if (UserSettings != nullptr)
			{
				const FDisplayMetrics& DisplayMetrics = FSlateApplication::Get().GetCachedDisplayMetricsByRef();
				const int32 MonitorIndex = Viewport->GetCurrentMonitorIndex();
				if (DisplayMetrics.MonitorInfo.IsValidIndex(MonitorIndex))
				{
					const FString& DisplayID = DisplayMetrics.MonitorInfo[MonitorIndex].ID;
					if (DisplayID != UserSettings->GetDisplayID())
					{
						UserSettings->SetDisplayProperties(DisplayID, MonitorIndex);

						WindowDisplayChangedDelegate.Broadcast();
					}
				}
			}
		}
	}
}

void UGameViewportClient::HandleWindowDPIScaleChanged(TSharedRef<SWindow> InWindow)
{
#if WITH_EDITOR
	if (InWindow == GetWindow())
	{
		RequestUpdateDPIScale();
	}
#endif
}

void UGameViewportClient::SetViewMode(EViewModeIndex ViewMode, FOutputDevice* Output)
{
	ViewModeIndex = ViewMode;

	if (ViewModeIndex == VMI_StationaryLightOverlap)
	{
		if (Output)
		{
			Output->Logf(TEXT("This view mode is currently not supported in game."));
		}
		
		ViewModeIndex = VMI_Lit;
	}

	if (FPlatformProperties::SupportsWindowedMode() == false)
	{
		if(ViewModeIndex == VMI_Unlit
			|| ViewModeIndex == VMI_StationaryLightOverlap
			|| ViewModeIndex == VMI_Lit_DetailLighting
			|| ViewModeIndex == VMI_ReflectionOverride)
		{
			if (Output)
			{
				Output->Logf(TEXT("This view mode is currently not supported on consoles."));
			}
			ViewModeIndex = VMI_Lit;
		}
	}

#if UE_BUILD_TEST || UE_BUILD_SHIPPING
	if (Output)
	{
		Output->Logf(TEXT("Debug viewmodes not allowed in Test or Shipping builds."));
	}
	ViewModeIndex = VMI_Lit;
#else

	if ((ViewModeIndex != VMI_Lit && ViewModeIndex != VMI_ShaderComplexity) && !AllowDebugViewmodes())
	{
		if (Output)
		{
			Output->Logf(TEXT("Debug viewmodes not allowed on consoles by default.  See AllowDebugViewmodes()."));
		}
		ViewModeIndex = VMI_Lit;
	}

	if (!GRHISupportsRayTracing || !GRHISupportsRayTracingShaders)
	{
		if (ViewModeIndex == VMI_PathTracing)
		{
			if (Output)
			{
				Output->Logf(TEXT("Path Tracing view mode requires ray tracing support. It is not supported on this system."));
			}
			ViewModeIndex = VMI_Lit;
		}

		if (ViewModeIndex == VMI_RayTracingDebug)
		{
			if (Output)
			{
				Output->Logf(TEXT("Ray tracing view mode requires ray tracing support. It is not supported on this system."));
			}
			ViewModeIndex = VMI_Lit;
		}
	}
#endif

	ApplyViewMode((EViewModeIndex)ViewModeIndex, true, EngineShowFlags);
}

bool UGameViewportClient::SetHardwareCursor(EMouseCursor::Type CursorShape, FName GameContentPath, FVector2D HotSpot)
{
	TSharedPtr<ICursor> PlatformCursor = FSlateApplication::Get().GetPlatformCursor();
	if (!PlatformCursor)
	{
		return false;
	}

	void* HardwareCursor = HardwareCursorCache.FindRef(GameContentPath);
	if ( !HardwareCursor )
	{
		// Validate hot spot
		ensure(HotSpot.X >= 0.0f && HotSpot.X <= 1.0f);
		ensure(HotSpot.Y >= 0.0f && HotSpot.Y <= 1.0f);
		HotSpot.X = FMath::Clamp(HotSpot.X, 0.0f, 1.0f);
		HotSpot.Y = FMath::Clamp(HotSpot.Y, 0.0f, 1.0f);

		// Try to create cursor from file directly
		FString CursorPath = FPaths::ProjectContentDir() / GameContentPath.ToString();
		HardwareCursor = PlatformCursor->CreateCursorFromFile(CursorPath, HotSpot);
		if ( !HardwareCursor )
		{
			// Try to load from PNG
			HardwareCursor = LoadCursorFromPngs(*PlatformCursor, CursorPath, HotSpot);
			if ( !HardwareCursor )
			{
				UE_LOGF(LogInit, Error, "Failed to load cursor '%ls'", *CursorPath);
				return false;
			}
		}

		HardwareCursorCache.Add(GameContentPath, HardwareCursor);
	}

	HardwareCursors.Add(CursorShape, HardwareCursor);

	if ( bIsMouseOverClient )
	{
		PlatformCursor->SetTypeShape(CursorShape, HardwareCursor);
	}

	return true;
}

bool UGameViewportClient::IsSimulateInEditorViewport() const
{
	const FSceneViewport* GameViewport = GetGameViewport();

	return GameViewport ? GameViewport->GetPlayInEditorIsSimulate() : false;
}

bool UGameViewportClient::GetUseMouseForTouch() const
{
#if WITH_EDITOR
	return GetDefault<ULevelEditorPlaySettings>()->UseMouseForTouch || GetDefault<UInputSettings>()->bUseMouseForTouch;
#else
	return GetDefault<UInputSettings>()->bUseMouseForTouch;
#endif
}

void* UGameViewportClient::LoadCursorFromPngs(ICursor& PlatformCursor, const FString& InPathToCursorWithoutExtension, FVector2D InHotSpot)
{
	if (!PlatformCursor.IsCreateCursorFromRGBABufferSupported())
	{
		return nullptr;
	}

	TArray<TSharedPtr<FPngFileData>> CursorPngFiles;
	if (!LoadAvailableCursorPngs(CursorPngFiles, InPathToCursorWithoutExtension))
	{
		return nullptr;
	}

	check(CursorPngFiles.Num() > 0);
	TSharedPtr<FPngFileData> NearestCursor = CursorPngFiles[0];
	float PlatformScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(0, 0);
	for (TSharedPtr<FPngFileData>& FileData : CursorPngFiles)
	{
		const float NewDelta = FMath::Abs(FileData->ScaleFactor - PlatformScaleFactor);
		if (NewDelta < FMath::Abs(NearestCursor->ScaleFactor - PlatformScaleFactor))
		{
			NearestCursor = FileData;
		}
	}

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper>PngImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

	if (PngImageWrapper.IsValid() && PngImageWrapper->SetCompressed(NearestCursor->FileData.GetData(), NearestCursor->FileData.Num()))
	{
		TArray64<uint8> RawImageData;
		if (PngImageWrapper->GetRaw(ERGBFormat::RGBA, 8, RawImageData))
		{
			const int32 Width = PngImageWrapper->GetWidth();
			const int32 Height = PngImageWrapper->GetHeight();

			return PlatformCursor.CreateCursorFromRGBABuffer((FColor*) RawImageData.GetData(), Width, Height, InHotSpot);
		}
	}

	return nullptr;
}

void UGameViewportClient::AddDebugDisplayProperty(class UObject* Obj, TSubclassOf<class UObject> WithinClass, const FName& PropertyName, bool bSpecialProperty /*= false*/)
{
	// If this property already exists than don't add a new one
	for (const FDebugDisplayProperty& Prop : DebugProperties)
	{
		if (Prop.Obj == Obj && Prop.PropertyName == PropertyName)
		{
			return;
		}
	}

	FDebugDisplayProperty& NewProp = DebugProperties.AddDefaulted_GetRef();
	NewProp.Obj = Obj;
	NewProp.WithinClass = WithinClass;
	NewProp.PropertyName = PropertyName;
	NewProp.bSpecialProperty = bSpecialProperty;
}

bool UGameViewportClient::LoadAvailableCursorPngs(TArray< TSharedPtr<FPngFileData> >& Results, const FString& InPathToCursorWithoutExtension)
{
	FString CursorsWithSizeSearch = FPaths::GetCleanFilename(InPathToCursorWithoutExtension) + TEXT("*.png");

	TArray<FString> PngCursorFiles;
	IFileManager::Get().FindFilesRecursive(PngCursorFiles, *FPaths::GetPath(InPathToCursorWithoutExtension), *CursorsWithSizeSearch, true, false, false);

	bool bFoundCursor = false;

	for (const FString& FullCursorPath : PngCursorFiles)
	{
		FString CursorFile = FPaths::GetBaseFilename(FullCursorPath);

		FString Dummy;
		FString ScaleFactorSection;
		FString ScaleFactor;

		if (CursorFile.Split(TEXT("@"), &Dummy, &ScaleFactorSection, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
		{
			if (ScaleFactorSection.Split(TEXT("x"), &ScaleFactor, &Dummy) == false)
			{
				ScaleFactor = ScaleFactorSection;
			}
		}
		else
		{
			ScaleFactor = TEXT("1");
		}

		if (FCString::IsNumeric(*ScaleFactor) == false)
		{
			UE_LOGF(LogInit, Error, "Failed to load cursor '%ls', non-numeric characters in the scale factor.", *FullCursorPath);
			continue;
		}

		TSharedPtr<FPngFileData> PngFileData = MakeShared<FPngFileData>();
		PngFileData->FileName = FullCursorPath;
		PngFileData->ScaleFactor = FCString::Atof(*ScaleFactor);

		if (FFileHelper::LoadFileToArray(PngFileData->FileData, *FullCursorPath, FILEREAD_Silent))
		{
			UE_LOGF(LogInit, Log, "Loading Cursor '%ls'.", *FullCursorPath);
		}

		Results.Add(PngFileData);

		bFoundCursor = true;
	}

	Results.StableSort([](const TSharedPtr<FPngFileData>& InFirst, const TSharedPtr<FPngFileData>& InSecond) -> bool
	{
		return InFirst->ScaleFactor < InSecond->ScaleFactor;
	});

	return bFoundCursor;
}

#undef LOCTEXT_NAMESPACE

