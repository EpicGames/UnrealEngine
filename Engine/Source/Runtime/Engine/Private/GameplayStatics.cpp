// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/GameplayStatics.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Engine/OverlapResult.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "EngineLogs.h"
#include "Misc/PackageName.h"
#include "Kismet/GameplayStaticsTypes.h"
#include "Misc/EngineVersion.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "GameFramework/DamageType.h"
#include "Kismet/KismetSystemLibrary.h"
#include "SceneView.h"
#include "Components/PrimitiveComponent.h"
#include "Math/InverseRotationMatrix.h"
#include "UObject/Package.h"
#include "Engine/CollisionProfile.h"
#include "ParticleHelper.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LocalPlayer.h"
#include "DrawDebugHelpers.h"
#include "EngineUtils.h"
#include "AudioDevice.h"
#include "SaveGameSystem.h"
#include "DVRStreaming.h"
#include "PlatformFeatures.h"
#include "GameFramework/Character.h"
#include "Sound/DialogueWave.h"
#include "GameFramework/SaveGame.h"
#include "GameFramework/GameStateBase.h"
#include "GameFramework/PlayerState.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Components/DecalComponent.h"
#include "Components/ForceFeedbackComponent.h"
#include "LandscapeProxy.h"
#include "Logging/MessageLog.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/BodySetup.h"
#include "Misc/EngineVersion.h"
#include "Engine/SceneCapture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Sound/SoundCue.h"
#include "Audio/ActorSoundParameterInterface.h"
#include "Audio/AudioTraceUtil.h"
#include "Engine/DamageEvents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayStatics)

#if WITH_ACCESSIBILITY
#include "Framework/Application/SlateApplication.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

#define LOCTEXT_NAMESPACE "GameplayStatics"

namespace GameplayStatics
{
	AActor* GetActorOwnerFromWorldContextObject(UObject* WorldContextObject)
	{
		if (AActor* Actor = Cast<AActor>(WorldContextObject))
		{
			return Actor;
		}
		return WorldContextObject->GetTypedOuter<AActor>();
	}
	const AActor* GetActorOwnerFromWorldContextObject(const UObject* WorldContextObject)
	{
		if (const AActor* Actor = Cast<const AActor>(WorldContextObject))
		{
			return Actor;
		}
		return WorldContextObject->GetTypedOuter<AActor>();
	}
}

static const int UE_SAVEGAME_FILE_TYPE_TAG = 0x53415647;		// "SAVG"

struct FSaveGameFileVersion
{
	enum Type
	{
		InitialVersion = 1,
		// serializing custom versions into the savegame data to handle that type of versioning
		AddedCustomVersions = 2,
		// added a new UE5 version number to FPackageFileSummary
		PackageFileSummaryVersionChange = 3,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

DECLARE_CYCLE_STAT(TEXT("BreakHitResult"), STAT_BreakHitResult, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("MakeHitResult"), STAT_MakeHitResult, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("SpawnTime"), STAT_SpawnTime, STATGROUP_Game);

//////////////////////////////////////////////////////////////////////////
// FSaveGameHeader

// This is the engine-level header for save game versioning and is not useful for game-specific version changes
// To implement those, you would need to save the version number into the save game object using something like ULocalPlayerSaveGame
struct FSaveGameHeader
{
	FSaveGameHeader();
	FSaveGameHeader(TSubclassOf<USaveGame> ObjectType);

	void Empty();
	bool IsEmpty() const;

	void Read(FMemoryReader& MemoryReader);
	void Write(FMemoryWriter& MemoryWriter);

	int32 FileTypeTag;
	int32 SaveGameFileVersion;
	FPackageFileVersion PackageFileUEVersion;
	FEngineVersion SavedEngineVersion;
	int32 CustomVersionFormat;
	FCustomVersionContainer CustomVersions;
	FString SaveGameClassName;
};

FSaveGameHeader::FSaveGameHeader()
	: FileTypeTag(0)
	, SaveGameFileVersion(0)
	, CustomVersionFormat(static_cast<int32>(ECustomVersionSerializationFormat::Unknown))
{}

FSaveGameHeader::FSaveGameHeader(TSubclassOf<USaveGame> ObjectType)
	: FileTypeTag(UE_SAVEGAME_FILE_TYPE_TAG)
	, SaveGameFileVersion(FSaveGameFileVersion::LatestVersion)
	, PackageFileUEVersion(GPackageFileUEVersion)
	, SavedEngineVersion(FEngineVersion::Current())
	, CustomVersionFormat(static_cast<int32>(ECustomVersionSerializationFormat::Latest))
	, CustomVersions(FCurrentCustomVersions::GetAll())
	, SaveGameClassName(ObjectType->GetPathName())
{}

void FSaveGameHeader::Empty()
{
	FileTypeTag = 0;
	SaveGameFileVersion = 0;
	PackageFileUEVersion.Reset();
	SavedEngineVersion.Empty();
	CustomVersionFormat = (int32)ECustomVersionSerializationFormat::Unknown;
	CustomVersions.Empty();
	SaveGameClassName.Empty();
}

bool FSaveGameHeader::IsEmpty() const
{
	return (FileTypeTag == 0);
}

void FSaveGameHeader::Read(FMemoryReader& MemoryReader)
{
	Empty();

	MemoryReader << FileTypeTag;

	if (FileTypeTag != UE_SAVEGAME_FILE_TYPE_TAG)
	{
		// This is a very old saved game, back up the file pointer to the beginning and assume version 1
		// This is unlikely to work without additional licensee-specific modifications to this code
		MemoryReader.Seek(0);
		SaveGameFileVersion = FSaveGameFileVersion::InitialVersion;
	}
	else
	{
		// Read version for this file format
		MemoryReader << SaveGameFileVersion;

		// Read engine and UE version information
		if (SaveGameFileVersion >= FSaveGameFileVersion::PackageFileSummaryVersionChange)
		{
			MemoryReader << PackageFileUEVersion;
		}
		else
		{
			int32 OldUe4Version;
			MemoryReader << OldUe4Version;

			PackageFileUEVersion = FPackageFileVersion::CreateUE4Version(OldUe4Version);
		}

		MemoryReader << SavedEngineVersion;

		MemoryReader.SetUEVer(PackageFileUEVersion);
		MemoryReader.SetEngineVer(SavedEngineVersion);

		if (SaveGameFileVersion >= FSaveGameFileVersion::AddedCustomVersions)
		{
			MemoryReader << CustomVersionFormat;

			CustomVersions.Serialize(MemoryReader, static_cast<ECustomVersionSerializationFormat>(CustomVersionFormat));
			MemoryReader.SetCustomVersions(CustomVersions);
		}

		// This code does not handle SetLicenseeUEVer because save games are not expected to work across major licensee changes to the engine
		// If your game wants to support advanced backward compatibility, you will probably want to implement a version number on the object itself
		// ULocalPlayerSaveGame has an example of how to implement a version number in a way that can be accessed after serialization
	}

	// Get the class name
	MemoryReader << SaveGameClassName;
}

void FSaveGameHeader::Write(FMemoryWriter& MemoryWriter)
{
	// write file type tag. identifies this file type and indicates it's using proper versioning
	// since older UE versions did not version this data.
	MemoryWriter << FileTypeTag;

	// Write version for this file format
	MemoryWriter << SaveGameFileVersion;

	// Write out engine and UE version information
	MemoryWriter << PackageFileUEVersion;
	MemoryWriter << SavedEngineVersion;

	// Write out custom version data
	MemoryWriter << CustomVersionFormat;
	CustomVersions.Serialize(MemoryWriter, static_cast<ECustomVersionSerializationFormat>(CustomVersionFormat));

	// Write the class name so we know what class to load to
	MemoryWriter << SaveGameClassName;
}

//////////////////////////////////////////////////////////////////////////
// UGameplayStatics

UGameplayStatics::UGameplayStatics(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

class UGameInstance* UGameplayStatics::GetGameInstance(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetGameInstance() : nullptr;
}

int32 UGameplayStatics::GetNumPlayerStates(const UObject* WorldContextObject)
{
	AGameStateBase* GameState = GetGameState(WorldContextObject);

	if (GameState)
	{
		return GameState->PlayerArray.Num();
	}

	return 0;
}

class APlayerState* UGameplayStatics::GetPlayerState(const UObject* WorldContextObject, int32 PlayerStateIndex)
{
	AGameStateBase* GameState = GetGameState(WorldContextObject);

	if (GameState && GameState->PlayerArray.IsValidIndex(PlayerStateIndex))
	{
		return GameState->PlayerArray[PlayerStateIndex];
	}

	return nullptr;
}

class APlayerState* UGameplayStatics::GetPlayerStateFromUniqueNetId(const UObject* WorldContextObject, const FUniqueNetIdRepl& UniqueId)
{
	AGameStateBase* GameState = GetGameState(WorldContextObject);

	if (GameState)
	{
		return GameState->GetPlayerStateFromUniqueNetId(UniqueId);
	}

	return nullptr;
}

int32 UGameplayStatics::GetNumPlayerControllers(const UObject* WorldContextObject)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return World->GetNumPlayerControllers();
	}
	return 0;
}

int32 UGameplayStatics::GetNumLocalPlayerControllers(const UObject* WorldContextObject)
{
	int32 Count = 0;
	UGameInstance* GameInstance = GetGameInstance(WorldContextObject);

	// We only want Local Players that have valid player controllers
	if (GameInstance)
	{
		const TArray<ULocalPlayer*>& LocalPlayers = GameInstance->GetLocalPlayers();
		for (ULocalPlayer* LocalPlayer : LocalPlayers)
		{
			if (APlayerController* PC = LocalPlayer->PlayerController)
			{
				Count++;
			}
		}
	}
	return Count;
}

class APlayerController* UGameplayStatics::GetPlayerController(const UObject* WorldContextObject, int32 PlayerIndex) 
{
	// The order for the player controller iterator is not consistent across map transfer/etc so we don't want to use that index
	// 99% of the time people pass in index 0 and want the primary local player controller
	// After we've finished iterating the local player controllers, iterate the GameState list to find remote ones in a consistent order

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (!World)
	{
		return nullptr;
	}

	// Don't use the game instance if the passed in world isn't the primary active world
	UGameInstance* GameInstance = World->GetGameInstance();
	const bool bUseGameInstance = GameInstance && GameInstance->GetWorld() == World;

	int32 Index = 0;
	if (bUseGameInstance)
	{		
		const TArray<ULocalPlayer*>& LocalPlayers = GameInstance->GetLocalPlayers();
		for (ULocalPlayer* LocalPlayer : LocalPlayers)
		{
			// Only count local players with an actual PC as part of the indexing
			if (APlayerController* PC = LocalPlayer->PlayerController)
			{
				if (Index == PlayerIndex)
				{
					return PC;
				}
				Index++;
			}
		}
	}

	// If we have a game state, use the consistent order there to pick up remote player controllers
	AGameStateBase* GameState = World->GetGameState();
	if (GameState)
	{
		for (APlayerState* PlayerState : GameState->PlayerArray)
		{
			// Ignore local player controllers we would have found in the previous pass
			APlayerController* PC = PlayerState ? PlayerState->GetPlayerController() : nullptr;
			if (PC && !(bUseGameInstance && PC->GetLocalPlayer()))
			{
				if (Index == PlayerIndex)
				{
					return PC;
				}
				Index++;
			}
		}
	}

	// Fallback to the old behavior with a raw iterator, but only if we didn't find any potential player controllers with the other methods
	if (Index == 0)
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (Index == PlayerIndex)
			{
				return PlayerController;
			}
			Index++;
		}
	}

	return nullptr;
}

APlayerController* UGameplayStatics::GetPlayerControllerFromID(const UObject* WorldContextObject, int32 ControllerID)
{
	return GetPlayerControllerFromPlatformUser(WorldContextObject, FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerID));
}

APlayerController* UGameplayStatics::GetPlayerControllerFromPlatformUser(const UObject* WorldContextObject, FPlatformUserId UserId)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			FPlatformUserId PlayerControllerUserID = PlayerController->GetPlatformUserId();
			if (PlayerControllerUserID.IsValid() && PlayerControllerUserID == UserId)
			{
				return PlayerController;
			}
		}
	}
	return nullptr;
}

ACharacter* UGameplayStatics::GetPlayerCharacter(const UObject* WorldContextObject, int32 PlayerIndex)
{
	APlayerController* PC = GetPlayerController(WorldContextObject, PlayerIndex);
	return PC ? Cast<ACharacter>(PC->GetPawn()) : nullptr;
}

APawn* UGameplayStatics::GetPlayerPawn(const UObject* WorldContextObject, int32 PlayerIndex)
{
	APlayerController* PC = GetPlayerController(WorldContextObject, PlayerIndex);
	return PC ? PC->GetPawnOrSpectator() : nullptr;
}

APlayerCameraManager* UGameplayStatics::GetPlayerCameraManager(const UObject* WorldContextObject, int32 PlayerIndex)
{
	APlayerController* const PC = GetPlayerController(WorldContextObject, PlayerIndex);
	return PC ? PC->PlayerCameraManager : nullptr;
}

bool UGameplayStatics::IsAnyLocalPlayerCameraWithinRange(const UObject* WorldContextObject, const FVector& Location, float MaximumRange)
{
	if (!GEngine || IsRunningDedicatedServer())
	{
		return false;
	}
	
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return false;
	}

	const float MaximumRangeSq = MaximumRange * MaximumRange; 

	for (FConstPlayerControllerIterator PlayerControllerIterator = World->GetPlayerControllerIterator();
		 PlayerControllerIterator; 
		 ++PlayerControllerIterator)
	{
		const APlayerController* PlayerController = PlayerControllerIterator->Get();
		if (!PlayerController || !PlayerController->IsLocalController())
		{
			continue;
		}

		if (const APlayerCameraManager* PlayerCameraManager = PlayerController->PlayerCameraManager)
		{
			if (FVector::DistSquared(PlayerCameraManager->GetCameraLocation(), Location) <= MaximumRangeSq)
			{
				return true;
			}
		}
	}

	return false;
}

APlayerController* UGameplayStatics::CreatePlayer(const UObject* WorldContextObject, int32 ControllerId, bool bSpawnPlayerController)
{
	return CreatePlayerFromPlatformUser(WorldContextObject, FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId), bSpawnPlayerController);
}

APlayerController* UGameplayStatics::CreatePlayerFromPlatformUser(const UObject* WorldContextObject, FPlatformUserId UserId, bool bSpawnPlayerController)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	FString Error;

	ULocalPlayer* LocalPlayer = World ? World->GetGameInstance()->CreateLocalPlayer(UserId, Error, bSpawnPlayerController) : nullptr;

	if (Error.Len() > 0)
	{
		UE_LOGF(LogPlayerManagement, Error, "Failed to Create Player: %ls", *Error);
	}

	return (LocalPlayer ? LocalPlayer->PlayerController : nullptr);
}

void UGameplayStatics::RemovePlayer(APlayerController* PlayerController, bool bDestroyPawn)
{
	if (PlayerController)
	{
		if (UWorld* World = PlayerController->GetWorld())
		{
			if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
			{
				APawn* PlayerPawn = (bDestroyPawn ? PlayerController->GetPawn().Get() : nullptr);
				if (World->GetGameInstance()->RemoveLocalPlayer(LocalPlayer) && PlayerPawn)
				{
					PlayerPawn->Destroy();
				}
			}
		}
	}
}

int32 UGameplayStatics::GetPlayerControllerID(APlayerController* PlayerController)
{
	FPlatformUserId UserID = PlayerController ? PlayerController->GetPlatformUserId() : PLATFORMUSERID_NONE;
	return FGenericPlatformMisc::GetUserIndexForPlatformUser(UserID);
}

void UGameplayStatics::SetPlayerControllerID(APlayerController* PlayerController, int32 ControllerId)
{
	SetPlayerPlatformUserId(PlayerController, FPlatformMisc::GetPlatformUserForUserIndex(ControllerId));
}

void UGameplayStatics::SetPlayerPlatformUserId(APlayerController* PlayerController, FPlatformUserId UserId)
{
	if (PlayerController)
	{
		if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
		{
			LocalPlayer->SetPlatformUserId(UserId);
		}
	}
}

AGameModeBase* UGameplayStatics::GetGameMode(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetAuthGameMode() : NULL;
}

AGameStateBase* UGameplayStatics::GetGameState(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetGameState() : nullptr;
}

class UClass* UGameplayStatics::GetObjectClass(const UObject* Object)
{
	return Object ? Object->GetClass() : nullptr;
}

bool UGameplayStatics::ObjectIsA(const UObject* Object, TSubclassOf<UObject> ObjectClass)
{
	return (Object && ObjectClass) ? Object->IsA(ObjectClass) : false;
}

float UGameplayStatics::GetGlobalTimeDilation(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetWorldSettings()->TimeDilation : 1.f;
}

void UGameplayStatics::SetGlobalTimeDilation(const UObject* WorldContextObject, float TimeDilation)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World != nullptr)
	{
		AWorldSettings* const WorldSettings = World->GetWorldSettings();
		if (WorldSettings != nullptr)
		{
			float const ActualTimeDilation = WorldSettings->SetTimeDilation(TimeDilation);
			if (TimeDilation != ActualTimeDilation)
			{
				UE_LOGF(LogBlueprintUserMessages, Warning, "Time Dilation must be between %f and %f.  Clamped value to that range.", WorldSettings->MinGlobalTimeDilation, WorldSettings->MaxGlobalTimeDilation);
			}
		}
	}
}

bool UGameplayStatics::SetGamePaused(const UObject* WorldContextObject, bool bPaused)
{
	UGameInstance* const GameInstance = GetGameInstance( WorldContextObject );
	APlayerController* const PC = GameInstance ? GameInstance->GetFirstLocalPlayerController() : nullptr;
	return PC ? PC->SetPause(bPaused) : false;
}

bool UGameplayStatics::IsGamePaused(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->IsPaused() : false;
}

void UGameplayStatics::SetForceDisableSplitscreen(const UObject* WorldContextObject, bool bDisable)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		UGameViewportClient* GameViewportClient = World->GetGameViewport();
		if (GameViewportClient)
		{
			GameViewportClient->SetForceDisableSplitscreen(bDisable);
		}
	}
}

bool UGameplayStatics::IsSplitscreenForceDisabled(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		UGameViewportClient* GameViewportClient = World->GetGameViewport();
		if (GameViewportClient)
		{
			return GameViewportClient->IsSplitscreenForceDisabled();
		}
	}
	return false;
}

void UGameplayStatics::SetEnableWorldRendering(const UObject* WorldContextObject, bool bEnable)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		UGameViewportClient* GameViewportClient = World->GetGameViewport();
		if (GameViewportClient)
		{
			GameViewportClient->bDisableWorldRendering = !bEnable;
		}
	}
}

bool UGameplayStatics::GetEnableWorldRendering(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		UGameViewportClient* const GameViewportClient = World->GetGameViewport();
		if (GameViewportClient)
		{
			return !GameViewportClient->bDisableWorldRendering;
		}
	}

	return false;
}

EMouseCaptureMode UGameplayStatics::GetViewportMouseCaptureMode(const UObject* WorldContextObject)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		UGameViewportClient* const GameViewportClient = World->GetGameViewport();
		if (GameViewportClient)
		{
			return GameViewportClient->GetMouseCaptureMode();
		}
	}

	return EMouseCaptureMode::NoCapture;
}

void UGameplayStatics::SetViewportMouseCaptureMode(const UObject* WorldContextObject, const EMouseCaptureMode MouseCaptureMode)
{
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		UGameViewportClient* const GameViewportClient = World->GetGameViewport();
		if (GameViewportClient)
		{
			GameViewportClient->SetMouseCaptureMode(MouseCaptureMode);
		}
	}
}

/** @RETURN True if weapon trace from Origin hits component VictimComp.  OutHitResult will contain properties of the hit. */
static bool ComponentIsDamageableFrom(UPrimitiveComponent* VictimComp, FVector const& Origin, AActor const* IgnoredActor, const TArray<AActor*>& IgnoreActors, ECollisionChannel TraceChannel, FHitResult& OutHitResult)
{
	FCollisionQueryParams LineParams(SCENE_QUERY_STAT(ComponentIsVisibleFrom), true, IgnoredActor);
	LineParams.AddIgnoredActors( IgnoreActors );

	// Do a trace from origin to middle of box
	UWorld* const World = VictimComp->GetWorld();
	check(World);

	FVector const TraceEnd = VictimComp->Bounds.Origin;
	FVector TraceStart = Origin;
	if (Origin == TraceEnd)
	{
		// tiny nudge so LineTraceSingle doesn't early out with no hits
		TraceStart.Z += 0.01f;
	}

	// Only do a line trace if there is a valid channel, if it is invalid then result will have no fall off
	if (TraceChannel != ECollisionChannel::ECC_MAX)
	{
		bool const bHadBlockingHit = World->LineTraceSingleByChannel(OutHitResult, TraceStart, TraceEnd, TraceChannel, LineParams);
		//::DrawDebugLine(World, TraceStart, TraceEnd, FLinearColor::Red, true);

		// If there was a blocking hit, it will be the last one
		if (bHadBlockingHit)
		{
			if (OutHitResult.Component == VictimComp)
			{
				// if blocking hit was the victim component, it is visible
				return true;
			}
			else
			{
				// if we hit something else blocking, it's not
				UE_LOGF(LogDamage, Log, "Radial Damage to %ls blocked by %ls (%ls)", *GetNameSafe(VictimComp), *OutHitResult.GetHitObjectHandle().GetName(), *GetNameSafe(OutHitResult.Component.Get()));
				return false;
			}
		}
	}
	else
	{
		UE_LOGF(LogDamage, Warning, "ECollisionChannel::ECC_MAX is not valid! No falloff is added to damage");
	}

	// didn't hit anything, assume nothing blocking the damage and victim is consequently visible
	// but since we don't have a hit result to pass back, construct a simple one, modeling the damage as having hit a point at the component's center.
	FVector const FakeHitLoc = VictimComp->GetComponentLocation();
	FVector const FakeHitNorm = (Origin - FakeHitLoc).GetSafeNormal();		// normal points back toward the epicenter
	OutHitResult = FHitResult(VictimComp->GetOwner(), VictimComp, FakeHitLoc, FakeHitNorm);
	return true;
}

bool UGameplayStatics::ApplyRadialDamage(const UObject* WorldContextObject, float BaseDamage, const FVector& Origin, float DamageRadius, TSubclassOf<UDamageType> DamageTypeClass, const TArray<AActor*>& IgnoreActors, AActor* DamageCauser, AController* InstigatedByController, bool bDoFullDamage, ECollisionChannel DamagePreventionChannel )
{
	float DamageFalloff = bDoFullDamage ? 0.f : 1.f;
	return ApplyRadialDamageWithFalloff(WorldContextObject, BaseDamage, 0.f, Origin, 0.f, DamageRadius, DamageFalloff, DamageTypeClass, IgnoreActors, DamageCauser, InstigatedByController, DamagePreventionChannel);
}

bool UGameplayStatics::ApplyRadialDamageWithFalloff(const UObject* WorldContextObject, float BaseDamage, float MinimumDamage, const FVector& Origin, float DamageInnerRadius, float DamageOuterRadius, float DamageFalloff, TSubclassOf<class UDamageType> DamageTypeClass, const TArray<AActor*>& IgnoreActors, AActor* DamageCauser, AController* InstigatedByController, ECollisionChannel DamagePreventionChannel)
{
	FCollisionQueryParams SphereParams(SCENE_QUERY_STAT(ApplyRadialDamage),  false, DamageCauser);

	SphereParams.AddIgnoredActors(IgnoreActors);

	// query scene to see what we hit
	TArray<FOverlapResult> Overlaps;
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		World->OverlapMultiByObjectType(Overlaps, Origin, FQuat::Identity, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllDynamicObjects), FCollisionShape::MakeSphere(DamageOuterRadius), SphereParams);
	}

	// collate into per-actor list of hit components
	TMap<AActor*, TArray<FHitResult> > OverlapComponentMap;
	for (const FOverlapResult& Overlap : Overlaps)
	{
		AActor* const OverlapActor = Overlap.OverlapObjectHandle.FetchActor();

		if (OverlapActor &&
			OverlapActor->CanBeDamaged() &&
			OverlapActor != DamageCauser &&
			Overlap.Component.IsValid())
		{
			FHitResult Hit;
			if (ComponentIsDamageableFrom(Overlap.Component.Get(), Origin, DamageCauser, IgnoreActors, DamagePreventionChannel, Hit))
			{
				TArray<FHitResult>& HitList = OverlapComponentMap.FindOrAdd(OverlapActor);
				HitList.Add(Hit);
			}
		}
	}

	bool bAppliedDamage = false;

	if (OverlapComponentMap.Num() > 0)
	{
		// make sure we have a good damage type
		TSubclassOf<UDamageType> const ValidDamageTypeClass = DamageTypeClass ? DamageTypeClass : TSubclassOf<UDamageType>(UDamageType::StaticClass());

		FRadialDamageEvent DmgEvent;
		DmgEvent.DamageTypeClass = ValidDamageTypeClass;
		DmgEvent.Origin = Origin;
		DmgEvent.Params = FRadialDamageParams(BaseDamage, MinimumDamage, DamageInnerRadius, DamageOuterRadius, DamageFalloff);

		// call damage function on each affected actors
		for (TMap<AActor*, TArray<FHitResult> >::TIterator It(OverlapComponentMap); It; ++It)
		{
			AActor* const Victim = It.Key();
			TArray<FHitResult> const& ComponentHits = It.Value();
			DmgEvent.ComponentHits = ComponentHits;

			Victim->TakeDamage(BaseDamage, DmgEvent, InstigatedByController, DamageCauser);

			bAppliedDamage = true;
		}
	}

	return bAppliedDamage;
}

float UGameplayStatics::ApplyPointDamage(AActor* DamagedActor, float BaseDamage, FVector const& HitFromDirection, FHitResult const& HitInfo, AController* EventInstigator, AActor* DamageCauser, TSubclassOf<UDamageType> DamageTypeClass)
{
	if (DamagedActor && BaseDamage != 0.f)
	{
		// make sure we have a good damage type
		TSubclassOf<UDamageType> const ValidDamageTypeClass = DamageTypeClass ? DamageTypeClass : TSubclassOf<UDamageType>(UDamageType::StaticClass());
		FPointDamageEvent PointDamageEvent(BaseDamage, HitInfo, HitFromDirection, ValidDamageTypeClass);

		return DamagedActor->TakeDamage(BaseDamage, PointDamageEvent, EventInstigator, DamageCauser);
	}

	return 0.f;
}

float UGameplayStatics::ApplyDamage(AActor* DamagedActor, float BaseDamage, AController* EventInstigator, AActor* DamageCauser, TSubclassOf<UDamageType> DamageTypeClass)
{
	if ( DamagedActor && (BaseDamage != 0.f) )
	{
		// make sure we have a good damage type
		TSubclassOf<UDamageType> const ValidDamageTypeClass = DamageTypeClass ? DamageTypeClass : TSubclassOf<UDamageType>(UDamageType::StaticClass());
		FDamageEvent DamageEvent(ValidDamageTypeClass);

		return DamagedActor->TakeDamage(BaseDamage, DamageEvent, EventInstigator, DamageCauser);
	}

	return 0.f;
}

UObject* UGameplayStatics::SpawnObject(TSubclassOf<UObject> ObjectClass, UObject* Outer)
{
	if (*ObjectClass == nullptr)
	{
		UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnObject no class specified");
		return nullptr;
	}

	if (!Outer)
	{
		UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnObject null outer");
		return nullptr;
	}

	if (ObjectClass->ClassWithin && !Outer->IsA(ObjectClass->ClassWithin))
	{
		UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnObject outer %ls is not %ls", *GetPathNameSafe(Outer), *GetPathNameSafe(ObjectClass->ClassWithin));
		return nullptr;
	}

	return NewObject<UObject>(Outer, ObjectClass, NAME_None, RF_StrongRefOnFrame);
}

class AActor* UGameplayStatics::BeginSpawningActorFromBlueprint(const UObject* WorldContextObject, const class UBlueprint* Blueprint, const FTransform& SpawnTransform, bool bNoCollisionFail)
{
	if (Blueprint && Blueprint->GeneratedClass)
	{
		if( Blueprint->GeneratedClass->IsChildOf(AActor::StaticClass()) )
		{
			ESpawnActorCollisionHandlingMethod const CollisionHandlingOverride = bNoCollisionFail ? ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding : ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			return BeginDeferredActorSpawnFromClass(WorldContextObject, *Blueprint->GeneratedClass, SpawnTransform, CollisionHandlingOverride);
		}
		else
		{
			UE_LOGF(LogScript, Warning, "UGameplayStatics::BeginSpawningActorFromBlueprint: %ls is not an actor class", *Blueprint->GeneratedClass->GetName() );
		}
	}
	return nullptr;
}

class AActor* UGameplayStatics::BeginDeferredActorSpawnFromClass(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, const FTransform& SpawnTransform, ESpawnActorCollisionHandlingMethod CollisionHandlingOverride /*= ESpawnActorCollisionHandlingMethod::Undefined*/, AActor* Owner /*= nullptr*/, ESpawnActorScaleMethod TransformScaleMethod /*= ESpawnActorScaleMethod::MultiplyWithRoot*/)
{
	SCOPE_CYCLE_COUNTER(STAT_SpawnTime);
	if (UClass* Class = *ActorClass)
	{
		// If the WorldContextObject is a Pawn we will use that as the instigator.
		// Otherwise if the WorldContextObject is an Actor we will share its instigator.
		// If the value is set via the exposed parameter on SpawnNode it will be overwritten anyways, so this is safe to specify here
		UObject* MutableWorldContextObject = const_cast<UObject*>(WorldContextObject);
		APawn* AutoInstigator = Cast<APawn>(MutableWorldContextObject);
		if (AutoInstigator == nullptr)
		{
			if (AActor* ContextActor = Cast<AActor>(MutableWorldContextObject))
			{
				AutoInstigator = ContextActor->GetInstigator();
			}
		}

		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			return World->SpawnActorDeferred<AActor>(Class, SpawnTransform, Owner, AutoInstigator, CollisionHandlingOverride, TransformScaleMethod);
		}
		else
		{
			//@TODO: RuntimeErrors: Overlogging
			UE_LOGF(LogScript, Warning, "UGameplayStatics::BeginDeferredActorSpawnFromClass: %ls can not be spawned in NULL world", *Class->GetName());		
		}
	}
	else
	{
		UE_LOGF(LogScript, Warning, "UGameplayStatics::BeginDeferredActorSpawnFromClass: can not spawn an actor from a NULL class");
	}
	return nullptr;
}

AActor* UGameplayStatics::FinishSpawningActor(AActor* Actor, const FTransform& SpawnTransform, ESpawnActorScaleMethod TransformScaleMethod)
{
	SCOPE_CYCLE_COUNTER(STAT_SpawnTime);
	if (Actor)
	{
		Actor->FinishSpawning(SpawnTransform, false, nullptr, TransformScaleMethod);
	}

	return Actor;
}

void UGameplayStatics::LoadStreamLevel(const UObject* WorldContextObject, FName LevelName, bool bMakeVisibleAfterLoad, bool bShouldBlockOnLoad, FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FStreamLevelAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FStreamLevelAction* NewAction = new FStreamLevelAction(true, LevelName, bMakeVisibleAfterLoad, bShouldBlockOnLoad, LatentInfo, World);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
	}
}

void UGameplayStatics::LoadStreamLevelBySoftObjectPtr(const UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, bool bMakeVisibleAfterLoad, bool bShouldBlockOnLoad, FLatentActionInfo LatentInfo)
{
	const FName LevelName = FName(*FPackageName::ObjectPathToPackageName(Level.ToString()));
	LoadStreamLevel(WorldContextObject, LevelName, bMakeVisibleAfterLoad, bShouldBlockOnLoad, LatentInfo);
}

void UGameplayStatics::UnloadStreamLevel(const UObject* WorldContextObject, FName LevelName, FLatentActionInfo LatentInfo, bool bShouldBlockOnUnload)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FStreamLevelAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FStreamLevelAction* NewAction = new FStreamLevelAction(false, LevelName, false, bShouldBlockOnUnload, LatentInfo, World );
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction );
		}
	}
}

void UGameplayStatics::UnloadStreamLevelBySoftObjectPtr(const UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, FLatentActionInfo LatentInfo, bool bShouldBlockOnUnload)
{
	const FName LevelName = FName(*FPackageName::ObjectPathToPackageName(Level.ToString()));
	UnloadStreamLevel(WorldContextObject, LevelName, LatentInfo, bShouldBlockOnUnload);
}

ULevelStreaming* UGameplayStatics::GetStreamingLevel(const UObject* WorldContextObject, FName InPackageName)
{
	if (InPackageName != NAME_None)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			FString SearchPackageName = FStreamLevelAction::MakeSafeLevelName(InPackageName, World);
			if (FPackageName::IsShortPackageName(SearchPackageName))
			{
				// Make sure MyMap1 and Map1 names do not resolve to a same streaming level
				SearchPackageName = TEXT("/") + SearchPackageName;
			}

			for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
			{
				// We check only suffix of package name, to handle situations when packages were saved for play into a temporary folder
				// Like Saved/Autosaves/PackageName
				if (LevelStreaming && LevelStreaming->GetWorldAssetPackageName().EndsWith(SearchPackageName, ESearchCase::IgnoreCase))
				{
					return LevelStreaming;
				}
			}
		}
	}
	
	return NULL;
}

void UGameplayStatics::FlushLevelStreaming(const UObject* WorldContextObject)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		World->FlushLevelStreaming();
	}
}

void UGameplayStatics::CancelAsyncLoading()
{

}

void UGameplayStatics::OpenLevel(const UObject* WorldContextObject, FName LevelName, bool bAbsolute, FString Options)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World == nullptr)
	{
		return;
	}

	const ETravelType TravelType = (bAbsolute ? TRAVEL_Absolute : TRAVEL_Relative);
	FWorldContext &WorldContext = GEngine->GetWorldContextFromWorldChecked(World);
	FString Cmd = LevelName.ToString();
	if (Options.Len() > 0)
	{
		Cmd += FString("?") + Options;
	}
	FURL TestURL(&WorldContext.LastURL, *Cmd, TravelType);
	if (TestURL.IsLocalInternal())
	{
		// make sure the file exists if we are opening a local file
		if (!GEngine->MakeSureMapNameIsValid(TestURL.Map))
		{
			UE_LOGF(LogLevel, Warning, "WARNING: The map '%ls' does not exist.", *TestURL.Map);
		}
	}

	GEngine->SetClientTravel( World, *Cmd, TravelType );
}

void UGameplayStatics::OpenLevelBySoftObjectPtr(const UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, bool bAbsolute, FString Options)
{
	const FName LevelName = FName(*FPackageName::ObjectPathToPackageName(Level.ToString()));
	UGameplayStatics::OpenLevel(WorldContextObject, LevelName, bAbsolute, Options);
}

FString UGameplayStatics::GetCurrentLevelName(const UObject* WorldContextObject, bool bRemovePrefixString)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FString LevelName = World->GetMapName();
		if (bRemovePrefixString)
		{
			LevelName.RemoveFromStart(World->StreamingLevelsPrefix);
		}
		return LevelName;
	}
	return FString();
}

FVector UGameplayStatics::GetActorArrayAverageLocation(const TArray<AActor*>& Actors)
{
	FVector LocationSum(0,0,0); // sum of locations
	int32 ActorCount = 0; // num actors
	// iterate over actors
	for(int32 ActorIdx=0; ActorIdx<Actors.Num(); ActorIdx++)
	{
		AActor* A = Actors[ActorIdx];
		// Check actor is non-null, not deleted, and has a root component
		if (IsValid(A) && A->GetRootComponent())
		{
			LocationSum += A->GetActorLocation();
			ActorCount++;
		}
	}

	// Find average
	FVector Average(0,0,0);
	if(ActorCount > 0)
	{
		Average = LocationSum/((float)ActorCount);
	}
	return Average;
}

void UGameplayStatics::GetActorArrayBounds(const TArray<AActor*>& Actors, bool bOnlyCollidingComponents, FVector& Center, FVector& BoxExtent)
{
	FBox ActorBounds(ForceInit);
	// Iterate over actors and accumulate bouding box
	for(int32 ActorIdx=0; ActorIdx<Actors.Num(); ActorIdx++)
	{
		AActor* A = Actors[ActorIdx];
		// Check actor is non-null, not deleted
		if(IsValid(A))
		{
			ActorBounds += A->GetComponentsBoundingBox(!bOnlyCollidingComponents);
		}
	}

	// if a valid box, get its center and extent
	Center = BoxExtent = FVector::ZeroVector;
	if(ActorBounds.IsValid)
	{
		Center = ActorBounds.GetCenter();
		BoxExtent = ActorBounds.GetExtent();
	}
}

AActor* UGameplayStatics::GetActorOfClass(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_GetActorOfClass);

	// We do nothing if no class is provided
	if (ActorClass)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			if (TActorIterator<AActor> It(World, ActorClass); It)
			{
				AActor* Actor = *It;
				return Actor;
			}
		}
	}

	return nullptr;
}

void UGameplayStatics::GetAllActorsOfClass(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, TArray<AActor*>& OutActors)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_GetAllActorsOfClass);
	OutActors.Reset();

	// We do nothing if no class is provided, rather than giving ALL actors!
	if (ActorClass)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			for (TActorIterator<AActor> It(World, ActorClass); It; ++It)
			{
				AActor* Actor = *It;
				OutActors.Add(Actor);
			}
		}
	}
}

void UGameplayStatics::GetAllActorsWithInterface(const UObject* WorldContextObject, TSubclassOf<UInterface> Interface, TArray<AActor*>& OutActors)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_GetAllActorsWithInterface);
	OutActors.Reset();

	// We do nothing if no interface provided, rather than giving ALL actors!
	if (!Interface)
	{
		return;
	}

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		for (FActorIterator It(World); It; ++It)
		{
			AActor* Actor = *It;
			if (Actor->GetClass()->ImplementsInterface(Interface))
			{
				OutActors.Add(Actor);
			}
		}
	}
}

void UGameplayStatics::GetAllActorsWithTag(const UObject* WorldContextObject, FName Tag, TArray<AActor*>& OutActors)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_GetAllActorsWithTag);
	OutActors.Reset();

	// We do nothing if no tag is provided, rather than giving ALL actors!
	if (Tag.IsNone())
	{
		return;
	}
	
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		for (FActorIterator It(World); It; ++It)
		{
			AActor* Actor = *It;
			if (Actor->ActorHasTag(Tag))
			{
				OutActors.Add(Actor);
			}
		}
	}
}


void UGameplayStatics::GetAllActorsOfClassWithTag(const UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, FName Tag, TArray<AActor*>& OutActors)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_GetAllActorsOfClass);
	OutActors.Reset();

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	// We do nothing if no class is provided, rather than giving ALL actors!
	if (ActorClass && World)
	{
		for (TActorIterator<AActor> It(World, ActorClass); It; ++It)
		{
			AActor* Actor = *It;
			if (IsValid(Actor) && Actor->ActorHasTag(Tag))
			{
				OutActors.Add(Actor);
			}
		}
	}
}

AActor* UGameplayStatics::FindNearestActor(FVector Origin, const TArray<AActor*>& ActorsToCheck, float& Distance)
{
	AActor* NearestActor = nullptr;
	float DistanceFromNearestActor = Distance = TNumericLimits<float>::Max();

	for (AActor* ActorToCheck : ActorsToCheck)
	{
		if (ActorToCheck)
		{
			const float DistanceFromActorToCheck = (Origin - ActorToCheck->GetActorLocation()).SizeSquared();
			if (DistanceFromActorToCheck < DistanceFromNearestActor)
			{
				NearestActor = ActorToCheck;
				DistanceFromNearestActor = DistanceFromActorToCheck;
			}
		}
	}

	if (NearestActor)
	{
		Distance = FMath::Sqrt(DistanceFromNearestActor);
	}

	return NearestActor;
}


void UGameplayStatics::PlayWorldCameraShake(const UObject* WorldContextObject, TSubclassOf<class UCameraShakeBase> Shake, FVector Epicenter, float InnerRadius, float OuterRadius, float Falloff, bool bOrientShakeTowardsEpicenter)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if(World != nullptr)
	{
		APlayerCameraManager::PlayWorldCameraShake(World, Shake, Epicenter, InnerRadius, OuterRadius, Falloff, bOrientShakeTowardsEpicenter);
	}
}

UParticleSystemComponent* CreateParticleSystem(UParticleSystem* EmitterTemplate, UWorld* World, AActor* Actor, bool bAutoDestroy, EPSCPoolMethod PoolingMethod)
{
	//Defaulting to creating systems from a pool. Can be disabled via fx.ParticleSystemPool.Enable 0
	UParticleSystemComponent* PSC = nullptr;

	if (FApp::CanEverRender() && World && !World->IsNetMode(NM_DedicatedServer))
	{
		if (PoolingMethod != EPSCPoolMethod::None)
		{
			//If system is set to auto destroy the we should be safe to automatically allocate from a the world pool.
			PSC = World->GetPSCPool().CreateWorldParticleSystem(EmitterTemplate, World, PoolingMethod);
		}
		else
		{
			PSC = NewObject<UParticleSystemComponent>((Actor ? Actor : (UObject*)World), NAME_None, RF_Transient);
			PSC->bAutoDestroy = bAutoDestroy;
			PSC->bAllowAnyoneToDestroyMe = true;
			PSC->SecondsBeforeInactive = 0.0f;
			PSC->bAutoActivate = false;
			PSC->SetTemplate(EmitterTemplate);
			PSC->bOverrideLODMethod = false;
		}
	}

	return PSC;
}

UParticleSystemComponent* UGameplayStatics::SpawnEmitterAtLocation(const UObject* WorldContextObject, UParticleSystem* EmitterTemplate, FVector SpawnLocation, FRotator SpawnRotation, bool bAutoDestroy, EPSCPoolMethod PoolingMethod, bool bAutoActivateSystem)
{
	return SpawnEmitterAtLocation(WorldContextObject, EmitterTemplate, SpawnLocation, SpawnRotation, FVector(1.f), bAutoDestroy, PoolingMethod, bAutoActivateSystem);
}

UParticleSystemComponent* UGameplayStatics::InternalSpawnEmitterAtLocation(UWorld* World, UParticleSystem* EmitterTemplate, FVector SpawnLocation, FRotator SpawnRotation, FVector SpawnScale, bool bAutoDestroy, EPSCPoolMethod PoolingMethod, bool bAutoActivateSystem)
{
	check(World && EmitterTemplate);

	UParticleSystemComponent* PSC = CreateParticleSystem(EmitterTemplate, World, World->GetWorldSettings(), bAutoDestroy, PoolingMethod);
	if (PSC)
	{
		PSC->SetUsingAbsoluteLocation(true);
		PSC->SetUsingAbsoluteRotation(true);
		PSC->SetUsingAbsoluteScale(true);
		PSC->SetRelativeLocation_Direct(SpawnLocation);
		PSC->SetRelativeRotation_Direct(SpawnRotation);
		PSC->SetRelativeScale3D_Direct(SpawnScale);

		PSC->RegisterComponentWithWorld(World);
		if (bAutoActivateSystem)
		{
			PSC->ActivateSystem(true);
		}
	

		// Notify the texture streamer so that PSC gets managed as a dynamic component.
		IStreamingManager::Get().NotifyPrimitiveUpdated(PSC);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (PSC->Template && PSC->Template->IsImmortal())
		{
			UE_LOGF(LogParticles, Warning, "GameplayStatics::SpawnEmitterAtLocation spawned potentially immortal particle system! %ls (%ls) may stay in world despite never spawning particles after burst spawning is over.",
				*(PSC->GetPathName()), *(PSC->Template->GetPathName())
			);
		}
#endif

		return PSC;
	}

	return nullptr;
}

UParticleSystemComponent* UGameplayStatics::SpawnEmitterAtLocation(const UObject* WorldContextObject, UParticleSystem* EmitterTemplate, FVector SpawnLocation, FRotator SpawnRotation, FVector SpawnScale, bool bAutoDestroy, EPSCPoolMethod PoolingMethod, bool bAutoActivateSystem)
{
	UParticleSystemComponent* PSC = nullptr;
	if (EmitterTemplate)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			PSC = InternalSpawnEmitterAtLocation(World, EmitterTemplate, SpawnLocation, SpawnRotation, SpawnScale, bAutoDestroy, PoolingMethod, bAutoActivateSystem);
		}
	}
	return PSC;
}

UParticleSystemComponent* UGameplayStatics::SpawnEmitterAtLocation(UWorld* World, UParticleSystem* EmitterTemplate, const FTransform& SpawnTransform, bool bAutoDestroy, EPSCPoolMethod PoolingMethod, bool bAutoActivateSystem)
{
	UParticleSystemComponent* PSC = nullptr;
	if (World && EmitterTemplate)
	{
		PSC = InternalSpawnEmitterAtLocation(World, EmitterTemplate, SpawnTransform.GetLocation(), SpawnTransform.GetRotation().Rotator(), SpawnTransform.GetScale3D(), bAutoDestroy, PoolingMethod, bAutoActivateSystem);
	}
	return PSC;
}

UParticleSystemComponent* UGameplayStatics::SpawnEmitterAttached(UParticleSystem* EmitterTemplate, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, bool bAutoDestroy, EPSCPoolMethod PoolingMethod, bool bAutoActivateSystem)
{
	return SpawnEmitterAttached(EmitterTemplate, AttachToComponent, AttachPointName, Location, Rotation, FVector(1.f), LocationType, bAutoDestroy, PoolingMethod, bAutoActivateSystem);
}

UParticleSystemComponent* UGameplayStatics::SpawnEmitterAttached(UParticleSystem* EmitterTemplate, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, FVector Scale, EAttachLocation::Type LocationType, bool bAutoDestroy, EPSCPoolMethod PoolingMethod, bool bAutoActivateSystem)
{
	UParticleSystemComponent* PSC = nullptr;
	if (EmitterTemplate)
	{
		if (AttachToComponent == nullptr)
		{
			UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnEmitterAttached: NULL AttachComponent specified!");
		}
		else
		{
			UWorld* const World = AttachToComponent->GetWorld();
			if (World && !World->IsNetMode(NM_DedicatedServer))
			{
				PSC = CreateParticleSystem(EmitterTemplate, World, AttachToComponent->GetOwner(), bAutoDestroy, PoolingMethod);

				if (PSC != nullptr)
				{
					PSC->SetupAttachment(AttachToComponent, AttachPointName);

					if (LocationType == EAttachLocation::KeepWorldPosition)
					{
						const FTransform ParentToWorld = AttachToComponent->GetSocketTransform(AttachPointName);
						const FTransform ComponentToWorld(Rotation, Location, Scale);
						const FTransform RelativeTM = ComponentToWorld.GetRelativeTransform(ParentToWorld);
						PSC->SetRelativeLocation_Direct(RelativeTM.GetLocation());
						PSC->SetRelativeRotation_Direct(RelativeTM.GetRotation().Rotator());
						PSC->SetRelativeScale3D_Direct(RelativeTM.GetScale3D());
					}
					else
					{
						PSC->SetRelativeLocation_Direct(Location);
						PSC->SetRelativeRotation_Direct(Rotation);

						if (LocationType == EAttachLocation::SnapToTarget)
						{
							// SnapToTarget indicates we "keep world scale", this indicates we we want the inverse of the parent-to-world scale 
							// to calculate world scale at Scale 1, and then apply the passed in Scale
							const FTransform ParentToWorld = AttachToComponent->GetSocketTransform(AttachPointName);
							PSC->SetRelativeScale3D_Direct(Scale * ParentToWorld.GetSafeScaleReciprocal(ParentToWorld.GetScale3D()));
						}
						else
						{
							PSC->SetRelativeScale3D_Direct(Scale);
						}
					}

					PSC->RegisterComponentWithWorld(World);
					if(bAutoActivateSystem)
					{
						PSC->ActivateSystem(true);
					}

					// Notify the texture streamer so that PSC gets managed as a dynamic component.
					IStreamingManager::Get().NotifyPrimitiveUpdated(PSC);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (PSC->Template && PSC->Template->IsImmortal())
					{
						const FString OnScreenMessage = FString::Printf(TEXT("SpawnEmitterAttached spawned potentially immortal particle system! %s (%s) may stay in world despite never spawning particles after burst spawning is over."), *(PSC->GetPathName()), *(PSC->Template->GetName()));
						GEngine->AddOnScreenDebugMessage((uint64)((PTRINT)AttachToComponent), 3.f, FColor::Red, OnScreenMessage);
						UE_LOGF(LogParticles, Log, "GameplayStatics::SpawnEmitterAttached spawned potentially immortal particle system! %ls (%ls) may stay in world despite never spawning particles after burst spawning is over.",
							*(PSC->GetPathName()), *(PSC->Template->GetPathName())
						);
					}
#endif
				}
			}
		}
	}
	return PSC;
}

// FRED_TODO: propagate hit object handles further instead of converting to an actor here
void UGameplayStatics::BreakHitResult(const FHitResult& Hit, bool& bBlockingHit, bool& bInitialOverlap, float& Time, float& Distance, FVector& Location, FVector& ImpactPoint, FVector& Normal, FVector& ImpactNormal, UPhysicalMaterial*& PhysMat, AActor*& HitActor, UPrimitiveComponent*& HitComponent, FName& HitBoneName, FName& BoneName, int32& HitItem, int32& ElementIndex, int32& FaceIndex, FVector& TraceStart, FVector& TraceEnd)
{
	SCOPE_CYCLE_COUNTER(STAT_BreakHitResult);
	bBlockingHit = Hit.bBlockingHit;
	bInitialOverlap = Hit.bStartPenetrating;
	Time = Hit.Time;
	Distance = Hit.Distance;
	Location = Hit.Location;
	ImpactPoint = Hit.ImpactPoint;
	Normal = Hit.Normal;
	ImpactNormal = Hit.ImpactNormal;	
	PhysMat = Hit.PhysMaterial.Get();
	HitActor = Hit.GetHitObjectHandle().FetchActor();
	HitComponent = Hit.GetComponent();
	HitBoneName = Hit.BoneName;
	BoneName = Hit.MyBoneName;
	HitItem = Hit.Item;
	ElementIndex = Hit.ElementIndex;
	TraceStart = Hit.TraceStart;
	TraceEnd = Hit.TraceEnd;
	FaceIndex = Hit.FaceIndex;
}

FHitResult UGameplayStatics::MakeHitResult(bool bBlockingHit, bool bInitialOverlap, float Time, float Distance, FVector Location, FVector ImpactPoint, FVector Normal, FVector ImpactNormal, class UPhysicalMaterial* PhysMat, class AActor* HitActor, class UPrimitiveComponent* HitComponent, FName HitBoneName, FName BoneName, int32 HitItem, int32 ElementIndex, int32 FaceIndex, FVector TraceStart, FVector TraceEnd)
{
	SCOPE_CYCLE_COUNTER(STAT_MakeHitResult);
	FHitResult Hit;
	Hit.bBlockingHit = bBlockingHit;
	Hit.bStartPenetrating = bInitialOverlap;
	Hit.Time = Time;
	Hit.Distance = Distance;
	Hit.Location = Location;
	Hit.ImpactPoint = ImpactPoint;
	Hit.Normal = Normal;
	Hit.ImpactNormal = ImpactNormal;
	Hit.PhysMaterial = PhysMat;
	Hit.HitObjectHandle = FActorInstanceHandle(HitActor, HitComponent, HitItem);
	Hit.Component = HitComponent;
	Hit.BoneName = HitBoneName;
	Hit.MyBoneName = BoneName;
	Hit.Item = HitItem;
	Hit.ElementIndex = ElementIndex;
	Hit.TraceStart = TraceStart;
	Hit.TraceEnd = TraceEnd;
	Hit.FaceIndex = FaceIndex;
	return Hit;
}

EPhysicalSurface UGameplayStatics::GetSurfaceType(const struct FHitResult& Hit)
{
	UPhysicalMaterial* const HitPhysMat = Hit.PhysMaterial.Get();
	return UPhysicalMaterial::DetermineSurfaceType( HitPhysMat );
}

bool UGameplayStatics::FindCollisionUV(const struct FHitResult& Hit, int32 UVChannel, FVector2D& UV)
{
	bool bSuccess = false;

	if (!UPhysicsSettings::Get()->bSupportUVFromHitResults)
	{
		FMessageLog("PIE").Warning(LOCTEXT("CollisionUVNoSupport", "Calling FindCollisionUV but 'Support UV From Hit Results' is not enabled in project settings. This is required for finding UV for collision results."));
	}
	else
	{
		UPrimitiveComponent* HitPrimComp = Hit.Component.Get();
		if (HitPrimComp)
		{
			UBodySetup* BodySetup = HitPrimComp->GetBodySetup();
			if (BodySetup)
			{
				const FVector LocalHitPos = HitPrimComp->GetComponentToWorld().InverseTransformPosition(Hit.Location);

				bSuccess = BodySetup->CalcUVAtLocation(LocalHitPos, Hit.FaceIndex, UVChannel, UV);
			}
		}
	}

	return bSuccess;
}

bool UGameplayStatics::AreAnyListenersWithinRange(const UObject* WorldContextObject, const FVector& Location, float MaximumRange)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return false;
	}
	
	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld)
	{
		return false;
	}
	
	// If there is no valid world from the world context object then there certainly are no listeners
	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		return AudioDevice->LocationIsAudible(Location, MaximumRange);
	}	

	return false;
}

bool UGameplayStatics::GetClosestListenerLocation(const UObject* WorldContextObject, const FVector& Location, float MaximumRange, const bool bAllowAttenuationOverride, FVector& ListenerPosition)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return false;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld)
	{
		return false;
	}

	// If there is no valid world from the world context object then there certainly are no listeners
	FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice();
	if (!AudioDevice)
	{
		return false;
	}

	float OutDistSq;
	const int32 ClosestListenerIndex = AudioDevice->FindClosestListenerIndex(Location, OutDistSq, bAllowAttenuationOverride);
	if (ClosestListenerIndex == INDEX_NONE || ((MaximumRange * MaximumRange) < OutDistSq))
	{
		return false;
	}

	return AudioDevice->GetListenerPosition(ClosestListenerIndex, ListenerPosition, bAllowAttenuationOverride);
}

void UGameplayStatics::SetGlobalPitchModulation(const UObject* WorldContextObject, float PitchModulation, float TimeSec)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->SetGlobalPitchModulation(PitchModulation, TimeSec);
	}
}

void UGameplayStatics::SetSoundClassDistanceScale(const UObject* WorldContextObject, USoundClass* SoundClass, float DistanceAttenuationScale, float TimeSec)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->SetSoundClassDistanceScale(SoundClass, DistanceAttenuationScale, TimeSec);
	}
}

void UGameplayStatics::SetGlobalListenerFocusParameters(const UObject* WorldContextObject, float FocusAzimuthScale, float NonFocusAzimuthScale, float FocusDistanceScale, float NonFocusDistanceScale, float FocusVolumeScale, float NonFocusVolumeScale, float FocusPriorityScale, float NonFocusPriorityScale)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		FGlobalFocusSettings NewFocusSettings;
		NewFocusSettings.FocusAzimuthScale = FMath::Max(FocusAzimuthScale, 0.0f);
		NewFocusSettings.NonFocusAzimuthScale = FMath::Max(NonFocusAzimuthScale, 0.0f);
		NewFocusSettings.FocusDistanceScale = FMath::Max(FocusDistanceScale, 0.0f);
		NewFocusSettings.NonFocusDistanceScale = FMath::Max(NonFocusDistanceScale, 0.0f);
		NewFocusSettings.FocusVolumeScale = FMath::Max(FocusVolumeScale, 0.0f);
		NewFocusSettings.NonFocusVolumeScale = FMath::Max(NonFocusVolumeScale, 0.0f);
		NewFocusSettings.FocusPriorityScale = FMath::Max(FocusPriorityScale, 0.0f);
		NewFocusSettings.NonFocusPriorityScale = FMath::Max(NonFocusPriorityScale, 0.0f);

		AudioDevice->SetGlobalFocusSettings(NewFocusSettings);
	}
}

void UGameplayStatics::PlaySound2D(const UObject* WorldContextObject, USoundBase* Sound, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundConcurrency* ConcurrencySettings, const AActor* OwningActor, bool bIsUISound)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_PlaySound2D);

	if (!Sound || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		FActiveSound NewActiveSound;
		NewActiveSound.SetSound(Sound);
		NewActiveSound.SetWorld(ThisWorld);

		NewActiveSound.SetPitch(PitchMultiplier);
		NewActiveSound.SetVolume(VolumeMultiplier);

		NewActiveSound.RequestedStartTime = FMath::Max(0.f, StartTime);

		NewActiveSound.bIsUISound = bIsUISound;
		NewActiveSound.bAllowSpatialization = false;

		if (ConcurrencySettings)
		{
			NewActiveSound.ConcurrencySet.Add(ConcurrencySettings);
		}

		NewActiveSound.Priority = Sound->Priority;
		NewActiveSound.SubtitlePriority = Sound->GetSubtitlePriority();

		// If OwningActor isn't supplied to this function, derive an owner from the WorldContextObject
		const AActor* ActiveSoundOwner = OwningActor ? OwningActor : GameplayStatics::GetActorOwnerFromWorldContextObject(WorldContextObject);

		NewActiveSound.SetOwner(ActiveSoundOwner);

		TArray<FAudioParameter> Params;
		UActorSoundParameterInterface::Fill(ActiveSoundOwner, Params);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const FString* EventLogID = &Audio::Trace::EventLog::ID::PlayRequestSound2D;
#else
		const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		AudioDevice->AddNewActiveSound(NewActiveSound, &Params, EventLogID);
	}
}

UAudioComponent* UGameplayStatics::CreateSound2D(const UObject* WorldContextObject, USoundBase* Sound, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundConcurrency* ConcurrencySettings, bool bPersistAcrossLevelTransition, bool bAutoDestroy)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_CreateSound2D);

	if (!Sound || !GEngine || !GEngine->UseSound())
	{
		return nullptr;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return nullptr;
	}

	// Derive an owner from the WorldContextObject
	AActor* WorldContextOwner = GameplayStatics::GetActorOwnerFromWorldContextObject(const_cast<UObject*>(WorldContextObject));

	FAudioDevice::FCreateComponentParams Params = bPersistAcrossLevelTransition
		? FAudioDevice::FCreateComponentParams(ThisWorld->GetAudioDeviceRaw())
		: FAudioDevice::FCreateComponentParams(ThisWorld, WorldContextOwner);

	if (ConcurrencySettings)
	{
		Params.ConcurrencySet.Add(ConcurrencySettings);
	}

	UAudioComponent* AudioComponent = FAudioDevice::CreateComponent(Sound, Params);
	if (AudioComponent)
	{
		AudioComponent->SetVolumeMultiplier(VolumeMultiplier);
		AudioComponent->SetPitchMultiplier(PitchMultiplier);
		AudioComponent->bAllowSpatialization = false;
		AudioComponent->bIsUISound = true;
		AudioComponent->bAutoDestroy = bAutoDestroy;
		AudioComponent->bIgnoreForFlushing = bPersistAcrossLevelTransition;
		AudioComponent->SubtitlePriority = Sound->GetSubtitlePriority();
		AudioComponent->bStopWhenOwnerDestroyed = false;
	}
	return AudioComponent;
}

UAudioComponent* UGameplayStatics::SpawnSound2D(const UObject* WorldContextObject, USoundBase* Sound, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundConcurrency* ConcurrencySettings, bool bPersistAcrossLevelTransition, bool bAutoDestroy)
{
	UAudioComponent* AudioComponent = CreateSound2D(WorldContextObject, Sound, VolumeMultiplier, PitchMultiplier, StartTime, ConcurrencySettings, bPersistAcrossLevelTransition, bAutoDestroy);
	if (AudioComponent)
	{
		AudioComponent->Play(StartTime);
	}
	return AudioComponent;
}

void UGameplayStatics::PlaySoundAtLocation(const UObject* WorldContextObject, class USoundBase* Sound, FVector Location, FRotator Rotation, float VolumeMultiplier, float PitchMultiplier, float StartTime, class USoundAttenuation* AttenuationSettings, class USoundConcurrency* ConcurrencySettings, const AActor* OwningActor, const UInitialActiveSoundParams* InitialParams)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_PlaySoundAtLocation);

	if (!Sound || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		TArray<FAudioParameter> Params;
		if (InitialParams)
		{
			Params.Append(InitialParams->AudioParams);
		}

		// If OwningActor isn't supplied to this function, derive an owner from the WorldContextObject
		const AActor* ActiveSoundOwner = OwningActor ? OwningActor : GameplayStatics::GetActorOwnerFromWorldContextObject(WorldContextObject);
		UActorSoundParameterInterface::Fill(ActiveSoundOwner, Params);

		AudioDevice->PlaySoundAtLocation(Sound, ThisWorld, VolumeMultiplier, PitchMultiplier, StartTime, Location, Rotation, AttenuationSettings, ConcurrencySettings, MoveTemp(Params), ActiveSoundOwner);
	}
}

UAudioComponent* UGameplayStatics::SpawnSoundAtLocation(const UObject* WorldContextObject, USoundBase* Sound, FVector Location, FRotator Rotation, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundAttenuation* AttenuationSettings, USoundConcurrency* ConcurrencySettings, bool bAutoDestroy)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_SpawnSoundAtLocation);

	if (!Sound || !GEngine || !GEngine->UseSound())
	{
		return nullptr;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback || ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		return nullptr;
	}

	const bool bIsInGameWorld = ThisWorld->IsGameWorld();

	// Derive an owner from the WorldContextObject
	AActor* WorldContextOwner = GameplayStatics::GetActorOwnerFromWorldContextObject(const_cast<UObject*>(WorldContextObject));

	FAudioDevice::FCreateComponentParams Params(ThisWorld, WorldContextOwner);
	Params.SetLocation(Location);
	Params.AttenuationSettings = AttenuationSettings;
	
	if (ConcurrencySettings)
	{
		Params.ConcurrencySet.Add(ConcurrencySettings);
	}

	UAudioComponent* AudioComponent = FAudioDevice::CreateComponent(Sound, Params);

	if (AudioComponent)
	{
		AudioComponent->SetWorldLocationAndRotation(Location, Rotation);
		AudioComponent->SetVolumeMultiplier(VolumeMultiplier);
		AudioComponent->SetPitchMultiplier(PitchMultiplier);
		AudioComponent->bAllowSpatialization	= Params.ShouldUseAttenuation();
		AudioComponent->bIsUISound				= !bIsInGameWorld;
		AudioComponent->bAutoDestroy			= bAutoDestroy;
		AudioComponent->SubtitlePriority		= Sound->GetSubtitlePriority();
		AudioComponent->bStopWhenOwnerDestroyed = false;
		AudioComponent->Play(StartTime);
	}

	return AudioComponent;
}

UAudioComponent* UGameplayStatics::SpawnSoundAttached(USoundBase* Sound, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, bool bStopWhenAttachedToDestroyed, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundAttenuation* AttenuationSettings, USoundConcurrency* ConcurrencySettings, bool bAutoDestroy)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameplayStatics_SpawnSoundAttached);

	if (!Sound)
	{
		return nullptr;
	}

	if (!AttachToComponent)
	{
		UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnSoundAttached: NULL AttachComponent specified! Trying to spawn sound [%ls],", *Sound->GetName());
		return nullptr;
	}

	UWorld* const ThisWorld = AttachToComponent->GetWorld();
	if (ThisWorld && ThisWorld->IsNetMode(NM_DedicatedServer))
	{
		// FAudioDevice::CreateComponent will fail to create the AudioComponent in a real dedicated server, but we need to check netmode here for Editor support.
		return nullptr;
	}

	// Location used to check whether to create a component if out of range
	FVector TestLocation = Location;
	if (LocationType == EAttachLocation::KeepRelativeOffset)
	{
		if (AttachPointName != NAME_None)
		{
			TestLocation = AttachToComponent->GetSocketTransform(AttachPointName).TransformPosition(Location);
		}
		else
		{
			TestLocation = AttachToComponent->GetComponentTransform().TransformPosition(Location);
		}
	}
	else if (LocationType == EAttachLocation::SnapToTarget || LocationType == EAttachLocation::SnapToTargetIncludingScale)
	{
		// If AttachPointName is NAME_None, will return just the component position
		TestLocation = AttachToComponent->GetSocketLocation(AttachPointName);
	}

	FAudioDevice::FCreateComponentParams Params(ThisWorld, AttachToComponent->GetOwner());
	Params.SetLocation(TestLocation);
	Params.bStopWhenOwnerDestroyed = bStopWhenAttachedToDestroyed;
	Params.AttenuationSettings = AttenuationSettings;

	if (ConcurrencySettings)
	{
		Params.ConcurrencySet.Add(ConcurrencySettings);
	}

	UAudioComponent* AudioComponent = FAudioDevice::CreateComponent(Sound, Params);
	if (AudioComponent)
	{
		if (UWorld* ComponentWorld = AudioComponent->GetWorld())
		{
			const bool bIsInGameWorld = ComponentWorld->IsGameWorld();

			if (LocationType == EAttachLocation::SnapToTarget || LocationType == EAttachLocation::SnapToTargetIncludingScale)
			{
				AudioComponent->AttachToComponent(AttachToComponent, FAttachmentTransformRules::SnapToTargetNotIncludingScale, AttachPointName);
			}
			else
			{
				AudioComponent->AttachToComponent(AttachToComponent, FAttachmentTransformRules::KeepRelativeTransform, AttachPointName);
				if (LocationType == EAttachLocation::KeepWorldPosition)
				{
					AudioComponent->SetWorldLocationAndRotation(Location, Rotation);
				}
				else
				{
					AudioComponent->SetRelativeLocationAndRotation(Location, Rotation);
				}
			}

			AudioComponent->SetVolumeMultiplier(VolumeMultiplier);
			AudioComponent->SetPitchMultiplier(PitchMultiplier);
			AudioComponent->bAllowSpatialization = Params.ShouldUseAttenuation();
			AudioComponent->bIsUISound = !bIsInGameWorld;
			AudioComponent->bAutoDestroy = bAutoDestroy;
			AudioComponent->SubtitlePriority = Sound->GetSubtitlePriority();
			AudioComponent->Play(StartTime);
		}
	}

	return AudioComponent;
}

void UGameplayStatics::PlayDialogue2D(const UObject* WorldContextObject, UDialogueWave* Dialogue, const FDialogueContext& Context, float VolumeMultiplier, float PitchMultiplier, float StartTime)
{
	if (Dialogue)
	{
		PlaySound2D(WorldContextObject, Dialogue->GetWaveFromContext(Context), VolumeMultiplier, PitchMultiplier, StartTime);
	}
}

UAudioComponent* UGameplayStatics::SpawnDialogue2D(const UObject* WorldContextObject, UDialogueWave* Dialogue, const FDialogueContext& Context, float VolumeMultiplier, float PitchMultiplier, float StartTime, bool bAutoDestroy)
{
	if (Dialogue)
	{
		return SpawnSound2D(WorldContextObject, Dialogue->GetWaveFromContext(Context), VolumeMultiplier, PitchMultiplier, StartTime, nullptr, false, bAutoDestroy);
	}
	return nullptr;
}

void UGameplayStatics::PlayDialogueAtLocation(const UObject* WorldContextObject, UDialogueWave* Dialogue, const FDialogueContext& Context, FVector Location, FRotator Rotation, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundAttenuation* AttenuationSettings)
{
	if (Dialogue)
	{
		PlaySoundAtLocation(WorldContextObject, Dialogue->GetWaveFromContext(Context), Location, Rotation, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings);
	}
}

UAudioComponent* UGameplayStatics::SpawnDialogueAtLocation(const UObject* WorldContextObject, UDialogueWave* Dialogue, const FDialogueContext& Context, FVector Location, FRotator Rotation, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundAttenuation* AttenuationSettings, bool bAutoDestroy)
{
	if (Dialogue)
	{
		return SpawnSoundAtLocation(WorldContextObject, Dialogue->GetWaveFromContext(Context), Location, Rotation, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, nullptr, bAutoDestroy);
	}
	return nullptr;
}

UAudioComponent* UGameplayStatics::SpawnDialogueAttached(UDialogueWave* Dialogue, const FDialogueContext& Context, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, bool bStopWhenAttachedToDestroyed, float VolumeMultiplier, float PitchMultiplier, float StartTime, USoundAttenuation* AttenuationSettings, bool bAutoDestroy)
{
	if (Dialogue)
	{
		return SpawnSoundAttached(Dialogue->GetWaveFromContext(Context), AttachToComponent, AttachPointName, Location, Rotation, LocationType, bStopWhenAttachedToDestroyed, VolumeMultiplier, PitchMultiplier, StartTime, AttenuationSettings, nullptr, bAutoDestroy);
	}
	return nullptr;
}

void UGameplayStatics::SetSubtitlesEnabled(bool bEnabled)
{
	if (GEngine)
	{
		GEngine->bSubtitlesEnabled = bEnabled;
	}
}

bool UGameplayStatics::AreSubtitlesEnabled()
{
	if (GEngine)
	{
		return GEngine->bSubtitlesEnabled;
	}
	return 0;
}

void UGameplayStatics::SetBaseSoundMix(const UObject* WorldContextObject, USoundMix* InSoundMix)
{
	if (!InSoundMix || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->SetBaseSoundMix(InSoundMix);
	}
}

void UGameplayStatics::PushSoundMixModifier(const UObject* WorldContextObject, USoundMix* InSoundMixModifier)
{
	if (!InSoundMixModifier || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->PushSoundMixModifier(InSoundMixModifier);
	}
}

void UGameplayStatics::PrimeSound(USoundBase* InSound)
{
	if (!InSound || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	if (USoundCue* InSoundCue = Cast<USoundCue>(InSound))
	{
		InSoundCue->PrimeSoundCue();
	}
	else if (USoundWave* InSoundWave = Cast<USoundWave>(InSound))
	{
#if WITH_EDITORONLY_DATA
		InSoundWave->CachePlatformData(false); // (an ensure told me to do this)
#endif // WITH_EDITORONLY_DATA

		if (InSoundWave->HasStreamingChunks() && InSoundWave->GetNumChunks() > 1)
		{
			IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(InSoundWave->GetSoundWaveProxy()->GetSoundWaveDataRef(), 1, [](EAudioChunkLoadResult) {});
		}
	}
}

TArray<FName> UGameplayStatics::GetAvailableSpatialPluginNames(const UObject* WorldContextObject)
{
	if(const UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
		{
			return AudioDevice->GetAvailableSpatializationPluginNames();
		}
	}

	return {};
}

FName UGameplayStatics::GetActiveSpatialPluginName(const UObject* WorldContextObject)
{
	if(const UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
		{
			return AudioDevice->GetCurrentSpatializationPluginInterfaceInfo().PluginName;
		}
	}

	return {};
}

bool UGameplayStatics::SetActiveSpatialPluginByName(const UObject* WorldContextObject, FName InPluginName)
{
	if(const UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
		{
			return AudioDevice->SetCurrentSpatializationPlugin(InPluginName);
		}
	}

	return {};
}

void UGameplayStatics::PrimeAllSoundsInSoundClass(class USoundClass* InSoundClass)
{
	for (TObjectIterator<USoundWave> Itr; Itr; ++Itr)
	{
		const USoundClass* SoundClass = Itr->GetSoundClass();	
		if (SoundClass && (SoundClass->GetName() == InSoundClass->GetName()))
		{
			PrimeSound(*Itr);
		}
	}
}

void UGameplayStatics::UnRetainAllSoundsInSoundClass(USoundClass* InSoundClass)
{
	for (TObjectIterator<USoundWave> Itr; Itr; ++Itr)
	{
		const USoundClass* SoundClass = Itr->GetSoundClass();
		if (SoundClass && (SoundClass->GetName() == InSoundClass->GetName()))
		{
			Itr->ReleaseCompressedAudio();
		}
	}
}

void UGameplayStatics::SetSoundMixClassOverride(const UObject* WorldContextObject, class USoundMix* InSoundMixModifier, class USoundClass* InSoundClass, float Volume, float Pitch, float FadeInTime, bool bApplyToChildren)
{
	if (!InSoundMixModifier || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->SetSoundMixClassOverride(InSoundMixModifier, InSoundClass, Volume, Pitch, FadeInTime, bApplyToChildren);
	}
}

void UGameplayStatics::ClearSoundMixClassOverride(const UObject* WorldContextObject, class USoundMix* InSoundMixModifier, class USoundClass* InSoundClass, float FadeOutTime)
{
	if (!InSoundMixModifier || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->ClearSoundMixClassOverride(InSoundMixModifier, InSoundClass, FadeOutTime);
	}
}

void UGameplayStatics::PopSoundMixModifier(const UObject* WorldContextObject, USoundMix* InSoundMixModifier)
{
	if (InSoundMixModifier == nullptr || GEngine == nullptr || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (ThisWorld == nullptr || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->PopSoundMixModifier(InSoundMixModifier);
	}
}

void UGameplayStatics::ClearSoundMixModifiers(const UObject* WorldContextObject)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->ClearSoundMixModifiers();
	}
}

void UGameplayStatics::ActivateReverbEffect(const UObject* WorldContextObject, class UReverbEffect* ReverbEffect, FName TagName, float Priority, float Volume, float FadeTime)
{
	if (ReverbEffect == nullptr || !GEngine || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->ActivateReverbEffect(ReverbEffect, TagName, Priority, Volume, FadeTime);
	}
}

void UGameplayStatics::DeactivateReverbEffect(const UObject* WorldContextObject, FName TagName)
{
	if (GEngine == nullptr || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->DeactivateReverbEffect(TagName);
	}
}

class UReverbEffect* UGameplayStatics::GetCurrentReverbEffect(const UObject* WorldContextObject)
{
	if (GEngine == nullptr || !GEngine->UseSound())
	{
		return nullptr;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return nullptr;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		return AudioDevice->GetCurrentReverbEffect();
	}
	return nullptr;
}

void UGameplayStatics::SetMaxAudioChannelsScaled(const UObject* WorldContextObject, float MaxChannelCountScale)
{
	if (GEngine == nullptr || !GEngine->UseSound())
	{
		return;
	}

	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		AudioDevice->SetMaxChannelsScaled(MaxChannelCountScale);
	}
}

int32 UGameplayStatics::GetMaxAudioChannelCount(const UObject* WorldContextObject)
{
	if (GEngine == nullptr || !GEngine->UseSound())
	{
		return 0;
	}


	UWorld* ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!ThisWorld || !ThisWorld->bAllowAudioPlayback)
	{
		return 0;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		return AudioDevice->GetMaxChannels();
	}

	return 0;
}


UDecalComponent* CreateDecalComponent(class UMaterialInterface* DecalMaterial, FVector DecalSize, UWorld* World, AActor* Actor, float LifeSpan)
{
	if (World == nullptr || World->GetNetMode() == NM_DedicatedServer)
	{
		return nullptr;
	}

	UDecalComponent* DecalComp = NewObject<UDecalComponent>((Actor ? Actor : (UObject*)World));
	DecalComp->bAllowAnyoneToDestroyMe = true;
	DecalComp->SetDecalMaterial(DecalMaterial);
	DecalComp->DecalSize = DecalSize;
	DecalComp->SetUsingAbsoluteScale(true);
	DecalComp->RegisterComponentWithWorld(World);

	if (LifeSpan > 0.f)
	{
		DecalComp->SetLifeSpan(LifeSpan);
	}

	return DecalComp;
}

UDecalComponent* UGameplayStatics::SpawnDecalAtLocation(const UObject* WorldContextObject, class UMaterialInterface* DecalMaterial, FVector DecalSize, FVector Location, FRotator Rotation, float LifeSpan)
{
	if (DecalMaterial)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			UDecalComponent* DecalComp = CreateDecalComponent(DecalMaterial, DecalSize, World, World->GetWorldSettings(), LifeSpan);
			if (DecalComp)
			{
				DecalComp->SetWorldLocationAndRotation(Location, Rotation);
			}
			return DecalComp;
		}
	}
	return nullptr;
}

UDecalComponent* UGameplayStatics::SpawnDecalAttached(class UMaterialInterface* DecalMaterial, FVector DecalSize, class USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, float LifeSpan)
{
	if (DecalMaterial)
	{
		if (!AttachToComponent)
		{
			UE_LOGF(LogScript, Warning, "UGameplayStatics::SpawnDecalAttached: NULL AttachComponent specified!");
		}
		else
		{
			UPrimitiveComponent* AttachToPrimitive = Cast<UPrimitiveComponent>(AttachToComponent);
			if (!AttachToPrimitive || AttachToPrimitive->bReceivesDecals)
			{
				if (AttachToPrimitive && Cast<AWorldSettings>(AttachToPrimitive->GetOwner()))
				{
					// special case: don't attach to component when it's owned by invisible WorldSettings (decals on BSP brush)
					return SpawnDecalAtLocation(AttachToPrimitive->GetOwner(), DecalMaterial, DecalSize, Location, Rotation, LifeSpan);
				}
				else
				{
					UDecalComponent* DecalComp = CreateDecalComponent(DecalMaterial, DecalSize, AttachToComponent->GetWorld(), AttachToComponent->GetOwner(), LifeSpan);
					if (DecalComp)
					{
						DecalComp->AttachToComponent(AttachToComponent, FAttachmentTransformRules::KeepRelativeTransform, AttachPointName);
						if (LocationType == EAttachLocation::KeepWorldPosition)
						{
							DecalComp->SetWorldLocationAndRotation(Location, Rotation);
						}
						else
						{
							DecalComp->SetRelativeLocationAndRotation(Location, Rotation);
						}
					}
					
					return DecalComp;
				}
			}
		}
	}
	return nullptr;
}

UForceFeedbackComponent* CreateForceFeedbackComponent(UForceFeedbackEffect* FeedbackEffect, AActor* Actor, const bool bLooping, const float IntensityMultiplier, UForceFeedbackAttenuation* AttenuationSettings, const bool bAutoDestroy)
{
	UForceFeedbackComponent* ForceFeedbackComp = NewObject<UForceFeedbackComponent>(Actor);
	ForceFeedbackComp->bAutoActivate = false;
	ForceFeedbackComp->bAutoDestroy = bAutoDestroy;
	ForceFeedbackComp->bLooping = bLooping;
	ForceFeedbackComp->ForceFeedbackEffect = FeedbackEffect;
	ForceFeedbackComp->IntensityMultiplier = IntensityMultiplier;
	ForceFeedbackComp->AttenuationSettings = AttenuationSettings;
	ForceFeedbackComp->RegisterComponent();

	return ForceFeedbackComp;
}

UForceFeedbackComponent* UGameplayStatics::SpawnForceFeedbackAtLocation(const UObject* WorldContextObject, UForceFeedbackEffect* ForceFeedbackEffect, const FVector Location, const FRotator Rotation, const bool bLooping, const float IntensityMultiplier, const float StartTime, UForceFeedbackAttenuation* AttenuationSettings, const bool bAutoDestroy)
{
	if (ForceFeedbackEffect)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			UForceFeedbackComponent* ForceFeedbackComp = CreateForceFeedbackComponent(ForceFeedbackEffect, World->GetWorldSettings(), bLooping, IntensityMultiplier, AttenuationSettings, bAutoDestroy);
			ForceFeedbackComp->SetWorldLocationAndRotation(Location, Rotation);
			ForceFeedbackComp->Play(StartTime);
			return ForceFeedbackComp;
		}
	}
	return nullptr;
}

UForceFeedbackComponent* UGameplayStatics::SpawnForceFeedbackAttached(UForceFeedbackEffect* ForceFeedbackEffect, USceneComponent* AttachToComponent, FName AttachPointName, FVector Location, FRotator Rotation, EAttachLocation::Type LocationType, const bool bStopWhenAttachedToDestroyed, const bool bLooping, const float IntensityMultiplier, const float StartTime, UForceFeedbackAttenuation* AttenuationSettings, const bool bAutoDestroy)
{
	if (ForceFeedbackEffect && AttachToComponent)
	{
		UForceFeedbackComponent* ForceFeedbackComp = CreateForceFeedbackComponent(ForceFeedbackEffect, AttachToComponent->GetOwner(), bLooping, IntensityMultiplier, AttenuationSettings, bAutoDestroy);
		ForceFeedbackComp->bStopWhenOwnerDestroyed = bStopWhenAttachedToDestroyed;
		ForceFeedbackComp->AttachToComponent(AttachToComponent, FAttachmentTransformRules::KeepRelativeTransform, AttachPointName);
		if (LocationType == EAttachLocation::KeepWorldPosition)
		{
			ForceFeedbackComp->SetWorldLocationAndRotation(Location, Rotation);
		}
		else
		{
			ForceFeedbackComp->SetRelativeLocationAndRotation(Location, Rotation);
		}
		ForceFeedbackComp->Play(StartTime);
		return ForceFeedbackComp;
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////

USaveGame* UGameplayStatics::CreateSaveGameObject(TSubclassOf<USaveGame> SaveGameClass)
{
	// Don't save if no class or if class is the abstract base class.
	if (*SaveGameClass && (*SaveGameClass != USaveGame::StaticClass()))
	{
		return NewObject<USaveGame>(GetTransientPackage(), SaveGameClass);
	}
	return nullptr;
}

bool UGameplayStatics::SaveGameToMemory(USaveGame* SaveGameObject, TArray<uint8>& OutSaveData )
{
	if (SaveGameObject)
	{
		FMemoryWriter MemoryWriter(OutSaveData, true);

		FSaveGameHeader SaveHeader(SaveGameObject->GetClass());
		SaveHeader.Write(MemoryWriter);

		// Then save the object state, replacing object refs and names with strings
		FObjectAndNameAsStringProxyArchive Ar(MemoryWriter, false);
		SaveGameObject->Serialize(Ar);

		return true; // Not sure if there's a failure case here.
	}

	return false;
}

bool UGameplayStatics::SaveDataToSlot(const TArray<uint8>& InSaveData, const FString& SlotName, const int32 UserIndex)
{
	ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();

	if (SaveSystem && InSaveData.Num() > 0 && SlotName.Len() > 0)
	{
		// Stuff that data into the save system with the desired file name
		return SaveSystem->SaveGame(false, *SlotName, UserIndex, InSaveData);
	}

	return false;
}

void UGameplayStatics::AsyncSaveGameToSlot(USaveGame* SaveGameObject, const FString& SlotName, const int32 UserIndex, FAsyncSaveGameToSlotDelegate SavedDelegate)
{
	ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();

	TSharedRef<TArray<uint8>> ObjectBytes(new TArray<uint8>());

	if (SaveSystem && (SlotName.Len() > 0) && 
		SaveGameToMemory(SaveGameObject, *ObjectBytes) && (ObjectBytes->Num() > 0) )
	{
		FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);

		SaveSystem->SaveGameAsync(false, *SlotName, PlatformUserId, ObjectBytes, 
			[SavedDelegate, UserIndex](const FString& SlotName, FPlatformUserId PlatformUserId, bool bSuccess)
			{
				check(IsInGameThread());
				SavedDelegate.ExecuteIfBound(SlotName, UserIndex, bSuccess);
			}
		);
	}
	else
	{
		SavedDelegate.ExecuteIfBound(SlotName, UserIndex, false);
	}
}

bool UGameplayStatics::SaveGameToSlot(USaveGame* SaveGameObject, const FString& SlotName, const int32 UserIndex)
{
	// This is a wrapper around the functions reading to/from a byte array
	TArray<uint8> ObjectBytes;
	if (SaveGameToMemory(SaveGameObject, ObjectBytes))
	{
		return SaveDataToSlot(ObjectBytes, SlotName, UserIndex);
	}
	return false;
}

bool UGameplayStatics::DoesSaveGameExist(const FString& SlotName, const int32 UserIndex)
{
	if (ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem())
	{
		return SaveSystem->DoesSaveGameExist(*SlotName, UserIndex);
	}
	return false;
}

bool UGameplayStatics::DeleteGameInSlot(const FString& SlotName, const int32 UserIndex)
{
	if (ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem())
	{
		return SaveSystem->DeleteGame(false, *SlotName, UserIndex);
	}
	return false;
}

USaveGame* UGameplayStatics::LoadGameFromMemory(const TArray<uint8>& InSaveData)
{
	if (InSaveData.Num() == 0)
	{
		// Empty buffer, return instead of causing a bad serialize that could crash
		return nullptr;
	}

	USaveGame* OutSaveGameObject = nullptr;

	FMemoryReader MemoryReader(InSaveData, true);

	FSaveGameHeader SaveHeader;
	SaveHeader.Read(MemoryReader);

	// Try and find it, and failing that, load it
	UClass* SaveGameClass = UClass::TryFindTypeSlow<UClass>(SaveHeader.SaveGameClassName);
	if (SaveGameClass == nullptr)
	{
		SaveGameClass = LoadObject<UClass>(nullptr, *SaveHeader.SaveGameClassName);
	}

	// If we have a class, try and load it.
	if (SaveGameClass != nullptr)
	{
		OutSaveGameObject = NewObject<USaveGame>(GetTransientPackage(), SaveGameClass);

		FObjectAndNameAsStringProxyArchive Ar(MemoryReader, true);
		OutSaveGameObject->Serialize(Ar);
	}

	return OutSaveGameObject;
}

bool UGameplayStatics::LoadDataFromSlot(TArray<uint8>& OutSaveData, const FString& SlotName, const int32 UserIndex)
{
	ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	// If we have a save system and a valid name..
	if (SaveSystem && (SlotName.Len() > 0))
	{
		if (SaveSystem->LoadGame(false, *SlotName, UserIndex, OutSaveData))
		{
			return true;
		}
	}

	// Clear buffer on a failed read
	OutSaveData.Reset();
	return false;
}

void UGameplayStatics::AsyncLoadGameFromSlot(const FString& SlotName, const int32 UserIndex, FAsyncLoadGameFromSlotDelegate LoadedDelegate)
{
	ISaveGameSystem* SaveSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (SaveSystem && (SlotName.Len() > 0))
	{
		FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);

		SaveSystem->LoadGameIfExistsAsync(false, *SlotName, PlatformUserId,
			[LoadedDelegate, UserIndex](const FString& SlotName, FPlatformUserId PlatformUserId, bool bSuccess, const TArray<uint8>& Data)
			{
				check(IsInGameThread());

				USaveGame* LoadedGame = nullptr;
				if (bSuccess)
				{
					LoadedGame = LoadGameFromMemory(Data);
				}

				LoadedDelegate.ExecuteIfBound(SlotName, UserIndex, LoadedGame);
			}
		);
	}
	else
	{
		LoadedDelegate.ExecuteIfBound(SlotName, UserIndex, nullptr);
	}
}

USaveGame* UGameplayStatics::LoadGameFromSlot(const FString& SlotName, const int32 UserIndex)
{
	// This is a wrapper around the functions reading to/from a byte array
	TArray<uint8> ObjectBytes;
	if (LoadDataFromSlot(ObjectBytes, SlotName, UserIndex))
	{
		return LoadGameFromMemory(ObjectBytes);
	}

	return nullptr;
}

FMemoryReader UGameplayStatics::StripSaveGameHeader(const TArray<uint8>& SaveData)
{
	FMemoryReader MemoryReader(SaveData, /*bIsPersistent =*/true);

	FSaveGameHeader SaveHeader;
	SaveHeader.Read(MemoryReader);

	return MemoryReader;
}

double UGameplayStatics::GetWorldDeltaSeconds(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetDeltaSeconds() : 0.0;
}

double UGameplayStatics::GetTimeSeconds(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetTimeSeconds() : 0.0;
}

double UGameplayStatics::GetUnpausedTimeSeconds(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetUnpausedTimeSeconds() : 0.0;
}

double UGameplayStatics::GetRealTimeSeconds(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetRealTimeSeconds() : 0.0;
}

double UGameplayStatics::GetAudioTimeSeconds(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetAudioTimeSeconds() : 0.0;
}

void UGameplayStatics::GetAccurateRealTime(int32& Seconds, double& PartialSeconds)
{
	double TimeSeconds = FPlatformTime::Seconds() - GStartTime;
	Seconds = floor(TimeSeconds);
	PartialSeconds = TimeSeconds - double(Seconds);
}

void UGameplayStatics::EnableLiveStreaming(bool Enable)
{
	if (IDVRStreamingSystem* StreamingSystem = IPlatformFeaturesModule::Get().GetStreamingSystem())
	{
		StreamingSystem->EnableStreaming(Enable);
	}
}

FString UGameplayStatics::GetPlatformName()
{
	// the string that BP users care about is actually the platform name that we'd name the .ini file directory (Windows, not WindowsEditor)
	return FPlatformProperties::IniPlatformName();
}

bool UGameplayStatics::BlueprintSuggestProjectileVelocity(const UObject* WorldContextObject, FVector& OutTossVelocity, FVector StartLocation, FVector EndLocation, float LaunchSpeed, float OverrideGravityZ, ESuggestProjVelocityTraceOption::Type TraceOption, float CollisionRadius, bool bFavorHighArc, bool bDrawDebug, bool bAcceptClosestOnNoSolutions)
{
	// simple pass-through to the C++ interface
	FSuggestProjectileVelocityParameters ProjectileParams = FSuggestProjectileVelocityParameters(WorldContextObject, StartLocation, EndLocation, LaunchSpeed);
	ProjectileParams.bFavorHighArc = bFavorHighArc;
	ProjectileParams.CollisionRadius = CollisionRadius;
	ProjectileParams.OverrideGravityZ = OverrideGravityZ;
	ProjectileParams.TraceOption = TraceOption;
	ProjectileParams.bDrawDebug = bDrawDebug;
	ProjectileParams.bAcceptClosestOnNoSolutions = bAcceptClosestOnNoSolutions;

	return SuggestProjectileVelocity(ProjectileParams, OutTossVelocity);
}

bool UGameplayStatics::SuggestProjectileVelocity(const UObject* WorldContextObject, FVector& TossVelocity, FVector StartLocation, FVector EndLocation, float TossSpeed, bool bHighArc , float CollisionRadius, float OverrideGravityZ, ESuggestProjVelocityTraceOption::Type TraceOption, FCollisionResponseParams& ResponseParam, TArray<AActor*> ActorsToIgnore, bool bDrawDebug, bool bAcceptClosestOnNoSolutions)
{
	FSuggestProjectileVelocityParameters ProjectileParams = FSuggestProjectileVelocityParameters(WorldContextObject,StartLocation, EndLocation, TossSpeed);
	ProjectileParams.bFavorHighArc = bHighArc;
	ProjectileParams.CollisionRadius = CollisionRadius;
	ProjectileParams.OverrideGravityZ = OverrideGravityZ;
	ProjectileParams.TraceOption = TraceOption;
	ProjectileParams.ResponseParam = ResponseParam;
	ProjectileParams.ActorsToIgnore = ActorsToIgnore;
	ProjectileParams.bDrawDebug = bDrawDebug;
	ProjectileParams.bAcceptClosestOnNoSolutions = bAcceptClosestOnNoSolutions;

	return SuggestProjectileVelocity(ProjectileParams, TossVelocity);
}

// note: this will automatically fall back to line test if radius is small enough
// Based on analytic solution to ballistic angle of launch http://en.wikipedia.org/wiki/Trajectory_of_a_projectile#Angle_required_to_hit_coordinate_.28x.2Cy.29
bool UGameplayStatics::SuggestProjectileVelocity(const FSuggestProjectileVelocityParameters& ProjectileParams, FVector& OutTossVelocity)
{
	const FVector FlightDelta = ProjectileParams.End - ProjectileParams.Start;
	const FVector DirXY = FlightDelta.GetSafeNormal2D();
	const float DeltaXY = FlightDelta.Size2D();

	const float DeltaZ = FlightDelta.Z;

	const float TossSpeedSq = FMath::Square(ProjectileParams.TossSpeed);

	const UWorld* const World = GEngine->GetWorldFromContextObject(ProjectileParams.WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World == nullptr)
	{
		return false;
	}
	const float GravityZ = FMath::IsNearlyEqual(ProjectileParams.OverrideGravityZ, 0.0f) ? -World->GetGravityZ() : -ProjectileParams.OverrideGravityZ;

	FVector PrioritizedProjVelocities[2];
	int32 NumSolutions;

	// v^4 - g*(g*x^2 + 2*y*v^2)
	const float InsideTheSqrt = FMath::Square(TossSpeedSq) - GravityZ * ((GravityZ * FMath::Square(DeltaXY)) + (2.f * DeltaZ * TossSpeedSq));

	if (InsideTheSqrt < 0.f)
	{
		if (ProjectileParams.bAcceptClosestOnNoSolutions)
		{
			NumSolutions = 1;
			// To get closest, we want to maximise the displacement in the direction of the target
			// R = v^2/g * 1/cos^2(theta) * (sin(2phi + theta) - sin(theta))
			// R = range, theta= angle to target (incline), phi = launch angle
			// 2phi + theta maximises at PI/2 rads
			// phi = PI/4 - theta/2    for max R
			// alpha = phi + theta
			// Create a vector at angle alpha up from DirXY and * by TossSpeed
			const FVector DirFlightDelta = FlightDelta.GetSafeNormal();
			const float AngleToTarget = FMath::Acos(FVector::DotProduct(DirFlightDelta, DirXY));
			const float AngleAlpha = UE_PI / 4 - (AngleToTarget / 2) + AngleToTarget;
			const FVector Cross = DirXY.Cross(FVector(0, 0, 1));
			const FVector TrajectoryHeading = DirXY.RotateAngleAxisRad(AngleAlpha, Cross);
			PrioritizedProjVelocities[0] = TrajectoryHeading * ProjectileParams.TossSpeed;
		}
		else
		{
			return false;
		}
	}
	else
	{
		// if we got here, there are 2 solutions: one high-angle and one low-angle.
		NumSolutions = 2;

		const float SqrtPart = FMath::Sqrt(InsideTheSqrt);

		// this is the tangent of the firing angle for the first (+) solution
		const float TanSolutionAngleA = (TossSpeedSq + SqrtPart) / (GravityZ * DeltaXY);
		// this is the tangent of the firing angle for the second (-) solution
		const float TanSolutionAngleB = (TossSpeedSq - SqrtPart) / (GravityZ * DeltaXY);

		// mag in the XY dir = sqrt( TossSpeedSq / (TanSolutionAngle^2 + 1) );
		const float MagXYSq_A = TossSpeedSq / (FMath::Square(TanSolutionAngleA) + 1.f);
		const float MagXYSq_B = TossSpeedSq / (FMath::Square(TanSolutionAngleB) + 1.f);

		float PrioritizedSolutionsMagXYSq[2];
		PrioritizedSolutionsMagXYSq[0] = ProjectileParams.bFavorHighArc ? FMath::Min(MagXYSq_A, MagXYSq_B) : FMath::Max(MagXYSq_A, MagXYSq_B);
		PrioritizedSolutionsMagXYSq[1] = ProjectileParams.bFavorHighArc ? FMath::Max(MagXYSq_A, MagXYSq_B) : FMath::Min(MagXYSq_A, MagXYSq_B);

		float PrioritizedSolutionZSign[2];
		PrioritizedSolutionZSign[0] = ProjectileParams.bFavorHighArc ?
			(MagXYSq_A < MagXYSq_B) ? FMath::Sign(TanSolutionAngleA) : FMath::Sign(TanSolutionAngleB) :
			(MagXYSq_A > MagXYSq_B) ? FMath::Sign(TanSolutionAngleA) : FMath::Sign(TanSolutionAngleB);
		PrioritizedSolutionZSign[1] = ProjectileParams.bFavorHighArc ?
			(MagXYSq_A > MagXYSq_B) ? FMath::Sign(TanSolutionAngleA) : FMath::Sign(TanSolutionAngleB) :
			(MagXYSq_A < MagXYSq_B) ? FMath::Sign(TanSolutionAngleA) : FMath::Sign(TanSolutionAngleB);

		// solutions in priority order
		for (int32 CurrentSolutionIdx = 0; (CurrentSolutionIdx < 2); ++CurrentSolutionIdx)
		{
			const float MagXY = FMath::Sqrt(PrioritizedSolutionsMagXYSq[CurrentSolutionIdx]);
			const float MagZ = FMath::Sqrt(TossSpeedSq - PrioritizedSolutionsMagXYSq[CurrentSolutionIdx]);		// pythagorean
			const float ZSign = PrioritizedSolutionZSign[CurrentSolutionIdx];

			PrioritizedProjVelocities[CurrentSolutionIdx] = (DirXY * MagXY) + (FVector::UpVector * MagZ * ZSign);
		}
	}

	// trace if desired
	if (ProjectileParams.TraceOption == ESuggestProjVelocityTraceOption::DoNotTrace)
	{
		OutTossVelocity = PrioritizedProjVelocities[0];
		const float MagXY = OutTossVelocity.Size2D();
#if ENABLE_DRAW_DEBUG
		if (ProjectileParams.bDrawDebug)
		{
			static const float StepSize = 0.125f;
			FVector TraceStart = ProjectileParams.Start;
			for (float Step = 0.f; Step < 1.f; Step += StepSize)
			{
				const float TimeInFlight = (Step + StepSize) * DeltaXY / MagXY;

				// d = vt + .5 a t^2
				const FVector TraceEnd = ProjectileParams.Start + OutTossVelocity * TimeInFlight + FVector(0.f, 0.f, 0.5f * -GravityZ * FMath::Square(TimeInFlight) - ProjectileParams.CollisionRadius);

				DrawDebugLine(World, TraceStart, TraceEnd, FColor::Yellow, true);
				TraceStart = TraceEnd;
			}
		}
#endif // ENABLE_DRAW_DEBUG
		return true;
	}
	else
	{
		// need to trace to validate
		int32 ValidSolutionIdx = INDEX_NONE;

		for (int32 SolutionIdx = 0; SolutionIdx < NumSolutions; ++SolutionIdx)
		{
			// trace each solution
			if (!IsProjectileTrajectoryBlocked(World, ProjectileParams.Start, PrioritizedProjVelocities[SolutionIdx], DeltaXY, GravityZ, ProjectileParams.CollisionRadius, ProjectileParams.TraceOption, ProjectileParams.ResponseParam, ProjectileParams.ActorsToIgnore, ProjectileParams.bDrawDebug))
			{
				OutTossVelocity = PrioritizedProjVelocities[SolutionIdx];
				return true;
			}
		}

		return false;
	}
}

bool UGameplayStatics::IsProjectileTrajectoryBlocked(const UWorld* World, FVector StartLocation, FVector& ProjectileVelocity, float TargetDeltaXY, float GravityZ, float CollisionRadius, ESuggestProjVelocityTraceOption::Type TraceOption, const FCollisionResponseParams& ResponseParam, const TArray<AActor*>& ActorsToIgnore, bool bDrawDebug)
{
	
	// iterate along the arc, doing stepwise traces
	static const float StepSize = 0.125f;
	const float MagXY = ProjectileVelocity.Size2D();
	FVector TraceStart = StartLocation;

	for (float Step = 0.f; Step < 1.f; Step += StepSize)
	{
		const float TimeInFlight = (Step + StepSize) * TargetDeltaXY / MagXY;

		// d = vt + .5 a t^2
		const FVector TraceEnd = StartLocation + ProjectileVelocity * TimeInFlight + FVector(0.f, 0.f, 0.5f * -GravityZ * FMath::Square(TimeInFlight) - CollisionRadius);

		if ((TraceOption == ESuggestProjVelocityTraceOption::OnlyTraceWhileAscending) && (TraceEnd.Z < TraceStart.Z))
		{
			// falling, we are done tracing
			if (!bDrawDebug)
			{
				// if we're drawing, we continue stepping without the traces
				// else we can just trivially end the iteration loop
				break;
			}
		}
		else
		{
			FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(SuggestProjVelTrace), true);
			QueryParams.AddIgnoredActors(ActorsToIgnore);
			if (World->SweepTestByChannel(TraceStart, TraceEnd, FQuat::Identity, ECC_WorldDynamic, FCollisionShape::MakeSphere(CollisionRadius), QueryParams, ResponseParam))
			{


#if ENABLE_DRAW_DEBUG
				if (bDrawDebug)
				{
					// draw failed segment in red
					DrawDebugLine(World, TraceStart, TraceEnd, FColor::Red, true);
				}
#endif // ENABLE_DRAW_DEBUG
				// hit something, failed
				return true;
			}
		}

#if ENABLE_DRAW_DEBUG
		if (bDrawDebug)
		{
			DrawDebugLine(World, TraceStart, TraceEnd, FColor::Yellow, true);
		}
#endif // ENABLE_DRAW_DEBUG

		// advance
		TraceStart = TraceEnd;
	}

	return false;
}

// note: this will automatically fall back to line test if radius is small enough
bool UGameplayStatics::PredictProjectilePath(const UObject* WorldContextObject, const FPredictProjectilePathParams& PredictParams, FPredictProjectilePathResult& PredictResult)
{
	PredictResult.Reset();
	bool bBlockingHit = false;

	UWorld const* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World && PredictParams.SimFrequency > UE_KINDA_SMALL_NUMBER)
	{
		const float SubstepDeltaTime = 1.f / PredictParams.SimFrequency;
		const float GravityZ = FMath::IsNearlyEqual(PredictParams.OverrideGravityZ, 0.0f) ? World->GetGravityZ() : PredictParams.OverrideGravityZ;
		const float ProjectileRadius = PredictParams.ProjectileRadius;

		FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(PredictProjectilePath), PredictParams.bTraceComplex);
		FCollisionObjectQueryParams ObjQueryParams;
		const bool bTraceWithObjectType = (PredictParams.ObjectTypes.Num() > 0);
		const bool bTracePath = PredictParams.bTraceWithCollision && (PredictParams.bTraceWithChannel || bTraceWithObjectType);
		if (bTracePath)
		{
			QueryParams.AddIgnoredActors(PredictParams.ActorsToIgnore);
			if (bTraceWithObjectType)
			{
				for (auto Iter = PredictParams.ObjectTypes.CreateConstIterator(); Iter; ++Iter)
				{
					const ECollisionChannel& Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
					ObjQueryParams.AddObjectTypesToQuery(Channel);
				}
			}
		}

		FVector CurrentVel = PredictParams.LaunchVelocity;
		FVector TraceStart = PredictParams.StartLocation;
		FVector TraceEnd = TraceStart;
		float CurrentTime = 0.f;
		PredictResult.PathData.Reserve(FMath::Min(128, FMath::CeilToInt(PredictParams.MaxSimTime * PredictParams.SimFrequency)));
		PredictResult.AddPoint(TraceStart, CurrentVel, CurrentTime);

		FHitResult ObjectTraceHit(NoInit);
		FHitResult ChannelTraceHit(NoInit);
		ObjectTraceHit.Time = 1.f;
		ChannelTraceHit.Time = 1.f;

		const float MaxSimTime = PredictParams.MaxSimTime;
		while (CurrentTime < MaxSimTime)
		{
			// Limit step to not go further than total time.
			const float PreviousTime = CurrentTime;
			const float ActualStepDeltaTime = FMath::Min(MaxSimTime - CurrentTime, SubstepDeltaTime);
			CurrentTime += ActualStepDeltaTime;

			// Integrate (Velocity Verlet method)
			TraceStart = TraceEnd;
			FVector OldVelocity = CurrentVel;
			CurrentVel = OldVelocity + FVector(0.f, 0.f, GravityZ * ActualStepDeltaTime);
			TraceEnd = TraceStart + (OldVelocity + CurrentVel) * (0.5f * ActualStepDeltaTime);
			PredictResult.LastTraceDestination.Set(TraceEnd, CurrentVel, CurrentTime);

			if (bTracePath)
			{
				bool bObjectHit = false;
				bool bChannelHit = false;
				if (bTraceWithObjectType)
				{
					bObjectHit = World->SweepSingleByObjectType(ObjectTraceHit, TraceStart, TraceEnd, FQuat::Identity, ObjQueryParams, FCollisionShape::MakeSphere(ProjectileRadius), QueryParams);
				}
				if (PredictParams.bTraceWithChannel)
				{
					bChannelHit = World->SweepSingleByChannel(ChannelTraceHit, TraceStart, TraceEnd, FQuat::Identity, PredictParams.TraceChannel, FCollisionShape::MakeSphere(ProjectileRadius), QueryParams);
				}

				// See if there were any hits.
				if (bObjectHit || bChannelHit)
				{
					// Hit! We are done. Choose trace with earliest hit time.
					PredictResult.HitResult = (ObjectTraceHit.Time < ChannelTraceHit.Time) ? ObjectTraceHit : ChannelTraceHit;
					const float HitTimeDelta = ActualStepDeltaTime * PredictResult.HitResult.Time;
					const float TotalTimeAtHit = PreviousTime + HitTimeDelta;
					const FVector VelocityAtHit = OldVelocity + FVector(0.f, 0.f, GravityZ * HitTimeDelta);
					PredictResult.AddPoint(PredictResult.HitResult.Location, VelocityAtHit, TotalTimeAtHit);
					bBlockingHit = true;
					break;
				}
			}

			PredictResult.AddPoint(TraceEnd, CurrentVel, CurrentTime);
		}

		// Draw debug path
#if ENABLE_DRAW_DEBUG
		if (PredictParams.DrawDebugType != EDrawDebugTrace::None)
		{
			const bool bPersistent = PredictParams.DrawDebugType == EDrawDebugTrace::Persistent;
			const float LifeTime = (PredictParams.DrawDebugType == EDrawDebugTrace::ForDuration) ? PredictParams.DrawDebugTime : 0.f;
			const float DrawRadius = (ProjectileRadius > 0.f) ? ProjectileRadius : 5.f;

			// draw the path
			for (const FPredictProjectilePathPointData& PathPt : PredictResult.PathData)
			{
				::DrawDebugSphere(World, PathPt.Location, DrawRadius, 12, FColor::Green, bPersistent, LifeTime);
			}
			// draw the impact point
			if (bBlockingHit)
			{
				::DrawDebugSphere(World, PredictResult.HitResult.Location, DrawRadius + 1.0f, 12, FColor::Red, bPersistent, LifeTime);
			}
		}
#endif //ENABLE_DRAW_DEBUG
	}

	return bBlockingHit;
}

bool UGameplayStatics::Blueprint_PredictProjectilePath_Advanced(const UObject* WorldContextObject, const FPredictProjectilePathParams& PredictParams, FPredictProjectilePathResult& PredictResult)
{
	return PredictProjectilePath(WorldContextObject, PredictParams, PredictResult);
}

// BP wrapper to general-purpose function.
bool UGameplayStatics::Blueprint_PredictProjectilePath_ByObjectType(
	const UObject* WorldContextObject,
	FHitResult& OutHit,
	TArray<FVector>& OutPathPositions,
	FVector& OutLastTraceDestination,
	FVector StartPos,
	FVector LaunchVelocity,
	bool bTracePath,
	float ProjectileRadius,
	const TArray<TEnumAsByte<EObjectTypeQuery> >& ObjectTypes,
	bool bTraceComplex,
	const TArray<AActor*>& ActorsToIgnore,
	EDrawDebugTrace::Type DrawDebugType,
	float DrawDebugTime,
	float SimFrequency,
	float MaxSimTime,
	float OverrideGravityZ)
{
	FPredictProjectilePathParams Params = FPredictProjectilePathParams(ProjectileRadius, StartPos, LaunchVelocity, MaxSimTime);
	Params.bTraceWithCollision = bTracePath;
	Params.bTraceComplex = bTraceComplex;
	Params.ActorsToIgnore = ActorsToIgnore;
	Params.DrawDebugType = DrawDebugType;
	Params.DrawDebugTime = DrawDebugTime;
	Params.SimFrequency = SimFrequency;
	Params.OverrideGravityZ = OverrideGravityZ;
	Params.ObjectTypes = ObjectTypes; // Object trace
	Params.bTraceWithChannel = false;

	// Do the trace
	FPredictProjectilePathResult PredictResult;
	bool bHit = PredictProjectilePath(WorldContextObject, Params, PredictResult);

	// Fill in results.
	OutHit = PredictResult.HitResult;
	OutLastTraceDestination = PredictResult.LastTraceDestination.Location;
	OutPathPositions.Empty(PredictResult.PathData.Num());
	for (const FPredictProjectilePathPointData& PathPoint : PredictResult.PathData)
	{
		OutPathPositions.Add(PathPoint.Location);
	}
	return bHit;
}

// BP wrapper to general-purpose function.
bool UGameplayStatics::Blueprint_PredictProjectilePath_ByTraceChannel(
	const UObject* WorldContextObject,
	FHitResult& OutHit,
	TArray<FVector>& OutPathPositions,
	FVector& OutLastTraceDestination,
	FVector StartPos,
	FVector LaunchVelocity,
	bool bTracePath,
	float ProjectileRadius,
	TEnumAsByte<ECollisionChannel> TraceChannel,
	bool bTraceComplex,
	const TArray<AActor*>& ActorsToIgnore,
	EDrawDebugTrace::Type DrawDebugType,
	float DrawDebugTime,
	float SimFrequency,
	float MaxSimTime,
	float OverrideGravityZ)
{
	FPredictProjectilePathParams Params = FPredictProjectilePathParams(ProjectileRadius, StartPos, LaunchVelocity, MaxSimTime);
	Params.bTraceWithCollision = bTracePath;
	Params.bTraceComplex = bTraceComplex;
	Params.ActorsToIgnore = ActorsToIgnore;
	Params.DrawDebugType = DrawDebugType;
	Params.DrawDebugTime = DrawDebugTime;
	Params.SimFrequency = SimFrequency;
	Params.OverrideGravityZ = OverrideGravityZ;
	Params.TraceChannel = TraceChannel; // Trace by channel

	// Do the trace
	FPredictProjectilePathResult PredictResult;
	bool bHit = PredictProjectilePath(WorldContextObject, Params, PredictResult);

	// Fill in results.
	OutHit = PredictResult.HitResult;
	OutLastTraceDestination = PredictResult.LastTraceDestination.Location;
	OutPathPositions.Empty(PredictResult.PathData.Num());
	for (const FPredictProjectilePathPointData& PathPoint : PredictResult.PathData)
	{
		OutPathPositions.Add(PathPoint.Location);
	}
	return bHit;
}


bool UGameplayStatics::SuggestProjectileVelocity_CustomArc(const UObject* WorldContextObject, FVector& OutLaunchVelocity, FVector StartPos, FVector EndPos, float OverrideGravityZ /*= 0*/, float ArcParam /*= 0.5f */)
{
	/* Make sure the start and end aren't the same location */
	FVector const StartToEnd = EndPos - StartPos;
	float const StartToEndDist = StartToEnd.Size();

	UWorld const* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World && StartToEndDist > UE_KINDA_SMALL_NUMBER)
	{
		const float GravityZ = FMath::IsNearlyEqual(OverrideGravityZ, 0.0f) ? World->GetGravityZ() : OverrideGravityZ;

		// choose arc according to the arc param
		FVector const StartToEndDir = StartToEnd / StartToEndDist;
		FVector LaunchDir = FMath::Lerp(FVector::UpVector, StartToEndDir, ArcParam).GetSafeNormal();

		// v = sqrt ( g * dx^2 / ( (dx tan(angle) + dz) * 2 * cos(angle))^2 ) )

		FRotator const LaunchRot = LaunchDir.Rotation();
		float const Angle = FMath::DegreesToRadians(LaunchRot.Pitch);

		float const Dx = StartToEnd.Size2D();
		float const Dz = StartToEnd.Z;
		float const NumeratorInsideSqrt = (GravityZ * FMath::Square(Dx) * 0.5f);
		float const DenominatorInsideSqrt = (Dz - (Dx * FMath::Tan(Angle))) * FMath::Square(FMath::Cos(Angle));
		float const InsideSqrt = NumeratorInsideSqrt / DenominatorInsideSqrt;
		if (InsideSqrt >= 0.f)
		{
			// there exists a solution
			float const Speed = FMath::Sqrt(InsideSqrt);	// this is the mag of the vertical component
			OutLaunchVelocity = LaunchDir * Speed;
			return true;
		}
	}

	OutLaunchVelocity = FVector::ZeroVector;
	return false;
}

bool UGameplayStatics::SuggestProjectileVelocity_MovingTarget(const UObject* WorldContextObject, FVector& OutLaunchVelocity, FVector ProjectileStartLocation, AActor* TargetActor, FVector TargetLocationOffset /*= FVector::ZeroVector*/, double GravityZOverride /*= 0.f*/, double TimeToTarget /*= 1.f*/, EDrawDebugTrace::Type DrawDebugType /*= EDrawDebugTrace::Type::None*/, float DrawDebugTime /*= 3.f*/, FLinearColor DrawDebugColor /*= FLinearColor::Red*/)
{
	// Generally solved using the logic below. In code we also adjust for TargetLocationOffset and a GravityZOverride.
	// 
	// pp = projectile position
	// pp0 = projectile initial position
	// vp0 = projectile initial velocity (OutLaunchVelocity)
	// g = gravity vector
	// pt = target position
	// pt0 = target initial position
	// vt0 = target initial velocity
	// 
	// Projectile position as a function of time
	// pp = pp0 + vp0*t + (g/2)*(t^2)
	// 
	// Target position as a function of time
	// pt = pt0 + vt0*t
	// 
	// Since we want the projectile position and target position to be the same, we can set the equations equal to each other to get:
	// vp0 = (pt0 + vt0*t - pp0 - (g/2)*(t^2)) / t
	
	if (!GEngine)
	{
		return false;
	}

	const UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return false;
	}

	if (!TargetActor)
	{
		return false;
	}

	// Clamp TimeToTarget to ensure non-zero value
	TimeToTarget = FMath::Clamp(TimeToTarget, 0.1, TimeToTarget);

	// Find the target's velocity
	FVector TargetVelocity = TargetActor->GetVelocity();

	// If the target is a character moving on ground or the floor result is walkable (and therefore used for movement),
	// GetVelocity() will return a vector with a Z value of 0, so we need to adjust for slope.
	if (ACharacter* TargetCharacter = Cast<ACharacter>(TargetActor))
	{
		if (UCharacterMovementComponent* TargetMovementComp = TargetCharacter->GetCharacterMovement())
		{
			FFindFloorResult FloorResult = TargetMovementComp->CurrentFloor;
			if (TargetMovementComp->IsMovingOnGround() || FloorResult.IsWalkableFloor())
			{
				FHitResult FloorHit = FloorResult.HitResult;
				const double FloorAngleRadians = FMath::Acos(FloorHit.ImpactNormal.Z);

				// z = x * tan(theta)
				const double ZMagnitude = TargetVelocity.Size2D() * FMath::Tan(FloorAngleRadians);
				TargetVelocity.Z = TargetVelocity.Dot(FloorHit.ImpactNormal) < 0.0 ? ZMagnitude : -ZMagnitude;
			}
		}
	}

	// Find projectile's velocity using the target's velocity
	const FVector TargetCurrentLocationPlusOffset = TargetActor->GetActorLocation() + TargetLocationOffset;
	const double GravityZ = FMath::IsNearlyZero(GravityZOverride) ? World->GetGravityZ() : GravityZOverride;
	const FVector GravityVector = FVector(0.0, 0.0, GravityZ);
	
	OutLaunchVelocity = (TargetCurrentLocationPlusOffset + (TargetVelocity * TimeToTarget) - ProjectileStartLocation - (0.5 * GravityVector * FMath::Square(TimeToTarget))) / TimeToTarget;

#if ENABLE_DRAW_DEBUG
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		const bool bPersistentLines = DrawDebugType == EDrawDebugTrace::Persistent;
		const float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawDebugTime : 0.f;
		
		// Draw bounding box of target at current location
		const float BoxThickness = 2.f;
		const FQuat CurrentRotation = TargetActor->GetActorRotation().Quaternion();
		FBox BodyBox(ForceInit);
		FVector BoundingBoxExtent = FVector::ZeroVector;
		FVector BoundingBoxCenter = TargetActor->GetActorLocation();
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(TargetActor->GetRootComponent()))
		{
			BodyBox = PrimitiveComponent->Bounds.GetBox();
			BodyBox.GetCenterAndExtents(BoundingBoxCenter, BoundingBoxExtent);
		}

		DrawDebugBox(World, BoundingBoxCenter, BoundingBoxExtent, DrawDebugColor.ToFColor(true), bPersistentLines, LifeTime, 0, BoxThickness);

		// Draw arrow indicating aim direction from start location
		const float DirectionalArrowLength = 30.f;
		const float DirectionalArrowsize = 5.f;
		const float DirectionalArrowThickness = 2.f;
		DrawDebugDirectionalArrow(World, ProjectileStartLocation, ProjectileStartLocation + OutLaunchVelocity.GetSafeNormal() * DirectionalArrowLength, DirectionalArrowsize, DrawDebugColor.ToFColor(true), bPersistentLines, LifeTime, 0, DirectionalArrowThickness);

		// Draw sphere at the target's final location (where the projectile should pass through after TimeToTarget seconds)
		const float SphereRadius = 16.f;
		const int32 SphereSegments = 16;
		const float SphereThickness = 2.f;
		const FVector TargetFinalLocation = TargetCurrentLocationPlusOffset + (TargetVelocity * TimeToTarget);
		DrawDebugSphere(World, TargetFinalLocation, SphereRadius, SphereSegments, DrawDebugColor.ToFColor(true), bPersistentLines, LifeTime, 0, SphereThickness);
	}
#endif //ENABLE_DRAW_DEBUG

	return true;
}

FIntVector UGameplayStatics::GetWorldOriginLocation(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->OriginLocation : FIntVector::ZeroValue;
}

void UGameplayStatics::SetWorldOriginLocation(const UObject* WorldContextObject, FIntVector NewLocation)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( World )
	{
		World->RequestNewWorldOrigin(NewLocation);
	}
}

FVector UGameplayStatics::RebaseLocalOriginOntoZero(UObject* WorldContextObject, FVector WorldLocation)
{
	return FRepMovement::RebaseOntoZeroOrigin(WorldLocation, GetWorldOriginLocation(WorldContextObject));
}

FVector UGameplayStatics::RebaseZeroOriginOntoLocal(UObject* WorldContextObject, FVector WorldLocation)
{
	return FRepMovement::RebaseOntoLocalOrigin(WorldLocation, GetWorldOriginLocation(WorldContextObject));
}

int32 UGameplayStatics::GrassOverlappingSphereCount(const UObject* WorldContextObject, const UStaticMesh* Mesh, FVector CenterPosition, float Radius)
{
	int32 Count = 0;

	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		const FSphere Sphere(CenterPosition, Radius);

		// check every landscape
		for (TActorIterator<ALandscapeProxy> It(World); It; ++It)
		{
			ALandscapeProxy* L = *It;
			if (L)
			{
				for (UHierarchicalInstancedStaticMeshComponent const* HComp : L->FoliageComponents)
				{
					if (HComp && (HComp->GetStaticMesh() == Mesh))
					{
						Count += HComp->GetOverlappingSphereCount(Sphere);
					}
				}
			}
		}
	}

	return Count;
}


bool UGameplayStatics::DeprojectScreenToWorld(APlayerController const* Player, const FVector2D& ScreenPosition, FVector& WorldPosition, FVector& WorldDirection)
{
	ULocalPlayer* const LP = Player ? Player->GetLocalPlayer() : nullptr;
	if (LP && LP->ViewportClient)
	{
		// get the projection data
		FSceneViewProjectionData ProjectionData;
		if (LP->GetProjectionData(LP->ViewportClient->Viewport, /*out*/ ProjectionData))
		{
			FMatrix const InvViewProjMatrix = ProjectionData.ComputeViewProjectionMatrix().InverseFast();
			FSceneView::DeprojectScreenToWorld(ScreenPosition, ProjectionData.GetConstrainedViewRect(), InvViewProjMatrix, /*out*/ WorldPosition, /*out*/ WorldDirection);
			return true;
		}
	}

	// something went wrong, zero things and return false
	WorldPosition = FVector::ZeroVector;
	WorldDirection = FVector::ZeroVector;
	return false;
}

bool UGameplayStatics::DeprojectSceneCaptureToWorld(ASceneCapture2D const* SceneCapture2D, const FVector2D& TargetUV, FVector& WorldPosition, FVector& WorldDirection)
{
	if (SceneCapture2D)
	{
		return DeprojectSceneCaptureComponentToWorld(SceneCapture2D->GetCaptureComponent2D(), TargetUV, WorldPosition, WorldDirection);
	}

	// something went wrong, zero things and return false
	WorldPosition = FVector::ZeroVector;
	WorldDirection = FVector::ZeroVector;
	return false;
}

bool UGameplayStatics::DeprojectSceneCaptureComponentToWorld(USceneCaptureComponent2D* SceneCaptureComponent2D, const FVector2D& TargetUV, FVector& WorldPosition, FVector& WorldDirection)
{
	if (SceneCaptureComponent2D)
	{
		if (SceneCaptureComponent2D->TextureTarget)
		{
			FMinimalViewInfo ViewInfo;
			SceneCaptureComponent2D->GetCameraView(0.0f, ViewInfo);

			FMatrix ProjectionMatrix;
			if (SceneCaptureComponent2D->bUseCustomProjectionMatrix)
			{
				ProjectionMatrix = AdjustProjectionMatrixForRHI(SceneCaptureComponent2D->CustomProjectionMatrix);
			}
			else//
			{
				ProjectionMatrix = AdjustProjectionMatrixForRHI(ViewInfo.CalculateProjectionMatrix());
			}
			FMatrix InvProjectionMatrix = ProjectionMatrix.Inverse();

			// A view matrix is the inverse of the viewer's matrix, so an inverse view matrix is just the viewer's matrix.
			// To save precision, we directly compute the viewer's matrix, plus it also avoids the cost of the inverse.
			// The matrix to convert from world coordinate space to view coordinate space also needs to be included (this
			// is the transpose of the similar matrix used in CalculateViewProjectionMatricesFromMinimalView).
			FMatrix InvViewMatrix = FMatrix(
				FPlane(0, 1, 0, 0),
				FPlane(0, 0, 1, 0),
				FPlane(1, 0, 0, 0),
				FPlane(0, 0, 0, 1)) * FRotationTranslationMatrix(ViewInfo.Rotation, ViewInfo.Location);

			FIntPoint TargetSize = FIntPoint(SceneCaptureComponent2D->TextureTarget->SizeX, SceneCaptureComponent2D->TextureTarget->SizeY);

			FSceneView::DeprojectScreenToWorld(
				TargetUV * FVector2D(TargetSize),
				FIntRect(FIntPoint(0, 0), TargetSize),
				InvViewMatrix,
				InvProjectionMatrix,
				WorldPosition,
				WorldDirection);

			return true;
		}
	}

	// something went wrong, zero things and return false
	WorldPosition = FVector::ZeroVector;
	WorldDirection = FVector::ZeroVector;
	return false;
}

bool UGameplayStatics::ProjectWorldToScreen(APlayerController const* Player, const FVector& WorldPosition, FVector2D& ScreenPosition, bool bPlayerViewportRelative)
{
	ULocalPlayer* const LP = Player ? Player->GetLocalPlayer() : nullptr;
	if (LP && LP->ViewportClient)
	{
		// get the projection data
		FSceneViewProjectionData ProjectionData;
		if (LP->GetProjectionData(LP->ViewportClient->Viewport, /*out*/ ProjectionData))
		{
			FMatrix const ViewProjectionMatrix = ProjectionData.ComputeViewProjectionMatrix();
			bool bResult = FSceneView::ProjectWorldToScreen(WorldPosition, ProjectionData.GetConstrainedViewRect(), ViewProjectionMatrix, ScreenPosition);

			if (bPlayerViewportRelative)
			{
				ScreenPosition -= FVector2D(ProjectionData.GetConstrainedViewRect().Min);
			}

			bResult = bResult && Player->PostProcessWorldToScreen(WorldPosition, ScreenPosition, bPlayerViewportRelative);
			return bResult;
		}
	}

	ScreenPosition = FVector2D::ZeroVector;
	return false;
}

FVector UGameplayStatics::TransformWorldToFirstPerson(const FMinimalViewInfo& ViewInfo, const FVector& WorldPosition, bool bIgnoreFirstPersonScale)
{
	return ViewInfo.TransformWorldToFirstPerson(WorldPosition, bIgnoreFirstPersonScale);
}

FVector UGameplayStatics::TransformFirstPersonToWorld(const FMinimalViewInfo& ViewInfo, const FVector& FirstPersonPosition, bool bScaleWasIgnoredOnInput)
{
	return ViewInfo.TransformFirstPersonToWorld(FirstPersonPosition, bScaleWasIgnoredOnInput);
}

void UGameplayStatics::CalculateViewProjectionMatricesFromViewTarget(AActor* InViewTarget, FMatrix& OutViewMatrix, FMatrix& OutProjectionMatrix, FMatrix& OutViewProjectionMatrix)
{
	if (InViewTarget)
	{
		FMinimalViewInfo MinimalViewInfo;
		InViewTarget->CalcCamera(0.f, MinimalViewInfo);

		// This is kinda weird odd one-off, can this be integrated into the MinimalViewInfo?
		TOptional<FMatrix> CustomProjectionMatrix;
		if (ASceneCapture2D* SceneCapture2D = Cast<ASceneCapture2D>(InViewTarget))
		{
			if (USceneCaptureComponent2D* SceneCaptureComponent2D = SceneCapture2D->GetCaptureComponent2D())
			{
				if (SceneCaptureComponent2D && SceneCaptureComponent2D->bUseCustomProjectionMatrix)
				{
					CustomProjectionMatrix = SceneCaptureComponent2D->CustomProjectionMatrix;
				}
			}
		}

		CalculateViewProjectionMatricesFromMinimalView(MinimalViewInfo, CustomProjectionMatrix, OutViewMatrix, OutProjectionMatrix, OutViewProjectionMatrix);
	}
}

void UGameplayStatics::CalculateViewProjectionMatricesFromMinimalView(const FMinimalViewInfo& MinimalViewInfo, const TOptional<FMatrix>& CustomProjectionMatrix, FMatrix& OutViewMatrix, FMatrix& OutProjectionMatrix, FMatrix& OutViewProjectionMatrix)
{
	if (CustomProjectionMatrix.IsSet())
	{
		OutProjectionMatrix = AdjustProjectionMatrixForRHI(CustomProjectionMatrix.GetValue());
	}
	else
	{
		OutProjectionMatrix = AdjustProjectionMatrixForRHI(MinimalViewInfo.CalculateProjectionMatrix());
	}

	FMatrix ViewRotationMatrix = FInverseRotationMatrix(MinimalViewInfo.Rotation) * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	OutViewMatrix = FTranslationMatrix(-MinimalViewInfo.Location) * ViewRotationMatrix;
	//OutInvProjectionMatrix = OutProjectionMatrix.Inverse();
	//OutInvViewMatrix = ViewRotationMatrix.GetTransposed() * FTranslationMatrix(MinimalViewInfo.Location);

	OutViewProjectionMatrix = OutViewMatrix * OutProjectionMatrix;
	//OutInvViewProjectionMatrix = OutInvProjectionMatrix * OutInvViewMatrix;
}

void UGameplayStatics::GetViewProjectionMatrix(FMinimalViewInfo DesiredView, FMatrix &ViewMatrix, FMatrix &ProjectionMatrix, FMatrix &ViewProjectionMatrix)
{
	TOptional<FMatrix> CustomMatrix;
	CalculateViewProjectionMatricesFromMinimalView(DesiredView, CustomMatrix, ViewMatrix, ProjectionMatrix, ViewProjectionMatrix);
}

bool UGameplayStatics::GrabOption( FString& Options, FString& Result )
{
	FString QuestionMark("?");

	if( Options.Left(1).Equals(QuestionMark, ESearchCase::CaseSensitive) )
	{
		// Get result.
		Result = Options.Mid(1, MAX_int32);
		const int32 QMIdx = Result.Find(QuestionMark, ESearchCase::CaseSensitive);
		if (QMIdx != INDEX_NONE)
		{
			Result.LeftInline(QMIdx, EAllowShrinking::No);
		}

		// Update options.
		Options.MidInline(1, MAX_int32, EAllowShrinking::No);
		if (Options.Contains(QuestionMark, ESearchCase::CaseSensitive))
		{
			Options.MidInline(Options.Find(QuestionMark, ESearchCase::CaseSensitive), MAX_int32, EAllowShrinking::No);
		}
		else
		{
			Options = FString();
		}

		return true;
	}

	return false;
}

void UGameplayStatics::GetKeyValue( const FString& Pair, FString& Key, FString& Value )
{
	const int32 EqualSignIndex = Pair.Find(TEXT("="), ESearchCase::CaseSensitive);
	if( EqualSignIndex != INDEX_NONE )
	{
		Key = Pair.Left(EqualSignIndex);
		Value = Pair.Mid(EqualSignIndex + 1, MAX_int32);
	}
	else
	{
		Key = Pair;
		Value = TEXT("");
	}
}

FString UGameplayStatics::ParseOption( FString Options, const FString& Key )
{
	FString ReturnValue;
	FString Pair, PairKey, PairValue;
	while( GrabOption( Options, Pair ) )
	{
		GetKeyValue( Pair, PairKey, PairValue );
		if (Key == PairKey)
		{
			ReturnValue = MoveTemp(PairValue);
			break;
		}
	}
	return ReturnValue;
}

bool UGameplayStatics::HasOption( FString Options, const FString& Key )
{
	FString Pair, PairKey, PairValue;
	while( GrabOption( Options, Pair ) )
	{
		GetKeyValue( Pair, PairKey, PairValue );
		if (Key == PairKey)
		{
			return true;
		}
	}
	return false;
}

int32 UGameplayStatics::GetIntOption( const FString& Options, const FString& Key, int32 DefaultValue)
{
	const FString InOpt = ParseOption( Options, Key );
	if ( !InOpt.IsEmpty() )
	{
		return FCString::Atoi(*InOpt);
	}
	return DefaultValue;
}

bool UGameplayStatics::HasLaunchOption(const FString& OptionToCheck)
{
	return FParse::Param(FCommandLine::Get(), *OptionToCheck);
}

void UGameplayStatics::AnnounceAccessibleString(const FString& AnnouncementString)
{
#if WITH_ACCESSIBILITY
	if (!AnnouncementString.IsEmpty())
	{
		FSlateApplication::Get().GetAccessibleMessageHandler()->MakeAccessibleAnnouncement(AnnouncementString);
	}
#endif
}

#undef LOCTEXT_NAMESPACE

