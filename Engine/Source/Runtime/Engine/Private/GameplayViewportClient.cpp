// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayViewportClient.h"
#include "UnrealClient.h"

bool FGameplayViewportClient::InputChar(FViewport* const Viewport, const int32 ControllerId, const TCHAR Character)
{
	// Absorb all keys so game input events are not routed to the Slate editor frame
	return Viewport->IsSlateViewport() && Viewport->IsPlayInEditorViewport();
}

bool FGameplayViewportClient::IsFocused(FViewport* Viewport)
{
	return Viewport->HasFocus() || Viewport->HasMouseCapture();
}

bool FGameplayViewportClient::RequiresHitProxyStorage()
{
	// This is only true for editor VPCs
	return false;
}

bool FGameplayViewportClient::ShouldDPIScaleSceneCanvas() const
{
	// Scene View Family does its own DPI scaling
	return false;
}
