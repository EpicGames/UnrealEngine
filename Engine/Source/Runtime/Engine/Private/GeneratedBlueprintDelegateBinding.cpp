// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/GeneratedBlueprintDelegateBinding.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeneratedBlueprintDelegateBinding)

void UGeneratedBlueprintBinding::BindDynamicDelegates(UObject* InInstance) const
{
	if (!InInstance)
	{
		return;
	}

	for(const FGeneratedBlueprintDelegateBinding& Binding : GeneratedBlueprintBindings)
	{
		if (FMulticastDelegateProperty* MulticastDelegateProp = FindFProperty<FMulticastDelegateProperty>(InInstance->GetClass(), Binding.DelegatePropertyName))
		{
			// Get the function we want to bind
			if (UFunction* FunctionToBind = InInstance->GetClass()->FindFunctionByName(Binding.FunctionNameToBind))
			{
				// Bind function on the instance to this delegate
				FScriptDelegate Delegate;
				Delegate.BindUFunction(InInstance, Binding.FunctionNameToBind);
				MulticastDelegateProp->AddDelegate(MoveTemp(Delegate), InInstance);
			}
		}
	}
}

void UGeneratedBlueprintBinding::UnbindDynamicDelegates(UObject* InInstance) const
{
	if (!InInstance)
	{
		return;
	}

	for (const FGeneratedBlueprintDelegateBinding& Binding : GeneratedBlueprintBindings)
	{
		if (FMulticastDelegateProperty* MulticastDelegateProp = FindFProperty<FMulticastDelegateProperty>(InInstance->GetClass(), Binding.DelegatePropertyName))
		{
			// Unbind function on the instance to this delegate
			FScriptDelegate Delegate;
			Delegate.BindUFunction(InInstance, Binding.FunctionNameToBind);
			MulticastDelegateProp->RemoveDelegate(Delegate, InInstance);
		}
	}
}

void UGeneratedBlueprintBinding::UnbindDynamicDelegatesForProperty(UObject* InInstance, const FObjectProperty* InObjectProperty) const
{
	// Intentionally left un-implimented. We operate on the entire object, so we aren't interested in individual property unbinds like component removal would be
	Super::UnbindDynamicDelegatesForProperty(InInstance, InObjectProperty);
}

