// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericQuadTree.h"

DEFINE_LOG_CATEGORY(LogQuadTree);
