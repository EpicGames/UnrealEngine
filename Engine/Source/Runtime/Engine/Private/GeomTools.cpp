// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 =============================================================================*/

#include "GeomTools.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"
#include "Math/ConvexHull2d.h"

DEFINE_LOG_CATEGORY_STATIC(LogGeomTools, Log, All);

	
/** */
static FClipSMVertex GetVert(const UStaticMesh* StaticMesh, int32 VertIndex)
{
	FClipSMVertex Result;
	const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[0];
	Result.Pos = LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(VertIndex);
	Result.TangentX = LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertIndex);
	Result.TangentY = LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(VertIndex);
	Result.TangentZ = LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertIndex);
	const int32 NumUVs = LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	for(int32 UVIndex = 0;UVIndex < NumUVs;UVIndex++)
	{
		Result.UVs[UVIndex] = FVector2D(LODModel.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertIndex,UVIndex));
	}
	for(int32 UVIndex = NumUVs;UVIndex < UE_ARRAY_COUNT(Result.UVs);UVIndex++)
	{
		Result.UVs[UVIndex] = FVector2D::ZeroVector;
	}
	Result.Color = FColor( 255, 255, 255 );
	if( LODModel.VertexBuffers.ColorVertexBuffer.GetNumVertices() > 0 )
	{
		Result.Color = LODModel.VertexBuffers.ColorVertexBuffer.VertexColor(VertIndex);
	}
	return Result;
}

/** Take two static mesh verts and interpolate all values between them */
FClipSMVertex InterpolateVert(const FClipSMVertex& V0, const FClipSMVertex& V1, float Alpha)
{
	FClipSMVertex Result;

	// Handle dodgy alpha
	if(FMath::IsNaN(Alpha) || !FMath::IsFinite(Alpha))
	{
		Result = V1;
		return Result;
	}

	Result.Pos = FMath::Lerp(V0.Pos, V1.Pos, Alpha);
	Result.TangentX = FMath::Lerp(V0.TangentX,V1.TangentX,Alpha);
	Result.TangentY = FMath::Lerp(V0.TangentY,V1.TangentY,Alpha);
	Result.TangentZ = FMath::Lerp(V0.TangentZ,V1.TangentZ,Alpha);
	for(int32 i=0; i<8; i++)
	{
		Result.UVs[i] = FMath::Lerp(V0.UVs[i], V1.UVs[i], Alpha);
	}
	
	Result.Color.R = FMath::Clamp( FMath::TruncToInt(FMath::Lerp(float(V0.Color.R), float(V1.Color.R), Alpha)), 0, 255 );
	Result.Color.G = FMath::Clamp( FMath::TruncToInt(FMath::Lerp(float(V0.Color.G), float(V1.Color.G), Alpha)), 0, 255 );
	Result.Color.B = FMath::Clamp( FMath::TruncToInt(FMath::Lerp(float(V0.Color.B), float(V1.Color.B), Alpha)), 0, 255 );
	Result.Color.A = FMath::Clamp( FMath::TruncToInt(FMath::Lerp(float(V0.Color.A), float(V1.Color.A), Alpha)), 0, 255 );
	return Result;
}

/** Extracts the triangles from a static-mesh as clippable triangles. */
void FGeomTools::GetClippableStaticMeshTriangles(TArray<FClipSMTriangle>& OutClippableTriangles,const UStaticMesh* StaticMesh)
{
	const FStaticMeshLODResources& RenderData = StaticMesh->GetRenderData()->LODResources[0];
	FIndexArrayView Indices = RenderData.IndexBuffer.GetArrayView();
	for(int32 SectionIndex = 0;SectionIndex < RenderData.Sections.Num();SectionIndex++)
	{
		const FStaticMeshSection& Section = RenderData.Sections[SectionIndex];
		for(uint32 TriangleIndex = 0;TriangleIndex < Section.NumTriangles;TriangleIndex++)
		{
			FClipSMTriangle ClipTriangle(0);

			// Copy the triangle's attributes.
			ClipTriangle.MaterialIndex = Section.MaterialIndex;
			ClipTriangle.NumUVs = RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
			ClipTriangle.SmoothingMask = 0;
			ClipTriangle.bOverrideTangentBasis = true;

			// Extract the vertices for this triangle.
			const uint32 BaseIndex = Section.FirstIndex + TriangleIndex * 3;
			for(uint32 TriangleVertexIndex = 0;TriangleVertexIndex < 3;TriangleVertexIndex++)
			{
				const uint32 VertexIndex = Indices[BaseIndex + TriangleVertexIndex];
				ClipTriangle.Vertices[TriangleVertexIndex] = GetVert(StaticMesh,VertexIndex);
			}

			// Compute the triangle's gradients and normal.
			ClipTriangle.ComputeGradientsAndNormal();

			// Add the triangle to the output array.
			OutClippableTriangles.Add(ClipTriangle);
		}
	}
}

/** Take the input mesh and cut it with supplied plane, creating new verts etc. Also outputs new edges created on the plane. */
void FGeomTools::ClipMeshWithPlane( TArray<FClipSMTriangle>& OutTris, TArray<FUtilEdge3D>& OutClipEdges, const TArray<FClipSMTriangle>& InTris, const FPlane& Plane )
{
	// Iterate over each source triangle
	for(int32 TriIdx=0; TriIdx<InTris.Num(); TriIdx++)
	{
		const FClipSMTriangle* SrcTri = &InTris[TriIdx];

		// Calculate which verts are beyond clipping plane
		float PlaneDist[3];
		for(int32 i=0; i<3; i++)
		{
			PlaneDist[i] = Plane.PlaneDot(FVector(SrcTri->Vertices[i].Pos));
		}

		TArray<FClipSMVertex> FinalVerts;
		FUtilEdge3D NewClipEdge;
		int32 ClippedEdges = 0;

		for(int32 EdgeIdx = 0; EdgeIdx < 3; EdgeIdx++)
		{
			int32 ThisVert = EdgeIdx;

			// If start vert is inside, add it.
			if(PlaneDist[ThisVert] < 0.0f)
			{
				FinalVerts.Add( SrcTri->Vertices[ThisVert] );
			}

			// If start and next vert are on opposite sides, add intersection
			int32 NextVert = (EdgeIdx+1)%3;

			if((PlaneDist[EdgeIdx] < 0.0f) != (PlaneDist[NextVert] < 0.0f))
			{
				// Find distance along edge that plane is
				float Alpha = -PlaneDist[ThisVert] / (PlaneDist[NextVert] - PlaneDist[ThisVert]);
				// Interpolate vertex params to that point
				FClipSMVertex InterpVert = InterpolateVert(SrcTri->Vertices[ThisVert], SrcTri->Vertices[NextVert], FMath::Clamp(Alpha,0.0f,1.0f));
				// Save vert
				FinalVerts.Add(InterpVert);

				// When we make a new edge on the surface of the clip plane, save it off.
				if(ClippedEdges == 0)
				{
					NewClipEdge.V0 = InterpVert.Pos;
				}
				else
				{
					NewClipEdge.V1 = InterpVert.Pos;
				}
				ClippedEdges++;
			}
		}

		// Triangulate the clipped polygon.
		for(int32 VertexIndex = 2;VertexIndex < FinalVerts.Num();VertexIndex++)
		{
			FClipSMTriangle NewTri = *SrcTri;
			NewTri.Vertices[0] = FinalVerts[0];
			NewTri.Vertices[1] = FinalVerts[VertexIndex - 1];
			NewTri.Vertices[2] = FinalVerts[VertexIndex];
			NewTri.bOverrideTangentBasis = true;
			OutTris.Add(NewTri);
		}

		// If we created a new edge, save that off here as well
		if(ClippedEdges == 2)
		{
			OutClipEdges.Add(NewClipEdge);
		}
	}
}

/** Take a set of 3D Edges and project them onto the supplied plane. Also returns matrix use to convert them back into 3D edges. */
void FGeomTools::ProjectEdges( TArray<FUtilEdge2D>& Out2DEdges, FMatrix& ToWorld, const TArray<FUtilEdge3D>& In3DEdges, const FPlane& InPlane )
{
	// Build matrix to transform verts into plane space
	FVector BasisX, BasisY, BasisZ;
	BasisZ = InPlane;
	BasisZ.FindBestAxisVectors(BasisX, BasisY);
	ToWorld = FMatrix( BasisX, BasisY, InPlane, BasisZ * InPlane.W );

	Out2DEdges.AddUninitialized( In3DEdges.Num() );
	for(int32 i=0; i<In3DEdges.Num(); i++)
	{
		FVector P = ToWorld.InverseTransformPosition(FVector(In3DEdges[i].V0));
		Out2DEdges[i].V0.X = P.X;
		Out2DEdges[i].V0.Y = P.Y;

		P = ToWorld.InverseTransformPosition(FVector(In3DEdges[i].V1));
		Out2DEdges[i].V1.X = P.X;
		Out2DEdges[i].V1.Y = P.Y;
	}
}

/** End of one edge and start of next must be less than this to connect them. */
static float EdgeMatchTolerance = 0.01f;

/** Util to look for best next edge start from Start. Returns false if no good next edge found. Edge is removed from InEdgeSet when found. */
static bool FindNextEdge(FUtilEdge2D& OutNextEdge, const FVector2D& Start, TArray<FUtilEdge2D>& InEdgeSet)
{
	float ClosestDistSqr = UE_BIG_NUMBER;
	int32 OutEdgeIndex = INDEX_NONE;
	// Search set of edges for one that starts closest to Start
	for(int32 i=0; i<InEdgeSet.Num(); i++)
	{
		float DistSqr = (InEdgeSet[i].V0 - Start).SizeSquared();
		if(DistSqr < ClosestDistSqr)
		{
			ClosestDistSqr = DistSqr;
			OutNextEdge = InEdgeSet[i];
			OutEdgeIndex = i;
		}

		DistSqr = (InEdgeSet[i].V1 - Start).SizeSquared();
		if(DistSqr < ClosestDistSqr)
		{
			ClosestDistSqr = DistSqr;
			OutNextEdge = InEdgeSet[i];
			Swap(OutNextEdge.V0, OutNextEdge.V1);
			OutEdgeIndex = i;
		}
	}

	// If next edge starts close enough return it
	if(ClosestDistSqr < FMath::Square(EdgeMatchTolerance))
	{
		check(OutEdgeIndex != INDEX_NONE);
		InEdgeSet.RemoveAt(OutEdgeIndex);
		return true;
	}

	// No next edge found.
	return false;
}

/** 
 *	Make sure that polygon winding is always consistent - cross product between successive edges is positive. 
 *	This function also remove co-linear edges.
 */
static void FixPolyWinding(FUtilPoly2D& Poly)
{
	float TotalAngle = 0.f;
	for(int32 i=Poly.Verts.Num()-1; i>=0; i--)
	{
		// Triangle is 'this' vert plus the one before and after it
		int32 AIndex = (i==0) ? Poly.Verts.Num()-1 : i-1;
		int32 BIndex = i;
		int32 CIndex = (i+1)%Poly.Verts.Num();

		float ABDistSqr = (Poly.Verts[BIndex].Pos - Poly.Verts[AIndex].Pos).SizeSquared();
		FVector2D ABEdge = (Poly.Verts[BIndex].Pos - Poly.Verts[AIndex].Pos).GetSafeNormal();

		float BCDistSqr = (Poly.Verts[CIndex].Pos - Poly.Verts[BIndex].Pos).SizeSquared();
		FVector2D BCEdge = (Poly.Verts[CIndex].Pos - Poly.Verts[BIndex].Pos).GetSafeNormal();

		// See if points are co-incident or edges are co-linear - if so, remove.
		if(ABDistSqr < 0.0001f || BCDistSqr < 0.0001f || ABEdge.Equals(BCEdge, 0.01f))
		{
			Poly.Verts.RemoveAt(i);
		}
		else
		{
			TotalAngle += FMath::Asin(ABEdge ^ BCEdge);
		}
	}

	// If total angle is negative, reverse order.
	if(TotalAngle < 0.f)
	{
		int32 NumVerts = Poly.Verts.Num();

		TArray<FUtilVertex2D> NewVerts;
		NewVerts.AddUninitialized(NumVerts);

		for(int32 i=0; i<NumVerts; i++)
		{
			NewVerts[i] = Poly.Verts[NumVerts-(1+i)];
		}
		Poly.Verts = NewVerts;
	}
}

/** Given a set of edges, find the set of closed polygons created by them. */
void FGeomTools::Buid2DPolysFromEdges(TArray<FUtilPoly2D>& OutPolys, const TArray<FUtilEdge2D>& InEdges, const FColor& VertColor)
{
	TArray<FUtilEdge2D> EdgeSet = InEdges;

	// While there are still edges to process..
	while(EdgeSet.Num() > 0)
	{
		// Initialise new polygon with the first edge in the set
		FUtilPoly2D NewPoly;
		FUtilEdge2D FirstEdge = EdgeSet.Pop();

		NewPoly.Verts.Add(FUtilVertex2D(FirstEdge.V0, VertColor));
		NewPoly.Verts.Add(FUtilVertex2D(FirstEdge.V1, VertColor));

		// Now we keep adding edges until we can't find any more
		FVector2D PolyEnd = NewPoly.Verts[ NewPoly.Verts.Num()-1 ].Pos;
		FUtilEdge2D NextEdge;
		while( FindNextEdge(NextEdge, PolyEnd, EdgeSet) )
		{
			NewPoly.Verts.Add(FUtilVertex2D(NextEdge.V1, VertColor));
			PolyEnd = NewPoly.Verts[ NewPoly.Verts.Num()-1 ].Pos;
		}

		// After walking edges see if we have a closed polygon.
		float CloseDistSqr = (NewPoly.Verts[0].Pos - NewPoly.Verts[ NewPoly.Verts.Num()-1 ].Pos).SizeSquared();
		if(NewPoly.Verts.Num() >= 4 && CloseDistSqr < FMath::Square(EdgeMatchTolerance))
		{
			// Remove last vert - its basically a duplicate of the first.
			NewPoly.Verts.RemoveAt( NewPoly.Verts.Num()-1 );

			// Make sure winding is correct.
			FixPolyWinding(NewPoly);

			// Add to set of output polys.
			OutPolys.Add(NewPoly);
		}
	}
}

/** Given three direction vectors, indicates if A and B are on the same 'side' of Vec. */
bool FGeomTools::VectorsOnSameSide(const FVector3f& Vec, const FVector3f& A, const FVector3f& B, const float SameSideDotProductEpsilon)
{
	const FVector3f CrossA = Vec ^ A;
	const FVector3f CrossB = Vec ^ B;
	float DotWithEpsilon = SameSideDotProductEpsilon + ( CrossA | CrossB );
	return !(DotWithEpsilon < 0.0f);
}

/** Util to see if P lies within triangle created by A, B and C. */
bool FGeomTools::PointInTriangle(const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& P, const float InsideTriangleDotProductEpsilon)
{
	// Cross product indicates which 'side' of the vector the point is on
	// If its on the same side as the remaining vert for all edges, then its inside.	
	if( VectorsOnSameSide(B-A, P-A, C-A, InsideTriangleDotProductEpsilon) &&
		VectorsOnSameSide(C-B, P-B, A-B, InsideTriangleDotProductEpsilon) &&
		VectorsOnSameSide(A-C, P-C, B-C, InsideTriangleDotProductEpsilon) )
	{
		return true;
	}
	else
	{
		return false;
	}
}

/** Compare all aspects of two verts of two triangles to see if they are the same. */
static bool VertsAreEqual(const FClipSMVertex& A,const FClipSMVertex& B)
{
	if( !A.Pos.Equals(B.Pos, UE_THRESH_POINTS_ARE_SAME) )
	{
		return false;
	}

	if( !A.TangentX.Equals(B.TangentX, UE_THRESH_NORMALS_ARE_SAME) )
	{
		return false;
	}
	
	if( !A.TangentY.Equals(B.TangentY, UE_THRESH_NORMALS_ARE_SAME) )
	{
		return false;
	}
	
	if( !A.TangentZ.Equals(B.TangentZ, UE_THRESH_NORMALS_ARE_SAME) )
	{
		return false;
	}

	if( A.Color != B.Color )
	{
		return false;
	}

	for(int32 i=0; i<UE_ARRAY_COUNT(A.UVs); i++)
	{
		if( !A.UVs[i].Equals(B.UVs[i], 1.0f / 1024.0f) )
		{
			return false;
		}
	}

	return true;
}

/** Determines whether two edges may be merged. */
static bool AreEdgesMergeable(
	const FClipSMVertex& V0,
	const FClipSMVertex& V1,
	const FClipSMVertex& V2
	)
{
	const FVector3f MergedEdgeVector = V2.Pos - V0.Pos;
	const float MergedEdgeLengthSquared = MergedEdgeVector.SizeSquared();
	if(MergedEdgeLengthSquared > UE_DELTA)
	{
		// Find the vertex closest to A1/B0 that is on the hypothetical merged edge formed by A0-B1.
		const float IntermediateVertexEdgeFraction =
			((V2.Pos - V0.Pos) | (V1.Pos - V0.Pos)) / MergedEdgeLengthSquared;
		const FClipSMVertex InterpolatedVertex = InterpolateVert(V0,V2,IntermediateVertexEdgeFraction);

		// The edges are merge-able if the interpolated vertex is close enough to the intermediate vertex.
		return VertsAreEqual(InterpolatedVertex,V1);
	}
	else
	{
		return true;
	}
}

/** Given a polygon, decompose into triangles and append to OutTris. */
bool FGeomTools::TriangulatePoly(TArray<FClipSMTriangle>& OutTris, const FClipSMPolygon& InPoly, bool bKeepColinearVertices)
{
	// Can't work if not enough verts for 1 triangle
	if(InPoly.Vertices.Num() < 3)
	{
		// Return true because poly is already a tri
		return true;
	}

	// Vertices of polygon in order - make a copy we are going to modify.
	TArray<FClipSMVertex> PolyVerts = InPoly.Vertices;

	// Keep iterating while there are still vertices
	while(true)
	{
		if (!bKeepColinearVertices)
		{
			// Cull redundant vertex edges from the polygon.
			for (int32 VertexIndex = 0; VertexIndex < PolyVerts.Num(); VertexIndex++)
			{
				const int32 I0 = (VertexIndex + 0) % PolyVerts.Num();
				const int32 I1 = (VertexIndex + 1) % PolyVerts.Num();
				const int32 I2 = (VertexIndex + 2) % PolyVerts.Num();
				if (AreEdgesMergeable(PolyVerts[I0], PolyVerts[I1], PolyVerts[I2]))
				{
					PolyVerts.RemoveAt(I1);
					VertexIndex--;
				}
			}
		}

		if(PolyVerts.Num() < 3)
		{
			break;
		}
		else
		{
			// Look for an 'ear' triangle
			bool bFoundEar = false;
			for(int32 EarVertexIndex = 0;EarVertexIndex < PolyVerts.Num();EarVertexIndex++)
			{
				// Triangle is 'this' vert plus the one before and after it
				const int32 AIndex = (EarVertexIndex==0) ? PolyVerts.Num()-1 : EarVertexIndex-1;
				const int32 BIndex = EarVertexIndex;
				const int32 CIndex = (EarVertexIndex+1)%PolyVerts.Num();

				// Check that this vertex is convex (cross product must be positive)
				const FVector3f ABEdge = PolyVerts[BIndex].Pos - PolyVerts[AIndex].Pos;
				const FVector3f ACEdge = PolyVerts[CIndex].Pos - PolyVerts[AIndex].Pos;
				const float TriangleDeterminant = (ABEdge ^ ACEdge) | (FVector3f)InPoly.FaceNormal;
				if(TriangleDeterminant < 0.0f)
				{
					continue;
				}

				bool bFoundVertInside = false;
				// Look through all verts before this in array to see if any are inside triangle
				for(int32 VertexIndex = 0;VertexIndex < PolyVerts.Num();VertexIndex++)
				{
					if(	VertexIndex != AIndex && VertexIndex != BIndex && VertexIndex != CIndex &&
						PointInTriangle(PolyVerts[AIndex].Pos, PolyVerts[BIndex].Pos, PolyVerts[CIndex].Pos, PolyVerts[VertexIndex].Pos) )
					{
						bFoundVertInside = true;
						break;
					}
				}

				// Triangle with no verts inside - its an 'ear'! 
				if(!bFoundVertInside)
				{
					// Add to output list..
					FClipSMTriangle NewTri(0);
					NewTri.CopyFace(InPoly);
					NewTri.Vertices[0] = PolyVerts[AIndex];
					NewTri.Vertices[1] = PolyVerts[BIndex];
					NewTri.Vertices[2] = PolyVerts[CIndex];
					OutTris.Add(NewTri);

					// And remove vertex from polygon
					PolyVerts.RemoveAt(EarVertexIndex);

					bFoundEar = true;
					break;
				}
			}

			// If we couldn't find an 'ear' it indicates something is bad with this polygon - discard triangles and return.
			if(!bFoundEar)
			{
				UE_LOGF(LogGeomTools, Log, "Triangulation of poly failed.");
				OutTris.Empty();
				return false;
			}
		}
	}

	return true;
}


/** Transform triangle from 2D to 3D static-mesh triangle. */
FClipSMPolygon FGeomTools::Transform2DPolygonToSMPolygon(const FUtilPoly2D& InPoly, const FMatrix& InMatrix)
{
	FClipSMPolygon Result(0);

	for(int32 VertexIndex = 0;VertexIndex < InPoly.Verts.Num();VertexIndex++)
	{
		const FUtilVertex2D& InVertex = InPoly.Verts[VertexIndex];

		FClipSMVertex* OutVertex = new(Result.Vertices) FClipSMVertex;
		FMemory::Memzero(OutVertex,sizeof(*OutVertex));
		OutVertex->Pos = (FVector4f)InMatrix.TransformPosition( FVector(InVertex.Pos.X, InVertex.Pos.Y, 0.f) );
		OutVertex->Color = InVertex.Color;
		OutVertex->UVs[0] = InVertex.UV;
	}

	// Assume that the matrix defines the polygon's normal.
	Result.FaceNormal = InMatrix.TransformVector(FVector(0,0,-1)).GetSafeNormal();

	return Result;
}

/** Does a simple box map onto this 2D polygon. */
void FGeomTools::GeneratePlanarFitPolyUVs(FUtilPoly2D& Polygon)
{
	// First work out 2D bounding box for tris.
	FVector2D Min(UE_BIG_NUMBER, UE_BIG_NUMBER);
	FVector2D Max(-UE_BIG_NUMBER, -UE_BIG_NUMBER);
	for(int32 VertexIndex = 0;VertexIndex < Polygon.Verts.Num();VertexIndex++)
	{
		const FUtilVertex2D& Vertex = Polygon.Verts[VertexIndex];
		Min.X = FMath::Min(Vertex.Pos.X, Min.X);
		Min.Y = FMath::Min(Vertex.Pos.Y, Min.Y);
		Max.X = FMath::Max(Vertex.Pos.X, Max.X);
		Max.Y = FMath::Max(Vertex.Pos.Y, Max.Y);
	}

	const FVector2D Extent = Max - Min;

	// Then use this to generate UVs
	for(int32 VertexIndex = 0;VertexIndex < Polygon.Verts.Num();VertexIndex++)
	{
		FUtilVertex2D& Vertex = Polygon.Verts[VertexIndex];
		Vertex.UV.X = (Vertex.Pos.X - Min.X)/Extent.X;
		Vertex.UV.Y = (Vertex.Pos.Y - Min.Y)/Extent.Y;
	}
}

void FGeomTools::GeneratePlanarTilingPolyUVs(FUtilPoly2D& Polygon, float TileSize)
{
	for (int32 VertexIndex = 0; VertexIndex < Polygon.Verts.Num(); VertexIndex++)
	{
		FUtilVertex2D& Vertex = Polygon.Verts[VertexIndex];
		Vertex.UV.X = Vertex.Pos.X / TileSize;
		Vertex.UV.Y = Vertex.Pos.Y / TileSize;
	}
}


/** Computes a transform from triangle parameter space into the space defined by an attribute that varies on the triangle's surface. */
static FMatrix ComputeTriangleParameterToAttribute(
	const FVector AttributeV0,
	const FVector AttributeV1,
	const FVector AttributeV2
	)
{
	const FVector AttributeOverS = AttributeV1 - AttributeV0;
	const FVector AttributeOverT = AttributeV2 - AttributeV0;
	const FVector AttributeOverNormal = (AttributeOverS ^ AttributeOverT).GetSafeNormal();
	return FMatrix(
		FPlane(	AttributeOverS.X,		AttributeOverS.Y,		AttributeOverS.Z,		0	),
		FPlane(	AttributeOverT.X,		AttributeOverT.Y,		AttributeOverT.Z,		0	),
		FPlane(	AttributeOverNormal.X,	AttributeOverNormal.Y,	AttributeOverNormal.Z,	0	),
		FPlane(	0,						0,						0,						1	)
		);
}

/** Converts a color into a vector. */
static FVector ColorToVector(const FLinearColor& Color)
{
	return FVector(Color.R,Color.G,Color.B);
}

void FClipSMTriangle::ComputeGradientsAndNormal()
{
	// Compute the transform from triangle parameter space to local space.
	const FMatrix ParameterToLocal = ComputeTriangleParameterToAttribute((FVector)Vertices[0].Pos, (FVector)Vertices[1].Pos, (FVector)Vertices[2].Pos);
	const FMatrix LocalToParameter = ParameterToLocal.Inverse();

	// Compute the triangle's normal.
	FaceNormal = ParameterToLocal.TransformVector(FVector(0,0,1));

	// Compute the normal's gradient in local space.
	const FMatrix ParameterToTangentX = ComputeTriangleParameterToAttribute((FVector)Vertices[0].TangentX, (FVector)Vertices[1].TangentX, (FVector)Vertices[2].TangentX);
	const FMatrix ParameterToTangentY = ComputeTriangleParameterToAttribute((FVector)Vertices[0].TangentY, (FVector)Vertices[1].TangentY, (FVector)Vertices[2].TangentY);
	const FMatrix ParameterToTangentZ = ComputeTriangleParameterToAttribute((FVector)Vertices[0].TangentZ, (FVector)Vertices[1].TangentZ, (FVector)Vertices[2].TangentZ);
	TangentXGradient = LocalToParameter * ParameterToTangentX;
	TangentYGradient = LocalToParameter * ParameterToTangentY;
	TangentZGradient = LocalToParameter * ParameterToTangentZ;

	// Compute the color's gradient in local space.
	const FVector Color0 = ColorToVector(Vertices[0].Color);
	const FVector Color1 = ColorToVector(Vertices[1].Color);
	const FVector Color2 = ColorToVector(Vertices[2].Color);
	const FMatrix ParameterToColor = ComputeTriangleParameterToAttribute(Color0,Color1,Color2);
	ColorGradient = LocalToParameter * ParameterToColor;

	for(int32 UVIndex = 0;UVIndex < NumUVs;UVIndex++)
	{
		// Compute the UV's gradient in local space.
		const FVector UV0(Vertices[0].UVs[UVIndex],0);
		const FVector UV1(Vertices[1].UVs[UVIndex],0);
		const FVector UV2(Vertices[2].UVs[UVIndex],0);
		const FMatrix ParameterToUV = ComputeTriangleParameterToAttribute(UV0,UV1,UV2);
		UVGradient[UVIndex] = LocalToParameter * ParameterToUV;
	}
}

/** Util that tries to combine two triangles if possible. */
static bool MergeTriangleIntoPolygon(
	FClipSMPolygon& Polygon,
	const FClipSMTriangle& Triangle)
{
	// The triangles' attributes must match the polygon.
	if(Polygon.MaterialIndex != Triangle.MaterialIndex)
	{
		return false;
	}
	if(Polygon.bOverrideTangentBasis != Triangle.bOverrideTangentBasis)
	{
		return false;
	}
	if(Polygon.NumUVs != Triangle.NumUVs)
	{
		return false;
	}
	if(!Polygon.bOverrideTangentBasis && Polygon.SmoothingMask != Triangle.SmoothingMask)
	{
		return false;
	}

	// The triangle must have the same normal as the polygon
	if(!Triangle.FaceNormal.Equals(Polygon.FaceNormal,UE_THRESH_NORMALS_ARE_SAME))
	{
		return false;
	}

	// The triangle must have the same attribute gradients as the polygon
	if(!Triangle.TangentXGradient.Equals(Polygon.TangentXGradient))
	{
		return false;
	}
	if(!Triangle.TangentYGradient.Equals(Polygon.TangentYGradient))
	{
		return false;
	}
	if(!Triangle.TangentZGradient.Equals(Polygon.TangentZGradient))
	{
		return false;
	}
	if(!Triangle.ColorGradient.Equals(Polygon.ColorGradient))
	{
		return false;
	}
	for(int32 UVIndex = 0;UVIndex < Triangle.NumUVs;UVIndex++)
	{
		if(!Triangle.UVGradient[UVIndex].Equals(Polygon.UVGradient[UVIndex]))
		{
			return false;
		}
	}

	for(int32 PolygonEdgeIndex = 0;PolygonEdgeIndex < Polygon.Vertices.Num();PolygonEdgeIndex++)
	{
		const uint32 PolygonEdgeVertex0 = PolygonEdgeIndex + 0;
		const uint32 PolygonEdgeVertex1 = (PolygonEdgeIndex + 1) % Polygon.Vertices.Num();

		for(uint32 TriangleEdgeIndex = 0;TriangleEdgeIndex < 3;TriangleEdgeIndex++)
		{
			const uint32 TriangleEdgeVertex0 = TriangleEdgeIndex + 0;
			const uint32 TriangleEdgeVertex1 = (TriangleEdgeIndex + 1) % 3;

			// If the triangle and polygon share an edge, then the triangle is in the same plane (implied by the above normal check),
			// and may be merged into the polygon.
			if(	VertsAreEqual(Polygon.Vertices[PolygonEdgeVertex0],Triangle.Vertices[TriangleEdgeVertex1]) &&
				VertsAreEqual(Polygon.Vertices[PolygonEdgeVertex1],Triangle.Vertices[TriangleEdgeVertex0]))
			{
				// Add the triangle's vertex that isn't in the adjacent edge to the polygon in between the vertices of the adjacent edge.
				const int32 TriangleOppositeVertexIndex = (TriangleEdgeIndex + 2) % 3;
				Polygon.Vertices.Insert(Triangle.Vertices[TriangleOppositeVertexIndex],PolygonEdgeVertex1);

				return true;
			}
		}
	}

	// Could not merge triangles.
	return false;
}

/** Given a set of triangles, remove those which share an edge and could be collapsed into one triangle. */
void FGeomTools::RemoveRedundantTriangles(TArray<FClipSMTriangle>& Tris)
{
	TArray<FClipSMPolygon> Polygons;

	// Merge the triangles into polygons.
	while(Tris.Num() > 0)
	{
		// Start building a polygon from the last triangle in the array.
		const FClipSMTriangle InitialTriangle = Tris.Pop();
		FClipSMPolygon MergedPolygon(0);
		MergedPolygon.CopyFace(InitialTriangle);
		MergedPolygon.Vertices.Add(InitialTriangle.Vertices[0]);
		MergedPolygon.Vertices.Add(InitialTriangle.Vertices[1]);
		MergedPolygon.Vertices.Add(InitialTriangle.Vertices[2]);

		// Find triangles that can be merged into the polygon.
		for(int32 CandidateTriangleIndex = 0;CandidateTriangleIndex < Tris.Num();CandidateTriangleIndex++)
		{
			const FClipSMTriangle& MergeCandidateTriangle = Tris[CandidateTriangleIndex];
			if(MergeTriangleIntoPolygon(MergedPolygon, MergeCandidateTriangle))
			{
				// Remove the merged triangle from the array.
				Tris.RemoveAtSwap(CandidateTriangleIndex);

				// Restart the search for mergeable triangles from the start of the array.
				CandidateTriangleIndex = -1;
			}
		}

		// Add the merged polygon to the array.
		Polygons.Add(MergedPolygon);
	}

	// Triangulate each polygon and add it to the output triangle array.
	for(int32 PolygonIndex = 0;PolygonIndex < Polygons.Num();PolygonIndex++)
	{
		TArray<FClipSMTriangle> Triangles;
		TriangulatePoly(Triangles,Polygons[PolygonIndex]);
		Tris.Append(Triangles);
	}
}

/** Util class for clipping polygon to a half space in 2D */
class FSplitLine2D
{
private:
	float X, Y, W;

public:
	FSplitLine2D() 
	{}

	FSplitLine2D( const FVector2D& InBase, const FVector2D &InNormal )
	{
		X = InNormal.X;
		Y = InNormal.Y;
		W = (InBase | InNormal);
	}


	float PlaneDot( const FVector2D &P ) const
	{
		return X*P.X + Y*P.Y - W;
	}
};

/** Split 2D polygons with a 3D plane. */
void FGeomTools::Split2DPolysWithPlane(FUtilPoly2DSet& PolySet, const FPlane& Plane, const FColor& ExteriorVertColor, const FColor& InteriorVertColor)
{
	// Break down world-space plane into normal and base
	FVector WNormal =  FVector(Plane.X, Plane.Y, Plane.Z);
	FVector WBase = WNormal * Plane.W;

	// Convert other plane into local space
	FVector LNormal = PolySet.PolyToWorld.InverseTransformVector(WNormal);

	// If planes are parallel, see if it clips away everything
	if(FMath::Abs(LNormal.Z) > (1.f - 0.001f))
	{
		// Check distance of this plane from the other
		float Dist = Plane.PlaneDot(PolySet.PolyToWorld.GetOrigin());
		// Its in front - remove all polys
		if(Dist > 0.f)
		{
			PolySet.Polys.Empty();
		}

		return;
	}

	FVector LBase = PolySet.PolyToWorld.InverseTransformPosition(WBase);

	// Project 0-plane normal into other plane - we will trace along this line to find intersection of two planes.
	FVector NormInOtherPlane = FVector(0,0,1) - (LNormal * (FVector(0,0,1) | LNormal));

	// Find direction of plane-plane intersect line
	//FVector LineDir = LNormal ^ FVector(0,0,1);
	// Cross this with other plane normal to find vector in other plane which will intersect this plane (0-plane)
	//FVector V = LineDir ^ LNormal;

	// Find second point along vector
	FVector VEnd = LBase - (10.f * NormInOtherPlane);
	// Find intersection
	FVector Intersect = FMath::LinePlaneIntersection(LBase, VEnd, FVector::ZeroVector, FVector(0,0,1));
	check(!Intersect.ContainsNaN());

	// Make 2D line.
	FVector2D Normal2D = FVector2D(LNormal.X, LNormal.Y).GetSafeNormal();
	FVector2D Base2D = FVector2D(Intersect.X, Intersect.Y);
	FSplitLine2D Plane2D(Base2D, Normal2D);

	for(int32 PolyIndex=PolySet.Polys.Num()-1; PolyIndex>=0; PolyIndex--)
	{
		FUtilPoly2D& Poly = PolySet.Polys[PolyIndex];
		int32 NumPVerts = Poly.Verts.Num();

		// Calculate distance of verts from clipping line
		TArray<float> PlaneDist;
		PlaneDist.AddZeroed(NumPVerts);
		for(int32 i=0; i<NumPVerts; i++)
		{
			PlaneDist[i] = Plane2D.PlaneDot(Poly.Verts[i].Pos);
		}

		TArray<FUtilVertex2D> FinalVerts;
		for(int32 ThisVert=0; ThisVert<NumPVerts; ThisVert++)
		{
			bool bStartInside = (PlaneDist[ThisVert] < 0.f);
			// If start vert is inside, add it.
			if(bStartInside)
			{
				FinalVerts.Add( Poly.Verts[ThisVert] );
			}

			// If start and next vert are on opposite sides, add intersection
			int32 NextVert = (ThisVert+1)%NumPVerts;

			if(PlaneDist[ThisVert] * PlaneDist[NextVert] < 0.f)
			{
				// Find distance along edge that plane is
				float Alpha = -PlaneDist[ThisVert] / (PlaneDist[NextVert] - PlaneDist[ThisVert]);
				FVector2D NewVertPos = FMath::Lerp(Poly.Verts[ThisVert].Pos, Poly.Verts[NextVert].Pos, Alpha);

				// New color based on whether we are cutting an 'interior' edge.
				FColor NewVertColor = (Poly.Verts[ThisVert].bInteriorEdge) ? InteriorVertColor : ExteriorVertColor;

				FUtilVertex2D NewVert = FUtilVertex2D(NewVertPos, NewVertColor);

				// We mark this the start of an interior edge if the edge we cut started inside.
				if(bStartInside || Poly.Verts[ThisVert].bInteriorEdge)
				{
					NewVert.bInteriorEdge = true;
				}

				// Save vert
				FinalVerts.Add(NewVert);
			}
		}

		// If we have no verts left, all clipped away, remove from set of polys
		if(FinalVerts.Num() == 0)
		{
			PolySet.Polys.RemoveAt(PolyIndex);
		}
		// Copy new set of verts back to poly.
		else
		{
			Poly.Verts = FinalVerts;
		}
	}
}



bool FGeomTools2D::IsPolygonWindingCCW(const TArray<FVector2D>& Points)
{
	float Sum = 0.0f;
	const int PointCount = Points.Num();
	for (int PointIndex = 0; PointIndex < PointCount; ++PointIndex)
	{
		const FVector2D& A = Points[PointIndex];
		const FVector2D& B = Points[(PointIndex + 1) % PointCount];
		Sum += (B.X - A.X) * (B.Y + A.Y);
	}
	return (Sum < 0.0f);
}

bool FGeomTools2D::IsPolygonWindingCCW(const TArray<FIntPoint>& Points)
{
	int32 Sum = 0;
	const int PointCount = Points.Num();
	for (int PointIndex = 0; PointIndex < PointCount; ++PointIndex)
	{
		const FIntPoint& A = Points[PointIndex];
		const FIntPoint& B = Points[(PointIndex + 1) % PointCount];
		Sum += (B.X - A.X) * (B.Y + A.Y);
	}
	return (Sum < 0);
}

// Note: We need to simplify non-simple polygons before this
static bool IsPolygonConvex(const TArray<FVector2D>& Points)
{
	const int PointCount = Points.Num();
	float Sign = 0;
	for (int32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
	{
		const FVector2D& A = Points[PointIndex];
		const FVector2D& B = Points[(PointIndex + 1) % PointCount];
		const FVector2D& C = Points[(PointIndex + 2) % PointCount];
		float Det = (B.X - A.X) * (C.Y - B.Y) - (B.Y - A.Y) * (C.X - B.X);
		float DetSign = FMath::Sign(Det);
		if (DetSign != 0)
		{
			if (Sign == 0)
			{
				Sign = DetSign;
			}
			else if (Sign != DetSign)
			{
				return false;
			}
		}
	}

	return true;
}


static bool IsAdditivePointInTriangle(const FVector2D& TestPoint, const FVector2D& A, const FVector2D& B, const FVector2D& C)
{
	const FVector2D AP = TestPoint - A;
	const FVector2D BP = TestPoint - B;
	const FVector2D AB = B - A;
	const FVector2D AC = C - A;
	const FVector2D BC = C - B;
	if (FVector2D::CrossProduct(AB, AP) <= 0.0f) return false;
	if (FVector2D::CrossProduct(AP, AC) <= 0.0f) return false;
	if (FVector2D::CrossProduct(BC, BP) <= 0.0f) return false;
	if (AP.SizeSquared() < 2.0f) return false;
	if (BP.SizeSquared() < 2.0f) return false;
	const FVector2D CP = TestPoint - C;
	if (CP.SizeSquared() < 2.0f) return false;
	return true;
}

static float VectorSign(const FVector2D& Vec, const FVector2D& A, const FVector2D& B)
{
	return FMath::Sign((B.X - A.X) * (Vec.Y - A.Y) - (B.Y - A.Y) * (Vec.X - A.X));
}

// Returns true when the point is inside the triangle
// Should not return true when the point is on one of the edges
static bool IsPointInTriangle(const FVector2D& TestPoint, const FVector2D& A, const FVector2D& B, const FVector2D& C)
{
	float BA = VectorSign(B, A, TestPoint);
	float CB = VectorSign(C, B, TestPoint);
	float AC = VectorSign(A, C, TestPoint);

	// point is in the same direction of all 3 tri edge lines
	// must be inside, regardless of tri winding
	return BA == CB && CB == AC;
}

// Returns true when the point is on the line segment limited by A and B
static bool IsPointOnLineSegment(const FVector2D& TestPoint, const FVector2D& A, const FVector2D& B)
{
	FVector2D BA = B - A;
	FVector2D PA = TestPoint - A;
	double SizeSquaredBA = FVector2D::DotProduct(BA, BA);
	double AreaCompareThreshold = UE_DOUBLE_KINDA_SMALL_NUMBER * SizeSquaredBA;
	double ParallelogramArea = BA.X * PA.Y - BA.Y * PA.X;

	return  TestPoint.X >= FMath::Min(A.X, B.X) && TestPoint.X <= FMath::Max(A.X, B.X) && // X within AB.X, including ON A or B
		TestPoint.Y >= FMath::Min(A.Y, B.Y) && TestPoint.Y <= FMath::Max(A.Y, B.Y) && // Y within AB.Y, including ON A or B
		FMath::Abs(ParallelogramArea) < AreaCompareThreshold; // Area is smaller than allowed epsilon = point on line
}

static void JoinSubtractiveToAdditive(TArray<FVector2D>& AdditivePoly, const TArray<FVector2D>& SubtractivePoly, const int AdditiveJoinIndex, const int SubtractiveJoinIndex)
{
	TArray<FVector2D> NewAdditivePoly;
	for (int AdditiveIndex = 0; AdditiveIndex < AdditivePoly.Num(); ++AdditiveIndex)
	{
		NewAdditivePoly.Add(AdditivePoly[AdditiveIndex]);
		if (AdditiveIndex == AdditiveJoinIndex)
		{
			for (int SubtractiveIndex = SubtractiveJoinIndex; SubtractiveIndex < SubtractivePoly.Num(); ++SubtractiveIndex)
			{
				NewAdditivePoly.Add(SubtractivePoly[SubtractiveIndex]);
			}
			for (int SubtractiveIndex = 0; SubtractiveIndex <= SubtractiveJoinIndex; ++SubtractiveIndex)
			{
				NewAdditivePoly.Add(SubtractivePoly[SubtractiveIndex]);
			}
			NewAdditivePoly.Add(AdditivePoly[AdditiveIndex]);
		}
	}
	AdditivePoly = NewAdditivePoly;
}


static void JoinMutuallyVisible(TArray<FVector2D>& AdditivePoly, const TArray<FVector2D>& SubtractivePoly)
{
	const int NumAdditivePoly = AdditivePoly.Num();
	const int NumSubtractivePoly = SubtractivePoly.Num();
	if (NumSubtractivePoly == 0)
	{
		return;
	}

	// Search the inner (subtractive) polygon for the point of maximum x-value
	int IndexMaxX = 0;
	{
		float MaxX = SubtractivePoly[0].X;
		for (int Index = 1; Index < NumSubtractivePoly; ++Index)
		{
			if (SubtractivePoly[Index].X > MaxX)
			{
				MaxX = SubtractivePoly[Index].X;
				IndexMaxX = Index;
			}
		}
	}
	const FVector2D PointMaxX = SubtractivePoly[IndexMaxX];

	// Intersect a ray from point M facing to the right (a, ab) with the additive shape edges (c, d)
	// Find the closest intersecting point to M (left-most point)
	int EdgeStartPointIndex = 0;
	int EdgeEndPointIndex = 0;
	bool bIntersectedAtVertex = false;
	float LeftMostIntersectX = UE_MAX_FLT;
	const FVector2D A = PointMaxX;
	const FVector2D AB = FVector2D(1.0f, 0.0f);
	for (int AdditiveIndex = 0; AdditiveIndex < NumAdditivePoly; ++AdditiveIndex)
	{
		const FVector2D& C = AdditivePoly[AdditiveIndex];
		const FVector2D& D = AdditivePoly[(AdditiveIndex + 1) % NumAdditivePoly];
		const FVector2D CD = D - C;
		// Only check edges from the inside, as edges will overlap after mutually visible points are merged.
		if (CD.Y > 0.0f)
		{
			const float DetS = AB.X * CD.Y - AB.Y * CD.X;
			const float DetT = CD.X * AB.Y - CD.Y * AB.X;
			if (DetS != 0.0f && DetT != 0.0f)
			{
				const float S = (A.Y * CD.X - C.Y * CD.X - A.X * CD.Y + C.X * CD.Y) / DetS;
				const float T = (C.Y * AB.X - A.Y * AB.X - C.X * AB.Y + A.X * AB.Y) / DetT;
				if (S >= 0.0f && T >= 0.0f && T <= 1.0f)
				{
					const float IntersectX = A.X + AB.X * S;
					if (IntersectX < LeftMostIntersectX)
					{
						LeftMostIntersectX = IntersectX;
						EdgeStartPointIndex = AdditiveIndex;
						EdgeEndPointIndex = (AdditiveIndex + 1) % NumAdditivePoly;
						if (T < FLT_EPSILON)
						{
							bIntersectedAtVertex = true;
						}
						else if (T > 1.0f - FLT_EPSILON)
						{
							bIntersectedAtVertex = true;
							EdgeStartPointIndex = EdgeEndPointIndex;
						}
					}
				}
			}
		}
	}

	// If the ray intersected a vertex, points are mutually visible
	if (bIntersectedAtVertex)
	{
		JoinSubtractiveToAdditive(AdditivePoly, SubtractivePoly, EdgeStartPointIndex, IndexMaxX);
		return;
	}

	// Otherwise, set P to be the edge endpoint with maximum x value
	const FVector2D Intersect(LeftMostIntersectX, PointMaxX.Y);
	const int IndexP = (AdditivePoly[EdgeStartPointIndex].X > AdditivePoly[EdgeEndPointIndex].X) ? EdgeStartPointIndex : EdgeEndPointIndex;
	const FVector2D P = AdditivePoly[IndexP];

	// Search the vertices of the additive shape. If all of these are outside the triangle (M, intersect, P) then M and P are mutually visible
	// If any vertices lie inside the triangle, find the one that minimizes the angle between (1, 0) and the line M R. (If multiple vertices minimize the angle, choose closest to M)
	const FVector2D TriA = PointMaxX;
	const FVector2D TriB = (P.Y < Intersect.Y) ? P : Intersect;
	const FVector2D TriC = (P.Y < Intersect.Y) ? Intersect : P;
	float CosAngleMax = 0.0f;
	float DistanceMin = UE_MAX_FLT;
	int IndexR = -1;
	for (int AdditiveIndex = 0; AdditiveIndex < NumAdditivePoly; ++AdditiveIndex)
	{
		// Ignore point P
		if (AdditiveIndex == IndexP)
		{
			continue;
		}

		if (IsAdditivePointInTriangle(AdditivePoly[AdditiveIndex], TriA, TriB, TriC))
		{
			const FVector2D MR = AdditivePoly[AdditiveIndex] - PointMaxX;
			const float CosAngle = MR.X / MR.Size();
			const float DSq = MR.SizeSquared();
			if (CosAngle > CosAngleMax || (CosAngle == CosAngleMax && DSq < DistanceMin))
			{
				CosAngleMax = CosAngle;
				DistanceMin = DSq;
				IndexR = AdditiveIndex;
			}
		}
	}

	JoinSubtractiveToAdditive(AdditivePoly, SubtractivePoly, (IndexR == -1) ? IndexP : IndexR, IndexMaxX);
}

/** Util that tries to combine two triangles if possible. */
// Unlike the implementation in GeomTools, we only care about 2D points and not any other vertex attributes
// If bConvex == true, the triangles are only merged if the resulting polygon is convex
static bool MergeTriangleIntoPolygon(
	TArray<FVector2D>& PolygonVertices,
	const FVector2D& TriangleVertexA, const FVector2D& TriangleVertexB, const FVector2D& TriangleVertexC, bool bConvex = false)
{
	// Create indexable copy
	FVector2D TriangleVertices[3] = { TriangleVertexA, TriangleVertexB, TriangleVertexC };

	for (int32 PolygonEdgeIndex = 0; PolygonEdgeIndex < PolygonVertices.Num(); PolygonEdgeIndex++)
	{
		const uint32 PolygonEdgeVertex0 = PolygonEdgeIndex + 0;
		const uint32 PolygonEdgeVertex1 = (PolygonEdgeIndex + 1) % PolygonVertices.Num();

		for (uint32 TriangleEdgeIndex = 0; TriangleEdgeIndex < 3; TriangleEdgeIndex++)
		{
			const uint32 TriangleEdgeVertex0 = TriangleEdgeIndex + 0;
			const uint32 TriangleEdgeVertex1 = (TriangleEdgeIndex + 1) % 3;

			// If the triangle and polygon share an edge, then the triangle is in the same plane (implied by the above normal check),
			// and may be merged into the polygon.
			if (PolygonVertices[PolygonEdgeVertex0].Equals(TriangleVertices[TriangleEdgeVertex1], UE_THRESH_POINTS_ARE_SAME) &&
				PolygonVertices[PolygonEdgeVertex1].Equals(TriangleVertices[TriangleEdgeVertex0], UE_THRESH_POINTS_ARE_SAME))
			{
				bool bMergeTriangle = true;
				if (bConvex)
				{
					TArray<FVector2D> TmpPolygonVertcies = PolygonVertices;
					const int32 TriangleOppositeVertexIndex = (TriangleEdgeIndex + 2) % 3;
					TmpPolygonVertcies.Insert(TriangleVertices[TriangleOppositeVertexIndex], PolygonEdgeVertex1);
					bMergeTriangle = IsPolygonConvex(TmpPolygonVertcies);
				}

				if (bMergeTriangle)
				{
					// Add the triangle's vertex that isn't in the adjacent edge to the polygon in between the vertices of the adjacent edge.
					const int32 TriangleOppositeVertexIndex = (TriangleEdgeIndex + 2) % 3;
					PolygonVertices.Insert(TriangleVertices[TriangleOppositeVertexIndex], PolygonEdgeVertex1);

					return true;
				}
			}
		}
	}

	// Could not merge triangles.
	return false;
}


TArray<TArray<FVector2D>> FGeomTools2D::ReducePolygons(const TArray<TArray<FVector2D>>& Polygons, const TArray<bool>& PolygonNegativeWinding)
{
	TArray<TArray<FVector2D>> ReturnPolygons;

	int NumPolygons = Polygons.Num();

	TArray<float> MaxXValues; // per polygon
	for (int PolyIndex = 0; PolyIndex < NumPolygons; ++PolyIndex)
	{
		float MaxX = -UE_BIG_NUMBER;
		const TArray<FVector2D>& Vertices = Polygons[PolyIndex];
		for (int VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
		{
			MaxX = FMath::Max(Vertices[VertexIndex].X, MaxX);
		}
		MaxXValues.Add(MaxX);
	}

	// Iterate through additive shapes
	for (int PolyIndex = 0; PolyIndex < NumPolygons; ++PolyIndex)
	{
		if (!PolygonNegativeWinding[PolyIndex])
		{
			TArray<FVector2D> Verts(Polygons[PolyIndex]);

			// Store indexes of subtractive shapes that lie inside the additive shape
			TArray<int> SubtractiveShapeIndices;
			for (int J = 0; J < NumPolygons; ++J)
			{
				const TArray<FVector2D>& CandidateShape = Polygons[J];
				if (PolygonNegativeWinding[J])
				{
					if (IsPointInPolygon(CandidateShape[0], Verts))
					{
						SubtractiveShapeIndices.Add(J);
					}
				}
			}

			// Remove subtractive shapes that lie inside the subtractive shapes we've found
			for (int J = 0; J < SubtractiveShapeIndices.Num();)
			{
				const TArray<FVector2D>& ourShape = Polygons[SubtractiveShapeIndices[J]];
				bool bRemoveOurShape = false;
				for (int K = 0; K < SubtractiveShapeIndices.Num(); ++K)
				{
					if (J == K)
					{
						continue;
					}
					if (IsPointInPolygon(ourShape[0], Polygons[SubtractiveShapeIndices[K]]))
					{
						bRemoveOurShape = true;
						break;
					}
				}
				if (bRemoveOurShape)
				{
					SubtractiveShapeIndices.RemoveAt(J);
				}
				else
				{
					++J;
				}
			}

			// Sort subtractive shapes from right to left by their points' maximum x value
			const int NumSubtractiveShapes = SubtractiveShapeIndices.Num();
			for (int J = 0; J < NumSubtractiveShapes; ++J)
			{
				for (int K = J + 1; K < NumSubtractiveShapes; ++K)
				{
					if (MaxXValues[SubtractiveShapeIndices[J]] < MaxXValues[SubtractiveShapeIndices[K]])
					{
						int tempIndex = SubtractiveShapeIndices[J];
						SubtractiveShapeIndices[J] = SubtractiveShapeIndices[K];
						SubtractiveShapeIndices[K] = tempIndex;
					}
				}
			}

			for (int SubtractiveIndex = 0; SubtractiveIndex < NumSubtractiveShapes; ++SubtractiveIndex)
			{
				JoinMutuallyVisible(Verts, Polygons[SubtractiveShapeIndices[SubtractiveIndex]]);
			}

			// Add new hole-less polygon to our output shapes
			ReturnPolygons.Add(Verts);
		}
	}

	return ReturnPolygons;
}


void FGeomTools2D::CorrectPolygonWinding(TArray<FVector2D>& OutVertices, const TArray<FVector2D>& Vertices, const bool bNegativeWinding)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomTools2D::CorrectPolygonWinding);
	if (Vertices.Num() >= 3)
	{
		// Make sure the polygon winding is correct
		if ((!bNegativeWinding && !IsPolygonWindingCCW(Vertices)) ||
			(bNegativeWinding && IsPolygonWindingCCW(Vertices)))
		{
			// Reverse vertices
			for (int32 VertexIndex = Vertices.Num() - 1; VertexIndex >= 0; --VertexIndex)
			{
				new (OutVertices) FVector2D(Vertices[VertexIndex]);
			}
		}
		else
		{
			// Copy vertices
			for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
			{
				new (OutVertices) FVector2D(Vertices[VertexIndex]);
			}
		}
	}
}

// Assumes polygons are valid, closed and the winding is correct and matches bWindingCCW
bool FGeomTools2D::ArePolygonsValid(const TArray<TArray<FVector2D>>& Polygons)
{
	const int PolygonCount = Polygons.Num();

	// Find every pair of shapes that overlap
	for (int PolygonIndexA = 0; PolygonIndexA < PolygonCount; ++PolygonIndexA)
	{
		const TArray<FVector2D>& PolygonA = Polygons[PolygonIndexA];
		const bool bIsWindingA_CCW = IsPolygonWindingCCW(PolygonA);
		for (int PolygonIndexB = PolygonIndexA + 1; PolygonIndexB < PolygonCount; ++PolygonIndexB)
		{
			const TArray<FVector2D>& PolygonB = Polygons[PolygonIndexB];
			const bool bIsWindingB_CCW = IsPolygonWindingCCW(PolygonB);

			// Additive polygons are allowed to intersect
			if (bIsWindingA_CCW && bIsWindingB_CCW)
			{
				continue;
			}

			// Test each shapeA edge against each shapeB edge for intersection
			for (int VertexA = 0; VertexA < PolygonA.Num(); ++VertexA)
			{
				const FVector2D& A0 = PolygonA[VertexA];
				const FVector2D& A1 = PolygonA[(VertexA + 1) % PolygonA.Num()];
				const FVector2D A10 = A1 - A0;

				for (int VertexB = 0; VertexB < PolygonB.Num(); ++VertexB)
				{
					const FVector2D B0 = PolygonB[VertexB];
					const FVector2D B1 = PolygonB[(VertexB + 1) % PolygonB.Num()];
					const FVector2D B10 = B1 - B0;

					const float DetS = A10.X * B10.Y - A10.Y * B10.X;
					const float DetT = B10.X * A10.Y - B10.Y * A10.X;
					if (DetS != 0.0f && DetT != 0.0f)
					{
						const float S = (A0.Y * B10.X - B0.Y * B10.X - A0.X * B10.Y + B0.X * B10.Y) / DetS;
						const float T = (B0.Y * A10.X - A0.Y * A10.X - B0.X * A10.Y + A0.X * A10.Y) / DetT;
						if (S >= 0.0f && S <= 1.0f && T >= 0.0f && T <= 1.0f)
						{
							// Edges intersect
							UE_LOGF(LogGeomTools, Log, "Edges in polygon %d and %d intersect", PolygonIndexA, PolygonIndexB);
							return false;
						}
					}
				}
			}
		}
	}

	// Make sure no polygons are self-intersecting
	// Disabled for now until we decide what to do with this - contour tracing can generate invalid polys & degenerate edges
	//for (int PolygonIndex = 0; PolygonIndex < PolygonCount; ++PolygonIndex)
	//{
	//	const TArray<FVector2D>& Polygon = Polygons[PolygonIndex].Vertices;
	//	const int PointCount = Polygon.Num();
	//	for (int PointIndexA = 0; PointIndexA < PointCount; ++PointIndexA)
	//	{
	//		const FVector2D& A0 = Polygon[PointIndexA];
	//		const FVector2D& A1 = Polygon[(PointIndexA + 1) % PointCount];
	//		const FVector2D A10 = A1 - A0;
	//		for (int PointIndexB = 0; PointIndexB < PointCount - 3; ++PointIndexB)
	//		{
	//			const FVector2D& B0 = Polygon[(PointIndexA + 2 + PointIndexB) % PointCount];
	//			const FVector2D& B1 = Polygon[(PointIndexA + 3 + PointIndexB) % PointCount];
	//			const FVector2D B10 = B1 - B0;

	//			const float DetS = A10.X * B10.Y - A10.Y * B10.X;
	//			const float DetT = B10.X * A10.Y - B10.Y * A10.X;
	//			if (DetS != 0.0f && DetT != 0.0f)
	//			{
	//				const float S = (A0.Y * B10.X - B0.Y * B10.X - A0.X * B10.Y + B0.X * B10.Y) / DetS;
	//				const float T = (B0.Y * A10.X - A0.Y * A10.X - B0.X * A10.Y + A0.X * A10.Y) / DetT;
	//				if (S >= 0.0f && S <= 1.0f && T >= 0.0f && T <= 1.0f)
	//				{
	//					// Edges intersect
	//					UE_LOGF(LogGeomTools, Log, "Polygon %d is self intersecting", PolygonIndex);
	//					return false;
	//				}
	//			}
	//		}
	//	}
	//}

	return true;
}

// Determines whether two edges may be merged.
// Very similar to the one in GeomTools.cpp, but only considers positions (as opposed to other vertex attributes)
static bool AreEdgesMergeable(const FVector2D& V0, const FVector2D& V1, const FVector2D& V2)
{
	const FVector2D MergedEdgeVector = V2 - V0;
	const float MergedEdgeLengthSquared = MergedEdgeVector.SizeSquared();
	if (MergedEdgeLengthSquared > UE_DELTA)
	{
		// Find the vertex closest to A1/B0 that is on the hypothetical merged edge formed by A0-B1.
		const float IntermediateVertexEdgeFraction = ((V2 - V0) | (V1 - V0)) / MergedEdgeLengthSquared;
		const FVector2D InterpolatedVertex = V0 + (V2 - V0) * IntermediateVertexEdgeFraction;

		// The edges are merge-able if the interpolated vertex is close enough to the intermediate vertex.
		return InterpolatedVertex.Equals(V1, UE_THRESH_POINTS_ARE_SAME);
	}
	else
	{
		return true;
	}
}

// Nearly identical function in GeomTools.cpp - Identical points (exact same fp values, not nearlyEq) are not considered to be inside 
// the triangle when ear clipping. These are generated by ReducePolygons when additive and subtractive polygons are merged.
// Expected input - PolygonVertices in CCW order, not overlapping
bool FGeomTools2D::TriangulatePoly(TArray<FVector2D>& OutTris, const TArray<FVector2D>& InPolyVerts, bool bKeepColinearVertices)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomTools2D::TriangulatePoly);
	// Can't work if not enough verts for 1 triangle
	if (InPolyVerts.Num() < 3)
	{
		// Return true because poly is already a tri
		return true;
	}

	// Vertices of polygon in order - make a copy we are going to modify.
	TArray<FVector2D> PolyVerts = InPolyVerts;

	// Keep iterating while there are still vertices
	while (true)
	{
		if (!bKeepColinearVertices)
		{
			// Cull redundant vertex edges from the polygon.
			for (int32 VertexIndex = 0; VertexIndex < PolyVerts.Num(); VertexIndex++)
			{
				const int32 I0 = (VertexIndex + 0) % PolyVerts.Num();
				const int32 I1 = (VertexIndex + 1) % PolyVerts.Num();
				const int32 I2 = (VertexIndex + 2) % PolyVerts.Num();
				if (AreEdgesMergeable(PolyVerts[I0], PolyVerts[I1], PolyVerts[I2]))
				{
					PolyVerts.RemoveAt(I1);
					VertexIndex--;
				}
			}
		}

		if (PolyVerts.Num() < 3)
		{
			break;
		}
		else
		{
			// Look for an 'ear' triangle
			bool bFoundEar = false;
			for (int32 EarVertexIndex = 0; EarVertexIndex < PolyVerts.Num(); EarVertexIndex++)
			{
				// Triangle is 'this' vert plus the one before and after it
				const int32 AIndex = (EarVertexIndex == 0) ? PolyVerts.Num() - 1 : EarVertexIndex - 1;
				const int32 BIndex = EarVertexIndex;
				const int32 CIndex = (EarVertexIndex + 1) % PolyVerts.Num();

				// Check that this vertex is convex (cross product must be positive)
				const FVector2D ABEdge = PolyVerts[BIndex] - PolyVerts[AIndex];
				const FVector2D ACEdge = PolyVerts[CIndex] - PolyVerts[AIndex];
				if ((ABEdge ^ ACEdge) < 0.0f)
				{
					continue;
				}

				bool bFoundVertInside = false;
				// Look through all verts before this in array to see if any are inside triangle
				for (int32 VertexIndex = 0; VertexIndex < PolyVerts.Num(); VertexIndex++)
				{
					const FVector2D& CurrentVertex = PolyVerts[VertexIndex];
					// Test indices first, then make sure we arent comparing identical points
					// These might have been added by the convex / concave splitting
					// If a point is not in the triangle, it may be on the new edge we're adding, which isn't allowed as 
					// it will create a partition in the polygon
					if (VertexIndex != AIndex && VertexIndex != BIndex && VertexIndex != CIndex &&
						CurrentVertex != PolyVerts[AIndex] && CurrentVertex != PolyVerts[BIndex] && CurrentVertex != PolyVerts[CIndex] &&
						(IsPointInTriangle(CurrentVertex, PolyVerts[AIndex], PolyVerts[BIndex], PolyVerts[CIndex]) || IsPointOnLineSegment(CurrentVertex, PolyVerts[CIndex], PolyVerts[AIndex])))
					{
						bFoundVertInside = true;
						break;
					}
				}

				// Triangle with no verts inside - its an 'ear'! 
				if (!bFoundVertInside)
				{
					// Add to output list..
					OutTris.Add(PolyVerts[AIndex]);
					OutTris.Add(PolyVerts[BIndex]);
					OutTris.Add(PolyVerts[CIndex]);

					// And remove vertex from polygon
					PolyVerts.RemoveAt(EarVertexIndex);

					bFoundEar = true;
					break;
				}
			}

			// If we couldn't find an 'ear' it indicates something is bad with this polygon - discard triangles and return.
			if (!bFoundEar)
			{
				// If we couldn't find an 'ear' it indicates something is bad with this polygon - discard triangles and return.
				UE_LOGF(LogGeomTools, Log, "Triangulation of poly failed.");
				OutTris.Empty();
				return false;
			}
		}
	}

	return true;
}

// 2D version of GeomTools RemoveRedundantTriangles
void FGeomTools2D::RemoveRedundantTriangles(TArray<FVector2D>& OutTriangles, const TArray<FVector2D>& InTriangleVertices)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomTools2D::RemoveRedundantTriangles);
	struct FLocalTriangle
	{
		int VertexA, VertexB, VertexC;
	};

	TArray<FLocalTriangle> Triangles;
	for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < InTriangleVertices.Num(); TriangleVertexIndex += 3)
	{
		FLocalTriangle* NewTriangle = new (Triangles) FLocalTriangle();
		NewTriangle->VertexA = TriangleVertexIndex + 0;
		NewTriangle->VertexB = TriangleVertexIndex + 1;
		NewTriangle->VertexC = TriangleVertexIndex + 2;
	}

	while (Triangles.Num() > 0)
	{
		TArray<FVector2D> PolygonVertices;

		const FLocalTriangle InitialTriangle = Triangles.Pop(EAllowShrinking::No);
		PolygonVertices.Add(InTriangleVertices[InitialTriangle.VertexA]);
		PolygonVertices.Add(InTriangleVertices[InitialTriangle.VertexB]);
		PolygonVertices.Add(InTriangleVertices[InitialTriangle.VertexC]);

		// Find triangles that can be merged into the polygon.
		for (int32 CandidateTriangleIndex = 0; CandidateTriangleIndex < Triangles.Num(); ++CandidateTriangleIndex)
		{
			const FLocalTriangle& MergeCandidateTriangle = Triangles[CandidateTriangleIndex];
			if (MergeTriangleIntoPolygon(PolygonVertices, InTriangleVertices[MergeCandidateTriangle.VertexA], InTriangleVertices[MergeCandidateTriangle.VertexB], InTriangleVertices[MergeCandidateTriangle.VertexC]))
			{
				// Remove the merged triangle from the array.
				Triangles.RemoveAtSwap(CandidateTriangleIndex);

				// Restart the search for mergeable triangles from the start of the array.
				CandidateTriangleIndex = -1;
			}
		}

		// Triangulate merged polygon and append to triangle array
		TArray<FVector2D> TriangulatedPoly;
		TriangulatePoly(/*out*/TriangulatedPoly, PolygonVertices);
		OutTriangles.Append(TriangulatedPoly);
	}
}

// Find convex polygons from triangle soup
void FGeomTools2D::GenerateConvexPolygonsFromTriangles(TArray<TArray<FVector2D>>& OutPolygons, const TArray<FVector2D>& InTriangleVertices)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomTools2D::GenerateConvexPolygonsFromTriangles);
	struct FLocalTriangle
	{
		int VertexA, VertexB, VertexC;
	};

	TArray<FLocalTriangle> Triangles;
	for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < InTriangleVertices.Num(); TriangleVertexIndex += 3)
	{
		FLocalTriangle* NewTriangle = new (Triangles) FLocalTriangle();
		NewTriangle->VertexA = TriangleVertexIndex + 0;
		NewTriangle->VertexB = TriangleVertexIndex + 1;
		NewTriangle->VertexC = TriangleVertexIndex + 2;
	}

	while (Triangles.Num() > 0)
	{
		TArray<FVector2D> PolygonVertices;

		const FLocalTriangle InitialTriangle = Triangles.Pop(EAllowShrinking::No);
		PolygonVertices.Add(InTriangleVertices[InitialTriangle.VertexA]);
		PolygonVertices.Add(InTriangleVertices[InitialTriangle.VertexB]);
		PolygonVertices.Add(InTriangleVertices[InitialTriangle.VertexC]);

		// Find triangles that can be merged into the polygon.
		for (int32 CandidateTriangleIndex = 0; CandidateTriangleIndex < Triangles.Num(); ++CandidateTriangleIndex)
		{
			const FLocalTriangle& MergeCandidateTriangle = Triangles[CandidateTriangleIndex];
			if (MergeTriangleIntoPolygon(PolygonVertices, InTriangleVertices[MergeCandidateTriangle.VertexA], InTriangleVertices[MergeCandidateTriangle.VertexB], InTriangleVertices[MergeCandidateTriangle.VertexC], true))
			{
				// Remove the merged triangle from the array.
				Triangles.RemoveAtSwap(CandidateTriangleIndex);

				// Restart the search for mergeable triangles from the start of the array.
				CandidateTriangleIndex = -1;
			}
		}

		OutPolygons.Add(PolygonVertices);
	}
}

// Creates a convex hull that contains the given points
void FGeomTools2D::GenerateConvexHullFromPoints(TArray<FVector2D>& OutConvexHull, TArray<FVector2D>& SourcePoints)
{
	int SourcePointsCount = SourcePoints.Num();
	if (SourcePointsCount < 3)
	{
		return;
	}

	TArray<int32> HullIndices;
	ConvexHull2D::ComputeConvexHull(SourcePoints, HullIndices);
	OutConvexHull.Empty(HullIndices.Num());

	for (int32 HullIdx : HullIndices)
	{
		OutConvexHull.Add(SourcePoints[HullIdx]);
	}
}

bool FGeomTools2D::IsPointInPolygon(const FVector2D& TestPoint, const TArray<FVector2D>& PolygonPoints)
{
	const int NumPoints = PolygonPoints.Num();
	float AngleSum = 0.0f;
	for (int PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
	{
		const FVector2D& VecAB = PolygonPoints[PointIndex] - TestPoint;
		const FVector2D& VecAC = PolygonPoints[(PointIndex + 1) % NumPoints] - TestPoint;
		const float Angle = FMath::Sign(FVector2D::CrossProduct(VecAB, VecAC)) * FMath::Acos(FMath::Clamp(FVector2D::DotProduct(VecAB, VecAC) / (VecAB.Size() * VecAC.Size()), -1.0f, 1.0f));
		AngleSum += Angle;
	}
	return (FMath::Abs(AngleSum) > 0.001f);
}
