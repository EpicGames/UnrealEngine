// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCacheVertexFactory.cpp: Geometry Cache vertex factory implementation
=============================================================================*/

#include "GeometryCacheVertexFactory.h"
#include "GlobalRenderResources.h"
#include "MeshBatch.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "Misc/DelayedAutoRegister.h"
#include "PackedNormal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderUtils.h"

/*-----------------------------------------------------------------------------
FGeometryCacheVertexFactoryShaderParameters
-----------------------------------------------------------------------------*/

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGeometryCacheVertexFactoryUniformBufferParameters, "GeomCache");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGeometryCacheManualVertexFetchUniformBufferParameters, "GeomCacheMVF");

/** Shader parameters for use with TGPUSkinVertexFactory */
class FGeometryCacheVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FGeometryCacheVertexFactoryShaderParameters, NonVirtual);
public:

	/**
	* Bind shader constants by name
	* @param	ParameterMap - mapping of named shader constants to indices
	*/
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		MeshOrigin.Bind(ParameterMap, TEXT("MeshOrigin"));
		MeshExtension.Bind(ParameterMap, TEXT("MeshExtension"));
		MotionBlurDataOrigin.Bind(ParameterMap, TEXT("MotionBlurDataOrigin"));
		MotionBlurDataExtension.Bind(ParameterMap, TEXT("MotionBlurDataExtension"));
		MotionBlurPositionScale.Bind(ParameterMap, TEXT("MotionBlurPositionScale"));
	}

	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* GenericVertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		// Ensure the vertex factory matches this parameter object and cast relevant objects
		check(GenericVertexFactory->GetType() == &FGeometryCacheVertexVertexFactory::StaticType);
		const FGeometryCacheVertexVertexFactory* GCVertexFactory = static_cast<const FGeometryCacheVertexVertexFactory*>(GenericVertexFactory);

		FGeometryCacheVertexFactoryUserData* BatchData = (FGeometryCacheVertexFactoryUserData*)BatchElement.VertexFactoryUserData;

		// Check the passed in vertex buffers make sense
		checkf(BatchData->PositionBuffer->IsInitialized(), TEXT("Batch position Vertex buffer was not initialized! Name %s"), *BatchData->PositionBuffer->GetFriendlyName());
		checkf(BatchData->MotionBlurDataBuffer->IsInitialized(), TEXT("Batch motion blur data buffer was not initialized! Name %s"), *BatchData->MotionBlurDataBuffer->GetFriendlyName());

		VertexStreams.Add(FVertexInputStream(GCVertexFactory->PositionStreamIndex, 0, BatchData->PositionBuffer->VertexBufferRHI));
		VertexStreams.Add(FVertexInputStream(GCVertexFactory->MotionBlurDataStreamIndex, 0, BatchData->MotionBlurDataBuffer->VertexBufferRHI));

		ShaderBindings.Add(MeshOrigin, BatchData->MeshOrigin);
		ShaderBindings.Add(MeshExtension, BatchData->MeshExtension);
		ShaderBindings.Add(MotionBlurDataOrigin, BatchData->MotionBlurDataOrigin);
		ShaderBindings.Add(MotionBlurDataExtension, BatchData->MotionBlurDataExtension);
		ShaderBindings.Add(MotionBlurPositionScale, BatchData->MotionBlurPositionScale);

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGeometryCacheVertexFactoryUniformBufferParameters>(), BatchData->UniformBuffer);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGeometryCacheManualVertexFetchUniformBufferParameters>(), BatchData->ManualVertexFetchUniformBuffer);
	}

private:
	LAYOUT_FIELD(FShaderParameter, MeshOrigin);
	LAYOUT_FIELD(FShaderParameter, MeshExtension);
	LAYOUT_FIELD(FShaderParameter, MotionBlurDataOrigin);
	LAYOUT_FIELD(FShaderParameter, MotionBlurDataExtension);
	LAYOUT_FIELD(FShaderParameter, MotionBlurPositionScale);
};

IMPLEMENT_TYPE_LAYOUT(FGeometryCacheVertexFactoryShaderParameters);

/*-----------------------------------------------------------------------------
FGPUSkinPassthroughVertexFactory
-----------------------------------------------------------------------------*/
void FGeometryCacheVertexVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	const bool bUseGPUScene = UseGPUScene(Parameters.Platform) && GetMaxSupportedFeatureLevel(Parameters.Platform) > ERHIFeatureLevel::ES3_1;
	const bool bSupportsPrimitiveIdStream = Parameters.VertexFactoryType->SupportsPrimitiveIdStream();

	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bSupportsPrimitiveIdStream && bUseGPUScene);
}

void FGeometryCacheVertexVertexFactory::SetData(const FDataType& InData)
{
	SetData(FRHICommandListImmediate::Get(), InData);
}

void FGeometryCacheVertexVertexFactory::SetData(FRHICommandListBase& RHICmdList, const FDataType& InData)
{
	// The shader code makes assumptions that the color component is a FColor, performing swizzles on ES3 and Metal platforms as necessary
	// If the color is sent down as anything other than VET_Color then you'll get an undesired swizzle on those platforms
	check((InData.ColorComponent.Type == VET_None) || (InData.ColorComponent.Type == VET_Color));

	Data = InData;
	// This will call InitRHI below where the real action happens
	UpdateRHI(RHICmdList);
}

class FDefaultGeometryCacheVertexBuffer : public FVertexBuffer
{
public:
	FShaderResourceViewRHIRef SRV;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("DefaultGeometryCacheVertexBuffer"), 2)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FVector4f> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		Initializer[0] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		Initializer[1] = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

		VertexBufferRHI = Initializer.Finalize();
		SRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
	}

	virtual void ReleaseRHI() override
	{
		SRV.SafeRelease();
		FVertexBuffer::ReleaseRHI();
	}
};
TGlobalResource<FDefaultGeometryCacheVertexBuffer> GDefaultGeometryCacheVertexBuffer;

class FDummyTangentBuffer : public FVertexBuffer
{
public:
	FShaderResourceViewRHIRef SRV;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("DummyTangentBuffer"), 2)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FVector4f> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		Initializer[0] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		Initializer[1] = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

		VertexBufferRHI = Initializer.Finalize();
		SRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8_SNORM));
	}

	virtual void ReleaseRHI() override
	{
		SRV.SafeRelease();
		FVertexBuffer::ReleaseRHI();
	}
};
TGlobalResource<FDummyTangentBuffer> GDummyTangentBuffer;

void FGeometryCacheVertexVertexFactory::GetVertexElements(
	ERHIFeatureLevel::Type FeatureLevel, 
	EVertexInputStreamType InputStreamType, 
	FDataType& StreamComponentData, 
	FVertexDeclarationElementList& OutElements, 
	FVertexStreamList& InOutStreams, 
	int32& OutPositionStreamIndex, 
	int32& OutMotionBlurDataStreamIndex)
{
	check(InputStreamType == EVertexInputStreamType::Default);

	if (StreamComponentData.PositionComponent.VertexBuffer != NULL || StreamComponentData.bIsDummyData)
	{
		OutElements.Add(AccessStreamComponent(StreamComponentData.PositionComponent, 0, InOutStreams));
		OutPositionStreamIndex = OutElements.Last().StreamIndex;
	}

	// only tangent,normal are used by the stream. the binormal is derived in the shader
	uint8 TangentBasisAttributes[2] = { 1, 2 };
	for (int32 AxisIndex = 0; AxisIndex < 2; AxisIndex++)
	{
		if (StreamComponentData.TangentBasisComponents[AxisIndex].VertexBuffer != NULL || StreamComponentData.bIsDummyData)
		{
			OutElements.Add(AccessStreamComponent(StreamComponentData.TangentBasisComponents[AxisIndex], TangentBasisAttributes[AxisIndex], InOutStreams));
		}
	}

	if (StreamComponentData.ColorComponent.VertexBuffer)
	{
		OutElements.Add(AccessStreamComponent(StreamComponentData.ColorComponent, 3, InOutStreams));
	}
	else
	{
		//If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
		//This wastes 4 bytes of bandwidth per vertex, but prevents having to compile out twice the number of vertex factories.
		FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color);
		OutElements.Add(AccessStreamComponent(NullColorComponent, 3, InOutStreams));
	}

	if (StreamComponentData.MotionBlurDataComponent.VertexBuffer || StreamComponentData.bIsDummyData)
	{
		OutElements.Add(AccessStreamComponent(StreamComponentData.MotionBlurDataComponent, 4, InOutStreams));
	}
	else if (StreamComponentData.PositionComponent.VertexBuffer != NULL)
	{
		OutElements.Add(AccessStreamComponent(StreamComponentData.PositionComponent, 4, InOutStreams));
	}
	OutMotionBlurDataStreamIndex = OutElements.Last().StreamIndex;

	if (StreamComponentData.TextureCoordinates.Num())
	{
		const int32 BaseTexCoordAttribute = 5;	
		AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, StreamComponentData.TextureCoordinates, OutElements, InOutStreams);
	}
}

void FGeometryCacheVertexVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& OutElements)
{
	OutElements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(float)*3u, false));
	
	if (VertexInputStreamType == EVertexInputStreamType::PositionAndNormalOnly)
	{
		// 2-axis TangentBasis components in a single buffer, hence *2u
		OutElements.Add(FVertexElement(1, 0, VET_PackedNormal, 1, sizeof(FPackedNormal) * 2u, false));
	}

	switch (VertexInputStreamType)
	{
		case EVertexInputStreamType::Default:
		{
			// Make sure all required elements are added because manual vertex fetch is not supported (see GetVertexElements above)
			uint8 TangentBasisAttributes[2] = { 1, 2 };
			for (int32 AxisIndex = 0; AxisIndex < 2; AxisIndex++)
			{
				OutElements.Add(FVertexElement(TangentBasisAttributes[AxisIndex], 0, VET_PackedNormal, TangentBasisAttributes[AxisIndex], sizeof(FPackedNormal), false));
			}
			OutElements.Add(FVertexElement(3, 0, VET_Color, 3, sizeof(FColor), false));
			OutElements.Add(FVertexElement(4, 0, VET_Float3, 4, sizeof(FVector3f), false));
			for (int32 CoordinateIndex = 0; CoordinateIndex < MAX_STATIC_TEXCOORDS / 2; CoordinateIndex++)
			{
				OutElements.Add(FVertexElement(5 + CoordinateIndex, 0, VET_Half4, 5 + CoordinateIndex, sizeof(FVector2f), false));
			}

			OutElements.Add(FVertexElement(9, 0, VET_UInt, 13, sizeof(uint32), true));

			break;
		}
		case EVertexInputStreamType::PositionOnly:
		{
			OutElements.Add(FVertexElement(1, 0, VET_UInt, 1, sizeof(uint32), true));
			break;
		}
		case EVertexInputStreamType::PositionAndNormalOnly:
		{
			OutElements.Add(FVertexElement(2, 0, VET_UInt, 2, sizeof(uint32), true));
			break;
		}
		default:
			checkNoEntry();
	}
}

void FGeometryCacheVertexVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FDataType& StreamComponentData, FVertexDeclarationElementList& OutElements)
{
	FVertexStreamList VertexStreams;
	int32 PositionStreamIndex = -1;
	int32 MotionBlurDataStreamIndex = -1;
	GetVertexElements(FeatureLevel, InputStreamType, StreamComponentData, OutElements, VertexStreams, PositionStreamIndex, MotionBlurDataStreamIndex);

	if (UseGPUScene(GMaxRHIShaderPlatform) 
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		OutElements.Add(FVertexElement(VertexStreams.Num(), 0, VET_UInt, 13, sizeof(uint32), true));
	}
}

void FGeometryCacheVertexVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Position needs to be separate from the rest (we just theck tangents here)
	check(Data.PositionComponent.VertexBuffer != Data.TangentBasisComponents[0].VertexBuffer);
	// Motion Blur data also needs to be separate from the rest
	check(Data.MotionBlurDataComponent.VertexBuffer != Data.TangentBasisComponents[0].VertexBuffer);
	check(Data.MotionBlurDataComponent.VertexBuffer != Data.PositionComponent.VertexBuffer);	

	// If the vertex buffer containing position is not the same vertex buffer containing the rest of the data,
	// then initialize PositionStream and PositionDeclaration.
	if (Data.PositionComponent.VertexBuffer != Data.TangentBasisComponents[0].VertexBuffer)
	{
		{
			FVertexDeclarationElementList PositionOnlyStreamElements;
			PositionOnlyStreamElements.Add(AccessStreamComponent(Data.PositionComponent, 0, EVertexInputStreamType::PositionOnly));
			AddPrimitiveIdStreamElement(EVertexInputStreamType::PositionOnly, PositionOnlyStreamElements, 1, 1);
			InitDeclaration(PositionOnlyStreamElements, EVertexInputStreamType::PositionOnly);
		}

		{
			FVertexDeclarationElementList PositionAndNormalOnlyStreamElements;
			PositionAndNormalOnlyStreamElements.Add(AccessStreamComponent(Data.PositionComponent, 0, EVertexInputStreamType::PositionAndNormalOnly));
			PositionAndNormalOnlyStreamElements.Add(AccessStreamComponent(Data.TangentBasisComponents[1], 1, EVertexInputStreamType::PositionAndNormalOnly));
			AddPrimitiveIdStreamElement(EVertexInputStreamType::PositionAndNormalOnly, PositionAndNormalOnlyStreamElements, 2, 2);
			InitDeclaration(PositionAndNormalOnlyStreamElements, EVertexInputStreamType::PositionAndNormalOnly);
		}
	}

	FVertexDeclarationElementList Elements;
	GetVertexElements(GetFeatureLevel(), EVertexInputStreamType::Default, Data, Elements, Streams, PositionStreamIndex, MotionBlurDataStreamIndex);
	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, 13, 13);

	check(Streams.Num() > 0);
	check(PositionStreamIndex >= 0);
	check(MotionBlurDataStreamIndex >= 0);
	check(MotionBlurDataStreamIndex != PositionStreamIndex);

	InitDeclaration(Elements);

	check(IsValidRef(GetDeclaration()));
}

void FGeometryCacheVertexVertexFactory::CreateManualVertexFetchUniformBuffer(
	const FVertexBuffer* PositionBuffer,
	const FVertexBuffer* MotionBlurBuffer,
	FGeometryCacheVertexFactoryUserData& OutUserData) const
{
	CreateManualVertexFetchUniformBuffer(FRHICommandListImmediate::Get(), PositionBuffer, MotionBlurBuffer, OutUserData);
}

void FGeometryCacheVertexVertexFactory::CreateManualVertexFetchUniformBuffer(
	FRHICommandListBase& RHICmdList,
	const FVertexBuffer* PoistionBuffer, 
	const FVertexBuffer* MotionBlurBuffer,
	FGeometryCacheVertexFactoryUserData& OutUserData) const
{
	FGeometryCacheManualVertexFetchUniformBufferParameters ManualVertexFetchParameters;

	if (PoistionBuffer != NULL)
	{
		OutUserData.PositionSRV = RHICmdList.CreateShaderResourceView(
			PoistionBuffer->VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
		// Position will need per-component fetch since we don't have R32G32B32 pixel format
		ManualVertexFetchParameters.Position = OutUserData.PositionSRV;
	}
	else
	{
		ManualVertexFetchParameters.Position = GDefaultGeometryCacheVertexBuffer.SRV;
	}

	if (Data.TangentBasisComponents[0].VertexBuffer != NULL)
	{
		OutUserData.TangentXSRV = RHICmdList.CreateShaderResourceView(
			Data.TangentBasisComponents[0].VertexBuffer->VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8_SNORM));
		ManualVertexFetchParameters.TangentX = OutUserData.TangentXSRV;
	}
	else
	{
		ManualVertexFetchParameters.TangentX = GDummyTangentBuffer.SRV;
	}

	if (Data.TangentBasisComponents[1].VertexBuffer != NULL)
	{
		OutUserData.TangentZSRV = RHICmdList.CreateShaderResourceView(
			Data.TangentBasisComponents[1].VertexBuffer->VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8_SNORM));
		ManualVertexFetchParameters.TangentZ = OutUserData.TangentZSRV;
	}
	else
	{
		ManualVertexFetchParameters.TangentZ = GDummyTangentBuffer.SRV;
	}

	if (Data.ColorComponent.VertexBuffer)
	{
		OutUserData.ColorSRV = RHICmdList.CreateShaderResourceView(
			Data.ColorComponent.VertexBuffer->VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_B8G8R8A8));
		ManualVertexFetchParameters.Color = OutUserData.ColorSRV;
	}
	else
	{
		OutUserData.ColorSRV = GNullColorVertexBuffer.VertexBufferSRV;
		ManualVertexFetchParameters.Color = OutUserData.ColorSRV;
	}

	if (MotionBlurBuffer)
	{
		OutUserData.MotionBlurDataSRV = RHICmdList.CreateShaderResourceView(
			MotionBlurBuffer->VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
		ManualVertexFetchParameters.MotionBlurData = OutUserData.MotionBlurDataSRV;
	}
	else if (PoistionBuffer != NULL)
	{
		ManualVertexFetchParameters.MotionBlurData = OutUserData.PositionSRV;
	}
	else
	{
		ManualVertexFetchParameters.MotionBlurData = GDefaultGeometryCacheVertexBuffer.SRV;
	}

	if (Data.TextureCoordinates.Num())
	{
		checkf(Data.TextureCoordinates.Num() <= 1, TEXT("We're assuming FGeometryCacheSceneProxy uses only one TextureCoordinates vertex buffer"));
		OutUserData.TexCoordsSRV = RHICmdList.CreateShaderResourceView(
			Data.TextureCoordinates[0].VertexBuffer->VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
		// TexCoords will need per-component fetch since we don't have R32G32 pixel format
		ManualVertexFetchParameters.TexCoords = OutUserData.TexCoordsSRV;
	}
	else
	{
		ManualVertexFetchParameters.TexCoords = GDefaultGeometryCacheVertexBuffer.SRV;
	}

	OutUserData.ManualVertexFetchUniformBuffer = FGeometryCacheManualVertexFetchUniformBufferParametersRef::CreateUniformBufferImmediate(ManualVertexFetchParameters, UniformBuffer_SingleFrame);
}

bool FGeometryCacheVertexVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	// Should this be platform or mesh type based? Returning true should work in all cases, but maybe too expensive? 
	// return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5) && !IsConsolePlatform(Platform);
	// TODO currently GeomCache supports only 4 UVs which could cause compilation errors when trying to compile shaders which use > 4
	return Parameters.MaterialParameters.bIsUsedWithGeometryCache || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FGeometryCacheVertexVertexFactory, SF_Vertex, FGeometryCacheVertexFactoryShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FGeometryCacheVertexVertexFactory, SF_RayHitGroup, FGeometryCacheVertexFactoryShaderParameters);
#endif
IMPLEMENT_VERTEX_FACTORY_TYPE(FGeometryCacheVertexVertexFactory, "/Engine/Private/GeometryCacheVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPositionOnly
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsPSOPrecaching
);
