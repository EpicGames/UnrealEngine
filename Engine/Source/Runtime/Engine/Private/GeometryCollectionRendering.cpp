// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionRendering.h"
#include "GlobalRenderResources.h"
#include "MaterialDomain.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/DelayedAutoRegister.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "PrimitiveUniformShaderParameters.h"
#include "RenderUtils.h"
#include "SceneInterface.h"

IMPLEMENT_TYPE_LAYOUT(FGeometryCollectionVertexFactoryShaderParameters);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGeometryCollectionVertexFactoryUniformShaderParameters, "GeometryCollectionVF");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGCBoneLooseParameters, "GCBoneLooseParameters");

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FGeometryCollectionVertexFactory, SF_Vertex, FGeometryCollectionVertexFactoryShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FGeometryCollectionVertexFactory, "/Engine/Private/GeometryCollectionVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsPositionOnly
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsManualVertexFetch
	| EVertexFactoryFlags::SupportsPSOPrecaching
);

bool FGeometryCollectionVertexFactory::UseShaderBoneTransform(EShaderPlatform ShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsTypedBufferSRVs(ShaderPlatform);
}

bool FGeometryCollectionVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	if (!Parameters.MaterialParameters.bIsUsedWithGeometryCollections && !Parameters.MaterialParameters.bIsSpecialEngineMaterial)
	{
		return false;
	}

	// Only compile this permutation inside the editor - it's not applicable in games, but occasionally the editor needs it.
	if (Parameters.MaterialParameters.MaterialDomain == MD_UI)
	{
		return !!WITH_EDITOR;
	}

	return true;
}

//
// Modify compile environment to enable instancing
// @param OutEnvironment - shader compile environment to modify
//
void FGeometryCollectionVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	const FStaticFeatureLevel MaxSupportedFeatureLevel = GetMaxSupportedFeatureLevel(Parameters.Platform);
	// TODO: support GPUScene on mobile
	const bool bUseGPUScene = UseGPUScene(Parameters.Platform);
	const bool bSupportsPrimitiveIdStream = Parameters.VertexFactoryType->SupportsPrimitiveIdStream();

	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bSupportsPrimitiveIdStream && bUseGPUScene);

	if (RHISupportsManualVertexFetch(Parameters.Platform))
	{
		OutEnvironment.SetDefineIfUnset(TEXT("MANUAL_VERTEX_FETCH"), TEXT("1"));
	}
	else if (UseShaderBoneTransform(Parameters.Platform))
	{
		OutEnvironment.SetDefineIfUnset(TEXT("USE_SHADER_BONE_TRANSFORM"), TEXT("1"));
	}

	// Geometry collections use a custom hit proxy per bone
	OutEnvironment.SetDefine(TEXT("USE_PER_VERTEX_HITPROXY_ID"), 1);

	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_LOCAL_SPACE"), TEXT("1"));
}

void FGeometryCollectionVertexFactory::ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors)
{
	if (Type->SupportsPrimitiveIdStream()
		&& UseGPUScene(Platform)
		&& ParameterMap.ContainsParameterAllocation(FPrimitiveUniformShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
	{
		OutErrors.AddUnique(*FString::Printf(TEXT("Shader attempted to bind the Primitive uniform buffer even though Vertex Factory %s computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(Parameters).Member instead of Primitive.Member."), Type->GetName()));
	}
}

void FGeometryCollectionVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	uint8 StreamIndex = 0;
	Elements.Add(FVertexElement(StreamIndex++, 0, VET_Float3, 0, 0, false));
		
	if (VertexInputStreamType == EVertexInputStreamType::PositionAndNormalOnly)
	{
		// 2-axis TangentBasis components in a single buffer, hence *2u
		Elements.Add(FVertexElement(StreamIndex++, 4, VET_PackedNormal, 2, 0, false));
	}

	// Primitive ID stream
	if (UseGPUScene(GMaxRHIShaderPlatform)
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		if (VertexInputStreamType == EVertexInputStreamType::Default)
		{
			Elements.Add(FVertexElement(StreamIndex++, 0, VET_UInt, 13, 0, true));
		}
		else
		{
			Elements.Add(FVertexElement(StreamIndex++, 0, VET_UInt, 1, 0, true));
		}
	}

	if (VertexInputStreamType == EVertexInputStreamType::Default && !RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		// Normals
		Elements.Add(FVertexElement(StreamIndex, 0, VET_PackedNormal, 1, 0, false));
		Elements.Add(FVertexElement(StreamIndex++, 4, VET_PackedNormal, 2, 0, false));

		// Color
		Elements.Add(FVertexElement(StreamIndex++, 0, VET_Color, 3, 0, false));

		// TexCoords
		int8 UVStreamIndex = StreamIndex;
		Elements.Add(FVertexElement(StreamIndex, 0, VET_Half4, 4, 0, false));
		Elements.Add(FVertexElement(StreamIndex, 8, VET_Half4, 5, 0, false));
		Elements.Add(FVertexElement(StreamIndex, 16, VET_Half4, 6, 0, false));
		Elements.Add(FVertexElement(StreamIndex++, 24, VET_Half4, 7, 0, false));

		// Light map data
		if (IsStaticLightingAllowed())
		{
			Elements.Add(FVertexElement(UVStreamIndex, 0, VET_Half2, 15, 0, false));
		}
	}
}

void FGeometryCollectionVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FGeometryCollectionVertexFactory_InitRHI);

	// We create different streams based on feature level
	check(HasValidFeatureLevel());

	// VertexFactory needs to be able to support max possible shader platform and feature level
	// in case if we switch feature level at runtime.
	const bool bCanUseGPUScene = UseGPUScene(GShaderPlatformForFeatureLevel[GetFeatureLevel()]);
	const bool bUseManualVertexFetch = SupportsManualVertexFetch(GetFeatureLevel());

	// If the vertex buffer containing position is not the same vertex buffer containing the rest of the data,
	// then initialize PositionStream and PositionDeclaration.
	if (Data.PositionComponent.VertexBuffer != Data.TangentBasisComponents[0].VertexBuffer)
	{
		auto AddDeclaration = [this](EVertexInputStreamType InputStreamType, bool bAddNormal)
		{
			FVertexDeclarationElementList StreamElements;
			StreamElements.Add(AccessStreamComponent(Data.PositionComponent, 0, InputStreamType));

			bAddNormal = bAddNormal && Data.TangentBasisComponents[1].VertexBuffer != NULL;
			if (bAddNormal)
			{
				StreamElements.Add(AccessStreamComponent(Data.TangentBasisComponents[1], 2, InputStreamType));
			}

			AddPrimitiveIdStreamElement(InputStreamType, StreamElements, 1, 1);

			InitDeclaration(StreamElements, InputStreamType);
		};

		AddDeclaration(EVertexInputStreamType::PositionOnly, false);
		AddDeclaration(EVertexInputStreamType::PositionAndNormalOnly, true);
	}

	FVertexDeclarationElementList Elements;
	if (Data.PositionComponent.VertexBuffer != nullptr)
	{
		Elements.Add(AccessStreamComponent(Data.PositionComponent, 0));
	}

	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, 13, 13);

	// Only the tangent and normal are used by the stream; the bitangent is derived in the shader.
	uint8 TangentBasisAttributes[2] = { 1, 2 };
	for (int32 AxisIndex = 0; AxisIndex < 2; AxisIndex++)
	{
		if (Data.TangentBasisComponents[AxisIndex].VertexBuffer != nullptr)
		{
			Elements.Add(AccessStreamComponent(Data.TangentBasisComponents[AxisIndex], TangentBasisAttributes[AxisIndex]));
		}
	}

	if (Data.ColorComponentsSRV == nullptr)
	{
		Data.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
		Data.ColorIndexMask = 0;
	}

	ColorStreamIndex = INDEX_NONE;
	if (Data.ColorComponent.VertexBuffer)
	{
		Elements.Add(AccessStreamComponent(Data.ColorComponent, 3));
		ColorStreamIndex = Elements.Last().StreamIndex;
	}
	else
	{
		// If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
		// This wastes 4 bytes per vertex, but prevents having to compile out twice the number of vertex factories.
		FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
		Elements.Add(AccessStreamComponent(NullColorComponent, 3));
		ColorStreamIndex = Elements.Last().StreamIndex;
	}

	if (Data.TextureCoordinates.Num())
	{
		const int32 BaseTexCoordAttribute = 4;			
		AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, Data.TextureCoordinates, Elements, Streams);
	}

	if (IsStaticLightingAllowed())
	{
		if (Data.LightMapCoordinateComponent.VertexBuffer)
		{
			Elements.Add(AccessStreamComponent(Data.LightMapCoordinateComponent, 15));
		}
		else if (Data.TextureCoordinates.Num())
		{
			Elements.Add(AccessStreamComponent(Data.TextureCoordinates[0], 15));
		}
	}

	check(Streams.Num() > 0);

	InitDeclaration(Elements);
	check(IsValidRef(GetDeclaration()));

	const int32 DefaultBaseVertexIndex = 0;
	const int32 DefaultPreSkinBaseVertexIndex = 0;

	if (bCanUseGPUScene)
	{
		SCOPED_LOADTIMER(FGeometryCollectionVertexFactory_InitRHI_CreateLocalVFUniformBuffer);

		FGeometryCollectionVertexFactoryUniformShaderParameters UniformParameters;

		UniformParameters.LODLightmapDataIndex = Data.LODLightmapDataIndex;
		int32 ColorIndexMask = 0;

		if (bUseManualVertexFetch)
		{
			UniformParameters.VertexFetch_PositionBuffer = GetPositionsSRV();
			UniformParameters.VertexFetch_PackedTangentsBuffer = GetTangentsSRV();
			UniformParameters.VertexFetch_TexCoordBuffer = GetTextureCoordinatesSRV();

			UniformParameters.VertexFetch_ColorComponentsBuffer = GetColorComponentsSRV();
			ColorIndexMask = (int32)GetColorIndexMask();
		}
		else
		{
			UniformParameters.VertexFetch_PositionBuffer = GNullColorVertexBuffer.VertexBufferSRV;
			UniformParameters.VertexFetch_PackedTangentsBuffer = GNullColorVertexBuffer.VertexBufferSRV;
			UniformParameters.VertexFetch_TexCoordBuffer = GNullColorVertexBuffer.VertexBufferSRV;
		}

		if (!UniformParameters.VertexFetch_ColorComponentsBuffer)
		{
			UniformParameters.VertexFetch_ColorComponentsBuffer = GNullColorVertexBuffer.VertexBufferSRV;
		}

		const int32 NumTexCoords = GetNumTexcoords();
		const int32 LightMapCoordinateIndex = GetLightMapCoordinateIndex();

		UniformParameters.VertexFetch_Parameters = { ColorIndexMask, NumTexCoords, LightMapCoordinateIndex, 0 };

		UniformBuffer = TUniformBufferRef<FGeometryCollectionVertexFactoryUniformShaderParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_MultiFrame);
		
		FGCBoneLooseParameters LooseParameters;

		LooseParameters.VertexFetch_BoneTransformBuffer = GetBoneTransformSRV();
		LooseParameters.VertexFetch_BonePrevTransformBuffer = GetBonePrevTransformSRV();
		LooseParameters.VertexFetch_BoneMapBuffer = GetBoneMapSRV();

		LooseParameterUniformBuffer = FGCBoneLooseParametersRef::CreateUniformBufferImmediate(LooseParameters, UniformBuffer_MultiFrame);
	}

	check(IsValidRef(GetDeclaration()));
}

void FGeometryCollectionVertexFactory::ReleaseRHI()
{
	UniformBuffer.SafeRelease();
	LooseParameterUniformBuffer.SafeRelease();
	FVertexFactory::ReleaseRHI();
}

void FGeometryCollectionVertexFactoryShaderParameters::GetElementShaderBindings(
	const FSceneInterface* Scene,
	const FSceneView* View,
	const FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams) const
{
	check(VertexFactory->GetType() == &FGeometryCollectionVertexFactory::StaticType);
	const auto* TypedVertexFactory = static_cast<const FGeometryCollectionVertexFactory*>(VertexFactory);

	const bool bUseShaderBoneTransform = TypedVertexFactory->SupportsManualVertexFetch(FeatureLevel) || TypedVertexFactory->UseShaderBoneTransform(Scene->GetShaderPlatform());

	FRHIUniformBuffer* VertexFactoryUniformBuffer = TypedVertexFactory->GetUniformBuffer();

	if (bUseShaderBoneTransform || UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]))
	{
		check(VertexFactoryUniformBuffer != nullptr);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGeometryCollectionVertexFactoryUniformShaderParameters>(), VertexFactoryUniformBuffer);
	}

	// We only want to set the SRV parameters if we support manual vertex fetch.
	if (bUseShaderBoneTransform)
	{
		FUniformBufferRHIRef LooseParameterBuffer = TypedVertexFactory->GetLooseParameterBuffer();
		check(LooseParameterBuffer != nullptr);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGCBoneLooseParameters>(), LooseParameterBuffer);
	}
}

#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FGeometryCollectionVertexFactory, SF_RayHitGroup, FGeometryCollectionVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FGeometryCollectionVertexFactory, SF_Compute, FGeometryCollectionVertexFactoryShaderParameters);
#endif // RHI_RAYTRACING
