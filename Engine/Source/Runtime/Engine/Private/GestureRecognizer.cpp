// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	 GestureRecognizer - handles detecting when gestures happen
=============================================================================*/

#include "GestureRecognizer.h"

#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerInput.h"

void FGestureRecognizer::DetectGestures(TArrayView<const UE::FTouchState> TouchStates, UPlayerInput* PlayerInput, float DeltaTime)
{
	// how many fingers currently held down?
	int32 TouchCount = 0;
	for (int32 TouchIndex = 0; TouchIndex < TouchStates.Num(); TouchIndex++)
	{
		if (TouchStates[TouchIndex].Pressure != 0)
		{
			TouchCount++;
		}
	}

	// Further processing is only needed if there were or are active touches.
	if (PreviousTouchCount != 0 || TouchCount != 0)
	{
		// place new anchor points
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(AnchorPoints); Index++)
		{
			if (PreviousTouchCount < Index + 1 && TouchCount >= Index + 1)
			{
				AnchorPoints[Index] = FVector2D(TouchStates[Index].ViewportRelativeLocation);
			}
		}

		// handle different types of two finger gestures
		if (TouchCount >= 2)
		{
			float* CurrentAlpha = CurrentGestureValues.Find(EKeys::Gesture_Pinch);

			//#jira UE-51262 values for pinch input produce very different results for same area on android device
			FVector2D CurrentPinchPoint_Start(TouchStates[0].ViewportRelativeLocation);
			FVector2D CurrentPinchPoint_End(TouchStates[1].ViewportRelativeLocation);
			const float PinchThreshold = 10000.0f;

			if (CurrentAlpha == nullptr)
			{
				//if the pinch points are close enough to those from the last pinch, continue with the previous starting distance
				if (AnchorDistanceSq == 0 ||
					(LastPinchPoint_Start - CurrentPinchPoint_Start).SizeSquared() > PinchThreshold ||
					(LastPinchPoint_End - CurrentPinchPoint_End).SizeSquared() > PinchThreshold)
				{
					// remember the starting distance
					SetAnchorDistanceSquared(CurrentPinchPoint_Start, CurrentPinchPoint_End);
				}

				// alpha of 1 is initial pinch anchor distance
				CurrentGestureValues.Add(EKeys::Gesture_Pinch, 1.0f);
				CurrentAlpha = CurrentGestureValues.Find(EKeys::Gesture_Pinch);
			}

			// calculate current alpha
			float NewDistanceSq = (CurrentPinchPoint_Start - CurrentPinchPoint_End).SizeSquared();
			*CurrentAlpha = NewDistanceSq / AnchorDistanceSq;
			HandleGesture(PlayerInput, EKeys::Gesture_Pinch, true, false);

			LastPinchPoint_Start = CurrentPinchPoint_Start;
			LastPinchPoint_End = CurrentPinchPoint_End;

			// calculate the angle of the vector between the touch points
			float NewAngle = FMath::Atan2(CurrentPinchPoint_Start.Y - CurrentPinchPoint_End.Y, CurrentPinchPoint_Start.X - CurrentPinchPoint_End.X);
			NewAngle = FRotator::ClampAxis(FMath::RadiansToDegrees(NewAngle));

			float* CurrentAngle = CurrentGestureValues.Find(EKeys::Gesture_Rotate);
			if (CurrentAngle == nullptr)
			{
				// save the starting angle
				StartAngle = NewAngle;

				// use 0 as the initial angle value; subsequent angles will be relative
				CurrentGestureValues.Add(EKeys::Gesture_Rotate, 0.0f);
				HandleGesture(PlayerInput, EKeys::Gesture_Rotate, true, false);
			}
			else
			{
				*CurrentAngle = NewAngle - StartAngle;

				// Gestures are only processed for IE_Pressed events, so treat this like another "start"
				HandleGesture(PlayerInput, EKeys::Gesture_Rotate, true, false);
			}
		}

		// Handle pinch and rotate release.
		if (PreviousTouchCount >= 2 && TouchCount < 2)
		{
			HandleGesture(PlayerInput, EKeys::Gesture_Pinch, false, true);
			HandleGesture(PlayerInput, EKeys::Gesture_Rotate, false, true);
		}

		if (PreviousTouchCount == 0 && TouchCount == 1)
		{
			// initialize the flick
			FlickTime = 0.0f;
		}
		else if (PreviousTouchCount == 1 && TouchCount == 1)
		{
			// remember the position for when we let go
			const FVector2f& Loc0 = TouchStates[0].ViewportRelativeLocation;
			FlickCurrent = FVector2D(Loc0);
			FlickTime += DeltaTime;
		}
		else if (PreviousTouchCount >= 1 && TouchCount == 0)
		{
			// must be a fast flick
			if (FlickTime < 0.25f && (FlickCurrent - AnchorPoints[0]).SizeSquared() > 10000.f)
			{
				// this is the angle from +X in screen space, meaning right is 0, up is 90, left is 180, down is 270
				float Angle = FMath::Atan2(-(FlickCurrent.Y - AnchorPoints[0].Y), FlickCurrent.X - AnchorPoints[0].X);
				Angle = FRotator::ClampAxis(FMath::RadiansToDegrees(Angle));

				// flicks are one-shot, so we start and end in the same frame
				CurrentGestureValues.Add(EKeys::Gesture_Flick, Angle);
				HandleGesture(PlayerInput, EKeys::Gesture_Flick, true, true);
			}
		}
	}	

	// remember for next frame
	PreviousTouchCount = TouchCount;
}

void FGestureRecognizer::SetAnchorDistanceSquared(const FVector2D FirstPoint, const FVector2D SecontPoint)
{
	AnchorDistanceSq = (FirstPoint - SecontPoint).SizeSquared();
}

void FGestureRecognizer::HandleGesture(UPlayerInput* PlayerInput, FKey Gesture, bool bStarted, bool bEnded)
{
	float* Value = CurrentGestureValues.Find(Gesture);
	if (Value)
	{
		const ULocalPlayer* OwningLP = PlayerInput->GetOwningLocalPlayer();
		const FInputDeviceId DefaultDevice = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(OwningLP ? OwningLP->GetPlatformUserId() : PLATFORMUSERID_NONE);
		const uint64 CurrentTimestamp = FPlatformTime::Cycles64();
		
		const EInputEvent GestureEvent = (bStarted ? IE_Pressed : bEnded ? IE_Released : IE_Repeat);
		PlayerInput->InputGesture(DefaultDevice, Gesture, GestureEvent, *Value, CurrentTimestamp);

		// remove it if the gesture is complete
		if (bEnded)
		{
			CurrentGestureValues.Remove(Gesture);
		}
	}
}

