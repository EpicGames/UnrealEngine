// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomVisualizationData.h"
#include "HAL/IConsoleManager.h"
#include "Materials/Material.h"
#include "Misc/ConfigCacheIni.h"
#include "SceneView.h"

#define LOCTEXT_NAMESPACE "FGroomVisualizationData"

static int32 GHairStrandsPluginEnable = 0;

static TAutoConsoleVariable<int32> CVarHairStrandsGlobalEnable(
	TEXT("r.HairStrands.Enable"), 1,
	TEXT("Enable/Disable the entire hair strands system. This affects all geometric representations (i.e., strands, cards, and meshes)."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

bool IsGroomEnabled()
{
	return GHairStrandsPluginEnable > 0 && CVarHairStrandsGlobalEnable.GetValueOnAnyThread() > 0;
}

void SetGroomEnabled(bool In)
{
	GHairStrandsPluginEnable = In ? 1 : 0;
}

EGroomViewMode GetGroomViewMode(const FSceneView& View)
{
	EGroomViewMode Out = EGroomViewMode::None;
	if (IsGroomEnabled())
	{
		static TConsoleVariableData<int32>* CVarGroomViewMode = nullptr; 
		if (CVarGroomViewMode == nullptr)
		{
			if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(FGroomVisualizationData::GetVisualizeConsoleCommandName()))
			{
				CVarGroomViewMode = ConsoleVariable->AsVariableInt();
			}
		}

		if (CVarGroomViewMode)
		{
			const uint32 ViewMode = CVarGroomViewMode->GetValueOnRenderThread();
			switch (ViewMode)
			{
			case 1:  return EGroomViewMode::MacroGroups;
			case 2:  return EGroomViewMode::LightBounds;
			case 3:  return EGroomViewMode::MacroGroupScreenRect;
			case 4:  return EGroomViewMode::DeepOpacityMaps;
			case 5:  return EGroomViewMode::SamplePerPixel;
			case 6:  return EGroomViewMode::TAAResolveType;
			case 7:  return EGroomViewMode::CoverageType;
			case 8:  return EGroomViewMode::VoxelsDensity;
			case 9:	 return EGroomViewMode::None;
			case 10: return EGroomViewMode::None;
			case 11: return EGroomViewMode::None;
			case 12: return EGroomViewMode::MeshProjection;
			case 13: return EGroomViewMode::Coverage;
			case 14: return EGroomViewMode::MaterialDepth;
			case 15: return EGroomViewMode::MaterialBaseColor;
			case 16: return EGroomViewMode::MaterialRoughness;
			case 17: return EGroomViewMode::MaterialSpecular;
			case 18: return EGroomViewMode::MaterialTangent;
			case 19: return EGroomViewMode::Tile;
			case 20: return EGroomViewMode::None;
			case 21: return EGroomViewMode::SimHairStrands;
			case 22: return EGroomViewMode::RenderHairStrands;
			case 23: return EGroomViewMode::RootUV;
			case 24: return EGroomViewMode::RootUDIM;
			case 25: return EGroomViewMode::UV;
			case 26: return EGroomViewMode::Seed;
			case 27: return EGroomViewMode::Dimension;
			case 28: return EGroomViewMode::RadiusVariation;
			case 29: return EGroomViewMode::Color;
			case 30: return EGroomViewMode::Roughness;
			case 31: return EGroomViewMode::Cluster;
			case 32: return EGroomViewMode::ClusterAABB;
			case 33: return EGroomViewMode::Tangent;
			case 34: return EGroomViewMode::ControlPoints;
			case 35: return EGroomViewMode::Group;
			case 36: return EGroomViewMode::LODColoration;
			case 37: return EGroomViewMode::CardGuides;
			case 38: return EGroomViewMode::AO;
			case 39: return EGroomViewMode::ClumpID;
			case 40: return EGroomViewMode::Memory;
			case 41: return EGroomViewMode::Bounds;
			default: break;
			}
		}

		const FGroomVisualizationData& VisualizationData = GetGroomVisualizationData();
		if (View.Family && View.Family->EngineShowFlags.VisualizeGroom)
		{
			Out = VisualizationData.GetViewMode(View.CurrentGroomVisualizationMode);
		}
		else if (View.Family && View.Family->EngineShowFlags.LODColoration)
		{
			Out = EGroomViewMode::LODColoration;
		}
	}
	return Out;
}

const TCHAR* GetGroomViewModeName(EGroomViewMode In)
{
	switch (In)
	{
	case EGroomViewMode::None:						return TEXT("NoneDebug");
	case EGroomViewMode::MacroGroups:				return TEXT("MacroGroups");
	case EGroomViewMode::LightBounds:				return TEXT("LightBounds");
	case EGroomViewMode::MacroGroupScreenRect:		return TEXT("MacroGroupScreenRect");
	case EGroomViewMode::DeepOpacityMaps:			return TEXT("DeepOpacityMaps");
	case EGroomViewMode::SamplePerPixel:			return TEXT("SamplePerPixel");
	case EGroomViewMode::TAAResolveType:			return TEXT("TAAResolveType");
	case EGroomViewMode::CoverageType:				return TEXT("CoverageType");
	case EGroomViewMode::VoxelsDensity:				return TEXT("VoxelsDensity");
	case EGroomViewMode::MeshProjection:			return TEXT("MeshProjection");
	case EGroomViewMode::Coverage:					return TEXT("Coverage");
	case EGroomViewMode::MaterialDepth:				return TEXT("MaterialDepth");
	case EGroomViewMode::MaterialBaseColor:			return TEXT("MaterialBaseColor");
	case EGroomViewMode::MaterialRoughness:			return TEXT("MaterialRoughness");
	case EGroomViewMode::MaterialSpecular:			return TEXT("MaterialSpecular");
	case EGroomViewMode::MaterialTangent:			return TEXT("MaterialTangent");
	case EGroomViewMode::Tile:						return TEXT("Tile");
	case EGroomViewMode::SimHairStrands:			return TEXT("SimHairStrands");
	case EGroomViewMode::RenderHairStrands:			return TEXT("RenderHairStrands");
	case EGroomViewMode::RootUV:					return TEXT("RootUV");
	case EGroomViewMode::RootUDIM:					return TEXT("RootUDIM");
	case EGroomViewMode::UV:						return TEXT("UV");
	case EGroomViewMode::Seed:						return TEXT("Seed");
	case EGroomViewMode::Dimension:					return TEXT("Dimension");
	case EGroomViewMode::RadiusVariation:			return TEXT("RadiusVariation");
	case EGroomViewMode::Color:						return TEXT("Color");
	case EGroomViewMode::Roughness:					return TEXT("Roughness");
	case EGroomViewMode::AO:						return TEXT("AO");
	case EGroomViewMode::ClumpID:					return TEXT("ClumpID");
	case EGroomViewMode::Cluster:					return TEXT("Cluster");
	case EGroomViewMode::ClusterAABB:				return TEXT("ClusterAABB");
	case EGroomViewMode::Tangent:					return TEXT("Tangent");
	case EGroomViewMode::ControlPoints:				return TEXT("ControlPoints");
	case EGroomViewMode::Group:						return TEXT("Group");
	case EGroomViewMode::LODColoration:				return TEXT("LODColoration");
	case EGroomViewMode::CardGuides:				return TEXT("CardGuides");
	case EGroomViewMode::Memory:					return TEXT("Memory");
	case EGroomViewMode::Bounds:					return TEXT("Bounds");
	}
	return TEXT("None");
}

static FGroomVisualizationData GGroomVisualizationData;

static FString ConfigureConsoleCommand(FGroomVisualizationData::TModeMap& ModeMap)
{
	FString AvailableVisualizationModes;
	for (FGroomVisualizationData::TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FGroomVisualizationData::FModeRecord& Record = It.Value();
		AvailableVisualizationModes += FString("\n  ");
		AvailableVisualizationModes += Record.ModeString;
	}

	FString Out;
	Out = TEXT("When the viewport view-mode is set to 'Groom Visualization', this command specifies which of the various channels to display. Values entered other than the allowed values shown below will be ignored.");
	Out += AvailableVisualizationModes;

	IConsoleManager::Get().RegisterConsoleVariable(
		FGroomVisualizationData::GetVisualizeConsoleCommandName(),
		0,
		*Out,
		ECVF_Cheat);

	return Out;
}

static void AddVisualizationMode(
	FGroomVisualizationData::TModeMap& ModeMap,
	bool DefaultComposited,
	const EGroomViewMode Mode,
	const FText& ModeText,
	const FText& ModeDesc)
{
	const TCHAR* ModeString = GetGroomViewModeName(Mode);
	const FName ModeName = FName(ModeString);

	FGroomVisualizationData::FModeRecord& Record = ModeMap.Emplace(ModeName);
	Record.ModeString			= FString(ModeString);
	Record.ModeName				= ModeName;
	Record.ModeText				= ModeText;
	Record.ModeDesc				= ModeDesc;
	Record.Mode					= Mode;
	Record.DefaultComposited	= DefaultComposited;
}

void FGroomVisualizationData::Initialize()
{
	UE::TScopeLock Lock(Mutex);
	if (!bIsInitialized && IsGroomEnabled())
	{
		AddVisualizationMode(ModeMap, true, EGroomViewMode::None,						LOCTEXT("NoneDebug", "None"),								LOCTEXT("NoneDebugDesc", "No debug mode"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MacroGroups,				LOCTEXT("MacroGroups", "Instances"),						LOCTEXT("MacroGroupsDesc", "Instances info"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::LightBounds,				LOCTEXT("LightBounds", "Light Bound"),						LOCTEXT("LightBoundsDesc", "All DOMs light bounds"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MacroGroupScreenRect,		LOCTEXT("MacroGroupScreenRect", "Screen Bounds"),			LOCTEXT("MacroGroupScreenRectDesc", "Screen projected instances"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::DeepOpacityMaps,			LOCTEXT("DeepOpacityMaps", "Deep Shadows"),					LOCTEXT("DeepOpacityMapsDesc", "Deep opacity maps"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::SamplePerPixel,				LOCTEXT("SamplePerPixel", "Sample Per Pixel"),				LOCTEXT("SamplePerPixelDesc", "Sub-pixel sample count"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::TAAResolveType,				LOCTEXT("TAAResolveType", "AA Type"),						LOCTEXT("TAAResolveTypeDesc", "TAA resolve type (regular/responsive)"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::CoverageType,				LOCTEXT("CoverageType", "Coverage Type"),					LOCTEXT("CoverageTypeDesc", "Type of hair coverage - Fully covered : Green / Partially covered : Red"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::VoxelsDensity,				LOCTEXT("VoxelsDensity", "Voxels"),							LOCTEXT("VoxelsDensityDesc", "Hair density volume"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MeshProjection,				LOCTEXT("MeshProjection", "Root Bindings"),					LOCTEXT("MeshProjectionDesc", "Hair mesh projection"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Coverage,					LOCTEXT("Coverage", "Coverage"),							LOCTEXT("CoverageDesc", "Hair coverage"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MaterialDepth,				LOCTEXT("MaterialDepth", "Depth"),							LOCTEXT("MaterialDepthDesc", "Hair material depth"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MaterialBaseColor,			LOCTEXT("MaterialBaseColor", "BaseColor"),					LOCTEXT("MaterialBaseColorDesc", "Hair material base color"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MaterialRoughness,			LOCTEXT("MaterialRoughness", "Roughness"),					LOCTEXT("MaterialRoughnessDesc", "Hair material roughness"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MaterialSpecular,			LOCTEXT("MaterialSpecular", "Specular"),					LOCTEXT("MaterialSpecularDesc", "Hair material specular"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::MaterialTangent,			LOCTEXT("MaterialTangent", "Tangent"),						LOCTEXT("MaterialTangentDesc", "Hair material tangent"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Tile,						LOCTEXT("Tile", "Tile"),									LOCTEXT("TileDesc", "Hair tile cotegorization"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::SimHairStrands,				LOCTEXT("SimHairStrands", "Guides"),						LOCTEXT("SimHairStrandsDesc", "Simulation strands"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::RenderHairStrands,			LOCTEXT("RenderHairStrands", "Strands Guides Influences"),	LOCTEXT("RenderHairStrandsDesc", "Rendering strands influences"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::CardGuides, 				LOCTEXT("CardGuides", "Cards Guides"),						LOCTEXT("CardGuidesDesc", "Cards Guides"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::RootUV,						LOCTEXT("RootUV", "Root UV"),								LOCTEXT("RootUVDesc", "Roots UV"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::RootUDIM,					LOCTEXT("RootUDIM", "Root UDIM"),							LOCTEXT("RootUDIMDesc", "Roots UV UDIM texture index"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::UV,							LOCTEXT("UV", "UV"),										LOCTEXT("UVDesc", "Hair UV"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Seed,						LOCTEXT("Seed", "Seed"),									LOCTEXT("SeedDesc", "Hair seed"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Dimension,					LOCTEXT("Dimension", "Dimension"),							LOCTEXT("DimensionDesc", "Hair dimensions"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::RadiusVariation,			LOCTEXT("RadiusVariation", "Radius Variation"),				LOCTEXT("RadiusVariationDesc", "Hair radius variation"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Tangent,					LOCTEXT("Tangent", "Tangent"),								LOCTEXT("TangentDesc", "Hair tangent"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::ControlPoints,				LOCTEXT("ControlPoints", "Control Points"),					LOCTEXT("ControlPointsDesc", "Hair control points"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Color,						LOCTEXT("Color", "Per-CV Color"),							LOCTEXT("ColorDesc", "Control point color (optional)"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Roughness,					LOCTEXT("Roughness", "Per-CV Roughness"),					LOCTEXT("RoughnessDesc", "Control point roughness  (optional)"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Cluster,					LOCTEXT("Cluster", "Clusters"),								LOCTEXT("ClusterDesc", "Hair clusters"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::ClusterAABB,				LOCTEXT("ClusterAABB", "Clusters Bounds"),					LOCTEXT("ClusterAABBDesc", "Hair clusters AABBs"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Group,						LOCTEXT("Group", "Groups"),									LOCTEXT("GroupDesc", "Hair groups"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::LODColoration,				LOCTEXT("LODColoration", "LOD Color"),						LOCTEXT("LODColorationDesc", "Hair LOD coloring"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::AO,							LOCTEXT("AO", "Per-CV AO"),									LOCTEXT("AODesc", "Control point AO  (optional)"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::ClumpID,					LOCTEXT("ClumpID", "Clump ID"),								LOCTEXT("ClumpIDDesc", "Curve clumpID  (optional)"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Memory,						LOCTEXT("Memory", "Memory"),								LOCTEXT("MemoryDesc", "Memory"));
		AddVisualizationMode(ModeMap, true, EGroomViewMode::Bounds,						LOCTEXT("Bounds", "Bounds"),								LOCTEXT("BoundsDesc", "Bounds"));

		ConsoleDocumentationVisualizationMode = ConfigureConsoleCommand(ModeMap);
	}
	bIsInitialized = true;
}

FText FGroomVisualizationData::GetModeDisplayName(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeText;
	}
	else
	{
		return FText::GetEmpty();
	}
}

EGroomViewMode FGroomVisualizationData::GetViewMode(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->Mode;
	}
	else
	{
		return EGroomViewMode::None;
	}
}

bool FGroomVisualizationData::GetModeDefaultComposited(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->DefaultComposited;
	}
	else
	{
		return false;
	}
}

FGroomVisualizationData& GetGroomVisualizationData()
{
	if (!GGroomVisualizationData.IsInitialized())
	{
		GGroomVisualizationData.Initialize();
	}

	return GGroomVisualizationData;
}

#undef LOCTEXT_NAMESPACE
