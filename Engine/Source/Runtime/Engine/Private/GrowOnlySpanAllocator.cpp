// Copyright Epic Games, Inc. All Rights Reserved.

#include "GrowOnlySpanAllocator.h"
#include "CoreGlobals.h"


// Allocate a range.  Returns allocated StartOffset.
int32 FGrowOnlySpanAllocator::Allocate(int32 Num)
{
	const int32 FoundIndex = SearchFreeList(Num);

	// Use an existing free span if one is found
	if (FoundIndex != INDEX_NONE)
	{
		FLinearAllocation FreeSpan = FreeSpans[FoundIndex];

		if (FreeSpan.Num > Num)
		{
			// Update existing free span with remainder
			FreeSpans[FoundIndex] = FLinearAllocation(FreeSpan.StartOffset + Num, FreeSpan.Num - Num);
		}
		else
		{
			// Fully consumed the free span
			FreeSpans.RemoveAtSwap(FoundIndex);
		}
				
		return FreeSpan.StartOffset;
	}

	// New allocation
	int32 StartOffset = MaxSize;
	MaxSize = MaxSize + Num;

	return StartOffset;
}

// Free an already allocated range.  
void FGrowOnlySpanAllocator::Free(int32 BaseOffset, int32 Num)
{
	check(BaseOffset + Num <= MaxSize);

	FLinearAllocation NewFreeSpan(BaseOffset, Num);

	// If we are deferring consolidation, just store the item, merging happens on allocate
	if (bDeferMerges)
	{
		FreeSpans.Add(NewFreeSpan);
		return;
	}

#if DO_GUARD_SLOW
	// Detect double delete
	for (int32 i = 0; i < FreeSpans.Num(); i++)
	{
		FLinearAllocation CurrentSpan = FreeSpans[i];
		checkSlow(!(CurrentSpan.Contains(NewFreeSpan)));
	}
#endif

	int32 SpanBeforeIndex = INDEX_NONE;
	int32 SpanAfterIndex = INDEX_NONE;

	// Search for existing free spans we can merge with
	for (int32 i = 0; i < FreeSpans.Num(); i++)
	{
		FLinearAllocation CurrentSpan = FreeSpans[i];

		if (CurrentSpan.StartOffset == NewFreeSpan.StartOffset + NewFreeSpan.Num)
		{
			SpanAfterIndex = i;
		}

		if (CurrentSpan.StartOffset + CurrentSpan.Num == NewFreeSpan.StartOffset)
		{
			SpanBeforeIndex = i;
		}
	}

	if (SpanBeforeIndex != INDEX_NONE)
	{
		// Merge span before with new free span
		FLinearAllocation& SpanBefore = FreeSpans[SpanBeforeIndex];
		SpanBefore.Num += NewFreeSpan.Num;

		if (SpanAfterIndex != INDEX_NONE)
		{
			// Also merge span after with span before
			FLinearAllocation SpanAfter = FreeSpans[SpanAfterIndex];
			SpanBefore.Num += SpanAfter.Num;
			FreeSpans.RemoveAtSwap(SpanAfterIndex);
		}
	}
	else if (SpanAfterIndex != INDEX_NONE)
	{
		// Merge span after with new free span
		FLinearAllocation& SpanAfter = FreeSpans[SpanAfterIndex];
		SpanAfter.StartOffset = NewFreeSpan.StartOffset;
		SpanAfter.Num += NewFreeSpan.Num;
	}
	else 
	{
		// Couldn't merge, store new free span
		FreeSpans.Add(NewFreeSpan);
	}
}


void FGrowOnlySpanAllocator::BeginDeferMerges()
{
	check(!bDeferMerges);
	bDeferMerges = true;

}
void FGrowOnlySpanAllocator::EndDeferMerges()
{
	check(bDeferMerges);
	bDeferMerges = false;

	// Consolidation 
	// 1. Sort the Free list by span start
	FreeSpans.Sort();

	// alternate free list, used during consolidation to avoid N^2 worst case, retained to avoid re-allocations.
	TArray<FLinearAllocation, TInlineAllocator<10>> FreeSpansTmp;
	FreeSpansTmp.Reset(FreeSpans.Num());

	int32 PrevEndOffset = INDEX_NONE;

	// 2. Loop over and fuse all adjacent, copy into new free list (to avoid compaction)
	for (int32 Index = 0; Index < FreeSpans.Num(); ++Index)
	{
		FLinearAllocation Alloc = FreeSpans[Index];
		// Continues the previous one, fuse
		if (PrevEndOffset == Alloc.StartOffset)
		{
			FreeSpansTmp.Last().Num += Alloc.Num;
		}
		else
		{
			FreeSpansTmp.Add(Alloc);
		}
		PrevEndOffset = FreeSpansTmp.Last().Num + FreeSpansTmp.Last().StartOffset;
	}
	// 3. Swap old and new free lists
	Swap(FreeSpans, FreeSpansTmp);
}


int32 FGrowOnlySpanAllocator::SearchFreeList(int32 Num)
{
	// Search free list for first matching
	for (int32 i = 0; i < FreeSpans.Num(); i++)
	{
		FLinearAllocation CurrentSpan = FreeSpans[i];

		if (CurrentSpan.Num >= Num)
		{
			return i;
		}
	}

	return INDEX_NONE;
}
