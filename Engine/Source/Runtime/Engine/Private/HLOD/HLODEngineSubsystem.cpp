// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLOD/HLODEngineSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODEngineSubsystem)

#if WITH_EDITOR

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDependencyGatherer.h"
#include "EngineUtils.h"
#include "Engine/LODActor.h"
#include "Engine/HLODProxy.h"
#include "Engine/World.h"
#include "Editor.h"
#include "UnrealEngine.h"
#include "HierarchicalLOD.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "UObject/ObjectSaveContext.h"

void UHLODEngineSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	bDisableHLODCleanupOnLoad = false;
	bDisableHLODSpawningOnLoad = false;

	Super::Initialize(Collection);
	RegisterRecreateLODActorsDelegates();
}

void UHLODEngineSubsystem::Deinitialize()
{
	UnregisterRecreateLODActorsDelegates();
	Super::Deinitialize();
}

void UHLODEngineSubsystem::DisableHLODCleanupOnLoad(bool bInDisableHLODCleanup)
{
	bDisableHLODCleanupOnLoad = bInDisableHLODCleanup;
}

void UHLODEngineSubsystem::DisableHLODSpawningOnLoad(bool bInDisableHLODSpawning)
{
	bDisableHLODSpawningOnLoad = bInDisableHLODSpawning;

	UnregisterRecreateLODActorsDelegates();
	RegisterRecreateLODActorsDelegates();
}

void UHLODEngineSubsystem::OnSaveLODActorsToHLODPackagesChanged()
{
	UnregisterRecreateLODActorsDelegates();
	RegisterRecreateLODActorsDelegates();
}

void UHLODEngineSubsystem::UnregisterRecreateLODActorsDelegates()
{
	FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitializationDelegateHandle);
	FWorldDelegates::LevelAddedToWorld.Remove(OnLevelAddedToWorldDelegateHandle);
	FEditorDelegates::PreSaveWorldWithContext.Remove(OnPreSaveWorlDelegateHandle);
}

void UHLODEngineSubsystem::RegisterRecreateLODActorsDelegates()
{
	if (GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages && !bDisableHLODSpawningOnLoad)
	{
		OnPostWorldInitializationDelegateHandle = FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UHLODEngineSubsystem::RecreateLODActorsForWorld);
		OnLevelAddedToWorldDelegateHandle = FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UHLODEngineSubsystem::RecreateLODActorsForLevel);
		OnPreSaveWorlDelegateHandle = FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &UHLODEngineSubsystem::OnPreSaveWorld);
	}
}

void UHLODEngineSubsystem::RecreateLODActorsForWorld(UWorld* InWorld, const UWorld::InitializationValues InInitializationValues)
{
	// For each level in this world
	for (ULevel* Level : InWorld->GetLevels())
	{
		RecreateLODActorsForLevel(Level, InWorld);
	}
}

void UHLODEngineSubsystem::RecreateLODActorsForLevel(ULevel* InLevel, UWorld* InWorld)
{
	bool bShouldRecreateActors = InWorld && !InWorld->bIsTearingDown && !InWorld->IsPreviewWorld();
	if (!bShouldRecreateActors)
	{
		return;
	}

	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	// First, destroy invalid HLOD actors. If needed, they will be recreated below.
	if (!bDisableHLODCleanupOnLoad)
	{
		CleanupHLODs(InLevel);
	}

	// Legacy LODActors are not necessary anymore when using World Partition
	if (InWorld->IsPartitionedWorld())
	{
		return;
	}

	// Look for HLODProxy packages associated with this level
	int32 NumLODLevels = InLevel->GetWorldSettings()->GetHierarchicalLODSetup().Num();
	for (int32 LODIndex = 0; LODIndex < NumLODLevels; ++LODIndex)
	{
		// Obtain HLOD package for the current HLOD level
		UHLODProxy* HLODProxy = Utilities->RetrieveLevelHLODProxy(InLevel, LODIndex);
		if (HLODProxy)
		{
			// Spawn LODActors from the HLODDesc, if any is found
			HLODProxy->SpawnLODActors(InLevel);
		}
	}
}

bool UHLODEngineSubsystem::CleanupHLODs(ULevel* InLevel)
{
	bool bPerformedCleanup = false;

	for (AActor* Actor : InLevel->Actors)
	{
		if (ALODActor* LODActor = Cast<ALODActor>(Actor))
		{
			bPerformedCleanup |= CleanupHLOD(LODActor);
		}
	}

	return bPerformedCleanup;
}

bool UHLODEngineSubsystem::CleanupHLODs(UWorld* InWorld)
{
	bool bPerformedCleanup = false;

	for (TActorIterator<ALODActor> It(InWorld); It; ++It)
	{
		bPerformedCleanup |= CleanupHLOD(*It);
	}

	return bPerformedCleanup;
}

bool UHLODEngineSubsystem::CleanupHLOD(ALODActor* InLODActor)
{
	bool bShouldDestroyActor = false;

	if (InLODActor->GetLevel()->GetWorldSettings()->IsPartitionedWorld())
	{
		UE_LOGF(LogEngine, Warning, "Deleting legacy LODActor %ls found in World Partition map. Resave %ls to silence warning.", *InLODActor->GetName(), *InLODActor->GetOutermost()->GetPathName());
		bShouldDestroyActor = true;
	}
	if (InLODActor->GetLevel()->GetWorldSettings()->GetHierarchicalLODSetup().Num() == 0)
	{
		UE_LOGF(LogEngine, Warning, "Deleting LODActor %ls found in map with no HLOD setup or disabled HLOD system. Resave %ls to silence warning.", *InLODActor->GetName(), *InLODActor->GetOutermost()->GetPathName());
		bShouldDestroyActor = true;
	}
	else if (!InLODActor->GetProxy() || InLODActor->GetProxy()->GetMap() != TSoftObjectPtr<UWorld>(InLODActor->GetLevel()->GetTypedOuter<UWorld>()))
	{
		UE_LOGF(LogEngine, Warning, "Deleting LODActor %ls with invalid HLODProxy. Resave %ls to silence warning.", *InLODActor->GetName(), *InLODActor->GetOutermost()->GetPathName());
		bShouldDestroyActor = true;
	}
	else if (GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages)
	{
		// LODActors will be recreated from the data contained in the HLODProxy packages
		bShouldDestroyActor = true;
	}

	if (bShouldDestroyActor)
	{
		InLODActor->GetWorld()->EditorDestroyActor(InLODActor, true);
		ReleaseUniquelyNamedObject(InLODActor);
	}

	return bShouldDestroyActor;
}

void UHLODEngineSubsystem::OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext)
{
	// When cooking, make sure that the LODActors are not transient
	if (InWorld && InWorld->PersistentLevel && ObjectSaveContext.IsCooking())
	{
		for (AActor* Actor : InWorld->PersistentLevel->Actors)
		{
			if (ALODActor* LODActor = Cast<ALODActor>(Actor))
			{
				if (LODActor->WasBuiltFromHLODDesc())
				{
					EObjectFlags TransientFlags = EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient;
					if (LODActor->HasAnyFlags(TransientFlags))
					{
						LODActor->ClearFlags(TransientFlags);

						ForEachObjectWithOuter(LODActor, [TransientFlags](UObject* Subobject)
						{
							Subobject->ClearFlags(TransientFlags);
						}, EGetObjectsFlags::IncludeNestedObjects);
					}
				}
			}
		}
	}
}

class FHLODDependencyGatherer : public IAssetDependencyGatherer
{
public:
	virtual void GatherDependencies(FGatherDependenciesContext& Context) const override;
};

void FHLODDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Context) const
{
	if (!GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages)
	{
		return;
	}

	// Record a dependency on the paths to HLODProxy packages that can be associated with
	// this level if they exist
	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	// TODO: GetHLODPackageName is usually constructed from the World's packagename, but this can be replaced by ULevelStreaming::PackageNameToLoad
	// We need to write the list of LevelStreaming PackageNameToLoad into the AssetData so we can read it from here.
	FString Wildcard = Utilities->GetWildcardOfHLODPackagesForPackage(Context.GetAssetData().PackageName.ToString());
	FARFilter PossibleAssetsFilter;
	PossibleAssetsFilter.PackagePaths.Add(FName(FPathViews::GetPath(Wildcard)));
	TArray<FAssetData> FilteredAssets;
	Context.GetAssetRegistryState().GetAssets(Context.CompileFilter(PossibleAssetsFilter), {}, FilteredAssets);
	FString PackageName;
	for (const FAssetData& FilteredAsset : FilteredAssets)
	{
		FilteredAsset.PackageName.ToString(PackageName);
		if (PackageName.MatchesWildcard(Wildcard))
		{
			Context.GetOutDependencies().Emplace(IAssetDependencyGatherer::FGathereredDependency{ FilteredAsset.PackageName,
				UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build });
		}
	}
}
REGISTER_ASSETDEPENDENCY_GATHERER(FHLODDependencyGatherer, UWorld);

#endif // WITH_EDITOR

