// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLOD/HLODProxyDesc.h"
#include "Engine/World.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODProxyDesc)

#if WITH_EDITOR
#include "Engine/LODActor.h"
#include "Engine/Level.h"
#include "GameFramework/WorldSettings.h"
#include "LevelUtils.h"
#include "Engine/LevelStreaming.h"
#include "Materials/MaterialInstanceDynamic.h"
#endif

#if WITH_EDITOR

FHLODISMComponentDesc::FHLODISMComponentDesc(const UInstancedStaticMeshComponent* InISMComponent)
{
	StaticMesh = InISMComponent->GetStaticMesh();
	Material = InISMComponent->GetMaterial(0);

	Instances.Reset(InISMComponent->GetInstanceCount());

	const int32 NumCustomDataFloats = InISMComponent->NumCustomDataFloats;

	for (int32 InstanceIndex = 0; InstanceIndex < InISMComponent->GetInstanceCount(); ++InstanceIndex)
	{
		FTransform InstanceTransform;
		InISMComponent->GetInstanceTransform(InstanceIndex, InstanceTransform);
		Instances.Emplace(InstanceTransform);

		if (NumCustomDataFloats > 0)
		{
			if (ensure(InISMComponent->PerInstanceSMCustomData.IsValidIndex(InstanceIndex * NumCustomDataFloats)))
			{
				FCustomPrimitiveData InstanceCustomData;
				InstanceCustomData.Data.SetNumUninitialized(NumCustomDataFloats);
				
				check(InISMComponent->PerInstanceSMCustomData.GetTypeSize() == InstanceCustomData.Data.GetTypeSize());

				void* Dest = 
				FMemory::Memcpy(InstanceCustomData.Data.GetData(), &InISMComponent->PerInstanceSMCustomData[InstanceIndex * NumCustomDataFloats], NumCustomDataFloats * InstanceCustomData.Data.GetTypeSize());

				InstancesCustomPrimitiveData.Emplace(MoveTemp(InstanceCustomData));
			}
		}
	}
}

bool FHLODISMComponentDesc::operator==(const FHLODISMComponentDesc& Other) const
{
	if (StaticMesh != Other.StaticMesh)
	{
		return false;
	}

	UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
	UMaterialInstance* OtherMaterialInstance = Cast<UMaterialInstance>(Other.Material);
	if (MaterialInstance && OtherMaterialInstance)
	{
		if (!MaterialInstance->Equivalent(OtherMaterialInstance))
		{
			return false;
		}
	}
	else if (Material != Other.Material)
	{
		return false;
	}

	if (Instances.Num() != Other.Instances.Num())
	{
		return false;
	}

	for (int32 InstanceIdx = 0; InstanceIdx < Instances.Num(); ++InstanceIdx)
	{
		const float Tolerance = 0.1f;
		if (!Instances[InstanceIdx].Equals(Other.Instances[InstanceIdx], Tolerance))
		{
			return false;
		}
	}

	if (InstancesCustomPrimitiveData != Other.InstancesCustomPrimitiveData)
	{
		return false;
	}

	return true;
}

FTransform RemoveStreamingLevelTransform(ULevel* InLevel, const FTransform InTransform)
{
	ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(InLevel);
	if (StreamingLevel)
	{
		return InTransform.GetRelativeTransform(StreamingLevel->LevelTransform);
	}

	return InTransform;
}

bool UHLODProxyDesc::UpdateFromLODActor(const ALODActor* InLODActor)
{
	// Check if there's any difference between the LODActor & its description
	if (!ShouldUpdateDesc(InLODActor))
	{
		return false;
	}

	// A difference was detected, copy all parameters from the LODActor
	
	Modify();

	SubActors.Reset(InLODActor->SubActors.Num());
	SubHLODDescs.Reset();
	
	for (AActor* SubActor : InLODActor->SubActors)
	{
		if (ALODActor* SubLODActor = Cast<ALODActor>(SubActor))
		{
			check(SubLODActor->ProxyDesc);
			SubHLODDescs.Emplace(SubLODActor->ProxyDesc);
		}
		else if (SubActor)
		{
			SubActors.Emplace(SubActor->GetFName());
		}
	}

	// Sort the arrays to ensure a stable order for comparisons
	SubActors.Sort(FNameLexicalLess());
	SubHLODDescs.Sort(FSoftObjectPtrLexicalLess());

	StaticMesh = InLODActor->StaticMeshComponent ? InLODActor->StaticMeshComponent->GetStaticMesh() : nullptr;

	const TMap<FHLODInstancingKey, TObjectPtr<UInstancedStaticMeshComponent>>& ISMComponents = InLODActor->InstancedStaticMeshComponents;
	ISMComponentsDesc.Reset(ISMComponents.Num());
	for (auto const& Pair : ISMComponents)
	{
		if (Pair.Key.IsValid() && Pair.Value->GetInstanceCount() != 0)
		{
			FHLODISMComponentDesc& ISMComponentDesc = ISMComponentsDesc.Emplace_GetRef(Pair.Value);

			// MIDs are not assets and are normally outered to their owner component.
			// We need to duplicate them here to make sure we don't create references to actors in the source level.
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(ISMComponentDesc.Material))
			{ 
				ISMComponentDesc.Material = DuplicateObject<UMaterialInstanceDynamic>(MID, this);
			}
		}
	}

	LODDrawDistance = InLODActor->GetDrawDistance();
	bOverrideMaterialMergeSettings = InLODActor->bOverrideMaterialMergeSettings;
	MaterialSettings = InLODActor->MaterialSettings;
	bOverrideTransitionScreenSize = InLODActor->bOverrideTransitionScreenSize;
	TransitionScreenSize = InLODActor->TransitionScreenSize;
	bOverrideScreenSize = InLODActor->bOverrideScreenSize;
	ScreenSize = InLODActor->ScreenSize;

	LODLevel = InLODActor->LODLevel;
	LODActorTag = InLODActor->LODActorTag;

	Location = RemoveStreamingLevelTransform(InLODActor->GetLevel(), FTransform(InLODActor->GetActorLocation())).GetTranslation();

	HLODBakingTransform = InLODActor->GetWorldSettings()->HLODBakingTransform;

	return true;
}

bool UHLODProxyDesc::ShouldUpdateDesc(const ALODActor* InLODActor) const
{
	TArray<FName> LocalSubActors;
	TArray<TSoftObjectPtr<UHLODProxyDesc>> LocalSubHLODDescs;

	LocalSubActors.Reset(InLODActor->SubActors.Num());
	for (AActor* SubActor : InLODActor->SubActors)
	{
		if (ALODActor* SubLODActor = Cast<ALODActor>(SubActor))
		{
			check(SubLODActor->ProxyDesc);
			LocalSubHLODDescs.Emplace(SubLODActor->ProxyDesc);
		}
		else if (SubActor)
		{
			LocalSubActors.Emplace(SubActor->GetFName());
		}
	}

	// Sort the arrays to ensure a stable order for the comparisons below
	LocalSubActors.Sort(FNameLexicalLess());
	LocalSubHLODDescs.Sort(FSoftObjectPtrLexicalLess());

	if (LocalSubActors != SubActors)
	{
		return true;
	}

	if (LocalSubHLODDescs != SubHLODDescs)
	{
		return true;
	}

	UStaticMesh* LocalStaticMesh = InLODActor->StaticMeshComponent ? ToRawPtr(InLODActor->StaticMeshComponent->GetStaticMesh()) : nullptr;
	if (StaticMesh != LocalStaticMesh)
	{
		return true;
	}

	TArray<FHLODISMComponentDesc> LocalISMComponentsDesc;
	const TMap<FHLODInstancingKey, TObjectPtr<UInstancedStaticMeshComponent>>& ISMComponents = InLODActor->InstancedStaticMeshComponents;
	LocalISMComponentsDesc.Reset(ISMComponents.Num());
	for (auto const& Pair : ISMComponents)
	{
		if (Pair.Key.IsValid())
		{
			LocalISMComponentsDesc.Emplace(Pair.Value);
		}
	}

	if (LocalISMComponentsDesc != ISMComponentsDesc)
	{
		return true;
	}

	if (LODDrawDistance != InLODActor->GetDrawDistance())
	{
		return true;
	}

	if (bOverrideMaterialMergeSettings != InLODActor->bOverrideMaterialMergeSettings)
	{
		return true;
	}

	if (MaterialSettings != InLODActor->MaterialSettings)
	{
		return true;
	}

	if (bOverrideTransitionScreenSize != InLODActor->bOverrideTransitionScreenSize)
	{
		return true;
	}

	if (TransitionScreenSize != InLODActor->TransitionScreenSize)
	{
		return true;
	}
	
	if (bOverrideScreenSize != InLODActor->bOverrideScreenSize)
	{
		return true;
	}
	
	if (ScreenSize != InLODActor->ScreenSize)
	{
		return true;
	}

	if (LODLevel != InLODActor->LODLevel)
	{
		return true;
	}

	if (LODActorTag != InLODActor->LODActorTag)
	{
		return true;
	}

	FVector LODActorLocation = RemoveStreamingLevelTransform(InLODActor->GetLevel(), FTransform(InLODActor->GetActorLocation())).GetTranslation();
	const float Tolerance = 0.1f;
	if (!Location.Equals(LODActorLocation, Tolerance))
	{
		return true;
	}

	if (!HLODBakingTransform.Equals(InLODActor->GetWorldSettings()->HLODBakingTransform))
	{
		return true;
	}

	return false;
}

ALODActor* UHLODProxyDesc::SpawnLODActor(ULevel* InLevel) const
{
	const bool bWasWorldPackageDirty = InLevel->GetOutermost()->IsDirty();

	FActorSpawnParameters ActorSpawnParameters;
	ActorSpawnParameters.Name = MakeUniqueObjectName(InLevel, ALODActor::StaticClass());
	ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	ActorSpawnParameters.OverrideLevel = InLevel;
	ActorSpawnParameters.bHideFromSceneOutliner = true;
	ActorSpawnParameters.ObjectFlags = EObjectFlags::RF_Transient | EObjectFlags::RF_DuplicateTransient;

	FTransform ActorTransform(Location);

	// If level is a streamed level with a transform and the transform was already applied,
	// make sure to spawn this new LODActor with a proper transform.
	if (InLevel->bAlreadyMovedActors)
	{
		ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(InLevel);
		if (StreamingLevel)
		{
			ActorTransform = ActorTransform * StreamingLevel->LevelTransform;
		}
	}

	ALODActor* LODActor = InLevel->GetWorld()->SpawnActor<ALODActor>(ALODActor::StaticClass(), ActorTransform, ActorSpawnParameters);
	if (!LODActor)
	{
		return nullptr;
	}

	LODActor->SetStaticMesh(StaticMesh);

	for (const FHLODISMComponentDesc& ISMComponentDesc : ISMComponentsDesc)
	{
		UStaticMesh* ISMStaticMesh = ISMComponentDesc.StaticMesh;
		UMaterialInterface* ISMMaterial = ISMComponentDesc.Material;

		if (!ISMStaticMesh || !ISMMaterial || ISMComponentDesc.Instances.IsEmpty())
		{
			continue;
		}

		if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(ISMMaterial))
		{
			ISMMaterial = DuplicateObject<UMaterialInstanceDynamic>(MID, LODActor);
		}
		
		// Apply transform to HISM instances
		const bool bTransformInstances = !ActorTransform.Equals(FTransform::Identity);
		if (bTransformInstances)
		{
			TArray<FTransform> Transforms = ISMComponentDesc.Instances;
			for (FTransform& Transform : Transforms)
			{
				Transform *= ActorTransform;
			}

			LODActor->AddInstances(ISMStaticMesh, ISMMaterial, Transforms, ISMComponentDesc.InstancesCustomPrimitiveData);
		}
		else
		{
			LODActor->AddInstances(ISMStaticMesh, ISMMaterial, ISMComponentDesc.Instances, ISMComponentDesc.InstancesCustomPrimitiveData);
		}
	}

	LODActor->SetDrawDistance(LODDrawDistance);
	LODActor->bOverrideMaterialMergeSettings = bOverrideMaterialMergeSettings;
	LODActor->MaterialSettings = MaterialSettings;
	LODActor->bOverrideTransitionScreenSize = bOverrideTransitionScreenSize;
	LODActor->TransitionScreenSize = TransitionScreenSize;
	LODActor->bOverrideScreenSize = bOverrideScreenSize;
	LODActor->ScreenSize = ScreenSize;
	LODActor->Key = Key;
	LODActor->LODLevel = LODLevel;
	LODActor->LODActorTag = LODActorTag;
	
	LODActor->CachedNumHLODLevels = InLevel->GetWorldSettings()->GetNumHierarchicalLODLevels();

	TArray<AActor*> SubActorsToAdd;
	SubActorsToAdd.Reset(SubActors.Num());

	// Add sub LODActors spawned from SubActorsDescs
	for (AActor* Actor : InLevel->Actors)
	{
		if (ALODActor* SubLODActor = Cast<ALODActor>(Actor))
		{
			if (SubLODActor->ProxyDesc && SubHLODDescs.Contains(TSoftObjectPtr<UHLODProxyDesc>(SubLODActor->ProxyDesc.Get())))
			{
				check(SubLODActor != LODActor);
				SubActorsToAdd.Add(SubLODActor);
			}
		}
	}

	// Find all subactors from the level
	Algo::Transform(SubActors, SubActorsToAdd, [InLevel, LODActor](const FName& ActorName)
	{
		AActor* Actor = FindObjectFast<AActor>(InLevel, ActorName);
		check(Actor != LODActor);
		return Actor;
	});

	// Remove null entries
	SubActorsToAdd.RemoveAll([](AActor* Actor) { return Actor == nullptr; });

	LODActor->AddSubActors(SubActorsToAdd);

	LODActor->ProxyDesc = const_cast<UHLODProxyDesc*>(this);
	LODActor->bBuiltFromHLODDesc = true;

	// Don't dirty the level file after spawning a transient actor
	if (!bWasWorldPackageDirty)
	{
		InLevel->GetOutermost()->SetDirtyFlag(false);
	}

	LODActor->GetWorldSettings()->HLODBakingTransform = HLODBakingTransform;

	return LODActor;
}

#endif // #if WITH_EDITOR

