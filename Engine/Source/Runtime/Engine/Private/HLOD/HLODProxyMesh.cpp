// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLOD/HLODProxyMesh.h"
#include "Engine/LODActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODProxyMesh)

FHLODProxyMesh::FHLODProxyMesh()
	: StaticMesh(nullptr)
{
}

#if WITH_EDITOR
FHLODProxyMesh::FHLODProxyMesh(ALODActor* InLODActor, UStaticMesh* InStaticMesh, const FName& InKey)
	: LODActor(InLODActor)
	, StaticMesh(InStaticMesh)
	, Key(InKey)
{
}

FHLODProxyMesh::FHLODProxyMesh(UStaticMesh* InStaticMesh, const FName& InKey)
	: StaticMesh(InStaticMesh)
	, Key(InKey)
{
}

bool FHLODProxyMesh::operator==(const FHLODProxyMesh& InHLODProxyMesh) const
{
	return LODActor == InHLODProxyMesh.LODActor &&
		   StaticMesh == InHLODProxyMesh.StaticMesh &&
		   Key == InHLODProxyMesh.Key;
}
#endif

const UStaticMesh* FHLODProxyMesh::GetStaticMesh() const
{
	return StaticMesh;
}

const TLazyObjectPtr<ALODActor>& FHLODProxyMesh::GetLODActor() const
{
	return LODActor;
}

const FName& FHLODProxyMesh::GetKey() const
{
	return Key;
}

