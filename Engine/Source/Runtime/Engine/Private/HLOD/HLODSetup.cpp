// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLOD/HLODSetup.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODSetup)


FHierarchicalSimplification::FHierarchicalSimplification()
	: TransitionScreenSize(0.315f)
	, OverrideDrawDistance(10000)
	, bUseOverrideDrawDistance(false)
	, bAllowSpecificExclusion(false)
	, bOnlyGenerateClustersForVolumes(false)
	, bReusePreviousLevelClusters(false)
	, SimplificationMethod(EHierarchicalSimplificationMethod::Merge)
	, DesiredBoundRadius(2000)
	, DesiredFillingPercentage(50)
	, MinNumberOfActorsToBuild(2)
{
	MergeSetting.bMergeMaterials = true;
	MergeSetting.bGenerateLightMapUV = true;
	ProxySetting.MaterialSettings.MaterialMergeType = EMaterialMergeType::MaterialMergeType_Simplygon;
	ProxySetting.bCreateCollision = false;
}

#if WITH_EDITORONLY_DATA

bool FHierarchicalSimplification::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FHierarchicalSimplification::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::HierarchicalSimplificationMethodEnumAdded)
		{
			SimplificationMethod = bSimplifyMesh_DEPRECATED ? EHierarchicalSimplificationMethod::Simplify : EHierarchicalSimplificationMethod::Merge;
		}
	}
}

#endif

FMaterialProxySettings* FHierarchicalSimplification::GetSimplificationMethodMaterialSettings()
{
	switch (SimplificationMethod)
	{
	case EHierarchicalSimplificationMethod::Merge:
		return &MergeSetting.MaterialSettings;

	case EHierarchicalSimplificationMethod::Simplify:
		return &ProxySetting.MaterialSettings;

	case EHierarchicalSimplificationMethod::Approximate:
		return &ApproximateSettings.MaterialSettings;
	}

	return nullptr;
}

UHierarchicalLODSetup::UHierarchicalLODSetup()
{
	HierarchicalLODSetup.AddDefaulted();
	OverrideBaseMaterial = nullptr;
}
