// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/HLODProxy.h"

#include "Concepts/StaticStructProvider.h"
#include "Engine/Level.h"
#include "Engine/LODActor.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "HLOD/HLODProxyDesc.h"
#include "HLOD/HLODSetup.h"
#include "Materials/Material.h"

#if WITH_EDITOR
#include "Algo/ForEach.h"
#include "DerivedDataCacheKey.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/ArchiveCrc32.h"
#include "HierarchicalLOD.h"
#include "ObjectTools.h"
#endif
#include "Materials/MaterialInstanceConstant.h"
#include "Engine/Texture.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshComponentLODInfo.h"
#include "LevelUtils.h"
#include "Engine/LevelStreaming.h"
#include "StaticMeshResources.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODProxy)

DEFINE_LOG_CATEGORY_STATIC(LogHLODProxy, Log, All);

#if WITH_EDITOR

void UHLODProxy::SetMap(const UWorld* InMap)
{
	OwningMap = const_cast<UWorld*>(InMap);
}

TSoftObjectPtr<UWorld> UHLODProxy::GetMap() const
{
	return OwningMap;
}

UHLODProxyDesc* UHLODProxy::AddLODActor(ALODActor* InLODActor)
{
	check(InLODActor->ProxyDesc == nullptr);

	// Create a new HLODProxyDesc and populate it from the provided InLODActor.
	UHLODProxyDesc* HLODProxyDesc = NewObject<UHLODProxyDesc>(this);
	HLODProxyDesc->UpdateFromLODActor(InLODActor);

	InLODActor->Proxy = this;
	InLODActor->ProxyDesc = HLODProxyDesc;
	InLODActor->bBuiltFromHLODDesc = true;

	HLODActors.Emplace(HLODProxyDesc);

	MarkPackageDirty();

	return HLODProxyDesc;
}

void UHLODProxy::AddMesh(ALODActor* InLODActor, UStaticMesh* InStaticMesh, const FName& InKey)
{
	// If the Save LOD Actors to HLOD packages feature is enabled, ensure that if a LODActor hasn't been rebuilt yet with
	// the feature on that we can still update its mesh properly.
	if (GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages && HLODActors.Contains(InLODActor->ProxyDesc))
	{
		check(InLODActor->Proxy == this);
		HLODActors[InLODActor->ProxyDesc] = FHLODProxyMesh(InStaticMesh, InKey);
		InLODActor->UpdateProxyDesc();
	}
	else
	{
		InLODActor->Proxy = this;
		FHLODProxyMesh NewProxyMesh(InLODActor, InStaticMesh, InKey);
		ProxyMeshes.AddUnique(NewProxyMesh);
	}
}

void UHLODProxy::Clean()
{
	// The level we reference must be loaded to clean this package
	check(OwningMap.IsNull() || OwningMap.ToSoftObjectPath().ResolveObject() != nullptr);

	// Remove all entries that have invalid actors
	int32 NumRemoved = ProxyMeshes.RemoveAll([this](const FHLODProxyMesh& InProxyMesh)
	{ 
		TLazyObjectPtr<ALODActor> LODActor = InProxyMesh.GetLODActor();

		bool bRemoveEntry = false;

		// Invalid actor means that it has been deleted so we shouldnt hold onto its data
		if(!LODActor.IsValid())
		{
			bRemoveEntry = true;
		}
		else if(LODActor.Get()->Proxy == nullptr)
		{
			// No proxy means we are also invalid
			bRemoveEntry = true;
		}
		else if(!LODActor.Get()->Proxy->ContainsDataForActor(LODActor.Get()))
		{
			// actor and proxy are valid, but key differs (unbuilt)
			bRemoveEntry = true;
		}

		if (bRemoveEntry)
		{
			RemoveAssets(InProxyMesh);
		}

		return bRemoveEntry;
	});

	// Ensure the HLOD descs are up to date.
	if (GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages)
	{
		UWorld* World = Cast<UWorld>(OwningMap.ToSoftObjectPath().ResolveObject());

		// Don't check HLODs for levels that are not visible, as they haven't got any LODActors yet, which means they'll always get cleaned out and dirtied.
		if (World && World->PersistentLevel->bIsVisible)
		{
			UpdateHLODDescs(World->PersistentLevel);
		}
	}
	else if (HLODActors.Num() > 0)
	{
		for (auto ItHLODActor = HLODActors.CreateIterator(); ItHLODActor; ++ItHLODActor)
		{
			RemoveAssets(ItHLODActor.Value());
		}

		HLODActors.Empty();
		Modify();
	}
}

bool UHLODProxy::IsEmpty() const
{
	return HLODActors.Num() == 0 && ProxyMeshes.Num() == 0;
}

void UHLODProxy::DeletePackage()
{
	UPackage* Package = GetOutermost();

	// Must not destroy objects during iteration, so gather a list
	TArray<UObject*> ObjectsToDestroy;
	ForEachObjectWithOuter(Package, [&ObjectsToDestroy](UObject* InObject)
	{
		ObjectsToDestroy.Add(InObject);
	});

	// Perform destruction
	for (UObject* ObjectToDestroy : ObjectsToDestroy)
	{
		DestroyObject(ObjectToDestroy);
	}

	ObjectTools::DeleteObjectsUnchecked({ Package });
}

void UHLODProxy::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (!OwningMap.IsValid())
	{
		return;
	}

	// Always rebuild key on save here.
	// We don't do this while cooking as keys rely on platform derived data which is context-dependent during cook
	if (!ObjectSaveContext.IsCooking())
	{
		if (GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages)
		{
			if (UWorld* World = Cast<UWorld>(OwningMap.ToSoftObjectPath().ResolveObject()))
			{
				for (AActor* Actor : World->PersistentLevel->Actors)
				{
					if (ALODActor* LODActor = Cast<ALODActor>(Actor))
					{
						if (LODActor->ProxyDesc && LODActor->ProxyDesc->GetOutermost() == GetOutermost())
						{
							LODActor->ProxyDesc->Key = UHLODProxy::GenerateKeyForActor(LODActor);
						}
					}
				}
			}
		}
	}
}

void UHLODProxy::UpdateHLODDescs(const ULevel* InLevel)
{
	// Gather a map of all the HLODProxyDescs used by LODActors in the level
	TMap<const UHLODProxyDesc*, ALODActor*> LODActors;
	for (AActor* Actor : InLevel->Actors)
	{
		if (ALODActor* LODActor = Cast<ALODActor>(Actor))
		{
			if (LODActor->ProxyDesc && LODActor->ProxyDesc->GetOutermost() == GetOutermost())
			{
				LODActors.Emplace(LODActor->ProxyDesc, LODActor);
			}
		}
	}

	// For each HLODProxyDesc stored in this proxy, ensure that it is up to date with the associated LODActor
	// Purge the HLODProxyDesc that are unused (not referenced by any LODActor)
	for (auto ItHLODActor = HLODActors.CreateIterator(); ItHLODActor; ++ItHLODActor)
	{
		UHLODProxyDesc* HLODProxyDesc = ItHLODActor.Key();
		ALODActor** LODActor = LODActors.Find(HLODProxyDesc);
		if (LODActor)
		{
			HLODProxyDesc->UpdateFromLODActor(*LODActor);
		}
		else
		{
			// Remove assets associated with this actor
			RemoveAssets(ItHLODActor.Value());

			Modify();
			ItHLODActor.RemoveCurrent();
		}
	}
}

const AActor* UHLODProxy::FindFirstActor(const ALODActor* LODActor)
{
	auto RecursiveFindFirstActor = [&](const ALODActor* InLODActor)
	{
		const AActor* FirstActor = InLODActor->SubActors.IsValidIndex(0) ? ToRawPtr(InLODActor->SubActors[0]) : nullptr;
		while (FirstActor != nullptr && FirstActor->IsA<ALODActor>())
		{
			const ALODActor* SubLODActor = Cast<ALODActor>(FirstActor);
			if (SubLODActor)
			{
				FirstActor = SubLODActor->SubActors.IsValidIndex(0) ? ToRawPtr(SubLODActor->SubActors[0]) : nullptr; 
			}
			else
			{
				// Unable to find a valid actor
				FirstActor = nullptr;
			}
		}
		return FirstActor;
	};

	// Retrieve first 'valid' AActor (non ALODActor)
	const AActor* FirstValidActor = nullptr;

	for (int32 Index = 0; Index < LODActor->SubActors.Num(); ++Index)
	{
		const AActor* SubActor = LODActor->SubActors[Index];

		if (const ALODActor* SubLODActor = Cast<ALODActor>(SubActor))
		{
			SubActor = RecursiveFindFirstActor(SubLODActor);
		}

		if (SubActor != nullptr)
		{
			FirstValidActor = SubActor;
			break;
		}
	}

	return FirstValidActor;
}

void UHLODProxy::ExtractStaticMeshComponentsFromLODActor(const ALODActor* LODActor, TArray<UStaticMeshComponent*>& InOutComponents)
{
	check(LODActor);

	for (const AActor* ChildActor : LODActor->SubActors)
	{
		if(ChildActor)
		{
			TArray<UStaticMeshComponent*> ChildComponents;
			if (ChildActor->IsA<ALODActor>())
			{
				ExtractStaticMeshComponentsFromLODActor(Cast<ALODActor>(ChildActor), ChildComponents);
			}
			else
			{
				ChildActor->GetComponents(ChildComponents);
			}

			InOutComponents.Append(ChildComponents);
		}
	}
}

void UHLODProxy::ExtractComponents(const ALODActor* LODActor, TArray<UPrimitiveComponent*>& InOutComponents)
{
	check(LODActor);

	for (const AActor* Actor : LODActor->SubActors)
	{
		if(Actor)
		{
			TArray<UStaticMeshComponent*> Components;

			if (Actor->IsA<ALODActor>())
			{
				ExtractStaticMeshComponentsFromLODActor(Cast<ALODActor>(Actor), Components);
			}
			else
			{
				Actor->GetComponents(Components);
			}

			Components.RemoveAll([&](UStaticMeshComponent* Val)
			{
				return Val->GetStaticMesh() == nullptr || !Val->ShouldGenerateAutoLOD(LODActor->LODLevel - 1);
			});

			InOutComponents.Append(Components);
		}
	}
}

uint32 UHLODProxy::GetCRC(UMaterialInterface* InMaterialInterface, uint32 InCRC)
{
	FArchiveCrc32 Ar(InCRC);

	UMaterialInterface* MaterialInterface = InMaterialInterface;
	while(MaterialInterface)
	{
		// Walk material parent chain for instances with known states (we cant support MIDs directly as they are always changing)
		if(UMaterialInstance* MI = Cast<UMaterialInstance>(MaterialInterface))
		{
			if(UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(MI))
			{
				Ar << MIC->ParameterStateId;
			}
			MaterialInterface = MI->Parent;
		}
		else if(UMaterial* Material = Cast<UMaterial>(MaterialInterface))
		{
			Ar << Material->StateId;
			MaterialInterface = nullptr;
		}
	}

	return Ar.GetCrc();
}

uint32 UHLODProxy::GetCRC(UTexture* InTexture, uint32 InCRC)
{
	// Default to just the path name if we don't have render data
	 if (InTexture->GetRunningPlatformData() != nullptr)
     {
         FTexturePlatformData* PlatformData = *InTexture->GetRunningPlatformData();
         if (PlatformData != nullptr)
         {
			 if (const FString* KeyString = PlatformData->DerivedDataKey.TryGet<FString>())
			 {
				 return FCrc::StrCrc32(**KeyString, InCRC);
			 }
			 else if (const UE::DerivedData::FCacheKeyProxy* KeyProxy = PlatformData->DerivedDataKey.TryGet<UE::DerivedData::FCacheKeyProxy>())
			 {
				 const UE::DerivedData::FCacheKey* Key = KeyProxy->AsCacheKey();
				 FAnsiStringView BucketStringView = Key->Bucket.ToString();
				 FArchiveCrc32 Ar(InCRC);
				 Ar.Serialize(const_cast<ANSICHAR*>(BucketStringView.GetData()), BucketStringView.Len());
				 Ar.Serialize(const_cast<FIoHash*>(&Key->Hash), sizeof(FIoHash));
				 return Ar.GetCrc();
			 }
         }
     }
 
     // Default to just the path name if we don't have render data
     return FCrc::StrCrc32(*InTexture->GetPathName(), InCRC);
}

uint32 UHLODProxy::GetCRC(UStaticMesh* InStaticMesh, uint32 InCRC, bool bInConsiderPhysicData)
{
	FArchiveCrc32 Ar(InCRC);

	// Default to just the path name if we don't have render data
	FString DerivedDataKey = InStaticMesh->GetRenderData() ? InStaticMesh->GetRenderData()->DerivedDataKey : InStaticMesh->GetPathName();
	Ar << DerivedDataKey;

	int32 LightMapCoordinateIndex = InStaticMesh->GetLightMapCoordinateIndex();
	Ar << LightMapCoordinateIndex;

	if (bInConsiderPhysicData)
	{
		if (UBodySetup* BodySetup = InStaticMesh->GetBodySetup())
		{
			// Incorporate physics data
			Ar << InStaticMesh->GetBodySetup()->BodySetupGuid;
		}
	}

	return Ar.GetCrc();
}

static void AppendRoundedTransform(const FRotator& ComponentRotation, const FVector& ComponentLocation, const FVector& ComponentScale, FArchive& Ar)
{
	// Include transform - round sufficiently to ensure stability
	FIntVector Location(FMath::RoundToInt(ComponentLocation.X), FMath::RoundToInt(ComponentLocation.Y), FMath::RoundToInt(ComponentLocation.Z));
	Ar << Location;

	FRotator Rotator(ComponentRotation.GetDenormalized());
	const int32 MAX_DEGREES = 360;
	FIntVector Rotation(FMath::RoundToInt(Rotator.Pitch) % MAX_DEGREES, FMath::RoundToInt(Rotator.Yaw) % MAX_DEGREES, FMath::RoundToInt(Rotator.Roll) % MAX_DEGREES);
	Ar << Rotation;

	const float SCALE_FACTOR = 100;
	FIntVector Scale(FMath::RoundToInt(ComponentScale.X * SCALE_FACTOR), FMath::RoundToInt(ComponentScale.Y * SCALE_FACTOR), FMath::RoundToInt(ComponentScale.Z * SCALE_FACTOR));
	Ar << Scale;
}

static void AppendRoundedTransform(const FTransform& InTransform, FArchive& Ar)
{
	AppendRoundedTransform(InTransform.Rotator(), InTransform.GetLocation(), InTransform.GetScale3D(), Ar);
}

uint32 UHLODProxy::GetCRC(const FTransform& InTransform, uint32 InCRC)
{
	FArchiveCrc32 Ar(InCRC);
	AppendRoundedTransform(InTransform, Ar);
	return Ar.GetCrc();
}

uint32 UHLODProxy::GetCRC(UStaticMeshComponent* InComponent, uint32 InCRC, const FTransform& TransformComponents)
{
	FArchiveCrc32 Ar(InCRC);

	FVector  ComponentLocation = InComponent->GetComponentLocation();
	FRotator ComponentRotation = InComponent->GetComponentRotation();
	FVector  ComponentScale = InComponent->GetComponentScale();

	ComponentLocation = TransformComponents.TransformPosition(ComponentLocation);
	ComponentRotation = TransformComponents.TransformRotation(ComponentRotation.Quaternion()).Rotator();
	AppendRoundedTransform(ComponentRotation, ComponentLocation, ComponentScale, Ar);
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - Transform = %x", Ar.GetCrc());

	// Include other relevant properties
	Ar << InComponent->ForcedLodModel;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - ForcedLodModel = %x", Ar.GetCrc());

	Ar << InComponent->HLODBatchingPolicy;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - HLODBatchingPolicy = %x", Ar.GetCrc());

	bool bCastShadow = InComponent->CastShadow;
	Ar << bCastShadow;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - bCastShadow = %x", Ar.GetCrc());

	bool bCastStaticShadow = InComponent->bCastStaticShadow;
	Ar << bCastStaticShadow;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - bCastStaticShadow = %x", Ar.GetCrc());

	bool bCastDynamicShadow = InComponent->bCastDynamicShadow;
	Ar << bCastDynamicShadow;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - bCastDynamicShadow = %x", Ar.GetCrc());

	bool bCastFarShadow = InComponent->bCastFarShadow;
	Ar << bCastFarShadow;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - bCastFarShadow = %x", Ar.GetCrc());

	int32 Width, Height;
	InComponent->GetLightMapResolution(Width, Height);
	Ar << Width;
	Ar << Height;
	UE_LOGF(LogHLODProxy, VeryVerbose, "         - LightMapResolution = %x", Ar.GetCrc());

	if (!InComponent->GetCustomPrimitiveData().Data.IsEmpty())
	{
		Ar << InComponent->GetCustomPrimitiveData();
		UE_LOGF(LogHLODProxy, VeryVerbose, "         - CustomPrimitiveData = %x", Ar.GetCrc());
	}
	
	// incorporate vertex colors
	for(FStaticMeshComponentLODInfo& LODInfo : InComponent->LODData)
	{
		if(LODInfo.OverrideVertexColors)
		{
			Ar.Serialize((uint8*)LODInfo.OverrideVertexColors->GetVertexData(), LODInfo.OverrideVertexColors->GetNumVertices() * LODInfo.OverrideVertexColors->GetStride());
			UE_LOGF(LogHLODProxy, VeryVerbose, "         - OverrideVertexColors = %x", Ar.GetCrc());
		}
	}

	// Include instances data in case of a ISMC
	if (UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		for (const FInstancedStaticMeshInstanceData& InstanceData : ISMC->PerInstanceSMData)
		{
			AppendRoundedTransform(FTransform(InstanceData.Transform), Ar);
			UE_LOGF(LogHLODProxy, VeryVerbose, "         - Instance Transform = %x", Ar.GetCrc());
		}

		Ar << ISMC->PerInstanceSMCustomData;
		UE_LOGF(LogHLODProxy, VeryVerbose, "         - PerInstanceSMCustomData = %x", Ar.GetCrc());

		Ar << ISMC->InstancingRandomSeed;
		UE_LOGF(LogHLODProxy, VeryVerbose, "         - InstancingRandomSeed = %x", Ar.GetCrc());

		for (FInstancedStaticMeshRandomSeed& ISMCRandomSeed : ISMC->AdditionalRandomSeeds)
		{
			Ar << ISMCRandomSeed.StartInstanceIndex;
			UE_LOGF(LogHLODProxy, VeryVerbose, "         - StartInstanceIndex = %x", Ar.GetCrc());

			Ar << ISMCRandomSeed.RandomSeed;
			UE_LOGF(LogHLODProxy, VeryVerbose, "         - RandomSeed = %x", Ar.GetCrc());
		}
	}

	return Ar.GetCrc();
}

// Key that forms the basis of the HLOD proxy key. Bump this key (i.e. generate a new GUID) when you want to force a rebuild of ALL HLOD proxies
#define HLOD_PROXY_BASE_KEY		TEXT("20D42B32B31542FDA1DAF99D4DA83601")

FName UHLODProxy::GenerateKeyForActor(const ALODActor* LODActor, bool bMustUndoLevelTransform)
{
	FString Key = HLOD_PROXY_BASE_KEY;

	UWorld* LODActorWorld = LODActor->GetLevel()->GetTypedOuter<UWorld>();
	const TArray<FHierarchicalSimplification>& HierarchicalLODSetups = LODActorWorld->GetWorldSettings()->GetHierarchicalLODSetup();

	// If the HLOD aren't created with collision data, no need to consider this when computing the checksum
	bool bConsiderPhysicData = false;
	if (HierarchicalLODSetups.IsValidIndex(LODActor->LODLevel - 1))
	{
		const FHierarchicalSimplification& HLODSettings = HierarchicalLODSetups[LODActor->LODLevel - 1];
		switch (HLODSettings.SimplificationMethod)
		{
		case EHierarchicalSimplificationMethod::Simplify:
			bConsiderPhysicData = HLODSettings.ProxySetting.bCreateCollision;
			break;
		case EHierarchicalSimplificationMethod::Merge:
			bConsiderPhysicData = HLODSettings.MergeSetting.bMergePhysicsData;
			break;
		default:
			bConsiderPhysicData = false;
		}
	}
	
	// Base us off the unique object ID
	{
		const UObject* Obj = LODActor->ProxyDesc ? ToRawPtr(Cast<const UObject>(LODActor->ProxyDesc)) : Cast<const UObject>(LODActor);
		FUniqueObjectGuid ObjectGUID = FUniqueObjectGuid::GetOrCreateIDForObject(Obj);
		Key += TEXT("_");
		Key += ObjectGUID.GetGuid().ToString(EGuidFormats::Digits);
	}

	// Accumulate a bunch of settings into a CRC
	{
		uint32 CRC = 0;

		// Get the HLOD settings CRC
		{
			TArray<FHierarchicalSimplification>& BuildLODLevelSettings = LODActor->GetLevel()->GetWorldSettings()->GetHierarchicalLODSetup();
			if(BuildLODLevelSettings.IsValidIndex(LODActor->LODLevel - 1))
			{
				FArchiveCrc32 Ar(CRC);
				Ar << BuildLODLevelSettings[LODActor->LODLevel - 1];
				CRC = Ar.GetCrc();
			}
		}

		// HLODBakingTransform
		CRC = GetCRC(LODActor->GetLevel()->GetWorldSettings()->HLODBakingTransform, CRC);

		// screen size + override
		{
			if(LODActor->bOverrideScreenSize)
			{
				CRC = FCrc::MemCrc32(&LODActor->ScreenSize, sizeof(float), CRC);
			}
		}

		// material merge settings override
		{
			if (LODActor->bOverrideMaterialMergeSettings)
			{
				FArchiveCrc32 Ar(CRC);
				Ar << const_cast<ALODActor*>(LODActor)->MaterialSettings;
				CRC = Ar.GetCrc();
			}
		}

		Key += TEXT("_");
		Key += BytesToHex((uint8*)&CRC, sizeof(uint32));
	}

	// get the base material CRC
	{
		UMaterialInterface* BaseMaterial = LODActor->GetLevel()->GetWorldSettings()->GetHierarchicalLODBaseMaterial();
		uint32 CRC = GetCRC(BaseMaterial);
		Key += TEXT("_");
		Key += BytesToHex((uint8*)&CRC, sizeof(uint32));
	}

	// We get the CRC of the first actor name and various static mesh components
	{
		uint32 CRC = 0;

		TArray<UPrimitiveComponent*> Components;
		ExtractComponents(LODActor, Components);
		
		// Components can be offset by their streaming level transform. Undo that transform to have the same signature
		// when computing CRC for a sub level or a persistent level.
		FTransform TransformComponents = FTransform::Identity;
		if (bMustUndoLevelTransform)
		{
			ULevelStreaming* SteamingLevel = FLevelUtils::FindStreamingLevel(LODActor->GetLevel());
			if (SteamingLevel)
			{
				TransformComponents = SteamingLevel->LevelTransform.Inverse();
			}
		}

		// We get the CRC of each component
		TArray<uint32> ComponentsCRCs;
		for(UPrimitiveComponent* Component : Components)
		{
			if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
			{
				uint32 ComponentCRC = 0;

				// CRC component
				ComponentCRC = GetCRC(StaticMeshComponent, ComponentCRC, TransformComponents);

				if(UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
				{
					// CRC static mesh
					ComponentCRC = GetCRC(StaticMesh, ComponentCRC, bConsiderPhysicData);

					// CRC materials
					const int32 NumMaterials = StaticMeshComponent->GetNumMaterials();
					for(int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
					{
						UMaterialInterface* MaterialInterface = StaticMeshComponent->GetMaterial(MaterialIndex);
						if (MaterialInterface)
						{
							ComponentCRC = GetCRC(MaterialInterface, ComponentCRC);

							TArray<UTexture*> Textures;
							MaterialInterface->GetUsedTextures(Textures);
							for (UTexture* Texture : Textures)
							{
								ComponentCRC = GetCRC(Texture, ComponentCRC);
							}
						}
					}
				}

				ComponentsCRCs.Add(ComponentCRC);
			}
		}

		// Sort the components CRCs to ensure the order of components won't have an impact on the final CRC
		ComponentsCRCs.Sort();

		// Append all components CRCs
		for (uint32 ComponentCRC : ComponentsCRCs)
		{
			CRC = HashCombineFast(CRC, ComponentCRC);
		}

		Key += TEXT("_");
		Key += BytesToHex((uint8*)&CRC, sizeof(uint32));
	}

	// Mesh reduction method
	{
		// NOTE: This mimics code in the editor only FMeshReductionManagerModule::StartupModule(). If that changes then this should too
		FString HLODMeshReductionModuleName;
		GConfig->GetString(TEXT("/Script/Engine.ProxyLODMeshSimplificationSettings"), TEXT("r.ProxyLODMeshReductionModule"), HLODMeshReductionModuleName, GEngineIni);
		// If nothing was requested, default to simplygon for mesh merging reduction
		if (HLODMeshReductionModuleName.IsEmpty())
		{
			HLODMeshReductionModuleName = TEXT("SimplygonMeshReduction");
		}

		Key += TEXT("_");
		Key += HLODMeshReductionModuleName;
	}

	return FName(*Key);
}

void UHLODProxy::SpawnLODActors(ULevel* InLevel)
{
	for (const auto& Pair : HLODActors)
	{
		// Spawn LODActor
		ALODActor* LODActor = Pair.Key->SpawnLODActor(InLevel);
		if (LODActor)
		{
			LODActor->Proxy = this;
		}
	}
}

void UHLODProxy::PostLoad()
{
	Super::PostLoad();

	// PKG_ContainsMapData required so FEditorFileUtils::GetDirtyContentPackages can treat this as a map package
	GetOutermost()->SetPackageFlags(PKG_ContainsMapData);
}

void UHLODProxy::DestroyObject(UObject* InObject)
{
	if (IsValid(InObject))
	{
		InObject->MarkPackageDirty();

		InObject->ClearFlags(RF_Public | RF_Standalone);
		InObject->SetFlags(RF_Transient);
		InObject->Rename(nullptr, GetTransientPackage());
		InObject->MarkAsGarbage();
	
		if (InObject->IsRooted())
		{
			InObject->RemoveFromRoot();
		}
	}
}

void UHLODProxy::RemoveAssets(const FHLODProxyMesh& ProxyMesh)
{
	UPackage* Outermost = GetOutermost();

	// Destroy the static mesh
	UStaticMesh* StaticMesh = const_cast<UStaticMesh*>(ProxyMesh.GetStaticMesh());
	if (StaticMesh)
	{
		FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(StaticMesh);

		// Destroy every materials
		for (const FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
		{
			UMaterialInterface* Material = StaticMaterial.MaterialInterface;

			if (Material)
			{
				// Destroy every textures
				TArray<UTexture*> Textures;
				Material->GetUsedTextures(Textures);
				for (UTexture* Texture : Textures)
				{
					if (Texture->GetOutermost() == Outermost)
					{
						DestroyObject(Texture);
					}
				}

				if (Material->GetOutermost() == Outermost)
				{
					DestroyObject(Material);
				}
			}
		}

		if (StaticMesh->GetOutermost() == Outermost)
		{
			DestroyObject(StaticMesh);

			Algo::ForEach(RecreateRenderStateContext.GetComponentsUsingMesh(StaticMesh), [](UStaticMeshComponent* Component)
			{
				Component->SetStaticMesh(nullptr);
			});
		}
	}
}

bool UHLODProxy::SetHLODBakingTransform(const FTransform& InTransform)
{
	bool bChanged = false;

	int32 NewTransformCRC = GetCRC(InTransform);

	for (auto ItHLODActor = HLODActors.CreateIterator(); ItHLODActor; ++ItHLODActor)
	{
		UHLODProxyDesc* HLODProxyDesc = ItHLODActor.Key();

		int32 OldTransformCRC = GetCRC(HLODProxyDesc->HLODBakingTransform);
		if (OldTransformCRC != NewTransformCRC)
		{
			HLODProxyDesc->HLODBakingTransform = InTransform;
			bChanged = true;
		}
	}

	return bChanged;
}

#endif // #if WITH_EDITOR

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR

bool UHLODProxy::ContainsDataForActor(const ALODActor* InLODActor) const
{
#if WITH_EDITOR
	FName Key;

	// Only re-generate the key in non-PIE worlds
	if(InLODActor->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		Key = InLODActor->GetKey();
	}
	else
	{
		Key = GenerateKeyForActor(InLODActor);
	}
#else
	FName Key = InLODActor->GetKey();
#endif

	if(Key == NAME_None)
	{
		return false;
	}

	for (const auto& Pair : HLODActors)
	{
		const FHLODProxyMesh& ProxyMesh = Pair.Value;
		if(ProxyMesh.GetKey() == Key)
		{
			return true;
		}
	}

	for(const FHLODProxyMesh& ProxyMesh : ProxyMeshes)
	{
		if(ProxyMesh.GetKey() == Key)
		{
			return true;
		}
	}

	return false;
}

#endif

