// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HModel.cpp: HModel implementation
=============================================================================*/

#include "HModel.h"
#include "SceneView.h"
#include "Model.h"
#include "Raster.h"
#include "Engine/Polys.h"
#include "Engine/Brush.h"
#include "GenericPlatform/ICursor.h"

IMPLEMENT_HIT_PROXY(HModel,HHitProxy);


/**
 * The state of the model hit test.
 */
class FModelHitState
{
public:

	uint32 SurfaceIndex;
	float SurfaceW;
	bool bHitSurface;

	int32 HitX;
	int32 HitY;

	/** Initialization constructor. */
	FModelHitState(int32 InHitX,int32 InHitY):
		SurfaceIndex(INDEX_NONE),
		SurfaceW(FLT_MAX),
		bHitSurface(false),
		HitX(InHitX),
		HitY(InHitY)
	{}
};

/**
 * A rasterization policy which is used to determine the BSP surface clicked on.
 */
class FModelHitRasterPolicy
{
public:
	typedef FVector4 InterpolantType;

	FModelHitRasterPolicy(uint32 InSurfaceIndex,FModelHitState& InHitState):
		SurfaceIndex(InSurfaceIndex),
		HitState(InHitState)
	{}

	void ProcessPixel(int32 X,int32 Y,const FVector4& Vertex,bool BackFacing)
	{
		const float InvW = 1.0f / Vertex.W;
		if(!BackFacing && InvW < HitState.SurfaceW)
		{
			HitState.SurfaceW = InvW;
			HitState.SurfaceIndex = SurfaceIndex;
			HitState.bHitSurface = true;
		}
	}

	int32 GetMinX() const { return HitState.HitX; }
	int32 GetMaxX() const { return HitState.HitX; } //-V524
	int32 GetMinY() const { return HitState.HitY; }
	int32 GetMaxY() const { return HitState.HitY; } //-V524

private:
	uint32 SurfaceIndex;
	FModelHitState& HitState;
};

bool HModel::ResolveSurface(const FSceneView* View,int32 X,int32 Y,uint32& OutSurfaceIndex) const
{
	FModelHitState HitState(X,Y);

	for(int32 NodeIndex = 0;NodeIndex < Model->Nodes.Num();NodeIndex++)
	{
		FBspNode& Node = Model->Nodes[NodeIndex];
		FBspSurf& Surf = Model->Surfs[Node.iSurf];

		bool bIsHidden = false;
#if WITH_EDITOR
		bIsHidden = (Surf.Actor && Surf.Actor->IsHiddenEd());
#endif
		const bool bIsPortal = (Surf.PolyFlags & PF_Portal) != 0;
		if(!bIsPortal && !bIsHidden)
		{
			// Convert the BSP node to a FPoly.
			FPoly NodePolygon;
			for(int32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
			{
				NodePolygon.Vertices.Add(Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex]);
			}

			// Clip the node's FPoly against the view's near clipping plane.
			if(	!View->bHasNearClippingPlane ||
				NodePolygon.Split(-FVector3f(View->NearClippingPlane),FVector3f(View->NearClippingPlane * View->NearClippingPlane.W)))
			{
				for(int32 LeadingVertexIndex = 2;LeadingVertexIndex < NodePolygon.Vertices.Num();LeadingVertexIndex++)
				{
					const int32 TriangleVertexIndices[3] = { 0, LeadingVertexIndex, LeadingVertexIndex - 1 };
					FVector4 Vertices[3];
					for(uint32 VertexIndex = 0;VertexIndex < 3;VertexIndex++)
					{
						FVector4 ScreenPosition = View->WorldToScreen((FVector)NodePolygon.Vertices[TriangleVertexIndices[VertexIndex]]);
						float InvW = 1.0f / ScreenPosition.W;
						float SizeX = View->UnscaledViewRect.Width();
						float SizeY = View->UnscaledViewRect.Height();
						Vertices[VertexIndex] = FVector4(
							View->UnscaledViewRect.Min.X + (0.5f + ScreenPosition.X * 0.5f * InvW) * SizeX,
							View->UnscaledViewRect.Min.Y + (0.5f - ScreenPosition.Y * 0.5f * InvW) * SizeY,
							ScreenPosition.Z,
							InvW
							);
					}

					const FVector4 EdgeA = Vertices[2] - Vertices[0];
					const FVector4 EdgeB = Vertices[1] - Vertices[0];
					FTriangleRasterizer<FModelHitRasterPolicy> Rasterizer(FModelHitRasterPolicy((uint32)Node.iSurf,HitState));
					Rasterizer.DrawTriangle(
						Vertices[0],
						Vertices[1],
						Vertices[2],
						FVector2D(Vertices[0].X,Vertices[0].Y),
						FVector2D(Vertices[1].X,Vertices[1].Y),
						FVector2D(Vertices[2].X,Vertices[2].Y),
						(Surf.PolyFlags & PF_TwoSided) ? false : (EdgeA.X * EdgeB.Y - EdgeA.Y * EdgeB.X < 0.0f) // Check if a surface is twosided when it is selected in the editor viewport
						);
				}
			}
		}
	}

	OutSurfaceIndex = HitState.SurfaceIndex;
	return HitState.bHitSurface;
}

void HModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Component);
	Collector.AddReferencedObject(Model);
}

EMouseCursor::Type HModel::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}
