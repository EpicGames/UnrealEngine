// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HUD.cpp: Heads up Display related functionality
=============================================================================*/

#include "GameFramework/HUD.h"
#include "Engine/GameViewportClient.h"
#include "Materials/MaterialInterface.h"
#include "SceneView.h"
#include "Engine/Engine.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "GlobalRenderResources.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/GameModeBase.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Components/LineBatchComponent.h"
#include "Engine/Canvas.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "DisplayDebugHelpers.h"
#include "DrawDebugHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/MovementComponent.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HUD)

DEFINE_LOG_CATEGORY_STATIC(LogHUD, Log, All);

#define LOCTEXT_NAMESPACE "HUD"

FOnShowDebugInfo AHUD::OnShowDebugInfo;
FOnHUDPostRender AHUD::OnHUDPostRender;

// Should we visualize the safe zone?
TAutoConsoleVariable<int32> GSafeZoneVisualizationModeCVar(
	TEXT("r.DebugSafeZone.Mode"),
	0,
	TEXT("The safe zone visualization mode (0..1)\n")
	TEXT(" 0: Disabled (default)\n")
	TEXT(" 1: Show Title Safe Zone"));

// How opaque should the safe zone visualization be?
TAutoConsoleVariable<float> GSafeZoneVisualizationAlphaCVar(
	TEXT("r.DebugSafeZone.OverlayAlpha"),
	0.2f,
	TEXT("The alpha value of the safe zone overlay (0..1)\n")
	TEXT(" default: 0.2"));

TAutoConsoleVariable<int32> GMaxDebugTextStringsPerActorCVar(
	TEXT("r.DebugSafeZone.MaxDebugTextStringsPerActor"),
	128,
	TEXT("The maximum number of debug strings that can be attached to a given actor (<=0 : no limit)"));

#if ENABLE_DRAW_DEBUG
TAutoConsoleVariable<int32> GDrawCurrentDebugTargetBoundingBox(
	TEXT("r.Debug.DrawCurrentDebugTargetBoundingBox"),
	1,
	TEXT("Draw the bounding box of the currently selected debug target (Default: 1)"));
#endif // ENABLE_DRAW_DEBUG

AHUD::AHUD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.TickGroup = TG_DuringPhysics;
	PrimaryActorTick.bCanEverTick = true;
	SetHidden(true);
	bReplicates = false;

	bLostFocusPaused = false;
	bShowHUD = true;

	SetCanBeDamaged(false);
	bEnableDebugTextShadow = false;
}

void AHUD::SetCanvas(class UCanvas* InCanvas, class UCanvas* InDebugCanvas)
{
	Canvas = InCanvas;
	DebugCanvas = InDebugCanvas;
}


void AHUD::Draw3DLine(FVector Start, FVector End, FColor LineColor)
{
	if (ULineBatchComponent* LineBatcher = GetWorld()->GetLineBatcher(UWorld::ELineBatcherType::World))
	{
		LineBatcher->DrawLine(Start, End, LineColor, SDPG_World);
	}
}

void AHUD::Draw2DLine(int32 X1, int32 Y1, int32 X2, int32 Y2, FColor LineColor)
{
	check(Canvas);

	FCanvasLineItem LineItem(FVector2D(X1, Y1), FVector2D(X2, Y2));
	LineItem.SetColor(LineColor);
	LineItem.Draw(Canvas->Canvas);
}

void AHUD::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	PlayerOwner = Cast<APlayerController>(GetOwner());

	// e.g. getting material pointers to control effects for gameplay
	NotifyBindPostProcessEffects();
}



void AHUD::NotifyBindPostProcessEffects()
{
	// overload with custom code e.g. getting material pointers to control effects for gameplay.
}

FVector2D AHUD::GetCoordinateOffset() const
{
	FVector2D Offset(0.f, 0.f);

	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(GetOwningPlayerController()->Player);

	if (LocalPlayer)
	{
		// Create a view family for the game viewport
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			LocalPlayer->ViewportClient->Viewport,
			GetWorld()->Scene,
			LocalPlayer->ViewportClient->EngineShowFlags)
			.SetRealtimeUpdate(true));

		// Calculate a view where the player is to update the streaming from the players start location
		FVector ViewLocation;
		FRotator ViewRotation;
		FSceneView* SceneView = LocalPlayer->CalcSceneView(&ViewFamily, /*out*/ ViewLocation, /*out*/ ViewRotation, LocalPlayer->ViewportClient->Viewport);

		if (SceneView)
		{
			Offset.X = -SceneView->UnscaledViewRect.Min.X; // And this will deal with the viewport offset if its a split screen
			Offset.Y = -SceneView->UnscaledViewRect.Min.Y;
		}
	}

	return Offset;
}

void AHUD::PostRender()
{
	// Theres nothing we can really do without a canvas or a world - so leave now in that case
	if ((GetWorld() == nullptr) || (Canvas == nullptr))
	{
		return;
	}
	// Set up delta time
	RenderDelta = GetWorld()->TimeSeconds - LastHUDRenderTime;

	if (PlayerOwner != nullptr)
	{
		// draw any debug text in real-time
		DrawDebugTextList();
	}

	if (bShowDebugInfo)
	{
		if (DebugCanvas)
		{
			DebugCanvas->DisplayDebugManager.Initialize(DebugCanvas, GEngine->GetTinyFont(), FVector2D(4.f, 50.f));
			ShowDebugInfo(DebugCanvas->DisplayDebugManager.GetMaxCharHeightRef(), DebugCanvas->DisplayDebugManager.GetYPosRef());
		}
	}
	else if (bShowHUD && FApp::CanEverRender())
	{
		DrawHUD();

		// No need to do work to determine hit box candidates if there will never be any
		if (HitBoxMap.Num() > 0)
		{
			ULocalPlayer* LocalPlayer = GetOwningPlayerController() ? Cast<ULocalPlayer>(GetOwningPlayerController()->Player) : nullptr;

			if (LocalPlayer && LocalPlayer->ViewportClient)
			{
				TArray<FVector2D> ContactPoints;

				if (!FSlateApplication::Get().IsFakingTouchEvents())
				{
					FVector2D MousePosition;
					if (LocalPlayer->ViewportClient->GetMousePosition(MousePosition))
					{
						ContactPoints.Add(MousePosition);
					}
				}

				for (int32 FingerIndex = 0; FingerIndex < EKeys::NUM_TOUCH_KEYS; ++FingerIndex)
				{
					FVector2f TouchLocation;
					bool bPressed = false;

					GetOwningPlayerController()->GetInputTouchState((ETouchIndex::Type)FingerIndex, TouchLocation.X, TouchLocation.Y, bPressed);

					if (bPressed)
					{
						ContactPoints.Add((FVector2D)TouchLocation);
					}
				}

				const FVector2D ContactPointOffset = GetCoordinateOffset();

				if (!ContactPointOffset.IsZero())
				{
					for (FVector2D& ContactPoint : ContactPoints)
					{
						ContactPoint += ContactPointOffset;
					}
				}
				UpdateHitBoxCandidates(MoveTemp(ContactPoints));
			}
		}
		else if (HitBoxesOver.Num() > 0)
		{
			// We still need to dispatch any end cursor over messages even if we don't have any hitboxes anymore
			for (const FName& HitBoxName : HitBoxesOver)
			{
				NotifyHitBoxEndCursorOver(HitBoxName);
			}
			HitBoxesOver.Reset();
		}
	}

	if (bShowHitBoxDebugInfo)
	{
		RenderHitBoxes(Canvas->Canvas);
	}

	DrawSafeZoneOverlay();

	OnHUDPostRender.Broadcast(this, DebugCanvas);

	LastHUDRenderTime = GetWorld()->TimeSeconds;
}

void AHUD::DrawActorOverlays(FVector Viewpoint, FRotator ViewRotation)
{
	// determine rendered camera position
	FVector ViewDir = ViewRotation.Vector();
	int32 i = 0;
	while (i < PostRenderedActors.Num())
	{
		if (PostRenderedActors[i] != nullptr)
		{
			PostRenderedActors[i]->PostRenderFor(PlayerOwner, Canvas, Viewpoint, ViewDir);
			i++;
		}
		else
		{
			PostRenderedActors.RemoveAt(i, 1);
		}
	}
}

void AHUD::DrawSafeZoneOverlay()
{
#if ENABLE_DRAW_DEBUG
	const int32 DebugSafeZoneMode = GSafeZoneVisualizationModeCVar.GetValueOnGameThread();

	if ((DebugSafeZoneMode > 0) && (DebugCanvas != nullptr) && (DebugCanvas->Canvas != nullptr))
	{

		const float Width = DebugCanvas->SizeX;
		const float Height = DebugCanvas->SizeY;
		FMargin SafeMargin;
		FSlateApplication::Get().GetSafeZoneSize(SafeMargin, FVector2D(Width, Height));
		const float UnsafeZoneAlpha = GSafeZoneVisualizationAlphaCVar.GetValueOnGameThread();
		const FLinearColor UnsafeZoneColor(1.0f, 0.5f, 0.5f, UnsafeZoneAlpha);

		const float HeightOfSides = Height - SafeMargin.GetTotalSpaceAlong<Orient_Vertical>();

		FCanvasTileItem TileItem(FVector2D::ZeroVector, GWhiteTexture, UnsafeZoneColor);
		TileItem.BlendMode = SE_BLEND_Translucent;

		// Top bar
		TileItem.Position = FVector2D::ZeroVector;
		TileItem.Size = FVector2D(Width, SafeMargin.Top);
		DebugCanvas->DrawItem(TileItem);

		// Bottom bar
		TileItem.Position = FVector2D(0.0f, Height - SafeMargin.Bottom);
		TileItem.Size = FVector2D(Width, SafeMargin.Bottom);
		DebugCanvas->DrawItem(TileItem);

		// Left bar
		TileItem.Position = FVector2D(0.0f, SafeMargin.Top);
		TileItem.Size = FVector2D(SafeMargin.Left, HeightOfSides);
		DebugCanvas->DrawItem(TileItem);

		// Right bar
		TileItem.Position = FVector2D(Width - SafeMargin.Right, SafeMargin.Top);
		TileItem.Size = FVector2D(SafeMargin.Right, HeightOfSides);
		DebugCanvas->DrawItem(TileItem);
	}
#endif
}

void AHUD::RemovePostRenderedActor(AActor* A)
{
	for (int32 i = 0; i < PostRenderedActors.Num(); i++)
	{
		if (PostRenderedActors[i] == A)
		{
			PostRenderedActors[i] = nullptr;
			return;
		}
	}
}

void AHUD::AddPostRenderedActor(AActor* A)
{
	// make sure that A is not already in list
	for (int32 i = 0; i < PostRenderedActors.Num(); i++)
	{
		if (PostRenderedActors[i] == A)
		{
			return;
		}
	}

	// add A at first empty slot
	for (int32 i = 0; i < PostRenderedActors.Num(); i++)
	{
		if (PostRenderedActors[i] == nullptr)
		{
			PostRenderedActors[i] = A;
			return;
		}
	}

	// no empty slot found, so grow array
	PostRenderedActors.Add(A);
}

void AHUD::ShowHUD()
{
	bShowHUD = !bShowHUD;
}

namespace ShowDebugNames
{
	static const FName Reset("Reset");
	static const FName HitBox("HitBox");
	static const FName Animation("Animation");
	static const FName Physics("Physics");
}

void AHUD::ShowDebug(FName DebugType)
{
	const bool bPreviousShowDebugInfo = bShowDebugInfo;

	if (DebugType == NAME_None)
	{
		bShowDebugInfo = !bShowDebugInfo;
	}
	else if (DebugType == ShowDebugNames::HitBox)
	{
		bShowHitBoxDebugInfo = !bShowHitBoxDebugInfo;
	}
	else if (DebugType == ShowDebugNames::Reset)
	{
		DebugDisplay.Reset();
		bShowDebugInfo = false;
		SaveConfig();
	}
	else
	{
		bool bRemoved = false;
		if (bShowDebugInfo)
		{
			// remove debugtype if already in array
			if (0 != DebugDisplay.Remove(DebugType))
			{
				bRemoved = true;
			}
		}
		if (!bRemoved)
		{
			// Since we prevent removing items when bShowDebugInfo is false, add them only once.
			DebugDisplay.AddUnique(DebugType);
		}

		bShowDebugInfo = true;
		SaveConfig();
	}

	// Reset Target to ourselves when enabled/disabled
	if (bShowDebugInfo != bPreviousShowDebugInfo)
	{
		ShowDebugTargetActor = nullptr;
	}
}

void AHUD::ShowDebugToggleSubCategory(FName Category)
{
	if (Category == ShowDebugNames::Reset)
	{
		ToggledDebugCategories.Reset();
		SaveConfig();
	}
	else
	{
		if (0 == ToggledDebugCategories.Remove(Category))
		{
			ToggledDebugCategories.Add(Category);
		}
		SaveConfig();
	}
}

void AHUD::ShowDebugForReticleTargetToggle(TSubclassOf<AActor> DesiredClass)
{
	bShowDebugForReticleTarget = !bShowDebugForReticleTarget;
	ShowDebugTargetDesiredClass = DesiredClass;
}

bool AHUD::ShouldDisplayDebug(const FName& DebugType) const
{
	return bShowDebugInfo && DebugDisplay.Contains(DebugType);
}

void AHUD::ShowDebugInfo(float& YL, float& YPos)
{
	if (DebugCanvas != nullptr && DebugCanvas->Canvas != nullptr)
	{
		// Darken background, so we can read text better.
		FLinearColor BackgroundColor(0.f, 0.f, 0.f, 0.2f);
		DebugCanvas->Canvas->DrawTile(0, 0, DebugCanvas->ClipX, DebugCanvas->ClipY, 0.f, 0.f, 0.f, 0.f, BackgroundColor);

		FDebugDisplayInfo DisplayInfo(DebugDisplay, ToggledDebugCategories);
		ShowDebugTargetActor = GetCurrentDebugTargetActor();

		// Draw Header.
		{
			FDisplayDebugManager& DisplayDebugManager = DebugCanvas->DisplayDebugManager;
			DisplayDebugManager.SetDrawColor(FColor(255, 0, 0));
			DisplayDebugManager.DrawString(FString::Printf(TEXT("Showing Debug for %s, Press [PageUp] and [PageDown] to cycle between targets."), *GetNameSafe(ShowDebugTargetActor)));
			DisplayDebugManager.DrawString(TEXT("Press [Right Shift] or [Left Shift] to move the debug menu a column over to see more of the debug menu"));
		}

		if (IsValid(ShowDebugTargetActor))
		{
			// Draw box around Actor being debugged.
#if ENABLE_DRAW_DEBUG
			if (GDrawCurrentDebugTargetBoundingBox.GetValueOnGameThread() > 0)
			{
				FVector BoundsOrigin, BoundsExtent;
				ShowDebugTargetActor->GetActorBounds(true, BoundsOrigin, BoundsExtent);

				// Expand extent a little bit
				BoundsExtent *= 1.1f;
				DrawDebugBox(GetWorld(), BoundsOrigin, BoundsExtent, FColor::Green, false, -1.f, 0, 2.f);
			}
#endif
			ShowDebugTargetActor->DisplayDebug(DebugCanvas, DisplayInfo, YL, YPos);

			if (!bShowDebugForReticleTarget && ShowDebugTargetActor->GetLocalRole() == ROLE_SimulatedProxy)
			{
				PlayerOwner->DisplayDebug(DebugCanvas, DisplayInfo, YL, YPos);
			}
		}

		if (ShouldDisplayDebug(NAME_Game))
		{
			AGameModeBase* AuthGameMode = GetWorld()->GetAuthGameMode();
			if (AuthGameMode)
			{
				AuthGameMode->DisplayDebug(DebugCanvas, DisplayInfo, YL, YPos);
			}
		}

		if (bShowDebugInfo)
		{
			OnShowDebugInfo.Broadcast(this, DebugCanvas, DisplayInfo, YL, YPos);
		}
	}
}

AActor* AHUD::GetCurrentDebugTargetActor()
{
	AActor* DebugTargetActor = nullptr;

	// Find targets through the reticle.
	if (bShowDebugForReticleTarget && PlayerOwner->PlayerCameraManager)
	{
		FRotator CamRot; FVector CamLoc; PlayerOwner->GetPlayerViewPoint(CamLoc, CamRot);

		FCollisionQueryParams TraceParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true, PlayerOwner->PlayerCameraManager->ViewTarget.Target);
		FHitResult Hit;
		bool bHit = GetWorld()->LineTraceSingleByChannel(Hit, CamLoc, CamRot.Vector() * 100000.f + CamLoc, ECC_WorldDynamic, TraceParams);
		if (bHit)
		{
			AActor* HitActor = Hit.HitObjectHandle.FetchActor();
			if (HitActor && ((ShowDebugTargetDesiredClass == nullptr) || HitActor->IsA(ShowDebugTargetDesiredClass)))
			{
				DebugTargetActor = HitActor;
			}
		}

		// If we hit something new, return this.
		// Otherwise fall back to our last successful hit.
		return DebugTargetActor ? DebugTargetActor : ToRawPtr(ShowDebugTargetActor);
	}
	else
	{
		// Otherwise we use our Cached DebugTargetActor.
		DebugTargetActor = ShowDebugTargetActor;
	}

	// If we have no one to view, default to current view target.
	if ((DebugTargetActor == nullptr) && PlayerOwner->PlayerCameraManager && PlayerOwner->PlayerCameraManager->ViewTarget.Target)
	{
		DebugTargetActor = PlayerOwner->PlayerCameraManager->ViewTarget.Target;
	}

	return DebugTargetActor;
}

void AHUD::AddActorToDebugList(AActor* InActor, TArray<AActor*>& InOutList, UWorld* InWorld)
{
	// Only consider actors that are visible, not destroyed and in the same world.
	if (IsValid(InActor) && (InActor->GetWorld() == InWorld) && InActor->WasRecentlyRendered())
	{
		InOutList.AddUnique(InActor);
	}
}

void AHUD::AddComponentOwnerToDebugList(UActorComponent* InComponent, TArray<AActor*>& InOutList, UWorld* InWorld)
{
	if (InComponent && InComponent->GetOwner())
	{
		AddActorToDebugList(InComponent->GetOwner(), InOutList, InWorld);
	}
}

void AHUD::GetDebugActorList(TArray<AActor*>& InOutList)
{
	UWorld* World = GetWorld();

	// By default, add all Pawns.
	for (TActorIterator<APawn> It(World); It; ++It)
	{
		AddActorToDebugList(*It, InOutList, World);
	}

	// If we're viewing animations, add all actors using an AnimInstance.
	if (ShouldDisplayDebug(ShowDebugNames::Animation))
	{
		for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			if (It->GetAnimInstance())
			{
				AddComponentOwnerToDebugList(*It, InOutList, World);
			}
		}
	}

	// If we're viewing physics, add all actors using a movement component.
	if (ShouldDisplayDebug(ShowDebugNames::Physics))
	{
		for (TObjectIterator<UMovementComponent> It; It; ++It)
		{
			AddComponentOwnerToDebugList(*It, InOutList, World);
		}
	}
}

void AHUD::NextDebugTarget()
{
	TArray<AActor*> Targets;
	GetDebugActorList(Targets);

	// If we have an existing target, find it in list.
	// As list can change as actors are spawned and destroyed.
	if (ShowDebugTargetActor)
	{
		if (!Targets.IsValidIndex(CurrentTargetIndex) || (Targets[CurrentTargetIndex] != ShowDebugTargetActor))
		{
			int32 FoundIndex;
			if (Targets.Find(ShowDebugTargetActor, FoundIndex))
			{
				CurrentTargetIndex = FoundIndex;
			}
		}
	}

	CurrentTargetIndex = Targets.Num() > 0 ? (CurrentTargetIndex + 1) % Targets.Num() : INDEX_NONE;
	if (Targets.IsValidIndex(CurrentTargetIndex))
	{
		ShowDebugTargetActor = Targets[CurrentTargetIndex];
	}
	else if (PlayerOwner->PlayerCameraManager && PlayerOwner->PlayerCameraManager->ViewTarget.Target)
	{
		ShowDebugTargetActor = PlayerOwner->PlayerCameraManager->ViewTarget.Target;
	}
}

void AHUD::PreviousDebugTarget()
{
	TArray<AActor*> Targets;
	GetDebugActorList(Targets);

	// If we have an existing target, find it in list.
	// As list can change as actors are spawned and destroyed.
	if (ShowDebugTargetActor)
	{
		if (!Targets.IsValidIndex(CurrentTargetIndex) || (Targets[CurrentTargetIndex] != ShowDebugTargetActor))
		{
			int32 FoundIndex;
			if (Targets.Find(ShowDebugTargetActor, FoundIndex))
			{
				CurrentTargetIndex = FoundIndex;
			}
		}
	}

	CurrentTargetIndex--;
	if (CurrentTargetIndex < 0)
	{
		CurrentTargetIndex = Targets.Num() - 1;
	}

	if (Targets.IsValidIndex(CurrentTargetIndex))
	{
		ShowDebugTargetActor = Targets[CurrentTargetIndex];
	}
	else if (PlayerOwner->PlayerCameraManager && PlayerOwner->PlayerCameraManager->ViewTarget.Target)
	{
		ShowDebugTargetActor = PlayerOwner->PlayerCameraManager->ViewTarget.Target;
	}
}

void AHUD::DrawHUD()
{
	HitBoxMap.Reset();
	HitBoxHits.Reset();
	if (bShowOverlays && (PlayerOwner != nullptr))
	{
		FVector ViewPoint;
		FRotator ViewRotation;
		PlayerOwner->GetPlayerViewPoint(ViewPoint, ViewRotation);
		DrawActorOverlays(ViewPoint, ViewRotation);
	}

	// Blueprint draw
	ReceiveDrawHUD(Canvas->SizeX, Canvas->SizeY);
}

UFont* AHUD::GetFontFromSizeIndex(int32 FontSizeIndex) const
{
	switch (FontSizeIndex)
	{
	case 0: return GEngine->GetTinyFont();
	case 1: return GEngine->GetSmallFont();
	case 2: return GEngine->GetMediumFont();
	case 3: return GEngine->GetLargeFont();
	}

	return GEngine->GetLargeFont();
}

void AHUD::OnLostFocusPause(bool bEnable)
{
	if (bLostFocusPaused == bEnable)
		return;

	if (GetNetMode() != NM_Client)
	{
		bLostFocusPaused = bEnable;
		PlayerOwner->SetPause(bEnable);
	}
}

void AHUD::DrawDebugTextList()
{
	if ((DebugTextList.Num() > 0) && (DebugCanvas != nullptr) && (DebugCanvas->Canvas != nullptr))
	{
		FRotator CameraRot;
		FVector CameraLoc;
		PlayerOwner->GetPlayerViewPoint(CameraLoc, CameraRot);

		FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::GetEmpty(), GEngine->GetSmallFont(), FLinearColor::White);
		for (int32 Idx = 0; Idx < DebugTextList.Num(); Idx++)
		{
			if (DebugTextList[Idx].SrcActor == nullptr)
			{
				DebugTextList.RemoveAt(Idx--, 1);
				continue;
			}

			if (DebugTextList[Idx].Font != nullptr)
			{
				TextItem.Font = DebugTextList[Idx].Font;
			}
			else
			{
				TextItem.Font = GEngine->GetSmallFont();
			}

			float const Alpha = FMath::IsNearlyZero(DebugTextList[Idx].Duration) ? 0.f : (1.f - (DebugTextList[Idx].TimeRemaining / DebugTextList[Idx].Duration));
			FVector WorldTextLoc;
			if (DebugTextList[Idx].bAbsoluteLocation)
			{
				WorldTextLoc = FMath::Lerp(DebugTextList[Idx].SrcActorOffset, DebugTextList[Idx].SrcActorDesiredOffset, Alpha);
			}
			else
			{
				FVector Offset = FMath::Lerp(DebugTextList[Idx].SrcActorOffset, DebugTextList[Idx].SrcActorDesiredOffset, Alpha);

				if (DebugTextList[Idx].bKeepAttachedToActor)
				{
					WorldTextLoc = DebugTextList[Idx].SrcActor->GetActorLocation() + Offset;
				}
				else
				{
					WorldTextLoc = DebugTextList[Idx].OrigActorLocation + Offset;
				}
			}

			if (bEnableDebugTextShadow || DebugTextList[Idx].bDrawShadow)
			{
				TextItem.EnableShadow(FLinearColor::Black);
			}
			else
			{
				TextItem.DisableShadow();
			}

			// don't draw text behind the camera
			if (((WorldTextLoc - CameraLoc) | CameraRot.Vector()) > 0.f)
			{
				FVector ScreenLoc = DebugCanvas->Project(WorldTextLoc);
				TextItem.SetColor(DebugTextList[Idx].TextColor);
				TextItem.Text = FText::FromString(DebugTextList[Idx].DebugText);
				TextItem.Scale = FVector2D(DebugTextList[Idx].FontScale, DebugTextList[Idx].FontScale);
				DebugCanvas->DrawItem(TextItem, FVector2D(FMath::CeilToFloat(ScreenLoc.X), FMath::CeilToFloat(ScreenLoc.Y)));
			}

			UE_LOGF(LogHUD, VeryVerbose, "[%f] %ls: \"%ls\" Loc: [%ls] "
				, GetWorld()->GetTimeSeconds()
				, *GetNameSafe(DebugTextList[Idx].SrcActor)
				, *DebugTextList[Idx].DebugText
				, *WorldTextLoc.ToString());

			// do this at the end so even small durations get at least one frame
			if (DebugTextList[Idx].TimeRemaining != -1.f)
			{
				DebugTextList[Idx].TimeRemaining -= RenderDelta;
			}
		}

		// Clear out the list of expired ones (going from the back to reduce copying the remaining portion of the list as it is shrunk, since order matters)
		for (int32 Idx = DebugTextList.Num() - 1; Idx >= 0; --Idx)
		{
			if (DebugTextList[Idx].TimeRemaining != -1.f)
			{
				if (DebugTextList[Idx].TimeRemaining <= 0.f)
				{
					DebugTextList.RemoveAt(Idx, 1);
				}
			}
		}
	}
}

/// @cond DOXYGEN_WARNINGS

int32 AHUD::FindDebugTextListIntervalForActor(AActor* InSrcActor, int32& OutFirstIdx, int32& OutLastIdx) const
{
	OutFirstIdx = DebugTextList.IndexOfByPredicate([=](const FDebugTextInfo& Element) { return (Element.SrcActor == InSrcActor); });
	OutLastIdx = OutFirstIdx;
	if (OutFirstIdx == INDEX_NONE)
	{
		return 0;
	}

	const int32 NumElements = DebugTextList.Num();
	while (((OutLastIdx + 1) < NumElements) && (DebugTextList[OutLastIdx + 1].SrcActor == InSrcActor))
	{
		++OutLastIdx;
	}

	return (OutLastIdx - OutFirstIdx + 1);
}

void AHUD::AddDebugText_Implementation(const FString& DebugText,
	AActor* SrcActor,
	float Duration,
	FVector Offset,
	FVector DesiredOffset,
	FColor TextColor,
	bool bSkipOverwriteCheck,
	bool bAbsoluteLocation,
	bool bKeepAttachedToActor,
	UFont* InFont,
	float FontScale,
	bool bDrawShadow
)
{
	// set a default color
	if (TextColor == FColor::Transparent)
	{
		TextColor = FColor::White;
	}

	// and a default source actor of our pawn
	if (SrcActor != nullptr)
	{
		if (DebugText.Len() == 0)
		{
			RemoveDebugText(SrcActor);
		}
		else
		{
			FDebugTextInfo* DebugTextInfo = nullptr;

			// find existing elements corresponding to this actor if any :
			int32 FirstIdx, LastIdx;
			int32 NumElements = FindDebugTextListIntervalForActor(SrcActor, FirstIdx, LastIdx);
			if (NumElements > 0)
			{
				if (!bSkipOverwriteCheck)
				{
					// just replace the existing entry :
					DebugTextInfo = &DebugTextList[FirstIdx];
				}
				else
				{
					// if there's a max number per actor, respect it : 
					int32 MaxNumStrings = GMaxDebugTextStringsPerActorCVar.GetValueOnGameThread();
					if ((MaxNumStrings <= 0) || (NumElements < MaxNumStrings))
					{
						DebugTextInfo = &DebugTextList.EmplaceAt_GetRef(LastIdx + 1);
					}
				}
			}
			else
			{
				DebugTextInfo = &DebugTextList.Emplace_GetRef();
			}

			if (DebugTextInfo != nullptr)
			{
				// assign the new text and actor
				DebugTextInfo->SrcActor = SrcActor;
				DebugTextInfo->SrcActorOffset = Offset;
				DebugTextInfo->SrcActorDesiredOffset = DesiredOffset;
				DebugTextInfo->DebugText = DebugText;
				DebugTextInfo->TimeRemaining = Duration;
				DebugTextInfo->Duration = Duration;
				DebugTextInfo->TextColor = TextColor;
				DebugTextInfo->bAbsoluteLocation = bAbsoluteLocation;
				DebugTextInfo->bKeepAttachedToActor = bKeepAttachedToActor;
				DebugTextInfo->OrigActorLocation = SrcActor->GetActorLocation();
				DebugTextInfo->Font = InFont;
				DebugTextInfo->FontScale = FontScale;
				DebugTextInfo->bDrawShadow = bDrawShadow;
			}
		}
	}
}

/** Remove all debug text for the specific actor. */
void AHUD::RemoveDebugText_Implementation(AActor* SrcActor, bool bLeaveDurationText)
{
	// remove from the back to reduce copying the remaining portion of the list as it is shrunk, since order matters :
	int32 FirstIdx, LastIdx;
	int32 NumElements = FindDebugTextListIntervalForActor(SrcActor, FirstIdx, LastIdx);
	if (NumElements > 0)
	{
		if (!bLeaveDurationText)
		{
			// immediate removal : remove the list altogether : 
			DebugTextList.RemoveAt(FirstIdx, NumElements);
		}
		else
		{
			// remove those with no expiry date 
			for (int32 Idx = LastIdx; Idx >= FirstIdx; --Idx)
			{
				if (DebugTextList[Idx].TimeRemaining == -1.f)
				{
					DebugTextList.RemoveAt(Idx, 1);
				}
			}
		}
	}
}

/** Remove all debug text */
void AHUD::RemoveAllDebugStrings_Implementation()
{
	DebugTextList.Reset();
}

/// @endcond

void AHUD::NotifyHitBoxClick(FName BoxName)
{
	// dispatch BP event
	ReceiveHitBoxClick(BoxName);
}

void AHUD::NotifyHitBoxRelease(FName BoxName)
{
	// dispatch BP event
	ReceiveHitBoxRelease(BoxName);
}

void AHUD::NotifyHitBoxBeginCursorOver(FName BoxName)
{
	// dispatch BP event
	ReceiveHitBoxBeginCursorOver(BoxName);
}

void AHUD::NotifyHitBoxEndCursorOver(FName BoxName)
{
	// dispatch BP event
	ReceiveHitBoxEndCursorOver(BoxName);
}

void AHUD::GetTextSize(const FString& Text, float& OutWidth, float& OutHeight, class UFont* Font, float Scale) const
{
	if (IsCanvasValid_WarnIfNot())
	{
		Canvas->TextSize(Font ? Font : GEngine->GetMediumFont(), Text, OutWidth, OutHeight, Scale, Scale);
	}
}

void AHUD::DrawText(FString const& Text, FLinearColor Color, float ScreenX, float ScreenY, UFont* Font, float Scale, bool bScalePosition)
{
	if (IsCanvasValid_WarnIfNot())
	{
		if (bScalePosition)
		{
			ScreenX *= Scale;
			ScreenY *= Scale;
		}
		FCanvasTextItem TextItem(FVector2D(ScreenX, ScreenY), FText::FromString(Text), Font ? Font : GEngine->GetMediumFont(), Color);
		TextItem.Scale = FVector2D(Scale, Scale);
		Canvas->DrawItem(TextItem);
	}
}

void AHUD::DrawMaterial(UMaterialInterface* Material, float ScreenX, float ScreenY, float ScreenW, float ScreenH, float MaterialU, float MaterialV, float MaterialUWidth, float MaterialVHeight, float Scale, bool bScalePosition, float Rotation, FVector2D RotPivot)
{
	if (IsCanvasValid_WarnIfNot() && Material)
	{
		FCanvasTileItem TileItem(FVector2D(ScreenX, ScreenY), Material->GetRenderProxy(), FVector2D(ScreenW, ScreenH) * Scale, FVector2D(MaterialU, MaterialV), FVector2D(MaterialU + MaterialUWidth, MaterialV + MaterialVHeight));
		TileItem.Rotation = FRotator(0, Rotation, 0);
		TileItem.PivotPoint = RotPivot;
		if (bScalePosition)
		{
			TileItem.Position *= Scale;
		}
		Canvas->DrawItem(TileItem);
	}
}

void AHUD::DrawMaterialSimple(UMaterialInterface* Material, float ScreenX, float ScreenY, float ScreenW, float ScreenH, float Scale, bool bScalePosition)
{
	if (IsCanvasValid_WarnIfNot() && Material)
	{
		FCanvasTileItem TileItem(FVector2D(ScreenX, ScreenY), Material->GetRenderProxy(), FVector2D(ScreenW, ScreenH) * Scale);
		if (bScalePosition)
		{
			TileItem.Position *= Scale;
		}
		Canvas->DrawItem(TileItem);
	}
}

void AHUD::DrawTexture(UTexture* Texture, float ScreenX, float ScreenY, float ScreenW, float ScreenH, float TextureU, float TextureV, float TextureUWidth, float TextureVHeight, FLinearColor Color, EBlendMode BlendMode, float Scale, bool bScalePosition, float Rotation, FVector2D RotPivot)
{
	if (IsCanvasValid_WarnIfNot() && Texture)
	{
		FCanvasTileItem TileItem(FVector2D(ScreenX, ScreenY), Texture->GetResource(), FVector2D(ScreenW, ScreenH) * Scale, FVector2D(TextureU, TextureV), FVector2D(TextureU + TextureUWidth, TextureV + TextureVHeight), Color);
		TileItem.Rotation = FRotator(0, Rotation, 0);
		TileItem.PivotPoint = RotPivot;
		if (bScalePosition)
		{
			TileItem.Position *= Scale;
		}
		TileItem.BlendMode = FCanvas::BlendToSimpleElementBlend(BlendMode);
		Canvas->DrawItem(TileItem);
	}
}

void AHUD::DrawTextureSimple(UTexture* Texture, float ScreenX, float ScreenY, float Scale, bool bScalePosition)
{
	if (IsCanvasValid_WarnIfNot() && Texture)
	{
		FCanvasTileItem TileItem(FVector2D(ScreenX, ScreenY), Texture->GetResource(), FLinearColor::White);
		if (bScalePosition)
		{
			TileItem.Position *= Scale;
		}
		// Apply the scale to the size (which will have been setup from the texture in the constructor).
		TileItem.Size *= Scale;
		TileItem.BlendMode = SE_BLEND_Translucent;
		Canvas->DrawItem(TileItem);
	}
}

void AHUD::DrawMaterialTriangle(UMaterialInterface* Material, FVector2D V0_Pos, FVector2D V1_Pos, FVector2D V2_Pos, FVector2D V0_UV, FVector2D V1_UV, FVector2D V2_UV, FLinearColor V0_Color, FLinearColor V1_Color, FLinearColor V2_Color)
{
	if (IsCanvasValid_WarnIfNot() && Material)
	{
		FCanvasTriangleItem TriangleItem(V0_Pos, V1_Pos, V2_Pos, V0_UV, V1_UV, V2_UV, nullptr);
		TriangleItem.TriangleList[0].V0_Color = V0_Color;
		TriangleItem.TriangleList[0].V1_Color = V1_Color;
		TriangleItem.TriangleList[0].V2_Color = V2_Color;
		TriangleItem.MaterialRenderProxy = Material->GetRenderProxy();
		Canvas->DrawItem(TriangleItem);
	}
}

FVector AHUD::Project(FVector Location, bool bClampToZeroPlane) const
{
	if (IsCanvasValid_WarnIfNot())
	{
		return Canvas->Project(Location, bClampToZeroPlane);
	}
	return FVector(0, 0, 0);
}

void AHUD::Deproject(float ScreenX, float ScreenY, FVector& WorldPosition, FVector& WorldDirection) const
{
	WorldPosition = WorldDirection = FVector(0, 0, 0);
	if (IsCanvasValid_WarnIfNot())
	{
		Canvas->Deproject(FVector2D(ScreenX, ScreenY), WorldPosition, WorldDirection);
	}
}


void AHUD::GetActorsInSelectionRectangle(TSubclassOf<class AActor> ClassFilter, const FVector2D& FirstPoint, const FVector2D& SecondPoint, TArray<AActor*>& OutActors, bool bIncludeNonCollidingComponents, bool bActorMustBeFullyEnclosed)
{
	// Because this is a HUD function it is likely to get called each tick,
	// so make sure any previous contents of the out actor array have been cleared!
	OutActors.Reset();

	//Create Selection Rectangle from Points
	FBox2D SelectionRectangle(ForceInit);

	//This method ensures that an appropriate rectangle is generated, 
	//		no matter what the coordinates of first and second point actually are.
	SelectionRectangle += FirstPoint;
	SelectionRectangle += SecondPoint;


	//The Actor Bounds Point Mapping
	const FVector BoundsPointMapping[8] =
	{
		FVector(1.f, 1.f, 1.f),
		FVector(1.f, 1.f, -1.f),
		FVector(1.f, -1.f, 1.f),
		FVector(1.f, -1.f, -1.f),
		FVector(-1.f, 1.f, 1.f),
		FVector(-1.f, 1.f, -1.f),
		FVector(-1.f, -1.f, 1.f),
		FVector(-1.f, -1.f, -1.f) };

	//~~~

	//For Each Actor of the Class Filter Type
	for (TActorIterator<AActor> Itr(GetWorld(), ClassFilter); Itr; ++Itr)
	{
		AActor* EachActor = *Itr;

		//Get Actor Bounds				//casting to base class, checked by template in the .h
		const FBox EachActorBounds = EachActor->GetComponentsBoundingBox(bIncludeNonCollidingComponents); /* All Components? */

		//Center
		const FVector BoxCenter = EachActorBounds.GetCenter();

		//Extents
		const FVector BoxExtents = EachActorBounds.GetExtent();

		// Build 2D bounding box of actor in screen space
		FBox2D ActorBox2D(ForceInit);
		for (uint8 BoundsPointItr = 0; BoundsPointItr < 8; BoundsPointItr++)
		{
			// Project vert into screen space.
			const FVector ProjectedWorldLocation = Project(BoxCenter + (BoundsPointMapping[BoundsPointItr] * BoxExtents), true);
			// Add to 2D bounding box if point is on the front side of the camera
			if (ProjectedWorldLocation.Z > 0.f)
			{
				ActorBox2D += FVector2D(ProjectedWorldLocation.X, ProjectedWorldLocation.Y);
			}
		}
		// Only consider actor boxes that have valid points inside
		if (ActorBox2D.bIsValid)
		{
			//Selection Box must fully enclose the Projected Actor Bounds
			if (bActorMustBeFullyEnclosed)
			{
				if (SelectionRectangle.IsInside(ActorBox2D))
				{
					OutActors.Add(EachActor);
				}
			}
			//Partial Intersection with Projected Actor Bounds
			else if (SelectionRectangle.Intersect(ActorBox2D))
			{
				OutActors.Add(EachActor);
			}
		}
	}
}

void AHUD::DrawRect(FLinearColor Color, float ScreenX, float ScreenY, float Width, float Height)
{
	if (IsCanvasValid_WarnIfNot())
	{
		FCanvasTileItem TileItem(FVector2D(ScreenX, ScreenY), GWhiteTexture, FVector2D(Width, Height), Color);
		TileItem.BlendMode = SE_BLEND_Translucent;
		Canvas->DrawItem(TileItem);
	}
}

void AHUD::DrawLine(float StartScreenX, float StartScreenY, float EndScreenX, float EndScreenY, FLinearColor LineColor, float LineThickness)
{
	if (IsCanvasValid_WarnIfNot())
	{
		FCanvasLineItem LineItem(FVector2D(StartScreenX, StartScreenY), FVector2D(EndScreenX, EndScreenY));
		LineItem.SetColor(LineColor);
		LineItem.LineThickness = LineThickness;
		Canvas->DrawItem(LineItem);
	}
}


APlayerController* AHUD::GetOwningPlayerController() const
{
	return PlayerOwner;
}

APawn* AHUD::GetOwningPawn() const
{
	return PlayerOwner ? PlayerOwner->GetPawn() : nullptr;
}

void AHUD::RenderHitBoxes(FCanvas* InCanvas)
{
	for (const FHUDHitBox& HitBox : HitBoxMap)
	{
		FLinearColor BoxColor = FLinearColor::White;
		if (HitBoxHits.Contains(const_cast<FHUDHitBox*>(&HitBox)))
		{
			BoxColor = FLinearColor::Red;
		}
		HitBox.Draw(InCanvas, BoxColor);
	}
}

void AHUD::UpdateHitBoxCandidates(TArray<FVector2D> InContactPoints)
{
	HitBoxHits.Reset();
	for (FHUDHitBox& HitBox : HitBoxMap)
	{
		bool bAdded = false;
		for (int32 ContactPointIndex = InContactPoints.Num() - 1; ContactPointIndex >= 0; --ContactPointIndex)
		{
			if (HitBox.Contains(InContactPoints[ContactPointIndex]))
			{
				if (!bAdded)
				{
					HitBoxHits.Add(&HitBox);
					bAdded = true;
				}
				if (HitBox.ConsumesInput())
				{
					InContactPoints.RemoveAtSwap(ContactPointIndex);
				}
				else
				{
					break;
				}
			}
		}
		if (InContactPoints.Num() == 0)
		{
			break;
		}
	}

	TSet<FName> NotOverHitBoxes = HitBoxesOver;
	TArray<FName> NewlyOverHitBoxes;

	// Now figure out which boxes we are over and deal with begin/end cursor over messages 
	for (FHUDHitBox* HitBox : HitBoxHits)
	{
		const FName HitBoxName = HitBox->GetName();
		if (HitBoxesOver.Contains(HitBoxName))
		{
			NotOverHitBoxes.Remove(HitBoxName);
		}
		else
		{
			NewlyOverHitBoxes.AddUnique(HitBoxName);
		}
	}

	// Dispatch the end cursor over messages
	for (const FName& HitBoxName : NotOverHitBoxes)
	{
		NotifyHitBoxEndCursorOver(HitBoxName);
		HitBoxesOver.Remove(HitBoxName);
	}

	// Dispatch the newly over hitbox messages
	for (const FName& HitBoxName : NewlyOverHitBoxes)
	{
		NotifyHitBoxBeginCursorOver(HitBoxName);
		HitBoxesOver.Add(HitBoxName);
	}
}

const FHUDHitBox* AHUD::GetHitBoxAtCoordinates(FVector2D InHitLocation, const bool bIsConsumingInput) const
{
	if (HitBoxMap.Num() > 0)
	{
		InHitLocation -= GetCoordinateOffset();

		for (const FHUDHitBox& HitBox : HitBoxMap)
		{
			if ((!bIsConsumingInput || HitBox.ConsumesInput()) && HitBox.Contains(InHitLocation))
			{
				return &HitBox;
			}
		}
	}
	return nullptr;
}

void AHUD::GetHitBoxesAtCoordinates(FVector2D InHitLocation, TArray<const FHUDHitBox*>& OutHitBoxes) const
{
	OutHitBoxes.Reset();

	if (HitBoxMap.Num() > 0)
	{
		InHitLocation -= GetCoordinateOffset();

		for (const FHUDHitBox& HitBox : HitBoxMap)
		{
			if (HitBox.Contains(InHitLocation))
			{
				OutHitBoxes.Add(&HitBox);
			}
		}
	}
}

const FHUDHitBox* AHUD::GetHitBoxWithName(const FName InName) const
{
	for (const FHUDHitBox& HitBox : HitBoxMap)
	{
		if (HitBox.GetName() == InName)
		{
			return &HitBox;
		}
	}
	return nullptr;
}

bool AHUD::AnyCurrentHitBoxHits() const
{
	return HitBoxHits.Num() != 0;
}

bool AHUD::UpdateAndDispatchHitBoxClickEvents(FVector2D ClickLocation, const EInputEvent InEventType)
{
	const bool bIsClickEvent = (InEventType == IE_Pressed || InEventType == IE_DoubleClick);

	// Early out to avoid unnecessary expense of calling GetCoordinateOffset
	if ((bIsClickEvent && HitBoxMap.Num() == 0) || (!bIsClickEvent && HitBoxHits.Num() == 0))
	{
		return false;
	}

	ClickLocation += GetCoordinateOffset();

	bool bHit = false;

	// If this is a click event we may not have the hit box in the hit list yet (particularly for touch events) so we need to check all HitBoxes
	if (bIsClickEvent)
	{
		for (FHUDHitBox& HitBox : HitBoxMap)
		{
			if (HitBox.Contains(ClickLocation))
			{
				bHit = true;

				NotifyHitBoxClick(HitBox.GetName());

				if (HitBox.ConsumesInput())
				{
					break;	//Early out if this box consumed the click
				}
			}
		}
	}
	else
	{
		for (FHUDHitBox* HitBoxHit : HitBoxHits)
		{
			if (HitBoxHit->Contains(ClickLocation))
			{
				bHit = true;

				if (InEventType == IE_Released)
				{
					NotifyHitBoxRelease(HitBoxHit->GetName());
				}

				if (HitBoxHit->ConsumesInput() == true)
				{
					break;	//Early out if this box consumed the click
				}
			}
		}
	}
	return bHit;
}

void AHUD::AddHitBox(FVector2D Position, FVector2D Size, FName Name, bool bConsumesInput, int32 Priority)
{
	if (GetHitBoxWithName(Name) == nullptr)
	{
		bool bAdded = false;
		for (int32 Index = 0; Index < HitBoxMap.Num(); ++Index)
		{
			if (HitBoxMap[Index].GetPriority() < Priority)
			{
				HitBoxMap.Insert(FHUDHitBox(Position, Size, Name, bConsumesInput, Priority), Index);
				bAdded = true;
				break;
			}
		}
		if (!bAdded)
		{
			HitBoxMap.Add(FHUDHitBox(Position, Size, Name, bConsumesInput, Priority));
		}
	}
	else
	{
		UE_LOGF(LogHUD, Warning, "Failed to add hitbox named %ls as a hitbox with this name already exists", *Name.ToString());
	}
}

bool AHUD::IsCanvasValid_WarnIfNot() const
{
	const bool bIsValid = Canvas != nullptr;
	if (!bIsValid)
	{
		FMessageLog("PIE").Warning()
			->AddToken(FUObjectToken::Create(const_cast<AHUD*>(this)))
			->AddToken(FTextToken::Create(LOCTEXT("PIE_Warning_Message_CanvasCallOutsideOfDrawCanvas", "Canvas Draw functions may only be called during the handling of the DrawHUD event")));
	}

	return bIsValid;
}


#undef LOCTEXT_NAMESPACE

