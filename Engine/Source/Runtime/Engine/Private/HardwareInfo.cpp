// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================
	HardwareInfo.cpp: Implements the FHardwareInfo class
=============================================================================*/


#include "HardwareInfo.h"
#include "CoreGlobals.h"


static TMap< FName, FString > HardwareDetailsMap;

ENGINE_API const FName NAME_RHI( "RHI" );
ENGINE_API const FName NAME_TextureFormat( "TextureFormat" );
ENGINE_API const FName NAME_DeviceType( "DeviceType" );

void FHardwareInfo::RegisterHardwareInfo( const FName SpecIdentifier, const FString& HardwareInfo )
{
	// Ensure we are adding a valid identifier to the map
	check(	SpecIdentifier == NAME_RHI || 
			SpecIdentifier == NAME_TextureFormat ||
			SpecIdentifier == NAME_DeviceType );

	HardwareDetailsMap.Add( SpecIdentifier, HardwareInfo );
}

FString FHardwareInfo::GetHardwareInfo(const FName SpecIdentifier)
{
	return HardwareDetailsMap.FindRef(SpecIdentifier);
}

const FString FHardwareInfo::GetHardwareDetailsString()
{
	FString DetailsString;

	int32 DetailsAdded = 0;

	for( TMap< FName, FString >::TConstIterator SpecIt( HardwareDetailsMap ); SpecIt; ++SpecIt )
	{
		// Separate all entries with a comma
		if( DetailsAdded++ > 0 )
		{
			DetailsString += TEXT( ", " );
		}

		FString SpecID = SpecIt.Key().ToString();
		FString SpecValue = SpecIt.Value();
		
		DetailsString += ( ( SpecID + TEXT( "=" ) ) + SpecValue );
	}
	return DetailsString;
}
