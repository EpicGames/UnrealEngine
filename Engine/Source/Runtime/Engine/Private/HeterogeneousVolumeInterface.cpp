// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeterogeneousVolumeInterface.cpp: Primitive scene proxy implementation.
=============================================================================*/

#include "HeterogeneousVolumeInterface.h"
