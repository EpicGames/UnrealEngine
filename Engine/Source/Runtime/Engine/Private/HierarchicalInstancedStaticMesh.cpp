// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InstancedStaticMesh.cpp: Static mesh rendering code.
=============================================================================*/

#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "StaticMeshComponentLODInfo.h"
#include "Templates/Greater.h"
#include "EngineLogs.h"
#include "EngineStats.h"
#include "Engine/Level.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "UObject/UObjectIterator.h"
#include "RenderUtils.h"
#include "UnrealEngine.h"
#include "Containers/Ticker.h"
#include "InstancedStaticMeshDelegates.h"
#include "UObject/ReleaseObjectVersion.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Algo/AnyOf.h"
#include "UObject/UnrealType.h"
#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif
#include "NaniteSceneProxy.h"
#include "HierarchicalStaticMeshSceneProxy.h"
#include "InstancedStaticMesh/ISMInstanceUpdateChangeSet.h"

#if WITH_EDITOR
static float GDebugBuildTreeAsyncDelayInSeconds = 0.f;
static FAutoConsoleVariableRef CVarDebugBuildTreeAsyncDelayInSeconds(
	TEXT("foliage.DebugBuildTreeAsyncDelayInSeconds"),
	GDebugBuildTreeAsyncDelayInSeconds,
	TEXT("Adds a delay (in seconds) to BuildTreeAsync tasks for debugging"));
#endif

static TAutoConsoleVariable<int32> CVarFoliageSplitFactor(
	TEXT("foliage.SplitFactor"),
	16,
	TEXT("This controls the branching factor of the foliage tree."));

static TAutoConsoleVariable<int32> CVarForceLOD(
	TEXT("foliage.ForceLOD"),
	-1,
	TEXT("If greater than or equal to zero, forces the foliage LOD to that level."));

static TAutoConsoleVariable<int32> CVarOnlyLOD(
	TEXT("foliage.OnlyLOD"),
	-1,
	TEXT("If greater than or equal to zero, only renders the foliage LOD at that level."));

static TAutoConsoleVariable<int32> CVarDisableCull(
	TEXT("foliage.DisableCull"),
	0,
	TEXT("If greater than zero, no culling occurs based on frustum."));

static TAutoConsoleVariable<int32> CVarCullAll(
	TEXT("foliage.CullAll"),
	0,
	TEXT("If greater than zero, everything is considered culled."),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarDitheredLOD(
	TEXT("foliage.DitheredLOD"),
	1,
	TEXT("If greater than zero, dithered LOD is used, otherwise popping LOD is used."));

static TAutoConsoleVariable<int32> CVarOverestimateLOD(
	TEXT("foliage.OverestimateLOD"),
	0,
	TEXT("If greater than zero and dithered LOD is not used, then we use an overestimate of LOD instead of an underestimate."));

static TAutoConsoleVariable<int32> CVarMaxTrianglesToRender(
	TEXT("foliage.MaxTrianglesToRender"),
	100000000,
	TEXT("This is an absolute limit on the number of foliage triangles to render in one traversal. This is used to prevent a silly LOD parameter mistake from causing the OS to kill the GPU."));

TAutoConsoleVariable<float> CVarFoliageMinimumScreenSize(
	TEXT("foliage.MinimumScreenSize"),
	0.000005f,
	TEXT("This controls the screen size at which we cull foliage instances entirely."),
	ECVF_Scalability);

TAutoConsoleVariable<int32> CVarFoliageMaxEndCullDistance(
	TEXT("foliage.MaxEndCullDistance"),
	0,
	TEXT("Max distance for end culling (0 disabled)."));

TAutoConsoleVariable<float> CVarFoliageLODDistanceScale(
	TEXT("foliage.LODDistanceScale"),
	1.0f,
	TEXT("Scale factor for the distance used in computing LOD for foliage."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}),
	ECVF_Scalability);

TAutoConsoleVariable<float> CVarRandomLODRange(
	TEXT("foliage.RandomLODRange"),
	0.0f,
	TEXT("Random distance added to each instance distance to compute LOD."));

static TAutoConsoleVariable<int32> CVarMinVertsToSplitNode(
	TEXT("foliage.MinVertsToSplitNode"),
	8192,
	TEXT("Controls the accuracy between culling and LOD accuracy and culling and CPU performance."));

static TAutoConsoleVariable<bool> CVarApplyViewOcclusionCullToShadow(
	TEXT("foliage.ApplyViewOcclusionCullToShadow"),
	true,
	TEXT("Re-use main view occlusion culling results to speed up the shadow view. Shadows for fully occluded objects may be missing."));

static TAutoConsoleVariable<int32> CVarMaxOcclusionQueriesPerComponent(
	TEXT("foliage.MaxOcclusionQueriesPerComponent"),
	16,
	TEXT("Controls the granularity of occlusion culling. 16-128 is a reasonable range."));

static TAutoConsoleVariable<int32> CVarMinOcclusionQueriesPerComponent(
	TEXT("foliage.MinOcclusionQueriesPerComponent"),
	6,
	TEXT("Controls the granularity of occlusion culling. 2 should be the Min."));

static TAutoConsoleVariable<int32> CVarMinInstancesPerOcclusionQuery(
	TEXT("foliage.MinInstancesPerOcclusionQuery"),
	256,
	TEXT("Controls the granualrity of occlusion culling. 1024 to 65536 is a reasonable range. This is not exact, actual minimum might be off by a factor of two."));

static TAutoConsoleVariable<float> CVarFoliageDensityScale(
	TEXT("foliage.DensityScale"),
	1.0,
	TEXT("Controls the amount of foliage to render. Foliage must opt-in to density scaling through the foliage type."),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarFoliageUseInstanceRuns(
	TEXT("foliage.InstanceRuns"),
	0,
	TEXT("Whether to use the InstanceRuns feature of FMeshBatch to compress foliage draw call data sent to the renderer.  Not supported by the Mesh Draw Command pipeline."));

#if RHI_RAYTRACING
static TAutoConsoleVariable<int32> CVarRayTracingHISM(
	TEXT("r.RayTracing.Geometry.HierarchicalInstancedStaticMesh"),
	1,
	TEXT("Defines if HierarchicalInstancedStaticMesh are added to the ray tracing scene.\n")
	TEXT(" 0: raytracing cached MDCs for HISMs are not created.\n")
	TEXT(" 1: raytracing cached MDCs for HISMs are created and instances are added to the RayTracing scene (default).\n")
	TEXT(" 2: HISMss are runtime culled from the RayTracing scene, but keeps the cached state for fast toggling."));
#endif

DECLARE_CYCLE_STAT(TEXT("Traversal Time"),STAT_FoliageTraversalTime,STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Build Time"), STAT_FoliageBuildTime, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Batch Time"),STAT_FoliageBatchTime,STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Foliage Create Proxy"), STAT_FoliageCreateProxy, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Foliage Post Load"), STAT_FoliagePostLoad, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("HISMC_AddInstance"), STAT_HISMCAddInstance, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("HISMC_AddInstances"), STAT_HISMCAddInstances, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("HISMC_RemoveInstance"), STAT_HISMCRemoveInstance, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("HISMC_GetDynamicMeshElement"), STAT_HISMCGetDynamicMeshElement, STATGROUP_Foliage);

DECLARE_DWORD_COUNTER_STAT(TEXT("Runs"), STAT_FoliageRuns, STATGROUP_Foliage);
DECLARE_DWORD_COUNTER_STAT(TEXT("Mesh Batches"), STAT_FoliageMeshBatches, STATGROUP_Foliage);
DECLARE_DWORD_COUNTER_STAT(TEXT("Triangles"), STAT_FoliageTriangles, STATGROUP_Foliage);
DECLARE_DWORD_COUNTER_STAT(TEXT("Instances"), STAT_FoliageInstances, STATGROUP_Foliage);
DECLARE_DWORD_COUNTER_STAT(TEXT("Occlusion Culled Instances"), STAT_OcclusionCulledFoliageInstances, STATGROUP_Foliage);
DECLARE_DWORD_COUNTER_STAT(TEXT("Traversals"),STAT_FoliageTraversals,STATGROUP_Foliage);
DECLARE_MEMORY_STAT(TEXT("Instance Buffers"),STAT_FoliageInstanceBuffers,STATGROUP_Foliage);

static void FoliageCVarSinkFunction()
{
	static float CachedFoliageDensityScale = 1.0f;
	float FoliageDensityScale = CVarFoliageDensityScale.GetValueOnGameThread();

	if (FoliageDensityScale != CachedFoliageDensityScale)
	{
		CachedFoliageDensityScale = FoliageDensityScale;
		FoliageDensityScale = FMath::Clamp(FoliageDensityScale, 0.0f, 1.0f);

		for (auto* Component : TObjectRange<UHierarchicalInstancedStaticMeshComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
		{
#if WITH_EDITOR
			if (Component->bCanEnableDensityScaling)
#endif
			{
				if (Component->bEnableDensityScaling && Component->CurrentDensityScaling != FoliageDensityScale)
				{
					Component->CurrentDensityScaling = FoliageDensityScale;
					Component->BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/true);
				}
			}
		}
	}
}

static FAutoConsoleVariableSink CVarFoliageSink(FConsoleCommandDelegate::CreateStatic(&FoliageCVarSinkFunction));


// ----------------------------------------------------------------------------------

UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::FClusterBuilder(TArray<FMatrix> InTransforms, TArray<float> InCustomDataFloats, int32 InNumCustomDataFloats, const FBox& InInstBox, int32 InMaxInstancesPerLeaf, float InDensityScaling, int32 InInstancingRandomSeed, bool InGenerateInstanceScalingRange)
	: OriginalNum(InTransforms.Num())
	, InstBox(InInstBox)
	, MaxInstancesPerLeaf(InMaxInstancesPerLeaf)
	, InstancingRandomSeed(InInstancingRandomSeed)
	, DensityScaling(InDensityScaling)
	, GenerateInstanceScalingRange(InGenerateInstanceScalingRange)
	, Transforms(MoveTemp(InTransforms))
	, CustomDataFloats(MoveTemp(InCustomDataFloats))
	, NumCustomDataFloats(InNumCustomDataFloats)
{
}

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::Split(int32 InNum)
{
	checkSlow(InNum);
	Clusters.Reset();
	Split(0, InNum - 1);
	Clusters.Sort();
	checkSlow(Clusters.Num() > 0);
	int32 At = 0;
	for (auto& Cluster : Clusters)
	{
		checkSlow(At == Cluster.Start);
		At += Cluster.Num;
	}
	checkSlow(At == InNum);
}

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::Split(int32 Start, int32 End)
{
	int32 NumRange = 1 + End - Start;
	FBox ClusterBounds(ForceInit);
	for (int32 Index = Start; Index <= End; Index++)
	{
		ClusterBounds += SortPoints[SortIndex[Index]];
	}
	if (NumRange <= BranchingFactor)
	{
		Clusters.Add(FRunPair(Start, NumRange));
		return;
	}
	checkSlow(NumRange >= 2);
	SortPairs.Reset();
	int32 BestAxis = -1;
	float BestAxisValue = -1.0f;
	for (int32 Axis = 0; Axis < 3; Axis++)
	{
		float ThisAxisValue = ClusterBounds.Max[Axis] - ClusterBounds.Min[Axis];
		if (!Axis || ThisAxisValue > BestAxisValue)
		{
			BestAxis = Axis;
			BestAxisValue = ThisAxisValue;
		}
	}
	for (int32 Index = Start; Index <= End; Index++)
	{
		FSortPair Pair;

		Pair.Index = SortIndex[Index];
		Pair.d = SortPoints[Pair.Index][BestAxis];
		SortPairs.Add(Pair);
	}
	SortPairs.Sort();
	for (int32 Index = Start; Index <= End; Index++)
	{
		SortIndex[Index] = SortPairs[Index - Start].Index;
	}

	int32 Half = NumRange / 2;

	int32 EndLeft = Start + Half - 1;
	int32 StartRight = 1 + End - Half;

	if (NumRange & 1)
	{
		if (SortPairs[Half].d - SortPairs[Half - 1].d < SortPairs[Half + 1].d - SortPairs[Half].d)
		{
			EndLeft++;
		}
		else
		{
			StartRight--;
		}
	}
	checkSlow(EndLeft + 1 == StartRight);
	checkSlow(EndLeft >= Start);
	checkSlow(End >= StartRight);

	Split(Start, EndLeft);
	Split(StartRight, End);
}

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::BuildInstanceBuffer()
{
	// build new instance buffer
	FRandomStream RandomStream = FRandomStream(InstancingRandomSeed);
	BuiltInstanceData = MakeUnique<FStaticMeshInstanceData>(/*bInUseHalfFloat = */true);
		
	int32 NumInstances = Result->InstanceReorderTable.Num();
	int32 NumRenderInstances = Result->SortedInstances.Num();
		
	if (NumRenderInstances > 0)
	{
		BuiltInstanceData->AllocateInstances(NumRenderInstances, NumCustomDataFloats, GIsEditor ? EResizeBufferFlags::AllowSlackOnGrow|EResizeBufferFlags::AllowSlackOnReduce : EResizeBufferFlags::None, false); // In Editor always permit overallocation, to prevent too much realloc

		FVector2D LightmapUVBias = FVector2D(-1.0f, -1.0f);
		FVector2D ShadowmapUVBias = FVector2D(-1.0f, -1.0f);

		// we loop over all instances to ensure that render instances will get same RandomID regardless of density settings
		for (int32 i = 0; i < NumInstances; ++i)
		{
			int32 RenderIndex = Result->InstanceReorderTable[i];
			float RandomID = RandomStream.GetFraction();
			if (RenderIndex >= 0)
			{
				// LWC_TODO: Precision loss here has been compensated for by use of TranslatedInstanceSpaceOrigin.
				BuiltInstanceData->SetInstance(RenderIndex, FMatrix44f(Transforms[i]), RandomID, LightmapUVBias, ShadowmapUVBias);
				for (int32 DataIndex = 0; DataIndex < NumCustomDataFloats; ++DataIndex)
				{
					BuiltInstanceData->SetInstanceCustomData(RenderIndex, DataIndex, CustomDataFloats[NumCustomDataFloats * i + DataIndex]);
				}
			}
			// correct light/shadow map bias will be setup on game thread side if needed
		}
	}
}

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::Init()
{
	SortIndex.Empty();
	SortPoints.SetNumUninitialized(OriginalNum);
					
	FRandomStream DensityRand = FRandomStream(InstancingRandomSeed);

	SortIndex.Empty(OriginalNum*DensityScaling);

	for (int32 Index = 0; Index < OriginalNum; Index++)
	{
		SortPoints[Index] = Transforms[Index].GetOrigin();

		if (DensityScaling < 1.0f && DensityRand.GetFraction() > DensityScaling)
		{
			continue;
		}

		SortIndex.Add(Index);
	}

	Num = SortIndex.Num();

	OcclusionLayerTarget = CVarMaxOcclusionQueriesPerComponent.GetValueOnAnyThread();
	int32 MinInstancesPerOcclusionQuery = CVarMinInstancesPerOcclusionQuery.GetValueOnAnyThread();

	if (Num / MinInstancesPerOcclusionQuery < OcclusionLayerTarget)
	{
		OcclusionLayerTarget = Num / MinInstancesPerOcclusionQuery;
		if (OcclusionLayerTarget < CVarMinOcclusionQueriesPerComponent.GetValueOnAnyThread())
		{
			OcclusionLayerTarget = 0;
		}
	}
	InternalNodeBranchingFactor = CVarFoliageSplitFactor.GetValueOnAnyThread();
		
	if (Num / MaxInstancesPerLeaf < InternalNodeBranchingFactor) // if there are less than InternalNodeBranchingFactor leaf nodes
	{
		MaxInstancesPerLeaf = FMath::Clamp<int32>(Num / InternalNodeBranchingFactor, 1, 1024); // then make sure we have at least InternalNodeBranchingFactor leaves
	}
}
	

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::BuildTreeAndBufferAsync(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
#if WITH_EDITOR
	if (!FMath::IsNearlyZero(GDebugBuildTreeAsyncDelayInSeconds))
	{
		UE_LOGF(LogStaticMesh, Warning, "BuildTree Debug Delay %5.1f (CVar foliage.DebugBuildTreeAsyncDelayInSeconds)", GDebugBuildTreeAsyncDelayInSeconds);
		FPlatformProcess::Sleep(GDebugBuildTreeAsyncDelayInSeconds);
	}
#endif
	BuildTreeAndBuffer();
}

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::BuildTreeAndBuffer()
{
	BuildTree();
	BuildInstanceBuffer();
}

void UHierarchicalInstancedStaticMeshComponent::FClusterBuilder::BuildTree()
{
	Init();
		
	Result = MakeUnique<FClusterTree>();

	if (Num == 0)
	{
		// Can happen if all instances are excluded due to scalability
		// It doesn't only happen with a scalability factor of 0 - 
		// even with a scalability factor of 0.99, if there's only one instance of this type you can end up with Num == 0 if you're unlucky
		Result->InstanceReorderTable.Init(INDEX_NONE, OriginalNum);
		return;
	}

	bool bIsOcclusionLayer = false;
	BranchingFactor = MaxInstancesPerLeaf;
	if (BranchingFactor > 2 && OcclusionLayerTarget && Num / BranchingFactor <= OcclusionLayerTarget)
	{
		BranchingFactor = FMath::Max<int32>(2, (Num + OcclusionLayerTarget - 1) / OcclusionLayerTarget);
		OcclusionLayerTarget = 0;
		bIsOcclusionLayer = true;
	}
	Split(Num);
	if (bIsOcclusionLayer)
	{
		Result->OutOcclusionLayerNum = Clusters.Num();
		bIsOcclusionLayer = false;
	}

	Result->SortedInstances.Append(SortIndex);
		
	NumRoots = Clusters.Num();
	Result->Nodes.Init(FClusterNode(), Clusters.Num());

	for (int32 Index = 0; Index < NumRoots; Index++)
	{
		FClusterNode& Node = Result->Nodes[Index];
		Node.FirstInstance = Clusters[Index].Start;
		Node.LastInstance = Clusters[Index].Start + Clusters[Index].Num - 1;
		FBox NodeBox(ForceInit);
		for (int32 InstanceIndex = Node.FirstInstance; InstanceIndex <= Node.LastInstance; InstanceIndex++)
		{
			const FMatrix& ThisInstTrans = Transforms[Result->SortedInstances[InstanceIndex]];
			FBox ThisInstBox = InstBox.TransformBy(ThisInstTrans);
			NodeBox += ThisInstBox;

			if (GenerateInstanceScalingRange)
			{
				FVector3f CurrentScale(ThisInstTrans.GetScaleVector());

				Node.MinInstanceScale = Node.MinInstanceScale.ComponentMin(CurrentScale);
				Node.MaxInstanceScale = Node.MaxInstanceScale.ComponentMax(CurrentScale);
			}
		}
		Node.BoundMin = (FVector3f)NodeBox.Min;
		Node.BoundMax = (FVector3f)NodeBox.Max;
	}
	TArray<int32> NodesPerLevel;
	NodesPerLevel.Add(NumRoots);
	int32 LOD = 0;

	TArray<int32> InverseSortIndex;
	TArray<int32> RemapSortIndex;
	TArray<int32> InverseInstanceIndex;
	TArray<int32> OldInstanceIndex;
	TArray<int32> LevelStarts;
	TArray<int32> InverseChildIndex;
	TArray<FClusterNode> OldNodes;

	while (NumRoots > 1)
	{
		SortIndex.Reset();
		SortPoints.Reset();
		SortIndex.AddUninitialized(NumRoots);
		SortPoints.AddUninitialized(NumRoots);
		for (int32 Index = 0; Index < NumRoots; Index++)
		{
			SortIndex[Index] = Index;
			FClusterNode& Node = Result->Nodes[Index];
			SortPoints[Index] = (FVector)(Node.BoundMin + Node.BoundMax) * 0.5f;
		}
		BranchingFactor = InternalNodeBranchingFactor;
		if (BranchingFactor > 2 && OcclusionLayerTarget && NumRoots / BranchingFactor <= OcclusionLayerTarget)
		{
			BranchingFactor = FMath::Max<int32>(2, (NumRoots + OcclusionLayerTarget - 1) / OcclusionLayerTarget);
			OcclusionLayerTarget = 0;
			bIsOcclusionLayer = true;
		}
		Split(NumRoots);
		if (bIsOcclusionLayer)
		{
			Result->OutOcclusionLayerNum = Clusters.Num();
			bIsOcclusionLayer = false;
		}

		InverseSortIndex.Reset();
		InverseSortIndex.AddUninitialized(NumRoots);
		for (int32 Index = 0; Index < NumRoots; Index++)
		{
			InverseSortIndex[SortIndex[Index]] = Index;
		}

		{
			// rearrange the instances to match the new order of the old roots
			RemapSortIndex.Reset();
			RemapSortIndex.AddUninitialized(Num);
			int32 OutIndex = 0;
			for (int32 Index = 0; Index < NumRoots; Index++)
			{
				FClusterNode& Node = Result->Nodes[SortIndex[Index]];
				for (int32 InstanceIndex = Node.FirstInstance; InstanceIndex <= Node.LastInstance; InstanceIndex++)
				{
					RemapSortIndex[OutIndex++] = InstanceIndex;
				}
			}
			InverseInstanceIndex.Reset();
			InverseInstanceIndex.AddUninitialized(Num);
			for (int32 Index = 0; Index < Num; Index++)
			{
				InverseInstanceIndex[RemapSortIndex[Index]] = Index;
			}
			for (int32 Index = 0; Index < Result->Nodes.Num(); Index++)
			{
				FClusterNode& Node = Result->Nodes[Index];
				Node.FirstInstance = InverseInstanceIndex[Node.FirstInstance];
				Node.LastInstance = InverseInstanceIndex[Node.LastInstance];
			}
			OldInstanceIndex.Reset();
			Swap(OldInstanceIndex, Result->SortedInstances);
			Result->SortedInstances.AddUninitialized(Num);
			for (int32 Index = 0; Index < Num; Index++)
			{
				Result->SortedInstances[Index] = OldInstanceIndex[RemapSortIndex[Index]];
			}
		}
		{
			// rearrange the nodes to match the new order of the old roots
			RemapSortIndex.Reset();
			int32 NewNum = Result->Nodes.Num() + Clusters.Num();
			// RemapSortIndex[new index] == old index
			RemapSortIndex.AddUninitialized(NewNum);
			LevelStarts.Reset();
			LevelStarts.Add(Clusters.Num());
			for (int32 Index = 0; Index < NodesPerLevel.Num() - 1; Index++)
			{
				LevelStarts.Add(LevelStarts[Index] + NodesPerLevel[Index]);
			}

			for (int32 Index = 0; Index < NumRoots; Index++)
			{
				FClusterNode& Node = Result->Nodes[SortIndex[Index]];
				RemapSortIndex[LevelStarts[0]++] = SortIndex[Index];

				int32 LeftIndex = Node.FirstChild;
				int32 RightIndex = Node.LastChild;
				int32 LevelIndex = 1;
				while (RightIndex >= 0)
				{
					int32 NextLeftIndex = MAX_int32;
					int32 NextRightIndex = -1;
					for (int32 ChildIndex = LeftIndex; ChildIndex <= RightIndex; ChildIndex++)
					{
						RemapSortIndex[LevelStarts[LevelIndex]++] = ChildIndex;
						int32 LeftChild = Result->Nodes[ChildIndex].FirstChild;
						int32 RightChild = Result->Nodes[ChildIndex].LastChild;
						if (LeftChild >= 0 && LeftChild <  NextLeftIndex)
						{
							NextLeftIndex = LeftChild;
						}
						if (RightChild >= 0 && RightChild >  NextRightIndex)
						{
							NextRightIndex = RightChild;
						}
					}
					LeftIndex = NextLeftIndex;
					RightIndex = NextRightIndex;
					LevelIndex++;
				}
			}
			checkSlow(LevelStarts[LevelStarts.Num() - 1] == NewNum);
			InverseChildIndex.Reset();
			// InverseChildIndex[old index] == new index
			InverseChildIndex.AddUninitialized(NewNum);
			for (int32 Index = Clusters.Num(); Index < NewNum; Index++)
			{
				InverseChildIndex[RemapSortIndex[Index]] = Index;
			}
			for (int32 Index = 0; Index < Result->Nodes.Num(); Index++)
			{
				FClusterNode& Node = Result->Nodes[Index];
				if (Node.FirstChild >= 0)
				{
					Node.FirstChild = InverseChildIndex[Node.FirstChild];
					Node.LastChild = InverseChildIndex[Node.LastChild];
				}
			}
			{
				Swap(OldNodes, Result->Nodes);
				Result->Nodes.Empty(NewNum);
				for (int32 Index = 0; Index < Clusters.Num(); Index++)
				{
					Result->Nodes.Add(FClusterNode());
				}
				Result->Nodes.AddUninitialized(OldNodes.Num());
				for (int32 Index = 0; Index < OldNodes.Num(); Index++)
				{
					Result->Nodes[InverseChildIndex[Index]] = OldNodes[Index];
				}
			}
			int32 OldIndex = Clusters.Num();
			int32 InstanceTracker = 0;
			for (int32 Index = 0; Index < Clusters.Num(); Index++)
			{
				FClusterNode& Node = Result->Nodes[Index];
				Node.FirstChild = OldIndex;
				OldIndex += Clusters[Index].Num;
				Node.LastChild = OldIndex - 1;
				Node.FirstInstance = Result->Nodes[Node.FirstChild].FirstInstance;
				checkSlow(Node.FirstInstance == InstanceTracker);
				Node.LastInstance = Result->Nodes[Node.LastChild].LastInstance;
				InstanceTracker = Node.LastInstance + 1;
				checkSlow(InstanceTracker <= Num);
				FBox NodeBox(ForceInit);
				for (int32 ChildIndex = Node.FirstChild; ChildIndex <= Node.LastChild; ChildIndex++)
				{
					FClusterNode& ChildNode = Result->Nodes[ChildIndex];
					NodeBox += (FVector)ChildNode.BoundMin;
					NodeBox += (FVector)ChildNode.BoundMax;

					if (GenerateInstanceScalingRange)
					{
						Node.MinInstanceScale = Node.MinInstanceScale.ComponentMin(ChildNode.MinInstanceScale);
						Node.MaxInstanceScale = Node.MaxInstanceScale.ComponentMax(ChildNode.MaxInstanceScale);
					}
				}
				Node.BoundMin = (FVector3f)NodeBox.Min;
				Node.BoundMax = (FVector3f)NodeBox.Max;
			}
			NumRoots = Clusters.Num();
			NodesPerLevel.Insert(NumRoots, 0);
		}
	}

	// Save inverse map
	Result->InstanceReorderTable.Init(INDEX_NONE, OriginalNum);
	for (int32 Index = 0; Index < Num; Index++)
	{
		Result->InstanceReorderTable[Result->SortedInstances[Index]] = Index;
	}

	// Output a general scale of 1 if we dont want the scaling range
	if (!GenerateInstanceScalingRange)
	{
		Result->Nodes[0].MinInstanceScale = FVector3f::OneVector;
		Result->Nodes[0].MaxInstanceScale = FVector3f::OneVector;
	}
}

bool UHierarchicalInstancedStaticMeshComponent::FClusterTree::PrintLevel(int32 NodeIndex, int32 Level, int32 CurrentLevel, int32 Parent)
{
	const FClusterNode& Node = Nodes[NodeIndex];
	if (Level == CurrentLevel)
	{
		UE_LOGF(LogConsoleResponse, Display, "Level %2d  Parent %3d",
			Level,
			Parent
			);
		FVector Extent(Node.BoundMax - Node.BoundMin);
		UE_LOGF(LogConsoleResponse, Display, "    Bound (%5.1f, %5.1f, %5.1f) [(%5.1f, %5.1f, %5.1f) - (%5.1f, %5.1f, %5.1f)]",
			Extent.X, Extent.Y, Extent.Z, 
			Node.BoundMin.X, Node.BoundMin.Y, Node.BoundMin.Z, 
			Node.BoundMax.X, Node.BoundMax.Y, Node.BoundMax.Z
			);
		UE_LOGF(LogConsoleResponse, Display, "    children %3d [%3d,%3d]   instances %3d [%3d,%3d]",
			(Node.FirstChild < 0) ? 0 : 1 + Node.LastChild - Node.FirstChild, Node.FirstChild, Node.LastChild,
			1 + Node.LastInstance - Node.FirstInstance, Node.FirstInstance, Node.LastInstance
			);
		return true;
	}
	else if (Node.FirstChild < 0)
	{
		return false;
	}
	bool Ret = false;
	for (int32 Child = Node.FirstChild; Child <= Node.LastChild; Child++)
	{
		Ret = PrintLevel(Child, Level, CurrentLevel + 1, NodeIndex) || Ret;
	}
	return Ret;
}

static void TestFoliage(const TArray<FString>& Args)
{
	UE_LOGF(LogConsoleResponse, Display, "Running Foliage test.");
	TArray<FInstancedStaticMeshInstanceData> Instances;

	FMatrix Temp;
	Temp.SetIdentity();
	FRandomStream RandomStream(0x238946);
	for (int32 i = 0; i < 1000; i++)
	{
		Instances.Add(FInstancedStaticMeshInstanceData());
		Temp.SetOrigin(FVector(RandomStream.FRandRange(0.0f, 1.0f), RandomStream.FRandRange(0.0f, 1.0f), 0.0f) * 10000.0f);
		Instances[i].Transform = Temp;
	}

	FBox TempBox(ForceInit);
	TempBox += FVector(-100.0f, -100.0f, -100.0f);
	TempBox += FVector(100.0f, 100.0f, 100.0f);

	TArray<FMatrix> InstanceTransforms;
	TArray<float> InstanceCustomDataDummy;
	InstanceTransforms.AddUninitialized(Instances.Num());
	for (int32 Index = 0; Index < Instances.Num(); Index++)
	{
		InstanceTransforms[Index] = Instances[Index].Transform;
	}

	UHierarchicalInstancedStaticMeshComponent::FClusterBuilder Builder(InstanceTransforms, InstanceCustomDataDummy, 0, TempBox, 16, 1.0f, 1, 0);
	Builder.BuildTree();

	int32 Level = 0;

	UE_LOGF(LogConsoleResponse, Display, "-----");

	while(Builder.Result->PrintLevel(0, Level++, 0, -1))
	{
	}
}

static FAutoConsoleCommand TestFoliageCmd(
	TEXT("foliage.Test"),
	TEXT("Useful for debugging."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&TestFoliage)
	);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static uint32 GDebugTag = 1;
	static uint32 GCaptureDebugRuns = 0;
#endif

static void FreezeFoliageCulling(const TArray<FString>& Args)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogConsoleResponse, Display, "Freezing Foliage Culling.");
	GDebugTag++;
	GCaptureDebugRuns = GDebugTag;
#endif
}

static FAutoConsoleCommand FreezeFoliageCullingCmd(
	TEXT("foliage.Freeze"),
	TEXT("Useful for debugging. Freezes the foliage culling and LOD."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FreezeFoliageCulling)
	);

static void UnFreezeFoliageCulling(const TArray<FString>& Args)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogConsoleResponse, Display, "Unfreezing Foliage Culling.");
	GDebugTag++;
	GCaptureDebugRuns = 0;
#endif
}

static FAutoConsoleCommand UnFreezeFoliageCullingCmd(
	TEXT("foliage.UnFreeze"),
	TEXT("Useful for debugging. Freezes the foliage culling and LOD."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&UnFreezeFoliageCulling)
	);

void ToggleFreezeFoliageCulling()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TArray<FString> Args;

	if (GCaptureDebugRuns == 0)
	{
		FreezeFoliageCulling(Args);
	}
	else
	{
		UnFreezeFoliageCulling(Args);
	}
#endif
}

FHierarchicalInstancedStaticMeshDelegates::FOnTreeBuilt FHierarchicalInstancedStaticMeshDelegates::OnTreeBuilt;

SIZE_T FHierarchicalStaticMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FHierarchicalStaticMeshSceneProxy::FHierarchicalStaticMeshSceneProxy(UHierarchicalInstancedStaticMeshComponent* InComponent, ERHIFeatureLevel::Type InFeatureLevel)
: FInstancedStaticMeshSceneProxy((UInstancedStaticMeshComponent*)InComponent, InFeatureLevel)
, ClusterTreePtr(InComponent->ClusterTreePtr.ToSharedRef())
, ClusterTree(*InComponent->ClusterTreePtr)
, UnbuiltBounds(InComponent->UnbuiltInstanceBoundsList)
, FirstUnbuiltIndex(InComponent->NumBuiltInstances > 0 ? InComponent->NumBuiltInstances : InComponent->NumBuiltRenderInstances)
, InstanceCountToRender(InComponent->InstanceCountToRender)
, ViewRelevance(InComponent->GetViewRelevanceType())
, bDitheredLODTransitions(InComponent->SupportsDitheredLODTransitions(GetScene().GetShaderPlatform()))
, SceneProxyCreatedFrameNumberRenderThread(UINT32_MAX)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
, CaptureTag(0)
#endif
{
	SetupOcclusion(InComponent);

	// Dynamic draw path without Nanite isn't supported by Lumen
	bVisibleInLumenScene = false;

	bIsHierarchicalInstancedStaticMesh = true;
	bIsLandscapeGrass = (ViewRelevance == EHISMViewRelevanceType::Grass);

#if RHI_RAYTRACING
	bDynamicRayTracingInstances = true;
#endif

	// Store LODDistanceScale so it can be used in FInstancedStaticMeshVertexFactoryShaderParameters::GetElementShaderBindings when dither LOD transitions are enabled
	float LODDistanceScale = InComponent->InstanceLODDistanceScale * CVarFoliageLODDistanceScale.GetValueOnGameThread();
	UserData_AllInstances.LODDistanceScale = LODDistanceScale;
	UserData_AllInstances.AverageInstancesScale = InComponent->GetAverageScale();
}

void FHierarchicalStaticMeshSceneProxy::SetupOcclusion(UHierarchicalInstancedStaticMeshComponent* InComponent)
{
	FirstOcclusionNode = 0;
	LastOcclusionNode = 0;
	if (ClusterTree.Num() && InComponent->OcclusionLayerNumNodes)
	{
		while (true)
		{
			int32 NextFirstOcclusionNode = ClusterTree[FirstOcclusionNode].FirstChild;
			int32 NextLastOcclusionNode = ClusterTree[LastOcclusionNode].LastChild;

			if (NextFirstOcclusionNode < 0 || NextLastOcclusionNode < 0)
			{
				break;
			}
			int32 NumNodes = 1 + NextLastOcclusionNode - NextFirstOcclusionNode;
			if (NumNodes > InComponent->OcclusionLayerNumNodes)
			{
				break;
			}
			FirstOcclusionNode = NextFirstOcclusionNode;
			LastOcclusionNode = NextLastOcclusionNode;
		}
	}
	int32 NumNodes = 1 + LastOcclusionNode - FirstOcclusionNode;
	if (NumNodes < 2)
	{
		FirstOcclusionNode = -1;
		LastOcclusionNode = -1;
		NumNodes = 0;
		if (ClusterTree.Num())
		{
			//UE_LOGF(LogTemp, Display, "No SubOcclusion %d inst", 1 + ClusterTree[0].LastInstance - ClusterTree[0].FirstInstance);
		}
	}
	else
	{
		//int32 NumPerNode = (1 + ClusterTree[0].LastInstance - ClusterTree[0].FirstInstance) / NumNodes;
		//UE_LOGF(LogTemp, Display, "Occlusion level %d   %d inst / node", NumNodes, NumPerNode);
		OcclusionBounds.Reserve(NumNodes);
		FMatrix XForm = InComponent->GetRenderMatrix();

		const float MaxWorldPositionOffset = GetMaxWorldPositionOffsetExtent();

		for (int32 Index = FirstOcclusionNode; Index <= LastOcclusionNode; Index++)
		{
			OcclusionBounds.Add(FBoxSphereBounds(FBox(ClusterTree[Index].BoundMin, ClusterTree[Index].BoundMax).ExpandBy(MaxWorldPositionOffset).TransformBy(XForm)));
		}
	}
}

void FHierarchicalStaticMeshSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FInstancedStaticMeshSceneProxy::CreateRenderThreadResources(RHICmdList);
	SceneProxyCreatedFrameNumberRenderThread = GFrameNumberRenderThread;
}
	
FPrimitiveViewRelevance FHierarchicalStaticMeshSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	bool bShowInstancedMesh = true;
	switch (ViewRelevance)
	{
	case EHISMViewRelevanceType::Grass:
		bShowInstancedMesh = View->Family->EngineShowFlags.InstancedGrass;
		break;
	case EHISMViewRelevanceType::Foliage:
		bShowInstancedMesh = View->Family->EngineShowFlags.InstancedFoliage;
		break;
	case EHISMViewRelevanceType::HISM:
		bShowInstancedMesh = View->Family->EngineShowFlags.InstancedStaticMeshes;
		break;
	default:
		break;
	}
	if (bShowInstancedMesh)
	{
		Result = FStaticMeshSceneProxy::GetViewRelevance(View);
		Result.bDynamicRelevance = true;
		Result.bStaticRelevance = false;

		// Remove relevance for primitives marked for runtime virtual texture only.
		if (RuntimeVirtualTextures.Num() > 0 && !ShouldRenderInMainPass())
		{
			Result.bDynamicRelevance = false;
		}
	}
	return Result;
}

void FHierarchicalStaticMeshSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	if (RuntimeVirtualTextures.Num() > 0)
	{
		// Create non-hierarchal static mesh batches for use by the runtime virtual texture rendering.
		//todo[vt]: Build an acceleration structure better suited for VT rendering maybe with batches aligned to VT pages?
		FInstancedStaticMeshSceneProxy::DrawStaticElements(PDI);
	}
}

void FHierarchicalStaticMeshSceneProxy::ApplyWorldOffset(FRHICommandListBase& RHICmdList, FVector InOffset)
{
	FInstancedStaticMeshSceneProxy::ApplyWorldOffset(RHICmdList, InOffset);
		
	for (FBoxSphereBounds& Item : OcclusionBounds)
	{
		Item.Origin+= InOffset;
	}
}

struct FFoliageRenderInstanceParams : public FOneFrameResource
{
	bool bNeedsSingleLODRuns;
	bool bNeedsMultipleLODRuns;
	bool bOverestimate;
	mutable TArray<uint32, SceneRenderingAllocator> MultipleLODRuns[MAX_STATIC_MESH_LODS];
	mutable TArray<uint32, SceneRenderingAllocator> SingleLODRuns[MAX_STATIC_MESH_LODS];
	mutable int32 TotalSingleLODInstances[MAX_STATIC_MESH_LODS];
	mutable int32 TotalMultipleLODInstances[MAX_STATIC_MESH_LODS];

	FFoliageRenderInstanceParams(bool InbNeedsSingleLODRuns, bool InbNeedsMultipleLODRuns, bool InbOverestimate)
		: bNeedsSingleLODRuns(InbNeedsSingleLODRuns)
		, bNeedsMultipleLODRuns(InbNeedsMultipleLODRuns)
		, bOverestimate(InbOverestimate)
	{
		for (int32 Index = 0; Index < MAX_STATIC_MESH_LODS; Index++)
		{
			TotalSingleLODInstances[Index] = 0;
			TotalMultipleLODInstances[Index] = 0;
		}
	}
	static FORCEINLINE_DEBUGGABLE void AddRun(TArray<uint32, SceneRenderingAllocator>& Array, int32 FirstInstance, int32 LastInstance)
	{
		if (Array.Num() && Array.Last() + 1 == FirstInstance)
		{
			Array.Last() = (uint32)LastInstance;
		}
		else
		{
			Array.Add((uint32)FirstInstance);
			Array.Add((uint32)LastInstance);
		}
	}
	FORCEINLINE_DEBUGGABLE void AddRun(int32 MinLod, int32 MaxLod, int32 FirstInstance, int32 LastInstance) const
	{
		if (bNeedsSingleLODRuns)
		{
			int32 CurrentLOD = bOverestimate ? MaxLod : MinLod;

			if (CurrentLOD < MAX_STATIC_MESH_LODS)
			{
				AddRun(SingleLODRuns[CurrentLOD], FirstInstance, LastInstance);
				TotalSingleLODInstances[CurrentLOD] += 1 + LastInstance - FirstInstance;
			}
		}
		if (bNeedsMultipleLODRuns)
		{
			for (int32 Lod = MinLod; Lod <= MaxLod; Lod++)
			{
				if (Lod < MAX_STATIC_MESH_LODS)
				{
					TotalMultipleLODInstances[Lod] += 1 + LastInstance - FirstInstance;
					AddRun(MultipleLODRuns[Lod], FirstInstance, LastInstance);
				}
			}
		}
	}

	FORCEINLINE_DEBUGGABLE void AddRun(int32 MinLod, int32 MaxLod, const FClusterNode& Node) const
	{
		AddRun(MinLod, MaxLod, Node.FirstInstance, Node.LastInstance);
	}
};

struct FFoliageCullInstanceParams : public FFoliageRenderInstanceParams
{
	FConvexVolume ViewFrustumLocal;
	int32 MinInstancesToSplit[MAX_STATIC_MESH_LODS];
	const TArray<FClusterNode>& Tree;
	const FSceneView* View;
	FVector ViewOriginInLocalZero;
	FVector ViewOriginInLocalOne;
	int32 LODs;
	float LODPlanesMax[MAX_STATIC_MESH_LODS];
	float LODPlanesMin[MAX_STATIC_MESH_LODS];
	int32 FirstOcclusionNode;
	int32 LastOcclusionNode;
	const TArray<bool>* OcclusionResults;
	int32 OcclusionResultsStart;
	float MaxWPODisplacement;


	FFoliageCullInstanceParams(bool InbNeedsSingleLODRuns, bool InbNeedsMultipleLODRuns, bool InbOverestimate, const TArray<FClusterNode>& InTree)
	:	FFoliageRenderInstanceParams(InbNeedsSingleLODRuns, InbNeedsMultipleLODRuns, InbOverestimate)
	,	Tree(InTree)
	,	FirstOcclusionNode(-1)
	,	LastOcclusionNode(-1)
	,	OcclusionResults(nullptr)
	,	OcclusionResultsStart(0)
	,	MaxWPODisplacement(0)
	{
	}
};

static bool GUseVectorCull = true;

static void ToggleUseVectorCull(const TArray<FString>& Args)
{
	GUseVectorCull = !GUseVectorCull;
}

static FAutoConsoleCommand ToggleUseVectorCullCmd(
	TEXT("foliage.ToggleVectorCull"),
	TEXT("Useful for debugging. Toggles the optimized cull."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&ToggleUseVectorCull)
	);

static uint32 GFrameNumberRenderThread_CaptureFoliageRuns = MAX_uint32;

static void LogFoliageFrame(const TArray<FString>& Args)
{
	GFrameNumberRenderThread_CaptureFoliageRuns = GFrameNumberRenderThread + 2;
}

static FAutoConsoleCommand LogFoliageFrameCmd(
	TEXT("foliage.LogFoliageFrame"),
	TEXT("Useful for debugging. Logs all foliage rendered in a frame."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&LogFoliageFrame)
	);

const VectorRegister		VECTOR_HALF_HALF_HALF_ZERO				= DECLARE_VECTOR_REGISTER(0.5f, 0.5f, 0.5f, 0.0f);

template<bool TUseVector>
static FORCEINLINE_DEBUGGABLE bool CullNode(const FFoliageCullInstanceParams& Params, const FVector& BoundMin, const FVector& BoundMax, bool& bOutFullyContained)
{
	if (TUseVector)
	{
		checkSlow(Params.ViewFrustumLocal.PermutedPlanes.Num() == 4);

		//@todo, once we have more than one mesh per tree, these should be aligned
		VectorRegister BoxMin = VectorLoadFloat3(&BoundMin);
		VectorRegister BoxMax = VectorLoadFloat3(&BoundMax);

		VectorRegister BoxDiff = VectorSubtract(BoxMax,BoxMin);
		VectorRegister BoxSum = VectorAdd(BoxMax,BoxMin);

		// Load the origin & extent
		VectorRegister Orig = VectorMultiply(VECTOR_HALF_HALF_HALF_ZERO, BoxSum);
		VectorRegister Ext = VectorMultiply(VECTOR_HALF_HALF_HALF_ZERO, BoxDiff);
		// Splat origin into 3 vectors
		VectorRegister OrigX = VectorReplicate(Orig, 0);
		VectorRegister OrigY = VectorReplicate(Orig, 1);
		VectorRegister OrigZ = VectorReplicate(Orig, 2);
		// Splat the abs for the pushout calculation
		VectorRegister AbsExtentX = VectorReplicate(Ext, 0);
		VectorRegister AbsExtentY = VectorReplicate(Ext, 1);
		VectorRegister AbsExtentZ = VectorReplicate(Ext, 2);
		// Since we are moving straight through get a pointer to the data
		const FPlane* RESTRICT PermutedPlanePtr = (FPlane*)Params.ViewFrustumLocal.PermutedPlanes.GetData();
		// Process four planes at a time until we have < 4 left
		// Load 4 planes that are already all Xs, Ys, ...
		VectorRegister PlanesX = VectorLoadAligned(&PermutedPlanePtr[0]);
		VectorRegister PlanesY = VectorLoadAligned(&PermutedPlanePtr[1]);
		VectorRegister PlanesZ = VectorLoadAligned(&PermutedPlanePtr[2]);
		VectorRegister PlanesW = VectorLoadAligned(&PermutedPlanePtr[3]);
		// Calculate the distance (x * x) + (y * y) + (z * z) - w
		VectorRegister DistX = VectorMultiply(OrigX,PlanesX);
		VectorRegister DistY = VectorMultiplyAdd(OrigY,PlanesY,DistX);
		VectorRegister DistZ = VectorMultiplyAdd(OrigZ,PlanesZ,DistY);
		VectorRegister Distance = VectorSubtract(DistZ,PlanesW);
		// Now do the push out FMath::Abs(x * x) + FMath::Abs(y * y) + FMath::Abs(z * z)
		VectorRegister PushX = VectorMultiply(AbsExtentX,VectorAbs(PlanesX));
		VectorRegister PushY = VectorMultiplyAdd(AbsExtentY,VectorAbs(PlanesY),PushX);
		VectorRegister PushOut = VectorMultiplyAdd(AbsExtentZ,VectorAbs(PlanesZ),PushY);
		VectorRegister PushOutNegative = VectorNegate(PushOut);

		bOutFullyContained = !VectorAnyGreaterThan(Distance,PushOutNegative);
		// Check for completely outside
		return !!VectorAnyGreaterThan(Distance,PushOut);
	}
	FVector Center = (BoundMin + BoundMax) * 0.5f;
	FVector Extent = (BoundMax - BoundMin) * 0.5f;
	if (!Params.ViewFrustumLocal.IntersectBox(Center, Extent, bOutFullyContained)) 
	{
		return true;
	}
	return false;
}

inline void CalcLOD(int32& InOutMinLOD, int32& InOutMaxLOD, const FVector& BoundMin, const FVector& BoundMax, const FVector& ViewOriginInLocalZero, const FVector& ViewOriginInLocalOne, const float (&LODPlanesMin)[MAX_STATIC_MESH_LODS], const float (&LODPlanesMax)[MAX_STATIC_MESH_LODS])
{
	if (InOutMinLOD != InOutMaxLOD)
	{
		const FVector Center = (BoundMax + BoundMin) * 0.5f;
		const float DistCenterZero = FVector::Dist(Center, ViewOriginInLocalZero);
		const float DistCenterOne = FVector::Dist(Center, ViewOriginInLocalOne);
		const float HalfWidth = FVector::Dist(BoundMax, BoundMin) * 0.5f;
		const float NearDot = FMath::Min(DistCenterZero, DistCenterOne) - HalfWidth;
		const float FarDot = FMath::Max(DistCenterZero, DistCenterOne) + HalfWidth;

		while (InOutMaxLOD > InOutMinLOD && NearDot > LODPlanesMax[InOutMinLOD])
		{
			InOutMinLOD++;
		}
		while (InOutMaxLOD > InOutMinLOD && FarDot < LODPlanesMin[InOutMaxLOD - 1])
		{
			InOutMaxLOD--;
		}
	}
}

inline bool CanGroup(const FVector& BoundMin, const FVector& BoundMax, const FVector& ViewOriginInLocalZero, const FVector& ViewOriginInLocalOne, float MaxDrawDist)
{
	const FVector Center = (BoundMax + BoundMin) * 0.5f;
	const float DistCenterZero = FVector::Dist(Center, ViewOriginInLocalZero);
	const float DistCenterOne = FVector::Dist(Center, ViewOriginInLocalOne);
	const float HalfWidth = FVector::Dist(BoundMax, BoundMin) * 0.5f;
	const float FarDot = FMath::Max(DistCenterZero, DistCenterOne) + HalfWidth;

	// We are sure that everything in the bound won't be distance culled
	return FarDot < MaxDrawDist;
}



template<bool TUseVector, bool THasWPODisplacement>
void FHierarchicalStaticMeshSceneProxy::Traverse(const FFoliageCullInstanceParams& Params, int32 Index, int32 MinLOD, int32 MaxLOD, bool bFullyContained) const
{
	const FClusterNode& Node = Params.Tree[Index];

	FVector BoundMin = (FVector)Node.BoundMin;
	FVector BoundMax = (FVector)Node.BoundMax;

	if (THasWPODisplacement)
	{
		BoundMin -= FVector(Params.MaxWPODisplacement);
		BoundMax += FVector(Params.MaxWPODisplacement);
	}

	if (!bFullyContained)
	{
		if (CullNode<TUseVector>(Params, BoundMin, BoundMax, bFullyContained))
		{
			return;
		}
	}

	if (MinLOD != MaxLOD)
	{
		CalcLOD(MinLOD, MaxLOD, BoundMin, BoundMax, Params.ViewOriginInLocalZero, Params.ViewOriginInLocalOne, Params.LODPlanesMin, Params.LODPlanesMax);

		if (MinLOD >= Params.LODs)
		{
			return;
		}
	}

	if (Index >= Params.FirstOcclusionNode && Index <= Params.LastOcclusionNode)
	{
		check(Params.OcclusionResults != NULL);
		const TArray<bool>& OcclusionResultsArray = *Params.OcclusionResults;
		if (OcclusionResultsArray[Params.OcclusionResultsStart + Index - Params.FirstOcclusionNode])
		{
			INC_DWORD_STAT_BY(STAT_OcclusionCulledFoliageInstances, 1 + Node.LastInstance - Node.FirstInstance);
			return;
		}
	}

	bool bShouldGroup = Node.FirstChild < 0
		|| ((Node.LastInstance - Node.FirstInstance + 1) < Params.MinInstancesToSplit[MinLOD]
			&& CanGroup(BoundMin, BoundMax, Params.ViewOriginInLocalZero, Params.ViewOriginInLocalOne, Params.LODPlanesMax[Params.LODs - 1]));
	bool bSplit = (!bFullyContained || MinLOD < MaxLOD || Index < Params.FirstOcclusionNode)
		&& !bShouldGroup;

	if (!bSplit)
	{
		MaxLOD = FMath::Min(MaxLOD, Params.LODs - 1);
		Params.AddRun(MinLOD, MaxLOD, Node);
		return;
	}
	for (int32 ChildIndex = Node.FirstChild; ChildIndex <= Node.LastChild; ChildIndex++)
	{
		Traverse<TUseVector, THasWPODisplacement>(Params, ChildIndex, MinLOD, MaxLOD, bFullyContained);
	}
}

struct FFoliageElementParams
{
	const FInstancingUserData* PassUserData;
	const FSceneView* View;
	int32 ViewIndex;
	bool bSelectionRenderEnabled;
	bool bBatchRenderSelection;
	bool bIsWireframe;
	bool bUseHoveredMaterial;
	bool bUseInstanceRuns;
	bool bBlendLODs;
	ERHIFeatureLevel::Type FeatureLevel;
	bool ShadowFrustum;
	float FinalCullDistance;
};

void FHierarchicalStaticMeshSceneProxy::FillDynamicMeshElements(const FSceneView* View, FMeshElementCollector& Collector, const FFoliageElementParams& ElementParams, const FFoliageRenderInstanceParams& Params) const
{
	SCOPE_CYCLE_COUNTER(STAT_FoliageBatchTime);
	int64 TotalTriangles = 0;

	const int32 NumVertexFactories = GetNumRenderDataVertexFactories();

	int32 OnlyLOD = FMath::Min<int32>(CVarOnlyLOD.GetValueOnRenderThread(), NumVertexFactories - 1);
	int32 FirstLOD = FMath::Max((OnlyLOD < 0) ? 0 : OnlyLOD, static_cast<int32>(this->GetCurrentFirstLODIdx_Internal()));
	int32 LastLODPlusOne = (OnlyLOD < 0) ? NumVertexFactories : (OnlyLOD+1);

	const bool bUseGPUScene = UseGPUScene(GetScene().GetShaderPlatform());

	for (int32 LODIndex = FirstLOD; LODIndex < LastLODPlusOne; LODIndex++)
	{
		const bool bDitherLODEnabled = ElementParams.bBlendLODs;
		const uint32 InstancedLODRange = bDitherLODEnabled ? 1 : 0;

		TArray<uint32, SceneRenderingAllocator>& RunArray = bDitherLODEnabled ? Params.MultipleLODRuns[LODIndex] : Params.SingleLODRuns[LODIndex];

		// No need to create uniform buffer if array is empty for given LOD
		if (!RunArray.Num())
		{
			continue;
		}

		const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];
		const int32 TotalNumSections = LODModel.Sections.Num();

		{
			FInstancedStaticMeshVFLooseUniformShaderParametersRef LooseUniformBuffer;
			if (InstancedRenderData)
			{
				LooseUniformBuffer = CreateLooseUniformBuffer(View, InstancedLODRange, LODIndex, EUniformBufferUsage::UniformBuffer_SingleFrame);
			}

			for (int32 SectionIndex = 0; SectionIndex < TotalNumSections; SectionIndex++)
			{
				const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];

				// No need to allocate mesh batch if section is empty
				if (Section.NumTriangles == 0)
				{
					continue;
				}

				if (bUseGPUScene)
				{
					FMeshBatch& MeshBatch = Collector.AllocateMesh();
					INC_DWORD_STAT(STAT_FoliageMeshBatches);

					if (!FStaticMeshSceneProxy::GetMeshElement(LODIndex, 0, SectionIndex, GetDepthPriorityGroup(ElementParams.View), ElementParams.bBatchRenderSelection, true, MeshBatch))
					{
						continue;
					}

					checkSlow(MeshBatch.GetNumPrimitives() > 0);
					MeshBatch.bCanApplyViewModeOverrides = true;
					MeshBatch.bUseSelectionOutline = ElementParams.bBatchRenderSelection;
					MeshBatch.bUseWireframeSelectionColoring = ElementParams.bBatchRenderSelection;
					MeshBatch.bUseAsOccluder = ShouldUseAsOccluder();
					FMeshBatchElement& MeshBatchElement = MeshBatch.Elements[0];
					if (InstancedRenderData)
					{
						MeshBatch.VertexFactory = &InstancedRenderData->VertexFactories[LODIndex];
					}
					// For dither LOD to work we need to override the VF UB with per-view dynamic data
					else if (bDitherLODEnabled)
					{
						const FLocalVertexFactory* VF = static_cast<const FLocalVertexFactory*>(MeshBatch.VertexFactory);
						FLocalVertexFactoryInstanceDitherParameters InstanceDitherParameters = GetInstanceDitherParameters(View, InstancedLODRange, LODIndex);
						MeshBatchElement.VertexFactoryUserData = CreateLocalVFUniformBuffer(VF, VF->GetLightMapCoordinateIndex(), nullptr, 0, 0, &InstanceDitherParameters, EUniformBufferUsage::UniformBuffer_SingleFrame);
					}

					MeshBatchElement.UserData = ElementParams.PassUserData;
					MeshBatchElement.bUserDataIsColorVertexBuffer = false;
					MeshBatchElement.MaxScreenSize = 1.0;
					MeshBatchElement.MinScreenSize = 0.0;
					MeshBatchElement.InstancedLODIndex = LODIndex;
					MeshBatchElement.InstancedLODRange = InstancedLODRange;
					MeshBatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
					MeshBatchElement.LooseParametersUniformBuffer = LooseUniformBuffer;
					MeshBatchElement.bForceInstanceCulling = true; // force ISM through Generic path even for a single instance cases

					int32 TotalInstances = bDitherLODEnabled ? Params.TotalMultipleLODInstances[LODIndex] : Params.TotalSingleLODInstances[LODIndex];
					{
						const int64 Tris = int64(TotalInstances) * int64(MeshBatchElement.NumPrimitives);
						TotalTriangles += Tris;
#if STATS
						if (GFrameNumberRenderThread_CaptureFoliageRuns == GFrameNumberRenderThread)
						{
							if (ElementParams.FinalCullDistance > 9.9E8)
							{
								UE_LOGF(LogStaticMesh, Display, "lod:%1d/%1d   section:%1d/%1d   runs:%4d   inst:%8d   tris:%9lld   cast shadow:%1d   cull:-NONE!!-   shadow:%1d     %ls %ls",
									LODIndex, NumVertexFactories, SectionIndex, TotalNumSections, RunArray.Num() / 2,
									TotalInstances, Tris, (int)MeshBatch.CastShadow, ElementParams.ShadowFrustum,
									*StaticMesh->GetPathName(),
									*MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(ElementParams.FeatureLevel).GetFriendlyName());
							}
							else
							{
								UE_LOGF(LogStaticMesh, Display, "lod:%1d/%1d   section:%1d/%1d   runs:%4d   inst:%8d   tris:%9lld   cast shadow:%1d   cull:%8.0f   shadow:%1d     %ls %ls",
									LODIndex, NumVertexFactories, SectionIndex, TotalNumSections, RunArray.Num() / 2,
									TotalInstances, Tris, (int)MeshBatch.CastShadow, ElementParams.FinalCullDistance, ElementParams.ShadowFrustum,
									*StaticMesh->GetPathName(),
									*MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(ElementParams.FeatureLevel).GetFriendlyName());
							}
						}
#endif // STATS
					}

					//MeshBatchElement.NumInstances = TotalInstances;
					// The index was used as an offset, but the dynamic buffer thing uses a resource view to make this not needed (using PrimitiveInstanceSceneDataOffset as a temp. debug help)
					MeshBatchElement.UserIndex = 0;

					// Note: this call overrides the UserIndex to mean the command index, which is used to fetch the offset to the instance array
					//Collector.AllocateInstancedBatchArguments(ElementParams.ViewIndex, MeshBatch, PrimitiveInstanceSceneDataOffset, PrimitiveInstanceDataCount, RunArray);

					// We use this existing hook to send info about the runs over to the visible mesh batch
					MeshBatchElement.NumInstances = RunArray.Num() / 2;
					MeshBatchElement.InstanceRuns = &RunArray[0];
					MeshBatchElement.bIsInstanceRuns = true;

					if (TotalTriangles < (int64)CVarMaxTrianglesToRender.GetValueOnRenderThread())
					{
						Collector.AddMesh(ElementParams.ViewIndex, MeshBatch);

						if (OverlayMaterial != nullptr)
						{
							FMeshBatch& OverlayMeshBatch = Collector.AllocateMesh();
							OverlayMeshBatch = MeshBatch;
							OverlayMeshBatch.bOverlayMaterial = true;
							OverlayMeshBatch.CastShadow = false;
							OverlayMeshBatch.bSelectable = false;
							OverlayMeshBatch.MaterialRenderProxy = OverlayMaterial->GetRenderProxy();
							// make sure overlay is always rendered on top of base mesh
							OverlayMeshBatch.MeshIdInPrimitive += TotalNumSections;
							Collector.AddMesh(ElementParams.ViewIndex, OverlayMeshBatch);
						}
					}
				}
				else if (ensure(InstancedRenderData))
				{
					ensure(DoesPlatformUseInstancedStaticMeshVertexFactory(GetScene().GetShaderPlatform()));
					int32 NumBatches = 1;
					int32 CurrentRun = 0;
					int32 CurrentInstance = 0;
					int32 RemainingInstances = bDitherLODEnabled ? Params.TotalMultipleLODInstances[LODIndex] : Params.TotalSingleLODInstances[LODIndex];
					int32 RemainingRuns = RunArray.Num() / 2;

					if (!ElementParams.bUseInstanceRuns)
					{
						NumBatches = FMath::DivideAndRoundUp(RemainingRuns, (int32)FInstancedStaticMeshVertexFactory::NumBitsForVisibilityMask());
					}

#if STATS
					INC_DWORD_STAT_BY(STAT_FoliageInstances, RemainingInstances);
#endif
					bool bDidStats = false;
					for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
					{
						FMeshBatch& MeshElement = Collector.AllocateMesh();
						INC_DWORD_STAT(STAT_FoliageMeshBatches);

						if (!FStaticMeshSceneProxy::GetMeshElement(LODIndex, 0, SectionIndex, GetDepthPriorityGroup(ElementParams.View), ElementParams.bBatchRenderSelection, true, MeshElement))
						{
							continue;
						}
						checkSlow(MeshElement.GetNumPrimitives() > 0);

						MeshElement.VertexFactory = &InstancedRenderData->VertexFactories[LODIndex];
						FMeshBatchElement& BatchElement0 = MeshElement.Elements[0];

						BatchElement0.UserData = ElementParams.PassUserData;
						BatchElement0.bUserDataIsColorVertexBuffer = false;
						BatchElement0.MaxScreenSize = 1.0;
						BatchElement0.MinScreenSize = 0.0;
						BatchElement0.InstancedLODIndex = LODIndex;
						BatchElement0.InstancedLODRange = InstancedLODRange;
						BatchElement0.PrimitiveUniformBuffer = GetUniformBuffer();
						BatchElement0.LooseParametersUniformBuffer = LooseUniformBuffer;
						MeshElement.bCanApplyViewModeOverrides = true;
						MeshElement.bUseSelectionOutline = ElementParams.bBatchRenderSelection;
						MeshElement.bUseWireframeSelectionColoring = ElementParams.bBatchRenderSelection;
						MeshElement.bUseAsOccluder = ShouldUseAsOccluder();

						if (!bDidStats)
						{
							bDidStats = true;
							int64 Tris = int64(RemainingInstances) * int64(BatchElement0.NumPrimitives);
							TotalTriangles += Tris;
#if STATS
							if (GFrameNumberRenderThread_CaptureFoliageRuns == GFrameNumberRenderThread)
							{
								if (ElementParams.FinalCullDistance > 9.9E8)
								{
									UE_LOGF(LogStaticMesh, Display, "lod:%1d/%1d   section:%1d/%1d   runs:%4d   inst:%8d   tris:%9lld   cast shadow:%1d   cull:-NONE!!-   shadow:%1d     %ls %ls",
										LODIndex, InstancedRenderData->VertexFactories.Num(), SectionIndex, TotalNumSections, RunArray.Num() / 2,
										RemainingInstances, Tris, (int)MeshElement.CastShadow, ElementParams.ShadowFrustum,
										*StaticMesh->GetPathName(),
										*MeshElement.MaterialRenderProxy->GetIncompleteMaterialWithFallback(ElementParams.FeatureLevel).GetFriendlyName());
								}
								else
								{
									UE_LOGF(LogStaticMesh, Display, "lod:%1d/%1d   section:%1d/%1d   runs:%4d   inst:%8d   tris:%9lld   cast shadow:%1d   cull:%8.0f   shadow:%1d     %ls %ls",
										LODIndex, InstancedRenderData->VertexFactories.Num(), SectionIndex, TotalNumSections, RunArray.Num() / 2,
										RemainingInstances, Tris, (int)MeshElement.CastShadow, ElementParams.FinalCullDistance, ElementParams.ShadowFrustum,
										*StaticMesh->GetPathName(),
										*MeshElement.MaterialRenderProxy->GetIncompleteMaterialWithFallback(ElementParams.FeatureLevel).GetFriendlyName());
								}
							}
#endif
						}
						if (ElementParams.bUseInstanceRuns)
						{
							BatchElement0.NumInstances = RunArray.Num() / 2;
							BatchElement0.InstanceRuns = &RunArray[0];
							BatchElement0.bIsInstanceRuns = true;
#if STATS
							INC_DWORD_STAT_BY(STAT_FoliageRuns, BatchElement0.NumInstances);
#endif
						}
						else
						{
							const uint32 NumElementsThisBatch = FMath::Min(RemainingRuns, (int32)FInstancedStaticMeshVertexFactory::NumBitsForVisibilityMask());

							MeshElement.Elements.Reserve(NumElementsThisBatch);
							check(NumElementsThisBatch);

							for (uint32 InstanceRun = 0; InstanceRun < NumElementsThisBatch; ++InstanceRun)
							{
								FMeshBatchElement* NewBatchElement;

								if (InstanceRun == 0)
								{
									NewBatchElement = &MeshElement.Elements[0];
								}
								else
								{
									NewBatchElement = &MeshElement.Elements.AddDefaulted_GetRef();
									*NewBatchElement = MeshElement.Elements[0];
								}

								const int32 InstanceOffset = RunArray[CurrentRun];
								NewBatchElement->UserIndex = InstanceOffset;
								NewBatchElement->NumInstances = 1 + RunArray[CurrentRun + 1] - InstanceOffset;

								if (--RemainingRuns)
								{
									CurrentRun += 2;
									check(CurrentRun + 1 < RunArray.Num());
								}
							}
						}

						if (TotalTriangles < (int64)CVarMaxTrianglesToRender.GetValueOnRenderThread())
						{
							Collector.AddMesh(ElementParams.ViewIndex, MeshElement);

							if (OverlayMaterial != nullptr)
							{
								FMeshBatch& OverlayMeshBatch = Collector.AllocateMesh();
								OverlayMeshBatch = MeshElement;
								OverlayMeshBatch.bOverlayMaterial = true;
								OverlayMeshBatch.CastShadow = false;
								OverlayMeshBatch.bSelectable = false;
								OverlayMeshBatch.MaterialRenderProxy = OverlayMaterial->GetRenderProxy();
								// make sure overlay is always rendered on top of base mesh
								OverlayMeshBatch.MeshIdInPrimitive += TotalNumSections;
								Collector.AddMesh(ElementParams.ViewIndex, OverlayMeshBatch);
							}
						}
					}
				}
			}
		}
	}
#if STATS
	TotalTriangles*= (OverlayMaterial != nullptr ? 2 : 1);
	TotalTriangles = FMath::Min<int64>(TotalTriangles, MAX_int32);
	INC_DWORD_STAT_BY(STAT_FoliageTriangles, (uint32)TotalTriangles);
	INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, (uint32)TotalTriangles);
#endif
}

void FHierarchicalStaticMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	if (Views[0]->bRenderFirstInstanceOnly)
	{
		FInstancedStaticMeshSceneProxy::GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_HierarchicalInstancedStaticMeshSceneProxy_GetMeshElements);
	SCOPE_CYCLE_COUNTER(STAT_HISMCGetDynamicMeshElement);

	bool bMultipleSections = bDitheredLODTransitions && CVarDitheredLOD.GetValueOnRenderThread() > 0;
	// Disable multiple selections when forced LOD is set
	bMultipleSections = bMultipleSections && ForcedLodModel <= 0 && CVarForceLOD.GetValueOnRenderThread() < 0;

	bool bSingleSections = !bMultipleSections;
	bool bOverestimate = CVarOverestimateLOD.GetValueOnRenderThread() > 0;

	int32 MinVertsToSplitNode = CVarMinVertsToSplitNode.GetValueOnRenderThread();

	const FMatrix WorldToLocal = GetLocalToWorld().Inverse();

	const bool bApplyViewOcclusionCullToShadow = CVarApplyViewOcclusionCullToShadow.GetValueOnAnyThread();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

			FFoliageElementParams ElementParams;
			ElementParams.bSelectionRenderEnabled = GIsEditor && ViewFamily.EngineShowFlags.Selection;
			ElementParams.PassUserData = &UserData_AllInstances;
			ElementParams.bBatchRenderSelection = ElementParams.bSelectionRenderEnabled && IsSelected();
			ElementParams.bIsWireframe = ViewFamily.EngineShowFlags.Wireframe;
			ElementParams.bUseHoveredMaterial = IsHovered();
			ElementParams.bUseInstanceRuns = (CVarFoliageUseInstanceRuns.GetValueOnRenderThread() > 0);
			ElementParams.FeatureLevel = GetScene().GetFeatureLevel();
			ElementParams.ViewIndex = ViewIndex;
			ElementParams.View = View;

			// Render built instances
			if (ClusterTree.Num())
			{
				FFoliageCullInstanceParams& InstanceParams = Collector.AllocateOneFrameResource<FFoliageCullInstanceParams>(bSingleSections, bMultipleSections, bOverestimate, ClusterTree);
				InstanceParams.LODs = RenderData->LODResources.Num();

				InstanceParams.View = View;

				bool bUseVectorCull = GUseVectorCull;
				bool bIsOrtho = false;

				bool bDisableCull = !!CVarDisableCull.GetValueOnRenderThread();
				ElementParams.ShadowFrustum = !!View->GetDynamicMeshElementsShadowCullFrustum();
				if (View->GetDynamicMeshElementsShadowCullFrustum())
				{
					for (int32 Index = 0; Index < View->GetDynamicMeshElementsShadowCullFrustum()->Planes.Num(); Index++)
					{
						FPlane Src = View->GetDynamicMeshElementsShadowCullFrustum()->Planes[Index];
						FPlane Norm = Src / Src.Size();
						// remove world space preview translation
						Norm.W -= (FVector(Norm) | View->GetPreShadowTranslation());
						FPlane Local = Norm.TransformBy(WorldToLocal);
						FPlane LocalNorm = Local / Local.Size();
						InstanceParams.ViewFrustumLocal.Planes.Add(LocalNorm);
					}
					bUseVectorCull = InstanceParams.ViewFrustumLocal.Planes.Num() == 4;
				}
				else
				{
					// build view frustum with no near plane / no far plane in frustum (far plane culling is done later in the function) : 
					static constexpr bool bViewFrustumUsesNearPlane = false;
					static constexpr bool bViewFrustumUsesFarPlane = false;
					const bool bIsPerspectiveProjection = View->ViewMatrices.IsPerspectiveProjection();

					// Instanced stereo needs to use the right plane from the right eye when constructing the frustum bounds to cull against.
					// Otherwise we'll cull objects visible in the right eye, but not the left.
					if (View->bIsSinglePassStereo && IStereoRendering::IsStereoEyeView(*View) && GEngine->StereoRenderingDevice.IsValid())
					{
						// TODO: Stereo culling frustum needs to use the culling origin instead of the view origin.
						InstanceParams.ViewFrustumLocal = View->GetCullingFrustum();
						for (FPlane& Plane : InstanceParams.ViewFrustumLocal.Planes)
						{
							Plane = Plane.TransformBy(WorldToLocal);
						}
						InstanceParams.ViewFrustumLocal.Init();

						// Invalid bounds retrieved, so skip render of this frame :
						if (bIsPerspectiveProjection && (InstanceParams.ViewFrustumLocal.Planes.Num() != 4))
						{
							// Report the error as a warning (instead of an ensure or a check) as the problem can come from improper user data (invalid transform or view-proj matrix) : 
							ensureMsgf(false, TEXT("Invalid frustum, skipping render of HISM"));
							continue;
						}
					}
					else
					{
						FMatrix LocalViewProjForCulling = GetLocalToWorld() * View->ViewMatrices.GetWorldToClip();

						GetViewFrustumBounds(InstanceParams.ViewFrustumLocal, LocalViewProjForCulling, bViewFrustumUsesNearPlane, bViewFrustumUsesFarPlane);

						// Invalid bounds retrieved, so skip render of this frame :
						if (bIsPerspectiveProjection && (InstanceParams.ViewFrustumLocal.Planes.Num() != 4))
						{
							// Report the error as a warning (instead of an ensure or a check) as the problem can come from improper user data (invalid transform or view-proj matrix) : 
							ensureMsgf(false, TEXT("Invalid frustum, skipping render of HISM : culling view projection matrix:%s"), *LocalViewProjForCulling.ToString());
							continue;
						}
					}

					if (bIsPerspectiveProjection)
					{
						check(InstanceParams.ViewFrustumLocal.Planes.Num() == 4);

						FMatrix ThreePlanes;
						ThreePlanes.SetIdentity();
						ThreePlanes.SetAxes(&InstanceParams.ViewFrustumLocal.Planes[0], &InstanceParams.ViewFrustumLocal.Planes[1], &InstanceParams.ViewFrustumLocal.Planes[2]);
						FVector ProjectionOrigin = ThreePlanes.Inverse().GetTransposed().TransformVector(FVector(InstanceParams.ViewFrustumLocal.Planes[0].W, InstanceParams.ViewFrustumLocal.Planes[1].W, InstanceParams.ViewFrustumLocal.Planes[2].W));

						for (int32 Index = 0; Index < InstanceParams.ViewFrustumLocal.Planes.Num(); Index++)
						{
							FPlane Src = InstanceParams.ViewFrustumLocal.Planes[Index];
							FVector Normal = Src.GetSafeNormal();
							InstanceParams.ViewFrustumLocal.Planes[Index] = FPlane(Normal, Normal | ProjectionOrigin);
						}
					}
					else
					{
						bIsOrtho = true;
						bUseVectorCull = false;
					}
				}
				if (!InstanceParams.ViewFrustumLocal.Planes.Num())
				{
					bDisableCull = true;
				}
				else
				{
					InstanceParams.ViewFrustumLocal.Init();
				}

				ElementParams.bBlendLODs = bMultipleSections;

				InstanceParams.ViewOriginInLocalZero = WorldToLocal.TransformPosition(View->GetTemporalLODOrigin(0, bMultipleSections));
				InstanceParams.ViewOriginInLocalOne = WorldToLocal.TransformPosition(View->GetTemporalLODOrigin(1, bMultipleSections));
				InstanceParams.MaxWPODisplacement = GetMaxWorldPositionOffsetExtent();

				float MinSize = bIsOrtho ? 0.0f : CVarFoliageMinimumScreenSize.GetValueOnRenderThread();
				float LODScale = UserData_AllInstances.LODDistanceScale;
				int MaxEndCullDistance = CVarFoliageMaxEndCullDistance.GetValueOnRenderThread();
				float LODRandom = CVarRandomLODRange.GetValueOnRenderThread();
				float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;
								
				FVector AverageScale(InstanceParams.Tree[0].MinInstanceScale + (InstanceParams.Tree[0].MaxInstanceScale - InstanceParams.Tree[0].MinInstanceScale) / 2.0f);
				FBoxSphereBounds ScaledBounds = RenderData->Bounds.TransformBy(FTransform(FQuat::Identity, FVector::ZeroVector, AverageScale));
				float SphereRadius = ScaledBounds.SphereRadius + InstanceParams.MaxWPODisplacement;

				float FinalCull = MAX_flt;
				if (MinSize > 0.0)
				{
					FinalCull = ComputeBoundsDrawDistance(MinSize, SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
				}
				if (View->SceneViewInitOptions.OverrideFarClippingPlaneDistance > 0.0f)
				{
					FinalCull = FMath::Min(FinalCull, View->SceneViewInitOptions.OverrideFarClippingPlaneDistance * MaxDrawDistanceScale);
				}
				int32 EndCullDistance = UserData_AllInstances.EndCullDistance * MaxDrawDistanceScale;
				if (MaxEndCullDistance > 0)
				{
					if (EndCullDistance > 0)
					{
						EndCullDistance = FMath::Min(MaxEndCullDistance, EndCullDistance);
					}
					else
					{
						EndCullDistance = MaxEndCullDistance;
					}
				}
				if (EndCullDistance > 0.0f)
				{
					FinalCull = FMath::Min(FinalCull, EndCullDistance);
				}
				ElementParams.FinalCullDistance = FinalCull;

				for (int32 LODIndex = 1; LODIndex < InstanceParams.LODs; LODIndex++)
				{
					float Distance = ComputeBoundsDrawDistance(RenderData->ScreenSize[LODIndex].GetValue(), SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
					InstanceParams.LODPlanesMin[LODIndex - 1] = FMath::Min(FinalCull - LODRandom, Distance - LODRandom);
					InstanceParams.LODPlanesMax[LODIndex - 1] = FMath::Min(FinalCull, Distance);
				}
				InstanceParams.LODPlanesMin[InstanceParams.LODs - 1] = FinalCull - LODRandom;
				InstanceParams.LODPlanesMax[InstanceParams.LODs - 1] = FinalCull;

				// Added assert guard to track issue UE-53944
				check(InstanceParams.LODs <= 8);
				check(RenderData != nullptr);
			
				for (int32 LODIndex = 0; LODIndex < InstanceParams.LODs; LODIndex++)
				{
					InstanceParams.MinInstancesToSplit[LODIndex] = 2;

					// Added assert guard to track issue UE-53944
					check(RenderData->LODResources.IsValidIndex(LODIndex));

					int32 NumVerts = RenderData->LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
					if (NumVerts)
					{
						InstanceParams.MinInstancesToSplit[LODIndex] = MinVertsToSplitNode / NumVerts;
					}
				}

				// Culling objects from the shadow view using main view occlusion culling results is technically wrong, but can give good perf improvements
				// For VSM, it's likely better to turn this off as non-visible pages aren't rendered anyway and we care more about quality
				bool bApplyOcclusionCull = !ElementParams.ShadowFrustum || bApplyViewOcclusionCullToShadow;
				if (FirstOcclusionNode >= 0 && LastOcclusionNode >= 0 && FirstOcclusionNode <= LastOcclusionNode && bApplyOcclusionCull)
				{
					uint32 ViewId = View->GetViewKey();
					const FFoliageOcclusionResults* OldResults = OcclusionResults.Find(ViewId);
					if (OldResults &&
						OldResults->FrameNumberRenderThread == GFrameNumberRenderThread &&
						1 + LastOcclusionNode - FirstOcclusionNode == OldResults->NumResults &&
						// OcclusionResultsArray[Params.OcclusionResultsStart + Index - Params.FirstOcclusionNode]

						OldResults->Results.IsValidIndex(OldResults->ResultsStart) &&
						OldResults->Results.IsValidIndex(OldResults->ResultsStart + LastOcclusionNode - FirstOcclusionNode)
						)
					{
						InstanceParams.FirstOcclusionNode = FirstOcclusionNode;
						InstanceParams.LastOcclusionNode = LastOcclusionNode;
						InstanceParams.OcclusionResults = &OldResults->Results;
						InstanceParams.OcclusionResultsStart = OldResults->ResultsStart;
					}
				}

				INC_DWORD_STAT(STAT_FoliageTraversals);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (GCaptureDebugRuns == GDebugTag && CaptureTag == GDebugTag)
				{
					for (int32 LODIndex = 0; LODIndex < InstanceParams.LODs; LODIndex++)
					{
						for (int32 Run = 0; Run < SingleDebugRuns[LODIndex].Num(); Run++)
						{
							InstanceParams.SingleLODRuns[LODIndex].Add(SingleDebugRuns[LODIndex][Run]);
						}
						InstanceParams.TotalSingleLODInstances[LODIndex] = SingleDebugTotalInstances[LODIndex];
						for (int32 Run = 0; Run < MultipleDebugRuns[LODIndex].Num(); Run++)
						{
							InstanceParams.MultipleLODRuns[LODIndex].Add(MultipleDebugRuns[LODIndex][Run]);
						}
						InstanceParams.TotalMultipleLODInstances[LODIndex] = MultipleDebugTotalInstances[LODIndex];
					}
				}
				else
#endif
				{
					SCOPE_CYCLE_COUNTER(STAT_FoliageTraversalTime);

					int32 UseMinLOD = ClampedMinLOD;

					int32 DebugMin = FMath::Min(CVarMinLOD.GetValueOnRenderThread(), InstanceParams.LODs - 1);
					if (DebugMin >= 0)
					{
						UseMinLOD = FMath::Max(UseMinLOD, DebugMin);
					}
					int32 UseMaxLOD = InstanceParams.LODs;

					int32 Force = CVarForceLOD.GetValueOnRenderThread() >= 0 ? CVarForceLOD.GetValueOnRenderThread() : (ForcedLodModel > 0 ? ForcedLodModel : -1); 
					if (Force >= 0)
					{
						UseMinLOD = FMath::Clamp(Force, 0, InstanceParams.LODs - 1);
						UseMaxLOD = FMath::Clamp(Force, 0, InstanceParams.LODs - 1);
					}

					// Clamp the min LOD to available LOD taking mesh streaming into account as well
					const int8 CurFirstLODIdx = GetCurrentFirstLODIdx_RenderThread();
					UseMinLOD = FMath::Max(UseMinLOD, CurFirstLODIdx);

					if (CVarCullAll.GetValueOnRenderThread() < 1)
					{
						const bool bHasWPODisplacement = InstanceParams.MaxWPODisplacement != 0.0f;
												
						if (bUseVectorCull)
						{
							if (bHasWPODisplacement)
							{
								Traverse<true, true>(InstanceParams, 0, UseMinLOD, UseMaxLOD, bDisableCull);
							}
							else
							{
								Traverse<true, false>(InstanceParams, 0, UseMinLOD, UseMaxLOD, bDisableCull);
							}
						}
						else
						{
							if (bHasWPODisplacement)
							{
								Traverse<false, true>(InstanceParams, 0, UseMinLOD, UseMaxLOD, bDisableCull);
							}
							else
							{
								Traverse<false, false>(InstanceParams, 0, UseMinLOD, UseMaxLOD, bDisableCull);
							}
						}
					}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (GCaptureDebugRuns == GDebugTag && CaptureTag != GDebugTag)
					{
						CaptureTag = GDebugTag;
						for (int32 LODIndex = 0; LODIndex < InstanceParams.LODs; LODIndex++)
						{
							SingleDebugRuns[LODIndex].Empty();
							SingleDebugTotalInstances[LODIndex] = InstanceParams.TotalSingleLODInstances[LODIndex];
							for (int32 Run = 0; Run < InstanceParams.SingleLODRuns[LODIndex].Num(); Run++)
							{
								SingleDebugRuns[LODIndex].Add(InstanceParams.SingleLODRuns[LODIndex][Run]);
							}
							MultipleDebugRuns[LODIndex].Empty();
							MultipleDebugTotalInstances[LODIndex] = InstanceParams.TotalMultipleLODInstances[LODIndex];
							for (int32 Run = 0; Run < InstanceParams.MultipleLODRuns[LODIndex].Num(); Run++)
							{
								MultipleDebugRuns[LODIndex].Add(InstanceParams.MultipleLODRuns[LODIndex][Run]);
							}
						}
					}
#endif
				}

				FillDynamicMeshElements(View, Collector, ElementParams, InstanceParams);
			}

			int32 UnbuiltInstanceCount = InstanceCountToRender - FirstUnbuiltIndex;

			// Render unbuilt instances
			if (UnbuiltInstanceCount > 0)
			{
				FFoliageRenderInstanceParams& InstanceParams = Collector.AllocateOneFrameResource<FFoliageRenderInstanceParams>(true, false, false);

				// disable LOD blending for unbuilt instances as we haven't calculated the correct LOD.
				ElementParams.bBlendLODs = false;

				if (UnbuiltInstanceCount < 1000 && UnbuiltBounds.Num() >= UnbuiltInstanceCount)
				{
					const int32 NumLODs = RenderData->LODResources.Num();

					int32 Force = CVarForceLOD.GetValueOnRenderThread() >= 0 ? CVarForceLOD.GetValueOnRenderThread() : (ForcedLodModel > 0 ? ForcedLodModel : -1);
					if (Force >= 0)
					{
						Force = FMath::Clamp(Force, 0, NumLODs - 1);
						int32 LastInstanceIndex = FirstUnbuiltIndex + UnbuiltInstanceCount - 1;
						InstanceParams.AddRun(Force, Force, FirstUnbuiltIndex, LastInstanceIndex);
					}
					else
					{
						FVector ViewOriginInLocalZero = WorldToLocal.TransformPosition(View->GetTemporalLODOrigin(0, bMultipleSections));
						FVector ViewOriginInLocalOne  = WorldToLocal.TransformPosition(View->GetTemporalLODOrigin(1, bMultipleSections));
						float LODPlanesMax[MAX_STATIC_MESH_LODS];
						float LODPlanesMin[MAX_STATIC_MESH_LODS];

						const bool bIsOrtho = !View->ViewMatrices.IsPerspectiveProjection();
						const float MinSize = bIsOrtho ? 0.0f : CVarFoliageMinimumScreenSize.GetValueOnRenderThread();
						const float LODScale = UserData_AllInstances.LODDistanceScale;
						int MaxEndCullDistance = CVarFoliageMaxEndCullDistance.GetValueOnRenderThread();
						const float LODRandom = CVarRandomLODRange.GetValueOnRenderThread();
						const float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;
						const float SphereRadius = RenderData->Bounds.SphereRadius + GetMaxWorldPositionOffsetExtent();

						checkSlow(NumLODs > 0);

						float FinalCull = MAX_flt;
						if (MinSize > 0.0)
						{
							FinalCull = ComputeBoundsDrawDistance(MinSize, SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
						}
						if (View->SceneViewInitOptions.OverrideFarClippingPlaneDistance > 0.0f)
						{
							FinalCull = FMath::Min(FinalCull, View->SceneViewInitOptions.OverrideFarClippingPlaneDistance * MaxDrawDistanceScale);
						}
						int32 EndCullDistance = UserData_AllInstances.EndCullDistance * MaxDrawDistanceScale;
						if (MaxEndCullDistance > 0)
						{
							if (EndCullDistance > 0)
							{
								EndCullDistance = FMath::Min(MaxEndCullDistance, EndCullDistance);
							}
							else
							{
								EndCullDistance = MaxEndCullDistance;
							}
						}
						if (EndCullDistance > 0.0f)
						{
							FinalCull = FMath::Min(FinalCull, EndCullDistance);
						}
						ElementParams.FinalCullDistance = FinalCull;

						for (int32 LODIndex = 1; LODIndex < NumLODs; LODIndex++)
						{
							float Distance = ComputeBoundsDrawDistance(RenderData->ScreenSize[LODIndex].GetValue(), SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
							LODPlanesMin[LODIndex - 1] = FMath::Min(FinalCull - LODRandom, Distance - LODRandom);
							LODPlanesMax[LODIndex - 1] = FMath::Min(FinalCull, Distance);
						}
						LODPlanesMin[NumLODs - 1] = FinalCull - LODRandom;
						LODPlanesMax[NumLODs - 1] = FinalCull;

						// NOTE: in case of unbuilt we can't really apply the instance scales so the LOD won't be optimal until the build is completed

						// calculate runs
						int32 MinLOD = ClampedMinLOD;
						int32 MaxLOD = NumLODs;

						// Clamp the min LOD to available LOD taking mesh streaming into account as well
						const int8 CurFirstLODIdx = GetCurrentFirstLODIdx_RenderThread();
						MinLOD = FMath::Max(MinLOD, CurFirstLODIdx);

						CalcLOD(MinLOD, MaxLOD, UnbuiltBounds[0].Min, UnbuiltBounds[0].Max, ViewOriginInLocalZero, ViewOriginInLocalOne, LODPlanesMin, LODPlanesMax);
						int32 FirstIndexInRun = 0;
						for (int32 Index = 1; Index < UnbuiltInstanceCount; ++Index)
						{
							int32 TempMinLOD = ClampedMinLOD;
							int32 TempMaxLOD = NumLODs;
							CalcLOD(TempMinLOD, TempMaxLOD, UnbuiltBounds[Index].Min, UnbuiltBounds[Index].Max, ViewOriginInLocalZero, ViewOriginInLocalOne, LODPlanesMin, LODPlanesMax);
							if (TempMinLOD != MinLOD)
							{
								if (MinLOD < NumLODs)
								{
									InstanceParams.AddRun(MinLOD, MinLOD, FirstIndexInRun + FirstUnbuiltIndex, (Index - 1) + FirstUnbuiltIndex - 1);
								}
								MinLOD = TempMinLOD;
								FirstIndexInRun = Index;
							}
						}
						InstanceParams.AddRun(MinLOD, MinLOD, FirstIndexInRun + FirstUnbuiltIndex, FirstUnbuiltIndex + UnbuiltInstanceCount - 1);
					}
				}
				else
				{
					// more than 1000, render them all at lowest LOD (until we have an updated tree)
					const int8 LowestLOD = (RenderData->LODResources.Num() - 1);
					InstanceParams.AddRun(LowestLOD, LowestLOD, FirstUnbuiltIndex, FirstUnbuiltIndex + UnbuiltInstanceCount - 1);
				}
				FillDynamicMeshElements(View, Collector, ElementParams, InstanceParams);
			}

			if (View->Family->EngineShowFlags.HISMCOcclusionBounds)
			{
				for (auto& OcclusionBound : OcclusionBounds)
				{
					DrawWireBox(Collector.GetPDI(ViewIndex), OcclusionBound.GetBox(), FColor(255, 0, 0), View->Family->EngineShowFlags.Game ? SDPG_World : SDPG_Foreground);
				}
			}

			if (View->Family->EngineShowFlags.HISMCClusterTree)
			{
				FColor StartingColor(100, 0, 0);
				const float MaxWorldPositionOffset = GetMaxWorldPositionOffsetExtent();

				for (const FClusterNode& CulsterNode : ClusterTree)
				{
					DrawWireBox(Collector.GetPDI(ViewIndex), GetLocalToWorld(), FBox(CulsterNode.BoundMin, CulsterNode.BoundMax).ExpandBy(MaxWorldPositionOffset), StartingColor, View->Family->EngineShowFlags.Game ? SDPG_World : SDPG_Foreground);
					StartingColor.R += 5;
					StartingColor.G += 5;
					StartingColor.B += 5;
				}
			}

			if (View->Family->EngineShowFlags.InstancedStaticMeshes)
			{
				RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
			}

		}
	}
}

void FHierarchicalStaticMeshSceneProxy::AcceptOcclusionResults(const FSceneView* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults)
{ 
	// Don't accept subprimitive occlusion results from a previously-created sceneproxy - the tree may have been different
	if (OcclusionBounds.Num() == NumResults && SceneProxyCreatedFrameNumberRenderThread < GFrameNumberRenderThread)
	{
		// This lock is necessary to guard against access from multiple views.
		OcclusionResultsMutex.Lock();

		uint32 ViewId = View->GetViewKey();
		FFoliageOcclusionResults* OldResults = OcclusionResults.Find(ViewId);
		if (OldResults)
		{
			OldResults->FrameNumberRenderThread = GFrameNumberRenderThread;
			OldResults->Results = *Results;
			OldResults->ResultsStart = ResultsStart;
			OldResults->NumResults = NumResults;
		}
		else
		{
			// now is a good time to clean up any stale entries
			for (auto Iter = OcclusionResults.CreateIterator(); Iter; ++Iter)
			{
				if (Iter.Value().FrameNumberRenderThread != GFrameNumberRenderThread)
				{
					Iter.RemoveCurrent();
				}
			}
			OcclusionResults.Add(ViewId, FFoliageOcclusionResults(Results, ResultsStart, NumResults));
		}

		OcclusionResultsMutex.Unlock();
	}
}

const TArray<FBoxSphereBounds>* FHierarchicalStaticMeshSceneProxy::GetOcclusionQueries(const FSceneView* View) const 
{
	return &OcclusionBounds;
}

bool FHierarchicalStaticMeshSceneProxy::GetInstanceDrawDistanceMinMax(FVector2f& OutDistanceMinMax) const
{
	// HISM already does distance culling before submitting instances, so there is no need to do distance culling on GPU as well.
	// Also GPU distance culling breaks dithered fade out for HISM. This is because we apply fade out to instances only when they go 
	// beyond the culling distance, and at that point they will be GPU culled!
	// We could apply a tweak to the draw distance returned here to account for this. But there isn't really any point
	// because we know that the correct distance culling is already applied on the CPU side. So any tweaking of the value would be
	// to ensure that we don't cull. So we might as well just not do the work on the GPU, which is what happens when we return false here.
	OutDistanceMinMax = FVector2f(0, 0);
	return false;
}

FBoxSphereBounds UHierarchicalInstancedStaticMeshComponent::CalcBounds(const FTransform& BoundTransform) const
{
	ensure(BuiltInstanceBounds.IsValid || ClusterTreePtr->Num() == 0);

	if (BuiltInstanceBounds.IsValid || UnbuiltInstanceBounds.IsValid)
	{
		FBoxSphereBounds Result = (BuiltInstanceBounds + UnbuiltInstanceBounds).TransformBy(BoundTransform);
		Result.BoxExtent *= BoundsScale;
		Result.SphereRadius *= BoundsScale;
		return Result;
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UHierarchicalInstancedStaticMeshComponent_CalcBounds_SlowPath);
		return Super::CalcBounds(BoundTransform);
	}
}

FBox UHierarchicalInstancedStaticMeshComponent::GetClusterTreeBounds(TArray<FClusterNode> const& InClusterTree, const FVector& InOffset)
{
	// Return top node of cluster tree. Apply offset on node bounds.
	return (InClusterTree.Num() > 0 ? FBox(InOffset + FVector(InClusterTree[0].BoundMin), InOffset + FVector(InClusterTree[0].BoundMax)) : FBox(ForceInit));
}

UHierarchicalInstancedStaticMeshComponent::UHierarchicalInstancedStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ClusterTreePtr(MakeShareable(new TArray<FClusterNode>))
	, bUseTranslatedInstanceSpace(false)
	, TranslatedInstanceSpaceOrigin(ForceInitToZero)
	, NumBuiltInstances(0)
	, NumBuiltRenderInstances(0)
	, UnbuiltInstanceBounds(ForceInit)
	, bEnableDensityScaling(false)
	, CurrentDensityScaling(1.0f)
	, OcclusionLayerNumNodes(0)
	, InstanceCountToRender(0)
	, bIsAsyncBuilding(false)
	, bIsOutOfDate(false)
	, bConcurrentChanges(false)
	, bAutoRebuildTreeOnInstanceChanges(true)
#if WITH_EDITOR
	, bCanEnableDensityScaling(true)
#endif
{
	PrimitiveInstanceDataManager.SetMode(FPrimitiveInstanceDataManager::EMode::Legacy);
	bCanEverAffectNavigation = true;
	bUseAsOccluder = false;
}

UHierarchicalInstancedStaticMeshComponent::~UHierarchicalInstancedStaticMeshComponent()
{
	if (ProxySize)
	{
		DEC_DWORD_STAT_BY(STAT_FoliageInstanceBuffers, ProxySize);
	}
	ProxySize = 0;
}

#if WITH_EDITOR

void UHierarchicalInstancedStaticMeshComponent::PostStaticMeshCompilation()
{
	BuildTreeIfOutdated(false, true);

	Super::PostStaticMeshCompilation();
}

void UHierarchicalInstancedStaticMeshComponent::PostEditUndo()
{
	Super::PostEditUndo();

	// Force because the Outdated Condition will fail (compared values will match)
	// Since we don't know what changed we can't really send a command to the InstanceUpdateCmdBuffer to reflect the changes so we do the Build Non-Async
	BuildTreeIfOutdated(/*Async*/false, /*ForceUpdate*/true);
}

void UHierarchicalInstancedStaticMeshComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if ((PropertyChangedEvent.Property != NULL && PropertyChangedEvent.Property->GetFName() == "InstancingRandomSeed"))
	{
		InitializeInstancingRandomSeed();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if ((PropertyChangedEvent.Property != NULL && PropertyChangedEvent.Property->GetFName() == "PerInstanceSMData") ||
		(PropertyChangedEvent.Property != NULL && PropertyChangedEvent.Property->GetFName() == "PerInstanceSMCustomData") ||
		(PropertyChangedEvent.Property != NULL && PropertyChangedEvent.Property->GetFName() == "Transform") ||
		(PropertyChangedEvent.Property != NULL && PropertyChangedEvent.Property->GetFName() == "StaticMesh"))
	{
		if (FApp::CanEverRender())
		{
			// Since we don't know what changed we can't really send a command to the InstanceUpdateCmdBuffer to reflect the changes so we do the Build Non-Async
			BuildTreeIfOutdated(/*Async*/false, /*ForceUpdate*/true);
		}
	}
}
#endif

void UHierarchicalInstancedStaticMeshComponent::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	// Make sure to build tree before Save/Duplicate
	if (Ar.IsSaving() && Ar.IsPersistent())
	{
		BuildTreeIfOutdated(/*Async*/false, /*ForceUpdate*/false);
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		ClusterTreePtr = MakeShareable(new TArray<FClusterNode>);
	}

	if (Ar.IsLoading() && Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::HISMCClusterTreeMigration)
	{
		// Skip the serialized tree, we will regenerate it correctly to contains the new data
		TArray<FClusterNode_DEPRECATED> ClusterTree_DEPRECATED;
		ClusterTree_DEPRECATED.BulkSerialize(Ar);
	}
	else
	{
		ClusterTreePtr->BulkSerialize(Ar);
	}

	if (Ar.IsLoading() && !BuiltInstanceBounds.IsValid)
	{
		TArray<FClusterNode>& ClusterTree = *ClusterTreePtr.Get();
		BuiltInstanceBounds = GetClusterTreeBounds(ClusterTree, TranslatedInstanceSpaceOrigin);
	}
}

void UHierarchicalInstancedStaticMeshComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	
	if (ClusterTreePtr.IsValid())
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ClusterTreePtr->GetAllocatedSize());
	}
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SortedInstances.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(UnbuiltInstanceBoundsList.GetAllocatedSize());
}

void UHierarchicalInstancedStaticMeshComponent::PostEditImport()
{
	Super::PostEditImport();

	// Node cluster isn't exported so we need to rebuild the tree.
	BuildTreeIfOutdated(/*Async*/false,/*ForceUpdate*/true);
}

void UHierarchicalInstancedStaticMeshComponent::PostLoad()
{
	// ConditionalPostLoad the level lightmap data since we expect it to be loaded before PostLoadPerInstanceData which is called from Super::PostLoad()
	AActor* Owner = GetOwner();
	if (Owner != nullptr)
	{
		ULevel* OwnerLevel = Owner->GetLevel();
		if (OwnerLevel != nullptr && OwnerLevel->MapBuildData != nullptr)
		{
			OwnerLevel->MapBuildData->ConditionalPostLoad();
		}
	}

	Super::PostLoad();
}

void UHierarchicalInstancedStaticMeshComponent::RemoveInstancesInternal(TConstArrayView<int32> InstanceIndices)
{
	if ( !InstanceIndices.IsEmpty())
	{
		bIsOutOfDate = true;
		bConcurrentChanges |= IsAsyncBuilding();
	}

	for (int32 Index = 0; Index < InstanceIndices.Num(); ++Index)
	{
		int32 InstanceIndex = InstanceIndices[Index];
		// Note: force removeAtSwap behavior:
		Super::RemoveInstanceInternal(InstanceIndex, false, true);
		// InstanceReorderTable could be empty for a 'bad' HISMC, (eg. missing mesh)
		if (InstanceReorderTable.IsValidIndex(InstanceIndex))
		{
			// Due to scalability it's possible that we try to remove an instance that is not valid in the reorder table as it was removed already from render
			int32 RenderIndex = InstanceReorderTable[InstanceIndex];
			InstanceReorderTable.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
		}
	}

	PerInstanceSMData.Shrink();
	// InstanceReorderTable is not shrink as the build tree will override it so we save the cost of the realloc
}

bool UHierarchicalInstancedStaticMeshComponent::RemoveInstances(const TArray<int32>& InstancesToRemove)
{
	return RemoveInstances(InstancesToRemove, false /*bInstanceArrayAlreadySortedInReverseOrder*/);
}

bool UHierarchicalInstancedStaticMeshComponent::RemoveInstances(const TArray<int32>& InstancesToRemove, bool bInstanceArrayAlreadySortedInReverseOrder)
{
	LLM_SCOPE(ELLMTag::StaticMesh);

	if (InstancesToRemove.Num() == 0)
	{
		return true;
	}

	SCOPE_CYCLE_COUNTER(STAT_HISMCRemoveInstance);

	auto RemoveInstanceFromSortedArray = [this](const TArray<int32>& SortedInstancesToRemove) -> bool
	{
		if (!PerInstanceSMData.IsValidIndex(SortedInstancesToRemove[0]) || !PerInstanceSMData.IsValidIndex(SortedInstancesToRemove.Last()))
		{
			return false;
		}

		RemoveInstancesInternal(SortedInstancesToRemove);

		if (bAutoRebuildTreeOnInstanceChanges)
		{
			BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/false);
		}

		MarkRenderStateDirty();

		return true;
	};

	bool bSuccess = false;
	if (bInstanceArrayAlreadySortedInReverseOrder)
	{
		bSuccess = RemoveInstanceFromSortedArray(InstancesToRemove);
	}
	else
	{
		TArray<int32> SortedInstancesToRemove = InstancesToRemove;

		// Sort so RemoveAtSwaps don't alter the indices of items still to remove
		SortedInstancesToRemove.Sort(TGreater<int32>());
		bSuccess = RemoveInstanceFromSortedArray(SortedInstancesToRemove);
	}

	return bSuccess;
}

bool UHierarchicalInstancedStaticMeshComponent::RemoveInstance(int32 InstanceIndex)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	SCOPE_CYCLE_COUNTER(STAT_HISMCRemoveInstance);

	RemoveInstancesInternal(MakeArrayView<const int32>(&InstanceIndex, 1));

	if (bAutoRebuildTreeOnInstanceChanges)
	{
		BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/false);
	}

	MarkRenderStateDirty();
	
	return true;
}

bool UHierarchicalInstancedStaticMeshComponent::UpdateInstanceTransform(int32 InstanceIndex, const FTransform& NewInstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	bIsOutOfDate = true;
	// invalidate the results of the current async build we need to modify the tree
	bConcurrentChanges |= IsAsyncBuilding();
	
	const int32 RenderIndex = GetRenderIndex(InstanceIndex);
	const FMatrix OldTransform = PerInstanceSMData[InstanceIndex].Transform;
	const FTransform NewLocalTransform = bWorldSpace ? NewInstanceTransform.GetRelativeTransform(GetComponentTransform()) : NewInstanceTransform;
	const FVector NewLocalLocation = NewLocalTransform.GetTranslation();

	const bool bIsOmittedInstance = (RenderIndex == INDEX_NONE);
	const bool bIsBuiltInstance = !bIsOmittedInstance && RenderIndex < NumBuiltRenderInstances;

	bool bAllowInPlaceUpdateForRotationOrScaleChange = true;

	// Code path using 'bDoInPlaceUpdate' indicates that it updates the cluster tree but
	// bounds are not updated until next tree rebuild and some overlapping queries rely on those bounds.
	// We want to make sure a manipulation in an Editor world will fully update the information so queries
	// will return proper information to callers (e.g. navigation rebuild and preview)
#if WITH_EDITOR
	if (const UWorld* World = GetWorld())
	{
		const bool bIsGameWorld = World->IsGameWorld();
		bAllowInPlaceUpdateForRotationOrScaleChange = bIsGameWorld;
	}
#endif // WITH_EDITOR

	// if we are only updating rotation/scale then we update the instance directly in the cluster tree
	const bool bDoInPlaceUpdate = bAllowInPlaceUpdateForRotationOrScaleChange && bIsBuiltInstance && NewLocalLocation.Equals(OldTransform.GetOrigin());

	bool Result = Super::UpdateInstanceTransform(InstanceIndex, NewInstanceTransform, bWorldSpace, bMarkRenderStateDirty, bTeleport);
	
	// The tree will be fully rebuilt once the static mesh compilation is finished, no need for incremental update in that case.
	if (Result && GetStaticMesh() && !GetStaticMesh()->IsCompiling() && GetStaticMesh()->HasValidRenderData())
	{
		const FBox NewInstanceBounds = GetStaticMesh()->GetBounds().GetBox().TransformBy(NewLocalTransform);
		
		if (bDoInPlaceUpdate)
		{
			// If the new bounds are larger than the old ones, then expand the bounds on the tree to make sure culling works correctly
			const FBox OldInstanceBounds = GetStaticMesh()->GetBounds().GetBox().TransformBy(OldTransform);
			if (!OldInstanceBounds.IsInside(NewInstanceBounds))
			{
				BuiltInstanceBounds += NewInstanceBounds;
			}
		}
		else
		{
			UnbuiltInstanceBounds += NewInstanceBounds;
			UnbuiltInstanceBoundsList.Add(NewInstanceBounds);

			BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/false);
		}
	}

	return Result;
}

bool UHierarchicalInstancedStaticMeshComponent::SetCustomDataValue(int32 InstanceIndex, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex) || CustomDataIndex < 0 || CustomDataIndex >= NumCustomDataFloats)
	{
		return false;
	}

	if (IsAsyncBuilding())
	{
		// invalidate the results of the current async build we need to modify the tree
		bConcurrentChanges = true;
	}

	bool Result = Super::SetCustomDataValue(InstanceIndex, CustomDataIndex, CustomDataValue, bMarkRenderStateDirty);
	return Result;
}

bool UHierarchicalInstancedStaticMeshComponent::SetCustomData(int32 InstanceIndex, TArrayView<const float> InCustomData, bool bMarkRenderStateDirty)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex) || InCustomData.Num() == 0)
	{
		return false;
	}

	if (IsAsyncBuilding())
	{
		// invalidate the results of the current async build we need to modify the tree
		bConcurrentChanges = true;
	}

	bool Result = Super::SetCustomData(InstanceIndex, InCustomData, bMarkRenderStateDirty);
	return Result;
}

bool UHierarchicalInstancedStaticMeshComponent::BatchUpdateInstancesTransforms(int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	return BatchUpdateInstancesTransformsInternal(StartInstanceIndex, MakeArrayView(NewInstancesTransforms), bWorldSpace, bMarkRenderStateDirty, bTeleport);
}

bool UHierarchicalInstancedStaticMeshComponent::BatchUpdateInstancesTransforms(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	return BatchUpdateInstancesTransformsInternal(StartInstanceIndex, NewInstancesTransforms, bWorldSpace, bMarkRenderStateDirty, bTeleport);
}

bool UHierarchicalInstancedStaticMeshComponent::BatchUpdateInstancesTransformsInternal(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	bool BatchResult = true;

	int32 InstanceIndex = StartInstanceIndex;
	for(const FTransform& NewInstanceTransform : NewInstancesTransforms)
	{
		bool Result = UpdateInstanceTransform(InstanceIndex, NewInstanceTransform, bWorldSpace, bMarkRenderStateDirty, bTeleport);
		BatchResult = BatchResult && Result;
		
		InstanceIndex++;
	}

	return BatchResult;
}

bool UHierarchicalInstancedStaticMeshComponent::BatchUpdateInstancesTransform(int32 StartInstanceIndex, int32 NumInstances, const FTransform& NewInstancesTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	bool BatchResult = true;

	int32 EndInstanceIndex = StartInstanceIndex + NumInstances;
	for(int32 InstanceIndex = StartInstanceIndex; InstanceIndex < EndInstanceIndex; ++InstanceIndex)
	{
		bool Result = UpdateInstanceTransform(InstanceIndex, NewInstancesTransform, bWorldSpace, bMarkRenderStateDirty, bTeleport);
		BatchResult = BatchResult && Result;
	}

	return BatchResult;
}

bool UHierarchicalInstancedStaticMeshComponent::BatchUpdateInstancesData(int32 StartInstanceIndex, int32 NumInstances, FInstancedStaticMeshInstanceData* StartInstanceData, bool bMarkRenderStateDirty, bool bTeleport)
{
	bool BatchResult = true;

	Super::BatchUpdateInstancesData(StartInstanceIndex, NumInstances, StartInstanceData, bMarkRenderStateDirty, bTeleport);
	BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/false);

	return BatchResult;
}

void UHierarchicalInstancedStaticMeshComponent::ApplyComponentInstanceData(FInstancedStaticMeshComponentInstanceData* InstancedMeshData)
{
	UInstancedStaticMeshComponent::ApplyComponentInstanceData(InstancedMeshData);
	// Force rebuild, since anything may have changed in the ApplyComponentInstanceData.
	BuildTreeIfOutdated(/*Async*/false, /*ForceUpdate*/true);
}

void UHierarchicalInstancedStaticMeshComponent::PreAllocateInstancesMemory(int32 AddedInstanceCount)
{
	Super::PreAllocateInstancesMemory(AddedInstanceCount);

	InstanceReorderTable.Reserve(InstanceReorderTable.Num() + AddedInstanceCount);
	UnbuiltInstanceBoundsList.Reserve(UnbuiltInstanceBoundsList.Num() + AddedInstanceCount);
}

int32 UHierarchicalInstancedStaticMeshComponent::AddInstance(const FTransform& InstanceTransform, bool bWorldSpace)
{
	SCOPE_CYCLE_COUNTER(STAT_HISMCAddInstance);

	// Initialize the seed here even if the rebuild doesn't happen now to prevent races with being renamed by the time the mesh compiles
	InitializeInstancingRandomSeed();

	int32 InstanceIndex = UInstancedStaticMeshComponent::AddInstance(InstanceTransform, bWorldSpace);

	// The tree will be fully rebuilt once the static mesh compilation is finished, no need for incremental update in that case.
	if (InstanceIndex != INDEX_NONE && GetStaticMesh() && !GetStaticMesh()->IsCompiling() && GetStaticMesh()->HasValidRenderData(false))
	{	
		check(InstanceIndex == InstanceReorderTable.Num());

		bIsOutOfDate = true;
		bConcurrentChanges |= IsAsyncBuilding();
	
		int32 InitialBufferOffset = InstanceCountToRender - InstanceReorderTable.Num(); // Until the build is done, we need to always add at the end of the buffer/reorder table
		InstanceReorderTable.Add(InitialBufferOffset + InstanceIndex); // add to the end until the build is completed
		++InstanceCountToRender;

		const FBox NewInstanceBounds = GetStaticMesh()->GetBounds().GetBox().TransformBy(PerInstanceSMData[InstanceIndex].Transform);
		UnbuiltInstanceBounds += NewInstanceBounds;
		UnbuiltInstanceBoundsList.Add(NewInstanceBounds);

		if (bAutoRebuildTreeOnInstanceChanges)
		{
			BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/false);
		}
	}

	return InstanceIndex;
}

TArray<int32> UHierarchicalInstancedStaticMeshComponent::AddInstances(const TArray<FTransform>& InstanceTransforms, bool bShouldReturnIndices, bool bWorldSpace, bool bUpdateNavigation)
{
	SCOPE_CYCLE_COUNTER(STAT_HISMCAddInstances);

	// Initialize the seed here even if the rebuild doesn't happen now to prevent races with being renamed by the time the mesh compiles
	InitializeInstancingRandomSeed();

	TArray<int32> InstanceIndices = UInstancedStaticMeshComponent::AddInstances(InstanceTransforms, true, bWorldSpace, bUpdateNavigation);

	// The tree will be fully rebuilt once the static mesh compilation is finished, no need for incremental update in that case.
	if (InstanceIndices.Num() > 0 && GetStaticMesh() && !GetStaticMesh()->IsCompiling() && GetStaticMesh()->HasValidRenderData(false))
	{
		bIsOutOfDate = true;
		bConcurrentChanges |= IsAsyncBuilding();

		const int32 Count = InstanceIndices.Num();
		
		InstanceReorderTable.Reserve(InstanceReorderTable.Num() + Count);
		UnbuiltInstanceBoundsList.Reserve(UnbuiltInstanceBoundsList.Num() + Count);

		const int32 InitialBufferOffset = InstanceCountToRender - InstanceReorderTable.Num();

		for (const int32 InstanceIndex : InstanceIndices)
		{
			InstanceReorderTable.Add(InitialBufferOffset + InstanceIndex);
			++InstanceCountToRender;

			const FBox NewInstanceBounds = GetStaticMesh()->GetBounds().GetBox().TransformBy(PerInstanceSMData[InstanceIndex].Transform);
			UnbuiltInstanceBounds += NewInstanceBounds;
			UnbuiltInstanceBoundsList.Add(NewInstanceBounds);
		}

		if (bAutoRebuildTreeOnInstanceChanges)
		{
			BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/false);
		}
	}

	return bShouldReturnIndices ? InstanceIndices : TArray<int32>();
}

void UHierarchicalInstancedStaticMeshComponent::ClearInstances()
{
	bIsOutOfDate = true;
	bConcurrentChanges |= IsAsyncBuilding();
	
	ClusterTreePtr = MakeShareable(new TArray<FClusterNode>);
	NumBuiltInstances = 0;
	NumBuiltRenderInstances = 0;
	InstanceCountToRender = 0;
	SortedInstances.Empty();
	UnbuiltInstanceBounds.Init();
	UnbuiltInstanceBoundsList.Empty();

	if (ProxySize)
	{
		DEC_DWORD_STAT_BY(STAT_FoliageInstanceBuffers, ProxySize);
	}

	UInstancedStaticMeshComponent::ClearInstances();
}

int32 UHierarchicalInstancedStaticMeshComponent::GetVertsForLOD(int32 LODIndex)
{
	if (GetStaticMesh() && GetStaticMesh()->HasValidRenderData(true, LODIndex))
	{
		return GetStaticMesh()->GetNumVertices(LODIndex);
	}
	return 0;
}

int32 UHierarchicalInstancedStaticMeshComponent::DesiredInstancesPerLeaf()
{
	int32 LOD0Verts = GetVertsForLOD(0);
	int32 VertsToSplit = CVarMinVertsToSplitNode.GetValueOnAnyThread();
	if (LOD0Verts)
	{
		return FMath::Clamp(VertsToSplit / LOD0Verts, 1, 1024);
	}
	return 16;
}

float UHierarchicalInstancedStaticMeshComponent::ActualInstancesPerLeaf()
{
	const TArray<FClusterNode>& ClusterTree = *ClusterTreePtr;
	if (ClusterTree.Num())
	{
		int32 NumLeaves = 0;
		int32 NumInstances = 0;
		for (int32 Index = ClusterTree.Num() - 1; Index >= 0; Index--)
		{
			if (ClusterTree[Index].FirstChild >= 0)
			{
				break;
			}
			NumLeaves++;
			NumInstances += 1 + ClusterTree[Index].LastInstance - ClusterTree[Index].FirstInstance;
		}
		if (NumLeaves)
		{
			return float(NumInstances) / float(NumLeaves);
		}
	}
	return 0.0f;
}

void UHierarchicalInstancedStaticMeshComponent::PostBuildStats()
{
#if 0
	const TArray<FClusterNode>& ClusterTree = *ClusterTreePtr;
	FString MeshName = GetStaticMesh() ? GetStaticMesh()->GetPathName() : FString("null");
	check(PerInstanceRenderData.IsValid());

	// Non-nanite HISMs that were built with AcceptPrebuiltTree won't have PerInstanceSMData, so get the instance count from the buffer
	int32 NumInst = PerInstanceSMData.Num() == 0 ? PerInstanceRenderData->InstanceBuffer.GetNumInstances() : PerInstanceSMData.Num();
	bool bIsGrass = GetViewRelevanceType() == EHISMViewRelevanceType::Grass;

	UE_LOGF(LogStaticMesh, Display, "Built a foliage hierarchy with %d instances, %d nodes, %f instances / leaf (desired %d) and %d verts in LOD0. Grass? %d    %ls", NumInst, ClusterTree.Num(), ActualInstancesPerLeaf(), DesiredInstancesPerLeaf(), GetVertsForLOD(0), bIsGrass), *MeshName);
#endif
}

// Deprecated in 5.7
void UHierarchicalInstancedStaticMeshComponent::BuildComponentInstanceData(ERHIFeatureLevel::Type FeatureLevel, FInstanceUpdateComponentDesc& OutData)
{
	return BuildComponentInstanceData(GetFeatureLevelShaderPlatform_Checked(FeatureLevel), OutData);
}

void UHierarchicalInstancedStaticMeshComponent::BuildComponentInstanceData(EShaderPlatform InShaderPlatform, FInstanceUpdateComponentDesc& OutData)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UInstancedStaticMeshComponent_BuildRenderData);

	OutData.PrimitiveLocalToWorld = GetRenderMatrix();
	OutData.PrimitiveMaterialDesc = GetUsedMaterialPropertyDesc(InShaderPlatform);
	OutData.Flags = MakeInstanceDataFlags(OutData.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom, OutData.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceCustomData);
	OutData.Flags.bHasPerInstanceDynamicData = false;
	OutData.StaticMeshBounds = GetStaticMesh()->GetBounds();
	OutData.NumProxyInstances = InstanceCountToRender;
	OutData.NumSourceInstances = PerInstanceSMData.Num();
	OutData.NumCustomDataFloats = NumCustomDataFloats;

	OutData.BuildChangeSet = [this](FISMInstanceUpdateChangeSet &ChangeSet)
	{
		BuildInstanceDataDeltaChangeSetCommon(ChangeSet);
		ChangeSet.SetInstanceTransforms(MakeStridedView(PerInstanceSMData, &FInstancedStaticMeshInstanceData::Transform), -TranslatedInstanceSpaceOrigin);
		ChangeSet.SetInstancePrevTransforms(MakeArrayView(PerInstancePrevTransform), -TranslatedInstanceSpaceOrigin);
		ChangeSet.LegacyInstanceReorderTable = InstanceReorderTable;
	};
}

void UHierarchicalInstancedStaticMeshComponent::BuildTree()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHierarchicalInstancedStaticMeshComponent::BuildTree);

	check(!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject));
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData)
	checkSlow(IsInGameThread());

	// If we try to build the tree with the static mesh not fully loaded, we can end up in an inconsistent state which ends in a crash later
	checkSlow(!GetStaticMesh() || !GetStaticMesh()->HasAnyFlags(RF_NeedPostLoad));

	QUICK_SCOPE_CYCLE_COUNTER(STAT_UHierarchicalInstancedStaticMeshComponent_BuildTree);

	// The tree will be fully rebuilt once the static mesh compilation is finished, no need for incremental update in that case.
	if (PerInstanceSMData.Num() > 0 && GetStaticMesh() && !GetStaticMesh()->IsCompiling() && GetStaticMesh()->HasValidRenderData(false))
	{
		// Build the tree in translated space to maintain precision.
		TranslatedInstanceSpaceOrigin = CalcTranslatedInstanceSpaceOrigin();

		InitializeInstancingRandomSeed();
		TArray<FMatrix> InstanceTransforms;
		GetInstanceTransforms(InstanceTransforms, -TranslatedInstanceSpaceOrigin);

		FClusterBuilder Builder(InstanceTransforms, PerInstanceSMCustomData, NumCustomDataFloats, GetStaticMesh()->GetBounds().GetBox(), DesiredInstancesPerLeaf(), CurrentDensityScaling, InstancingRandomSeed, PerInstanceSMData.Num() > 0);
		Builder.BuildTreeAndBuffer();

		ApplyBuildTree(Builder, /*bWasAsyncBuild*/false);
	}
	else
	{
		ApplyEmpty();
	}
}

void UHierarchicalInstancedStaticMeshComponent::ApplyBuildTreeAsync(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent, TSharedRef<FClusterBuilder, ESPMode::ThreadSafe> Builder, double StartTime)
{
	check(IsInGameThread());

	bIsAsyncBuilding = false;
	BuildTreeAsyncTasks.Empty();

	// We did a sync build while async building. The sync build is newer so we will use that.
	if (!bIsOutOfDate)
	{
		bConcurrentChanges = false;
		return;
	}

	// We did some changes during an async build
	if (bConcurrentChanges)
	{
		bConcurrentChanges = false;

		UE_LOGF(LogStaticMesh, Verbose, "Discarded foliage hierarchy of %d elements build due to concurrent removal (%.1fs)", Builder->Result->InstanceReorderTable.Num(), (float)(FPlatformTime::Seconds() - StartTime));

		// There were changes while we were building, it's too slow to fix up the result now, so build async again.
		BuildTreeAsync();
		return;
	}

	// Completed the build
	ApplyBuildTree(Builder.Get(), /*bWasAsyncBuild*/true);
}

void UHierarchicalInstancedStaticMeshComponent::ApplyEmpty()
{
	bIsOutOfDate = false;
	ClusterTreePtr = MakeShareable(new TArray<FClusterNode>);
	NumBuiltInstances = 0;
	NumBuiltRenderInstances = 0;
	InstanceCountToRender = 0;
	InstanceReorderTable.Empty();
	SortedInstances.Empty();
	UnbuiltInstanceBoundsList.Empty();
	BuiltInstanceBounds.Init();
	CacheMeshExtendedBounds = (GetStaticMesh() && (GetStaticMesh()->IsCompiling() || GetStaticMesh()->HasValidRenderData(false))) ? GetStaticMesh()->GetBounds() : FBoxSphereBounds(ForceInitToZero);
	PrimitiveInstanceDataManager.Invalidate(PerInstanceSMData.Num());
	FHierarchicalInstancedStaticMeshDelegates::OnTreeBuilt.Broadcast(this, /*bWasAsyncBuild*/false);
}

void UHierarchicalInstancedStaticMeshComponent::ApplyBuildTree(FClusterBuilder& Builder, const bool bWasAsyncBuild)
{
	bIsOutOfDate = false;

	check(Builder.Result->InstanceReorderTable.Num() == PerInstanceSMData.Num());

	NumBuiltInstances = Builder.Result->InstanceReorderTable.Num();
	NumBuiltRenderInstances = Builder.Result->SortedInstances.Num();

	ClusterTreePtr = MakeShareable(new TArray<FClusterNode>(MoveTemp(Builder.Result->Nodes)));

	InstanceReorderTable = MoveTemp(Builder.Result->InstanceReorderTable);
	SortedInstances = MoveTemp(Builder.Result->SortedInstances);
	CacheMeshExtendedBounds = GetStaticMesh()->GetBounds();
	TUniquePtr<FStaticMeshInstanceData> BuiltInstanceData = MoveTemp(Builder.BuiltInstanceData);

	OcclusionLayerNumNodes = Builder.Result->OutOcclusionLayerNum;

	// Get the new bounds taking into account the translated space used when building the tree.
	const TArray<FClusterNode>& ClusterTree = *ClusterTreePtr;
	BuiltInstanceBounds = GetClusterTreeBounds(ClusterTree, TranslatedInstanceSpaceOrigin);

	UnbuiltInstanceBounds.Init();
	UnbuiltInstanceBoundsList.Empty();

	check(BuiltInstanceData.IsValid());
	check(BuiltInstanceData->GetNumInstances() == NumBuiltRenderInstances);

	InstanceCountToRender = NumBuiltInstances;

	check(InstanceReorderTable.Num() == PerInstanceSMData.Num());

	// create per-instance hit-proxies if needed
	TArray<TRefCountPtr<HHitProxy>> HitProxies;
	CreateHitProxyData(HitProxies);
	SetPerInstanceLightMapAndEditorData(*BuiltInstanceData, HitProxies);

	// Make sure it gets rebuilt from scratch to reflect the new instance ordering
	// we could actually do it incrementally since no data needs to be uploaded (the instances are the same as before, just need to mark changed Indexes for all & implement the general swap functionality)
	// BUT: the hitproxy data was rebuilt right here & the per instance random is handled differently inside the tree builder so, nope.
	PrimitiveInstanceDataManager.MarkForRebuildFromLegacy(MoveTemp(BuiltInstanceData), InstanceReorderTable, HitProxies);

	PostBuildStats();
	MarkRenderStateDirty();

	FHierarchicalInstancedStaticMeshDelegates::OnTreeBuilt.Broadcast(this, bWasAsyncBuild);
}

bool UHierarchicalInstancedStaticMeshComponent::BuildTreeIfOutdated(bool Async, bool ForceUpdate)
{
	// The tree will be fully rebuilt once the static mesh compilation is finished.
	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || (GetStaticMesh() && GetStaticMesh()->IsCompiling()))
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UHierarchicalInstancedStaticMeshComponent::BuildTreeIfOutdated);

	if (ForceUpdate 
		|| bIsOutOfDate
		|| PrimitiveInstanceDataManager.HasAnyInstanceChanges()
		|| InstanceReorderTable.Num() != PerInstanceSMData.Num()
		|| NumBuiltInstances != PerInstanceSMData.Num() 
		|| (GetStaticMesh() != nullptr && CacheMeshExtendedBounds != GetStaticMesh()->GetBounds())
		|| UnbuiltInstanceBoundsList.Num() > 0
		|| GetLinkerUEVersion() < VER_UE4_REBUILD_HIERARCHICAL_INSTANCE_TREES
		|| GetLinkerCustomVersion(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::HISMCClusterTreeMigration)
	{
		if (!GetStaticMesh())
		{
			ApplyEmpty();
		}
		else if (!GetStaticMesh()->HasAnyFlags(RF_NeedLoad)) // we can build the tree if the static mesh is not even loaded, and we can't call PostLoad as the load is not even done
		{
			// Make sure if any of those conditions is true, we mark ourselves out of date so the Async Build completes
			bIsOutOfDate = true;

			GetStaticMesh()->ConditionalPostLoad();

			// Trying to do async processing on the begin play does not work, as this will be dirty but not ready for rendering
			const bool bForceSync = (NumBuiltInstances == 0 && GetWorld() && !GetWorld()->HasBegunPlay());

			if (Async && !bForceSync)
			{
				if (IsAsyncBuilding())
				{
					// invalidate the results of the current async build we need to modify the tree
					bConcurrentChanges = true;
				}
				else
				{
					BuildTreeAsync();
				}
			}
			else
			{
				BuildTree();
			}

			return true;
		}
	}

	return false;
}

FVector UHierarchicalInstancedStaticMeshComponent::CalcTranslatedInstanceSpaceOrigin() const
{
	// Foliage is often built in world space which can cause problems with large world coordinates because
	// the instance transforms in the renderer are single precision, and the HISM culling is also single precision.
	// We should fix the HISM culling to be double precision.
	// But the instance transforms (relative to the owner primitive) will probably stay single precision to not bloat memory.
	// A fix for that is to have authoring tools set sensible primitive transforms (instead of identity).
	// But until that happens we set a translated instance space here.
	// For simplicity we use the first instance as the origin of the translated space.
	return bUseTranslatedInstanceSpace && PerInstanceSMData.Num() ? PerInstanceSMData[0].Transform.GetOrigin() : FVector::Zero();
}

void UHierarchicalInstancedStaticMeshComponent::GetInstanceTransforms(TArray<FMatrix>& InstanceTransforms, FVector const& Offset) const
{
	double StartTime = FPlatformTime::Seconds();
	int32 Num = PerInstanceSMData.Num();

	InstanceTransforms.SetNumUninitialized(Num);
	for (int32 Index = 0; Index < Num; Index++)
	{
		InstanceTransforms[Index] = PerInstanceSMData[Index].Transform.ConcatTranslation(Offset);
	}

	UE_LOGF(LogStaticMesh, Verbose, "Copied %d transforms in %.3fs.", Num, float(FPlatformTime::Seconds() - StartTime));
}

void UHierarchicalInstancedStaticMeshComponent::BuildTreeAsync()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHierarchicalInstancedStaticMeshComponent::BuildTreeAsync);

	check(!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject));
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData)
	check(IsInGameThread());

	// If we try to build the tree with the static mesh not fully loaded, we can end up in an inconsistent state which ends in a crash later
	checkSlow(!GetStaticMesh() || !GetStaticMesh()->HasAnyFlags(RF_NeedPostLoad));

	check(!bIsAsyncBuilding);
	check(BuildTreeAsyncTasks.Num() == 0);

	// Verify that the mesh is valid before using it.
	// The tree will be fully rebuilt once the static mesh compilation is finished, no need to do it now.
	if (PerInstanceSMData.Num() > 0 && GetStaticMesh() && !GetStaticMesh()->IsCompiling() && GetStaticMesh()->HasValidRenderData(false))
	{
		double StartTime = FPlatformTime::Seconds();
		
		// Build the tree in translated space to maintain precision.
		TranslatedInstanceSpaceOrigin = CalcTranslatedInstanceSpaceOrigin();

		InitializeInstancingRandomSeed();
		TArray<FMatrix> InstanceTransforms;
		GetInstanceTransforms(InstanceTransforms, -TranslatedInstanceSpaceOrigin);
		
		TSharedRef<FClusterBuilder, ESPMode::ThreadSafe> Builder(new FClusterBuilder(InstanceTransforms, PerInstanceSMCustomData, NumCustomDataFloats, GetStaticMesh()->GetBounds().GetBox(), DesiredInstancesPerLeaf(), CurrentDensityScaling, InstancingRandomSeed, PerInstanceSMData.Num() > 0));

		bIsAsyncBuilding = true;

		FGraphEventRef BuildTreeAsyncResult(
			FDelegateGraphTask::CreateAndDispatchWhenReady(FDelegateGraphTask::FDelegate::CreateRaw(&Builder.Get(), &FClusterBuilder::BuildTreeAndBufferAsync), GET_STATID(STAT_FoliageBuildTime), NULL, ENamedThreads::GameThread, ENamedThreads::AnyBackgroundThreadNormalTask));

		BuildTreeAsyncTasks.Add(BuildTreeAsyncResult);

		// add a dependent task to run on the main thread when build is complete
		FGraphEventRef PostBuildTreeAsyncResult(
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[WeakPtr = TWeakObjectPtr(this), Builder, StartTime]()
				{
					ExecuteOnGameThread(
						TEXT("ApplyBuildTreeAsync_GameThread"),
						[WeakPtr, Builder, StartTime]()
						{
							if (UHierarchicalInstancedStaticMeshComponent* Ptr = WeakPtr.Get())
							{
								FGraphEventRef Dummy;
								Ptr->ApplyBuildTreeAsync(ENamedThreads::GameThread, Dummy, Builder, StartTime);
							}
						}
					);
				},
				GET_STATID(STAT_FoliageBuildTime),
				BuildTreeAsyncResult
			)
		);

		BuildTreeAsyncTasks.Add(PostBuildTreeAsyncResult);
	}
	else
	{
		ApplyEmpty();
	}
}

void UHierarchicalInstancedStaticMeshComponent::PropagateLightingScenarioChange()
{
	UInstancedStaticMeshComponent::PropagateLightingScenarioChange();

	if (!GIsEditor)
	{
		// Need to immediately kill the current proxy (instead of waiting until the async tree build is finished) as the underlying lightmap data (from MapBuildRegistry) can be going away
		MarkRenderStateDirty();
		BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/true);
	}
}

void UHierarchicalInstancedStaticMeshComponent::SetPerInstanceLightMapAndEditorData(FStaticMeshInstanceData& PerInstanceData, const TArray<TRefCountPtr<HHitProxy>>& HitProxies)
{
	int32 NumInstances = PerInstanceData.GetNumInstances();
	
	const FMeshMapBuildData* MeshMapBuildData = nullptr;

#if WITH_EDITOR
	MeshMapBuildData = FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(this, 0);
#endif

	if (MeshMapBuildData == nullptr && LODData.Num() > 0)
	{
		MeshMapBuildData = GetMeshMapBuildData(LODData[0], false);
	}

	if (MeshMapBuildData != nullptr || GIsEditor)
	{
		for (int32 RenderIndex = 0; RenderIndex < NumInstances; ++RenderIndex)
		{
			int32 Index = SortedInstances[RenderIndex];
						
			FVector2D LightmapUVBias = FVector2D(-1.0f, -1.0f);
			FVector2D ShadowmapUVBias = FVector2D(-1.0f, -1.0f);

			if (MeshMapBuildData != nullptr && MeshMapBuildData->PerInstanceLightmapData.IsValidIndex(Index))
			{
				LightmapUVBias = FVector2D(MeshMapBuildData->PerInstanceLightmapData[Index].LightmapUVBias);
				ShadowmapUVBias = FVector2D(MeshMapBuildData->PerInstanceLightmapData[Index].ShadowmapUVBias);

				PerInstanceData.SetInstanceLightMapData(RenderIndex, LightmapUVBias, ShadowmapUVBias);
			}

	#if WITH_EDITOR
			if (GIsEditor)
			{
				// Record if the instance is selected
				FColor HitProxyColor(ForceInit);
				bool bSelected = !SelectedInstances.IsValidIndex(Index) || SelectedInstances[Index];

				if (HitProxies.IsValidIndex(Index))
				{
					HitProxyColor = HitProxies[Index]->Id.GetColor();
				}

				PerInstanceData.SetInstanceEditorData(RenderIndex, HitProxyColor, bSelected);
			}
	#endif
		}
	}
}

FPrimitiveSceneProxy* UHierarchicalInstancedStaticMeshComponent::CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);

	GetOrCreateInstanceDataSceneProxy();

	if (bCreateNanite)
	{
		return ::new Nanite::FSceneProxy(NaniteMaterials, this);
	}
	
	return ::new FHierarchicalStaticMeshSceneProxy(this, GetWorld()->GetFeatureLevel());
}

FPrimitiveSceneProxy* UHierarchicalInstancedStaticMeshComponent::CreateSceneProxy()
{
	static const auto NaniteProxyRenderModeVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.ProxyRenderMode"));
	const int32 NaniteProxyRenderMode = (NaniteProxyRenderModeVar != nullptr) ? (NaniteProxyRenderModeVar->GetInt() != 0) : 0;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_HierarchicalInstancedStaticMeshComponent_CreateSceneProxy);
	SCOPE_CYCLE_COUNTER(STAT_FoliageCreateProxy);

	PrimitiveInstanceDataManager.ResetComponentDirtyTracking();

	if (ProxySize)
	{
		DEC_DWORD_STAT_BY(STAT_FoliageInstanceBuffers, ProxySize);
	}
	ProxySize = 0;

	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for UHierarchicalInstancedStaticMeshComponent %ls (UHierarchicalInstancedStaticMeshComponent PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	// Verify that the mesh is valid & that we have instances before creating a proxy.
	const bool bAreMeshAndInstancesValid =
#if WITH_EDITOR
		bIsInstanceDataApplyCompleted && 
#endif
		// Make sure we have instances, or an update with instances on the way, or already built instances (for the external data (landscape grass) mode).
		(GetNumInstances() > 0 ||  PrimitiveInstanceDataManager.GetMaxInstanceIndex() > 0 || NumBuiltRenderInstances > 0) &&
		// Make sure we have an actual static mesh.
		GetStaticMesh() &&
		!GetStaticMesh()->IsCompiling() &&
		GetStaticMesh()->HasValidRenderData(false);

	if (!bAreMeshAndInstancesValid)
	{
		return nullptr;
	}
		
	check(InstancingRandomSeed != 0);


	// NOTE: Purposefully skipping UInstancedStaticMeshComponent implementation
	FPrimitiveSceneProxy* PrimitiveSceneProxy = UStaticMeshComponent::CreateSceneProxy();

	if (PrimitiveSceneProxy != nullptr)
	{
		FInstanceUpdateComponentDesc ComponentData;
		BuildComponentInstanceData(PrimitiveSceneProxy->GetScene().GetShaderPlatform(), ComponentData);
		PrimitiveInstanceDataManager.FlushChanges(MoveTemp(ComponentData), true);
	}

	// Estimate the allocated data (it is platform dependent and ought to really track actual allocations anyway)
	ProxySize = GetNumRenderInstances() * sizeof(FVector4f) * (5 + FMath::DivideAndRoundUp(NumCustomDataFloats, 4));
	INC_DWORD_STAT_BY(STAT_FoliageInstanceBuffers, ProxySize);

	return PrimitiveSceneProxy;
}

void UHierarchicalInstancedStaticMeshComponent::UpdateDensityScaling()
{
	float OldDensityScaling = CurrentDensityScaling;

#if WITH_EDITOR
	CurrentDensityScaling = bCanEnableDensityScaling && bEnableDensityScaling ? CVarFoliageDensityScale.GetValueOnGameThread() : 1.0f;
#else
	CurrentDensityScaling = bEnableDensityScaling ? CVarFoliageDensityScale.GetValueOnGameThread() : 1.0f;
#endif

	CurrentDensityScaling = FMath::Clamp(CurrentDensityScaling, 0.0f, 1.0f);
	BuildTreeIfOutdated(/*Async*/true, /*ForceUpdate*/OldDensityScaling != CurrentDensityScaling);
}

void UHierarchicalInstancedStaticMeshComponent::OnPostLoadPerInstanceData()
{
	SCOPE_CYCLE_COUNTER(STAT_FoliagePostLoad);
	TRACE_CPUPROFILER_EVENT_SCOPE(UHierarchicalInstancedStaticMeshComponent::OnPostLoadPerInstanceData);

	// Tree will be fully rebuilt when staticmesh has finished compiling
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) && !IsCompiling())
	{
		bool bForceTreeBuild = false;

		if (bEnableDensityScaling && GetWorld() && GetWorld()->IsGameWorld())
		{
			CurrentDensityScaling = FMath::Clamp(CVarFoliageDensityScale.GetValueOnGameThread(), 0.0f, 1.0f);
			bForceTreeBuild = CurrentDensityScaling < 1.0f;
		}

		if (CurrentDensityScaling == 0.f)
		{
			// Not going to render anything
			ClearInstances();
		}
		else
		{
			bool bAsyncTreeBuild = true;

#if WITH_EDITOR
			// If InstanceReorderTable contains invalid indices force a synchronous tree rebuild
			bool bHasInvalidIndices = Algo::AnyOf(InstanceReorderTable, [this](int32 ReorderIndex) { return !PerInstanceSMData.IsValidIndex(ReorderIndex); });
			if (bHasInvalidIndices)
			{
				bAsyncTreeBuild = false;
				bForceTreeBuild = true;
			}
#endif

			// Update the instance data if the reorder table do not match the per instance sm data
			if (!bForceTreeBuild && PerInstanceSMData.Num() > 0 && PerInstanceSMData.Num() != InstanceReorderTable.Num())
			{
				bForceTreeBuild = true;
			}

			// Update the instance data if the lighting scenario isn't the owner level
			if (!bForceTreeBuild)
			{
				if (AActor* Owner = GetOwner())
				{
					if (ULevel* OwnerLevel = Owner->GetLevel())
					{
						UWorld* OwnerWorld = OwnerLevel ? OwnerLevel->OwningWorld.Get() : nullptr;
						if (OwnerWorld && OwnerWorld->GetActiveLightingScenario() != nullptr && OwnerWorld->GetActiveLightingScenario() != OwnerLevel)
						{
							bForceTreeBuild = true;
						}
					}
				}
			}

			if (!bForceTreeBuild)
			{
				NumBuiltRenderInstances = NumBuiltInstances;
				InstanceCountToRender = NumBuiltInstances;
			}

			// If any of the data is out of sync, build the tree now!
			BuildTreeIfOutdated(bAsyncTreeBuild, bForceTreeBuild);
		}
	}
}

static void GatherInstanceTransformsInArea(const UHierarchicalInstancedStaticMeshComponent& Component, const FBox& AreaBox, int32 Child, TArray<FTransform>& InstanceData)
{
	const TArray<FClusterNode>& ClusterTree = *Component.ClusterTreePtr;
	if (ClusterTree.Num())
	{
		const FClusterNode& ChildNode = ClusterTree[Child];

		const FTransform ToWorldTransform = FTransform(Component.TranslatedInstanceSpaceOrigin) * Component.GetComponentTransform();
		const FBox WorldNodeBox = FBox(ChildNode.BoundMin, ChildNode.BoundMax).TransformBy(ToWorldTransform);

#if 0
		// Keeping this as it can be useful to debug but disabled as it can be spammy. 
		UE_VLOG_BOX(&Component, LogStaticMesh, VeryVerbose, FBox(ChildNode.BoundMin, ChildNode.BoundMax), FColor::Red, TEXT("LocalNodeBox"));
		UE_VLOG_BOX(&Component, LogStaticMesh, VeryVerbose, WorldNodeBox, FColor::Green, TEXT("WorldNodeBox"));
		UE_VLOG_BOX(&Component, LogStaticMesh, VeryVerbose, AreaBox, FColor::Blue, TEXT("AreaBox"));
#endif

		if (AreaBox.Intersect(WorldNodeBox))
		{
			if (ChildNode.FirstChild < 0 || AreaBox.IsInside(WorldNodeBox))
			{
				// Unfortunately ordering of PerInstanceSMData does not match ordering of cluster tree, so we have to use remaping
				const bool bUseRemaping = Component.SortedInstances.Num() > 0;
			
				// In case there no more subdivision or node is completely encapsulated by a area box
				// add all instances to the result
				for (int32 i = ChildNode.FirstInstance; i <= ChildNode.LastInstance; ++i)
				{
					int32 SortedIdx = bUseRemaping ? Component.SortedInstances[i] : i;

					FTransform InstanceToComponent;
					if (Component.PerInstanceSMData.IsValidIndex(SortedIdx))
					{
						InstanceToComponent = FTransform(Component.PerInstanceSMData[SortedIdx].Transform);
					}
					
					if (!InstanceToComponent.GetScale3D().IsZero())
					{
						InstanceData.Add(InstanceToComponent*Component.GetComponentTransform());
					}
				}
			}
			else
			{
				for (int32 i = ChildNode.FirstChild; i <= ChildNode.LastChild; ++i)
				{
					GatherInstanceTransformsInArea(Component, AreaBox, i, InstanceData);
				}
			}
		}
	}
}

int32 UHierarchicalInstancedStaticMeshComponent::GetOverlappingSphereCount(const FSphere& Sphere) const
{
	int32 Count = 0;
	TArray<FTransform> Transforms;
	const FBox AABB(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));
	GatherInstanceTransformsInArea(*this, AABB, 0, Transforms);
	const FBoxSphereBounds MeshBounds = GetStaticMesh()->GetBounds();

	for (const FTransform& TM : Transforms)
	{
		const FVector Center = TM.GetLocation();
		const FSphere InstanceSphere(Center, MeshBounds.SphereRadius * TM.GetScale3D().GetMax());
		
		if (Sphere.Intersects(InstanceSphere))
		{
			++Count;
		}
	}

	return Count;
}

int32 UHierarchicalInstancedStaticMeshComponent::GetOverlappingBoxCount(const FBox& Box) const
{
	TArray<FTransform> Transforms;
	GatherInstanceTransformsInArea(*this, Box, 0, Transforms);
	
	int32 Count = 0;
	const FBoxSphereBounds MeshBounds = GetStaticMesh()->GetBounds();
	for(FTransform& T : Transforms)
	{
		const FBox OtherBox(MeshBounds.TransformBy(T).GetBox());
		if(Box.Intersect(OtherBox))
		{
			Count++;
		}
	}

	return Count;
}

void UHierarchicalInstancedStaticMeshComponent::GetOverlappingBoxTransforms(const FBox& Box, TArray<FTransform>& OutTransforms) const
{
	OutTransforms.Reset();
	
	GatherInstanceTransformsInArea(*this, Box, 0, OutTransforms);

	const FBoxSphereBounds MeshBounds = GetStaticMesh()->GetBounds();
	OutTransforms.RemoveAllSwap([&MeshBounds, &Box](const FTransform& Transform) -> bool
	{
		const FBox OtherBox(MeshBounds.TransformBy(Transform).GetBox());
		return !Box.Intersect(OtherBox); 
	});
}

void UHierarchicalInstancedStaticMeshComponent::GetTree(TArray<FClusterNode>& OutClusterTree) const
{
	OutClusterTree = *ClusterTreePtr;
}

FVector UHierarchicalInstancedStaticMeshComponent::GetAverageScale() const
{
	const TArray<FClusterNode>& ClusterTree = *ClusterTreePtr;
	if (ClusterTree.Num())
	{
		return FVector(ClusterTree[0].MinInstanceScale + (ClusterTree[0].MaxInstanceScale - ClusterTree[0].MinInstanceScale) / 2.0f);
	}
	else
	{
		return FVector::Zero();
	}
}


// recursive helper to gather all instances with locations inside the specified area. Supply a Filter to exclude leaf nodes based on the instance transform.
static void GatherInstancesOverlappingArea(const UHierarchicalInstancedStaticMeshComponent& Component, const FBox& AreaBox, int32 Child, TFunctionRef<bool(const FMatrix&)> Filter, TArray<int32>& OutInstanceIndices)
{
	const TArray<FClusterNode>& ClusterTree = *Component.ClusterTreePtr;
	const FClusterNode& ChildNode = ClusterTree[Child];

	const FTransform ToWorldTransform = FTransform(Component.TranslatedInstanceSpaceOrigin) * Component.GetComponentTransform();
	const FBox WorldNodeBox = FBox(ChildNode.BoundMin, ChildNode.BoundMax).TransformBy(ToWorldTransform);

	if (AreaBox.Intersect(WorldNodeBox))
	{
		if (ChildNode.FirstChild < 0 || AreaBox.IsInside(WorldNodeBox))
		{
			// Unfortunately ordering of PerInstanceSMData does not match ordering of cluster tree, so we have to use remaping
			const bool bUseRemaping = Component.SortedInstances.Num() > 0;

			// In case there no more subdivision or node is completely encapsulated by a area box
			// add all instances to the result
			for (int32 i = ChildNode.FirstInstance; i <= ChildNode.LastInstance; ++i)
			{
				int32 SortedIdx = bUseRemaping ? Component.SortedInstances[i] : i;
				if (Component.PerInstanceSMData.IsValidIndex(SortedIdx))
				{
					const FMatrix& Matrix = Component.PerInstanceSMData[SortedIdx].Transform;
					if (Filter(Matrix))
					{
						OutInstanceIndices.Add(SortedIdx);
					}
				}
			}
		}
		else
		{
			for (int32 i = ChildNode.FirstChild; i <= ChildNode.LastChild; ++i)
			{
				GatherInstancesOverlappingArea(Component, AreaBox, i, Filter, OutInstanceIndices);
			}
		}
	}
}

TArray<int32> UHierarchicalInstancedStaticMeshComponent::GetInstancesOverlappingSphere(const FVector& Center, float Radius, bool bSphereInWorldSpace) const
{
	if (ClusterTreePtr.IsValid() && ClusterTreePtr->Num())
	{
		TArray<int32> Result;
		FSphere Sphere(Center, Radius);
		
		FBox WorldSpaceAABB(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));
		if (bSphereInWorldSpace)
		{
			Sphere = Sphere.TransformBy(GetComponentTransform().Inverse());
		}
		else
		{
			WorldSpaceAABB = WorldSpaceAABB.TransformBy(GetComponentTransform());
		}

		if (GetStaticMesh() != nullptr)
		{
			const float StaticMeshBoundsRadius = GetStaticMesh()->GetBounds().SphereRadius;
			GatherInstancesOverlappingArea(*this, WorldSpaceAABB, 0,
				[Sphere, StaticMeshBoundsRadius](const FMatrix& InstanceTransform)->bool
				{
					FSphere InstanceSphere(InstanceTransform.GetOrigin(), StaticMeshBoundsRadius * InstanceTransform.GetScaleVector().GetMax());
					return Sphere.Intersects(InstanceSphere);
				},
				Result);
		}
		return Result;
	}
	else
	{
		return Super::GetInstancesOverlappingSphere(Center, Radius, bSphereInWorldSpace);
	}
}

TArray<int32> UHierarchicalInstancedStaticMeshComponent::GetInstancesOverlappingBox(const FBox& InBox, bool bBoxInWorldSpace) const
{
	if (ClusterTreePtr.IsValid() && ClusterTreePtr->Num())
	{
		TArray<int32> Result;

		FBox WorldSpaceBox(InBox);
		FBox LocalSpaceSpaceBox(InBox);
		if (bBoxInWorldSpace)
		{
			LocalSpaceSpaceBox = LocalSpaceSpaceBox.TransformBy(GetComponentTransform().Inverse());
		}
		else
		{
			WorldSpaceBox = WorldSpaceBox.TransformBy(GetComponentTransform());
		}


		if (GetStaticMesh() != nullptr)
		{
			const FBox StaticMeshBox = GetStaticMesh()->GetBounds().GetBox();
			GatherInstancesOverlappingArea(*this, WorldSpaceBox, 0,
				[LocalSpaceSpaceBox, StaticMeshBox](const FMatrix& InstanceTransform)->bool
				{
					FBox InstanceBox = StaticMeshBox.TransformBy(InstanceTransform);
					return LocalSpaceSpaceBox.Intersect(InstanceBox);
				},
				Result);
		}

		return Result;
	}
	else
	{
		return Super::GetInstancesOverlappingBox(InBox, bBoxInWorldSpace);
	}
}

static void RebuildFoliageTrees(const TArray<FString>& Args)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogConsoleResponse, Display, "Rebuild Foliage Trees");
#endif
	for (TObjectIterator<UHierarchicalInstancedStaticMeshComponent> It; It; ++It)
	{
		UHierarchicalInstancedStaticMeshComponent* Comp = *It;
		if (IsValid(Comp) && !Comp->IsTemplate())
		{
			Comp->BuildTreeIfOutdated(/*Async*/false, /*ForceUpdate*/true);
			Comp->MarkRenderStateDirty();
		}
	}
}

static FAutoConsoleCommand RebuildFoliageTreesCmd(
	TEXT("foliage.RebuildFoliageTrees"),
	TEXT("Rebuild the trees for non-grass foliage."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&RebuildFoliageTrees)
	);

