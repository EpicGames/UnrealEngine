// Copyright Epic Games, Inc. All Rights Reserved.
#include "HighResScreenshot.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "ImageWriteTask.h"
#include "Modules/ModuleManager.h"
#include "Materials/Material.h"
#include "Slate/SceneViewport.h"
#include "ImageWriteQueue.h"

static TAutoConsoleVariable<int32> CVarSaveEXRCompressionQuality(
	TEXT("r.SaveEXR.CompressionQuality"),
	1,
	TEXT("Defines how we save HDR screenshots in the EXR format.\n")
	TEXT(" 0: no compression\n")
	TEXT(" 1: default compression which can be slow (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<FString> CVarHighResScreenshotCmd(
	TEXT("r.HighResScreenshot.AdditionalCmds"), TEXT(""),
	TEXT("Additional command to execute when a high res screenshot is requested."),
	ECVF_Default);

static void RunHighResScreenshotAdditionalCommands()
{
	FString Cmds = CVarHighResScreenshotCmd.GetValueOnGameThread();

	if (!Cmds.IsEmpty())
	{
		GEngine->Exec(GWorld, *Cmds);
	}
}

DEFINE_LOG_CATEGORY(LogHighResScreenshot);

FHighResScreenshotConfig& GetHighResScreenshotConfig()
{
	static FHighResScreenshotConfig Instance;
	return Instance;
}

const float FHighResScreenshotConfig::MinResolutionMultipler = 1.0f;
const float FHighResScreenshotConfig::MaxResolutionMultipler = 10.0f;

FHighResScreenshotConfig::FHighResScreenshotConfig()
	: ResolutionMultiplier(FHighResScreenshotConfig::MinResolutionMultipler)
	, ResolutionMultiplierScale(0.0f)
	, bMaskEnabled(false)
	, bDateTimeBasedNaming(false)
	, bDumpBufferVisualizationTargets(false)
{
	ChangeViewport(TWeakPtr<FSceneViewport>());
	SetHDRCapture(false);
	SetForce128BitRendering(false);
}

void FHighResScreenshotConfig::Init()
{
	ImageWriteQueue = &FModuleManager::LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();

#if WITH_EDITOR
	HighResScreenshotMaterial = LoadObject<UMaterial>(NULL, TEXT("/Engine/EngineMaterials/HighResScreenshot.HighResScreenshot"));
	HighResScreenshotMaskMaterial = LoadObject<UMaterial>(NULL, TEXT("/Engine/EngineMaterials/HighResScreenshotMask.HighResScreenshotMask"));
	HighResScreenshotCaptureRegionMaterial = LoadObject<UMaterial>(NULL, TEXT("/Engine/EngineMaterials/HighResScreenshotCaptureRegion.HighResScreenshotCaptureRegion"));

	if (HighResScreenshotMaterial)
	{
		HighResScreenshotMaterial->AddToRoot();
	}
	if (HighResScreenshotMaskMaterial)
	{
		HighResScreenshotMaskMaterial->AddToRoot();
	}
	if (HighResScreenshotCaptureRegionMaterial)
	{
		HighResScreenshotCaptureRegionMaterial->AddToRoot();
	}
#endif
}

void FHighResScreenshotConfig::PopulateImageTaskParams(FImageWriteTask& InOutTask)
{
	static const TConsoleVariableData<int32>* CVarDumpFramesAsHDR = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BufferVisualizationDumpFramesAsHDR"));

	const bool bCaptureHDREnabledInUI = bCaptureHDR && bDumpBufferVisualizationTargets;

	const bool bLocalCaptureHDR = bCaptureHDREnabledInUI || CVarDumpFramesAsHDR->GetValueOnAnyThread();

	InOutTask.Format = bLocalCaptureHDR ? EImageFormat::EXR : EImageFormat::PNG;

	InOutTask.CompressionQuality = (int32)EImageCompressionQuality::Default;
	if (bLocalCaptureHDR && CVarSaveEXRCompressionQuality.GetValueOnAnyThread() == 0)
	{
		InOutTask.CompressionQuality = (int32)EImageCompressionQuality::Uncompressed;
	}
}

void FHighResScreenshotConfig::ChangeViewport(TWeakPtr<FSceneViewport> InViewport)
{
	if (FSceneViewport* Viewport = TargetViewport.Pin().Get())
	{
		// Force an invalidate on the old viewport to make sure we clear away the capture region effect
		Viewport->Invalidate();
	}

	UnscaledCaptureRegion = FIntRect(0, 0, 0, 0);
	CaptureRegion = UnscaledCaptureRegion;
	bMaskEnabled = false;
	bDateTimeBasedNaming = false;
	bDumpBufferVisualizationTargets = false;
	ResolutionMultiplier = FHighResScreenshotConfig::MinResolutionMultipler;
	ResolutionMultiplierScale = 0.0f;
	TargetViewport = InViewport;
}

bool FHighResScreenshotConfig::ParseConsoleCommand(const FString& InCmd, FOutputDevice& Ar)
{
	GScreenshotResolutionX = 0;
	GScreenshotResolutionY = 0;
	ResolutionMultiplier = FHighResScreenshotConfig::MinResolutionMultipler;
	ResolutionMultiplierScale = 0.0f;

	if( GetHighResScreenShotInput(*InCmd, Ar, GScreenshotResolutionX, GScreenshotResolutionY, ResolutionMultiplier, CaptureRegion, bMaskEnabled, bDumpBufferVisualizationTargets, bCaptureHDR, FilenameOverride, bDateTimeBasedNaming) )
	{
		GScreenshotResolutionX *= ResolutionMultiplier;
		GScreenshotResolutionY *= ResolutionMultiplier;

		uint32 MaxTextureDimension = GetMax2DTextureDimension();

		// Check that we can actually create a destination texture of this size
		if ( GScreenshotResolutionX > MaxTextureDimension || GScreenshotResolutionY > MaxTextureDimension )
		{
			Ar.Logf(TEXT("Error: Screenshot size exceeds the maximum allowed texture size (%d x %d)"), GetMax2DTextureDimension(), GetMax2DTextureDimension());
			return false;
		}

		GIsHighResScreenshot = true;
		RunHighResScreenshotAdditionalCommands();
		return true;
	}

	return false;
}

template<class FColorType, typename TChannelType>
bool MergeMaskIntoAlphaInternal(TArray<FColorType>& InBitmap, const FIntRect& ViewRect, bool bMaskEnabled, TChannelType AlphaMultipler)
{
	bool bWritten = false;

	TArray<FColor>* MaskArray = FScreenshotRequest::GetHighresScreenshotMaskColorArray();
	const FIntPoint& MaskExtents = FScreenshotRequest::GetHighresScreenshotMaskExtents();

	bool bMaskMatches = !bMaskEnabled || (InBitmap.Num() == MaskArray->Num()) || (InBitmap.Num() == ViewRect.Area() && ViewRect.Max.X <= MaskExtents.X && ViewRect.Max.Y <= MaskExtents.Y);
	ensureMsgf(bMaskMatches, TEXT("Highres screenshot MaskArray doesn't match screenshot size.  Skipping Masking. MaskSize: %i, ScreenshotSize: %i"), MaskArray->Num(), InBitmap.Num());
	if (bMaskEnabled && bMaskMatches)
	{
		// If this is a high resolution screenshot and we are using the masking feature,
		// Get the results of the mask rendering pass and insert into the alpha channel of the screenshot.
		if (InBitmap.Num() == MaskArray->Num())
		{
			// Exact match, copy verbatim
			for (int32 i = 0; i < InBitmap.Num(); ++i)
			{
				InBitmap[i].A = TChannelType((*MaskArray)[i].R) * AlphaMultipler;
			}
		}
		else
		{
			// Need to pull a rectangle out of the mask array
			int32 RectOffsetX = ViewRect.Min.X;
			int32 RectOffsetY = ViewRect.Min.Y;
			int32 OutputOffset = 0;
			int32 MaskStride = MaskExtents.X;

			for (int32 j = ViewRect.Min.Y; j < ViewRect.Max.Y; j++)
			{
				for (int32 i = ViewRect.Min.X; i < ViewRect.Max.X; i++, OutputOffset++)
				{
					InBitmap[OutputOffset].A = TChannelType((*MaskArray)[j * MaskStride + i].R) * AlphaMultipler;
				}
			}
		}
		bWritten = true;
	}
	else
	{
		// Ensure that all pixels' alpha is set to 255
		for (auto& Color : InBitmap)
		{
			Color.A = TChannelType(255) * AlphaMultipler;
		}
	}

	return bWritten;
}

bool FHighResScreenshotConfig::MergeMaskIntoAlpha(TArray<FColor>& InBitmap, const FIntRect& ViewRect)
{
	return MergeMaskIntoAlphaInternal<FColor, uint8>(InBitmap, ViewRect, bMaskEnabled, 1);
}

bool FHighResScreenshotConfig::MergeMaskIntoAlpha(TArray<FLinearColor>& InBitmap, const FIntRect& ViewRect)
{
	return MergeMaskIntoAlphaInternal<FLinearColor, float>(InBitmap, ViewRect, bMaskEnabled, 1.0f / 255.0f);
}

void FHighResScreenshotConfig::SetHDRCapture(bool bCaptureHDRIN)
{
	bCaptureHDR = bCaptureHDRIN;
}

void FHighResScreenshotConfig::SetForce128BitRendering(bool bForce)
{
	bForce128BitRendering = bForce;
}

bool FHighResScreenshotConfig::SetResolution(uint32 ResolutionX, uint32 ResolutionY, float ResolutionScale)
{
	if ((ResolutionX * ResolutionScale) > GetMax2DTextureDimension() || (ResolutionY * ResolutionScale) > GetMax2DTextureDimension())
	{
		// TODO LOG
		//Ar.Logf(TEXT("Error: Screenshot size exceeds the maximum allowed texture size (%d x %d)"), GetMax2DTextureDimension(), GetMax2DTextureDimension());
		return false;
	}

	UnscaledCaptureRegion = FIntRect(0, 0, 0, 0);
	CaptureRegion = UnscaledCaptureRegion;
	bMaskEnabled = false;

	GScreenshotResolutionX = (ResolutionX * ResolutionScale);
	GScreenshotResolutionY = (ResolutionY * ResolutionScale);
	GIsHighResScreenshot = true;
	RunHighResScreenshotAdditionalCommands();

	return true;
}

void FHighResScreenshotConfig::SetFilename(FString Filename)
{
	FilenameOverride = Filename;
}

void FHighResScreenshotConfig::SetMaskEnabled(bool bShouldMaskBeEnabled)
{
	bMaskEnabled = bShouldMaskBeEnabled;
}
