// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HitProxies.cpp: Hit proxy implementation.
=============================================================================*/

#include "HitProxies.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "GenericPlatform/ICursor.h"

/// @cond DOXYGEN_WARNINGS

#include UE_INLINE_GENERATED_CPP_BY_NAME(HitProxies)

IMPLEMENT_HIT_PROXY_BASE( HHitProxy, NULL );
IMPLEMENT_HIT_PROXY(HObject,HHitProxy);

/// @endcond

const FHitProxyId FHitProxyId::InvisibleHitProxyId( INDEX_NONE - 1 );


/** The global list of allocated hit proxies, indexed by hit proxy ID. */

class FHitProxyArray
{
	TSparseArray<HHitProxy*> HitProxies;
	FTransactionallySafeCriticalSection Lock;
public:

	static FHitProxyArray& Get()
	{
		static FHitProxyArray Singleton;
		return Singleton;
	}

	void Remove(int32 Index)
	{
		UE::TScopeLock ScopeLock(Lock);
		HitProxies.RemoveAt(Index);
	}

	int32 Add(HHitProxy* Proxy)
	{
		UE::TScopeLock ScopeLock(Lock);
		return HitProxies.Add(Proxy);
	}

	HHitProxy* GetHitProxyById(int32 Index)
	{
		UE::TScopeLock ScopeLock(Lock);
		if(Index >= 0 && Index < HitProxies.GetMaxIndex() && HitProxies.IsAllocated(Index))
		{
			return HitProxies[Index];
		}
		else
		{
			return NULL;
		}
	}

	const TSparseArray<HHitProxy*>& GetAllHitProxies() const
	{
		return HitProxies;
	}
};

static struct FForceInitHitProxyBeforeMain
{
	FForceInitHitProxyBeforeMain()
	{
		// we don't want this to be initialized by two threads at once, so we will set it up before main starts
		FHitProxyArray::Get();
	}
} ForceInitHitProxyBeforeMain;

FHitProxyId::FHitProxyId(FColor Color)
{
	Index = ((int32)Color.R << 16) | ((int32)Color.G << 8) | ((int32)Color.B << 0) | ((int32)Color.A << 24);
}

FColor FHitProxyId::GetColor() const
{
	return FColor(
		((Index >> 16) & 0xff),
		((Index >> 8) & 0xff),
		((Index >> 0) & 0xff),
		((Index >> 24) & 0xff)
		);
}

HHitProxy::HHitProxy(EHitProxyPriority InPriority):
	Priority(InPriority),
	OrthoPriority(InPriority)
{
	InitHitProxy();
}

HHitProxy::HHitProxy(EHitProxyPriority InPriority, EHitProxyPriority InOrthoPriority):
	Priority(InPriority),
	OrthoPriority(InOrthoPriority)
{
	InitHitProxy();
}

HHitProxy::~HHitProxy()
{
	// Remove this hit proxy from the global array.
	FHitProxyArray::Get().Remove(Id.Index);
}

void HHitProxy::InitHitProxy()
{
	// Allocate an entry in the global hit proxy array for this hit proxy, and use the index as the hit proxy's ID.
	Id = FHitProxyId(FHitProxyArray::Get().Add(this));
}

bool HHitProxy::IsA(HHitProxyType* TestType) const
{
	bool bIsInstance = false;
	for(HHitProxyType* Type = GetType();Type;Type = Type->GetParent())
	{
		if(Type == TestType)
		{
			bIsInstance = true;
			break;
		}
	}
	return bIsInstance;
}

EMouseCursor::Type HHitProxy::GetMouseCursor()
{
	return EMouseCursor::Default;
}

FTypedElementHandle HHitProxy::GetElementHandle() const
{
	return FTypedElementHandle();
}

HHitProxy* GetHitProxyById(FHitProxyId Id)
{
	int32 IndexIgnoringAlpha = Id.Index & 0xffffff;
	return FHitProxyArray::Get().GetHitProxyById(IndexIgnoringAlpha);
}

ENGINE_API const TSparseArray<HHitProxy*>& GetAllHitProxies()
{
	return FHitProxyArray::Get().GetAllHitProxies();
}
