// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPartition/ISMComponentBatcher.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Serialization/ArchiveCrc32.h"
#include "Misc/TransformUtilities.h"
#include "Templates/TypeHash.h"

void FISMComponentBatcher::Add(const UActorComponent* InComponent)
{
	AddInternal(InComponent, TOptional<TFunctionRef<FTransform(const FTransform&)>>(), TOptional<TFunctionRef<bool(const FBox&)>>());
}

void FISMComponentBatcher::Add(const UActorComponent* InComponent, TFunctionRef<FTransform(const FTransform&)> InTransformFunc)
{
	AddInternal(InComponent, InTransformFunc, TOptional<TFunctionRef<bool(const FBox&)>>());
}

void FISMComponentBatcher::Add(const UActorComponent* InComponent, TFunctionRef<bool(const FBox&)> InFilterFunc)
{
	AddInternal(InComponent, TOptional<TFunctionRef<FTransform(const FTransform&)>>(), InFilterFunc);
}

void FISMComponentBatcher::Add(const UActorComponent* InComponent, TFunctionRef<FTransform(const FTransform&)> InTransformFunc, TFunctionRef<bool(const FBox&)> InFilterFunc)
{
	AddInternal(InComponent, InTransformFunc, InFilterFunc);
}

void FISMComponentBatcher::AddInternal(const UActorComponent* InComponent, TOptional<TFunctionRef<FTransform(const FTransform&)>> InTransformFunc, TOptional<TFunctionRef<bool(const FBox&)>> InFilterFunc)
{
	Hash = 0; // Invalidate
	int32 NewNumCustomDataFloats = 0;
	int32 NewNumInstances = 0;
	int32 InitialInstanceCount = NumInstances;

	// Compute number of instances & custom data float to add
	if (const UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		NewNumCustomDataFloats = ISMC->NumCustomDataFloats;
		NewNumInstances = ISMC->GetInstanceCount();
	}
	else if (const UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(InComponent))
	{
		NewNumCustomDataFloats = SMC->GetCustomPrimitiveData().Data.Num();
		NewNumInstances = 1;
	}
	else if (const UInstancedSkinnedMeshComponent* ISKMC = Cast<UInstancedSkinnedMeshComponent>(InComponent))
	{
		NewNumCustomDataFloats = ISKMC->GetNumCustomDataFloats();
		NewNumInstances = ISKMC->GetInstanceCount();
	}

	// If we must increase the number of custom data floats, this means we have to add the proper padding between each existing
	// custom data float set.
	if (NewNumCustomDataFloats > NumCustomDataFloats)
	{
		TArray<float> NewInstancesCustomData;
		NewInstancesCustomData.AddZeroed(NewNumCustomDataFloats * NumInstances);

		for (int32 InstanceIdx = 0; InstanceIdx < NumInstances; ++InstanceIdx)
		{
			int32 SrcCustomDataOffset = NumCustomDataFloats * InstanceIdx;
			int32 DestCustomDataOffset = NewNumCustomDataFloats * InstanceIdx;
			for (int32 CustomDataIdx = 0; CustomDataIdx < NumCustomDataFloats; ++CustomDataIdx)
			{
				NewInstancesCustomData[DestCustomDataOffset + CustomDataIdx] = InstancesCustomData[SrcCustomDataOffset + CustomDataIdx];
			}
		}

		InstancesCustomData = MoveTemp(NewInstancesCustomData);
		NumCustomDataFloats = NewNumCustomDataFloats;
	}

	int32 ReserveNum = NumInstances + NewNumInstances;
	InstancesTransformsWS.Reserve(ReserveNum);
	InstancesCustomData.Reserve(ReserveNum * NumCustomDataFloats);

	// Add instances
	if (const UInstancedStaticMeshComponent* ISMC = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		FBox StaticMeshBox = ISMC->GetStaticMesh() ? ISMC->GetStaticMesh()->GetBounds().GetBox() : FBox();

		// Instancing Random Seed
		InstancingRandomSeed  = ISMC->InstancingRandomSeed;

		// Add each instance
		int32 NextRandomSeedsIndex = ISMC->AdditionalRandomSeeds.IsValidIndex(0) ? 0 : INDEX_NONE;
		int32 NextRandomSeedsInstanceStart = NextRandomSeedsIndex != INDEX_NONE ? ISMC->AdditionalRandomSeeds[NextRandomSeedsIndex].StartInstanceIndex : INDEX_NONE;

		for (int32 InstanceIdx = 0, RemappedInstanceIdx=0; InstanceIdx < ISMC->GetInstanceCount(); InstanceIdx++, RemappedInstanceIdx++)
		{
			// Add additional random seeds
			if (InstanceIdx == NextRandomSeedsInstanceStart)
			{
				AdditionalRandomSeeds.Add({ RemappedInstanceIdx, ISMC->AdditionalRandomSeeds[NextRandomSeedsIndex].RandomSeed });
				NextRandomSeedsIndex = ISMC->AdditionalRandomSeeds.IsValidIndex(NextRandomSeedsIndex+1) ? NextRandomSeedsIndex+1 : INDEX_NONE;
				NextRandomSeedsInstanceStart = NextRandomSeedsIndex != INDEX_NONE ? ISMC->AdditionalRandomSeeds[NextRandomSeedsIndex].StartInstanceIndex : INDEX_NONE;
			}
			
			FTransform InstanceTransformWS;
			if (ISMC->GetInstanceTransform(InstanceIdx, InstanceTransformWS, /*bWorldSpace*/ true))
			{
				if (InTransformFunc.IsSet())
				{
					InstanceTransformWS = InTransformFunc.GetValue()(InstanceTransformWS);
				}

				if (InFilterFunc.IsSet())
				{
					bool bFilterPass = InFilterFunc.GetValue()(StaticMeshBox.TransformBy(InstanceTransformWS));
					if (!bFilterPass)
					{
						RemappedInstanceIdx--;
						continue;
					}
				}

				NumInstances++;

				// Add instance transform
				InstancesTransformsWS.Add(InstanceTransformWS);

				// Add per instance custom data, if any
				if (NumCustomDataFloats > 0)
				{
					if (ISMC->NumCustomDataFloats > 0)
					{
						TConstArrayView<float> InstanceCustomData(&ISMC->PerInstanceSMCustomData[InstanceIdx * ISMC->NumCustomDataFloats], ISMC->NumCustomDataFloats);
						InstancesCustomData.Append(InstanceCustomData);
					}

					InstancesCustomData.AddDefaulted(NumCustomDataFloats - ISMC->NumCustomDataFloats);
				}
			}
		}
	}
	else if (const UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(InComponent))
	{
		FBox StaticMeshBox = SMC->GetStaticMesh() ? SMC->GetStaticMesh()->GetBounds().GetBox() : FBox();

		FTransform InstanceTransformWS = SMC->GetComponentTransform();
		if (InTransformFunc.IsSet())
		{
			InstanceTransformWS = InTransformFunc.GetValue()(InstanceTransformWS);
		}

		if (InFilterFunc.IsSet())
		{
			bool bFilterPass = InFilterFunc.GetValue()(StaticMeshBox.TransformBy(InstanceTransformWS));
			if (!bFilterPass)
			{
				return;
			}
		}

		NumInstances++;

		// Add transform
		InstancesTransformsWS.Add(InstanceTransformWS);

		// Add custom data
		InstancesCustomData.Append(SMC->GetCustomPrimitiveData().Data);
		InstancesCustomData.AddDefaulted(NumCustomDataFloats - SMC->GetCustomPrimitiveData().Data.Num());
	}
	else if (const UInstancedSkinnedMeshComponent* ISKMC = Cast<UInstancedSkinnedMeshComponent>(InComponent))
	{
		// Add each instance
		FPrimitiveInstanceId InstanceId;
		for (InstanceId.Id = 0; InstanceId.Id < ISKMC->GetInstanceCount(); InstanceId.Id++)
		{
			NumInstances++;

			// Add instance transform
			FTransform InstanceTransformWS;

			if (ISKMC->GetInstanceTransform(InstanceId, InstanceTransformWS, /*bWorldSpace*/ true))
			{
				if (InTransformFunc.IsSet())
				{
					InstanceTransformWS = InTransformFunc.GetValue()(InstanceTransformWS);
				}
				InstancesTransformsWS.Add(InstanceTransformWS);

				ISKMC->GetInstanceAnimationIndex(InstanceId, AnimationIndices.AddZeroed_GetRef());

				// Add per instance custom data, if any
				if (NumCustomDataFloats > 0)
				{
					int32 StartIndex = InstancesCustomData.Num(); // Get the current end index
					InstancesCustomData.AddDefaulted(NumCustomDataFloats); // Append new elements

					if (ISKMC->GetNumCustomDataFloats() > 0)
					{
						ISKMC->GetCustomData(InstanceId, TArrayView<float>(InstancesCustomData.GetData() + StartIndex, NumCustomDataFloats));
					}
				}
			}
		}
	}

	const uint32 NumInstancesAdded = NumInstances - InitialInstanceCount;
	if (bBuildMappingInfo && NumInstancesAdded != 0)
	{
		ComponentsToInstancesMap.Emplace(InComponent, InitialInstanceCount, NumInstancesAdded);
	}
}

void FISMComponentBatcher::InitComponent(UInstancedStaticMeshComponent* ISMComponent) const
{
	ISMComponent->NumCustomDataFloats = NumCustomDataFloats;
	ISMComponent->AddInstances(InstancesTransformsWS, /*bShouldReturnIndices*/false, /*bWorldSpace*/true);
	ISMComponent->PerInstanceSMCustomData = InstancesCustomData;

	ISMComponent->InstancingRandomSeed = InstancingRandomSeed;
	ISMComponent->AdditionalRandomSeeds = AdditionalRandomSeeds;
}

void FISMComponentBatcher::InitComponent(UInstancedSkinnedMeshComponent* ISMComponent) const
{
	ISMComponent->SetNumCustomDataFloats(NumCustomDataFloats);
	ISMComponent->AddInstances(InstancesTransformsWS, AnimationIndices, /*bShouldReturnIndices*/false, /*bWorldSpace*/true);

	if (NumCustomDataFloats)
	{
		FPrimitiveInstanceId InstanceId;
		for (InstanceId.Id = 0; InstanceId.Id < ISMComponent->GetInstanceCount(); InstanceId.Id++)
		{
			ISMComponent->SetCustomData(InstanceId, TArrayView<const float>(&InstancesCustomData[InstanceId.Id], NumCustomDataFloats));
		}
	}
}

void FISMComponentBatcher::ComputeHash() const
{
	uint32 CRC = 0;
	for (const FTransform& InstanceTransform : InstancesTransformsWS)
	{
		CRC = HashCombine(TransformUtilities::GetRoundedTransformCRC32(InstanceTransform), CRC);
	}
	
	FArchiveCrc32 Ar(CRC);
	FISMComponentBatcher& This = *const_cast<FISMComponentBatcher*>(this);

	Ar << This.InstancesCustomData;
	Ar << This.InstancingRandomSeed;
	Ar << This.AdditionalRandomSeeds;
	Ar << This.AnimationIndices;

	Hash = Ar.GetCrc();
}

TArray<FISMComponentBatcher::FComponentToInstancesMapping> FISMComponentBatcher::GetComponentsToInstancesMap()
{
	return MoveTemp(ComponentsToInstancesMap);
}
