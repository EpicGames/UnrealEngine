// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPartition/ISMComponentData.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMComponentData)

FISMComponentData::FISMComponentData()
#if WITH_EDITORONLY_DATA
	: Component(nullptr)
	, bInvalidateLightingCache(false)
	, bAutoRebuildTreeOnInstanceChanges(false)
	, bWasModifyCalled(false)
#endif
{

}

#if WITH_EDITOR
void FISMComponentData::RegisterDelegates()
{
	if (Component && Component->IsA<UHierarchicalInstancedStaticMeshComponent>())
	{
		if (Component->GetStaticMesh())
		{
			Component->GetStaticMesh()->GetOnExtendedBoundsChanged().AddRaw(this, &FISMComponentData::HandleComponentMeshBoundsChanged);
		}
	}
}

void FISMComponentData::UnregisterDelegates()
{
	if (Component && Component->IsA<UHierarchicalInstancedStaticMeshComponent>())
	{
		if (Component->GetStaticMesh())
		{
			Component->GetStaticMesh()->GetOnExtendedBoundsChanged().RemoveAll(this);
		}
	}
}

void FISMComponentData::HandleComponentMeshBoundsChanged(const FBoxSphereBounds& NewBounds)
{
	check(Component);
	if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(Component))
	{
		HISMComponent->BuildTreeIfOutdated(true, false);
	}
}
#endif

