// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPartition/ISMComponentDescriptor.h"
#include "Concepts/StaticStructProvider.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMComponentDescriptor)

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Serialization/ArchiveCrc32.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "VT/RuntimeVirtualTexture.h"
#include "Algo/Transform.h"

bool GISMComponentDescriptorCopiesCanEverAffectNavigation = true;
static FAutoConsoleVariableRef CVarSMComponentDescriptorCopiesCanEverAffectNavigation(
	TEXT("ISM.ComponentDescriptor.CanEverAffectNavigation"),
	GISMComponentDescriptorCopiesCanEverAffectNavigation,
	TEXT("If true CanEverAffectNavigation property will be copied from the StaticMesh and applied to the ISM"));

FISMComponentDescriptorBase::FISMComponentDescriptorBase()
{
	// Note: should not really be used - prefer using FISMComponentDescriptor & FSoftISMDescriptor instead
	InitFrom(UHierarchicalInstancedStaticMeshComponent::StaticClass()->GetDefaultObject<UHierarchicalInstancedStaticMeshComponent>());
}

FISMComponentDescriptorBase::FISMComponentDescriptorBase(ENoInit) {}
FISMComponentDescriptorBase::FISMComponentDescriptorBase(const FISMComponentDescriptorBase&) = default;
FISMComponentDescriptorBase::~FISMComponentDescriptorBase() = default;

FISMComponentDescriptor::FISMComponentDescriptor()
	: FISMComponentDescriptorBase(NoInit)
{
	// Make sure we have proper defaults
	InitFrom(UHierarchicalInstancedStaticMeshComponent::StaticClass()->GetDefaultObject<UHierarchicalInstancedStaticMeshComponent>());
}

FISMComponentDescriptor::FISMComponentDescriptor(const FSoftISMComponentDescriptor& Other)
	: FISMComponentDescriptorBase(Other)
{
	StaticMesh = Other.StaticMesh.LoadSynchronous();
	Algo::Transform(Other.OverrideMaterials, OverrideMaterials, [](TSoftObjectPtr<UMaterialInterface> Material) { return Material.LoadSynchronous(); });
	OverlayMaterial = Other.OverlayMaterial.LoadSynchronous();
	Algo::Transform(Other.RuntimeVirtualTextures, RuntimeVirtualTextures, [](TSoftObjectPtr<URuntimeVirtualTexture> RVT) { return RVT.LoadSynchronous(); });
	Hash = Other.Hash;
}

FISMComponentDescriptor::FISMComponentDescriptor(const FISMComponentDescriptor&) = default;
FISMComponentDescriptor::~FISMComponentDescriptor() = default;

FSoftISMComponentDescriptor::FSoftISMComponentDescriptor()
	: FISMComponentDescriptorBase(NoInit)
{
	// Make sure we have proper defaults
	InitFrom(UHierarchicalInstancedStaticMeshComponent::StaticClass()->GetDefaultObject<UHierarchicalInstancedStaticMeshComponent>());
}

FSoftISMComponentDescriptor::FSoftISMComponentDescriptor(const FISMComponentDescriptor& Other)
	: FISMComponentDescriptorBase(Other)
{
	StaticMesh = Other.StaticMesh;
	Algo::Transform(Other.OverrideMaterials, OverrideMaterials, [](TObjectPtr<UMaterialInterface> Material) { return Material; });
	OverlayMaterial = Other.OverlayMaterial;
	Algo::Transform(Other.RuntimeVirtualTextures, RuntimeVirtualTextures, [](TObjectPtr<URuntimeVirtualTexture> RVT) { return RVT; });
	Hash = Other.Hash;
}

FSoftISMComponentDescriptor::FSoftISMComponentDescriptor(const FSoftISMComponentDescriptor& Other) = default;
FSoftISMComponentDescriptor::~FSoftISMComponentDescriptor() = default;

FISMComponentDescriptor FISMComponentDescriptor::CreateFrom(const TSubclassOf<UStaticMeshComponent>& From)
{
	FISMComponentDescriptor ComponentDescriptor;

	ComponentDescriptor.InitFrom(From->GetDefaultObject<UStaticMeshComponent>());
	ComponentDescriptor.ComputeHash();

	return ComponentDescriptor;
}

void FISMComponentDescriptorBase::InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance)
{
	check(Template);
	bEnableDiscardOnLoad = false;

	// Disregard the template class if it does not stem from an instanced mesh component
	if (Template->IsA(UInstancedStaticMeshComponent::StaticClass()))
	{
		ComponentClass = Template->GetClass();
	}

	Mobility = Template->Mobility;
	VirtualTextureRenderPassType = Template->VirtualTextureRenderPassType;
	LightmapType = Template->GetLightmapType();
	LightingChannels = Template->LightingChannels;
	RayTracingGroupId = Template->RayTracingGroupId;
	RayTracingGroupCullingPriority = Template->RayTracingGroupCullingPriority;
	bHasCustomNavigableGeometry = Template->bHasCustomNavigableGeometry;
	CustomDepthStencilWriteMask = Template->CustomDepthStencilWriteMask;
	VirtualTextureCullMips = Template->VirtualTextureCullMips;
	TranslucencySortPriority = Template->TranslucencySortPriority;
	OverriddenLightMapRes = Template->OverriddenLightMapRes;
	CustomDepthStencilValue = Template->CustomDepthStencilValue;
	bCastShadow = Template->CastShadow;
	bEmissiveLightSource = Template->bEmissiveLightSource;
	bAffectDynamicIndirectLighting = Template->bAffectDynamicIndirectLighting;
	bAffectDynamicIndirectLightingWhileHidden = Template->bAffectIndirectLightingWhileHidden;
	bAffectDistanceFieldLighting = Template->bAffectDistanceFieldLighting;
	bCastDynamicShadow = Template->bCastDynamicShadow;
	bCastStaticShadow = Template->bCastStaticShadow;
	bCastVolumetricTranslucentShadow = Template->bCastVolumetricTranslucentShadow;
	bCastContactShadow = Template->bCastContactShadow;
	bSelfShadowOnly = Template->bSelfShadowOnly;
	bCastFarShadow = Template->bCastFarShadow;
	bCastInsetShadow = Template->bCastInsetShadow;
	bCastCinematicShadow = Template->bCastCinematicShadow;
	bCastHiddenShadow = Template->bCastHiddenShadow;
	bCastShadowAsTwoSided = Template->bCastShadowAsTwoSided;
	bReceivesDecals = Template->bReceivesDecals;
	bOverrideLightMapRes = Template->bOverrideLightMapRes;
	bUseAsOccluder = Template->bUseAsOccluder;
	bRenderCustomDepth = Template->bRenderCustomDepth;
	bHiddenInGame = Template->bHiddenInGame;
	bIsEditorOnly = Template->bIsEditorOnly;
	bVisible = Template->GetVisibleFlag();
	bSelectable = Template->bSelectable;
	bHasPerInstanceHitProxies = Template->bHasPerInstanceHitProxies;
	bVisibleInRayTracing = Template->bVisibleInRayTracing;
	bVisibleInReflections = Template->bVisibleInReflections;
	bRenderInMainPass = Template->bRenderInMainPass;
	bRenderInDepthPass = Template->bRenderInDepthPass;	
	bEvaluateWorldPositionOffset = Template->bEvaluateWorldPositionOffset;
	WorldPositionOffsetDisableDistance = Template->WorldPositionOffsetDisableDistance;
	NanitePixelProgrammableDistance = Template->NanitePixelProgrammableDistance;
	ShadowCacheInvalidationBehavior = Template->ShadowCacheInvalidationBehavior;
	DetailMode = Template->DetailMode;
	// Determine if this instance must render with reversed culling based on both scale and the component property
	const bool bIsLocalToWorldDeterminantNegative = Template->GetRenderMatrix().Determinant() < 0;
	bReverseCulling = Template->bReverseCulling != bIsLocalToWorldDeterminantNegative;
	bUseDefaultCollision = Template->bUseDefaultCollision;
	bGenerateOverlapEvents = Template->GetGenerateOverlapEvents();
	bOverrideNavigationExport = Template->bOverrideNavigationExport;
	bForceNavigationObstacle = Template->bForceNavigationObstacle;
	bFillCollisionUnderneathForNavmesh = Template->bFillCollisionUnderneathForNavmesh;
	bRasterizeAsFilledConvexVolume = Template->bRasterizeAsFilledConvexVolume;

	if (GISMComponentDescriptorCopiesCanEverAffectNavigation)
	{
		bCanEverAffectNavigation = Template->CanEverAffectNavigation();
	}

#if WITH_EDITORONLY_DATA
	HLODBatchingPolicy = Template->HLODBatchingPolicy;
	bIncludeInHLOD = Template->bEnableAutoLODGeneration;
	bConsiderForActorPlacementWhenHidden = Template->bConsiderForActorPlacementWhenHidden;
#endif

	if (const UInstancedStaticMeshComponent* ISMTemplate = Cast<UInstancedStaticMeshComponent>(Template))
	{
		InstanceMinDrawDistance = ISMTemplate->InstanceMinDrawDistance;
		InstanceStartCullDistance = ISMTemplate->InstanceStartCullDistance;
		InstanceEndCullDistance = ISMTemplate->InstanceEndCullDistance;
		bUseGpuLodSelection = ISMTemplate->bUseGpuLodSelection;

		// HISM Specific
		if (const UHierarchicalInstancedStaticMeshComponent* HISMTemplate = Cast<UHierarchicalInstancedStaticMeshComponent>(Template))
		{
			bEnableDensityScaling = HISMTemplate->bEnableDensityScaling;
			InstanceLODDistanceScale = HISMTemplate->InstanceLODDistanceScale;
		}
	}

	if (bInitBodyInstance)
	{
		// If we are building this batch live in the editor, then the Body Properties will be in a different runtime state.
		if (Template->BodyInstance.BodySetup.Get() == nullptr)
		{
			BodyInstance.CopyBodyInstancePropertiesFrom(&Template->BodyInstance);
		}
		else
		{
			BodyInstance.CopyRuntimeBodyInstancePropertiesFrom(&Template->BodyInstance);
		}
	}
}

void FISMComponentDescriptor::InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance)
{
	StaticMesh = Template->GetStaticMesh();
	StaticMeshComponent = Template;
	OverrideMaterials = Template->OverrideMaterials;
	OverlayMaterial = Template->OverlayMaterial;
	RuntimeVirtualTextures = Template->RuntimeVirtualTextures;

	Super::InitFrom(Template, bInitBodyInstance);
}

void FSoftISMComponentDescriptor::InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance)
{
	StaticMesh = Template->GetStaticMesh();
	Algo::Transform(Template->OverrideMaterials, OverrideMaterials, [](TObjectPtr<UMaterialInterface> Material) { return Material; });
	OverlayMaterial = Template->OverlayMaterial;
	Algo::Transform(Template->RuntimeVirtualTextures, RuntimeVirtualTextures, [](TObjectPtr<URuntimeVirtualTexture> RVT) { return RVT; });

	Super::InitFrom(Template, bInitBodyInstance);
}

void FISMComponentDescriptorBase::PostLoadFixup(UObject* Loader)
{
	check(Loader);

	// Necessary to update the collision Response Container from the array
	BodyInstance.FixupData(Loader);
}

UMaterialInterface* FISMComponentDescriptorBase::DuplicateMaterialInterface(const UMaterialInterface& InMaterialInterface, UObject& NewOuter)
{
	// Regenerate a new unique number since the template MIDs don't have stable numbers during cook
	const FName BaseName = FName(InMaterialInterface.GetFName(), 0);
	const FName NewName = MakeUniqueObjectName(&NewOuter, UMaterialInterface::StaticClass(), BaseName);
	return DuplicateObject<UMaterialInterface>(&InMaterialInterface, &NewOuter, NewName);
}

bool FISMComponentDescriptorBase::operator!=(const FISMComponentDescriptorBase& Other) const
{
	return !(*this == Other);
}

bool FISMComponentDescriptor::operator!=(const FISMComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FSoftISMComponentDescriptor::operator!=(const FSoftISMComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FISMComponentDescriptorBase::operator==(const FISMComponentDescriptorBase& Other) const
{
	return ComponentClass == Other.ComponentClass &&
	Mobility == Other.Mobility &&
	VirtualTextureRenderPassType == Other.VirtualTextureRenderPassType &&
	LightmapType == Other.LightmapType &&
	GetLightingChannelMaskForStruct(LightingChannels) == GetLightingChannelMaskForStruct(Other.LightingChannels) &&
	RayTracingGroupId == Other.RayTracingGroupId &&
	RayTracingGroupCullingPriority == Other.RayTracingGroupCullingPriority &&
	bHasCustomNavigableGeometry == Other.bHasCustomNavigableGeometry &&
	CustomDepthStencilWriteMask == Other.CustomDepthStencilWriteMask &&
	InstanceMinDrawDistance == Other.InstanceMinDrawDistance &&
	InstanceStartCullDistance == Other.InstanceStartCullDistance &&
	InstanceEndCullDistance == Other.InstanceEndCullDistance &&
	VirtualTextureCullMips == Other.VirtualTextureCullMips &&
	TranslucencySortPriority == Other.TranslucencySortPriority &&
	OverriddenLightMapRes == Other.OverriddenLightMapRes &&
	CustomDepthStencilValue == Other.CustomDepthStencilValue &&
	bCastShadow == Other.bCastShadow &&
	bEmissiveLightSource == Other.bEmissiveLightSource &&
	bAffectDynamicIndirectLighting == Other.bAffectDynamicIndirectLighting &&
	bAffectDynamicIndirectLightingWhileHidden == Other.bAffectDynamicIndirectLightingWhileHidden &&
	bAffectDistanceFieldLighting == Other.bAffectDistanceFieldLighting &&
	bCastDynamicShadow == Other.bCastDynamicShadow &&
	bCastStaticShadow == Other.bCastStaticShadow &&
	bCastVolumetricTranslucentShadow == Other.bCastVolumetricTranslucentShadow &&
	bCastContactShadow == Other.bCastContactShadow &&
	bSelfShadowOnly == Other.bSelfShadowOnly &&
	bCastFarShadow == Other.bCastFarShadow &&
	bCastInsetShadow == Other.bCastInsetShadow &&
	bCastCinematicShadow == Other.bCastCinematicShadow &&
	bCastHiddenShadow == Other.bCastHiddenShadow &&
	bCastShadowAsTwoSided == Other.bCastShadowAsTwoSided &&
	bReceivesDecals == Other.bReceivesDecals &&
	bOverrideLightMapRes == Other.bOverrideLightMapRes &&
	bUseAsOccluder == Other.bUseAsOccluder &&
	bRenderCustomDepth == Other.bRenderCustomDepth &&
	bEnableDiscardOnLoad == Other.bEnableDiscardOnLoad &&
	bHiddenInGame == Other.bHiddenInGame &&
	bIsEditorOnly == Other.bIsEditorOnly &&
	bVisible == Other.bVisible &&
	bVisibleInRayTracing == Other.bVisibleInRayTracing &&
	bVisibleInReflections == Other.bVisibleInReflections &&
	bRenderInMainPass == Other.bRenderInMainPass &&
	bRenderInDepthPass == Other.bRenderInDepthPass &&
	bSelectable == Other.bSelectable &&
	bHasPerInstanceHitProxies == Other.bHasPerInstanceHitProxies &&
	bEvaluateWorldPositionOffset == Other.bEvaluateWorldPositionOffset &&
	bReverseCulling == Other.bReverseCulling &&
	bUseGpuLodSelection == Other.bUseGpuLodSelection &&
	bUseDefaultCollision == Other.bUseDefaultCollision &&
	bGenerateOverlapEvents == Other.bGenerateOverlapEvents &&
	bOverrideNavigationExport == Other.bOverrideNavigationExport &&
	bForceNavigationObstacle == Other.bForceNavigationObstacle &&
	bCanEverAffectNavigation == Other.bCanEverAffectNavigation &&
	bFillCollisionUnderneathForNavmesh == Other.bFillCollisionUnderneathForNavmesh &&
	bRasterizeAsFilledConvexVolume == Other.bRasterizeAsFilledConvexVolume &&
	WorldPositionOffsetDisableDistance == Other.WorldPositionOffsetDisableDistance &&
	NanitePixelProgrammableDistance == Other.NanitePixelProgrammableDistance &&
	ShadowCacheInvalidationBehavior == Other.ShadowCacheInvalidationBehavior &&
	DetailMode == Other.DetailMode &&
#if WITH_EDITORONLY_DATA
	HLODBatchingPolicy == Other.HLODBatchingPolicy &&
	bIncludeInHLOD == Other.bIncludeInHLOD &&
	bConsiderForActorPlacementWhenHidden == Other.bConsiderForActorPlacementWhenHidden &&
#endif // WITH_EDITORONLY_DATA
	BodyInstance.GetCollisionEnabled() == Other.BodyInstance.GetCollisionEnabled() && 
	BodyInstance.GetCollisionResponse() == Other.BodyInstance.GetCollisionResponse() &&
	BodyInstance.DoesUseCollisionProfile() == Other.BodyInstance.DoesUseCollisionProfile() &&
	(!BodyInstance.DoesUseCollisionProfile() || (BodyInstance.GetCollisionProfileName() == Other.BodyInstance.GetCollisionProfileName()));
}

bool FISMComponentDescriptor::operator==(const FISMComponentDescriptor& Other) const
{	
	return (Hash == 0 || Other.Hash == 0 || Hash == Other.Hash) && // Check hash first, other checks are in case of Hash collision
		StaticMesh == Other.StaticMesh &&
		OverrideMaterials == Other.OverrideMaterials &&
		OverlayMaterial == Other.OverlayMaterial &&
		RuntimeVirtualTextures == Other.RuntimeVirtualTextures &&
		LocalTransform.Equals(Other.LocalTransform) &&
		Super::operator==(Other);
}

bool FSoftISMComponentDescriptor::operator==(const FSoftISMComponentDescriptor& Other) const
{
	return (Hash == 0 || Other.Hash == 0 || Hash == Other.Hash) && // Check hash first, other checks are in case of Hash collision
		StaticMesh == Other.StaticMesh &&
		OverrideMaterials == Other.OverrideMaterials &&
		OverlayMaterial == Other.OverlayMaterial &&
		RuntimeVirtualTextures == Other.RuntimeVirtualTextures &&
		Super::operator==(Other);
}

uint32 FISMComponentDescriptorBase::ComputeHash() const
{
	FArchiveCrc32 CrcArchive;

	Hash = 0; // we don't want the hash to impact the calculation
	CrcArchive << *this;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

uint32 FISMComponentDescriptor::ComputeHash() const
{
	Super::ComputeHash();

	FISMComponentDescriptor& MutableSelf = *const_cast<FISMComponentDescriptor*>(this);
	FArchiveCrc32 CrcArchive(Hash);
	CrcArchive << MutableSelf.StaticMesh;
	CrcArchive << MutableSelf.OverrideMaterials;
	CrcArchive << MutableSelf.OverlayMaterial;
	CrcArchive << MutableSelf.RuntimeVirtualTextures;
	CrcArchive << MutableSelf.LocalTransform;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

uint32 FSoftISMComponentDescriptor::ComputeHash() const
{
	Super::ComputeHash();

	FSoftISMComponentDescriptor& MutableSelf = *const_cast<FSoftISMComponentDescriptor*>(this);
	FArchiveCrc32 CrcArchive(Hash);
	CrcArchive << MutableSelf.StaticMesh;
	CrcArchive << MutableSelf.OverrideMaterials;
	CrcArchive << MutableSelf.OverlayMaterial;
	CrcArchive << MutableSelf.RuntimeVirtualTextures;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

UInstancedStaticMeshComponent* FISMComponentDescriptorBase::CreateComponent(UObject* Outer, FName Name, EObjectFlags ObjectFlags) const
{
	UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(Outer, ComponentClass, Name, ObjectFlags);
	
	InitComponent(ISMComponent);

	return ISMComponent;
}

void FISMComponentDescriptorBase::InitComponent(UInstancedStaticMeshComponent* ISMComponent) const
{
	ISMComponent->Mobility = Mobility;
	ISMComponent->VirtualTextureRenderPassType = VirtualTextureRenderPassType;
	ISMComponent->SetLightmapType(LightmapType);
	ISMComponent->LightingChannels = LightingChannels;
	ISMComponent->RayTracingGroupId = RayTracingGroupId;
	ISMComponent->RayTracingGroupCullingPriority = RayTracingGroupCullingPriority;
	ISMComponent->bHasCustomNavigableGeometry = bHasCustomNavigableGeometry;
	ISMComponent->CustomDepthStencilWriteMask = CustomDepthStencilWriteMask;
	ISMComponent->BodyInstance.CopyBodyInstancePropertiesFrom(&BodyInstance);
	ISMComponent->InstanceMinDrawDistance = InstanceMinDrawDistance;
	ISMComponent->InstanceStartCullDistance = InstanceStartCullDistance;
	ISMComponent->InstanceEndCullDistance = InstanceEndCullDistance;
	ISMComponent->VirtualTextureCullMips = VirtualTextureCullMips;
	ISMComponent->TranslucencySortPriority = TranslucencySortPriority;
	ISMComponent->OverriddenLightMapRes = OverriddenLightMapRes;
	ISMComponent->CustomDepthStencilValue = CustomDepthStencilValue;
	ISMComponent->CastShadow = bCastShadow;
	ISMComponent->bEmissiveLightSource = bEmissiveLightSource;
	ISMComponent->bAffectDynamicIndirectLighting = bAffectDynamicIndirectLighting;
	ISMComponent->bAffectIndirectLightingWhileHidden = bAffectDynamicIndirectLightingWhileHidden;
	ISMComponent->bAffectDistanceFieldLighting = bAffectDistanceFieldLighting;
	ISMComponent->bCastDynamicShadow = bCastDynamicShadow;
	ISMComponent->bCastStaticShadow = bCastStaticShadow;
	ISMComponent->bCastVolumetricTranslucentShadow = bCastVolumetricTranslucentShadow;
	ISMComponent->bCastContactShadow = bCastContactShadow;
	ISMComponent->bSelfShadowOnly = bSelfShadowOnly;
	ISMComponent->bCastFarShadow = bCastFarShadow;
	ISMComponent->bCastInsetShadow = bCastInsetShadow;
	ISMComponent->bCastCinematicShadow = bCastCinematicShadow;
	ISMComponent->bCastHiddenShadow = bCastHiddenShadow;
	ISMComponent->bCastShadowAsTwoSided = bCastShadowAsTwoSided;	
	ISMComponent->bReceivesDecals = bReceivesDecals;
	ISMComponent->bOverrideLightMapRes = bOverrideLightMapRes;
	ISMComponent->bUseAsOccluder = bUseAsOccluder;
	ISMComponent->bRenderCustomDepth = bRenderCustomDepth;
	ISMComponent->bHiddenInGame = bHiddenInGame;
	ISMComponent->bIsEditorOnly = bIsEditorOnly;
	ISMComponent->bRenderInMainPass = bRenderInMainPass;
	ISMComponent->bRenderInDepthPass = bRenderInDepthPass;
	ISMComponent->SetVisibleFlag(bVisible);
	ISMComponent->bVisibleInRayTracing = bVisibleInRayTracing;
	ISMComponent->bVisibleInReflections = bVisibleInReflections;
	ISMComponent->bSelectable = bSelectable;
	ISMComponent->bHasPerInstanceHitProxies = bHasPerInstanceHitProxies;
	ISMComponent->bEvaluateWorldPositionOffset = bEvaluateWorldPositionOffset;
	ISMComponent->bReverseCulling = bReverseCulling;
	ISMComponent->bUseGpuLodSelection = bUseGpuLodSelection;
	ISMComponent->bUseDefaultCollision = bUseDefaultCollision;
	ISMComponent->SetGenerateOverlapEvents(bGenerateOverlapEvents);
	ISMComponent->bOverrideNavigationExport = bOverrideNavigationExport;
	ISMComponent->bForceNavigationObstacle = bForceNavigationObstacle;
	ISMComponent->bFillCollisionUnderneathForNavmesh = bFillCollisionUnderneathForNavmesh;
	ISMComponent->bRasterizeAsFilledConvexVolume = bRasterizeAsFilledConvexVolume;
	ISMComponent->WorldPositionOffsetDisableDistance = WorldPositionOffsetDisableDistance;
	ISMComponent->NanitePixelProgrammableDistance = NanitePixelProgrammableDistance;
	ISMComponent->ShadowCacheInvalidationBehavior = ShadowCacheInvalidationBehavior;
	ISMComponent->DetailMode = DetailMode;
	
	if (GISMComponentDescriptorCopiesCanEverAffectNavigation)
	{
		ISMComponent->SetCanEverAffectNavigation(bCanEverAffectNavigation);
	}

	#if WITH_EDITORONLY_DATA
	ISMComponent->HLODBatchingPolicy = HLODBatchingPolicy;
	ISMComponent->bEnableAutoLODGeneration = bIncludeInHLOD;
	ISMComponent->bConsiderForActorPlacementWhenHidden = bConsiderForActorPlacementWhenHidden;
#endif // WITH_EDITORONLY_DATA

	// HISM Specific
	if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(ISMComponent))
	{
		HISMComponent->bEnableDensityScaling = bEnableDensityScaling;
		HISMComponent->InstanceLODDistanceScale = InstanceLODDistanceScale;
	}
}

void FISMComponentDescriptor::InitComponent(UInstancedStaticMeshComponent* ISMComponent) const
{
	ISMComponent->SetStaticMesh(StaticMesh);

	auto GetMaterial = [ISMComponent](UMaterialInterface* MaterialInterface)
	{
		if (MaterialInterface && !MaterialInterface->IsAsset())
		{
			// If the material is equivalent to its parent, just take a reference to its parent rather than making another redundant object 
			if (UMaterialInstance* Instance = Cast<UMaterialInstance>(MaterialInterface); Instance && Instance->IsRedundant())
			{
				MaterialInterface = Instance->Parent;
			}
			else
			{
				// As override materials are normally outered to their owner component, we need to duplicate them here to make sure we don't create
				// references to actors in other levels (for packed level instances or HLOD actors).
				MaterialInterface = DuplicateMaterialInterface(*MaterialInterface, *ISMComponent);

				// If the MID we just duplicated has a nanite override that's also not an asset, duplicate that too
				UMaterialInstanceDynamic* OverrideMID = Cast<UMaterialInstanceDynamic>(MaterialInterface);
				UMaterialInterface* NaniteOverride = OverrideMID ? OverrideMID->GetNaniteOverride() : nullptr;
				if (NaniteOverride && !NaniteOverride->IsAsset())
				{
					OverrideMID->SetNaniteOverride(DuplicateMaterialInterface(*NaniteOverride, *ISMComponent));
				}
			}
		}

		return MaterialInterface;
	};

	ISMComponent->OverrideMaterials.Empty(OverrideMaterials.Num());
	for (UMaterialInterface* OverrideMaterial : OverrideMaterials)
	{
		ISMComponent->OverrideMaterials.Add(GetMaterial(OverrideMaterial));
	}
	ISMComponent->OverlayMaterial = GetMaterial(OverlayMaterial);
	ISMComponent->RuntimeVirtualTextures = RuntimeVirtualTextures;

	Super::InitComponent(ISMComponent);
}

void FSoftISMComponentDescriptor::InitComponent(UInstancedStaticMeshComponent* ISMComponent) const
{
	ISMComponent->SetStaticMesh(StaticMesh.LoadSynchronous());

	auto GetMaterial = [ISMComponent](const TSoftObjectPtr<UMaterialInterface>& MaterialInterfacePtr)
	{
		UMaterialInterface* MaterialInterface = MaterialInterfacePtr.LoadSynchronous();
		if (MaterialInterface && !MaterialInterface->IsAsset())
		{
			// If the material is equivalent to its parent, just take a reference to its parent rather than making another redundant object 
			if (UMaterialInstance* Instance = Cast<UMaterialInstance>(MaterialInterface); Instance && Instance->IsRedundant())
			{
				MaterialInterface = Instance->Parent;
			}
			else
			{
				// As override materials are normally outered to their owner component, we need to duplicate them here to make sure we don't create
				// references to actors in other levels (for packed level instances or HLOD actors).
				MaterialInterface = DuplicateMaterialInterface(*MaterialInterface, *ISMComponent);

				// If the MID we just duplicated has a nanite override that's also not an asset, duplicate that too
				UMaterialInstanceDynamic* OverrideMID = Cast<UMaterialInstanceDynamic>(MaterialInterface);
				UMaterialInterface* NaniteOverride = OverrideMID ? OverrideMID->GetNaniteOverride() : nullptr; 
				if (NaniteOverride && !NaniteOverride->IsAsset())
				{
					OverrideMID->SetNaniteOverride(DuplicateMaterialInterface(*NaniteOverride, *ISMComponent));
				}
			}
		}
		
		return MaterialInterface;
	};

	ISMComponent->OverrideMaterials.Empty(OverrideMaterials.Num());
	for (const TSoftObjectPtr<UMaterialInterface>& OverrideMaterialPtr : OverrideMaterials)
	{
		ISMComponent->OverrideMaterials.Add(GetMaterial(OverrideMaterialPtr));
	}
	ISMComponent->OverlayMaterial = GetMaterial(OverlayMaterial);

	ISMComponent->RuntimeVirtualTextures.Empty(RuntimeVirtualTextures.Num());
	for (const TSoftObjectPtr<URuntimeVirtualTexture>& RuntimeVirtualTexturePtr : RuntimeVirtualTextures)
	{
		if (URuntimeVirtualTexture* RuntimeVirtualTexture = RuntimeVirtualTexturePtr.LoadSynchronous())
		{
			ISMComponent->RuntimeVirtualTextures.Add(RuntimeVirtualTexture);
		}
	}

	Super::InitComponent(ISMComponent);
}
