// Copyright Epic Games, Inc. All Rights Reserved.
#include "ISMPartition/ISMPartitionActor.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "EngineUtils.h"
#include "ISMPartition/ISMPartitionClient.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMPartitionActor)

#if WITH_EDITOR
#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY(LogISMPartition);

AISMPartitionActor::AISMPartitionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, UpdateDepth(0)
	, bWasModifyCalled(false)
#endif
{
#if WITH_EDITORONLY_DATA
	bListedInSceneOutliner = false;
#endif
}

#if WITH_EDITOR
static FAutoConsoleCommand DumpISMPartitionActors(
	TEXT("ism.Editor.DumpISMPartitionActors"),
	TEXT("Output stats about ISMPartitionActor(s)"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			if (!World->IsPlayInEditor())
			{
				for (TActorIterator<AISMPartitionActor> It(World); It; ++It)
				{
					AISMPartitionActor* ISMPartitionActor = *It;
					ISMPartitionActor->OutputStats();
				}

				// Also output total ism count
				int32 TotalCount = 0;
				int32 ComponentCount = 0;
				for (TObjectIterator<UInstancedStaticMeshComponent> It(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage); It; ++It)
				{
					UInstancedStaticMeshComponent* ISMComponent = *It;
					if (ISMComponent)
					{
						TotalCount += ISMComponent->GetInstanceCount();
						ComponentCount++;
					}
				}
				UE_LOGF(LogISMPartition, Log, "ISM Component Count: %d Instance Count: %d", ComponentCount, TotalCount);
			}
		}
	})
);


void AISMPartitionActor::OutputStats() const
{
	UE_LOGF(LogISMPartition, Log, "ISM Partition: %ls (%ls)", *GetActorLabel(), *GetPathName());
	int32 TotalCount = 0;
	for (int32 DescriptorIndex = 0; DescriptorIndex < Descriptors.Num(); ++DescriptorIndex)
	{
		int32 Count = DescriptorComponents[DescriptorIndex].Instances.Num();
		if (Count > 0)
		{
			UE_LOGF(LogISMPartition, Log, "StaticMesh: %ls Instance Count: %d", *Descriptors[DescriptorIndex].StaticMesh->GetPathName(), Count);
		}
		TotalCount += Count;
	}
	UE_LOGF(LogISMPartition, Log, "Total Instance Count: %d", TotalCount);
}

void AISMPartitionActor::PreEditUndo()
{
	Super::PreEditUndo();

	for (FISMComponentData& ComponentData : DescriptorComponents)
	{
		ComponentData.UnregisterDelegates();
	}
}

void AISMPartitionActor::PostEditUndo()
{
	Super::PostEditUndo();

	for (FISMComponentData& ComponentData : DescriptorComponents)
	{
		ComponentData.RegisterDelegates();
	}
}

FISMClientHandle AISMPartitionActor::RegisterClient(const FGuid& ClientGuid)
{
	ModifyActor();
	int32 Index = Clients.Find(ClientGuid);
	if (Index == INDEX_NONE)
	{
		Index = Clients.Find(FGuid());
		if (Index == INDEX_NONE)
		{
			Index = Clients.Num();
			Clients.Add(ClientGuid);
		}
		else // Fill hole
		{
			Clients[Index] = ClientGuid;
		}
	}
	return FISMClientHandle(Index, ClientGuid);
}

void AISMPartitionActor::UnregisterClient(FISMClientHandle& Handle)
{
	ModifyActor();
	check(Handle.Guid == Clients[Handle.Index]);
	RemoveISMInstances(Handle);
	
	ClientInstanceManagers.Remove(Handle.Guid);

	int32 ClientIndex = Handle.Index;
	Clients[Handle.Index] = FGuid();
	Handle.Index = -1;
	Handle.Guid = FGuid();

	if (ClientIndex == Clients.Num() - 1)
	{
		Clients.RemoveAt(ClientIndex);
	}
}

void AISMPartitionActor::RegisterClientInstanceManager(const FISMClientHandle& Handle, IISMPartitionInstanceManager* ClientInstanceManager)
{
	check(Handle.Guid == Clients[Handle.Index]);

	FISMClientInstanceManagerData& ClientInstanceManagerData = ClientInstanceManagers.FindOrAdd(Handle.Guid);
	ClientInstanceManagerData.SetInstanceManager(ClientInstanceManager);
}

void AISMPartitionActor::RegisterClientInstanceManagerProvider(const FISMClientHandle& Handle, IISMPartitionInstanceManagerProvider* ClientInstanceManagerProvider)
{
	check(Handle.Guid == Clients[Handle.Index]);

	FISMClientInstanceManagerData& ClientInstanceManagerData = ClientInstanceManagers.FindOrAdd(Handle.Guid);
	ClientInstanceManagerData.SetInstanceManagerProvider(ClientInstanceManagerProvider);
}

int32 AISMPartitionActor::RegisterISMComponentDescriptor(const FISMComponentDescriptor& Descriptor)
{
	check(Descriptor.Hash != 0);
	ModifyActor();
	int32 Index = Descriptors.IndexOfByPredicate([&](const FISMComponentDescriptor& ExistingDescriptor) { return ExistingDescriptor == Descriptor; });
	if (Index == INDEX_NONE)
	{
		Index = Descriptors.IndexOfByPredicate([&](const FISMComponentDescriptor& ExistingDescriptor) { return ExistingDescriptor.Hash == 0; });
		if (Index == INDEX_NONE)
		{
			check(Descriptors.Num() == DescriptorComponents.Num());
			Index = Descriptors.Num();
			Descriptors.Add(Descriptor);
			DescriptorComponents.Add(FISMComponentData());
		}
		else
		{
			Descriptors[Index] = Descriptor;
			check(DescriptorComponents[Index].Instances.Num() == 0);
		}
	}
	return Index;
}

void AISMPartitionActor::ReserveISMInstances(const FISMClientHandle& Handle, int32 AddedInstanceCount, const TSortedMap<int32, TArray<FTransform>>& InstanceDefinition)
{
	check(Handle.Guid == Clients[Handle.Index]);
	BeginUpdate();
	ModifyActor();
	for (const auto& Pair : InstanceDefinition)
	{
		const int32 DescriptorIndex = Pair.Key;

		FISMComponentData& ComponentData = DescriptorComponents[DescriptorIndex];
		if (!ComponentData.Component)
		{
			CreateComponent(Descriptors[DescriptorIndex], ComponentData);
		}
		ModifyComponent(ComponentData);
		ComponentData.Component->PreAllocateInstancesMemory(AddedInstanceCount);

		if (ComponentData.ClientInstances.Num() <= Handle.Index)
		{
			ComponentData.ClientInstances.AddDefaulted(Handle.Index - ComponentData.ClientInstances.Num() + 1);
		}

		FISMClientData& ISMClientData = ComponentData.ClientInstances[Handle.Index];
		ISMClientData.Instances.Reserve(ISMClientData.Instances.Num() + AddedInstanceCount);
	}
	EndUpdate();
}

TArray<FSMInstanceId> AISMPartitionActor::AddISMInstance(const FISMClientHandle& Handle, const FTransform& InstanceTransform, const TSortedMap<int32, TArray<FTransform>>& InstanceDefinition)
{
	check(Handle.Guid == Clients[Handle.Index]);
	// Create Descriptor entry
	BeginUpdate();
	ModifyActor();

	TArray<FSMInstanceId> AddedInstances;
	for (const auto& Pair : InstanceDefinition)
	{
		const int32 DescriptorIndex = Pair.Key;
		
		FISMComponentData& ComponentData = DescriptorComponents[DescriptorIndex];
		if (!ComponentData.Component)
		{
			CreateComponent(Descriptors[DescriptorIndex], ComponentData);
		}

		if (ComponentData.ClientInstances.Num() <= Handle.Index)
		{
			ComponentData.ClientInstances.AddDefaulted(Handle.Index - ComponentData.ClientInstances.Num() + 1);
		}

		FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
		int32 ClientIndex = ClientData.Instances.Num();
		FISMClientInstance& ClientInstance = ClientData.Instances.Emplace_GetRef();
		ClientInstance.ComponentIndices.Reserve(Pair.Value.Num());
		ComponentData.Instances.Reserve(ComponentData.Instances.Num() + Pair.Value.Num());

		// Add Instances and creating mappings for faster removal
		for (const FTransform& LocalTransform : Pair.Value)
		{
			int32 ComponentInstanceIndex = ComponentData.Instances.Num();
			ComponentData.Instances.Emplace(Handle.Index, ClientIndex, ClientInstance.ComponentIndices.Num());			
			ClientInstance.ComponentIndices.Add(ComponentInstanceIndex);
			
			FTransform NewInstanceTransform = LocalTransform * InstanceTransform;

			int32 NewInstanceIdx = AddInstanceToComponent(ComponentData, NewInstanceTransform);
			if (NewInstanceIdx != INDEX_NONE)
			{
				AddedInstances.Emplace(FSMInstanceId{ ComponentData.Component, NewInstanceIdx });
			}
		}
	}

	EndUpdate();

	return AddedInstances;
}

void AISMPartitionActor::RemoveISMInstances(const FISMClientHandle& Handle)
{
	check(Handle.Guid == Clients[Handle.Index]);
	BeginUpdate();
	
	for (int32 DescriptorIndex = Descriptors.Num()-1; DescriptorIndex >= 0; --DescriptorIndex)
	{
		FISMComponentData& ComponentData = DescriptorComponents[DescriptorIndex];
		if (ComponentData.ClientInstances.IsValidIndex(Handle.Index))
		{
			FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
			for (int32 InstanceIndex = ClientData.Instances.Num() - 1; InstanceIndex >= 0; --InstanceIndex)
			{
				RemoveISMInstancesInternal(ComponentData, ClientData, InstanceIndex);
			}
			ClientData.Instances.Empty(); // Free the memory
		}

		bool bIsEmpty = DestroyComponentIfEmpty(Descriptors[DescriptorIndex], ComponentData);
		// If Component is empty and its the last descriptor in the list we can remove it without breaking indices
		if (bIsEmpty && DescriptorIndex == Descriptors.Num() - 1)
		{
			Descriptors.RemoveAt(DescriptorIndex);
			DescriptorComponents.RemoveAt(DescriptorIndex);
		}
	}
		
	EndUpdate();
}

bool AISMPartitionActor::DestroyComponentIfEmpty(FISMComponentDescriptor& Descriptor, FISMComponentData& ComponentData)
{
	if (ComponentData.Component && !ComponentData.Component->GetInstanceCount())
	{
		ModifyActor();
		Descriptor.Hash = 0; // FreeList
		ComponentData.Instances.Empty();
		ComponentData.UnregisterDelegates();
		ComponentData.Component->Modify();
		ComponentData.Component->DestroyComponent();
		ComponentData.Component = nullptr;
		return true;
	}

	return false;
}

void AISMPartitionActor::InvalidateComponentLightingCache(FISMComponentData& ComponentData)
{
	if (UpdateDepth)
	{
		ComponentData.bInvalidateLightingCache = true;
	}
	else
	{
		ComponentData.Component->InvalidateLightingCache();
		ComponentData.bInvalidateLightingCache = false;
	}
}

void AISMPartitionActor::RemoveInstanceFromComponent(FISMComponentData& ComponentData, int32 ComponentInstanceIndex)
{
	if (ComponentData.Component)
	{
		ModifyComponent(ComponentData);
		ComponentData.Component->RemoveInstance(ComponentInstanceIndex);
		InvalidateComponentLightingCache(ComponentData);
	}
}

int32 AISMPartitionActor::AddInstanceToComponent(FISMComponentData& ComponentData, const FTransform& WorldTransform)
{
	check(ComponentData.Component);
	ModifyComponent(ComponentData);
	int32 NewInstanceIdx = ComponentData.Component->AddInstance(WorldTransform, /*bWorldSpace*/true);
	InvalidateComponentLightingCache(ComponentData);
	return NewInstanceIdx;
}

void AISMPartitionActor::UpdateInstanceTransform(FISMComponentData& ComponentData, int32 ComponentInstanceIndex, const FTransform& WorldTransform, bool bTeleport)
{
	check(ComponentData.Component);
	ModifyComponent(ComponentData);
	ComponentData.Component->UpdateInstanceTransform(ComponentInstanceIndex, WorldTransform, true, true, bTeleport);
	InvalidateComponentLightingCache(ComponentData);
}

void AISMPartitionActor::ModifyComponent(FISMComponentData& ComponentData)
{
	check(ComponentData.Component);
	if (!ComponentData.bWasModifyCalled || UpdateDepth == 0)
	{
		// Only store the bModfied inside a BeginUpdate/EndUpdate
		ComponentData.bWasModifyCalled = UpdateDepth > 0;
		ComponentData.Component->Modify();
	}
}

void AISMPartitionActor::CreateComponent(const FISMComponentDescriptor& ComponentDescriptor, FISMComponentData& ComponentData)
{
	ModifyActor();

	check(!ComponentData.Component);
	ComponentData.Component = ComponentDescriptor.CreateComponent(this, NAME_None, RF_Transactional);
	ComponentData.Component->bSelectable = true;
	ComponentData.Component->bHasPerInstanceHitProxies = true;
	ComponentData.RegisterDelegates();
	AddInstanceComponent(ComponentData.Component);

	ComponentData.Component->SetupAttachment(RootComponent);

	if (GetRootComponent()->IsRegistered())
	{
		ComponentData.Component->RegisterComponent();
	}

	// Use only instance translation as a component transform
	ComponentData.Component->SetWorldTransform(RootComponent->GetComponentTransform());

	// Add the new component to the transaction buffer so it will get destroyed on undo
	ComponentData.Component->Modify();

	check(UpdateDepth > 0);
	if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(ComponentData.Component))
	{
		ComponentData.bAutoRebuildTreeOnInstanceChanges = HISMComponent->bAutoRebuildTreeOnInstanceChanges;
		HISMComponent->bAutoRebuildTreeOnInstanceChanges = false;
	}
}

void AISMPartitionActor::ModifyActor()
{
	if (!bWasModifyCalled || UpdateDepth == 0)
	{
		bWasModifyCalled = UpdateDepth > 0;
		Modify();
	}
}

void AISMPartitionActor::RemoveISMInstancesInternal(FISMComponentData& ComponentData, FISMClientData& ClientData, int32 InstanceIndex)
{
	if (!ClientData.Instances.Num())
	{
		return;
	}

	BeginUpdate();
	ModifyActor();
		
	check(ComponentData.Component);
	const bool bRemoveSwap = ComponentData.Component->SupportsRemoveSwap();
	FISMClientInstance& ClientInstance = ClientData.Instances[InstanceIndex];
	for (int32 ComponentInstanceIndex : ClientInstance.ComponentIndices)
	{
		bool OutIsRemoveAtSwap = false;
		RemoveInstanceFromComponent(ComponentData, ComponentInstanceIndex);

		if (bRemoveSwap)
		{
			ComponentData.Instances.RemoveAtSwap(ComponentInstanceIndex, EAllowShrinking::No);

			// Make sure we have the proper index (if we removed the last element, or the only element)
			ComponentInstanceIndex = FMath::Min(ComponentInstanceIndex, ComponentData.Instances.Num() - 1);
			if (ComponentInstanceIndex >= 0)
			{
				// Fix up because of swapping
				const FISMComponentInstance& ComponentInstance = ComponentData.Instances[ComponentInstanceIndex];
				FISMClientData& SwapClientData = ComponentData.ClientInstances[ComponentInstance.ClientIndex];
				SwapClientData.Instances[ComponentInstance.InstanceIndex].ComponentIndices[ComponentInstance.InstanceSubIndex] = ComponentInstanceIndex;
			}
		}
		else
		{
			ComponentData.Instances.RemoveAt(ComponentInstanceIndex);

			// No swap means we need to update all instances after the removed instance
			for (int32 IndexToUpdate = ComponentInstanceIndex; IndexToUpdate < ComponentData.Instances.Num(); ++IndexToUpdate)
			{
				const FISMComponentInstance& ComponentInstance = ComponentData.Instances[IndexToUpdate];
				FISMClientData& SwapClientData = ComponentData.ClientInstances[ComponentInstance.ClientIndex];
				SwapClientData.Instances[ComponentInstance.InstanceIndex].ComponentIndices[ComponentInstance.InstanceSubIndex] = IndexToUpdate;
			}
		}
	}

	ClientData.Instances.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
	if (ClientData.Instances.IsValidIndex(InstanceIndex))
	{
		FISMClientInstance& SwapClientInstance = ClientData.Instances[InstanceIndex];
		for (int32 ComponentInstanceIndex : SwapClientInstance.ComponentIndices)
		{
			FISMComponentInstance& ComponentInstance = ComponentData.Instances[ComponentInstanceIndex];
			ComponentInstance.InstanceIndex = InstanceIndex;
		}
	}

	EndUpdate();
}

void AISMPartitionActor::RemoveISMInstance(const FISMClientHandle& Handle, int32 InstanceIndex, bool* bOutIsEmpty)
{
	check(Handle.Guid == Clients[Handle.Index]);
	TOptional<bool> bIsEmpty;
	for (FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (ComponentData.ClientInstances.IsValidIndex(Handle.Index))
		{
			FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
			if (ClientData.Instances.Num())
			{
				RemoveISMInstancesInternal(ComponentData, ClientData, InstanceIndex);

				if (!bIsEmpty.IsSet())
				{
					bIsEmpty = ClientData.Instances.IsEmpty();
				}
				else
				{
					// Client instances should match across components
					check(bIsEmpty == ClientData.Instances.IsEmpty());
				}
			}
		}
	}

	if (bOutIsEmpty)
	{
		*bOutIsEmpty = !bIsEmpty.IsSet() || bIsEmpty.GetValue();
	}
}

void AISMPartitionActor::SelectISMInstances(const FISMClientHandle& Handle, bool bSelect, const TSet<int32>& Indices)
{
	check(Handle.Guid == Clients[Handle.Index]);
	if (Indices.Num())
	{
		for (FISMComponentData& ComponentData : DescriptorComponents)
		{
			if (ComponentData.ClientInstances.IsValidIndex(Handle.Index))
			{
				FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
				if (ClientData.Instances.Num())
				{
					for (int32 SelectIndex : Indices)
					{
						FISMClientInstance& ClientInstance = ClientData.Instances[SelectIndex];
						for (int32 ComponentInstanceIndex : ClientInstance.ComponentIndices)
						{
							ComponentData.Component->SelectInstance(bSelect, ComponentInstanceIndex);
						}
					}
				}
			}
		}
	}
}

void AISMPartitionActor::SetISMInstanceTransform(const FISMClientHandle& Handle, int32 InstanceIndex, const FTransform& NewTransform, bool bTeleport, const TSortedMap<int32, TArray<FTransform>>& InstanceDefinition)
{
	check(Handle.Guid == Clients[Handle.Index]);
	BeginUpdate();

	// We don't store the previous transform so for now we need the Definition to compute the new world positions
	for (const auto& Pair : InstanceDefinition)
	{
		const int32 DescriptorIndex = Pair.Key;

		FISMComponentData& ComponentData = DescriptorComponents[DescriptorIndex];
		FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
		FISMClientInstance& ClientInstance = ClientData.Instances[InstanceIndex];
		check(ClientInstance.ComponentIndices.Num() == Pair.Value.Num());

		// Add Instances and creating mappings for faster removal
		for (int32 Index = 0; Index < ClientInstance.ComponentIndices.Num(); ++Index)
		{
			const FTransform& LocalTransform = Pair.Value[Index];
			
			int32 ComponentInstanceIndex = ClientInstance.ComponentIndices[Index];

			UpdateInstanceTransform(ComponentData, ComponentInstanceIndex, LocalTransform * NewTransform, bTeleport);
		}
	}

	EndUpdate();
}

bool AISMPartitionActor::IsISMComponent(const UPrimitiveComponent* Component) const
{
	for (const FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (ComponentData.Component == Component)
		{
			return true;
		}
	}

	return false;
}

int32 AISMPartitionActor::GetISMInstanceIndex(const FISMClientHandle& Handle, const UInstancedStaticMeshComponent* ISMComponent, int32 ComponentIndex) const
{
	check(Handle.Guid == Clients[Handle.Index]);
	for (const FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (ComponentData.Component == ISMComponent)
		{
			const FISMComponentInstance& ComponentInstance = ComponentData.Instances[ComponentIndex];
			if (ComponentInstance.ClientIndex == Handle.Index)
			{
				return ComponentInstance.InstanceIndex;
			}
			
			break;
		}
	}

	return INDEX_NONE;
}

FBox AISMPartitionActor::GetISMInstanceBounds(const FISMClientHandle& Handle, const TSet<int32>& Indices) const
{
	check(Handle.Guid == Clients[Handle.Index]);
	FBox BoundingBox(EForceInit::ForceInit);
	FTransform InstanceWorldTransform;
	for (int32 i : Indices)
	{
		for (const FISMComponentData& ComponentData : DescriptorComponents)
		{
			if (ComponentData.Component && ComponentData.Component->GetStaticMesh())
			{
				const FBox StaticMeshBoundingBox = ComponentData.Component->GetStaticMesh()->GetBoundingBox();
				const FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
				for (const FISMClientInstance& ClientInstance : ClientData.Instances)
				{
					for (int32 ComponentInstanceIndex : ClientInstance.ComponentIndices)
					{
						ComponentData.Component->GetInstanceTransform(ComponentInstanceIndex, InstanceWorldTransform, true);
						BoundingBox += StaticMeshBoundingBox.TransformBy(InstanceWorldTransform);
					}
				}
			}
		}

		
	}
	return BoundingBox;
}

void AISMPartitionActor::BeginUpdate()
{
	if (UpdateDepth == 0)
	{
		for (FISMComponentData& ComponentData : DescriptorComponents)
		{
			if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(ComponentData.Component))
			{
				ComponentData.bAutoRebuildTreeOnInstanceChanges = HISMComponent->bAutoRebuildTreeOnInstanceChanges;
				HISMComponent->bAutoRebuildTreeOnInstanceChanges = false;
			}

			check(!ComponentData.bWasModifyCalled);
			check(!bWasModifyCalled);
		}
	}
	++UpdateDepth;
}

void AISMPartitionActor::EndUpdate()
{
	--UpdateDepth;
	check(UpdateDepth >= 0);
	if (UpdateDepth == 0)
	{
		bWasModifyCalled = false;
		for (FISMComponentData& ComponentData : DescriptorComponents)
		{
			
			if (ComponentData.Component && ComponentData.bInvalidateLightingCache)
			{
				ComponentData.Component->InvalidateLightingCache();
				ComponentData.bInvalidateLightingCache = false;
			}
			if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(ComponentData.Component))
			{
				HISMComponent->bAutoRebuildTreeOnInstanceChanges = ComponentData.bAutoRebuildTreeOnInstanceChanges;

				if (ComponentData.bWasModifyCalled)
				{
					HISMComponent->BuildTreeIfOutdated(true, false);
				}
			}

			ComponentData.bWasModifyCalled = false;
		}
	}
}

void AISMPartitionActor::UpdateHISMTrees(bool bAsync, bool bForce)
{
	for (FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(ComponentData.Component))
		{
			HISMComponent->BuildTreeIfOutdated(bAsync, bForce);
		}
	}
}

void AISMPartitionActor::ForEachClientComponent(const FISMClientHandle& Handle, TFunctionRef<bool(UInstancedStaticMeshComponent*)> Callback) const
{
	check(Handle.Guid == Clients[Handle.Index]);
	for (const FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (ComponentData.Component)
		{
			const FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
			if (ClientData.Instances.Num())
			{
				if (!Callback(ComponentData.Component))
				{
					return;
				}
			}
		}
	}
}

void AISMPartitionActor::ForEachClientSMInstance(const FISMClientHandle& Handle, TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	check(Handle.Guid == Clients[Handle.Index]);
	for (const FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (ComponentData.Component && ComponentData.ClientInstances.IsValidIndex(Handle.Index))
		{
			const FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
			for (const FISMClientInstance& ClientInstance : ClientData.Instances)
			{
				for (int32 ComponentIndex : ClientInstance.ComponentIndices)
				{
					if (!Callback(FSMInstanceId{ ComponentData.Component, ComponentIndex }))
					{
						return;
					}
				}
			}
		}
	}
}

void AISMPartitionActor::ForEachClientSMInstance(const FISMClientHandle& Handle, int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const
{
	check(Handle.Guid == Clients[Handle.Index]);
	for (const FISMComponentData& ComponentData : DescriptorComponents)
	{
		if (ComponentData.Component && ComponentData.ClientInstances.IsValidIndex(Handle.Index))
		{
			const FISMClientData& ClientData = ComponentData.ClientInstances[Handle.Index];
			if (ClientData.Instances.IsValidIndex(InstanceIndex))
			{
				const FISMClientInstance& ClientInstance = ClientData.Instances[InstanceIndex];
				for (int32 ComponentIndex : ClientInstance.ComponentIndices)
				{
					if (!Callback(FSMInstanceId{ ComponentData.Component, ComponentIndex }))
					{
						return;
					}
				}
			}
		}
	}
}
#endif

FText AISMPartitionActor::GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.GetISMPartitionInstanceDisplayName();
#else
	return FText();
#endif
}

FText AISMPartitionActor::GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.GetISMPartitionInstanceTooltip();
#else
	return FText();
#endif
}

bool AISMPartitionActor::CanEditSMInstance(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.CanEditISMPartitionInstance();
#else
	return false;
#endif
}

bool AISMPartitionActor::CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType WorldType) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.CanMoveISMPartitionInstance(WorldType);
#else
	return false;
#endif
}

bool AISMPartitionActor::GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.GetISMPartitionInstanceTransform(OutInstanceTransform, bWorldSpace);
#else
	return false;
#endif
}

bool AISMPartitionActor::SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.SetISMPartitionInstanceTransform(InstanceTransform, bWorldSpace, bTeleport);
#else
	return false;
#endif
}

void AISMPartitionActor::NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	InstanceManager.NotifyISMPartitionInstanceMovementStarted();
#endif
}

void AISMPartitionActor::NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	InstanceManager.NotifyISMPartitionInstanceMovementOngoing();
#endif
}

void AISMPartitionActor::NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	InstanceManager.NotifyISMPartitionInstanceMovementEnded();
#endif
}

void AISMPartitionActor::NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected)
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	InstanceManager.NotifyISMPartitionInstanceSelectionChanged(bIsSelected);
#endif
}

void AISMPartitionActor::ForEachSMInstanceInSelectionGroup(const FSMInstanceId& InstanceId, TFunctionRef<bool(FSMInstanceId)> Callback)
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	ForEachClientSMInstance(InstanceManager.GetInstanceId().Handle, InstanceManager.GetInstanceId().Index, Callback);
#else
	Callback(InstanceId);
#endif
}

bool AISMPartitionActor::CanDeleteSMInstance(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.CanDeleteISMPartitionInstance();
#else
	return false;
#endif
}

bool AISMPartitionActor::DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds)
{
#if WITH_EDITOR
	// Batch by the client manager
	TMap<IISMPartitionInstanceManager*, TArray<FISMClientInstanceId>> BatchedInstancesToDelete;
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
		TArray<FISMClientInstanceId>& ClientInstanceIds = BatchedInstancesToDelete.FindOrAdd(InstanceManager.GetInstanceManager());
		ClientInstanceIds.Add(InstanceManager.GetInstanceId());
	}

	bool bDidDelete = false;
	for (TTuple<IISMPartitionInstanceManager*, TArray<FISMClientInstanceId>>& BatchedInstancesToDeletePair : BatchedInstancesToDelete)
	{
		bDidDelete |= BatchedInstancesToDeletePair.Key->DeleteISMPartitionInstances(BatchedInstancesToDeletePair.Value);
	}
	return bDidDelete;
#else
	return false;
#endif
}

bool AISMPartitionActor::CanDuplicateSMInstance(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
	return InstanceManager.CanDuplicateISMPartitionInstance();
#else
	return false;
#endif
}

bool AISMPartitionActor::DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds)
{
#if WITH_EDITOR
	// Batch by the client manager
	TMap<IISMPartitionInstanceManager*, TArray<FISMClientInstanceId>> BatchedInstancesToDuplicate;
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManagerChecked(InstanceId);
		TArray<FISMClientInstanceId>& ClientInstanceIds = BatchedInstancesToDuplicate.FindOrAdd(InstanceManager.GetInstanceManager());
		ClientInstanceIds.Add(InstanceManager.GetInstanceId());
	}

	bool bDidDuplicate = false;
	for (const TTuple<IISMPartitionInstanceManager*, TArray<FISMClientInstanceId>>& BatchedInstancesToDuplicatePair : BatchedInstancesToDuplicate)
	{
		TArray<FISMClientInstanceId> NewClientInstanceIds;
		if (BatchedInstancesToDuplicatePair.Key->DuplicateISMPartitionInstances(BatchedInstancesToDuplicatePair.Value, NewClientInstanceIds))
		{
			bDidDuplicate = true;

			OutNewInstanceIds.Reserve(OutNewInstanceIds.Num() + NewClientInstanceIds.Num());
			for (const FISMClientInstanceId& NewClientInstanceId : NewClientInstanceIds)
			{
				for (const FISMComponentData& DescriptorComponent : DescriptorComponents)
				{
					if (DescriptorComponent.Component)
					{
						const FISMClientData& ClientData = DescriptorComponent.ClientInstances[NewClientInstanceId.Handle.Index];
						if (ClientData.Instances.IsValidIndex(NewClientInstanceId.Index))
						{
							// We only return the first ISMC instance, as all ISMC instances belonging to this client instance are treated as one (see ForEachSMInstanceInSelectionGroup)
							OutNewInstanceIds.Add(FSMInstanceId{ DescriptorComponent.Component, ClientData.Instances[NewClientInstanceId.Index].ComponentIndices[0] });
							break;
						}
					}
				}
			}
		}
	}
	return bDidDuplicate;
#else
	return false;
#endif
}

#if WITH_EDITOR
FISMPartitionInstanceManager AISMPartitionActor::GetISMPartitionInstanceManager(const FSMInstanceId& InstanceId) const
{
	if (GIsEditor)
	{
		for (const FISMComponentData& DescriptorComponent : DescriptorComponents)
		{
			if (DescriptorComponent.Component == InstanceId.ISMComponent)
			{
				const FISMComponentInstance& ComponentInstance = DescriptorComponent.Instances[InstanceId.InstanceIndex];

				const FGuid& ClientGuid = Clients[ComponentInstance.ClientIndex];
				if (const FISMClientInstanceManagerData* ClientInstanceManagerData = ClientInstanceManagers.Find(ClientGuid))
				{
					const FISMClientHandle ClientHandle(ComponentInstance.ClientIndex, ClientGuid);
					return FISMPartitionInstanceManager(FISMClientInstanceId{ ClientHandle, ComponentInstance.InstanceIndex }, ClientInstanceManagerData->ResolveInstanceManager(ClientHandle));
				}
			}
		}
	}

	return FISMPartitionInstanceManager();
}

FISMPartitionInstanceManager AISMPartitionActor::GetISMPartitionInstanceManagerChecked(const FSMInstanceId& InstanceId) const
{
	FISMPartitionInstanceManager InstanceManager = GetISMPartitionInstanceManager(InstanceId);
	check(InstanceManager);
	return InstanceManager;
}
#endif

ISMInstanceManager* AISMPartitionActor::GetSMInstanceManager(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	if (GetISMPartitionInstanceManager(InstanceId))
	{
		return this;
	}
#endif

	return nullptr;
}

