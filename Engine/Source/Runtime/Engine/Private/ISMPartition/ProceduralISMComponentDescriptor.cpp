// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPartition/ProceduralISMComponentDescriptor.h"

#include "ISMPartition/ISMComponentDescriptor.h"
#include "Materials/MaterialInterface.h"
#include "Serialization/ArchiveCrc32.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralISMComponentDescriptor)

FProceduralISMComponentDescriptor::FProceduralISMComponentDescriptor()
	: NumInstances(0)
	, NumCustomFloats(0)
{
	*this = FSoftISMComponentDescriptor();
}

FProceduralISMComponentDescriptor& FProceduralISMComponentDescriptor::operator=(const FSoftISMComponentDescriptor& Other)
{
	InstanceMinDrawDistance = Other.InstanceMinDrawDistance;
	InstanceStartCullDistance = Other.InstanceStartCullDistance;
	InstanceEndCullDistance = Other.InstanceEndCullDistance;
	OverlayMaterial = Other.OverlayMaterial.LoadSynchronous();
	StaticMesh = Other.StaticMesh;
	Mobility = Other.Mobility;
	VirtualTextureRenderPassType = Other.VirtualTextureRenderPassType;
	LightingChannels = Other.LightingChannels;
	CustomDepthStencilWriteMask = Other.CustomDepthStencilWriteMask;
	VirtualTextureCullMips = Other.VirtualTextureCullMips;
	TranslucencySortPriority = Other.TranslucencySortPriority;
	CustomDepthStencilValue = Other.CustomDepthStencilValue;
	bCastShadow = Other.bCastShadow;
	bEmissiveLightSource = Other.bEmissiveLightSource;
	bCastDynamicShadow = Other.bCastDynamicShadow;
	bCastStaticShadow = Other.bCastStaticShadow;
	bCastContactShadow = Other.bCastContactShadow;
	bCastShadowAsTwoSided = Other.bCastShadowAsTwoSided;
	bCastHiddenShadow = Other.bCastHiddenShadow;
	bReceivesDecals = Other.bReceivesDecals;
	bUseAsOccluder = Other.bUseAsOccluder;
	bRenderCustomDepth = Other.bRenderCustomDepth;
	bEvaluateWorldPositionOffset = Other.bEvaluateWorldPositionOffset;
	bReverseCulling = Other.bReverseCulling;
	WorldPositionOffsetDisableDistance = Other.WorldPositionOffsetDisableDistance;
	ShadowCacheInvalidationBehavior = Other.ShadowCacheInvalidationBehavior;
	DetailMode = Other.DetailMode;
	bVisibleInRayTracing = Other.bVisibleInRayTracing;
	bVisibleInReflections = Other.bVisibleInReflections;
	RayTracingGroupId = Other.RayTracingGroupId;
	RayTracingGroupCullingPriority = Other.RayTracingGroupCullingPriority;

	Algo::Transform(Other.OverrideMaterials, OverrideMaterials, [](const TSoftObjectPtr<UMaterialInterface>& OverrideMaterial)
	{
		return OverrideMaterial.LoadSynchronous();
	});

	Algo::Transform(Other.RuntimeVirtualTextures, RuntimeVirtualTextures, [](const TSoftObjectPtr<URuntimeVirtualTexture>& RVT)
	{
		return RVT.LoadSynchronous();
	});

	return *this;
}

uint32 FProceduralISMComponentDescriptor::ComputeHash() const
{
	FArchiveCrc32 CrcArchive;

	Hash = 0; // we don't want the hash to impact the calculation
	CrcArchive << *this;
	Hash = CrcArchive.GetCrc();

	return Hash;
}

bool FProceduralISMComponentDescriptor::operator!=(const FProceduralISMComponentDescriptor& Other) const
{
	return !(*this == Other);
}

bool FProceduralISMComponentDescriptor::operator==(const FProceduralISMComponentDescriptor& Other) const
{
	return StaticMesh == Other.StaticMesh
		&& OverrideMaterials == Other.OverrideMaterials
		&& OverlayMaterial == Other.OverlayMaterial
		&& RuntimeVirtualTextures == Other.RuntimeVirtualTextures
		&& NumInstances == Other.NumInstances
		&& NumCustomFloats == Other.NumCustomFloats
		&& WorldBounds == Other.WorldBounds
		&& InstanceMinDrawDistance == Other.InstanceMinDrawDistance
		&& InstanceStartCullDistance == Other.InstanceStartCullDistance
		&& InstanceEndCullDistance == Other.InstanceEndCullDistance
		&& Mobility == Other.Mobility
		&& VirtualTextureRenderPassType == Other.VirtualTextureRenderPassType
		&& GetLightingChannelMaskForStruct(LightingChannels) == GetLightingChannelMaskForStruct(Other.LightingChannels)
		&& CustomDepthStencilWriteMask == Other.CustomDepthStencilWriteMask
		&& VirtualTextureCullMips == Other.VirtualTextureCullMips
		&& TranslucencySortPriority == Other.TranslucencySortPriority
		&& CustomDepthStencilValue == Other.CustomDepthStencilValue
		&& bCastShadow == Other.bCastShadow
		&& bEmissiveLightSource == Other.bEmissiveLightSource
		&& bCastDynamicShadow == Other.bCastDynamicShadow
		&& bCastStaticShadow == Other.bCastStaticShadow
		&& bCastContactShadow == Other.bCastContactShadow
		&& bCastShadowAsTwoSided == Other.bCastShadowAsTwoSided
		&& bCastHiddenShadow == Other.bCastHiddenShadow
		&& bReceivesDecals == Other.bReceivesDecals
		&& bUseAsOccluder == Other.bUseAsOccluder
		&& bRenderCustomDepth == Other.bRenderCustomDepth
		&& bEvaluateWorldPositionOffset == Other.bEvaluateWorldPositionOffset
		&& bReverseCulling == Other.bReverseCulling
		&& WorldPositionOffsetDisableDistance == Other.WorldPositionOffsetDisableDistance
		&& ShadowCacheInvalidationBehavior == Other.ShadowCacheInvalidationBehavior
		&& DetailMode == Other.DetailMode
		&& bVisibleInRayTracing == Other.bVisibleInRayTracing
		&& bVisibleInReflections == Other.bVisibleInReflections
		&& RayTracingGroupId == Other.RayTracingGroupId
		&& RayTracingGroupCullingPriority == Other.RayTracingGroupCullingPriority;
}
