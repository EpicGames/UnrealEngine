// Copyright Epic Games, Inc. All Rights Reserved.

#include "IStereoLayers.h"

#include "Engine/TextureRenderTarget2D.h"

IStereoLayers::FLayerDesc IStereoLayers::GetDebugCanvasLayerDesc(UTextureRenderTarget2D* Texture)
{
	// Default debug layer desc
	IStereoLayers::FLayerDesc StereoLayerDesc;
	StereoLayerDesc.Transform = FTransform(FVector(100.f, 0, 0));
	StereoLayerDesc.QuadSize = FVector2D(120.f, 120.f);
	StereoLayerDesc.PositionType = IStereoLayers::ELayerType::FaceLocked;
	StereoLayerDesc.TextureObj = Texture;
	StereoLayerDesc.Flags = IStereoLayers::ELayerFlags::LAYER_FLAG_TEX_CONTINUOUS_UPDATE;
	StereoLayerDesc.Flags |= IStereoLayers::ELayerFlags::LAYER_FLAG_QUAD_PRESERVE_TEX_RATIO;
	return StereoLayerDesc;
}