// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "Modules/ModuleManager.h"

/**
* Singleton-like access to this module's interface.  This is just for convenience!
* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
*
* @return Returns singleton instance, loading the module on demand if needed
*/
IWorldPartitionEditorModule& IWorldPartitionEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<IWorldPartitionEditorModule>("WorldPartitionEditor");
}