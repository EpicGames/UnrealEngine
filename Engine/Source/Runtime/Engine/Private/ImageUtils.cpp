// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ImageUtils.cpp: Image utility functions.
=============================================================================*/

#include "ImageUtils.h"

#include "CubemapUnwrapUtils.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/VolumeTexture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTargetCube.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Logging/MessageLog.h"
#include "Misc/FileHelper.h"
#include "Misc/ObjectThumbnail.h"
#include "Modules/ModuleManager.h"
#include "ImageCoreUtils.h"
#include "DDSFile.h"
#include "TextureResource.h"

#ifdef LoadImage // Seems like some windows include is leaking through (LoadImage define can be set to LoadImageW)
#undef LoadImage
#endif

DEFINE_LOG_CATEGORY_STATIC(LogImageUtils, Log, All);

#define LOCTEXT_NAMESPACE "ImageUtils"


static IImageWrapperModule * GetOrLoadImageWrapperModule()
{
	static FName ImageWrapperName("ImageWrapper");

	if ( IsInGameThread() )
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(ImageWrapperName);
		return &ImageWrapperModule;
	}
	else
	{
		IImageWrapperModule * ImageWrapperModule = FModuleManager::GetModulePtr<IImageWrapperModule>(ImageWrapperName);

		if ( ImageWrapperModule == nullptr )
		{
			UE_LOGF(LogImageUtils,Error,"Not on GameThread, cannot load ImageWrapper.  Do on main thread in startup.");

			// LoadModule needs to be done on the Game thread as part of your initialization
			//   before any thread tries to run this code
			// Engine does this.  If you are making a stand-alone app and hit this error,
			//   add this to your initialization on the Game thread :
			// FModuleManager::LoadModuleChecked<IImageWrapperModule>("ImageWrapper");
		}

		return ImageWrapperModule;
	}
}


bool FImageUtils::SaveImageAutoFormat(const TCHAR * FileName, const FImageView & InImage, int32 Quality)
{
	IImageWrapperModule * ImageWrapperModule = GetOrLoadImageWrapperModule();
	if ( ImageWrapperModule == nullptr )
	{
		return false;
	}

	EImageFormat ToFormat = ImageWrapperModule->GetDefaultOutputFormat(InImage.Format);
	check( ToFormat != EImageFormat::Invalid );
	
	bool bHasExtension = FCString::Strchr(FileName,TEXT('.')) != nullptr;

	FString Scratch;
	if ( !bHasExtension || ImageWrapperModule->GetImageFormatFromExtension(FileName) != ToFormat )
	{
		const TCHAR * Extension = ImageWrapperModule->GetExtension(ToFormat);
		
		Scratch = FPaths::GetBaseFilename(FileName, false);
		Scratch += TEXT('.');
		Scratch += Extension;
		FileName = *Scratch;
	}

	TArray64<uint8> Buffer;
	if ( ! ImageWrapperModule->CompressImage(Buffer,ToFormat,InImage,Quality) )
	{
		return false;
	}

	return FFileHelper::SaveArrayToFile(Buffer,FileName);
}

bool FImageUtils::LoadImage(const TCHAR * Filename, FImage & OutImage)
{
	TArray64<uint8> Buffer;
	if ( ! FFileHelper::LoadFileToArray(Buffer, Filename, FILEREAD_Silent) )
	{
		return false;
	}

	return DecompressImage(Buffer.GetData(),Buffer.Num(),OutImage);
}

bool FImageUtils::SaveImageByExtension(const TCHAR * Filename, const FImageView & InImage, int32 Quality)
{
	// if no extension provided, add one :
	if ( FCString::Strchr(Filename,TEXT('.')) == nullptr )
	{
		return SaveImageAutoFormat(Filename,InImage,Quality);
	}

	TArray64<uint8> Buffer;
	if ( !  CompressImage(Buffer, Filename, InImage,Quality) )
	{
		return false;
	}

	return FFileHelper::SaveArrayToFile(Buffer,Filename);
}

// OutImage is allocated and filled, any existing contents are discarded
bool FImageUtils::DecompressImage(const void* InCompressedData, int64 InCompressedSize, FImage & OutImage)
{
	IImageWrapperModule * ImageWrapperModule = GetOrLoadImageWrapperModule();
	if ( ImageWrapperModule == nullptr )
	{
		return false;
	}

	return ImageWrapperModule->DecompressImage(InCompressedData,InCompressedSize,OutImage);
}

// OutData is filled with the file format encoded data
//	 lossy conversion may be done if necessary
bool FImageUtils::CompressImage(TArray64<uint8> & OutData, const TCHAR * ToFormatExtension, const FImageView & InImage, int32 Quality)
{
	IImageWrapperModule * ImageWrapperModule = GetOrLoadImageWrapperModule();
	if ( ImageWrapperModule == nullptr )
	{
		return false;
	}
	
	EImageFormat ToFormat = EImageFormat::Invalid;
	
	if ( ToFormatExtension != nullptr )
	{
		ToFormat = ImageWrapperModule->GetImageFormatFromExtension(ToFormatExtension);
	}

	if ( ToFormat == EImageFormat::Invalid )
	{
		ToFormat = ImageWrapperModule->GetDefaultOutputFormat(InImage.Format);
	}

	return ImageWrapperModule->CompressImage(OutData,ToFormat,InImage,Quality);
}

bool FImageUtils::ExportTextureSourceToDDS(TArray64<uint8> & OutData, UTexture * Texture, int BlockIndex, int LayerIndex)
{
#if WITH_EDITORONLY_DATA	

	const FTextureSource & Source = Texture->Source;

	if ( ! Source.IsValid() )
	{
		UE_LOGF(LogImageUtils,Warning,"ExportTextureSourceToDDS: Texture does not have source (%ls)",*Texture->GetName());
		return false;
	}

	check( BlockIndex < Source.GetNumBlocks() );
	check( LayerIndex < Source.GetNumLayers() );
	
	FTextureSourceBlock Block;
	Source.GetBlock(BlockIndex, Block);

	int32 SizeX = Block.SizeX;
	int32 SizeY = Block.SizeY;
	int32 NumSlices = Block.NumSlices;
	int32 NumMips = Block.NumMips;

	int32 Dimension;
	int32 SizeZ;
	int32 ArraySize;
	uint32 CreateFlags = 0;
	ETextureClass TextureClass = Texture->GetTextureClass();
	if ( TextureClass == ETextureClass::TwoD )
	{
		Dimension = 2;
		SizeZ = 1;
		ArraySize = NumSlices;
	}
	else if ( TextureClass == ETextureClass::Cube )
	{
		Dimension = 2;
		SizeZ = 1;

		// IsLongLatCubemap() is not right
		// just guess from NumSlices

		//bool bIsLongLatCubemap = Source.IsLongLatCubemap();
		bool bIsLongLatCubemap = ( NumSlices == 1 ); 

		if ( bIsLongLatCubemap )
		{
			// just export as 2d texture
			check( NumSlices == 1 );
			ArraySize = 1;
		}
		else
		{
			check( NumSlices == 6 );
			ArraySize = 6;
			CreateFlags = UE::DDS::FDDSFile::CREATE_FLAG_CUBEMAP;
		}
	}
	else if ( TextureClass == ETextureClass::CubeArray )
	{
		Dimension = 2;
		SizeZ = 1;

		// IsLongLatCubemap() should be right for CubeArrays
		bool bIsLongLatCubemap = Source.IsLongLatCubemap() || (NumSlices%6) != 0; 

		if ( bIsLongLatCubemap )
		{
			// just export as 2d texture array
			ArraySize = NumSlices;
		}
		else
		{
			check( (NumSlices%6) == 0 );
			ArraySize = NumSlices;
			CreateFlags = UE::DDS::FDDSFile::CREATE_FLAG_CUBEMAP;
		}
	}
	else if ( TextureClass == ETextureClass::Volume )
	{
		Dimension = 3;
		SizeZ = NumSlices;
		ArraySize = 1;
	}
	else
	{
		checkf(false, TEXT("unexpected TextureClass"));
		return false;
	}
	
	ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(Source.GetFormat(LayerIndex));
	EGammaSpace GammaSpace = Source.GetGammaSpace(LayerIndex);
	UE::DDS::EDXGIFormat DXGIFormat = UE::DDS::DXGIFormatFromRawFormat(RawFormat,GammaSpace);
	
	if ( RawFormat == ERawImageFormat::BGRE8 )
	{
		// convert BGRE to Linear float? or just export the bytes as they are in BGRA ?
		UE_LOGF(LogImageUtils,Warning,"DDS export to BGRE8 will write the bytes as they were in BGRA, not converted to float linear");
	}

	UE_LOGF(LogImageUtils,Display,"Exporting DDS Dimension=%d SizeX=%d SizeY=%d SizeZ=%d NumMips=%d ArraySize=%d",Dimension, SizeX,SizeY,SizeZ,NumMips,ArraySize);	
	
	UE::DDS::EDDSError Error;
	UE::DDS::FDDSFile * DDS = UE::DDS::FDDSFile::CreateEmpty(Dimension, SizeX,SizeY,SizeZ,NumMips,ArraySize, DXGIFormat,CreateFlags, &Error);
	if ( DDS == nullptr || Error != UE::DDS::EDDSError::OK )
	{
		UE_LOGF(LogImageUtils,Warning,"FDDSFile::CreateEmpty (Error=%d)",(int)Error);			
		return false;
	}
	
	// delete DDS at scope exit :
	TUniquePtr<UE::DDS::FDDSFile> DDSPtr(DDS);

	check( DDS->Validate() == UE::DDS::EDDSError::OK );
	
	// blit into the mips:
	
	for(int MipIndex=0;MipIndex<NumMips;MipIndex++)
	{
		FTextureSource::FMipLock MipLock(FTextureSource::ELockState::ReadOnly,const_cast<FTextureSource *>(&Source),BlockIndex,LayerIndex,MipIndex);

		if ( DDS->Dimension == 3 )
		{
			check( DDS->Mips.Num() == DDS->MipCount );
			check( DDS->Mips[MipIndex].Depth == MipLock.Image.NumSlices );

			DDS->FillMip( MipLock.Image, MipIndex );
		}
		else
		{
			// DDS->Mips[] contains both mips and arrays
			check( DDS->Mips.Num() == DDS->MipCount * NumSlices );

			for(int SliceIndex=0;SliceIndex<NumSlices;SliceIndex++)
			{
				FImageView MipSlice = MipLock.Image.GetSlice(SliceIndex);
				
				// DDS Mips[] array has whole mip chain of each slice, then next slice
				// we have the opposite (all slices of top mip first, then next mip)
				int DDSMipIndex = SliceIndex * DDS->MipCount + MipIndex;
				
				DDS->FillMip( MipSlice, DDSMipIndex );
			}
		}
	}

	Error = DDS->WriteDDS(OutData);
	if ( Error != UE::DDS::EDDSError::OK )
	{
		UE_LOGF(LogImageUtils,Warning,"FDDSFile::WriteDDS failed (Error=%d)",(int)Error);			
		return false;
	}

	return true;
#else
	return false;
#endif
}

bool FImageUtils::ExportRenderTargetToDDS(TArray64<uint8> & OutData, UTextureRenderTarget * TexRT)
{
	FImage Image;
	if ( ! GetRenderTargetImage(TexRT,Image) )
	{
		UE_LOGF(LogImageUtils,Warning,"ExportRenderTargetToDDS : GetRenderTargetImage failed");
		return false;
	}

	// some code dupe with GetRenderTargetImage to identify 2d/cube/vol :
	ETextureClass TexRTClass = TexRT->GetRenderTargetTextureClass();
	check( TexRTClass != ETextureClass::Invalid && TexRTClass != ETextureClass::RenderTarget );
	bool bIsCube = TexRTClass == ETextureClass::Cube || TexRTClass == ETextureClass::CubeArray;
	
	int64 SizeX = FMath::RoundToInt64( TexRT->GetSurfaceWidth() ); // GetSurfaceWidth returns SizeX but as float
	int64 SizeY = FMath::RoundToInt64( TexRT->GetSurfaceHeight() );
	int64 TexRT_SizeZ = FMath::RoundToInt64( TexRT->GetSurfaceDepth() );
	int64 TexRT_ArraySize = TexRT->GetSurfaceArraySize();
	// TexRT_SizeZ , ArraySize are 0 if not used, not 1

	int32 NumMips = 1;

	int32 Dimension;
	int32 SizeZ;
	int32 ArraySize;
	uint32 CreateFlags = 0;
	if ( TexRT_SizeZ <= 1 && !bIsCube ) // 2D
	{
		Dimension = 2;
		SizeZ = 1;
		ArraySize = TexRT_ArraySize ? TexRT_ArraySize : 1;
	}
	else if ( bIsCube )
	{
		Dimension = 2;
		SizeZ = 1;

		check( TexRT_ArraySize == 6 );
		ArraySize = 6;
		CreateFlags = UE::DDS::FDDSFile::CREATE_FLAG_CUBEMAP;
	}
	else if ( TexRT_SizeZ > 1 )
	{
		Dimension = 3;
		SizeZ = TexRT_SizeZ;
		ArraySize = 1;
	}
	else
	{
		checkf(false, TEXT("unexpected TextureClass"));
		return false;
	}
	
	UE::DDS::EDXGIFormat DXGIFormat = UE::DDS::DXGIFormatFromRawFormat(Image.Format,Image.GammaSpace);
	
	UE_LOG(LogImageUtils,Display,TEXT("Exporting DDS Dimension=%d SizeX=%" INT64_FMT " SizeY=%" INT64_FMT " SizeZ=%d NumMips=%d ArraySize=%d"),Dimension, SizeX,SizeY,SizeZ,NumMips,ArraySize);
	
	UE::DDS::EDDSError Error;
	UE::DDS::FDDSFile * DDS = UE::DDS::FDDSFile::CreateEmpty(Dimension, SizeX,SizeY,SizeZ,NumMips,ArraySize, DXGIFormat,CreateFlags, &Error);
	if ( DDS == nullptr || Error != UE::DDS::EDDSError::OK )
	{
		UE_LOGF(LogImageUtils,Warning,"FDDSFile::CreateEmpty (Error=%d)",(int)Error);			
		return false;
	}
	
	// delete DDS at scope exit :
	TUniquePtr<UE::DDS::FDDSFile> DDSPtr(DDS);

	check( DDS->Validate() == UE::DDS::EDDSError::OK );
	
	// blit into the mips:
	const int32 MipIndex = 0;

	if ( DDS->Dimension == 3 )
	{
		check( DDS->Mips.Num() == DDS->MipCount );
		check( DDS->Mips[MipIndex].Depth == Image.NumSlices );

		DDS->FillMip( Image, MipIndex );
	}
	else
	{
		// DDS->Mips[] contains both mips and arrays
		check( DDS->Mips.Num() == DDS->MipCount * Image.NumSlices );

		for(int SliceIndex=0;SliceIndex<Image.NumSlices;SliceIndex++)
		{
			FImageView MipSlice = Image.GetSlice(SliceIndex);
				
			// DDS Mips[] array has whole mip chain of each slice, then next slice
			// we have the opposite (all slices of top mip first, then next mip)
			int DDSMipIndex = SliceIndex * DDS->MipCount + MipIndex;
				
			DDS->FillMip( MipSlice, DDSMipIndex );
		}
	}

	Error = DDS->WriteDDS(OutData);
	if ( Error != UE::DDS::EDDSError::OK )
	{
		UE_LOGF(LogImageUtils,Warning,"FDDSFile::WriteDDS failed (Error=%d)",(int)Error);			
		return false;
	}

	return true;
}

bool FImageUtils::GetRawData(UTextureRenderTarget2D* TexRT, TArray64<uint8>& RawData)
{
	// DEPRECATED , use GetRenderTargetImage

	RawData.Empty();

	FImage Image;
	if ( ! GetRenderTargetImage(TexRT,Image) )
	{
		return false;
	}

	RawData = MoveTemp(Image.RawData);
	return true;
}

bool FImageUtils::GetRenderTargetImage(UTextureRenderTarget* TexRT, FImage & Image)
{
	return GetRenderTargetImage(TexRT,Image,FIntRect(0,0,0,0));
}

bool FImageUtils::GetRenderTargetImage(UTextureRenderTarget* TexRT, FImage & OutImage, const FIntRect & InRectOrZero)
{
	OutImage = FImage();
	
	FRenderTarget* RenderTarget = TexRT->GameThread_GetRenderTargetResource();
	EPixelFormat RTFormat = TexRT->GetFormat();

	ERawImageFormat::Type ReadFormat = UTextureRenderTarget::GetReadPixelsFormat(RTFormat,false);
	
	// we have to identify cubes because they are treated differently
	ETextureClass TexRTClass = TexRT->GetRenderTargetTextureClass();
	check( TexRTClass != ETextureClass::Invalid && TexRTClass != ETextureClass::RenderTarget );
	bool bIsCube = TexRTClass == ETextureClass::Cube || TexRTClass == ETextureClass::CubeArray;
	
	int64 TexRT_SizeX = FMath::RoundToInt64( TexRT->GetSurfaceWidth() ); // GetSurfaceWidth returns SizeX but as float
	int64 TexRT_SizeY = FMath::RoundToInt64( TexRT->GetSurfaceHeight() );
	int64 TexRT_SizeZ = FMath::RoundToInt64( TexRT->GetSurfaceDepth() );
	int64 TexRT_ArraySize = TexRT->GetSurfaceArraySize();

	if ( TexRT_SizeX <= 0 || TexRT_SizeY <= 0 )
	{
		return false;
	}

	int64 NumSlices;
	if ( bIsCube )
	{
		check( TexRT_SizeZ == 0 );
		check( TexRT_ArraySize == 6 );
		NumSlices = 6;
	}
	else if ( TexRT_SizeZ != 0 )
	{
		check( TexRT_ArraySize == 0 );
		NumSlices = TexRT_SizeZ;
	}
	else if ( TexRT_ArraySize != 0 )
	{
		check( TexRT_SizeZ == 0 );
		NumSlices = TexRT_ArraySize;
	}
	else
	{
		// 2D
		NumSlices = 1;
	}
	
	// UTextureRenderTarget2D returns 0 for Depth and ArraySize, not 1

	// RCM_MinMax means don't renormalize, just read the pixels as they are
	//	default RCM_UNorm does funny scalings
	FReadSurfaceDataFlags ReadFlags(RCM_MinMax, CubeFace_MAX);
	
	FIntRect Rect = InRectOrZero;
	if (InRectOrZero == FIntRect(0, 0, 0, 0))
	{
		Rect = FIntRect(0, 0, TexRT_SizeX, TexRT_SizeY);
	}

	int64 RectSizeX = Rect.Width();
	int64 RectSizeY = Rect.Height();
	if ( ! ensure( Rect.Min.X >= 0 && Rect.Min.Y >= 0 && 
		Rect.Max.X <= TexRT_SizeX && Rect.Max.Y <= TexRT_SizeY &&
		RectSizeX >= 0 && RectSizeY >= 0 ) )
	{
		return false;
	}
	if ( RectSizeX == 0 || RectSizeY == 0 )
	{
		return false; // or is that a success to grab zero pixels?
	}

	if ( ReadFormat == ERawImageFormat::RGBA16F )
	{
		// ReadFloat16Pixels does no conversions
		//	must be used only exactly with FloatRGBA type

		OutImage.Init(RectSizeX,RectSizeY,NumSlices,ERawImageFormat::RGBA16F,EGammaSpace::Linear);
		
		TArray<FFloat16Color> Colors;
		
		for (int32 SliceIndex = 0; SliceIndex < NumSlices; ++SliceIndex)
		{
			ReadFlags.SetCubeFace(  bIsCube ? (ECubeFace)(SliceIndex % 6) : CubeFace_MAX);
			ReadFlags.SetArrayIndex(bIsCube ? (SliceIndex/6) : SliceIndex);

			if ( ! RenderTarget->ReadFloat16Pixels(Colors,ReadFlags,Rect) )
			{
				return false;
			}

			FImageView ImageSlice = OutImage.GetSlice(SliceIndex);
			check( ImageSlice.GetImageSizeBytes() == Colors.Num() * sizeof(Colors[0]) );
			memcpy( ImageSlice.RawData, Colors.GetData(), ImageSlice.GetImageSizeBytes() );
		}
	}
	else if ( ReadFormat == ERawImageFormat::BGRA8 )
	{
		// ?? not clear TexRT->IsSRGB is right , see other notes on various issues there
		//	mainly we are trying to catch the check for whether the _SRGB or non _SRGB BGRA8 format as chosen
		EGammaSpace GammaSpace = TexRT->IsSRGB() ? EGammaSpace::sRGB : EGammaSpace::Linear;
		
		OutImage.Init(RectSizeX,RectSizeY,NumSlices,ERawImageFormat::BGRA8,GammaSpace);
		
		// "LinearToGamma" is basically moot; that would only be used if we were reading float pixels to FColor
		//	but in that case the ReadFormat should have been float, so we won't be here
		//	gamma conversion will be handled by FImage after the pixel read, not inside RHI
		ReadFlags.SetLinearToGamma( GammaSpace == EGammaSpace::sRGB );

		TArray<FColor> Colors;
		
		for (int32 SliceIndex = 0; SliceIndex < NumSlices; ++SliceIndex)
		{
			ReadFlags.SetCubeFace(  bIsCube ? (ECubeFace)(SliceIndex % 6) : CubeFace_MAX);
			ReadFlags.SetArrayIndex(bIsCube ? (SliceIndex/6) : SliceIndex);

			if ( ! RenderTarget->ReadPixels(Colors,ReadFlags,Rect) )
			{
				return false;
			}
		
			FImageView ImageSlice = OutImage.GetSlice(SliceIndex);
			check( ImageSlice.GetImageSizeBytes() == Colors.Num() * sizeof(Colors[0]) );
			memcpy( ImageSlice.RawData, Colors.GetData(), ImageSlice.GetImageSizeBytes() );
		}
	}
	else if ( ReadFormat == ERawImageFormat::RGBA32F )
	{
		OutImage.Init(RectSizeX,RectSizeY,NumSlices,ERawImageFormat::RGBA32F,EGammaSpace::Linear);
		
		TArray<FLinearColor> Colors;
		
		for (int32 SliceIndex = 0; SliceIndex < NumSlices; ++SliceIndex)
		{
			ReadFlags.SetCubeFace(  bIsCube ? (ECubeFace)(SliceIndex % 6) : CubeFace_MAX);
			ReadFlags.SetArrayIndex(bIsCube ? (SliceIndex/6) : SliceIndex);

			if ( ! RenderTarget->ReadLinearColorPixels(Colors,ReadFlags,Rect) )
			{
				return false;
			}
			
			FImageView ImageSlice = OutImage.GetSlice(SliceIndex);
			check( ImageSlice.GetImageSizeBytes() == Colors.Num() * sizeof(Colors[0]) );
			memcpy( ImageSlice.RawData, Colors.GetData(), ImageSlice.GetImageSizeBytes() );
		}
	}
	else
	{
		check(0); // unexpected ReadFormat
	}
	
	return true;
}


// prefer FImageCore::ResizeImage
void FImageUtils::ImageResize(int32 SrcWidth, int32 SrcHeight, const TArray<FColor> &SrcData, int32 DstWidth, int32 DstHeight, TArray<FColor> &DstData, bool bResizeSRGBinLinearSpace, bool bForceOpaqueOutput)
{
	int64 Num = (int64)DstWidth*DstHeight;
	DstData.Empty(Num);
	DstData.AddZeroed(Num);

	ImageResize(SrcWidth, SrcHeight, TArrayView<const FColor>(SrcData), DstWidth, DstHeight, TArrayView<FColor>(DstData), bResizeSRGBinLinearSpace, bForceOpaqueOutput);
}

// prefer FImageCore::ResizeImage
void FImageUtils::ImageResize(int32 SrcWidth, int32 SrcHeight, const TArrayView<const FColor> &SrcData, int32 DstWidth, int32 DstHeight, const TArrayView<FColor> &DstData, bool bResizeSRGBinLinearSpace, bool bForceOpaqueOutput)
{
	check(SrcData.Num() >= (int64)SrcWidth * SrcHeight);
	check(DstData.Num() >= (int64)DstWidth * DstHeight);
	
	if ( SrcData.Num() == 0 || DstData.Num() == 0 )
	{
		return;
	}

	// bResizeSRGBinLinearSpace means convert SRGB colors to Linear before resizing
	//	if it's false, we tell resize that our U8's are linear so it does no transforms
	EGammaSpace Gamma = bResizeSRGBinLinearSpace ? EGammaSpace::sRGB : EGammaSpace::Linear; 

	FImageView SrcImage(&SrcData[0],SrcWidth,SrcHeight,Gamma);
	FImageView DstImage(&DstData[0],DstWidth,DstHeight,Gamma);
	
	FImageCore::ResizeImage(SrcImage,DstImage);

	if ( bForceOpaqueOutput )
	{
		FImageCore::SetAlphaOpaque(DstImage);
	}
}

// prefer FImageCore::ResizeImage
void FImageUtils::ImageResize(int32 SrcWidth, int32 SrcHeight, const TArray64<FLinearColor>& SrcData, int32 DstWidth, int32 DstHeight, TArray64<FLinearColor>& DstData)
{
	int64 Num = (int64)DstWidth*DstHeight;
	DstData.Empty(Num);
	DstData.AddZeroed(Num);

	ImageResize(SrcWidth, SrcHeight, TArrayView64<const FLinearColor>(SrcData), DstWidth, DstHeight, TArrayView64<FLinearColor>(DstData));
}

// prefer FImageCore::ResizeImage
void FImageUtils::ImageResize(int32 SrcWidth, int32 SrcHeight, const TArrayView64<const FLinearColor>& SrcData, int32 DstWidth, int32 DstHeight, const TArrayView64<FLinearColor>& DstData)
{
	check(SrcData.Num() >= (int64)SrcWidth * SrcHeight);
	check(DstData.Num() >= (int64)DstWidth * DstHeight);
	
	if ( SrcData.Num() == 0 || DstData.Num() == 0 )
	{
		return;
	}

	FImageView SrcImage(&SrcData[0],SrcWidth,SrcHeight);
	FImageView DstImage(&DstData[0],DstWidth,DstHeight);
	
	FImageCore::ResizeImage(SrcImage,DstImage);
}

/**
 * Creates a 2D texture from a array of raw color data.
 *
 * @param SrcWidth		Source image width.
 * @param SrcHeight		Source image height.
 * @param SrcData		Source image data.
 * @param Outer			Outer for the texture object.
 * @param Name			Name for the texture object.
 * @param Flags			Object flags for the texture object.
 * @param InParams		Params about how to set up the texture.
 * @return				Returns a pointer to the constructed 2D texture object.
 *
 */
UTexture2D* FImageUtils::CreateTexture2D(int32 SrcWidth, int32 SrcHeight, const TArray<FColor> &SrcData, UObject* Outer, const FString& Name, const EObjectFlags &Flags, const FCreateTexture2DParameters& InParams)
{
#if WITH_EDITOR
	UTexture2D* Tex2D;

	Tex2D = NewObject<UTexture2D>(Outer, FName(*Name), Flags);
	Tex2D->Source.Init(SrcWidth, SrcHeight, /*NumSlices=*/ 1, /*NumMips=*/ 1, TSF_BGRA8);
	
	// if bUseAlpha is off, alpha is changed to 255
	uint8 AlphaOr = InParams.bUseAlpha ? 0 : 0xFF;

	// Create base mip for the texture we created.
	uint8* MipData = Tex2D->Source.LockMip(0);
	for( int32 y=0; y<SrcHeight; y++ )
	{
		// when UseAlpha is true, this could/should just be a memcpy of the whole array
		uint8* DestPtr = &MipData[(int64) y * SrcWidth * sizeof(FColor)];
		const FColor* SrcPtr = &SrcData[(int64) y * SrcWidth];
		for( int32 x=0; x<SrcWidth; x++ )
		{
			*DestPtr++ = SrcPtr->B;
			*DestPtr++ = SrcPtr->G;
			*DestPtr++ = SrcPtr->R;
			*DestPtr++ = SrcPtr->A | AlphaOr;
			SrcPtr++;
		}
	}
	Tex2D->Source.UnlockMip(0);

	// Set the Source Guid/Hash if specified
	if (InParams.SourceGuidHash.IsValid())
	{
		Tex2D->Source.SetId(InParams.SourceGuidHash, true);
	}

	// Set compression options.
	Tex2D->SRGB = InParams.bSRGB;
	Tex2D->CompressionSettings = InParams.CompressionSettings;
	Tex2D->MipGenSettings = InParams.MipGenSettings;
	if( !InParams.bUseAlpha )
	{
		Tex2D->CompressionNoAlpha = true;
	}
	Tex2D->DeferCompression	= InParams.bDeferCompression;
	if (InParams.TextureGroup != TEXTUREGROUP_MAX)
	{
		Tex2D->LODGroup = InParams.TextureGroup;
	}

	Tex2D->VirtualTextureStreaming = InParams.bVirtualTexture;
	
	Tex2D->SetModernSettingsForNewOrChangedTexture();

	Tex2D->PostEditChange();
	return Tex2D;
#else
	UE_LOGF(LogImageUtils, Fatal,"CreateTexture2D not supported without WITH_EDITOR.");
	return nullptr;
#endif
}
	
UTexture * FImageUtils::CreateTexture(ETextureClass TextureClass, const FImageView & Image, UObject* Outer, const FString& Name, EObjectFlags Flags, bool DoPostEditChange )
{
#if WITH_EDITOR
	UTexture * Tex;

	switch(TextureClass)
	{
	case ETextureClass::TwoD:
		Tex = NewObject<UTexture2D>(Outer, FName(*Name), Flags);
		break;
	case ETextureClass::Cube:
		Tex = NewObject<UTextureCube>(Outer, FName(*Name), Flags);
		break;
	case ETextureClass::Array:
		Tex = NewObject<UTexture2DArray>(Outer, FName(*Name), Flags);
		break;
	case ETextureClass::CubeArray:
		Tex = NewObject<UTextureCubeArray>(Outer, FName(*Name), Flags);
		break;
	case ETextureClass::Volume:
		Tex = NewObject<UVolumeTexture>(Outer, FName(*Name), Flags);
		break;
	default:
		UE_LOGF(LogImageUtils, Fatal,"CreateTexture invalid TextureClass.");
		return nullptr;
	}
	
	Tex->Source.Init(Image);
	
	Tex->SetModernSettingsForNewOrChangedTexture();

	if ( DoPostEditChange )
	{
		// note: Tex->CompressionSettings , SRGB, etc. have not been set yet
		//	calling PostEditChange usually doesn't make sense
		//	you should first set those things up, then call PostEditChange yourself outside
 
		Tex->PostEditChange();
	}

	return Tex;
#else
	UE_LOGF(LogImageUtils, Fatal,"CreateTexture not supported without WITH_EDITOR.");
	return nullptr;
#endif
}

void FImageUtils::CropAndScaleImage( int32 SrcWidth, int32 SrcHeight, int32 DesiredWidth, int32 DesiredHeight, const TArray<FColor> &SrcData, TArray<FColor> &DstData  )
{
	//	this is used by the old Thumbnail code, not the new paths

	// Get the aspect ratio, and calculate the dimension of the image to crop
	float DesiredAspectRatio = (float)DesiredWidth/(float)DesiredHeight;

	float MaxHeight = (float)SrcWidth / DesiredAspectRatio;
	float MaxWidth = (float)SrcWidth;

	if ( MaxHeight > (float)SrcHeight)
	{
		MaxHeight = (float)SrcHeight;
		MaxWidth = MaxHeight * DesiredAspectRatio;
	}

	// Store crop width and height as ints for convenience
	int32 CropWidth = FMath::FloorToInt(MaxWidth);
	int32 CropHeight = FMath::FloorToInt(MaxHeight);

	// Array holding the cropped image
	TArray<FColor> CroppedData;
	CroppedData.AddUninitialized(CropWidth*CropHeight);

	int32 CroppedSrcTop  = 0;
	int32 CroppedSrcLeft = 0;

	// Set top pixel if we are cropping height
	if ( CropHeight < SrcHeight )
	{
		CroppedSrcTop = ( SrcHeight - CropHeight ) / 2;
	}

	// Set width pixel if cropping width
	if ( CropWidth < SrcWidth )
	{
		CroppedSrcLeft = ( SrcWidth - CropWidth ) / 2;
	}

	const int32 DataSize = sizeof(FColor);

	//Crop the image
	for (int32 Row = 0; Row < CropHeight; Row++)
	{
	 	//Row*Side of a row*byte per color
	 	int32 SrcPixelIndex = (CroppedSrcTop+Row)*SrcWidth + CroppedSrcLeft;
	 	const void* SrcPtr = &(SrcData[SrcPixelIndex]);
	 	void* DstPtr = &(CroppedData[Row*CropWidth]);
	 	FMemory::Memcpy(DstPtr, SrcPtr, CropWidth*DataSize);
	}

	// Scale the image
	DstData.AddUninitialized(DesiredWidth*DesiredHeight);

	// Resize the image
	FImageUtils::ImageResize( MaxWidth, MaxHeight, CroppedData, DesiredWidth, DesiredHeight, DstData, true, true );
}

void FImageUtils::ThumbnailCompressImageArray( int32 ImageWidth, int32 ImageHeight, const TArray<FColor> &SrcData, TArray<uint8> &DstData )
{
	FObjectThumbnail TempThumbnail;
	TempThumbnail.SetImageSize( ImageWidth, ImageHeight );
	TArray<uint8>& ThumbnailByteArray = TempThumbnail.AccessImageData();

	// Copy image into destination thumb
	int32 MemorySize = ImageWidth*ImageHeight*sizeof(FColor);
	ThumbnailByteArray.AddUninitialized(MemorySize);
	FMemory::Memcpy(ThumbnailByteArray.GetData(), SrcData.GetData(), MemorySize);
	
	FColor * MutableSrcData = (FColor *)ThumbnailByteArray.GetData();

	// Thumbnails are saved as RGBA but FColors are stored as BGRA. An option to swap the order upon compression may be added at 
	// some point. At the moment, manually swapping Red and Blue 
	for (int32 Index = 0; Index < ImageWidth*ImageHeight; Index++ )
	{
		uint8 TempRed = MutableSrcData[Index].R;
		MutableSrcData[Index].R = MutableSrcData[Index].B;
		MutableSrcData[Index].B = TempRed;
	}

	// Compress data - convert into thumbnail current format
	TempThumbnail.CompressImageData();
	DstData = TempThumbnail.AccessCompressedImageData();
}

void FImageUtils::PNGCompressImageArray(int32 ImageWidth, int32 ImageHeight, const TArrayView64<const FColor>& SrcData, TArray64<uint8>& DstData)
{
	DstData.Reset();

	if (SrcData.Num() > 0 && ImageWidth > 0 && ImageHeight > 0)
	{
		FImageView Image((void *)SrcData.GetData(),ImageWidth,ImageHeight, ERawImageFormat::BGRA8);

		CompressImage(DstData,TEXT("png"),Image);
	}
}

UTexture2D* FImageUtils::CreateCheckerboardTexture(FColor ColorOne, FColor ColorTwo, int32 CheckerSize)
{
	CheckerSize = FMath::Min<uint32>( FMath::RoundUpToPowerOfTwo(CheckerSize), 4096 );
	const int32 HalfPixelNum = CheckerSize >> 1;

	// Create the texture
	UTexture2D* CheckerboardTexture = UTexture2D::CreateTransient(CheckerSize, CheckerSize, PF_B8G8R8A8);

	// Lock the checkerboard texture so it can be modified
	FColor* MipData = reinterpret_cast<FColor*>( CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE) );

	// Fill in the colors in a checkerboard pattern
	for ( int32 RowNum = 0; RowNum < CheckerSize; ++RowNum )
	{
		for ( int32 ColNum = 0; ColNum < CheckerSize; ++ColNum )
		{
			FColor& CurColor = MipData[( ColNum + ( RowNum * CheckerSize ) )];

			if ( ColNum < HalfPixelNum )
			{
				CurColor = ( RowNum < HalfPixelNum ) ? ColorOne : ColorTwo;
			}
			else
			{
				CurColor = ( RowNum < HalfPixelNum ) ? ColorTwo : ColorOne;
			}
		}
	}

	// Unlock the texture
	CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Unlock();
	CheckerboardTexture->UpdateResource();

	return CheckerboardTexture;
}

UTexture2DArray* FImageUtils::CreateCheckerboardTexture2DArray(FColor ColorOne, FColor ColorTwo, int32 CheckerSize, int32 ArraySize)
{
	CheckerSize = FMath::Min<uint32>(FMath::RoundUpToPowerOfTwo(CheckerSize), 4096);
	const int32 HalfCheckerSize = CheckerSize >> 1;

	// Create the texture
	UTexture2DArray* CheckerboardTexture = UTexture2DArray::CreateTransient(CheckerSize, CheckerSize, ArraySize, PF_B8G8R8A8);

	// Lock the checkerboard texture so it can be modified
	FColor* MipData = reinterpret_cast<FColor*>(CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));

	// Fill in the colors in a checkerboard pattern
	for (int32 ArrayIndex = 0; ArrayIndex < ArraySize; ArrayIndex++)
	{
		for (int32 RowIndex = 0; RowIndex < CheckerSize; RowIndex++)
		{
			for (int32 ColumnIndex = 0; ColumnIndex < CheckerSize; ColumnIndex++)
			{
				*MipData++ = ColumnIndex < HalfCheckerSize == RowIndex < HalfCheckerSize ? ColorOne : ColorTwo;
			}
		}
	}

	// Unlock the texture
	CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Unlock();
	CheckerboardTexture->UpdateResource();

	return CheckerboardTexture;
}

UTextureCube* FImageUtils::CreateCheckerboardCubeTexture(FColor ColorOne, FColor ColorTwo, int32 CheckerSize)
{
	CheckerSize = FMath::Min<uint32>(FMath::RoundUpToPowerOfTwo(CheckerSize), 4096);
	const int32 HalfPixelNum = CheckerSize >> 1;

	// Create the texture
	UTextureCube* CheckerboardTexture = UTextureCube::CreateTransient(CheckerSize, CheckerSize, PF_B8G8R8A8);

	// Lock the checkerboard texture so it can be modified
	FColor* MipData = reinterpret_cast<FColor*>(CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));

	// Fill in the colors in a checkerboard pattern
	for (int32 Face = 0; Face < 6; ++Face)
	{
		for (int32 RowNum = 0; RowNum < CheckerSize; ++RowNum)
		{
			for (int32 ColNum = 0; ColNum < CheckerSize; ++ColNum)
			{
				FColor& CurColor = MipData[(ColNum + (RowNum * CheckerSize))];

				if (ColNum < HalfPixelNum)
				{
					CurColor = (RowNum < HalfPixelNum) ? ColorOne : ColorTwo;
				}
				else
				{
					CurColor = (RowNum < HalfPixelNum) ? ColorTwo : ColorOne;
				}
			}
		}
		MipData += CheckerSize * CheckerSize;
	}

	// Unlock the texture
	CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Unlock();
	CheckerboardTexture->UpdateResource();

	return CheckerboardTexture;
}

UTextureCubeArray* FImageUtils::CreateCheckerboardTextureCubeArray(FColor ColorOne, FColor ColorTwo, int32 CheckerSize, int32 ArraySize)
{
	CheckerSize = FMath::Min<uint32>(FMath::RoundUpToPowerOfTwo(CheckerSize), 4096);
	const int32 HalfCheckerSize = CheckerSize >> 1;

	// Create the texture
	UTextureCubeArray* CheckerboardTexture = UTextureCubeArray::CreateTransient(CheckerSize, CheckerSize, ArraySize, PF_B8G8R8A8);

	// Lock the checkerboard texture so it can be modified
	FColor* MipData = reinterpret_cast<FColor*>(CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));

	// Fill in the colors in a checkerboard pattern
	for (int32 ArrayIndex = 0; ArrayIndex < ArraySize; ArrayIndex++)
	{
		for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			for (int32 RowIndex = 0; RowIndex < CheckerSize; RowIndex++)
			{
				for (int32 ColumnIndex = 0; ColumnIndex < CheckerSize; ColumnIndex++)
				{
					*MipData++ = ColumnIndex < HalfCheckerSize == RowIndex < HalfCheckerSize ? ColorOne : ColorTwo;
				}
			}
		}
	}

	// Unlock the texture
	CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Unlock();
	CheckerboardTexture->UpdateResource();

	return CheckerboardTexture;
}

UVolumeTexture* FImageUtils::CreateCheckerboardVolumeTexture(FColor ColorOne, FColor ColorTwo, int32 CheckerSize)
{
	CheckerSize = FMath::Min<uint32>(FMath::RoundUpToPowerOfTwo(CheckerSize), 4096);
	const int32 HalfCheckerSize = CheckerSize >> 1;

	// Create the texture
	UVolumeTexture* CheckerboardTexture = UVolumeTexture::CreateTransient(CheckerSize, CheckerSize, CheckerSize, PF_B8G8R8A8);

	// Lock the checkerboard texture so it can be modified
	FColor* MipData = reinterpret_cast<FColor*>(CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));

	// Fill in the colors in a checkerboard pattern
	for (int32 SliceIndex = 0; SliceIndex < CheckerSize; SliceIndex++)
	{
		for (int32 RowIndex = 0; RowIndex < CheckerSize; RowIndex++)
		{
			for (int32 ColumnIndex = 0; ColumnIndex < CheckerSize; ColumnIndex++)
			{
				*MipData++ = ColumnIndex < HalfCheckerSize == RowIndex < HalfCheckerSize ? ColorOne : ColorTwo;
			}
		}
	}

	// Unlock the texture
	CheckerboardTexture->GetPlatformData()->Mips[0].BulkData.Unlock();
	CheckerboardTexture->UpdateResource();

	return CheckerboardTexture;
}

/*------------------------------------------------------------------------------
HDR file format helper.
------------------------------------------------------------------------------*/
// DEPRECATED
// only used for cube maps for GenerateLongLatUnwrap
// do not use HDR for exports, it is very lossy, use EXR instead for float output
// if you need HDR, do not use this export code
// instead use HdrImageWrapper via FImageUtils::CompressImage
class FHDRExportHelper
{
public:

	/**
	* Writes HDR format image to an FArchive
	* This function unwraps the cube image on to a 2D surface.
	* @param TexCube - A cube source cube texture to read from.
	* @param Ar - Archive object to write HDR data to.
	* @return true on successful export.
	*/
	// this does GenerateLongLatUnwrap
	bool ExportHDR(UTextureCube* TexCube, FArchive& Ar)
	{
		check(TexCube != nullptr);

		// Generate 2D image.
		TArray64<uint8> RawData;
		bool bUnwrapSuccess = CubemapHelpers::GenerateLongLatUnwrap(TexCube, RawData, Size, Format);
		bool bAcceptableFormat = (Format == PF_B8G8R8A8 || Format == PF_FloatRGBA);
		if (bUnwrapSuccess == false || bAcceptableFormat == false)
		{
			return false;
		}

		WriteHDRImage(RawData, Ar);

		return true;
	}

	/**
	* Writes HDR format image to an FArchive
	* This function unwraps the cube image on to a 2D surface.
	* @param TexCube - A cube source render target cube texture to read from.
	* @param Ar - Archive object to write HDR data to.
	* @return true on successful export.
	*/
	// this does GenerateLongLatUnwrap
	bool ExportHDR(UTextureRenderTargetCube* TexCube, FArchive& Ar)
	{
		check(TexCube != nullptr);

		// Generate 2D image.
		TArray64<uint8> RawData;
		bool bUnwrapSuccess = CubemapHelpers::GenerateLongLatUnwrap(TexCube, RawData, Size, Format);
		bool bAcceptableFormat = (Format == PF_B8G8R8A8 || Format == PF_FloatRGBA);
		if (bUnwrapSuccess == false || bAcceptableFormat == false)
		{
			return false;
		}

		WriteHDRImage(RawData, Ar);

		return true;
	}

private:
	void WriteScanLine(FArchive& Ar, const TArray<uint8>& ScanLine)
	{
		const uint8* LineEnd = ScanLine.GetData() + ScanLine.Num();
		const uint8* LineSource = ScanLine.GetData();
		TArray<uint8> Output;
		Output.Reserve(ScanLine.Num() * 2);
		while (LineSource < LineEnd)
		{
			int32 CurrentPos = 0;
			int32 NextPos = 0;
			int32 CurrentRunLength = 0;
			while (CurrentRunLength <= 4 && NextPos < 128 && LineSource + NextPos < LineEnd)
			{
				CurrentPos = NextPos;
				CurrentRunLength = 0;
				while (CurrentRunLength < 127 && CurrentPos + CurrentRunLength < 128 && LineSource + NextPos < LineEnd && LineSource[CurrentPos] == LineSource[NextPos])
				{
					NextPos++;
					CurrentRunLength++;
				}
			}

			if (CurrentRunLength > 4)
			{
				// write a non run: LineSource[0] to LineSource[CurrentPos]
				if (CurrentPos > 0)
				{
					Output.Add(CurrentPos);
					for (int32 i = 0; i < CurrentPos; i++)
					{
						Output.Add(LineSource[i]);
					}
				}
				Output.Add((uint8)(128 + CurrentRunLength));
				Output.Add(LineSource[CurrentPos]);
			}
			else
			{
				// write a non run: LineSource[0] to LineSource[NextPos]
				Output.Add((uint8)(NextPos));
				for (int32 i = 0; i < NextPos; i++)
				{
					Output.Add((uint8)(LineSource[i]));
				}
			}
			LineSource += NextPos;
		}
		Ar.Serialize(Output.GetData(), Output.Num());
	}

	template<typename TSourceColorType>
	void WriteHDRBits(FArchive& Ar, TSourceColorType* SourceTexels)
	{
		const int32 NumChannels = 4;
		const int32 SizeX = Size.X;
		const int32 SizeY = Size.Y;
		TArray<uint8> ScanLine[NumChannels];
		for (int32 Channel = 0; Channel < NumChannels; Channel++)
		{
			ScanLine[Channel].Reserve(SizeX);
		}

		for (int32 y = 0; y < SizeY; y++)
		{
			// write RLE header
			uint8 RLEheader[4];
			RLEheader[0] = 2;
			RLEheader[1] = 2;
			RLEheader[2] = SizeX >> 8;
			RLEheader[3] = SizeX & 0xFF;
			Ar.Serialize(&RLEheader[0], sizeof(RLEheader));

			for (int32 Channel = 0; Channel < NumChannels; Channel++)
			{
				ScanLine[Channel].Reset();
			}

			for (int32 x = 0; x < SizeX; x++)
			{
				FLinearColor LinearColor(*SourceTexels);
				FColor RGBEColor = LinearColor.ToRGBE();

				ScanLine[0].Add(RGBEColor.R);
				ScanLine[1].Add(RGBEColor.G);
				ScanLine[2].Add(RGBEColor.B);
				ScanLine[3].Add(RGBEColor.A);
				SourceTexels++;
			}

			for (int32 Channel = 0; Channel < NumChannels; Channel++)
			{
				WriteScanLine(Ar, ScanLine[Channel]);
			}
		}
	}

	void WriteHDRHeader(FArchive& Ar)
	{
		const int32 MaxHeaderSize = 256;
		char Header[MAX_SPRINTF];
		FCStringAnsi::Sprintf(Header, "#?RADIANCE\nFORMAT=32-bit_rle_rgbe\n\n-Y %d +X %d\n", Size.Y, Size.X);
		Header[MaxHeaderSize - 1] = 0;
		int32 Len = FMath::Min(FCStringAnsi::Strlen(Header), MaxHeaderSize);
		Ar.Serialize(Header, Len);
	}

	void WriteHDRImage(const TArray64<uint8>& RawData, FArchive& Ar)
	{
		WriteHDRHeader(Ar);
		if (Format == PF_FloatRGBA)
		{
			WriteHDRBits(Ar, (FFloat16Color*)RawData.GetData());
		}
		else
		{
			check( Format == PF_B8G8R8A8 );
			WriteHDRBits(Ar, (FColor*)RawData.GetData());
		}
		// can't actually write BGRE8 pixels
		// use HdrImageWrapper instead
	}

	FIntPoint Size;
	EPixelFormat Format;
};

bool FImageUtils::ExportRenderTarget2DAsHDR(UTextureRenderTarget2D* TexRT, FArchive& Ar)
{
	FImage Image;
	if ( ! GetRenderTargetImage(TexRT,Image) )
	{
		return false;
	}

	TArray64<uint8> CompressedData;
	if ( ! CompressImage(CompressedData,TEXT("HDR"),Image) )
	{
		return false;
	}

	Ar.Serialize((void*)CompressedData.GetData(), CompressedData.GetAllocatedSize());

	return true;
}

bool FImageUtils::ExportRenderTarget2DAsPNG(UTextureRenderTarget2D* TexRT, FArchive& Ar)
{
	FImage Image;
	if ( ! GetRenderTargetImage(TexRT,Image) )
	{
		return false;
	}

	TArray64<uint8> CompressedData;
	if ( ! CompressImage(CompressedData,TEXT("PNG"),Image) )
	{
		return false;
	}

	Ar.Serialize((void*)CompressedData.GetData(), CompressedData.GetAllocatedSize());

	return true;
}

bool FImageUtils::ExportRenderTarget2DAsEXR(UTextureRenderTarget2D* TexRT, FArchive& Ar)
{
	FImage Image;
	if ( ! GetRenderTargetImage(TexRT,Image) )
	{
		return false;
	}

	TArray64<uint8> CompressedData;
	if ( ! CompressImage(CompressedData,TEXT("EXR"),Image) )
	{
		return false;
	}

	Ar.Serialize((void*)CompressedData.GetData(), CompressedData.GetAllocatedSize());

	return true;
}

bool FImageUtils::ExportTexture2DAsHDR(UTexture2D* Tex, FArchive& Ar)
{
	UE_LOGF(LogImageUtils, Warning, "HDR file format is very lossy, use EXR or PNG instead." );
	
	FImage Image;
	if ( ! GetTexture2DSourceImage(Tex,Image) )
	{
		return false;
	}

	TArray64<uint8> CompressedData;
	if ( ! CompressImage(CompressedData,TEXT("HDR"),Image) )
	{
		return false;
	}

	Ar.Serialize((void*)CompressedData.GetData(), CompressedData.GetAllocatedSize());

	return true;
}

// if Texture source is available, get it as an FImage
bool FImageUtils::GetTexture2DSourceImage(UTexture2D* Texture, FImage & OutImage)
{
#if WITH_EDITORONLY_DATA	
	return Texture->Source.GetMipImage(OutImage,0);
#else

	UE_LOGF(LogImageUtils,Warning,"GetTexture2DSourceImage from PlatformData not implemented yet");

	// @todo Oodle : could export texture from platformdata
	// only a few formats would be possible to grab to an FImage
	//   (for DDS export we could do all formats)
	// eg. see ExportHDR below

	/*
	EPixelFormat PixelFormat = Texture->GetFormat();
	// this is PlatformData GetMipData , not Source :
	Texture->GetMipData(0, (void**)RawData2.GetData());
	if (Texture->GetPlatformData()->Mips.Num() == 0)
	*/
	
	//uint8* MipData = static_cast<uint8*>(NewTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
	//int64 MipDataSize = NewTexture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();
	
	//if (NewFormat == PF_B8G8R8A8)
	//else if (NewFormat == PF_FloatRGBA)

	return false;
#endif
}

UTexture2D* FImageUtils::ImportFileAsTexture2D(const FString& Filename)
{
	UTexture2D* NewTexture = nullptr;
	TArray64<uint8> Buffer;
	if (FFileHelper::LoadFileToArray(Buffer, *Filename, FILEREAD_Silent))
	{
		// note this make a Transient / PlatformData only Texture (no TextureSource)
		NewTexture = FImageUtils::ImportBufferAsTexture2D(Buffer);

		if(!NewTexture)
		{
			UE_LOGF(LogImageUtils, Warning, "Error creating texture. %ls is not a supported file format", *Filename)
		}	
	}
	else
	{
		UE_LOGF(LogImageUtils, Warning, "Error creating texture. %ls could not be found", *Filename)
	}

	return NewTexture;
}

UTexture2D* FImageUtils::ImportBufferAsTexture2D(TArrayView64<const uint8> Buffer)
{
	FImage Image;
	if ( ! DecompressImage(Buffer.GetData(),Buffer.Num(), Image) )
	{
		UE_LOGF(LogImageUtils, Warning, "Error creating texture. Couldn't determine the file format");
		return nullptr;
	}

	// note this make a Transient / PlatformData only Texture (no TextureSource)
	return CreateTexture2DFromImage(Image);
}

UTexture2D* FImageUtils::CreateTexture2DFromImage(const FImageView & Image)
{
	ERawImageFormat::Type PixelFormatRawFormat;
	EPixelFormat PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(Image.Format,&PixelFormatRawFormat);
						
	UTexture2D* NewTexture = UTexture2D::CreateTransient(Image.SizeX, Image.SizeY, PixelFormat);
	if (NewTexture == nullptr)
	{
		UE_LOGF(LogImageUtils, Warning, "Error in CreateTransient");
		return nullptr;
	}

	NewTexture->bNotOfflineProcessed = true;

	uint8* MipData = static_cast<uint8*>(NewTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
	check( MipData != nullptr );
	int64 MipDataSize = NewTexture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();

	FImageView MipImage(MipData,Image.SizeX,Image.SizeY,1,PixelFormatRawFormat,Image.GammaSpace);
	check( MipImage.GetImageSizeBytes() <= MipDataSize ); // is it exactly == ?

	// copy into texture and convert if necessary :
	FImageCore::CopyImage(Image,MipImage);
				
	NewTexture->GetPlatformData()->Mips[0].BulkData.Unlock();

	NewTexture->UpdateResource();

	return NewTexture;
}

UTexture2D* FImageUtils::ImportBufferAsTexture2D(const TArray<uint8>& Buffer)
{
	return ImportBufferAsTexture2D(TArrayView64<const uint8>(Buffer.GetData(), Buffer.Num()));
}

bool FImageUtils::ExportRenderTargetCubeAsHDR(UTextureRenderTargetCube* TexRT, FArchive& Ar)
{
	// this does GenerateLongLatUnwrap
	FHDRExportHelper Exporter;
	return Exporter.ExportHDR(TexRT, Ar);
}

bool FImageUtils::ExportTextureCubeAsHDR(UTextureCube* TexRT, FArchive& Ar)
{
	// this does GenerateLongLatUnwrap
	FHDRExportHelper Exporter;
	return Exporter.ExportHDR(TexRT, Ar);
}

#undef LOCTEXT_NAMESPACE
