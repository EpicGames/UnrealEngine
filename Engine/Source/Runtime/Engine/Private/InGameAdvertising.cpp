// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InGameAdvertising.cpp: Base implementation for ingame advertising management
=============================================================================*/ 

#include "Engine/InGameAdManager.h"
#include "Engine/PlatformInterfaceBase.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

UInGameAdManager::UInGameAdManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bShouldPauseWhileAdOpen = true;
}


void UInGameAdManager::Init()
{
}
void UInGameAdManager::ShowBanner(bool bShowOnBottomOfScreen)
{
}
void UInGameAdManager::HideBanner()
{
}
void UInGameAdManager::ForceCloseAd()
{
}


void UInGameAdManager::OnUserClickedBanner()
{
	// pause if we want to pause while ad is open - This will need to be revised if we do !	
// 	if (bShouldPauseWhileAdOpen && GetNetMode() == NM_Standalone)
// 	{
// 		// pause the first player controller
// 		if (GEngine && GEngine->GamePlayers.Num() && GEngine->GamePlayers[0])
// 		{
// 			GEngine->GamePlayers[0]->PlayerController->ConsoleCommand(TEXT("PAUSE"));
// 		}
// 	}

	// call the delegates
	const TArray<FOnUserClickedBanner> Delegates = ClickedBannerDelegates;
	// Iterate through the delegate list
	for (int32 Index = 0; Index < Delegates.Num(); Index++)
	{
		// Send the notification of completion
		Delegates[Index].ExecuteIfBound();
	}
}



void UInGameAdManager::OnUserClosedAd()
{
	// unpause if we want to pause while ad is open - This will need to be revised if we do !	
// 	if (bShouldPauseWhileAdOpen && GetNetMode() == NM_Standalone)
// 	{
// 		if (GEngine && GEngine->GamePlayers.Num() && GEngine->GamePlayers[0])
// 		{
// 			GEngine->GamePlayers[0]->PlayerController->ConsoleCommand(TEXT("PAUSE"));
// 		}
// 	}

	// call the delegates
	const TArray<FOnUserClosedAdvertisement> Delegates = ClosedAdDelegates;
	// Iterate through the delegate list
	for (int32 Index = 0; Index < Delegates.Num(); Index++)
	{
		// Send the notification of completion
		Delegates[Index].ExecuteIfBound();
	}
}

void UInGameAdManager::SetPauseWhileAdOpen(bool bShouldPause)
{
	bShouldPauseWhileAdOpen = bShouldPause;
}



void UInGameAdManager::AddClickedBannerDelegate(FOnUserClickedBanner InDelegate)
{
	// Add this delegate to the array if not already present
	if (ClickedBannerDelegates.Find(InDelegate) == INDEX_NONE)
	{
		ClickedBannerDelegates.Add(InDelegate);
	}
}


void UInGameAdManager::ClearClickedBannerDelegate(FOnUserClickedBanner InDelegate)
{
	// Remove this delegate from the array if found
	int32 RemoveIndex = ClickedBannerDelegates.Find(InDelegate);
	if (RemoveIndex != INDEX_NONE)
	{
		ClickedBannerDelegates.RemoveAt(RemoveIndex,1);
	}
}



void UInGameAdManager::AddClosedAdDelegate(FOnUserClosedAdvertisement InDelegate)
{
	// Add this delegate to the array if not already present
	if (ClosedAdDelegates.Find(InDelegate) == INDEX_NONE)
	{
		ClosedAdDelegates.Add(InDelegate);
	}
}

void UInGameAdManager::ClearClosedAdDelegate(FOnUserClosedAdvertisement InDelegate)
{
	// Remove this delegate from the array if found
	int32 RemoveIndex = ClosedAdDelegates.Find(InDelegate);
	if (RemoveIndex != INDEX_NONE)
	{
		ClosedAdDelegates.RemoveAt(RemoveIndex,1);
	}
}
