// Copyright Epic Games, Inc. All Rights Reserved.

#include "InGamePerformanceTracker.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogInGamePerformanceTracker, Log, All);

//////////////////////////////////////////////////////////////////////////

void FInGameCycleHistory::NextFrame()
{
	check(FrameIdx < FrameCycles.Num());

	//Replace the current frame cycles with the existing frame in the history.
	TotalCycles += CurrFrameCycles;
	TotalCycles -= FrameCycles[FrameIdx];
	FrameCycles[FrameIdx] = CurrFrameCycles;
	CurrFrameCycles = 0;

	FrameIdx = (FrameIdx + 1) % FrameCycles.Num();
	CachedAverageCycles = TotalCycles / FrameCycles.Num();
}

//////////////////////////////////////////////////////////////////////////

IConsoleVariable* FInGamePerformanceTracker::Enabled = IConsoleManager::Get().RegisterConsoleVariable(TEXT("InGamePerformanceTracking.Enabled"), 0, TEXT("If in-game performance tracking is enabled. Most games will likely not use or need this so it should be left disabled."), ECVF_Default);
int32 FInGamePerformanceTracker::CachedEnabled;
IConsoleVariable* FInGamePerformanceTracker::HistorySize = IConsoleManager::Get().RegisterConsoleVariable(TEXT("InGamePerformanceTracking.HistorySize"), 30, TEXT("How many frames in game performance tracking should store in it's history."), ECVF_Default);

FInGamePerformanceTracker::FInGamePerformanceTracker()
: History(HistorySize->GetInt())
, DirectSectionTime_EntryCount(0)
, DirectSectionTime_BeginCycles(0)
{

}

FInGamePerformanceTracker::FInGamePerformanceTracker(uint32 FrameHistorySize)
: History(FrameHistorySize)
, DirectSectionTime_EntryCount(0)
, DirectSectionTime_BeginCycles(0)
{
}

void FInGamePerformanceTracker::Tick()
{
	check(DirectSectionTime_EntryCount == 0);//There's a mis matched direct section tracking entry/exit calls.
	CachedEnabled = Enabled->GetInt();
	if (CachedEnabled)
	{
		History.NextFrame();
	}

	//UE_LOGF(LogInGamePerformanceTracker, Log, "%ls - %6.4f", *Name.ToString(), GetAverageTimeSeconds());
}	

void FInGamePerformanceTracker::EnterTimedSection()
{
	check(IsInGameThread());//This is only safe single threaded so for now assume it must be game thread.
	if (DirectSectionTime_EntryCount++ == 0 && CachedEnabled)
	{
		DirectSectionTime_BeginCycles = FPlatformTime::Cycles();
	}
}

void FInGamePerformanceTracker::ExitTimedSection()
{
	check(IsInGameThread());//This is only safe single threaded so for now assume it must be game thread.
	check(DirectSectionTime_EntryCount > 0); //Mismatched calls to enter/exit
	if (--DirectSectionTime_EntryCount == 0 && CachedEnabled)
	{
		History.AddCycles(FPlatformTime::Cycles() - DirectSectionTime_BeginCycles);
		DirectSectionTime_BeginCycles = 0;
	}
}

//////////////////////////////////////////////////////////////////////////

FInGameScopedCycleCounter::FInGameScopedCycleCounter(class UWorld* InWorld, EInGamePerfTrackers Tracker, EInGamePerfTrackerThreads TrackerThread, bool bEnabled)
: FInGameCycleCounter(InWorld && bEnabled && InWorld->PerfTrackers ? &InWorld->PerfTrackers->GetInGamePerformanceTracker(Tracker, TrackerThread) : nullptr)
{
	Begin();
}

FInGameScopedCycleCounter::~FInGameScopedCycleCounter()
{
	End();
}


//////////////////////////////////////////////////////////////////////////

FWorldInGamePerformanceTrackers::FWorldInGamePerformanceTrackers()
{

}

FWorldInGamePerformanceTrackers::~FWorldInGamePerformanceTrackers()
{

}
