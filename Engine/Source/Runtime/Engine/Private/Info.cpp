// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/Info.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Info)

AInfo::AInfo(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	RootComponent = SpriteComponent;
	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTexture;
			FName ID_Info;
			FText NAME_Info;
			FConstructorStatics()
				: SpriteTexture(TEXT("/Engine/EditorResources/S_Actor"))
				, ID_Info(TEXT("Info"))
				, NAME_Info(NSLOCTEXT("SpriteCategory", "Info", "Info"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.SpriteTexture.Get();
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Info;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Info;
		SpriteComponent->bIsScreenSizeScaled = true;
	}

	bIsSpatiallyLoaded = false;
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = false;
	bAllowTickBeforeBeginPlay = true;
	bReplicates = false;
	SetNetUpdateFrequency(10.0f);
	SetHidden(true);
	SetReplicatingMovement(false);
	SetCanBeDamaged(false);
	bEnableAutoLODGeneration = false;
}

#if WITH_EDITORONLY_DATA
/** Returns SpriteComponent subobject **/
UBillboardComponent* AInfo::GetSpriteComponent() const { return SpriteComponent; }
#endif

#if WITH_EDITOR
void AInfo::PostLoad()
{
	Super::PostLoad();

	// Fixes Actors that were saved prior to their constructor using ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite"))
	// Those Components are MarkForGarbage in their UActorComponent::PostInitProperties and so we need to clear the Component pointers
	if (RootComponent != nullptr && !IsValid(RootComponent))
	{
		RootComponent = nullptr;
	}

#if WITH_EDITORONLY_DATA
	if (SpriteComponent != nullptr && !IsValid(SpriteComponent))
	{
		SpriteComponent = nullptr;
	}
#endif
}
#endif
