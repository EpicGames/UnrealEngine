// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InheritableComponentHandler.h"
#include "Engine/Engine.h"
#include "Engine/SCS_Node.h"
#include "EngineLogs.h"
#include "UObject/BlueprintsObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InheritableComponentHandler)

#if WITH_EDITOR
#include "Kismet2/BlueprintEditorUtils.h"
#else
#include "UObject/LinkerLoad.h"
#endif // WITH_EDITOR

// UInheritableComponentHandler

const FString UInheritableComponentHandler::SCSDefaultSceneRootOverrideNamePrefix(TEXT("ICH-"));

void UInheritableComponentHandler::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);
}

void UInheritableComponentHandler::FixComponentTemplateNames()
{
#if WITH_EDITOR
	if (!GIsDuplicatingClassForReinstancing)
	{
		for (int32 Index = Records.Num() - 1; Index >= 0; --Index)
		{
			FComponentOverrideRecord& Record = Records[Index];
			if (Record.ComponentTemplate)
			{
				if (GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::SCSHasComponentTemplateClass)
				{
					// Fix up component class on load, if it's not already set.
					if (Record.ComponentClass == nullptr)
					{
						Record.ComponentClass = Record.ComponentTemplate->GetClass();
					}
				}

				// Fix up component template name on load, if it doesn't match the original template name. Otherwise, archetype lookups will fail for this template.
				// For example, this can occur after a component variable rename in a parent BP class, but before a child BP class with an override template is loaded.
				// Note: If the key maps to an SCS node, the node's variable GUID will be used for the lookup instead of the name below (that's only used for UCS keys).
				if (UActorComponent* OriginalTemplate = Record.ComponentKey.GetOriginalTemplate(Record.ComponentTemplate->GetFName()))
				{
					FName ExpectedTemplateName = OriginalTemplate->GetFName();
					if (USCS_Node* SCSNode = Record.ComponentKey.FindSCSNode())
					{
						// We append a prefix onto SCS default scene root node overrides. This is done to ensure that the override template does not collide with our owner's own SCS default scene root node template.
						if (SCSNode == SCSNode->GetSCS()->GetDefaultSceneRootNode())
						{
							ExpectedTemplateName = FName(SCSDefaultSceneRootOverrideNamePrefix + ExpectedTemplateName.ToString());
						}
					}

					if (ExpectedTemplateName != Record.ComponentTemplate->GetFName())
					{
						FixComponentTemplateName(Record.ComponentTemplate, ExpectedTemplateName);
					}
				}

				if (!CastChecked<UActorComponent>(Record.ComponentTemplate->GetArchetype())->IsEditableWhenInherited())
				{
					Record.ComponentTemplate->MarkAsGarbage(); // hack needed to be able to identify if NewObject returns this back to us in the future
					Records.RemoveAtSwap(Index);
				}
			}
		}
	}
#endif
}

#if WITH_EDITOR
EDataValidationResult UInheritableComponentHandler::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	// If nothing has performed any validation yet, start with a valid result.
	if (Result == EDataValidationResult::NotValidated)
	{
		Result = EDataValidationResult::Valid;
	}

	for (const FComponentOverrideRecord& Record : Records)
	{
		if (const UActorComponent* ActorComponent = Record.ComponentTemplate)
		{
			EDataValidationResult OverrideResult = ActorComponent->IsDataValid(Context);
			Result = CombineDataValidationResults(Result, OverrideResult);
		}
	}

	return Result;
}

UActorComponent* UInheritableComponentHandler::CreateOverridenComponentTemplate(const FComponentKey& Key)
{
	for (int32 Index = 0; Index < Records.Num(); ++Index)
	{
		FComponentOverrideRecord& Record = Records[Index];
		if (Record.ComponentKey.Match(Key))
		{
			if (Record.ComponentTemplate)
			{
				return Record.ComponentTemplate;
			}
			Records.RemoveAtSwap(Index);
			break;
		}
	}

	UActorComponent* BestArchetype = FindBestArchetype(Key);
	if (!BestArchetype)
	{
		UE_LOGF(LogBlueprint, Warning, "CreateOverridenComponentTemplate '%ls': cannot find archetype for component '%ls' from '%ls'",
			*GetPathNameSafe(this), *Key.GetSCSVariableName().ToString(), *GetPathNameSafe(Key.GetComponentOwner()));
		return NULL;
	}
	
	FName NewComponentTemplateName = BestArchetype->GetFName();
	if (USCS_Node* SCSNode = Key.FindSCSNode())
	{
		const USCS_Node* DefaultSceneRootNode = SCSNode->GetSCS()->GetDefaultSceneRootNode();

		// If this template will override an inherited DefaultSceneRoot node from a parent class's SCS, adjust the template name so that we don't reallocate our owner class's SCS DefaultSceneRoot node template.
		// Note: This is currently the only case where a child class can have both an SCS node template and an override template associated with the same variable name, that is not considered to be a collision.
		if (SCSNode == DefaultSceneRootNode && BestArchetype == DefaultSceneRootNode->ComponentTemplate)
		{
			NewComponentTemplateName = FName(*(SCSDefaultSceneRootOverrideNamePrefix + BestArchetype->GetName()));
		}
	}

	ensure(Cast<UBlueprintGeneratedClass>(GetOuter()));
	
	// If we find an existing object with our name that the object recycling system won't allow for we need to deal with it 
	// or else the NewObject call below will fatally assert
	UObject* ExistingObj = FindObjectFast<UObject>(GetOuter(), NewComponentTemplateName);
	if (ExistingObj && (!ExistingObj->GetClass()->IsChildOf(BestArchetype->GetClass()) || ExistingObj->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad)))
	{
		// If this isn't an unnecessary component there is something else we need to investigate
		// but if it is, just consign it to oblivion as its purpose is no longer required with the allocation
		// of an object of the same name
		UActorComponent* ExistingComp = Cast<UActorComponent>(ExistingObj);
		if (ensure(ExistingComp) && ensure(UnnecessaryComponents.RemoveSwap(ExistingComp) > 0 || GetPackage()->HasAnyPackageFlags(PKG_ForDiffing)))
		{
			ExistingObj->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
			ExistingObj->MarkAsGarbage();
		}
	}

	UActorComponent* NewComponentTemplate = NewObject<UActorComponent>(
		GetOuter(), BestArchetype->GetClass(), NewComponentTemplateName, RF_ArchetypeObject | RF_Public | RF_InheritableComponentTemplate, BestArchetype);

	// HACK: NewObject can return a pre-existing object which will not have been initialized to the archetype.  When we remove the old handlers, we mark them pending
	//       kill so we can identify that situation here (see UE-13987/UE-13990)
	if (!::IsValid(NewComponentTemplate))
	{
		NewComponentTemplate->ClearGarbage();
		UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
		CopyParams.bDoDelta = false;
		// No good can come of replacing references to BestArchetype with references to NewComponentTemplate
		CopyParams.bNotifyObjectReplacement = false;
		UEngine::CopyPropertiesForUnrelatedObjects(BestArchetype, NewComponentTemplate, CopyParams);
	}

	// Clear transient flag if it was transient before and re copy off archetype
	if (NewComponentTemplate->HasAnyFlags(RF_Transient))
	{
		const int32 ComponentIndex = UnnecessaryComponents.Find(NewComponentTemplate);
		if (ComponentIndex != INDEX_NONE)
		{
			NewComponentTemplate->ClearFlags(RF_Transient);
			UnnecessaryComponents.RemoveAtSwap(ComponentIndex);

			UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
			CopyParams.bDoDelta = false;
			// No good can come of replacing references to BestArchetype with references to NewComponentTemplate
			CopyParams.bNotifyObjectReplacement = false;
			UEngine::CopyPropertiesForUnrelatedObjects(BestArchetype, NewComponentTemplate, CopyParams);
		}
	}

	FComponentOverrideRecord NewRecord;
	NewRecord.ComponentKey = Key;
	NewRecord.ComponentClass = NewComponentTemplate->GetClass();
	NewRecord.ComponentTemplate = NewComponentTemplate;
	Records.Emplace(MoveTemp(NewRecord));

	return NewComponentTemplate;
}

void UInheritableComponentHandler::RemoveOverridenComponentTemplate(const FComponentKey& Key)
{
	for (int32 Index = 0; Index < Records.Num(); ++Index)
	{
		const FComponentOverrideRecord& Record = Records[Index];
		if (Record.ComponentKey.Match(Key))
		{
			Record.ComponentTemplate->MarkAsGarbage(); // hack needed to be able to identify if NewObject returns this back to us in the future
			Records.RemoveAtSwap(Index);
			break;
		}
	}
}

void UInheritableComponentHandler::UpdateOwnerClass(UBlueprintGeneratedClass* OwnerClass)
{
	for (FComponentOverrideRecord& Record : Records)
	{
		UActorComponent* OldComponentTemplate = Record.ComponentTemplate;
		if (OldComponentTemplate && (OwnerClass != OldComponentTemplate->GetOuter()))
		{
			Record.ComponentTemplate = DuplicateObject(OldComponentTemplate, OwnerClass, OldComponentTemplate->GetFName());
		}
	}
}

void UInheritableComponentHandler::ValidateTemplates()
{
	for (int32 Index = 0; Index < Records.Num();)
	{
		bool bIsValidAndNecessary = false;
		FComponentOverrideRecord& Record = Records[Index];
		{
			FComponentKey& ComponentKey = Record.ComponentKey;
			
			FName VarName = ComponentKey.GetSCSVariableName();
			if (ComponentKey.RefreshVariableName())
			{
				FName NewName = ComponentKey.GetSCSVariableName();
				UE_LOGF(LogBlueprint, Log, "ValidateTemplates '%ls': variable old name '%ls' new name '%ls'",
					*GetPathNameSafe(this), *VarName.ToString(), *NewName.ToString());
				VarName = NewName;

				MarkPackageDirty();
			}

			if (IsRecordValid(Record))
			{
				if (IsRecordNecessary(Record))
				{
					bIsValidAndNecessary = true;
				}
				else
				{
					// Set transient flag so this object does not get used as an archetype for subclasses
					if (Record.ComponentTemplate)
					{
						Record.ComponentTemplate->SetFlags(RF_Transient);
						UnnecessaryComponents.AddUnique(Record.ComponentTemplate);
					}

					UE_LOGF(LogBlueprint, Log, "ValidateTemplates '%ls': overridden template is unnecessary and will be removed - component '%ls' from '%ls'",
						*GetPathNameSafe(this), *VarName.ToString(), *GetPathNameSafe(ComponentKey.GetComponentOwner()));
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Log, "ValidateTemplates '%ls': overridden template is invalid and will be removed - component '%ls' from '%ls' (this can happen when a class was recently reparented)",
					*GetPathNameSafe(this), *VarName.ToString(), *GetPathNameSafe(ComponentKey.GetComponentOwner()));
			}
		}

		if (bIsValidAndNecessary)
		{
			++Index;
		}
		else
		{
#if WITH_EDITOR
			if (Record.ComponentTemplate)
			{
				// Mark the export as invalid to ensure the loader won't reload the object, 
				// should the package owning this object be reloaded.
				// Note, Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
				FLinkerLoad::InvalidateExport(Record.ComponentTemplate);
				// in editor, move the component template aside so its name is free:
				Record.ComponentTemplate->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
				Record.ComponentTemplate->ClearFlags(RF_Standalone);
				Record.ComponentTemplate->RemoveFromRoot();
			}
#endif // WITH_EDITOR
			Records.RemoveAtSwap(Index);
		}
	}
}

bool UInheritableComponentHandler::IsValid() const
{
	for (const FComponentOverrideRecord& Record : Records)
	{
		if (!IsRecordValid(Record))
		{
			return false;
		}
	}
	return true;
}

bool UInheritableComponentHandler::IsRecordValid(const FComponentOverrideRecord& Record) const
{
	UClass* OwnerClass = Cast<UClass>(GetOuter());
	if (!ensure(OwnerClass))
	{
		return false;
	}

	if (!Record.ComponentTemplate)
	{
		// Note: We still consider the record to be valid, even if the template is missing, if we have valid class information. This typically will indicate that the template object was filtered at load time (to save space, e.g. dedicated server).
		return Record.ComponentClass != nullptr;
	}

	if (Record.ComponentTemplate->GetOuter() != OwnerClass)
	{
		return false;
	}

	if (!Record.ComponentKey.IsValid())
	{
		return false;
	}

	UClass* ComponentOwner = Record.ComponentKey.GetComponentOwner();
	if (!ComponentOwner || !OwnerClass->IsChildOf(ComponentOwner))
	{
		return false;
	}

	// Note: If the original template is missing, we consider the record to be unnecessary, but not invalid.
	UActorComponent* OriginalTemplate = Record.ComponentKey.GetOriginalTemplate(Record.ComponentTemplate->GetFName());
	if (OriginalTemplate != nullptr && OriginalTemplate->GetClass() != Record.ComponentTemplate->GetClass())
	{
		return false;
	}
	
	return true;
}

struct FComponentComparisonHelper
{
	static bool AreIdentical(UObject* ObjectA, UObject* ObjectB)
	{
		if (!ObjectA || !ObjectB || (ObjectA->GetClass() != ObjectB->GetClass()))
		{
			return false;
		}

		bool Result = true;
		for (FProperty* Prop = ObjectA->GetClass()->PropertyLink; Prop && Result; Prop = Prop->PropertyLinkNext)
		{
			bool bConsiderProperty = Prop->ShouldDuplicateValue(); //Should the property be compared at all?
			if (bConsiderProperty)
			{
				for (int32 Idx = 0; (Idx < Prop->ArrayDim) && Result; Idx++)
				{
					if (!Prop->Identical_InContainer(ObjectA, ObjectB, Idx, PPF_DeepComparison))
					{
						Result = false;
					}
				}
			}
		}
		if (Result)
		{
			// Allow the component to compare its native/ intrinsic properties.
			Result = ObjectA->AreNativePropertiesIdenticalTo(ObjectB);
		}
		return Result;
	}
};

bool UInheritableComponentHandler::IsRecordNecessary(const FComponentOverrideRecord& Record) const
{
	// If the record's template was not loaded, check to see if the class information is valid.
	if (Record.ComponentTemplate == nullptr)
	{
		if (Record.ComponentClass != nullptr)
		{
			UObject* ComponentCDO = Record.ComponentClass->GetDefaultObject();
			if (ComponentCDO != nullptr)
			{
				// The record is considered necessary if the class information is valid but the template was not loaded due to client/server exclusion at load time (e.g. uncooked dedicated server).
				return !UObject::CanCreateInCurrentContext(ComponentCDO);
			}
		}
		
		// Otherwise, we don't need to keep the record if the template is NULL.
		return false;
	}
	else
	{
		const FName TemplateName = Record.ComponentTemplate->GetFName();

		// Consider the record to be unnecessary if the original template no longer exists.
		UActorComponent* OriginalTemplate = Record.ComponentKey.GetOriginalTemplate(TemplateName);
		if (OriginalTemplate == nullptr)
		{
			return false;
		}
	
		UActorComponent* ChildComponentTemplate = Record.ComponentTemplate;
		UActorComponent* ParentComponentTemplate = FindBestArchetype(Record.ComponentKey, TemplateName);
		check(ChildComponentTemplate && ParentComponentTemplate && (ParentComponentTemplate != ChildComponentTemplate));
		return !FComponentComparisonHelper::AreIdentical(ChildComponentTemplate, ParentComponentTemplate);
	}
}

UActorComponent* UInheritableComponentHandler::FindBestArchetype(const FComponentKey& Key, const FName& TemplateName) const
{
	UActorComponent* ClosestArchetype = nullptr;

	UBlueprintGeneratedClass* ActualBPGC = Cast<UBlueprintGeneratedClass>(GetOuter());
	if (ActualBPGC && Key.GetComponentOwner() && (ActualBPGC != Key.GetComponentOwner()))
	{
		if (UBlueprint* ClassGeneratedByBP = Cast<UBlueprint>(ActualBPGC->ClassGeneratedBy))
		{
			// During reparenting the outer's Class isn't always the Blueprint's class when the ICH is updating so reference
			// the Blueprint's ParentClass instead
			ActualBPGC = Cast<UBlueprintGeneratedClass>(ClassGeneratedByBP->ParentClass);
		}
		else
		{
			// Blueprint assets aren't available in cooked editors (only the BPGC), so just use the super class directly
			// since we know it will be up-to-date
			ActualBPGC = Cast<UBlueprintGeneratedClass>(ActualBPGC->GetSuperClass());
		}

		while (!ClosestArchetype && ActualBPGC)
		{
			if (ActualBPGC->InheritableComponentHandler)
			{
				ClosestArchetype = ActualBPGC->InheritableComponentHandler->GetOverridenComponentTemplate(Key);
			}
			ActualBPGC = Cast<UBlueprintGeneratedClass>(ActualBPGC->GetSuperClass());
		}

		if (!ClosestArchetype)
		{
			ClosestArchetype = Key.GetOriginalTemplate(TemplateName);
		}
	}

	return ClosestArchetype;
}

bool UInheritableComponentHandler::RefreshTemplateName(const FComponentKey& OldKey)
{
	for (FComponentOverrideRecord& Record : Records)
	{
		if (Record.ComponentKey.Match(OldKey))
		{
			Record.ComponentKey.RefreshVariableName();
			return true;
		}
	}
	return false;
}

FComponentKey UInheritableComponentHandler::FindKey(UActorComponent* ComponentTemplate) const
{
	for (const FComponentOverrideRecord& Record : Records)
	{
		if (Record.ComponentTemplate == ComponentTemplate)
		{
			return Record.ComponentKey;
		}
	}
	return FComponentKey();
}

#endif

void UInheritableComponentHandler::PreloadAllTemplates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInheritableComponentHandler::PreloadAllTemplates);

	for (const FComponentOverrideRecord& Record : Records)
	{
		if (Record.ComponentTemplate)
		{
			Record.ComponentTemplate->ConditionalPreload();

			TArray<UObject*> ComponentTemplateSubobjects;
			// Can't use ForEachObjectWithOuter here as Preloading may modify UObject hash tables (it will most likely create new objects)
			GetObjectsWithOuter(Record.ComponentTemplate, ComponentTemplateSubobjects);
			for (UObject* SubObj : ComponentTemplateSubobjects)
			{
				SubObj->ConditionalPreload();
			}
		}
	}
}

void UInheritableComponentHandler::PreloadAll()
{
	ConditionalPreload();
	
	PreloadAllTemplates();
	// This will get component names up to date - since these component 
	// templates are used by GetArchetypeFromRequiredInfo we want to have real
	// names as quickly as possible. The circumstances that require
	// this are not clear to me, but that the logic exists and occasionally
	// runs means we should run it ASAP - otherwise we may use the wrong
	// archetype on construction.
	FixComponentTemplateNames();
}

FComponentKey UInheritableComponentHandler::FindKey(const FName VariableName) const
{
	for (const FComponentOverrideRecord& Record : Records)
	{
		if (Record.ComponentKey.GetSCSVariableName() == VariableName || (Record.ComponentTemplate && Record.ComponentTemplate->GetFName() == VariableName))
		{
			return Record.ComponentKey;
		}
	}
	return FComponentKey();
}

UActorComponent* UInheritableComponentHandler::GetOverridenComponentTemplate(const FComponentKey& Key) const
{
	const FComponentOverrideRecord* Record = FindRecord(Key);
	return Record ? Record->ComponentTemplate : nullptr;
}

const FBlueprintCookedComponentInstancingData* UInheritableComponentHandler::GetOverridenComponentTemplateData(const FComponentKey& Key) const
{
	const FComponentOverrideRecord* Record = FindRecord(Key);
	return Record ? &Record->CookedComponentInstancingData : nullptr;
}

const FComponentOverrideRecord* UInheritableComponentHandler::FindRecord(const FComponentKey& Key) const
{
	for (const FComponentOverrideRecord& Record : Records)
	{
		if (Record.ComponentKey.Match(Key))
		{
			return &Record;
		}
	}
	return nullptr;
}

void UInheritableComponentHandler::FixComponentTemplateName(UActorComponent* ComponentTemplate, const FName NewName)
{
	// If we found a collision, temporarily rename the associated template object to something unique so that it no longer
	// collides with the one we're trying to correct here. This will be fixed up when we later encounter this record during
	// PostLoad() validation and see that it still doesn't match its original template name.
	if (UObject* ExistingObject = (UObject*)FindObjectWithOuter(ComponentTemplate->GetOuter(), nullptr, NewName))
	{
		ExistingObject->Rename(nullptr, nullptr, REN_AllowPackageLinkerMismatch | REN_DontCreateRedirectors);
	}

	// Now that we're sure there are no collisions with other records, we can safely rename this one to its new name.
	ComponentTemplate->Rename(*NewName.ToString(), nullptr, REN_AllowPackageLinkerMismatch | REN_DontCreateRedirectors);
}

// FComponentOverrideRecord

FComponentKey::FComponentKey(const USCS_Node* SCSNode) : OwnerClass(nullptr)
{
	if (SCSNode)
	{
		const USimpleConstructionScript* ParentSCS = SCSNode->GetSCS();
		OwnerClass      = ParentSCS ? ParentSCS->GetOwnerClass() : nullptr;
		AssociatedGuid  = SCSNode->VariableGuid;
		SCSVariableName = SCSNode->GetVariableName();
	}
}

#if WITH_EDITOR
FComponentKey::FComponentKey(UBlueprint* Blueprint, const FUCSComponentId& UCSComponentID)
{
	OwnerClass     = Blueprint->GeneratedClass;
	AssociatedGuid = UCSComponentID.GetAssociatedGuid();
}
#endif // WITH_EDITOR

bool FComponentKey::Match(const FComponentKey& OtherKey) const
{
	return (OwnerClass == OtherKey.OwnerClass) && (AssociatedGuid == OtherKey.AssociatedGuid);
}

USCS_Node* FComponentKey::FindSCSNode() const
{
	USimpleConstructionScript* ParentSCS = (OwnerClass && IsSCSKey()) ? CastChecked<UBlueprintGeneratedClass>(OwnerClass)->SimpleConstructionScript : nullptr;
	return ParentSCS ? ParentSCS->FindSCSNodeByGuid(AssociatedGuid) : nullptr;
}

UActorComponent* FComponentKey::GetOriginalTemplate(const FName& TemplateName) const
{
	UActorComponent* ComponentTemplate = nullptr;
	if (IsSCSKey())
	{
		if (USCS_Node* SCSNode = FindSCSNode())
		{
			ComponentTemplate = SCSNode->ComponentTemplate;
		}
	}
#if WITH_EDITOR
	else if (IsUCSKey())
	{
		ComponentTemplate = FBlueprintEditorUtils::FindUCSComponentTemplate(*this, TemplateName);
	}
#endif // WITH_EDITOR
	return ComponentTemplate;
}

bool FComponentKey::RefreshVariableName()
{
	if (IsValid() && IsSCSKey())
	{
		USCS_Node* SCSNode = FindSCSNode();
		const FName UpdatedName = SCSNode ? SCSNode->GetVariableName() : NAME_None;

		if (UpdatedName != SCSVariableName)
		{
			SCSVariableName = UpdatedName;
			return true;
		}
	}
	return false;
}

