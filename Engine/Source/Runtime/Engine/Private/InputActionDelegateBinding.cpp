// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputActionDelegateBinding.h"
#include "Components/InputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputActionDelegateBinding)

UInputActionDelegateBinding::UInputActionDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInputActionDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	TArray<FInputActionBinding> BindsToAdd;

	for (int32 BindIndex=0; BindIndex < InputActionDelegateBindings.Num(); ++BindIndex)
	{
		const FBlueprintInputActionDelegateBinding& Binding = InputActionDelegateBindings[BindIndex];

		FInputActionBinding AB( Binding.InputActionName, Binding.InputKeyEvent );
		AB.bConsumeInput = Binding.bConsumeInput;
		AB.bExecuteWhenPaused = Binding.bExecuteWhenPaused;
		AB.ActionDelegate.BindDelegate(ObjectToBindTo, Binding.FunctionNameToBind);

		if (Binding.bOverrideParentBinding)
		{
			for (int32 ExistingIndex = InputComponent->GetNumActionBindings() - 1; ExistingIndex >= 0; --ExistingIndex)
			{
				const FInputActionBinding& ExistingBind = InputComponent->GetActionBinding(ExistingIndex);
				if (ExistingBind.GetActionName() == AB.GetActionName() && ExistingBind.KeyEvent == AB.KeyEvent)
				{
					InputComponent->RemoveActionBinding(ExistingIndex);
				}
			}
		}

		// To avoid binds in the same layer being removed by the parent override temporarily put them in this array and add later
		BindsToAdd.Add(AB);
	}

	for (int32 Index=0; Index < BindsToAdd.Num(); ++Index)
	{
		InputComponent->AddActionBinding(BindsToAdd[Index]);
	}
}

