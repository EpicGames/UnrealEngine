// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputAxisDelegateBinding.h"
#include "Components/InputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputAxisDelegateBinding)


// Determines whether or not the Dynamic Delegate for the given binding
// is a NAME_None function delegate. That implies it is a "dummy" delegate
// only used for capturing axis values (for things like GetAxisValue).
static inline bool IsFunctionNameNone(const FInputAxisBinding& Binding)
{
	const FInputAxisHandlerDynamicSignature& DynamicDelegate = Binding.AxisDelegate.GetDynamicDelegate();
	const FInputAxisHandlerSignature& NonDynamicDelegate = Binding.AxisDelegate.GetDelegate();

	// Dynamic delegates will return unbound if the function is NAME_None.
	// Therefore, just ensure that we don't have a normal delegate bound.
	return !NonDynamicDelegate.IsBound() && DynamicDelegate.GetFunctionName().IsNone();
}


UInputAxisDelegateBinding::UInputAxisDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInputAxisDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	TArray<FInputAxisBinding> BindsToAdd;

	for (int32 BindIndex=0; BindIndex<InputAxisDelegateBindings.Num(); ++BindIndex)
	{
		bool bFoundNameNoneDelegate = false;
		const FBlueprintInputAxisDelegateBinding& Binding = InputAxisDelegateBindings[BindIndex];

		// Only allow a single NAME_None delegate for any given axis.
		if (Binding.FunctionNameToBind == NAME_None)
		{
			for (const FInputAxisBinding& ExistingBinding : InputComponent->AxisBindings)
			{
				if (Binding.InputAxisName == ExistingBinding.AxisName && IsFunctionNameNone(ExistingBinding))
				{
					bFoundNameNoneDelegate = true;
					break;
				}
			}
		}

		if (!bFoundNameNoneDelegate)
		{
			FInputAxisBinding AB( Binding.InputAxisName );
			AB.bConsumeInput = Binding.bConsumeInput;
			AB.bExecuteWhenPaused = Binding.bExecuteWhenPaused;
			AB.AxisDelegate.BindDelegate(ObjectToBindTo, Binding.FunctionNameToBind);

			if (Binding.bOverrideParentBinding)
			{
				for (int32 ExistingIndex = InputComponent->AxisBindings.Num() - 1; ExistingIndex >= 0; --ExistingIndex)
				{
					const FInputAxisBinding& ExistingBind = InputComponent->AxisBindings[ExistingIndex];
					if (ExistingBind.AxisName == AB.AxisName && !IsFunctionNameNone(ExistingBind))
					{
							InputComponent->AxisBindings.RemoveAt(ExistingIndex);
					}
				}
			}

			// To avoid binds in the same layer being removed by the parent override temporarily put them in this array and add later
			BindsToAdd.Add(AB);
		}
	}

	for (int32 Index=0; Index < BindsToAdd.Num(); ++Index)
	{
		InputComponent->AxisBindings.Add(BindsToAdd[Index]);
	}
}

