// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputAxisKeyDelegateBinding.h"
#include "Components/InputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputAxisKeyDelegateBinding)

UInputAxisKeyDelegateBinding::UInputAxisKeyDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInputAxisKeyDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	TArray<FInputAxisKeyBinding> BindsToAdd;

	for (int32 BindIndex=0; BindIndex<InputAxisKeyDelegateBindings.Num(); ++BindIndex)
	{
		const FBlueprintInputAxisKeyDelegateBinding& Binding = InputAxisKeyDelegateBindings[BindIndex];

		FInputAxisKeyBinding AB( Binding.AxisKey );
		AB.bConsumeInput = Binding.bConsumeInput;
		AB.bExecuteWhenPaused = Binding.bExecuteWhenPaused;
		AB.AxisDelegate.BindDelegate(ObjectToBindTo, Binding.FunctionNameToBind);

		if (Binding.bOverrideParentBinding)
		{
			for (int32 ExistingIndex = InputComponent->AxisKeyBindings.Num() - 1; ExistingIndex >= 0; --ExistingIndex)
			{
				const FInputAxisKeyBinding& ExistingBind = InputComponent->AxisKeyBindings[ExistingIndex];
				if (ExistingBind.AxisKey == AB.AxisKey)
				{
					InputComponent->AxisKeyBindings.RemoveAt(ExistingIndex);
				}
			}
		}

		// To avoid binds in the same layer being removed by the parent override temporarily put them in this array and add later
		BindsToAdd.Add(AB);
	}

	for (int32 Index=0; Index < BindsToAdd.Num(); ++Index)
	{
		InputComponent->AxisKeyBindings.Add(BindsToAdd[Index]);
	}
}

