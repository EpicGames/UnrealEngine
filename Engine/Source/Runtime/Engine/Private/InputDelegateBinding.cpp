// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputDelegateBinding.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameFramework/Actor.h"
#include "Components/InputComponent.h"
#include "HAL/IConsoleManager.h"

static bool bAlwaysAllowInputDelegateBindings = true;
static FAutoConsoleVariableRef CVarAlwaysAllowInputDelegateBindings(TEXT("Input.bAlwaysAllowInputDelegateBindings"),
	bAlwaysAllowInputDelegateBindings,
	TEXT("If true then UInputDelegateBinding::SupportsInputDelegate will always return true. Otherwise, only blueprint generated class will support dynamic input delegates"));

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputDelegateBinding)

TSet<UClass*> UInputDelegateBinding::InputBindingClasses;

UInputDelegateBinding::UInputDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (IsTemplate())
	{
		// Auto register the class
		InputBindingClasses.Emplace(GetClass());
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UInputDelegateBinding::SupportsInputDelegate(const UClass* InClass)
{
	// We want to treat every class as supporting input delegate binding no matter what,
	// because even with a native UClass it can still have blueprint generated subobjects/components
	// who need to be dynamically bound.
	return bAlwaysAllowInputDelegateBindings || Cast<UBlueprintGeneratedClass>(InClass);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UInputDelegateBinding::BindInputDelegates(const UClass* InClass, UInputComponent* InputComponent, UObject* InObjectToBindTo /* = nullptr */)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (InClass && InputComponent && SupportsInputDelegate(InClass))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		ensureMsgf(InputComponent, TEXT("Attempting to bind input delegates to an invalid Input Component!"));
		
		// If there was an object given to bind to use that, otherwise fall back to the input component's owner
		// which will be an AActor.
		UObject* ObjectToBindTo = InObjectToBindTo ? InObjectToBindTo : InputComponent->GetOwner();
		
		BindInputDelegates(InClass->GetSuperClass(), InputComponent, ObjectToBindTo);

		for(UClass* BindingClass : InputBindingClasses)
		{
			UInputDelegateBinding* BindingObject = CastChecked<UInputDelegateBinding>(
				UBlueprintGeneratedClass::GetDynamicBindingObject(InClass, BindingClass)
				, ECastCheckedType::NullAllowed);
			if (BindingObject)
			{
				BindingObject->BindToInputComponent(InputComponent, ObjectToBindTo);
			}
		}
	}
}

void UInputDelegateBinding::BindInputDelegatesWithSubojects(AActor* InActor, UInputComponent* InputComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInputDelegateBinding::BindInputDelegatesWithSubojects);
	
	ensureMsgf(InActor && InputComponent, TEXT("Attempting to bind input delegates to an invalid actor or input component!"));

	const UClass* ActorClass = InActor ? InActor->GetClass() : nullptr;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ActorClass && InputComponent && SupportsInputDelegate(ActorClass))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Bind any input delegates on the base actor class
		UInputDelegateBinding::BindInputDelegates(ActorClass, InputComponent, InputComponent->GetOwner());

		// Bind any input delegates on the actor's components
		TInlineComponentArray<UActorComponent*> ComponentArray;
		InActor->GetComponents(ComponentArray);
		for(UActorComponent* Comp : ComponentArray)
		{
			const UClass* CompClass = Comp ? Comp->GetClass() : nullptr;
			if(CompClass && Comp != InputComponent)
			{
				UInputDelegateBinding::BindInputDelegates(CompClass, InputComponent, Comp);	
			}			
		}
	}
}
