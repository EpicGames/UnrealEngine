// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/InputDeviceLibrary.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/InputDeviceSubsystem.h"
#include "Engine/World.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputDeviceLibrary)

DEFINE_LOG_CATEGORY(LogInputDevices);

APlayerController* UInputDeviceLibrary::GetPlayerControllerFromPlatformUser(const FPlatformUserId UserId)
{
	if (UserId.IsValid())
	{
		if (const UInputDeviceSubsystem* System = UInputDeviceSubsystem::Get())
		{
			if (const UWorld* World = System->GetTickableGameObjectWorld())
			{
				for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
				{
					APlayerController* PlayerController = Iterator->Get();
					const FPlatformUserId PlayerControllerUserID = PlayerController ? PlayerController->GetPlatformUserId() : PLATFORMUSERID_NONE;
					if (PlayerControllerUserID.IsValid() && PlayerControllerUserID == UserId)
					{
						return PlayerController;
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogInputDevices, Warning, "Attempting to find the player controller for an invalid Platform User Id");
	}

	return nullptr;
}

APlayerController* UInputDeviceLibrary::GetPlayerControllerFromInputDevice(const FInputDeviceId DeviceId)
{
	return GetPlayerControllerFromPlatformUser(IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceId));
}

bool UInputDeviceLibrary::IsDevicePropertyHandleValid(const FInputDevicePropertyHandle& InHandle)
{
	return InHandle.IsValid();
}

int32 UInputDeviceLibrary::GetAllInputDevicesForUser(const FPlatformUserId UserId, TArray<FInputDeviceId>& OutInputDevices)
{
	return IPlatformInputDeviceMapper::Get().GetAllInputDevicesForUser(UserId, OutInputDevices);
}

int32 UInputDeviceLibrary::GetAllInputDevices(TArray<FInputDeviceId>& OutInputDevices)
{
	return IPlatformInputDeviceMapper::Get().GetAllInputDevices(OutInputDevices);
}

int32 UInputDeviceLibrary::GetAllConnectedInputDevices(TArray<FInputDeviceId>& OutInputDevices)
{
	return IPlatformInputDeviceMapper::Get().GetAllConnectedInputDevices(OutInputDevices);
}

int32 UInputDeviceLibrary::GetAllActiveUsers(TArray<FPlatformUserId>& OutUsers)
{
	return IPlatformInputDeviceMapper::Get().GetAllActiveUsers(OutUsers);
}

FPlatformUserId UInputDeviceLibrary::GetUserForUnpairedInputDevices()
{
	return IPlatformInputDeviceMapper::Get().GetUserForUnpairedInputDevices();
}

FPlatformUserId UInputDeviceLibrary::GetPrimaryPlatformUser()
{
	return IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser();
}

bool UInputDeviceLibrary::IsUnpairedUserId(const FPlatformUserId PlatformId)
{
	return IPlatformInputDeviceMapper::Get().IsUnpairedUserId(PlatformId);
}

bool UInputDeviceLibrary::IsInputDeviceMappedToUnpairedUser(const FInputDeviceId InputDevice)
{
	return IPlatformInputDeviceMapper::Get().IsInputDeviceMappedToUnpairedUser(InputDevice);
}

FInputDeviceId UInputDeviceLibrary::GetDefaultInputDevice()
{
	return IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
}

FPlatformUserId UInputDeviceLibrary::GetUserForInputDevice(FInputDeviceId DeviceId)
{
	return IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceId);
}

FInputDeviceId UInputDeviceLibrary::GetPrimaryInputDeviceForUser(FPlatformUserId UserId)
{
	return IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(UserId);
}

EInputDeviceConnectionState UInputDeviceLibrary::GetInputDeviceConnectionState(const FInputDeviceId DeviceId)
{
	return IPlatformInputDeviceMapper::Get().GetInputDeviceConnectionState(DeviceId);
}

bool UInputDeviceLibrary::IsValidInputDevice(FInputDeviceId DeviceId)
{
	return DeviceId.IsValid();
}

bool UInputDeviceLibrary::IsValidPlatformId(FPlatformUserId UserId)
{
	return UserId.IsValid();
}

FPlatformUserId UInputDeviceLibrary::PlatformUserId_None()
{
	return PLATFORMUSERID_NONE;
}

FInputDeviceId UInputDeviceLibrary::InputDeviceId_None()
{
	return INPUTDEVICEID_NONE;
}

bool UInputDeviceLibrary::EqualEqual_PlatformUserId(FPlatformUserId A, FPlatformUserId B)
{
	return A == B;
}

bool UInputDeviceLibrary::NotEqual_PlatformUserId(FPlatformUserId A, FPlatformUserId B)
{
	return A != B;
}

bool UInputDeviceLibrary::EqualEqual_InputDeviceId(FInputDeviceId A, FInputDeviceId B)
{
	return A == B;
}

bool UInputDeviceLibrary::NotEqual_InputDeviceId(FInputDeviceId A, FInputDeviceId B)
{
	return A != B;
}
