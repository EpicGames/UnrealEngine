// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputKeyDelegateBinding.h"
#include "Components/InputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputKeyDelegateBinding)

UInputKeyDelegateBinding::UInputKeyDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInputKeyDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	TArray<FInputKeyBinding> BindsToAdd;

	for (int32 BindIndex=0; BindIndex<InputKeyDelegateBindings.Num(); ++BindIndex)
	{
		const FBlueprintInputKeyDelegateBinding& Binding = InputKeyDelegateBindings[BindIndex];

		FInputKeyBinding KB( Binding.InputChord, Binding.InputKeyEvent );
		KB.bConsumeInput = Binding.bConsumeInput;
		KB.bExecuteWhenPaused = Binding.bExecuteWhenPaused;
		KB.KeyDelegate.BindDelegate(ObjectToBindTo, Binding.FunctionNameToBind);

		if (Binding.bOverrideParentBinding)
		{
			for (int32 ExistingIndex = InputComponent->KeyBindings.Num() - 1; ExistingIndex >= 0; --ExistingIndex)
			{
				const FInputKeyBinding& ExistingBind = InputComponent->KeyBindings[ExistingIndex];
				if (ExistingBind.Chord == KB.Chord && ExistingBind.KeyEvent == KB.KeyEvent)
				{
					InputComponent->KeyBindings.RemoveAt(ExistingIndex);
				}
			}
		}

		// To avoid binds in the same layer being removed by the parent override temporarily put them in this array and add later
		BindsToAdd.Add(KB);
	}

	for (int32 Index=0; Index < BindsToAdd.Num(); ++Index)
	{
		InputComponent->KeyBindings.Add(BindsToAdd[Index]);
	}
}

