// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputKeyEventArgs.h"

#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

FInputKeyEventArgs::FInputKeyEventArgs(
		FViewport* InViewport,
		const FInputDeviceId InInputDevice,
		const FKey& InKey,
		const EInputEvent InEvent,
		const float InAmountDepressed,
		const bool bInIsTouchEvent,
		const uint64 InEventTimestamp)
	: Viewport(InViewport)
	, InputDevice(InInputDevice)
	, Key(InKey)
	, Event(InEvent)
	, AmountDepressed(InAmountDepressed)
	, bIsTouchEvent(bInIsTouchEvent)
	, EventTimestamp(InEventTimestamp)
{
	// Populate the legacy ControllerId based on the newer FPlatformUserId. This is used in some cases in PIE
	// to pass input events along to the next viewport client and "fake" that the input came from that viewports
	// primary player.
	const FPlatformUserId UserID = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(InInputDevice);
	ControllerId = FPlatformMisc::GetUserIndexForPlatformUser(UserID);
}

FInputKeyEventArgs FInputKeyEventArgs::CreateSimulated(
	const FKey& InKey,
	const EInputEvent InEvent,
	const float AmountDepressed,
	const int32 InNumSamplesOverride /* = -1 */,
	const FInputDeviceId InputDevice /*= INPUTDEVICEID_NONE*/,
	const bool bIsTouchEvent /*= false*/,
	FViewport* Viewport /*= nullptr*/
	)
{
	FInputKeyEventArgs OutArgs = FInputKeyEventArgs(
		/*InViewport*/ Viewport,
		/*InInputDevice*/ InputDevice,
		/*InKey*/ InKey,
		/*InEvent*/ InEvent,
		/*InAmountDepressed*/ AmountDepressed,
		/*bInIsTouchEvent*/ bIsTouchEvent,
		/*InEventTimestamp*/ FPlatformTime::Cycles64()	// Set the timestamp to right now
		);
	
	OutArgs.NumSamples =
		InNumSamplesOverride == -1 ?
		(InKey.IsAnalog() ? 1 : 0)  :	// By default, set the number of samples to 1 for analog keys and 0 for digital keys.
		InNumSamplesOverride;			// But, if you have specified a specific number of samples then use that.	

	// Flag this event as being a simulated input event
	OutArgs.bIsSimulatedInput = true;

	return OutArgs;
}


FPlatformUserId FInputKeyEventArgs::GetPlatformUser() const
{
	return IPlatformInputDeviceMapper::Get().GetUserForInputDevice(InputDevice);
}