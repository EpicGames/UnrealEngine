// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputTouchDelegateBinding.h"
#include "Components/InputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputTouchDelegateBinding)

UInputTouchDelegateBinding::UInputTouchDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInputTouchDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	TArray<FInputTouchBinding> BindsToAdd;

	for (int32 BindIndex=0; BindIndex<InputTouchDelegateBindings.Num(); ++BindIndex)
	{
		const FBlueprintInputTouchDelegateBinding& Binding = InputTouchDelegateBindings[BindIndex];

		FInputTouchBinding TB( Binding.InputKeyEvent );
		TB.bConsumeInput = Binding.bConsumeInput;
		TB.bExecuteWhenPaused = Binding.bExecuteWhenPaused;
		TB.TouchDelegate.BindDelegate(ObjectToBindTo, Binding.FunctionNameToBind);

		if (Binding.bOverrideParentBinding)
		{
			for (int32 ExistingIndex = InputComponent->TouchBindings.Num() - 1; ExistingIndex >= 0; --ExistingIndex)
			{
				if (InputComponent->TouchBindings[ExistingIndex].KeyEvent != TB.KeyEvent)
				{
					InputComponent->TouchBindings.RemoveAt(ExistingIndex);
				}
			}
		}

		// To avoid binds in the same layer being removed by the parent override temporarily put them in this array and add later
		BindsToAdd.Add(TB);
	}

	for (int32 Index=0; Index < BindsToAdd.Num(); ++Index)
	{
		InputComponent->TouchBindings.Add(BindsToAdd[Index]);
	}
}

