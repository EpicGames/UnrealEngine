// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/InputVectorAxisDelegateBinding.h"
#include "Components/InputComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputVectorAxisDelegateBinding)


UInputVectorAxisDelegateBinding::UInputVectorAxisDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UInputVectorAxisDelegateBinding::BindToInputComponent(UInputComponent* InputComponent, UObject* ObjectToBindTo) const
{
	TArray<FInputVectorAxisBinding> BindsToAdd;

	for (int32 BindIndex=0; BindIndex<InputAxisKeyDelegateBindings.Num(); ++BindIndex)
	{
		const FBlueprintInputAxisKeyDelegateBinding& Binding = InputAxisKeyDelegateBindings[BindIndex];

		FInputVectorAxisBinding VAB( Binding.AxisKey );
		VAB.bConsumeInput = Binding.bConsumeInput;
		VAB.bExecuteWhenPaused = Binding.bExecuteWhenPaused;
		VAB.AxisDelegate.BindDelegate(ObjectToBindTo, Binding.FunctionNameToBind);

		if (Binding.bOverrideParentBinding)
		{
			for (int32 ExistingIndex = InputComponent->VectorAxisBindings.Num() - 1; ExistingIndex >= 0; --ExistingIndex)
			{
				const FInputVectorAxisBinding& ExistingBind = InputComponent->VectorAxisBindings[ExistingIndex];
				if (ExistingBind.AxisKey == VAB.AxisKey)
				{
					InputComponent->VectorAxisBindings.RemoveAt(ExistingIndex);
				}
			}
		}

		// To avoid binds in the same layer being removed by the parent override temporarily put them in this array and add later
		BindsToAdd.Add(VAB);
	}

	for (int32 Index=0; Index < BindsToAdd.Num(); ++Index)
	{
		InputComponent->VectorAxisBindings.Add(BindsToAdd[Index]);
	}
}

