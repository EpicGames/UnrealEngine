// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceData/InstanceDataHelpers.h"
#include "Engine/InstancedStaticMesh.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/TypeHash.h"

#if WITH_EDITOR
#include "InstanceData/InstanceDataUpdateUtils.h"
#endif

uint32 FInstanceDataHelpers::HashPrimitiveLocalToWorld(const FRenderTransform& M, const FVector& PrimitiveWorldSpaceOffset)
{
	// Round to reduce risk of floating point precision causing otherwise same transforms from being considered different
	return HashCombineFast(
		GetTypeHash(FMath::RoundToInt(M.TransformRows[0][0])), GetTypeHash(FMath::RoundToInt(M.TransformRows[0][1])), GetTypeHash(FMath::RoundToInt(M.TransformRows[0][2])),
		GetTypeHash(FMath::RoundToInt(M.TransformRows[1][0])), GetTypeHash(FMath::RoundToInt(M.TransformRows[1][1])), GetTypeHash(FMath::RoundToInt(M.TransformRows[1][2])),
		GetTypeHash(FMath::RoundToInt(M.TransformRows[2][0])), GetTypeHash(FMath::RoundToInt(M.TransformRows[2][1])), GetTypeHash(FMath::RoundToInt(M.TransformRows[2][2])),
		GetTypeHash(FMath::RoundToInt(M.Origin[0])), GetTypeHash(FMath::RoundToInt(M.Origin[1])), GetTypeHash(FMath::RoundToInt(M.Origin[2])),
		GetTypeHash(FMath::RoundToInt(PrimitiveWorldSpaceOffset[0])), GetTypeHash(FMath::RoundToInt(PrimitiveWorldSpaceOffset[1])), GetTypeHash(FMath::RoundToInt(PrimitiveWorldSpaceOffset[2]))
	);
}

void FInstanceDataHelpers::GenerateInstanceRandomIDs(int32 InstancingRandomSeed, const TArray<FInstancedStaticMeshRandomSeed>& AdditionalRandomSeeds, TArrayView<float> OutRandomIDs)
{
	FRandomStream RandomStream(InstancingRandomSeed);
	TArray<FInstancedStaticMeshRandomSeed>::TConstIterator SeedsIt = AdditionalRandomSeeds.CreateConstIterator();
	int32 SeedResetIndex = SeedsIt ? SeedsIt->StartInstanceIndex : INDEX_NONE;
	for (int32 Index = 0; Index < OutRandomIDs.Num(); ++Index)
	{
		if (Index == SeedResetIndex)
		{
			RandomStream = FRandomStream(SeedsIt->RandomSeed);
			++SeedsIt;
			SeedResetIndex = SeedsIt ? SeedsIt->StartInstanceIndex : INDEX_NONE;
		}
		OutRandomIDs[Index] = RandomStream.GetFraction();
	}
}

TArray<float> FInstanceDataHelpers::GenerateInstanceRandomIDs(int32 InstanceCount, int32 InstancingRandomSeed, const TArray<FInstancedStaticMeshRandomSeed>& AdditionalRandomSeeds)
{
	TArray<float> RandomIDs;
	RandomIDs.SetNumUninitialized(InstanceCount);
	GenerateInstanceRandomIDs(InstancingRandomSeed, AdditionalRandomSeeds, RandomIDs);
	return RandomIDs;
}

#if WITH_EDITOR

FInstanceDataHelpers::FSpatialHashResult FInstanceDataHelpers::BuildSpatialHashData(int32 NumInstances, TFunctionRef<FSphere(int32)> GetWorldSpaceInstanceSphere)
{
	int32 MinLevel = 0;
	static const auto CVarInstanceHierarchyMinCellSize = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SceneCulling.MinCellSize"));
	if (CVarInstanceHierarchyMinCellSize)
	{
		MinLevel = RenderingSpatialHash::CalcLevel(CVarInstanceHierarchyMinCellSize->GetValueOnAnyThread() - 1.0);
	}

	FSpatialHashSortBuilder SortBuilder;
	SortBuilder.BuildOptimizedSpatialHashOrder(NumInstances, MinLevel, GetWorldSpaceInstanceSphere);

	// Build reorder table and compress spatial hashes in a single pass over the sorted instances.
	// This logic is extracted from FInstanceDataManager::PrecomputeOptimizationData and
	// FPrimitiveInstanceDataManager::PrecomputeOptimizationData with no behavioral changes.
	FSpatialHashResult Result;
	Result.ReorderTable.SetNumUninitialized(NumInstances);

	FInstanceSceneDataBuffers::FCompressedSpatialHashItem CurrentItem;
	CurrentItem.NumInstances = 0;

	bool bIsIdentityIndexMap = true;

	for (int32 Index = 0; Index < SortBuilder.SortedInstances.Num(); ++Index)
	{
		// Reorder table
		int32 ComponentInstanceIndex = SortBuilder.SortedInstances[Index].InstanceIndex;
		bIsIdentityIndexMap = bIsIdentityIndexMap && (Index == ComponentInstanceIndex);
		Result.ReorderTable[Index] = ComponentInstanceIndex;

		// Append cell-relative bounds
		FSphere CellRelInstanceSphere = GetWorldSpaceInstanceSphere(ComponentInstanceIndex);
		CellRelInstanceSphere.Center -= RenderingSpatialHash::CalcWorldCellCenter(SortBuilder.SortedInstances[Index].InstanceLoc);

		// Compress consecutive same-location instances into ranges
		bool bSameLoc = CurrentItem.NumInstances > 0 && CurrentItem.Location == SortBuilder.SortedInstances[Index].InstanceLoc;
		if (bSameLoc)
		{
			CurrentItem.NumInstances += 1;
			CurrentItem.ExplicitBounds += FRenderBounds(CellRelInstanceSphere);
		}
		else
		{
			if (CurrentItem.NumInstances > 0)
			{
				Result.SpatialHashes.Add(CurrentItem);
			}
			CurrentItem.Location = SortBuilder.SortedInstances[Index].InstanceLoc;
			CurrentItem.NumInstances = 1;
			CurrentItem.ExplicitBounds = FRenderBounds(CellRelInstanceSphere);
		}
	}
	if (CurrentItem.NumInstances > 0)
	{
		Result.SpatialHashes.Add(CurrentItem);
	}

	// Don't store a 1:1 mapping
	if (bIsIdentityIndexMap)
	{
		Result.ReorderTable.Reset();
	}

	return Result;
}

#endif