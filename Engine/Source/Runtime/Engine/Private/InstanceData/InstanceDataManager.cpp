// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceData/InstanceDataManager.h"
#include "InstanceData/InstanceDataHelpers.h"
#include "InstanceData/InstanceUpdateChangeSet.h"
#include "InstanceData/InstanceDataUpdateUtils.h"
#include "Components/PrimitiveComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneInterface.h"
#include "Math/DoubleFloat.h"
#include "RenderUtils.h"
#include "Rendering/MotionVectorSimulation.h"

#if 0

#define LOG_INST_DATA(_Format_, ...) \
{ \
	FString Tmp = FString::Printf(_Format_, ##__VA_ARGS__);\
	UE_LOGF(LogInstanceProxy, Log, "%p, %ls", PrimitiveComponent.IsValid() ? PrimitiveComponent.Get() : nullptr, *Tmp);\
}

#else
	#define LOG_INST_DATA(_Format_, ...) 
#endif

static TAutoConsoleVariable<bool> CVarInstanceDataManagerBuildOptimizedForTesting(
	TEXT("r.InstanceData.BuildOptimizedForTesting"),
	false,
	TEXT("."));



template <typename IndexRemapType>
void ApplyDataChanges(FInstanceUpdateChangeSet& ChangeSet, const IndexRemapType& IndexRemap, int32 PostUpdateNumInstances, FInstanceSceneDataBuffers::FWriteView& ProxyData)
{
	ProxyData.PrimitiveToRelativeWorld = ChangeSet.PrimitiveToRelativeWorld;
	ProxyData.PrimitiveWorldSpaceOffset = ChangeSet.PrimitiveWorldSpaceOffset;
		
	// TODO: DISP - Fix me (this comment came along from FPrimitiveSceneProxy::SetInstanceLocalBounds and is probably still true...)
	const FVector3f PadExtent = GetLocalBoundsPadExtent(ProxyData.PrimitiveToRelativeWorld, ChangeSet.AbsMaxDisplacement);
	if (!ChangeSet.Flags.bHasPerInstanceLocalBounds)
	{
		ProxyData.InstanceLocalBounds = MoveTemp(ChangeSet.InstanceLocalBounds);

		for (FRenderBounds& Bounds : ProxyData.InstanceLocalBounds)
		{
			Bounds.Min -= PadExtent;
			Bounds.Max += PadExtent;
		}
	}
	else
	{
		ChangeSet.GetLocalBoundsReader().Scatter(ProxyData.InstanceLocalBounds, [PadExtent](FRenderBounds& Bounds) -> void 
			{	
				Bounds.Min -= PadExtent;
				Bounds.Max += PadExtent;
			}, IndexRemap);
	}

	// unpack transform deltas
	ApplyTransformUpdates(ChangeSet.GetTransformDelta(), IndexRemap, ChangeSet.PrimitiveToRelativeWorld, ChangeSet.Transforms, PostUpdateNumInstances, ProxyData.InstanceToPrimitiveRelative);
	if (ChangeSet.Flags.bHasPerInstanceDynamicData)
	{
		FRenderTransform PrevPrimitiveToRelativeWorld = ChangeSet.PreviousPrimitiveToRelativeWorld.Get(ChangeSet.PrimitiveToRelativeWorld);
		ApplyTransformUpdates(ChangeSet.GetTransformDelta(), IndexRemap, PrevPrimitiveToRelativeWorld, ChangeSet.PrevTransforms, PostUpdateNumInstances, ProxyData.PrevInstanceToPrimitiveRelative);
	}
	else
	{
		ProxyData.PrevInstanceToPrimitiveRelative.Reset();
	}

	ApplyAttributeChanges(ChangeSet, IndexRemap, ProxyData);
}

class FUpdatableInstanceDataSceneProxy : public FInstanceDataSceneProxy
{
public:
	using FInstanceDataSceneProxy::FInstanceDataSceneProxy;

	virtual FInstanceDataUpdateTaskInfo *GetUpdateTaskInfo() override
	{ 
		return &InstanceDataUpdateTaskInfo; 
	}

	virtual void Update(FInstanceUpdateChangeSet&& ChangeSet)
	{
		SCOPED_NAMED_EVENT(FISMCInstanceDataSceneProxy_Update, FColor::Emerald);
		check(!ChangeSet.IsFullUpdate());

	#if WITH_EDITOR
		// replace the HP container.
		if (ChangeSet.HitProxyContainer)
		{
			HitProxyContainer = MoveTemp(ChangeSet.HitProxyContainer);
		}
	#endif

		DecStatCounters();

		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

		int32 PostUpdateNumInstances = ChangeSet.NumSourceInstances;

		// Handle data movement, needs old & new ID maps
		// These can only be caused by removes, which means an item can only ever move towards lower index in the array.
		// Thus, we can always safely overwrite the data in the new slot, since we do them in increasing order.
		// NOTE: If we start allowing some other kind of permutation of the ISM data, this assumption will break.
		// TODO: Add validation code somewhere in the pipeline.
		const auto IndexDelta = ChangeSet.GetIndexChangedDelta();
		for (auto It = IndexDelta.GetIterator(); It; ++It)
		{
			// Index in the source (e.g., component)
			const int32 ToIndex = It.GetIndex();
			if (!ChangeSet.InstanceAttributeTracker.TestFlag<FInstanceAttributeTracker::EFlag::Added>(ToIndex))
			{
				int32 ItemIndex = It.GetItemIndex();
				FPrimitiveInstanceId InstanceId = ChangeSet.bIdentityIdMap ? FPrimitiveInstanceId{ToIndex} : ChangeSet.IndexToIdMapDeltaData[ItemIndex];
				if (InstanceIdIndexMap.IsValidId(InstanceId))
				{
					const int32 FromIndex =  InstanceIdIndexMap.IdToIndex(InstanceId);

					ProxyData.InstanceToPrimitiveRelative[ToIndex] = ProxyData.InstanceToPrimitiveRelative[FromIndex];
					CondMove(ProxyData.Flags.bHasPerInstanceCustomData, ProxyData.InstanceCustomData, FromIndex, ToIndex, ProxyData.NumCustomDataFloats);
					CondMove(ProxyData.Flags.bHasPerInstanceRandom, ProxyData.InstanceRandomIDs, FromIndex, ToIndex);
					CondMove(ProxyData.Flags.bHasPerInstanceLMSMUVBias, ProxyData.InstanceLightShadowUVBias, FromIndex, ToIndex);
					CondMove(ProxyData.Flags.bHasPerInstanceSkinningData, ProxyData.InstanceSkinningData, FromIndex, ToIndex);
	#if WITH_EDITOR
					CondMove(ProxyData.Flags.bHasPerInstanceEditorData, ProxyData.InstanceEditorData, FromIndex, ToIndex);
	#endif
				}
			}
		}
		ProxyData.Flags = ChangeSet.Flags;

		UpdateIdMapping(ChangeSet, FIdentityIndexRemap(), InstanceIdIndexMap);
		check(ChangeSet.NumSourceInstances == InstanceIdIndexMap.GetMaxInstanceIndex());

		FIdentityIndexRemap IndexRemap;
		ApplyDataChanges(ChangeSet, IndexRemap, PostUpdateNumInstances, ProxyData);

		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

		InstanceSceneDataBuffers.ValidateData();

		IncStatCounters();
	}

	virtual void Build(FInstanceUpdateChangeSet&& ChangeSet)
	{
		DecStatCounters();
		check(ChangeSet.IsFullUpdate());
		// TODO: add similar checks: checkSlow(!ChangeSet.GetTransformDelta().IsDelta());
		// TODO: add similar checks: checkSlow(!ChangeSet.GetCustomDataDelta().IsDelta() || (!ChangeSet.Flags.bHasPerInstanceCustomData && ChangeSet.GetCustomDataDelta().IsEmpty()));
		// TODO: add similar checks: checkSlow(!ChangeSet.GetInstanceLightShadowUVBiasDelta().IsDelta() || ChangeSet.GetInstanceLightShadowUVBiasDelta().IsEmpty());
	#if WITH_EDITOR
		// TODO: add similar checks: checkSlow(!ChangeSet.GetInstanceEditorDataDelta().IsDelta() || ChangeSet.GetInstanceEditorDataDelta().IsEmpty());
		// replace the HP container.
		if (ChangeSet.HitProxyContainer)
		{
			HitProxyContainer = MoveTemp(ChangeSet.HitProxyContainer);
		}
	#endif

		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView WriteView = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

		WriteView.Flags = ChangeSet.Flags;

		if (ChangeSet.PrecomputedOptimizationData)
		{
			// We don't store an ID mapping for this case, since we assume a full rebuild is needed to handle any changes at all.
			// TODO: it may be possible that we'd want to keep the ID mapping at some point if we had a case where a specific instance needed 
			//       to be addressed on the RT. However, since it is not free to keep the remap around we don't do that until needed.
			InstanceIdIndexMap.Reset(ChangeSet.NumSourceInstances);

			// If the optimization data contains an index remap, we must rearrange the instance data to match (or spatial hash ranges won't match).
			// Otherwise we can take the most optimal unsorted path (which allows moving attribute arrays).
			if (ChangeSet.PrecomputedOptimizationData->ProxyIndexToComponentIndexRemap.IsEmpty())
			{
				ApplyDataChanges(ChangeSet, FIdentityIndexRemap(), InstanceIdIndexMap.GetMaxInstanceIndex(), WriteView);
			}
			else
			{
				FSrcIndexRemap SortedInstancesRemap(ChangeSet.PrecomputedOptimizationData->ProxyIndexToComponentIndexRemap);
				ApplyDataChanges(ChangeSet, SortedInstancesRemap, InstanceIdIndexMap.GetMaxInstanceIndex(), WriteView);
			}
	
			InstanceSceneDataBuffers.SetImmutable(FInstanceSceneDataImmutable(ChangeSet.PrecomputedOptimizationData->Hashes), WriteView.AccessTag);
		}
		else
		{
			UpdateIdMapping(ChangeSet, FIdentityIndexRemap(), InstanceIdIndexMap);
			check(ChangeSet.NumSourceInstances == InstanceIdIndexMap.GetMaxInstanceIndex());

			FIdentityIndexRemap IndexRemap;
			ApplyDataChanges(ChangeSet, IndexRemap, InstanceIdIndexMap.GetMaxInstanceIndex(), WriteView);
		}
		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

		InstanceSceneDataBuffers.ValidateData();

		IncStatCounters();
	}

protected:
	FInstanceDataUpdateTaskInfo InstanceDataUpdateTaskInfo;
	FInstanceIdIndexMap InstanceIdIndexMap;
#if WITH_EDITOR
	/**Container for hitproxies that are used by the instances, uses the FDeferredCleanupInterface machinery to delete itself back on the game thread when replaced. */
	TPimplPtr<FOpaqueHitProxyContainer> HitProxyContainer;
#endif
};

FInstanceDataManager::FInstanceDataManager(UPrimitiveComponent* InPrimitiveComponent)
	: bNumCustomDataChanged(false)
	, bTransformChangedAllInstances(false)
	, bPrimitiveTransformChanged(false)
	, bAnyInstanceChange(false)
	, PrimitiveComponent(InPrimitiveComponent) 
{
	// Don't do anything if this is not a "real" ISM being tracked (this logic shopuld move out).
	if (PrimitiveComponent.IsValid() && PrimitiveComponent->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		TrackingState = ETrackingState::Disabled;
	}
	LOG_INST_DATA(TEXT("FInstanceDataManager %s, TrackingState=%s"), *PrimitiveComponent->GetFullName(), TrackingState == ETrackingState::Disabled ? TEXT("Disabled") : TEXT("Initial"));
}

FPrimitiveInstanceId FInstanceDataManager::Add(int32 InInstanceAddAtIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return FPrimitiveInstanceId{};
	}

	ValidateMapping();

	// 1. determine if we need to enable explicit tracking, this happens when an instance is inserted (i.e., not added at the end).
	bool bIsInsert = InInstanceAddAtIndex != GetMaxInstanceIndex();

	// Create explicit mapping if we need it now
	if (bIsInsert && HasIdentityMapping())
	{
		CreateExplicitIdentityMapping();
	}

	MarkComponentRenderInstancesDirty();

	if (HasIdentityMapping())
	{
		int32 InstanceId = NumInstances++;
		MarkChangeHelper<EChangeFlag::Added>(InstanceId);

		LOG_INST_DATA(TEXT("Add(IDX: %d, bInsert: %d) -> Id: %d"), InInstanceAddAtIndex, bInsert ? 1 : 0, InstanceId);
		return FPrimitiveInstanceId { InstanceId };
	}

	FPrimitiveInstanceId InstanceId{ValidInstanceIdMask.FindAndSetFirstZeroBit(IdSearchStartIndex)};
	if (!InstanceId.IsValid())
	{
		InstanceId = FPrimitiveInstanceId{ValidInstanceIdMask.Add(true)};
	}
	// Optimize search for next time
	IdSearchStartIndex = InstanceId.Id;
	IdToIndexMap.SetNumUninitialized(ValidInstanceIdMask.Num());

	// if these do not line up, then we are inserting an instance, this is a thing in the editor
	if (InInstanceAddAtIndex != IndexToIdMap.Num())
	{
		check(bIsInsert);
		IdToIndexMap[InstanceId.Id] = InInstanceAddAtIndex;
		LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), InstanceId.Id, InInstanceAddAtIndex);
		// first move all the existing data down one step by inserting the new one
		IndexToIdMap.Insert(InstanceId, InInstanceAddAtIndex);
		// then update all the relevant id->index mappings
		for (int32 Index = InInstanceAddAtIndex + 1; Index < IndexToIdMap.Num(); ++Index)
		{
			FPrimitiveInstanceId MovedId = IndexToIdMap[Index];
			IdToIndexMap[MovedId.Id] = Index;
			LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), MovedId.GetAsIndex(), Index);
			InstanceUpdateTracker.MarkIndex<EChangeFlag::IndexChanged>(Index, GetMaxInstanceIndex());
		}

		// TODO: DO this instead of all the InstanceUpdateTracker.MarkIndex<EChangeFlag::IndexChanged>(Index, GetMaxInstanceIndex()); 
		// The proxy update can't handle the reordering anyway and not sure we should bother.
		// InstanceUpdateTracker.MarkAllChanged();
	}
	else
	{
		int32 InstanceIndex = IndexToIdMap.Num();
		IdToIndexMap[InstanceId.Id] = InstanceIndex;
		LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), InstanceId.Id, InstanceIndex);

		IndexToIdMap.Add(InstanceId);
	}
	NumInstances = IndexToIdMap.Num();
	check(ValidInstanceIdMask.Num() >= NumInstances);
	MarkChangeHelper<EChangeFlag::Added>(InstanceId);
	LOG_INST_DATA(TEXT("Add(IDX: %d, bIsInsert: %d) -> Id: %d"), InInstanceAddAtIndex, bIsInsert, InstanceId.Id);

	ValidateMapping();
	return InstanceId;
}

void FInstanceDataManager::RemoveAtSwap(int32 InstanceIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	ValidateMapping();

	FPrimitiveInstanceId InstanceId = IndexToId(InstanceIndex);
	// resize to the max at once so we don't have to grow piecemeal
	InstanceUpdateTracker.RemoveAtSwap(InstanceId, InstanceIndex, GetMaxInstanceIndex());

	// If the remove would cause reordering, we create the explicit mapping
	const bool bCausesReordering = InstanceIndex != NumInstances - 1;
	if (bCausesReordering && HasIdentityMapping())
	{
		CreateExplicitIdentityMapping();
	}

	MarkComponentRenderInstancesDirty();
	FreeInstanceId(InstanceId);

	// If we still have the identity mapping, we must be removing the last item
	if (HasIdentityMapping())
	{
		check(!bCausesReordering);
		--NumInstances;
		LOG_INST_DATA(TEXT("RemoveAtSwap(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);
		return;
	}

	FPrimitiveInstanceId LastInstanceId = IndexToIdMap.Pop();
	NumInstances = IndexToIdMap.Num();
	check(ValidInstanceIdMask.Num() >= NumInstances);

	if (InstanceId != LastInstanceId)
	{
		IdToIndexMap[LastInstanceId.Id] = InstanceIndex;
		LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), LastInstanceId.Id, InstanceIndex);
		IndexToIdMap[InstanceIndex] = LastInstanceId;
	}
	ValidateMapping();
	LOG_INST_DATA(TEXT("RemoveAtSwap(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);
}
	
void FInstanceDataManager::RemoveAt(int32 InstanceIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	ValidateMapping();

	FPrimitiveInstanceId InstanceId = IndexToId(InstanceIndex);

	InstanceUpdateTracker.RemoveAt(InstanceId, InstanceIndex, GetMaxInstanceIndex());
		
	const bool bCausesReordering = InstanceIndex != NumInstances - 1;
	if (bCausesReordering && HasIdentityMapping())
	{
		CreateExplicitIdentityMapping();
	}

	MarkComponentRenderInstancesDirty();
	FreeInstanceId(InstanceId);

	// If we still have the identity mapping, do the simplified tracking update
	if (HasIdentityMapping())
	{
		check(!bCausesReordering);
		--NumInstances;
		LOG_INST_DATA(TEXT("RemoveAt(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);
		return;
	}

	if (InstanceIndex == IndexToIdMap.Num() - 1)
	{
		IndexToIdMap.SetNum(InstanceIndex);
	}
	else
	{
		IndexToIdMap.RemoveAt(InstanceIndex);
		for (int32 Index = InstanceIndex; Index < IndexToIdMap.Num(); ++Index)
		{
			FPrimitiveInstanceId MovedId = IndexToIdMap[Index];
			IdToIndexMap[MovedId.Id] = Index;
			LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), MovedId.GetAsIndex(), Index);
		}
	}
	NumInstances = IndexToIdMap.Num();
	check(ValidInstanceIdMask.Num() >= NumInstances);
	LOG_INST_DATA(TEXT("RemoveAt(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);

	ValidateMapping();
}

void FInstanceDataManager::TransformChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("TransformChanged(IDX: %d)"), InstanceIndex);
	MarkChangeHelper<EChangeFlag::TransformChanged>( InstanceIndex);
}

void FInstanceDataManager::TransformChanged(FPrimitiveInstanceId InstanceId)
{
	LOG_INST_DATA(TEXT("TransformChanged(ID: %d)"), InstanceId.Id);
	MarkChangeHelper<EChangeFlag::TransformChanged>(InstanceId);
}

void FInstanceDataManager::TransformsChangedAll()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("TransformsChangedAll(%s)"), TEXT(""));
	bTransformChangedAllInstances = true;
	MarkComponentRenderInstancesDirty();
}

void FInstanceDataManager::CustomDataChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("CustomDataChanged(IDX: %d)"), InstanceIndex);
	MarkChangeHelper<EChangeFlag::CustomDataChanged>(InstanceIndex);
}

void FInstanceDataManager::SkinningDataChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("SkinningDataChanged(IDX: %d)"), InstanceIndex);
	MarkChangeHelper<EChangeFlag::SkinningDataChanged>(InstanceIndex);
}

void FInstanceDataManager::BakedLightingDataChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("BakedLightingDataChanged(IDX: %d)"), InstanceIndex);
	AllChangedFlags.bHasPerInstanceLMSMUVBias = true;
	MarkComponentRenderInstancesDirty();
}

void FInstanceDataManager::BakedLightingDataChangedAll()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("BakedLightingDataChangedAll(%s)"), TEXT(""));
	AllChangedFlags.bHasPerInstanceLMSMUVBias = true;
	MarkComponentRenderInstancesDirty();
}

void FInstanceDataManager::NumCustomDataChanged()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("NumCustomDataChanged(%s)"), TEXT(""));
	bNumCustomDataChanged = true;
	MarkComponentRenderInstancesDirty();
}

#if WITH_EDITOR

void FInstanceDataManager::EditorDataChangedAll()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("EditorDataChangedAll(%s)"), TEXT(""));
	AllChangedFlags.bHasPerInstanceEditorData = true;
	MarkComponentRenderInstancesDirty();
}

#endif

void FInstanceDataManager::PrimitiveTransformChanged()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("PrimitiveTransformChanged(%s)"), TEXT(""));
	bPrimitiveTransformChanged = true;
	MarkComponentRenderInstancesDirty();
}

void FInstanceDataManager::ClearInstances()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}
	// Reset any instance allocations
	Reset(0);
	ValidInstanceIdMask.Empty();
	IdSearchStartIndex = 0;
	// reset the change tracking back to initial state
	ClearChangeTracking();
	MarkComponentRenderInstancesDirty();
}

bool FInstanceDataManager::HasAnyInstanceChanges() const
{
	return bAnyInstanceChange || bNumCustomDataChanged || AllChangedFlags.Packed != 0u || bTransformChangedAllInstances
		|| InstanceUpdateTracker.HasAnyChanges();
}

/**
 * Describes what has changed, that can be derived from the primitive desc, or internal tracking state.
 */
struct FChangeDesc2
{
	FInstanceDataFlags ChangedFlags;
	union
	{
		struct 
		{
			uint8 bUntrackedState : 1;
			uint8 bInstancesChanged : 1;
			uint8 bPrimitiveTransformChanged : 1;
			uint8 bMaterialUsageFlagsChanged : 1;
			uint8 bMaxDisplacementChanged : 1;
			uint8 bStaticMeshBoundsChanged : 1;
		};
		uint32 Packed;
	};
	
	FChangeDesc2(bool bFullChange = false)
	{
		Packed = 0u;
		bUntrackedState = bFullChange;
	}

	bool HasAnyChange() const
	{
		return Packed != 0u || ChangedFlags.Packed != 0u;
	}
};

static float ComputeAbsMaxDisplacement(const FInstanceDataManagerSourceDataDesc& ComponentData)
{
	return FMath::Max(-ComponentData.PrimitiveMaterialDesc.MinMaxMaterialDisplacement.X, ComponentData.PrimitiveMaterialDesc.MinMaxMaterialDisplacement.Y)
		+ ComponentData.PrimitiveMaterialDesc.MaxWorldPositionOffsetDisplacement;
}

void FInstanceDataManager::GatherDefaultData(const FInstanceDataManagerSourceDataDesc& ComponentData, FInstanceUpdateChangeSet& ChangeSet) const
{
	// Collect the delta data to be able to update the index mapping.
	ChangeSet.MaxInstanceId = GetMaxAllocatedInstanceId();
	ChangeSet.bIdentityIdMap = IsIdentity();

	const FVector3f PrimitiveWorldSpacePositionHigh = FDFVector3{ ComponentData.PrimitiveLocalToWorld.GetOrigin() }.High;
	ChangeSet.PrimitiveWorldSpaceOffset = FVector{ PrimitiveWorldSpacePositionHigh };
	ChangeSet.PrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(PrimitiveWorldSpacePositionHigh, ComponentData.PrimitiveLocalToWorld).M;
	ChangeSet.AbsMaxDisplacement = ComputeAbsMaxDisplacement(ComponentData);
	if (ChangeSet.Flags.bHasPerInstanceCustomData)
	{
		ChangeSet.NumCustomDataFloats = ComponentData.NumCustomDataFloats;
	}

	// Only gather data after the rest is initialized
	if (!ChangeSet.bIdentityIdMap)
	{
		Gather(ChangeSet.GetIndexChangedDelta(), ChangeSet.IndexToIdMapDeltaData, IndexToIdMap);
	}
		
	// Patch up the special local bounds data.
	if (!ChangeSet.Flags.bHasPerInstanceLocalBounds)
	{
		check(ChangeSet.InstanceLocalBounds.IsEmpty());
		// This is the odd one out.
		ChangeSet.SetSharedLocalBounds(ComponentData.MeshBounds);
	}
}

void FInstanceDataManager::InitChangeSet(const FInstanceDataManagerSourceDataDesc& ComponentData, FInstanceUpdateChangeSet& ChangeSet) const
{
	// This path is intended only to work with a full update.
	check(ChangeSet.bNeedFullUpdate);
	check(ChangeSet.Flags == ChangeSet.ForceFullFlags);

	GatherDefaultData(ComponentData, ChangeSet);
}

void FInstanceDataManager::InitChangeSet(const FChangeDesc2 &ChangeDesc, const FInstanceDataManagerSourceDataDesc& ComponentData, FInstanceUpdateChangeSet& ChangeSet)
{
	ChangeSet.ForceFullFlags = ChangeDesc.ChangedFlags; 
	// also trigger full copy if num changed.
	ChangeSet.ForceFullFlags.bHasPerInstanceCustomData = ChangeSet.ForceFullFlags.bHasPerInstanceCustomData || bNumCustomDataChanged;
	ChangeSet.Flags = ComponentData.Flags;

	GatherDefaultData(ComponentData, ChangeSet);
}

bool FInstanceDataManager::FlushChanges(FInstanceDataManagerSourceDataDesc&& ComponentData)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return false;
	}

	// if bHasPerInstanceCustomData is set, then NumCustomDataFloats must be non-zero
	check(!ComponentData.Flags.bHasPerInstanceCustomData || ComponentData.NumCustomDataFloats != 0);

	// for a proxy that is not visible to the render thread we can dispatch the update task immediately, saving queueing a RT command and potentially increasing overlap by some ammount.
	const bool bIsUnattached = !InstanceDataProxy.IsValid() || InstanceDataProxy.GetSharedReferenceCount() == 1;
	if (!InstanceDataProxy.IsValid())
	{
		InstanceDataProxy = GetOrCreateProxyInternal();
	}

	float NewAbsMaxDisplacement = ComputeAbsMaxDisplacement(ComponentData);

	FChangeDesc2 ChangeDesc;
	ChangeDesc.bUntrackedState = GetState() != ETrackingState::Tracked && GetState() != ETrackingState::Optimized;

	// Figure out the deltas.
	if (!ChangeDesc.bUntrackedState)
	{
		ChangeDesc.bInstancesChanged =  HasAnyInstanceChanges();

		// TODO: make all this depend on tracking rather than diffing (we may do diffing as a means of debugging perhaps)
		ChangeDesc.bPrimitiveTransformChanged = bTransformChangedAllInstances || bPrimitiveTransformChanged; 
		ChangeDesc.bMaterialUsageFlagsChanged = Flags != ComponentData.Flags;
		ChangeDesc.bMaxDisplacementChanged = AbsMaxDisplacement != NewAbsMaxDisplacement;
		ChangeDesc.bStaticMeshBoundsChanged = !MeshBounds.Equals(ComponentData.MeshBounds);
		ChangeDesc.ChangedFlags.Packed = Flags.Packed ^ ComponentData.Flags.Packed;
	}
	
	//TBD: leave in the proxy?
	// FMatrix PrevPrimitiveLocalToWorld = PrimitiveLocalToWorld;
	
	// Update the tracked state
	// PrimitiveLocalToWorld = ComponentData.PrimitiveLocalToWorld;
	AbsMaxDisplacement = NewAbsMaxDisplacement;
	MeshBounds = ComponentData.MeshBounds;
	Flags = ComponentData.Flags;

	// TODO: detect change and toggle full update
	NumCustomDataFloats = ComponentData.NumCustomDataFloats;

	if (!ChangeDesc.HasAnyChange())
	{
		return false;
	}
	
	// If we got here & the state is "optimized" then we know the precomputed data is now invalid and we ditch it.
	// This should not happen in a cooked client, ideally.
	if (GetState() == ETrackingState::Optimized)
	{
#if !UE_BUILD_SHIPPING
		static bool bLoggedOnce = false;
		if (!bLoggedOnce)
		{
			UE_LOGF(LogTemp, Log, "Discarded PrecomputedOptimizationData");
			bLoggedOnce = true;
		}
#endif
		PrecomputedOptimizationData.Reset();
	}

	// After an update has been sent, we need to track all deltas.
	ETrackingState SuccessorTrackingState = ETrackingState::Tracked;
	{
#if WITH_EDITOR
		// We should rebuild the optimization data if the cvar is set & this is not the first update and we have cooked data.
		// TODO: this may not be 100% robust against things like invalidations triggered by unexpected stuff, like BP machinations.
		//       Most such events also need to clear PrecomputedOptimizationData, but must be carefully done to not clear when actually wanted (in cooked). 
		bool bRebuildOptimized = CVarInstanceDataManagerBuildOptimizedForTesting.GetValueOnGameThread()
			&& !(ChangeDesc.bUntrackedState && PrecomputedOptimizationData.IsValid());
#else
		constexpr bool bRebuildOptimized = false;
#endif
		bool bNeedFullUpdate = ChangeDesc.bUntrackedState 
			// Note: this is a bit inefficient and we could resend just the relevant attributes, but happens only when the material changes which is hopefully not something we need to optimize for.
			|| ChangeDesc.bMaterialUsageFlagsChanged 
			// it was optimized so we need to build everything 
			|| GetState() == ETrackingState::Optimized
			|| bRebuildOptimized;

		// NOTE: Moving the update tracker to the change set implicitly resets it.
		FInstanceUpdateChangeSet ChangeSet(
			bNeedFullUpdate, 
			MoveTemp(InstanceUpdateTracker),
			ComponentData.NumInstances);

		ChangeSet.bUpdateAllInstanceTransforms = ChangeDesc.bPrimitiveTransformChanged || bTransformChangedAllInstances;
		// Initialize the change set before collecting instance change data.
		InitChangeSet(ChangeDesc, ComponentData, ChangeSet);

		// Callback to the owner to fill in change data.
		ComponentData.BuildChangeSet(ChangeSet);

		// make sure the custom data change is correctly tracked
		check(!ChangeSet.Flags.bHasPerInstanceCustomData || NumCustomDataFloats == ChangeSet.NumCustomDataFloats);
		
		// TODO: add more cross validation here somehow
		//check(!ChangeSet.Flags.bHasPerInstanceCustomData || ChangeSet.PerInstanceCustomData.);
		check(ChangeSet.Flags.bHasPerInstanceLocalBounds || ChangeSet.InstanceLocalBounds.Num() == 1);
		//check(!ChangeSet.Flags.bHasPerInstanceLocalBounds || ChangeSet.InstanceLocalBounds.Num() == ChangeSet.GetTransformDelta().GetNumItems());


		// If we have per-instance previous local to world, they are expected to be in the local space of the _previous_ local to world. If they are in fact not (e.g., if someone sets them explicitly from world space) then, well, this won't be correct
		if (ChangeSet.Flags.bHasPerInstanceDynamicData)
		{
			// TODO: move supplying this out to the ComponentData desc.
			TOptional<FTransform> PreviousTransform = FMotionVectorSimulation::Get().GetPreviousTransform(PrimitiveComponent.Get());
			if (PreviousTransform.IsSet())
			{
				ChangeSet.PreviousPrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(FVector3f{ ChangeSet.PrimitiveWorldSpaceOffset }, PreviousTransform.GetValue().ToMatrixWithScale()).M;
			}
			// TODO: Move this to the proxy
			//else
			//{
			//	ChangeSet.PreviousPrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(FVector3f{ ChangeSet.PrimitiveWorldSpaceOffset }, PrevPrimitiveLocalToWorld).M;				
			//}
		}

		// Note: this does affect TrackingState but we reset this below anyways.
		ClearChangeTracking();

		// Assemble header info to enable nonblocking primitive update.
		FInstanceDataBufferHeader InstanceDataBufferHeader;

		InstanceDataBufferHeader.NumInstances = ChangeSet.NumSourceInstances;
		InstanceDataBufferHeader.PayloadDataStride = FInstanceSceneDataBuffers::CalcPayloadDataStride(ChangeSet.Flags, ChangeSet.NumCustomDataFloats, 0);
		InstanceDataBufferHeader.Flags = ChangeSet.Flags;

		CSV_CUSTOM_STAT_GLOBAL(NumInstanceTransformUpdates, ChangeSet.Transforms.Num(), ECsvCustomStatOp::Accumulate);

#if WITH_EDITOR
		if (bRebuildOptimized)
		{
			check(ChangeSet.IsFullUpdate());
			LOG_INST_DATA(TEXT("Optimized Rebuild"));
			// check(ComponentData.ComponentMobility == EComponentMobility::Static || ComponentData.ComponentMobility == EComponentMobility::Stationary);
			DispatchInstanceDataUpdateTask(bIsUnattached, InstanceDataProxy, InstanceDataBufferHeader, [ChangeSet = MoveTemp(ChangeSet), InstanceDataProxy = InstanceDataProxy, NumHeaderInstances = InstanceDataBufferHeader.NumInstances] () mutable 
			{
				// Build the optimization data on the fly. This path is only for testing purposes as we otherwise want the data to be pre-cooked & passed along.
				ChangeSet.PrecomputedOptimizationData = MakeShared<FPrecomputedInstanceSpatialHashData>(PrecomputeOptimizationData(ChangeSet));
				InstanceDataProxy->Build(MoveTemp(ChangeSet));
				check(NumHeaderInstances == InstanceDataProxy->GeInstanceSceneDataBuffers()->GetNumInstances());
			});
			SuccessorTrackingState = ETrackingState::Optimized;
		}
		else 
#endif // WITH_EDITOR
		if (bNeedFullUpdate)
		{
			LOG_INST_DATA(TEXT("Full Build %s"), TEXT(""));
			if (PrecomputedOptimizationData.IsValid())
			{
				if (PrecomputedOptimizationData->PrimitiveLocalToWorldHash == FInstanceDataHelpers::HashPrimitiveLocalToWorld(ChangeSet.PrimitiveToRelativeWorld, ChangeSet.PrimitiveWorldSpaceOffset))
				{
					LOG_INST_DATA(TEXT("  Optimized Build (%s)"), PrecomputedOptimizationData.IsValid() ? TEXT("Precomputed") : TEXT(""));
					ChangeSet.PrecomputedOptimizationData = PrecomputedOptimizationData;
					SuccessorTrackingState = ETrackingState::Optimized;
				}
				else
				{
					PrecomputedOptimizationData.Reset();
				}
			}
			DispatchInstanceDataUpdateTask(bIsUnattached, InstanceDataProxy, InstanceDataBufferHeader, [ChangeSet = MoveTemp(ChangeSet), InstanceDataProxy = InstanceDataProxy, NumHeaderInstances = InstanceDataBufferHeader.NumInstances] () mutable 
			{
				InstanceDataProxy->Build(MoveTemp(ChangeSet));
				check(NumHeaderInstances == InstanceDataProxy->GeInstanceSceneDataBuffers()->GetNumInstances());
			});
		}
		else
		{
			LOG_INST_DATA(TEXT("Delta Update %s"), TEXT(""));
			check(!ChangeDesc.bUntrackedState);
			DispatchInstanceDataUpdateTask(bIsUnattached, InstanceDataProxy, InstanceDataBufferHeader, [ChangeSet = MoveTemp(ChangeSet), InstanceDataProxy = InstanceDataProxy, NumHeaderInstances = InstanceDataBufferHeader.NumInstances] () mutable 
			{
				InstanceDataProxy->Update(MoveTemp(ChangeSet));
				check(NumHeaderInstances == InstanceDataProxy->GeInstanceSceneDataBuffers()->GetNumInstances());
			});
		}
	}

	TrackingState = SuccessorTrackingState;
	return true;
}

void FInstanceDataManager::ClearChangeTracking()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	// When tracking data is cleared, we loose connection to previously tracked state until the next update is sent.
	TrackingState = ETrackingState::Initial;

	InstanceUpdateTracker.Reset();
	bNumCustomDataChanged = false;
	AllChangedFlags.Packed = 0u;
	bTransformChangedAllInstances = false;
	bPrimitiveTransformChanged = false;
	bAnyInstanceChange = false;
}

int32 FInstanceDataManager::GetMaxAllocatedInstanceId() const
{
	return HasIdentityMapping() ? NumInstances : ValidInstanceIdMask.Num();
}

void FInstanceDataManager::CreateExplicitIdentityMapping()
{
	check(HasIdentityMapping());
	IndexToIdMap.SetNumUninitialized(NumInstances);
	IdToIndexMap.SetNumUninitialized(NumInstances);
	for (int32 Index = 0; Index < NumInstances; ++Index)
	{
		IndexToIdMap[Index] = FPrimitiveInstanceId{Index};
		IdToIndexMap[Index] = Index;
	}
	ValidInstanceIdMask.Reset();
	ValidInstanceIdMask.SetNum(NumInstances, true);
	IdSearchStartIndex = NumInstances;
}

template<FInstanceDataManager::EChangeFlag Flag>
void FInstanceDataManager::MarkChangeHelper(int32 InstanceIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	if (GetState() != ETrackingState::Tracked)
	{
		bAnyInstanceChange = true;
		MarkComponentRenderInstancesDirty();
		return;
	}
	InstanceUpdateTracker.MarkIndex<Flag>(InstanceIndex, GetMaxInstanceIndex());
	MarkComponentRenderInstancesDirty();
}

template<FInstanceDataManager::EChangeFlag Flag>
void FInstanceDataManager::MarkChangeHelper(FPrimitiveInstanceId InstanceId)
{
	if (GetState() != ETrackingState::Tracked)
	{
		bAnyInstanceChange = true;
		MarkComponentRenderInstancesDirty();
		return;
	}
	MarkChangeHelper<Flag>(IdToIndex(InstanceId));
}

void FInstanceDataManager::MarkComponentRenderInstancesDirty()
{
	if (UPrimitiveComponent *PrimitiveComponentPtr = PrimitiveComponent.Get())
	{
		PrimitiveComponentPtr->MarkRenderInstancesDirty();
	}
}

bool FInstanceDataManager::HasIdentityMapping() const
{
	return IndexToIdMap.IsEmpty();
}

void FInstanceDataManager::FreeInstanceId(FPrimitiveInstanceId InstanceId)
{
	LOG_INST_DATA(TEXT("FreeInstanceId(Id: %d)"), InstanceId.Id);

	if (!HasIdentityMapping())
	{
		IdToIndexMap[InstanceId.Id] = INDEX_NONE;
		ValidInstanceIdMask[InstanceId.Id] = false;
		// Must start from the lowest free index since we'd otherwise get holes when adding things.
		IdSearchStartIndex = FMath::Min(IdSearchStartIndex, InstanceId.Id);
	}

	LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), InstanceId.Id, INDEX_NONE);
}

TSharedPtr<FUpdatableInstanceDataSceneProxy, ESPMode::ThreadSafe> FInstanceDataManager::GetOrCreateProxyInternal()
{
	LOG_INST_DATA(TEXT("GetOrCreateProxy"));
	if (!InstanceDataProxy)
	{
		InstanceDataProxy = MakeShared<FUpdatableInstanceDataSceneProxy>();
	}

	return InstanceDataProxy;
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> FInstanceDataManager::GetOrCreateProxy()
{
	return GetOrCreateProxyInternal();
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> FInstanceDataManager::GetProxy()
{
	return InstanceDataProxy;
}

#if DO_GUARD_SLOW
void FInstanceDataManager::ValidateMapping() const
{
	check(HasIdentityMapping() || IndexToIdMap.Num() == NumInstances);
	for (int32 Index = 0; Index < IndexToIdMap.Num(); ++Index)
	{
		FPrimitiveInstanceId Id = IndexToIdMap[Index];
		check(ValidInstanceIdMask[Id.GetAsIndex()]);
		check(Index == IdToIndexMap[Id.GetAsIndex()]);
	}
	for (int32 Id = 0; Id < IdToIndexMap.Num(); ++Id)
	{
		int32 Index = IdToIndexMap[Id];
		if (Index != INDEX_NONE)
		{
			check(ValidInstanceIdMask[Id]);
			check(IndexToIdMap[Index].GetAsIndex() == Id);
		}
		else
		{
			check(!ValidInstanceIdMask[Id]);
		}
	}
	int32 FirstFalse = ValidInstanceIdMask.Find(false);
	check(FirstFalse < 0 || FirstFalse >= IdSearchStartIndex);
}
#endif

SIZE_T FInstanceDataManager::GetAllocatedSize() const
{
	return ValidInstanceIdMask.GetAllocatedSize() +
		InstanceUpdateTracker.GetAllocatedSize();
}

bool FInstanceDataManager::ShouldUsePrecomputed()
{
	static const auto CVarPrecomputed = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SceneCulling.Precomputed"));
	
	return CVarPrecomputed && CVarPrecomputed->GetValueOnAnyThread() != 0;
}


#if WITH_EDITOR

bool FInstanceDataManager::ShouldWriteCookedData(const ITargetPlatform* TargetPlatform, int32 NumInstancesToBuildFor)
{
	EComponentMobility::Type Mobility = PrimitiveComponent.IsValid() ? PrimitiveComponent->Mobility.GetValue() : EComponentMobility::Type::Movable;

	// Only cook for static & stationary
	bool bValidTypeAndMobility = true;//(Mobility == EComponentMobility::Static || Mobility == EComponentMobility::Stationary);

	static const auto MinInstanceCountToOptimizeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.InstanceData.MinInstanceCountToOptimize"));
	// Default to 2 if the cvar doesn't exist for some reason.
	const int32 MinInstanceCountToOptimizeFor = MinInstanceCountToOptimizeCVar != nullptr ? MinInstanceCountToOptimizeCVar->GetInt() : 2;

	return bValidTypeAndMobility
		&& NumInstancesToBuildFor >= MinInstanceCountToOptimizeFor
		&& ShouldUsePrecomputed()
		&& DoesTargetPlatformSupportNanite(TargetPlatform);

}

void FInstanceDataManager::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform, FInstanceDataManagerSourceDataDesc&& ComponentData)
{
	// Already precomputed, we don't need to do it twice (could add checks to see that it is not incorrect for some obscure reason)
	if (PrecomputedOptimizationData.IsValid())
	{
		return;
	}
	
	bool bShouldBuild = ShouldWriteCookedData(TargetPlatform, ComponentData.NumInstances);

	// TODO: we could kick an async thread here if that is preferrable for the cooker?
	if (bShouldBuild && ComponentData.BuildChangeSet)
	{
		uint32 StartTime = FPlatformTime::Cycles();
		PrecomputedOptimizationData = MakeShared<FPrecomputedInstanceSpatialHashData>(PrecomputeOptimizationData(MoveTemp(ComponentData)));
		uint32 EndtTime = FPlatformTime::Cycles();

		UE_LOGF(LogTemp, Log, "Build Instance Spatial Hashes (%.2fms), Instances: %d, Hashes: %d, Remap Size: %d", FPlatformTime::ToMilliseconds( EndtTime - StartTime ), ComponentData.NumInstances, PrecomputedOptimizationData->Hashes.Num(), PrecomputedOptimizationData->ProxyIndexToComponentIndexRemap.Num());
	}	
}

FPrecomputedInstanceSpatialHashData FInstanceDataManager::PrecomputeOptimizationData(FInstanceDataManagerSourceDataDesc&& ComponentData)
{
	FInstanceDataFlags RequestedFlags;
	// Only retain the per instance local bounds flag, if present in the component, since none of the other data is used in the optimization process.
	RequestedFlags.bHasPerInstanceLocalBounds = ComponentData.Flags.bHasPerInstanceLocalBounds;
	FInstanceUpdateChangeSet ChangeSet(ComponentData.NumInstances, RequestedFlags);
	InitChangeSet(ComponentData, ChangeSet);

	// Callback to the owner to fill in change-set data.
	// Note: this makes a copy of the data, which is somewhat wasteful but gets the format converted from whatever the owner might have, otherwise we'd need some other abstraction for the data here.
	ComponentData.BuildChangeSet(ChangeSet);

	return PrecomputeOptimizationData(ChangeSet);
}

FPrecomputedInstanceSpatialHashData FInstanceDataManager::PrecomputeOptimizationData(FInstanceUpdateChangeSet& ChangeSet)
{
	FInstanceDataHelpers::FSpatialHashResult SpatialHashResult = FInstanceDataHelpers::BuildSpatialHashData(ChangeSet.NumSourceInstances,
		[&](int32 InstanceIndex) -> FSphere
		{
			int32 BoundsIndex = FMath::Min(ChangeSet.InstanceLocalBounds.Num() - 1, InstanceIndex);
			// this can totally be optimized
			FSphere3f LocalInstanceSphere = FSphere3f(ChangeSet.InstanceLocalBounds[BoundsIndex].GetCenter(), ChangeSet.InstanceLocalBounds[BoundsIndex].GetExtent().Size());
			FRenderTransform InstanceLocalToWorld = ChangeSet.Transforms[InstanceIndex] * ChangeSet.PrimitiveToRelativeWorld;
			FSphere Result = FSphere(LocalInstanceSphere.TransformBy(InstanceLocalToWorld.ToMatrix44f()));
			Result.Center += ChangeSet.PrimitiveWorldSpaceOffset;
			return Result;
		}
	);

	FPrecomputedInstanceSpatialHashData Result;
	Result.Hashes = MoveTemp(SpatialHashResult.SpatialHashes);
	Result.ProxyIndexToComponentIndexRemap = MoveTemp(SpatialHashResult.ReorderTable);
	Result.PrimitiveLocalToWorldHash = FInstanceDataHelpers::HashPrimitiveLocalToWorld(ChangeSet.PrimitiveToRelativeWorld, ChangeSet.PrimitiveWorldSpaceOffset);
	return Result;
}

void FInstanceDataManager::WriteCookedRenderData(FArchive& Ar, FInstanceDataManagerSourceDataDesc&& ComponentData)
{
	bool bHasCookedData = false;

	bool bShouldBuild = ShouldWriteCookedData(Ar.CookingTarget(), ComponentData.NumInstances);

	if (bShouldBuild)
	{
		if (!PrecomputedOptimizationData.IsValid())
		{
			if (ComponentData.BuildChangeSet)
			{
				PrecomputedOptimizationData = MakeShared<const FPrecomputedInstanceSpatialHashData>(PrecomputeOptimizationData(MoveTemp(ComponentData)));
			}
		}
		
		if (PrecomputedOptimizationData.IsValid())
		{
			// We have to copy the whole thing to be able to serialize?
			FPrecomputedInstanceSpatialHashData OptData = *PrecomputedOptimizationData;

			// Serialize the stuff we need.
			bHasCookedData = true;
			Ar << bHasCookedData;
			Ar << OptData.PrimitiveLocalToWorldHash;

			OptData.Hashes.BulkSerialize(Ar);
			OptData.ProxyIndexToComponentIndexRemap.BulkSerialize(Ar);
		}
	}

	if (!bHasCookedData)
	{
		// write the bool if we didn't write any data previously
		Ar << bHasCookedData;
	}
}
#endif // WITH_EDITOR

void FInstanceDataManager::ReadCookedRenderData(FArchive& Ar)
{
	bool bHasCookedData = false;
	Ar << bHasCookedData;

	if (bHasCookedData)
	{
		FPrecomputedInstanceSpatialHashData Tmp;
		Ar << Tmp.PrimitiveLocalToWorldHash;

		// TODO: Pack the data representation to far fewer bits
		Tmp.Hashes.BulkSerialize(Ar);
		// TODO: RLE-compress
		Tmp.ProxyIndexToComponentIndexRemap.BulkSerialize(Ar);

		// Ditch the precomputed data if it has been disabled (in the runtime), even if the cook was done with the data enabled.
		if (ShouldUsePrecomputed())
		{
			PrecomputedOptimizationData = MakeShared<const FPrecomputedInstanceSpatialHashData>(Tmp);
		}
		else
		{
			PrecomputedOptimizationData.Reset();
		}
	}
}

void FInstanceDataManager::Serialize(FArchive& Ar, bool bCookedOrCooking)
{
	FInstanceIdIndexMap::Serialize(Ar);

	// if we're loading a non-identity map then restore the ID allocation
	if (Ar.IsLoading())
	{
		ValidInstanceIdMask.Empty(IdToIndexMap.Num());
		if (!IdToIndexMap.IsEmpty())
		{
			ValidInstanceIdMask.SetNum(IdToIndexMap.Num(), false);

			for (int32 InstanceIndex = 0; InstanceIndex < IndexToIdMap.Num(); ++InstanceIndex)
			{
				FPrimitiveInstanceId Id = IndexToIdMap[InstanceIndex];
				ValidInstanceIdMask[Id.GetAsIndex()] = true;
			}

		}
		IdSearchStartIndex = 0;
		ClearChangeTracking();
	}
}

TArray<int32> FInstanceDataManager::Optimize(FInstanceDataManagerSourceDataDesc&& ComponentData, bool bShouldRetainIdMap)
{
#if WITH_EDITOR
	FPrecomputedInstanceSpatialHashData OptData = PrecomputeOptimizationData(MoveTemp(ComponentData));
	
	// Note: Currently this just ditches the spatial hash data again. This is the simple and robust solution because of something mutates the data
	//       nothing breaks as we recompute the optimization data anyway during cook. In the case where nothing has changed we'll detect the identity reodering and ditch the reorder table.

	//
	if (bShouldRetainIdMap)
	{
		if (!OptData.ProxyIndexToComponentIndexRemap.IsEmpty())
		{
			if (HasIdentityMapping())
			{
				CreateExplicitIdentityMapping();
			}
		
			TArray<FPrimitiveInstanceId> OldIndexToIdMap = MoveTemp(IndexToIdMap);
			IndexToIdMap.SetNumUninitialized(OldIndexToIdMap.Num());

			for (int32 NewIndex = 0; NewIndex < OptData.ProxyIndexToComponentIndexRemap.Num(); ++NewIndex)
			{
				int32 OldIndex = OptData.ProxyIndexToComponentIndexRemap[NewIndex];
				FPrimitiveInstanceId InstanceId = OldIndexToIdMap[OldIndex];
				Update(InstanceId, NewIndex);
			}
		}
	}
	else
	{
		// As we've moved the instances around we'll drop the ID mapping noew
		Reset(GetMaxInstanceIndex());
	}
	return OptData.ProxyIndexToComponentIndexRemap;
#else
	return TArray<int32>();
#endif // WITH_EDITOR
}
