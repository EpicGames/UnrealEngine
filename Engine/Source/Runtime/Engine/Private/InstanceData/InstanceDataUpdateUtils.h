// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "RenderingThread.h"
#include "RenderTransform.h"
#include "InstanceDataSceneProxy.h"
#include "InstanceData/InstanceDataManager.h"
#include "InstanceData/InstanceUpdateChangeSet.h"

#define INSTANCE_DATA_UPDATE_ENABLE_ASYNC_TASK 1

/**
 * Write InSource that was previously gathered using the above methods to the final destination array OutDest
 * using the same delta information. 
 * If there is no delta and the index remap is identity, it performs a move of the source data to the final array, saving a malloc & copy.
 */
template <typename DeltaType, typename ValueType, typename IndexRemapType>
void Scatter(const DeltaType& Delta, TArray<ValueType>& OutDest, int32 DestNumElements, TArray<ValueType>&& InSource, const IndexRemapType& IndexRemap, int32 ElementStride = 1)
{
	check(InSource.Num() == Delta.GetNumItems() * ElementStride);
	if (Delta.IsDelta() || !IndexRemap.IsIdentity())
	{
		OutDest.SetNumUninitialized(DestNumElements * ElementStride);
		for (auto It = Delta.GetIterator(); It; ++It)
		{
			int32 SrcIndex = It.GetItemIndex();
			int32 DestIndex = It.GetIndex();

			if (IndexRemap.Remap(SrcIndex, DestIndex))
			{
				FMemory::Memcpy(&OutDest[DestIndex * ElementStride], &InSource[SrcIndex * ElementStride], ElementStride * sizeof(ValueType));
			}
		}
	}
	else
	{
		check(InSource.Num() == DestNumElements * ElementStride);
		OutDest = MoveTemp(InSource);
	}
}

/**
 * Also takes a flag to optionally reset the array in case the attribute is disabled
 */
template <typename DeltaType, typename ValueType, typename IndexRemapType>
inline void Scatter(bool bHasData, const DeltaType& Delta, TArray<ValueType>& DestData, int32 NumOutElements, TArray<ValueType>&& InData, const IndexRemapType& IndexRemap, int32 ElementStride = 1)
{
	if (bHasData)
	{
		::Scatter(Delta, DestData, NumOutElements, MoveTemp(InData), IndexRemap, ElementStride);
	}
	else
	{
		DestData.Reset();
	}
}

/**
 * Gather the needed values from InSource to OutDest, according to the delta.
 * If there is no delta, it will perform a bulk copy.
 */
template <typename DeltaType, typename ValueType, typename InValueArrayType>
void Gather(const DeltaType& Delta, TArray<ValueType>& OutDest, const InValueArrayType& InSource, int32 ElementStride = 1)
{
	// strides & element count matches - just copy the data
	if (InSource.Num() == Delta.GetNumItems() * ElementStride)
	{
		OutDest = InSource;
	}
	else if (Delta.IsEmpty())
	{
		OutDest.Reset();
	}
	else if (Delta.IsDelta() || ElementStride != 1)
	{
		OutDest.Reset(Delta.GetNumItems() * ElementStride);
		for (auto It = Delta.GetIterator(); It; ++It)
		{
			check(OutDest.Num() < Delta.GetNumItems() * ElementStride);
			OutDest.Append(&InSource[It.GetIndex() * ElementStride], ElementStride);
		}
	}
}

/**
 */
struct FIdentityIndexRemap
{
	FORCEINLINE constexpr bool IsIdentity() const { return true; }
	FORCEINLINE int32 operator[](int32 InIndex) const { return InIndex; }
	FORCEINLINE constexpr bool RemapDestIndex(int32 Index) const { return true; }
	FORCEINLINE bool Remap(int32& SrcIndex, int32& DstIndex) const  { return true; }
};

/**
 * Uses an array to remap the source index, leaving the destination index unchanged.
 */
struct FSrcIndexRemap
{
	FORCEINLINE constexpr bool IsIdentity() const { return false; }
	FORCEINLINE FSrcIndexRemap(const TArray<int32>& InIndexRemap) : IndexRemap(InIndexRemap) {}
	FORCEINLINE bool RemapDestIndex(int32& Index) const { return true;}
	FORCEINLINE bool Remap(int32& SrcIndex, int32& DstIndex) const 
	{ 
		SrcIndex = IndexRemap[SrcIndex];
		return true; 
	}
	const TArray<int32>& IndexRemap;
};

/**
 * Helper function to conditionally move a single element
 */
template<typename ValueType>
void CondMove(bool bCondition, TArray<ValueType>& Data, int32 FromIndex, int32 ToIndex, int32 NumElements = 1)
{
	if (bCondition)
	{
		FMemory::Memcpy(&Data[ToIndex * NumElements], &Data[FromIndex * NumElements], NumElements * sizeof(ValueType));
	}
}

/**
 * Vector register version of FRenderTransform, used to preload the primitive to world transform into registers
 */
struct FRenderTransformVectorRegister
{
	VectorRegister4f R0;
	VectorRegister4f R1;
	VectorRegister4f R2;
	VectorRegister4f Origin;

	FORCEINLINE FRenderTransformVectorRegister(const FRenderTransform& RenderTransform)
	{
		//  we can use unaligmed vectorized load since we know there is data beyond the three rows (the origin), so it is ok to load whatever into the 4th component.
		R0 = VectorLoad(&RenderTransform.TransformRows[0].X);
		R1 = VectorLoad(&RenderTransform.TransformRows[1].X);
		R2 = VectorLoad(&RenderTransform.TransformRows[2].X);
		// But not for the origin
		Origin = VectorLoadFloat3(&RenderTransform.Origin);
	}
};

FORCEINLINE_DEBUGGABLE FRenderTransform VectorMatrixMultiply(const FRenderTransform& LocalToPrimitive, const FRenderTransformVectorRegister& PrimitiveToWorld)
{
	FRenderTransform Result;

	// First row of result (Matrix1[0] * Matrix2).
	{
		//  we can use unaligmed vectorized load since we know there is data beyond the three rows (the origin), so it is ok to load whatever into the 4th component.
		const VectorRegister4Float ARow = VectorLoad(&LocalToPrimitive.TransformRows[0].X);
		VectorRegister4Float R0 = VectorMultiply(VectorReplicate(ARow, 0), PrimitiveToWorld.R0);
		R0 = VectorMultiplyAdd(VectorReplicate(ARow, 1), PrimitiveToWorld.R1, R0);
		R0 = VectorMultiplyAdd(VectorReplicate(ARow, 2), PrimitiveToWorld.R2, R0);
	
		// We can use unaligmed vectorized store since we know there is data beyond the three floats that is written later
		// Note: stomps the X of the TransformRows[1]
		VectorStore(R0, &Result.TransformRows[0].X);		
	}

	// Second row of result (Matrix1[1] * Matrix2).
	{
		//  we can use unaligmed vectorized load since we know there is data beyond the three rows (the origin), so it is ok to load whatever into the 4th component.
		const VectorRegister4Float ARow = VectorLoad(&LocalToPrimitive.TransformRows[1].X);
		VectorRegister4Float R1 = VectorMultiply(VectorReplicate(ARow, 0), PrimitiveToWorld.R0);
		R1 = VectorMultiplyAdd(VectorReplicate(ARow, 1), PrimitiveToWorld.R1, R1);
		R1 = VectorMultiplyAdd(VectorReplicate(ARow, 2), PrimitiveToWorld.R2, R1);

		// We can use unaligmed vectorized store since we know there is data beyond the three floats that is written later
		// Note: stomps the X of the TransformRows[2]
		VectorStore(R1, &Result.TransformRows[1].X);
	}

	// Third row of result (Matrix1[2] * Matrix2).
	{
		//  we can use unaligmed vectorized load since we know there is data beyond the three rows (the origin), so it is ok to load whatever into the 4th component.
		const VectorRegister4Float ARow = VectorLoad(&LocalToPrimitive.TransformRows[2].X);
		VectorRegister4Float R2 = VectorMultiply(VectorReplicate(ARow, 0), PrimitiveToWorld.R0);
		R2 = VectorMultiplyAdd(VectorReplicate(ARow, 1), PrimitiveToWorld.R1, R2);
		R2 = VectorMultiplyAdd(VectorReplicate(ARow, 2), PrimitiveToWorld.R2, R2);

		// We can use unaligmed vectorized store since we know there is data beyond the three floats that is written later
		// Note: stomps the X of the Origin
		VectorStore(R2, &Result.TransformRows[2].X);
	}

	// Fourth row of result (Matrix1[3] * Matrix2).
	{
		//  can _NOT_ use VectorLoad, or we'll run off the end of the FRenderTransform struct.
		const VectorRegister4Float ARow = VectorLoadFloat3(&LocalToPrimitive.Origin);
		
		// Add B3 at once (instead of mult by 1.0 which would have been the fourth value in the 4x4 version of the matrix)
		VectorRegister4Float R3 = VectorMultiplyAdd(VectorReplicate(ARow, 0), PrimitiveToWorld.R0, PrimitiveToWorld.Origin);
		R3 = VectorMultiplyAdd(VectorReplicate(ARow, 1), PrimitiveToWorld.R1, R3);
		R3 = VectorMultiplyAdd(VectorReplicate(ARow, 2), PrimitiveToWorld.R2, R3);

		VectorStoreFloat3(R3, &Result.Origin);
	}
	return Result;
}


/**
 * Helper function to apply transform update that selectively performs Orthogonalize only if the primitive transform has any non-uniform scale.
 */

template <typename DeltaType, typename IndexRemapType>
FORCEINLINE void ApplyTransformUpdates(const DeltaType& DeltaRange, const IndexRemapType& IndexRemap, const FRenderTransform& PrimitiveToRelativeWorld, const TArray<FRenderTransform>& InstanceTransforms, int32 PostUpdateNumTransforms, TArray<FRenderTransform>& OutInstanceToPrimitiveRelative)
{
	OutInstanceToPrimitiveRelative.SetNumUninitialized(PostUpdateNumTransforms);

	if (DeltaRange.IsEmpty())
	{
		return;
	}

	if (PrimitiveToRelativeWorld.IsScaleNonUniform())
	{
		FRenderTransformVectorRegister PrimitiveToRelativeWorldVR(PrimitiveToRelativeWorld);
		for (auto It = DeltaRange.GetIterator(); It; ++It)
		{
			int32 ItemIndex = It.GetItemIndex();
			int32 InstanceIndex = It.GetIndex();

			if (IndexRemap.Remap(ItemIndex, InstanceIndex))
			{
				FRenderTransform LocalToPrimitiveRelativeWorld = VectorMatrixMultiply(InstanceTransforms[ItemIndex], PrimitiveToRelativeWorldVR);
				// Remove shear
				LocalToPrimitiveRelativeWorld.Orthogonalize();
				OutInstanceToPrimitiveRelative[InstanceIndex] = LocalToPrimitiveRelativeWorld;
			}
		}
	}
	else
	{
		FRenderTransformVectorRegister PrimitiveToRelativeWorldVR(PrimitiveToRelativeWorld);
		for (auto It = DeltaRange.GetIterator(); It; ++It)
		{
			int32 ItemIndex = It.GetItemIndex();
			int32 InstanceIndex = It.GetIndex();

			if (IndexRemap.Remap(ItemIndex, InstanceIndex))
			{
				OutInstanceToPrimitiveRelative[InstanceIndex] = VectorMatrixMultiply(InstanceTransforms[ItemIndex], PrimitiveToRelativeWorldVR);
			}
		}
	}

};

inline FVector3f GetLocalBoundsPadExtent(const FRenderTransform& LocalToWorld, float PadAmount)
{
	if (FMath::Abs(PadAmount) < UE_SMALL_NUMBER)
	{
		return FVector3f::ZeroVector;
	}

	FVector3f Scale = LocalToWorld.GetScale();
	return FVector3f(
		Scale.X > 0.0f ? PadAmount / Scale.X : 0.0f,
		Scale.Y > 0.0f ? PadAmount / Scale.Y : 0.0f,
		Scale.Z > 0.0f ? PadAmount / Scale.Z : 0.0f);
}


template <typename IndexRemapType>
inline void UpdateIdMapping(FInstanceUpdateChangeSet& ChangeSet, const IndexRemapType& IndexRemap, FInstanceIdIndexMap& OutInstanceIdIndexMap)
{
	// update mapping, create explicit mapping if needed
	if (ChangeSet.bIdentityIdMap && IndexRemap.IsIdentity())
	{
		// Reset to identity mapping with the new number of instances
		OutInstanceIdIndexMap.Reset(ChangeSet.NumSourceInstances);
	}
	else 
	{
		auto IndexDelta = ChangeSet.GetIndexChangedDelta();
		bool bIsFull = !IndexDelta.IsDelta() || ChangeSet.NumSourceInstances == ChangeSet.IndexToIdMapDeltaData.Num();

		// Efficient full-data update path if there is no index remap
		if (bIsFull && IndexRemap.IsIdentity())
		{
			OutInstanceIdIndexMap.RebuildFromIndexToIdMap(MoveTemp(ChangeSet.IndexToIdMapDeltaData), ChangeSet.MaxInstanceId);
		}
		else
		{
			// General path that handles incremental removes and other updates.
			OutInstanceIdIndexMap.ResizeExplicit(ChangeSet.NumSourceInstances, ChangeSet.MaxInstanceId);

			// If any were removed, we need to clear the associated IDs _before_ updating (since they may have been added again)
			for (TConstSetBitIterator<> It(ChangeSet.InstanceAttributeTracker.GetRemovedIterator()); It; ++It)
			{
				// There may be more bits set as things that are marked as removed may no longer be in the map
				if (It.GetIndex() >= OutInstanceIdIndexMap.GetMaxInstanceId())
				{
					break;
				}
				OutInstanceIdIndexMap.SetInvalid(FPrimitiveInstanceId{It.GetIndex()});
			}

			// Update index mappings
			for (auto It = IndexDelta.GetIterator(); It; ++It)
			{
				int32 NewInstanceIndex = It.GetIndex();
				int32 ItemIndex = It.GetItemIndex();
			
				IndexRemap.Remap(ItemIndex, NewInstanceIndex);

				FPrimitiveInstanceId InstanceId = ChangeSet.bIdentityIdMap ? FPrimitiveInstanceId{ItemIndex} : ChangeSet.IndexToIdMapDeltaData[ItemIndex];
				OutInstanceIdIndexMap.Update(InstanceId, NewInstanceIndex);
			}
		}
	}
}

template <typename IndexRemapType>
inline void ApplyAttributeChanges(FInstanceUpdateChangeSet& ChangeSet, const IndexRemapType& IndexRemap, FInstanceSceneDataBuffers::FWriteView& ProxyData)
{
	ChangeSet.GetCustomDataReader().Scatter(ProxyData.InstanceCustomData, IndexRemap);
	ProxyData.NumCustomDataFloats = ChangeSet.NumCustomDataFloats;
	check(ProxyData.Flags.bHasPerInstanceCustomData || ProxyData.NumCustomDataFloats == 0);
	ChangeSet.GetSkinningDataReader().Scatter(ProxyData.InstanceSkinningData, IndexRemap);
	ChangeSet.GetLightShadowUVBiasReader().Scatter(ProxyData.InstanceLightShadowUVBias, IndexRemap);

#if WITH_EDITOR
	ChangeSet.GetEditorDataReader().Scatter(ProxyData.InstanceEditorData, IndexRemap);
#endif

	// Delayed per instance random generation, moves it off the GT and RT, but still sucks
	if (ChangeSet.Flags.bHasPerInstanceRandom)
	{
		// TODO: only need to process added instances? No help for ISM since the move path would be taken.
		// TODO: OTOH for HISM there is no meaningful data, so just skipping and letting the SetNumZeroed fill in the blanks is fine.

		// TODO: Move this to the caller, i.e., the update lambda?
		ProxyData.InstanceRandomIDs.SetNumZeroed(ChangeSet.NumSourceInstances);
		if (ChangeSet.GeneratePerInstanceRandomIds)
		{
			// NOTE: this is not super efficient(!)
			TArray<float> TmpInstanceRandomIDs;
			TmpInstanceRandomIDs.SetNumZeroed(ChangeSet.NumSourceInstances);
			ChangeSet.GeneratePerInstanceRandomIds(TmpInstanceRandomIDs);
			FIdentityDeltaRange PerInstanceRandomDelta(TmpInstanceRandomIDs.Num());
			Scatter(PerInstanceRandomDelta, ProxyData.InstanceRandomIDs, ChangeSet.NumSourceInstances, MoveTemp(TmpInstanceRandomIDs), IndexRemap);
		}
		//else 
		//{
		//	IndexRemap.Scatter(true, PerInstanceRandomDelta, ProxyData.InstanceRandomIDs, ChangeSet.NumSourceInstances, MoveTemp(ChangeSet.InstanceRandomIDs));
		//}
	}
	else
	{
		ProxyData.InstanceRandomIDs.Reset();
	}
}

template <typename TaskLambdaType>
void BeginInstanceDataUpdateTask(FInstanceDataUpdateTaskInfo& InstanceDataUpdateTaskInfo, TaskLambdaType&& TaskLambda, const FInstanceDataBufferHeader& InInstanceDataBufferHeader)
{
	// Make sure any previous tasks are done.
	InstanceDataUpdateTaskInfo.WaitForUpdateCompletion();
	InstanceDataUpdateTaskInfo.InstanceDataBufferHeader = InInstanceDataBufferHeader;

#if INSTANCE_DATA_UPDATE_ENABLE_ASYNC_TASK
	InstanceDataUpdateTaskInfo.UpdateTaskHandle = UE::Tasks::Launch(TEXT("FInstanceDataUpdateTaskInfo::BeginUpdateTask"), MoveTemp(TaskLambda));
#else
	InstanceDataUpdateTaskInfo.UpdateTaskHandle = UE::Tasks::FTask();
	TaskLambda();
#endif
}

template <typename ProxyType, typename TaskLambdaType>
void DispatchInstanceDataUpdateTask(bool bIsUnattached, const TSharedPtr<ProxyType, ESPMode::ThreadSafe>& InstanceDataProxy, const FInstanceDataBufferHeader& InstanceDataBufferHeader, TaskLambdaType&& TaskLambda)
{
	FInstanceDataUpdateTaskInfo *InstanceDataUpdateTaskInfo = InstanceDataProxy->GetUpdateTaskInfo();
#if DO_CHECK
	auto OuterTaskLambda = [InnerTaskLambda = MoveTemp(TaskLambda), InstanceDataBufferHeader, InstanceDataProxy = InstanceDataProxy]() mutable
	{
		FInstanceDataUpdateTaskInfo *InstanceDataUpdateTaskInfo = InstanceDataProxy->GetUpdateTaskInfo();
		check(!InstanceDataUpdateTaskInfo || InstanceDataUpdateTaskInfo->GetHeader() == InstanceDataBufferHeader);
		InnerTaskLambda();
		check(!InstanceDataUpdateTaskInfo || InstanceDataUpdateTaskInfo->GetHeader() == InstanceDataBufferHeader);
		check(!InstanceDataUpdateTaskInfo || InstanceDataBufferHeader.NumInstances == InstanceDataProxy->GeInstanceSceneDataBuffers()->GetNumInstances());
		// check(InstanceDataBufferHeader.Flags == InstanceDataProxy->GetData().GetFlags());
		const FInstanceDataFlags HeaderFlags = InstanceDataBufferHeader.Flags;
		const bool bHasAnyPayloadData = HeaderFlags.bHasPerInstanceHierarchyOffset 
			|| HeaderFlags.bHasPerInstanceLocalBounds 
			|| HeaderFlags.bHasPerInstanceDynamicData 
			|| HeaderFlags.bHasPerInstanceLMSMUVBias 
			|| HeaderFlags.bHasPerInstanceCustomData 
			|| HeaderFlags.bHasPerInstancePayloadExtension 
			|| HeaderFlags.bHasPerInstanceSkinningData 
			|| HeaderFlags.bHasPerInstanceEditorData;
		check(bHasAnyPayloadData || InstanceDataBufferHeader.PayloadDataStride == 0);
		check(!bHasAnyPayloadData || InstanceDataBufferHeader.PayloadDataStride != 0);
	};
#else
	TaskLambdaType OuterTaskLambda = MoveTemp(TaskLambda);
#endif
	// Dispatch from any thread.
	if (bIsUnattached)
	{
		if (InstanceDataUpdateTaskInfo)
		{
			BeginInstanceDataUpdateTask(*InstanceDataUpdateTaskInfo, MoveTemp(OuterTaskLambda), InstanceDataBufferHeader);
		}
		else
		{
			OuterTaskLambda();
		}
	}
	else
	{
		// Mutating an existing data, must dispatch from RT (such that it does not happen mid-frame).
		// (One could imagine other scheduling mechanisms)
		ENQUEUE_RENDER_COMMAND(UpdateInstanceProxyData)(
			[InstanceDataUpdateTaskInfo, 
			InstanceDataBufferHeader, 
			OuterTaskLambda = MoveTemp(OuterTaskLambda)](FRHICommandList& RHICmdList) mutable
		{
			if (InstanceDataUpdateTaskInfo)
			{
				BeginInstanceDataUpdateTask(*InstanceDataUpdateTaskInfo, MoveTemp(OuterTaskLambda), InstanceDataBufferHeader);
			}
			else
			{
				OuterTaskLambda();
			}
		});
	}
}

#if WITH_EDITOR

class FSpatialHashSortBuilder
{
public:
	struct FSortedInstanceItem
	{
		RenderingSpatialHash::FLocation64 InstanceLoc;
		int32 InstanceIndex;
	};

	template <typename GetWorldSpaceInstanceSphereFuncType>
	void BuildOptimizedSpatialHashOrder(int32 NumInstances, int32 MinLevel, GetWorldSpaceInstanceSphereFuncType&& GetWorldSpaceInstanceSphere)
	{
		int32 FirstLevel = MinLevel;

		SortedInstances.Reserve(NumInstances);
		for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
		{
			FSphere InstanceWorldSpaceSphere = GetWorldSpaceInstanceSphere(InstanceIndex);

			RenderingSpatialHash::FLocation64 InstanceLoc = RenderingSpatialHash::CalcLevelAndLocationClamped(InstanceWorldSpaceSphere.Center, InstanceWorldSpaceSphere.W, FirstLevel);

			FSortedInstanceItem& Item = SortedInstances.AddDefaulted_GetRef();
			Item.InstanceLoc = InstanceLoc;
			Item.InstanceIndex = InstanceIndex;
		}
	
		// Sort the instances according to hash location (first level, then coordinate) and last on instance Index.
		SortedInstances.Sort(
			[](const FSortedInstanceItem& A, const FSortedInstanceItem& B) -> bool
			{
				if (A.InstanceLoc.Level != B.InstanceLoc.Level)
				{
					return A.InstanceLoc.Level < B.InstanceLoc.Level;
				}
				if (A.InstanceLoc.Coord.X != B.InstanceLoc.Coord.X)
				{
					return A.InstanceLoc.Coord.X < B.InstanceLoc.Coord.X;
				}
				if (A.InstanceLoc.Coord.Y != B.InstanceLoc.Coord.Y)
				{
					return A.InstanceLoc.Coord.Y < B.InstanceLoc.Coord.Y;
				}
				if (A.InstanceLoc.Coord.Z != B.InstanceLoc.Coord.Z)
				{
					return A.InstanceLoc.Coord.Z < B.InstanceLoc.Coord.Z;
				}
				return A.InstanceIndex < B.InstanceIndex;
			}
		);
	}
	TArray<FSortedInstanceItem> SortedInstances;
};

#endif //WITH_EDITOR
