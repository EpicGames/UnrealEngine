// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceData/InstanceUpdateChangeSet.h"
#include "InstanceData/InstanceDataUpdateUtils.h"
#include "Engine/InstancedStaticMesh.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Rendering/RenderingSpatialHash.h"
#include "Rendering/MotionVectorSimulation.h"
#include "SceneInterface.h"



#if WITH_EDITOR

/**
 * Ugly little wrapper to make sure hitproxies that are kept alive by the proxy on the RT are deleted on the GT
 */
class FOpaqueHitProxyContainer
{
public:
	FOpaqueHitProxyContainer(const TArray<TRefCountPtr<HHitProxy>>& InHitProxies) : HitProxies(InHitProxies) {}
	~FOpaqueHitProxyContainer();

private:
	FOpaqueHitProxyContainer(const FOpaqueHitProxyContainer &) = delete;
	void operator=(const FOpaqueHitProxyContainer &) = delete;

	TArray<TRefCountPtr<HHitProxy>> HitProxies;
};

FOpaqueHitProxyContainer::~FOpaqueHitProxyContainer()
{
	struct DeferDeleteHitProxies : FDeferredCleanupInterface
	{
		DeferDeleteHitProxies(TArray<TRefCountPtr<HHitProxy>>&& InHitProxies) : HitProxies(MoveTemp(InHitProxies)) {}
		TArray<TRefCountPtr<HHitProxy>> HitProxies;
	};

	BeginCleanup(new DeferDeleteHitProxies(MoveTemp(HitProxies)));
}


void FInstanceUpdateChangeSet::SetEditorData(const TArray<TRefCountPtr<HHitProxy>>& HitProxies, const TBitArray<> &InSelectedInstances)//, bool bWasHitProxiesReallocated)
{
	HitProxyContainer = MakeOpaqueHitProxyContainer(HitProxies);
	for (int32 Index = 0; Index < HitProxies.Num(); ++Index)
	{
		// Record if the instance is selected
		FColor HitProxyColor(ForceInit);
		bool bSelected = !InSelectedInstances.IsValidIndex(Index) || InSelectedInstances[Index];
		if (HitProxies.IsValidIndex(Index))
		{
			HitProxyColor = HitProxies[Index]->Id.GetColor();
		}
		InstanceEditorData.Add(FInstanceEditorData::Pack(HitProxyColor, bSelected));
	}
	SelectedInstances = InSelectedInstances;
}

#endif

void FInstanceUpdateChangeSet::SetSharedLocalBounds(const FRenderBounds &Bounds)
{
	check(!Flags.bHasPerInstanceLocalBounds);
	InstanceLocalBounds.SetNum(1);
	InstanceLocalBounds[0] = Bounds;
}

#if WITH_EDITOR

TPimplPtr<FOpaqueHitProxyContainer> MakeOpaqueHitProxyContainer(const TArray<TRefCountPtr<HHitProxy>>& InHitProxies)
{
	return MakePimpl<FOpaqueHitProxyContainer>(InHitProxies);
}

#endif
