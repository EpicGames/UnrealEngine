// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataSceneProxy.h"
#include "Engine/InstancedStaticMesh.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Rendering/RenderingSpatialHash.h"
#include "Rendering/MotionVectorSimulation.h"

DECLARE_STATS_GROUP( TEXT("InstanceData"), STATGROUP_InstanceData, STATCAT_Advanced );

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Instances"), STAT_InstanceDataInstanceCount, STATGROUP_InstanceData);
DECLARE_MEMORY_STAT(TEXT("Nanite Proxy Instance Memory"), STAT_ProxyInstanceMemory, STATGROUP_InstanceData);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Dynamic Data Instances"), STAT_InstanceHasDynamicCount, STATGROUP_InstanceData);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Skinning Data Instances"), STAT_InstanceHasSkinningCount, STATGROUP_InstanceData);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("LMSM Data Instances"), STAT_InstanceHasLMSMBiasCount, STATGROUP_InstanceData);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Custom Data Instances"), STAT_InstanceHasCustomDataCount, STATGROUP_InstanceData);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Random Data Instances"), STAT_InstanceHasRandomCount, STATGROUP_InstanceData);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Local Bounds Instances"), STAT_InstanceHasLocalBounds, STATGROUP_InstanceData);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Hierarchy Offset Instances"), STAT_InstanceHasHierarchyOffset, STATGROUP_InstanceData);

const FInstanceDataBufferHeader FInstanceDataBufferHeader::SinglePrimitiveHeader = { 1, 0, FInstanceDataFlags(), false };

void FInstanceIdIndexMap::Reset(int32 InNumInstances)
{
	IndexToIdMap.Reset();
	IdToIndexMap.Reset();
	NumInstances = InNumInstances;
}

void FInstanceIdIndexMap::ResizeExplicit(int32 InNumInstances, int32 MaxInstanceId)
{
	if (IsIdentity())
	{
		// Create explicit mapping & before applying changes
		CreateExplicitIdentityMapping();
	}
	IndexToIdMap.SetNumUninitialized(InNumInstances);
	NumInstances = InNumInstances;
	if (MaxInstanceId != IdToIndexMap.Num())
	{
		int32 OldCount = IdToIndexMap.Num();
		IdToIndexMap.SetNumUninitialized(MaxInstanceId);
		for (int32 Index = OldCount; Index < MaxInstanceId; ++Index)
		{
			IdToIndexMap[Index] = INDEX_NONE;
		}
	}
}

void FInstanceIdIndexMap::CreateExplicitIdentityMapping()
{
	check(IsIdentity());
	IndexToIdMap.SetNumUninitialized(NumInstances);
	IdToIndexMap.SetNumUninitialized(NumInstances);
	for (int32 Index = 0; Index < NumInstances; ++Index)
	{
		IndexToIdMap[Index] = FPrimitiveInstanceId{Index};
		IdToIndexMap[Index] = Index;
	}
}

void FInstanceIdIndexMap::Serialize(FArchive& Ar)
{
	// Serialize the smaller of the two maps (1:1 with instances, whereas the other might have holes).
	Ar << IndexToIdMap;
	Ar << NumInstances;

	int32 MaxInstanceId = GetMaxInstanceId();
	Ar << MaxInstanceId;
	
	if (Ar.IsLoading()) 
	{
		// if we're loading a non-identity map then restore the ID to index map as well
		if (!IndexToIdMap.IsEmpty())
		{
			RebuildFromIndexToIdMap(MaxInstanceId);
		}
		else
		{
			// identity mapping, make sure the array is empty
			IdToIndexMap.Reset();
		}
	}
}

void FInstanceIdIndexMap::RebuildFromIndexToIdMap(int32 MaxInstanceId)
{
	IdToIndexMap.SetNum(MaxInstanceId);
	for (int32 Index = 0; Index < MaxInstanceId; ++Index)
	{
		IdToIndexMap[Index] = INDEX_NONE;
	}
	for (int32 InstanceIndex = 0; InstanceIndex < IndexToIdMap.Num(); ++InstanceIndex)
	{
		FPrimitiveInstanceId Id = IndexToIdMap[InstanceIndex];
		IdToIndexMap[Id.Id] = InstanceIndex;
	}
}

void FInstanceIdIndexMap::RebuildFromIndexToIdMap(TArray<FPrimitiveInstanceId>&& InIndexToIdMap, int32 MaxInstanceId)
{
	IndexToIdMap = MoveTemp(InIndexToIdMap);
	RebuildFromIndexToIdMap(MaxInstanceId);
}

FInstanceSceneDataBuffers::FInstanceSceneDataBuffers()
{
}

FInstanceSceneDataBuffers::FInstanceSceneDataBuffers(bool InbInstanceDataIsGPUOnly)
	: bInstanceDataIsGPUOnly(InbInstanceDataIsGPUOnly)
{
}

uint32 FInstanceSceneDataBuffers::CalcPayloadDataStride(FInstanceDataFlags Flags, int32 InNumCustomDataFloats, int32 InNumPayloadExtensionFloat4s)
{
	static_assert(sizeof(FRenderTransform) == sizeof(float) * 3 * 4); // Sanity check
	static_assert(sizeof(FRenderBounds) == sizeof(float) * 3 * 2); // Sanity check

	// This count is per instance.
	uint32 PayloadDataCount = 0;

	// Random ID is packed into scene data currently
	if (FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(GMaxRHIShaderPlatform))
	{
		PayloadDataCount += Flags.bHasPerInstanceDynamicData ? 2 : 0;	// Compressed transform
	}
	else
	{
		PayloadDataCount += Flags.bHasPerInstanceDynamicData ? 3 : 0;	// FRenderTransform
	}
		
	// Hierarchy is packed in with local bounds if they are both present (almost always the case)
	if (Flags.bHasPerInstanceLocalBounds)
	{
		PayloadDataCount += 2; // FRenderBounds and possibly uint32 for hierarchy offset & another uint32 for EditorData
	}
	else if (Flags.bHasPerInstanceHierarchyOffset || Flags.bHasPerInstanceSkinningData)
	{
		PayloadDataCount += 1; // uint32 for hierarchy offset (float4 packed) & instance skinning data is packed in the same float4
	}

	PayloadDataCount += Flags.bHasPerInstanceEditorData ? 1 : 0; // FVector4

	PayloadDataCount += Flags.bHasPerInstanceLMSMUVBias ? 1 : 0; // FVector4

	if (Flags.bHasPerInstancePayloadExtension)
	{
		PayloadDataCount += uint32(InNumPayloadExtensionFloat4s);
	}

	if (Flags.bHasPerInstanceCustomData)
	{
		PayloadDataCount += FMath::DivideAndRoundUp(uint32(InNumCustomDataFloats), 4u);
	}

	return PayloadDataCount;
}


uint32 FInstanceSceneDataBuffers::GetPayloadDataStride(FAccessTag AccessTag) const
{
	ValidateAccess(AccessTag);

	int32 NumPayloadExtensionFloat4s = 0;
	if (Flags.bHasPerInstancePayloadExtension)
	{
		const uint32 InstanceCount   = GetNumInstances();
		const uint32 ExtensionCount = InstancePayloadExtension.Num();
		if (InstanceCount > 0)
		{
			NumPayloadExtensionFloat4s = ExtensionCount / InstanceCount;
		}
	}

	return CalcPayloadDataStride(Flags, NumCustomDataFloats, NumPayloadExtensionFloat4s);
}


FRenderBounds FInstanceSceneDataBuffers::GetInstanceLocalBounds(int32 InstanceIndex, FAccessTag AccessTag) const
{
	ValidateAccess(AccessTag);
	// Get the clamped bound in case there are no per-instance unique bounds (the common case)
	return InstanceLocalBounds[FMath::Min(InstanceIndex,InstanceLocalBounds.Num() - 1)];
}

FRenderBounds FInstanceSceneDataBuffers::GetInstancePrimitiveRelativeBounds(int32 InstanceIndex, FAccessTag AccessTag) const
{
	ValidateAccess(AccessTag);
	return GetInstanceLocalBounds(InstanceIndex).TransformBy(InstanceToPrimitiveRelative[InstanceIndex]);
}

FBoxSphereBounds FInstanceSceneDataBuffers::GetInstanceWorldBounds(int32 InstanceIndex, FAccessTag AccessTag) const
{
	ValidateAccess(AccessTag);
	FRenderBounds PrimitiveRelativeBoundingBox = GetInstancePrimitiveRelativeBounds(InstanceIndex);

	FBoxSphereBounds WorldSpaceBounds = PrimitiveRelativeBoundingBox.ToBoxSphereBounds();
	WorldSpaceBounds.Origin += PrimitiveWorldSpaceOffset;

	return WorldSpaceBounds;
}

FMatrix FInstanceSceneDataBuffers::GetInstanceToWorld(int32 InstanceIndex, FAccessTag AccessTag) const
{
	ValidateAccess(AccessTag);
	FMatrix InstanceToWorld = InstanceToPrimitiveRelative[InstanceIndex].ToMatrix();
	return InstanceToWorld.ConcatTranslation(PrimitiveWorldSpaceOffset);
}

FRenderTransform FInstanceSceneDataBuffers::ComputeInstanceToPrimitiveRelative(const FMatrix44f &InstanceToPrimitive, FAccessTag AccessTag)
{
	ValidateAccess(AccessTag);
	FRenderTransform InstanceToPrimitiveRelativeWorld = FRenderTransform(InstanceToPrimitive) * PrimitiveToRelativeWorld;
	// Remove shear
	InstanceToPrimitiveRelativeWorld.Orthogonalize();

	return InstanceToPrimitiveRelativeWorld;
}

void FInstanceSceneDataBuffers::SetPrimitiveLocalToWorld(const FMatrix &PrimitiveLocalToWorld, FAccessTag AccessTag)
{
	ValidateAccess(AccessTag);
	const FVector3f PrimitiveWorldSpacePositionHigh = FDFVector3{ PrimitiveLocalToWorld.GetOrigin() }.High;
	PrimitiveWorldSpaceOffset = FVector{ PrimitiveWorldSpacePositionHigh };
	PrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(PrimitiveWorldSpacePositionHigh, PrimitiveLocalToWorld).M;
}

FInstanceDataBufferHeader FInstanceSceneDataBuffers::GetHeader(FAccessTag AccessTag) const
{
	ValidateAccess(AccessTag);
	// Bit ugly, only this way to keep FInstanceDataBufferHeader fwd-declarable...
	return FInstanceDataBufferHeader{ GetNumInstances(), GetPayloadDataStride(), Flags, bInstanceDataIsGPUOnly };
}

template <typename ArrayType>
static void ValidateArray(bool bFlag, const ArrayType &Array, int32 NumInstances, bool bInstanceDataIsGPUOnly, int32 ElementStride = 1)
{
	const bool bDataShouldBePresent = bFlag && !bInstanceDataIsGPUOnly;
	check(bDataShouldBePresent || Array.IsEmpty());
	check(!bDataShouldBePresent || Array.Num() == NumInstances * ElementStride);
}

void FInstanceSceneDataBuffers::ValidateData() const
{
	ValidateArray(Flags.bHasPerInstanceCustomData, InstanceCustomData, GetNumInstances(), bInstanceDataIsGPUOnly, NumCustomDataFloats);
	ValidateArray(Flags.bHasPerInstanceRandom, InstanceRandomIDs, GetNumInstances(), bInstanceDataIsGPUOnly);
	ValidateArray(Flags.bHasPerInstanceLMSMUVBias, InstanceLightShadowUVBias, GetNumInstances(), bInstanceDataIsGPUOnly);
	ValidateArray(Flags.bHasPerInstanceHierarchyOffset, InstanceHierarchyOffset, GetNumInstances(), bInstanceDataIsGPUOnly);
	ValidateArray(Flags.bHasPerInstanceDynamicData, PrevInstanceToPrimitiveRelative, GetNumInstances(), bInstanceDataIsGPUOnly);
	ValidateArray(Flags.bHasPerInstanceSkinningData, InstanceSkinningData, GetNumInstances(), bInstanceDataIsGPUOnly);
#if WITH_EDITOR
	ValidateArray(Flags.bHasPerInstanceEditorData, InstanceEditorData, GetNumInstances(), bInstanceDataIsGPUOnly);
#endif
	// TODO: These don't follow the common pattern.
	// ValidateArray(Flags.bHasPerInstanceLocalBounds, InstanceLocalBounds, GetNumInstances(), bInstanceDataIsGPUOnly);
	// ValidateArray(Flags.bHasPerInstancePayloadExtension, InstancePayloadExtension, GetNumInstances(), bInstanceDataIsGPUOnly);
}

void FInstanceSceneDataBuffers::SetImmutable(FInstanceSceneDataImmutable &&ImmutableData, FAccessTag AccessTag)
{
	ValidateAccess(AccessTag);
	Flags.bHasCompressedSpatialHash = !ImmutableData.GetCompressedInstanceSpatialHashes().IsEmpty();
	Immutable = MakeShared<FInstanceSceneDataImmutable>(MoveTemp(ImmutableData));
}

FSingleInstanceDataBuffers::FSingleInstanceDataBuffers()
{
	PrimitiveToRelativeWorld.SetIdentity();
	InstanceLocalBounds.SetNumZeroed(1);
	InstanceToPrimitiveRelative.Emplace(PrimitiveToRelativeWorld);
}

void FSingleInstanceDataBuffers::UpdateDefaultInstance(const FMatrix &PrimitiveLocalToWorld, const FRenderBounds LocalBounds)
{
	SetPrimitiveLocalToWorld(PrimitiveLocalToWorld);
	InstanceToPrimitiveRelative.Reset(1);
	InstanceToPrimitiveRelative.Emplace(PrimitiveToRelativeWorld);
	InstanceLocalBounds[0] = LocalBounds;
}


void FInstanceDataUpdateTaskInfo::WaitForUpdateCompletion()
{
	UpdateTaskHandle.Wait();
}

FInstanceDataSceneProxy::FInstanceDataSceneProxy()
{
}

FInstanceDataSceneProxy::FInstanceDataSceneProxy(FInstanceSceneDataBuffers&& InInstanceSceneDataBuffers)
	: InstanceSceneDataBuffers(MoveTemp(InInstanceSceneDataBuffers))
{
}

FInstanceDataSceneProxy::~FInstanceDataSceneProxy() 
{
	DecStatCounters();
}

void FInstanceDataSceneProxy::IncStatCounters()
{
	FInstanceSceneDataBuffers::FReadView Buffer = InstanceSceneDataBuffers.GetReadView();
	// TODO: Should report much finer granularity than what this code is doing (i.e. dynamic vs static, per stream sizes, etc..)
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceToPrimitiveRelative.GetAllocatedSize());
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.PrevInstanceToPrimitiveRelative.GetAllocatedSize());
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceCustomData.GetAllocatedSize());
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceRandomIDs.GetAllocatedSize());
#if WITH_EDITOR
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceEditorData.GetAllocatedSize());
#endif
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceLightShadowUVBias.GetAllocatedSize());
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceLocalBounds.GetAllocatedSize());
	INC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceHierarchyOffset.GetAllocatedSize());

	int32 NumIntances = InstanceSceneDataBuffers.GetNumInstances();

	INC_DWORD_STAT_BY(STAT_InstanceDataInstanceCount, NumIntances);

	INC_DWORD_STAT_BY(STAT_InstanceHasDynamicCount, Buffer.Flags.bHasPerInstanceDynamicData ? NumIntances : 0);
	INC_DWORD_STAT_BY(STAT_InstanceHasSkinningCount, Buffer.Flags.bHasPerInstanceSkinningData ? NumIntances : 0);
	INC_DWORD_STAT_BY(STAT_InstanceHasLMSMBiasCount, Buffer.Flags.bHasPerInstanceLMSMUVBias ? NumIntances : 0);
	INC_DWORD_STAT_BY(STAT_InstanceHasCustomDataCount, Buffer.Flags.bHasPerInstanceCustomData ? NumIntances : 0);
	INC_DWORD_STAT_BY(STAT_InstanceHasRandomCount, Buffer.Flags.bHasPerInstanceRandom ? NumIntances : 0);
	INC_DWORD_STAT_BY(STAT_InstanceHasLocalBounds, Buffer.Flags.bHasPerInstanceLocalBounds ? NumIntances : 0);
	INC_DWORD_STAT_BY(STAT_InstanceHasHierarchyOffset, Buffer.Flags.bHasPerInstanceHierarchyOffset ? NumIntances : 0);
}

void FInstanceDataSceneProxy::DecStatCounters()
{
	FInstanceSceneDataBuffers::FReadView Buffer = InstanceSceneDataBuffers.GetReadView();
	// TODO: Should report much finer granularity than what this code is doing (i.e. dynamic vs static, per stream sizes, etc..)
	// TODO: Also should be reporting this for all proxies, not just the Nanite ones
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceToPrimitiveRelative.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.PrevInstanceToPrimitiveRelative.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceCustomData.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceRandomIDs.GetAllocatedSize());
#if WITH_EDITOR
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceEditorData.GetAllocatedSize());
#endif
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceLightShadowUVBias.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceLocalBounds.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_ProxyInstanceMemory, Buffer.InstanceHierarchyOffset.GetAllocatedSize());

	int32 NumInstances = InstanceSceneDataBuffers.GetNumInstances();
	DEC_DWORD_STAT_BY(STAT_InstanceDataInstanceCount, NumInstances);

	DEC_DWORD_STAT_BY(STAT_InstanceHasDynamicCount, Buffer.Flags.bHasPerInstanceDynamicData ? NumInstances : 0);
	DEC_DWORD_STAT_BY(STAT_InstanceHasSkinningCount, Buffer.Flags.bHasPerInstanceSkinningData ? NumInstances : 0);
	DEC_DWORD_STAT_BY(STAT_InstanceHasLMSMBiasCount, Buffer.Flags.bHasPerInstanceLMSMUVBias ? NumInstances : 0);
	DEC_DWORD_STAT_BY(STAT_InstanceHasCustomDataCount, Buffer.Flags.bHasPerInstanceCustomData ? NumInstances : 0);
	DEC_DWORD_STAT_BY(STAT_InstanceHasRandomCount, Buffer.Flags.bHasPerInstanceRandom ? NumInstances : 0);
	DEC_DWORD_STAT_BY(STAT_InstanceHasLocalBounds, Buffer.Flags.bHasPerInstanceLocalBounds ? NumInstances : 0);
	DEC_DWORD_STAT_BY(STAT_InstanceHasHierarchyOffset, Buffer.Flags.bHasPerInstanceHierarchyOffset ? NumInstances : 0);
}
