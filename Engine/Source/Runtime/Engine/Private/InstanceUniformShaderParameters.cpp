// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceUniformShaderParameters.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneDefinitions.h"


bool FInstanceSceneShaderData::SupportsCompressedTransforms()
{
	return FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(GMaxRHIShaderPlatform);
}

uint32 FInstanceSceneShaderData::GetDataStrideInFloat4s()
{
	if (FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(GMaxRHIShaderPlatform))
	{
		return CompressedTransformDataStrideInFloat4s;
	}
	else
	{
		return UnCompressedTransformDataStrideInFloat4s;
	}
}

void FInstanceSceneShaderData::Build
(
	uint32 PrimitiveId,
	uint32 RelativeId,
	uint32 InstanceFlags,
	uint32 LastUpdateFrame,
	uint32 CustomDataCount,
	float RandomID,
	bool bIsVisible
)
{
	BuildInternal(PrimitiveId, RelativeId, InstanceFlags, LastUpdateFrame, CustomDataCount, RandomID, FRenderTransform::Identity, bIsVisible, SupportsCompressedTransforms());
}

void FInstanceSceneShaderData::Build
(
	uint32 PrimitiveId,
	uint32 RelativeId,
	uint32 InstanceFlags,
	uint32 LastUpdateFrame,
	uint32 CustomDataCount,
	float RandomID,
	const FRenderTransform& LocalToPrimitive,
	const FRenderTransform& PrimitiveToWorld,
	bool bIsVisible
)
{
	FRenderTransform LocalToWorld = LocalToPrimitive * PrimitiveToWorld;

	// Remove shear
	LocalToWorld.Orthogonalize();

	BuildInternal(PrimitiveId, RelativeId, InstanceFlags, LastUpdateFrame, CustomDataCount, RandomID, LocalToWorld, bIsVisible, SupportsCompressedTransforms());
}

