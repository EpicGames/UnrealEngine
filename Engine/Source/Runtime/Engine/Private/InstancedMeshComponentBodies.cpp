// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedMeshComponentBodies.h"
#include "PhysicsEngine/BodySetup.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "CollisionQueryParams.h"

FInstancedMeshComponentBodies::FInstancedMeshComponentBodies(UPrimitiveComponent* InOwner)
	: Owner(InOwner)
{
	check(Owner);
}

FInstancedMeshComponentBodies::~FInstancedMeshComponentBodies()
{
	if (!ensure(!IsAsyncDestroying()))
	{
		AsyncDestroy();
	}
	ClearAll();
}

void FInstancedMeshComponentBodies::InitInstanceBody(FBodyInstance* Body, int32 Idx, const FTransform& Transform, bool bRuntime)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InitInstanceBody(Body, &Owner->BodyInstance, Idx, Transform, bRuntime, Owner->GetBodySetup());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FInstancedMeshComponentBodies::InitInstanceBody(
	FBodyInstance* Body, FBodyInstance* ReferenceBody, int32 Idx, const FTransform& Transform,
	bool bRuntime, UBodySetup* BodySetup)
{
	check(Body);
	check(ReferenceBody);
	check(BodySetup);

	if (bRuntime)
	{
		Body->CopyRuntimeBodyInstancePropertiesFrom(ReferenceBody);
	}
	else
	{
		Body->CopyBodyInstancePropertiesFrom(ReferenceBody);
	}

	Body->InstanceBodyIndex = Idx;
	Body->bSimulatePhysics = false;
	Body->bAutoWeld = false;
	Body->InitBody(BodySetup, Transform, Owner, Owner->GetWorld()->GetPhysicsScene(), nullptr);
}

void FInstancedMeshComponentBodies::CreateAll(TConstArrayView<FTransform> WorldTransforms, bool bSkipWalkableSlopeOverrideFromBodySetup)
{
	check(Bodies.Num() == 0);

	const int32 NumInstances = WorldTransforms.Num();

	UBodySetup* BodySetup = Owner->GetBodySetup();
	if (!BodySetup)
	{
		Bodies.AddZeroed(NumInstances);
		return;
	}

	FBodyInstance& ReferenceBody = Owner->BodyInstance;
	FPhysScene* PhysScene = Owner->GetWorld()->GetPhysicsScene();
	const EComponentMobility::Type Mobility = Owner->Mobility;

	// When bSkipWalkableSlopeOverrideFromBodySetup is set, the caller has already written
	// ReferenceBody.WalkableSlopeOverride on GT in OnAsyncCreatePhysicsStateBegin_GameThread.
	// Skipping here avoids re-reading BodySetup from the worker (UActorComponent::OnAsyncCreatePhysicsState
	// falls through to OnCreatePhysicsState in editor PIE; designer mid-edits would tear the struct)
	// and avoids clobbering the GT-captured value. In cooked, BodySetup is immutable post-PostLoad and
	// the worker read is uncontested -- bSkipWalkableSlopeOverrideFromBodySetup stays false.
	if (!bSkipWalkableSlopeOverrideFromBodySetup && !ReferenceBody.GetOverrideWalkableSlopeOnInstance())
	{
		ReferenceBody.SetWalkableSlopeOverride(BodySetup->WalkableSlopeOverride, false);
	}

	Bodies.SetNumUninitialized(NumInstances);

	TArray<FBodyInstance*> SanitizedBodies;
	SanitizedBodies.Reserve(NumInstances);

	TArray<FTransform> StaticTransforms;
	StaticTransforms.Reserve(NumInstances);

	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
	{
		const FTransform& InstanceTM = WorldTransforms[InstanceIndex];
		if (InstanceTM.GetScale3D().IsNearlyZero())
		{
			Bodies[InstanceIndex] = nullptr;
		}
		else
		{
			FBodyInstance* Instance = new FBodyInstance;

			SanitizedBodies.Add(Instance);
			Bodies[InstanceIndex] = Instance;
			Instance->CopyBodyInstancePropertiesFrom(&ReferenceBody);
			Instance->InstanceBodyIndex = InstanceIndex;
			Instance->bAutoWeld = false;
			Instance->bSimulatePhysics = false;

			if (Mobility == EComponentMobility::Movable)
			{
				Instance->InitBody(BodySetup, InstanceTM, Owner, PhysScene, nullptr);
			}
			else
			{
				StaticTransforms.Add(InstanceTM);
			}
		}
	}

	if (SanitizedBodies.Num() > 0 && Mobility != EComponentMobility::Movable)
	{
		FBodyInstance::InitStaticBodies(MoveTemp(SanitizedBodies), MoveTemp(StaticTransforms), BodySetup, Owner, PhysScene);
	}
}

void FInstancedMeshComponentBodies::ClearAll()
{
	for (FBodyInstance*& Instance : Bodies)
	{
		if (Instance)
		{
			Instance->TermBody();
			delete Instance;
		}
	}

	Bodies.Empty();
}

void FInstancedMeshComponentBodies::UpdateTransform(
	int32 Idx, const FTransform& WorldTransform, bool bTeleport)
{
	if (!Bodies.IsValidIndex(Idx))
	{
		return;
	}

	FBodyInstance*& Body = Bodies[Idx];
	UBodySetup* BodySetup = Owner->GetBodySetup();

	if (BodySetup == nullptr || WorldTransform.GetScale3D().IsNearlyZero())
	{
		if (Body)
		{
			Body->TermBody();
			delete Body;
			Body = nullptr;
		}
	}
	else
	{
		if (Body)
		{
			Body->SetBodyTransform(WorldTransform, TeleportFlagToEnum(bTeleport));
			Body->UpdateBodyScale(WorldTransform.GetScale3D());
		}
		else
		{			
			Body = new FBodyInstance();
			InitInstanceBody(Body, Idx, WorldTransform, false);			
		}
	}
}

void FInstancedMeshComponentBodies::RemoveAt(int32 Idx)
{
	if (!Bodies.IsValidIndex(Idx))
	{
		return;
	}

	if (FBodyInstance* Body = Bodies[Idx])
	{
		Body->TermBody();
		delete Body;
	}
	
	Bodies.RemoveAt(Idx);

	for (int32 Index = Idx; Index < Bodies.Num(); ++Index)
	{
		if (Bodies[Index])
		{
			Bodies[Index]->InstanceBodyIndex = Index;
		}
	}
}

void FInstancedMeshComponentBodies::RemoveAtSwap(int32 Idx)
{
	if (!Bodies.IsValidIndex(Idx))
	{
		return;
	}

	if (FBodyInstance* Body = Bodies[Idx])
	{
		Body->TermBody();
		delete Body;
	}

	Bodies.RemoveAtSwap(Idx);

	if (Bodies.IsValidIndex(Idx) && Bodies[Idx])
	{
		Bodies[Idx]->InstanceBodyIndex = Idx;
	}	
}

void FInstancedMeshComponentBodies::Insert(
	int32 Idx, const FTransform& WorldTransform)
{
	UBodySetup* BodySetup = Owner->GetBodySetup();
	if (BodySetup == nullptr || WorldTransform.GetScale3D().IsNearlyZero())
	{
		Bodies.Insert(nullptr, Idx);
	}
	else
	{
		FBodyInstance* NewBody = new FBodyInstance();
		int32 BodyIndex = Bodies.Insert(NewBody, Idx);
		check(Idx == BodyIndex);
		InitInstanceBody(NewBody, BodyIndex, WorldTransform, false);
	}

	// Fix up InstanceBodyIndex for all bodies shifted by the insertion
	for (int32 Index = Idx + 1; Index < Bodies.Num(); ++Index)
	{
		if (Bodies[Index])
		{
			Bodies[Index]->InstanceBodyIndex = Index;
		}
	}
}

void FInstancedMeshComponentBodies::Recreate(int32 Idx, const FTransform& InstanceTransform)
{
	if (!Bodies.IsValidIndex(Idx))
	{
		return;
	}

	FBodyInstance* OldBody = Bodies[Idx];
	if (!OldBody)
	{
		return;
	}

	FBodyInstance* NewBody = nullptr;
	if (UBodySetup* BodySetup = OldBody->GetBodySetup())
	{
		NewBody = new FBodyInstance;
		NewBody->CopyRuntimeBodyInstancePropertiesFrom(OldBody);
		NewBody->InstanceBodyIndex = Idx;
		NewBody->bSimulatePhysics = false;
		NewBody->bAutoWeld = false;
		NewBody->InitBody(BodySetup, InstanceTransform, Owner, Owner->GetWorld()->GetPhysicsScene(), nullptr);
	}
	Bodies[Idx] = NewBody;

	OldBody->TermBody();
	delete OldBody;
}

FBodyInstance* FInstancedMeshComponentBodies::GetBodyInstance(int32 Index) const
{
	if (Index != INDEX_NONE && Bodies.IsValidIndex(Index))
	{
		return Bodies[Index];
	}
	return &Owner->BodyInstance;
}

Chaos::FPhysicsObject* FInstancedMeshComponentBodies::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!Bodies.IsValidIndex(Id) || !Bodies[Id] || !Bodies[Id]->GetPhysicsActor())
	{
		return nullptr;
	}
	return Bodies[Id]->GetPhysicsActor()->GetPhysicsObject();
}

TArray<Chaos::FPhysicsObject*> FInstancedMeshComponentBodies::GetAllPhysicsObjects() const
{
	TArray<Chaos::FPhysicsObject*> Objects;
	Objects.Reserve(Bodies.Num());
	for (int32 Index = 0; Index < Bodies.Num(); ++Index)
	{
		Objects.Add(GetPhysicsObjectById(Index));
	}
	return Objects;
}

void FInstancedMeshComponentBodies::BeginAsyncDestroy()
{
	check(AsyncDestroyPayload.IsEmpty());
	AsyncDestroyPayload = MoveTemp(Bodies);
}

void FInstancedMeshComponentBodies::AsyncDestroy()
{
	check(Bodies.IsEmpty());
	for (FBodyInstance*& Instance : AsyncDestroyPayload)
	{
		if (Instance)
		{
			Instance->TermBody();
			delete Instance;
		}
	}
	AsyncDestroyPayload.Empty();
}

bool FInstancedMeshComponentBodies::IsAsyncDestroying() const
{
	return !AsyncDestroyPayload.IsEmpty();
}

// ---- Collision queries ----

bool FInstancedMeshComponentBodies::LineTrace(
	FHitResult& OutHit, const FVector& Start, const FVector& End,
	bool bTraceComplex, bool bReturnPhysicalMaterial) const
{
	bool bHaveHit = false;
	float MinTime = UE_MAX_FLT;
	FHitResult Hit;

	for (FBodyInstance* Body : Bodies)
	{
		if (!Body)
		{
			continue;
		}

		if (Body->LineTrace(Hit, Start, End, bTraceComplex, bReturnPhysicalMaterial))
		{
			bHaveHit = true;
			if (MinTime > Hit.Time)
			{
				MinTime = Hit.Time;
				OutHit = Hit;
			}
		}
	}

	return bHaveHit;
}

bool FInstancedMeshComponentBodies::Sweep(
	FHitResult& OutHit, const FVector& Start, const FVector& End,
	const FQuat& Rot, const FCollisionShape& Shape, bool bTraceComplex) const
{
	bool bHaveHit = false;
	FHitResult Hit;

	for (FBodyInstance* Body : Bodies)
	{
		if (!Body)
		{
			continue;
		}

		if (Body->Sweep(Hit, Start, End, Rot, Shape, bTraceComplex))
		{
			if (!bHaveHit || Hit.Time < OutHit.Time)
			{
				OutHit = Hit;
			}
			bHaveHit = true;
		}
	}

	return bHaveHit;
}

bool FInstancedMeshComponentBodies::Overlap(
	const FVector& Pos, const FQuat& Rot, const FCollisionShape& Shape) const
{
	for (FBodyInstance* Body : Bodies)
	{
		if (!Body)
		{
			continue;
		}

		if (Body->OverlapTest(Pos, Rot, Shape))
		{
			return true;
		}
	}

	return false;
}

bool FInstancedMeshComponentBodies::OverlapComponent(
	FBodyInstance* OtherBody, const FVector& Pos, const FQuat& Quat) const
{
	if (!OtherBody)
	{
		return false;
	}

	return OtherBody->OverlapTestForBodies(Pos, Quat, Bodies);
}

bool FInstancedMeshComponentBodies::OverlapMulti(
	TArray<FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot,
	ECollisionChannel Channel, const FComponentQueryParams& Params,
	const FCollisionObjectQueryParams& ObjectParams) const
{
	const FTransform WorldToComponent(Owner->GetComponentTransform().Inverse());
	const FCollisionResponseParams ResponseParams(Owner->GetCollisionResponseToChannels());

	FComponentQueryParams ParamsWithSelf = Params;
	ParamsWithSelf.AddIgnoredComponent(Owner);

	bool bHaveBlockingHit = false;
	for (FBodyInstance* Body : Bodies)
	{
		if (!Body)
		{
			continue;
		}

		if (Body->OverlapMulti(OutOverlaps, Owner->GetWorld(), &WorldToComponent, Pos, Rot, Channel, ParamsWithSelf, ResponseParams, ObjectParams))
		{
			bHaveBlockingHit = true;
		}
	}

	return bHaveBlockingHit;
}
