// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Chaos/PhysicsObject.h"
#include "PhysicsInterfaceDeclaresCore.h"

class UPrimitiveComponent;
class UBodySetup;
struct FCollisionShape;
struct FCollisionQueryParams;
struct FComponentQueryParams;
struct FCollisionObjectQueryParams;
struct FCollisionResponseParams;
struct FHitResult;
struct FOverlapResult;

/**
 * Manages per-instance physics body instances for instanced mesh components.
 * Owns the bodies array and async destroy payload. Provides collision query helpers.
 * Derives BodyInstance, PhysScene, Mobility, World, and collision response from the owning component.
 */
class FInstancedMeshComponentBodies
{
public:
	explicit FInstancedMeshComponentBodies(UPrimitiveComponent* InOwner);
	~FInstancedMeshComponentBodies();

	/**
	 * Initialize a single body: copies properties from reference, sets index, calls InitBody.
	 * NOTE: Do not call directly. TODO: Move implementation to private overload once external
	 * refs have been deprecated.
	 */
	UE_DEPRECATED(5.8, "This method will be removed in a future release.")
	void InitInstanceBody(FBodyInstance* Body, FBodyInstance* ReferenceBody, int32 Idx,
		const FTransform& Transform, bool bRuntime, UBodySetup* BodySetup);

	/** Create all instance bodies from world-space transforms.
	 *  @param bSkipWalkableSlopeOverrideFromBodySetup  If true, do not read BodySetup->WalkableSlopeOverride
	 *      (caller has already pre-cached it on GT to avoid an editor-PIE race with a designer mid-edit). */
	void CreateAll(TConstArrayView<FTransform> WorldTransforms, bool bSkipWalkableSlopeOverrideFromBodySetup = false);

	/** Terminate and delete all instance bodies. */
	void ClearAll();

	/** Update a single body's transform; handles zero-scale deletion and lazy creation. */
	void UpdateTransform(int32 Idx, const FTransform& WorldTransform, bool bTeleport);

	/** Remove a body at the given index. */
	void RemoveAt(int32 Idx);
	
	/** Remove a body at the given index, move the last body into the empty slot. */
	void RemoveAtSwap(int32 Idx);

	/** Insert a new body at the given index. */
	void Insert(int32 Idx, const FTransform& WorldTransform);

	/** Recreate a body at the given index, copying runtime properties from the existing body. */
	void Recreate(int32 Idx, const FTransform& InstanceTransform);

	/** Get body by index, or owner's default BodyInstance if Index is INDEX_NONE. */
	FBodyInstance* GetBodyInstance(int32 Index) const;

	Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const;
	TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const;

	bool LineTrace(FHitResult& OutHit, const FVector& Start, const FVector& End,
		bool bTraceComplex, bool bReturnPhysicalMaterial) const;

	bool Sweep(FHitResult& OutHit, const FVector& Start, const FVector& End,
		const FQuat& Rot, const FCollisionShape& Shape, bool bTraceComplex) const;

	bool Overlap(const FVector& Pos, const FQuat& Rot, const FCollisionShape& Shape) const;

	bool OverlapComponent(FBodyInstance* OtherBody, const FVector& Pos, const FQuat& Quat) const;

	/** Multi-overlap query. Derives World, WorldToComponent, and ResponseParams from owner. */
	bool OverlapMulti(TArray<FOverlapResult>& OutOverlaps, const FVector& Pos, const FQuat& Rot,
		ECollisionChannel Channel, const FComponentQueryParams& Params,
		const FCollisionObjectQueryParams& ObjectParams) const;

	/** Move bodies to async payload for off-thread destruction. */
	void BeginAsyncDestroy();

	/** Destroy bodies in the async payload. */
	void AsyncDestroy();

	/** Returns true if an async destroy is in progress. */
	bool IsAsyncDestroying() const;

	const TArray<FBodyInstance*>& GetBodies() const { return Bodies; }
	int32 Num() const { return Bodies.Num(); }
	bool IsValidIndex(int32 Idx) const { return Bodies.IsValidIndex(Idx); }
	bool IsEmpty() const { return Bodies.IsEmpty(); }
	FBodyInstance*& operator[](int32 Idx) { return Bodies[Idx]; }
	FBodyInstance* operator[](int32 Idx) const { return Bodies[Idx]; }

	// Range-for iteration
	auto begin()       { return Bodies.begin(); }
	auto end()         { return Bodies.end(); }
	auto begin() const { return Bodies.begin(); }
	auto end()   const { return Bodies.end(); }

private:
	/** Initialize a single body: copies properties from owner's BodyInstance, sets index, calls InitBody. */
	void InitInstanceBody(FBodyInstance* Body, int32 Idx, const FTransform& Transform, bool bRuntime);

	UPrimitiveComponent* Owner;
	TArray<FBodyInstance*> Bodies;
	TArray<FBodyInstance*> AsyncDestroyPayload;
};
