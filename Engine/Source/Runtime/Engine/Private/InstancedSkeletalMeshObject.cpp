// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedSkeletalMeshObject.h"
#include "SkeletalRenderGPUSkin.h"
#include "SceneInterface.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/RenderCommandPipes.h"

extern void InitGPUSkinVertexFactoryComponents(FGPUSkinDataType* VertexFactoryData, const FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers& VertexBuffers, FGPUBaseSkinVertexFactory* VertexFactory);

FInstancedSkeletalMeshObject::FInstancedSkeletalMeshObject(
	const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc,
	FSkeletalMeshRenderData* InRenderData,
	ERHIFeatureLevel::Type InFeatureLevel,
	EType InType)
	: FSkeletalMeshObject(InMeshDesc, InRenderData, InFeatureLevel)
	, TransformProvider(InMeshDesc.TransformProvider)
	, Type(InType)
{
	const int32 NumLODs = InRenderData->LODRenderData.Num();
	const bool bIsGPUSkin = Type == EType::GPUSkin;

	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		LODs.Emplace(InRenderData, LODIndex, InFeatureLevel);
	}

	bUseGpuLodSelection = NumLODs > 1 && InMeshDesc.bUseGpuLodSelection && ((bIsGPUSkin && InRenderData->HasUnifiedBoneMap()) || Type == EType::Static);
	bForceAnimateSockets = InMeshDesc.bForceAnimateSockets;
	BoneTransformStorageMode = (bIsGPUSkin || InRenderData->HasUnifiedBoneMap()) ? EBoneTransformStorageMode::BoneMap : EBoneTransformStorageMode::Direct;

	InitResources(InMeshDesc);

	bSupportsStaticRelevance = true;

	if (Type == EType::Nanite)
	{
		NaniteMaterials = MakeUnique<Nanite::FMaterialAudit>();
		AuditMaterials(&InMeshDesc, *NaniteMaterials, true /* Set material usage flags */);
	}
}

FInstancedSkeletalMeshObject::~FInstancedSkeletalMeshObject()
{
	delete DynamicData;
	DynamicData = nullptr;
}

void FInstancedSkeletalMeshObject::InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc)
{
	checkf(InMeshDesc.Scene && InMeshDesc.Scene->GetSkeletalMeshUpdater(), TEXT("Scene cannot be null when creating a SkeletalMeshObject."));
	UpdateHandle = InMeshDesc.Scene->GetSkeletalMeshUpdater()->Create(this);

	for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
	{
		FLOD& LOD = LODs[LODIndex];

		if (LOD.RenderData->LODRenderData[LODIndex].GetNumVertices() > 0)
		{
			const FSkelMeshComponentLODInfo* InitLODInfo = nullptr;

			if (InMeshDesc.LODInfo.IsValidIndex(LODIndex))
			{
				InitLODInfo = &InMeshDesc.LODInfo[LODIndex];
			}

			LOD.InitResources(InitLODInfo, FeatureLevel, Type, BoneTransformStorageMode);
		}
	}
}

void FInstancedSkeletalMeshObject::ReleaseResources()
{
	UpdateHandle.Release();

	for (FLOD& LOD : LODs)
	{
		LOD.ReleaseResources();
	}
}

void FInstancedSkeletalMeshObject::Update(
	int32 LODIndex,
	const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
	const FPrimitiveSceneProxy* InSceneProxy,
	const USkinnedAsset* InSkinnedAsset,
	const FMorphTargetWeightMap& InActiveMorphTargets,
	const TArray<float>& MorphTargetWeights,
	EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode,
	const FExternalMorphWeightData& InExternalMorphWeightData)
{
	FDynamicData* NewDynamicData = new FDynamicData;
	NewDynamicData->LODIndex = LODIndex;
	UpdateHandle.Update(NewDynamicData);
}

FSkinningSceneExtensionProxy* FInstancedSkeletalMeshObject::CreateSceneExtensionProxy(const USkinnedAsset* InSkinnedAsset, bool bAllowScaling) const
{
	if (Type == EType::Nanite || (Type == EType::GPUSkin && IsGPUSkinSceneExtensionEnabled()))
	{
		return new FInstancedSkinningSceneExtensionProxy(TransformProvider, this, InSkinnedAsset, bAllowScaling, bForceAnimateSockets);
	}
	// EType::Static does not register with the skinning extension (uses FLocalVertexFactory)
	return nullptr;
}

void FInstancedSkeletalMeshObject::UpdateSceneExtensionHeader(FRHICommandListBase& RHICmdList, const FSkinningHeader& SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV) const
{
	if (Type == EType::GPUSkin)
	{
		check(DynamicData);

		const auto UpdateVertexFactory = [&] (int32 LODIndex)
		{
			for (const TUniquePtr<FGPUBaseSkinVertexFactory>& VertexFactory : LODs[LODIndex].VertexFactories)
			{
				if (VertexFactory)
				{
					VertexFactory->UpdateUniformBuffer(RHICmdList, &SkinningHeader, TransformDataBufferSRV);
				}
			}
		};

		if (bUseGpuLodSelection)
		{
			for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
			{
				UpdateVertexFactory(LODIndex);
			}
		}
		else
		{
			UpdateVertexFactory(DynamicData->LODIndex);
		}
	}
}

void FInstancedSkeletalMeshObject::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(LODs.GetAllocatedSize());

	for (FLOD& LOD : LODs)
	{
		LOD.GetResourceSizeEx(CumulativeResourceSize);
	}
}

const FVertexFactory* FInstancedSkeletalMeshObject::GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	const FLOD& LOD = LODs[LODIndex];

	if (Type == EType::Static)
	{
		return &LOD.LocalVertexFactory;
	}

#if RHI_RAYTRACING
	if (VFMode == ESkinVertexFactoryMode::RayTracing)
	{
		return &LOD.LocalVertexFactory;
	}
#endif

	if (LOD.VertexFactories.IsValidIndex(ChunkIdx))
	{
		return LOD.VertexFactories[ChunkIdx].Get();
	}

	return nullptr;
}

const FVertexFactory* FInstancedSkeletalMeshObject::GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const
{
	return GetSkinVertexFactory(nullptr, LODIndex, ChunkIdx, VFMode);
}

#if RHI_RAYTRACING
const FRayTracingGeometry* FInstancedSkeletalMeshObject::GetStaticRayTracingGeometry() const
{
	const int32 RayTracingLODIndex = GetRayTracingLOD();
	return &LODs[RayTracingLODIndex].RenderData->LODRenderData[RayTracingLODIndex].StaticRayTracingGeometry;
}
#endif

FInstancedSkeletalMeshObject::FLOD::FLOD(FSkeletalMeshRenderData* InRenderData, int32 InLOD, ERHIFeatureLevel::Type InFeatureLevel)
	: RenderData(InRenderData)
	, LocalVertexFactory(InFeatureLevel, "FInstancedSkeletalMeshObjectLOD")
	, LODIndex(InLOD)
{}

void FInstancedSkeletalMeshObject::FLOD::InitResources(const FSkelMeshComponentLODInfo* InLODInfo, ERHIFeatureLevel::Type InFeatureLevel, EType InType, EBoneTransformStorageMode InBoneTransformStorageMode)
{
	check(RenderData);
	check(RenderData->LODRenderData.IsValidIndex(LODIndex));

	FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];

	FSkeletalMeshObjectGPUSkin::FVertexFactoryBuffers VertexBuffers;
	VertexBuffers.StaticVertexBuffers = &LODData.StaticVertexBuffers;
	VertexBuffers.ColorVertexBuffer = FSkeletalMeshObject::GetColorVertexBuffer(LODData, InLODInfo);
	VertexBuffers.SkinWeightVertexBuffer = FSkeletalMeshObject::GetSkinWeightVertexBuffer(LODData, InLODInfo);
	VertexBuffers.NumVertices = LODData.GetNumVertices();

#if RHI_RAYTRACING
	const bool bRayTracingEnabled = IsRayTracingEnabled() && RenderData->bSupportRayTracing;
	if (bRayTracingEnabled)
	{
		RenderData->InitStaticRayTracingGeometry(LODIndex);
		bStaticRayTracingGeometryInitialized = true;
	}
#else
	const bool bRayTracingEnabled = false;
#endif

	ENQUEUE_RENDER_COMMAND(FInstancedSkeletalMeshObjectLOD_InitResources)(UE::RenderCommandPipe::SkeletalMesh,
		[this, &LODData, VertexBuffers, bRayTracingEnabled, InFeatureLevel, InType](FRHICommandList& RHICmdList)
	{
		if (InType == EType::GPUSkin)
		{
			TConstArrayView<FSkelMeshRenderSection> Sections = LODData.RenderSections;
			VertexFactories.Empty(Sections.Num());
			uint32 BoneOffset = 0;

			const bool bHasUnifiedBoneMap = RenderData->HasUnifiedBoneMap();

			for (const FSkelMeshRenderSection& Section : Sections)
			{
				FGPUBaseSkinVertexFactory* VertexFactory = nullptr;

				if (Section.IsValid())
				{
					const uint32 NumBones = Section.BoneMap.Num();

					const FGPUBaseSkinVertexFactory::FInitializer Initializer
					{
						  .FeatureLevel    = InFeatureLevel
						, .NumBones        = NumBones
						, .BoneOffset      = BoneOffset
						, .NumVertices     = VertexBuffers.NumVertices
						, .BaseVertexIndex = Section.BaseVertexIndex
					};

					if (VertexBuffers.SkinWeightVertexBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::DefaultBoneInfluence)
					{
						VertexFactory = new TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::DefaultBoneInfluence>(Initializer);
					}
					else
					{
						VertexFactory = new TGPUSkinVertexFactory<GPUSkinBoneInfluenceType::UnlimitedBoneInfluence>(Initializer);
					}

					VertexFactory->GetShaderData().UpdatedFrameNumber = ~0u;

					FGPUSkinDataType Data;
					InitGPUSkinVertexFactoryComponents(&Data, VertexBuffers, VertexFactory);
					VertexFactory->SetData(RHICmdList, &Data);
					VertexFactory->InitResource(RHICmdList);

					BoneOffset += bHasUnifiedBoneMap ? 0 : Section.BoneMap.Num();
				}

				VertexFactories.Emplace(VertexFactory);
			}
		}

		// Initialize LocalVertexFactory for Static mode (all rendering) and for ray tracing in other modes
		if (InType == EType::Static
#if RHI_RAYTRACING
			|| bRayTracingEnabled
#endif
		)
		{
			FPositionVertexBuffer* PositionVertexBufferPtr = &LODData.StaticVertexBuffers.PositionVertexBuffer;
			FStaticMeshVertexBuffer* StaticMeshVertexBufferPtr = &LODData.StaticVertexBuffers.StaticMeshVertexBuffer;

			FLocalVertexFactory::FDataType Data;
			PositionVertexBufferPtr->BindPositionVertexBuffer(&LocalVertexFactory, Data);
			StaticMeshVertexBufferPtr->BindTangentVertexBuffer(&LocalVertexFactory, Data);
			StaticMeshVertexBufferPtr->BindPackedTexCoordVertexBuffer(&LocalVertexFactory, Data);
			StaticMeshVertexBufferPtr->BindLightMapVertexBuffer(&LocalVertexFactory, Data, 0);

			LocalVertexFactory.SetData(RHICmdList, Data);
			LocalVertexFactory.InitResource(RHICmdList);
		}
	});
}

void FInstancedSkeletalMeshObject::FLOD::ReleaseResources()
{
	check(RenderData);

	ENQUEUE_RENDER_COMMAND(FInstancedSkeletalMeshObjectLOD_ReleaseResources)(UE::RenderCommandPipe::SkeletalMesh,
		[this](FRHICommandList& RHICmdList)
	{
		for (auto& VertexFactory : VertexFactories)
		{
			if (VertexFactory)
			{
				VertexFactory->ReleaseResource();
			}
		}

		LocalVertexFactory.ReleaseResource();
	});

#if RHI_RAYTRACING
	if (bStaticRayTracingGeometryInitialized)
	{
		RenderData->ReleaseStaticRayTracingGeometry(LODIndex);
	}
#endif
}

//////////////////////////////////////////////////////////////////////////

class FInstancedSkeletalMeshUpdatePacket : public TSkeletalMeshUpdatePacket<FInstancedSkeletalMeshObject, FInstancedSkeletalMeshObject::FDynamicData>
{
public:
	void Add(FInstancedSkeletalMeshObject* MeshObject, FInstancedSkeletalMeshObject::FDynamicData* MeshDynamicData)
	{
		FInstancedSkeletalMeshObject::FDynamicData::Release(MeshObject->DynamicData);
		MeshObject->DynamicData = MeshDynamicData;
	}

	void Free(FInstancedSkeletalMeshObject::FDynamicData* MeshDynamicData)
	{
		FInstancedSkeletalMeshObject::FDynamicData::Release(MeshDynamicData);
	}
};

REGISTER_SKELETAL_MESH_UPDATE_BACKEND(FInstancedSkeletalMeshUpdatePacket);
