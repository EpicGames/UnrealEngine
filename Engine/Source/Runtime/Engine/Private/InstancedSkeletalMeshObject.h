// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalRenderPublic.h"
#include "SkeletalMeshUpdater.h"
#include "GPUSkinVertexFactory.h"
#include "Animation/AnimBank.h"
#include "InstancedSkinnedMeshSceneProxy.h"

struct FSkinningHeader;

class FInstancedSkeletalMeshObject : public FSkeletalMeshObject
{
public:
	enum class EType : uint8
	{
		Nanite,
		GPUSkin,
		Static
	};

	FInstancedSkeletalMeshObject(
		const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc,
		FSkeletalMeshRenderData* InRenderData,
		ERHIFeatureLevel::Type InFeatureLevel,
		EType Type);

	~FInstancedSkeletalMeshObject();

	virtual void InitResources(const FSkinnedMeshSceneProxyDesc& InMeshDesc) override;
	virtual void ReleaseResources();
	virtual TArray<FTransform>* GetComponentSpaceTransforms() const override { return nullptr; }
	virtual TConstArrayView<FMatrix44f> GetReferenceToLocalMatrices() const override { return {}; }
	virtual TConstArrayView<FMatrix44f> GetPrevReferenceToLocalMatrices() const override { return {}; }
	virtual FSkinningSceneExtensionProxy* CreateSceneExtensionProxy(const USkinnedAsset* InSkinnedAsset, bool bAllowScaling) const override;
	virtual void UpdateSceneExtensionHeader(FRHICommandListBase& RHICmdList, const FSkinningHeader& SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV) const override;
	virtual void UpdateSkinWeightBuffer(const TArrayView<const FSkelMeshComponentLODInfo> InLODInfo) override {}

	virtual const FVertexFactory* GetSkinVertexFactory(const FSceneView* View, int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const override;
	virtual const FVertexFactory* GetStaticSkinVertexFactory(int32 LODIndex, int32 ChunkIdx, ESkinVertexFactoryMode VFMode) const override;

	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	virtual bool IsNaniteMesh()  const override { return Type == EType::Nanite; }
	virtual bool IsGPUSkinMesh() const override { return Type == EType::GPUSkin; }
	
#if RHI_RAYTRACING
	// TODO: Support skinning in ray tracing (currently representing with static geometry)
	virtual const FRayTracingGeometry* GetStaticRayTracingGeometry() const override;
#endif

	void Update(
		int32 LODIndex,
		const FSkinnedMeshSceneProxyDynamicData& InDynamicData,
		const FPrimitiveSceneProxy* InSceneProxy,
		const USkinnedAsset* InSkinnedAsset,
		const FMorphTargetWeightMap& InActiveMorphTargets,
		const TArray<float>& MorphTargetWeights,
		EPreviousBoneTransformUpdateMode PreviousBoneTransformUpdateMode,
		const FExternalMorphWeightData& InExternalMorphWeightData) override;

	virtual int32 GetLOD() const override
	{
		return DynamicData ? DynamicData->LODIndex : 0;
	}

	virtual bool HaveValidDynamicData() const override
	{
		return DynamicData != nullptr;
	}

	virtual const Nanite::FMaterialAudit* GetNaniteMaterials() const override { return NaniteMaterials.Get(); }

	EType GetType() const
	{
		return Type;
	}

protected:
	class FDynamicData : public TSkeletalMeshDynamicData<FInstancedSkeletalMeshObject, FDynamicData>
	{
	};

	struct FLOD
	{
		FSkeletalMeshRenderData* RenderData;
		FLocalVertexFactory	LocalVertexFactory;
		TArray<TUniquePtr<FGPUBaseSkinVertexFactory>> VertexFactories;
		int32 LODIndex;
		bool bStaticRayTracingGeometryInitialized = false;

		FLOD(FSkeletalMeshRenderData* InRenderData, int32 InLOD, ERHIFeatureLevel::Type InFeatureLevel);

		void InitResources(const FSkelMeshComponentLODInfo* InLODInfo, ERHIFeatureLevel::Type InFeatureLevel, EType Type, EBoneTransformStorageMode InBoneTransformStorageMode);
		void ReleaseResources();

		void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(VertexFactories.GetAllocatedSize());
		}
	};

	TObjectPtr<UTransformProviderData> TransformProvider;
	TArray<FLOD> LODs;
	TUniquePtr<Nanite::FMaterialAudit> NaniteMaterials;
	FSkeletalMeshUpdateHandle UpdateHandle;
	FDynamicData* DynamicData = nullptr;
	EType Type;
	bool bUseGpuLodSelection = false;
	bool bForceAnimateSockets = false;

	friend class FInstancedSkeletalMeshUpdatePacket;
};