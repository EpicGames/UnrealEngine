// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedSkinnedMeshSceneProxy.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Rendering/SkeletalMeshRenderData.h"

FInstancedSkinnedMeshData::FInstancedSkinnedMeshData(const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc)
{
	InstanceMinDrawDistance     = InMeshDesc.InstanceMinDrawDistance;
	InstanceStartCullDistance   = InMeshDesc.InstanceStartCullDistance;
	InstanceEndCullDistance     = InMeshDesc.InstanceEndCullDistance;
	InstanceDataSceneProxy      = InMeshDesc.InstanceDataSceneProxy;
	AnimationMinScreenSize      = InMeshDesc.AnimationMinScreenSize;
}

bool FInstancedSkinnedMeshData::GetInstanceDrawDistanceMinMax(FVector2f& OutCullRange) const
{
	if (InstanceEndCullDistance > 0)
	{
		OutCullRange = FVector2f(float(InstanceMinDrawDistance), float(InstanceEndCullDistance));
		return true;
	}
	else
	{
		OutCullRange = FVector2f(0.0f);
		return false;
	}
}

void FInstancedSkinnedMeshData::SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance)
{
	InstanceStartCullDistance = StartCullDistance;
	InstanceEndCullDistance = EndCullDistance;
}

FNaniteInstancedSkinnedMeshSceneProxy::FNaniteInstancedSkinnedMeshSceneProxy(const Nanite::FMaterialAudit& MaterialAudit, const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InRenderData)
	: Super(MaterialAudit, InMeshDesc, InRenderData, false /* bAllowScale */)
	, Data(InMeshDesc)
{
#if WITH_EDITOR
	const bool bSupportInstancePicking = HasPerInstanceHitProxies() && SMInstanceElementDataUtil::SMInstanceElementsEnabled();
	HitProxyMode = bSupportInstancePicking ? EHitProxyMode::PerInstance : EHitProxyMode::MaterialSection;

	if (HitProxyMode == EHitProxyMode::PerInstance)
	{
		if (InMeshDesc.SelectedInstances.Find(true) != INDEX_NONE)
		{
			bHasSelectedInstances = true;
			SetSelection_GameThread(true);
		}
	}
#endif

	bAlwaysHasVelocity = true;
	bInstancedSkinnedMesh = true;
	bDynamicRayTracingGeometry = false;

	SetupInstanceSceneDataBuffers(Data.InstanceDataSceneProxy->GeInstanceSceneDataBuffers());
}

FInstancedSkinnedMeshSceneProxy::FInstancedSkinnedMeshSceneProxy(const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InRenderData)
	: Super(InMeshDesc, InRenderData)
	, Data(InMeshDesc)
{
#if WITH_EDITOR
	if (InMeshDesc.SelectedInstances.Find(true) != INDEX_NONE)
	{
		bHasSelectedInstances = true;
		SetSelection_GameThread(true);
	}
#endif

	bAlwaysHasVelocity = true;
	bInstancedSkinnedMesh = true;
	bUseGpuLodSelection = NumLODs > 1 && InMeshDesc.bUseGpuLodSelection && (InMeshDesc.bRenderStatic || InRenderData->HasUnifiedBoneMap());

	SetupInstanceSceneDataBuffers(Data.InstanceDataSceneProxy->GeInstanceSceneDataBuffers());
}