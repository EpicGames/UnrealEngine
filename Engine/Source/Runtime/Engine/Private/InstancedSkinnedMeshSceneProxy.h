// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedSkinnedMeshSceneProxyDesc.h"
#include "InstanceDataSceneProxy.h"
#include "NaniteSceneProxy.h"
#include "SkeletalMeshSceneProxy.h"

struct FInstancedSkinnedMeshData
{
	FInstancedSkinnedMeshData(const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc);

	bool GetInstanceDrawDistanceMinMax(FVector2f& OutCullRange) const;

	void SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance);

	FInstanceDataUpdateTaskInfo *GetInstanceDataUpdateTaskInfo() const
	{
		return InstanceDataSceneProxy ? InstanceDataSceneProxy->GetUpdateTaskInfo() : nullptr;
	}

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> InstanceDataSceneProxy;
	float AnimationMinScreenSize = 0.0f;
	uint32 InstanceMinDrawDistance = 0;
	uint32 InstanceStartCullDistance = 0;
	uint32 InstanceEndCullDistance = 0;
};

class FNaniteInstancedSkinnedMeshSceneProxy : public Nanite::FSkinnedSceneProxy
{
public:
	using Super = Nanite::FSkinnedSceneProxy;

	FNaniteInstancedSkinnedMeshSceneProxy(const Nanite::FMaterialAudit& MaterialAudit, UInstancedSkinnedMeshComponent* InComponent, FSkeletalMeshRenderData* InRenderData)
		: FNaniteInstancedSkinnedMeshSceneProxy(MaterialAudit, FInstancedSkinnedMeshSceneProxyDesc(InComponent), InRenderData)
	{}

	FNaniteInstancedSkinnedMeshSceneProxy(const Nanite::FMaterialAudit& MaterialAudit, const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InRenderData);

#if RHI_RAYTRACING
	virtual ERayTracingPrimitiveFlags GetRayTracingPrimitiveFlags() override
	{
		static const auto RayTracingInstancedSkeletalMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.InstancedSkeletalMeshes"));

		if (RayTracingInstancedSkeletalMeshesCVar && RayTracingInstancedSkeletalMeshesCVar->GetValueOnRenderThread() <= 0)
		{
			return ERayTracingPrimitiveFlags::Exclude;
		}

		return Super::GetRayTracingPrimitiveFlags();
	}
#endif

	virtual bool GetInstanceDrawDistanceMinMax(FVector2f& OutCullRange) const override
	{
		return Data.GetInstanceDrawDistanceMinMax(OutCullRange);
	}

	virtual void SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance) override
	{
		Data.SetInstanceCullDistance_RenderThread(StartCullDistance, EndCullDistance);
	}

	virtual FInstanceDataUpdateTaskInfo *GetInstanceDataUpdateTaskInfo() const override
	{
		return Data.GetInstanceDataUpdateTaskInfo();
	}

	// FPrimitiveSceneProxy interface.
	virtual SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	virtual float GetAnimationMinScreenSize() const  override
	{
		return Data.AnimationMinScreenSize;
	}

private:
	FInstancedSkinnedMeshData Data;
};

class FInstancedSkinnedMeshSceneProxy : public FSkeletalMeshSceneProxy
{
public:
	using Super = FSkeletalMeshSceneProxy;
	
	FInstancedSkinnedMeshSceneProxy(const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InRenderData);

#if RHI_RAYTRACING
	virtual ERayTracingPrimitiveFlags GetRayTracingPrimitiveFlags() override
	{
		static const auto RayTracingInstancedSkeletalMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.InstancedSkeletalMeshes"));

		if (RayTracingInstancedSkeletalMeshesCVar && RayTracingInstancedSkeletalMeshesCVar->GetValueOnRenderThread() <= 0)
		{
			return ERayTracingPrimitiveFlags::Exclude;
		}

		return Super::GetRayTracingPrimitiveFlags();
	}

	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override
	{
	}
#endif

	virtual bool GetInstanceDrawDistanceMinMax(FVector2f& OutCullRange) const override
	{
		return Data.GetInstanceDrawDistanceMinMax(OutCullRange);
	}

	virtual void SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance) override
	{
		Data.SetInstanceCullDistance_RenderThread(StartCullDistance, EndCullDistance);
	}

	virtual FInstanceDataUpdateTaskInfo *GetInstanceDataUpdateTaskInfo() const override
	{
		return Data.GetInstanceDataUpdateTaskInfo();
	}

	// FPrimitiveSceneProxy interface.
	virtual SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	virtual float GetAnimationMinScreenSize() const  override
	{
		return Data.AnimationMinScreenSize;
	}

	virtual float GetGpuLodInstanceRadius() const override
	{
		// Note that StaticMeshBounds.SphereRadius is a better fit, but doesn't match the value on the GPU used for LOD culling.
		// That's because GPUScene drops the bounds sphere radius and uses the box. So we end up with the sphere encompassing the box, encompassing the sphere :(
		return bUseGpuLodSelection ? PreSkinnedLocalBounds.BoxExtent.Length() : 0.0f;
	}

	virtual uint8 GetCurrentFirstLODIdx_RenderThread() const override
	{
		return GetCurrentFirstLODIdx_Internal();
	}

	virtual FDesiredLODLevel GetDesiredLODLevel_RenderThread(const FSceneView* View) const override
	{
		if (bUseGpuLodSelection)
		{
			return FDesiredLODLevel::CreateFirst(GetCurrentFirstLODIdx_Internal());
		}
		else
		{
			return Super::GetDesiredLODLevel_RenderThread(View);
		}
	}

private:
	FInstancedSkinnedMeshData Data;
};