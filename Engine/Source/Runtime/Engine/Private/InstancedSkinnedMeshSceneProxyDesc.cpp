// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedSkinnedMeshSceneProxyDesc.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "InstancedSkeletalMeshObject.h"
#include "InstancedSkinnedMeshSceneProxy.h"
#include "Rendering/NaniteResourcesHelper.h"
#include "Rendering/SkeletalMeshRenderData.h"

FSkeletalMeshObject* FInstancedSkinnedMeshSceneProxyDesc::CreateMeshObject(const FInstancedSkinnedMeshSceneProxyDesc& InMeshDesc, FSkeletalMeshRenderData* InRenderData, ERHIFeatureLevel::Type InFeatureLevel)
{
	if (InMeshDesc.ShouldNaniteSkin())
	{
		return new FInstancedSkeletalMeshObject(InMeshDesc, InRenderData, InFeatureLevel, FInstancedSkeletalMeshObject::EType::Nanite);
	}
	else if (InMeshDesc.bRenderStatic)
	{
		return new FInstancedSkeletalMeshObject(InMeshDesc, InRenderData, InFeatureLevel, FInstancedSkeletalMeshObject::EType::Static);
	}
	else if (!InMeshDesc.ShouldCPUSkin() && IsGPUSkinSceneExtensionEnabled())
	{
		return new FInstancedSkeletalMeshObject(InMeshDesc, InRenderData, InFeatureLevel, FInstancedSkeletalMeshObject::EType::GPUSkin);
	}
	return nullptr;
}

FPrimitiveSceneProxy* FInstancedSkinnedMeshSceneProxyDesc::CreateSceneProxy(const FInstancedSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin, bool bShouldNaniteSkin, bool bIsEnabled)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	FPrimitiveSceneProxy* Result = nullptr;
	FSkeletalMeshRenderData* SkelMeshRenderData = Desc.GetSkinnedAsset()->GetResourceForRendering();

	FSkeletalMeshObject* MeshObject = Desc.MeshObject;

	// Only create a scene proxy for rendering if properly initialized
	if (SkelMeshRenderData &&
		SkelMeshRenderData->LODRenderData.IsValidIndex(Desc.PredictedLODLevel) &&
		!bHideSkin &&
		MeshObject)
	{
		if (bIsEnabled)
		{
			if (bShouldNaniteSkin)
			{
				Nanite::FMaterialAudit NaniteMaterials{};
				const bool bSetMaterialUsageFlags = true;
				Nanite::FNaniteResourcesHelper::AuditMaterials(&Desc, NaniteMaterials, bSetMaterialUsageFlags);
				Result = ::new FNaniteInstancedSkinnedMeshSceneProxy(NaniteMaterials, Desc, SkelMeshRenderData);
			}
			else if (!MeshObject->IsNaniteMesh())
			{
				Result = ::new FInstancedSkinnedMeshSceneProxy(Desc, SkelMeshRenderData);
			}
		}

		if (Result == nullptr)
		{
			Result = FSkinnedMeshSceneProxyDesc::CreateSceneProxy(Desc, bHideSkin);
		}
	}

	return Result;
}

FInstancedSkinnedMeshSceneProxyDesc::FInstancedSkinnedMeshSceneProxyDesc(const UInstancedSkinnedMeshComponent* InComponent)
{
	InitializeFromInstancedSkinnedMeshComponent(InComponent);
}

void FInstancedSkinnedMeshSceneProxyDesc::InitializeFromInstancedSkinnedMeshComponent(const UInstancedSkinnedMeshComponent* InComponent)
{
	InitializeFromSkinnedMeshComponent(InComponent);

	TransformProvider = InComponent->TransformProvider;

	AnimationMinScreenSize = InComponent->AnimationMinScreenSize;
	InstanceMinDrawDistance = InComponent->InstanceMinDrawDistance;
	InstanceStartCullDistance = InComponent->InstanceStartCullDistance;
	InstanceEndCullDistance = InComponent->InstanceEndCullDistance;
#if WITH_EDITOR
	SelectedInstances = InComponent->SelectedInstances;
#endif

	bAllowAlwaysVisible = true;

	InstanceDataSceneProxy = InComponent->GetInstanceDataSceneProxy();

	bUseGpuLodSelection = InComponent->bUseGpuLodSelection;
	bForceAnimateSockets = InComponent->bForceAnimateSockets;
}