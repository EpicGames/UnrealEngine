// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InstancedStaticMesh.cpp: Static mesh rendering code.
=============================================================================*/

#include "Engine/InstancedStaticMesh.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "InstanceData/InstanceDataHelpers.h"
#include "Engine/Level.h"
#include "Engine/OverlapResult.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "Components/LightComponent.h"
#include "EngineLogs.h"
#include "Logging/MessageLog.h"
#include "Materials/Material.h"
#include "UnrealEngine.h"
#include "AI/NavigationSystemHelpers.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "MeshDrawShaderBindings.h"
#include "Misc/UObjectToken.h"
#include "Misc/DelayedAutoRegister.h"
#include "PhysicsEngine/BodySetup.h"
#include "GameFramework/WorldSettings.h"
#include "ComponentRecreateRenderStateContext.h"
#include "UObject/MobileObjectVersion.h"
#include "EngineStats.h"
#include "Interfaces/ITargetPlatform.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshMaterialShader.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "RenderUtils.h"
#include "StaticMeshComponentLODInfo.h"
#include "NaniteSceneProxy.h"
#include "MaterialCachedData.h"
#include "Collision.h"
#include "CollisionDebugDrawingPublic.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include "InstancedMeshComponentBodies.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ObjectSaveContext.h"
#include "GenericPlatform/ICursor.h"
#include "NaniteVertexFactory.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "InstancedStaticMesh/ISMInstanceDataManager.h"
#include "InstancedStaticMeshComponentHelper.h"

#if WITH_EDITOR
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#endif // WITH_EDITOR
#include "Templates/Greater.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/EditorObjectVersion.h"

#include "InstancedStaticMesh/ISMInstanceUpdateChangeSet.h"

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#endif

#if UE_BUILD_SHIPPING && !WITH_EDITORONLY_DATA
static_assert(sizeof(FInstancedStaticMeshVertexFactory) <= 672, "FInstancedStaticMeshVertexFactory was optimized to fit within 672 bytes bin of MallocBinned3");
#endif

IMPLEMENT_TYPE_LAYOUT(FInstancedStaticMeshVertexFactoryShaderParameters);

CSV_DEFINE_CATEGORY_MODULE(ENGINE_API, InstancedStaticMeshComponent, true);

const int32 InstancedStaticMeshMaxTexCoord = 8;
static const uint32 MaxSimulatedInstances = 256;

IMPLEMENT_HIT_PROXY(HInstancedStaticMeshInstance, HHitProxy);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FInstancedStaticMeshVertexFactoryUniformShaderParameters, "InstanceVF");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FInstancedStaticMeshVFLooseUniformShaderParameters, "InstancedVFLooseParameters");

TAutoConsoleVariable<int32> CVarGpuLodSelection(
	TEXT("r.InstancedStaticMeshes.GpuLod"),
	1,
	TEXT("Whether to enable GPU LOD selection on InstancedStaticMesh."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { FGlobalComponentRecreateRenderStateContext Context; }),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAllowCreateEmptyISMs(
	TEXT("r.InstancedStaticMeshes.AllowCreateEmpty"),
	0,
	TEXT("Whether to allow creation of empty ISMS."));

TAutoConsoleVariable<int32> CVarMinLOD(
	TEXT("foliage.MinLOD"),
	-1,
	TEXT("Used to discard the top LODs for performance evaluation. -1: Disable all effects of this cvar."),
	ECVF_Scalability | ECVF_Default);

static FAutoConsoleVariableDeprecated CVarCullAllInVertexShader_Deprecated(TEXT("foliage.CullAllInVertexShader"), TEXT("foliage.CullAllInVertexShader_Unused"), TEXT("5.8"));
static TAutoConsoleVariable<int32> CVarCullAllInVertexShader(
	TEXT("foliage.CullAllInVertexShader_Unused"),
	0,
	TEXT("This cvar is no longer implemented as the support would add overhead and complexity."));

static TAutoConsoleVariable<int32> CVarRayTracingRenderInstances(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes"),
	1,
	TEXT("Include static mesh instances in ray tracing effects (default = 1 (Instances enabled in ray tracing))"));

static TAutoConsoleVariable<int32> CVarRayTracingInstancedStaticMeshesMinLOD(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.MinLOD"),
	0,
	TEXT("Clamps minimum LOD to this value (default = 0, highest resolution LOD may be used)"));

static TAutoConsoleVariable<int32> CVarRayTracingRenderInstancesCulling(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.Culling"),
	1,
	TEXT("Enable culling for instances in ray tracing (default = 1 (Culling enabled))"));

static TAutoConsoleVariable<float> CVarRayTracingInstancesCullClusterMaxRadiusMultiplier(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.CullClusterMaxRadiusMultiplier"),
	20.0f, 
	TEXT("Multiplier for the maximum instance size (default = 20)"));

static TAutoConsoleVariable<float> CVarRayTracingInstancesCullClusterRadius(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.CullClusterRadius"),
	10000.0f, // 100 m
	TEXT("Ignore instances outside of this radius in ray tracing effects (default = 10000 (100m))"));

static TAutoConsoleVariable<float> CVarRayTracingInstancesLowScaleThreshold(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.LowScaleRadiusThreshold"),
	50.0f, // Instances with a radius smaller than this threshold get culled after CVarRayTracingInstancesLowScaleCullRadius
	TEXT("Threshold that classifies instances as small (default = 50cm))"));

static TAutoConsoleVariable<float> CVarRayTracingInstancesLowScaleCullRadius(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.LowScaleCullRadius"),
	1000.0f, 
	TEXT("Cull radius for small instances (default = 1000 (10m))"));

static TAutoConsoleVariable<float> CVarRayTracingInstancesCullAngle(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.CullAngle"),
	2.0f,
	TEXT("Solid angle to test instance bounds against for culling (default 2 degrees)\n")
	TEXT("  -1 => use distance based culling")
);

static TAutoConsoleVariable<int32> CVarRayTracingInstancesEvaluateWPO(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.EvaluateWPO"),
	0,
	TEXT("Whether to evaluate WPO on instanced static meshes\n")
	TEXT("  0 - off (default)")
	TEXT("  1 - on for all with WPO")
	TEXT(" -1 - on only for meshes with evaluate WPO enabled")
);

FAutoConsoleVariableSink CVarRayTracingInstancesEvaluateWPOSink(FConsoleCommandDelegate::CreateLambda([]()
	{
		static int32 CachedValue = CVarRayTracingInstancesEvaluateWPO.GetValueOnGameThread();

		int32 Value = CVarRayTracingInstancesEvaluateWPO.GetValueOnGameThread();

		if (CachedValue != Value)
		{
			CachedValue = Value;

			// NV-JIRA UE-668: Do this as a task on the game thread to break up a possible
			// reentry call to USkeletalMeshComponent::PostAnimEvaluation if BP toggles this CVar
			FFunctionGraphTask::CreateAndDispatchWhenReady([]()
				{
					// Easiest way to update all static scene proxies is to recreate them
					FlushRenderingCommands();
					FGlobalComponentReregisterContext ReregisterContext;

				}, TStatId(), nullptr, ENamedThreads::GameThread);
		}
	}));

static TAutoConsoleVariable<float> CVarRayTracingInstancesSimulationClusterRadius(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.SimulationClusterRadius"),
	500.0f, // 5 m
	TEXT("Bucket instances based on distance to camera for simulating WPO (default = 500 (5m), disable if <= 0)"));

static TAutoConsoleVariable<int32> CVarRayTracingSimulatedInstanceCount(
	TEXT("r.RayTracing.Geometry.InstancedStaticMeshes.SimulationCount"),
	1,
	TEXT("Maximum number of instances to simulate per instanced static mesh, presently capped to 256")
);

#if WITH_EDITOR
static TAutoConsoleVariable<int32> CVarEnableViewportSMInstanceSelection(
	TEXT("TypedElements.EnableViewportSMInstanceSelection"),
	1,
	TEXT("Enable direct selection of Instanced Static Mesh Component (ISMC) Instances in the Level Editor Viewport")
);
#endif

static TAutoConsoleVariable<int32> CVarISMForceRemoveAtSwap(
	TEXT("r.InstancedStaticMeshes.ForceRemoveAtSwap"),
	0,
	TEXT("Force the RemoveAtSwap optimization when removing instances from an ISM."));

static TAutoConsoleVariable<int32> CVarISMFetchInstanceCountFromScene(
	TEXT("r.InstancedStaticMeshes.FetchInstanceCountFromScene"),
	1,
	TEXT("Enables the data path that allows instance count to be fetched from the Scene rather than the Mesh Draw Commands (MDCs), which removes the need to re-cache MDCs when instance count changes."));

static int32 GConservativeBoundsThreshold = 30;
FAutoConsoleVariableRef CVarISMConservativeBoundsThreshold(
	TEXT("r.InstancedStaticMeshes.ConservativeBounds.Threshold"),
	GConservativeBoundsThreshold,
	TEXT("Number of instances in an ISM before we start using conservative bounds. Set to -1 to disable conservative bounds."));

static TAutoConsoleVariable<int32> CVarISMInvalidateCachedNavigationBoundsOnPartialNavigationUpdates(
	TEXT("r.InstancedStaticMeshes.InvalidateCachedNavigationBoundsOnPartialNavigationUpdates"),
	1,
	TEXT("Invalidate cached navigation bounds when doing a partial navigation update"));

class FISMExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec_Runtime(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		if(FParse::Command(&Cmd, TEXT("LIST ISM PHYSICS")))
		{
			Ar.Logf(TEXT("----- BEGIN ISM PHYSICS LISTING -------------------------"));
			Ar.Logf(TEXT("Component Name, Component Class Name, Mesh Name, Num Bodies, Num Shapes"));

			int32 BodiesTotal = 0;
			int32 ShapesTotal = 0;
			for(TObjectIterator<UInstancedStaticMeshComponent> It; It; ++It)
			{
				UInstancedStaticMeshComponent* ISMComponent = *It;
				const UWorld* ComponentWorld = ISMComponent->GetWorld();
				if (ComponentWorld!=InWorld || !ComponentWorld || !ComponentWorld->IsGameWorld())
				{
					continue;
				}

				const FString ComponentName = ISMComponent->GetPathName(ISMComponent->GetOwner()->GetOuter());

				if(ISMComponent->ShouldCreatePhysicsState())
				{
					const TArray<FBodyInstance*>& Bodies = ISMComponent->GetInstanceBodies();
					if(Bodies.Num() == 0)
					{
						continue;
					}

					int32 NumValidInstanceBodies = 0;
					int32 NumInstanceShapes = 0;
					TArray<FPhysicsShapeHandle> Shapes;
					for(const FBodyInstance* Body : Bodies)
					{
						if(!Body)
						{
							continue;
						}
						++NumValidInstanceBodies;
						
						FPhysicsCommand::ExecuteRead(Body->GetPhysicsActor(), [&Body, &Shapes, &NumInstanceShapes](const FPhysicsActorHandle& Actor)
						{
							Shapes.Reset();
							Body->GetAllShapes_AssumesLocked(Shapes);

							NumInstanceShapes += Shapes.Num();
						});
					}

					UStaticMesh* Mesh = ISMComponent->GetStaticMesh();

					Ar.Logf(TEXT("%s, %s, %s, %d, %d"),
							*ComponentName,
							*ISMComponent->GetClass()->GetName(),
							Mesh ? *Mesh->GetPathName() : TEXT("None"),
							NumValidInstanceBodies,
							NumInstanceShapes);

					BodiesTotal += NumValidInstanceBodies;
					ShapesTotal += NumInstanceShapes;
				}
			}
			Ar.Logf(TEXT("\nTotal Bodies, Total Shapes\n%d, %d"), BodiesTotal, ShapesTotal);
			Ar.Logf(TEXT("----- END ISM PHYSICS LISTING ---------------------------"));

			return true;
		}
		else if(FParse::Command(&Cmd, TEXT("LIST ISM")))
		{
			if (InWorld == nullptr)
			{
				return true;
			}
			Ar.Logf(TEXT("Name, Num Instances, Has Previous Transform, Num Custom Floats, Has Random, Has Custom Data, Has Dynamic Data, Has LMSMUVBias, Has LocalBounds, Has Instance Hiearchy Offset"));
			ERHIFeatureLevel::Type FeatureLevel = InWorld->GetFeatureLevel();
			for (TObjectIterator<UInstancedStaticMeshComponent> It; It; ++It)
			{
				UInstancedStaticMeshComponent* ISMComponent = *It;
				const UWorld* ComponentWorld = ISMComponent->GetWorld();
				if (ComponentWorld != InWorld || !ComponentWorld || !ComponentWorld->IsGameWorld())
				{
					continue;
				}

				UStaticMesh* Mesh = ISMComponent->GetStaticMesh();
				FPrimitiveMaterialPropertyDescriptor MatDesc = ISMComponent->GetUsedMaterialPropertyDesc(GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
				FInstanceDataFlags Flags = ISMComponent->MakeInstanceDataFlags(MatDesc.bAnyMaterialHasPerInstanceRandom, MatDesc.bAnyMaterialHasPerInstanceCustomData);

				Ar.Logf(TEXT("%s, %d, %d, %d, %d, %d, %d, %d, %d, %d"),
					Mesh ? *Mesh->GetFullName() : TEXT(""),
					ISMComponent->GetInstanceCount(),
					ISMComponent->PerInstancePrevTransform.Num() > 0,
					ISMComponent->NumCustomDataFloats,
					Flags.bHasPerInstanceRandom,
					Flags.bHasPerInstanceCustomData,
					Flags.bHasPerInstanceDynamicData,
					Flags.bHasPerInstanceLMSMUVBias,
					Flags.bHasPerInstanceLocalBounds,
					Flags.bHasPerInstanceHierarchyOffset);
			}
			return true;
		}
		return false;
	}
};
static FISMExecHelper GISMExecHelper;

class FDummyFloatBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create the texture RHI.
		const int32 NumFloats = 4;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("DummyFloatBuffer"), sizeof(float) * NumFloats)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState()
			.SetInitActionZeroData();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
	}
};

TGlobalResource<FDummyFloatBuffer> GDummyFloatBuffer;

FInstancedStaticMeshDelegates::FOnInstanceIndexUpdated FInstancedStaticMeshDelegates::OnInstanceIndexUpdated;

/** InstancedStaticMeshInstance hit proxy */
void HInstancedStaticMeshInstance::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Component);
}

FTypedElementHandle HInstancedStaticMeshInstance::GetElementHandle() const
{
#if WITH_EDITOR
	if (Component)
	{
		if (CVarEnableViewportSMInstanceSelection.GetValueOnAnyThread() != 0)
		{
			// Prefer per-instance selection if available
			// This may fail to return a handle if the feature is disabled, or if per-instance editing is disabled for this component
			if (FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(Component, InstanceIndex))
			{
				return ElementHandle;
			}
		}

		// If per-instance selection isn't possible, fallback to general per-component selection (which may choose to select the owner actor instead)
		return UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component);
	}
#endif	// WITH_EDITOR
	return FTypedElementHandle();
}

EMouseCursor::Type HInstancedStaticMeshInstance::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

FStaticMeshInstanceBuffer::FStaticMeshInstanceBuffer(ERHIFeatureLevel::Type InFeatureLevel, bool InRequireCPUAccess)
	: FRenderResource(InFeatureLevel)
	, RequireCPUAccess(InRequireCPUAccess)
	, bFlushToGPUPending(false)
{
}

FStaticMeshInstanceBuffer::~FStaticMeshInstanceBuffer()
{
	CleanUp();
}

/** Delete existing resources */
void FStaticMeshInstanceBuffer::CleanUp()
{
	InstanceData.Reset();
}

void FStaticMeshInstanceBuffer::InitFromPreallocatedData(FStaticMeshInstanceData& Other)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FStaticMeshInstanceBuffer_InitFromPreallocatedData);

	InstanceData = MakeShared<FStaticMeshInstanceData, ESPMode::ThreadSafe>();
	Swap(Other, *InstanceData.Get());
	InstanceData->SetAllowCPUAccess(RequireCPUAccess);
}

/**
 * Specialized assignment operator, only used when importing LOD's.  
 */
void FStaticMeshInstanceBuffer::operator=(const FStaticMeshInstanceBuffer &Other)
{
	checkf(0, TEXT("Unexpected assignment call"));
}

void FStaticMeshInstanceBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStaticMeshInstanceBuffer::InitRHI");

	check(InstanceData);
	if (InstanceData->GetNumInstances() > 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FStaticMeshInstanceBuffer_InitRHI);
		SCOPED_LOADTIMER(FStaticMeshInstanceBuffer_InitRHI);

		LLM_SCOPE(ELLMTag::InstancedMesh);
		auto AccessFlags = BUF_Static;
		CreateVertexBuffer(RHICmdList, InstanceData->GetOriginResourceArray(), AccessFlags | BUF_ShaderResource, 16, PF_A32B32G32R32F, InstanceOriginBuffer.VertexBufferRHI, InstanceOriginSRV);
		CreateVertexBuffer(RHICmdList, InstanceData->GetTransformResourceArray(), AccessFlags | BUF_ShaderResource, InstanceData->GetTranslationUsesHalfs() ? 8 : 16, InstanceData->GetTranslationUsesHalfs() ? PF_FloatRGBA : PF_A32B32G32R32F, InstanceTransformBuffer.VertexBufferRHI, InstanceTransformSRV);
		CreateVertexBuffer(RHICmdList, InstanceData->GetLightMapResourceArray(), AccessFlags | BUF_ShaderResource, 8, PF_R16G16B16A16_SNORM, InstanceLightmapBuffer.VertexBufferRHI, InstanceLightmapSRV);
		if (InstanceData->GetNumCustomDataFloats() > 0)
		{
			CreateVertexBuffer(RHICmdList, InstanceData->GetCustomDataResourceArray(), AccessFlags | BUF_ShaderResource, 4, PF_R32_FLOAT, InstanceCustomDataBuffer.VertexBufferRHI, InstanceCustomDataSRV);
			// Make sure we still create custom data SRV on platforms that do not support/use MVF 
			if (InstanceCustomDataSRV == nullptr)
			{
				InstanceCustomDataSRV = RHICmdList.CreateShaderResourceView(
					InstanceCustomDataBuffer.VertexBufferRHI, 
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_R32_FLOAT));
			}
		}
		else
		{
			InstanceCustomDataSRV = GDummyFloatBuffer.ShaderResourceViewRHI;
		}
	}
}

void FStaticMeshInstanceBuffer::ReleaseRHI()
{
	InstanceOriginSRV.SafeRelease();
	InstanceTransformSRV.SafeRelease();
	InstanceLightmapSRV.SafeRelease();
	InstanceCustomDataSRV.SafeRelease();

	InstanceOriginBuffer.ReleaseRHI();
	InstanceTransformBuffer.ReleaseRHI();
	InstanceLightmapBuffer.ReleaseRHI();
	InstanceCustomDataBuffer.ReleaseRHI();
}

void FStaticMeshInstanceBuffer::InitResource(FRHICommandListBase& RHICmdList)
{
	FRenderResource::InitResource(RHICmdList);
	InstanceOriginBuffer.InitResource(RHICmdList);
	InstanceTransformBuffer.InitResource(RHICmdList);
	InstanceLightmapBuffer.InitResource(RHICmdList);
	InstanceCustomDataBuffer.InitResource(RHICmdList);
}

void FStaticMeshInstanceBuffer::ReleaseResource()
{
	FRenderResource::ReleaseResource();
	InstanceOriginBuffer.ReleaseResource();
	InstanceTransformBuffer.ReleaseResource();
	InstanceLightmapBuffer.ReleaseResource();
	InstanceCustomDataBuffer.ReleaseResource();
}

SIZE_T FStaticMeshInstanceBuffer::GetResourceSize() const
{
	if (InstanceData && InstanceData->GetNumInstances() > 0)
	{
		return InstanceData->GetResourceSize();
	}
	return 0;
}

void FStaticMeshInstanceBuffer::CreateVertexBuffer(FRHICommandListBase& RHICmdList, FResourceArrayInterface* InResourceArray, EBufferUsageFlags InUsage, uint32 InStride, uint8 InFormat, FBufferRHIRef& OutVertexBufferRHI, FShaderResourceViewRHIRef& OutInstanceSRV)
{
	check(InResourceArray);
	check(InResourceArray->GetResourceDataSize() > 0);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FStaticMeshInstanceBuffer"), InResourceArray->GetResourceDataSize())
		.AddUsage(InUsage)
		.SetInitActionResourceArray(InResourceArray)
		.DetermineInitialState();

	// TODO: possibility over allocated the vertex buffer when we support partial update for when working in the editor
	OutVertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

	if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		OutInstanceSRV = RHICmdList.CreateShaderResourceView(
			OutVertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(static_cast<EPixelFormat>(InFormat)));
	}
}

void FStaticMeshInstanceBuffer::BindInstanceVertexBuffer(const class FVertexFactory* VertexFactory, FInstancedStaticMeshDataType& InstancedStaticMeshData) const
{
	if (InstanceData->GetNumInstances())
	{
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			check(InstanceOriginSRV);
			check(InstanceTransformSRV);
			check(InstanceLightmapSRV);
		}
		check(InstanceCustomDataSRV); // Should not be nullptr, but can be assigned a dummy buffer
	}

	{
		InstancedStaticMeshData.InstanceOriginSRV = InstanceOriginSRV;
		InstancedStaticMeshData.InstanceTransformSRV = InstanceTransformSRV;
		InstancedStaticMeshData.InstanceLightmapSRV = InstanceLightmapSRV;
		InstancedStaticMeshData.InstanceCustomDataSRV = InstanceCustomDataSRV;
		InstancedStaticMeshData.NumCustomDataFloats = InstanceData->GetNumCustomDataFloats();
	}

	{
		InstancedStaticMeshData.InstanceOriginComponent = FVertexStreamComponent(
			&InstanceOriginBuffer,
			0,
			16,
			VET_Float4,
			EVertexStreamUsage::ManualFetch | EVertexStreamUsage::Instancing
		);

		EVertexElementType TransformType = InstanceData->GetTranslationUsesHalfs() ? VET_Half4 : VET_Float4;
		uint32 TransformStride = InstanceData->GetTranslationUsesHalfs() ? 8 : 16;

		InstancedStaticMeshData.InstanceTransformComponent[0] = FVertexStreamComponent(
			&InstanceTransformBuffer,
			0 * TransformStride,
			3 * TransformStride,
			TransformType,
			EVertexStreamUsage::ManualFetch | EVertexStreamUsage::Instancing
		);
		InstancedStaticMeshData.InstanceTransformComponent[1] = FVertexStreamComponent(
			&InstanceTransformBuffer,
			1 * TransformStride,
			3 * TransformStride,
			TransformType,
			EVertexStreamUsage::ManualFetch | EVertexStreamUsage::Instancing
		);
		InstancedStaticMeshData.InstanceTransformComponent[2] = FVertexStreamComponent(
			&InstanceTransformBuffer,
			2 * TransformStride,
			3 * TransformStride,
			TransformType,
			EVertexStreamUsage::ManualFetch | EVertexStreamUsage::Instancing
		);

		InstancedStaticMeshData.InstanceLightmapAndShadowMapUVBiasComponent = FVertexStreamComponent(
			&InstanceLightmapBuffer,
			0,
			8,
			VET_Short4N,
			EVertexStreamUsage::ManualFetch | EVertexStreamUsage::Instancing
		);
	}
}

void FStaticMeshInstanceBuffer::FlushGPUUpload(FRHICommandListBase& RHICmdList)
{
	if (bFlushToGPUPending)
	{
		if (!IsInitialized())
		{
			InitResource(RHICmdList);
		}
		else
		{
			UpdateRHI(RHICmdList);
		}
		bFlushToGPUPending = false;
	}
}


void FStaticMeshInstanceData::Serialize(FArchive& Ar)
{	
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	const bool bCookConvertTransformsToFullFloat = Ar.IsCooking() && bUseHalfFloat && !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::HalfFloatVertexFormat);

	if (bCookConvertTransformsToFullFloat)
	{
		bool bSaveUseHalfFloat = false;
		Ar << bSaveUseHalfFloat;
	}
	else
	{
		Ar << bUseHalfFloat;
	}

	Ar << NumInstances;

	if (!Ar.IsLoading() || Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::PerInstanceCustomData)
	{
		Ar << NumCustomDataFloats;
	}

	if (Ar.IsLoading())
	{
		const int64 NumTotalCustomDataFloats = (int64)NumCustomDataFloats * NumInstances;
		if (!IntFitsIn<int32>(NumTotalCustomDataFloats))
		{
			// Sanitize inputs. Allocate no custom data to avoid out of range access
			NumCustomDataFloats = 0;
			ensureMsgf(false, TEXT("Total Custom Instance Data Floats count is out of range."));
		}

		AllocateBuffers(NumInstances);
	}

	InstanceOriginData->Serialize(Ar);
	InstanceLightmapData->Serialize(Ar);

	if (bCookConvertTransformsToFullFloat)
	{
		TStaticMeshVertexData<FInstanceTransformMatrix<float>> FullInstanceTransformData;
		FullInstanceTransformData.ResizeBuffer(NumInstances);

		FInstanceTransformMatrix<FFloat16>* Src = (FInstanceTransformMatrix<FFloat16>*)InstanceTransformData->GetDataPointer();
		FInstanceTransformMatrix<float>* Dest = (FInstanceTransformMatrix<float>*)FullInstanceTransformData.GetDataPointer();
		for (int32 Idx = 0; Idx < NumInstances; Idx++)
		{
			Dest->InstanceTransform1[0] = Src->InstanceTransform1[0];
			Dest->InstanceTransform1[1] = Src->InstanceTransform1[1];
			Dest->InstanceTransform1[2] = Src->InstanceTransform1[2];
			Dest->InstanceTransform1[3] = Src->InstanceTransform1[3];
			Dest->InstanceTransform2[0] = Src->InstanceTransform2[0];
			Dest->InstanceTransform2[1] = Src->InstanceTransform2[1];
			Dest->InstanceTransform2[2] = Src->InstanceTransform2[2];
			Dest->InstanceTransform2[3] = Src->InstanceTransform2[3];
			Dest->InstanceTransform3[0] = Src->InstanceTransform3[0];
			Dest->InstanceTransform3[1] = Src->InstanceTransform3[1];
			Dest->InstanceTransform3[2] = Src->InstanceTransform3[2];
			Dest->InstanceTransform3[3] = Src->InstanceTransform3[3];
			Src++;
			Dest++;
		}

		FullInstanceTransformData.Serialize(Ar);
	}
	else
	{
		InstanceTransformData->Serialize(Ar);
	}

	if (!Ar.IsLoading() || Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::PerInstanceCustomData)
	{
		InstanceCustomData->Serialize(Ar);
	}

	if (Ar.IsLoading())
	{
		InstanceOriginDataPtr = InstanceOriginData->GetDataPointer();
		InstanceLightmapDataPtr = InstanceLightmapData->GetDataPointer();
		InstanceTransformDataPtr = InstanceTransformData->GetDataPointer();
		InstanceCustomDataPtr = InstanceCustomData->GetDataPointer();
	}
}


/**
 * Should we cache the material's shadertype on this platform with this vertex factory? 
 */
bool FInstancedStaticMeshVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return DoesPlatformUseInstancedStaticMeshVertexFactory(Parameters.Platform)
			&& (Parameters.MaterialParameters.bIsUsedWithInstancedStaticMeshes || Parameters.MaterialParameters.bIsSpecialEngineMaterial)
			&& FLocalVertexFactory::ShouldCompilePermutation(Parameters);
}

void FInstancedStaticMeshVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("IS_INSTANCED_STATIC_MESH_VF"), TEXT("1"));

	if (RHISupportsManualVertexFetch(Parameters.Platform))
	{
		OutEnvironment.SetDefineIfUnset(TEXT("MANUAL_VERTEX_FETCH"), TEXT("1"));
	}

	if (UseGPUScene(Parameters.Platform))
	{
		// USE_INSTANCE_CULLING - set up additional instancing attributes (basic instancing is the default)
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), TEXT("1"));
	}
	else
	{
		OutEnvironment.SetDefine(TEXT("USE_INSTANCING"), TEXT("1"));
	}

	FLocalVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

/**
 * Get vertex elements used when during PSO precaching materials using this vertex factory type
 */
void FInstancedStaticMeshVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	// Fallback to local vertex factory because manual vertex fetch is supported
	FLocalVertexFactory::GetPSOPrecacheVertexFetchElements(VertexInputStreamType, Elements);
	
	const bool bCanUseGPUScene = UseGPUScene(GMaxRHIShaderPlatform);
	const bool bMobileUsesGPUScene = MobileSupportsGPUScene();
	
	if (GMaxRHIFeatureLevel > ERHIFeatureLevel::ES3_1 || !bMobileUsesGPUScene)
	{
		uint8 StreamIndex = Elements.Last().StreamIndex + 1;

		// InstanceOriginComponent
		Elements.Add(FVertexElement(StreamIndex++, 0, VET_Float4, 8, 0, false));

		// InstanceTransformComponent
		Elements.Add(FVertexElement(StreamIndex, 0, VET_Float4, 9, 0, false));
		Elements.Add(FVertexElement(StreamIndex, 8, VET_Float4, 10, 0, false));
		Elements.Add(FVertexElement(StreamIndex++, 16, VET_Float4, 11, 0, false));

		// InstanceLightmapAndShadowMapUVBiasComponent
		Elements.Add(FVertexElement(StreamIndex++, 16, VET_Short4N, 12, 0, false));
	}
}

/**
 * Copy the data from another vertex factory
 * @param Other - factory to copy from
 */
void FInstancedStaticMeshVertexFactory::Copy(const FInstancedStaticMeshVertexFactory& Other)
{
	FInstancedStaticMeshVertexFactory* VertexFactory = this;
	const FLocalVertexFactory::FDataType* DataCopy = &Other.Data;
	const FInstancedStaticMeshDataType* InstanceDataCopy = &Other.InstanceData;
	ENQUEUE_RENDER_COMMAND(FInstancedStaticMeshVertexFactoryCopyData)(
	[VertexFactory, DataCopy, InstanceDataCopy] (FRHICommandListBase&)
	{
		VertexFactory->Data = *DataCopy;
		VertexFactory->InstanceData = *InstanceDataCopy;
	});
	BeginUpdateResourceRHI(this);
}

void FInstancedStaticMeshVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FInstancedStaticMeshVertexFactory::InitRHI");

	SCOPED_LOADTIMER(FInstancedStaticMeshVertexFactory_InitRHI);

	check(HasValidFeatureLevel());

	const ERHIFeatureLevel::Type ThisFeatureLevel = GetFeatureLevel();
	const bool bCanUseGPUScene = UseGPUScene(GShaderPlatformForFeatureLevel[ThisFeatureLevel]);
	const bool bUseManualVertexFetch = GetType()->SupportsManualVertexFetch(ThisFeatureLevel);

	FVertexDeclarationElementList Elements;
	GetVertexElements(ThisFeatureLevel, EVertexInputStreamType::Default, bUseManualVertexFetch, Data, InstanceData, Elements, Streams);

	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, 13, 13);

	// we don't need per-vertex shadow or lightmap rendering
	InitDeclaration(Elements);

	if (!bCanUseGPUScene)
	{
		FInstancedStaticMeshVertexFactoryUniformShaderParameters UniformParameters;
		UniformParameters.VertexFetch_InstanceOriginBuffer = GetInstanceOriginSRV();
		UniformParameters.VertexFetch_InstanceTransformBuffer = GetInstanceTransformSRV();
		UniformParameters.VertexFetch_InstanceLightmapBuffer = GetInstanceLightmapSRV();
		UniformParameters.InstanceCustomDataBuffer = GetInstanceCustomDataSRV();
		UniformParameters.NumCustomDataFloats = InstanceData.NumCustomDataFloats;
		UniformBuffer = TUniformBufferRef<FInstancedStaticMeshVertexFactoryUniformShaderParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_MultiFrame, EUniformBufferValidation::None);
	}
}

void FInstancedStaticMeshVertexFactory::GetVertexElements(
	ERHIFeatureLevel::Type FeatureLevel,
	EVertexInputStreamType InputStreamType,
	bool bSupportsManualVertexFetch,
	FDataType& Data,
	FInstancedStaticMeshDataType& InstanceData,
	FVertexDeclarationElementList& Elements)
{
	FVertexStreamList VertexStreams;
	GetVertexElements(FeatureLevel, InputStreamType, bSupportsManualVertexFetch, Data, InstanceData, Elements, VertexStreams);

	if (UseGPUScene(GMaxRHIShaderPlatform) 
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		Elements.Add(FVertexElement(VertexStreams.Num(), 0, VET_UInt, 13, sizeof(uint32), true));
	}
}

void FInstancedStaticMeshVertexFactory::GetVertexElements(
	ERHIFeatureLevel::Type FeatureLevel, 
	EVertexInputStreamType InputStreamType, 
	bool bSupportsManualVertexFetch, 
	FDataType& Data, 
	FInstancedStaticMeshDataType& InstanceData,
	FVertexDeclarationElementList& Elements, 
	FVertexStreamList& Streams)
{
	if (Data.PositionComponent.VertexBuffer != NULL)
	{
		Elements.Add(AccessStreamComponent(Data.PositionComponent, 0, Streams));
	}

	if (!bSupportsManualVertexFetch)
	{
		// only tangent,normal are used by the stream. the binormal is derived in the shader
		uint8 TangentBasisAttributes[2] = { 1, 2 };
		for (int32 AxisIndex = 0; AxisIndex < 2; AxisIndex++)
		{
			if (Data.TangentBasisComponents[AxisIndex].VertexBuffer != NULL)
			{
				Elements.Add(AccessStreamComponent(Data.TangentBasisComponents[AxisIndex], TangentBasisAttributes[AxisIndex], Streams));
			}
		}

		if (Data.ColorComponentsSRV == nullptr)
		{
			Data.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
			Data.ColorIndexMask = 0;
		}

		if (Data.ColorComponent.VertexBuffer)
		{
			Elements.Add(AccessStreamComponent(Data.ColorComponent, 3, Streams));
		}
		else
		{
			//If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
			//This wastes 4 bytes of bandwidth per vertex, but prevents having to compile out twice the number of vertex factories.
			FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
			Elements.Add(AccessStreamComponent(NullColorComponent, 3, Streams));
		}

		if (Data.TextureCoordinates.Num())
		{
			const int32 BaseTexCoordAttribute = 4;				
			AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, Data.TextureCoordinates, Elements, Streams);
		}

		// PreSkinPosition attribute is only used for GPUSkinPassthrough variation of local vertex factory.
		// It is not used by ISM so fill with dummy buffer.
		if (IsGPUSkinPassThroughSupported(GMaxRHIShaderPlatform))
		{
			FVertexStreamComponent NullComponent(&GNullVertexBuffer, 0, 0, VET_Float4);
			Elements.Add(AccessStreamComponent(NullComponent, 14, Streams));
		}

		if (IsStaticLightingAllowed())
		{
			if (Data.LightMapCoordinateComponent.VertexBuffer)
			{
				Elements.Add(AccessStreamComponent(Data.LightMapCoordinateComponent, 15, Streams));
			}
			else if (Data.TextureCoordinates.Num())
			{
				Elements.Add(AccessStreamComponent(Data.TextureCoordinates[0], 15, Streams));
			}
		}
	}

	const bool bCanUseGPUScene = UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]);
	const bool bMobileUsesGPUScene = MobileSupportsGPUScene();
	
	if (FeatureLevel > ERHIFeatureLevel::ES3_1 || !bMobileUsesGPUScene)
	{
		// toss in the instanced location stream
		check(bCanUseGPUScene || InstanceData.InstanceOriginComponent.VertexBuffer);
		if (InstanceData.InstanceOriginComponent.VertexBuffer)
		{
			Elements.Add(AccessStreamComponent(InstanceData.InstanceOriginComponent, 8, Streams));
		}

		check(bCanUseGPUScene || InstanceData.InstanceTransformComponent[0].VertexBuffer);
		if (InstanceData.InstanceTransformComponent[0].VertexBuffer)
		{
			Elements.Add(AccessStreamComponent(InstanceData.InstanceTransformComponent[0], 9, Streams));
			Elements.Add(AccessStreamComponent(InstanceData.InstanceTransformComponent[1], 10, Streams));
			Elements.Add(AccessStreamComponent(InstanceData.InstanceTransformComponent[2], 11, Streams));
		}

		if (InstanceData.InstanceLightmapAndShadowMapUVBiasComponent.VertexBuffer)
		{
			Elements.Add(AccessStreamComponent(InstanceData.InstanceLightmapAndShadowMapUVBiasComponent, 12, Streams));
		}
	}
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FInstancedStaticMeshVertexFactory, SF_Vertex, FInstancedStaticMeshVertexFactoryShaderParameters);
// pixel shader may need access to InstanceCustomDataBuffer in non-GPUScene case
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FInstancedStaticMeshVertexFactory, SF_Pixel, FInstancedStaticMeshVertexFactoryShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FInstancedStaticMeshVertexFactory, SF_RayHitGroup, FInstancedStaticMeshVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FInstancedStaticMeshVertexFactory, SF_Compute, FInstancedStaticMeshVertexFactoryShaderParameters);
#endif

IMPLEMENT_VERTEX_FACTORY_TYPE(FInstancedStaticMeshVertexFactory,"/Engine/Private/LocalVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsLightmapBaking
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::DoesNotSupportNullPixelShader
	| EVertexFactoryFlags::SupportsManualVertexFetch
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsLumenMeshCards
);

FInstancedStaticMeshRenderData::FInstancedStaticMeshRenderData(const FInstancedStaticMeshSceneProxyDesc* InDesc, ERHIFeatureLevel::Type InFeatureLevel)
	: Component(Cast<UInstancedStaticMeshComponent>(InDesc->Component))
	, LightMapCoordinateIndex(InDesc->GetStaticMesh()->GetLightMapCoordinateIndex())
	, LODModels(InDesc->GetStaticMesh()->GetRenderData()->LODResources)
	, FeatureLevel(InFeatureLevel)
{
	if (IsRayTracingAllowed() && InDesc->GetStaticMesh()->bSupportRayTracing && InDesc->GetStaticMesh()->GetRenderData()->RayTracingProxy)
	{
		RayTracingProxyLODs = &InDesc->GetStaticMesh()->GetRenderData()->RayTracingProxy->LODs;
	}

	// Allocate the vertex factories for each LOD
	InitVertexFactories();
	RegisterSpeedTreeWind(InDesc);
}

void FInstancedStaticMeshRenderData::ReleaseResources(FSceneInterface* Scene, const UStaticMesh* StaticMesh)
{
	// unregister SpeedTree wind with the scene
	if (Scene && StaticMesh && StaticMesh->SpeedTreeWind.IsValid())
	{
		for (int32 LODIndex = 0; LODIndex < VertexFactories.Num(); LODIndex++)
		{
			Scene->RemoveSpeedTreeWind_RenderThread(&VertexFactories[LODIndex], StaticMesh);
		}
	}

	for (int32 LODIndex = 0; LODIndex < VertexFactories.Num(); LODIndex++)
	{
		VertexFactories[LODIndex].ReleaseResource();
	}

	for (int32 LODIndex = 0; LODIndex < RayTracingProxyVertexFactories.Num(); LODIndex++)
	{
		RayTracingProxyVertexFactories[LODIndex].ReleaseResource();
	}
}

void FInstancedStaticMeshVertexFactory::InitInstancedStaticMeshVertexFactoryComponents(
	const FStaticMeshVertexBuffers& VertexBuffers,
	const FColorVertexBuffer* ColorVertexBuffer,
	const FStaticMeshInstanceBuffer* InstanceBuffer,
	const FInstancedStaticMeshVertexFactory* VertexFactory,
	int32 LightMapCoordinateIndex,
	bool bRHISupportsManualVertexFetch,
	FInstancedStaticMeshVertexFactory::FDataType& OutData,
	FInstancedStaticMeshDataType& OutInstanceData)
{
	VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(VertexFactory, OutData);
	VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactory, OutData);
	VertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactory, OutData);
	
	if (LightMapCoordinateIndex < (int32)VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() && LightMapCoordinateIndex >= 0)
	{
		VertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactory, OutData, LightMapCoordinateIndex);
	}

	if (ColorVertexBuffer != nullptr)
	{
		ColorVertexBuffer->BindColorVertexBuffer(VertexFactory, OutData);
	}
	else
	{
		// shouldn't this check if ISM component actually has a color data for override?
		FColorVertexBuffer::BindDefaultColorVertexBuffer(VertexFactory, OutData, bRHISupportsManualVertexFetch ? FColorVertexBuffer::NullBindStride::FColorSizeForComponentOverride : FColorVertexBuffer::NullBindStride::ZeroForDefaultBufferBind);
	}

	if (InstanceBuffer)
	{
		InstanceBuffer->BindInstanceVertexBuffer(VertexFactory, OutInstanceData);
	}
}

void FInstancedStaticMeshRenderData::BindBuffersToVertexFactories(FRHICommandListBase& RHICmdList, FStaticMeshInstanceBuffer* InstanceBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FInstancedStaticMeshRenderData::BindBuffersToVertexFactories");

	if (InstanceBuffer)
	{
		InstanceBuffer->FlushGPUUpload(RHICmdList);
	}

	const bool bRHISupportsManualVertexFetch = RHISupportsManualVertexFetch(GShaderPlatformForFeatureLevel[FeatureLevel]);
	for (int32 LODIndex = 0; LODIndex < VertexFactories.Num(); LODIndex++)
	{
		const FStaticMeshLODResources* RenderData = &LODModels[LODIndex];

		FInstancedStaticMeshDataType InstanceData;
		FInstancedStaticMeshVertexFactory::FDataType Data;
		// Assign to the vertex factory for this LOD.
		FInstancedStaticMeshVertexFactory& VertexFactory = VertexFactories[LODIndex];
		const FColorVertexBuffer* ColorVertexBuffer = RenderData->bHasColorVertexData ? &(RenderData->VertexBuffers.ColorVertexBuffer) : nullptr;
		//@todo: replacement for non UInstancedStaticMeshComponent case
		if (Component && Component->LODData.IsValidIndex(LODIndex) && Component->LODData[LODIndex].OverrideVertexColors)
		{
			ColorVertexBuffer = Component->LODData[LODIndex].OverrideVertexColors;
		}
		FInstancedStaticMeshVertexFactory::InitInstancedStaticMeshVertexFactoryComponents(RenderData->VertexBuffers, ColorVertexBuffer, InstanceBuffer, &VertexFactory, LightMapCoordinateIndex, bRHISupportsManualVertexFetch, Data, InstanceData);
		VertexFactory.SetData(RHICmdList, Data, InstanceBuffer ? &InstanceData : nullptr);
		VertexFactory.InitResource(RHICmdList);
	}

	for (int32 LODIndex = 0; LODIndex < RayTracingProxyVertexFactories.Num(); LODIndex++)
	{
		const FStaticMeshRayTracingProxyLOD* RenderData = &(*RayTracingProxyLODs)[LODIndex];

		FInstancedStaticMeshDataType InstanceData;
		FInstancedStaticMeshVertexFactory::FDataType Data;
		// Assign to the vertex factory for this LOD.
		FInstancedStaticMeshVertexFactory& VertexFactory = RayTracingProxyVertexFactories[LODIndex];
		FInstancedStaticMeshVertexFactory::InitInstancedStaticMeshVertexFactoryComponents(*RenderData->VertexBuffers, /*ColorVertexBuffer*/ nullptr, InstanceBuffer, &VertexFactory, LightMapCoordinateIndex, bRHISupportsManualVertexFetch, Data, InstanceData);
		VertexFactory.SetData(RHICmdList, Data, InstanceBuffer ? &InstanceData : nullptr);
		VertexFactory.InitResource(RHICmdList);
	}
}

void FInstancedStaticMeshRenderData::InitVertexFactories()
{
	// Allocate the vertex factories for each LOD
	for (int32 LODIndex = 0; LODIndex < LODModels.Num(); LODIndex++)
	{
		VertexFactories.Add(new FInstancedStaticMeshVertexFactory(FeatureLevel));
	}

	if (RayTracingProxyLODs)
	{
		for (int32 LODIndex = 0; LODIndex < RayTracingProxyLODs->Num(); LODIndex++)
		{
			RayTracingProxyVertexFactories.Add(new FInstancedStaticMeshVertexFactory(FeatureLevel));
		}
	}
}

void FInstancedStaticMeshRenderData::RegisterSpeedTreeWind(const FInstancedStaticMeshSceneProxyDesc* InProxyDesc)
{
	// register SpeedTree wind with the scene
	if (InProxyDesc->GetStaticMesh()->SpeedTreeWind.IsValid())
	{
		for (int32 LODIndex = 0; LODIndex < LODModels.Num(); LODIndex++)
		{
			if (InProxyDesc->GetScene())
			{
				InProxyDesc->GetScene()->AddSpeedTreeWind(&VertexFactories[LODIndex], InProxyDesc->GetStaticMesh());
			}
		}
	}
}

SIZE_T FInstancedStaticMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FInstancedStaticMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_InstancedStaticMeshSceneProxy_GetMeshElements);

	const bool bSelectionRenderEnabled = GIsEditor && ViewFamily.EngineShowFlags.Selection;

	bool bBatchRenderSelection = bSelectionRenderEnabled && IsSelected();
	const bool bIsWireframe = ViewFamily.EngineShowFlags.Wireframe;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

			{
				const int32 FirstLODIndex = RenderData->GetFirstValidLODIdx(RenderData->CurrentFirstLODIdx);
				const int32 LODIndex = FMath::Max(GetLOD(View), FirstLODIndex);
				const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];

				FInstancedStaticMeshVFLooseUniformShaderParametersRef LooseUniformBuffer = CreateLooseUniformBuffer(View, /*InstancedLODRange=*/0, LODIndex, EUniformBufferUsage::UniformBuffer_SingleFrame);

				for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
				{
					const int32 NumBatches = GetNumMeshBatches();

					for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
					{
						FMeshBatch& MeshElement = Collector.AllocateMesh();

						if (GetMeshElement(LODIndex, BatchIndex, SectionIndex, GetDepthPriorityGroup(View), bBatchRenderSelection, true, MeshElement))
						{
							//@todo-rco this is only supporting selection on the first element
							MeshElement.Elements[0].UserData = &UserData_AllInstances;
							MeshElement.Elements[0].bUserDataIsColorVertexBuffer = false;
							MeshElement.Elements[0].LooseParametersUniformBuffer = LooseUniformBuffer;
							MeshElement.bCanApplyViewModeOverrides = true;
							MeshElement.bUseSelectionOutline = bBatchRenderSelection;
							MeshElement.bUseWireframeSelectionColoring = bBatchRenderSelection;

							if (View->bRenderFirstInstanceOnly)
							{
								for (int32 ElementIndex = 0; ElementIndex < MeshElement.Elements.Num(); ElementIndex++)
								{
									MeshElement.Elements[ElementIndex].NumInstances = FMath::Min<uint32>(MeshElement.Elements[ElementIndex].NumInstances, 1);
								}
							}

							Collector.AddMesh(ViewIndex, MeshElement);
							INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, MeshElement.GetNumPrimitives());

							if (OverlayMaterial != nullptr)
							{
								FMeshBatch& OverlayMeshBatch = Collector.AllocateMesh();
								OverlayMeshBatch = MeshElement;
								OverlayMeshBatch.bOverlayMaterial = true;
								OverlayMeshBatch.CastShadow = false;
								OverlayMeshBatch.bSelectable = false;
								OverlayMeshBatch.MaterialRenderProxy = OverlayMaterial->GetRenderProxy();
								// make sure overlay is always rendered on top of base mesh
								OverlayMeshBatch.MeshIdInPrimitive += LODModel.Sections.Num();
								Collector.AddMesh(ViewIndex, OverlayMeshBatch);
								
								INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, OverlayMeshBatch.GetNumPrimitives());
							}
						}
					}
				}
			}
			
			// Draw the bounds
			RenderBounds(PDI, View->Family->EngineShowFlags, GetBounds(), IsSelected());

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (View->Family->EngineShowFlags.VisualizeInstanceUpdates && InstanceDataSceneProxy)
			{
				InstanceDataSceneProxy->DebugDrawInstanceChanges(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags.Game ? SDPG_World : SDPG_Foreground);
			}
#endif

#if ENABLE_DRAW_DEBUG	
			if(AllowDebugViewmodes())
			{
				if (ViewFamily.EngineShowFlags.InstancedStaticMeshes)
				{
					RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
				}
			}
#endif
		}
	}
}

static uint32 GetTypeHash(const FInstancedStaticMeshVFLooseUniformShaderParameters& Params)
{
	uint32 HashParameters = 0;
	HashParameters = HashCombine(HashParameters, GetTypeHash(Params.InstancingFadeOutParams));
	HashParameters = HashCombine(HashParameters, GetTypeHash(Params.InstanceDitherParameters.InstancingTranslatedWorldViewOriginOne));
	HashParameters = HashCombine(HashParameters, GetTypeHash(Params.InstanceDitherParameters.InstancingTranslatedWorldViewOriginZero));
	HashParameters = HashCombine(HashParameters, GetTypeHash(Params.InstanceDitherParameters.InstancingViewZCompareOne));
	HashParameters = HashCombine(HashParameters, GetTypeHash(Params.InstanceDitherParameters.InstancingViewZCompareZero));
	HashParameters = HashCombine(HashParameters, GetTypeHash(Params.InstanceDitherParameters.InstancingViewZConstant));
	return HashParameters;
}

static bool operator== (const FInstancedStaticMeshVFLooseUniformShaderParameters& A, const FInstancedStaticMeshVFLooseUniformShaderParameters& B)
{
	static_assert(sizeof(FInstancedStaticMeshVFLooseUniformShaderParameters) == 96, "FInstancedStaticMeshVFLooseUniformShaderParameters size has changed, make sure operator== is updated");
	
	return A.InstancingFadeOutParams == B.InstancingFadeOutParams
		&& A.InstanceDitherParameters.InstancingTranslatedWorldViewOriginOne == B.InstanceDitherParameters.InstancingTranslatedWorldViewOriginOne
		&& A.InstanceDitherParameters.InstancingTranslatedWorldViewOriginZero == B.InstanceDitherParameters.InstancingTranslatedWorldViewOriginZero
		&& A.InstanceDitherParameters.InstancingViewZCompareOne == B.InstanceDitherParameters.InstancingViewZCompareOne
		&& A.InstanceDitherParameters.InstancingViewZCompareZero == B.InstanceDitherParameters.InstancingViewZCompareZero
		&& A.InstanceDitherParameters.InstancingViewZConstant == B.InstanceDitherParameters.InstancingViewZConstant;
}

// For deduplicating FInstancedStaticMeshVFLooseUniformShaderParameters UBs
class FInstancedStaticMeshVFUniformsCache : public FRenderResource
{
	const static int32 MAX_CACHE_SIZE = 10;
	TMap<FInstancedStaticMeshVFLooseUniformShaderParameters, FInstancedStaticMeshVFLooseUniformShaderParametersRef> CachedParams;

public:
	FInstancedStaticMeshVFUniformsCache()
		: FRenderResource()
	{
	}
	
	virtual void ReleaseRHI() override
	{
		CachedParams.Empty();
	}

	FInstancedStaticMeshVFLooseUniformShaderParametersRef CreateUniformBuffer(const FInstancedStaticMeshVFLooseUniformShaderParameters& Params, EUniformBufferUsage UniformBufferUsage)
	{
		LLM_SCOPE(ELLMTag::InstancedMesh);

		check(IsInRenderingThread());

		FInstancedStaticMeshVFLooseUniformShaderParametersRef* CachedUB = CachedParams.Find(Params);
		if (CachedUB)
		{
			return *CachedUB;
		}

		FInstancedStaticMeshVFLooseUniformShaderParametersRef UB = CreateUniformBufferImmediate(Params, UniformBufferUsage);
		if (CachedParams.Num() < MAX_CACHE_SIZE && UniformBufferUsage == UniformBuffer_MultiFrame)
		{
			CachedParams.Add(Params, UB);
		}

		return UB;
	}
};

static TGlobalResource<FInstancedStaticMeshVFUniformsCache> GInstancedStaticMeshVFUniformsCache;

FLocalVertexFactoryInstanceDitherParameters FInstancedStaticMeshSceneProxy::GetInstanceDitherParameters(const FSceneView* View, uint32 InstancedLODRange, uint32 InstancedLODIndex) const
{
	FLocalVertexFactoryInstanceDitherParameters Result = MakeNullLocalVertexFactoryInstanceDitherParameters();

	// InstancedLODRange is only set for HierarchicalInstancedStaticMeshes
	if (InstancedLODRange)
	{
		FVector4f InstancingViewZCompareZero(MIN_flt, MIN_flt, MAX_flt, 1.0f);
		FVector4f InstancingViewZCompareOne(MIN_flt, MIN_flt, MAX_flt, 0.0f);
		FVector4f InstancingViewZConstant(ForceInit);
		FVector4f InstancingTranslatedWorldViewOriginZero(ForceInit);
		FVector4f InstancingTranslatedWorldViewOriginOne(ForceInit);
		InstancingTranslatedWorldViewOriginOne.W = 1.0f;

		check(View);
		int32 FirstLOD = UserData_AllInstances.MinLOD;

		int32 DebugMin = FMath::Min(CVarMinLOD.GetValueOnRenderThread(), UserData_AllInstances.MeshRenderData->LODResources.Num() - 1);
		if (DebugMin >= 0)
		{
			FirstLOD = FMath::Max(FirstLOD, DebugMin);
		}

		FBoxSphereBounds ScaledBounds = UserData_AllInstances.MeshRenderData->Bounds.TransformBy(FTransform(FQuat::Identity, FVector::ZeroVector, UserData_AllInstances.AverageInstancesScale));
		float SphereRadius = ScaledBounds.SphereRadius;
		float MinSize = View->ViewMatrices.IsPerspectiveProjection() ? CVarFoliageMinimumScreenSize.GetValueOnRenderThread() : 0.0f;
		float LODScale = UserData_AllInstances.LODDistanceScale;
		float LODRandom = CVarRandomLODRange.GetValueOnRenderThread();
		float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;

		if (InstancedLODIndex)
		{
			InstancingViewZConstant.X = -1.0f;
		}
		else
		{
			// this is the first LOD, so we don't have a fade-in region
			InstancingViewZConstant.X = 0.0f;
		}
		InstancingViewZConstant.Y = 0.0f;
		InstancingViewZConstant.Z = 1.0f;

		// now we subtract off the lower segments, since they will be incorporated 
		InstancingViewZConstant.Y -= InstancingViewZConstant.X;
		InstancingViewZConstant.Z -= InstancingViewZConstant.X + InstancingViewZConstant.Y;
		//not using W InstancingViewZConstant.W -= InstancingViewZConstant.X + InstancingViewZConstant.Y + InstancingViewZConstant.Z;

		for (int32 SampleIndex = 0; SampleIndex < 2; SampleIndex++)
		{
			FVector4f& InstancingViewZCompare(SampleIndex ? InstancingViewZCompareOne : InstancingViewZCompareZero);

			float FinalCull = MAX_flt;
			if (MinSize > 0.0)
			{
				FinalCull = ComputeBoundsDrawDistance(MinSize, SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
			}
			if (UserData_AllInstances.EndCullDistance > 0.0f)
			{
				FinalCull = FMath::Min(FinalCull, UserData_AllInstances.EndCullDistance * MaxDrawDistanceScale);
			}

			InstancingViewZCompare.Z = FinalCull;
			if (int(InstancedLODIndex) < UserData_AllInstances.MeshRenderData->LODResources.Num() - 1)
			{
				float NextCut = ComputeBoundsDrawDistance(UserData_AllInstances.MeshRenderData->ScreenSize[InstancedLODIndex + 1].GetValue(), SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
				InstancingViewZCompare.Z = FMath::Min(NextCut, FinalCull);
			}

			InstancingViewZCompare.X = MIN_flt;
			if (int(InstancedLODIndex) > FirstLOD)
			{
				float CurCut = ComputeBoundsDrawDistance(UserData_AllInstances.MeshRenderData->ScreenSize[InstancedLODIndex].GetValue(), SphereRadius, View->ViewMatrices.GetViewToClip()) * LODScale;
				if (CurCut < FinalCull)
				{
					InstancingViewZCompare.Y = CurCut;
				}
				else
				{
					// this LOD is completely removed by one of the other two factors
					InstancingViewZCompare.Y = MIN_flt;
					InstancingViewZCompare.Z = MIN_flt;
				}
			}
			else
			{
				// this is the first LOD, so we don't have a fade-in region
				InstancingViewZCompare.Y = MIN_flt;
			}
		}

		const FVector PreViewTranslation = View->ViewMatrices.GetPreViewTranslation();
		InstancingTranslatedWorldViewOriginZero = (FVector3f)(View->GetTemporalLODOrigin(0) + PreViewTranslation); // LWC_TODO: precision loss
		InstancingTranslatedWorldViewOriginOne = (FVector3f)(View->GetTemporalLODOrigin(1) + PreViewTranslation); // LWC_TODO: precision loss

		float Alpha = View->GetTemporalLODTransition();
		InstancingTranslatedWorldViewOriginZero.W = 1.0f - Alpha;
		InstancingTranslatedWorldViewOriginOne.W = Alpha;

		InstancingViewZCompareZero.W = LODRandom;

		Result.InstancingViewZCompareZero = InstancingViewZCompareZero;
		Result.InstancingViewZCompareOne = InstancingViewZCompareOne;
		Result.InstancingViewZCompareOne.W = 1.0f;
		Result.InstancingViewZConstant = InstancingViewZConstant;
		Result.InstancingTranslatedWorldViewOriginZero = InstancingTranslatedWorldViewOriginZero;
		Result.InstancingTranslatedWorldViewOriginOne = InstancingTranslatedWorldViewOriginOne;
	}

	return Result;
}

FVector2f FInstancedStaticMeshSceneProxy::CalcInstancingFadeOutParams() const
{
	const float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;
	const float StartDistance = UserData_AllInstances.StartCullDistance * MaxDrawDistanceScale;
	const float EndDistance = UserData_AllInstances.EndCullDistance * MaxDrawDistanceScale;

	FVector2f Result = FVector2f(StartDistance, 0.f);
	if (EndDistance > 0.0f)
	{
		if (EndDistance > StartDistance)
		{
			Result.Y = 1.f / (float)(EndDistance - StartDistance);
		}
		else
		{
			Result.Y = 1.f;
		}
	}
	return Result;
}

FInstancedStaticMeshVFLooseUniformShaderParametersRef FInstancedStaticMeshSceneProxy::CreateLooseUniformBuffer(const FSceneView* View, uint32 InstancedLODRange, uint32 InstancedLODIndex, EUniformBufferUsage UniformBufferUsage) const
{
	FInstancedStaticMeshVFLooseUniformShaderParameters LooseParameters;

	LooseParameters.InstanceDitherParameters = GetInstanceDitherParameters(View, InstancedLODRange, InstancedLODIndex);
	const FVector2f InstancingFadeOutParams = CalcInstancingFadeOutParams();
	LooseParameters.InstancingFadeOutParams = FVector4f(InstancingFadeOutParams.X, InstancingFadeOutParams.Y, 1.0f, 1.0f);

	if (IsInRenderingThread())
	{
		return GInstancedStaticMeshVFUniformsCache.CreateUniformBuffer(LooseParameters, UniformBufferUsage);
	}
	else
	{
		return CreateUniformBufferImmediate(LooseParameters, UniformBufferUsage);
	}
}

FInstancedStaticMeshSceneProxyDesc::FInstancedStaticMeshSceneProxyDesc(const UInstancedStaticMeshComponent* InComponent)
	: FInstancedStaticMeshSceneProxyDesc()
{
	InitializeFromInstancedStaticMeshComponent(InComponent);
}

void FInstancedStaticMeshSceneProxyDesc::InitializeFromInstancedStaticMeshComponent(const UInstancedStaticMeshComponent* InComponent)
{
	InitializeFromStaticMeshComponent(InComponent);

	InstanceDataSceneProxy = InComponent->GetInstanceDataSceneProxy();
	check(!GetScene() || InstanceDataSceneProxy);
#if WITH_EDITOR
	bHasSelectedInstances = InComponent->SelectedInstances.Find(true) != INDEX_NONE;
#endif

	InstanceMinDrawDistance = InComponent->InstanceMinDrawDistance;
	InstanceStartCullDistance = InComponent->InstanceStartCullDistance;
	InstanceEndCullDistance = InComponent->InstanceEndCullDistance;
	InstanceLODDistanceScale = InComponent->InstanceLODDistanceScale;
	bUseGpuLodSelection = InComponent->bUseGpuLodSelection;
}

/** Initialization constructor. */
FInstancedStaticMeshSceneProxy::FInstancedStaticMeshSceneProxy(UInstancedStaticMeshComponent* InComponent, ERHIFeatureLevel::Type InFeatureLevel)
	: FInstancedStaticMeshSceneProxy(FInstancedStaticMeshSceneProxyDesc(InComponent), InFeatureLevel)
{

}

/** Initialization constructor. */
FInstancedStaticMeshSceneProxy::FInstancedStaticMeshSceneProxy(const FInstancedStaticMeshSceneProxyDesc& InProxyDesc, ERHIFeatureLevel::Type InFeatureLevel)
	:	FStaticMeshSceneProxy(InProxyDesc, true)
	,	StaticMesh(InProxyDesc.GetStaticMesh())
#if WITH_EDITOR
	,	bHasSelectedInstances(false)
#endif
	,	InstanceLODDistanceScale(InProxyDesc.InstanceLODDistanceScale)
#if RHI_RAYTRACING
	,	CachedRayTracingLODIndex(-1)
#endif
	,	StaticMeshBounds(StaticMesh->GetBounds())
	,	InstanceDataSceneProxy(InProxyDesc.InstanceDataSceneProxy)
{
#if WITH_EDITOR
	bHasSelectedInstances = InProxyDesc.bHasSelectedInstances;
#endif
	if (DoesPlatformUseInstancedStaticMeshVertexFactory(GetScene().GetShaderPlatform()))
	{
		InstancedRenderData = MakeUnique<FInstancedStaticMeshRenderData>(&InProxyDesc, InFeatureLevel);
	}
	SetupProxy(InProxyDesc);
}

void FInstancedStaticMeshSceneProxy::SetupProxy(const FInstancedStaticMeshSceneProxyDesc& InProxyDesc)
{
#if WITH_EDITOR
	if (bHasSelectedInstances)
	{
		// if we have selected indices, mark scene proxy as selected.
		SetSelection_GameThread(true);
	}
#endif

	bDoesMeshBatchesUseSceneInstanceCount = CVarISMFetchInstanceCountFromScene.GetValueOnAnyThread() != 0;

	SetupInstanceSceneDataBuffers(InstanceDataSceneProxy->GeInstanceSceneDataBuffers());

	bAnySegmentUsesWorldPositionOffset = false;

	// Make sure all the materials are okay to be rendered as an instanced mesh.
	for (int32 LODIndex = 0; LODIndex < LODs.Num(); LODIndex++)
	{
		FStaticMeshSceneProxy::FLODInfo& LODInfo = LODs[LODIndex];
		for (int32 SectionIndex = 0; SectionIndex < LODInfo.Sections.Num(); SectionIndex++)
		{
			FStaticMeshSceneProxy::FLODInfo::FSectionInfo& Section = LODInfo.Sections[SectionIndex];
			if (!Section.Material->CheckMaterialUsage_Concurrent(MATUSAGE_InstancedStaticMeshes))
			{
				Section.Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}
			bAnySegmentUsesWorldPositionOffset |= Section.Material->IsUsingWorldPositionOffset_Concurrent(GMaxRHIShaderPlatform);
		}
	}

#if RHI_RAYTRACING
	if (bSupportRayTracing)
	{
		const bool bWantsRayTracingWPO = bEvaluateWorldPositionOffset && bAnySegmentUsesWorldPositionOffset && InProxyDesc.bEvaluateWorldPositionOffsetInRayTracing;

		if (bWantsRayTracingWPO)
		{
			// Need to use these temporary variables since compiler doesn't accept 'bitfield bool' as bool&
			bool bHasRayTracingRepresentationTmp;
			bool bDynamicRayTracingGeometryTmp;
			GetRayTracingWPOConfig(bHasRayTracingRepresentationTmp, bDynamicRayTracingGeometryTmp);

			bHasRayTracingRepresentation = bHasRayTracingRepresentationTmp;
			bDynamicRayTracingGeometry = bDynamicRayTracingGeometryTmp;
		}
		else
		{
			bHasRayTracingRepresentation = true;
		}
	}

	// By default, the base class (FStaticMeshSceneProxy) creates DynamicRayTracingGeometries which is not actually used
	// because FInstancedStaticMeshSceneProxy creates it's own ray tracing geometries for each simulated instance.
	// disable bCreateDynamicRayTracingGeometries to avoid that
	bCreateDynamicRayTracingGeometries = false;
#endif // RHI_RAYTRACING

	if (OverlayMaterial != nullptr)
	{
		if (!OverlayMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_InstancedStaticMeshes))
		{
			OverlayMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
			UE_LOGF(LogStaticMesh, Error, "Overlay material with missing usage flag was applied to instanced static mesh %ls", *InProxyDesc.GetStaticMesh()->GetPathName());
		}
	}

	// Copy the parameters for LOD - all instances
	UserData_AllInstances.MeshRenderData = InProxyDesc.GetStaticMesh()->GetRenderData();
	UserData_AllInstances.MinDrawDistance = InProxyDesc.InstanceMinDrawDistance;
	UserData_AllInstances.StartCullDistance = InProxyDesc.InstanceStartCullDistance;
	UserData_AllInstances.EndCullDistance = InProxyDesc.InstanceEndCullDistance;
	UserData_AllInstances.LODDistanceScale = 1.0f;
	UserData_AllInstances.InstancingOffset = InProxyDesc.GetStaticMesh()->GetBoundingBox().GetCenter();
	UserData_AllInstances.MinLOD = ClampedMinLOD;
	UserData_AllInstances.RenderData = nullptr;

	// Only used by HISM, and thus set in the descendant ctor
	UserData_AllInstances.AverageInstancesScale = FVector::Zero();

	const bool bUseGPUScene = UseGPUScene(GetScene().GetShaderPlatform());
	
	const bool bEnableGpuLodSelection = CVarGpuLodSelection.GetValueOnAnyThread() != 0;
	bUseGpuLodSelection = InProxyDesc.bUseGpuLodSelection && bUseGPUScene && bEnableGpuLodSelection;

	if (bUseGPUScene)
	{
		if (PlatformGPUSceneUsesUniformBufferView(GetScene().GetShaderPlatform()))
		{
			// Only instance data comes from GPUScene on platforms that use uniform buffer views
			bVFRequiresPrimitiveUniformBuffer = true;
			for (int32 LODIdx = 0; LODIdx < LODs.Num(); ++LODIdx)
			{
				LODs[LODIdx].bCanUsePrecomputedLightingParametersFromGPUScene = false;
			}
		}
	}
}

bool FInstancedStaticMeshSceneProxy::CanSetupInstancedMeshBatch(int32 LODIndex) const
{
	if (InstancedRenderData)
	{
		// TODO: If this fires somehow our LODLooseUniformBuffers created in CreateRenderThreadResources is empty, a full dmp should be investigated
		check(LODLooseUniformBuffers.Num() == GetNumRenderDataVertexFactories());
		return LODLooseUniformBuffers.Contains(LODIndex) && LODIndex < GetNumRenderDataVertexFactories();
	}
	return LODIndex < GetNumRenderDataVertexFactories();
}

bool FInstancedStaticMeshSceneProxy::CanSetupRayTracingInstancedMeshBatch(int32 LODIndex) const
{
	if (InstancedRenderData)
	{
		return LODIndex < InstancedRenderData->RayTracingProxyVertexFactories.Num();
	}
	else
	{
		return LODIndex < RenderData->RayTracingProxy->LODVertexFactories->Num();
	}
}

void FInstancedStaticMeshSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	check(InstanceDataSceneProxy);
	FStaticMeshSceneProxy::CreateRenderThreadResources(RHICmdList);

	if (InstancedRenderData)
	{
		InstancedRenderData->BindBuffersToVertexFactories(RHICmdList, InstanceDataSceneProxy->GetLegacyInstanceBuffer());
		for (int32 i = 0; i < LODs.Num(); ++i)
		{
			FInstancedStaticMeshVFLooseUniformShaderParametersRef LooseUniformBuffer = CreateLooseUniformBuffer(nullptr, 0, i, EUniformBufferUsage::UniformBuffer_MultiFrame);
			LODLooseUniformBuffers.Add(i, LooseUniformBuffer);
		}
	}	
	// currently we setup loose uniform buffer for ray tracing based on InstancedLODIndex = 0
	// since currently non of the uniforms depend on the LOD index when InstancedLODRange = 0
	RayTracingLooseUniformBuffer = CreateLooseUniformBuffer(nullptr, /*InstancedLODRange*/ 0, /*InstancedLODIndex*/ 0, EUniformBufferUsage::UniformBuffer_MultiFrame);
}

void FInstancedStaticMeshSceneProxy::DestroyRenderThreadResources()
{
	if (InstancedRenderData)
	{
		InstancedRenderData->ReleaseResources(&GetScene(), StaticMesh);
	}
	FStaticMeshSceneProxy::DestroyRenderThreadResources();

#if RHI_RAYTRACING
	ReleaseRayTracingDynamicInstances();
#endif
}

void FInstancedStaticMeshSceneProxy::UpdateInstances_RenderThread(FRHICommandListBase& RHICmdList, const FBoxSphereBounds& InBounds, const FBoxSphereBounds& InLocalBounds)
{
	if (InstancedRenderData)
	{
		if (FStaticMeshInstanceBuffer *LegacyInstanceBuffer = InstanceDataSceneProxy->GetLegacyInstanceBuffer())
		{
			InstancedRenderData->BindBuffersToVertexFactories(RHICmdList, LegacyInstanceBuffer);
		}
	}
	return FPrimitiveSceneProxy::UpdateInstances_RenderThread(RHICmdList, InBounds, InLocalBounds);
}

void FInstancedStaticMeshSceneProxy::SetupInstancedMeshBatch(int32 LODIndex, int32 BatchIndex, FMeshBatch& OutMeshBatch) const
{
	FMeshBatchElement& BatchElement0 = OutMeshBatch.Elements[0];
	if (InstancedRenderData)
	{
		OutMeshBatch.VertexFactory = &InstancedRenderData->VertexFactories[LODIndex];
		BatchElement0.LooseParametersUniformBuffer = LODLooseUniformBuffers[LODIndex];
	}
	BatchElement0.UserData = (void*)&UserData_AllInstances;
	BatchElement0.bUserDataIsColorVertexBuffer = false;
	BatchElement0.InstancedLODIndex = LODIndex;
	BatchElement0.UserIndex = 0;
	BatchElement0.PrimitiveUniformBuffer = GetUniformBuffer();
	BatchElement0.bForceInstanceCulling = true; // force ISM through Generic path even for a single instance cases
	
	if (OutMeshBatch.MaterialRenderProxy)
	{
		// If the material on the mesh batch is translucent, then preserve the instance draw order to prevent flickering
		// TODO: This should use depth sorting of instances instead, once that is implemented for GPU Scene instances
		const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
		
		// NOTE: For now, this feature is not supported for mobile platforms
		if (FeatureLevel > ERHIFeatureLevel::ES3_1)
		{
			const FMaterial& Material = OutMeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
			BatchElement0.bPreserveInstanceOrder = IsTranslucentBlendMode(Material);
		}
	}
	BatchElement0.bFetchInstanceCountFromScene = CVarISMFetchInstanceCountFromScene.GetValueOnRenderThread() != 0;
	BatchElement0.NumInstances = GetInstanceDataHeader().NumInstances;
}

void FInstancedStaticMeshSceneProxy::SetupRayTracingInstancedMeshBatch(int32 LODIndex, int32 BatchIndex, FMeshBatch& OutMeshBatch) const
{
	if (InstancedRenderData)
	{
		OutMeshBatch.VertexFactory = &InstancedRenderData->RayTracingProxyVertexFactories[LODIndex];
	}
	FMeshBatchElement& BatchElement0 = OutMeshBatch.Elements[0];
	BatchElement0.UserData = (void*)&UserData_AllInstances;
	BatchElement0.bUserDataIsColorVertexBuffer = false;
	BatchElement0.InstancedLODIndex = LODIndex;
	BatchElement0.UserIndex = 0;
	BatchElement0.PrimitiveUniformBuffer = GetUniformBuffer();
	BatchElement0.LooseParametersUniformBuffer = RayTracingLooseUniformBuffer;
	BatchElement0.bForceInstanceCulling = true; // force ISM through Generic path even for a single instance cases

	if (OutMeshBatch.MaterialRenderProxy)
	{
		// If the material on the mesh batch is translucent, then preserve the instance draw order to prevent flickering
		// TODO: This should use depth sorting of instances instead, once that is implemented for GPU Scene instances
		const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();

		// NOTE: For now, this feature is not supported for mobile platforms
		if (FeatureLevel > ERHIFeatureLevel::ES3_1)
		{
			const FMaterial& Material = OutMeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
			BatchElement0.bPreserveInstanceOrder = IsTranslucentBlendMode(Material);
		}
	}
	BatchElement0.bFetchInstanceCountFromScene = CVarISMFetchInstanceCountFromScene.GetValueOnRenderThread() != 0;
	BatchElement0.NumInstances = GetInstanceDataHeader().NumInstances;
}

void FInstancedStaticMeshSceneProxy::GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const
{
	FStaticMeshSceneProxy::GetLightRelevance(LightSceneProxy, bDynamic, bRelevant, bLightMapped, bShadowMapped);

	if (GetInstanceDataHeader().NumInstances == 0)
	{
		bRelevant = false;
	}
}

bool FInstancedStaticMeshSceneProxy::GetShadowMeshElement(int32 LODIndex, int32 BatchIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch, bool bDitheredLODTransition) const
{
	if (CanSetupInstancedMeshBatch(LODIndex) && FStaticMeshSceneProxy::GetShadowMeshElement(LODIndex, BatchIndex, InDepthPriorityGroup, OutMeshBatch, bDitheredLODTransition))
	{
		SetupInstancedMeshBatch(LODIndex, BatchIndex, OutMeshBatch);
		return true;
	}
	return false;
}

/** Sets up a FMeshBatch for a specific LOD and element. */
bool FInstancedStaticMeshSceneProxy::GetMeshElement(int32 LODIndex, int32 BatchIndex, int32 ElementIndex, uint8 InDepthPriorityGroup, bool bUseSelectionOutline, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const
{
	if (CanSetupInstancedMeshBatch(LODIndex) && FStaticMeshSceneProxy::GetMeshElement(LODIndex, BatchIndex, ElementIndex, InDepthPriorityGroup, bUseSelectionOutline, bAllowPreCulledIndices, OutMeshBatch))
	{
		SetupInstancedMeshBatch(LODIndex, BatchIndex, OutMeshBatch);
		return true;
	}
	return false;
};

/** Sets up a wireframe FMeshBatch for a specific LOD. */
bool FInstancedStaticMeshSceneProxy::GetWireframeMeshElement(int32 LODIndex, int32 BatchIndex, const FMaterialRenderProxy* WireframeRenderProxy, uint8 InDepthPriorityGroup, bool bAllowPreCulledIndices, FMeshBatch& OutMeshBatch) const
{
	if (CanSetupInstancedMeshBatch(LODIndex) && FStaticMeshSceneProxy::GetWireframeMeshElement(LODIndex, BatchIndex, WireframeRenderProxy, InDepthPriorityGroup, bAllowPreCulledIndices, OutMeshBatch))
	{
		SetupInstancedMeshBatch(LODIndex, BatchIndex, OutMeshBatch);
		return true;
	}
	return false;
}

bool FInstancedStaticMeshSceneProxy::GetRayTracingMeshElement(int32 LODIndex, int32 BatchIndex, int32 SectionIndex, uint8 InDepthPriorityGroup, FMeshBatch& OutMeshBatch) const
{
	if (CanSetupRayTracingInstancedMeshBatch(LODIndex) && FStaticMeshSceneProxy::GetRayTracingMeshElement(LODIndex, BatchIndex, SectionIndex, InDepthPriorityGroup, OutMeshBatch))
	{
		SetupRayTracingInstancedMeshBatch(LODIndex, BatchIndex, OutMeshBatch);
		return true;
	}
	return false;
}

void FInstancedStaticMeshSceneProxy::GetDistanceFieldAtlasData(const FDistanceFieldVolumeData*& OutDistanceFieldData, float& SelfShadowBias) const
{
	FStaticMeshSceneProxy::GetDistanceFieldAtlasData(OutDistanceFieldData, SelfShadowBias);
}

HHitProxy* FInstancedStaticMeshSceneProxy::CreateHitProxies(UPrimitiveComponent* Component,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	return FInstancedStaticMeshSceneProxy::CreateHitProxies(Component->GetPrimitiveComponentInterface(), OutHitProxies);
}

HHitProxy* FInstancedStaticMeshSceneProxy::CreateHitProxies(IPrimitiveComponent* Component,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	if (HasPerInstanceHitProxies())
	{
		// Note: the instance data proxy handles the hitproxy lifetimes internally as the update cadence does not match FPrimitiveSceneInfo ctor cadence
		return nullptr;
	}
	return FStaticMeshSceneProxy::CreateHitProxies(Component, OutHitProxies);
}

bool FInstancedStaticMeshSceneProxy::GetInstanceDrawDistanceMinMax(FVector2f& OutDistanceMinMax) const
{
	if (UserData_AllInstances.EndCullDistance > 0)
	{
		OutDistanceMinMax = FVector2f(float(UserData_AllInstances.MinDrawDistance), float(UserData_AllInstances.EndCullDistance));
		return true;
	}
	else
	{
		OutDistanceMinMax = FVector2f(0.0f);
		return false;
	}
}

bool FInstancedStaticMeshSceneProxy::GetInstancingFadeOutParams(FVector2f& OutInstancingFadeOutParams) const
{
	if (UserData_AllInstances.StartCullDistance > 0)
	{
		OutInstancingFadeOutParams = CalcInstancingFadeOutParams();;
		return true;
	}
	return false;
}

float FInstancedStaticMeshSceneProxy::GetLodScreenSizeScale() const
{
	return InstanceLODDistanceScale > 0.f ? 1.f / InstanceLODDistanceScale : 1.f;
}

float FInstancedStaticMeshSceneProxy::GetGpuLodInstanceRadius() const
{
	// Note that StaticMeshBounds.SphereRadius is a better fit, but doesn't match the value on the GPU used for LOD culling.
	// That's because GPUScene drops the bounds sphere radius and uses the box. So we end up with the sphere encompassing the box, encompassing the sphere :(
	return bUseGpuLodSelection ? StaticMeshBounds.BoxExtent.Length() : 0.f;
}

FInstanceDataUpdateTaskInfo *FInstancedStaticMeshSceneProxy::GetInstanceDataUpdateTaskInfo() const
{
	return InstanceDataSceneProxy->GetUpdateTaskInfo();
}

void FInstancedStaticMeshSceneProxy::SetEvaluateWorldPositionOffsetInRayTracing(FRHICommandListBase& RHICmdList, bool bEvaluate)
{
#if RHI_RAYTRACING
	if (!bSupportRayTracing)
	{
		return;
	}

	bool bPrevDynamicRayTracingGeometry = bDynamicRayTracingGeometry;

	FStaticMeshSceneProxy::SetEvaluateWorldPositionOffsetInRayTracing(RHICmdList, bEvaluate);

	if (bPrevDynamicRayTracingGeometry && !bDynamicRayTracingGeometry)
	{
		ReleaseRayTracingDynamicInstances();
	}

	GetScene().UpdateCachedRayTracingState(this);
#endif // RHI_RAYTRACING
}

int32 FInstancedStaticMeshSceneProxy::GetNumRenderDataVertexFactories() const
{
	if (InstancedRenderData)
	{
		return InstancedRenderData->VertexFactories.Num();
	}
	else
	{
		return RenderData->LODVertexFactories.Num();
	}
}

#if RHI_RAYTRACING
void FInstancedStaticMeshSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	// Workaround: SetEvaluateWorldPositionOffsetInRayTracing(...) calls UpdateCachedRayTracingState(...)
	// however the update only happens after gathering relevant ray tracing primitives
	// so ERayTracingPrimitiveFlags::Dynamic is set for one frame after the WPO evaluation is disabled.
	if (!bDynamicRayTracingInstances && !bDynamicRayTracingGeometry)
	{
		return;
	}

	if (!CVarRayTracingRenderInstances.GetValueOnRenderThread())
	{
		// TODO: Exclude proxy during ray tracing primitive gather instead of doing this early out here.
		return;
	}

#if DO_CHECK
	if (!ensureMsgf(bSupportRayTracing, TEXT("If proxy doesn't support ray tracing it should've been excluded earlier.")))
	{
		return;
	}
#endif

	const FInstanceSceneDataBuffers* InstanceSceneDataBuffers = GetInstanceSceneDataBuffers();

	checkf(!InstanceSceneDataBuffers->IsInstanceDataGPUOnly(), TEXT("FInstancedStaticMeshSceneProxy does not support dynamic ray tracing geometry when IsInstanceDataGPUOnly()."));

	const uint32 InstanceCount = GetInstanceDataHeader().NumInstances;

	if (InstanceCount == 0u)
	{
		return;
	}

	check(InstanceSceneDataBuffers && InstanceSceneDataBuffers->GetNumInstances() == InstanceCount);

	// TODO: Should only do this if any instance uses static geometry
	Collector.AddReferencedGeometryGroup(RenderData->RayTracingGeometryGroupHandle);

	const int32 MinAllowedLODIndex = FMath::Clamp<int32>(CVarRayTracingInstancedStaticMeshesMinLOD.GetValueOnRenderThread(), 0, RenderData->LODResources.Num() - 1);

	const int32 RayTracingMinLOD = RenderData->RayTracingProxy->bUsingRenderingLODs ? FMath::Max(MinAllowedLODIndex, (int32)GetCurrentFirstLODIdx_RenderThread()) : 0;

	int32 LODIndex = RayTracingMinLOD;

	FStaticMeshRayTracingProxyLODArray& RayTracingLODs = RenderData->RayTracingProxy->LODs;

	FRayTracingGeometry* RayTracingGeometry = nullptr;

	// Select first LOD with valid ray tracing geometry
	// TODO: Should only do this if any instance uses static geometry
	for (; LODIndex < RayTracingLODs.Num(); ++LODIndex)
	{
		FRayTracingGeometry& CurrentRayTracingGeometry = *RayTracingLODs[LODIndex].RayTracingGeometry;

		if (CurrentRayTracingGeometry.HasPendingBuildRequest())
		{
			CurrentRayTracingGeometry.BoostBuildPriority();
		}
		else if (CurrentRayTracingGeometry.IsValid() && !CurrentRayTracingGeometry.IsEvicted())
		{
			RayTracingGeometry = &CurrentRayTracingGeometry;
			break;
		}
	}

	if (RayTracingGeometry == nullptr)
	{
		// TODO: Should only do this if any instance uses static geometry
		return;
	}

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	struct FVisibleInstance
	{
		uint32 InstanceIndex = 0;
		float DistanceToView = 0;
	};

	TArray<FMeshBatch>& RayTracingMeshBatches = Collector.AllocateMeshBatchArray();

	{
		int32 SectionCount = 0;
		if (InstancedRenderData)
		{
			SectionCount = (*InstancedRenderData->RayTracingProxyLODs)[LODIndex].Sections->Num();
		}
		else
		{
			SectionCount = RenderData->RayTracingProxy->LODs[LODIndex].Sections->Num();
		}

		for (int32 SectionIdx = 0; SectionIdx < SectionCount; ++SectionIdx)
		{
			FMeshBatch MeshBatch;

			bool bResult;
			if (RenderData->RayTracingProxy->bUsingRenderingLODs)
			{
				// when using rendering LODs we can reuse the main GetMeshElement(...)
				bResult = GetMeshElement(LODIndex, 0, SectionIdx, 0, false, false, MeshBatch);
			}
			else
			{
				// otherwise initialize MeshBatch using ray tracing proxy VB/IB/Section data
				bResult = GetRayTracingMeshElement(LODIndex, 0, SectionIdx, 0, MeshBatch);
			}

			if (bResult)
			{
				if (InstancedRenderData)
				{
					auto VF = RenderData->RayTracingProxy->bUsingRenderingLODs ? &InstancedRenderData->VertexFactories[LODIndex] : &InstancedRenderData->RayTracingProxyVertexFactories[LODIndex];
					check(MeshBatch.VertexFactory == VF);
				}
				else
				{
					auto VF = RenderData->RayTracingProxy->bUsingRenderingLODs ? &RenderData->LODVertexFactories[LODIndex].VertexFactory : &(*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex].VertexFactory;
					check(MeshBatch.VertexFactory == VF);
				}

				MeshBatch.ReverseCulling = bReverseCulling;

				RayTracingMeshBatches.Add(MeshBatch);
			}
		}
	}

	// Which index holds the reference to the particular simulated instance
	TArray<uint32> ActiveInstances;

	// Visible instances
	TArray<FVisibleInstance> VisibleInstances;
	TArray<FRayTracingInstance> RayTracingInstances;

	const uint32 RequestedSimulatedInstances = CVarRayTracingSimulatedInstanceCount.GetValueOnRenderThread();
	const uint32 SimulatedInstances = FMath::Min(RequestedSimulatedInstances == -1 ? InstanceCount : FMath::Clamp(RequestedSimulatedInstances, 1u, InstanceCount), MaxSimulatedInstances);

	if (bDynamicRayTracingGeometry)
	{
		SetupRayTracingDynamicInstances(Collector.GetRHICommandList(), SimulatedInstances, LODIndex);

		ActiveInstances.AddZeroed(SimulatedInstances);

		for (auto& Instance : ActiveInstances)
		{
			Instance = INDEX_NONE;
		}

		RayTracingInstances.Reserve(SimulatedInstances);
	}

	VisibleInstances.Reserve(InstanceCount);

	const FBox CurrentBounds = StaticMeshBounds.GetBox();

	constexpr float LocalToWorldScale = 1.0f;

	TArray<FVector, TInlineAllocator<2, SceneRenderingAllocator>> ViewPositions;
	ViewPositions.Reserve(Views.Num());
	
	for (const FSceneView* View : Views)
	{
		ViewPositions.Add(View->ViewLocation);
	}

	auto GetDistanceToInstance = [&ViewPositions, InstanceSceneDataBuffers](int32 InstanceIndex, float& OutInstanceRadius, float& OutDistanceToInstanceCenter, float& OutDistanceToInstanceStart)
	{
		const FBoxSphereBounds& InstanceWorldBounds = InstanceSceneDataBuffers->GetInstanceWorldBounds(InstanceIndex);

		OutDistanceToInstanceCenter = std::numeric_limits<float>::infinity();

		for (const FVector& ViewPosition : ViewPositions)
		{
			const FVector VToInstanceCenter = ViewPosition - InstanceWorldBounds.Origin;

			OutDistanceToInstanceCenter = FMath::Min(OutDistanceToInstanceCenter, float(VToInstanceCenter.Size()));
		}

		OutInstanceRadius = float(InstanceWorldBounds.SphereRadius);
		OutDistanceToInstanceStart = FMath::Abs(OutDistanceToInstanceCenter - OutInstanceRadius);
	};

	if (CVarRayTracingRenderInstancesCulling.GetValueOnRenderThread() > 0)
	{
		// whether to use angular culling instead of distance, angle is halved as it is compared against the projection of the radius rather than the diameter
		const float CullAngle = FMath::Min(CVarRayTracingInstancesCullAngle.GetValueOnRenderThread(), 179.9f) * 0.5f;
		const float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;

		const float UserNearCullDistance = UserData_AllInstances.MinDrawDistance * MaxDrawDistanceScale;
		const float UserFarCullDistance = UserData_AllInstances.EndCullDistance * MaxDrawDistanceScale;
		if (CullAngle < 0.0f || UserFarCullDistance > 0.0f)
		{
			//
			//  Distance based culling
			//    Check nodes for being within minimum distances
			//
			const float Inf = std::numeric_limits<float>::infinity();

			const float BVHFarCullRadius = FMath::Min(UserFarCullDistance > 0 ? UserFarCullDistance : Inf, CVarRayTracingInstancesCullClusterRadius.GetValueOnRenderThread());
			const float BVHLowScaleThreshold = CVarRayTracingInstancesLowScaleThreshold.GetValueOnRenderThread();
			const float BVHLowScaleRadius = CVarRayTracingInstancesLowScaleCullRadius.GetValueOnRenderThread();
			const bool ApplyGeneralCulling = BVHFarCullRadius > 0.0f;
			const bool ApplyLowScaleCulling = BVHLowScaleThreshold > 0.0f && BVHLowScaleRadius > 0.0f;

			for (uint32 InstanceIndex = 0; InstanceIndex < InstanceCount; InstanceIndex++)
			{
				float InstanceRadius, DistanceToInstanceCenter, DistanceToInstanceStart;
				GetDistanceToInstance(InstanceIndex, InstanceRadius, DistanceToInstanceCenter, DistanceToInstanceStart);

				// Cull instance based on distance
				if (((DistanceToInstanceCenter < UserNearCullDistance) || (DistanceToInstanceCenter > BVHFarCullRadius)) && ApplyGeneralCulling)
					continue;

				// Special culling for small scale objects
				if (InstanceRadius < BVHLowScaleThreshold && ApplyLowScaleCulling)
				{
					if (DistanceToInstanceStart > BVHLowScaleRadius)
						continue;
				}

				VisibleInstances.Add(FVisibleInstance{ InstanceIndex, DistanceToInstanceStart });
			}
		}
		else
		{
			//
			// Angle based culling
			//  Instead of culling objects based on distance check the radius of bounding sphere against a minimum culling angle
			//  This ensures objects essentially cull based on size as seen from viewer rather than distance. Provides much less
			//  popping for the same number of instances
			//
			float Ratio = FMath::Tan(CullAngle / 360.0f * 2.0f * UE_PI);

			for (uint32 InstanceIndex = 0; InstanceIndex < InstanceCount; InstanceIndex++)
			{
				float InstanceRadius, DistanceToInstanceCenter, DistanceToInstanceStart;
				GetDistanceToInstance(InstanceIndex, InstanceRadius, DistanceToInstanceCenter, DistanceToInstanceStart);

				if (DistanceToInstanceCenter * Ratio <= InstanceRadius * LocalToWorldScale)
				{
					VisibleInstances.Add(FVisibleInstance{ InstanceIndex, DistanceToInstanceStart });
				}
			}
		}
	}
	else
	{
		// No culling
		for (uint32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
		{
			VisibleInstances.Add(FVisibleInstance{ InstanceIndex, -1.0f });
		}
	}

	// Bucket which visible instances to simulate based on camera distance
	if (bDynamicRayTracingGeometry && VisibleInstances.Num() > 0)
	{
		float SimulationClusterRadius = CVarRayTracingInstancesSimulationClusterRadius.GetValueOnRenderThread();

		if (SimulationClusterRadius > 0.0)
		{
			if (SimulatedInstances < (uint32)VisibleInstances.Num())
			{
				// In no culling case, we are missing distance to view, so fill it in now
				if (CVarRayTracingRenderInstancesCulling.GetValueOnRenderThread() == 0)
				{
					for (uint32 InstanceIndex = 0; InstanceIndex < (uint32)VisibleInstances.Num(); InstanceIndex++)
					{
						float InstanceRadius, DistanceToInstanceCenter, DistanceToInstanceStart;
						GetDistanceToInstance(InstanceIndex, InstanceRadius, DistanceToInstanceCenter, DistanceToInstanceStart);

						VisibleInstances[InstanceIndex].DistanceToView = DistanceToInstanceStart;
					}
				}

				uint32 PartitionIndex = 0;

				// Let's partition the array so the closest elements are the ones we want to simulate
				for (uint32 InstanceIndex = (uint32)VisibleInstances.Num() - 1; PartitionIndex < InstanceIndex; )
				{
					if (VisibleInstances[InstanceIndex].DistanceToView > SimulationClusterRadius)
					{
						InstanceIndex--;
					}
					else
					{
						Swap(VisibleInstances[PartitionIndex], VisibleInstances[InstanceIndex]);

						PartitionIndex++;
					}
				}

				// And partial sort those that fall within this bucket
				Algo::SortBy(MakeArrayView(VisibleInstances.GetData(), PartitionIndex), &FVisibleInstance::DistanceToView);
			}
		}
	}

	FRayTracingInstance StaticRayTracingInstance;
	StaticRayTracingInstance.Geometry = RayTracingGeometry;
	StaticRayTracingInstance.MaterialsView = RayTracingMeshBatches;
	//preallocate the worst-case to prevent an explosion of reallocs
	//#dxr_todo: possibly track used instances and reserve based on previous behavior
	StaticRayTracingInstance.PrimitiveInstanceIndices.Reserve(InstanceCount);

	// Add all visible instances
	for (FVisibleInstance VisibleInstance : VisibleInstances)
	{
		const uint32 InstanceIndex = VisibleInstance.InstanceIndex;

		FMatrix InstanceToWorld = InstanceSceneDataBuffers->GetInstanceToWorld(InstanceIndex);
		const uint32 DynamicInstanceIdx = InstanceIndex % SimulatedInstances;

		const bool bSupportsRayTracingDynamicGeometry = InstancedRenderData 
			? InstancedRenderData->RayTracingProxyVertexFactories[LODIndex].GetType()->SupportsRayTracingDynamicGeometry()
			: (*RenderData->RayTracingProxy->LODVertexFactories)[LODIndex].VertexFactory.GetType()->SupportsRayTracingDynamicGeometry();
		if (bDynamicRayTracingGeometry && bSupportsRayTracingDynamicGeometry)
		{
			FRayTracingInstance* DynamicInstance = nullptr;

			if (ActiveInstances[DynamicInstanceIdx] == INDEX_NONE)
			{
				// first case of this dynamic instance, setup the material and add it
				const FStaticMeshRayTracingProxyLOD& RayTracingLOD = RenderData->RayTracingProxy->LODs[LODIndex];
				const uint32 NumVertices = RayTracingLOD.VertexBuffers->PositionVertexBuffer.GetNumVertices();

				FRayTracingDynamicInstanceData& DynamicData = RayTracingDynamicInstanceData[DynamicInstanceIdx];

				ActiveInstances[DynamicInstanceIdx] = RayTracingInstances.Num();
				FRayTracingInstance& RayTracingInstance = RayTracingInstances.AddDefaulted_GetRef();
				RayTracingInstance.Geometry = &DynamicData.Geometry;
				RayTracingInstance.MaterialsView = RayTracingMeshBatches;
				RayTracingInstance.PrimitiveInstanceIndices.Reserve(InstanceCount);

				DynamicInstance = &RayTracingInstance;

				FRayTracingDynamicGeometryUpdateParams UpdateParams;
				UpdateParams.MeshBatchesView = RayTracingMeshBatches;
				UpdateParams.bUsingIndirectDraw = false;
				UpdateParams.NumVertices = NumVertices;
				UpdateParams.VertexBufferSize = uint32((SIZE_T)NumVertices * sizeof(FVector3f));
				UpdateParams.NumTriangles = DynamicData.Geometry.GetInitializer().TotalPrimitiveCount;
				UpdateParams.Geometry = &DynamicData.Geometry;
				UpdateParams.Buffer = nullptr;
				UpdateParams.bApplyWorldPositionOffset = true;
				UpdateParams.InstanceId = InstanceIndex;
				UpdateParams.WorldToInstance = (FMatrix44f) InstanceToWorld.Inverse();

				Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
			}
			else
			{
				DynamicInstance = &RayTracingInstances[ActiveInstances[DynamicInstanceIdx]];
			}

			DynamicInstance->PrimitiveInstanceIndices.Add(InstanceIndex);

		}
		else
		{
			StaticRayTracingInstance.PrimitiveInstanceIndices.Add(InstanceIndex);
		}
	}

	if (StaticRayTracingInstance.PrimitiveInstanceIndices.Num() > 0)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			Collector.AddRayTracingInstance(ViewIndex, StaticRayTracingInstance);
		}
	}

	for (const FRayTracingInstance& Instance : RayTracingInstances)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			Collector.AddRayTracingInstance(ViewIndex, Instance);
		}
	}
}

void FInstancedStaticMeshSceneProxy::SetupRayTracingDynamicInstances(FRHICommandListBase& RHICmdList, int32 NumDynamicInstances, int32 LODIndex)
{
	if (NumDynamicInstances == RayTracingDynamicInstanceData.Num() && LODIndex == CachedRayTracingLODIndex)
	{
		return;
	}

	CachedRayTracingLODIndex = LODIndex;

	// if either NumDynamicInstances or LOD changed
	// need to recreate RayTracingDynamicData array
	// FRayTracingGeometry is no relocate-able so can't grow or shrink array
	// TODO: Investigate manually re-registering with FRayTracingGeometryManager, using sparse array or other alternative data structure instead

	// release entries

	for (FRayTracingDynamicInstanceData& DynamicData : RayTracingDynamicInstanceData)
	{
		DynamicData.Geometry.ReleaseResource();
		DynamicData.VertexBuffer.Release();
	}

	// clear and resize
	RayTracingDynamicInstanceData.Empty(NumDynamicInstances);

	// create new geometries

	FStaticMeshRayTracingProxyLOD& RayTracingLOD = RenderData->RayTracingProxy->LODs[LODIndex];

	for (int32 Item = 0; Item < NumDynamicInstances; Item++)
	{
		FRayTracingDynamicInstanceData& DynamicData = RayTracingDynamicInstanceData.AddDefaulted_GetRef();

		FRayTracingGeometryInitializer Initializer = RayTracingLOD.RayTracingGeometry->GetInitializer();
		for (FRayTracingGeometrySegment& Segment : Initializer.Segments)
		{
			Segment.VertexBuffer = nullptr;
		}
		Initializer.bAllowUpdate = true;
		Initializer.bFastBuild = true;

		DynamicData.Geometry.SetInitializer(MoveTemp(Initializer));
		DynamicData.Geometry.InitResource(RHICmdList);
	}
}

void FInstancedStaticMeshSceneProxy::ReleaseRayTracingDynamicInstances()
{
	for (auto& DynamicRayTracingItem : RayTracingDynamicInstanceData)
	{
		DynamicRayTracingItem.Geometry.ReleaseResource();
		DynamicRayTracingItem.VertexBuffer.Release();
	}

	RayTracingDynamicInstanceData.Empty();
}

void FInstancedStaticMeshSceneProxy::GetRayTracingWPOConfig(bool& bOutHasRayTracingRepresentation, bool& bOutDynamicRayTracingGeometry) const
{
	FStaticMeshSceneProxy::GetRayTracingWPOConfig(bOutHasRayTracingRepresentation, bOutDynamicRayTracingGeometry);

	const int32 WPOEvalMode = CVarRayTracingInstancesEvaluateWPO.GetValueOnAnyThread();

	bOutDynamicRayTracingGeometry = WPOEvalMode < 0 ? bOutDynamicRayTracingGeometry : WPOEvalMode != 0;
	bOutDynamicRayTracingGeometry &= bAnySegmentUsesWorldPositionOffset;

	const FInstanceSceneDataBuffers* InstanceSceneDataBuffers = GetInstanceSceneDataBuffers();

	if (InstanceSceneDataBuffers->IsInstanceDataGPUOnly())
	{
		UE_LOGF(LogStaticMesh, Warning, "Dynamic ray tracing geometry not currently supported for GPU instanced static meshes (%ls). Falling back to static ray tracing geometry.", *StaticMesh->GetPathName());

		// TODO: Update FInstancedStaticMeshSceneProxy to make it compatible with GPU Scene when using WPO materials.
		bOutDynamicRayTracingGeometry = false;
	}
}

#endif // RHI_RAYTRACING

void FInstancedStaticMeshSceneProxy::SetInstanceCullDistance_RenderThread(float StartCullDistance, float EndCullDistance)
{
	UserData_AllInstances.StartCullDistance = StartCullDistance;
	UserData_AllInstances.EndCullDistance = EndCullDistance;

	if (InstancedRenderData)
	{
		for (int32 i = 0; i < LODs.Num(); ++i)
		{
			FInstancedStaticMeshVFLooseUniformShaderParametersRef LooseUniformBuffer = CreateLooseUniformBuffer(nullptr, 0, i, EUniformBufferUsage::UniformBuffer_MultiFrame);
			LODLooseUniformBuffers.Add(i, LooseUniformBuffer);
		}
	}
	// currently we setup loose uniform buffer for ray tracing based on InstancedLODIndex = 0
	// since currently non of the uniforms depend on the LOD index when InstancedLODRange = 0
	RayTracingLooseUniformBuffer = CreateLooseUniformBuffer(nullptr, /*InstancedLODRange*/ 0, /*InstancedLODIndex*/ 0, EUniformBufferUsage::UniformBuffer_MultiFrame);
}

void FInstancedStaticMeshSceneProxy::GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const
{
	OutBounds = StaticMeshBounds;
}

/*-----------------------------------------------------------------------------
	UInstancedStaticMeshComponent
-----------------------------------------------------------------------------*/

UInstancedStaticMeshComponent::UInstancedStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsInstanceDataApplyCompleted(true)
	, PrimitiveInstanceDataManager(this)
	, InstancePhysicsBodies(MakeUnique<FInstancedMeshComponentBodies>(this))
{
	Mobility = EComponentMobility::Movable;
	BodyInstance.bSimulatePhysics = false;

	// Can't mesh paint on instanced mesh.
	bSupportMeshPainting = false;
	bEnableVertexColorMeshPainting = false;
	bEnableTextureColorMeshPainting = false;

	bMultiBodyOverlap = true;

	bUseGpuLodSelection = true;
	bInheritPerInstanceData = false;
	bIsUpdatingBounds = false;
#if WITH_EDITORONLY_DATA
	bWalkableSlopeOverrideCachedGT = false;
#endif

#if STATS
	{
		UObject const* StatObject = this->AdditionalStatObject();
		if (!StatObject)
		{
			StatObject = this;
		}
		StatId = StatObject->GetStatID(true);
	}
#endif
}

UInstancedStaticMeshComponent::UInstancedStaticMeshComponent(FVTableHelper& Helper)
	: Super(Helper)
	, bIsInstanceDataApplyCompleted(true)
	, PrimitiveInstanceDataManager(this)
	, InstancePhysicsBodies(MakeUnique<FInstancedMeshComponentBodies>(this))
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UInstancedStaticMeshComponent::~UInstancedStaticMeshComponent()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TStructOnScope<FActorComponentInstanceData> UInstancedStaticMeshComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData;
#if WITH_EDITOR
	InstanceData.InitializeAs<FInstancedStaticMeshComponentInstanceData>(this);
	FInstancedStaticMeshComponentInstanceData* StaticMeshInstanceData = InstanceData.Cast<FInstancedStaticMeshComponentInstanceData>();

	// Fill in info (copied from UStaticMeshComponent::GetComponentInstanceData)
	StaticMeshInstanceData->CachedStaticLighting.Transform = GetComponentTransform();

	for (const FStaticMeshComponentLODInfo& LODDataEntry : LODData)
	{
		StaticMeshInstanceData->CachedStaticLighting.MapBuildDataIds.Add(LODDataEntry.OriginalMapBuildDataId);
		StaticMeshInstanceData->CachedStaticLighting.MapBuildDataIds.Add(LODDataEntry.MapBuildDataId);
	}

	// Back up per-instance info (this is strictly for Comparison in UInstancedStaticMeshComponent::ApplyComponentInstanceData 
	// as this Property will get serialized by base class FActorComponentInstanceData through FComponentPropertyWriter which uses the PPF_ForceTaggedSerialization to backup all properties even the custom serialized ones
	StaticMeshInstanceData->PerInstanceSMData = PerInstanceSMData;

	// Back up instance selection
	StaticMeshInstanceData->SelectedInstances = SelectedInstances;

	// Back up random seed
	StaticMeshInstanceData->InstancingRandomSeed = InstancingRandomSeed;
	StaticMeshInstanceData->AdditionalRandomSeeds = AdditionalRandomSeeds;

	// Back up per-instance hit proxies
	StaticMeshInstanceData->bHasPerInstanceHitProxies = bHasPerInstanceHitProxies;
#endif
	return InstanceData;
}

void UInstancedStaticMeshComponent::GetComponentChildElements(TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate)
{
#if WITH_EDITOR
	for (int32 InstanceIndex = 0; InstanceIndex < PerInstanceSMData.Num(); ++InstanceIndex)
	{
		if (FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(this, InstanceIndex, bAllowCreate))
		{
			OutElementHandles.Add(MoveTemp(ElementHandle));
		}
	}
#endif	// WITH_EDITOR
}

void UInstancedStaticMeshComponent::PreApplyComponentInstanceData(struct FInstancedStaticMeshComponentInstanceData* InstancedMeshData)
{
#if WITH_EDITOR
	// Prevent proxy recreate while traversing the ::ApplyToComponent stack
	bIsInstanceDataApplyCompleted = false;
#endif
}

void UInstancedStaticMeshComponent::ApplyComponentInstanceData(FInstancedStaticMeshComponentInstanceData* InstancedMeshData)
{
#if WITH_EDITOR
	check(InstancedMeshData);

	ON_SCOPE_EXIT
	{		
		bIsInstanceDataApplyCompleted = true;	
	};

	if (GetStaticMesh() != InstancedMeshData->StaticMesh)
	{
		return;
	}

	bool bMatch = false;

	// If we should inherit from archetype do it here after data was applied and before comparing (RerunConstructionScript will serialize SkipSerialization properties and reapply them even if we want to inherit them)
	const UInstancedStaticMeshComponent* Archetype = Cast<UInstancedStaticMeshComponent>(GetArchetype());
	if (ShouldInheritPerInstanceData(Archetype))
	{
		ApplyInheritedPerInstanceData(Archetype);
	}

	// Check for any instance having moved as that would invalidate static lighting
	if (PerInstanceSMData.Num() == InstancedMeshData->PerInstanceSMData.Num() &&
		InstancedMeshData->CachedStaticLighting.Transform.Equals(GetComponentTransform()))
	{
		bMatch = true;

		for (int32 InstanceIndex = 0; InstanceIndex < PerInstanceSMData.Num(); ++InstanceIndex)
		{
			if (!PerInstanceSMData[InstanceIndex].Transform.Equals(InstancedMeshData->PerInstanceSMData[InstanceIndex].Transform))
			{
				bMatch = false;
				break;
			}
		}
	}

	// Restore static lighting if appropriate
	if (bMatch)
	{
		const int32 NumLODLightMaps = InstancedMeshData->CachedStaticLighting.MapBuildDataIds.Num()/2;
		SetLODDataCount(NumLODLightMaps, NumLODLightMaps);

		for (int32 i = 0; i < NumLODLightMaps; ++i)
		{
			LODData[i].OriginalMapBuildDataId = InstancedMeshData->CachedStaticLighting.MapBuildDataIds[(i*2)];
			LODData[i].MapBuildDataId = InstancedMeshData->CachedStaticLighting.MapBuildDataIds[(i*2)+1];
		}
	}

	SelectedInstances = InstancedMeshData->SelectedInstances;

	InstancingRandomSeed = InstancedMeshData->InstancingRandomSeed;
	AdditionalRandomSeeds = InstancedMeshData->AdditionalRandomSeeds;

	bHasPerInstanceHitProxies = InstancedMeshData->bHasPerInstanceHitProxies;

	// TODO: restore ID mapping either from the serialized stuff, or the InstancedMeshData
	PrimitiveInstanceDataManager.Invalidate(PerInstanceSMData.Num());
#endif
}

bool UInstancedStaticMeshComponent::IsHLODRelevant() const
{
	if (GetInstanceCount() == 0)
	{
		return false;
	}

	return Super::IsHLODRelevant();
}

#if WITH_EDITOR
void UInstancedStaticMeshComponent::ComputeHLODHash(FHLODHashBuilder& HashBuilder) const
{
	Super::ComputeHLODHash(HashBuilder);

	HashBuilder.HashField(PerInstanceSMData, GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMData));
	HashBuilder.HashField(PerInstanceSMCustomData, GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMCustomData));
	HashBuilder.HashField(InstancingRandomSeed, GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, InstancingRandomSeed));
	HashBuilder.HashField(AdditionalRandomSeeds, GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, AdditionalRandomSeeds));
}
#endif

void UInstancedStaticMeshComponent::SendRenderInstanceData_Concurrent()
{
	Super::SendRenderInstanceData_Concurrent();

	PrimitiveInstanceDataManager.ResetComponentDirtyTracking();

	// If the primitive isn't hidden update its instances.
	const bool bDetailModeAllowsRendering = DetailMode <= GetCachedScalabilityCVars().DetailMode;
	// The proxy may not be created, this can happen when a SM is async loading for example.
	if (bDetailModeAllowsRendering && (ShouldRender() || bCastHiddenShadow || bAffectIndirectLightingWhileHidden || bRayTracingFarField))
	{
		if (SceneProxy != nullptr)
		{
			// Make sure the instance data proxy is up to date:
			FInstanceUpdateComponentDesc ComponentData;
			BuildComponentInstanceData(SceneProxy->GetScene().GetShaderPlatform(), ComponentData);
			if (PrimitiveInstanceDataManager.FlushChanges(MoveTemp(ComponentData), false))
			{
				UpdateBounds();
				GetWorld()->Scene->UpdatePrimitiveInstances(this);
			}
		}
		else
		{
			UpdateBounds();
			GetWorld()->Scene->AddPrimitive(this);

			// A scene proxy was not created during render state creation possibly due to lacking instance data.
			// In that case, this component hasn't been added to the streamers so do it now
			ConditionalNotifyStreamingPrimitiveUpdated_Concurrent();
		}
	}
}

const TArray<FBodyInstance*>& UInstancedStaticMeshComponent::GetInstanceBodies() const
{
	return InstancePhysicsBodies->GetBodies();
}

FBodyInstance* UInstancedStaticMeshComponent::GetBodyInstance(FName BoneName, bool bGetWelded, int32 Index) const
{
	return InstancePhysicsBodies->GetBodyInstance(Index);
}

FPrimitiveSceneProxy* UInstancedStaticMeshComponent::CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	// Override which constructor is used for the Nanite scene proxy or create FInstancedStaticMeshSceneProxy

	LLM_SCOPE(ELLMTag::InstancedMesh);

	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogStaticMesh, Verbose, "Skipping CreateSceneProxy for UInstancedStaticMeshComponent %ls (UInstancedStaticMeshComponent PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	GetOrCreateInstanceDataSceneProxy();

	if (bCreateNanite)
	{
		return ::new Nanite::FSceneProxy(NaniteMaterials, this);
	}

	return ::new FInstancedStaticMeshSceneProxy(this, GetWorld()->GetFeatureLevel());
}

FPrimitiveSceneProxy* UInstancedStaticMeshComponent::CreateSceneProxy()
{
	LLM_SCOPE(ELLMTag::InstancedMesh);

	ProxySize = 0;

	PrimitiveInstanceDataManager.ResetComponentDirtyTracking();

	// Verify that instance data is valid before using it.
	const bool bIsInstanceDataValid =
#if WITH_EDITOR
		bIsInstanceDataApplyCompleted && 
#endif
		// make sure we have instances - or have explicitly permitted creating empty ISMs
		(PerInstanceSMData.Num() > 0 || CVarAllowCreateEmptyISMs.GetValueOnGameThread());

	if (!bIsInstanceDataValid)
	{
		return nullptr;
	}

	check(InstancingRandomSeed != 0);
		
	FPrimitiveSceneProxy* PrimitiveSceneProxy = Super::CreateSceneProxy();
	if (PrimitiveSceneProxy != nullptr)
	{
		FInstanceUpdateComponentDesc ComponentData;
		BuildComponentInstanceData(PrimitiveSceneProxy->GetScene().GetShaderPlatform(), ComponentData);
		PrimitiveInstanceDataManager.FlushChanges(MoveTemp(ComponentData), true);
	}
	return PrimitiveSceneProxy;
}

FMatrix UInstancedStaticMeshComponent::GetRenderMatrix() const
{
	// Apply the translated space to the render matrix.
	return (FTransform(GetTranslatedInstanceSpaceOrigin()) * GetComponentTransform()).ToMatrixWithScale();
}

void UInstancedStaticMeshComponent::CreateHitProxyData(TArray<TRefCountPtr<HHitProxy>>& HitProxies)
{
	if (GIsEditor && bHasPerInstanceHitProxies)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UInstancedStaticMeshComponent_CreateHitProxyData);
		
		int32 NumProxies = PerInstanceSMData.Num();
		HitProxies.Empty(NumProxies);

		for (int32 InstanceIdx = 0; InstanceIdx < NumProxies; ++InstanceIdx)
		{
			HitProxies.Add(new HInstancedStaticMeshInstance(this, InstanceIdx));
		}
	}
	else
	{
		HitProxies.Empty();
	}
}

void UInstancedStaticMeshComponent::BuildLegacyRenderData(FStaticMeshInstanceData& OutData)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UInstancedStaticMeshComponent_BuildRenderData);

	int32 NumInstances = PerInstanceSMData.Num();
	if (NumInstances == 0)
	{
		return;
	}
	
	OutData.AllocateInstances(NumInstances, NumCustomDataFloats, EResizeBufferFlags::None, true); // In Editor always permit overallocation, to prevent too much realloc

	const FMeshMapBuildData* MeshMapBuildData = nullptr;

#if WITH_EDITOR
	MeshMapBuildData = FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(this, 0);
#endif

	if (MeshMapBuildData == nullptr && LODData.Num() > 0)
	{
		MeshMapBuildData = GetMeshMapBuildData(LODData[0], false);
	}
	
	check(InstancingRandomSeed != 0);
	FRandomStream RandomStream = FRandomStream(InstancingRandomSeed);

	auto AdditionalRandomSeedsIt = AdditionalRandomSeeds.CreateIterator();
	int32 SeedResetIndex = AdditionalRandomSeedsIt ? AdditionalRandomSeedsIt->StartInstanceIndex : INDEX_NONE;
	
	const FVector TranslatedInstanceSpaceOffset = -GetTranslatedInstanceSpaceOrigin();

	for (int32 Index = 0; Index < NumInstances; ++Index)
	{
		const int32 RenderIndex = GetRenderIndex(Index);
		if (RenderIndex == INDEX_NONE) 
		{
			// could be skipped by density settings
			continue;
		}

		// Reset the random stream if necessary
		if (Index == SeedResetIndex)
		{
			RandomStream = FRandomStream(AdditionalRandomSeedsIt->RandomSeed);
			AdditionalRandomSeedsIt++;
			SeedResetIndex = AdditionalRandomSeedsIt ? AdditionalRandomSeedsIt->StartInstanceIndex : INDEX_NONE;
		}

		const FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[Index];
		FVector2D LightmapUVBias = FVector2D(-1.0f, -1.0f);
		FVector2D ShadowmapUVBias = FVector2D(-1.0f, -1.0f);

		if (MeshMapBuildData != nullptr && MeshMapBuildData->PerInstanceLightmapData.IsValidIndex(Index))
		{
			LightmapUVBias = FVector2D(MeshMapBuildData->PerInstanceLightmapData[Index].LightmapUVBias);
			ShadowmapUVBias = FVector2D(MeshMapBuildData->PerInstanceLightmapData[Index].ShadowmapUVBias);
		}
	
		// LWC_TODO: Precision loss here is compensated for by use of TranslatedInstanceSpaceOffset.
		OutData.SetInstance(RenderIndex, FMatrix44f(InstanceData.Transform.ConcatTranslation(TranslatedInstanceSpaceOffset)), RandomStream.GetFraction(), LightmapUVBias, ShadowmapUVBias);

		for (int32 CustomDataIndex = 0; CustomDataIndex < NumCustomDataFloats; ++CustomDataIndex)
		{
			OutData.SetInstanceCustomData(RenderIndex, CustomDataIndex, PerInstanceSMCustomData[Index * NumCustomDataFloats + CustomDataIndex]);
		}
	}
}

void UInstancedStaticMeshComponent::BuildInstanceDataDeltaChangeSetCommon(FISMInstanceUpdateChangeSet &ChangeSet)
{
#if WITH_EDITOR
	if (ChangeSet.Flags.bHasPerInstanceEditorData)
	{
		// TODO: the way hit proxies are managed seems daft, why don't we just add them when needed and store them in an array alonside the instances?
		//       this will always force us to update all the hit proxy data for every instances.
		TArray<TRefCountPtr<HHitProxy>> HitProxies;
		CreateHitProxyData(HitProxies);
		ChangeSet.SetEditorData(HitProxies, SelectedInstances);
	}
#endif
	if (ChangeSet.Flags.bHasPerInstanceRandom)
	{
		// TODO: we need to change how this is done(!), These need to be stored off when an instance is created and then persisted like other data.
		//       Otherwise there is no efficient way to update just one in the middle.

		// Does this always have to crunch through all the instances from start to end each time anything changes? 
		check(InstancingRandomSeed != 0);
		ChangeSet.GeneratePerInstanceRandomIds =
			[InstancingRandomSeed = InstancingRandomSeed,
			AdditionalRandomSeeds = AdditionalRandomSeeds](TArray<float> &InstanceRandomIDs)
			{
				FInstanceDataHelpers::GenerateInstanceRandomIDs(InstancingRandomSeed, AdditionalRandomSeeds, InstanceRandomIDs);
			};
	}

	if (ChangeSet.Flags.bHasPerInstanceLMSMUVBias)
	{
		const FMeshMapBuildData* MeshMapBuildData = nullptr;

#if WITH_EDITOR
		MeshMapBuildData = FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(this, 0);
#endif

		if (MeshMapBuildData == nullptr && LODData.Num() > 0)
		{
			MeshMapBuildData = GetMeshMapBuildData(LODData[0], false);
		}

		for (int32 Index = 0; Index < ChangeSet.PostUpdateNumInstances; ++Index)
		{
			FVector2D LightmapUVBias = FVector2D(-1.0f, -1.0f);
			FVector2D ShadowmapUVBias = FVector2D(-1.0f, -1.0f);

			if (MeshMapBuildData != nullptr && MeshMapBuildData->PerInstanceLightmapData.IsValidIndex(Index))
			{
				LightmapUVBias = FVector2D(MeshMapBuildData->PerInstanceLightmapData[Index].LightmapUVBias);
				ShadowmapUVBias = FVector2D(MeshMapBuildData->PerInstanceLightmapData[Index].ShadowmapUVBias);
			}
			ChangeSet.AddInstanceLightShadowUVBias(FVector4f(LightmapUVBias.X, LightmapUVBias.Y, ShadowmapUVBias.X, ShadowmapUVBias.Y));
		}
	}
	ChangeSet.SetCustomData(MakeArrayView(PerInstanceSMCustomData), NumCustomDataFloats);
}

// Deprecated 5.7
void UInstancedStaticMeshComponent::BuildComponentInstanceData(ERHIFeatureLevel::Type FeatureLevel, FInstanceUpdateComponentDesc& OutData)
{
	BuildComponentInstanceData(GetFeatureLevelShaderPlatform_Checked(FeatureLevel), OutData);
}

void UInstancedStaticMeshComponent::BuildComponentInstanceData(EShaderPlatform InShaderPlatform, FInstanceUpdateComponentDesc& OutData)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UInstancedStaticMeshComponent_BuildRenderData);

	OutData.PrimitiveMaterialDesc = GetUsedMaterialPropertyDesc(InShaderPlatform);
	OutData.Flags = MakeInstanceDataFlags(OutData.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom, OutData.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceCustomData);
	OutData.ComponentMobility = Mobility;
	OutData.PrimitiveLocalToWorld = GetRenderMatrix();
	OutData.StaticMeshBounds = GetStaticMesh()->GetBounds();
	OutData.NumProxyInstances = PerInstanceSMData.Num();
	OutData.NumSourceInstances = PerInstanceSMData.Num();
	OutData.NumCustomDataFloats = NumCustomDataFloats;

	// Function that only gets called if we actually need to flush any changes.
	OutData.BuildChangeSet = [this](FISMInstanceUpdateChangeSet &ChangeSet)
	{
		BuildInstanceDataDeltaChangeSetCommon(ChangeSet);

		const bool bUpdateConservativeBounds = bUseConservativeBounds && GConservativeBoundsThreshold >= 0 && PerInstanceSMData.Num() > GConservativeBoundsThreshold;
		if (bUpdateConservativeBounds)
		{
			if (ChangeSet.IsFullUpdate())
			{
				CachedConservativeInstanceBounds.Init();
			}
			else if (!CachedConservativeInstanceBounds.IsValid && !Bounds.GetBox().GetSize().IsZero())
			{
				// Initialize to current bounds in local space.
				CachedConservativeInstanceBounds = Bounds.GetBox().TransformBy(GetComponentTransform().Inverse());
			}

			// Use the variation of SetInstanceTransforms() that updates the conservative bounds.
			// Do this here, rather than in a separate loop, to take advantage of the fact that we are already iterating over the transform data.
			ChangeSet.SetInstanceTransforms(MakeStridedView(PerInstanceSMData, &FInstancedStaticMeshInstanceData::Transform), GetStaticMesh()->GetBounds().GetBox(), CachedConservativeInstanceBounds);
		}
		else
		{
			CachedConservativeInstanceBounds.Init();

			ChangeSet.SetInstanceTransforms(MakeStridedView(PerInstanceSMData, &FInstancedStaticMeshInstanceData::Transform));
		}

		ChangeSet.SetInstancePrevTransforms(MakeArrayView(PerInstancePrevTransform));
	};
}

void UInstancedStaticMeshComponent::InitInstanceBody(int32 InstanceIdx, FBodyInstance* InstanceBodyInstance, FBodyInstance* ReferenceBodyInstance, bool bRuntimeBodyInstance, UBodySetup* BodySetup)
{
	if (!GetStaticMesh())
	{
		UE_LOGF(LogStaticMesh, Warning, "Unable to create a body instance for %ls in Actor %ls. No StaticMesh set.", *GetName(), GetOwner() ? *GetOwner()->GetName() : TEXT("?"));
		return;
	}

	check(InstanceIdx < PerInstanceSMData.Num());
	check(InstanceIdx < InstancePhysicsBodies->Num());
	check(InstanceBodyInstance);
	check(ReferenceBodyInstance);
	check(BodySetup);

	const FTransform InstanceTransform = FTransform(PerInstanceSMData[InstanceIdx].Transform) * GetComponentTransform();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InstancePhysicsBodies->InitInstanceBody(InstanceBodyInstance, ReferenceBodyInstance, InstanceIdx, InstanceTransform,
		bRuntimeBodyInstance, BodySetup);	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UInstancedStaticMeshComponent::InitInstanceBody(int32 InstanceIdx, FBodyInstance* InstanceBodyInstance)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InitInstanceBody(InstanceIdx, InstanceBodyInstance, &BodyInstance, false, GetBodySetup());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UInstancedStaticMeshComponent::CreateAllInstanceBodies()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::CreateAllInstanceBodies);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UInstancedStaticMeshComponent_CreateAllInstanceBodies);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UInstancedStaticMeshComponent_CreateAllInstanceBodies);
	STAT(FScopeCycleCounter Context(StatId);)

	const int32 NumBodies = PerInstanceSMData.Num();

	UBodySetup* BodySetup = GetBodySetup();
	if (!BodySetup)
	{
		UE_LOGF(LogStaticMesh, Warning, "Instance Static Mesh Component unable to create InstanceBodies!");
		return;
	}

	TArray<FTransform> WorldTransforms;
	WorldTransforms.SetNumUninitialized(NumBodies);
	for (int32 i = 0; i < NumBodies; ++i)
	{
		WorldTransforms[i] = FTransform(PerInstanceSMData[i].Transform) * GetComponentTransform();
	}

#if WITH_EDITOR
	const bool bSkipWalkableSlopeOverride = bWalkableSlopeOverrideCachedGT;
#else
	constexpr bool bSkipWalkableSlopeOverride = false;
#endif
	InstancePhysicsBodies->CreateAll(WorldTransforms, bSkipWalkableSlopeOverride);
}

void UInstancedStaticMeshComponent::ClearAllInstanceBodies()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::ClearAllInstanceBodies);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UInstancedStaticMeshComponent_ClearAllInstanceBodies);
	STAT(FScopeCycleCounter Context(StatId);)

	InstancePhysicsBodies->ClearAll();
}


void UInstancedStaticMeshComponent::OnCreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::OnCreatePhysicsState)
	check(InstancePhysicsBodies->Num() == 0);

	FPhysScene* PhysScene = GetWorld()->GetPhysicsScene();

	if (!PhysScene)
	{
		return;
	}

	// Create all the bodies.
	CreateAllInstanceBodies();

	USceneComponent::OnCreatePhysicsState();

	// Since StaticMeshComponent was not called
	// Navigation relevancy needs to be handled here
	bNavigationRelevant = IsNavigationRelevant();
}

void UInstancedStaticMeshComponent::OnDestroyPhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::OnDestroyPhysicsState);
	USceneComponent::OnDestroyPhysicsState();

	// Release all physics representations
	ClearAllInstanceBodies();

	// Since StaticMeshComponent was not called
	// Navigation relevancy needs to be handled here
	bNavigationRelevant = IsNavigationRelevant();
}

void UInstancedStaticMeshComponent::OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	USceneComponent::OnAsyncDestroyPhysicsStateBegin_GameThread(OutRootedObjects);

	// Call UnRegisterForCollisionEvents on game thread before going async.
	// TermBody skips UnRegisterForCollisionEvents when called from a worker thread (see FBodyInstance::TermBody contract).
	if (BodyInstance.bNotifyRigidBodyCollision)
	{
		if (FPhysScene* PhysScene = GetWorld()->GetPhysicsScene())
		{
			PhysScene->UnRegisterForCollisionEvents(this);
		}
	}

	InstancePhysicsBodies->BeginAsyncDestroy();

	bNavigationRelevant = IsNavigationRelevant();
}

#if WITH_EDITOR
void UInstancedStaticMeshComponent::OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects)
{
	// GT-side capture of WalkableSlopeOverride from the (potentially designer-mutable) BodySetup.
	// The worker step (UActorComponent::OnAsyncCreatePhysicsState fallthrough -> OnCreatePhysicsState
	// -> CreateAllInstanceBodies -> InstancePhysicsBodies->CreateAll) checks bWalkableSlopeOverrideCachedGT
	// and skips its own read.
	if (UBodySetup* BodySetup = GetBodySetup())
	{
		if (!BodyInstance.GetOverrideWalkableSlopeOnInstance())
		{
			BodyInstance.SetWalkableSlopeOverride(BodySetup->WalkableSlopeOverride, false);
		}
	}
	bWalkableSlopeOverrideCachedGT = true;

	Super::OnAsyncCreatePhysicsStateBegin_GameThread(OutRootedObjects);
}
#endif

void UInstancedStaticMeshComponent::OnAsyncCreatePhysicsStateEnd_GameThread()
{
	Super::OnAsyncCreatePhysicsStateEnd_GameThread();

#if WITH_EDITOR
	bWalkableSlopeOverrideCachedGT = false;
#endif

	for (FBodyInstance* Body : *InstancePhysicsBodies)
	{
		if (Body)
		{
			Body->ClearCachedPhysicsCreationInputs();
		}
	}
}

bool UInstancedStaticMeshComponent::OnAsyncDestroyPhysicsState(const UE::FTimeout& Timeout)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::OnAsyncDestroyPhysicsState);
	USceneComponent::OnAsyncDestroyPhysicsState(Timeout);

	InstancePhysicsBodies->AsyncDestroy();
	return true;
}

Chaos::FPhysicsObject* UInstancedStaticMeshComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	return InstancePhysicsBodies->GetPhysicsObjectById(Id);
}

TArray<Chaos::FPhysicsObject*> UInstancedStaticMeshComponent::GetAllPhysicsObjects() const
{
	return InstancePhysicsBodies->GetAllPhysicsObjects();
}

bool UInstancedStaticMeshComponent::CanEditSimulatePhysics()
{
	// if instancedstaticmeshcomponent, we will never allow it
	return false;
}

uint32 FBoundsCacheElement::GetHash(const FTransform& InBoundTransform, const FBox& InInstanceBounds)
{
	// Compute a persistent hash as this value may be saved
	FVector Translation = InBoundTransform.GetTranslation();
	FQuat Rotation = InBoundTransform.GetRotation();
	FVector Scale3D = InBoundTransform.GetScale3D();

	uint64 Hash = 0;
	Hash = CityHash64WithSeed((const char*)&Translation, sizeof(Translation), Hash);
	Hash = CityHash64WithSeed((const char*)&Rotation, sizeof(Rotation), Hash);
	Hash = CityHash64WithSeed((const char*)&Scale3D, sizeof(Scale3D), Hash);
	Hash = CityHash64WithSeed((const char*)&InInstanceBounds.Min, sizeof(InInstanceBounds.Min), Hash);
	Hash = CityHash64WithSeed((const char*)&InInstanceBounds.Max, sizeof(InInstanceBounds.Max), Hash);
	return static_cast<uint32>(Hash) ^ static_cast<uint32>(Hash >> 32);
}

void UInstancedStaticMeshComponent::InvalidateCachedBounds()
{
	static_assert(UE_ARRAY_COUNT(CachedBounds) == int32(EBoundsCacheType::Count));
	for (int i = 0; i < (int)EBoundsCacheType::Count; ++i)
	{
		CachedBounds[i].Invalidate();
	}
}

FBoxSphereBounds UInstancedStaticMeshComponent::CalcBounds(const FTransform& InBoundTransform) const
{
	EBoundsCacheType Type = bIsUpdatingBounds ? EBoundsCacheType::Component : InBoundTransform.Equals(FTransform::Identity) ? EBoundsCacheType::Local : EBoundsCacheType::Other;
	FBoxSphereBounds NewBounds = GetOrCalcBounds(InBoundTransform, Type);
	NewBounds.BoxExtent *= BoundsScale;
	NewBounds.SphereRadius *= BoundsScale;
	return NewBounds;
}

void UInstancedStaticMeshComponent::CalcAndCacheNavigationBounds()
{
	GetOrCalcBounds(GetComponentTransform(), EBoundsCacheType::Navigation);
}

FBoxSphereBounds UInstancedStaticMeshComponent::GetOrCalcBounds(const FTransform& InBoundTransform, EBoundsCacheType InType) const
{
	if (GetStaticMesh() && PerInstanceSMData.Num() > 0)
	{
		const FBox InstanceBounds = (InType == EBoundsCacheType::Navigation) ? GetInstanceNavigationBounds() : GetStaticMesh()->GetBounds().GetBox();
		if (InstanceBounds.IsValid)
		{
			const UWorld* const World = GetWorld();
			const bool bIsGameWorld = World && World->IsGameWorld();
			const bool bShouldCache = InType != EBoundsCacheType::Other && (IsRegistered() || bIsGameWorld);
			if (!bShouldCache)
			{
				FBoxSphereBounds Value = CalcBoundsImpl(InBoundTransform, InstanceBounds);
				if (InType == EBoundsCacheType::Navigation)
				{
					// Store result for Navigation Cache as GetNavigationBounds() relies on it.
					CachedBounds[(int32)EBoundsCacheType::Navigation].CacheValue(Value);
				}
				return Value;
			}

			const uint32 Hash = FBoundsCacheElement::GetHash(InBoundTransform, InstanceBounds);
			int32 TypeAsInt = (int32)InType;
			FBoundsCacheElement& TargetBounds = CachedBounds[TypeAsInt];
			if (TargetBounds != Hash)
			{
				for (int32 TypeIndex = 0; TypeIndex < (int32)EBoundsCacheType::Count; ++TypeIndex)
				{
					if ((TypeIndex != TypeAsInt) && CachedBounds[TypeIndex] == Hash)
					{
						TargetBounds = CachedBounds[TypeIndex];
						return TargetBounds.GetValue();
					}
				}
				TargetBounds.CacheValue(CalcBoundsImpl(InBoundTransform, InstanceBounds), Hash);
			}
			return TargetBounds.GetValue();
		}
	}
	FBoxSphereBounds Value(InBoundTransform.GetLocation(), FVector::ZeroVector, 0.f);
	if (InType == EBoundsCacheType::Navigation)
	{
		// Store result for Navigation Cache as GetNavigationBounds() relies on it.
		CachedBounds[(int32)EBoundsCacheType::Navigation].CacheValue(Value);
	}
	return Value;
}

// Deprecated
FBoxSphereBounds UInstancedStaticMeshComponent::CalcBoundsImpl(const FTransform& BoundTransform, bool bForNavigation) const
{
	if (GetStaticMesh() && PerInstanceSMData.Num() > 0)
	{
		const FBox InstanceBounds = bForNavigation ? GetInstanceNavigationBounds() : GetStaticMesh()->GetBounds().GetBox();
		if (InstanceBounds.IsValid)
		{
			return CalcBoundsImpl(BoundTransform, InstanceBounds);
		}
	}
	return FBoxSphereBounds(BoundTransform.GetLocation(), FVector::ZeroVector, 0.f);
}

FBoxSphereBounds UInstancedStaticMeshComponent::CalcBoundsImpl(const FTransform& BoundTransform, const FBox& InstanceBounds) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::CalcBoundsImpl);
	check(PerInstanceSMData.Num() > 0);
	const FMatrix BoundTransformMatrix = BoundTransform.ToMatrixWithScale();
	FBoxSphereBounds::Builder BoundsBuilder;
	for (int32 InstanceIndex = 0; InstanceIndex < PerInstanceSMData.Num(); InstanceIndex++)
	{
		BoundsBuilder += InstanceBounds.TransformBy(PerInstanceSMData[InstanceIndex].Transform * BoundTransformMatrix);
	}
	return BoundsBuilder;
}

void UInstancedStaticMeshComponent::UpdateBounds()
{
#if WITH_EDITOR
	// Skip updating bounds to avoid stall on static mesh if we're still compiling and because
	// if this is called during registration, we can end up doing a flush that will then
	// try to recreate the render state as part of the postcompilation before we are even ready to do so
	// in the registration flow, causing asserts because InstancingRandomSeed has not yet been initialized.
	if (IsCompiling())
	{
		// Bounds will be updated when we finish compiling.
		return;
	}
#endif
	FGuardValue_Bitfield(bIsUpdatingBounds, true);
	
	if (GetNumRenderInstances() == 0)
	{
		Bounds = FBoxSphereBounds(GetComponentTransform().GetLocation(), FVector::ZeroVector, 0.f);
	}
	else if (bUseConservativeBounds && CachedConservativeInstanceBounds.IsValid)
	{
		Bounds = CachedConservativeInstanceBounds.TransformBy(GetComponentTransform());
	}
	else
	{
		Super::UpdateBounds();
	}

	if (bCanEverAffectNavigation && (IsRegistered() || IsPreRegistering()))
	{
		GetOrCalcBounds(GetComponentTransform(), EBoundsCacheType::Navigation);
	}
}

#if WITH_EDITOR
void UInstancedStaticMeshComponent::GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo, const TArray<ULightComponent*>& InRelevantLights, const FLightingBuildOptions& Options)
{
	if (HasValidSettingsForStaticLighting(false))
	{
		// We need to make sure this data is created before trying to create the FStaticMeshStaticLightingMesh
		UpdateStaticLightingData();
		UpdateMapBuildDataId();

		// create static lighting for LOD 0
		int32 LightMapWidth = 0;
		int32 LightMapHeight = 0;
		GetLightMapResolution(LightMapWidth, LightMapHeight);

		bool bFit = false;
		bool bReduced = false;
		while (1)
		{
			const int32 OneLessThanMaximumSupportedResolution = 1 << (GMaxTextureMipCount - 2);

			const int32 MaxInstancesInMaxSizeLightmap = (OneLessThanMaximumSupportedResolution / LightMapWidth) * ((OneLessThanMaximumSupportedResolution / 2) / LightMapHeight);
			if (PerInstanceSMData.Num() > MaxInstancesInMaxSizeLightmap)
			{
				if (LightMapWidth < 4 || LightMapHeight < 4)
				{
					break;
				}
				LightMapWidth /= 2;
				LightMapHeight /= 2;
				bReduced = true;
			}
			else
			{
				bFit = true;
				break;
			}
		}

		if (!bFit)
		{
			FMessageLog("LightingResults").Message(EMessageSeverity::Error)
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "FailedStaticLightingWarning", "The total lightmap size for this InstancedStaticMeshComponent is too big no matter how much we reduce the per-instance size, the number of mesh instances in this component must be reduced")));
			return;
		}
		if (bReduced)
		{
			FMessageLog("LightingResults").Message(EMessageSeverity::Warning)
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "ReducedStaticLightingWarning", "The total lightmap size for this InstancedStaticMeshComponent was too big and it was automatically reduced. Consider reducing the component's lightmap resolution or number of mesh instances in this component")));
		}

		const int32 LightMapSize = GetWorld()->GetWorldSettings()->PackedLightAndShadowMapTextureSize;
		const int32 MaxInstancesInDefaultSizeLightmap = (LightMapSize / LightMapWidth) * ((LightMapSize / 2) / LightMapHeight);
		if (PerInstanceSMData.Num() > MaxInstancesInDefaultSizeLightmap)
		{
			FMessageLog("LightingResults").Message(EMessageSeverity::Warning)
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "LargeStaticLightingWarning", "The total lightmap size for this InstancedStaticMeshComponent is large, consider reducing the component's lightmap resolution or number of mesh instances in this component")));
		}

		// TODO: Support separate static lighting in LODs for instanced meshes.

		if (!GetStaticMesh()->CanLODsShareStaticLighting())
		{
			//TODO: Detect if the UVs for all sub-LODs overlap the base LOD UVs and omit this warning if they do.
			FMessageLog("LightingResults").Message(EMessageSeverity::Warning)
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(NSLOCTEXT("InstancedStaticMesh", "UniqueStaticLightingForLODWarning", "Instanced meshes don't yet support unique static lighting for each LOD. Lighting on LOD 1+ may be incorrect unless lightmap UVs are the same for all LODs.")));
		}

		// Force sharing LOD 0 lightmaps for now.
		int32 NumLODs = 1;

		CachedMappings.Reset(PerInstanceSMData.Num() * NumLODs);
		CachedMappings.AddZeroed(PerInstanceSMData.Num() * NumLODs);

		NumPendingLightmaps = 0;

		for (int32 LODIndex = 0; LODIndex < NumLODs; LODIndex++)
		{
			const FStaticMeshLODResources& LODRenderData = GetStaticMesh()->GetRenderData()->LODResources[LODIndex];

			for (int32 InstanceIndex = 0; InstanceIndex < PerInstanceSMData.Num(); InstanceIndex++)
			{
				auto* StaticLightingMesh = new FStaticLightingMesh_InstancedStaticMesh(this, LODIndex, InstanceIndex, InRelevantLights);
				OutPrimitiveInfo.Meshes.Add(StaticLightingMesh);

				auto* InstancedMapping = new FStaticLightingTextureMapping_InstancedStaticMesh(this, LODIndex, InstanceIndex, StaticLightingMesh, LightMapWidth, LightMapHeight, GetStaticMesh()->GetLightMapCoordinateIndex(), true);
				OutPrimitiveInfo.Mappings.Add(InstancedMapping);

				CachedMappings[LODIndex * PerInstanceSMData.Num() + InstanceIndex].Mapping = InstancedMapping;
				NumPendingLightmaps++;
			}

			// Shrink LOD texture lightmaps by half for each LOD level (minimum 4x4 px)
			LightMapWidth  = FMath::Max(LightMapWidth  / 2, 4);
			LightMapHeight = FMath::Max(LightMapHeight / 2, 4);
		}
	}
}

void FStaticLightingTextureMapping_InstancedStaticMesh::Serialize(FArchive& Ar)
{
	FStaticMeshStaticLightingTextureMapping::Serialize(Ar);

	Ar << InstanceIndex;
	bool bHasQuantizedData = QuantizedData != nullptr;
	Ar << bHasQuantizedData;
	if (bHasQuantizedData)
	{
		if (Ar.IsLoading())
		{
			QuantizedData = TUniquePtr<FQuantizedLightmapData>(new FQuantizedLightmapData);
		}

		QuantizedData->Serialize(Ar);
	}
}


void UInstancedStaticMeshComponent::ApplyLightMapping(FStaticLightingTextureMapping_InstancedStaticMesh* InMapping, const FStaticLightingBuildContext* LightingContext)
{
	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GMaxRHIShaderPlatform);

	// restore the CachedMappings ptr (in case we're using DeferredMappings)
	uint32 Index = (InMapping->GetLODIndex() * PerInstanceSMData.Num()) + InMapping->InstanceIndex;
	CachedMappings[Index].Mapping = InMapping;
	
	NumPendingLightmaps--;

	if (NumPendingLightmaps == 0)
	{
		// Calculate the range of each coefficient in this light-map and repack the data to have the same scale factor and bias across all instances
		// TODO: Per instance scale?

		// generate the final lightmaps for all the mappings for this component
		TArray<TUniquePtr<FQuantizedLightmapData>> AllQuantizedData;
		for (auto& MappingInfo : CachedMappings)
		{
			FStaticLightingTextureMapping_InstancedStaticMesh* Mapping = MappingInfo.Mapping;
			AllQuantizedData.Add(MoveTemp(Mapping->QuantizedData));
		}

		bool bNeedsShadowMap = false;
		TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>> AllShadowMapData;
		for (auto& MappingInfo : CachedMappings)
		{
			FStaticLightingTextureMapping_InstancedStaticMesh* Mapping = MappingInfo.Mapping;
			bNeedsShadowMap = bNeedsShadowMap || (Mapping->ShadowMapData.Num() > 0);
			AllShadowMapData.Add(MoveTemp(Mapping->ShadowMapData));
		}

		UStaticMesh* ResolvedMesh = GetStaticMesh();
		if (LODData.Num() != ResolvedMesh->GetNumLODs())
		{
			MarkPackageDirty();
		}

		// Ensure LODData has enough entries in it, free not required.
		SetLODDataCount(ResolvedMesh->GetNumLODs(), ResolvedMesh->GetNumLODs());
		FStaticMeshComponentLODInfo& LODInfo = LODData[0];

		// Ensure this LODInfo has a valid OriginalMapBuildDataId
		if (LODInfo.CreateMapBuildDataId(0))
		{
			MarkPackageDirty();
		}

		UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForActor(GetOwner());
		FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(LODInfo.MapBuildDataId, true);

		MeshBuildData.PerInstanceLightmapData.Empty(AllQuantizedData.Num());
		MeshBuildData.PerInstanceLightmapData.AddZeroed(AllQuantizedData.Num());

		// Create a light-map for the primitive.
		// When using VT, shadow map data is included with lightmap allocation
		const ELightMapPaddingType PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
		TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>> EmptyShadowMapData;
		TRefCountPtr<FLightMap2D> NewLightMap = FLightMap2D::AllocateInstancedLightMap(Registry, this,
			MoveTemp(AllQuantizedData),
			bUseVirtualTextures ? MoveTemp(AllShadowMapData) : MoveTemp(EmptyShadowMapData),
			Registry, LODInfo.MapBuildDataId, Bounds, PaddingType, LMF_Streamed);

		// Create a shadow-map for the primitive, only needed when not using VT
		TRefCountPtr<FShadowMap2D> NewShadowMap = (bNeedsShadowMap && !bUseVirtualTextures)
			? FShadowMap2D::AllocateInstancedShadowMap(Registry, this, MoveTemp(AllShadowMapData), Registry, LODInfo.MapBuildDataId, Bounds, PaddingType, SMF_Streamed)
			: nullptr;

		MeshBuildData.LightMap = NewLightMap;
		MeshBuildData.ShadowMap = NewShadowMap;

		// Build the list of statically irrelevant lights.
		// TODO: This should be stored per LOD.
		TSet<FGuid> RelevantLights;
		TSet<FGuid> PossiblyIrrelevantLights;
		for (auto& MappingInfo : CachedMappings)
		{
			for (FGuid LightGuid : MappingInfo.Mapping->Mesh->RelevantLightsGuid)
			{
				// Check if the light is stored in the light-map.
				const bool bIsInLightMap = MeshBuildData.LightMap && MeshBuildData.LightMap->LightGuids.Contains(LightGuid);

				// Check if the light is stored in the shadow-map.
				const bool bIsInShadowMap = MeshBuildData.ShadowMap && MeshBuildData.ShadowMap->LightGuids.Contains(LightGuid);

				// If the light isn't already relevant to another mapping, add it to the potentially irrelevant list
				if (!bIsInLightMap && !bIsInShadowMap && !RelevantLights.Contains(LightGuid))
				{
					PossiblyIrrelevantLights.Add(LightGuid);
				}

				// Light is relevant
				if (bIsInLightMap || bIsInShadowMap)
				{
					RelevantLights.Add(LightGuid);
					PossiblyIrrelevantLights.Remove(LightGuid);
				}
			}
		}

		MeshBuildData.IrrelevantLights = PossiblyIrrelevantLights.Array();

		PrimitiveInstanceDataManager.BakedLightingDataChangedAll();

		MarkRenderStateDirty();
	}
}

FBox UInstancedStaticMeshComponent::GetStreamingBounds() const
{
	return (GetStaticMesh() && PerInstanceSMData.Num()) ? Super::GetStreamingBounds() : FBox(ForceInit);
}
#endif

void UInstancedStaticMeshComponent::PropagateLightingScenarioChange()
{
	FComponentRecreateRenderStateContext Context(this);

	PrimitiveInstanceDataManager.BakedLightingDataChangedAll();
}

void UInstancedStaticMeshComponent::GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const
{
	Super::GetLightAndShadowMapMemoryUsage(LightMapMemoryUsage, ShadowMapMemoryUsage);

	int32 NumInstances = PerInstanceSMData.Num();

	// Scale lighting demo by number of instances
	LightMapMemoryUsage *= NumInstances;
	ShadowMapMemoryUsage *= NumInstances;
}

// Deprecated version of PerInstanceSMData
struct FInstancedStaticMeshInstanceData_DEPRECATED
{
	FMatrix44f Transform;
	FVector2f LightmapUVBias;
	FVector2f ShadowmapUVBias;
	
	friend FArchive& operator<<(FArchive& Ar, FInstancedStaticMeshInstanceData_DEPRECATED& InstanceData)
	{
		// @warning BulkSerialize: FInstancedStaticMeshInstanceData is serialized as memory dump
		Ar << InstanceData.Transform << InstanceData.LightmapUVBias << InstanceData.ShadowmapUVBias;
		return Ar;
	}
};

static bool NeedRenderDataForTargetPlatform(const ITargetPlatform* TargetPlatform)
{
#if WITH_EDITOR
	const UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(TargetPlatform->IniPlatformName());
	if (DeviceProfile)
	{
		int32 CVarFoliageSaveRenderData = 1;
		if (DeviceProfile->GetConsolidatedCVarValue(TEXT("foliage.SaveRenderData"), CVarFoliageSaveRenderData))
		{
			return CVarFoliageSaveRenderData != 0;
		}
	}
#endif // WITH_EDITOR
	return true;
}

void UInstancedStaticMeshComponent::SerializeRenderData(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		PrimitiveInstanceDataManager.ReadCookedRenderData(Ar);
	}
	else if (Ar.IsSaving())
	{
#if WITH_EDITOR
		FInstanceUpdateComponentDesc ComponentData;
		if (GetStaticMesh() != nullptr)
		{
			UpdateComponentToWorld();
			BuildComponentInstanceData(GMaxRHIShaderPlatform, ComponentData);
		}
		PrimitiveInstanceDataManager.WriteCookedRenderData(Ar, MoveTemp(ComponentData), MakeStridedView(PerInstanceSMData, &FInstancedStaticMeshInstanceData::Transform));
#endif
	}
}

void UInstancedStaticMeshComponent::ApplyInheritedPerInstanceData(const UInstancedStaticMeshComponent* InArchetype)
{
	check(InArchetype);
	PerInstanceSMData = InArchetype->PerInstanceSMData;
	PerInstanceSMCustomData = InArchetype->PerInstanceSMCustomData;
	NumCustomDataFloats = InArchetype->NumCustomDataFloats;
}

bool UInstancedStaticMeshComponent::ShouldInheritPerInstanceData() const
{
	return ShouldInheritPerInstanceData(Cast<UInstancedStaticMeshComponent>(GetArchetype()));
}

bool UInstancedStaticMeshComponent::ShouldInheritPerInstanceData(const UInstancedStaticMeshComponent* InArchetype) const
{
	return (bInheritPerInstanceData || !bEditableWhenInherited) && InArchetype && InArchetype->IsInBlueprint() && !IsTemplate();
}

void UInstancedStaticMeshComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (!IsTemplate() && IsRegistered())
	{
		GetOrCalcBounds(FTransform::Identity, EBoundsCacheType::Local);
		GetOrCalcBounds(GetComponentTransform(), EBoundsCacheType::Component);
		GetOrCalcBounds(GetComponentTransform(), EBoundsCacheType::Navigation);
	}
	else
	{
		InvalidateCachedBounds();
	}

	if (!IsTemplate())
	{
		const bool bUseRandomSeedValue = !ObjectSaveContext.IsCooking();
		InitializeInstancingRandomSeed(bUseRandomSeedValue);
	}
}

void UInstancedStaticMeshComponent::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::InstancedMesh);
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FMobileObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);	
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	
	bool bCooked = Ar.IsCooking();
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SerializeInstancedStaticMeshRenderData || Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SerializeInstancedStaticMeshRenderData)
	{
		Ar << bCooked;
	}

	// Inherit properties when bEditableWhenInherited == false || bInheritPerInstanceData == true (when the component isn't a template and we are persisting data)
	const UInstancedStaticMeshComponent* Archetype = Cast<UInstancedStaticMeshComponent>(GetArchetype());
	const bool bInheritSkipSerializationProperties = ShouldInheritPerInstanceData(Archetype) && Ar.IsPersistent();
	
	// Check if we need have SkipSerialization property data to load/save
	bool bHasSkipSerializationPropertiesData = !bInheritSkipSerializationProperties;
	if (Ar.IsLoading())
	{
#if WITH_EDITOR
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ISMComponentEditableWhenInheritedSkipSerialization)
		{
			bHasSkipSerializationPropertiesData = true;
		}
		else
#endif
		{
			Ar << bHasSkipSerializationPropertiesData;
		}
	}
	else
	{
		Ar << bHasSkipSerializationPropertiesData;
	}
			
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.CustomVer(FMobileObjectVersion::GUID) < FMobileObjectVersion::InstancedStaticMeshLightmapSerialization)
	{
		TArray<FInstancedStaticMeshInstanceData_DEPRECATED> DeprecatedData;
		DeprecatedData.BulkSerialize(Ar);
		PerInstanceSMData.Reset(DeprecatedData.Num());
		Algo::Transform(DeprecatedData, PerInstanceSMData, [](const FInstancedStaticMeshInstanceData_DEPRECATED& OldData){ 
			return FInstancedStaticMeshInstanceData(FMatrix(OldData.Transform));
		});
	}
	else
#endif //WITH_EDITOR
	if (Ar.IsLoading())
	{
		// Read existing data if it was serialized
		TArray<FInstancedStaticMeshInstanceData> TempPerInstanceSMData;
		TArray<float> TempPerInstanceSMCustomData;
		if (bHasSkipSerializationPropertiesData)
		{
			TempPerInstanceSMData.BulkSerialize(Ar, Ar.UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES);
			if(Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::PerInstanceCustomData)
			{
				TempPerInstanceSMCustomData.BulkSerialize(Ar);
			}
		}
						
		// If we should inherit use Archetype Data
		if (bInheritSkipSerializationProperties)
		{
			ApplyInheritedPerInstanceData(Archetype);
		} 
		// It is possible for a component to lose its BP archetype between a save / load so in this case we have no per instance data (usually this component gets deleted through construction script)
		else if(bHasSkipSerializationPropertiesData)
		{
			PerInstanceSMData = MoveTemp(TempPerInstanceSMData);
			PerInstanceSMCustomData = MoveTemp(TempPerInstanceSMCustomData);
		}
	}
	else if(bHasSkipSerializationPropertiesData)
	{
		PerInstanceSMData.BulkSerialize(Ar, Ar.UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES);
		PerInstanceSMCustomData.BulkSerialize(Ar);
	}

	if (bCooked && (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SerializeInstancedStaticMeshRenderData || Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::SerializeInstancedStaticMeshRenderData))
	{
		SerializeRenderData(Ar);
	}
	
#if WITH_EDITOR
	if( Ar.IsTransacting() )
	{
		Ar << SelectedInstances;
	}
#endif
}

void UInstancedStaticMeshComponent::PreAllocateInstancesMemory(int32 AddedInstanceCount)
{
	PerInstanceSMData.Reserve(PerInstanceSMData.Num() + AddedInstanceCount);
	PerInstanceSMCustomData.Reserve(PerInstanceSMCustomData.Num() + AddedInstanceCount * NumCustomDataFloats);
}

int32 UInstancedStaticMeshComponent::AddInstanceInternal(int32 InstanceIndex, FInstancedStaticMeshInstanceData* InNewInstanceData, const FTransform& InstanceTransform, bool bWorldSpace)
{
	InvalidateCachedBounds();

	FInstancedStaticMeshInstanceData* NewInstanceData = InNewInstanceData;

	PrimitiveInstanceDataManager.Add(InstanceIndex, NewInstanceData != nullptr);

	if (NewInstanceData == nullptr)
	{
		NewInstanceData = &PerInstanceSMData.AddDefaulted_GetRef();
	}

	const FTransform LocalTransform = bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform;
	SetupNewInstanceData(*NewInstanceData, InstanceIndex, LocalTransform);

	// Add custom data to instance
	PerInstanceSMCustomData.AddZeroed(NumCustomDataFloats);
	if (bHasPreviousTransforms)
	{
		// Copy in the current transform - it is somewhat redundant should probably add tracking instead to ensure it gets set, or pass in a struct that can initialize the whole instance 
		// (same with custom data where we store a mob of zeroes)
		PerInstancePrevTransform.Add(NewInstanceData->Transform);
	}

#if WITH_EDITOR
	if (SelectedInstances.Num())
	{
		SelectedInstances.Add(false);
	}
#endif

	// Update navigation relevancy
	bNavigationRelevant = IsNavigationRelevant();

	// Perform navigation update if the component is relevant to navigation and registered.
	// Otherwise this will be handled in OnRegister (e.g. Editor manipulations).
	if (bNavigationRelevant && IsRegistered())
	{
		// If it's the first instance, register the component to the navigation system
		// since it was skipped because IsNavigationRelevant() requires at least one instance.
		if (GetInstanceCount() == 1)
		{
			FNavigationSystem::RegisterComponent(*this);
		}

		if (SupportsPartialNavigationUpdate())
		{
			PartialNavigationUpdate(InstanceIndex);
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	if (FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.IsBound())
	{
		FInstancedStaticMeshDelegates::FInstanceIndexUpdateData IndexUpdate{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Added, InstanceIndex };
		FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Broadcast(this, MakeArrayView(&IndexUpdate, 1));
	}
	
	return InstanceIndex;
}

int32 UInstancedStaticMeshComponent::AddInstance(const FTransform& InstanceTransform, bool bWorldSpace)
{
	return AddInstanceInternal(PerInstanceSMData.Num(), nullptr, InstanceTransform, bWorldSpace);
}

TArray<int32> UInstancedStaticMeshComponent::AddInstancesInternal(TConstArrayView<FTransform> InstanceTransforms, bool bShouldReturnIndices, bool bWorldSpace, bool bUpdateNavigation)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::AddInstancesInternal);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UInstancedStaticMeshComponent_AddInstancesInternal);

	InvalidateCachedBounds();

	const int32 Count = InstanceTransforms.Num();

	TArray<int32> NewInstanceIndices;

	if (bShouldReturnIndices)
	{
		NewInstanceIndices.Reserve(Count);
	}

	int32 InstanceIndex = PerInstanceSMData.Num();

	PerInstanceSMCustomData.AddZeroed(NumCustomDataFloats * Count);

#if WITH_EDITOR
	SelectedInstances.Add(false, Count);
#endif

	const int32 NumInstances = InstanceIndex + Count;
	PerInstanceSMData.Reserve(NumInstances);

	if (bHasPreviousTransforms)
	{
		PerInstancePrevTransform.Reserve(NumInstances);
	}
	// Update navigation relevancy
	// Note that we update navigation relevancy using the base class since the new instances are not added yet
	// and IsNavigationRelevant() takes the number of instance into account which can be 0 here.
	bNavigationRelevant = NumInstances > 0 && Super::IsNavigationRelevant();

	// Navigation update required if explicitly requested and if the component is relevant to navigation and registered.
	// Otherwise this will be handled in OnRegister (e.g. Editor manipulations).
	const bool bNavigationUpdateRequired = bUpdateNavigation && bNavigationRelevant && IsRegistered();
	const bool bPartialNavigationUpdateRequired = bNavigationUpdateRequired && SupportsPartialNavigationUpdate();

	TArray<FTransform> NavigationUpdateTransforms;
	if (bPartialNavigationUpdateRequired)
	{
		NavigationUpdateTransforms.Reserve(InstanceTransforms.Num());
	}
	
	bool bNeedsNavigationRegistration = false;

	for (const FTransform& InstanceTransform : InstanceTransforms)
	{
		FInstancedStaticMeshInstanceData& NewInstanceData = PerInstanceSMData.AddDefaulted_GetRef();
		PrimitiveInstanceDataManager.Add(PerInstanceSMData.Num() - 1, false);

		const FTransform LocalTransform = bWorldSpace ? InstanceTransform.GetRelativeTransform(GetComponentTransform()) : InstanceTransform;
		SetupNewInstanceData(NewInstanceData, InstanceIndex, LocalTransform);

		if (bHasPreviousTransforms)
		{
			// Copy in the current transform - it is somewhat redundant should probably add tracking instead to ensure it gets set, or pass in a struct that can initialize the whole instance 
			// (same with custom data where we store a mob of zeroes)
			PerInstancePrevTransform.Emplace(NewInstanceData.Transform);
		}

		if (bShouldReturnIndices)
		{
			NewInstanceIndices.Add(InstanceIndex);
		}

		if (bNavigationUpdateRequired)
		{
			// If it's the first instance, register the component to the navigation system
			// since it was skipped because IsNavigationRelevant() requires at least one instance.
			if (GetInstanceCount() == 1)
			{
				bNeedsNavigationRegistration = true;
			}

			if (bPartialNavigationUpdateRequired)
			{
				NavigationUpdateTransforms.Emplace(InstanceTransform);
			}
		}

		if (FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.IsBound())
		{
			FInstancedStaticMeshDelegates::FInstanceIndexUpdateData IndexUpdate{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Added, InstanceIndex };
			FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Broadcast(this, MakeArrayView(&IndexUpdate, 1));
		}

		++InstanceIndex;
	}

	if (bNavigationUpdateRequired)
	{
		if (bNeedsNavigationRegistration)
		{
			FNavigationSystem::RegisterComponent(*this);
		}

		if (bPartialNavigationUpdateRequired)
		{
			PartialNavigationUpdates(NavigationUpdateTransforms);
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	return NewInstanceIndices;
}

TArray<int32> UInstancedStaticMeshComponent::AddInstances(const TArray<FTransform>& InstanceTransforms, bool bShouldReturnIndices, bool bWorldSpace, bool bUpdateNavigation)
{
	return AddInstancesInternal(InstanceTransforms, bShouldReturnIndices, bWorldSpace, bUpdateNavigation);
}

TArray<FPrimitiveInstanceId> UInstancedStaticMeshComponent::AddInstancesById(const TArrayView<const FTransform>& InstanceTransforms, bool bWorldSpace, const bool bUpdateNavigation)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	TArray<FPrimitiveInstanceId> Ids;
	TArray<int32> Indices = AddInstancesInternal(InstanceTransforms, true, bWorldSpace, bUpdateNavigation);
	Algo::Transform(Indices, Ids, [&](int32 Index) { return FPrimitiveInstanceId{PrimitiveInstanceDataManager.IndexToId(Index) }; } );
	return Ids;
}

FPrimitiveInstanceId UInstancedStaticMeshComponent::AddInstanceById(const FTransform& InstanceTransform, bool bWorldSpace)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	int32 Index = AddInstanceInternal(PerInstanceSMData.Num(), nullptr, InstanceTransform, bWorldSpace);
	return FPrimitiveInstanceId{PrimitiveInstanceDataManager.IndexToId(Index) };
}

void UInstancedStaticMeshComponent::SetCustomDataById(const TArrayView<const FPrimitiveInstanceId>& InstanceIds, TArrayView<const float> CustomDataFloats)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	check(InstanceIds.Num() * NumCustomDataFloats == CustomDataFloats.Num());
	for (int32 DataIndex = 0; DataIndex < InstanceIds.Num(); ++DataIndex)
	{
		FPrimitiveInstanceId Id = InstanceIds[DataIndex];
		SetCustomData(PrimitiveInstanceDataManager.IdToIndex(Id), MakeArrayView(CustomDataFloats.GetData() + DataIndex * NumCustomDataFloats, NumCustomDataFloats));
	}
}

void UInstancedStaticMeshComponent::SetCustomDataValueById(FPrimitiveInstanceId InstanceId, int32 CustomDataIndex, float CustomDataValue)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	SetCustomDataValue(PrimitiveInstanceDataManager.IdToIndex(InstanceId), CustomDataIndex, CustomDataValue);
}

void UInstancedStaticMeshComponent::RemoveInstancesById(const TArrayView<const FPrimitiveInstanceId>& InstanceIds, const bool bUpdateNavigation)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	for (FPrimitiveInstanceId Id : InstanceIds)
	{
		RemoveInstanceInternal(PrimitiveInstanceDataManager.IdToIndex(Id), false, true, bUpdateNavigation);
	}
}

void UInstancedStaticMeshComponent::UpdateInstanceTransformById(FPrimitiveInstanceId InstanceId, const FTransform& NewInstanceTransform, bool bWorldSpace, bool bTeleport)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	UpdateInstanceTransform(PrimitiveInstanceDataManager.IdToIndex(InstanceId), NewInstanceTransform, bWorldSpace, false, bTeleport);
}

void UInstancedStaticMeshComponent::SetPreviousTransformById(FPrimitiveInstanceId InstanceId, const FTransform& NewPrevInstanceTransform, bool bWorldSpace)
{
	check(PrimitiveInstanceDataManager.GetMode() != FPrimitiveInstanceDataManager::EMode::ExternalLegacyData);
	check(bHasPreviousTransforms);

	int32 InstanceIndex = PrimitiveInstanceDataManager.IdToIndex(InstanceId);

	// TODO: Computing LocalTransform is useless when we're updating the world location for the entire mesh.
	// Should find some way around this for performance.
	FTransform LocalPrevTransform = bWorldSpace ? NewPrevInstanceTransform.GetRelativeTransform(GetComponentTransform()) : NewPrevInstanceTransform;
	PerInstancePrevTransform[InstanceIndex] = LocalPrevTransform.ToMatrixWithScale();

	PrimitiveInstanceDataManager.TransformChanged(InstanceId);
}

bool UInstancedStaticMeshComponent::IsValidId(FPrimitiveInstanceId InstanceId)
{
	return PrimitiveInstanceDataManager.IsValidId(InstanceId);
}

void UInstancedStaticMeshComponent::SetHasPerInstancePrevTransforms(bool bInHasPreviousTransforms)
{
	if (bInHasPreviousTransforms != bHasPreviousTransforms)
	{
		bHasPreviousTransforms = bInHasPreviousTransforms;

		if (!bHasPreviousTransforms)
		{
			PerInstancePrevTransform.Empty();
		}
		else if (PerInstancePrevTransform.Num() != PerInstanceSMData.Num())
		{
			PerInstancePrevTransform.Empty(PerInstanceSMData.Num());
			Algo::Transform(PerInstanceSMData, PerInstancePrevTransform, [](const FInstancedStaticMeshInstanceData &Tfm) { return Tfm.Transform;});
		}
	}
}

// Per Instance Custom Data - Updating custom data for specific instance
bool UInstancedStaticMeshComponent::SetCustomDataValue(int32 InstanceIndex, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex) || CustomDataIndex < 0 || CustomDataIndex >= NumCustomDataFloats)
	{
		return false;
	}

	PrimitiveInstanceDataManager.CustomDataChanged(InstanceIndex);
	Modify();

	PerInstanceSMCustomData[InstanceIndex * NumCustomDataFloats + CustomDataIndex] = CustomDataValue;

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UInstancedStaticMeshComponent::SetCustomData(int32 InstanceIndex, TArrayView<const float> InCustomData, bool bMarkRenderStateDirty)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex) || InCustomData.Num() == 0)
	{
		return false;
	}

	if (bMarkRenderStateDirty)
	{
		Modify();
	}

	const int32 NumToCopy = FMath::Min(InCustomData.Num(), NumCustomDataFloats);
	FMemory::Memcpy(&PerInstanceSMCustomData[InstanceIndex * NumCustomDataFloats], InCustomData.GetData(), NumToCopy * InCustomData.GetTypeSize());

	PrimitiveInstanceDataManager.CustomDataChanged(InstanceIndex);

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UInstancedStaticMeshComponent::SetCustomData(int32 InstanceIndexStart, int32 InstanceIndexEnd, TConstArrayView<float> CustomDataFloats, bool bMarkRenderStateDirty)
{
	const int32 CustomDataIndexStart = InstanceIndexStart * NumCustomDataFloats;
	checkf(PerInstanceSMCustomData.Num() >= (CustomDataIndexStart + CustomDataFloats.Num()) && CustomDataFloats.GetTypeSize() == PerInstanceSMCustomData.GetTypeSize(), TEXT("CustomDataFloats and offset must fit within PerInstanceSMCustomData"));
	
	if (CustomDataFloats.IsEmpty())
	{
		return false;
	}
	
	if (!ensureMsgf(InstanceIndexStart <= InstanceIndexEnd && InstanceIndexStart >= 0 && InstanceIndexEnd < PerInstanceSMData.Num(), TEXT("InstanceIndexStart and InstanceIndexEnd need to be valid indices for PerInstanceSMData")))
	{
		return false;
	}

	if (!ensureMsgf((InstanceIndexEnd - InstanceIndexStart + 1) * NumCustomDataFloats == CustomDataFloats.Num(), TEXT("CustomDataFloats ArrayNum must match the number of Instances * NumCustomDataFloats")))
	{
		return false;
	}

	if (bMarkRenderStateDirty)
	{
		Modify();
	}
	
	FMemory::Memcpy(&PerInstanceSMCustomData[CustomDataIndexStart], CustomDataFloats.GetData(), CustomDataFloats.Num() * CustomDataFloats.GetTypeSize());

	for (int32 InstanceIndex = InstanceIndexStart; InstanceIndex <= InstanceIndexEnd; ++InstanceIndex)
	{
		PrimitiveInstanceDataManager.CustomDataChanged(InstanceIndex);
	}
	
	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}
	
	return true;
}

void UInstancedStaticMeshComponent::SetNumCustomDataFloats(int32 InNumCustomDataFloats)
{
	if (FMath::Max(InNumCustomDataFloats, 0) != NumCustomDataFloats)
	{
		NumCustomDataFloats = FMath::Max(InNumCustomDataFloats, 0);

		// Clear out and reinit to 0
		PerInstanceSMCustomData.Empty(PerInstanceSMData.Num() * NumCustomDataFloats);
		PerInstanceSMCustomData.SetNumZeroed(PerInstanceSMData.Num() * NumCustomDataFloats);

		PrimitiveInstanceDataManager.NumCustomDataChanged();
	}
}

bool UInstancedStaticMeshComponent::SupportsRemoveSwap() const
{
	return bSupportRemoveAtSwap || CVarISMForceRemoveAtSwap.GetValueOnGameThread() != 0;
}

bool UInstancedStaticMeshComponent::RemoveInstanceInternal(int32 InstanceIndex, bool InstanceAlreadyRemoved, bool bForceRemoveAtSwap, const bool bUpdateNavigation)
{
#if WITH_EDITOR
	DeletionState = InstanceAlreadyRemoved ? EInstanceDeletionReason::EntryAlreadyRemoved : EInstanceDeletionReason::EntryRemoval;
#endif

	// For performance we would prefer to use RemoveAtSwap() but some old code may be relying on the old
	// RemoveAt() behavior, since there was no explicit contract about how instance indices can move around.
	const bool bUseRemoveAtSwap = bForceRemoveAtSwap || bSupportRemoveAtSwap || CVarISMForceRemoveAtSwap.GetValueOnGameThread() != 0;

	// When InstanceAlreadyRemoved, the entry has already been removed from PerInstanceSMData
	// (e.g., by the property editor), so the index may be past the end of the already-shrunk
	// array. Account for this by checking against the original (pre-removal) count.
	const bool bValidInstance = InstanceAlreadyRemoved
		? (InstanceIndex >= 0 && InstanceIndex <= PerInstanceSMData.Num())
		: PerInstanceSMData.IsValidIndex(InstanceIndex);

	if (bValidInstance)
	{
		if (bUseRemoveAtSwap)
		{
			PrimitiveInstanceDataManager.RemoveAtSwap(InstanceIndex);
		}
		else
		{
			PrimitiveInstanceDataManager.RemoveAt(InstanceIndex);
		}

		// remove instance
		if (!InstanceAlreadyRemoved)
		{
			// Navigation update required explicitly requested and if the component is relevant to navigation and registered.
			const bool bNavigationUpdateRequired = bUpdateNavigation && bNavigationRelevant && IsRegistered();

			// Note that it is done before removing the instance since partial update needs
			// the instance's transform to dirty the area and full update will dirty the whole covered by the navigation bounds.
			if (bNavigationUpdateRequired && SupportsPartialNavigationUpdate())
			{
				PartialNavigationUpdate(InstanceIndex);
			}

			if (bUseRemoveAtSwap)
			{
				PerInstanceSMData.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);
				PerInstanceSMCustomData.RemoveAtSwap(InstanceIndex * NumCustomDataFloats, NumCustomDataFloats, EAllowShrinking::No);
			}
			else
			{
				PerInstanceSMData.RemoveAt(InstanceIndex);
				PerInstanceSMCustomData.RemoveAt(InstanceIndex * NumCustomDataFloats, NumCustomDataFloats);
			}

			if (bNavigationUpdateRequired)
			{
				// If it's the last instance, unregister the component since component with no instances are not registered.
				// (because of GetInstanceCount() > 0 in UInstancedStaticMeshComponent::IsNavigationRelevant())
				if (GetInstanceCount() == 0)
				{
					bNavigationRelevant = false;
					FNavigationSystem::UnregisterComponent(*this);
				}
				else if (!SupportsPartialNavigationUpdate())
				{
					FullNavigationUpdate();
				}
			}
		}
		else if (NumCustomDataFloats > 0)
		{
			// When the instance was already removed (e.g., by the property editor), only PerInstanceSMData
			// was updated; PerInstanceSMCustomData still contains the stale entry and must be cleaned up.
			// Use ordered removal to match the ordered removal performed on PerInstanceSMData by the caller.
			PerInstanceSMCustomData.RemoveAt(InstanceIndex * NumCustomDataFloats, NumCustomDataFloats);
		}
	}

	if (bHasPreviousTransforms && PerInstancePrevTransform.IsValidIndex(InstanceIndex))
	{
		if (bUseRemoveAtSwap)
		{
			PerInstancePrevTransform.RemoveAtSwap(InstanceIndex);
		}
		else
		{
			PerInstancePrevTransform.RemoveAt(InstanceIndex);
		}
	}
#if WITH_EDITOR
	// remove selection flag if array is filled in
	if (SelectedInstances.IsValidIndex(InstanceIndex))
	{
		if (bUseRemoveAtSwap)
		{
			SelectedInstances.RemoveAtSwap(InstanceIndex);
		}
		else
		{
			SelectedInstances.RemoveAt(InstanceIndex);
		}
	}
#endif

	const int32 LastInstanceIndex = PerInstanceSMData.Num();

	// update the physics state
	if (bPhysicsStateCreated && InstancePhysicsBodies->IsValidIndex(InstanceIndex))
	{
		if (bUseRemoveAtSwap)
		{
			InstancePhysicsBodies->RemoveAtSwap(InstanceIndex);
		}
		else
		{
			InstancePhysicsBodies->RemoveAt(InstanceIndex);
		}
	}

	// Notify that these instances have been removed/relocated
	if (FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.IsBound())
	{
		if (bUseRemoveAtSwap)
		{
			TArray<FInstancedStaticMeshDelegates::FInstanceIndexUpdateData, TInlineAllocator<2>> IndexUpdates;
			IndexUpdates.Add(FInstancedStaticMeshDelegates::FInstanceIndexUpdateData{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Removed, InstanceIndex });
			if (InstanceIndex != LastInstanceIndex)
			{
				// used swap remove, so the last index has been moved to the spot we removed from
				IndexUpdates.Add(FInstancedStaticMeshDelegates::FInstanceIndexUpdateData{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Relocated, InstanceIndex, LastInstanceIndex });
			}

			FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Broadcast(this, IndexUpdates);
		}
		else
		{
			TArray<FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> IndexUpdates;
			IndexUpdates.Reserve(IndexUpdates.Num() + 1 + (PerInstanceSMData.Num() - InstanceIndex));

			IndexUpdates.Add(FInstancedStaticMeshDelegates::FInstanceIndexUpdateData{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Removed, InstanceIndex });
			for (int32 MovedInstanceIndex = InstanceIndex; MovedInstanceIndex < PerInstanceSMData.Num(); ++MovedInstanceIndex)
			{
				// ISMs use standard remove, so each instance above our removal point is shuffled down by 1
				IndexUpdates.Add(FInstancedStaticMeshDelegates::FInstanceIndexUpdateData{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Relocated, MovedInstanceIndex, MovedInstanceIndex + 1 });
			}

			FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Broadcast(this, IndexUpdates);
		}
	}

#if WITH_EDITOR
	DeletionState = EInstanceDeletionReason::NotDeleting;
#endif

	InvalidateCachedBounds();
	return true;
}

bool UInstancedStaticMeshComponent::RemoveInstance(int32 InstanceIndex)
{
	return RemoveInstanceInternal(InstanceIndex, false);
}

bool UInstancedStaticMeshComponent::RemoveInstances(const TArray<int32>& InstancesToRemove)
{
	return RemoveInstances(InstancesToRemove, false /*bInstanceArrayAlreadySortedInReverseOrder*/);
}

bool UInstancedStaticMeshComponent::RemoveInstances(const TArray<int32>& InstancesToRemove, bool bInstanceArrayAlreadySortedInReverseOrder)
{
	if(InstancesToRemove.IsEmpty())
	{
		return false;
	}
	
	auto RemoveInstancedWithSortedArray = [this](const TArray<int32>& SortedInstancesToRemove) -> bool
	{
		if (!PerInstanceSMData.IsValidIndex(SortedInstancesToRemove[0]) || !PerInstanceSMData.IsValidIndex(SortedInstancesToRemove.Last()))
		{
			return false;
		}

		for (const int32 InstanceIndex : SortedInstancesToRemove)
		{
			RemoveInstanceInternal(InstanceIndex, false);
		}
		return true;
	};

	bool bSuccess = false;
	if (bInstanceArrayAlreadySortedInReverseOrder)
	{
		bSuccess = RemoveInstancedWithSortedArray(InstancesToRemove);
	}
	else
	{
		// Sort so Remove doesn't alter the indices of items still to remove
		TArray<int32> SortedInstancesToRemove = InstancesToRemove;
		SortedInstancesToRemove.Sort(TGreater<int32>());
		bSuccess = RemoveInstancedWithSortedArray(SortedInstancesToRemove);
	}

	return bSuccess;
}

bool UInstancedStaticMeshComponent::GetInstanceTransform(int32 InstanceIndex, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	const FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[InstanceIndex];

	OutInstanceTransform = FTransform(InstanceData.Transform);
	if (bWorldSpace)
	{
		OutInstanceTransform = OutInstanceTransform * GetComponentTransform();
	}

	return true;
}

bool UInstancedStaticMeshComponent::GetInstancePrevTransform(int32 InstanceIndex, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	if (!PerInstancePrevTransform.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	const FMatrix& InstanceData = PerInstancePrevTransform[InstanceIndex];

	OutInstanceTransform = FTransform(InstanceData);
	if (bWorldSpace)
	{
		OutInstanceTransform = OutInstanceTransform * GetComponentTransform();
	}

	return true;
}


void UInstancedStaticMeshComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	// We are handling the physics move in UpdateComponentTransform below, so don't handle it at higher levels
	Super::OnUpdateTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);
	UpdateComponentTransform(UpdateTransformFlags, Teleport);
}

void UInstancedStaticMeshComponent::UpdateComponentTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	const bool bTeleport = TeleportEnumToFlag(Teleport);
	const bool bDoPartialNavigationUpdate = IsNavigationRelevant() && SupportsPartialNavigationUpdate();
	const bool bUpdateBodies = bPhysicsStateCreated && !(EUpdateTransformFlags::SkipPhysicsUpdate & UpdateTransformFlags);

	if (bDoPartialNavigationUpdate || bUpdateBodies)
	{
		TArray<FTransform> NavigationUpdateTransforms;
		if (bDoPartialNavigationUpdate)
		{
			NavigationUpdateTransforms.Reserve(PerInstanceSMData.Num());
		}

		for (int32 i = 0; i < PerInstanceSMData.Num(); i++)
		{
			const FTransform NewInstanceTransform(FTransform(PerInstanceSMData[i].Transform) * GetComponentTransform());

			// Append instance's previous and new transforms to dirty both areas
			if (bDoPartialNavigationUpdate)
			{
				NavigationUpdateTransforms.Append(
					{
						FTransform(PerInstanceSMData[i].Transform) * PreviousComponentTransform,
						NewInstanceTransform
					});
			}

			// Send new transforms to physics
			if (bUpdateBodies)
			{
				UpdateInstanceBodyTransform(i, NewInstanceTransform, bTeleport);
			}
		}

		// Only handle partial updates since modifying the transform is handled by base class (i.e. SceneComponent).
		if (bDoPartialNavigationUpdate)
		{
			PartialNavigationUpdates(NavigationUpdateTransforms);
		}
	}

	// TODO: bTeleport???
	PrimitiveInstanceDataManager.PrimitiveTransformChanged();

	PreviousComponentTransform = GetComponentTransform();
}

void UInstancedStaticMeshComponent::RecreateInstanceBody(int32 InstanceBodyIndex)
{
	if (PerInstanceSMData.IsValidIndex(InstanceBodyIndex))
	{
		const FTransform InstanceTM = FTransform(PerInstanceSMData[InstanceBodyIndex].Transform) * GetComponentTransform();
		InstancePhysicsBodies->Recreate(InstanceBodyIndex, InstanceTM);
	}
}

void UInstancedStaticMeshComponent::UpdateInstanceBodyTransform(int32 InstanceIndex, const FTransform& WorldSpaceInstanceTransform, bool bTeleport)
{
	check(bPhysicsStateCreated);

	InstancePhysicsBodies->UpdateTransform(InstanceIndex, WorldSpaceInstanceTransform, bTeleport);
}

bool UInstancedStaticMeshComponent::UpdateInstanceTransform(const int32 InstanceIndex, const FTransform& NewInstanceTransform, const bool bWorldSpace, const bool bMarkRenderStateDirty, const bool bTeleport)
{
	if (!PerInstanceSMData.IsValidIndex(InstanceIndex))
	{
		return false;
	}

	Modify();
	InvalidateCachedBounds();

	FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[InstanceIndex];

	const FTransform PreviousTransform(InstanceData.Transform);

	// TODO: Computing LocalTransform is useless when we're updating the world location for the entire mesh.
	// Should find some way around this for performance.

	// Render data uses local transform of the instance
	const FTransform LocalTransform = bWorldSpace ? NewInstanceTransform.GetRelativeTransform(GetComponentTransform()) : NewInstanceTransform;
	InstanceData.Transform = LocalTransform.ToMatrixWithScale();
	PrimitiveInstanceDataManager.TransformChanged(InstanceIndex);

	if (bPhysicsStateCreated)
	{
		// Physics uses world transform of the instance
		const FTransform WorldTransform = bWorldSpace ? NewInstanceTransform : (LocalTransform * GetComponentTransform());
		UpdateInstanceBodyTransform(InstanceIndex, WorldTransform, bTeleport);
	}

	// Navigation update required if the component is relevant to navigation and registered.
	if (bNavigationRelevant && IsRegistered())
	{
		if (SupportsPartialNavigationUpdate())
		{
			// Perform partial update after instance gets updated since we need NavigationBounds using up to date instances.
			// Append instance's previous and new transforms to dirty both areas
			PartialNavigationUpdates(
				{
					PreviousTransform * GetComponentTransform(),
					(bWorldSpace ? NewInstanceTransform : NewInstanceTransform * GetComponentTransform())
				} );
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UInstancedStaticMeshComponent::BatchUpdateInstancesTransforms(int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, const TArray<FTransform>& NewInstancesPrevTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	// Number of current and prev transforms must match 
	check(NewInstancesTransforms.Num() == NewInstancesPrevTransforms.Num());

	// Early out if trying to update an invalid range
	if (!PerInstanceSMData.IsValidIndex(StartInstanceIndex) || !PerInstanceSMData.IsValidIndex(StartInstanceIndex + NewInstancesTransforms.Num() - 1))
	{
		return false;
	}

	// If the new transform index range is ok for PerInstanceSMData, it must also be ok for PerInstancePrevTransform
	check(PerInstancePrevTransform.IsValidIndex(StartInstanceIndex) && PerInstancePrevTransform.IsValidIndex(StartInstanceIndex + NewInstancesPrevTransforms.Num() - 1));

	Modify();
	InvalidateCachedBounds();

	// Navigation update required if the component is relevant to navigation and registered.
	const bool bNavigationUpdateRequired = bNavigationRelevant && IsRegistered();
	const bool bPartialNavigationUpdateRequired = bNavigationUpdateRequired && SupportsPartialNavigationUpdate();

	TArray<FTransform> NavigationUpdateTransforms;
	if (bPartialNavigationUpdateRequired)
	{
		NavigationUpdateTransforms.Reserve(NewInstancesTransforms.Num());
	}

	for (int32 Index = 0; Index < NewInstancesTransforms.Num(); Index++)
	{
		const int32 InstanceIndex = StartInstanceIndex + Index;

		const FTransform& NewInstanceTransform = NewInstancesTransforms[Index];
		const FTransform& NewInstancePrevTransform = NewInstancesPrevTransforms[Index];

		FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[InstanceIndex];
		FMatrix& PrevInstanceData = PerInstancePrevTransform[InstanceIndex];

		// Append instance's previous and new transforms to dirty both areas
		if (bPartialNavigationUpdateRequired)
		{
			NavigationUpdateTransforms.Append(
			{
				FTransform(InstanceData.Transform) * GetComponentTransform(),
				(bWorldSpace ? NewInstanceTransform : NewInstanceTransform * GetComponentTransform())
			});
		}
		
		// TODO: Computing LocalTransform is useless when we're updating the world location for the entire mesh.
		// Should find some way around this for performance.

		// Render data uses local transform of the instance
		FTransform LocalTransform = bWorldSpace ? NewInstanceTransform.GetRelativeTransform(GetComponentTransform()) : NewInstanceTransform;
		InstanceData.Transform = LocalTransform.ToMatrixWithScale();
		PrimitiveInstanceDataManager.TransformChanged(InstanceIndex);

		FTransform LocalPrevTransform = bWorldSpace ? NewInstancePrevTransform.GetRelativeTransform(GetComponentTransform()) : NewInstancePrevTransform;
		PrevInstanceData = LocalPrevTransform.ToMatrixWithScale();

		if (bPhysicsStateCreated)
		{
			// Physics uses world transform of the instance
			FTransform WorldTransform = bWorldSpace ? NewInstanceTransform : (LocalTransform * GetComponentTransform());
			UpdateInstanceBodyTransform(InstanceIndex, WorldTransform, bTeleport);
		}
	}

	if (bNavigationUpdateRequired)
	{
		if (bPartialNavigationUpdateRequired)
		{
			PartialNavigationUpdates(NavigationUpdateTransforms);
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UInstancedStaticMeshComponent::BatchUpdateInstancesTransforms(int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	return BatchUpdateInstancesTransformsInternal(StartInstanceIndex, MakeArrayView(NewInstancesTransforms), bWorldSpace, bMarkRenderStateDirty, bTeleport);
}

bool UInstancedStaticMeshComponent::BatchUpdateInstancesTransforms(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	return BatchUpdateInstancesTransformsInternal(StartInstanceIndex, NewInstancesTransforms, bWorldSpace, bMarkRenderStateDirty, bTeleport);
}

bool UInstancedStaticMeshComponent::BatchUpdateInstancesTransformsInternal(int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UInstancedStaticMeshComponent_BatchUpdateInstancesTransforms);
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UInstancedStaticMeshComponent::BatchUpdateInstancesTransforms");

	if (!PerInstanceSMData.IsValidIndex(StartInstanceIndex) || !PerInstanceSMData.IsValidIndex(StartInstanceIndex + NewInstancesTransforms.Num() - 1))
	{
		return false;
	}

	Modify();
	InvalidateCachedBounds();

	// Navigation update required if the component is relevant to navigation and registered.
	const bool bNavigationUpdateRequired = bNavigationRelevant && IsRegistered();
	const bool bPartialNavigationUpdateRequired = bNavigationUpdateRequired && SupportsPartialNavigationUpdate();

	TArray<FTransform> NavigationUpdateTransforms;
	if (bPartialNavigationUpdateRequired)
	{
		// Reserve enough space for previous and new transforms
		NavigationUpdateTransforms.Reserve(NewInstancesTransforms.Num() * 2);
	}

	int32 InstanceIndex = StartInstanceIndex;
	for (const FTransform& NewInstanceTransform : NewInstancesTransforms)
	{
		FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[InstanceIndex];

		// Append instance's previous and new transforms to dirty both areas
		if (bPartialNavigationUpdateRequired)
		{
			NavigationUpdateTransforms.Append(
			{
				FTransform(InstanceData.Transform) * GetComponentTransform(),
				(bWorldSpace ? NewInstanceTransform : NewInstanceTransform * GetComponentTransform())
			});
		}
		
		// TODO: Computing LocalTransform is useless when we're updating the world location for the entire mesh.
		// Should find some way around this for performance.

		// Render data uses local transform of the instance
		FTransform LocalTransform = bWorldSpace ? NewInstanceTransform.GetRelativeTransform(GetComponentTransform()) : NewInstanceTransform;
		InstanceData.Transform = LocalTransform.ToMatrixWithScale();
		PrimitiveInstanceDataManager.TransformChanged(InstanceIndex);

		if (bPhysicsStateCreated)
		{
			// Physics uses world transform of the instance
			FTransform WorldTransform = bWorldSpace ? NewInstanceTransform : (LocalTransform * GetComponentTransform());
			UpdateInstanceBodyTransform(InstanceIndex, WorldTransform, bTeleport);
		}

		InstanceIndex++;
	}

	if (bNavigationUpdateRequired)
	{
		if (bPartialNavigationUpdateRequired)
		{
			PartialNavigationUpdates(NavigationUpdateTransforms);
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UInstancedStaticMeshComponent::BatchUpdateInstancesTransform(int32 StartInstanceIndex, int32 NumInstances, const FTransform& NewInstancesTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	if(!PerInstanceSMData.IsValidIndex(StartInstanceIndex) || !PerInstanceSMData.IsValidIndex(StartInstanceIndex + NumInstances - 1))
	{
		return false;
	}

	Modify();
	InvalidateCachedBounds();

	// Navigation update required if the component is relevant to navigation and registered.
	const bool bNavigationUpdateRequired = bNavigationRelevant && IsRegistered();
	const bool bPartialNavigationUpdateRequired = bNavigationUpdateRequired && SupportsPartialNavigationUpdate();

	TArray<FTransform> NavigationUpdateTransforms;
	if (bPartialNavigationUpdateRequired)
	{
		NavigationUpdateTransforms.Reserve(NumInstances);
	}
	
	int32 EndInstanceIndex = StartInstanceIndex + NumInstances;
	for(int32 InstanceIndex = StartInstanceIndex; InstanceIndex < EndInstanceIndex; ++InstanceIndex)
	{
		FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[InstanceIndex];

		// Append instance's previous and new transforms to dirty both areas
		if (bPartialNavigationUpdateRequired)
		{
			NavigationUpdateTransforms.Append(
			{
				FTransform(InstanceData.Transform) * GetComponentTransform(),
				(bWorldSpace ? NewInstancesTransform : NewInstancesTransform * GetComponentTransform())
			});
		}
		
		// TODO: Computing LocalTransform is useless when we're updating the world location for the entire mesh.
		// Should find some way around this for performance.

		// Render data uses local transform of the instance
		FTransform LocalTransform = bWorldSpace ? NewInstancesTransform.GetRelativeTransform(GetComponentTransform()) : NewInstancesTransform;
		InstanceData.Transform = LocalTransform.ToMatrixWithScale();
		PrimitiveInstanceDataManager.TransformChanged(InstanceIndex);

		if(bPhysicsStateCreated)
		{
			// Physics uses world transform of the instance
			FTransform WorldTransform = bWorldSpace ? NewInstancesTransform : (LocalTransform * GetComponentTransform());
			UpdateInstanceBodyTransform(InstanceIndex, WorldTransform, bTeleport);
		}
	}

	if (bNavigationUpdateRequired)
	{
		if (bPartialNavigationUpdateRequired)
		{
			PartialNavigationUpdates(NavigationUpdateTransforms);
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	if(bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

bool UInstancedStaticMeshComponent::BatchUpdateInstancesData(int32 StartInstanceIndex, int32 NumInstances, FInstancedStaticMeshInstanceData* StartInstanceData, bool bMarkRenderStateDirty, bool bTeleport)
{
	if (!PerInstanceSMData.IsValidIndex(StartInstanceIndex) || !PerInstanceSMData.IsValidIndex(StartInstanceIndex + NumInstances - 1))
	{
		return false;
	}

	Modify();
	InvalidateCachedBounds();

	// Navigation update required if the component is relevant to navigation and registered.
	const bool bNavigationUpdateRequired = bNavigationRelevant && IsRegistered();
	const bool bPartialNavigationUpdateRequired = bNavigationUpdateRequired && SupportsPartialNavigationUpdate();

	TArray<FTransform> NavigationUpdateTransforms;
	if (bPartialNavigationUpdateRequired)
	{
		NavigationUpdateTransforms.Reserve(NumInstances);
	}
	
	for (int32 Index = 0; Index < NumInstances; ++Index)
	{
		int32 InstanceIndex = StartInstanceIndex + Index;
		FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[InstanceIndex];

		// Append instance's previous and new transforms to dirty both areas
		if (bPartialNavigationUpdateRequired)
		{
			NavigationUpdateTransforms.Append(
			{
				FTransform(InstanceData.Transform) * GetComponentTransform(),
				FTransform(StartInstanceData[Index].Transform) * GetComponentTransform()
			});
		}
		
		InstanceData = StartInstanceData[Index];
		PrimitiveInstanceDataManager.TransformChanged(InstanceIndex);

		if (bPhysicsStateCreated)
		{
			// Physics uses world transform of the instance
			FTransform WorldTransform = FTransform(InstanceData.Transform) * GetComponentTransform();
			UpdateInstanceBodyTransform(InstanceIndex, WorldTransform, bTeleport);
		}
	}

	if (bNavigationUpdateRequired)
	{
		if (bPartialNavigationUpdateRequired)
		{
			PartialNavigationUpdates(NavigationUpdateTransforms);
		}
		else
		{
			FullNavigationUpdate();
		}
	}

	if (bMarkRenderStateDirty)
	{
		MarkRenderStateDirty();
	}

	return true;
}

TArray<int32> UInstancedStaticMeshComponent::GetInstancesOverlappingSphere(const FVector& Center, float Radius, bool bSphereInWorldSpace) const
{
	TArray<int32> Result;

	if (UStaticMesh* Mesh = GetStaticMesh())
	{
		FSphere Sphere(Center, Radius);
		if (bSphereInWorldSpace)
		{
			Sphere = Sphere.TransformBy(GetComponentTransform().Inverse());
		}

		FSphere StaticMeshSphere = Mesh->GetBounds().GetSphere();

		for (int32 Index = 0; Index < PerInstanceSMData.Num(); Index++)
		{
			if (Sphere.Intersects(StaticMeshSphere.TransformBy(PerInstanceSMData[Index].Transform)))
			{
				Result.Add(Index);
			}
		}
	}

	return Result;
}

TArray<int32> UInstancedStaticMeshComponent::GetInstancesOverlappingBox(const FBox& InBox, bool bBoxInWorldSpace) const
{
	TArray<int32> Result;

	if (UStaticMesh* Mesh = GetStaticMesh())
	{
		FBox Box(InBox);
		if (bBoxInWorldSpace)
		{
			Box = Box.TransformBy(GetComponentTransform().Inverse());
		}

		FBox StaticMeshBox = Mesh->GetBounds().GetBox();
		for (int32 Index = 0; Index < PerInstanceSMData.Num(); Index++)
		{
			if (Box.Intersect(StaticMeshBox.TransformBy(PerInstanceSMData[Index].Transform)))
			{
				Result.Add(Index);
			}
		}
	}
	return Result;
}

bool UInstancedStaticMeshComponent::ShouldCreatePhysicsState() const
{
	return !bDisableCollision && (IsRegistered() || (IsPreRegistering() && AllowsAsyncPhysicsStateCreation())) && !IsBeingDestroyed() && GetStaticMesh() && !GetStaticMesh()->IsCompiling() && (bAlwaysCreatePhysicsState || IsCollisionEnabled());
}

float UInstancedStaticMeshComponent::GetTextureStreamingTransformScale() const
{
	// By default if there are no per instance data, use a scale of 1.
	// This is required because some derived class use the instancing system without filling the per instance data. (like landscape grass)
	// In those cases, we assume the instance are spreaded across the bounds with a scale of 1.
	float TransformScale = 1.f; 

	if (PerInstanceSMData.Num() > 0)
	{
		TransformScale = Super::GetTextureStreamingTransformScale();

		float WeightedAxisScaleSum = 0;
		float WeightSum = 0;

		for (int32 InstanceIndex = 0; InstanceIndex < PerInstanceSMData.Num(); InstanceIndex++)
		{
			const float AxisScale = PerInstanceSMData[InstanceIndex].Transform.GetMaximumAxisScale();
			const float Weight = AxisScale; // The weight is the axis scale since we want to weight by surface coverage.
			WeightedAxisScaleSum += AxisScale * Weight;
			WeightSum += Weight;
		}

		if (WeightSum > UE_SMALL_NUMBER)
		{
			TransformScale *= WeightedAxisScaleSum / WeightSum;
		}
	}
	return TransformScale;
}

bool UInstancedStaticMeshComponent::GetMaterialStreamingData(int32 MaterialIndex, FPrimitiveMaterialInfo& MaterialData) const
{
	// Same thing as StaticMesh but we take the full bounds to cover the instances.
	if (GetStaticMesh())
	{
		MaterialData.Material = GetMaterial(MaterialIndex);
		MaterialData.UVChannelData = GetStaticMesh()->GetUVChannelData(MaterialIndex);
		MaterialData.PackedRelativeBox = PackedRelativeBox_Identity;
	}
	return MaterialData.IsValid();
}

bool UInstancedStaticMeshComponent::BuildTextureStreamingDataImpl(ETextureStreamingBuildType BuildType, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<FGuid>& DependentResources, bool& bOutSupportsBuildTextureStreamingData)
{
#if WITH_EDITORONLY_DATA // Only rebuild the data in editor 
	if (GetInstanceCount() > 0)
	{
		return Super::BuildTextureStreamingDataImpl(BuildType, QualityLevel, FeatureLevel, DependentResources, bOutSupportsBuildTextureStreamingData);
	}
#endif
	return true;
}

void UInstancedStaticMeshComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	// Don't only look the instance count but also if the bound is valid, as derived classes might not set PerInstanceSMData.
	if (GetInstanceCount() > 0 || Bounds.SphereRadius > 0)
	{
		return Super::GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
	}
}

void UInstancedStaticMeshComponent::ClearInstances()
{
	InvalidateCachedBounds();

#if WITH_EDITOR
	DeletionState = EInstanceDeletionReason::Clearing;
#endif

	const int32 PrevNumInstances = GetInstanceCount();

	// Navigation update required if the component is relevant to navigation and registered.
	// Note that it is done before removing the instance since partial update needs
	// the instances transform to dirty all the areas.
	const bool bNavigationUpdateRequired = bNavigationRelevant && IsRegistered();
	if (bNavigationUpdateRequired && SupportsPartialNavigationUpdate())
	{
		PartialNavigateUpdateForCurrentInstances();
	}

	// Clear all the per-instance data
	PerInstanceSMData.Empty();
	PerInstanceSMCustomData.Empty();
	InstanceReorderTable.Empty();

	ProxySize = 0;

	// Release any physics representations
	ClearAllInstanceBodies();

	// Force full recreation of the instance data when proxy is created
	PrimitiveInstanceDataManager.Invalidate(PerInstanceSMData.Num());

	// Notify that these instances have been cleared
	if (FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.IsBound())
	{
		FInstancedStaticMeshDelegates::FInstanceIndexUpdateData IndexUpdate{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Cleared, PrevNumInstances - 1 };
		FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Broadcast(this, MakeArrayView(&IndexUpdate, 1));
	}

	if (bNavigationUpdateRequired)
	{
		// Unregister since components with no instances are not registered.
		FNavigationSystem::UnregisterComponent(*this);
		bNavigationRelevant = false;
	}

#if WITH_EDITOR
	DeletionState = EInstanceDeletionReason::NotDeleting;
#endif
}

int32 UInstancedStaticMeshComponent::GetInstanceCount() const
{
	return PerInstanceSMData.Num();
}

bool UInstancedStaticMeshComponent::IsValidInstance(int32 InstanceIndex) const
{
	return PerInstanceSMData.IsValidIndex(InstanceIndex);
}

void UInstancedStaticMeshComponent::ForEachInstanceMatrix(TFunctionRef<void(const FMatrix&)> Func) const
{
	for (const FInstancedStaticMeshInstanceData& InstanceData : PerInstanceSMData)
	{
		Func(InstanceData.Transform);
	}
}

void UInstancedStaticMeshComponent::SetCullDistances(int32 StartCullDistance, int32 EndCullDistance)
{
	if (InstanceStartCullDistance != StartCullDistance || InstanceEndCullDistance != EndCullDistance)
	{
		InstanceStartCullDistance = StartCullDistance;
		InstanceEndCullDistance = EndCullDistance;

		if (GetScene() && SceneProxy)
		{
			GetScene()->UpdateInstanceCullDistance(this, StartCullDistance, EndCullDistance);
		}
	}
}

TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe> UInstancedStaticMeshComponent::GetOrCreateInstanceDataSceneProxy()
{
	if (FSceneInterface *Scene = GetScene())
	{
		return PrimitiveInstanceDataManager.GetOrCreateProxy(Scene->GetFeatureLevel());
	}
	return nullptr;
}

const TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe>&  UInstancedStaticMeshComponent::GetInstanceDataSceneProxy() const
{
	return PrimitiveInstanceDataManager.GetProxy();
}

void UInstancedStaticMeshComponent::SetBakedLightingDataChanged(int32 InInstanceIndex)
{
	PrimitiveInstanceDataManager.BakedLightingDataChanged(InInstanceIndex);
}

void UInstancedStaticMeshComponent::SetBakedLightingDataChangedAll()
{
	PrimitiveInstanceDataManager.BakedLightingDataChangedAll();
}

void UInstancedStaticMeshComponent::InvalidateInstanceDataTracking()
{
	PrimitiveInstanceDataManager.Invalidate(GetNumInstances());
}

void UInstancedStaticMeshComponent::SetupNewInstanceData(FInstancedStaticMeshInstanceData& InOutNewInstanceData, int32 InInstanceIndex, const FTransform& InInstanceTransform)
{
	InOutNewInstanceData.Transform = InInstanceTransform.ToMatrixWithScale();
	PrimitiveInstanceDataManager.TransformChanged(InInstanceIndex);

	if (bPhysicsStateCreated && !IsAsyncDestroyPhysicsStateRunning())
	{
		const FTransform WorldTransform = InInstanceTransform * GetComponentTransform();
		InstancePhysicsBodies->Insert(InInstanceIndex, WorldTransform);
	}
}

static bool ComponentRequestsCPUAccess(UInstancedStaticMeshComponent* InComponent, ERHIFeatureLevel::Type FeatureLevel)
{
	bool bNeedsCPUAccess = false;

	// Ray tracing needs instance transforms on CPU
	bNeedsCPUAccess |= IsRayTracingEnabled();

	const UStaticMesh* StaticMesh = InComponent->GetStaticMesh();

	// Check mesh distance fields
	if (StaticMesh)
	{
		if ((FeatureLevel > ERHIFeatureLevel::ES3_1) || IsMobileDistanceFieldEnabled(GMaxRHIShaderPlatform))
		{
			// Mirror the conditions used in the FPrimitiveSceneProxy since these are used in IncludePrimitiveInDistanceFieldSceneData in RendererScene.cpp to filter the 
			// primitives that are included in the distance field scene. If these are not in sync, the host copy may be discarded and thus crashing in the distance field update.
			auto ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
			bNeedsCPUAccess |= PrimitiveNeedsDistanceFieldSceneData(
				ShouldAllPrimitivesHaveDistanceField(ShaderPlatform),
				/* bCastsDynamicIndirectShadow */ InComponent->bCastDynamicShadow && InComponent->CastShadow && InComponent->bCastDistanceFieldIndirectShadow && InComponent->Mobility != EComponentMobility::Static && InComponent->FirstPersonPrimitiveType != EFirstPersonPrimitiveType::FirstPerson,
				InComponent->bAffectDistanceFieldLighting,
				true, /* conservatively overestimate DrawInGame - it has complex logic in the Proxy. */
				InComponent->bCastHiddenShadow,
				/* bCastsDynamicShadow */ InComponent->bCastDynamicShadow && InComponent->CastShadow && !InComponent->GetShadowIndirectOnly(),
				InComponent->bAffectDynamicIndirectLighting,
				InComponent->bAffectIndirectLightingWhileHidden);
		}

		// Check Nanite
		if (FeatureLevel >= ERHIFeatureLevel::SM5)
		{
			// TODO: Call UseNanite(GetScene()->GetShaderPlatform())?
			//static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite"));

		#if WITH_EDITOR
			const bool bHasNaniteData = StaticMesh->IsNaniteEnabled();
		#else
			const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
			const bool bHasNaniteData = RenderData->HasValidNaniteData();
		#endif

			bNeedsCPUAccess |= bHasNaniteData;
		}
	}

	return bNeedsCPUAccess;
}

void UInstancedStaticMeshComponent::GetInstancesMinMaxScale(FVector& MinScale, FVector& MaxScale) const
{
	if (PerInstanceSMData.Num() > 0)
	{
		MinScale = FVector(MAX_flt);
		MaxScale = FVector(-MAX_flt);

		for (int32 i = 0; i < PerInstanceSMData.Num(); ++i)
		{
			const FInstancedStaticMeshInstanceData& InstanceData = PerInstanceSMData[i];
			FVector ScaleVector = InstanceData.Transform.GetScaleVector();

			MinScale = MinScale.ComponentMin(ScaleVector);
			MaxScale = MaxScale.ComponentMax(ScaleVector);
		}
	}
	else
	{
		MinScale = FVector(1.0f);
		MaxScale = FVector(1.0f);
	}
}

void UInstancedStaticMeshComponent::OnRegister()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::OnRegister())
	Super::OnRegister();
	
	PreviousComponentTransform = GetComponentTransform();

	if (FApp::CanEverRender() && !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		InitializeInstancingRandomSeed();
		// Propagate the current number of instances to the manager, because of the multifarious ways the engine shoves data into the properties it is possible for the count to get out of sync.
		// At this point we need to let the manager reset if needed, also at this point we may assume that we have no idea of the state of individual members.
		PrimitiveInstanceDataManager.OnRegister(PerInstanceSMData.Num());
	}

	// A component using partial updates will not dirty the whole area covered by the navigation bound (base class default behavior)
	// so it needs to dirty areas around its current instances that are getting added to the scene.
	if (bNavigationRelevant && SupportsPartialNavigationUpdate())
	{
		PartialNavigateUpdateForCurrentInstances();
	}
}

void UInstancedStaticMeshComponent::OnUnregister()
{
	// A component using partial updates will not dirty the whole area covered by the navigation bound (base class default behavior)
	// so it needs to dirty areas around its current instances that are getting removed from the scene.
	if (bNavigationRelevant && SupportsPartialNavigationUpdate())
	{
		PartialNavigateUpdateForCurrentInstances();
	}

	Super::OnUnregister();
}

void UInstancedStaticMeshComponent::OnActorEnableCollisionChanged()
{
	Super::OnActorEnableCollisionChanged();
	
	for (FBodyInstance* InstanceBody : *InstancePhysicsBodies)
	{
		if (InstanceBody)
		{
			InstanceBody->UpdatePhysicsFilterData();
		}
	}
}

#if WITH_EDITOR

bool UInstancedStaticMeshComponent::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty)
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMData) ||
			InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMCustomData) ||
			InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, NumCustomDataFloats))
		{
			return !ShouldInheritPerInstanceData();
		}
	}

	return true;
}

bool UInstancedStaticMeshComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	if (bConsiderOnlyBSP)
	{
		return false;
	}

	for (int32 Index = 0; Index < PerInstanceSMData.Num(); ++Index)
	{
		if (IsInstanceTouchingSelectionBox(Index, InSelBBox, bMustEncompassEntireComponent) && !bMustEncompassEntireComponent)
		{
			return true;
		}
		else if (bMustEncompassEntireComponent)
		{
			return false;
		}
	}

	if (PerInstanceSMData.Num() == 0)
	{
		return Super::ComponentIsTouchingSelectionBox(InSelBBox, bConsiderOnlyBSP, bMustEncompassEntireComponent);
	}

	return bMustEncompassEntireComponent;
}

bool UInstancedStaticMeshComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	if (bConsiderOnlyBSP)
	{
		return false;
	}

	for (int32 Index = 0; Index < PerInstanceSMData.Num(); ++Index)
	{
		if (IsInstanceTouchingSelectionFrustum(Index, InFrustum, bMustEncompassEntireComponent) && !bMustEncompassEntireComponent)
		{
			return true;
		}
		else if (bMustEncompassEntireComponent)
		{
			return false;
		}
	}

	if (PerInstanceSMData.Num() == 0)
	{
		return Super::ComponentIsTouchingSelectionFrustum(InFrustum, bConsiderOnlyBSP, bMustEncompassEntireComponent);		
	}

	return bMustEncompassEntireComponent;
}

bool UInstancedStaticMeshComponent::IsInstanceTouchingSelectionBox(int32 InstanceIndex, const FBox& InBox, const bool bMustEncompassEntireInstance) const
{
	if (PerInstanceSMData.IsValidIndex(InstanceIndex))
	{
		const UStaticMesh* Mesh = GetStaticMesh();
		if (Mesh && Mesh->HasValidRenderData())
		{
			const FBoxSphereBounds& MeshBoundingBox = Mesh->GetBounds();

			FTransform InstanceTransfrom;
			const bool bWorldSpace = true;
			GetInstanceTransform(InstanceIndex, InstanceTransfrom, bWorldSpace);

			const FBox& InstanceBoundingBox = MeshBoundingBox.TransformBy(InstanceTransfrom).GetBox();

			// If the bounds are fully contains assume the static mesh instance is fully contained
			if (InstanceBoundingBox.IsInside(InBox))
			{
				return true;
			}

			// Test the bounding box first to avoid heavy computation.
			if (InBox.Intersect(InstanceBoundingBox))
			{
				TArray<FVector> Vertex;

				const int32 MinLODIdx = GetStaticMesh()->GetMinLODIdx();
				const FStaticMeshLODResources* LODModel = GetStaticMesh()->GetRenderData()->GetCurrentFirstLOD(MinLODIdx);
				if (!LODModel)
				{
					return false;
				}

				const FIndexArrayView Indices = LODModel->IndexBuffer.GetArrayView();
				const FPositionVertexBuffer& Vertices = LODModel->VertexBuffers.PositionVertexBuffer;

				for (const FStaticMeshSection& Section : LODModel->Sections)
				{
					// Iterate over each triangle.
					for (int32 TriangleIndex = 0; TriangleIndex < (int32)Section.NumTriangles; TriangleIndex++)
					{
						Vertex.Empty(3);

						int32 FirstIndex = TriangleIndex * 3 + Section.FirstIndex;
						for (int32 i = 0; i < 3; i++)
						{
							int32 VertexIndex = Indices[FirstIndex + i];
							FVector LocalPosition(Vertices.VertexPosition(VertexIndex));
							Vertex.Emplace(InstanceTransfrom.TransformPosition(LocalPosition));
						}

						// Check if the triangle is colliding with the bounding box.
						FSeparatingAxisPointCheck ThePointCheck(Vertex, InBox.GetCenter(), InBox.GetExtent(), false);
						if (!bMustEncompassEntireInstance && ThePointCheck.bHit)
						{
							// Needn't encompass entire component: any intersection, we consider as touching
							return true;
						}
						else if (bMustEncompassEntireInstance && !ThePointCheck.bHit)
						{
							// Must encompass entire component: any non intersection, we consider as not touching
							return false;
						}
					}
				}

				// Either:
				// a) It must encompass the entire component and all points were intersected (return true), or;
				// b) It needn't encompass the entire component but no points were intersected (return false)
				return bMustEncompassEntireInstance;
			}
		}
	}

	return false;
}

bool UInstancedStaticMeshComponent::IsInstanceTouchingSelectionFrustum(int32 InstanceIndex, const FConvexVolume& InFrustum, const bool bMustEncompassEntireInstance) const
{
	if (PerInstanceSMData.IsValidIndex(InstanceIndex))
	{
		const UStaticMesh* Mesh = GetStaticMesh();
		if (Mesh && Mesh->HasValidRenderData())
		{
			const FBoxSphereBounds& MeshBoundingBox = Mesh->GetBounds();

			FTransform InstanceTransfrom;
			const bool bWorldSpace = true;
			GetInstanceTransform(InstanceIndex, InstanceTransfrom, bWorldSpace);

			const FBoxSphereBounds& InstanceBoundingBox = MeshBoundingBox.TransformBy(InstanceTransfrom);

			// Test the bounding box first to avoid heavy computation.
			bool bIsFullyInside = false;
			if (InFrustum.IntersectBox(InstanceBoundingBox.Origin, InstanceBoundingBox.BoxExtent, bIsFullyInside))
			{
				// If the bounds are fully contains assume the static mesh instance is fully contained
				if (bIsFullyInside)
				{
					return true;
				}

				const int32 MinLODIdx = GetStaticMesh()->GetMinLODIdx();
				const FStaticMeshLODResources* LODModel = GetStaticMesh()->GetRenderData()->GetCurrentFirstLOD(MinLODIdx);
				if (!LODModel)
				{
					return false;
				}

				const FIndexArrayView Indices = LODModel->IndexBuffer.GetArrayView();
				const FPositionVertexBuffer& Vertices = LODModel->VertexBuffers.PositionVertexBuffer;

				for (const FStaticMeshSection& Section : LODModel->Sections)
				{
					// Iterate over each triangle.
					for (int32 TriangleIndex = 0; TriangleIndex < (int32)Section.NumTriangles; TriangleIndex++)
					{
						int32 Index = TriangleIndex * 3 + Section.FirstIndex;

						FVector PointA(Vertices.VertexPosition(Indices[Index]));
						FVector PointB(Vertices.VertexPosition(Indices[Index + 1]));
						FVector PointC(Vertices.VertexPosition(Indices[Index + 2]));

						PointA = InstanceTransfrom.TransformPosition(PointA);
						PointB = InstanceTransfrom.TransformPosition(PointB);
						PointC = InstanceTransfrom.TransformPosition(PointC);

						bool bFullyContained = false;
						bool bIntersect = InFrustum.IntersectTriangle(PointA, PointB, PointC, bFullyContained);

						if (!bMustEncompassEntireInstance && bIntersect)
						{
							// Needn't encompass entire component: any intersection, we consider as touching
							return true;
						}
						else if (bMustEncompassEntireInstance && !bFullyContained)
						{
							// Must encompass entire component: any non intersection, we consider as not touching
							return false;
						}
					}
				}

				// Either:
				// a) It must encompass the entire component and all points were intersected (return true), or;
				// b) It needn't encompass the entire component but no points were intersected (return false)
				return bMustEncompassEntireInstance;
			}
		}
	}

	return false;
}
#endif //WITH_EDITOR

// Helper function to construct a base-set of instance data flags that in
FInstanceDataFlags UInstancedStaticMeshComponent::MakeInstanceDataFlags(bool bAnyMaterialHasPerInstanceRandom, bool bAnyMaterialHasPerInstanceCustomData) const
{
	FInstanceDataFlags Flags;
	Flags.bHasPerInstanceRandom = bAnyMaterialHasPerInstanceRandom;
	Flags.bHasPerInstanceCustomData = bAnyMaterialHasPerInstanceCustomData && NumCustomDataFloats != 0;
#if WITH_EDITOR
	Flags.bHasPerInstanceEditorData = GIsEditor != 0 && bHasPerInstanceHitProxies;
#endif
	
	Flags.bHasPerInstanceLMSMUVBias = IsStaticLightingAllowed();

	Flags.bHasPerInstanceDynamicData = PerInstancePrevTransform.Num() > 0 && PerInstancePrevTransform.Num() == GetInstanceCount();
	check(!Flags.bHasPerInstanceDynamicData || Mobility != EComponentMobility::Static);

	return Flags;
}

void UInstancedStaticMeshComponent::PostLoad()
{
	Super::PostLoad();

	// If world is instanced, clear our cached bounds as they will likely not be valid
	if (ULevel* Level = GetComponentLevel())
	{
		if (Level->IsInstancedLevel())
		{
			InvalidateCachedBounds();
		}
	}

	// Ensure we have the proper amount of per instance custom float data.
	// We have instances, and we have custom float data per instance, but we don't have the right amount of custom float data allocated.
	if ((PerInstanceSMData.Num() * NumCustomDataFloats) != PerInstanceSMCustomData.Num() && PerInstanceSMData.Num() > 0)
	{
		// No custom data at all, add all zeroes
		if (PerInstanceSMCustomData.Num() == 0)
		{
			UE_LOGF(LogStaticMesh, Warning, "%ls has %d instances, and %d NumCustomDataFloats, but no PerInstanceSMCustomData.  Allocating %d custom floats.", *GetFullName(), PerInstanceSMData.Num(), NumCustomDataFloats, PerInstanceSMData.Num() * NumCustomDataFloats);
			PerInstanceSMCustomData.AddZeroed(PerInstanceSMData.Num() * NumCustomDataFloats);
		}
		else
		{
			// Custom data exists, so we preserve it in our new allocation
			UE_LOGF(LogStaticMesh, Warning, "%ls has %d instances, and %d NumCustomDataFloats, but has %d PerInstanceSMCustomData.  Allocating %d custom floats to match.", *GetFullName(), PerInstanceSMData.Num(), NumCustomDataFloats, PerInstanceSMCustomData.Num(), PerInstanceSMData.Num() * NumCustomDataFloats);

			const int32 NumCustomDataFloatsPrev = PerInstanceSMCustomData.Num() / PerInstanceSMData.Num();

			TArray<float> OldPerInstanceSMCustomData;
			Exchange(OldPerInstanceSMCustomData, PerInstanceSMCustomData);

			// Allocate the proper amount and zero it
			PerInstanceSMCustomData.AddZeroed(PerInstanceSMData.Num() * NumCustomDataFloats);

			const int32 NumFloatsToCopy = FMath::Min(NumCustomDataFloatsPrev, NumCustomDataFloats);

			// Copy over existing data 
			for (int32 InstanceID = 0; InstanceID < PerInstanceSMData.Num(); InstanceID++)
			{
				for (int32 CopyID = 0; CopyID < NumFloatsToCopy; CopyID++)
				{
					PerInstanceSMCustomData[InstanceID * NumCustomDataFloats + CopyID] = OldPerInstanceSMCustomData[InstanceID * NumCustomDataFloatsPrev + CopyID];
				}
			}
		}
	}

	if (!HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		PrimitiveInstanceDataManager.PostLoad(PerInstanceSMData.Num());
	}

	// Has different implementation in HISMC
	OnPostLoadPerInstanceData();
}

void UInstancedStaticMeshComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	FInstancedStaticMeshComponentHelper::CollectPSOPrecacheData(*this, BasePrecachePSOParams, OutParams);
}

void UInstancedStaticMeshComponent::OnPostLoadPerInstanceData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::OnPostLoadPerInstanceData);

	if (!HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		InitializeInstancingRandomSeed();
	}

	if (AActor* Owner = GetOwner())
	{
		ULevel* OwnerLevel = Owner->GetLevel();
		UWorld* OwnerWorld = OwnerLevel ? OwnerLevel->OwningWorld.Get() : nullptr;
		ULevel* ActiveLightingScenario = OwnerWorld ? OwnerWorld->GetActiveLightingScenario() : nullptr;

		if (ActiveLightingScenario && ActiveLightingScenario != OwnerLevel)
		{
			//update the instance data if the lighting scenario isn't the owner level
			PrimitiveInstanceDataManager.BakedLightingDataChangedAll();
		}
	}
}

void UInstancedStaticMeshComponent::FullNavigationUpdate()
{
	FNavigationSystem::UpdateComponentData(*this); // just update everything
}

void UInstancedStaticMeshComponent::PartialNavigateUpdateForCurrentInstances()
{
	if (!IsNavigationRelevant() || PerInstanceSMData.IsEmpty())
	{
		return;
	}

	TArray<FTransform> NavigationUpdateTransforms;
	NavigationUpdateTransforms.Reserve(PerInstanceSMData.Num());
	for (int32 i = 0; i < PerInstanceSMData.Num(); i++)
	{
		NavigationUpdateTransforms.Add(FTransform(PerInstanceSMData[i].Transform) * GetComponentTransform());
	}

	PartialNavigationUpdates(NavigationUpdateTransforms);
}

void UInstancedStaticMeshComponent::PartialNavigationUpdate(int32 InstanceIdx)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedStaticMeshComponent::PartialNavigationUpdate);
	if (!IsNavigationRelevant())
	{
		return;
	}

	FTransform InstanceTransform;
	if (GetInstanceTransform(InstanceIdx, InstanceTransform, /*bWorldSpace*/true))
	{
		PartialNavigationUpdates({InstanceTransform});
	}
}

void UInstancedStaticMeshComponent::PartialNavigationUpdates(const TConstArrayView<FTransform> InstanceTransforms)
{
	if (!IsNavigationRelevant() || InstanceTransforms.IsEmpty())
	{
		return;
	}

	const FBox InstanceBounds = GetInstanceNavigationBounds();
	if (InstanceBounds.IsValid)
	{
		if (CVarISMInvalidateCachedNavigationBoundsOnPartialNavigationUpdates.GetValueOnAnyThread() != 0)
		{
			// Make sure to invalidate cached navigation bounds that might have been computed from a different number of instances.
			CachedBounds[static_cast<int32>(EBoundsCacheType::Navigation)].Invalidate();
		}

		// Dirty areas around the modified instances
		TArray<FBox> DirtyAreas;
		DirtyAreas.Reserve(InstanceTransforms.Num());
		for (const FTransform& InstanceTransform : InstanceTransforms)
		{
			DirtyAreas.Emplace(InstanceBounds.TransformBy(InstanceTransform));
		}
		FNavigationSystem::OnObjectBoundsChanged(*this, GetNavigationBounds(), DirtyAreas);
	}
}

bool UInstancedStaticMeshComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	return FInstancedStaticMeshComponentHelper::DoCustomNavigableGeometryExport(*this, GeomExport, FNavDataPerInstanceTransformDelegate::CreateUObject(this, &UInstancedStaticMeshComponent::GetNavigationPerInstanceTransforms));
}

extern float DebugLineLifetime;

bool UInstancedStaticMeshComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	bool bHaveHit = InstancePhysicsBodies->LineTrace(OutHit, Start, End, Params.bTraceComplex, Params.bReturnPhysicalMaterial);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UWorld* const World = GetWorld();
	if (World && World->DebugDrawSceneQueries(Params.TraceTag))
	{
		TArray<FHitResult> Hits;
		if (bHaveHit)
		{
			Hits.Add(OutHit);
		}
		DrawLineTraces(GetWorld(), Start, End, Hits, DebugLineLifetime);
	}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	return bHaveHit;
}

bool UInstancedStaticMeshComponent::SweepComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FQuat& ShapeWorldRotation, const FCollisionShape& CollisionShape, bool bTraceComplex)
{
	return InstancePhysicsBodies->Sweep(OutHit, Start, End, ShapeWorldRotation, CollisionShape, bTraceComplex);
}

bool UInstancedStaticMeshComponent::OverlapComponent(const FVector& Pos, const FQuat& Rot, const FCollisionShape& CollisionShape) const
{
	return InstancePhysicsBodies->Overlap(Pos, Rot, CollisionShape);
}

bool UInstancedStaticMeshComponent::ComponentOverlapComponentImpl(class UPrimitiveComponent* PrimComp, const FVector Pos, const FQuat& Quat, const FCollisionQueryParams& Params)
{
	//We do not support Instanced static meshes vs Instanced static meshes
	if (PrimComp->IsA<UInstancedStaticMeshComponent>())
	{
		UE_LOGF(LogCollision, Warning, "ComponentOverlapComponent : (%ls) Does not support InstancedStaticMesh with Physics Asset", *PrimComp->GetPathName());
		return false;
	}

	return InstancePhysicsBodies->OverlapComponent(PrimComp->GetBodyInstance(), Pos, Quat);
}

bool UInstancedStaticMeshComponent::ComponentOverlapMultiImpl(TArray<struct FOverlapResult>& OutOverlaps, const class UWorld* InWorld, const FVector& Pos, const FQuat& Rot, ECollisionChannel TestChannel, const struct FComponentQueryParams& Params, const struct FCollisionObjectQueryParams& ObjectQueryParams) const
{
	OutOverlaps.Reset();

	return InstancePhysicsBodies->OverlapMulti(OutOverlaps, Pos, Rot, TestChannel, Params, ObjectQueryParams);
}

void UInstancedStaticMeshComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	FInstancedStaticMeshComponentHelper::GetNavigationData(*this, Data, FNavDataPerInstanceTransformDelegate::CreateUObject(this, &UInstancedStaticMeshComponent::GetNavigationPerInstanceTransforms));
}

FBox UInstancedStaticMeshComponent::GetNavigationBounds() const
{
	return GetOrCalcBounds(GetComponentTransform(), EBoundsCacheType::Navigation).GetBox();
}

bool UInstancedStaticMeshComponent::IsNavigationRelevant() const
{
	return GetInstanceCount() > 0 && Super::IsNavigationRelevant();
}

bool UInstancedStaticMeshComponent::ShouldSkipDirtyAreaOnAddOrRemove() const
{
	// If partial navigation updates are supported then we don't want to dirty the
	// whole area covered by the navigation bounds when added to the navigation octree,
	// instead we use the partial update to push the list of dirty areas.
	return SupportsPartialNavigationUpdate();
}

FBox UInstancedStaticMeshComponent::GetInstanceNavigationBounds() const
{
	return FInstancedStaticMeshComponentHelper::GetInstanceNavigationBounds(*this);
}

void UInstancedStaticMeshComponent::GetNavigationPerInstanceTransforms(const FBox& AreaBox, TArray<FTransform>& InstanceData) const
{
	FInstancedStaticMeshComponentHelper::GetNavigationPerInstanceTransforms(*this, AreaBox, InstanceData);
}

void UInstancedStaticMeshComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// component stuff
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(InstancePhysicsBodies->GetBodies().GetAllocatedSize());
	for (FBodyInstance* Body : *InstancePhysicsBodies)
	{
		if (Body != nullptr && Body->IsValidBodyInstance())
		{
			Body->GetBodyInstanceResourceSizeEx(CumulativeResourceSize);
		}
	}
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(PerInstanceSMData.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(PerInstanceSMCustomData.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(InstanceReorderTable.GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(PrimitiveInstanceDataManager.GetAllocatedSize());
}

void UInstancedStaticMeshComponent::BeginDestroy()
{
	// Notify that these instances have been cleared due to the destroy
	if (FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.IsBound())
	{
		FInstancedStaticMeshDelegates::FInstanceIndexUpdateData IndexUpdate{ FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Destroyed, GetInstanceCount() - 1 };
		FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.Broadcast(this, MakeArrayView(&IndexUpdate, 1));
	}

	Super::BeginDestroy();
}

void UInstancedStaticMeshComponent::SetLODDistanceScale(float InLODDistanceScale)
{
	if (InstanceLODDistanceScale != InLODDistanceScale)
	{
		InstanceLODDistanceScale = InLODDistanceScale;
		MarkRenderStateDirty();
	}
}

#if WITH_EDITOR
void UInstancedStaticMeshComponent::PostStaticMeshCompilation()
{
	// Multiple operations (including 'UpdateBounds') can early out when mesh is compiling so we need to 
	// override PostStaticMeshCompilation to make sure that UpdateBounds is completed before 
	// UStaticMeshComponent::PostStaticMeshCompilation calls FNavigationSubsystem::UpdateComponentData, which relies
	// on updated and cached bounds to create a valid FNavigationElement.

	UpdateBounds();

	Super::PostStaticMeshCompilation();
}

void UInstancedStaticMeshComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if(PropertyChangedEvent.Property != NULL)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMData))
		{
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd
				|| PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
			{
				int32 AddedAtIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
				check(AddedAtIndex != INDEX_NONE);

				AddInstanceInternal(AddedAtIndex, &PerInstanceSMData[AddedAtIndex], PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd ? FTransform::Identity : FTransform(PerInstanceSMData[AddedAtIndex].Transform), /*bWorldSpace*/false);

				// added via the property editor, so we will want to interactively work with instances
				bHasPerInstanceHitProxies = true;

				// This path may reorder the instances by insertion and this is not handled by the incremental updates in the proxy, so we force a full update.
				PrimitiveInstanceDataManager.ClearChangeTracking();
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
			{
				int32 RemovedAtIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
				check(RemovedAtIndex != INDEX_NONE);

				RemoveInstanceInternal(RemovedAtIndex, true);
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
			{
				ClearInstances();
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
			{
				// This, presumably means the whole array was replaced - this certainly invalidates any tracking.
				PrimitiveInstanceDataManager.Invalidate(PerInstanceSMData.Num());
			}
			
			MarkRenderStateDirty();
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FInstancedStaticMeshInstanceData, Transform))
		{
			InvalidateCachedBounds();
			// No need to update when using partial update since OnRegister will take care of it.
			if (!SupportsPartialNavigationUpdate())
			{
				FullNavigationUpdate();
			}

			// Mark all instances as changed because we don't know which one actually did.
			PrimitiveInstanceDataManager.TransformsChangedAll();
		}
		else if (PropertyChangedEvent.Property->GetFName() == "NumCustomDataFloats")
		{
			// Can't just call SetNumCustomDataFloats because it doesn't do anything if the value is the same, and the edtior has already modified the value...
			NumCustomDataFloats = FMath::Max(NumCustomDataFloats, 0);

			// Clear out and reinit to 0
			PerInstanceSMCustomData.Empty(PerInstanceSMData.Num() * NumCustomDataFloats);
			PerInstanceSMCustomData.SetNumZeroed(PerInstanceSMData.Num() * NumCustomDataFloats);
			PrimitiveInstanceDataManager.NumCustomDataChanged();
		}
		else if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName() == "PerInstanceSMCustomData")
		{
			int32 ChangedCustomValueIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
			if (ensure(NumCustomDataFloats > 0))
			{
				int InstanceIndex = ChangedCustomValueIndex / NumCustomDataFloats;
				PrimitiveInstanceDataManager.CustomDataChanged(InstanceIndex);
		}
	}
	}
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UInstancedStaticMeshComponent::PostEditUndo()
{
	InvalidateCachedBounds();

	// Need to do this before, because the proxy is recreated up this stack, which ends up hitting a PrimitiveInstanceDataManager.FlushChanges.
	PrimitiveInstanceDataManager.Invalidate(PerInstanceSMData.Num());
	Super::PostEditUndo();

	FNavigationSystem::UpdateComponentData(*this);

	MarkRenderStateDirty();
}

void UInstancedStaticMeshComponent::BeginCacheForCookedPlatformData( const ITargetPlatform* TargetPlatform )
{
	Super::BeginCacheForCookedPlatformData(TargetPlatform);

	// Early out to avoid call to BuildComponentInstanceData if we don't need to call BeginCacheForCookedPlatformData
	bool bNeedPrimitiveInstanceDataManagerCookedData = PrimitiveInstanceDataManager.ShouldWriteCookedData(TargetPlatform, PerInstanceSMData.Num());

	FInstanceUpdateComponentDesc ComponentData;
	if (GetStaticMesh() != nullptr)
	{
		UpdateComponentToWorld();

		if (bNeedPrimitiveInstanceDataManagerCookedData)
		{
			// Calling this function can be expensive, because of its call to GetUsedMaterials, which immediately forces completion of in-flight
			// static mesh compilation, reducing parallelism.  That's why we try to avoid the call where it's not necessary.
			//
			// TODO:  In the future, we could make it so the following function is itself more efficient.  The component data is only affected by
			// materials where displacement is used (rare), and we could skip the call to GetUsedMaterials if none of the original material references
			// in the static mesh have displacement.  This isn't completely trivial, as the static mesh currently protects access to the material
			// references, also forcing completion of in-flight compilation when accessed.  You'd need some way to bypass that protection, perhaps
			// caching the relevant information (whether displacement may be present) on load.
			BuildComponentInstanceData(GMaxRHIShaderPlatform, ComponentData);
		}
	}

	if (bNeedPrimitiveInstanceDataManagerCookedData)
	{
		PrimitiveInstanceDataManager.BeginCacheForCookedPlatformData(TargetPlatform, MoveTemp(ComponentData), MakeStridedView(PerInstanceSMData, &FInstancedStaticMeshInstanceData::Transform));
	}
}
	
bool UInstancedStaticMeshComponent::IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform )
{
	return Super::IsCachedCookedPlatformDataLoaded(TargetPlatform);
}

#endif

bool UInstancedStaticMeshComponent::IsInstanceSelected(int32 InInstanceIndex) const
{
#if WITH_EDITOR
	if(SelectedInstances.IsValidIndex(InInstanceIndex))
	{
		return SelectedInstances[InInstanceIndex];
	}
#endif

	return false;
}

void UInstancedStaticMeshComponent::SelectInstance(bool bInSelected, int32 InInstanceIndex, int32 InInstanceCount)
{
#if WITH_EDITOR
	if (InInstanceCount > 0 && DeletionState == EInstanceDeletionReason::NotDeleting)
	{
		if (PerInstanceSMData.Num() != SelectedInstances.Num())
		{
			SelectedInstances.Init(false, PerInstanceSMData.Num());
		}

		check(InInstanceIndex >= 0 && InInstanceCount > 0);
		check(InInstanceIndex + InInstanceCount - 1 < SelectedInstances.Num());
		
		PrimitiveInstanceDataManager.EditorDataChangedAll();
		for (int32 InstanceIndex = InInstanceIndex; InstanceIndex < InInstanceIndex + InInstanceCount; InstanceIndex++)
		{
			if (SelectedInstances.IsValidIndex(InInstanceIndex))
			{
				SelectedInstances[InstanceIndex] = bInSelected;
			}
		}

		// Reset the flags if all are false (this is also used to ensure per primitive selection works as intended since it relies on this being cleared when per instance selection is disabled).
		if (SelectedInstances.Find(true) == INDEX_NONE)
		{
			SelectedInstances.Empty();
		}
		
		// We need to recreate to make sure it changes to the dynamic path, which makes selection outline rendering work.
		MarkRenderStateDirty();
	}
#endif
}

void UInstancedStaticMeshComponent::ClearInstanceSelection()
{
#if WITH_EDITOR
	SelectedInstances.Empty();
	PrimitiveInstanceDataManager.EditorDataChangedAll();
#endif
}

bool UInstancedStaticMeshComponent::CanEditSMInstance(const FSMInstanceId& InstanceId) const
{
	check(InstanceId.ISMComponent == this);
	return IsEditableWhenInherited();
}

bool UInstancedStaticMeshComponent::CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const
{
	check(InstanceId.ISMComponent == this);
	return InWorldType == ETypedElementWorldType::Editor || InstanceId.ISMComponent->Mobility == EComponentMobility::Movable;
}

bool UInstancedStaticMeshComponent::GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	check(InstanceId.ISMComponent == this);
	return GetInstanceTransform(InstanceId.InstanceIndex, OutInstanceTransform, bWorldSpace);
}

bool UInstancedStaticMeshComponent::SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	check(InstanceId.ISMComponent == this);
	return UpdateInstanceTransform(InstanceId.InstanceIndex, InstanceTransform, bWorldSpace, bMarkRenderStateDirty, bTeleport);
}

void UInstancedStaticMeshComponent::NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId)
{
	check(InstanceId.ISMComponent == this);
}

void UInstancedStaticMeshComponent::NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId)
{
	check(InstanceId.ISMComponent == this);
}

void UInstancedStaticMeshComponent::NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId)
{
	check(InstanceId.ISMComponent == this);
}

void UInstancedStaticMeshComponent::NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected)
{
	check(InstanceId.ISMComponent == this);
	SelectInstance(bIsSelected, InstanceId.InstanceIndex);
}

bool UInstancedStaticMeshComponent::DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds)
{
	TArray<int32> InstanceIndices;
	InstanceIndices.Reserve(InstanceIds.Num());
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		check(InstanceId.ISMComponent == this);
		InstanceIndices.Add(InstanceId.InstanceIndex);
	}

	Modify();
	return RemoveInstances(InstanceIndices);
}

bool UInstancedStaticMeshComponent::DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds)
{
	TArray<FTransform> NewInstanceTransforms;
	NewInstanceTransforms.Reserve(InstanceIds.Num());
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		check(InstanceId.ISMComponent == this);
		FTransform& NewInstanceTransform = NewInstanceTransforms.Add_GetRef(FTransform::Identity);
		GetInstanceTransform(InstanceId.InstanceIndex, NewInstanceTransform);
	}

	Modify();
	const TArray<int32> NewInstanceIndices = AddInstances(NewInstanceTransforms, /*bShouldReturnIndices*/true);

	OutNewInstanceIds.Reset(NewInstanceIndices.Num());
	for (int32 NewInstanceIndex : NewInstanceIndices)
	{
		OutNewInstanceIds.Add(FSMInstanceId{ this, NewInstanceIndex });
	}

	return true;
}

void UInstancedStaticMeshComponent::InitializeInstancingRandomSeed(const bool bAllowRandomSeedValue)
{
	auto GenerateSeedFromObjectPath = [this]() -> int32
	{
		TStringBuilder<256> ResultBuilder;
		GetPathName(nullptr, ResultBuilder);
		int32 RandomSeed = (int32)CityHash32((const char*)ResultBuilder.GetData(), ResultBuilder.Len() * sizeof(decltype(ResultBuilder)::ElementType));
		if (RandomSeed == 0)
		{
			// If the resulting hash is still zero (unlikely, but possible), we fall back to a fixed default value.
			RandomSeed = 42;
		}

		return RandomSeed;
	};

	// If we don't have a random seed for this instanced static mesh component yet, then go ahead and
	// generate one now.  This will be saved with the static mesh component and used for future generation
	// of random numbers for this component's instances. (Used by the PerInstanceRandom material expression)
	if (bAllowRandomSeedValue)
	{
		// This code path is assumed to be used when saving.
		// Make sure to generate a truly random seed which is not based on the object path.
		int32 SeedFromPath = GenerateSeedFromObjectPath();
		while (InstancingRandomSeed == 0 || InstancingRandomSeed == SeedFromPath)
		{
			InstancingRandomSeed = FMath::Rand();
		}
	}
	else
	{
		// To maintain determinism, the seed is generated from the component's path name.
		if (InstancingRandomSeed == 0)
		{
			InstancingRandomSeed = GenerateSeedFromObjectPath();
		}
	}	
}

void FInstancedStaticMeshVertexFactoryShaderParameters::GetElementShaderBindings(
	const class FSceneInterface* Scene,
	const FSceneView* View,
	const FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams
	) const
{
	// Decode VertexFactoryUserData as VertexFactoryUniformBuffer
	FRHIUniformBuffer* VertexFactoryUniformBuffer = static_cast<FRHIUniformBuffer*>(BatchElement.VertexFactoryUserData);
	FLocalVertexFactoryShaderParametersBase::GetElementShaderBindingsBase(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, VertexFactoryUniformBuffer, ShaderBindings, VertexStreams);

	const FInstancingUserData* InstancingUserData = (const FInstancingUserData*)BatchElement.UserData;
	const auto* InstancedVertexFactory = static_cast<const FInstancedStaticMeshVertexFactory*>(VertexFactory);
	const int32 InstanceOffsetValue = BatchElement.UserIndex;

	ShaderBindings.Add(InstanceOffset, InstanceOffsetValue);
	
	if (!UseGPUScene(Scene ? Scene->GetShaderPlatform() : GMaxRHIShaderPlatform))
	{
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FInstancedStaticMeshVertexFactoryUniformShaderParameters>(), InstancedVertexFactory->GetUniformBuffer());
		if (InstancedVertexFactory->SupportsManualVertexFetch(FeatureLevel))
		{
			ShaderBindings.Add(VertexFetch_InstanceOriginBufferParameter, InstancedVertexFactory->GetInstanceOriginSRV());
			ShaderBindings.Add(VertexFetch_InstanceTransformBufferParameter, InstancedVertexFactory->GetInstanceTransformSRV());
			ShaderBindings.Add(VertexFetch_InstanceLightmapBufferParameter, InstancedVertexFactory->GetInstanceLightmapSRV());
		}
		if (InstanceOffsetValue > 0 && VertexStreams.Num() > 0)
		{
			// GPUCULL_TODO: This here can still work together with the instance attributes for index, but note that all instance attributes then must assume they are offset wrt the on-the-fly generate buffer
			//          so with the new scheme there is no clear way this can work in the vanilla instancing way as there is an indirection. So either other attributes must be loaded in the shader or they
			//          would have to be copied as the instance ID is now - not good.
			VertexFactory->OffsetInstanceStreams(InstanceOffsetValue, InputStreamType, VertexStreams);
		}
	}

	FVector4f InstancingOffset(ForceInit);
	// InstancedLODRange is only set for HierarchicalInstancedStaticMeshes
	if (InstancingUserData && BatchElement.InstancedLODRange)
	{
		InstancingOffset = (FVector3f)InstancingUserData->InstancingOffset; // LWC_TODO: precision loss
	}
	ShaderBindings.Add(InstancingOffsetParameter, InstancingOffset);

	ShaderBindings.Add(Shader->GetUniformBufferParameter<FInstancedStaticMeshVFLooseUniformShaderParameters>(), BatchElement.LooseParametersUniformBuffer);
}
