// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMesh/ISMInstanceDataManager.h"
#include "InstancedStaticMesh/ISMInstanceUpdateChangeSet.h"
#include "InstanceData/InstanceDataHelpers.h"
#include "InstanceData/InstanceDataUpdateUtils.h"

#include "Engine/InstancedStaticMesh.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Rendering/MotionVectorSimulation.h"
#include "SceneInterface.h"

//UE_DISABLE_OPTIMIZATION

#define IDPROXY_ENABLE_ASYNC_TASK 1

#if 0

#define LOG_INST_DATA(_Format_, ...) \
{ \
	FString Tmp = FString::Printf(_Format_, ##__VA_ARGS__);\
	UE_LOGF(LogInstanceProxy, Log, "%p, %ls", PrimitiveComponent.IsValid() ? PrimitiveComponent.Get() : nullptr, *Tmp);\
}

#else
	#define LOG_INST_DATA(_Format_, ...) 
#endif

static TAutoConsoleVariable<float> CVarInstanceUpdateTaskDebugDelay(
	TEXT("r.InstanceUpdateTaskDebugDelay"),
	0.0f,
	TEXT("Instance update debug delay in seconds."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarInstanceDataResetTrackingOnRegister(
	TEXT("r.InstanceData.ResetTrackingOnRegister"),
	1,
	TEXT("Chicken switch to disable the new code to reset tracking & instance count during OnRegister, if this causes problems.\nTODO: Remove."));

static TAutoConsoleVariable<int32> CVarInstanceDataMinInstanceCountToOptimize(
	TEXT("r.InstanceData.MinInstanceCountToOptimize"),
	2,
	TEXT("Minimum number of instances to perform optimized build for (if enabled), can be used to disable optimized build for small ISMs as there is some overhead from doing so."));


/**
 * Helper function to hash the local to world that was used to build the precomputed data as we can't rely on it being unchanged, 
 * for example when sublevels with transforms are used.
 */
static uint32 HashLocalToWorld(const FMatrix& M)
{
	// round to reduce risk of floating point precision causing otherwise same transforms from being considered different
	uint32 Hash = HashCombineFast(
		GetTypeHash(FMath::RoundToInt64(M.M[0][0])), GetTypeHash(FMath::RoundToInt64(M.M[0][1])), GetTypeHash(FMath::RoundToInt64(M.M[0][2])),
		GetTypeHash(FMath::RoundToInt64(M.M[1][0])), GetTypeHash(FMath::RoundToInt64(M.M[1][1])), GetTypeHash(FMath::RoundToInt64(M.M[1][2])),
		GetTypeHash(FMath::RoundToInt64(M.M[2][0])), GetTypeHash(FMath::RoundToInt64(M.M[2][1])), GetTypeHash(FMath::RoundToInt64(M.M[2][2])),
		GetTypeHash(FMath::RoundToInt64(M.M[3][0])), GetTypeHash(FMath::RoundToInt64(M.M[3][1])), GetTypeHash(FMath::RoundToInt64(M.M[3][2]))
	);

	return Hash;
}

FPrimitiveInstanceDataManager::FPrimitiveInstanceDataManager(UPrimitiveComponent* InPrimitiveComponent) 
	: bNumCustomDataChanged(false)
	, bBakedLightingDataChanged(false)
	, bTransformChangedAllInstances(false)
#if WITH_EDITOR
	, bAnyEditorDataChanged(false)
#endif
	, bPrimitiveTransformChanged(false)
	, bAnyInstanceChange(false)
	, bComponentMarkedDirty(false)
	, bEnableTracking(false)
	, bFirstFlush(true)
	, PrimitiveComponent(InPrimitiveComponent) 
{
	// Don't do anything if this is not a "real" ISM being tracked (this logic shopuld move out).
	if (PrimitiveComponent.IsValid() && PrimitiveComponent->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		TrackingState = ETrackingState::Disabled;
	}
	LOG_INST_DATA(TEXT("FPrimitiveInstanceDataManager %s, TrackingState=%s"), *PrimitiveComponent->GetFullName(), TrackingState == ETrackingState::Disabled ? TEXT("Disabled") : TEXT("Initial"));
}

void FPrimitiveInstanceDataManager::SetMode(EMode InMode)
{
	if (InMode != Mode)
	{
		// This should never be called in mid-use
		check(GetMaxInstanceIndex() == 0);
		Invalidate(0);
	}
	Mode = InMode;
}

void FPrimitiveInstanceDataManager::Add(int32 InInstanceAddAtIndex, bool bInsert)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	ValidateMapping();

	// If the manager is marked for legacy-only mode, we should not see any tracking calls!
	check(Mode != EMode::ExternalLegacyData);

	// 1. determine if we need to enable explicit tracking, this happens when an instance is inserted.
	bool bIsActualInsert = bInsert && InInstanceAddAtIndex != GetMaxInstanceIndex();

	// Create explicit mapping if we need it now
	if (bIsActualInsert && HasIdentityMapping())
	{
		CreateExplicitIdentityMapping();
	}

	check(bInsert || InInstanceAddAtIndex == GetMaxInstanceIndex());

	MarkComponentRenderInstancesDirty();

	if (HasIdentityMapping())
	{
		int32 InstanceId = NumInstances++;
		MarkChangeHelper<EChangeFlag::Added>(InstanceId);

		LOG_INST_DATA(TEXT("Add(IDX: %d, bInsert: %d) -> Id: %d"), InInstanceAddAtIndex, bInsert ? 1 : 0, InstanceId);
		return;
	}

	FPrimitiveInstanceId InstanceId{ValidInstanceIdMask.FindAndSetFirstZeroBit(IdSearchStartIndex)};
	if (!InstanceId.IsValid())
	{
		InstanceId = FPrimitiveInstanceId{ValidInstanceIdMask.Add(true)};
	}
	// Optimize search for next time
	IdSearchStartIndex = InstanceId.Id;
	IdToIndexMap.SetNumUninitialized(ValidInstanceIdMask.Num());

	// if these do not line up, then we are inserting an instance, this is a thing in the editor
	if (InInstanceAddAtIndex != IndexToIdMap.Num())
	{
		check(bInsert);
		IdToIndexMap[InstanceId.Id] = InInstanceAddAtIndex;
		LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), InstanceId.Id, InInstanceAddAtIndex);
		// first move all the existing data down one step by inserting the new one
		IndexToIdMap.Insert(InstanceId, InInstanceAddAtIndex);
		// then update all the relevant id->index mappings
		for (int32 Index = InInstanceAddAtIndex + 1; Index < IndexToIdMap.Num(); ++Index)
		{
			FPrimitiveInstanceId MovedId = IndexToIdMap[Index];
			IdToIndexMap[MovedId.Id] = Index;
			LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), MovedId.GetAsIndex(), Index);
			InstanceUpdateTracker.MarkIndex<EChangeFlag::IndexChanged>(Index, GetMaxInstanceIndex());
		}
	}
	else
	{
		int32 InstanceIndex = IndexToIdMap.Num();
		IdToIndexMap[InstanceId.Id] = InstanceIndex;
		LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), InstanceId.Id, InstanceIndex);

		IndexToIdMap.Add(InstanceId);
	}
	NumInstances = IndexToIdMap.Num();
	check(ValidInstanceIdMask.Num() >= NumInstances);
	MarkChangeHelper<EChangeFlag::Added>(InstanceId);
	LOG_INST_DATA(TEXT("Add(IDX: %d, bInsert: %d) -> Id: %d"), InInstanceAddAtIndex, bInsert, InstanceId.Id);

	ValidateMapping();
}

void FPrimitiveInstanceDataManager::RemoveAtSwap(int32 InstanceIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	ValidateMapping();

	check(Mode != EMode::ExternalLegacyData);

	FPrimitiveInstanceId InstanceId = IndexToId(InstanceIndex);
	// resize to the max at once so we don't have to grow piecemeal
	InstanceUpdateTracker.RemoveAtSwap(InstanceId, InstanceIndex, GetMaxInstanceIndex());

	// If the remove would cause reordering, we create the explicit mapping
	const bool bCausesReordering = InstanceIndex != NumInstances - 1;
	if (bCausesReordering && HasIdentityMapping())
	{
		CreateExplicitIdentityMapping();
	}

	MarkComponentRenderInstancesDirty();

	FreeInstanceId(InstanceId);

	// If we still have the identity mapping, we must be removing the last item
	if (HasIdentityMapping())
	{
		check(!bCausesReordering);
		--NumInstances;
		LOG_INST_DATA(TEXT("RemoveAtSwap(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);
		return;
	}

	FPrimitiveInstanceId LastInstanceId = IndexToIdMap.Pop();
	NumInstances = IndexToIdMap.Num();
	check(ValidInstanceIdMask.Num() >= NumInstances);

	if (InstanceId != LastInstanceId)
	{
		IdToIndexMap[LastInstanceId.Id] = InstanceIndex;
		LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), LastInstanceId.Id, InstanceIndex);
		IndexToIdMap[InstanceIndex] = LastInstanceId;
	}
	ValidateMapping();
	LOG_INST_DATA(TEXT("RemoveAtSwap(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);
}
	
void FPrimitiveInstanceDataManager::RemoveAt(int32 InstanceIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	ValidateMapping();

	check(Mode != EMode::ExternalLegacyData);
	FPrimitiveInstanceId InstanceId = IndexToId(InstanceIndex);

	InstanceUpdateTracker.RemoveAt(InstanceId, InstanceIndex, GetMaxInstanceIndex());
		
	const bool bCausesReordering = InstanceIndex != NumInstances - 1;
	if (bCausesReordering && HasIdentityMapping())
	{
		CreateExplicitIdentityMapping();
	}

	MarkComponentRenderInstancesDirty();
	FreeInstanceId(InstanceId);

	// If we still have the identity mapping, do the simplified tracking update
	if (HasIdentityMapping())
	{
		check(!bCausesReordering);
		--NumInstances;
		LOG_INST_DATA(TEXT("RemoveAt(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);
		return;
	}

	if (InstanceIndex == IndexToIdMap.Num() - 1)
	{
		IndexToIdMap.SetNum(InstanceIndex);
	}
	else
	{
		IndexToIdMap.RemoveAt(InstanceIndex);
		for (int32 Index = InstanceIndex; Index < IndexToIdMap.Num(); ++Index)
		{
			FPrimitiveInstanceId MovedId = IndexToIdMap[Index];
			IdToIndexMap[MovedId.Id] = Index;
			LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), MovedId.GetAsIndex(), Index);
		}
	}
	NumInstances = IndexToIdMap.Num();
	check(ValidInstanceIdMask.Num() >= NumInstances);
	LOG_INST_DATA(TEXT("RemoveAt(IDX: %d) -> Id: %d"), InstanceIndex, InstanceId.Id);

	ValidateMapping();
}

void FPrimitiveInstanceDataManager::TransformChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("TransformChanged(IDX: %d)"), InstanceIndex);
	MarkChangeHelper<EChangeFlag::TransformChanged>( InstanceIndex);
}

void FPrimitiveInstanceDataManager::TransformChanged(FPrimitiveInstanceId InstanceId)
{
	LOG_INST_DATA(TEXT("TransformChanged(ID: %d)"), InstanceId.Id);
	MarkChangeHelper<EChangeFlag::TransformChanged>(InstanceId);
}

void FPrimitiveInstanceDataManager::TransformsChangedAll()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("TransformsChangedAll(%s)"), TEXT(""));
	bTransformChangedAllInstances = true;
	MarkComponentRenderInstancesDirty();
}

void FPrimitiveInstanceDataManager::CustomDataChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("CustomDataChanged(IDX: %d)"), InstanceIndex);
	MarkChangeHelper<EChangeFlag::CustomDataChanged>(InstanceIndex);
}

void FPrimitiveInstanceDataManager::BakedLightingDataChanged(int32 InstanceIndex)
{
	LOG_INST_DATA(TEXT("BakedLightingDataChanged(IDX: %d)"), InstanceIndex);
	bBakedLightingDataChanged = true;
	MarkComponentRenderInstancesDirty();
}

void FPrimitiveInstanceDataManager::BakedLightingDataChangedAll()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("BakedLightingDataChangedAll(%s)"), TEXT(""));
	bBakedLightingDataChanged = true;
	MarkComponentRenderInstancesDirty();
}

void FPrimitiveInstanceDataManager::NumCustomDataChanged()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("NumCustomDataChanged(%s)"), TEXT(""));
	bNumCustomDataChanged = true;
	MarkComponentRenderInstancesDirty();
}

#if WITH_EDITOR

void FPrimitiveInstanceDataManager::EditorDataChangedAll()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("EditorDataChangedAll(%s)"), TEXT(""));
	bAnyEditorDataChanged = true;
	MarkComponentRenderInstancesDirty();
}

#endif

void FPrimitiveInstanceDataManager::PrimitiveTransformChanged()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	LOG_INST_DATA(TEXT("PrimitiveTransformChanged(%s)"), TEXT(""));
	bPrimitiveTransformChanged = true;
	MarkComponentRenderInstancesDirty();
}

bool FPrimitiveInstanceDataManager::HasAnyInstanceChanges() const
{
	return bAnyInstanceChange || bNumCustomDataChanged || bBakedLightingDataChanged || bTransformChangedAllInstances
#if WITH_EDITOR
		|| bAnyEditorDataChanged 
#endif
		|| InstanceUpdateTracker.HasAnyChanges();
}

/**
 * Describes what has changed, that can be derived from the primitive desc, or internal tracking state.
 */
union FChangeDesc 
{
	struct 
	{
		uint8 bUntrackedState : 1;
		uint8 bInstancesChanged : 1;
		uint8 bPrimitiveTransformChanged : 1;
		uint8 bMaterialUsageFlagsChanged : 1;
		uint8 bMaxDisplacementChanged : 1;
		uint8 bStaticMeshBoundsChanged : 1;
	};
	uint8 Packed;
	
	FChangeDesc()
	{
		Packed = 0u;
	}

};

/**
 * Data captured that may be flushed to rebuild the proxy from legacy data AKA FStaticMeshInstanceData.
 * Is marshalled across via the FLegacyRebuildChangeSet.
 */
struct FLegacyBuildData
{
	TUniquePtr<FStaticMeshInstanceData> LegacyStaticMeshInstanceData;
	TArray<int32> LegacyInstanceReorderTable;
	FInstanceIdIndexMap InstanceIdIndexMap;
#if WITH_EDITOR
	TArray<TRefCountPtr<HHitProxy>> HitProxies;
#endif
};

struct FLegacyRebuildChangeSet
{
	FRenderBounds InstanceLocalBounds;
	FVector PrimitiveWorldSpaceOffset;
	FRenderTransform PrimitiveToRelativeWorld;
	FInstanceDataFlags Flags;
	float AbsMaxDisplacement = 0.0f;
	TUniquePtr<FStaticMeshInstanceData> LegacyStaticMeshInstanceData;
	TArray<int32> LegacyInstanceReorderTable;
	FInstanceIdIndexMap InstanceIdIndexMap;

#if WITH_EDITOR
	TPimplPtr<FOpaqueHitProxyContainer> HitProxyContainer;
#endif
	int32 NumCustomDataFloats = 0; 
};

void FPrimitiveInstanceDataManager::InitChangeSet(const FChangeDesc &ChangeDesc, const FInstanceUpdateComponentDesc &ComponentData, FISMInstanceUpdateChangeSet &ChangeSet)
{
	// Collect the delta data to be able to update the index mapping.
	ChangeSet.MaxInstanceId = GetMaxInstanceId();
	ChangeSet.bIdentityIdMap = IsIdentity();
	if (!IsIdentity())
	{
		Gather(ChangeSet.GetIndexChangedDelta(), ChangeSet.IndexToIdMapDeltaData, IndexToIdMap);
	}

#if WITH_EDITOR
	bAnyEditorDataChanged = false;
#endif	

	// trigger full copy of custom data if num changed (because this zero-fills the custom data array).
	ChangeSet.ForceFullFlags.bHasPerInstanceCustomData = ChangeSet.ForceFullFlags.bHasPerInstanceCustomData || bNumCustomDataChanged;

	bAnyInstanceChange = false;
	bTransformChangedAllInstances = false;
	bNumCustomDataChanged = false;
	bBakedLightingDataChanged = false;

	const FVector3f PrimitiveWorldSpacePositionHigh = FDFVector3{ PrimitiveLocalToWorld.GetOrigin() }.High;
	ChangeSet.PrimitiveWorldSpaceOffset = FVector{ PrimitiveWorldSpacePositionHigh };
	ChangeSet.PrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(PrimitiveWorldSpacePositionHigh, PrimitiveLocalToWorld).M;
	ChangeSet.Flags = Flags;
	ChangeSet.AbsMaxDisplacement = AbsMaxDisplacement;
	ChangeSet.NumCustomDataFloats = NumCustomDataFloats;

	// This is the odd one out
	ChangeSet.SetSharedLocalBounds(ComponentData.StaticMeshBounds);
}

bool FPrimitiveInstanceDataManager::FlushChanges(FInstanceUpdateComponentDesc &&ComponentData, bool bNewPrimitiveProxy)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return false;
	}

	// Always clear the flag such that any subsequent change marks it as needing update again.
	bComponentMarkedDirty = false;

	// Can't update if there is no proxy, if it was destroyed we must wait for a new scene proxy to come around and request a new one to be created.
	if (!Proxy)
	{
		return false;
	}

	// Determine if the instance data proxy is exposed to the RT such that we can do early dispatch, this is important for level loading, probably, where we could move the work earlier.
	// In fact, we could, e.g., for static ISMs kick this directly in post load perhaps.
	const bool bIsUnattached = bNewPrimitiveProxy && Proxy->bIsNew;
	Proxy->bIsNew = false;
	
	// BandAid: This is the first flush & we have not been informed correctly about the number of instances in the ISM so we need to patch that up here and now.
	if (bFirstFlush && GetMaxInstanceIndex() == 0 && ComponentData.NumSourceInstances != 0)
	{
		IdToIndexMap.Reset();
		IndexToIdMap.Reset();
		NumInstances = ComponentData.NumSourceInstances;
	}
	bFirstFlush = false;

	float NewAbsMaxDisplacement = FMath::Max(-ComponentData.PrimitiveMaterialDesc.MinMaxMaterialDisplacement.X, ComponentData.PrimitiveMaterialDesc.MinMaxMaterialDisplacement.Y)
		+ ComponentData.PrimitiveMaterialDesc.MaxWorldPositionOffsetDisplacement;

	bool bWasUpdateQueued = false;
	// Marked for externally managed update, this may be combined with tracked changes (which results in double updates)
	if (LegacyBuildData)
	{
		LOG_INST_DATA(TEXT("Rebuild with EState::External %s"), TEXT(""));

		if (Mode == EMode::ExternalLegacyData)
		{
			check(LegacyBuildData->LegacyInstanceReorderTable.IsEmpty());
			check(LegacyBuildData->InstanceIdIndexMap.GetMaxInstanceIndex() == LegacyBuildData->LegacyStaticMeshInstanceData->GetNumInstances());
		}
		else
		{
			check(LegacyBuildData->InstanceIdIndexMap.GetMaxInstanceIndex() == LegacyBuildData->LegacyInstanceReorderTable.Num());
		}

		// An "external build" is responsible for setting everything up in the right space.
		bPrimitiveTransformChanged = false;
		PrimitiveLocalToWorld = ComponentData.PrimitiveLocalToWorld;
		Flags = ComponentData.Flags;
		StaticMeshBounds = ComponentData.StaticMeshBounds;
		NumCustomDataFloats = ComponentData.NumCustomDataFloats;

		FLegacyRebuildChangeSet ExternalChangeSet;
		const FVector3f PrimitiveWorldSpacePositionHigh = FDFVector3{ PrimitiveLocalToWorld.GetOrigin() }.High;
		ExternalChangeSet.PrimitiveWorldSpaceOffset = FVector{ PrimitiveWorldSpacePositionHigh };
		ExternalChangeSet.PrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(PrimitiveWorldSpacePositionHigh, PrimitiveLocalToWorld).M;
		ExternalChangeSet.InstanceLocalBounds = ComponentData.StaticMeshBounds;
		ExternalChangeSet.Flags = Flags;

		ExternalChangeSet.LegacyStaticMeshInstanceData = MoveTemp(LegacyBuildData->LegacyStaticMeshInstanceData);
		ExternalChangeSet.LegacyInstanceReorderTable = MoveTemp(LegacyBuildData->LegacyInstanceReorderTable);
		ExternalChangeSet.InstanceIdIndexMap = MoveTemp(LegacyBuildData->InstanceIdIndexMap);
#if WITH_EDITOR
		ExternalChangeSet.HitProxyContainer = MakeOpaqueHitProxyContainer(LegacyBuildData->HitProxies);
#endif

		ExternalChangeSet.AbsMaxDisplacement = NewAbsMaxDisplacement;

		// Assemble header info to enable nonblocking primitive update.
		FInstanceDataBufferHeader InstanceDataBufferHeader;
		// Note: the buffers will contain the number of instances in the LegacyStaticMeshInstanceData, which may be different from the number in the ISMC & what is tracked (density scaling can do this)
		InstanceDataBufferHeader.NumInstances = ExternalChangeSet.LegacyStaticMeshInstanceData->GetNumInstances();
		InstanceDataBufferHeader.PayloadDataStride = FInstanceSceneDataBuffers::CalcPayloadDataStride(ExternalChangeSet.Flags, NumCustomDataFloats, 0);
		InstanceDataBufferHeader.Flags = ExternalChangeSet.Flags;

		DispatchInstanceDataUpdateTask(bIsUnattached, Proxy, InstanceDataBufferHeader, [ExternalChangeSet = MoveTemp(ExternalChangeSet), Proxy = Proxy] () mutable 
		{
			FISMCInstanceDataSceneProxy &ProxyRef = *Proxy;
			
			// Forcibly destroy any tracking state
			ProxyRef.InstanceIdIndexMap = MoveTemp(ExternalChangeSet.InstanceIdIndexMap);
#if WITH_EDITOR
			ProxyRef.HitProxyContainer = MoveTemp(ExternalChangeSet.HitProxyContainer);
#endif
			FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(&ProxyRef));
			FInstanceSceneDataBuffers::FWriteView WriteView = ProxyRef.InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
			WriteView.Flags = ExternalChangeSet.Flags;
			WriteView.PrimitiveToRelativeWorld = ExternalChangeSet.PrimitiveToRelativeWorld;
			WriteView.PrimitiveWorldSpaceOffset = ExternalChangeSet.PrimitiveWorldSpaceOffset;
			ProxyRef.InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

			// TODO: DISP - Fix me (this comment came along from FPrimitiveSceneProxy::SetInstanceLocalBounds and is probably still true...)
			const FVector3f PadExtent = FISMCInstanceDataSceneProxy::GetLocalBoundsPadExtent(ExternalChangeSet.PrimitiveToRelativeWorld, ExternalChangeSet.AbsMaxDisplacement);
			ExternalChangeSet.InstanceLocalBounds.Min -= PadExtent;
			ExternalChangeSet.InstanceLocalBounds.Max += PadExtent;

			ProxyRef.BuildFromLegacyData(MoveTemp(ExternalChangeSet.LegacyStaticMeshInstanceData), ExternalChangeSet.InstanceLocalBounds, MoveTemp(ExternalChangeSet.LegacyInstanceReorderTable));

			check(ProxyRef.GetData().GetNumInstances() == ProxyRef.GetUpdateTaskInfo()->GetHeader().NumInstances);
			// Some instances may not be represented in the data due to density scaling, this should only be true if there is a reorder table that has the same size as the 
			check(ProxyRef.GetData().GetNumInstances() <= ProxyRef.InstanceIdIndexMap.GetMaxInstanceIndex());
			check(ProxyRef.GetUpdateTaskInfo()->GetHeader().Flags == ProxyRef.InstanceSceneDataBuffers.GetFlags());
		});

		LegacyBuildData.Reset();
		bWasUpdateQueued = true;
	}

	FChangeDesc ChangeDesc;

	// TODO: We may decide to do so if other conditions are met (e.g., large change-set or marked for full invalidation).
	// TODO: Need to figure this out in some other way, e.g., attachment counter or whatnot, since the creation has been moved up we no longer know if this is a fresh one.
	ChangeDesc.bUntrackedState = GetState() != ETrackingState::Tracked && GetState() != ETrackingState::Optimized;

	// Figure out the deltas.
	if (!ChangeDesc.bUntrackedState)
	{
		ChangeDesc.bInstancesChanged =  HasAnyInstanceChanges();

		ChangeDesc.bPrimitiveTransformChanged = !ComponentData.PrimitiveLocalToWorld.Equals(PrimitiveLocalToWorld, UE_SMALL_NUMBER);
		ChangeDesc.bMaterialUsageFlagsChanged = Flags != ComponentData.Flags;
		ChangeDesc.bMaxDisplacementChanged = AbsMaxDisplacement != NewAbsMaxDisplacement;
		ChangeDesc.bStaticMeshBoundsChanged = !StaticMeshBounds.Equals(ComponentData.StaticMeshBounds);
	}
	
	// Yet another special case to handle externally managed data from landscape grass
	if (Mode == EMode::ExternalLegacyData && (!ComponentData.PrimitiveLocalToWorld.Equals(PrimitiveLocalToWorld, UE_SMALL_NUMBER)))
	{
		ChangeDesc.bPrimitiveTransformChanged = true;
	}

	// Init previous LocalToWorld to current on initial update
	FMatrix PrevPrimitiveLocalToWorld = GetState() == ETrackingState::Initial ? ComponentData.PrimitiveLocalToWorld : PrimitiveLocalToWorld;
	// Update the tracked state
	PrimitiveLocalToWorld = ComponentData.PrimitiveLocalToWorld;
	AbsMaxDisplacement = NewAbsMaxDisplacement;
	StaticMeshBounds = ComponentData.StaticMeshBounds;
	Flags = ComponentData.Flags;
	NumCustomDataFloats = ComponentData.NumCustomDataFloats;

	const bool bAnyChange = ChangeDesc.Packed != 0;

	if (!bAnyChange)
	{
		return bWasUpdateQueued;
	}
	
	// No need to send an update for an ISM with zero source instances && no instance updates 
	// except if the primitive transform changed, in which case we need to pipe that through if there are instances in the proxy.
	if (ComponentData.NumSourceInstances == 0 && !ChangeDesc.bInstancesChanged 
		&& !(ChangeDesc.bPrimitiveTransformChanged && ComponentData.NumProxyInstances != 0))
	{
		ClearChangeTracking();
		return bWasUpdateQueued;
	}

	// If we got here & the state is "optimized" then we know the precomputed data is now invalid and we ditch it.
	if (GetState() == ETrackingState::Optimized)
	{
#if !UE_BUILD_SHIPPING
		static bool bLoggedOnce = false;
		if (!bLoggedOnce)
		{
			UE_LOGF(LogInstanceProxy, Log, "Discarded PrecomputedOptimizationData");
			bLoggedOnce = true;
		}
#endif
		PrecomputedOptimizationData.Reset();
	}

	ETrackingState SuccessorTrackingState = ETrackingState::Tracked;
	// 
	{
		// TODO: Maybe specialize for only bPrimitiveTransformChanged (no need to send/modify anything _other_ than the transform data)
		// TODO: The states other than bCreateNewProxy _can_ be handled through delta updates (e.g., add instance + offset all the rest).
		bool bNeedFullUpdate = ChangeDesc.bUntrackedState || ChangeDesc.bMaterialUsageFlagsChanged || GetState() == ETrackingState::Optimized;

		// NOTE: Moving the update tracker to the change set implicitly resets it.
		FISMInstanceUpdateChangeSet ChangeSet(bNeedFullUpdate, MoveTemp(InstanceUpdateTracker), ComponentData.NumProxyInstances);
		ChangeSet.bUpdateAllInstanceTransforms = ChangeDesc.bPrimitiveTransformChanged || bTransformChangedAllInstances;
		ChangeSet.PostUpdateNumInstances = ComponentData.NumProxyInstances;
		ChangeSet.NumSourceInstances = ComponentData.NumSourceInstances;

		// Initialize the change set before collecting instance change data.
		InitChangeSet(ChangeDesc, ComponentData, ChangeSet);

		// Too many numbers, make sure they line up.
		if (Mode == EMode::Default)
		{
			check(ChangeSet.LegacyInstanceReorderTable.IsEmpty());
			check(ComponentData.NumSourceInstances == GetMaxInstanceIndex());
			check(ComponentData.NumProxyInstances == GetMaxInstanceIndex());
		}

		// Callback to the owner to fill in change data.
		ComponentData.BuildChangeSet(ChangeSet);
		
		// make sure the custom data change is correctly tracked
		check(!ChangeSet.Flags.bHasPerInstanceCustomData || NumCustomDataFloats == ChangeSet.NumCustomDataFloats);
		checkSlow(ChangeSet.Flags.bHasPerInstanceCustomData || ChangeSet.GetCustomDataDelta().IsEmpty() && ChangeSet.PerInstanceCustomData.IsEmpty());

		// If we have per-instance previous local to world, they are expected to be in the local space of the _previous_ local to world. If they are in fact not (e.g., if someone sets them explicitly from world space) then, well, this won't be correct
		if (ChangeSet.Flags.bHasPerInstanceDynamicData)
		{
			TOptional<FTransform> PreviousTransform = FMotionVectorSimulation::Get().GetPreviousTransform(PrimitiveComponent.Get());
			if (PreviousTransform.IsSet())
			{
				ChangeSet.PreviousPrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(FVector3f{ ChangeSet.PrimitiveWorldSpaceOffset }, PreviousTransform.GetValue().ToMatrixWithScale()).M;
			}
			else
			{
				ChangeSet.PreviousPrimitiveToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(FVector3f{ ChangeSet.PrimitiveWorldSpaceOffset }, PrevPrimitiveLocalToWorld).M;				
			}
		}

		// Assemble header info to enable nonblocking primitive update.
		FInstanceDataBufferHeader InstanceDataBufferHeader;

		InstanceDataBufferHeader.NumInstances = ChangeSet.PostUpdateNumInstances;
		InstanceDataBufferHeader.PayloadDataStride = FInstanceSceneDataBuffers::CalcPayloadDataStride(ChangeSet.Flags, ChangeSet.NumCustomDataFloats, 0);
		InstanceDataBufferHeader.Flags = ChangeSet.Flags;

		CSV_CUSTOM_STAT_GLOBAL(NumInstanceTransformUpdates, ChangeSet.Transforms.Num(), ECsvCustomStatOp::Accumulate);

		// Special handling for the case of external data & transform change, may want to extend this to others (but need to send data in that case)
		if (Mode == EMode::ExternalLegacyData && ChangeDesc.bPrimitiveTransformChanged)
		{
			LOG_INST_DATA(TEXT("Primitive Transform Update %s"), TEXT(""));
			SuccessorTrackingState = ETrackingState::Initial;
			DispatchInstanceDataUpdateTask(bIsUnattached, Proxy, InstanceDataBufferHeader, [ChangeSet = MoveTemp(ChangeSet), Proxy = Proxy, NumHeaderInstances = InstanceDataBufferHeader.NumInstances] () mutable 
			{
				Proxy->UpdatePrimitiveTransform(MoveTemp(ChangeSet));
				check(NumHeaderInstances == Proxy->GeInstanceSceneDataBuffers()->GetNumInstances());
			});
		}
		else if (bNeedFullUpdate)
		{
			LOG_INST_DATA(TEXT("Full Build %s"), TEXT(""));
			if (PrecomputedOptimizationData.IsValid())
			{
				check(Mode == EMode::Default);
				check(ComponentData.ComponentMobility == EComponentMobility::Static || ComponentData.ComponentMobility == EComponentMobility::Stationary);

				LOG_INST_DATA(TEXT("  Optimized Build (%s)"), PrecomputedOptimizationData.IsValid() ? TEXT("Precomputed") : TEXT(""));

				if (PrecomputedOptimizationData->PrimitiveLocalToWorldHash == HashLocalToWorld(ComponentData.PrimitiveLocalToWorld))
				{
					Proxy->bBuildOptimized = true;
					Proxy->PrecomputedOptimizationData = PrecomputedOptimizationData;
					SuccessorTrackingState = ETrackingState::Optimized;
				}
				else
				{
					PrecomputedOptimizationData.Reset();
				}
			}

			DispatchInstanceDataUpdateTask(bIsUnattached, Proxy, InstanceDataBufferHeader, [ChangeSet = MoveTemp(ChangeSet), Proxy = Proxy, NumHeaderInstances = InstanceDataBufferHeader.NumInstances] () mutable 
			{
				Proxy->Build(MoveTemp(ChangeSet));
				check(NumHeaderInstances == Proxy->GeInstanceSceneDataBuffers()->GetNumInstances());
			});
		}
		else
		{
			LOG_INST_DATA(TEXT("Delta Update %s"), TEXT(""));
			check(!ChangeDesc.bUntrackedState);
			DispatchInstanceDataUpdateTask(bIsUnattached, Proxy, InstanceDataBufferHeader, [ChangeSet = MoveTemp(ChangeSet), Proxy = Proxy, NumHeaderInstances = InstanceDataBufferHeader.NumInstances] () mutable 
			{
				Proxy->Update(MoveTemp(ChangeSet));
				check(NumHeaderInstances == Proxy->GeInstanceSceneDataBuffers()->GetNumInstances());
			});
		}
	}

	// After an update has been sent, we need to track all deltas.
	TrackingState = SuccessorTrackingState;

	return true;
}

void FPrimitiveInstanceDataManager::PostLoad(int32 InNumInstances)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	NumInstances = InNumInstances;
}

void FPrimitiveInstanceDataManager::ClearIdTracking(int32 InNumInstances)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	// Reset the mapping to identity & clear allocator, this looses all association with ID:Index that existed before
	IndexToIdMap.Empty();
	IdToIndexMap.Empty();
	NumInstances = InNumInstances;
	ValidInstanceIdMask.Empty();
	IdSearchStartIndex = 0;

	// Also clear the change tracking since it is not valid anymore
	ClearChangeTracking();
}

void FPrimitiveInstanceDataManager::ClearChangeTracking()
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	// When tracking data is cleared, we loose connection to previously tracked state until the next update is sent.
	TrackingState = ETrackingState::Initial;

	LegacyBuildData.Reset();
	InstanceUpdateTracker.Reset();
	bNumCustomDataChanged = false;
	bBakedLightingDataChanged = false;
	bTransformChangedAllInstances = false;

#if WITH_EDITOR
	bAnyEditorDataChanged = false;
#endif	
	bPrimitiveTransformChanged = false;
}

int32 FPrimitiveInstanceDataManager::GetMaxInstanceId() const
{
	return HasIdentityMapping() ? NumInstances : ValidInstanceIdMask.Num();
}

int32 FPrimitiveInstanceDataManager::GetMaxInstanceIndex() const
{
	return HasIdentityMapping() ? NumInstances : IndexToIdMap.Num();
}

void FPrimitiveInstanceDataManager::CreateExplicitIdentityMapping()
{
	check(HasIdentityMapping());
	IndexToIdMap.SetNumUninitialized(NumInstances);
	IdToIndexMap.SetNumUninitialized(NumInstances);
	for (int32 Index = 0; Index < NumInstances; ++Index)
	{
		IndexToIdMap[Index] = FPrimitiveInstanceId{Index};
		IdToIndexMap[Index] = Index;
	}
	ValidInstanceIdMask.Reset();
	ValidInstanceIdMask.SetNum(NumInstances, true);
	IdSearchStartIndex = NumInstances;
}

template<FPrimitiveInstanceDataManager::EChangeFlag Flag>
void FPrimitiveInstanceDataManager::MarkChangeHelper(int32 InstanceIndex)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	check(Mode != EMode::ExternalLegacyData);

	if (GetState() != ETrackingState::Tracked)
	{
		bAnyInstanceChange = true;
		MarkComponentRenderInstancesDirty();
		return;
	}
	InstanceUpdateTracker.MarkIndex<Flag>(InstanceIndex, GetMaxInstanceIndex());
	MarkComponentRenderInstancesDirty();
}

template<FPrimitiveInstanceDataManager::EChangeFlag Flag>
void FPrimitiveInstanceDataManager::MarkChangeHelper(FPrimitiveInstanceId InstanceId)
{
	check(Mode != EMode::ExternalLegacyData);

	if (GetState() != ETrackingState::Tracked)
	{
		bAnyInstanceChange = true;
		MarkComponentRenderInstancesDirty();
		return;
	}
	MarkChangeHelper<Flag>(IdToIndex(InstanceId));
}

void FPrimitiveInstanceDataManager::MarkComponentRenderInstancesDirty()
{
	if (!bComponentMarkedDirty)
	{
		if (UPrimitiveComponent *PrimitiveComponentPtr = PrimitiveComponent.Get())
		{
			PrimitiveComponentPtr->MarkRenderInstancesDirty();
			bComponentMarkedDirty = true;
		}
	}
}

bool FPrimitiveInstanceDataManager::HasIdentityMapping() const
{
	return IndexToIdMap.IsEmpty();
}

void FPrimitiveInstanceDataManager::FreeInstanceId(FPrimitiveInstanceId InstanceId)
{
	LOG_INST_DATA(TEXT("FreeInstanceId(Id: %d)"), InstanceId.Id);

	if (!HasIdentityMapping())
	{
		IdToIndexMap[InstanceId.Id] = INDEX_NONE;
		ValidInstanceIdMask[InstanceId.Id] = false;
		// Must start from the lowest free index since we'd otherwise get holes when adding things.
		IdSearchStartIndex = FMath::Min(IdSearchStartIndex, InstanceId.Id);
	}

	LOG_INST_DATA(TEXT("IdToIndexMap[%d] = %d"), InstanceId.Id, INDEX_NONE);
}

TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe> FPrimitiveInstanceDataManager::GetOrCreateProxy(ERHIFeatureLevel::Type InFeatureLevel)
{
	LOG_INST_DATA(TEXT("GetOrCreateProxy"));
	FStaticShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
	if (Proxy && !Proxy->CheckPlatformFeatureLevel(ShaderPlatform, InFeatureLevel))
	{
		// TODO: May need to add some attachment counter checks here?
		Proxy.Reset();
		ClearChangeTracking();
	}

	if (!Proxy)
	{
		if (!UseGPUScene(ShaderPlatform))
		{
			// Catch all proxy for both ISM / HISM in mobile mode.
			Proxy = MakeShared<FISMCInstanceDataSceneProxyNoGPUScene>(ShaderPlatform, InFeatureLevel, Mode == EMode::Legacy);
		}
		else if (Mode == EMode::Legacy || Mode == EMode::ExternalLegacyData)
		{
			Proxy = MakeShared<FISMCInstanceDataSceneProxyLegacyReordered>(ShaderPlatform, InFeatureLevel, Mode != EMode::ExternalLegacyData);
		}
		else
		{
			Proxy = MakeShared<FISMCInstanceDataSceneProxy>(ShaderPlatform, InFeatureLevel);
		}
	}

	return Proxy;
}

void FPrimitiveInstanceDataManager::Invalidate(int32 InNumInstances)
{
	LOG_INST_DATA(TEXT("Invalidate"));
	Proxy.Reset();
	ClearIdTracking(InNumInstances);
	// When the proxy is being replaced, we want to invalidate the owning proxy thing as the continuity is lost.
	if (UPrimitiveComponent *PrimitiveComponentPtr = PrimitiveComponent.Get())
	{
		PrimitiveComponentPtr->MarkRenderStateDirty();
		bComponentMarkedDirty = true;
	}
}

#if DO_GUARD_SLOW
void FPrimitiveInstanceDataManager::ValidateMapping() const
{
	check(HasIdentityMapping() || IndexToIdMap.Num() == NumInstances);
	for (int32 Index = 0; Index < IndexToIdMap.Num(); ++Index)
	{
		FPrimitiveInstanceId Id = IndexToIdMap[Index];
		check(ValidInstanceIdMask[Id.GetAsIndex()]);
		check(Index == IdToIndexMap[Id.GetAsIndex()]);
	}
	for (int32 Id = 0; Id < IdToIndexMap.Num(); ++Id)
	{
		int32 Index = IdToIndexMap[Id];
		if (Index != INDEX_NONE)
		{
			check(ValidInstanceIdMask[Id]);
			check(IndexToIdMap[Index].GetAsIndex() == Id);
		}
		else
		{
			check(!ValidInstanceIdMask[Id]);
		}
	}
	int32 FirstFalse = ValidInstanceIdMask.Find(false);
	check(FirstFalse < 0 || FirstFalse >= IdSearchStartIndex);
}
#endif

void FPrimitiveInstanceDataManager::MarkForRebuildFromLegacy(TUniquePtr<FStaticMeshInstanceData> &&InLegacyInstanceData, const TArray<int32> &InstanceReorderTable, const TArray<TRefCountPtr<HHitProxy>> &HitProxies)
{
	if (GetState() == ETrackingState::Disabled)
	{
		return;
	}

	check(InLegacyInstanceData);
	if (Mode == EMode::ExternalLegacyData)
	{
		check(InstanceReorderTable.IsEmpty());
		ClearIdTracking(InLegacyInstanceData->GetNumInstances());
		check(GetMaxInstanceIndex() == InLegacyInstanceData->GetNumInstances());
	}
	else
	{
		// TODO: restore the ID tracking when not in external mode
		ClearIdTracking(InstanceReorderTable.Num());
		//ClearChangeTracking();
	}

	LegacyBuildData = MakePimpl<FLegacyBuildData>();
	
	// Need to capture the InstanceIdIndexMap map here, as there may be further changes before the next flush is called.
	LegacyBuildData->LegacyStaticMeshInstanceData = MoveTemp(InLegacyInstanceData);
	LegacyBuildData->LegacyInstanceReorderTable = InstanceReorderTable;
	LegacyBuildData->InstanceIdIndexMap = FInstanceIdIndexMap(*this);
#if WITH_EDITOR
	LegacyBuildData->HitProxies = HitProxies; 
#endif

	if (Mode != EMode::ExternalLegacyData)
	{
		// Must actually track the changes because we may see incremental changes in the same frame,
		TrackingState = ETrackingState::Tracked;
	}
	MarkComponentRenderInstancesDirty();
}

SIZE_T FPrimitiveInstanceDataManager::GetAllocatedSize() const
{
	return ValidInstanceIdMask.GetAllocatedSize() +
		InstanceUpdateTracker.GetAllocatedSize();
}

void FPrimitiveInstanceDataManager::OnRegister(int32 InNumInstances)
{
	LOG_INST_DATA(TEXT("OnRegister(InNumInstances : %d) NumInstances: %d"), InNumInstances, NumInstances);

	if (CVarInstanceDataResetTrackingOnRegister.GetValueOnGameThread())
	{
		// We don't clear for external legacy, AKA landscape grass, because it manages its own data, doesn't use the tracking & we don't want to clear the externall queued update for this case.
		if (Mode != EMode::ExternalLegacyData && InNumInstances != NumInstances)
		{
			ClearIdTracking(InNumInstances);
		}
	}
}

bool FPrimitiveInstanceDataManager::ShouldUsePrecomputed()
{
	static const auto CVarPrecomputed = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SceneCulling.Precomputed"));
	
	return CVarPrecomputed && CVarPrecomputed->GetValueOnAnyThread() != 0;
}

#if WITH_EDITOR

// Calling this early can save performance if no features are enabled that require BeginCacheForCookedPlatformData to be called.  Specifically, that function
// does no work if this function returns false.  If additional features require BeginCacheForCookedPlatformData to be called, or logic is added outside the
// "bShouldBuild" condition in that function, this would need to be revisited.
//
// The big advantage of this early out is that unnecessarily computing the FInstanceUpdateComponentDesc argument to BeginCacheForCookedPlatformData can
// have catastrophic effects on cook performance, because it forces an early flush of the static mesh compilation queue where GetUsedMaterials() is called,
// reducing parallelism.  On a test project, this optimization saves 11% in overall cook time following a StaticMesh version bump.
bool FPrimitiveInstanceDataManager::ShouldWriteCookedData(const ITargetPlatform* TargetPlatform, int32 NumInstancesToBuildFor)
{
	EComponentMobility::Type Mobility = PrimitiveComponent.IsValid() ? PrimitiveComponent->Mobility.GetValue() : EComponentMobility::Type::Movable;

	// Only cook for static & stationary(?) regular ISMs
	bool bValidTypeAndMobility = Mode == EMode::Default && (Mobility == EComponentMobility::Static || Mobility == EComponentMobility::Stationary);

	return bValidTypeAndMobility
		&& NumInstancesToBuildFor >= CVarInstanceDataMinInstanceCountToOptimize.GetValueOnAnyThread()
		&& ShouldUsePrecomputed()
		&& DoesTargetPlatformSupportNanite(TargetPlatform);

}

void FPrimitiveInstanceDataManager::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform, FInstanceUpdateComponentDesc &&ComponentData, TStridedView<FMatrix> InstanceTransforms)
{
	// Already precomputed, we don't need to do it twice (could add checks to see that it is not incorrect for some obscure reason)
	if (PrecomputedOptimizationData.IsValid())
	{
		return;
	}
	
	bool bShouldBuild = ShouldWriteCookedData(TargetPlatform, InstanceTransforms.Num());

	// TODO: we could kick an async thread here if that is preferrable for the cooker?
	if (bShouldBuild && ComponentData.BuildChangeSet)
	{
		uint32 StartTime = FPlatformTime::Cycles();
		PrecomputedOptimizationData = MakeShared<FISMPrecomputedSpatialHashData>(PrecomputeOptimizationData(MoveTemp(ComponentData), InstanceTransforms));
		uint32 EndtTime = FPlatformTime::Cycles();

		UE_LOGF(LogInstanceProxy, Verbose, "Build Instance Spatial Hashes (%.2fms), Instances: %d, Hashes: %d, Remap Size: %d", FPlatformTime::ToMilliseconds( EndtTime - StartTime ), InstanceTransforms.Num(), PrecomputedOptimizationData->Hashes.Num(), PrecomputedOptimizationData->ProxyIndexToComponentIndexRemap.Num());
	}	
}


FISMPrecomputedSpatialHashData FPrimitiveInstanceDataManager::PrecomputeOptimizationData(FInstanceUpdateComponentDesc &&ComponentData, TStridedView<FMatrix> InstanceTransforms)
{
	const float LocalAbsMaxDisplacement = FMath::Max(-ComponentData.PrimitiveMaterialDesc.MinMaxMaterialDisplacement.X, ComponentData.PrimitiveMaterialDesc.MinMaxMaterialDisplacement.Y)
									+ ComponentData.PrimitiveMaterialDesc.MaxWorldPositionOffsetDisplacement;

	const FVector3f PadExtent = FISMCInstanceDataSceneProxy::GetLocalBoundsPadExtent(ComponentData.PrimitiveLocalToWorld, LocalAbsMaxDisplacement);
	FRenderBounds InstanceLocalBounds = ComponentData.StaticMeshBounds;
	InstanceLocalBounds.Min -= PadExtent;
	InstanceLocalBounds.Max += PadExtent;

	FSphere LocalInstanceSphere = InstanceLocalBounds.ToBoxSphereBounds().GetSphere();

	FInstanceDataHelpers::FSpatialHashResult SpatialHashResult = FInstanceDataHelpers::BuildSpatialHashData(InstanceTransforms.Num(),
		[&](int32 InstanceIndex) -> FSphere
		{
			FMatrix InstanceLocalToWorld = InstanceTransforms[InstanceIndex] * ComponentData.PrimitiveLocalToWorld;
			return LocalInstanceSphere.TransformBy(InstanceLocalToWorld);
		}
	);

	FISMPrecomputedSpatialHashData Result;
	Result.Hashes = MoveTemp(SpatialHashResult.SpatialHashes);
	Result.ProxyIndexToComponentIndexRemap = MoveTemp(SpatialHashResult.ReorderTable);
	Result.PrimitiveLocalToWorldHash = HashLocalToWorld(ComponentData.PrimitiveLocalToWorld);
	return Result;
}

void FPrimitiveInstanceDataManager::WriteCookedRenderData(FArchive& Ar, FInstanceUpdateComponentDesc &&ComponentData, TStridedView<FMatrix> InstanceTransforms)
{
	bool bHasCookedData = false;

	bool bShouldBuild = ShouldWriteCookedData(Ar.CookingTarget(), InstanceTransforms.Num());

	if (bShouldBuild)
	{
		if (!PrecomputedOptimizationData.IsValid())
		{
			if (ComponentData.BuildChangeSet)
			{
				PrecomputedOptimizationData = MakeShared<const FISMPrecomputedSpatialHashData>(PrecomputeOptimizationData(MoveTemp(ComponentData), InstanceTransforms));
			}
		}
		
		if (PrecomputedOptimizationData.IsValid())
		{
			// We have to copy the whole thing to be able to serialize?
			FISMPrecomputedSpatialHashData OptData = *PrecomputedOptimizationData;

			// Serialize the stuff we need.
			bHasCookedData = true;
			Ar << bHasCookedData;
			Ar << OptData.PrimitiveLocalToWorldHash;

			OptData.Hashes.BulkSerialize(Ar);
			OptData.ProxyIndexToComponentIndexRemap.BulkSerialize(Ar);
		}
	}

	if (!bHasCookedData)
	{
		// write the bool if we didn't write any data previously
		Ar << bHasCookedData;
	}
}
#endif // WITH_EDITOR

void FPrimitiveInstanceDataManager::ReadCookedRenderData(FArchive& Ar)
{
	bool bHasCookedData = false;
	Ar << bHasCookedData;
	if (bHasCookedData)
	{
		FISMPrecomputedSpatialHashData Tmp;

		Ar << Tmp.PrimitiveLocalToWorldHash;

		// TODO: Pack the data representation to far fewer bits
		Tmp.Hashes.BulkSerialize(Ar);
		// TODO: RLE-compress
		Tmp.ProxyIndexToComponentIndexRemap.BulkSerialize(Ar);

		// Ditch the precomputed data if it has been disabled (in the runtime), even if the cook was done with the data enabled.
		if (ShouldUsePrecomputed())
		{
			PrecomputedOptimizationData = MakeShared<const FISMPrecomputedSpatialHashData>(Tmp);
		}
		else
		{
			PrecomputedOptimizationData.Reset();
		}
	}
}
