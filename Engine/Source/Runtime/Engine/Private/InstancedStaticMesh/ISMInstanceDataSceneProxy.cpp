// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMesh/ISMInstanceDataSceneProxy.h"
#include "InstancedStaticMesh/ISMInstanceUpdateChangeSet.h"
#include "InstanceData/InstanceDataUpdateUtils.h"
#include "Engine/InstancedStaticMesh.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Rendering/RenderingSpatialHash.h"
#include "Rendering/MotionVectorSimulation.h"

// UE_DISABLE_OPTIMIZATION

DEFINE_LOG_CATEGORY(LogInstanceProxy);

#if 0
#define LOG_INST_DATA(_Format_, ...) UE_LOG(LogInstanceProxy, Log, _Format_, ##__VA_ARGS__)
#else
	#define LOG_INST_DATA(_Format_, ...) 
#endif

FISMCInstanceDataSceneProxy::FISMCInstanceDataSceneProxy(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel) 
	: ShaderPlatform(InShaderPlatform)
	, FeatureLevel(InFeatureLevel) 
{
	bUseLegacyRenderingPath = !UseGPUScene(ShaderPlatform);
}

struct FReorderTableIndexRemap
{
	FORCEINLINE constexpr bool IsIdentity() const { return false; }

	FReorderTableIndexRemap(const TArray<int32> &InReorderTable, int32 InMaxValidIndex) : ReorderTable(InReorderTable), MaxValidIndex(InMaxValidIndex) {}

	inline int32 ClampValidIndex(int32 Index) const
	{
		if (Index < MaxValidIndex)
		{
			return Index;
		}
		return INDEX_NONE;
	}

	inline int32 operator[](int32 InIndex) const 
	{ 
		if (ReorderTable.IsValidIndex(InIndex))
		{
			return ClampValidIndex(ReorderTable[InIndex]);
		}
		return ClampValidIndex(InIndex); 
	}

	FORCEINLINE bool RemapDestIndex(int32 &Index) const 
	{ 
		Index = operator[](Index);
		return Index != INDEX_NONE; 
	}

	FORCEINLINE bool Remap(int32 &SrcIndex, int32 &DstIndex) const 
	{ 
		RemapDestIndex(DstIndex);
		return DstIndex != INDEX_NONE; 
	}

	template <typename DeltaType, typename ValueType>
	inline void Scatter(bool bHasData, const DeltaType &Delta, TArray<ValueType> &DestData, int32 NumOutElements, TArray<ValueType> &&InData, int32 ElementStride = 1) const
	{
		if (bHasData)
		{
			::Scatter(Delta, DestData, NumOutElements, MoveTemp(InData), *this, ElementStride);
		}
		else
		{
			DestData.Reset();
		}
	}

	const TArray<int32> &ReorderTable;
	int32 MaxValidIndex = 0;
};

FVector3f FISMCInstanceDataSceneProxy::GetLocalBoundsPadExtent(const FRenderTransform& LocalToWorld, float PadAmount)
{
	if (FMath::Abs(PadAmount) < UE_SMALL_NUMBER)
	{
		return FVector3f::ZeroVector;
	}

	FVector3f Scale = LocalToWorld.GetScale();
	return FVector3f(
		Scale.X > 0.0f ? PadAmount / Scale.X : 0.0f,
		Scale.Y > 0.0f ? PadAmount / Scale.Y : 0.0f,
		Scale.Z > 0.0f ? PadAmount / Scale.Z : 0.0f);
}

template <typename IndexRemapType>
void FISMCInstanceDataSceneProxy::ApplyAttributeChanges(FISMInstanceUpdateChangeSet &ChangeSet, const IndexRemapType &IndexRemap, FInstanceSceneDataBuffers::FWriteView &ProxyData)
{
	if (ChangeSet.Flags.bHasPerInstanceCustomData)
	{
		ProxyData.NumCustomDataFloats = ChangeSet.NumCustomDataFloats;
		Scatter(ChangeSet.GetCustomDataDelta(), ProxyData.InstanceCustomData, ChangeSet.PostUpdateNumInstances, MoveTemp(ChangeSet.PerInstanceCustomData), IndexRemap, ProxyData.NumCustomDataFloats);
	}
	else
	{
		ProxyData.NumCustomDataFloats = 0;
		ProxyData.InstanceCustomData.Reset();
	}

	Scatter(ChangeSet.Flags.bHasPerInstanceLMSMUVBias, ChangeSet.GetInstanceLightShadowUVBiasDelta(), ProxyData.InstanceLightShadowUVBias, ChangeSet.PostUpdateNumInstances, MoveTemp(ChangeSet.InstanceLightShadowUVBias), IndexRemap);
#if WITH_EDITOR
	Scatter(ChangeSet.Flags.bHasPerInstanceEditorData, ChangeSet.GetInstanceEditorDataDelta(), ProxyData.InstanceEditorData, ChangeSet.PostUpdateNumInstances, MoveTemp(ChangeSet.InstanceEditorData), IndexRemap);

	// replace the HP container.
	if (ChangeSet.HitProxyContainer)
	{
		HitProxyContainer = MoveTemp(ChangeSet.HitProxyContainer);
	}

#endif

	// Delayed per instance random generation, moves it off the GT and RT, but still sucks
	if (ChangeSet.Flags.bHasPerInstanceRandom)
	{
		// TODO: only need to process added instances? No help for ISM since the move path would be taken.
		// TODO: OTOH for HISM there is no meaningful data, so just skipping and letting the SetNumZeroed fill in the blanks is fine.

		ProxyData.InstanceRandomIDs.SetNumZeroed(ChangeSet.PostUpdateNumInstances);
		if (ChangeSet.GeneratePerInstanceRandomIds)
		{
			// NOTE: this is not super efficient(!)
			TArray<float> TmpInstanceRandomIDs;
			TmpInstanceRandomIDs.SetNumZeroed(ChangeSet.PostUpdateNumInstances);
			ChangeSet.GeneratePerInstanceRandomIds(TmpInstanceRandomIDs);
			FIdentityDeltaRange PerInstanceRandomDelta(TmpInstanceRandomIDs.Num());
			Scatter(PerInstanceRandomDelta, ProxyData.InstanceRandomIDs, ChangeSet.PostUpdateNumInstances, MoveTemp(TmpInstanceRandomIDs), IndexRemap);
		}
		//else 
		//{
		//	IndexRemap.Scatter(true, PerInstanceRandomDelta, ProxyData.InstanceRandomIDs, ChangeSet.PostUpdateNumInstances, MoveTemp(ChangeSet.InstanceRandomIDs));
		//}
	}
	else
	{
		ProxyData.InstanceRandomIDs.Reset();
	}
}

template <typename IndexRemapType>
void FISMCInstanceDataSceneProxy::ApplyDataChanges(FISMInstanceUpdateChangeSet &ChangeSet, const IndexRemapType &IndexRemap, int32 PostUpdateNumInstances, FInstanceSceneDataBuffers::FWriteView &ProxyData)
{
	ProxyData.PrimitiveToRelativeWorld = ChangeSet.PrimitiveToRelativeWorld;
	ProxyData.PrimitiveWorldSpaceOffset = ChangeSet.PrimitiveWorldSpaceOffset;
		
	check(!ChangeSet.Flags.bHasPerInstanceLocalBounds);
	// TODO: delta support & always assume all bounds changed, and that there is in fact only one
	ProxyData.InstanceLocalBounds = MoveTemp(ChangeSet.InstanceLocalBounds);

	// TODO: DISP - Fix me (this comment came along from FPrimitiveSceneProxy::SetInstanceLocalBounds and is probably still true...)
	const FVector3f PadExtent = GetLocalBoundsPadExtent(ProxyData.PrimitiveToRelativeWorld, ChangeSet.AbsMaxDisplacement);
	for (FRenderBounds& Bounds : ProxyData.InstanceLocalBounds)
	{
		Bounds.Min -= PadExtent;
		Bounds.Max += PadExtent;
	}

	// unpack transform deltas
	ApplyTransformUpdates(ChangeSet.GetTransformDelta(), IndexRemap, ChangeSet.PrimitiveToRelativeWorld, ChangeSet.Transforms, PostUpdateNumInstances, ProxyData.InstanceToPrimitiveRelative);
	if (ChangeSet.Flags.bHasPerInstanceDynamicData)
	{
		FRenderTransform PrevPrimitiveToRelativeWorld = ChangeSet.PreviousPrimitiveToRelativeWorld.Get(ChangeSet.PrimitiveToRelativeWorld);
		ApplyTransformUpdates(ChangeSet.GetTransformDelta(), IndexRemap, PrevPrimitiveToRelativeWorld, ChangeSet.PrevTransforms, PostUpdateNumInstances, ProxyData.PrevInstanceToPrimitiveRelative);
	}
	else
	{
		ProxyData.PrevInstanceToPrimitiveRelative.Reset();
	}

	ApplyAttributeChanges(ChangeSet, IndexRemap, ProxyData);
}

void FISMCInstanceDataSceneProxy::BuildFromOptimizedDataBuffers(FISMInstanceUpdateChangeSet& ChangeSet, FInstanceIdIndexMap &OutInstanceIdIndexMap, FInstanceSceneDataBuffers::FWriteView &ProxyData)
{
	SCOPED_NAMED_EVENT(FISMCInstanceDataSceneProxy_BuildFromOptimizedDataBuffers, FColor::Emerald);

	ProxyData.PrimitiveToRelativeWorld = ChangeSet.PrimitiveToRelativeWorld;
	ProxyData.PrimitiveWorldSpaceOffset = ChangeSet.PrimitiveWorldSpaceOffset;
		
	check(!ChangeSet.Flags.bHasPerInstanceLocalBounds);
	
	// TODO: delta support & always assume all bounds changed, and that there is in fact only one
	ProxyData.InstanceLocalBounds = MoveTemp(ChangeSet.InstanceLocalBounds);

	// TODO: DISP - Fix me (this comment came along from FPrimitiveSceneProxy::SetInstanceLocalBounds and is probably still true...)
	const FVector3f PadExtent = GetLocalBoundsPadExtent(ProxyData.PrimitiveToRelativeWorld, ChangeSet.AbsMaxDisplacement);
	for (FRenderBounds& Bounds : ProxyData.InstanceLocalBounds)
	{
		Bounds.Min -= PadExtent;
		Bounds.Max += PadExtent;
	}

	// If preoptimized:
	if (PrecomputedOptimizationData.IsValid())
	{
		if (PrecomputedOptimizationData->ProxyIndexToComponentIndexRemap.IsEmpty())
		{
			ApplyTransformUpdates(ChangeSet.GetTransformDelta(), FIdentityIndexRemap(), ChangeSet.PrimitiveToRelativeWorld, ChangeSet.Transforms, ChangeSet.PostUpdateNumInstances, ProxyData.InstanceToPrimitiveRelative);
			ApplyAttributeChanges(ChangeSet, FIdentityIndexRemap(), ProxyData);
		}
		else
		{
			FSrcIndexRemap SortedInstancesRemap(PrecomputedOptimizationData->ProxyIndexToComponentIndexRemap);
			ApplyTransformUpdates(ChangeSet.GetTransformDelta(), SortedInstancesRemap, ChangeSet.PrimitiveToRelativeWorld, ChangeSet.Transforms, ChangeSet.PostUpdateNumInstances, ProxyData.InstanceToPrimitiveRelative);
			ApplyAttributeChanges(ChangeSet, SortedInstancesRemap, ProxyData);
		}

		// We don't store an ID mapping for this case, since we assume a full rebuild is needed to handle any changes at all.
		InstanceIdIndexMap.Reset(ChangeSet.PostUpdateNumInstances);
	
		InstanceSceneDataBuffers.SetImmutable(FInstanceSceneDataImmutable(PrecomputedOptimizationData->Hashes), ProxyData.AccessTag);

		// Clear the data, we're done with it and it is never coming back (until it is loaded again)
		PrecomputedOptimizationData.Reset();
		return;
	}
}

template <typename ValueArrayType>
static void PadArrayWithZeroData(bool bUsed, int32 DesiredSize, ValueArrayType &Array)
{
	if (bUsed)
	{
		Array.SetNumZeroed(DesiredSize);
	}
}

void FISMCInstanceDataSceneProxy::TestAndApplyInstanceBufferSizeFixup(int32 PostUpdateNumInstances)
{
	// This is a fixup (that should never run, hopefully) to make sure the number of instances in the buffer matches the PostUpdateNumInstances. This is important because PostUpdateNumInstances is what 
	// the renderer expects and will use that to index.
	if (!ensureMsgf(PostUpdateNumInstances == InstanceSceneDataBuffers.GetNumInstances(), TEXT("The number of instances does not match what was promised in the update dispatch! Padding with dummy transforms to prevent crashes!")))
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
		if (PostUpdateNumInstances != 0 && ProxyData.InstanceLocalBounds.IsEmpty())
		{
			static const FRenderBounds ZeroRenderBounds(FVector3f::ZeroVector, FVector3f::ZeroVector);
			ProxyData.InstanceLocalBounds.Add(ZeroRenderBounds);
		}

		// If it is empty, we can't figure out the number of float4s for each, so in this case the only safe thing is to remove all of them and mark as no existing.
		if (ProxyData.Flags.bHasPerInstancePayloadExtension && 
			(ProxyData.InstancePayloadExtension.IsEmpty() || ProxyData.InstanceToPrimitiveRelative.IsEmpty()
				// or it is not divisible, then it is likely messed up somehow.
				|| (ProxyData.InstancePayloadExtension.Num() % ProxyData.InstanceToPrimitiveRelative.Num()) != 0))
		{
			ProxyData.Flags.bHasPerInstancePayloadExtension = false;
		}
		else
		{
			int32 StrideInFloat4s = ProxyData.InstancePayloadExtension.Num() / ProxyData.InstanceToPrimitiveRelative.Num();
			PadArrayWithZeroData(ProxyData.Flags.bHasPerInstancePayloadExtension, PostUpdateNumInstances * StrideInFloat4s, ProxyData.InstancePayloadExtension);
		}

		// Fill the array with zero transforms, these are not particularly useful but should prevent memory violations in shipping builds.
		ProxyData.InstanceToPrimitiveRelative.SetNumZeroed(PostUpdateNumInstances);

		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceCustomData, PostUpdateNumInstances * ProxyData.NumCustomDataFloats, ProxyData.InstanceCustomData);
		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceDynamicData, PostUpdateNumInstances, ProxyData.PrevInstanceToPrimitiveRelative);
#if WITH_EDITOR
		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceEditorData, PostUpdateNumInstances, ProxyData.InstanceEditorData);
#endif
		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceHierarchyOffset, PostUpdateNumInstances, ProxyData.InstanceHierarchyOffset);
		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceLMSMUVBias, PostUpdateNumInstances, ProxyData.InstanceLightShadowUVBias);
		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceLocalBounds, PostUpdateNumInstances, ProxyData.InstanceLocalBounds);
		PadArrayWithZeroData(ProxyData.Flags.bHasPerInstanceRandom, PostUpdateNumInstances, ProxyData.InstanceRandomIDs);
		if (ProxyData.Flags.bHasPerInstanceVisible)
		{
			ProxyData.VisibleInstances.SetNum(PostUpdateNumInstances, false);
		}

		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
	}

}

void FISMCInstanceDataSceneProxy::Build(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	SCOPED_NAMED_EVENT(FISMCInstanceDataSceneProxy_Build, FColor::Emerald);

	DecStatCounters();
	check(ChangeSet.IsFullUpdate());
	checkSlow(!ChangeSet.GetTransformDelta().IsDelta());
	checkSlow(!ChangeSet.GetCustomDataDelta().IsDelta() || (!ChangeSet.Flags.bHasPerInstanceCustomData && ChangeSet.GetCustomDataDelta().IsEmpty()));
	checkSlow(!ChangeSet.GetInstanceLightShadowUVBiasDelta().IsDelta() || ChangeSet.GetInstanceLightShadowUVBiasDelta().IsEmpty());
#if WITH_EDITOR
	checkSlow(!ChangeSet.GetInstanceEditorDataDelta().IsDelta() || ChangeSet.GetInstanceEditorDataDelta().IsEmpty());
#endif

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView WriteView = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

	WriteView.Flags = ChangeSet.Flags;

	if (bBuildOptimized && ChangeSet.PostUpdateNumInstances)
	{
		BuildFromOptimizedDataBuffers(ChangeSet, InstanceIdIndexMap, WriteView);
	}
	else
	{
		UpdateIdMapping(ChangeSet, FIdentityIndexRemap());
		check(ChangeSet.PostUpdateNumInstances == InstanceIdIndexMap.GetMaxInstanceIndex());

		FIdentityIndexRemap IndexRemap;
		ApplyDataChanges(ChangeSet, IndexRemap, InstanceIdIndexMap.GetMaxInstanceIndex(), WriteView);
	}
	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	InstanceSceneDataBuffers.ValidateData();

	IncStatCounters();

	// This auto-resets such that following builds are _NOT_ doing the opt (these are symptoms of something that was expected to be static, was built anyway)
	bBuildOptimized = false;
}

template <typename IndexRemapType>
void FISMCInstanceDataSceneProxy::UpdateIdMapping(FISMInstanceUpdateChangeSet& ChangeSet, const IndexRemapType &IndexRemap)
{
	::UpdateIdMapping(ChangeSet, IndexRemap, InstanceIdIndexMap);
}

void FISMCInstanceDataSceneProxy::Update(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	SCOPED_NAMED_EVENT(FISMCInstanceDataSceneProxy_Update, FColor::Emerald);
	check(!ChangeSet.IsFullUpdate());

	DecStatCounters();

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);


	int32 PostUpdateNumInstances = ChangeSet.PostUpdateNumInstances;

	// Handle data movement, needs old & new ID maps
	// These can only be caused by removes, which means an item can only ever move towards lower index in the array.
	// Thus, we can always safely overwrite the data in the new slot, since we do them in increasing order.
	// NOTE: If we start allowing some other kind of permutation of the ISM data, this assumption will break.
	// TODO: Add validation code somewhere in the pipeline.
	const auto IndexDelta = ChangeSet.GetIndexChangedDelta();
	for (auto It = IndexDelta.GetIterator(); It; ++It)
	{
		// Index in the source (e.g., component)
		const int32 ToIndex = It.GetIndex();
		if (!ChangeSet.InstanceAttributeTracker.TestFlag<FInstanceAttributeTracker::EFlag::Added>(ToIndex))
		{
			int32 ItemIndex = It.GetItemIndex();
			FPrimitiveInstanceId InstanceId = ChangeSet.bIdentityIdMap ? FPrimitiveInstanceId{ToIndex} : ChangeSet.IndexToIdMapDeltaData[ItemIndex];
			if (InstanceIdIndexMap.IsValidId(InstanceId))
			{
				const int32 FromIndex =  InstanceIdIndexMap.IdToIndex(InstanceId);

				ProxyData.InstanceToPrimitiveRelative[ToIndex] = ProxyData.InstanceToPrimitiveRelative[FromIndex];
				CondMove(ProxyData.Flags.bHasPerInstanceCustomData, ProxyData.InstanceCustomData, FromIndex, ToIndex, ProxyData.NumCustomDataFloats);
				CondMove(ProxyData.Flags.bHasPerInstanceRandom, ProxyData.InstanceRandomIDs, FromIndex, ToIndex);
				CondMove(ProxyData.Flags.bHasPerInstanceLMSMUVBias, ProxyData.InstanceLightShadowUVBias, FromIndex, ToIndex);
#if WITH_EDITOR
				CondMove(ProxyData.Flags.bHasPerInstanceEditorData, ProxyData.InstanceEditorData, FromIndex, ToIndex);
#endif
			}
		}
	}
	ProxyData.Flags = ChangeSet.Flags;

	UpdateIdMapping(ChangeSet, FIdentityIndexRemap());
	check(ChangeSet.PostUpdateNumInstances == InstanceIdIndexMap.GetMaxInstanceIndex());

	FIdentityIndexRemap IndexRemap;
	ApplyDataChanges(ChangeSet, IndexRemap, PostUpdateNumInstances, ProxyData);

	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	InstanceSceneDataBuffers.ValidateData();

	IncStatCounters();
}

void FISMCInstanceDataSceneProxy::DebugDrawInstanceChanges(FPrimitiveDrawInterface* DebugPDI, ESceneDepthPriorityGroup SceneDepthPriorityGroup)
{
	InstanceDataUpdateTaskInfo.WaitForUpdateCompletion();
#if 0
	// TODO: The tracked changes are not available in the proxy. Need a new  mechanism to propagate this info, probably.
	for (int Index = 0; Index < InstanceSceneDataBuffers.GetNumInstances(); ++Index)
	{
		FPrimitiveInstanceId InstanceId = InstanceIdIndexMap.IndexToId(Index);

		FMatrix InstanceToWorld = InstanceSceneDataBuffers.GetInstanceToWorld(Index);
		DrawWireStar(DebugPDI, InstanceToWorld.GetOrigin(), 40.0f, ChangeMask.TransformChangedInstances.WasChanged(InstanceId) ? FColor::Red : FColor::Green, SceneDepthPriorityGroup);

		if (ChangeMask.CustomDataChangedInstances.WasChanged(InstanceId))
		{
			DrawCircle(DebugPDI, InstanceToWorld.GetOrigin(), FVector(1, 0, 0), FVector(0, 1, 0), FColor::Orange, 40.0f, 32, SceneDepthPriorityGroup);
		}
	}
#endif
}

FISMCInstanceDataSceneProxyLegacyReordered::FISMCInstanceDataSceneProxyLegacyReordered(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, bool bInLegacyReordered) 
	: FISMCInstanceDataSceneProxy(InShaderPlatform, InFeatureLevel)
	, bLegacyReordered(bInLegacyReordered)
{
}

void FISMCInstanceDataSceneProxyLegacyReordered::Update(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	check(!ChangeSet.IsFullUpdate());
	check(bLegacyReordered || ChangeSet.LegacyInstanceReorderTable.IsEmpty());
	DecStatCounters();

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

	ProxyData.Flags = ChangeSet.Flags;
	UpdateIdMapping(ChangeSet, FIdentityIndexRemap());

	LegacyInstanceReorderTable = MoveTemp(ChangeSet.LegacyInstanceReorderTable);
	FReorderTableIndexRemap IndexRemap(LegacyInstanceReorderTable, ChangeSet.PostUpdateNumInstances);

	// Use the index reorder table to scatter the data to the correct locations.
	ApplyDataChanges(ChangeSet, IndexRemap, ChangeSet.PostUpdateNumInstances, ProxyData);
	
	if (bLegacyReordered && !LegacyInstanceReorderTable.IsEmpty())
	{
		// Make sure any instance no longer represented in the reorder table is hidden.
		ProxyData.VisibleInstances.Init(false, ChangeSet.PostUpdateNumInstances);
		for (int32 InstanceIndex : LegacyInstanceReorderTable)
		{
			if (ProxyData.VisibleInstances.IsValidIndex(InstanceIndex))
			{
				ProxyData.VisibleInstances[InstanceIndex] = true;
			}
		}
		ProxyData.Flags.bHasPerInstanceVisible = true;
	}
	else
	{
		// Mark everything as visible from the start.
		ProxyData.VisibleInstances.Reset();
		ProxyData.Flags.bHasPerInstanceVisible = false;
	}	

	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	InstanceSceneDataBuffers.ValidateData();
	IncStatCounters();
}


void FISMCInstanceDataSceneProxyLegacyReordered::Build(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	DecStatCounters();
	check(ChangeSet.IsFullUpdate());
	checkSlow(!ChangeSet.GetTransformDelta().IsDelta());
	checkSlow(!ChangeSet.GetCustomDataDelta().IsDelta() || (!ChangeSet.Flags.bHasPerInstanceCustomData && ChangeSet.GetCustomDataDelta().IsEmpty()));
	checkSlow(!ChangeSet.GetInstanceLightShadowUVBiasDelta().IsDelta() || ChangeSet.GetInstanceLightShadowUVBiasDelta().IsEmpty());
	check(bLegacyReordered || ChangeSet.LegacyInstanceReorderTable.IsEmpty());
#if WITH_EDITOR
	checkSlow(!ChangeSet.GetInstanceEditorDataDelta().IsDelta() || ChangeSet.GetInstanceEditorDataDelta().IsEmpty());
#endif

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

	LegacyInstanceReorderTable = MoveTemp(ChangeSet.LegacyInstanceReorderTable);
	ProxyData.Flags = ChangeSet.Flags;

	UpdateIdMapping(ChangeSet, FIdentityIndexRemap());

	FReorderTableIndexRemap IndexRemap(LegacyInstanceReorderTable, ChangeSet.PostUpdateNumInstances);
	ApplyDataChanges(ChangeSet, IndexRemap, ChangeSet.PostUpdateNumInstances, ProxyData);

	// Is there is a reorder table and it does not have the same number as the instances, some must be hidden
	if (bLegacyReordered && ChangeSet.PostUpdateNumInstances != LegacyInstanceReorderTable.Num())
	{
		ProxyData.VisibleInstances.Init(false, ChangeSet.PostUpdateNumInstances);
		for (int32 InstanceIndex : LegacyInstanceReorderTable)
		{
			if (IndexRemap.RemapDestIndex(InstanceIndex))
			{
				ProxyData.VisibleInstances[InstanceIndex] = true;
			}
		}
		ProxyData.Flags.bHasPerInstanceVisible = true;
	}
	else
	{
		// Mark everything as visible from the start.
		ProxyData.VisibleInstances.Reset();
		ProxyData.Flags.bHasPerInstanceVisible = false;
	}	
	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	InstanceSceneDataBuffers.ValidateData();
	IncStatCounters();
}


void FISMCInstanceDataSceneProxyLegacyReordered::BuildFromLegacyData(TUniquePtr<FStaticMeshInstanceData> &&InExternalLegacyData, const FRenderBounds &InstanceLocalBounds, TArray<int32> &&InLegacyInstanceReorderTable)
{
	DecStatCounters();

	ExternalLegacyData = MoveTemp(InExternalLegacyData);
	check(bLegacyReordered || InLegacyInstanceReorderTable.IsEmpty());
	LegacyInstanceReorderTable = MoveTemp(InLegacyInstanceReorderTable);

	check(!bUseLegacyRenderingPath);
	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

	// Not supported in this path
	ProxyData.Flags.bHasPerInstanceDynamicData = false;
	ProxyData.PrevInstanceToPrimitiveRelative.Empty();
	check(!ProxyData.Flags.bHasPerInstancePayloadExtension);

	int32 NumInstances = ExternalLegacyData->GetNumInstances();
	ProxyData.VisibleInstances.Reset();
	ProxyData.VisibleInstances.SetNum(NumInstances, true);

	ProxyData.InstanceToPrimitiveRelative.Reset(NumInstances);

	ProxyData.InstanceLightShadowUVBias.SetNumZeroed(ProxyData.Flags.bHasPerInstanceLMSMUVBias ? NumInstances : 0);
	ProxyData.InstanceLocalBounds = MakeArrayView(&InstanceLocalBounds, 1);
	ProxyData.NumCustomDataFloats = ExternalLegacyData->GetNumCustomDataFloats();
	ProxyData.InstanceCustomData.SetNumZeroed(ProxyData.Flags.bHasPerInstanceCustomData ? NumInstances * ProxyData.NumCustomDataFloats : 0); 

	ProxyData.InstanceRandomIDs.SetNumZeroed(ProxyData.Flags.bHasPerInstanceRandom ? NumInstances : 0);

#if WITH_EDITOR
	ProxyData.InstanceEditorData.SetNumZeroed(ProxyData.Flags.bHasPerInstanceEditorData ? NumInstances : 0);
#endif
	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
	{
		FRenderTransform InstanceToPrimitive;
		ExternalLegacyData->GetInstanceTransform(InstanceIndex, InstanceToPrimitive);
		FRenderTransform LocalToPrimitiveRelativeWorld = InstanceToPrimitive * ProxyData.PrimitiveToRelativeWorld;
		// Remove shear
		LocalToPrimitiveRelativeWorld.Orthogonalize();
		ProxyData.InstanceToPrimitiveRelative.Add(LocalToPrimitiveRelativeWorld);

		if (ProxyData.Flags.bHasPerInstanceDynamicData)
		{
			// TODO: this doesn't exist...
		}

		if (ProxyData.Flags.bHasPerInstanceCustomData)
		{
			ExternalLegacyData->GetInstanceCustomDataValues(InstanceIndex, MakeArrayView(ProxyData.InstanceCustomData.GetData() + InstanceIndex * ProxyData.NumCustomDataFloats, ProxyData.NumCustomDataFloats));
		}

		if (ProxyData.Flags.bHasPerInstanceRandom)
		{
			ExternalLegacyData->GetInstanceRandomID(InstanceIndex, ProxyData.InstanceRandomIDs[InstanceIndex]);
		}

		if (ProxyData.Flags.bHasPerInstanceLMSMUVBias)
		{
			ExternalLegacyData->GetInstanceLightMapData(InstanceIndex, ProxyData.InstanceLightShadowUVBias[InstanceIndex]);
		}

#if WITH_EDITOR
		// TODO:
		if (ProxyData.Flags.bHasPerInstanceEditorData)
		{
			FColor HitProxyColor;
			bool bSelected;
			ExternalLegacyData->GetInstanceEditorData(InstanceIndex, HitProxyColor, bSelected);
			ProxyData.InstanceEditorData[InstanceIndex] = FInstanceEditorData::Pack(HitProxyColor, bSelected);
		}
#endif
	}
	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	InstanceSceneDataBuffers.ValidateData();
	IncStatCounters();
}

void FISMCInstanceDataSceneProxyLegacyReordered::UpdateInstancesTransforms(FInstanceSceneDataBuffers::FWriteView &ProxyData, const FStaticMeshInstanceData &LegacyInstanceData)
{
	ProxyData.PrevInstanceToPrimitiveRelative.Empty();
	check(!ProxyData.Flags.bHasPerInstancePayloadExtension);
	int32 NumInstances = LegacyInstanceData.GetNumInstances();
	ProxyData.InstanceToPrimitiveRelative.Reset(NumInstances);
	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
	{
		FRenderTransform InstanceToPrimitive;
		LegacyInstanceData.GetInstanceTransform(InstanceIndex, InstanceToPrimitive);
		FRenderTransform LocalToPrimitiveRelativeWorld = InstanceToPrimitive * ProxyData.PrimitiveToRelativeWorld;
		// Remove shear
		LocalToPrimitiveRelativeWorld.Orthogonalize();
		ProxyData.InstanceToPrimitiveRelative.Add(LocalToPrimitiveRelativeWorld);
	}
}

void FISMCInstanceDataSceneProxyLegacyReordered::UpdatePrimitiveTransform(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	check(ExternalLegacyData && InstanceSceneDataBuffers.GetNumInstances() == ExternalLegacyData->GetNumInstances() || InstanceSceneDataBuffers.GetNumInstances() == 0);

	if (ExternalLegacyData)
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
		ProxyData.PrimitiveToRelativeWorld = ChangeSet.PrimitiveToRelativeWorld;
		ProxyData.PrimitiveWorldSpaceOffset = ChangeSet.PrimitiveWorldSpaceOffset;
		UpdateInstancesTransforms(ProxyData, *ExternalLegacyData);
		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

		InstanceSceneDataBuffers.ValidateData();
	}

	TestAndApplyInstanceBufferSizeFixup(ChangeSet.PostUpdateNumInstances);
}

FISMCInstanceDataSceneProxyNoGPUScene::FISMCInstanceDataSceneProxyNoGPUScene(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, bool bInLegacyReordered) 
	: FISMCInstanceDataSceneProxyLegacyReordered(InShaderPlatform, InFeatureLevel, bInLegacyReordered)
{
}

FISMCInstanceDataSceneProxyNoGPUScene::~FISMCInstanceDataSceneProxyNoGPUScene()
{
	ReleaseStaticMeshInstanceBuffer();
}

template <typename IndexRemapType>
void FISMCInstanceDataSceneProxyNoGPUScene::ApplyDataChanges(FISMInstanceUpdateChangeSet &ChangeSet, const IndexRemapType &IndexRemap, int32 PostUpdateNumInstances, FInstanceSceneDataBuffers::FWriteView &ProxyData, FStaticMeshInstanceData &LegacyInstanceData)
{
	ProxyData.NumCustomDataFloats = ChangeSet.Flags.bHasPerInstanceCustomData ? ChangeSet.NumCustomDataFloats : 0;
	LegacyInstanceData.AllocateInstances(PostUpdateNumInstances, ProxyData.NumCustomDataFloats, GIsEditor ? EResizeBufferFlags::AllowSlackOnGrow|EResizeBufferFlags::AllowSlackOnReduce : EResizeBufferFlags::None, false); // In Editor always permit overallocation, to prevent too much realloc

	ProxyData.PrimitiveToRelativeWorld = ChangeSet.PrimitiveToRelativeWorld;
	ProxyData.PrimitiveWorldSpaceOffset = ChangeSet.PrimitiveWorldSpaceOffset;
		
	check(!ChangeSet.Flags.bHasPerInstanceLocalBounds);
	// TODO: delta support & always assume all bounds changed, and that there is in fact only one
	ProxyData.InstanceLocalBounds = MoveTemp(ChangeSet.InstanceLocalBounds);

	// TODO: DISP - Fix me (this comment came along from FPrimitiveSceneProxy::SetInstanceLocalBounds and is probably still true...)
	const FVector3f PadExtent = GetLocalBoundsPadExtent(ProxyData.PrimitiveToRelativeWorld, ChangeSet.AbsMaxDisplacement);
	for (FRenderBounds& Bounds : ProxyData.InstanceLocalBounds)
	{
		Bounds.Min -= PadExtent;
		Bounds.Max += PadExtent;
	}

	// TODO: Dont bother for delta updates perhaps since it didnt use to work anyway, though now we are potentially doing more of that...
	TArray<float> InstanceRandomIDs;
	// Delayed per instance random generation, moves it off the GT and RT, but still sucks
	if (ChangeSet.Flags.bHasPerInstanceRandom)
	{
		// TODO: only need to process added instances? No help for ISM since the move path would be taken.
		// TODO: OTOH for HISM there is no meaningful data, so just skipping and letting the SetNumZeroed fill in the blanks is fine.
		InstanceRandomIDs.SetNumZeroed(PostUpdateNumInstances);
		if (ChangeSet.GeneratePerInstanceRandomIds)
		{
			ChangeSet.GeneratePerInstanceRandomIds(InstanceRandomIDs);
		}
	}

	// unpack transform deltas
	// TODO: Only do if requested / needed
	ProxyData.InstanceToPrimitiveRelative.SetNumUninitialized(PostUpdateNumInstances);
	auto TransformDelta = ChangeSet.GetTransformDelta();
	for (auto It = TransformDelta.GetIterator(); It; ++It)
	{
		int32 PackedIndex = It.GetItemIndex();
		int32 InstanceIndex = It.GetIndex();

		if (IndexRemap.RemapDestIndex(InstanceIndex))
		{
			LegacyInstanceData.SetInstance(InstanceIndex, ChangeSet.Transforms[PackedIndex].ToMatrix44f(), ChangeSet.Flags.bHasPerInstanceRandom ? InstanceRandomIDs[InstanceIndex] : 0.0f);

			// TODO: Only do if requested / needed
			FRenderTransform LocalToPrimitiveRelativeWorld = ChangeSet.Transforms[PackedIndex] * ChangeSet.PrimitiveToRelativeWorld;
			// Remove shear
			LocalToPrimitiveRelativeWorld.Orthogonalize();
			ProxyData.InstanceToPrimitiveRelative[InstanceIndex] = LocalToPrimitiveRelativeWorld;
		}
	}

	if (ChangeSet.Flags.bHasPerInstanceCustomData)
	{
		auto CustomDataDelta = ChangeSet.GetCustomDataDelta();
		for (auto It = CustomDataDelta.GetIterator(); It; ++It)
		{
			int32 PackedIndex = It.GetItemIndex();
			int32 InstanceIndex = It.GetIndex();
			if (IndexRemap.RemapDestIndex(InstanceIndex))
			{
				for (int32 j = 0; j < ProxyData.NumCustomDataFloats; ++j)
				{
					LegacyInstanceData.SetInstanceCustomData(InstanceIndex, j, ChangeSet.PerInstanceCustomData[PackedIndex * ProxyData.NumCustomDataFloats + j]);
				}
			}
		}
	}
	if (ChangeSet.Flags.bHasPerInstanceLMSMUVBias)
	{
		auto InstanceLightShadowUVBiasDelta = ChangeSet.GetInstanceLightShadowUVBiasDelta();
		for (auto It = InstanceLightShadowUVBiasDelta.GetIterator(); It; ++It)
		{
			int32 PackedIndex = It.GetItemIndex();
			int32 InstanceIndex = It.GetIndex();
			if (IndexRemap.RemapDestIndex(InstanceIndex))
			{
				FVector4f Packed = ChangeSet.InstanceLightShadowUVBias[PackedIndex];
				FVector2D LightmapUVBias = FVector2D(Packed.X, Packed.Y);
				FVector2D ShadowmapUVBias = FVector2D(Packed.Z, Packed.W);

				LegacyInstanceData.SetInstanceLightMapData(InstanceIndex, LightmapUVBias, ShadowmapUVBias);
			}
		}
	}

#if WITH_EDITOR
	if (ChangeSet.Flags.bHasPerInstanceEditorData)
	{
		auto InstanceEditorDataDelta = ChangeSet.GetInstanceEditorDataDelta();
		for (auto It = InstanceEditorDataDelta.GetIterator(); It; ++It)
		{
			int32 PackedIndex = It.GetItemIndex();
			int32 InstanceIndex = It.GetIndex();
			if (IndexRemap.RemapDestIndex(InstanceIndex))
			{
				FColor HitProxyColor;
				bool bSelected;
				FInstanceEditorData::Unpack(ChangeSet.InstanceEditorData[PackedIndex], HitProxyColor, bSelected);

				LegacyInstanceData.SetInstanceEditorData(InstanceIndex, HitProxyColor, bSelected);
			}
		}
	}

	// replace the HP container.
	if (ChangeSet.HitProxyContainer)
	{
		HitProxyContainer = MoveTemp(ChangeSet.HitProxyContainer);
	}
#endif
}

FStaticMeshInstanceBuffer* FISMCInstanceDataSceneProxyNoGPUScene::GetLegacyInstanceBuffer() 
{ 
	if (bUseLegacyRenderingPath)
	{
		// Must sync to be sure the build is complete
		InstanceDataUpdateTaskInfo.WaitForUpdateCompletion();
		return LegacyInstanceBuffer.Get(); 
	}
	return nullptr;
}

void FISMCInstanceDataSceneProxyNoGPUScene::Update(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	check(!ChangeSet.IsFullUpdate());

	check(bLegacyReordered || ChangeSet.LegacyInstanceReorderTable.IsEmpty());

	DecStatCounters();

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);

	ProxyData.Flags = ChangeSet.Flags;
	// Handle deletions before updating the data.
	{
		FReorderTableIndexRemap IndexRemapOld(LegacyInstanceReorderTable, LegacyInstanceBuffer->GetNumInstances());

		// HISMTODO: move to own implementation
		ProxyData.VisibleInstances.SetNum(ChangeSet.PostUpdateNumInstances, true);
		ProxyData.Flags.bHasPerInstanceVisible = true;
		for (auto It = ChangeSet.InstanceAttributeTracker.GetRemovedIterator(); It; ++It)
		{
			// This is somewhat nonintuitive, but the current instance->index map is where we retain knowledge of where the instance used to be placed (in the component address space at last update)
			int32 InstanceIndex = InstanceIdIndexMap.IdToIndex(FPrimitiveInstanceId{It.GetIndex()});
			if (IndexRemapOld.RemapDestIndex(InstanceIndex))
			{
				LOG_INST_DATA(TEXT("Update/HideInstance, ID: %d, IDX: %d"), It.GetIndex(), InstanceIndex);
				LegacyInstanceBuffer->InstanceData->NullifyInstance(InstanceIndex);
			}
		}
	}

	UpdateIdMapping(ChangeSet, FIdentityIndexRemap());
	check(bLegacyReordered || ChangeSet.PostUpdateNumInstances == InstanceIdIndexMap.GetMaxInstanceIndex());

	LegacyInstanceReorderTable = MoveTemp(ChangeSet.LegacyInstanceReorderTable);
	FReorderTableIndexRemap IndexRemap(LegacyInstanceReorderTable, ChangeSet.PostUpdateNumInstances);

	ApplyDataChanges(ChangeSet, IndexRemap, ChangeSet.PostUpdateNumInstances, ProxyData, *LegacyInstanceBuffer->InstanceData.Get());

	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	LegacyInstanceBuffer->SetFlushToGPUPending();
		
	IncStatCounters();
}

void FISMCInstanceDataSceneProxyNoGPUScene::Build(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	DecStatCounters();
	check(ChangeSet.IsFullUpdate());
	checkSlow(!ChangeSet.GetTransformDelta().IsDelta());
	checkSlow(!ChangeSet.GetCustomDataDelta().IsDelta() || (!ChangeSet.Flags.bHasPerInstanceCustomData && ChangeSet.GetCustomDataDelta().IsEmpty()));
	checkSlow(!ChangeSet.GetInstanceLightShadowUVBiasDelta().IsDelta() || ChangeSet.GetInstanceLightShadowUVBiasDelta().IsEmpty());
#if WITH_EDITOR
	checkSlow(!ChangeSet.GetInstanceEditorDataDelta().IsDelta() || ChangeSet.GetInstanceEditorDataDelta().IsEmpty());
#endif

	check(bLegacyReordered || ChangeSet.LegacyInstanceReorderTable.IsEmpty());

	UpdateIdMapping(ChangeSet, FIdentityIndexRemap());
	check(bLegacyReordered || ChangeSet.PostUpdateNumInstances == InstanceIdIndexMap.GetMaxInstanceIndex());

	LegacyInstanceReorderTable = MoveTemp(ChangeSet.LegacyInstanceReorderTable);

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
	ProxyData.Flags = ChangeSet.Flags;

	FStaticMeshInstanceData LegacyInstanceData(/*bInUseHalfFloat = */true);
	FReorderTableIndexRemap IndexRemap(LegacyInstanceReorderTable, ChangeSet.PostUpdateNumInstances);
	ApplyDataChanges(ChangeSet, IndexRemap, ChangeSet.PostUpdateNumInstances, ProxyData, LegacyInstanceData);
	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);

	// Is there is a reorder table and it does not have the same number as the instances, some must be hidden
	if (bLegacyReordered && !LegacyInstanceReorderTable.IsEmpty())
	{
		TBitArray<> HiddenInstances;
		HiddenInstances.SetNum(ChangeSet.PostUpdateNumInstances, true);
		for (int32 InstanceIndex : LegacyInstanceReorderTable)
		{
			if (InstanceIndex != INDEX_NONE)
			{
				HiddenInstances[InstanceIndex]  = false;
			}
		}
		for (TConstSetBitIterator<> BitIt(HiddenInstances); BitIt; ++BitIt)
		{
			LegacyInstanceData.NullifyInstance(BitIt.GetIndex());
		}
	}

	// no need to provide CPU access sice we don't use this on the renderer any more, also no need to defer since we only create this data when actually needed.
	// TODO: strip out those flags & associated logic
	if (!LegacyInstanceBuffer)
	{
		LegacyInstanceBuffer = MakeUnique<FStaticMeshInstanceBuffer>(FeatureLevel, false);
	}
	LegacyInstanceBuffer->InitFromPreallocatedData(LegacyInstanceData);
	LegacyInstanceBuffer->SetFlushToGPUPending();

	IncStatCounters();
}

void FISMCInstanceDataSceneProxyNoGPUScene::BuildFromLegacyData(TUniquePtr<FStaticMeshInstanceData>&& InExternalLegacyData, const FRenderBounds & InstanceLocalBounds, TArray<int32>&& InLegacyInstanceReorderTable)
{
	ExternalLegacyData = MoveTemp(InExternalLegacyData);
	LegacyInstanceReorderTable = MoveTemp(InLegacyInstanceReorderTable);

	// NEW_INSTANCE_DATA_PATH_TODO: May not want to do this for every ISM, just those that actually have DF or anything else that will be accessed on the CPU?
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
		ProxyData.InstanceLocalBounds = MakeArrayView(&InstanceLocalBounds, 1);
		UpdateInstancesTransforms(ProxyData, *ExternalLegacyData);
		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
	}

	if (!LegacyInstanceBuffer)
	{
		LegacyInstanceBuffer = MakeUnique<FStaticMeshInstanceBuffer>(FeatureLevel, false);
	}	
	// Note: this passes ownership of the contained data
	LegacyInstanceBuffer->InitFromPreallocatedData(*ExternalLegacyData);
	LegacyInstanceBuffer->SetFlushToGPUPending();
}

void FISMCInstanceDataSceneProxyNoGPUScene::UpdatePrimitiveTransform(FISMInstanceUpdateChangeSet&& ChangeSet)
{
	check(LegacyInstanceBuffer && LegacyInstanceBuffer->GetNumInstances() == LegacyInstanceBuffer->GetNumInstances() || InstanceSceneDataBuffers.GetNumInstances() == 0);

	if (LegacyInstanceBuffer)
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
		FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
		ProxyData.PrimitiveToRelativeWorld = ChangeSet.PrimitiveToRelativeWorld;
		ProxyData.PrimitiveWorldSpaceOffset = ChangeSet.PrimitiveWorldSpaceOffset;
		UpdateInstancesTransforms(ProxyData, *LegacyInstanceBuffer->InstanceData.Get());
		InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
	}
}

void FISMCInstanceDataSceneProxyNoGPUScene::ReleaseStaticMeshInstanceBuffer()
{
	if (LegacyInstanceBuffer)
	{
		ENQUEUE_RENDER_COMMAND(FReleasePerInstanceRenderData)(
			[LegacyInstanceBufferRt = MoveTemp(LegacyInstanceBuffer)](FRHICommandList& RHICmdList) mutable
			{
				LegacyInstanceBufferRt->ReleaseResource();
				LegacyInstanceBufferRt.Reset();
			});
	}
}
