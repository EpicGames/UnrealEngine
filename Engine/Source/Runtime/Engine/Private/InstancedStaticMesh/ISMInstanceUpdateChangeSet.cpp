// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMesh/ISMInstanceUpdateChangeSet.h"
#include "InstanceData/InstanceDataUpdateUtils.h"
#include "Engine/InstancedStaticMesh.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Rendering/RenderingSpatialHash.h"
#include "Rendering/MotionVectorSimulation.h"
#include "SceneInterface.h"

void FISMInstanceUpdateChangeSet::SetInstanceTransforms(TStridedView<FMatrix> InInstanceTransforms, const FVector Offset)
{
	GetTransformWriter().Gather(
		[&, Offset](int32 InstanceIndex) -> FRenderTransform 
		{ 
			return FRenderTransform(InInstanceTransforms[InstanceIndex].ConcatTranslation(Offset));
		});
}

void FISMInstanceUpdateChangeSet::SetInstanceTransforms(TStridedView<FMatrix> InInstanceTransforms)
{
	GetTransformWriter().Gather(
		[&](int32 InstanceIndex) -> FRenderTransform 
		{ 
			return FRenderTransform(InInstanceTransforms[InstanceIndex]);
		});
}

void FISMInstanceUpdateChangeSet::SetInstanceTransforms(TStridedView<FMatrix> InInstanceTransforms, FBox const& InInstanceBounds, FBox& OutGatheredBounds)
{
	GetTransformWriter().Gather(
		[&](int32 InstanceIndex) -> FRenderTransform
		{
			const FMatrix& M = InInstanceTransforms[InstanceIndex];
			FRenderTransform Transform(M);
			OutGatheredBounds += InInstanceBounds.TransformBy(Transform.ToMatrix());
			return Transform;
		});
}

void FISMInstanceUpdateChangeSet::SetInstancePrevTransforms(TArrayView<FMatrix> InPrevInstanceTransforms, const FVector &Offset)
{
	GetPrevTransformWriter().Gather(
		[&, Offset](int32 InstanceIndex) -> FRenderTransform 
		{ 
			return FRenderTransform(InPrevInstanceTransforms[InstanceIndex].ConcatTranslation(Offset));
		});
}

void FISMInstanceUpdateChangeSet::SetInstancePrevTransforms(TArrayView<FMatrix> InPrevInstanceTransforms)
{
	GetPrevTransformWriter().Gather(
		[&](int32 InstanceIndex) -> FRenderTransform 
		{ 
			return FRenderTransform(InPrevInstanceTransforms[InstanceIndex]);
		});
}

void FISMInstanceUpdateChangeSet::SetCustomData(const TArrayView<const float>& InPerInstanceCustomData, int32 InNumCustomDataFloats)
{
	GetCustomDataWriter().Gather(InPerInstanceCustomData, InNumCustomDataFloats);
}

