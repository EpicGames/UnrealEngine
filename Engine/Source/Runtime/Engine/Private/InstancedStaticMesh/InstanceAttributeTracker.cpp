// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMesh/InstanceAttributeTracker.h"

FInstanceAttributeTracker::FInstanceAttributeTracker()
{
	Reset();
}

FInstanceAttributeTracker::FInstanceAttributeTracker(FInstanceAttributeTracker &&Other)
{
	Move(*this, Other);
}

void FInstanceAttributeTracker::operator=(FInstanceAttributeTracker &&Other)
{
	Move(*this, Other);
}

void FInstanceAttributeTracker::Reset()
{
	Removed.Reset();
	Data.Reset();
	MaxIndex = 0;
	FirstChangedIndex = MAX_int32;
	LastChangedIndex = 0;		
	FirstRemovedIdIndex = INT32_MAX;
	FMemory::Memset(NumChanged, 0, sizeof(NumChanged));
}
	
void FInstanceAttributeTracker::Move(FInstanceAttributeTracker &Dest, FInstanceAttributeTracker &Source)
{
	Dest.Removed = MoveTemp(Source.Removed);
	Dest.Data = MoveTemp(Source.Data);
	Dest.FirstRemovedIdIndex = Source.FirstRemovedIdIndex;
	Dest.MaxIndex = Source.MaxIndex;
	Dest.FirstChangedIndex = Source.FirstChangedIndex;
	Dest.LastChangedIndex = Source.LastChangedIndex;
	FMemory::Memcpy(Dest.NumChanged, Source.NumChanged, sizeof(Dest.NumChanged));
		
	Source.Reset();
}

TConstSetBitIterator<> FInstanceAttributeTracker::GetRemovedIterator() const
{
	return TConstSetBitIterator<>(Removed, FMath::Min(Removed.Num(), FirstRemovedIdIndex));
}

#if DO_GUARD_SLOW

void FInstanceAttributeTracker::Validate() const
{
	int32 NumChangedCounts[uint32(EFlag::Num)];
	FMemory::Memset(NumChangedCounts, 0, sizeof(NumChangedCounts));

	for (int32 Index = 0; Index < MaxIndex; ++Index)
	{
		uint32 Flags = GetFlags(Index);
		if (Flags & FToBit<EFlag::Added>::Bit)
		{
			NumChangedCounts[uint32(EFlag::Added)] += 1;
		}
		else
		{
			for (uint32 Flag = uint32(EFlag::TransformChanged); Flag < uint32(EFlag::Num); ++Flag)
			{
				// If it is set & not added
				if (Flags & (1u << Flag))
				{
					NumChangedCounts[Flag] += 1;
				}
			}
		}
	}
	for (uint32 Flag = 0u; Flag < uint32(EFlag::Num); ++Flag)
	{
		check(NumChangedCounts[Flag] == NumChanged[Flag]);
	}

	if (MaxIndex > 0)
	{
		int32 LastIndex = MaxIndex - 1;
		const int32 ElementIndex = LastIndex / MasksPerElement;
		const uint32 SubIndex = (LastIndex % MasksPerElement) + 1u;
		if (SubIndex < MasksPerElement)
		{
			const uint32 ElementSubShift = SubIndex * uint32(EFlag::Num);
			// Load and clear the mask for this index
			ElementType LastElement = Data[ElementIndex];
			check((LastElement >> ElementSubShift) == 0u);
		}
	}
}

#endif