// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMeshComponentHelper.h"

TGlobalResource<FInstancedStaticMeshComponentHelper::FDummyStaticMeshInstanceBuffer> FInstancedStaticMeshComponentHelper::DummyStaticMeshInstanceBuffer;