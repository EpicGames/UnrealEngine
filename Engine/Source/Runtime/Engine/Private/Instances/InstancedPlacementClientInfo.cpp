// Copyright Epic Games, Inc. All Rights Reserved.

#include "Instances/InstancedPlacementClientInfo.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedPlacementClientInfo)

#if WITH_EDITORONLY_DATA
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "Misc/ITransaction.h"
#include "Instances/InstancedPlacementPartitionActor.h"
#include "Engine/World.h"

FArchive& operator<<(FArchive& Ar, FPlacementInstance& Instance)
{
	Ar << Instance.ZOffset;
	Ar << Instance.Flags;
	Ar << Instance.WorldTransform;
	Ar << Instance.PreAlignRotation;

	return Ar;
}

FTransform FPlacementInstance::GetInstanceWorldTransform() const
{
	return WorldTransform;
}

void FPlacementInstance::SetInstanceWorldTransform(const FTransform& Transform)
{
	WorldTransform = Transform;
}

void FPlacementInstance::AlignToNormal(const FVector& InNormal, int32 AlignMaxAngle)
{
	Flags |= (uint32)EPlacementInstanceFlags::AlignToNormal;

	FRotator AlignRotation = InNormal.Rotation();
	// Static meshes are authored along the vertical axis rather than the X axis, so we add 90 degrees to the static mesh's Pitch.
	AlignRotation.Pitch -= 90.f;
	// Clamp its value inside +/- one rotation
	AlignRotation.Pitch = FRotator::NormalizeAxis(AlignRotation.Pitch);

	// limit the maximum pitch angle if it's > 0.
	if (AlignMaxAngle > 0)
	{
		int32 MaxPitch = AlignMaxAngle;
		if (AlignRotation.Pitch > MaxPitch)
		{
			AlignRotation.Pitch = MaxPitch;
		}
		else if (AlignRotation.Pitch < -MaxPitch)
		{
			AlignRotation.Pitch = -MaxPitch;
		}
	}

	PreAlignRotation = WorldTransform.GetRotation();
	WorldTransform.SetRotation(FQuat(AlignRotation) * PreAlignRotation);
}

#if WITH_EDITOR
void UInstancedPlacemenClientSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateGuid = FGuid::NewGuid();

	bool bSourceChanged = false;
	if (const FProperty* Property = PropertyChangedEvent.Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedPlacemenClientSettings, ObjectPath))
		{
			bSourceChanged = true;
		}
	}
	uint32 OriginalHash = InstancedComponentSettings.Hash;
	InstancedComponentSettings.ComputeHash();
}

void UInstancedPlacemenClientSettings::RegisterISMDescriptors(AInstancedPlacementPartitionActor* ParentPartitionActor, TSortedMap<int32, TArray<FTransform>>& ISMDefinition) const
{
}
#endif	// WITH_EDITOR

FClientPlacementInfo::FClientPlacementInfo()
{
}

FArchive& operator<<(FArchive& Ar, FClientPlacementInfo& PlacementInfo)
{
	if (Ar.IsTransacting())
	{
		PlacementInfo.Instances.BulkSerialize(Ar, true);
	}
	else
	{
		Ar << PlacementInfo.Instances;
	}

	Ar << PlacementInfo.ClientGuid;
	Ar << PlacementInfo.UpdateGuid;
	Ar << PlacementInfo.ISMDefinition;
	PlacementInfo.ClientHandle.Serialize(Ar);

	return Ar;
}

bool FClientPlacementInfo::Initialize(const FGuid& InClientGuid, const FString& InClientDisplayName, AInstancedPlacementPartitionActor* InParentPartitionActor, FClientDescriptorFunc RegisterDescriptorFunc)
{
	check(!IsInitialized());

	if (!InParentPartitionActor || !InClientGuid.IsValid())
	{
		return false;
	}

	// Register the descriptors and generate the ISM definition from the callback provided
	RegisterDescriptorFunc(InParentPartitionActor, ISMDefinition);
	if (ISMDefinition.IsEmpty())
	{
		return false;
	}

	ParentPartitionActor = InParentPartitionActor;
	ClientGuid = InClientGuid;

	ClientDisplayName = InClientDisplayName;
	if (ClientDisplayName.IsEmpty())
	{
		ClientDisplayName = ParentPartitionActor->GetName();
	}

	ClientHandle = ParentPartitionActor->RegisterClient(InClientGuid);
	ParentPartitionActor->RegisterClientInstanceManager(ClientHandle, this);
	//RegisterDelegates();

	return true;
}

void FClientPlacementInfo::Uninitialize()
{
	check(IsInitialized());
	//UnregisterDelegates();
	ParentPartitionActor->UnregisterClient(ClientHandle);
	ISMDefinition.Empty();
}

void FClientPlacementInfo::PostLoad(AInstancedPlacementPartitionActor* InParentPartitionActor)
{
#if WITH_EDITORONLY_DATA
	ParentPartitionActor = InParentPartitionActor;

	for (int32 Index = 0; Index < Instances.Num(); ++Index)
	{
		InstanceHash.InsertInstance(Instances[Index].GetInstanceWorldTransform().GetLocation(), Index);
	}
#endif

#if WITH_EDITOR
	if (GIsEditor && IsInitialized())
	{
		//RegisterDelegates();
	}
#endif
}
void FClientPlacementInfo::PostSerialize(FArchive& Ar, AInstancedPlacementPartitionActor* InParentPartitionActor)
{
#if WITH_EDITORONLY_DATA
	ParentPartitionActor = InParentPartitionActor;
#endif

#if WITH_EDITOR
	if (GIsEditor && IsInitialized() && Ar.IsLoading())
	{
		ParentPartitionActor->RegisterClientInstanceManager(ClientHandle, this);
	}
#endif	// WITH_EDITOR
}
bool FClientPlacementInfo::IsInitialized() const
{
	return ClientHandle.IsValid();
}

void FClientPlacementInfo::PreEditUndo()
{
	InstanceHash.CheckInstanceCount(Instances.Num());
}

void FClientPlacementInfo::PostEditUndo()
{
	// Regenerate instance hash
	// We regenerate it here instead of saving to transaction buffer to speed up modify operations
	// And also because not all edits of the hash get correctly saved with the frequency of modify operations we have
	InstanceHash.Empty();
	for (int32 InstanceIdx = 0; InstanceIdx < Instances.Num(); InstanceIdx++)
	{
		InstanceHash.InsertInstance(Instances[InstanceIdx].GetInstanceWorldTransform().GetLocation(), InstanceIdx);
	}

	InstanceHash.CheckInstanceCount(Instances.Num());
}

TArray<FSMInstanceId> FClientPlacementInfo::AddInstances(TArrayView<const FPlacementInstance> InstancesToAdd)
{
	check(IsInitialized());

	ParentPartitionActor->Modify();
	ParentPartitionActor->BeginUpdate();

	Instances.Reserve(Instances.Num() + InstancesToAdd.Num());

	TArray<FSMInstanceId> PlacedInstances;
	PlacedInstances.Reserve(InstancesToAdd.Num());
	for (const FPlacementInstance& Instance : InstancesToAdd)
	{
		PlacedInstances.Append(ParentPartitionActor->AddISMInstance(ClientHandle, Instance.GetInstanceWorldTransform(), ISMDefinition));
		int32 InstanceIndex = Instances.Add(Instance);
		InstanceHash.InsertInstance(Instance.GetInstanceWorldTransform().GetLocation(), InstanceIndex);
	}

	ParentPartitionActor->EndUpdate();

	return PlacedInstances;
}

void FClientPlacementInfo::RemoveInstancesFromPartitionActor(TArrayView<const FISMClientInstanceId> InstanceIds, bool bUpdateHISMTrees, TFunctionRef<void(int32)> RemoveFn)
{
	if (InstanceIds.IsEmpty())
	{
		return;
	}

	check(IsInitialized());

	TSet<int32> InstancesToRemove;
	InstancesToRemove.Reserve(InstanceIds.Num());
	for (const FISMClientInstanceId& InstanceId : InstanceIds)
	{
		InstancesToRemove.Add(InstanceId.Index);
	}

	ParentPartitionActor->Modify();
	ParentPartitionActor->BeginUpdate();

	while (InstancesToRemove.Num() > 0)
	{
		// Get an item from the set for processing
		auto It = InstancesToRemove.CreateConstIterator();
		int32 InstanceIndex = *It;
		int32 InstanceIndexToRemove = InstanceIndex;

		FPlacementInstance& Instance = Instances[InstanceIndex];

		// remove from hash
		InstanceHash.RemoveInstance(Instance.GetInstanceWorldTransform().GetLocation(), InstanceIndex);

		// remove this instance from the parent partition actor
		RemoveFn(InstanceIndex);

		// remove from instances array
		Instances.RemoveAtSwap(InstanceIndex, EAllowShrinking::No);

		// update hashes for swapped instance
		if (InstanceIndex != Instances.Num() && Instances.Num() > 0)
		{
			// Instance hash
			FPlacementInstance& SwappedInstance = Instances[InstanceIndex];
			InstanceHash.SwapInstance(SwappedInstance.GetInstanceWorldTransform().GetLocation(), Instances.Num(), InstanceIndex);

			// Removal list
			if (InstancesToRemove.Contains(Instances.Num()))
			{
				// The item from the end of the array that we swapped in to InstanceIndex is also on the list to remove.
				// Remove the item at the end of the array and leave InstanceIndex in the removal list.
				InstanceIndexToRemove = Instances.Num();
			}
		}

		// Remove the removed item from the removal list
		InstancesToRemove.Remove(InstanceIndexToRemove);
	}

	ParentPartitionActor->EndUpdate();

	Instances.Shrink();
	if (bUpdateHISMTrees)
	{
		ParentPartitionActor->UpdateHISMTrees(true, true);
	}
}

FText FClientPlacementInfo::GetISMPartitionInstanceDisplayName(const FISMClientInstanceId& InstanceId) const
{
#if WITH_EDITOR
	return FText::Format(NSLOCTEXT("PlacementClientInfo", "DisplayNameFmt", "{0} - Instance {1}"), FText::FromString(ClientDisplayName), InstanceId.Index);
#else
	return FText();
#endif
}

FText FClientPlacementInfo::GetISMPartitionInstanceTooltip(const FISMClientInstanceId& InstanceId) const
{
#if WITH_EDITOR
	const FText OwnerDisplayPath = FText::FromString(ParentPartitionActor->GetPathName(ParentPartitionActor->GetWorld())); // stops the path at the level of the world the object is in
	return FText::Format(NSLOCTEXT("PlacementClientInfo", "TooltipFmt", "Instance {0} on {1}"), InstanceId.Index, OwnerDisplayPath);
#else
	return FText();
#endif
}

bool FClientPlacementInfo::CanEditISMPartitionInstance(const FISMClientInstanceId& InstanceId) const
{
#if WITH_EDITOR
	return true;
#else
	return false;
#endif
}

bool FClientPlacementInfo::CanMoveISMPartitionInstance(const FISMClientInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const
{
#if WITH_EDITOR
	return true;
#else
	return false;
#endif
}

bool FClientPlacementInfo::GetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
#if WITH_EDITOR
	check(InstanceId.Handle == ClientHandle);
	OutInstanceTransform = Instances[InstanceId.Index].GetInstanceWorldTransform();
	if (!bWorldSpace)
	{
		OutInstanceTransform = OutInstanceTransform.GetRelativeTransform(ParentPartitionActor->GetActorTransform());
	}
	return true;
#else
	return false;
#endif
}

bool FClientPlacementInfo::SetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bTeleport)
{
#if WITH_EDITOR
	ParentPartitionActor->Modify();

	check(InstanceId.Handle == ClientHandle);
	bool bWasMoving = MovingInstances.Contains(InstanceId);
	if (!bWasMoving)
	{
		NotifyISMPartitionInstanceMovementStarted(InstanceId);
	}

	FTransform WorldTransform = bWorldSpace ? InstanceTransform : InstanceTransform * ParentPartitionActor->GetActorTransform();
	Instances[InstanceId.Index].SetInstanceWorldTransform(WorldTransform);
	ParentPartitionActor->SetISMInstanceTransform(ClientHandle, InstanceId.Index, WorldTransform, bTeleport, ISMDefinition);

	if (!bWasMoving)
	{
		NotifyISMPartitionInstanceMovementEnded(InstanceId);
	}

	return true;
#else
	return false;
#endif
}

void FClientPlacementInfo::NotifyISMPartitionInstanceMovementStarted(const FISMClientInstanceId& InstanceId)
{
#if WITH_EDITOR
	check(InstanceId.Handle == ClientHandle);
	MovingInstances.Add(InstanceId);
	InstanceHash.RemoveInstance(Instances[InstanceId.Index].GetInstanceWorldTransform().GetLocation(), InstanceId.Index);
#endif
}

void FClientPlacementInfo::NotifyISMPartitionInstanceMovementOngoing(const FISMClientInstanceId& InstanceId)
{
#if WITH_EDITOR
	check(InstanceId.Handle == ClientHandle);
	NotifyISMPartitionInstanceMovementEnded(InstanceId);
	NotifyISMPartitionInstanceMovementStarted(InstanceId);
#endif
}

void FClientPlacementInfo::NotifyISMPartitionInstanceMovementEnded(const FISMClientInstanceId& InstanceId)
{
#if WITH_EDITOR
	check(InstanceId.Handle == ClientHandle);
	FTransform InstanceWorldTransform = Instances[InstanceId.Index].GetInstanceWorldTransform();
	constexpr bool bTeleport = true;
	ParentPartitionActor->SetISMInstanceTransform(ClientHandle, InstanceId.Index, InstanceWorldTransform, bTeleport, ISMDefinition);
	InstanceHash.InsertInstance(Instances[InstanceId.Index].GetInstanceWorldTransform().GetLocation(), InstanceId.Index);
	MovingInstances.Remove(InstanceId);
#endif
}

void FClientPlacementInfo::NotifyISMPartitionInstanceSelectionChanged(const FISMClientInstanceId& InstanceId, const bool bIsSelected)
{
#if WITH_EDITOR
	check(IsInitialized());
	check(InstanceId.Handle == ClientHandle);

	if (AInstancedPlacementPartitionActor* ParentPartitionActorPtr = ParentPartitionActor.Get())
	{
		if (!GUndo || !GUndo->ContainsObject(ParentPartitionActorPtr))
		{
			// Actor modify can be a bit slow since when there is a transaction active it need to iterate the properties
			ParentPartitionActorPtr->Modify(false);
		}

		ParentPartitionActorPtr->SelectISMInstances(ClientHandle, bIsSelected, { InstanceId.Index });
	}
#endif
}

bool FClientPlacementInfo::DeleteISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds)
{
#if WITH_EDITOR
	auto RemoveFn = [this](int32 Index)
	{
		bool bOutIsEmpty = false;
		ParentPartitionActor->RemoveISMInstance(ClientHandle, Index, &bOutIsEmpty);
		if (bOutIsEmpty)
		{
			Uninitialize();
		}
	};

	constexpr bool bUpdateHISMTrees = true;
	RemoveInstancesFromPartitionActor(InstanceIds, bUpdateHISMTrees, RemoveFn);
	return true;
#else
	return false;
#endif
}

bool FClientPlacementInfo::DuplicateISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds, TArray<FISMClientInstanceId>& OutNewInstanceIds)
{
#if WITH_EDITOR
	if (InstanceIds.Num() == 0)
	{
		return false;
	}

	TArray<FPlacementInstance> InstancesToDuplicate;
	InstancesToDuplicate.Reserve(InstanceIds.Num());
	for (const FISMClientInstanceId& InstanceId : InstanceIds)
	{
		InstancesToDuplicate.Add(Instances[InstanceId.Index]);
	}

	TArray<FSMInstanceId> AddedInstances = AddInstances(InstancesToDuplicate);
	ParentPartitionActor->UpdateHISMTrees(true, true);

	OutNewInstanceIds.Reset(InstanceIds.Num());
	for (const FSMInstanceId& NewInstanceIds : AddedInstances)
	{
		OutNewInstanceIds.Emplace(FISMClientInstanceId{ ClientHandle, NewInstanceIds.InstanceIndex });
	}

	return OutNewInstanceIds.Num() > 0;
#else
	return false;
#endif
}

#endif	// WITH_EDITORONLY_DATA

