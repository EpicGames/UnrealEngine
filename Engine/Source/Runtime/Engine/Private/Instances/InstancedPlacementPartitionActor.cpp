// Copyright Epic Games, Inc. All Rights Reserved.

#include "Instances/InstancedPlacementPartitionActor.h"

#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedPlacementPartitionActor)

AInstancedPlacementPartitionActor::AInstancedPlacementPartitionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = false;
	SetActorEnableCollision(true);
}

void AInstancedPlacementPartitionActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	for (auto& Pair : PlacedClientInfo)
	{
		FClientPlacementInfo& ClientInfo = *Pair.Value;
		ClientInfo.PostLoad(this);
	}
#endif	// WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void AInstancedPlacementPartitionActor::PreEditUndo()
{
	Super::PreEditUndo();

	for (auto& Pair : PlacedClientInfo)
	{
		FClientPlacementInfo& ClientInfo = *Pair.Value;
		ClientInfo.PreEditUndo();
	}
}

void AInstancedPlacementPartitionActor::PostEditUndo()
{
	Super::PostEditUndo();

	for (auto& Pair : PlacedClientInfo)
	{
		FClientPlacementInfo& ClientInfo = *Pair.Value;
		ClientInfo.PostEditUndo();
	}
}
#endif

void AInstancedPlacementPartitionActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << PlacedClientInfo;
		
		for (auto& Pair : PlacedClientInfo)
		{
			FClientPlacementInfo& ClientInfo = *Pair.Value;
			ClientInfo.PostSerialize(Ar, this);
		}
	}
#endif	// WITH_EDITORONLY_DATA
}

#if WITH_EDITOR	
uint32 AInstancedPlacementPartitionActor::GetDefaultGridSize(UWorld* InWorld) const
{
	return InWorld->GetWorldSettings()->DefaultPlacementGridSize;
}

FGuid AInstancedPlacementPartitionActor::GetGridGuid() const
{
	return PlacementGridGuid;
}

void AInstancedPlacementPartitionActor::SetGridGuid(const FGuid& InGuid)
{
	PlacementGridGuid = InGuid;
}

FClientPlacementInfo* AInstancedPlacementPartitionActor::PreAddClientInstances(const FGuid& ClientGuid, const FString& InClientDisplayString, FClientDescriptorFunc RegisterDefinitionFunc)
{
	BeginUpdate();

	if (TUniqueObj<FClientPlacementInfo>* FoundClientInfo = PlacedClientInfo.Find(ClientGuid))
	{
		if (!(*FoundClientInfo)->IsInitialized())
		{
			if (!(*FoundClientInfo)->Initialize(ClientGuid, InClientDisplayString, this, RegisterDefinitionFunc))
			{
				return nullptr;
			}
		}

		return &FoundClientInfo->Get();
	}

	TUniqueObj<FClientPlacementInfo>& CreatedInfo = PlacedClientInfo.Add(ClientGuid);
	if (CreatedInfo->Initialize(ClientGuid, InClientDisplayString, this, RegisterDefinitionFunc))
	{
		return &(*CreatedInfo);
	}

	PlacedClientInfo.Remove(ClientGuid);
	return nullptr;
}

void AInstancedPlacementPartitionActor::PostAddClientInstances()
{
	EndUpdate();
}
#endif	// WITH_EDITOR

