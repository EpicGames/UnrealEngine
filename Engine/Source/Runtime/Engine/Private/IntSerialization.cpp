// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IntSerialization.cpp: UObject class for testing serialization int types
=============================================================================*/

#include "Engine/IntSerialization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntSerialization)

UIntSerialization::UIntSerialization(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{
}

/*void UIntSerilization::Serialize (FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << UnsignedInt16Variable << UnsignedInt32Variable << UnsignedInt64Variable << SignedInt8Variable << SignedInt16Variable << SignedInt64Variable << UnsignedInt8Variable << SignedInt32Variable;
}*/

