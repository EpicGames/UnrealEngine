// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/Interface.h"
#include "Interfaces/Interface_ActorSubobject.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "Interfaces/Interface_AsyncCompilation.h"

UInterface_CollisionDataProvider::UInterface_CollisionDataProvider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UInterface_PostProcessVolume::UInterface_PostProcessVolume( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
}

UInterface_PreviewMeshProvider::UInterface_PreviewMeshProvider( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{

}
UInterface_AsyncCompilation::UInterface_AsyncCompilation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

// Experimental
UInterface_ActorSubobject::UInterface_ActorSubobject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}