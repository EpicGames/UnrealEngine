// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/Interface_AssetUserData.h"

#include "Templates/SubclassOf.h"
#include "Engine/AssetUserData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Interface_AssetUserData)

void IInterface_AssetUserData::AddAssetUserData(UAssetUserData* InUserData)
{
}

UAssetUserData* IInterface_AssetUserData::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	return nullptr;
}

bool IInterface_AssetUserData::HasAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	return GetAssetUserDataOfClass(InUserDataClass) != nullptr;
}

bool IInterface_AssetUserData::AddAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	const UClass* DataClass = InUserDataClass.Get();
	if (DataClass != nullptr)
	{
		AddAssetUserData(NewObject<UAssetUserData>(Cast<UObject>(this), DataClass));
	}
	return GetAssetUserDataOfClass(InUserDataClass) != nullptr;
}

const TArray<UAssetUserData*>* IInterface_AssetUserData::GetAssetUserDataArray() const
{
	return nullptr;
}

void IInterface_AssetUserData::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
}
