// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/EnginePackageLocalizationCache.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"

FEnginePackageLocalizationCache::FEnginePackageLocalizationCache()
	: bIsScanningPath(false)
{
	FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	AssetRegistry.OnAssetAdded().AddRaw(this, &FEnginePackageLocalizationCache::HandleAssetAdded);
	AssetRegistry.OnAssetRemoved().AddRaw(this, &FEnginePackageLocalizationCache::HandleAssetRemoved);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FEnginePackageLocalizationCache::HandleAssetRenamed);
}

FEnginePackageLocalizationCache::~FEnginePackageLocalizationCache()
{
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetAdded().RemoveAll(this);
		AssetRegistry->OnAssetRemoved().RemoveAll(this);
		AssetRegistry->OnAssetRenamed().RemoveAll(this);
	}
}

void FEnginePackageLocalizationCache::FindLocalizedPackages(const TMap<FString, TArray<FString>>& NewSourceToLocalizedPaths, TMap<FName, TArray<FName>>& InOutSourcePackagesToLocalizedPackages)
{
	if (NewSourceToLocalizedPaths.Num() == 0)
	{
		return;
	}

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	int32 NumElementsPerKeyGuess = NewSourceToLocalizedPaths.CreateConstIterator()->Value.Num();
	int32 SizeGuess = NewSourceToLocalizedPaths.Num() * NumElementsPerKeyGuess;
#if WITH_EDITOR
	TArray<FString> LocalizedPackagePaths;
	LocalizedPackagePaths.Reserve(SizeGuess);
#endif
	TArray<FName> LocalizedPackagePathFNames;
	LocalizedPackagePathFNames.Reserve(SizeGuess);
	for (const TPair<FString, TArray<FString>>& Pair : NewSourceToLocalizedPaths)
	{
		for (const FString& LocalizedRoot : Pair.Value)
		{
#if WITH_EDITOR
			LocalizedPackagePaths.Add(LocalizedRoot);
#endif
			LocalizedPackagePathFNames.Add(*LocalizedRoot);
		}
	}

#if WITH_EDITOR
	// Make sure the asset registry has the data we need
	{
		// Set bIsScanningPath to avoid us processing newly added assets from this scan
		TGuardValue<bool> SetIsScanningPath(bIsScanningPath, true);
		AssetRegistry.ScanPathsSynchronous(LocalizedPackagePaths);
	}
#endif // WITH_EDITOR

	TArray<FAssetData> LocalizedAssetDataArray;
	bool bIncludeOnlyOnDiskAssets = !GIsEditor;
	bool bRecursive = true;
	AssetRegistry.GetAssetsByPaths(MoveTemp(LocalizedPackagePathFNames), LocalizedAssetDataArray, bRecursive, bIncludeOnlyOnDiskAssets);

	for (const FAssetData& LocalizedAssetData : LocalizedAssetDataArray)
	{
		const FName SourcePackageName = *FPackageName::GetSourcePackagePath(LocalizedAssetData.PackageName.ToString());

		TArray<FName>& PrioritizedLocalizedPackageNames = InOutSourcePackagesToLocalizedPackages.FindOrAdd(SourcePackageName);
		PrioritizedLocalizedPackageNames.AddUnique(LocalizedAssetData.PackageName);

		UE_LOGF(LogPackageLocalizationCache, Verbose, "Discovered localized package '%ls' for source package '%ls'", *LocalizedAssetData.PackageName.ToString(), *SourcePackageName.ToString());
	}
}

void FEnginePackageLocalizationCache::FindAssetGroupPackages(const FName InAssetGroupName, const FTopLevelAssetPath& InAssetClassName)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	// We use the localized paths to find the source assets for the group since it's much faster to scan those paths than perform a full scan
	TArray<FString> LocalizedRootPaths;
	{
		TArray<FString> RootPaths;
		FPackageName::QueryRootContentPaths(RootPaths);
		for (const FString& RootPath : RootPaths)
		{
			LocalizedRootPaths.Add(RootPath / TEXT("L10N"));
		}
	}

#if WITH_EDITOR
	// Make sure the asset registry has the data we need
	AssetRegistry.ScanPathsSynchronous(LocalizedRootPaths);
#endif // WITH_EDITOR

	// Build the filter to get all localized assets of the given class
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	for (const FString& LocalizedRootPath : LocalizedRootPaths)
	{
		Filter.PackagePaths.Add(*LocalizedRootPath);
	}
	Filter.bIncludeOnlyOnDiskAssets = !GIsEditor;
	Filter.ClassPaths.Add(InAssetClassName);
	Filter.bRecursiveClasses = false;

	TArray<FAssetData> LocalizedAssetsOfClass;
	AssetRegistry.GetAssets(Filter, LocalizedAssetsOfClass);

	for (const FAssetData& LocalizedAssetOfClass : LocalizedAssetsOfClass)
	{
		const FName SourcePackageName = *FPackageName::GetSourcePackagePath(LocalizedAssetOfClass.PackageName.ToString());
		PackageNameToAssetGroup.Add(SourcePackageName, InAssetGroupName);
	}
}

void FEnginePackageLocalizationCache::HandleAssetAdded(const FAssetData& InAssetData)
{
	if (bIsScanningPath)
	{
		// Ignore this, it came from the path we're currently scanning
		return;
	}

	// Convert the string outside the lock and loop	as this is called often while loading
	const FString PackagePath = InAssetData.PackageName.ToString();

	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		bPackageNameToAssetGroupDirty |= CultureCachePair.Value->AddPackage(PackagePath);
	}
}

void FEnginePackageLocalizationCache::HandleAssetRemoved(const FAssetData& InAssetData)
{
	const FString PackagePath = InAssetData.PackageName.ToString();

	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		bPackageNameToAssetGroupDirty |= CultureCachePair.Value->RemovePackage(PackagePath);
	}
}

void FEnginePackageLocalizationCache::HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
{
	const FString PackagePath = InAssetData.PackageName.ToString();
	const FString OldPackagePath = FPackageName::ObjectPathToPackageName(InOldObjectPath);

	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		bPackageNameToAssetGroupDirty |= CultureCachePair.Value->RemovePackage(OldPackagePath);
		bPackageNameToAssetGroupDirty |= CultureCachePair.Value->AddPackage(PackagePath);
	}
}
