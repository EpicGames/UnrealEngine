// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/StringTable.h"
#include "Internationalization/GatherableTextData.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/StringTableRegistry.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "UObject/GCObject.h"
#include "Templates/Casts.h"
#include "Application/SlateApplicationBase.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StringTable)

namespace UE::StringTable
{
#if WITH_EDITORONLY_DATA
	void GatherForLocalization(const FString& SourceLocation, const FStringTableConstRef& StringTable, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		auto FindOrAddTextData = [&StringTable, &PropertyLocalizationDataGatherer](const FString& InText) -> FGatherableTextData&
		{
			check(!InText.IsEmpty());

			FTextSourceData SourceData;
			SourceData.SourceString = InText;

			auto& GatherableTextDataArray = PropertyLocalizationDataGatherer.GetGatherableTextDataArray();
			FGatherableTextData* GatherableTextData = GatherableTextDataArray.FindByPredicate([&](const FGatherableTextData& Candidate)
			{
				return Candidate.NamespaceName.Equals(StringTable->GetNamespace(), ESearchCase::CaseSensitive)
					&& Candidate.SourceData.SourceString.Equals(SourceData.SourceString, ESearchCase::CaseSensitive)
					&& Candidate.SourceData.SourceStringMetaData == SourceData.SourceStringMetaData;
			});
			if (!GatherableTextData)
			{
				GatherableTextData = &GatherableTextDataArray[GatherableTextDataArray.AddDefaulted()];
				GatherableTextData->NamespaceName = StringTable->GetNamespace();
				GatherableTextData->SourceData = SourceData;
			}

			return *GatherableTextData;
		};

		StringTable->EnumerateSourceStrings([&SourceLocation, &StringTable, &FindOrAddTextData](const FString& InKey, const FString& InSourceString, const FString& InDevNotes) -> bool
		{
			if (!InSourceString.IsEmpty())
			{
				FGatherableTextData& GatherableTextData = FindOrAddTextData(InSourceString);

				FTextSourceSiteContext& SourceSiteContext = GatherableTextData.SourceSiteContexts[GatherableTextData.SourceSiteContexts.AddDefaulted()];
				SourceSiteContext.KeyName = InKey;
				SourceSiteContext.SiteDescription = SourceLocation;
				SourceSiteContext.IsEditorOnly = false;
				SourceSiteContext.IsOptional = false;
				SourceSiteContext.DevNotes = InDevNotes;

				StringTable->EnumerateMetaData(InKey, [&](const FName InMetaDataId, const FString& InMetaData)
				{
					SourceSiteContext.InfoMetaData.SetStringField(InMetaDataId.ToString(), InMetaData);
					return true; // continue enumeration
				});
			}

			return true; // continue enumeration
		});
	}

	namespace Private
	{
		void GatherAssetForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
		{
			FStringTableConstRef StringTable = CastChecked<UStringTable>(Object)->GetStringTable();
			const FString SourceLocation = Object->GetPathName();
			GatherForLocalization(SourceLocation, StringTable, PropertyLocalizationDataGatherer, GatherTextFlags);
		}
	}
#endif	// WITH_EDITORONLY_DATA

	namespace Private
	{
		bool ShouldAutoRegister(const FName InTableId)
		{
			return !FPackageName::IsTempPackage(FNameBuilder(InTableId).ToView());
		}
	}
}

class FStringTableEngineBridge : public IStringTableEngineBridge, public FGCObject
{
public:
	static void Initialize()
	{
		InstancePtr = &Get();
	}

	static FStringTableEngineBridge& Get()
	{
		static FStringTableEngineBridge Instance;
		return Instance;
	}

	void RegisterForGC(UStringTable* InStringTableAsset)
	{
		FScopeLock KeepAliveStringTablesLock(&KeepAliveStringTablesCS);
		KeepAliveStringTables.Add(InStringTableAsset);
	}

	void UnregisterForGC(UStringTable* InStringTableAsset)
	{
		FScopeLock KeepAliveStringTablesLock(&KeepAliveStringTablesCS);
		KeepAliveStringTables.RemoveSwap(InStringTableAsset);
	}

	void UnregisterForGCUnderPath(FStringView PluginPackageRoot)
	{
		KeepAliveStringTables.RemoveAllSwap(
			[&PluginPackageRoot](const TObjectPtr<UStringTable>& InStringTableAsset)
			{
				if (!InStringTableAsset || InStringTableAsset->HasAnyFlags(RF_MirroredGarbage | RF_NewerVersionExists))
				{
					return true;
				}

				const FNameBuilder PackageName(InStringTableAsset->GetPackage()->GetFName());
				if (PackageName.ToView().StartsWith(PluginPackageRoot))
				{
					UE_LOGF(LogStringTable, Log, "Removing string table '%ls' from keep-alive list", *InStringTableAsset->GetName());
					return true;
				}

				return false;
			});
	}

private:
	void HandleStringTableAssetAsyncLoadCompleted(const FName& InLoadedPackageName, UPackage* InLoadedPackage, EAsyncLoadingResult::Type InLoadingResult)
	{
		// Get the loading state to complete
		FAsyncLoadingStringTable AsyncLoadingState;
		{
			FScopeLock AsyncLoadingStringTablesLock(&AsyncLoadingStringTablesCS);
			AsyncLoadingStringTables.RemoveAndCopyValue(InLoadedPackageName, AsyncLoadingState);
		}

		// Calculate the name of the loaded string table based on the package name
		FName LoadedStringTableId;
		if (InLoadingResult == EAsyncLoadingResult::Succeeded)
		{
			check(InLoadedPackage);
			const FString LoadedPackageNameStr = InLoadedPackage->GetName();
			LoadedStringTableId = *FString::Printf(TEXT("%s.%s"), *LoadedPackageNameStr, *FPackageName::GetLongPackageAssetName(LoadedPackageNameStr));
		}

		// Notify any listeners of the result
		for (const FLoadStringTableAssetCallback& LoadedCallback : AsyncLoadingState.LoadedCallbacks)
		{
			check(LoadedCallback);
			LoadedCallback(AsyncLoadingState.RequestedTableId, LoadedStringTableId);
		}
	}

	//~ IStringTableEngineBridge interface
	virtual bool CanFindOrLoadStringTableAssetImpl() override
	{
		return IsInGameThread() && !IsGarbageCollecting() && !UE::IsSavingPackage();
	}

	virtual int32 LoadStringTableAssetImpl(const FName InTableId, FLoadStringTableAssetCallback InLoadedCallback) override
	{
		const FSoftObjectPath StringTableAssetReference = GetAssetReference(InTableId);
		if (StringTableAssetReference.IsValid())
		{
			UStringTable* StringTableAsset = Cast<UStringTable>(StringTableAssetReference.ResolveObject());
			if (StringTableAsset)
			{
				// Already loaded
				if (InLoadedCallback)
				{
					InLoadedCallback(InTableId, StringTableAsset->GetStringTableId());
				}
				return INDEX_NONE;
			}

			// Not loaded - either load synchronously or asynchronously depending on the environment
			if (IsAsyncLoading())
			{
				const FString StringTableAssetPackageNameStr = StringTableAssetReference.GetLongPackageName();
				const FName StringTableAssetPackageName = *StringTableAssetPackageNameStr;

				{
					FScopeLock AsyncLoadingStringTablesLock(&AsyncLoadingStringTablesCS);

					// Already being asynchronously loaded? If so, merge the request
					if (FAsyncLoadingStringTable* AsyncLoadingState = AsyncLoadingStringTables.Find(StringTableAssetPackageName))
					{
						if (InLoadedCallback)
						{
							AsyncLoadingState->LoadedCallbacks.Add(InLoadedCallback);
						}
						return AsyncLoadingState->AsyncLoadingId;
					}

					// Prepare for an asynchronous load
					FAsyncLoadingStringTable& NewAsyncLoadingState = AsyncLoadingStringTables.Add(StringTableAssetPackageName);
					NewAsyncLoadingState.RequestedTableId = InTableId;
					if (InLoadedCallback)
					{
						NewAsyncLoadingState.LoadedCallbacks.Add(InLoadedCallback);
					}
				}

				// Begin an asynchronous load
				// Note: The LoadPackageAsync callback may fire immediately if the request is invalid. This would remove the entry from AsyncLoadingStringTables, so it's valid for the find request below to fail in that case
				const int32 AsyncLoadingId = LoadPackageAsync(StringTableAssetPackageNameStr, FLoadPackageAsyncDelegate::CreateRaw(this, &FStringTableEngineBridge::HandleStringTableAssetAsyncLoadCompleted));

				if (AsyncLoadingId != INDEX_NONE)
				{
					// Load ongoing
					FScopeLock AsyncLoadingStringTablesLock(&AsyncLoadingStringTablesCS);
					if (FAsyncLoadingStringTable* AsyncLoadingState = AsyncLoadingStringTables.Find(StringTableAssetPackageName))
					{
						AsyncLoadingState->AsyncLoadingId = AsyncLoadingId;
					}
					return AsyncLoadingId;
				}

				// Load failed
				if (InLoadedCallback)
				{
					InLoadedCallback(InTableId, FName());
				}
				return INDEX_NONE;
			}
			else
			{
				// Attempt a synchronous load
				StringTableAsset = Cast<UStringTable>(StringTableAssetReference.TryLoad());
				if (InLoadedCallback)
				{
					InLoadedCallback(InTableId, StringTableAsset ? StringTableAsset->GetStringTableId() : FName());
				}
				return INDEX_NONE;
			}
		}

		// Not an asset - just say it's already loaded
		if (InLoadedCallback)
		{
			InLoadedCallback(InTableId, InTableId);
		}
		return INDEX_NONE;
	}

	virtual void FullyLoadStringTableAssetImpl(FName& InOutTableId) override
	{
		const FSoftObjectPath StringTableAssetReference = GetAssetReference(InOutTableId);
		if (StringTableAssetReference.IsValid())
		{
			UStringTable* StringTableAsset = Cast<UStringTable>(StringTableAssetReference.ResolveObject());
			if (!StringTableAsset || StringTableAsset->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad))
			{
				StringTableAsset = Cast<UStringTable>(StringTableAssetReference.TryLoad());
			}
			if (StringTableAsset)
			{
				InOutTableId = StringTableAsset->GetStringTableId();
			}
		}
	}

	virtual void RedirectStringTableAssetImpl(FName& InOutTableId) override
	{
		const FSoftObjectPath StringTableAssetReference = GetAssetReference(InOutTableId);
		if (StringTableAssetReference.IsValid())
		{
			UStringTable* StringTableAsset = Cast<UStringTable>(StringTableAssetReference.ResolveObject());
			if (StringTableAsset)
			{
				InOutTableId = StringTableAsset->GetStringTableId();
			}
		}
	}

	virtual void CollectStringTableAssetReferencesImpl(FName& InOutTableId, FStructuredArchive::FSlot Slot) override
	{
		check(Slot.GetUnderlyingArchive().IsObjectReferenceCollector());

		UObject* StringTableAsset = FStringTableRegistry::Get().FindStringTableAsset(InOutTableId);

		FSoftObjectPath StringTableAssetReference = GetAssetReference(InOutTableId);
		if (StringTableAssetReference.IsValid())
		{
			FStructuredArchiveRecord Record = Slot.EnterRecord();

			// Save a hard-reference so that dependent tables are loaded correctly
			Record << SA_VALUE(TEXT("Ptr"), StringTableAsset);

			// Save a weak-reference so that redirector fix-up during rename works
			Record << SA_VALUE(TEXT("Ref"), StringTableAssetReference);

			// This may have redirected our weak-reference
			InOutTableId = *StringTableAssetReference.ToString();
		}
		else
		{
			// We have to write something to the slot...
			Slot << StringTableAsset;
		}
	}

	virtual void CollectStringTableAssetSearchableNameImpl(const FName InTableId, const FTextKey InKey, const FArchive& Ar) override
	{
		if (Ar.IsSaving() && !InTableId.IsNone() && !InKey.IsEmpty())
		{
			if (UStringTable* StringTableAsset = FStringTableRegistry::Get().FindStringTableAsset(InTableId))
			{
				// Note which key we are pointing to for later searching
				FNameBuilder KeyStr;
				InKey.ToString(KeyStr);
				Ar.MarkSearchableName(StringTableAsset, FName(KeyStr));
			}
		}
	}

	virtual bool IsStringTableFromAssetImpl(const FName InTableId) override
	{
		const FSoftObjectPath StringTableAssetReference = GetAssetReference(InTableId);
		return StringTableAssetReference.IsValid();
	}

	virtual bool IsStringTableAssetBeingReplacedImpl(const UStringTable* InStringTableAsset) override
	{
		return InStringTableAsset && InStringTableAsset->HasAnyFlags(RF_NewerVersionExists);
	}

	static FSoftObjectPath GetAssetReference(const FName InTableId)
	{
		const FString StringTableAssetName = InTableId.ToString();

		FString StringTablePackageName = StringTableAssetName;
		{
			int32 DotIndex = INDEX_NONE;
			if (StringTablePackageName.FindChar(TEXT('.'), DotIndex))
			{
				StringTablePackageName.LeftInline(DotIndex, EAllowShrinking::No);
			}
		}

		FSoftObjectPath StringTableAssetReference;
		if (FPackageName::IsValidLongPackageName(StringTablePackageName, /*bIncludeReadOnlyRoots*/true) && FPackageName::DoesPackageExist(StringTablePackageName))
		{
			StringTableAssetReference.SetPath(StringTableAssetName);
		}
		
		return StringTableAssetReference;
	}

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		FScopeLock KeepAliveStringTablesLock(&KeepAliveStringTablesCS);
		KeepAliveStringTables.RemoveAllSwap([](const TObjectPtr<UStringTable>& InStringTableAsset) { return !InStringTableAsset || InStringTableAsset->HasAnyFlags(RF_MirroredGarbage | RF_NewerVersionExists); });
		Collector.AddReferencedObjects(KeepAliveStringTables);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FStringTableEngineBridge");
	}

private:
	struct FAsyncLoadingStringTable
	{
		int32 AsyncLoadingId = INDEX_NONE;
		FName RequestedTableId;
		TArray<FLoadStringTableAssetCallback> LoadedCallbacks;
	};

	/** Map of string table assets that are currently being async loaded (package name -> async loading state) */
	TMap<FName, FAsyncLoadingStringTable> AsyncLoadingStringTables;
	/** Critical section preventing concurrent modification of AsyncLoadingStringTables */
	mutable FCriticalSection AsyncLoadingStringTablesCS;

	/** Array of string table assets that have been loaded and should be kept alive */
	TArray<TObjectPtr<UStringTable>> KeepAliveStringTables;
	/** Critical section preventing concurrent modification of KeepAliveStringTables */
	mutable FCriticalSection KeepAliveStringTablesCS;
};

UStringTable::UStringTable()
	: StringTable(FStringTable::NewStringTable())
	, StringTableId(*GetPathName()) // Note: If you change this ID format, also fix HandleStringTableAssetAsyncLoadCompleted
{
	StringTable->SetOwnerAsset(this);
	StringTable->IsLoaded(!HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));

	StringTable->SetNamespace(GetName());

	if (IsAsset() && UE::StringTable::Private::ShouldAutoRegister(GetStringTableId()))
	{
		FStringTableEngineBridge::Get().RegisterForGC(this);
		FStringTableRegistry::Get().RegisterStringTable(GetStringTableId(), StringTable.ToSharedRef());
	}

#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UStringTable::StaticClass(), &UE::StringTable::Private::GatherAssetForLocalization); }
#endif
}

void UStringTable::InitializeEngineBridge()
{
	FStringTableEngineBridge::Initialize();
}

void UStringTable::FinishDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject)) // Needed to avoid a crash during shutdown for programs that don't call RequestEngineExit
	{
		FStringTableEngineBridge::Get().UnregisterForGC(this);
		if (!HasAnyFlags(RF_NewerVersionExists))
		{
			FStringTableRegistry::Get().UnregisterStringTable(GetStringTableId());
		}
	}

	StringTable.Reset();

	Super::FinishDestroy();
}

void UStringTable::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	StringTable->Serialize(Ar);
}

void UStringTable::PostLoad()
{
	Super::PostLoad();

	StringTable->IsLoaded(true);

	if (FSlateApplicationBase::IsInitialized())
	{
		// Ensure all invalidation panels are updated now that the string data is loaded
		FSlateApplicationBase::Get().InvalidateAllWidgets(false);
	}
}

bool UStringTable::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	const bool bRenamed = Super::Rename(NewName, NewOuter, Flags);
	if (bRenamed)
	{
		const bool bWasRegistered = FStringTableRegistry::Get().UnregisterStringTable(GetStringTableId());
		const bool bWasManuallyRegistered = bWasRegistered && !UE::StringTable::Private::ShouldAutoRegister(GetStringTableId());

		StringTableId = *GetPathName();
		
		if (bWasRegistered && (bWasManuallyRegistered || UE::StringTable::Private::ShouldAutoRegister(GetStringTableId())))
		{
			FStringTableRegistry::Get().RegisterStringTable(GetStringTableId(), StringTable.ToSharedRef());
		}
	}
	return bRenamed;
}

void UStringTable::OnPluginUnloaded(FStringView PluginName)
{
	auto PackageNameBasis = WriteToString<256>(TEXT("/"), PluginName, TEXT("/"));
	FStringTableEngineBridge::Get().UnregisterForGCUnderPath(PackageNameBasis);
}

FName UStringTable::GetStringTableId() const
{
	return StringTableId;
}

FStringTableConstRef UStringTable::GetStringTable() const
{
	return StringTable.ToSharedRef();
}

FStringTableRef UStringTable::GetMutableStringTable() const
{
	return StringTable.ToSharedRef();
}
