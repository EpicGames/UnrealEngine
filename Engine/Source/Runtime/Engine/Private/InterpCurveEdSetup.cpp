// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InterpCurveEdSetup.cpp: Implementation of distribution classes.
=============================================================================*/

#include "Engine/InterpCurveEdSetup.h"
#include "Templates/Casts.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterpCurveEdSetup)


void UInterpCurveEdSetup::PostLoad()
{
	Super::PostLoad();

	// Remove any curve entries that do not refer to FCurveEdInterface-derived objects.
	for(int32 i=0; i<Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];
		for(int32 j=Tab.Curves.Num()-1; j>=0; j--)
		{
			FCurveEdInterface* EdInterface = GetCurveEdInterfacePointer( Tab.Curves[j] );
			if(!EdInterface)
			{
				Tab.Curves.RemoveAt(j);
			}
		}
	}
}

UInterpCurveEdSetup::UInterpCurveEdSetup(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	Tabs.Add(FCurveEdTab(TEXT("Default"), 0.0f, 1.0f, -1.0f, 1.0f));
}

FCurveEdInterface* UInterpCurveEdSetup::GetCurveEdInterfacePointer(const FCurveEdEntry& Entry)
{
	UDistributionFloat* FloatDist = Cast<UDistributionFloat>(Entry.CurveObject);
	if(FloatDist)
	{
		return FloatDist;
	}

	UDistributionVector* VectorDist = Cast<UDistributionVector>(Entry.CurveObject);
	if(VectorDist)
	{
		return VectorDist;
	}

	return NULL;
}

bool UInterpCurveEdSetup::AddCurveToCurrentTab(
		UObject* InCurve, const FString& CurveName, const FColor& CurveColor, 
		FCurveEdEntry** OutCurveEntry, bool bInColorCurve, bool bInFloatingPointColor,
		bool bInClamp, float InClampLow, float InClampHigh)
{
	FCurveEdTab& Tab = Tabs[ActiveTab];

	// See if curve is already on tab. If so, do nothing.
	for( int32 i=0; i<Tab.Curves.Num(); i++ )
	{
		FCurveEdEntry& Curve = Tab.Curves[i];
		if( Curve.CurveObject == InCurve )
		{
			if ( OutCurveEntry )
			{
				*OutCurveEntry = &Curve;
			}
			return false;
		}
	}

	// Curve not there, so make new entry and record details.
	FCurveEdEntry* NewCurve = new(Tab.Curves) FCurveEdEntry();
	FMemory::Memzero(NewCurve, sizeof(FCurveEdEntry));

	NewCurve->CurveObject = InCurve;
	NewCurve->CurveName = CurveName;
	NewCurve->CurveColor = CurveColor;
	NewCurve->bColorCurve = bInColorCurve;
	NewCurve->bFloatingPointColorCurve = bInFloatingPointColor;
	NewCurve->bClamp = bInClamp;
	NewCurve->ClampLow = InClampLow;
	NewCurve->ClampHigh = InClampHigh;

	if ( OutCurveEntry )
	{
		*OutCurveEntry = NewCurve;
	}
	return true;
}

void UInterpCurveEdSetup::RemoveCurve(UObject* InCurve)
{
	for(int32 i=0; i<Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];
		for(int32 j=Tab.Curves.Num()-1; j>=0; j--)
		{			
			if(Tab.Curves[j].CurveObject == InCurve)
			{
				Tab.Curves.RemoveAt(j);
			}
		}
	}
}
	
void UInterpCurveEdSetup::ReplaceCurve(UObject* RemoveCurve, UObject* AddCurve)
{
	check(RemoveCurve);
	check(AddCurve);

	for(int32 i=0; i<Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];
		for(int32 j=0; j<Tab.Curves.Num(); j++)
		{			
			if(Tab.Curves[j].CurveObject == RemoveCurve)
			{
				Tab.Curves[j].CurveObject = AddCurve;
			}
		}
	}
}


void UInterpCurveEdSetup::CreateNewTab(const FString& InTabName)
{
	FCurveEdTab Tab;
	
	FMemory::Memzero(&Tab, sizeof(Tab));
	
	Tab.TabName			= InTabName;
	Tab.ViewStartInput	=  0.0f;
	Tab.ViewEndInput	=  1.0f;
	Tab.ViewStartOutput = -1.0;
	Tab.ViewEndOutput	=  1.0;

	Tabs.Add(Tab);
}


void UInterpCurveEdSetup::RemoveTab(const FString& InTabName)
{
	for (int32 i = 0; i < Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];
		if (Tab.TabName == InTabName)
		{
			Tabs.RemoveAt(i);
			break;
		}
	}
}

bool UInterpCurveEdSetup::ShowingCurve(UObject* InCurve)
{
	for(int32 i=0; i<Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];

		for(int32 j=0; j<Tab.Curves.Num(); j++)
		{
			if(Tab.Curves[j].CurveObject == InCurve)
				return true;
		}
	}

	return false;
}

void UInterpCurveEdSetup::ChangeCurveColor(UObject* InCurve, const FColor& CurveColor)
{
	for (int32 i=0; i<Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];
		for (int32 j=0; j<Tab.Curves.Num(); j++)
		{
			if (Tab.Curves[j].CurveObject == InCurve)
			{
				Tab.Curves[j].CurveColor = CurveColor;
			}
		}
	}
}

void UInterpCurveEdSetup::ChangeCurveName(UObject* InCurve, const FString& NewCurveName)
{
	for (int32 i=0; i<Tabs.Num(); i++)
	{
		FCurveEdTab& Tab = Tabs[i];
		for (int32 j=0; j<Tab.Curves.Num(); j++)
		{
			if (Tab.Curves[j].CurveObject == InCurve)
			{
				Tab.Curves[j].CurveName = NewCurveName;
			}
		}
	}
}


void UInterpCurveEdSetup::ResetTabs()
{
	Tabs.Empty();

	FCurveEdTab Tab;
	
	FMemory::Memzero(&Tab, sizeof(Tab));
	
	Tab.TabName			= TEXT("Default");
	Tab.ViewStartInput	=  0.0f;
	Tab.ViewEndInput	=  1.0f;
	Tab.ViewStartOutput = -1.0;
	Tab.ViewEndOutput	=  1.0;

	Tabs.Add(Tab);
}

