// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Components/SceneComponent.h"
#include "Engine/LatentActionManager.h"
#include "LatentActions.h"

/** Action that interpolates a component over time to a desired position */
class FInterpolateComponentToAction : public FPendingLatentAction
{
public:
	/** Time over which interpolation should happen */
	float TotalTime;
	/** Time so far elapsed for the interpolation */
	float TimeElapsed;
	/** If we are currently interpolating. If false, update will complete */
	bool bInterpolating;

	/** Function to execute on completion */
	FName ExecutionFunction;
	/** Link to fire on completion */
	int32 OutputLink;
	/** Object to call callback on upon completion */
	FWeakObjectPtr CallbackTarget;

	/** Component to interpolate */
	TWeakObjectPtr<USceneComponent> TargetComponent;

	/** If we should modify rotation */
	bool bInterpRotation;
	/** Rotation to interpolate from */
	FRotator InitialRotation;
	/** Rotation to interpolate to */
	FRotator TargetRotation;

	/** If we should modify location */
	bool bInterpLocation;
	/** Location to interpolate from */
	FVector InitialLocation;
	/** Location to interpolate to */
	FVector TargetLocation;

	/** Should we ease in (ie start slowly) during interpolation */
	bool bEaseIn;
	/** Should we east out (ie end slowly) during interpolation */
	bool bEaseOut;

	/** Force use of shortest Path for rotation **/
	bool bForceShortestRotationPath;

	FInterpolateComponentToAction(float Duration, const FLatentActionInfo& LatentInfo, USceneComponent* Component, bool bInEaseOut, bool bInEaseIn, bool bInForceShortestRotationPath)
		: TotalTime(Duration)
		, TimeElapsed(0.f)
		, bInterpolating(true)
		, ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
		, TargetComponent(Component)
		, bInterpRotation(true)
		, InitialRotation(FRotator::ZeroRotator)
		, TargetRotation(FRotator::ZeroRotator)
		, bInterpLocation(true)
		, InitialLocation(FVector::ZeroVector)
		, TargetLocation(FVector::ZeroVector)
		, bEaseIn(bInEaseIn)
		, bEaseOut(bInEaseOut)
		, bForceShortestRotationPath(bInForceShortestRotationPath)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		// Update elapsed time
		TimeElapsed += Response.ElapsedTime();

		bool bComplete = (TimeElapsed >= TotalTime);

		// If we have a component to modify..
		if(TargetComponent.IsValid() && bInterpolating)
		{
			// Work out 'Blend Percentage'
			const float BlendExp = 2.f;
			float DurationPct = TimeElapsed/TotalTime;
			float BlendPct;
			if(bEaseIn)
			{
				if(bEaseOut)
				{
					// EASE IN/OUT
					BlendPct = FMath::InterpEaseInOut(0.f, 1.f, DurationPct, BlendExp);
				}
				else
				{
					// EASE IN
					BlendPct = FMath::Lerp(0.f, 1.f, FMath::Pow(DurationPct, BlendExp));
				}
			}
			else
			{
				if(bEaseOut)
				{
					// EASE OUT
					BlendPct = FMath::Lerp(0.f, 1.f, FMath::Pow(DurationPct, 1.f / BlendExp));
				}
				else
				{
					// LINEAR
					BlendPct = FMath::Lerp(0.f, 1.f, DurationPct);
				}
			}

			// Update location
			if(bInterpLocation)
			{
				FVector NewLocation = bComplete ? TargetLocation : FMath::Lerp(InitialLocation, TargetLocation, BlendPct);
				TargetComponent->SetRelativeLocation(NewLocation, false);
			}

			if(bInterpRotation && TargetComponent.IsValid())
			{
				FRotator NewRotation;
				// If we are done just set the final rotation
				if (bComplete)
				{
					NewRotation = TargetRotation;
				}
				else if (bForceShortestRotationPath)
				{
					// We want the shortest path 
					FQuat AQuat(InitialRotation);
					FQuat BQuat(TargetRotation);

					FQuat Result = FQuat::Slerp(AQuat, BQuat, BlendPct);
					Result.Normalize();
					NewRotation = Result.Rotator();
				}
				else
				{
					// dont care about it being the shortest path - just lerp
					NewRotation = FMath::Lerp(InitialRotation, TargetRotation, BlendPct);
				}
				TargetComponent->SetRelativeRotation(NewRotation, false);
			}
		}

		Response.FinishAndTriggerIf(bComplete || !bInterpolating, ExecutionFunction, OutputLink, CallbackTarget);
	}

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override
	{
		static const FNumberFormattingOptions DelayTimeFormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(3)
			.SetMaximumFractionalDigits(3);
		return FText::Format(NSLOCTEXT("FInterpolateComponentToAction", "ActionTimeFmt", "Move ({0} seconds left)"), FText::AsNumber(TotalTime-TimeElapsed, &DelayTimeFormatOptions)).ToString();
	}
#endif
};
