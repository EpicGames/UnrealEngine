// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/KillZVolume.h"
#include "GameFramework/DamageType.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KillZVolume)

AKillZVolume::AKillZVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bNetLoadOnClient = false;
}

void AKillZVolume::ActorEnteredVolume(AActor* Other)
{
	Super::ActorEnteredVolume(Other);
	
	if ( Other )
	{
		const UDamageType* DamageType = GetDefault<UDamageType>();

		UWorld* World = GetWorld();
		if ( World )
		{
			AWorldSettings* WorldSettings = World->GetWorldSettings( true );
			if ( WorldSettings && WorldSettings->KillZDamageType )
			{
				DamageType = WorldSettings->KillZDamageType->GetDefaultObject<UDamageType>();
			}
		}

		Other->FellOutOfWorld(*DamageType);
	}
}

