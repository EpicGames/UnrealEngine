// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintAsyncActionBase.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintAsyncActionBase)

#if DO_ENSURE

#include "Async/TransactionallySafeMutex.h"
#include "Misc/NotNull.h"

namespace UE::BlueprintAsyncAction::Private
{
	static TAutoConsoleVariable<int32> CVarMaxActionCount(
		TEXT("bp.MaxAsyncActionCount"),
		10000,
		TEXT("The maximum number of async actions of a that can be active at a time before triggering an ensure. Set to <= 0 to disable")
	);

	/** Total number of async actions that are active */
	static int32 GlobalAsyncActionCount = 0;
	static UE::FTransactionallySafeMutex GlobalActionCountMutex;

	static void DebugRecordAsyncActionCreated(TNotNull<const UBlueprintAsyncActionBase*> Action)
	{
		UE::TScopeLock Lock(GlobalActionCountMutex);
		GlobalAsyncActionCount++;

		const int32 MaxActionCount = CVarMaxActionCount.GetValueOnAnyThread();
		if (MaxActionCount > 0)
		{
			ensureMsgf(GlobalAsyncActionCount <= MaxActionCount,
				TEXT("Exceeded the maximum number of allowed async actions. Last created action class was %s"),
				*GetNameSafe(Action->GetClass()));
		}
	}

	static void DebugRecordAsyncActionDestroyed()
	{
		UE::TScopeLock Lock(GlobalActionCountMutex);
		--GlobalAsyncActionCount;
	}
}
#endif // DO_ENSURE

//////////////////////////////////////////////////////////////////////////
// UBlueprintAsyncActionBase

UBlueprintAsyncActionBase::UBlueprintAsyncActionBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SetFlags(RF_StrongRefOnFrame);

#if DO_ENSURE
		UE::BlueprintAsyncAction::Private::DebugRecordAsyncActionCreated(this);
#endif // DO_ENSURE
	}
}

void UBlueprintAsyncActionBase::BeginDestroy()
{
	Super::BeginDestroy();

#if DO_ENSURE
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UE::BlueprintAsyncAction::Private::DebugRecordAsyncActionDestroyed();
	}
#endif // DO_ENSURE
}

void UBlueprintAsyncActionBase::Activate()
{
}

void UBlueprintAsyncActionBase::RegisterWithGameInstance(const UObject* WorldContextObject)
{
	UWorld* FoundWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (FoundWorld && FoundWorld->GetGameInstance())
	{
		RegisterWithGameInstance(FoundWorld->GetGameInstance());
	}
}

void UBlueprintAsyncActionBase::RegisterWithGameInstance(UGameInstance* GameInstance)
{
	if (GameInstance)
	{
		UGameInstance* OldGameInstance = RegisteredWithGameInstance.Get();
		if (OldGameInstance)
		{
			OldGameInstance->UnregisterReferencedObject(this);
		}

		GameInstance->RegisterReferencedObject(this);
		RegisteredWithGameInstance = GameInstance;
	}
}

void UBlueprintAsyncActionBase::SetReadyToDestroy()
{
	ClearFlags(RF_StrongRefOnFrame);

	UGameInstance* OldGameInstance = RegisteredWithGameInstance.Get();
	if (OldGameInstance)
	{
		OldGameInstance->UnregisterReferencedObject(this);
		RegisteredWithGameInstance.Reset();
	}
}

