// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintPathsLibrary.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintPathsLibrary)

UBlueprintPathsLibrary::UBlueprintPathsLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

};

bool UBlueprintPathsLibrary::ShouldSaveToUserDir()
{
	return FPaths::ShouldSaveToUserDir();
}

FString UBlueprintPathsLibrary::LaunchDir()
{
	return FPaths::LaunchDir();
}

FString UBlueprintPathsLibrary::EngineDir()
{
	return FPaths::EngineDir();
}

FString UBlueprintPathsLibrary::EngineUserDir()
{
	return FPaths::EngineUserDir();
}

FString UBlueprintPathsLibrary::EngineVersionAgnosticUserDir()
{
	return FPaths::EngineVersionAgnosticUserDir();
}

FString UBlueprintPathsLibrary::EngineContentDir()
{
	return FPaths::EngineContentDir();
}

FString UBlueprintPathsLibrary::EngineConfigDir()
{
	return FPaths::EngineConfigDir();
}

FString UBlueprintPathsLibrary::EngineIntermediateDir()
{
	return FPaths::EngineIntermediateDir();
}

FString UBlueprintPathsLibrary::EngineSavedDir()
{
	return FPaths::EngineSavedDir();
}

FString UBlueprintPathsLibrary::EnginePluginsDir()
{
	return FPaths::EnginePluginsDir();
}

FString UBlueprintPathsLibrary::EnterpriseDir()
{
	return FPaths::EnterpriseDir();
}

FString UBlueprintPathsLibrary::EnterprisePluginsDir()
{
	return FPaths::EnterprisePluginsDir();
}

FString UBlueprintPathsLibrary::EnterpriseFeaturePackDir()
{
	return FPaths::EnterpriseFeaturePackDir();
}

FString UBlueprintPathsLibrary::RootDir()
{
	return FPaths::RootDir();
}

FString UBlueprintPathsLibrary::ProjectDir()
{
	return FPaths::ProjectDir();
}

FString UBlueprintPathsLibrary::ProjectUserDir()
{
	return FPaths::ProjectUserDir();
}

FString UBlueprintPathsLibrary::ProjectContentDir()
{
	return FPaths::ProjectContentDir();
}

FString UBlueprintPathsLibrary::ProjectConfigDir()
{
	return FPaths::ProjectConfigDir();
}

FString UBlueprintPathsLibrary::ProjectSavedDir()
{
	return FPaths::ProjectSavedDir();
}

FString UBlueprintPathsLibrary::ProjectIntermediateDir()
{
	return FPaths::ProjectIntermediateDir();
}

FString UBlueprintPathsLibrary::ShaderWorkingDir()
{
	return FPaths::ShaderWorkingDir();
}

FString UBlueprintPathsLibrary::ProjectPluginsDir()
{
	return FPaths::ProjectPluginsDir();
}

FString UBlueprintPathsLibrary::ProjectModsDir()
{
	return FPaths::ProjectModsDir();
}

bool UBlueprintPathsLibrary::HasProjectPersistentDownloadDir()
{
	return FPaths::HasProjectPersistentDownloadDir();
}

FString UBlueprintPathsLibrary::ProjectPersistentDownloadDir()
{
	return FPaths::ProjectPersistentDownloadDir();
}

FString UBlueprintPathsLibrary::SourceConfigDir()
{
	return FPaths::SourceConfigDir();
}

FString UBlueprintPathsLibrary::GeneratedConfigDir()
{
	return FPaths::GeneratedConfigDir();
}

FString UBlueprintPathsLibrary::SandboxesDir()
{
	return FPaths::SandboxesDir();
}

FString UBlueprintPathsLibrary::ProfilingDir()
{
	return FPaths::ProfilingDir();
}

FString UBlueprintPathsLibrary::ScreenShotDir()
{
	return FPaths::ScreenShotDir();
}

FString UBlueprintPathsLibrary::BugItDir()
{
	return FPaths::BugItDir();
}

FString UBlueprintPathsLibrary::VideoCaptureDir()
{
	return FPaths::VideoCaptureDir();
}

FString UBlueprintPathsLibrary::ProjectLogDir()
{
	return FPaths::ProjectLogDir();
}

FString UBlueprintPathsLibrary::AutomationDir()
{
	return FPaths::AutomationDir();
}

FString UBlueprintPathsLibrary::AutomationTransientDir()
{
	return FPaths::AutomationTransientDir();
}

FString UBlueprintPathsLibrary::AutomationLogDir()
{
	return FPaths::AutomationLogDir();
}

FString UBlueprintPathsLibrary::CloudDir()
{
	return FPaths::CloudDir();
}

FString UBlueprintPathsLibrary::GameDevelopersDir()
{
	return FPaths::GameDevelopersDir();
}

FString UBlueprintPathsLibrary::GameUserDeveloperDir()
{
	return FPaths::GameUserDeveloperDir();
}

FString UBlueprintPathsLibrary::DiffDir()
{
	return FPaths::DiffDir();
}

const TArray<FString>& UBlueprintPathsLibrary::GetEngineLocalizationPaths()
{
	return FPaths::GetEngineLocalizationPaths();
}

const TArray<FString>& UBlueprintPathsLibrary::GetEditorLocalizationPaths()
{
	return FPaths::GetEditorLocalizationPaths();
}

const TArray<FString>& UBlueprintPathsLibrary::GetPropertyNameLocalizationPaths()
{
	return FPaths::GetPropertyNameLocalizationPaths();
}

const TArray<FString>& UBlueprintPathsLibrary::GetToolTipLocalizationPaths()
{
	return FPaths::GetToolTipLocalizationPaths();
}

const TArray<FString>& UBlueprintPathsLibrary::GetGameLocalizationPaths()
{
	return FPaths::GetGameLocalizationPaths();
}

const TArray<FString>& UBlueprintPathsLibrary::GetRestrictedFolderNames()
{
	return FPaths::GetRestrictedFolderNames();
}

bool UBlueprintPathsLibrary::IsRestrictedPath(const FString& InPath)
{
	return FPaths::IsRestrictedPath(InPath);
}

FString UBlueprintPathsLibrary::GameAgnosticSavedDir()
{
	return FPaths::GameAgnosticSavedDir();
}

FString UBlueprintPathsLibrary::EngineSourceDir()
{
	return FPaths::EngineSourceDir();
}

FString UBlueprintPathsLibrary::GameSourceDir()
{
	return FPaths::GameSourceDir();
}

FString UBlueprintPathsLibrary::FeaturePackDir()
{
	return FPaths::FeaturePackDir();
}

bool UBlueprintPathsLibrary::IsProjectFilePathSet()
{
	return FPaths::IsProjectFilePathSet();
}

FString UBlueprintPathsLibrary::GetProjectFilePath()
{
	return FPaths::GetProjectFilePath();
}

void UBlueprintPathsLibrary::SetProjectFilePath(const FString& NewGameProjectFilePath)
{
	FPaths::SetProjectFilePath(NewGameProjectFilePath);
}

FString UBlueprintPathsLibrary::GetExtension(const FString& InPath, bool bIncludeDot /*= false*/)
{
	return FPaths::GetExtension(InPath, bIncludeDot);
}

FString UBlueprintPathsLibrary::GetCleanFilename(const FString& InPath)
{
	return FPaths::GetCleanFilename(InPath);
}

FString UBlueprintPathsLibrary::GetBaseFilename(const FString& InPath, bool bRemovePath /*= true*/)
{
	return FPaths::GetBaseFilename(InPath, bRemovePath);
}

FString UBlueprintPathsLibrary::GetPath(const FString& InPath)
{
	return FPaths::GetPath(InPath);
}

FString UBlueprintPathsLibrary::ChangeExtension(const FString& InPath, const FString& InNewExtension)
{
	return FPaths::ChangeExtension(InPath, InNewExtension);
}

FString UBlueprintPathsLibrary::SetExtension(const FString& InPath, const FString& InNewExtension)
{
	return FPaths::SetExtension(InPath, InNewExtension);
}

bool UBlueprintPathsLibrary::FileExists(const FString& InPath)
{
	return FPaths::FileExists(InPath);
}

bool UBlueprintPathsLibrary::DirectoryExists(const FString& InPath)
{
	return FPaths::DirectoryExists(InPath);
}

bool UBlueprintPathsLibrary::IsDrive(const FString& InPath)
{
	return FPaths::IsDrive(InPath);
}

bool UBlueprintPathsLibrary::IsRelative(const FString& InPath)
{
	return FPaths::IsRelative(InPath);
}

void UBlueprintPathsLibrary::NormalizeFilename(const FString& InPath, FString& OutPath)
{
	OutPath = InPath;
	FPaths::NormalizeFilename(OutPath);
}

bool UBlueprintPathsLibrary::IsSamePath(const FString& PathA, const FString& PathB)
{
	return FPaths::IsSamePath(PathA, PathB);
}

void UBlueprintPathsLibrary::NormalizeDirectoryName(const FString& InPath, FString& OutPath)
{
	OutPath = InPath;
	FPaths::NormalizeDirectoryName(OutPath);
}

bool UBlueprintPathsLibrary::CollapseRelativeDirectories(const FString& InPath, FString& OutPath)
{
	OutPath = InPath;
	return FPaths::CollapseRelativeDirectories(OutPath);
}

void UBlueprintPathsLibrary::RemoveDuplicateSlashes(const FString& InPath, FString& OutPath)
{
	OutPath = InPath;
	FPaths::RemoveDuplicateSlashes(OutPath);
}

void UBlueprintPathsLibrary::MakeStandardFilename(const FString& InPath, FString& OutPath)
{
	OutPath = InPath;
	FPaths::MakeStandardFilename(OutPath);
}

void UBlueprintPathsLibrary::MakePlatformFilename(const FString& InPath, FString& OutPath)
{
	OutPath = InPath;
	FPaths::MakePlatformFilename(OutPath);
}

bool UBlueprintPathsLibrary::MakePathRelativeTo(const FString& InPath, const FString& InRelativeTo, FString& OutPath)
{
	OutPath = InPath;
	return FPaths::MakePathRelativeTo(OutPath, *InRelativeTo);
}

FString UBlueprintPathsLibrary::ConvertRelativePathToFull(const FString& InPath, const FString& InBasePath /*= TEXT("")*/ )
{
	if (InBasePath.Len() > 0)
	{
		return FPaths::ConvertRelativePathToFull(InBasePath, InPath);
	}
	else
	{
		return FPaths::ConvertRelativePathToFull(InPath);
	}
}

FString UBlueprintPathsLibrary::ConvertToSandboxPath(const FString& InPath, const FString& InSandboxName)
{
	return FPaths::ConvertToSandboxPath(InPath, *InSandboxName);
}

FString UBlueprintPathsLibrary::ConvertFromSandboxPath(const FString& InPath, const FString& InSandboxName)
{
	return FPaths::ConvertFromSandboxPath(InPath, *InSandboxName);
}

FString UBlueprintPathsLibrary::CreateTempFilename(const FString& Path, const FString& Prefix /*= TEXT("")*/, const FString& Extension /*= TEXT(".tmp")*/)
{
	return FPaths::CreateTempFilename(*Path, *Prefix, *Extension);
}

FString UBlueprintPathsLibrary::GetInvalidFileSystemChars()
{
	return FPaths::GetInvalidFileSystemChars();
}

FString UBlueprintPathsLibrary::MakeValidFileName(const FString& InString, const FString& InReplacementChar /*= TEXT("")*/)
{
	if (InReplacementChar.Len() > 0)
	{
		return FPaths::MakeValidFileName(InString, InReplacementChar[0]);
	}
	else
	{
		return FPaths::MakeValidFileName(InString);
	}
}

void UBlueprintPathsLibrary::ValidatePath(const FString& InPath, bool& bDidSucceed, FText& OutReason)
{
	bDidSucceed = FPaths::ValidatePath(InPath, &OutReason);
}

void UBlueprintPathsLibrary::Split(const FString& InPath, FString& PathPart, FString& FilenamePart, FString& ExtensionPart)
{
	FPaths::Split(InPath, PathPart, FilenamePart, ExtensionPart);
}

const FString& UBlueprintPathsLibrary::GetRelativePathToRoot()
{
	return FPaths::GetRelativePathToRoot();
}

FString UBlueprintPathsLibrary::Combine(const TArray<FString>& InPaths)
{
	FString OutString;
	if (InPaths.Num() == 1)
	{
		// Only one path so combination is just the first element
		OutString = InPaths[0];
	}
	else if (InPaths.Num() > 1)
	{
		// Multiple paths so perform a left fold on them
		OutString = InPaths[0];

		for (int PathIdx = 1; PathIdx < InPaths.Num(); ++PathIdx)
		{
			OutString = FPaths::Combine(OutString, InPaths[PathIdx]);
		}
	}

	return OutString;
}
