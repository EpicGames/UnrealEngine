// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintPlatformLibrary.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "LocalNotification.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintPlatformLibrary)

void UPlatformGameInstance::PostInitProperties()

{
    Super::PostInitProperties();

    FCoreDelegates::ApplicationWillDeactivateDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationWillDeactivateDelegate_Handler);
    FCoreDelegates::ApplicationHasReactivatedDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationHasReactivatedDelegate_Handler);
    FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationWillEnterBackgroundDelegate_Handler);
    FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationHasEnteredForegroundDelegate_Handler);
    FCoreDelegates::GetApplicationWillTerminateDelegate().AddUObject(this, &UPlatformGameInstance::ApplicationWillTerminateDelegate_Handler);
	FCoreDelegates::ApplicationShouldUnloadResourcesDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationShouldUnloadResourcesDelegate_Handler);
	FCoreDelegates::ApplicationReceivedStartupArgumentsDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationReceivedStartupArgumentsDelegate_Handler);
    FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationRegisteredForRemoteNotificationsDelegate_Handler);
    FCoreDelegates::ApplicationRegisteredForUserNotificationsDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationRegisteredForUserNotificationsDelegate_Handler);
    FCoreDelegates::ApplicationFailedToRegisterForRemoteNotificationsDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationFailedToRegisterForRemoteNotificationsDelegate_Handler);
    FCoreDelegates::ApplicationReceivedRemoteNotificationDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationReceivedRemoteNotificationDelegate_Handler);
    FCoreDelegates::ApplicationReceivedLocalNotificationDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationReceivedLocalNotificationDelegate_Handler);
    FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.AddUObject(this, &UPlatformGameInstance::ApplicationReceivedScreenOrientationChangedNotificationDelegate_Handler);
}

void UPlatformGameInstance::BeginDestroy()

{
	FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
	FCoreDelegates::GetApplicationWillTerminateDelegate().RemoveAll(this);
 	FCoreDelegates::ApplicationShouldUnloadResourcesDelegate.RemoveAll(this);
 	FCoreDelegates::ApplicationReceivedStartupArgumentsDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationRegisteredForUserNotificationsDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationFailedToRegisterForRemoteNotificationsDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationReceivedRemoteNotificationDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationReceivedLocalNotificationDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.RemoveAll(this);

    Super::BeginDestroy();
}

void UPlatformGameInstance::ApplicationReceivedScreenOrientationChangedNotificationDelegate_Handler(int32 inScreenOrientation)
{
	ApplicationReceivedScreenOrientationChangedNotificationDelegate.Broadcast((EScreenOrientation::Type)inScreenOrientation);
}

void UPlatformGameInstance::ApplicationReceivedRemoteNotificationDelegate_Handler(FString inFString, int32 inAppState)
{
	ApplicationReceivedRemoteNotificationDelegate.Broadcast(inFString, (EApplicationState::Type)inAppState);
}

void UPlatformGameInstance::ApplicationReceivedLocalNotificationDelegate_Handler(FString inFString, int32 inInt, int32 inAppState)
{
	ApplicationReceivedLocalNotificationDelegate.Broadcast(inFString, inInt, (EApplicationState::Type)inAppState);
}


//////////////////////////////////////////////////////////////////////////
// UBlueprintPlatformLibrary

UBlueprintPlatformLibrary::UBlueprintPlatformLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (platformService == nullptr)
	{
		FString ModuleName;
		GConfig->GetString(TEXT("LocalNotification"), TEXT("DefaultPlatformService"), ModuleName, GEngineIni);

		if (ModuleName.Len() > 0)
		{
			// load the module by name from the .ini
			if (ILocalNotificationModule* Module = FModuleManager::LoadModulePtr<ILocalNotificationModule>(*ModuleName))
			{
				platformService = Module->GetLocalNotificationService();
			}
		}
	}
}

void UBlueprintPlatformLibrary::ClearAllLocalNotifications()
{
	if(platformService == nullptr)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "ClearAllLocalNotifications(): No local notification service");
		return;
	}
	
	platformService->ClearAllLocalNotifications();
}

int32 UBlueprintPlatformLibrary::ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool inLocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent)
{
	if(platformService == nullptr)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "ScheduleLocalNotificationAtTime(): No local notification service");
		return -1;
	}

	UE_LOGF(LogBlueprintUserMessages, Log, "Scheduling notification %ls at %d/%d/%d %d:%d:%d %ls", *(Title.ToString()), FireDateTime.GetMonth(), FireDateTime.GetDay(), FireDateTime.GetYear(), FireDateTime.GetHour(), FireDateTime.GetMinute(), FireDateTime.GetSecond(), inLocalTime ? TEXT("Local") : TEXT("UTC"));
	
	return platformService->ScheduleLocalNotificationAtTime(FireDateTime, inLocalTime, Title, Body, Action, ActivationEvent);
}
       
int32 UBlueprintPlatformLibrary::ScheduleLocalNotificationFromNow(int32 inSecondsFromNow, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent)
{
	FDateTime TargetTime = FDateTime::Now();
	TargetTime += FTimespan::FromSeconds(inSecondsFromNow);

	return ScheduleLocalNotificationAtTime(TargetTime, true, Title, Body, Action, ActivationEvent);
}

int32 UBlueprintPlatformLibrary::ScheduleLocalNotificationBadgeAtTime(const FDateTime& FireDateTime, bool inLocalTime, const FString& ActivationEvent)
{
	if (platformService == nullptr)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "ScheduleLocalNotificationBadgeAtTime(): No local notification service");
		return -1;
	}

	UE_LOGF(LogBlueprintUserMessages, Log, "Scheduling notification badge %ls at %d/%d/%d %d:%d:%d %ls", *ActivationEvent, FireDateTime.GetMonth(), FireDateTime.GetDay(), FireDateTime.GetYear(), FireDateTime.GetHour(), FireDateTime.GetMinute(), FireDateTime.GetSecond(), inLocalTime ? TEXT("Local") : TEXT("UTC"));

	return platformService->ScheduleLocalNotificationBadgeAtTime(FireDateTime, inLocalTime, ActivationEvent);
}

void UBlueprintPlatformLibrary::ScheduleLocalNotificationBadgeFromNow(int32 inSecondsFromNow, const FString& ActivationEvent)
{
	FDateTime TargetTime = FDateTime::Now();
	TargetTime += FTimespan::FromSeconds(inSecondsFromNow);

	ScheduleLocalNotificationBadgeAtTime(TargetTime, true, ActivationEvent);
}

UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
void UBlueprintPlatformLibrary::CancelLocalNotification(const FString& ActivationEvent)
{
	if(platformService == nullptr)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "CancelLocalNotification(): No local notification service");
		return;
	}

	UE_LOGF(LogBlueprintUserMessages, Log, "Canceling notification %ls", *ActivationEvent);
	
	platformService->CancelLocalNotification(ActivationEvent);
}

UFUNCTION(BlueprintCallable, Category="Platform|LocalNotification")
void UBlueprintPlatformLibrary::CancelLocalNotificationById(int32 NotificationId)
{
	if(platformService == nullptr)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "CancelLocalNotification(): No local notification service");
		return;
	}
	
	UE_LOGF(LogBlueprintUserMessages, Log, "Canceling notification %d", NotificationId);
	
	platformService->CancelLocalNotification(NotificationId);
}

void UBlueprintPlatformLibrary::GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate)
{
	if(platformService == nullptr)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "GetLaunchNotification(): No local notification service");
		return;
	}
	
	platformService->GetLaunchNotification(NotificationLaunchedApp, ActivationEvent, FireDate);
}

EScreenOrientation::Type UBlueprintPlatformLibrary::GetDeviceOrientation()
{
	return ConvertToScreenOrientation(FPlatformMisc::GetDeviceOrientation());
}


EScreenOrientation::Type UBlueprintPlatformLibrary::GetAllowedDeviceOrientation()
{
	return ConvertToScreenOrientation(FPlatformMisc::GetAllowedDeviceOrientation());
}

void UBlueprintPlatformLibrary::SetAllowedDeviceOrientation(EScreenOrientation::Type NewAllowedDeviceOrientation)
{
	FPlatformMisc::SetAllowedDeviceOrientation(ConvertToDeviceScreenOrientation(NewAllowedDeviceOrientation));
}

EScreenOrientation::Type UBlueprintPlatformLibrary::ConvertToScreenOrientation(EDeviceScreenOrientation DeviceScreenOrientation)
{
	switch (DeviceScreenOrientation)
	{
	case EDeviceScreenOrientation::Portrait:
		return EScreenOrientation::Portrait;

	case EDeviceScreenOrientation::PortraitUpsideDown:
		return EScreenOrientation::PortraitUpsideDown;

	case EDeviceScreenOrientation::LandscapeLeft:
		return EScreenOrientation::LandscapeLeft;

	case EDeviceScreenOrientation::LandscapeRight:
		return EScreenOrientation::LandscapeRight;

	case EDeviceScreenOrientation::FaceUp:
		return EScreenOrientation::FaceUp;

	case EDeviceScreenOrientation::FaceDown:
		return EScreenOrientation::FaceDown;

	case EDeviceScreenOrientation::PortraitSensor:
		return EScreenOrientation::PortraitSensor;

	case EDeviceScreenOrientation::LandscapeSensor:
		return EScreenOrientation::LandscapeSensor;

	case EDeviceScreenOrientation::FullSensor:
		return EScreenOrientation::FullSensor;
	}
	return EScreenOrientation::Unknown;
}

EDeviceScreenOrientation UBlueprintPlatformLibrary::ConvertToDeviceScreenOrientation(EScreenOrientation::Type ScreenOrientation)
{
	switch (ScreenOrientation)
	{
	case EScreenOrientation::Portrait:
		return EDeviceScreenOrientation::Portrait;

	case EScreenOrientation::PortraitUpsideDown:
		return EDeviceScreenOrientation::PortraitUpsideDown;

	case EScreenOrientation::LandscapeLeft:
		return EDeviceScreenOrientation::LandscapeLeft;

	case EScreenOrientation::LandscapeRight:
		return EDeviceScreenOrientation::LandscapeRight;

	case EScreenOrientation::FaceUp:
		return EDeviceScreenOrientation::FaceUp;

	case EScreenOrientation::FaceDown:
		return EDeviceScreenOrientation::FaceDown;

	case EScreenOrientation::PortraitSensor:
		return EDeviceScreenOrientation::PortraitSensor;

	case EScreenOrientation::LandscapeSensor:
		return EDeviceScreenOrientation::LandscapeSensor;

	case EScreenOrientation::FullSensor:
		return EDeviceScreenOrientation::FullSensor;
	}
	return EDeviceScreenOrientation::Unknown;
}

ILocalNotificationService* UBlueprintPlatformLibrary::platformService = nullptr;

