// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintPropertyHelpers.h"
#include "UObject/UnrealType.h"

void UE::BlueprintPropertyHelpers::ResetToDefault(void* Instance, const FProperty* Type)
{
	// the goal of this function is match the semantic of InitializeValue. It's very similar
	// to ClearValue, except that it honors default values from UserDefinedStruct.
 	const int32 PropertySize = Type->GetElementSize() * Type->ArrayDim;
	if(Type->HasAnyPropertyFlags(CPF_ZeroConstructor))
	{
		// common case.. default value is zero, just zero the memory:
		FMemory::Memzero(Instance, static_cast<size_t>(PropertySize));
	}
	else
	{
		// copy from default value:
		void* StorageSpace = FMemory_Alloca(PropertySize);
		Type->InitializeValue(StorageSpace);
		Type->CopySingleValueToScriptVM(Instance, StorageSpace);
		Type->DestroyValue(StorageSpace);
	}
}
