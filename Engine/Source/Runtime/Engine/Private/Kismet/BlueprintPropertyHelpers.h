// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FProperty;

namespace UE::BlueprintPropertyHelpers
{
	void ResetToDefault(void* Instance, const FProperty* Type);
}
