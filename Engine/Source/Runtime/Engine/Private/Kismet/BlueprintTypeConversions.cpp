// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/BlueprintTypeConversions.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Stack.h"
#include "Types/SlateVector2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintTypeConversions)

#define MAKE_CONVERSION_FUNCTION_NAME(SourceType, DestType)													\
	Convert##SourceType##To##DestType

#define DEFINE_CONVERSION_FUNCTIONS(BASETYPE, DOUBLEVARIANT, FLOATVARIANT)									\
	DEFINE_FUNCTION(UBlueprintTypeConversions::MAKE_CONVERSION_EXEC_FUNCTION_NAME(DOUBLEVARIANT, FLOATVARIANT)) \
	{																										\
		using namespace UE::Kismet::BlueprintTypeConversions::Internal;										\
																											\
		void* DestAddr = Stack.MostRecentPropertyAddress;													\
		Stack.MostRecentProperty = nullptr;																	\
		Stack.StepCompiledIn(nullptr, nullptr);																\
		const void* SourceAddr = Stack.MostRecentPropertyAddress;											\
		P_FINISH;																							\
		ConvertType<DOUBLEVARIANT, FLOATVARIANT>(SourceAddr, DestAddr);										\
	}																										\
	DEFINE_FUNCTION(UBlueprintTypeConversions::MAKE_CONVERSION_EXEC_FUNCTION_NAME(FLOATVARIANT, DOUBLEVARIANT)) \
	{																										\
		using namespace UE::Kismet::BlueprintTypeConversions::Internal;										\
																											\
		void* DestAddr = Stack.MostRecentPropertyAddress;													\
		Stack.MostRecentProperty = nullptr;																	\
		Stack.StepCompiledIn(nullptr, nullptr);																\
		const void* SourceAddr = Stack.MostRecentPropertyAddress;											\
		P_FINISH;																							\
		ConvertType<FLOATVARIANT, DOUBLEVARIANT>(SourceAddr, DestAddr);										\
	}																										\
	UE::Kismet::BlueprintTypeConversions::Internal::FStructConversionEntry BASETYPE##Entry(					\
		&TBaseStructure<BASETYPE>::Get,																		\
		&TVariantStructure<BASETYPE>::Get,																	\
		&TVariantStructure<DOUBLEVARIANT>::Get,																\
		&TVariantStructure<FLOATVARIANT>::Get,																\
		TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(DOUBLEVARIANT, FLOATVARIANT))),						\
		TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FLOATVARIANT, DOUBLEVARIANT))),						\
		&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<DOUBLEVARIANT, FLOATVARIANT>,			\
		&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FLOATVARIANT, DOUBLEVARIANT>			\
	);

namespace UE::Kismet::BlueprintTypeConversions::Internal
{

template <typename TFrom, typename TTo>
FORCEINLINE_DEBUGGABLE void ConvertType(const void* InFromData, void* InToData)
{
	const TFrom* From = reinterpret_cast<const TFrom*>(InFromData);
	check(From);
	TTo* To = reinterpret_cast<TTo*>(InToData);
	check(To);

	// LWC_TODO - Ensure that we're using the correct constructor here.
	// Specifically, we need to use a constructor or function that calls a narrowing cast for safe conversions.
	*To = TTo(*From);
}

ConversionFunctionT FindConversionFunction(const FProperty* InFromProperty, const FProperty* InToProperty)
{
	ConversionFunctionT Result = nullptr;

	if (InFromProperty->IsA<FFloatProperty>() && InToProperty->IsA<FDoubleProperty>())
	{
		Result = &ConvertType<float, double>;
	}
	else if (InFromProperty->IsA<FDoubleProperty>() && InToProperty->IsA<FFloatProperty>())
	{
		Result = &ConvertType<double, float>;
	}
	else if (InFromProperty->IsA<FStructProperty>() && InToProperty->IsA<FStructProperty>())
	{
		const FStructProperty* InFromStructProperty = CastFieldChecked<FStructProperty>(InFromProperty);
		const FStructProperty* InToStructProperty = CastFieldChecked<FStructProperty>(InToProperty);
		const UScriptStruct* InFromStruct = InFromStructProperty->Struct;
		const UScriptStruct* InToStruct = InToStructProperty->Struct;

		// Only core struct types will have a specialized conversion function (eg: FVector)
		if (TOptional<ConversionFunctionPairT> ConversionPair = FStructConversionTable::Get().GetConversionFunction(InFromStruct, InToStruct))
		{
			Result = ConversionPair->Get<0>();
		}
	}

	return Result;
}

struct FStructConversionEntry
{
	using GetUScriptStructFunctionT = UScriptStruct * (*)(void);

	FStructConversionEntry(GetUScriptStructFunctionT InGetBaseStruct,
		GetUScriptStructFunctionT InGetVariantFromBaseStruct,
		GetUScriptStructFunctionT InGetDoubleVariantStruct,
		GetUScriptStructFunctionT InGetFloatVariantStruct,
		const TCHAR* InConvertDoubleToFloatFunctionName,
		const TCHAR* InConvertFloatToDoubleFunctionName,
		UE::Kismet::BlueprintTypeConversions::ConversionFunctionT InConvertDoubleToFloatImpl,
		UE::Kismet::BlueprintTypeConversions::ConversionFunctionT InConvertFloatToDoubleImpl)
		: GetBaseStruct(InGetBaseStruct)
		, GetVariantFromBaseStruct(InGetVariantFromBaseStruct)
		, GetDoubleVariantStruct(InGetDoubleVariantStruct)
		, GetFloatVariantStruct(InGetFloatVariantStruct)
		, ConvertDoubleToFloatFunctionName(InConvertDoubleToFloatFunctionName)
		, ConvertFloatToDoubleFunctionName(InConvertFloatToDoubleFunctionName)
		, ConvertDoubleToFloatImpl(InConvertDoubleToFloatImpl)
		, ConvertFloatToDoubleImpl(InConvertFloatToDoubleImpl)
	{
		NextEntry = FStructConversionEntry::GetListHead();
		FStructConversionEntry::GetListHead() = this;
	}

	GetUScriptStructFunctionT GetBaseStruct;
	GetUScriptStructFunctionT GetVariantFromBaseStruct;
	GetUScriptStructFunctionT GetDoubleVariantStruct;
	GetUScriptStructFunctionT GetFloatVariantStruct;
	const TCHAR* ConvertDoubleToFloatFunctionName;
	const TCHAR* ConvertFloatToDoubleFunctionName;
	UE::Kismet::BlueprintTypeConversions::ConversionFunctionT ConvertDoubleToFloatImpl;
	UE::Kismet::BlueprintTypeConversions::ConversionFunctionT ConvertFloatToDoubleImpl;
	FStructConversionEntry* NextEntry;

	static FStructConversionEntry*& GetListHead()
	{
		static FStructConversionEntry* ListHead = nullptr;
		return ListHead;
	}
};

} // namespace UE::Kismet::BlueprintTypeConversions::Internal

namespace UE::Kismet::BlueprintTypeConversions
{

FStructConversionTable* FStructConversionTable::Instance = nullptr;

FStructConversionTable::FStructConversionTable()
{
	using namespace UE::Kismet::BlueprintTypeConversions;

	UClass* BlueprintTypeConversionsClass = UBlueprintTypeConversions::StaticClass();
	check(BlueprintTypeConversionsClass);

	Internal::FStructConversionEntry* Entry = Internal::FStructConversionEntry::GetListHead();
	while (Entry)
	{
		const UScriptStruct* BaseStruct = Entry->GetBaseStruct();
		const UScriptStruct* VariantFromBaseStruct = Entry->GetVariantFromBaseStruct();
		const UScriptStruct* DoubleVariantStruct = Entry->GetDoubleVariantStruct();
		const UScriptStruct* FloatVariantStruct = Entry->GetFloatVariantStruct();

		// The base and float variant structs *must* exist.
		// However, we don't always have the double variant struct, as is the case with FBox2d and FVector2d.

		check(BaseStruct);
		check(FloatVariantStruct);

		if (VariantFromBaseStruct)
		{
			StructVariantLookupTable.Add(BaseStruct, VariantFromBaseStruct);
		}

		const UScriptStruct* FloatKeyEntry = FloatVariantStruct;
		const UScriptStruct* DoubleKeyEntry = DoubleVariantStruct ? DoubleVariantStruct : BaseStruct;

		StructVariantPairT ConversionKeyPair1 = { DoubleKeyEntry, FloatKeyEntry };
		ConversionFunctionPairT ConversionValuePair1 = 
			{ Entry->ConvertDoubleToFloatImpl,
			  BlueprintTypeConversionsClass->FindFunctionByName(Entry->ConvertDoubleToFloatFunctionName) };

		StructVariantPairT ConversionKeyPair2 = { FloatKeyEntry, DoubleKeyEntry };
		ConversionFunctionPairT ConversionValuePair2 = 
			{ Entry->ConvertFloatToDoubleImpl,
			  BlueprintTypeConversionsClass->FindFunctionByName(Entry->ConvertFloatToDoubleFunctionName) };

		ImplicitCastLookupTable.Add(ConversionKeyPair1, ConversionValuePair1);
		ImplicitCastLookupTable.Add(ConversionKeyPair2, ConversionValuePair2);

		Entry = Entry->NextEntry;
	}
}

const FStructConversionTable& FStructConversionTable::Get()
{
	if (Instance == nullptr)
	{
		Instance = new FStructConversionTable();
	}

	return *Instance;
}

TOptional<ConversionFunctionPairT> FStructConversionTable::GetConversionFunction(const UScriptStruct* InFrom, const UScriptStruct* InTo) const
{
	TOptional<ConversionFunctionPairT> Result;

	StructVariantPairT Key = GetVariantsFromStructs(InFrom, InTo);

	if (const ConversionFunctionPairT* Entry = ImplicitCastLookupTable.Find(Key))
	{
		Result = *Entry;
	}

	return Result;
}

FStructConversionTable::StructVariantPairT FStructConversionTable::GetVariantsFromStructs(const UScriptStruct* InStruct1, const UScriptStruct* InStruct2) const
{
	StructVariantPairT Result;

	if (const UScriptStruct* const* Variant = StructVariantLookupTable.Find(InStruct1))
	{
		Result.Get<0>() = *Variant;
	}
	else
	{
		Result.Get<0>() = InStruct1;
	}

	if (const UScriptStruct* const* Variant = StructVariantLookupTable.Find(InStruct2))
	{
		Result.Get<1>() = *Variant;
	}
	else
	{
		Result.Get<1>() = InStruct2;
	}

	return Result;
}

} // namespace UE::Kismet::BlueprintTypeConversions

UBlueprintTypeConversions::UBlueprintTypeConversions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

// Container conversions

DEFINE_FUNCTION(UBlueprintTypeConversions::execConvertArrayType)
{
	using namespace UE::Kismet;

	FArrayProperty* DestArrayProperty = CastFieldChecked<FArrayProperty>(Stack.MostRecentProperty);
	void* DestArrayAddr = Stack.MostRecentPropertyAddress;

	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn<FArrayProperty>(nullptr);
	const void* SourceArrayAddr = Stack.MostRecentPropertyAddress;
	const FArrayProperty* SourceArrayProperty = CastFieldChecked<FArrayProperty>(Stack.MostRecentProperty);

	P_FINISH;

	FScriptArrayHelper SourceArray(SourceArrayProperty, SourceArrayAddr);
	FScriptArrayHelper DestArray(DestArrayProperty, DestArrayAddr);

	int SourceArraySize = SourceArray.Num();
	DestArray.Resize(SourceArraySize);

	BlueprintTypeConversions::ConversionFunctionT ConversionFunction =
		BlueprintTypeConversions::Internal::FindConversionFunction(SourceArrayProperty->Inner, DestArrayProperty->Inner);
	check(ConversionFunction);

	for (int i = 0; i < SourceArraySize; ++i)
	{
		const void* SrcData = SourceArray.GetRawPtr(i);
		void* DestData = DestArray.GetRawPtr(i);
		(*ConversionFunction)(SrcData, DestData);
	}
}

DEFINE_FUNCTION(UBlueprintTypeConversions::execConvertSetType)
{
	using namespace UE::Kismet;

	FSetProperty* DestSetProperty = CastFieldChecked<FSetProperty>(Stack.MostRecentProperty);
	void* DestSetAddr = Stack.MostRecentPropertyAddress;

	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn<FSetProperty>(nullptr);
	const void* SourceSetAddr = Stack.MostRecentPropertyAddress;
	const FSetProperty* SourceSetProperty = CastFieldChecked<FSetProperty>(Stack.MostRecentProperty);

	P_FINISH;

	FScriptSetHelper SourceSet(SourceSetProperty, SourceSetAddr);
	FScriptSetHelper DestSet(DestSetProperty, DestSetAddr);

	DestSet.EmptyElements(SourceSet.Num());

	BlueprintTypeConversions::ConversionFunctionT ConversionFunction = 
		BlueprintTypeConversions::Internal::FindConversionFunction(SourceSetProperty->ElementProp, DestSetProperty->ElementProp);
	check(ConversionFunction);

	for (FScriptSetHelper::FIterator It(SourceSet); It; ++It)
	{
		const void* SrcData = SourceSet.GetElementPtr(It);
		const int32 NewIndex = DestSet.AddDefaultValue_Invalid_NeedsRehash();
		void* DestData = DestSet.GetElementPtr(NewIndex);
		(*ConversionFunction)(SrcData, DestData);
	}

	DestSet.Rehash();
}

DEFINE_FUNCTION(UBlueprintTypeConversions::execConvertMapType)
{
	using namespace UE::Kismet;

	FMapProperty* DestMapProperty = CastFieldChecked<FMapProperty>(Stack.MostRecentProperty);
	void* DestMapAddr = Stack.MostRecentPropertyAddress;

	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn<FMapProperty>(nullptr);
	const void* SourceMapAddr = Stack.MostRecentPropertyAddress;
	const FMapProperty* SourceMapProperty = CastFieldChecked<FMapProperty>(Stack.MostRecentProperty);

	P_FINISH;

	FScriptMapHelper SourceMap(SourceMapProperty, SourceMapAddr);
	FScriptMapHelper DestMap(DestMapProperty, DestMapAddr);

	DestMap.EmptyValues(SourceMap.Num());

	BlueprintTypeConversions::ConversionFunctionT KeyConversionFunction = 
		BlueprintTypeConversions::Internal::FindConversionFunction(SourceMapProperty->KeyProp, DestMapProperty->KeyProp);
	BlueprintTypeConversions::ConversionFunctionT ValueConversionFunction = 
		BlueprintTypeConversions::Internal::FindConversionFunction(SourceMapProperty->ValueProp, DestMapProperty->ValueProp);

	for (FScriptMapHelper::FIterator It(SourceMap); It; ++It)
	{
		const int32 NewIndex = DestMap.AddDefaultValue_Invalid_NeedsRehash();

		const void* SourceKeyRawData = SourceMap.GetKeyPtr(It);
		void* DestinationKeyRawData = DestMap.GetKeyPtr(NewIndex);

		if (KeyConversionFunction)
		{
			(*KeyConversionFunction)(SourceKeyRawData, DestinationKeyRawData);
		}
		else
		{
			SourceMapProperty->KeyProp->CopySingleValue(DestinationKeyRawData, SourceKeyRawData);
		}

		const void* SourceValueRawData = SourceMap.GetValuePtr(It);
		void* DestinationValueRawData = DestMap.GetValuePtr(NewIndex);

		if (ValueConversionFunction)
		{
			(*ValueConversionFunction)(SourceValueRawData, DestinationValueRawData);
		}
		else
		{
			SourceMapProperty->ValueProp->CopySingleValue(DestinationValueRawData, SourceValueRawData);
		}
	}

	DestMap.Rehash();
}

// Custom struct conversions

DEFINE_CONVERSION_FUNCTIONS(FVector, FVector3d, FVector3f)
DEFINE_CONVERSION_FUNCTIONS(FVector4, FVector4d, FVector4f)
DEFINE_CONVERSION_FUNCTIONS(FPlane, FPlane4d, FPlane4f)
DEFINE_CONVERSION_FUNCTIONS(FQuat, FQuat4d, FQuat4f)
DEFINE_CONVERSION_FUNCTIONS(FRotator, FRotator3d, FRotator3f)
DEFINE_CONVERSION_FUNCTIONS(FTransform, FTransform3d, FTransform3f)
DEFINE_CONVERSION_FUNCTIONS(FMatrix, FMatrix44d, FMatrix44f)

// LWC_TODO - This is a bit of a hack, but FBox2d and FVector2d don't have script structs due to FName collisions with their base type.
// So we need to add our own expanded version of DEFINE_CONVERSION_FUNCTIONS that returns a nullptr for the variant script structs.

UScriptStruct* ReturnNullVariant()
{
	return nullptr;
}

DEFINE_FUNCTION(UBlueprintTypeConversions::MAKE_CONVERSION_EXEC_FUNCTION_NAME(FBox2d, FBox2f))
{
	using namespace UE::Kismet::BlueprintTypeConversions::Internal;

	void* DestAddr = Stack.MostRecentPropertyAddress;
	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn(nullptr, nullptr);
	const void* SourceAddr = Stack.MostRecentPropertyAddress;
	P_FINISH;
	ConvertType<FBox2d, FBox2f>(SourceAddr, DestAddr);
}

DEFINE_FUNCTION(UBlueprintTypeConversions::MAKE_CONVERSION_EXEC_FUNCTION_NAME(FBox2f, FBox2d))
{
	using namespace UE::Kismet::BlueprintTypeConversions::Internal;

	void* DestAddr = Stack.MostRecentPropertyAddress;
	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn(nullptr, nullptr);
	const void* SourceAddr = Stack.MostRecentPropertyAddress;
	P_FINISH;
	ConvertType<FBox2f, FBox2d>(SourceAddr, DestAddr);
}

UE::Kismet::BlueprintTypeConversions::Internal::FStructConversionEntry FBox2DEntry(
	&TBaseStructure<FBox2D>::Get,
	&ReturnNullVariant,
	&ReturnNullVariant,
	&TVariantStructure<FBox2f>::Get,
	TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FBox2d, FBox2f))),
	TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FBox2f, FBox2d))),
	&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FBox2d, FBox2f>,
	&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FBox2f, FBox2d>
);

DEFINE_FUNCTION(UBlueprintTypeConversions::MAKE_CONVERSION_EXEC_FUNCTION_NAME(FVector2d, FVector2f))
{
	using namespace UE::Kismet::BlueprintTypeConversions::Internal;

	void* DestAddr = Stack.MostRecentPropertyAddress;
	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn(nullptr, nullptr);
	const void* SourceAddr = Stack.MostRecentPropertyAddress;
	P_FINISH;
	ConvertType<FVector2d, FVector2f>(SourceAddr, DestAddr);
}

DEFINE_FUNCTION(UBlueprintTypeConversions::MAKE_CONVERSION_EXEC_FUNCTION_NAME(FVector2f, FVector2d))
{
	using namespace UE::Kismet::BlueprintTypeConversions::Internal;

	void* DestAddr = Stack.MostRecentPropertyAddress;
	Stack.MostRecentProperty = nullptr;
	Stack.StepCompiledIn(nullptr, nullptr);
	const void* SourceAddr = Stack.MostRecentPropertyAddress;
	P_FINISH;
	ConvertType<FVector2f, FVector2d>(SourceAddr, DestAddr);
}

UE::Kismet::BlueprintTypeConversions::Internal::FStructConversionEntry FVector2DEntry(
	&TBaseStructure<FVector2D>::Get,
	&ReturnNullVariant,
	&ReturnNullVariant,
	&TVariantStructure<FVector2f>::Get,
	TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FVector2d, FVector2f))),
	TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FVector2f, FVector2d))),
	&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FVector2d, FVector2f>,
	&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FVector2f, FVector2d>
);

UE::Kismet::BlueprintTypeConversions::Internal::FStructConversionEntry FDeprecateVector2DEntry(
	&FDeprecateSlateVector2D::StaticStruct,
	&TVariantStructure<FVector2f>::Get,
	&TBaseStructure<FVector2D>::Get,
	&TVariantStructure<FVector2f>::Get,
	TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FVector2d, FVector2f))),
	TEXT(UE_STRINGIZE(MAKE_CONVERSION_FUNCTION_NAME(FVector2f, FVector2d))),
	&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FVector2d, FVector2f>,
	&UE::Kismet::BlueprintTypeConversions::Internal::ConvertType<FVector2f, FVector2d>
);