// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/ImportanceSamplingLibrary.h"
#include "Math/Sobol.h"
#include "Engine/Texture2D.h"
#include "TextureCompiler.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportanceSamplingLibrary)

#define LOCTEXT_NAMESPACE "ImportanceSamplingLibrary"

// when to switch from binary to linear search
// branch prediction makes linear search faster for small sizes
// set to 1 to use binary search all the way down
#define BINARY_SEARCH_LIMIT 64

// max MIP size to store and use for texture calculation
// largest MarginalCDF entry is the sum of all texel probabilities.
// per-texel PDF precision is then 24-bit float mantissa - 2*(mips-1)
// for 1024x1024 with 2^20 texels and 11 mips, that's 24-20 = 4 bits of probability precision
#define MAX_MIP_LEVELS 11

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/* Default constructor, must Initialize before use */
FImportanceTexture::FImportanceTexture() : Size(ForceInitToZero), NumMips(0), Texture(0), Weighting(EImportanceWeight::Luminance) {}

/* Constructor with initialization */
FImportanceTexture::FImportanceTexture(UTexture2D *SourceTexture, TEnumAsByte<EImportanceWeight::Type> WeightingFunc)
{
	Initialize(SourceTexture, WeightingFunc);
}

FImportanceTexture::FImportanceTexture(const FImportanceTexture& Other) = default;
FImportanceTexture::FImportanceTexture(FImportanceTexture&& Other) = default;
FImportanceTexture& FImportanceTexture::operator=(const FImportanceTexture& Other) = default;
FImportanceTexture& FImportanceTexture::operator=(FImportanceTexture&& Other) = default;

PRAGMA_ENABLE_DEPRECATION_WARNINGS

FLinearColor FImportanceTexture::GetColorTrilinear(FVector2D Position, float Mip) const
{
	float IntMip = FMath::FloorToFloat(Mip);
	float MipBlend = Mip - IntMip;
	FLinearColor Color0 = GetColorBilinear(Position, IntMip);
	FLinearColor Color1 = GetColorBilinear(Position, IntMip + 1.f);
	return FMath::Lerp(Color0, Color1, MipBlend);
}

FLinearColor FImportanceTexture::GetColorBilinear(FVector2D Position, int32 Mip) const
{
	if (LinearTextureData.IsEmpty())
	{
		return FLinearColor(EForceInit());
	}
	int32 iMip = FMath::Clamp(Mip, 0, NumMips - 1);
	FIntPoint MipSize(((Size.X - 1) >> iMip) + 1, ((Size.Y - 1) >> iMip) + 1);
	size_t LevelStart = 4 * (Size.X * Size.Y - MipSize.X * MipSize.Y) / 3;

	FVector2D TexelPos = Position * FVector2D(MipSize - FIntPoint(1,1));
	FIntPoint IntPos(
		FMath::Clamp(FMath::FloorToInt(TexelPos.X), 0, MipSize.X - 1),
		FMath::Clamp(FMath::FloorToInt(TexelPos.Y), 0, MipSize.Y - 1));
	FVector2D TexelBlend = TexelPos - FVector2D(IntPos);

	// At bottom MIP, return single texel
	size_t TextureOffset = LevelStart + IntPos.Y * MipSize.X + IntPos.X;
	FLinearColor Color00(LinearTextureData[TextureOffset]);
	if (MipSize.X == 1 || MipSize.Y == 1) return Color00;
	
	// MIP texel blending should be in linear space, so this includes conversions to/from sRGB
	FLinearColor Color01(LinearTextureData[TextureOffset + MipSize.X]);
	FLinearColor Color10(LinearTextureData[TextureOffset + 1]);
	FLinearColor Color11(LinearTextureData[TextureOffset + MipSize.X + 1]);
	return FMath::Lerp(
		FMath::Lerp(Color00, Color10, TexelBlend.X), 
		FMath::Lerp(Color01, Color11, TexelBlend.X), TexelBlend.Y);
}

static float ImportanceWeightLinear(FLinearColor LinearTexel, TEnumAsByte<EImportanceWeight::Type> WeightingFunc)
{
	switch (WeightingFunc) {
	case EImportanceWeight::Luminance:	return LinearTexel.GetLuminance();
	case EImportanceWeight::Red:		return LinearTexel.R;
	case EImportanceWeight::Green:		return LinearTexel.G;
	case EImportanceWeight::Blue:		return LinearTexel.B;
	case EImportanceWeight::Alpha:		return LinearTexel.A;
	}
	return 1.f;
}

float FImportanceTexture::ImportanceWeight(FColor Texel, TEnumAsByte<EImportanceWeight::Type> WeightingFunc) const
{
	FLinearColor LinearTexel = (Texture.IsValid() && Texture->SRGB) ? FLinearColor(Texel) : Texel.ReinterpretAsLinear();
	return ImportanceWeightLinear(LinearTexel, WeightingFunc);
}



template <typename ColorType>
static void FetchTextureData(TArray<FLinearColor>& LinearTextureData, FIntPoint& Size, int& NumMips, UTexture2D* SourceTexture)
{
	// Save a copy of all MIP data for later color lookups
	// Doing GetMipData for each sample would allocate and copy the entire MIP chain for each access
	TArray<ColorType*> MipData;
	int32 SourceMips = SourceTexture->GetNumMips();
	int32 FirstMip = FMath::Max(0, SourceMips - MAX_MIP_LEVELS);
	NumMips = SourceMips - FirstMip;
	MipData.AddZeroed(NumMips);
	// this gets the PlatformData mip data, not the Source:
	SourceTexture->GetMipData(FirstMip, reinterpret_cast<void**>(MipData.GetData()));

	// Copy just the needed MIP data and adjust size
	// todo: why is it being copied? just use MipData
	FIntPoint SrcSize = FIntPoint(SourceTexture->GetSizeX(), SourceTexture->GetSizeY());
	// todo: this mip size computation looks wrong
	Size = FIntPoint(((SrcSize.X - 1) >> FirstMip) + 1, ((SrcSize.Y - 1) >> FirstMip) + 1);
	FIntPoint LastMipSize(((Size.X - 1) >> (NumMips - 1)) + 1, ((Size.Y - 1) >> (NumMips - 1)) + 1);
	size_t MipDataSize = (4 * Size.X * Size.Y - LastMipSize.X * LastMipSize.Y) / 3;
	LinearTextureData.SetNumUninitialized(MipDataSize);
	for (int32 mip = 0; mip < NumMips; ++mip) {
		FIntPoint LevelSize(((Size.X - 1) >> mip) + 1, ((Size.Y - 1) >> mip) + 1);
		size_t LevelStart = (Size.X * Size.Y - LevelSize.X * LevelSize.Y) * 4 / 3;
		for (int Index = 0, NumPixels = LevelSize.X * LevelSize.Y; Index < NumPixels; Index++)
		{
			ColorType Texel = MipData[mip][Index];
			if constexpr(std::is_same_v<ColorType, FColor>)
			{
				// in RGBA8 case, we should include the possibility of non-SRGB textures
				LinearTextureData[LevelStart + Index] = SourceTexture->SRGB ? FLinearColor(Texel) : Texel.ReinterpretAsLinear();
			}
			else
			{
				LinearTextureData[LevelStart + Index] = FLinearColor(Texel);
			}
			
		}
	}

	// get rid of temporary copy of MIP data
	for (auto MipLevel : MipData)
	{
		FMemory::Free(MipLevel);
	}
}

void FImportanceTexture::Initialize(UTexture2D *SourceTexture, TEnumAsByte<EImportanceWeight::Type> WeightingFunc)
{
#if WITH_EDITOR
	// For now, this doesn't support being refreshed once the source texture compilation is done.
	// So force it to finish its compilation if it is not yet done to avoid getting the default's texture data.
	if (SourceTexture && SourceTexture->IsCompiling())
	{
		FTextureCompilingManager::Get().FinishCompilation({ SourceTexture });
	}
#endif

	// after this, safe to re-initialize
	Texture = SourceTexture;
	Weighting = WeightingFunc;

	if (!SourceTexture)
	{
		FFrame::KismetExecutionMessage(TEXT("Importance Texture is missing a source texture"), ELogVerbosity::Error);
		return;
	}

	switch (SourceTexture->GetPixelFormat())
	{
		case EPixelFormat::PF_B8G8R8A8:
			FetchTextureData<FColor>(LinearTextureData, Size, NumMips, SourceTexture);
			break;
		case EPixelFormat::PF_FloatRGBA:
			FetchTextureData<FFloat16Color>(LinearTextureData, Size, NumMips, SourceTexture);
			break;
		case EPixelFormat::PF_A32B32G32R32F:
			FetchTextureData<FLinearColor>(LinearTextureData, Size, NumMips, SourceTexture);
			break;
		default:
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Importance Texture does not support texture format %d"), SourceTexture->GetPixelFormat()), ELogVerbosity::Error);
			Texture = nullptr; // Reset pointer to we don't try to sample from this texture
			return;
	}

	// accumulate un-normalized marginal CDF for image, and conditional CDF for each row
	MarginalCDF.SetNumUninitialized(Size.Y + 1);
	ConditionalCDF.SetNumUninitialized((Size.X + 1) * Size.Y);
	MarginalCDF[0] = 0.f;

	for (int y = 1; y <= Size.Y; ++y)
	{
		// Accumulate along row

		FLinearColor *ColorRow = &LinearTextureData[(y - 1) * Size.X];
		float *CDFRow = &ConditionalCDF[(y - 1) * (Size.X + 1)];
		CDFRow[0] = 0.f;
		for (int x = 1; x <= Size.X; ++x)
		{
			CDFRow[x] = CDFRow[x - 1] + ImportanceWeightLinear(ColorRow[x - 1], WeightingFunc);
		}

		// add row total to global total
		MarginalCDF[y] = MarginalCDF[y - 1] + CDFRow[Size.X];
	}
}

void FImportanceTexture::ImportanceSample(const FVector2D & Rand, int Samples, float Intensity, FVector2D & SamplePosition, FLinearColor & SampleColor, float & SampleIntensity, float & SampleSize) const
{
	if (!Texture.IsValid())
	{
		return;
	}

	// find a row: binary search then linear
	float YRand = MarginalCDF[Size.Y] * FMath::Frac(Rand.Y);	// 0 <= YRand < PDFTotal normalization factor
	FIntPoint YTexel(0, Size.Y);	// range: X=low, Y=high
	while (YTexel.Y - YTexel.X > BINARY_SEARCH_LIMIT)
	{
		int32 Mid = YTexel.X + ((YTexel.Y - YTexel.X) >> 1);
		YTexel = (MarginalCDF[Mid] < YRand) ? FIntPoint(Mid, YTexel.Y) : FIntPoint(YTexel.X, Mid);
	}
	while (YTexel.X < YTexel.Y  &&  MarginalCDF[YTexel.X + 1] < YRand)
	{
		++YTexel.X;
	}
	YTexel.Y = YTexel.X + 1;

	// find a column
	const float *CDFRow = &ConditionalCDF[(Size.X + 1) * YTexel.X];
	float XRand = CDFRow[Size.X] * FMath::Frac(Rand.X);	// 0 <= XRand < row total
	FIntPoint XTexel(0, Size.X);	// range: X=low, Y=high
	while (XTexel.Y - XTexel.X > BINARY_SEARCH_LIMIT)
	{
		int32 Mid = XTexel.X + ((XTexel.Y - XTexel.X) >> 1);
		XTexel = (CDFRow[Mid] < XRand) ? FIntPoint(Mid, XTexel.Y) : FIntPoint(XTexel.X, Mid);
	}
	while (XTexel.X < XTexel.Y  &&  CDFRow[XTexel.X + 1] < XRand)
	{
		++XTexel.X;
	}
	XTexel.Y = XTexel.X + 1;

	// Final position
	FVector2D IntervalStart(CDFRow[XTexel.X], MarginalCDF[YTexel.X]);
	FVector2D IntervalEnd(CDFRow[XTexel.Y], MarginalCDF[YTexel.Y]);
	FVector2D Interval = IntervalEnd - IntervalStart;
	FVector2D TexelRand = (FVector2D(XRand, YRand) - IntervalStart) / Interval;
	SamplePosition = (FVector2D(float(XTexel.X), float(YTexel.X)) + TexelRand) / FVector2D(Size);

	// Final scaled probability density, scaled by Jacobian of mapping from unit square to texels (aka texture size) and PDF total normalization
	float Jacobian = Size.X * Size.Y / MarginalCDF[Size.Y];
	float Probability = Interval.X * Jacobian;

	// find size scaled by number of samples and probability
	float Scale = 1.f / (float(Samples) * Probability);
	SampleSize = 4.f * FMath::Sqrt(0.5f * Scale);

	// clamp size and position to fit inside texture
	//SampleSize = FMath::Min3(SampleSize, 4.f * SamplePosition.X, 4.f - 4.f * SamplePosition.X);
	//SampleSize = FMath::Min3(SampleSize, 4.f * SamplePosition.Y, 4.f - 4.f * SamplePosition.Y);

	// Color from mip texture, not normalized for total intensity so colors match texture.
	// Need to use SampleColor * SampleIntensity to get expected total color
	float MipLevel = 0.5 * FMath::Log2(Size.X * Size.Y * Scale);
	SampleColor = GetColorTrilinear(SamplePosition, MipLevel);
	SampleIntensity = Intensity * Scale * Jacobian;
}

UImportanceSamplingLibrary::UImportanceSamplingLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

float UImportanceSamplingLibrary::RandomSobolFloat(int32 Index, int32 Dimension, float Seed)
{
	Dimension = FMath::Clamp<int32>(Dimension, 0, FSobol::MaxDimension);
	return FSobol::Evaluate(Index, Dimension, int32(Seed * 0x1000000));
}

float UImportanceSamplingLibrary::NextSobolFloat(int32 Index, int32 Dimension, float Value)
{
	Dimension = FMath::Clamp<int32>(Dimension, 0, FSobol::MaxDimension);
	return FSobol::Next(Index, Dimension, Value);
}

FVector2D UImportanceSamplingLibrary::RandomSobolCell2D(int32 Index, int32 NumCells, FVector2D Cell, FVector2D Seed)
{
	int32 CellBits = FMath::Clamp<int32>(FGenericPlatformMath::CeilLogTwo(NumCells), 0, FSobol::MaxCell2DBits);
	return FSobol::Evaluate(Index, CellBits, Cell.IntPoint(), (Seed * 0x1000000).IntPoint());
}

FVector2D UImportanceSamplingLibrary::NextSobolCell2D(int32 Index, int32 NumCells, FVector2D Value)
{
	int32 CellBits = FMath::Clamp<int32>(FGenericPlatformMath::CeilLogTwo(NumCells), 0, FSobol::MaxCell2DBits);
	return FSobol::Next(Index, CellBits, Value);
}

FVector UImportanceSamplingLibrary::RandomSobolCell3D(int32 Index, int32 NumCells, FVector Cell, FVector Seed)
{
	int32 CellBits = FMath::Clamp<int32>(FGenericPlatformMath::CeilLogTwo(NumCells), 0, FSobol::MaxCell3DBits);
	FIntVector ICell = FIntVector(int32(Cell.X), int32(Cell.Y), int32(Cell.Z));
	FIntVector ISeed = FIntVector(int32(Seed.X * 0x1000000), int32(Seed.Y * 0x1000000), int32(Seed.Z * 0x1000000));
	return FSobol::Evaluate(Index, CellBits, ICell, ISeed);
}

FVector UImportanceSamplingLibrary::NextSobolCell3D(int32 Index, int32 NumCells, FVector Value)
{
	int32 CellBits = FMath::Clamp<int32>(FGenericPlatformMath::CeilLogTwo(NumCells), 0, FSobol::MaxCell3DBits);
	return FSobol::Next(Index, CellBits, Value);
}

FImportanceTexture UImportanceSamplingLibrary::MakeImportanceTexture(UTexture2D *SourceTexture, TEnumAsByte<EImportanceWeight::Type> WeightingFunc)
{
	return FImportanceTexture(SourceTexture, WeightingFunc);
}

void UImportanceSamplingLibrary::BreakImportanceTexture(const FImportanceTexture &ImportanceTexture, UTexture2D *&Texture, TEnumAsByte<EImportanceWeight::Type> &WeightingFunc)
{
	Texture = ImportanceTexture.Texture.Get();
	WeightingFunc = ImportanceTexture.Weighting;
}

void UImportanceSamplingLibrary::ImportanceSample(const FImportanceTexture &Texture, const FVector2D &Rand, int Samples, float Intensity,
	FVector2D &SamplePosition, FLinearColor &SampleColor, float &SampleIntensity, float &SampleSize)
{
	Texture.ImportanceSample(Rand, Samples, Intensity, SamplePosition, SampleColor, SampleIntensity, SampleSize);
}

#undef LOCTEXT_NAMESPACE
#undef BINARY_SEARCH_LIMIT
#undef MAX_MIP_LEVELS

