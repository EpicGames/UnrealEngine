// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetArrayLibrary.h"
#include "Kismet/BlueprintPropertyHelpers.h"
#include "GameFramework/Actor.h"
#include "Blueprint/BlueprintSupport.h"
#include "Math/RandomStream.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetArrayLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogArray, Warning, All);

#define LOCTEXT_NAMESPACE "UKismetArrayLibrary"

//////////////////////////////////////////////////////////////////////////
// UKismetArrayLibrary

const FName GetOutOfBoundsWarning = FName("GetOutOfBoundsWarning");
const FName SetOutOfBoundsWarning = FName("SetOutOfBoundsWarning");
const FName InsertOutOfBoundsWarning = FName("InsertOutOfBoundsWarning");
const FName RemoveOutOfBoundsWarning = FName("RemoveOutOfBoundsWarning");
const FName ResizeArrayNegativeWarning = FName("ResizeArrayNegativeWarning");
const FName SwapElementsInArrayWarning = FName("SwapElementsInArrayWarning");
const FName RandomAccessToEmptyArrayWarning = FName("RandomAccessToEmptyArrayWarning");

const FName UKismetArrayLibrary::ReachedMaximumContainerSizeWarning = FName("ReachedMaximumContainerSizeWarning");

namespace UE::KismetArray::Private
{
	template<class T, typename InLessThanFunc = TLess<T>, typename InGreaterThanFunc = TGreater<T>>
	void SortHelper(
		TArray<T>& InArray,
		bool bStableSort,
		EArraySortOrder SortOrder,
		InLessThanFunc&& LessThanFunc = {},
		InGreaterThanFunc&& GreaterThanFunc = {})
	{
		if (!bStableSort)
		{
			if (SortOrder == EArraySortOrder::Ascending)
			{
				InArray.Sort(Forward<InLessThanFunc>(LessThanFunc));
			}
			else
			{
				InArray.Sort(Forward<InGreaterThanFunc>(GreaterThanFunc));
			}
		}
		else
		{
			if (SortOrder == EArraySortOrder::Ascending)
			{
				InArray.StableSort(Forward<InLessThanFunc>(LessThanFunc));
			}
			else
			{
				InArray.StableSort(Forward<InGreaterThanFunc>(GreaterThanFunc));
			}
		}
	}
}

UKismetArrayLibrary::UKismetArrayLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			GetOutOfBoundsWarning,
			LOCTEXT("GetOutOfBoundsWarning", "Array read access out of bounds")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			SetOutOfBoundsWarning,
			LOCTEXT("SetOutOfBoundsWarning", "Array assignment access out of bounds")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			InsertOutOfBoundsWarning,
			LOCTEXT("InsertOutOfBoundsWarning", "Array insertion out of bounds")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			RemoveOutOfBoundsWarning,
			LOCTEXT("RemoveOutOfBoundsWarning", "Array removal out of bounds")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			ResizeArrayNegativeWarning,
			LOCTEXT("ResizeArrayNegativeWarning", "Array resized to negative size")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			SwapElementsInArrayWarning,
			LOCTEXT("SwapElementsInArrayWarning", "Array swap access out of bounds")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			RandomAccessToEmptyArrayWarning,
			LOCTEXT("RandomAccessToEmptyArrayWarning", "Random access to empty array")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			ReachedMaximumContainerSizeWarning,
			LOCTEXT("ReachedMaximumContainerSizeWarning", "Container has reached maximum size")
		)
	);
}

void UKismetArrayLibrary::FilterArray(const TArray<AActor*>& TargetArray, TSubclassOf<class AActor> FilterClass, TArray<AActor*>& FilteredArray)
{
	FilteredArray.Empty();
	for (auto It = TargetArray.CreateConstIterator(); It; It++)
	{
		AActor* TargetElement = (*It);
		if (TargetElement && TargetElement->IsA(FilterClass))
		{
			FilteredArray.Add(TargetElement);
		}
	}
}

void UKismetArrayLibrary::SortStringArray(TArray<FString>& TargetArray, bool bStableSort, EArraySortOrder SortOrder)
{
	UE::KismetArray::Private::SortHelper(TargetArray, bStableSort, SortOrder);
}

void UKismetArrayLibrary::SortNameArray(TArray<FName>& TargetArray, bool bStableSort, bool bLexicalSort, EArraySortOrder SortOrder)
{
	const auto LessThan = [bLexicalSort](const FName& A, const FName& B)
	{
		return bLexicalSort ? FNameLexicalLess{}(A, B) : FNameFastLess{}(A, B);
	};

	const auto GreaterThan = [bLexicalSort](const FName& A, const FName& B)
	{
		return bLexicalSort ? FNameLexicalLess{}(B, A) : FNameFastLess{}(B, A);
	};

	UE::KismetArray::Private::SortHelper(TargetArray, bStableSort, SortOrder, LessThan, GreaterThan);
}

void UKismetArrayLibrary::SortByteArray(TArray<uint8>& TargetArray, bool bStableSort, EArraySortOrder SortOrder)
{
	UE::KismetArray::Private::SortHelper(TargetArray, bStableSort, SortOrder);
}

void UKismetArrayLibrary::SortIntArray(TArray<int32>& TargetArray, bool bStableSort, EArraySortOrder SortOrder)
{
	UE::KismetArray::Private::SortHelper(TargetArray, bStableSort, SortOrder);
}

void UKismetArrayLibrary::SortInt64Array(TArray<int64>& TargetArray, bool bStableSort, EArraySortOrder SortOrder)
{
	UE::KismetArray::Private::SortHelper(TargetArray, bStableSort, SortOrder);
}

void UKismetArrayLibrary::SortFloatArray(TArray<double>& TargetArray, bool bStableSort, EArraySortOrder SortOrder)
{
	UE::KismetArray::Private::SortHelper(TargetArray, bStableSort, SortOrder);
}

int32 UKismetArrayLibrary::GenericArray_Add(void* TargetArray, const FArrayProperty* ArrayProp, const void* NewItem)
{
	int32 NewIndex = INDEX_NONE;
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		if (ArrayHelper.Num() < MaxSupportedArraySize)
		{
			FProperty* InnerProp = ArrayProp->Inner;

			NewIndex = ArrayHelper.AddValue();
			InnerProp->CopySingleValueToScriptVM(ArrayHelper.GetRawPtr(NewIndex), NewItem);
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted add to array '%s' beyond the maximum supported capacity!"), *ArrayProp->GetName()), ELogVerbosity::Warning, ReachedMaximumContainerSizeWarning);
		}
	}
	return NewIndex;
}

int32 UKismetArrayLibrary::GenericArray_AddUnique(void* TargetArray, const FArrayProperty* ArrayProp, const void* NewItem)
{
	int32 NewIndex = INDEX_NONE;
	if (TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		FProperty* InnerProp = ArrayProp->Inner;

		if (GenericArray_Find(TargetArray, ArrayProp, NewItem) == INDEX_NONE)
		{
			if (ArrayHelper.Num() < MaxSupportedArraySize)
			{
				NewIndex = ArrayHelper.AddValue();
				InnerProp->CopySingleValueToScriptVM(ArrayHelper.GetRawPtr(NewIndex), NewItem);
			}
			else
			{
				FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted add to array '%s' beyond the maximum supported capacity!"), *ArrayProp->GetName()), ELogVerbosity::Warning, ReachedMaximumContainerSizeWarning);
			}
		}
	}
	return NewIndex;
}

bool UKismetArrayLibrary::GenericArray_Identical(void* ArrayA, const FArrayProperty* ArrayAProp, void* ArrayB, const FArrayProperty* ArrayBProp)
{
	if (ArrayA && ArrayB)
	{
		FProperty* InnerAProp = ArrayAProp->Inner;

		if (InnerAProp->SameType(ArrayBProp->Inner))
		{
			FScriptArrayHelper ArrayAHelper(ArrayAProp, ArrayA);
			FScriptArrayHelper ArrayBHelper(ArrayBProp, ArrayB);

			const int32 ArrayANum = ArrayAHelper.Num();
			if (ArrayANum != ArrayBHelper.Num())
			{
				return false;
			}

			for (int32 Index = 0; Index < ArrayANum; ++Index)
			{
				if (!InnerAProp->Identical(ArrayAHelper.GetRawPtr(Index), ArrayBHelper.GetRawPtr(Index)))
				{
					return false;
				}
			}

			return true;
		}
	}

	return false;
}

void UKismetArrayLibrary::GenericArray_Append(void* TargetArray, const FArrayProperty* TargetArrayProp, void* SourceArray, const FArrayProperty* SourceArrayProperty)
{
	if(TargetArray && SourceArray)
	{
		FScriptArrayHelper TargetArrayHelper(TargetArrayProp, TargetArray);
		FScriptArrayHelper SourceArrayHelper(SourceArrayProperty, SourceArray);

		if(SourceArrayHelper.Num() > 0)
		{
			FProperty* InnerProp = TargetArrayProp->Inner;

			// Determine the maximum index value that we can store, and compute the difference between that and the index of the
			// last element in the target array. If the difference is smaller than the size of the source array, then we'll only
			// add that number of values (i.e. until we reach the maximum supported size). Otherwise, we append the full source array.
			const int32 MaxIdx = MaxSupportedArraySize - 1;
			const int32 AddNum = FMath::Min(SourceArrayHelper.Num(), MaxIdx - GetLastIndex(TargetArrayHelper));
			if (AddNum > 0)
			{
				int32 StartIdx = TargetArrayHelper.AddValues(AddNum);
				for (int32 x = 0; x < AddNum; ++x, ++StartIdx)
				{
					InnerProp->CopySingleValueToScriptVM(TargetArrayHelper.GetRawPtr(StartIdx), SourceArrayHelper.GetRawPtr(x));
				}
			}

			// Warn if we didn't add all the values from the source array (due to having reached our maximum supported size).
			if (AddNum < SourceArrayHelper.Num())
			{
				FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted append to array '%s' beyond the maximum supported capacity!"), *TargetArrayProp->GetName()), ELogVerbosity::Warning, ReachedMaximumContainerSizeWarning);
			}
		}
	}
}

void UKismetArrayLibrary::GenericArray_Insert(void* TargetArray, const FArrayProperty* ArrayProp, const void* NewItem, int32 Index)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		FProperty* InnerProp = ArrayProp->Inner;

		if (ArrayHelper.IsValidIndex(Index)
			|| (Index >= 0 && Index <= ArrayHelper.Num()) )
		{
			ArrayHelper.InsertValues(Index, 1);
			InnerProp->CopySingleValueToScriptVM(ArrayHelper.GetRawPtr(Index), NewItem);
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to insert an item into array %s out of bounds [%d/%d]!"), *ArrayProp->GetName(), Index, GetLastIndex(ArrayHelper)), ELogVerbosity::Warning, InsertOutOfBoundsWarning);
		}
	}
}

void UKismetArrayLibrary::GenericArray_Remove(void* TargetArray, const FArrayProperty* ArrayProp, int32 IndexToRemove)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		if( ArrayHelper.IsValidIndex(IndexToRemove) )
		{
			ArrayHelper.RemoveValues(IndexToRemove, 1);
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to remove an item from an invalid index from array %s [%d/%d]!"), *ArrayProp->GetName(), IndexToRemove, GetLastIndex(ArrayHelper)), ELogVerbosity::Warning, RemoveOutOfBoundsWarning);
		}
	}
}


bool UKismetArrayLibrary::GenericArray_RemoveItem(void* TargetArray, const FArrayProperty* ArrayProp, const void* Item)
{
	bool bRemoved = false;

	if( TargetArray )
	{
		int32 IndexToRemove = GenericArray_Find(TargetArray, ArrayProp, Item);
		while(IndexToRemove != INDEX_NONE)
		{
			GenericArray_Remove(TargetArray, ArrayProp, IndexToRemove);
			bRemoved = true; //removed

			// See if there is another in the array
			IndexToRemove = GenericArray_Find(TargetArray, ArrayProp, Item);
		}
	}

	return bRemoved;
}

void UKismetArrayLibrary::GenericArray_Shuffle(void* TargetArray, const FArrayProperty* ArrayProp)
{
	if (TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		int32 LastIndex = ArrayHelper.Num() - 1;
		for (int32 i = 0; i <= LastIndex; ++i)
		{
			int32 Index = FMath::RandRange(i, LastIndex);
			if (i != Index)
			{
				ArrayHelper.SwapValues(i, Index);
			}
		}
	}
}

void UKismetArrayLibrary::GenericArray_ShuffleFromStream(void* TargetArray, const FArrayProperty* ArrayProp, FRandomStream* RandomStream)
{
	if (TargetArray && RandomStream)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		int32 LastIndex = ArrayHelper.Num() - 1;
		for (int32 i = 0; i <= LastIndex; ++i)
		{
			const int32 Index = RandomStream->RandRange(i, LastIndex);
			if (i != Index)
			{
				ArrayHelper.SwapValues(i, Index);
			}
		}
	}
}

void UKismetArrayLibrary::GenericArray_Clear(void* TargetArray, const FArrayProperty* ArrayProp)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		ArrayHelper.EmptyValues();
	}
}

void UKismetArrayLibrary::GenericArray_Resize(void* TargetArray, const FArrayProperty* ArrayProp, int32 Size)
{
	if( TargetArray )
	{
		if(Size >= 0)
		{
			FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
			ArrayHelper.Resize(Size);
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to resize an array using negative size: Array = %s, Size = %d!"), *ArrayProp->GetName(), Size), ELogVerbosity::Warning, ResizeArrayNegativeWarning);
		}
	}
}

void UKismetArrayLibrary::GenericArray_Reverse(void* TargetArray, const FArrayProperty* ArrayProp)
{
	if (TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		const int32 ArraySize = ArrayHelper.Num();
		for (int32 i = 0, i2 = ArraySize - 1; i < ArraySize / 2 /*rounding down*/; ++i, --i2)
		{
			ArrayHelper.SwapValues(i, i2);
		}
	}
}

int32 UKismetArrayLibrary::GenericArray_Length(const void* TargetArray, const FArrayProperty* ArrayProp)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		return ArrayHelper.Num();
	}
	
	return 0;
}

bool UKismetArrayLibrary::GenericArray_IsEmpty(const void* TargetArray, const FArrayProperty* ArrayProp)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		return ArrayHelper.Num() == 0;
	}
	
	return true;
}

bool UKismetArrayLibrary::GenericArray_IsNotEmpty(const void* TargetArray, const FArrayProperty* ArrayProp)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		return ArrayHelper.Num() > 0;
	}

	return false;
}

int32 UKismetArrayLibrary::GenericArray_LastIndex(const void* TargetArray, const FArrayProperty* ArrayProp)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		return ArrayHelper.Num() - 1;
	}

	return INDEX_NONE;
}

void UKismetArrayLibrary::GenericArray_Get(void* TargetArray, const FArrayProperty* ArrayProp, int32 Index, void* Item)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		FProperty* InnerProp = ArrayProp->Inner;
		if( ArrayHelper.IsValidIndex(Index) )
		{
			InnerProp->CopyCompleteValueFromScriptVM(Item, ArrayHelper.GetRawPtr(Index));	
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to access index %d from array '%s' of length %d in '%s'!"),
				Index,
				*ArrayProp->GetName(),
				ArrayHelper.Num(),
				*ArrayProp->GetOwnerVariant().GetPathName()),
				ELogVerbosity::Warning,
				GetOutOfBoundsWarning);
			UE::BlueprintPropertyHelpers::ResetToDefault(Item, InnerProp);
		}
	}
}


void UKismetArrayLibrary::GenericArray_Set(void* TargetArray, const FArrayProperty* ArrayProp, int32 Index, const void* NewItem, bool bSizeToFit)
{
	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		FProperty* InnerProp = ArrayProp->Inner;

		// Expand the array, if desired
		if (!ArrayHelper.IsValidIndex(Index) && bSizeToFit && (Index >= 0) && (Index < MaxSupportedArraySize))
		{
			ArrayHelper.ExpandForIndex(Index);
		}

		if (ArrayHelper.IsValidIndex(Index))
		{
			InnerProp->CopySingleValueToScriptVM(ArrayHelper.GetRawPtr(Index), NewItem);
		}
		else if (Index >= MaxSupportedArraySize)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to set an index on array %s beyond its maximum supported capacity!"), *ArrayProp->GetName()), ELogVerbosity::Warning, ReachedMaximumContainerSizeWarning);
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to set an invalid index on array %s [%d/%d]!"), *ArrayProp->GetName(), Index, GetLastIndex(ArrayHelper)), ELogVerbosity::Warning, SetOutOfBoundsWarning);
		}
	}
}

void UKismetArrayLibrary::GenericArray_Swap(const void* TargetArray, const FArrayProperty* ArrayProp, int32 First, int32 Second)
{
	if (TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		if (ArrayHelper.IsValidIndex(First) && ArrayHelper.IsValidIndex(Second))
		{
			// If First and Second indices are the same - nothing to do
			if (First != Second)
			{
				ArrayHelper.SwapValues(First, Second);
			}
		}
		else if (ArrayHelper.Num() == 0)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to swap elements in empty array %s!"),
				*ArrayProp->GetName()),
				ELogVerbosity::Warning,
				SwapElementsInArrayWarning);
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to swap elements [%d] and [%d] in array %s. Available index range: [0 %d)!"),
				First,
				Second,
				*ArrayProp->GetName(),
				ArrayHelper.Num() - 1),
				ELogVerbosity::Warning,
				SwapElementsInArrayWarning);
		}
	}
}

int32 UKismetArrayLibrary::GenericArray_Find(const void* TargetArray, const FArrayProperty* ArrayProperty, const void* ItemToFind)
{
	int32 ResultIndex = INDEX_NONE;

	if( TargetArray )
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, TargetArray);
		FProperty* InnerProp = ArrayProperty->Inner;

		// compare against each element in the array
		for (int32 Idx = 0; Idx < ArrayHelper.Num() && ResultIndex == INDEX_NONE; Idx++)
		{
			if (InnerProp->Identical(ItemToFind,ArrayHelper.GetRawPtr(Idx)))
			{
				ResultIndex = Idx;
			}
		}
	}

	// assign the resulting index
	return ResultIndex;
}

void UKismetArrayLibrary::GenericArray_SetArrayPropertyByName(UObject* OwnerObject, FName ArrayPropertyName, const void* SrcArrayAddr)
{
	if (OwnerObject != NULL)
	{
		FArrayProperty* ArrayProp = FindFProperty<FArrayProperty>(OwnerObject->GetClass(), ArrayPropertyName);
		if (ArrayProp != NULL)
		{
			void* Dest = ArrayProp->ContainerPtrToValuePtr<void>(OwnerObject);
			ArrayProp->CopyValuesInternal(Dest, SrcArrayAddr, 1);
		}
	}
}

bool UKismetArrayLibrary::GenericArray_IsValidIndex(const void* TargetArray, const FArrayProperty* ArrayProp, int32 IndexToTest)
{
	bool ReturnBool = false;

	if (TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);

		ReturnBool = ArrayHelper.IsValidIndex(IndexToTest);
	}

	return ReturnBool;
}

void UKismetArrayLibrary::GenericArray_HandleBool(const FProperty* Property, void* ItemPtr)
{
	const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property);
	if (BoolProperty)
	{
		ensure((BoolProperty->GetElementSize() * BoolProperty->ArrayDim) == sizeof(uint8));
		BoolProperty->SetPropertyValue(ItemPtr, 0 != *(reinterpret_cast<uint8*>(ItemPtr)));
	}
}


void UKismetArrayLibrary::GenericArray_Random(void* TargetArray, const FArrayProperty* ArrayProp, void* OutItem, int32* OutIndex)
{
	*OutIndex = INDEX_NONE;
	if (TargetArray)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		FProperty* InnerProp = ArrayProp->Inner;

		if (ArrayHelper.Num() > 0)
		{
			const int32 Index = FMath::RandRange(0, ArrayHelper.Num() - 1);

			InnerProp->CopySingleValueToScriptVM(OutItem, ArrayHelper.GetRawPtr(Index));
			*OutIndex = Index;
		} 
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to access random index from empty array!")),
				ELogVerbosity::Warning,
				RandomAccessToEmptyArrayWarning);
			UE::BlueprintPropertyHelpers::ResetToDefault(OutItem, InnerProp);
		}
	}
}

void UKismetArrayLibrary::GenericArray_RandomFromStream(void* TargetArray, const FArrayProperty* ArrayProp, FRandomStream* RandomStream, void* OutItem, int32* OutIndex)
{
	*OutIndex = INDEX_NONE;
	if (TargetArray && RandomStream)
	{
		FScriptArrayHelper ArrayHelper(ArrayProp, TargetArray);
		FProperty* InnerProp = ArrayProp->Inner;

		if (ArrayHelper.Num() > 0)
		{
			const int32 Index = RandomStream->RandRange(0, ArrayHelper.Num() - 1);

			InnerProp->CopySingleValueToScriptVM(OutItem, ArrayHelper.GetRawPtr(Index));
			*OutIndex = Index;
		}
		else
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Attempted to access random index from empty array!")),
				ELogVerbosity::Warning,
				RandomAccessToEmptyArrayWarning);
			UE::BlueprintPropertyHelpers::ResetToDefault(OutItem, InnerProp);
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// Stubs for the UFunctions declared as kismet callable.  These are never actually called...the CustomThunk code calls the appropriate native function with a void* reference to the array

int32 UKismetArrayLibrary::Array_Add(TArray<int32>& TargetArray, const int32& NewItem)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return 0;
}

void UKismetArrayLibrary::Array_Shuffle(TArray<int32>& TargetArray)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

void UKismetArrayLibrary::Array_Insert(TArray<int32>& TargetArray, const int32& NewItem, int32 Index)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

void UKismetArrayLibrary::Array_Remove(TArray<int32>& TargetArray, int32 IndexToRemove)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

bool UKismetArrayLibrary::Array_RemoveItem(TArray<int32>& TargetArray, const int32& IndexToRemove)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return true;
}

void UKismetArrayLibrary::Array_Clear(TArray<int32>& TargetArray)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

static void Array_Resize(const TArray<int32>& TargetArray, int32 Size)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

int32 UKismetArrayLibrary::Array_Length(const TArray<int32>& TargetArray)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return 0;
}

bool UKismetArrayLibrary::Array_IsEmpty(const TArray<int32>& TargetArray)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return true;
}

bool UKismetArrayLibrary::Array_IsNotEmpty(const TArray<int32>& TargetArray)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return false;
}

int32 UKismetArrayLibrary::Array_LastIndex(const TArray<int32>& TargetArray)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return 0;
}

void UKismetArrayLibrary::Array_Get(const TArray<int32>& TargetArray, int32 Index, int32& Item)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

void UKismetArrayLibrary::Array_Set(TArray<int32>& TargetArray, int32 Index, const int32& NewItem, bool bSizeToFit)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

void UKismetArrayLibrary::Array_Swap(TArray<int32>& TargetArray, int32 First, int32 Second)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

int32 UKismetArrayLibrary::Array_Find(const TArray<int32>& TargetArray, const int32& ItemToFind)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
	return 0;
}

void UKismetArrayLibrary::SetArrayPropertyByName(UObject* Object, FName PropertyName, const TArray<int32>& Value)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.
	check(0);
}

bool UKismetArrayLibrary::Array_IsValidIndex(const TArray<int32>& TargetArray, int32 IndexToTest)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.
	check(0);
	return true;
}

void UKismetArrayLibrary::Array_Random(const TArray<int32>& TargetArray, int32& OutItem, int32& OutIndex)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}

void UKismetArrayLibrary::Array_RandomFromStream(const TArray<int32>& TargetArray, FRandomStream& RandomStream, int32& OutItem, int32& OutIndex)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.  Call the Generic* equivalent instead
	check(0);
}


#undef LOCTEXT_NAMESPACE

