// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetGuidLibrary.h"

#include "Cooker/CookRand.h"
#include "EngineLogs.h"
#include "Misc/Guid.h"
#include "Misc/PackagePath.h"
#include "UObject/Stack.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetGuidLibrary)


/* Guid functions
 *****************************************************************************/

UKismetGuidLibrary::UKismetGuidLibrary( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }


FString UKismetGuidLibrary::Conv_GuidToString( const FGuid& InGuid )
{
	return InGuid.ToString(EGuidFormats::Digits);
}


bool UKismetGuidLibrary::EqualEqual_GuidGuid( const FGuid& A, const FGuid& B )
{
	return A == B;
}


bool UKismetGuidLibrary::NotEqual_GuidGuid( const FGuid& A, const FGuid& B )
{
	return A != B;
}


bool UKismetGuidLibrary::IsValid_Guid( const FGuid& InGuid )
{
	return InGuid.IsValid();
}


void UKismetGuidLibrary::Invalidate_Guid( FGuid& InGuid )
{
	InGuid.Invalidate();
}


FGuid UKismetGuidLibrary::NewGuid()
{
#if WITH_EDITOR
	TRefCountPtr<UE::Cook::FCookRand> CookRand = UE::Cook::FCookRand::GetThreadScope();
	if (CookRand)
	{
		FPackagePath PackagePath;
		FString ObjectPath(TEXT("<Unknown>"));
		FString FunctionPath(TEXT("<Unknown>"));
		FFrame* TopFrame = FFrame::GetThreadLocalTopStackFrame();
		if (TopFrame)
		{
			if (TopFrame->Object)
			{
				FPackagePath::TryFromPackageName(TopFrame->Object->GetPackage()->GetName(), PackagePath);
				ObjectPath = TopFrame->Object->GetPathName();
			}
			if (TopFrame->Node)
			{
				FunctionPath = TopFrame->Node->GetPathName();
			}
		}
		FGuid Result = CookRand->RandomGuid();
		UE_ASSET_LOG(LogBlueprintUserMessages, Display, PackagePath,
			TEXT("NewGuid called by ConstructionScript during cooking. Returning a CookRand deterministic random guid.")
			TEXT("\n\tObject: %s")
			TEXT("\n\tFunction: %s")
			TEXT("\n\tResult: %s"),
			*ObjectPath, *FunctionPath, *LexToString(Result));
		return Result;
	}
#endif
	return FGuid::NewGuid();
}


void UKismetGuidLibrary::Parse_StringToGuid( const FString& GuidString, FGuid& OutGuid, bool& Success )
{
	Success = FGuid::Parse(GuidString, OutGuid);
}

