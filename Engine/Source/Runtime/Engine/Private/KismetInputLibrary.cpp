// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetInputLibrary.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetInputLibrary)

FSlateModifierKeysState::FSlateModifierKeysState(const FModifierKeysState& InModifierKeysState)
	: ModifierKeysStateMask(EModifierKey::FromBools(
		InModifierKeysState.IsControlDown(),
		InModifierKeysState.IsAltDown(),
		InModifierKeysState.IsShiftDown(),
		InModifierKeysState.IsCommandDown()
	))
{
}

//////////////////////////////////////////////////////////////////////////
// UKismetInputLibrary

#define LOCTEXT_NAMESPACE "KismetInputLibrary"


UKismetInputLibrary::UKismetInputLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UKismetInputLibrary::CalibrateTilt()
{
	GEngine->Exec(NULL, TEXT("CALIBRATEMOTION"));
}


bool UKismetInputLibrary::EqualEqual_KeyKey(FKey A, FKey B)
{
	return A == B;
}

bool UKismetInputLibrary::EqualEqual_InputChordInputChord( FInputChord A, FInputChord B )
{
	return A == B;
}

bool UKismetInputLibrary::Key_IsModifierKey(const FKey& Key)
{
	return Key.IsModifierKey();
}

bool UKismetInputLibrary::Key_IsGamepadKey(const FKey& Key)
{
	return Key.IsGamepadKey();
}

bool UKismetInputLibrary::Key_IsMouseButton(const FKey& Key)
{
	return Key.IsMouseButton();
}

bool UKismetInputLibrary::Key_IsKeyboardKey(const FKey& Key)
{
	return Key.IsBindableInBlueprints() && (Key.IsGamepadKey() == false && Key.IsMouseButton() == false && Key.IsTouch() == false);
}

bool UKismetInputLibrary::Key_IsAxis1D(const FKey& Key)
{
	return Key.IsAxis1D();
}

bool UKismetInputLibrary::Key_IsAxis2D(const FKey& Key)
{
	return Key.IsAxis2D();
}

bool UKismetInputLibrary::Key_IsAxis3D(const FKey& Key)
{
	return Key.IsAxis3D();
}

bool UKismetInputLibrary::Key_IsButtonAxis(const FKey& Key)
{
	return Key.IsButtonAxis();
}

bool UKismetInputLibrary::Key_IsAnalog(const FKey& Key)
{
	return Key.IsAnalog();
}

bool UKismetInputLibrary::Key_IsDigital(const FKey& Key)
{
	return Key.IsDigital();
}

bool UKismetInputLibrary::Key_IsValid(const FKey& Key)
{
	return Key.IsValid();
}

EUINavigationAction UKismetInputLibrary::Key_GetNavigationAction(const FKey& InKey)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FSlateApplication::IsInitialized())
	{
		return FSlateApplication::Get().GetNavigationActionForKey(InKey);
	}

	return EUINavigationAction::Invalid;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

EUINavigationAction UKismetInputLibrary::Key_GetNavigationActionFromKey(const FKeyEvent& InKeyEvent)
{
	if (FSlateApplication::IsInitialized())
	{
		return FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent);
	}

	return EUINavigationAction::Invalid;
}

EUINavigation UKismetInputLibrary::Key_GetNavigationDirectionFromKey(const FKeyEvent& InKeyEvent)
{
	if (FSlateApplication::IsInitialized())
	{
		return FSlateApplication::Get().GetNavigationDirectionFromKey(InKeyEvent);
	}

	return EUINavigation::Invalid;
}

EUINavigation UKismetInputLibrary::Key_GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent)
{
	if (FSlateApplication::IsInitialized())
	{
		return FSlateApplication::Get().GetNavigationDirectionFromAnalog(InAnalogEvent);
	}

	return EUINavigation::Invalid;
}


FText UKismetInputLibrary::Key_GetDisplayName(const FKey& Key, const bool bLongDisplayName)
{
	return Key.GetDisplayName(bLongDisplayName);
}

bool UKismetInputLibrary::InputEvent_IsRepeat(const FInputEvent& Input)
{
	return Input.IsRepeat();
}

bool UKismetInputLibrary::InputEvent_IsShiftDown(const FInputEvent& Input)
{
	return Input.IsShiftDown();
}

bool UKismetInputLibrary::InputEvent_IsLeftShiftDown(const FInputEvent& Input)
{
	return Input.IsLeftShiftDown();
}

bool UKismetInputLibrary::InputEvent_IsRightShiftDown(const FInputEvent& Input)
{
	return Input.IsRightShiftDown();
}

bool UKismetInputLibrary::InputEvent_IsControlDown(const FInputEvent& Input)
{
	return Input.IsControlDown();
}

bool UKismetInputLibrary::InputEvent_IsLeftControlDown(const FInputEvent& Input)
{
	return Input.IsLeftControlDown();
}

bool UKismetInputLibrary::InputEvent_IsRightControlDown(const FInputEvent& Input)
{
	return Input.IsRightControlDown();
}

bool UKismetInputLibrary::InputEvent_IsAltDown(const FInputEvent& Input)
{
	return Input.IsAltDown();
}

bool UKismetInputLibrary::InputEvent_IsLeftAltDown(const FInputEvent& Input)
{
	return Input.IsLeftAltDown();
}

bool UKismetInputLibrary::InputEvent_IsRightAltDown(const FInputEvent& Input)
{
	return Input.IsRightAltDown();
}

bool UKismetInputLibrary::InputEvent_IsCommandDown(const FInputEvent& Input)
{
	return Input.IsCommandDown();
}

bool UKismetInputLibrary::InputEvent_IsLeftCommandDown(const FInputEvent& Input)
{
	return Input.IsLeftCommandDown();
}

bool UKismetInputLibrary::InputEvent_IsRightCommandDown(const FInputEvent& Input)
{
	return Input.IsRightCommandDown();
}

bool UKismetInputLibrary::ModifierKeysState_IsShiftDown(const FSlateModifierKeysState& KeysState)
{
	return (KeysState.ModifierKeysStateMask & EModifierKey::Shift) != 0;
}

bool UKismetInputLibrary::ModifierKeysState_IsControlDown(const FSlateModifierKeysState& KeysState)
{
	return (KeysState.ModifierKeysStateMask & EModifierKey::Control) != 0;
}

bool UKismetInputLibrary::ModifierKeysState_IsAltDown(const FSlateModifierKeysState& KeysState)
{
	return (KeysState.ModifierKeysStateMask & EModifierKey::Alt) != 0;
}

bool UKismetInputLibrary::ModifierKeysState_IsCommandDown(const FSlateModifierKeysState& KeysState)
{
	return (KeysState.ModifierKeysStateMask & EModifierKey::Command) != 0;
}

FSlateModifierKeysState UKismetInputLibrary::GetModifierKeysState()
{
	if (FSlateApplication::IsInitialized())
	{
		return FSlateApplication::Get().GetModifierKeys();
	}
	return FSlateModifierKeysState();
}

FText UKismetInputLibrary::InputChord_GetDisplayName(const FInputChord& Key)
{
	return Key.GetInputText();
}

FKey UKismetInputLibrary::GetKey(const FKeyEvent& Input)
{
	return Input.GetKey();
}

int32 UKismetInputLibrary::GetUserIndex(const FKeyEvent& Input)
{
	return Input.GetUserIndex();
}

float UKismetInputLibrary::GetAnalogValue(const FAnalogInputEvent& Input)
{
	return Input.GetAnalogValue();
}


FVector2D UKismetInputLibrary::PointerEvent_GetScreenSpacePosition(const FPointerEvent& Input)
{
	return Input.GetScreenSpacePosition();
}

FVector2D UKismetInputLibrary::PointerEvent_GetLastScreenSpacePosition(const FPointerEvent& Input)
{
	return Input.GetLastScreenSpacePosition();
}

FVector2D UKismetInputLibrary::PointerEvent_GetCursorDelta(const FPointerEvent& Input)
{
	return Input.GetCursorDelta();
}

bool UKismetInputLibrary::PointerEvent_IsMouseButtonDown(const FPointerEvent& Input, FKey MouseButton)
{
	return Input.IsMouseButtonDown(MouseButton);
}

FKey UKismetInputLibrary::PointerEvent_GetEffectingButton(const FPointerEvent& Input)
{
	return Input.GetEffectingButton();
}

float UKismetInputLibrary::PointerEvent_GetWheelDelta(const FPointerEvent& Input)
{
	return Input.GetWheelDelta();
}

int32 UKismetInputLibrary::PointerEvent_GetUserIndex(const FPointerEvent& Input)
{
	return Input.GetUserIndex();
}

int32 UKismetInputLibrary::PointerEvent_GetPointerIndex(const FPointerEvent& Input)
{
	return Input.GetPointerIndex();
}

int32 UKismetInputLibrary::PointerEvent_GetTouchpadIndex(const FPointerEvent& Input)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Input.GetTouchpadIndex();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UKismetInputLibrary::PointerEvent_IsTouchEvent(const FPointerEvent& Input)
{
	return Input.IsTouchEvent();
}

ESlateGesture UKismetInputLibrary::PointerEvent_GetGestureType(const FPointerEvent& Input)
{
	static_assert( (int32)EGestureEvent::None == (int32)ESlateGesture::None, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Scroll == (int32)ESlateGesture::Scroll, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Magnify == (int32)ESlateGesture::Magnify, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Swipe == (int32)ESlateGesture::Swipe, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Rotate == (int32)ESlateGesture::Rotate, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::LongPress == (int32)ESlateGesture::LongPress, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Pan == (int32)ESlateGesture::Pan, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Tap == (int32)ESlateGesture::Tap, "EGestureEvent == ESlateGesture" );
	static_assert( (int32)EGestureEvent::Flick == (int32)ESlateGesture::Flick, "EGestureEvent == ESlateGesture" );

	switch ( Input.GetGestureType() )
	{
	case EGestureEvent::Scroll:
		return ESlateGesture::Scroll;
	case EGestureEvent::Magnify:
		return ESlateGesture::Magnify;
	case EGestureEvent::Swipe:
		return ESlateGesture::Swipe;
	case EGestureEvent::Rotate:
		return ESlateGesture::Rotate;
	case EGestureEvent::LongPress:
		return ESlateGesture::LongPress;
	case EGestureEvent::Pan:
		return ESlateGesture::Pan;
	case EGestureEvent::Tap:
		return ESlateGesture::Tap;
	case EGestureEvent::Flick:
		return ESlateGesture::Flick;
	case EGestureEvent::None:
	default:
		return ESlateGesture::None;
	}
}

FVector2D UKismetInputLibrary::PointerEvent_GetGestureDelta(const FPointerEvent& Input)
{
	return Input.GetGestureDelta();
}

#undef LOCTEXT_NAMESPACE

