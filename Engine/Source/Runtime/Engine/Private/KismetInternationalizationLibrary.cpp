// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetInternationalizationLibrary.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Internationalization/Culture.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetInternationalizationLibrary)

#define LOCTEXT_NAMESPACE "Kismet"

bool UKismetInternationalizationLibrary::SetCurrentCulture(const FString& Culture, const bool SaveToConfig)
{
	if (FInternationalization::Get().SetCurrentCulture(Culture))
	{
		if (!GIsEditor && SaveToConfig)
		{
			GConfig->SetString(TEXT("Internationalization"), TEXT("Culture"), *Culture, GGameUserSettingsIni);
			GConfig->EmptySection(TEXT("Internationalization.AssetGroupCultures"), GGameUserSettingsIni);
			GConfig->Flush(false, GGameUserSettingsIni);
		}
		return true;
	}

	return false;
}

FString UKismetInternationalizationLibrary::GetCurrentCulture()
{
	return FInternationalization::Get().GetCurrentCulture()->GetName();
}

bool UKismetInternationalizationLibrary::SetCurrentLanguage(const FString& Culture, const bool SaveToConfig)
{
	if (FInternationalization::Get().SetCurrentLanguage(Culture))
	{
		if (!GIsEditor && SaveToConfig)
		{
			GConfig->SetString(TEXT("Internationalization"), TEXT("Language"), *Culture, GGameUserSettingsIni);
			GConfig->Flush(false, GGameUserSettingsIni);
		}
		return true;
	}

	return false;
}

FString UKismetInternationalizationLibrary::GetCurrentLanguage()
{
	return FInternationalization::Get().GetCurrentLanguage()->GetName();
}

bool UKismetInternationalizationLibrary::SetCurrentLocale(const FString& Culture, const bool SaveToConfig)
{
	if (FInternationalization::Get().SetCurrentLocale(Culture))
	{
		if (!GIsEditor && SaveToConfig)
		{
			GConfig->SetString(TEXT("Internationalization"), TEXT("Locale"), *Culture, GGameUserSettingsIni);
			GConfig->Flush(false, GGameUserSettingsIni);
		}
		return true;
	}

	return false;
}

FString UKismetInternationalizationLibrary::GetCurrentLocale()
{
	return FInternationalization::Get().GetCurrentLocale()->GetName();
}

bool UKismetInternationalizationLibrary::SetCurrentLanguageAndLocale(const FString& Culture, const bool SaveToConfig)
{
	if (FInternationalization::Get().SetCurrentLanguageAndLocale(Culture))
	{
		if (!GIsEditor && SaveToConfig)
		{
			GConfig->SetString(TEXT("Internationalization"), TEXT("Language"), *Culture, GGameUserSettingsIni);
			GConfig->SetString(TEXT("Internationalization"), TEXT("Locale"), *Culture, GGameUserSettingsIni);
			GConfig->Flush(false, GGameUserSettingsIni);
		}
		return true;
	}

	return false;
}

bool UKismetInternationalizationLibrary::SetCurrentAssetGroupCulture(const FName AssetGroup, const FString& Culture, const bool SaveToConfig)
{
	if (FInternationalization::Get().SetCurrentAssetGroupCulture(AssetGroup, Culture))
	{
		if (!GIsEditor && SaveToConfig)
		{
			GConfig->RemoveKeyFromSection(TEXT("Internationalization.AssetGroupCultures"), AssetGroup, GGameUserSettingsIni);
			GConfig->AddToSection(TEXT("Internationalization.AssetGroupCultures"), AssetGroup, Culture, GGameUserSettingsIni);
			GConfig->Flush(false, GGameUserSettingsIni);
		}
		return true;
	}

	return false;
}

FString UKismetInternationalizationLibrary::GetCurrentAssetGroupCulture(const FName AssetGroup)
{
	return FInternationalization::Get().GetCurrentAssetGroupCulture(AssetGroup)->GetName();
}

void UKismetInternationalizationLibrary::ClearCurrentAssetGroupCulture(const FName AssetGroup, const bool SaveToConfig)
{
	FInternationalization::Get().ClearCurrentAssetGroupCulture(AssetGroup);

	if (!GIsEditor && SaveToConfig)
	{
		GConfig->RemoveKeyFromSection(TEXT("Internationalization.AssetGroupCultures"), AssetGroup, GGameUserSettingsIni);
		GConfig->Flush(false, GGameUserSettingsIni);
	}
}

FString UKismetInternationalizationLibrary::GetNativeCulture(const ELocalizedTextSourceCategory TextCategory)
{
	return FTextLocalizationManager::Get().GetNativeCultureName(TextCategory);
}

TArray<FString> UKismetInternationalizationLibrary::GetLocalizedCultures(const bool IncludeGame, const bool IncludeEngine, const bool IncludeEditor, const bool IncludeAdditional)
{
	ELocalizationLoadFlags LoadFlags = ELocalizationLoadFlags::None;
	LoadFlags |= (IncludeGame ? ELocalizationLoadFlags::Game : ELocalizationLoadFlags::None);
	LoadFlags |= (IncludeEngine ? ELocalizationLoadFlags::Engine : ELocalizationLoadFlags::None);
	LoadFlags |= (IncludeEditor ? ELocalizationLoadFlags::Editor : ELocalizationLoadFlags::None);
	LoadFlags |= (IncludeAdditional ? ELocalizationLoadFlags::Additional : ELocalizationLoadFlags::None);

	return GetLocalizedCultures(LoadFlags);
}

TArray<FString> UKismetInternationalizationLibrary::GetLocalizedCultures(const ELocalizationLoadFlags LoadFlags)
{
	return FTextLocalizationManager::Get().GetLocalizedCultureNames(LoadFlags);
}

FString UKismetInternationalizationLibrary::GetLocalizedLanguage()
{
	return FTextLocalizationManager::Get().GetLocalizedLanguageName();
}

FString UKismetInternationalizationLibrary::GetSuitableCulture(const TArray<FString>& AvailableCultures, const FString& CultureToMatch, const FString& FallbackCulture)
{
	const TArray<FString> PrioritizedCulturesToMatch = FInternationalization::Get().GetPrioritizedCultureNames(CultureToMatch);
	for (const FString& PrioritizedCulture : PrioritizedCulturesToMatch)
	{
		if (AvailableCultures.Contains(PrioritizedCulture))
		{
			return PrioritizedCulture;
		}
	}
	return FallbackCulture;
}

FString UKismetInternationalizationLibrary::GetCultureDisplayName(const FString& Culture, const bool Localized)
{
	const FCulturePtr CulturePtr = FInternationalization::Get().GetCulture(Culture);
	if (CulturePtr.IsValid())
	{
		return Localized
			? CulturePtr->GetDisplayName()
			: CulturePtr->GetNativeName();
	}
	return Culture;
}

bool UKismetInternationalizationLibrary::IsCultureRightToLeft(const FString& Culture)
{
	const FCulturePtr CulturePtr = FInternationalization::Get().GetCulture(Culture);
	return CulturePtr && CulturePtr->IsRightToLeft();
}

#undef LOCTEXT_NAMESPACE

