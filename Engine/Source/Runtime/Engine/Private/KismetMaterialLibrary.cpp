// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetMaterialLibrary.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetMaterialLibrary)

//////////////////////////////////////////////////////////////////////////
// UKismetMaterialLibrary

#define LOCTEXT_NAMESPACE "KismetMaterialLibrary"

void UKismetMaterialLibrary::SetScalarParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName, float ParameterValue)
{
	if (Collection)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Collection);

			const bool bFoundParameter = Instance->SetScalarParameterValue(ParameterName, ParameterValue);

			if (!bFoundParameter && !Instance->bLoggedMissingParameterWarning)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ParamName"), FText::FromName(ParameterName));
				FMessageLog("PIE").Warning()
					->AddToken(FTextToken::Create(LOCTEXT("SetScalarParamOn", "SetScalarParameterValue called on")))
					->AddToken(FUObjectToken::Create(Collection))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("WithInvalidParam", "with invalid ParameterName '{ParamName}'. This is likely due to a Blueprint error."), Arguments)));
				Instance->bLoggedMissingParameterWarning = true;
			}
		}
	}
}

void UKismetMaterialLibrary::SetVectorParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName, const FLinearColor& ParameterValue)
{
	if (Collection)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Collection);

			const bool bFoundParameter = Instance->SetVectorParameterValue(ParameterName, ParameterValue);

			if (!bFoundParameter && !Instance->bLoggedMissingParameterWarning)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ParamName"), FText::FromName(ParameterName));
				FMessageLog("PIE").Warning()
					->AddToken(FTextToken::Create(LOCTEXT("SetVectorParamOn", "SetVectorParameterValue called on")))
					->AddToken(FUObjectToken::Create(Collection))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("WithInvalidParam", "with invalid ParameterName '{ParamName}'. This is likely due to a Blueprint error."), Arguments)));
				Instance->bLoggedMissingParameterWarning = true;
			}
		}
	}
}

float UKismetMaterialLibrary::GetScalarParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName) 
{
	float ParameterValue = 0;

	if (Collection)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Collection);

			const bool bFoundParameter = Instance->GetScalarParameterValue(ParameterName, ParameterValue);

			if (!bFoundParameter && !Instance->bLoggedMissingParameterWarning)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ParamName"), FText::FromName(ParameterName));
				FMessageLog("PIE").Warning()
					->AddToken(FTextToken::Create(LOCTEXT("GetScalarParamOn", "GetScalarParameterValue called on")))
					->AddToken(FUObjectToken::Create(Collection))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("WithInvalidParam", "with invalid ParameterName '{ParamName}'. This is likely due to a Blueprint error."), Arguments)));
				Instance->bLoggedMissingParameterWarning = true;
			}
		}
	}

	return ParameterValue;
}

FLinearColor UKismetMaterialLibrary::GetVectorParameterValue(UObject* WorldContextObject, UMaterialParameterCollection* Collection, FName ParameterName) 
{
	FLinearColor ParameterValue = FLinearColor::Black;

	if (Collection)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Collection);

			const bool bFoundParameter = Instance->GetVectorParameterValue(ParameterName, ParameterValue);

			if (!bFoundParameter && !Instance->bLoggedMissingParameterWarning)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ParamName"), FText::FromName(ParameterName));
				FMessageLog("PIE").Warning()
					->AddToken(FTextToken::Create(LOCTEXT("GetVectorParamOn", "GetVectorParameterValue called on")))
					->AddToken(FUObjectToken::Create(Collection))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("WithInvalidParam", "with invalid ParameterName '{ParamName}'. This is likely due to a Blueprint error."), Arguments)));
				Instance->bLoggedMissingParameterWarning = true;
			}
		}
	}

	return ParameterValue;
}

class UMaterialInstanceDynamic* UKismetMaterialLibrary::CreateDynamicMaterialInstance(UObject* WorldContextObject, class UMaterialInterface* Parent, FName OptionalName, EMIDCreationFlags CreationFlags)
{
	UMaterialInstanceDynamic* NewMID = nullptr;

	if (Parent)
	{

		// In editor MIDs need to be created within a persistent object or else they will not be saved.
		// If this MID is created at runtime or specifically marked as transient then put it in the transient package.
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);
		UObject* MIDOuter = !EnumHasAnyFlags(CreationFlags, EMIDCreationFlags::Transient) && World && !World->IsGameWorld() ? WorldContextObject : nullptr;
		NewMID = UMaterialInstanceDynamic::Create(Parent, MIDOuter, OptionalName);
		if (MIDOuter == nullptr)
		{
			NewMID->SetFlags(RF_Transient);
		}
	}

	return NewMID;
}

#undef LOCTEXT_NAMESPACE

