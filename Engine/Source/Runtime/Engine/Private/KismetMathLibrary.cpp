// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetMathLibrary.h"
#include "Engine/Engine.h"
#include "Curves/CurveFloat.h"
#include "DrawDebugHelpers.h"

#include "Blueprint/BlueprintSupport.h"
#include "Math/ConvexHull2d.h"
#include "Math/DualQuat.h"
#include "Math/Box.h"

#include "Math/RandomStream.h"
#include "Misc/RuntimeErrors.h"
#include "Misc/QualifiedFrameTime.h"
#include "Templates/SubclassOf.h"

#include <type_traits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetMathLibrary)

#define LOCTEXT_NAMESPACE "UKismetMathLibrary"

/** Interpolate a linear alpha value using an ease mode and function. */
template<typename FloatType UE_REQUIRES(std::is_floating_point_v<FloatType>)>
FloatType EaseAlpha(FloatType InAlpha, uint8 EasingFunc, FloatType BlendExp, int32 Steps)
{
	switch (EasingFunc)
	{
	case EEasingFunc::Step:					return FMath::InterpStep<FloatType>(0.f, 1.f, InAlpha, Steps);
	case EEasingFunc::SinusoidalIn:			return FMath::InterpSinIn<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::SinusoidalOut:		return FMath::InterpSinOut<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::SinusoidalInOut:		return FMath::InterpSinInOut<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::EaseIn:				return FMath::InterpEaseIn<FloatType>(0.f, 1.f, InAlpha, BlendExp);
	case EEasingFunc::EaseOut:				return FMath::InterpEaseOut<FloatType>(0.f, 1.f, InAlpha, BlendExp);
	case EEasingFunc::EaseInOut:			return FMath::InterpEaseInOut<FloatType>(0.f, 1.f, InAlpha, BlendExp);
	case EEasingFunc::ExpoIn:				return FMath::InterpExpoIn<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::ExpoOut:				return FMath::InterpExpoOut<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::ExpoInOut:			return FMath::InterpExpoInOut<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::CircularIn:			return FMath::InterpCircularIn<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::CircularOut:			return FMath::InterpCircularOut<FloatType>(0.f, 1.f, InAlpha);
	case EEasingFunc::CircularInOut:		return FMath::InterpCircularInOut<FloatType>(0.f, 1.f, InAlpha);
	}
	return InAlpha;
}

const FName DivideByZeroWarning = FName("DivideByZeroWarning");
const FName NegativeSqrtWarning = FName("NegativeSqrtWarning");
const FName ZeroLengthProjectionWarning = FName("ZeroLengthProjectionWarning");
const FName InvalidDateWarning = FName("InvalidDateWarning");
const FName InvalidIndexConversionParameterWarning = FName("InvalidIndexConversionParameterWarning");

UKismetMathLibrary::UKismetMathLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			DivideByZeroWarning,
			LOCTEXT("DivideByZeroWarning", "Divide by zero")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			NegativeSqrtWarning,
			LOCTEXT("NegativeSqrtWarning", "Square root of negative number")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			ZeroLengthProjectionWarning,
			LOCTEXT("ZeroLengthProjectionWarning", "Projection onto vector of zero length")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration (
			InvalidDateWarning,
			LOCTEXT("InvalidDateWarning", "Invalid date warning")
		)
	);
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			InvalidIndexConversionParameterWarning,
			LOCTEXT("InvalidIndexConversionParameterWarning", "Invalid index conversion parameter warning")
		)
	);
}

void UKismetMathLibrary::ReportError_Divide_ByteByte()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_ByteByte"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Percent_ByteByte()
{
	FFrame::KismetExecutionMessage(TEXT("Modulo by zero: Percent_ByteByte"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_IntInt()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_IntInt"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_DoubleDouble()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_DoubleDouble"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_FloatFloat()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_FloatFloat"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_Int64Int64()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_Int64Int64"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Percent_IntInt()
{
	FFrame::KismetExecutionMessage(TEXT("Modulo by zero: Percent_IntInt"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Percent_Int64Int64()
{
	FFrame::KismetExecutionMessage(TEXT("Modulo by zero: Percent_Int64Int64"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Sqrt()
{
	FFrame::KismetExecutionMessage(TEXT("Attempt to take Sqrt() of negative number - returning 0."), ELogVerbosity::Warning, NegativeSqrtWarning);
}

void UKismetMathLibrary::ReportError_Divide_VectorFloat()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_VectorFloat"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_VectorInt()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_VectorInt"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_VectorVector()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_VectorVector"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_Vector2DVector2D()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_Vector2DVector2D"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_ProjectVectorOnToVector()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: ProjectVectorOnToVector with zero Target vector"), ELogVerbosity::Warning, ZeroLengthProjectionWarning);
}

void UKismetMathLibrary::ReportError_Divide_Vector2DFloat()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_Vector2DFloat"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_IntPointOnInt()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_IntPointInt"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_Divide_IntPointOnIntPoint()
{
	FFrame::KismetExecutionMessage(TEXT("Divide by zero: Divide_IntPointIntPoint"), ELogVerbosity::Warning, DivideByZeroWarning);
}

void UKismetMathLibrary::ReportError_DaysInMonth()
{
	FFrame::KismetExecutionMessage(TEXT("Invalid month (must be between 1 and 12): DaysInMonth"), ELogVerbosity::Warning, InvalidDateWarning);
}


// Include code in this source file if it's not being inlined in the header.
#if !KISMET_MATH_INLINE_ENABLED
#include "Kismet/KismetMathLibrary.inl"
#endif

bool UKismetMathLibrary::RandomBoolWithWeight(float Weight)
{
	//If the Weight equals to 0.0f then always return false
	if (Weight <= 0.0f)
	{
		return false;
	}
	else
	{
		//If the Weight is higher or equal to the random number then return true
		return Weight >= FMath::FRandRange(0.0f, 1.0f);
	}

}

bool UKismetMathLibrary::RandomBoolWithWeightFromStream(const FRandomStream& RandomStream, float Weight)
{
	//If the Weight equals to 0.0f then always return false
	if (Weight <= 0.0f)
	{
		return false;
	}
	else
	{
		//Create the random float from the specified stream
		float Number = UKismetMathLibrary::RandomFloatFromStream(RandomStream);

		//If the Weight is higher or equal to number generated from stream then return true
		return Weight >= Number;
	}

}

/* This function is custom thunked, the real function is GenericPercent_FloatFloat */
double UKismetMathLibrary::Percent_FloatFloat(double A, double B)
{
	check(0);
	return 0;
}

double UKismetMathLibrary::GenericPercent_FloatFloat(double A, double B)
{
	return (B != 0.0) ? FMath::Fmod(A, B) : 0.0;
}

bool UKismetMathLibrary::InRange_FloatFloat(double Value, double Min, double Max, bool InclusiveMin, bool InclusiveMax)
{
	return ((InclusiveMin ? (Value >= Min) : (Value > Min)) && (InclusiveMax ? (Value <= Max) : (Value < Max)));
}

double UKismetMathLibrary::Hypotenuse(double Width, double Height)
{
	// This implementation avoids overflow/underflow caused by squaring width and height:
	double Min = FMath::Abs(Width);
	double Max = FMath::Abs(Height);

	if (Min > Max)
	{
		Swap(Min, Max);
	}

	return (FMath::IsNearlyZero(Max) ? 0.0 : Max * FMath::Sqrt(1.0 + FMath::Square(Min/Max)));
}

double UKismetMathLibrary::Log(double A, double Base)
{
	double Result = 0.f;
	if(Base <= 0.f)
	{
		FFrame::KismetExecutionMessage(TEXT("Divide by zero: Log"), ELogVerbosity::Warning, DivideByZeroWarning);
	}
	else
	{
		Result = FMath::Loge(A) / FMath::Loge(Base);
	}
	return Result;
}

int32 UKismetMathLibrary::FMod(double Dividend, double Divisor, double& Remainder)
{
	int32 Result;
	if (Divisor != 0.f)
	{
		const double Quotient = Dividend / Divisor;
		Result = (Quotient < 0.f ? -1 : 1) * FMath::FloorToInt32(FMath::Abs(Quotient));
		Remainder = FMath::Fmod(Dividend, Divisor);
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("Attempted modulo 0 - returning 0."), ELogVerbosity::Warning, DivideByZeroWarning);

		Result = 0;
		Remainder = 0.f;
	}

	return Result;
}

int64 UKismetMathLibrary::FMod64(double Dividend, double Divisor, double& Remainder)
{
	int64 Result;
	if (Divisor != 0.f)
	{
		const double Quotient = Dividend / Divisor;
		Result = (Quotient < 0.f ? -1 : 1) * FMath::FloorToInt64(FMath::Abs(Quotient));
		Remainder = FMath::Fmod(Dividend, Divisor);
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("Attempted modulo 0 - returning 0."), ELogVerbosity::Warning, DivideByZeroWarning);

		Result = 0;
		Remainder = 0.f;
	}

	return Result;
}

double UKismetMathLibrary::NormalizeToRange(double Value, double RangeMin, double RangeMax)
{
	if (RangeMin == RangeMax)
	{
		if (Value < RangeMin)
		{
			return 0.f;
		}
		else
		{
			return 1.f;
		}
	}

	if (RangeMin > RangeMax)
	{
		Swap(RangeMin, RangeMax);
	}
	return (Value - RangeMin) / (RangeMax - RangeMin);
}

double UKismetMathLibrary::MapRangeUnclamped(double Value, double InRangeA, double InRangeB, double OutRangeA, double OutRangeB)
{
	return FMath::GetMappedRangeValueUnclamped(FVector2D(InRangeA, InRangeB), FVector2D(OutRangeA, OutRangeB), Value);
}

double UKismetMathLibrary::MapRangeClamped(double Value, double InRangeA, double InRangeB, double OutRangeA, double OutRangeB)
{
	return FMath::GetMappedRangeValueClamped(FVector2D(InRangeA, InRangeB), FVector2D(OutRangeA, OutRangeB), Value);
}

double UKismetMathLibrary::FInterpEaseInOut(double A, double B, double Alpha, double Exponent)
{
	return FMath::InterpEaseInOut<double>(A, B, Alpha, Exponent);
}

float UKismetMathLibrary::GetRuntimeFloatCurveValue(const FRuntimeFloatCurve& Curve, const float InTime, const float InDefaultValue /*= 0.0f*/)
{
	if (const FRichCurve* RichCurve = Curve.GetRichCurveConst())
	{
		return RichCurve->Eval(InTime, InDefaultValue);
	}

	return InDefaultValue;
}

float UKismetMathLibrary::MakePulsatingValue(float InCurrentTime, float InPulsesPerSecond, float InPhase)
{
	return FMath::MakePulsatingValue((double)InCurrentTime, InPulsesPerSecond, InPhase);
}

double UKismetMathLibrary::SafeDivide(double A, double B)
{
	return (B != 0.0f) ? (A / B) : 0.0f;
}

void UKismetMathLibrary::MaxOfIntArray(const TArray<int32>& IntArray, int32& IndexOfMaxValue, int32& MaxValue)
{
	MaxValue = FMath::Max(IntArray, &IndexOfMaxValue);
}

void UKismetMathLibrary::MinOfIntArray(const TArray<int32>& IntArray, int32& IndexOfMinValue, int32& MinValue)
{
	MinValue = FMath::Min<int32>(IntArray, &IndexOfMinValue);
}

void UKismetMathLibrary::MedianOfIntArray(TArray<int32> IntArray, float& MedianValue)
{
	if (IntArray.IsEmpty())
	{
		MedianValue = 0;
		return;
	}

	IntArray.Sort();
	
	if (IntArray.Num() % 2 == 1)
	{
		MedianValue = IntArray[(IntArray.Num() - 1) / 2];
	}
	else
	{
		MedianValue = IntArray[IntArray.Num() / 2] + IntArray[(IntArray.Num() / 2) - 1];
		MedianValue /= 2;
	}
}

void UKismetMathLibrary::AverageOfIntArray(const TArray<int32>& IntArray, float& AverageValue)
{
	AverageValue = 0;

	for (int32 Index = 0; Index < IntArray.Num(); ++Index)
	{
		AverageValue += IntArray[Index];
	}

	AverageValue /= IntArray.Num();
}

void UKismetMathLibrary::MaxOfFloatArray(const TArray<float>& FloatArray, int32& IndexOfMaxValue, float& MaxValue)
{
	MaxValue = FMath::Max(FloatArray, &IndexOfMaxValue);
}

void UKismetMathLibrary::MinOfFloatArray(const TArray<float>& FloatArray, int32& IndexOfMinValue, float& MinValue)
{
	MinValue = FMath::Min(FloatArray, &IndexOfMinValue);
}

void UKismetMathLibrary::MaxOfByteArray(const TArray<uint8>& ByteArray, int32& IndexOfMaxValue, uint8& MaxValue)
{
	MaxValue = FMath::Max(ByteArray, &IndexOfMaxValue);
}

void UKismetMathLibrary::MinOfByteArray(const TArray<uint8>& ByteArray, int32& IndexOfMinValue, uint8& MinValue)
{
	MinValue = FMath::Min(ByteArray, &IndexOfMinValue);
}

double UKismetMathLibrary::Ease(double A, double B, double Alpha, TEnumAsByte<EEasingFunc::Type> EasingFunc, double BlendExp, int32 Steps)
{
	return Lerp(A, B, EaseAlpha(Alpha, EasingFunc, BlendExp, Steps));
}

template <typename T>
void GenericSpringInterp(T& Current, const T Target, T& PrevTarget, bool& bPrevTargetValid, T& Velocity,
                         float Stiffness, float CriticalDamping, float DeltaTime,
                         float TargetVelocityAmount, float Mass, bool bInitializeFromTarget)
{
	if (bInitializeFromTarget && !bPrevTargetValid)
	{
		Current = Target;
	}
	if (DeltaTime > UE_SMALL_NUMBER)
	{
		if (!FMath::IsNearlyZero(Mass))
		{
			// Note that old target was stored in PrevTarget
			T TargetVel = bPrevTargetValid ? (Target - PrevTarget) * (TargetVelocityAmount / DeltaTime) : T(0.f);
			const float Omega = FMath::Sqrt(Stiffness / Mass); // angular frequency
			const float Frequency = Omega / (2.0f * UE_PI);
			T NewValue = Current;
			FMath::SpringDamper(Current, Velocity, Target, TargetVel, DeltaTime, Frequency, CriticalDamping);
			PrevTarget = Target;
			bPrevTargetValid = true;
		}
	}
}

float UKismetMathLibrary::FloatSpringInterp(float Current, float Target, FFloatSpringState& SpringState,
                                            float Stiffness, float CriticalDamping, float DeltaTime,
                                            float Mass, float TargetVelocityAmount, 
                                            bool bClamp, float MinValue, float MaxValue,
                                            bool bInitializeFromTarget)
{
	GenericSpringInterp(Current, Target, SpringState.PrevTarget, SpringState.bPrevTargetValid, SpringState.Velocity, Stiffness, CriticalDamping,
	                    DeltaTime, TargetVelocityAmount, Mass, bInitializeFromTarget);
	if (bClamp)
	{
		if (Current < MinValue)
		{
			Current = MinValue;
			if (SpringState.Velocity < 0.0f)
			{
				SpringState.Velocity = 0.0f;
			}
		}
		else if (Current > MaxValue)
		{
			Current = MaxValue;
			if (SpringState.Velocity > 0.0f)
			{
				SpringState.Velocity = 0.0f;
			}
		}
	}
	return Current;
}

FVector  UKismetMathLibrary::RotateAngleAxis(FVector InVect, float AngleDeg, FVector Axis)
{
	return InVect.RotateAngleAxis(AngleDeg, Axis.GetSafeNormal());
}

FVector UKismetMathLibrary::VEase(FVector A, FVector B, float Alpha, TEnumAsByte<EEasingFunc::Type> EasingFunc, float BlendExp, int32 Steps)
{
	return VLerp(A, B, EaseAlpha(Alpha, EasingFunc, BlendExp, Steps));
}

FVector UKismetMathLibrary::VectorSpringInterp(FVector Current, FVector Target, FVectorSpringState& SpringState,
                                               float Stiffness, float CriticalDamping, float DeltaTime,
                                               float Mass, float TargetVelocityAmount, 
                                               bool bClamp, FVector MinValue, FVector MaxValue,
                                               bool bInitializeFromTarget)
{
	GenericSpringInterp(Current, Target, SpringState.PrevTarget, SpringState.bPrevTargetValid, SpringState.Velocity, Stiffness,
	                    CriticalDamping, DeltaTime, TargetVelocityAmount, Mass, bInitializeFromTarget);
	if (bClamp)
	{
		for (int Index = 0 ; Index != 3 ; ++Index)
		{
			if (Current[Index] < MinValue[Index])
			{
				Current[Index] = MinValue[Index];
				if (SpringState.Velocity[Index] < 0.0f)
				{
					SpringState.Velocity[Index] = 0.0f;
				}
			}
			else if (Current[Index] > MaxValue[Index])
			{
				Current[Index] = MaxValue[Index];
				if (SpringState.Velocity[Index] > 0.0f)
				{
					SpringState.Velocity[Index] = 0.0f;
				}
			}
		}
	}
	return Current;
}

FQuat UKismetMathLibrary::QuaternionSpringInterp(FQuat Current, FQuat Target, UPARAM(ref) FQuaternionSpringState& SpringState,
                                                 float Stiffness, float CriticalDamping, float DeltaTime,
                                                 float Mass, float TargetVelocityAmount, bool bInitializeFromTarget)
{
	if ((Target | Current) < 0.0f)
	{
		Target = -Target;
	}
	FQuat VelocityQuat(SpringState.AngularVelocity.X, SpringState.AngularVelocity.Y, SpringState.AngularVelocity.Z, 0.0f);
	FQuat Velocity = 0.5f * Current * VelocityQuat;

  	GenericSpringInterp(Current, Target, SpringState.PrevTarget, SpringState.bPrevTargetValid, Velocity, Stiffness,
	                    CriticalDamping, DeltaTime, TargetVelocityAmount, Mass, bInitializeFromTarget);
	Current.Normalize();

	VelocityQuat = Current.Inverse() * 2.0f * Velocity;
	SpringState.AngularVelocity.Set(VelocityQuat.X, VelocityQuat.Y, VelocityQuat.Z);

	return Current;
}


void UKismetMathLibrary::ResetFloatSpringState(FFloatSpringState& SpringState)
{
	SpringState.Reset();
}

void UKismetMathLibrary::ResetVectorSpringState(FVectorSpringState& SpringState)
{
	SpringState.Reset();
}

void UKismetMathLibrary::ResetQuaternionSpringState(FQuaternionSpringState& SpringState)
{
	SpringState.Reset();
}

void UKismetMathLibrary::SetFloatSpringStateVelocity(FFloatSpringState& SpringState, float Velocity)
{
	SpringState.Velocity = Velocity;
}

void UKismetMathLibrary::SetVectorSpringStateVelocity(FVectorSpringState& SpringState, FVector Velocity)
{
	SpringState.Velocity = Velocity;
}

void UKismetMathLibrary::SetQuaternionSpringStateAngularVelocity(FQuaternionSpringState& SpringState, FVector AngularVelocity)
{
	SpringState.AngularVelocity = AngularVelocity;
}

FVector UKismetMathLibrary::RandomUnitVector()
{
	return FMath::VRand();
}

FVector UKismetMathLibrary::RandomUnitVectorInEllipticalConeInRadians(FVector ConeDir, float MaxYawInRadians, float MaxPitchInRadians)
{
	return FMath::VRandCone(ConeDir, MaxYawInRadians, MaxPitchInRadians);
}

FRotator UKismetMathLibrary::RandomRotator(bool bRoll)
{
	FRotator RRot;
	RRot.Yaw = FMath::FRand() * 360.f;
	RRot.Pitch = FMath::FRand() * 360.f;

	if (bRoll)
	{
		RRot.Roll = FMath::FRand() * 360.f;
	}
	else
	{
		RRot.Roll = 0;
	}
	return RRot;
}

FVector UKismetMathLibrary::FindClosestPointOnLine(FVector Point, FVector LineOrigin, FVector LineDirection)
{
	const FVector SafeDir = LineDirection.GetSafeNormal();
	const FVector ClosestPoint = LineOrigin + (SafeDir * ((Point-LineOrigin) | SafeDir));
	return ClosestPoint;
}

FVector UKismetMathLibrary::GetVectorArrayAverage(const TArray<FVector>& Vectors)
{
	FVector Sum(0.f);
	FVector Average(0.f);

	if (Vectors.Num() > 0)
	{
		for (int32 VecIdx=0; VecIdx<Vectors.Num(); VecIdx++)
		{
			Sum += Vectors[VecIdx];
		}

		Average = Sum / ((float)Vectors.Num());
	}

	return Average;
}

FRotator UKismetMathLibrary::TransformRotation(const FTransform& T, FRotator Rotation)
{
	return T.TransformRotation(Rotation.Quaternion()).Rotator();
}

FRotator UKismetMathLibrary::InverseTransformRotation(const FTransform& T, FRotator Rotation)
{
	return T.InverseTransformRotation(Rotation.Quaternion()).Rotator();
}

FRotator UKismetMathLibrary::ComposeRotators(FRotator A, FRotator B)
{
	FQuat AQuat = FQuat(A);
	FQuat BQuat = FQuat(B);

	return FRotator(BQuat*AQuat);
}

void UKismetMathLibrary::GetAxes(FRotator A, FVector& X, FVector& Y, FVector& Z)
{
	FRotationMatrix R(A);
	R.GetScaledAxes(X, Y, Z);
}

FRotator UKismetMathLibrary::RLerp(FRotator A, FRotator B, float Alpha, bool bShortestPath)
{
	// if shortest path, we use Quaternion to interpolate instead of using FRotator
	if (bShortestPath)
	{
		FQuat AQuat(A);
		FQuat BQuat(B);

		FQuat Result = FQuat::Slerp(AQuat, BQuat, Alpha);

		return Result.Rotator();
	}

	const FRotator DeltaAngle = B - A;
	return A + Alpha*DeltaAngle;
}

FRotator UKismetMathLibrary::REase(FRotator A, FRotator B, float Alpha, bool bShortestPath, TEnumAsByte<EEasingFunc::Type> EasingFunc, float BlendExp, int32 Steps)
{
	return RLerp(A, B, EaseAlpha(Alpha, EasingFunc, BlendExp, Steps), bShortestPath);
}

FRotator UKismetMathLibrary::NormalizedDeltaRotator(FRotator A, FRotator B)
{
	FRotator Delta = A - B;
	Delta.Normalize();
	return Delta;
}

FRotator UKismetMathLibrary::RotatorFromAxisAndAngle(FVector Axis, float Angle)
{
	FVector SafeAxis = Axis.GetSafeNormal(); // Make sure axis is unit length
	return FQuat(SafeAxis, FMath::DegreesToRadians(Angle)).Rotator();
}

float UKismetMathLibrary::ClampAxis(float Angle)
{
	return FRotator::ClampAxis(Angle);
}

float UKismetMathLibrary::NormalizeAxis(float Angle)
{
	return FRotator::NormalizeAxis(Angle);
}

FTransform UKismetMathLibrary::TLerp(const FTransform& A, const FTransform& B, float Alpha, TEnumAsByte<ELerpInterpolationMode::Type> LerpInterpolationMode)
{
	FTransform Result;

	FTransform NA = A;
	FTransform NB = B;
	NA.NormalizeRotation();
	NB.NormalizeRotation();

	// Quaternion interpolation
	if (LerpInterpolationMode == ELerpInterpolationMode::QuatInterp)
	{
		Result.Blend(NA, NB, Alpha);
		return Result;
	}
	// Euler Angle interpolation
	else if (LerpInterpolationMode == ELerpInterpolationMode::EulerInterp)
	{
		Result.SetTranslation(FMath::Lerp(NA.GetTranslation(), NB.GetTranslation(), Alpha));
		Result.SetScale3D(FMath::Lerp(NA.GetScale3D(), NB.GetScale3D(), Alpha));
		Result.SetRotation(FQuat(RLerp(NA.Rotator(), NB.Rotator(), Alpha, false)));
		return Result;
	}
	// Dual quaternion interpolation
	else
	{
		if ((NB.GetRotation() | NA.GetRotation()) < 0.0f)
		{
			NB.SetRotation(NB.GetRotation()*-1.0f);
		}
		return (FDualQuat(NA)*(1 - Alpha) + FDualQuat(NB)*Alpha).Normalized().AsFTransform(FMath::Lerp(NA.GetScale3D(), NB.GetScale3D(), Alpha));
	}
}

FTransform UKismetMathLibrary::TEase(const FTransform& A, const FTransform& B, float Alpha, TEnumAsByte<EEasingFunc::Type> EasingFunc, float BlendExp, int32 Steps)
{
	return TLerp(A, B, EaseAlpha(Alpha, EasingFunc, BlendExp, Steps));
}

FTransform UKismetMathLibrary::TInterpTo(const FTransform& Current, const FTransform& Target, float DeltaTime, float InterpSpeed)
{
	if (InterpSpeed <= 0.f)
	{
		return Target;
	}

	const float Alpha = FClamp(DeltaTime * InterpSpeed, 0.f, 1.f);

	return TLerp(Current, Target, Alpha);
}

bool UKismetMathLibrary::NearlyEqual_TransformTransform(const FTransform& A, const FTransform& B, float LocationTolerance, float RotationTolerance, float Scale3DTolerance)
{
	return
		FTransform::AreRotationsEqual(A, B, RotationTolerance) &&
		FTransform::AreTranslationsEqual(A, B, LocationTolerance) &&
		FTransform::AreScale3DsEqual(A, B, Scale3DTolerance);
}

float UKismetMathLibrary::Transform_Determinant(const FTransform& Transform)
{
	return Transform.ToMatrixWithScale().Determinant();
}

FMatrix UKismetMathLibrary::Conv_TransformToMatrix(const FTransform& Tranform)
{
	return Tranform.ToMatrixWithScale();
}

bool UKismetMathLibrary::ClassIsChildOf(TSubclassOf<class UObject> TestClass, TSubclassOf<class UObject> ParentClass)
{
	return ((*ParentClass != NULL) && (*TestClass != NULL)) ? (*TestClass)->IsChildOf(*ParentClass) : false;
}

/* Plane functions
 *****************************************************************************/
FPlane UKismetMathLibrary::MakePlaneFromPointAndNormal(FVector Point, FVector Normal)
{
	return FPlane(Point, Normal.GetSafeNormal());
}

/* DateTime functions
 *****************************************************************************/
FDateTime UKismetMathLibrary::MakeDateTime(int32 Year, int32 Month, int32 Day, int32 Hour, int32 Minute, int32 Second, int32 Millisecond)
{
	if (!FDateTime::Validate(Year, Month, Day, Hour, Minute, Second, Millisecond))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("DateTime in bad format (year %d, month %d, day %d, hour %d, minute %d, second %d, millisecond %d). E.g. year, month and day can't be zero."), Year, Month, Day, Hour, Minute, Second, Millisecond), ELogVerbosity::Warning, InvalidDateWarning);

		return FDateTime(1, 1, 1, 0, 0, 0, 0);
	}

	return FDateTime(Year, Month, Day, Hour, Minute, Second, Millisecond);
}

void UKismetMathLibrary::BreakDateTime(FDateTime InDateTime, int32& Year, int32& Month, int32& Day, int32& Hour, int32& Minute, int32& Second, int32& Millisecond)
{
	Year = GetYear(InDateTime);
	Month = GetMonth(InDateTime);
	Day = GetDay(InDateTime);
	Hour = GetHour(InDateTime);
	Minute = GetMinute(InDateTime);
	Second = GetSecond(InDateTime);
	Millisecond = GetMillisecond(InDateTime);
}


/* Timespan functions
*****************************************************************************/

FTimespan UKismetMathLibrary::MakeTimespan(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 Milliseconds)
{
	return FTimespan(Days, Hours, Minutes, Seconds, Milliseconds * 1000 * 1000);
}


FTimespan UKismetMathLibrary::MakeTimespan2(int32 Days, int32 Hours, int32 Minutes, int32 Seconds, int32 FractionNano)
{
	return FTimespan(Days, Hours, Minutes, Seconds, FractionNano);
}


void UKismetMathLibrary::BreakTimespan(FTimespan InTimespan, int32& Days, int32& Hours, int32& Minutes, int32& Seconds, int32& Milliseconds)
{
	Days = InTimespan.GetDays();
	Hours = InTimespan.GetHours();
	Minutes = InTimespan.GetMinutes();
	Seconds = InTimespan.GetSeconds();
	Milliseconds = InTimespan.GetFractionMilli();
}


void UKismetMathLibrary::BreakTimespan2(FTimespan InTimespan, int32& Days, int32& Hours, int32& Minutes, int32& Seconds, int32& FractionNano)
{
	Days = InTimespan.GetDays();
	Hours = InTimespan.GetHours();
	Minutes = InTimespan.GetMinutes();
	Seconds = InTimespan.GetSeconds();
	FractionNano = InTimespan.GetFractionNano();
}


FTimespan UKismetMathLibrary::FromDays(double Days)
{
	if (Days < FTimespan::MinValue().GetTotalDays())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DaysValue"), Days);
		LogRuntimeError(FText::Format(LOCTEXT("ClampDaysToMinTimespan", "Days value {DaysValue} is less than minimum days TimeSpan can represent. Clamping to MinValue."), Args));
		return FTimespan::MinValue();
	}
	else if (Days > FTimespan::MaxValue().GetTotalDays())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DaysValue"), Days);
		LogRuntimeError(FText::Format(LOCTEXT("ClampDaysToMaxTimespan", "Days value {DaysValue} is greater than maximum days TimeSpan can represent. Clamping to MaxValue."), Args));
		return FTimespan::MaxValue();
	}

	return FTimespan::FromDays(Days);
}


FTimespan UKismetMathLibrary::FromHours(double Hours)
{
	if (Hours < FTimespan::MinValue().GetTotalHours())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("HoursValue"), Hours);
		LogRuntimeError(FText::Format(LOCTEXT("ClampHoursToMinTimespan", "Hours value {HoursValue} is less than minimum hours TimeSpan can represent. Clamping to MinValue."), Args));
		return FTimespan::MinValue();
	}
	else if (Hours > FTimespan::MaxValue().GetTotalHours())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("HoursValue"), Hours);
		LogRuntimeError(FText::Format(LOCTEXT("ClampHoursToMaxTimespan", "Hours value {HoursValue} is greater than maximum hours TimeSpan can represent. Clamping to MaxValue."), Args));
		return FTimespan::MaxValue();
	}

	return FTimespan::FromHours(Hours);
}


FTimespan UKismetMathLibrary::FromMinutes(double Minutes)
{
	if (Minutes < FTimespan::MinValue().GetTotalMinutes())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MinutesValue"), Minutes);
		LogRuntimeError(FText::Format(LOCTEXT("ClampMinutesToMinTimespan", "Minutes value {MinutesValue} is less than minimum minutes TimeSpan can represent. Clamping to MinValue."), Args));
		return FTimespan::MinValue();
	}
	else if (Minutes > FTimespan::MaxValue().GetTotalMinutes())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MinutesValue"), Minutes);
		LogRuntimeError(FText::Format(LOCTEXT("ClampMinutesToMaxTimespan", "Minutes value {MinutesValue} is greater than maximum minutes TimeSpan can represent. Clamping to MaxValue."), Args));
		return FTimespan::MaxValue();
	}

	return FTimespan::FromMinutes(Minutes);
}


FTimespan UKismetMathLibrary::FromSeconds(double Seconds)
{
	if (Seconds < FTimespan::MinValue().GetTotalSeconds())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("SecondsValue"), Seconds);
		LogRuntimeError(FText::Format(LOCTEXT("ClampSecondsToMinTimespan", "Seconds value {SecondsValue} is less than minimum seconds TimeSpan can represent. Clamping to MinValue."), Args));
		return FTimespan::MinValue();
	}
	else if (Seconds > FTimespan::MaxValue().GetTotalSeconds())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("SecondsValue"), Seconds);
		LogRuntimeError(FText::Format(LOCTEXT("ClampSecondsToMaxTimespan", "Seconds value {SecondsValue} is greater than maximum seconds TimeSpan can represent. Clamping to MaxValue."), Args));
		return FTimespan::MaxValue();
	}

	return FTimespan::FromSeconds(Seconds);
}


FTimespan UKismetMathLibrary::FromMilliseconds(double Milliseconds)
{
	if (Milliseconds < FTimespan::MinValue().GetTotalMilliseconds())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MillisecondsValue"), Milliseconds);
		LogRuntimeError(FText::Format(LOCTEXT("ClampMillisecondsToMinTimespan", "Milliseconds value {MillisecondsValue} is less than minimum milliseconds TimeSpan can represent. Clamping to MinValue."), Args));
		return FTimespan::MinValue();
	}
	else if (Milliseconds > FTimespan::MaxValue().GetTotalMilliseconds())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MillisecondsValue"), Milliseconds);
		LogRuntimeError(FText::Format(LOCTEXT("ClampMillisecondsToMaxTimespan", "Milliseconds value {MillisecondsValue} is greater than maximum milliseconds TimeSpan can represent. Clamping to MaxValue."), Args));
		return FTimespan::MaxValue();
	}

	return FTimespan::FromMilliseconds(Milliseconds);
}

/* FrameTime functions
*****************************************************************************/

FQualifiedFrameTime UKismetMathLibrary::MakeQualifiedFrameTime(FFrameNumber Frame, FFrameRate FrameRate, float SubFrame)
{
	SubFrame = FMath::Clamp(SubFrame, 0.f, FFrameTime::MaxSubframe);
	return FQualifiedFrameTime(FFrameTime(Frame, SubFrame), FrameRate);
}

void UKismetMathLibrary::BreakQualifiedFrameTime(const FQualifiedFrameTime& InFrameTime, FFrameNumber& Frame, FFrameRate& FrameRate, float& SubFrame)
{
	Frame = InFrameTime.Time.GetFrame();
	SubFrame = InFrameTime.Time.GetSubFrame();
	FrameRate = InFrameTime.Rate;
}

/* FrameRate functions
*****************************************************************************/
FFrameRate UKismetMathLibrary::MakeFrameRate(int32 Numerator, int32 Denominator)
{
	// Prevent a denominator of zero as that will cause divide by zero errors when the framerate is used.
	if (Denominator <= 0)
	{
		Denominator = 1;
	}

	return FFrameRate(Numerator, Denominator);
}

void UKismetMathLibrary::BreakFrameRate(const FFrameRate& InFrameRate, int32& Numerator, int32& Denominator)
{
	Numerator = InFrameRate.Numerator;
	Denominator = InFrameRate.Denominator;
}



/* Rotator functions
*****************************************************************************/

FVector UKismetMathLibrary::GetForwardVector(FRotator InRot)
{
	return InRot.Vector();
}

FVector UKismetMathLibrary::GetRightVector(FRotator InRot)
{
	return FRotationMatrix(InRot).GetScaledAxis(EAxis::Y);
}

FVector UKismetMathLibrary::GetUpVector(FRotator InRot)
{
	return FRotationMatrix(InRot).GetScaledAxis(EAxis::Z);
}

FVector UKismetMathLibrary::CreateVectorFromYawPitch(float Yaw, float Pitch, float Length /*= 1.0f */)
{
	// FRotator::Vector() behaviour 
	float CP, SP, CY, SY;
	FMath::SinCos(&SP, &CP, FMath::DegreesToRadians(Pitch));
	FMath::SinCos(&SY, &CY, FMath::DegreesToRadians(Yaw));
	FVector V = FVector(CP*CY, CP*SY, SP) * Length;

	return V;
}

void UKismetMathLibrary::GetYawPitchFromVector(FVector InVec, float& Yaw, float& Pitch)
{
	FVector NormalizedVector = InVec.GetSafeNormal();
	// Find yaw.
	Yaw = FMath::RadiansToDegrees(FMath::Atan2(NormalizedVector.Y, NormalizedVector.X));

	// Find pitch.
	Pitch = FMath::RadiansToDegrees(FMath::Atan2(NormalizedVector.Z, FMath::Sqrt(NormalizedVector.X*NormalizedVector.X + NormalizedVector.Y*NormalizedVector.Y)));
}

void UKismetMathLibrary::GetAzimuthAndElevation(FVector InDirection, const FTransform& ReferenceFrame, float& Azimuth, float& Elevation)
{
	FVector2D Result = FMath::GetAzimuthAndElevation
	(
		InDirection.GetSafeNormal(),
		ReferenceFrame.GetUnitAxis(EAxis::X),
		ReferenceFrame.GetUnitAxis(EAxis::Y),
		ReferenceFrame.GetUnitAxis(EAxis::Z)
	);

	Azimuth = FMath::RadiansToDegrees(Result.X);
	Elevation = FMath::RadiansToDegrees(Result.Y);
}

void UKismetMathLibrary::BreakRotIntoAxes(const FRotator& InRot, FVector& X, FVector& Y, FVector& Z)
{
	FRotationMatrix(InRot).GetScaledAxes(X, Y, Z);
}

FRotator UKismetMathLibrary::MakeRotationFromAxes(FVector Forward, FVector Right, FVector Up)
{
	Forward.Normalize();
	Right.Normalize();
	Up.Normalize();

	FMatrix RotMatrix(Forward, Right, Up, FVector::ZeroVector);

	return RotMatrix.Rotator();
}

FRotator UKismetMathLibrary::FindRelativeLookAtRotation(const FTransform& StartTransform, const FVector& TargetLocation)
{
	return NormalizedDeltaRotator(FindLookAtRotation(StartTransform.GetLocation(), TargetLocation), StartTransform.GetRotation().Rotator());
}

int32 UKismetMathLibrary::RandomIntegerFromStream(const FRandomStream& Stream, int32 Max)
{
	return Stream.RandHelper(Max);
}

int32 UKismetMathLibrary::RandomIntegerInRangeFromStream(const FRandomStream& Stream, int32 Min, int32 Max)
{
	return Stream.RandRange(Min, Max);
}

bool UKismetMathLibrary::InRange_IntInt(int32 Value, int32 Min, int32 Max, bool InclusiveMin, bool InclusiveMax)
{
	return ((InclusiveMin ? (Value >= Min) : (Value > Min)) && (InclusiveMax ? (Value <= Max) : (Value < Max)));
}

bool UKismetMathLibrary::InRange_Int64Int64(int64 Value, int64 Min, int64 Max, bool InclusiveMin, bool InclusiveMax)
{
	return ((InclusiveMin ? (Value >= Min) : (Value > Min)) && (InclusiveMax ? (Value <= Max) : (Value < Max)));
}

bool UKismetMathLibrary::RandomBoolFromStream(const FRandomStream& Stream)
{
	return Stream.RandBool();
}

float UKismetMathLibrary::RandomFloatFromStream(const FRandomStream& Stream)
{
	return Stream.FRand();
}

float UKismetMathLibrary::RandomFloatInRangeFromStream(const FRandomStream& Stream, float Min, float Max)
{
	return Min + (Max - Min) * RandomFloatFromStream(Stream);
}

FVector UKismetMathLibrary::RandomUnitVectorFromStream(const FRandomStream& Stream)
{
	return Stream.VRand();
}

FVector UKismetMathLibrary::RandomPointInBoundingBoxFromStream(const FRandomStream& Stream, const FVector Center, const FVector HalfSize)
{
	const FVector BoxMin = Center - HalfSize;
	const FVector BoxMax = Center + HalfSize;
	return Stream.RandPointInBox(FBox(BoxMin, BoxMax));
}

FVector UKismetMathLibrary::RandomPointInBoundingBoxFromStream_Box(const FRandomStream& Stream, const FBox Box)
{
	return Stream.RandPointInBox(Box);
}

FRotator UKismetMathLibrary::RandomRotatorFromStream(const FRandomStream& Stream, bool bRoll)
{
	FRotator RRot;
	RRot.Yaw = RandomFloatFromStream(Stream) * 360.f;
	RRot.Pitch = RandomFloatFromStream(Stream) * 360.f;

	if (bRoll)
	{
		RRot.Roll = RandomFloatFromStream(Stream) * 360.f;
	}
	else
	{
		RRot.Roll = 0;
	}
	return RRot;
}

void UKismetMathLibrary::ResetRandomStream(const FRandomStream& Stream)
{
	Stream.Reset();
}

void UKismetMathLibrary::SeedRandomStream(FRandomStream& Stream)
{
	Stream.GenerateNewSeed();
}

void UKismetMathLibrary::SetRandomStreamSeed(FRandomStream& Stream, int32 NewSeed)
{
	Stream.Initialize(NewSeed);
}

void UKismetMathLibrary::MinAreaRectangle(class UObject* WorldContextObject, const TArray<FVector>& InPoints, const FVector& SampleSurfaceNormal,
                                          FVector& OutRectCenter, FRotator& OutRectRotation, float& OutRectLengthX, float& OutRectLengthY, bool bDebugDraw)
{
	const int32 InVertsNum = InPoints.Num();
	
	// Bail if we receive an empty InVerts array
	if (InVertsNum == 0)
	{
		OutRectCenter = FVector::ZeroVector;
		OutRectRotation = FRotator::ZeroRotator;
		OutRectLengthX = 0.0f;
		OutRectLengthY = 0.0f;
		return;
	}

	// Use 'Newell's Method' to compute a robust 'best fit' plane from the input points
	FVector PlaneNormal = FVector::ZeroVector;
	for (int32 Vert0 = InVertsNum - 1, Vert1 = 0; Vert1 < InVertsNum; Vert0 = Vert1++)
	{
		const FVector& P0 = InPoints[Vert0];
		const FVector& P1 = InPoints[Vert1];
		PlaneNormal.X += (P1.Y - P0.Y) * (P0.Z + P1.Z);
		PlaneNormal.Y += (P1.Z - P0.Z) * (P0.X + P1.X);
		PlaneNormal.Z += (P1.X - P0.X) * (P0.Y + P1.Y);
	}
	PlaneNormal.Normalize();
	if ((PlaneNormal | SampleSurfaceNormal) < 0.f)
	{
		PlaneNormal = -PlaneNormal;
	}

	// Transform the sample points to 2D
	const FMatrix SurfaceNormalMatrix = FRotationMatrix::MakeFromZX(PlaneNormal, FVector(1.f, 0.f, 0.f));
	TArray<FVector> TransformedVerts;
	TransformedVerts.SetNumUninitialized(InVertsNum);
	const FMatrix InverseSurfaceNormalMatrix = SurfaceNormalMatrix.Inverse();
	for (int32 Idx = 0; Idx < InVertsNum; ++Idx)
	{
		const FVector TransformedVert = InverseSurfaceNormalMatrix.TransformVector(InPoints[Idx]);
		TransformedVerts[Idx] = {TransformedVert.X, TransformedVert.Y, 0.0f};
	}

	// Compute the convex hull of the sample points
	TArray<int32> PolyVertIndices;
	ConvexHull2D::ComputeConvexHullLegacy(TransformedVerts, PolyVertIndices);

	// Minimum area rectangle as computed by the exhaustive search algorithm in http://www.geometrictools.com/Documentation/MinimumAreaRectangle.pdf
	FVector Side0, Side1;
	[&PolyVertIndices, &TransformedVerts, &OutRectCenter, &Side0, &Side1]
	{
		float MinArea = TNumericLimits<float>::Max();

		for (int32 Idx0 = PolyVertIndices.Num() - 1, Idx1 = 0, Num = PolyVertIndices.Num(); Idx1 < Num; Idx0 = Idx1++)
		{
			const FVector Origin = TransformedVerts[PolyVertIndices[Idx0]];
			const FVector U0 = (TransformedVerts[PolyVertIndices[Idx1]] - Origin).GetSafeNormal2D();
			const FVector U1{-U0.Y, U0.X, 0.0f};

			float MinU0 = 0.0f;
			float MaxU0 = 0.0f;
			float MaxU1 = 0.0f;

			for (int VertIdx = 0; VertIdx < Num; ++VertIdx)
			{
				const FVector D = TransformedVerts[PolyVertIndices[VertIdx]] - Origin;
				const float Dot0 = U0 | D;
				if (Dot0 < MinU0)
				{
					MinU0 = Dot0;
				}
				else if (Dot0 > MaxU0)
				{
					MaxU0 = Dot0;
				}

				const float Dot1 = U1 | D;
				if (Dot1 > MaxU1)
				{
					MaxU1 = Dot1;
				}
			}

			const float Side0Length = MaxU0 - MinU0;
			const float Side1Length = MaxU1;
			const float CurrentArea = Side0Length * Side1Length;
			if (CurrentArea < MinArea)
			{
				MinArea = CurrentArea;
				Side0 = U0 * Side0Length;
				Side1 = U1 * Side1Length;
				OutRectCenter = Origin + ((MinU0 + MaxU0) / 2.0f) * U0 + (MaxU1 / 2.0f) * U1;
			}
		}
	}();

	Side0 = SurfaceNormalMatrix.TransformVector(Side0);
	Side1 = SurfaceNormalMatrix.TransformVector(Side1);
	OutRectCenter = SurfaceNormalMatrix.TransformPosition(OutRectCenter);
	OutRectRotation = FRotationMatrix::MakeFromZX(PlaneNormal, Side0).Rotator();
	OutRectLengthX = Side0.Length();
	OutRectLengthY = Side1.Length();

#if ENABLE_DRAW_DEBUG
	if (bDebugDraw)
	{
		if (const UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			DrawDebugSphere(World, OutRectCenter, 10.0f, 12, FColor::Yellow, true);
			DrawDebugCoordinateSystem(World, OutRectCenter, OutRectRotation, 50.0f, true);

			auto DrawLine = [World, &OutRectCenter, &Side0, &Side1](float A, float B, float C, float D)
			{
				DrawDebugLine(World, OutRectCenter + A * Side0 + B * Side1, OutRectCenter + C * Side0 + D * Side1, FColor::Blue, true, -1, 0, 1.0f);
			};

			DrawLine(-0.5f, -0.5f,  0.5f, -0.5f);
			DrawLine( 0.5f, -0.5f,  0.5f,  0.5f);
			DrawLine( 0.5f,  0.5f, -0.5f,  0.5f);
			DrawLine(-0.5f,  0.5f, -0.5f, -0.5f);
		}
	}
#endif
}

bool UKismetMathLibrary::IsPointInBox(FVector Point, FVector BoxOrigin, FVector BoxExtent)
{
	const FBox Box(BoxOrigin - BoxExtent, BoxOrigin + BoxExtent);
	return Box.IsInsideOrOn(Point);
}

bool UKismetMathLibrary::IsPointInBox_Box(FVector Point, FBox Box)
{
	return Box.IsInsideOrOn(Point);
}

FBox UKismetMathLibrary::MakeBoxWithOrigin(const FVector& Origin, const FVector& Extent)
{
	return FBox::BuildAABB(Origin, Extent);
}

FOrientedBox UKismetMathLibrary::MakeOrientedBox(FVector Origin, FRotator Orientation, FVector Extent)
{
	FOrientedBox ToRet;
	ToRet.Center = Origin;
	GetAxes(Orientation, ToRet.AxisX, ToRet.AxisY, ToRet.AxisZ);
	ToRet.ExtentX = Extent.X;
	ToRet.ExtentY = Extent.Y;
	ToRet.ExtentZ = Extent.Z;
	return ToRet;
}

bool UKismetMathLibrary::Box_IsInside(const FBox& InnerTest, const FBox& OuterTest)
{
	return OuterTest.IsInside(InnerTest);
}

bool UKismetMathLibrary::Box_IsInsideOrOn(const FBox& InnerTest, const FBox& OuterTest)
{
	return OuterTest.IsInsideOrOn(InnerTest);
}

bool UKismetMathLibrary::Box_IsPointInside(const FBox& Box, const FVector& Point)
{
	return Box.IsInside(Point);
}

bool UKismetMathLibrary::Box_Intersects(const FBox& A, const FBox& B)
{
	return A.Intersect(B);
}

FBox UKismetMathLibrary::Box_ExpandBy(const FBox& Box, const FVector& Negative, const FVector& Positive)
{
	return Box.ExpandBy(Negative, Positive);
}

FBox UKismetMathLibrary::Box_Overlap(const FBox& A, const FBox& B)
{
	return A.Overlap(B);
}

FVector UKismetMathLibrary::Box_GetClosestPointTo(const FBox& Box, const FVector& Point)
{
	return Box.GetClosestPointTo(Point);
}

double UKismetMathLibrary::GetBoxVolume(const FBox& InBox)
{
	return InBox.GetVolume();
}

FVector UKismetMathLibrary::GetBoxSize(const FBox& InBox)
{
	return InBox.GetSize();
}

FVector UKismetMathLibrary::GetBoxCenter(const FBox& InBox)
{
	return InBox.GetCenter();
}

bool UKismetMathLibrary::IsPointInBoxWithTransform(FVector Point, const FTransform& BoxWorldTransform, FVector BoxExtent)
{
	// Put point in component space
	const FVector PointInComponentSpace = BoxWorldTransform.InverseTransformPosition(Point);
	// Now it's just a normal point-in-box test, with a box at the origin.
	const FBox Box(-BoxExtent, BoxExtent);
	return Box.IsInsideOrOn(PointInComponentSpace);
}

bool UKismetMathLibrary::IsPointInBoxWithTransform_Box(FVector Point, const FTransform& BoxWorldTransform, FBox Box)
{
	// Put point in component space
	const FVector PointInComponentSpace = BoxWorldTransform.InverseTransformPosition(Point);
	// Now it's just a normal point-in-box test, with a box at the origin.
	Box = Box.MoveTo(PointInComponentSpace);
	return Box.IsInsideOrOn(PointInComponentSpace);
}

void UKismetMathLibrary::GetSlopeDegreeAngles(const FVector& MyRightYAxis, const FVector& FloorNormal, const FVector& UpVector, float& OutSlopePitchDegreeAngle, float& OutSlopeRollDegreeAngle)
{
	const FVector FloorZAxis = FloorNormal;
	const FVector FloorXAxis = MyRightYAxis ^ FloorZAxis;
	const FVector FloorYAxis = FloorZAxis ^ FloorXAxis;

	OutSlopePitchDegreeAngle = 90.f - FMath::RadiansToDegrees(FMath::Acos(FloorXAxis | UpVector));
	OutSlopeRollDegreeAngle = 90.f - FMath::RadiansToDegrees(FMath::Acos(FloorYAxis | UpVector));
}

bool UKismetMathLibrary::LinePlaneIntersection(const FVector& LineStart, const FVector& LineEnd, const FPlane& APlane, float& T, FVector& Intersection)
{
	FVector RayDir = LineEnd - LineStart;

	// Check ray is not parallel to plane
	if ((RayDir | APlane) == 0.0f)
	{
		T = -1.0f;
		Intersection = FVector::ZeroVector;
		return false;
	}

	T = ((APlane.W - (LineStart | APlane)) / (RayDir | APlane));

	// Check intersection is not outside line segment
	if (T < 0.0f || T > 1.0f)
	{
		Intersection = FVector::ZeroVector;
		return false;
	}

	// Calculate intersection point
	Intersection = LineStart + RayDir * T;

	return true;
}

bool UKismetMathLibrary::LinePlaneIntersection_OriginNormal(const FVector& LineStart, const FVector& LineEnd, FVector PlaneOrigin, FVector PlaneNormal, float& T, FVector& Intersection)
{
	FVector RayDir = LineEnd - LineStart;

	// Check ray is not parallel to plane
	if ((RayDir | PlaneNormal) == 0.0f)
	{
		T = -1.0f;
		Intersection = FVector::ZeroVector;
		return false;
	}

	T = (((PlaneOrigin - LineStart) | PlaneNormal) / (RayDir | PlaneNormal));

	// Check intersection is not outside line segment
	if (T < 0.0f || T > 1.0f)
	{
		Intersection = FVector::ZeroVector;
		return false;
	}

	// Calculate intersection point
	Intersection = LineStart + RayDir * T;

	return true;
}

void UKismetMathLibrary::BreakRandomStream(const FRandomStream& InRandomStream, int32& InitialSeed)
{
	InitialSeed = InRandomStream.GetInitialSeed();
}

FRandomStream UKismetMathLibrary::MakeRandomStream(int32 InitialSeed)
{
	return FRandomStream(InitialSeed);
}

FRandomStream UKismetMathLibrary::MakeRandomStreamFromLocation(const FVector& Location, float DistanceInterval, bool bIncludeZ)
{
	uint32 Seed = GetTypeHash(DistanceInterval);
	if (FMath::IsNearlyZero(DistanceInterval))
	{
		Seed = HashCombine(Seed, GetTypeHash(Location.X));
		Seed = HashCombine(Seed, GetTypeHash(Location.Y));
		if (bIncludeZ)
		{
			Seed = HashCombine(Seed, GetTypeHash(Location.Z));
		}
	}
	else
	{
		FVector GridLocation = Location / DistanceInterval;
		const int32 X = static_cast<int32>(GridLocation.X);
		const int32 Y = static_cast<int32>(GridLocation.Y);
		Seed = HashCombine(Seed, GetTypeHash(X));
		Seed = HashCombine(Seed, GetTypeHash(Y));
		if (bIncludeZ)
		{
			const int32 Z = static_cast<int32>(GridLocation.Z);
			Seed = HashCombine(Seed, GetTypeHash(Z));
		}
	}

	return FRandomStream(Seed);
}

FVector UKismetMathLibrary::RandomUnitVectorInConeInRadiansFromStream(const FRandomStream& Stream, const FVector& ConeDir, float ConeHalfAngleInRadians)
{
	return Stream.VRandCone(ConeDir, ConeHalfAngleInRadians);
}

FVector UKismetMathLibrary::RandomUnitVectorInEllipticalConeInRadiansFromStream(const FRandomStream& Stream, const FVector& ConeDir, float MaxYawInRadians, float MaxPitchInRadians)
{
	return Stream.VRandCone(ConeDir, MaxYawInRadians, MaxPitchInRadians);
}

float UKismetMathLibrary::PerlinNoise1D(const float Value)
{
	return FMath::PerlinNoise1D(Value);
}

float UKismetMathLibrary::WeightedMovingAverage_Float(float CurrentSample, float PreviousSample, float Weight)
{
	return FMath::WeightedMovingAverage(CurrentSample, PreviousSample, Weight);
}

FVector UKismetMathLibrary::WeightedMovingAverage_FVector(FVector CurrentSample, FVector PreviousSample, float Weight)
{
	FVector OutVector;
	OutVector.X = FMath::WeightedMovingAverage<FVector::FReal>(CurrentSample.X, PreviousSample.X, Weight);
	OutVector.Y = FMath::WeightedMovingAverage<FVector::FReal>(CurrentSample.Y, PreviousSample.Y, Weight);
	OutVector.Z = FMath::WeightedMovingAverage<FVector::FReal>(CurrentSample.Z, PreviousSample.Z, Weight);
	return OutVector;
}

FRotator UKismetMathLibrary::WeightedMovingAverage_FRotator(FRotator CurrentSample, FRotator PreviousSample, float Weight)
{
	FRotator OutRotator;
	OutRotator.Yaw = FMath::Clamp(FMath::WeightedMovingAverage<FRotator::FReal>(CurrentSample.Yaw, PreviousSample.Yaw, Weight), -180.f, 180.f);
	OutRotator.Pitch = FMath::Clamp(FMath::WeightedMovingAverage<FRotator::FReal>(CurrentSample.Pitch, PreviousSample.Pitch, Weight), -180.f, 180.f);
	OutRotator.Roll = FMath::Clamp(FMath::WeightedMovingAverage<FRotator::FReal>(CurrentSample.Roll, PreviousSample.Roll, Weight), -180.f, 180.f);
	return OutRotator;
}

float UKismetMathLibrary::DynamicWeightedMovingAverage_Float(float CurrentSample, float PreviousSample, float MaxDistance, float MinWeight, float MaxWeight)
{
	return FMath::DynamicWeightedMovingAverage(CurrentSample, PreviousSample, MaxDistance, MinWeight, MaxWeight);
}

FVector UKismetMathLibrary::DynamicWeightedMovingAverage_FVector(FVector CurrentSample, FVector PreviousSample, float MaxDistance, float MinWeight, float MaxWeight)
{
	FVector OutVector;
	OutVector.X = FMath::DynamicWeightedMovingAverage<FVector::FReal>(CurrentSample.X, PreviousSample.X, MaxDistance, MinWeight, MaxWeight);
	OutVector.Y = FMath::DynamicWeightedMovingAverage<FVector::FReal>(CurrentSample.Y, PreviousSample.Y, MaxDistance, MinWeight, MaxWeight);
	OutVector.Z = FMath::DynamicWeightedMovingAverage<FVector::FReal>(CurrentSample.Z, PreviousSample.Z, MaxDistance, MinWeight, MaxWeight);
	return OutVector;
}

FRotator UKismetMathLibrary::DynamicWeightedMovingAverage_FRotator(FRotator CurrentSample, FRotator PreviousSample, float MaxDistance, float MinWeight, float MaxWeight)
{
	FRotator OutRotator;
	OutRotator.Yaw = FMath::Clamp(FMath::DynamicWeightedMovingAverage<FRotator::FReal>(CurrentSample.Yaw, PreviousSample.Yaw, MaxDistance, MinWeight, MaxWeight), -180.f, 180.f);
	OutRotator.Pitch = FMath::Clamp(FMath::DynamicWeightedMovingAverage<FRotator::FReal>(CurrentSample.Pitch, PreviousSample.Pitch, MaxDistance, MinWeight, MaxWeight), -180.f, 180.f);
	OutRotator.Roll = FMath::Clamp(FMath::DynamicWeightedMovingAverage<FRotator::FReal>(CurrentSample.Roll, PreviousSample.Roll, MaxDistance, MinWeight, MaxWeight), -180.f, 180.f);
	return OutRotator;
}

FIntPoint UKismetMathLibrary::Convert1DTo2D(int32 Index1D, int32 XSize)
{
	if (Index1D < 0)
	{
		FFrame::KismetExecutionMessage(TEXT("Index1D must be non-negative: Convert1DTo2D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return FIntPoint::ZeroValue;
	}

	if (XSize <= 0)
	{
		FFrame::KismetExecutionMessage(TEXT("XSize must be positive: Convert1DTo2D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return FIntPoint::ZeroValue;
	}

	int32 X = Index1D % XSize;
	int32 Y = Index1D / XSize;

	return FIntPoint{ X,Y };
}

FIntVector UKismetMathLibrary::Convert1DTo3D(int32 Index1D, int32 XSize, int32 YSize)
{
	if (Index1D < 0)
	{
		FFrame::KismetExecutionMessage(TEXT("Index1D must be non-negative: Convert1DTo3D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return FIntVector::ZeroValue;
	}

	if (XSize <= 0 || YSize <= 0)
	{
		FFrame::KismetExecutionMessage(TEXT("XSize and YSize must be positive: Convert1DTo3D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return FIntVector::ZeroValue;
	}

	int32 X = Index1D % XSize;
	int32 Y = (Index1D / XSize) % YSize;
	int32 Z = (Index1D / (XSize * YSize));

	return FIntVector{ X, Y, Z };
}

int32 UKismetMathLibrary::Convert2DTo1D(const FIntPoint& Index2D, int32 XSize)
{
	const bool bInvalidBounds =
		Index2D.X < 0 || 
		Index2D.X >= XSize || 
		Index2D.Y < 0
	;

	if (bInvalidBounds)
	{
		FFrame::KismetExecutionMessage(TEXT("Index2D out of bounds: Convert2DTo1D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return 0;
	}

	if (XSize <= 0)
	{
		FFrame::KismetExecutionMessage(TEXT("XSize must be positive: Convert2DTo1D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return 0;
	}

	return Index2D.X + (Index2D.Y * XSize);
}

int32 UKismetMathLibrary::Convert3DTo1D(const FIntVector& Index3D, int32 XSize, int32 YSize)
{
	const bool bInvalidBounds =
		Index3D.X < 0 || 
		Index3D.X >= XSize || 
		Index3D.Y < 0 || 
		Index3D.Y >= YSize || 
		Index3D.Z < 0
	;

	if (bInvalidBounds)
	{
		FFrame::KismetExecutionMessage(TEXT("Index3D out of bounds: Convert3DTo1D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return 0;
	}

	if (XSize <= 0 || YSize <= 0)
	{
		FFrame::KismetExecutionMessage(TEXT("XSize and YSize must be positive: Convert3DTo1D"), ELogVerbosity::Warning, InvalidIndexConversionParameterWarning);
		return 0;
	}

	return Index3D.X + (Index3D.Y * XSize) + (Index3D.Z * XSize * YSize);
}

#undef LOCTEXT_NAMESPACE

