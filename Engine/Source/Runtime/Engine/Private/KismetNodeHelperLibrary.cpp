// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetNodeHelperLibrary.h"
#include "Misc/RuntimeErrors.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetNodeHelperLibrary)

UKismetNodeHelperLibrary::UKismetNodeHelperLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UKismetNodeHelperLibrary::BitIsMarked(int32 Data, int32 Index)
{
	if (Index < 32)
	{
		return ((1 << Index) & Data) > 0;
	}
	return false;
}

void UKismetNodeHelperLibrary::MarkBit(int32& Data, int32 Index)
{
	if (Index < 32)
	{
		Data |= (1 << Index);
	}
}

void UKismetNodeHelperLibrary::ClearBit(int32& Data, int32 Index)
{
	if (Index < 32)
	{
		Data &= ~(1 << Index);
	}
}

void UKismetNodeHelperLibrary::ClearAllBits(int32& Data)
{
	Data = 0;
}

bool UKismetNodeHelperLibrary::HasUnmarkedBit(int32 Data, int32 NumBits)
{
	if (NumBits < 32)
	{
		for (int32 Idx = 0; Idx < NumBits; Idx++)
		{
			if (!BitIsMarked(Data, Idx))
			{
				return true;
			}
		}
	}
	return false;
}

bool UKismetNodeHelperLibrary::HasMarkedBit(int32 Data, int32 NumBits)
{
	if (NumBits < 32)
	{
		for (int32 Idx = 0; Idx < NumBits; Idx++)
		{
			if (BitIsMarked(Data, Idx))
			{
				return true;
			}
		}
	}
	return false;
}

int32 UKismetNodeHelperLibrary::GetUnmarkedBit(int32 Data, int32 StartIdx, int32 NumBits, bool bRandom)
{
	if (bRandom)
	{
		return GetRandomUnmarkedBit(Data, StartIdx, NumBits);
	}
	else
	{
		return GetFirstUnmarkedBit(Data, StartIdx, NumBits);
	}
}

int32 UKismetNodeHelperLibrary::GetRandomUnmarkedBit(int32 Data, int32 StartIdx, int32 NumBits)
{
	if (NumBits < 32)
	{
		if (HasUnmarkedBit(Data, NumBits))
		{
			int32 Idx;
			if (StartIdx >= 0 && StartIdx < NumBits)
			{
				Idx = StartIdx;
			}
			else
			{
				Idx = FMath::RandRange(0, NumBits-1);
			}

			do 
			{
				if (((1 << Idx) & Data) == 0)
				{
					Data |= (1 << Idx);
					return Idx;
				}

				Idx = FMath::RandRange(0, NumBits-1);
			}
			while (1);
		}
	}

	return INDEX_NONE;
}

int32 UKismetNodeHelperLibrary::GetFirstUnmarkedBit(int32 Data, int32 StartIdx, int32 NumBits)
{
	if (NumBits < 32)
	{
		if (HasUnmarkedBit(Data, NumBits))
		{
			int32 Idx = 0;
			if (StartIdx >= 0 && StartIdx < NumBits)
			{
				Idx = StartIdx;
			}

			do 
			{
				if (((1 << Idx) & Data) == 0)
				{
					Data |= (1 << Idx);
					return Idx;
				}
				Idx = (Idx + 1) % NumBits;
			}
			while (1);
		}
	}

	return INDEX_NONE;
}

FName UKismetNodeHelperLibrary::GetEnumeratorName(const UEnum* Enum, uint8 EnumeratorValue)
{
	return (nullptr != Enum) ? Enum->GetNameByValue(EnumeratorValue) : NAME_None;
}

FString UKismetNodeHelperLibrary::GetEnumeratorUserFriendlyName(const UEnum* Enum, uint8 EnumeratorValue)
{
	if (nullptr != Enum)
	{
		return Enum->GetValueOrBitfieldAsDisplayNameText(EnumeratorValue).ToString();
	}

	return FName().ToString();
}

uint8 UKismetNodeHelperLibrary::GetValidValue(const UEnum* Enum, uint8 EnumeratorValue)
{
	if (ensureAsRuntimeWarning(Enum != nullptr))
	{
		if (Enum->IsValidEnumValueOrBitfield(EnumeratorValue))
		{
			return EnumeratorValue;
		}
		return Enum->GetMaxEnumValue();
	}
	return INDEX_NONE;
}

uint8 UKismetNodeHelperLibrary::GetEnumeratorValueFromIndex(const UEnum* Enum, uint8 EnumeratorIndex)
{
	if (NULL != Enum)
	{
		return Enum->GetValueByIndex(EnumeratorIndex);
	}
	return INDEX_NONE;
}
