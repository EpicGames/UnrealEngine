// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetRenderingLibrary.h"
#include "Camera/CameraTypes.h"
#include "HAL/FileManager.h"
#include "Components/SkinnedMeshComponent.h"
#include "Materials/MaterialInterface.h"
#include "Engine/World.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "Serialization/BufferArchive.h"
#include "RHIContext.h"
#include "RHIUtilities.h"
#include "RenderingThread.h"
#include "RenderCaptureInterface.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "TextureResource.h"
#include "Logging/MessageLog.h"
#include "Engine/GameViewportClient.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/TextureRenderTargetCube.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "ImageUtils.h"
#include "ClearQuad.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCube.h"
#include "Engine/VolumeTexture.h"
#include "UObject/Package.h"
#include "EngineModule.h"
#include "SceneInterface.h"
#include "ShaderPipelineCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetRenderingLibrary)

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "PackageTools.h"
#endif

//////////////////////////////////////////////////////////////////////////
// UKismetRenderingLibrary

#define LOCTEXT_NAMESPACE "KismetRenderingLibrary"

UKismetRenderingLibrary::UKismetRenderingLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UKismetRenderingLibrary::ClearRenderTarget2D(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, FLinearColor ClearColor)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (TextureRenderTarget
		&& TextureRenderTarget->GetResource()
		&& World)
	{
		FTextureRenderTargetResource* RenderTargetResource = TextureRenderTarget->GameThread_GetRenderTargetResource();
		ENQUEUE_RENDER_COMMAND(ClearRTCommand)(
			[RenderTargetResource, ClearColor](FRHICommandList& RHICmdList)
		{
			FRHIRenderPassInfo RPInfo(RenderTargetResource->GetRenderTargetTexture(), ERenderTargetActions::DontLoad_Store);
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::Unknown, ERHIAccess::RTV));
			RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearRT"));
			DrawClearQuad(RHICmdList, ClearColor);
			RHICmdList.EndRenderPass();

			RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVMask));
		});
	}
}

UTextureRenderTarget2D* UKismetRenderingLibrary::CreateRenderTarget2D(UObject* WorldContextObject, int32 Width, int32 Height, ETextureRenderTargetFormat Format, FLinearColor ClearColor, bool bAutoGenerateMipMaps, bool bSupportUAVs)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (Width > 0 && Height > 0 && World)
	{
		UTextureRenderTarget2D* NewRenderTarget2D = NewObject<UTextureRenderTarget2D>(WorldContextObject);
		check(NewRenderTarget2D);
		NewRenderTarget2D->RenderTargetFormat = Format;
		NewRenderTarget2D->ClearColor = ClearColor;
		NewRenderTarget2D->bAutoGenerateMips = bAutoGenerateMipMaps;
		NewRenderTarget2D->bCanCreateUAV = bSupportUAVs;
		NewRenderTarget2D->InitAutoFormat(Width, Height);	
		NewRenderTarget2D->UpdateResourceImmediate(true);

		return NewRenderTarget2D; 
	}

	return nullptr;
}

UTextureRenderTarget2DArray* UKismetRenderingLibrary::CreateRenderTarget2DArray(UObject* WorldContextObject, int32 Width, int32 Height, int32 Slices, ETextureRenderTargetFormat Format, FLinearColor ClearColor, bool bAutoGenerateMipMaps, bool bSupportUAVs)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (Width > 0 && Height > 0 && Slices > 0 && World)
	{
		UTextureRenderTarget2DArray* NewRenderTarget = NewObject<UTextureRenderTarget2DArray>(WorldContextObject);
		check(NewRenderTarget);
		NewRenderTarget->ClearColor = ClearColor;
		NewRenderTarget->bCanCreateUAV = bSupportUAVs;
		NewRenderTarget->Init(Width, Height, Slices, GetPixelFormatFromRenderTargetFormat(Format));
		NewRenderTarget->UpdateResourceImmediate(true);
		return NewRenderTarget;
	}

	return nullptr;
}

UTextureRenderTargetVolume* UKismetRenderingLibrary::CreateRenderTargetVolume(UObject* WorldContextObject, int32 Width, int32 Height, int32 Depth, ETextureRenderTargetFormat Format, FLinearColor ClearColor, bool bAutoGenerateMipMaps, bool bSupportUAVs)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (Width > 0 && Height > 0 && Depth > 0 && World)
	{
		UTextureRenderTargetVolume* NewRenderTarget = NewObject<UTextureRenderTargetVolume>(WorldContextObject);
		check(NewRenderTarget);
		NewRenderTarget->ClearColor = ClearColor;
		NewRenderTarget->bCanCreateUAV = bSupportUAVs;
		NewRenderTarget->Init(Width, Height, Depth, GetPixelFormatFromRenderTargetFormat(Format));
		NewRenderTarget->UpdateResourceImmediate(true);
		return NewRenderTarget;
	}

	return nullptr;
}

// static 
void UKismetRenderingLibrary::ReleaseRenderTarget2D(UTextureRenderTarget2D* TextureRenderTarget)
{
	if (!TextureRenderTarget)
	{
		return;
	}

	TextureRenderTarget->ReleaseResource();
}

void UKismetRenderingLibrary::ResizeRenderTarget2D(UTextureRenderTarget2D* TextureRenderTarget, int32 Width, int32 Height)
{
	if (!TextureRenderTarget)
	{
		return;
	}

	// Resize function silently fails if either dimension isn't positive, so check for that here so we can warn the caller
	if ((Width > 0) && (Height > 0))
	{
		TextureRenderTarget->ResizeTarget(Width, Height);
	}
	else
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("ResizeRenderTarget2D_InvalidDimensions", "ResizeRenderTarget2D: Dimensions must be positive."));
	}
}

void UKismetRenderingLibrary::DrawMaterialToRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, UMaterialInterface* Material)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DrawMaterialToRenderTarget);

	if (!FApp::CanEverRender())
	{
		// Returning early to avoid warnings about missing resources that are expected when CanEverRender is false.
		return;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (!World)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("DrawMaterialToRenderTarget_InvalidWorldContextObject", "DrawMaterialToRenderTarget: WorldContextObject is not valid."));
	}
	else if (!Material)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("DrawMaterialToRenderTarget_InvalidMaterial", "DrawMaterialToRenderTarget[{0}]: Material must be non-null."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else if (!TextureRenderTarget)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("DrawMaterialToRenderTarget_InvalidTextureRenderTarget", "DrawMaterialToRenderTarget[{0}]: TextureRenderTarget must be non-null."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else if (!TextureRenderTarget->GetResource())
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("DrawMaterialToRenderTarget_ReleasedTextureRenderTarget", "DrawMaterialToRenderTarget[{0}]: render target has been released."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else
	{
		// This is a user-facing function, so we'd rather make sure that shaders are ready by the time we render, in order to ensure we don't draw with a fallback material :
		Material->EnsureIsComplete();

		World->FlushDeferredParameterCollectionInstanceUpdates();

		FTextureRenderTargetResource* RenderTargetResource = TextureRenderTarget->GameThread_GetRenderTargetResource();

		UCanvas* Canvas = World->GetCanvasForDrawMaterialToRenderTarget();

		FCanvas RenderCanvas(
			RenderTargetResource, 
			nullptr, 
			World,
			World->GetFeatureLevel());

		Canvas->Init(TextureRenderTarget->SizeX, TextureRenderTarget->SizeY, nullptr, &RenderCanvas);

		{
			RHI_BREADCRUMB_EVENT_GAMETHREAD_F("DrawMaterialToRenderTarget", "DrawMaterialToRenderTarget: %s", TextureRenderTarget->GetFName());

			ENQUEUE_RENDER_COMMAND(FlushDeferredResourceUpdateCommand)(
				[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
				{
					RenderTargetResource->FlushDeferredResourceUpdate(RHICmdList);
				});

			Canvas->K2_DrawMaterial(Material, FVector2D(0, 0), FVector2D(TextureRenderTarget->SizeX, TextureRenderTarget->SizeY), FVector2D(0, 0));

			RenderCanvas.Flush_GameThread();
			Canvas->Canvas = nullptr;

			//UpdateResourceImmediate must be called here to ensure mips are generated.
			TextureRenderTarget->UpdateResourceImmediate(false);
		}
	}
}

void UKismetRenderingLibrary::ExportRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, const FString& FilePath, const FString& FileName)
{
	FString TotalFileName = FPaths::Combine(*FilePath, *FileName);
	FText PathError;
	FPaths::ValidatePath(TotalFileName, &PathError);


	if (!TextureRenderTarget)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ExportRenderTarget_InvalidTextureRenderTarget", "ExportRenderTarget[{0}]: TextureRenderTarget must be non-null."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else if (!TextureRenderTarget->GetResource())
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ExportRenderTarget_ReleasedTextureRenderTarget", "ExportRenderTarget[{0}]: render target has been released."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else if (!PathError.IsEmpty())
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ExportRenderTarget_InvalidFilePath", "ExportRenderTarget[{0}]: Invalid file path provided: '{1}'"), FText::FromString(GetPathNameSafe(WorldContextObject)), PathError));
	}
	else if (FileName.IsEmpty())
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ExportRenderTarget_InvalidFileName", "ExportRenderTarget[{0}]: FileName must be non-empty."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else
	{
		FArchive* Ar = IFileManager::Get().CreateFileWriter(*TotalFileName);

		if (Ar)
		{
			FBufferArchive Buffer;

			bool bSuccess = false;
			if ((TextureRenderTarget->RenderTargetFormat == RTF_RGBA16f) || (TextureRenderTarget->RenderTargetFormat == RTF_RGBA32f))
			{
				// Note == is case insensitive
				if (FPaths::GetExtension(TotalFileName) == TEXT("HDR"))
				{
					bSuccess = FImageUtils::ExportRenderTarget2DAsHDR(TextureRenderTarget, Buffer);
				}
				else
				{
					bSuccess = FImageUtils::ExportRenderTarget2DAsEXR(TextureRenderTarget, Buffer);
				}
				
			}
			else
			{
				bSuccess = FImageUtils::ExportRenderTarget2DAsPNG(TextureRenderTarget, Buffer);
			}

			if (bSuccess)
			{
				Ar->Serialize(const_cast<uint8*>(Buffer.GetData()), Buffer.Num());
			}

			delete Ar;
		}
		else
		{
			FMessageLog("Blueprint").Warning(LOCTEXT("ExportRenderTarget_FileWriterFailedToCreate", "ExportRenderTarget: FileWrite failed to create."));
		}
	}
}

EPixelFormat ReadRenderTargetHelper(
	TArray<FColor>& OutLDRValues,
	TArray<FLinearColor>& OutHDRValues,
	UObject* WorldContextObject,
	UTextureRenderTarget2D* TextureRenderTarget,
	int32 X,
	int32 Y,
	int32 Width,
	int32 Height,
	bool bNormalize = true)
{
	EPixelFormat OutFormat = PF_Unknown;

	if (!TextureRenderTarget)
	{
		return OutFormat;
	}

	FTextureRenderTarget2DResource* RTResource = (FTextureRenderTarget2DResource*)TextureRenderTarget->GameThread_GetRenderTargetResource();
	if (!RTResource)
	{
		return OutFormat;
	}

	X = FMath::Clamp(X, 0, TextureRenderTarget->SizeX - 1);
	Y = FMath::Clamp(Y, 0, TextureRenderTarget->SizeY - 1);
	Width = FMath::Clamp(Width, 1, TextureRenderTarget->SizeX);
	Height = FMath::Clamp(Height, 1, TextureRenderTarget->SizeY);
	Width = Width - FMath::Max(X + Width - TextureRenderTarget->SizeX, 0);
	Height = Height - FMath::Max(Y + Height - TextureRenderTarget->SizeY, 0);

	FIntRect SampleRect(X, Y, X + Width, Y + Height);

	FReadSurfaceDataFlags ReadSurfaceDataFlags = bNormalize ? FReadSurfaceDataFlags() : FReadSurfaceDataFlags(RCM_MinMax);

	FRenderTarget* RenderTarget = TextureRenderTarget->GameThread_GetRenderTargetResource();
	OutFormat = TextureRenderTarget->GetFormat();

	const int32 NumPixelsToRead = Width * Height;

	switch (OutFormat)
	{
	case PF_B8G8R8A8:
		if (!RenderTarget->ReadPixels(OutLDRValues, ReadSurfaceDataFlags, SampleRect))
		{
			OutFormat = PF_Unknown;
		}
		else
		{
			check(OutLDRValues.Num() == NumPixelsToRead);
		}
		break;
	case PF_FloatRGBA:
	case PF_A32B32G32R32F:
		if (!RenderTarget->ReadLinearColorPixels(OutHDRValues, ReadSurfaceDataFlags, SampleRect))
		{
			OutFormat = PF_Unknown;
		}
		else
		{
			check(OutHDRValues.Num() == NumPixelsToRead);
		}
		break;
	default:
		OutFormat = PF_Unknown;
		break;
	}

	return OutFormat;
}

FColor UKismetRenderingLibrary::ReadRenderTargetUV(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, float U, float V)
{
	if (!TextureRenderTarget)
	{
		return FColor::Red;
	}

	U = FMath::Clamp(U, 0.0f, 1.0f);
	V = FMath::Clamp(V, 0.0f, 1.0f);
	int32 XPos = U * (float)TextureRenderTarget->SizeX;
	int32 YPos = V * (float)TextureRenderTarget->SizeY;

	return ReadRenderTargetPixel(WorldContextObject, TextureRenderTarget, XPos, YPos);
}

FColor UKismetRenderingLibrary::ReadRenderTargetPixel(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, int32 X, int32 Y)
{	
	TArray<FColor> Samples;
	TArray<FLinearColor> LinearSamples;

	switch (ReadRenderTargetHelper(Samples, LinearSamples, WorldContextObject, TextureRenderTarget, X, Y, 1, 1))
	{
	case PF_B8G8R8A8:
		check(Samples.Num() == 1 && LinearSamples.Num() == 0);
		return Samples[0];
	case PF_FloatRGBA:
	case PF_A32B32G32R32F:
		check(Samples.Num() == 0 && LinearSamples.Num() == 1);
		return LinearSamples[0].ToFColor(true);
	case PF_Unknown:
	default:
		return FColor::Red;
	}
}

bool UKismetRenderingLibrary::ReadRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, TArray<FColor>& OutSamples, bool bNormalize)
{
	if (WorldContextObject != nullptr && TextureRenderTarget != nullptr)
	{
		const int32 NumSamples = TextureRenderTarget->SizeX * TextureRenderTarget->SizeY;

		OutSamples.Reset(NumSamples);

		TArray<FLinearColor> LinearSamples;
		LinearSamples.Reserve(NumSamples);

		switch (ReadRenderTargetHelper(OutSamples, LinearSamples, WorldContextObject, TextureRenderTarget, 0, 0, TextureRenderTarget->SizeX, TextureRenderTarget->SizeY, bNormalize))
		{
		case PF_B8G8R8A8:
			check(OutSamples.Num() == NumSamples && LinearSamples.Num() == 0);
			return true;
		case PF_FloatRGBA:
		case PF_A32B32G32R32F:
			check(OutSamples.Num() == 0 && LinearSamples.Num() == NumSamples);
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				OutSamples.Add(LinearSamples[SampleIndex].ToFColor(true));
			}
			return true;
		case PF_Unknown:
		default:
			return false;
		}
	}

	return false;
}

FLinearColor UKismetRenderingLibrary::ReadRenderTargetRawPixel(UObject * WorldContextObject, UTextureRenderTarget2D * TextureRenderTarget, int32 X, int32 Y, bool bNormalize)
{
	TArray<FColor> Samples;
	TArray<FLinearColor> LinearSamples;

	switch (ReadRenderTargetHelper(Samples, LinearSamples, WorldContextObject, TextureRenderTarget, X, Y, 1, 1, bNormalize))
	{
	case PF_B8G8R8A8:
		check(Samples.Num() == 1 && LinearSamples.Num() == 0);
		return FLinearColor(float(Samples[0].R), float(Samples[0].G), float(Samples[0].B), float(Samples[0].A));
	case PF_FloatRGBA:
	case PF_A32B32G32R32F:
		check(Samples.Num() == 0 && LinearSamples.Num() == 1);
		return LinearSamples[0];
	case PF_Unknown:
	default:
		return FLinearColor::Red;
	}
}

ENGINE_API TArray<FLinearColor> UKismetRenderingLibrary::ReadRenderTargetRawPixelArea(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, int32 MinX, int32 MinY, int32 MaxX, int32 MaxY, bool bNormalize /*= true*/)
{
	TArray<FColor> Samples;
	TArray<FLinearColor> LinearSamples;

	switch (ReadRenderTargetHelper(Samples, LinearSamples, WorldContextObject, TextureRenderTarget, MinX, MinY, MaxX, MaxY, bNormalize))
	{
	case PF_B8G8R8A8:
		check(Samples.Num() > 0 && LinearSamples.Num() == 0);
		LinearSamples.SetNum(Samples.Num());
		for (int Idx = 0; Idx < Samples.Num(); Idx++)
		{
			LinearSamples[Idx] = FLinearColor(float(Samples[Idx].R), float(Samples[Idx].G), float(Samples[Idx].B), float(Samples[Idx].A));
		}
		return LinearSamples;

	case PF_FloatRGBA:
	case PF_A32B32G32R32F:
		check(Samples.Num() == 0 && LinearSamples.Num() > 0);
		return LinearSamples;

	case PF_Unknown:

	default:
		return TArray<FLinearColor>();
	}
}

FLinearColor UKismetRenderingLibrary::ReadRenderTargetRawUV(UObject * WorldContextObject, UTextureRenderTarget2D * TextureRenderTarget, float U, float V, bool bNormalize)
{
	if (!TextureRenderTarget)
	{
		return FLinearColor::Red;
	}

	U = FMath::Clamp(U, 0.0f, 1.0f);
	V = FMath::Clamp(V, 0.0f, 1.0f);
	int32 XPos = U * (float)TextureRenderTarget->SizeX;
	int32 YPos = V * (float)TextureRenderTarget->SizeY;

	return ReadRenderTargetRawPixel(WorldContextObject, TextureRenderTarget, XPos, YPos, bNormalize);
}

bool UKismetRenderingLibrary::ReadRenderTargetRaw(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, TArray<FLinearColor>& OutLinearSamples, bool bNormalize)
{
	if (WorldContextObject != nullptr && TextureRenderTarget != nullptr)
	{
		const int32 NumSamples = TextureRenderTarget->SizeX * TextureRenderTarget->SizeY;

		OutLinearSamples.Reset(NumSamples);
		TArray<FColor> Samples;
		Samples.Reserve(NumSamples);

		switch (ReadRenderTargetHelper(Samples, OutLinearSamples, WorldContextObject, TextureRenderTarget, 0, 0, TextureRenderTarget->SizeX, TextureRenderTarget->SizeY, bNormalize))
		{
		case PF_B8G8R8A8:
			check(Samples.Num() == NumSamples && OutLinearSamples.Num() == 0);
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				OutLinearSamples.Add(FLinearColor(float(Samples[SampleIndex].R), float(Samples[SampleIndex].G), float(Samples[SampleIndex].B), float(Samples[SampleIndex].A)));
			}
			return true;
		case PF_FloatRGBA:
		case PF_A32B32G32R32F:
			check(Samples.Num() == 0 && OutLinearSamples.Num() == NumSamples);
			return true;
		case PF_Unknown:
		default:
			return false;
		}
	}

	return false;
}

ENGINE_API TArray<FLinearColor> UKismetRenderingLibrary::ReadRenderTargetRawUVArea(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, FBox2D Area, bool bNormalize /*= true*/)
{

	if (!TextureRenderTarget)
	{
		return TArray<FLinearColor>();
	}

	int32 MinX = FMath::Clamp(Area.Min.X, 0.f, 1.f) * (float)TextureRenderTarget->SizeX;
	int32 MinY = FMath::Clamp(Area.Min.Y, 0.f, 1.f) * (float)TextureRenderTarget->SizeY;
	int32 MaxX = FMath::Clamp(Area.Max.X, 0.f, 1.f) * (float)TextureRenderTarget->SizeX;
	int32 MaxY = FMath::Clamp(Area.Max.Y, 0.f, 1.f) * (float)TextureRenderTarget->SizeY;

	return ReadRenderTargetRawPixelArea(WorldContextObject, TextureRenderTarget, MinX, MinY, MaxX, MaxY, bNormalize);
}

namespace UE::Kismet::RenderingLibrary
{
void ConvertRenderTargetToTextureEditorOnly(UObject* WorldContextObject, UTextureRenderTarget* InRenderTarget, UTexture* InTexture)
{
#if WITH_EDITOR
	if (InRenderTarget == nullptr)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ConvertRenderTargetToTexture_InvalidRenderTarget", "ConvertRenderTargetToTextureEditorOnly[{0}]: RenderTarget must be non-null."), FText::FromString(GetPathNameSafe(WorldContextObject))));
		return;
	}
	
	if (InRenderTarget->GetResource() == nullptr)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ConvertRenderTargetToTexture_ReleasedTextureRenderTarget", "ConvertRenderTargetToTextureEditorOnly[{0}]: render target has been released."), FText::FromString(GetPathNameSafe(WorldContextObject))));
		return;
	}
	
	if (InTexture == nullptr)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ConvertRenderTargetToTexture_InvalidTexture", "ConvertRenderTargetToTextureEditorOnly[{0}]: Texture must be non-null."), FText::FromString(GetPathNameSafe(WorldContextObject))));
		return;
	}

	// We don't want to create a new texture here, we already have one.  We want to preserve any configuration
	// by a developer and just update the minimal pixel data and format data.
	FText ErrorMessage;
	if (!InRenderTarget->CanConvertToTexture(&ErrorMessage))
	{
		FMessageLog("Blueprint").Warning(ErrorMessage);
		return;
	}

	if (!InRenderTarget->UpdateTexture(InTexture, /*InFlags = */CTF_Default, /*InAlphaOverride = */nullptr, /*InOnTextureChangingDelegate = */[](UTexture*){}, &ErrorMessage))
	{
		FMessageLog("Blueprint").Warning(ErrorMessage);
		return;
	}

	InTexture->Modify();
	InTexture->MarkPackageDirty();
	InTexture->PostEditChange();
	InTexture->UpdateResource();

#else // WITH_EDITOR
	FMessageLog("Blueprint").Error(LOCTEXT("ConvertRenderTargetToTexture_CannotCallAtRuntime", "ConvertRenderTarget: Can't convert render target to texture at run time. "));
#endif // !WITH_EDITOR
}

UTexture* RenderTargetCreateStaticTextureEditorOnly(UTextureRenderTarget* InRenderTarget, FString InName, TextureCompressionSettings InCompressionSettings, TextureMipGenSettings InMipSettings)
{
#if WITH_EDITOR
	if (InRenderTarget == nullptr)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTargetCreateStaticTexture_InvalidRenderTarget", "RenderTargetCreateStaticTextureEditorOnly: RenderTarget must be non-null."));
		return nullptr;
	}
	else if (InRenderTarget->GetResource() == nullptr)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("RenderTargetCreateStaticTexture_ReleasedRenderTarget", "RenderTargetCreateStaticTextureEditorOnly: RenderTarget has been released."));
		return nullptr;
	}
	else
	{
		FString Name;
		FString PackageName;
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		//Use asset name only if directories are specified, otherwise full path
		if (!InName.StartsWith(TEXT("/")))
		{
			FString AssetName = InRenderTarget->GetOutermost()->GetName();
			const FString SanitizedBasePackageName = UPackageTools::SanitizePackageName(AssetName);
			const FString PackagePath = FPackageName::GetLongPackagePath(SanitizedBasePackageName) + TEXT("/");
			AssetTools.CreateUniqueAssetName(PackagePath, InName, PackageName, Name);
		}
		else
		{
			AssetTools.CreateUniqueAssetName(InName, TEXT(""), PackageName, Name);
		}

		UObject* NewObj = nullptr;

		// create a static texture
		FText ErrorMessage;
		NewObj = InRenderTarget->ConstructTexture(CreatePackage(*PackageName), Name, InRenderTarget->GetMaskedFlags() | RF_Public | RF_Standalone, 
			static_cast<EConstructTextureFlags>(CTF_Default | CTF_AllowMips | CTF_SkipPostEdit), /*InAlphaOverride = */nullptr, &ErrorMessage);

		UTexture* NewTex = Cast<UTexture>(NewObj);
		if (NewTex == nullptr)
		{
			FMessageLog("Blueprint").Warning(ErrorMessage);
			return nullptr;
		}

		// package needs saving
		NewObj->MarkPackageDirty();

		// Update Compression and Mip settings
		NewTex->CompressionSettings = InCompressionSettings;
		NewTex->MipGenSettings = InMipSettings;
		NewTex->PostEditChange();

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(NewObj);

		return NewTex;
	}
#else // WITH_EDITOR
	FMessageLog("Blueprint").Error(LOCTEXT("RenderTargetCreateStaticTexture_CannotCallAtRuntime", "RenderTargetCreateStaticTextureEditorOnly: Can't create texture at run time. "));
	return nullptr;
	#endif // !WITH_EDITOR
}
} // namespace UE::Kismet::RenderingLibrary

UTexture2D* UKismetRenderingLibrary::RenderTargetCreateStaticTexture2DEditorOnly(UTextureRenderTarget2D* RenderTarget, FString InName, enum TextureCompressionSettings CompressionSettings, enum TextureMipGenSettings MipSettings)
{
	return Cast<UTexture2D>(UE::Kismet::RenderingLibrary::RenderTargetCreateStaticTextureEditorOnly(RenderTarget, InName, CompressionSettings, MipSettings));
}

UTexture2DArray* UKismetRenderingLibrary::RenderTargetCreateStaticTexture2DArrayEditorOnly(UTextureRenderTarget2DArray* RenderTarget, FString InName, enum TextureCompressionSettings CompressionSettings, enum TextureMipGenSettings MipSettings)
{
	return Cast<UTexture2DArray>(UE::Kismet::RenderingLibrary::RenderTargetCreateStaticTextureEditorOnly(RenderTarget, InName, CompressionSettings, MipSettings));
}

UTextureCube* UKismetRenderingLibrary::RenderTargetCreateStaticTextureCubeEditorOnly(UTextureRenderTargetCube* RenderTarget, FString InName, enum TextureCompressionSettings CompressionSettings, enum TextureMipGenSettings MipSettings)
{
	return Cast<UTextureCube>(UE::Kismet::RenderingLibrary::RenderTargetCreateStaticTextureEditorOnly(RenderTarget, InName, CompressionSettings, MipSettings));
}

UVolumeTexture* UKismetRenderingLibrary::RenderTargetCreateStaticVolumeTextureEditorOnly(UTextureRenderTargetVolume* RenderTarget, FString InName, enum TextureCompressionSettings CompressionSettings, enum TextureMipGenSettings MipSettings)
{
	return Cast<UVolumeTexture>(UE::Kismet::RenderingLibrary::RenderTargetCreateStaticTextureEditorOnly(RenderTarget, InName, CompressionSettings, MipSettings));
}

void UKismetRenderingLibrary::ConvertRenderTargetToTexture2DEditorOnly(UObject* WorldContextObject, UTextureRenderTarget2D* RenderTarget, UTexture2D* Texture)
{
	UE::Kismet::RenderingLibrary::ConvertRenderTargetToTextureEditorOnly(WorldContextObject, RenderTarget, Texture);
}

void UKismetRenderingLibrary::ConvertRenderTargetToTexture2DArrayEditorOnly(UObject* WorldContextObject, UTextureRenderTarget2DArray* RenderTarget, UTexture2DArray* Texture)
{
	UE::Kismet::RenderingLibrary::ConvertRenderTargetToTextureEditorOnly(WorldContextObject, RenderTarget, Texture);
}

void UKismetRenderingLibrary::ConvertRenderTargetToTextureCubeEditorOnly(UObject* WorldContextObject, UTextureRenderTargetCube* RenderTarget, UTextureCube* Texture)
{
	UE::Kismet::RenderingLibrary::ConvertRenderTargetToTextureEditorOnly(WorldContextObject, RenderTarget, Texture);
}

void UKismetRenderingLibrary::ConvertRenderTargetToTextureVolumeEditorOnly(UObject* WorldContextObject, UTextureRenderTargetVolume* RenderTarget, UVolumeTexture* Texture)
{
	UE::Kismet::RenderingLibrary::ConvertRenderTargetToTextureEditorOnly(WorldContextObject, RenderTarget, Texture);
}

void UKismetRenderingLibrary::ExportTexture2D(UObject* WorldContextObject, UTexture2D* Texture, const FString& FilePath, const FString& FileName)
{
	FString TotalFileName = FPaths::Combine(*FilePath, *FileName);
	FText PathError;
	FPaths::ValidatePath(TotalFileName, &PathError);

	if (Texture && !FileName.IsEmpty() && PathError.IsEmpty())
	{
		FArchive* Ar = IFileManager::Get().CreateFileWriter(*TotalFileName);

		if (Ar)
		{
			FBufferArchive Buffer;
			bool bSuccess = FImageUtils::ExportTexture2DAsHDR(Texture, Buffer);

			if (bSuccess)
			{
				Ar->Serialize(const_cast<uint8*>(Buffer.GetData()), Buffer.Num());
			}

			delete Ar;
		}
		else
		{
			FMessageLog("Blueprint").Warning(LOCTEXT("ExportTexture2D_FileWriterFailedToCreate", "ExportTexture2D: FileWrite failed to create."));
		}
	}

	else if (!Texture)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("ExportTexture2D_InvalidTextureRenderTarget", "ExportTexture2D: TextureRenderTarget must be non-null."));
	}
	if (!PathError.IsEmpty())
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("ExportTexture2D_InvalidFilePath", "ExportTexture2D: Invalid file path provided: '{0}'"), PathError));
	}
	if (FileName.IsEmpty())
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("ExportTexture2D_InvalidFileName", "ExportTexture2D: FileName must be non-empty."));
	}
}

UTexture2D* UKismetRenderingLibrary::ImportFileAsTexture2D(UObject* WorldContextObject, const FString& Filename)
{
	return FImageUtils::ImportFileAsTexture2D(Filename);
}

UTexture2D* UKismetRenderingLibrary::ImportBufferAsTexture2D(UObject* WorldContextObject, const TArray<uint8>& Buffer)
{
	return FImageUtils::ImportBufferAsTexture2D(Buffer);
}

void UKismetRenderingLibrary::BeginDrawCanvasToRenderTarget(UObject* WorldContextObject, UTextureRenderTarget2D* TextureRenderTarget, UCanvas*& Canvas, FVector2D& Size, FDrawToRenderTargetContext& Context)
{
	Canvas = nullptr;
	Size = FVector2D(0, 0);
	Context = FDrawToRenderTargetContext();
	
	if (!FApp::CanEverRender())
	{
		// Returning early to avoid warnings about missing resources that are expected when CanEverRender is false.
		return;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (!World)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("BeginDrawCanvasToRenderTarget_InvalidWorldContextObject", "BeginDrawCanvasToRenderTarget: WorldContextObject is not valid."));
	}
	else if (!TextureRenderTarget)
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("BeginDrawCanvasToRenderTarget_InvalidTextureRenderTarget", "BeginDrawCanvasToRenderTarget[{0}]: TextureRenderTarget must be non-null."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else if (!TextureRenderTarget->GetResource())
	{
		FMessageLog("Blueprint").Warning(FText::Format(LOCTEXT("BeginDrawCanvasToRenderTarget_ReleasedTextureRenderTarget", "BeginDrawCanvasToRenderTarget[{0}]: render target has been released."), FText::FromString(GetPathNameSafe(WorldContextObject))));
	}
	else
	{
		World->FlushDeferredParameterCollectionInstanceUpdates();

		Context.RenderTarget = TextureRenderTarget;

		Canvas = World->GetCanvasForRenderingToTarget();

		Size = FVector2D(TextureRenderTarget->SizeX, TextureRenderTarget->SizeY);

		FTextureRenderTargetResource* RenderTargetResource = TextureRenderTarget->GameThread_GetRenderTargetResource();
		FCanvas* NewCanvas = new FCanvas(
			RenderTargetResource,
			nullptr, 
			World,
			World->GetFeatureLevel(),
			// Draw immediately so that interleaved SetVectorParameter (etc) function calls work as expected
			FCanvas::CDM_ImmediateDrawing);
		Canvas->Init(TextureRenderTarget->SizeX, TextureRenderTarget->SizeY, nullptr, NewCanvas);

		ENQUEUE_RENDER_COMMAND(FlushDeferredResourceUpdateCommand)(
			[
				RenderTargetResource
#if WANTS_DRAW_MESH_EVENTS
				, Name = TextureRenderTarget->GetFName()
				, Breadcrumb = (Context.Breadcrumb = new TOptional<FRHIBreadcrumbEventManual>)
#endif
				](FRHICommandListImmediate& RHICmdList)
			{
#if  WANTS_DRAW_MESH_EVENTS
				Breadcrumb->Emplace(RHICmdList, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("DrawCanvasToTarget"), TEXT("DrawCanvasToTarget: %s"), RHI_GPU_STAT_ARGS_NONE)(Name));
#endif
				RenderTargetResource->FlushDeferredResourceUpdate(RHICmdList);
			});
	}
}

void UKismetRenderingLibrary::EndDrawCanvasToRenderTarget(UObject* WorldContextObject, const FDrawToRenderTargetContext& Context)
{
	if (!FApp::CanEverRender())
	{
		// Returning early to avoid warnings about missing resources that are expected when CanEverRender is false.
		return;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	if (World)
	{
		UCanvas* WorldCanvas = World->GetCanvasForRenderingToTarget();

		if (WorldCanvas->Canvas)
		{
			WorldCanvas->Canvas->Flush_GameThread();
			delete WorldCanvas->Canvas;
			WorldCanvas->Canvas = nullptr;
		}
		
		if (Context.RenderTarget)
		{
			FTextureRenderTargetResource* RenderTargetResource = Context.RenderTarget->GameThread_GetRenderTargetResource();

			ENQUEUE_RENDER_COMMAND(CanvasRenderTargetResolveCommand)(
				[
					RenderTargetResource
#if WANTS_DRAW_MESH_EVENTS
					, Breadcrumb = Context.Breadcrumb
#endif
				](FRHICommandListImmediate& RHICmdList)
				{
					// Note: If multisampled, it should have already been resolved by ~FCanvasRenderThreadScope()
					if (!RenderTargetResource->GetRenderTargetTexture()->GetDesc().IsMultisample())
					{
						TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), RenderTargetResource->TextureRHI, {});
					}

#if WANTS_DRAW_MESH_EVENTS
					(*Breadcrumb)->End(RHICmdList);
					delete Breadcrumb;
#endif
				}
			);

			// Remove references to the context now that we've resolved it, to avoid a crash when EndDrawCanvasToRenderTarget is called multiple times with the same context
			// const cast required, as BP will treat Context as an output without the const
			const_cast<FDrawToRenderTargetContext&>(Context) = FDrawToRenderTargetContext();
		}
		else
		{
#if WANTS_DRAW_MESH_EVENTS
			check(!Context.Breadcrumb);
#endif

			FMessageLog("Blueprint").Warning(LOCTEXT("EndDrawCanvasToRenderTarget_InvalidContext", "EndDrawCanvasToRenderTarget: Context must be valid."));
		}
	}
	else
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("EndDrawCanvasToRenderTarget_InvalidWorldContextObject", "EndDrawCanvasToRenderTarget: WorldContextObject is not valid."));
	}
}

FSkelMeshSkinWeightInfo UKismetRenderingLibrary::MakeSkinWeightInfo(int32 Bone0, uint8 Weight0, int32 Bone1, uint8 Weight1, int32 Bone2, uint8 Weight2, int32 Bone3, uint8 Weight3)
{
	FSkelMeshSkinWeightInfo Info;
	FMemory::Memzero(&Info, sizeof(FSkelMeshSkinWeightInfo));
	Info.Bones[0] = Bone0;
	Info.Weights[0] = Weight0;
	Info.Bones[1] = Bone1;
	Info.Weights[1] = Weight1;
	Info.Bones[2] = Bone2;
	Info.Weights[2] = Weight2;
	Info.Bones[3] = Bone3;
	Info.Weights[3] = Weight3;
	return Info;
}


void UKismetRenderingLibrary::BreakSkinWeightInfo(FSkelMeshSkinWeightInfo InWeight, int32& Bone0, uint8& Weight0, int32& Bone1, uint8& Weight1, int32& Bone2, uint8& Weight2, int32& Bone3, uint8& Weight3)
{
	FMemory::Memzero(&InWeight, sizeof(FSkelMeshSkinWeightInfo));
	Bone0 = InWeight.Bones[0];
	Weight0 = InWeight.Weights[0];
	Bone1 = InWeight.Bones[1];
	Weight1 = InWeight.Weights[1];
	Bone2 = InWeight.Bones[2];
	Weight2 = InWeight.Weights[2];
	Bone3 = InWeight.Bones[3];
	Weight3 = InWeight.Weights[3];
}

void UKismetRenderingLibrary::SetCastInsetShadowForAllAttachments(UPrimitiveComponent* PrimitiveComponent, bool bCastInsetShadow, bool bLightAttachmentsAsGroup)
{
	if (PrimitiveComponent)
	{
		// Update this primitive
		PrimitiveComponent->SetCastInsetShadow(bCastInsetShadow);
		PrimitiveComponent->SetLightAttachmentsAsGroup(bLightAttachmentsAsGroup);

		// Go through all potential children and update them 
		TArray<USceneComponent*, TInlineAllocator<8>> ProcessStack;
		ProcessStack.Append(PrimitiveComponent->GetAttachChildren());

		// Walk down the tree updating
		while (ProcessStack.Num() > 0)
		{
			USceneComponent* Current = ProcessStack.Pop(EAllowShrinking::No);
			UPrimitiveComponent* CurrentPrimitive = Cast<UPrimitiveComponent>(Current);

			if (CurrentPrimitive && CurrentPrimitive->ShouldComponentAddToScene())
			{
				if (bLightAttachmentsAsGroup)
				{
					// Clear all the children if the root primitive wants to light attachments as group
					// This is to make sure no child attachment in the chain overrides its parent
					CurrentPrimitive->SetLightAttachmentsAsGroup(false);
				}

				CurrentPrimitive->SetCastInsetShadow(bCastInsetShadow);
			}

			ProcessStack.Append(Current->GetAttachChildren());
		}
	}
	else
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("SetCastInsetShadowForAllAttachments_InvalidPrimitiveComponent", "SetCastInsetShadowForAllAttachments: PrimitiveComponent must be non-null."));
	}
}

ENGINE_API FMatrix UKismetRenderingLibrary::CalculateProjectionMatrix(const FMinimalViewInfo& MinimalViewInfo)
{
	return MinimalViewInfo.CalculateProjectionMatrix();
}

ENGINE_API void UKismetRenderingLibrary::EnablePathTracing(bool bEnablePathTracer)
{
	if (GEngine != nullptr && GEngine->GameViewport != nullptr)
	{
		FEngineShowFlags* EngineShowFlags = GEngine->GameViewport->GetEngineShowFlags();
		if (EngineShowFlags != nullptr)
		{
			EngineShowFlags->SetPathTracing(bEnablePathTracer);
		}
	}
}

ENGINE_API void UKismetRenderingLibrary::RefreshPathTracingOutput()
{
	if (GEngine != nullptr && GEngine->GameViewport != nullptr)
	{
		UWorld* World = GEngine->GameViewport->GetWorld();
		if (World != nullptr && World->Scene != nullptr)
		{
			World->Scene->InvalidatePathTracedOutput();
		}
	}
}

ENGINE_API int32 UKismetRenderingLibrary::NumPrecompilingPSOsRemaining()
{
	return (int32)FShaderPipelineCache::NumPrecompilesRemaining();
}

#undef LOCTEXT_NAMESPACE

