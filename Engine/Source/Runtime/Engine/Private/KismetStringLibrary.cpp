// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetStringLibrary.h"
#include "Math/Box.h"

#if WITH_EDITOR
#include "AsyncTreeDifferences.h"
#include "Containers/StringView.h"
#include "DiffUtils.h"
#include "Internationalization/Regex.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetStringLibrary)

//////////////////////////////////////////////////////////////////////////
// UKismetStringLibrary

UKismetStringLibrary::UKismetStringLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UKismetStringLibrary::Concat_StrStr(const FString& A, const FString& B)
{
	// faster, preallocating method
	FString StringResult;
	StringResult.Empty(A.Len() + B.Len() + 1); // adding one for the string terminator
	StringResult += A;
	StringResult += B;

	return StringResult;
}

bool UKismetStringLibrary::EqualEqual_StriStri(const FString& A, const FString& B)
{
	return FCString::Stricmp(*A, *B) == 0;
}

bool UKismetStringLibrary::EqualEqual_StrStr(const FString& A, const FString& B)
{
	return FCString::Strcmp(*A, *B) == 0;
}

bool UKismetStringLibrary::NotEqual_StriStri(const FString& A, const FString& B)
{
	return FCString::Stricmp(*A, *B) != 0;
}

bool UKismetStringLibrary::NotEqual_StrStr(const FString& A, const FString& B)
{
	return FCString::Strcmp(*A, *B) != 0;
}

int32 UKismetStringLibrary::Len(const FString& S)
{
	return S.Len();
}

bool UKismetStringLibrary::IsEmpty(const FString& InString)
{
	return InString.IsEmpty();
}

FString UKismetStringLibrary::Conv_DoubleToString(double InDouble)
{
	return FString::SanitizeFloat(InDouble);
}

FString UKismetStringLibrary::Conv_IntToString(int32 InInt)
{
	return FString::Printf(TEXT("%d"), InInt);	
}

FString UKismetStringLibrary::Conv_Int64ToString(int64 InInt)
{
	return FString::Printf(TEXT("%lld"), InInt);
}

FString UKismetStringLibrary::Conv_ByteToString(uint8 InByte)
{
	return FString::Printf(TEXT("%d"), InByte);	
}

FString UKismetStringLibrary::Conv_BoolToString(bool InBool)
{
	return InBool ? TEXT("true") : TEXT("false");	
}

FString UKismetStringLibrary::Conv_VectorToString(FVector InVec)
{
	return InVec.ToString();	
}

FString UKismetStringLibrary::Conv_Vector3fToString(FVector3f InVec)
{
	return InVec.ToString();
}

FString UKismetStringLibrary::Conv_IntVectorToString(FIntVector InIntVec)
{
	return InIntVec.ToString();
}

FString UKismetStringLibrary::Conv_IntVector2ToString(FIntVector2 InIntVec2)
{
	return InIntVec2.ToString();
}

FString UKismetStringLibrary::Conv_IntPointToString(FIntPoint InIntPoint)
{
	return InIntPoint.ToString();
}

FString UKismetStringLibrary::Conv_Vector2dToString(FVector2D InVec)
{
	return InVec.ToString();	
}

FString UKismetStringLibrary::Conv_RotatorToString(FRotator InRot)
{
	return InRot.ToString();	
}

FString UKismetStringLibrary::Conv_TransformToString(const FTransform& InTrans)
{
	return FString::Printf(TEXT("Translation: %s Rotation: %s Scale: %s"), *InTrans.GetTranslation().ToString(), *InTrans.Rotator().ToString(), *InTrans.GetScale3D().ToString());
}

FString UKismetStringLibrary::Conv_ObjectToString(class UObject* InObj)
{
	return (InObj != nullptr) ? InObj->GetName() : FString("None");
}

FString UKismetStringLibrary::Conv_BoxToString(const FBox& Box)
{
	return Box.ToString();
}

FString UKismetStringLibrary::Conv_BoxCenterAndExtentsToString(const FBox& Box)
{
	FVector Center = FVector::ZeroVector;
	FVector Extents = FVector::ZeroVector;
	Box.GetCenterAndExtents(OUT Center, OUT Extents);

	return FString::Printf(TEXT("Center: %s Extents: %s "), *Center.ToString(), *Extents.ToString());
}

FString UKismetStringLibrary::Conv_InputDeviceIdToString(FInputDeviceId InDeviceId)
{
	return FString::Printf(TEXT("%d"), InDeviceId.GetId());
}

FString UKismetStringLibrary::Conv_PlatformUserIdToString(FPlatformUserId InPlatformUserId)
{
	return FString::Printf(TEXT("%d"), InPlatformUserId.GetInternalId());
}

FString UKismetStringLibrary::Conv_ColorToString(FLinearColor C)
{
	return C.ToString();
}

FString UKismetStringLibrary::Conv_NameToString(FName InName)
{
	return InName.ToString();
}

FString UKismetStringLibrary::Conv_MatrixToString(const FMatrix& InMatrix)
{
	return InMatrix.ToString();
}

FName UKismetStringLibrary::Conv_StringToName(const FString& InString)
{
	return FName(*InString);
}

int32 UKismetStringLibrary::Conv_StringToInt(const FString& InString)
{
	return FCString::Atoi(*InString);
}

int64 UKismetStringLibrary::Conv_StringToInt64(const FString& InString)
{
	return FCString::Atoi64(*InString);
}

double UKismetStringLibrary::Conv_StringToDouble(const FString& InString)
{
	return FCString::Atod(*InString);
}

void UKismetStringLibrary::Conv_StringToVector(const FString& InString, FVector& OutConvertedVector, bool& OutIsValid)
{
	OutIsValid = OutConvertedVector.InitFromString(InString);
}

void UKismetStringLibrary::Conv_StringToVector3f(const FString& InString, FVector3f& OutConvertedVector, bool& OutIsValid)
{
	OutIsValid = OutConvertedVector.InitFromString(InString);
}

void UKismetStringLibrary::Conv_StringToVector2D(const FString& InString, FVector2D& OutConvertedVector2D, bool& OutIsValid)
{
	OutIsValid = OutConvertedVector2D.InitFromString(InString);
}

void UKismetStringLibrary::Conv_StringToRotator(const FString& InString, FRotator& OutConvertedRotator, bool& OutIsValid)
{
	OutIsValid = OutConvertedRotator.InitFromString(InString);
}

void UKismetStringLibrary::Conv_StringToColor(const FString& InString, FLinearColor& OutConvertedColor, bool& OutIsValid)
{
	OutIsValid = OutConvertedColor.InitFromString(InString);
}

FString UKismetStringLibrary::BuildString_Double(const FString& AppendTo, const FString& Prefix, double InDouble, const FString& Suffix)
{
	// despite the name, SanitizeFloat takes a double parameter
	const FString DoubleStr = FString::SanitizeFloat(InDouble);

	FString StringResult;
	StringResult.Empty(AppendTo.Len() + Prefix.Len() + DoubleStr.Len() + Suffix.Len() + 1);
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += DoubleStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Int(const FString& AppendTo, const FString& Prefix, int32 InInt, const FString& Suffix)
{
	// faster, preallocating method
	const FString IntStr = FString::Printf(TEXT("%d"), InInt);

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+IntStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += IntStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Bool(const FString& AppendTo, const FString& Prefix, bool InBool, const FString& Suffix)
{
	// faster, preallocating method
	const FString BoolStr = InBool ? TEXT("true") : TEXT("false");	

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+BoolStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += BoolStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Vector(const FString& AppendTo, const FString& Prefix, FVector InVector, const FString& Suffix)
{
	// faster, preallocating method
	const FString VecStr = InVector.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+VecStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += VecStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_IntVector(const FString& AppendTo, const FString& Prefix, FIntVector InIntVector, const FString& Suffix)
{
	// faster, preallocating method
	const FString VecStr = InIntVector.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len() + Prefix.Len() + VecStr.Len() + Suffix.Len() + 1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += VecStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_IntVector2(const FString& AppendTo, const FString& Prefix, FIntVector2 InIntVector2, const FString& Suffix)
{
	// faster, preallocating method
	const FString VecStr = InIntVector2.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len() + Prefix.Len() + VecStr.Len() + Suffix.Len() + 1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += VecStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Vector2d(const FString& AppendTo, const FString& Prefix, FVector2D InVector2d, const FString& Suffix)
{
	// faster, preallocating method
	const FString VecStr = InVector2d.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+VecStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += VecStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Rotator(const FString& AppendTo, const FString& Prefix, FRotator InRot, const FString& Suffix)
{
	// faster, preallocating method
	const FString RotStr = InRot.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+RotStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += RotStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Object(const FString& AppendTo, const FString& Prefix, class UObject* InObj, const FString& Suffix)
{
	// faster, preallocating method
	const FString ObjStr = (InObj != NULL) ? InObj->GetName() : FString("None");

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+ObjStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += ObjStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Color(const FString& AppendTo, const FString& Prefix, FLinearColor InColor, const FString& Suffix)
{
	// faster, preallocating method
	const FString ColorStr = InColor.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+ColorStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += ColorStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::BuildString_Name(const FString& AppendTo, const FString& Prefix, FName InName, const FString& Suffix)
{
	// faster, preallocating method
	const FString NameStr = InName.ToString();

	FString StringResult;
	StringResult.Empty(AppendTo.Len()+Prefix.Len()+NameStr.Len()+Suffix.Len()+1); // adding one for the string terminator
	StringResult += AppendTo;
	StringResult += Prefix;
	StringResult += NameStr;
	StringResult += Suffix;

	return StringResult;
}

FString UKismetStringLibrary::GetSubstring(const FString& SourceString, int32 StartIndex, int32 Length)
{
	return (Length >= 0 ? SourceString.Mid(StartIndex, Length) : FString());
}

int32 UKismetStringLibrary::FindSubstring(const FString& SearchIn, const FString& Substring, bool bUseCase, bool bSearchFromEnd, int32 StartPosition)
{
	ESearchCase::Type Case = bUseCase ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase;
	ESearchDir::Type Dir = bSearchFromEnd ? ESearchDir::FromEnd : ESearchDir::FromStart;

	return SearchIn.Find(Substring, Case, Dir, StartPosition);
}

bool UKismetStringLibrary::Contains(const FString& SearchIn, const FString& Substring, bool bUseCase, bool bSearchFromEnd)
{
	return FindSubstring(SearchIn, Substring, bUseCase, bSearchFromEnd) != -1;
}

int32 UKismetStringLibrary::GetCharacterAsNumber(const FString& SourceString, int32 Index)
{
	if ((Index >= 0) && (Index < SourceString.Len()))
	{
		return (int32)(SourceString.GetCharArray()[Index]);
	}
	else
	{
		//@TODO: Script error
		return 0;
	}
}

TArray<FString> UKismetStringLibrary::ParseIntoArray(const FString& SourceString, const FString& Delimiter, const bool CullEmptyStrings)
{
	TArray<FString> SeparatedStrings;
	const int32 nArraySize = SourceString.ParseIntoArray(SeparatedStrings, *Delimiter, CullEmptyStrings);
	return SeparatedStrings;
}

FString UKismetStringLibrary::JoinStringArray(const TArray<FString>& SourceArray, const FString&  Separator)
{
	return FString::Join(SourceArray, *Separator);
}

TArray<FString> UKismetStringLibrary::GetCharacterArrayFromString(const FString& SourceString)
{
	TArray<FString> SeparatedChars;

	if (!SourceString.IsEmpty())
	{
		for (auto CharIt(SourceString.CreateConstIterator()); CharIt; ++CharIt)
		{
			TCHAR Char = *CharIt;
			SeparatedChars.Add(FString::ConstructFromPtrSize(&Char, 1));
		}

		// Remove the null terminator on the end
		SeparatedChars.RemoveAt(SeparatedChars.Num() - 1, 1);
	}

	return SeparatedChars;
}

FString UKismetStringLibrary::ToUpper(const FString& SourceString)
{
	return SourceString.ToUpper();
}

FString UKismetStringLibrary::ToLower(const FString& SourceString)
{
	return SourceString.ToLower();
}

FString UKismetStringLibrary::LeftPad(const FString& SourceString, int32 ChCount)
{
	return SourceString.LeftPad(ChCount);
}

FString UKismetStringLibrary::RightPad(const FString& SourceString, int32 ChCount)
{
	return SourceString.RightPad(ChCount);
}

bool UKismetStringLibrary::IsNumeric(const FString& SourceString)
{
	return SourceString.IsNumeric();
}

bool UKismetStringLibrary::StartsWith(const FString& SourceString, const FString& InPrefix, ESearchCase::Type SearchCase)
{
	return SourceString.StartsWith(InPrefix,SearchCase);
}

bool UKismetStringLibrary::EndsWith(const FString& SourceString, const FString& InSuffix, ESearchCase::Type SearchCase)
{
	return SourceString.EndsWith(InSuffix,SearchCase);
}

bool UKismetStringLibrary::MatchesWildcard(const FString& SourceString, const FString& Wildcard, ESearchCase::Type SearchCase)
{
	return SourceString.MatchesWildcard(Wildcard, SearchCase);
}

FString UKismetStringLibrary::Trim(const FString& SourceString)
{
	FString Trimmed = SourceString.TrimStart();
	return Trimmed;
}

FString UKismetStringLibrary::TrimTrailing(const FString& SourceString)
{
	FString Trimmed = SourceString.TrimEnd();
	return Trimmed;
}

int32 UKismetStringLibrary::CullArray(const FString& SourceString,TArray<FString>& InArray)
{
	return SourceString.CullArray(&InArray);
}

FString UKismetStringLibrary::Reverse(const FString& SourceString)
{
//	return SourceString.Reverse();
	FString Reversed = SourceString;
	Reversed.ReverseString();
	return Reversed;
}

FString UKismetStringLibrary::Replace(const FString& SourceString, const FString& From, const FString& To, ESearchCase::Type SearchCase)
{
	return SourceString.Replace(*From, *To, SearchCase);
}

int32 UKismetStringLibrary::ReplaceInline(FString& SourceString, const FString& SearchText, const FString& ReplacementText, ESearchCase::Type SearchCase)
{
	return SourceString.ReplaceInline(*SearchText, *ReplacementText, SearchCase);
}

bool UKismetStringLibrary::Split(const FString& SourceString, const FString& InStr, FString& LeftS, FString& RightS, ESearchCase::Type SearchCase, ESearchDir::Type SearchDir)
{
	return SourceString.Split(InStr, &LeftS, &RightS, SearchCase, SearchDir);
}

FString UKismetStringLibrary::Left(const FString& SourceString, int32 Count)
{
	return SourceString.Left(Count);
}

FString UKismetStringLibrary::LeftChop(const FString& SourceString, int32 Count)
{
	return SourceString.LeftChop(Count);
}

FString UKismetStringLibrary::Right(const FString& SourceString, int32 Count)
{
	return SourceString.Right(Count);
}

FString UKismetStringLibrary::RightChop(const FString& SourceString, int32 Count)
{
	return SourceString.RightChop(Count);
}

FString UKismetStringLibrary::Mid(const FString& SourceString, int32 Start, int32 Count = MAX_int32)
{
	return (Count >= 0 ? SourceString.Mid(Start, Count) : FString());
}

#if WITH_EDITOR
namespace UE::Private
{
	struct FTokenizedStringNode
	{
		FStringView StringView;
		int32 LineIndex = INDEX_NONE; // line number in the full string (INDEX_NONE if this is a head node)
		int32 TokenIndex = INDEX_NONE; // index of token at current line (INDEX_NONE if this is a head or line node)
		
		bool operator==(const FTokenizedStringNode&) const = default;
	};
}

// methods that make FTokenizedStringNode diffable - specializations must be done at global scope:
template<>
class TTreeDiffSpecification<UE::Private::FTokenizedStringNode>
{
public:
	bool AreValuesEqual(const UE::Private::FTokenizedStringNode& TreeNodeA, const UE::Private::FTokenizedStringNode& TreeNodeB) const
	{
		return TreeNodeA.StringView.TrimStartAndEnd() == TreeNodeB.StringView.TrimStartAndEnd();
	}

	bool AreMatching(const UE::Private::FTokenizedStringNode& TreeNodeA, const UE::Private::FTokenizedStringNode& TreeNodeB) const
	{
		return TreeNodeA.LineIndex == TreeNodeB.LineIndex &&  TreeNodeA.TokenIndex == TreeNodeB.TokenIndex;
	}

	void GetChildren(const UE::Private::FTokenizedStringNode& InParent, TArray<UE::Private::FTokenizedStringNode>& OutChildren) const
	{
		if (InParent.LineIndex == INDEX_NONE)
		{
			// parent is the head. Each child should be a line
			int32 LineIndex = 0;
			FStringView RemainingView(InParent.StringView);
			while (!RemainingView.IsEmpty())
			{
				int32 ChildSize;
				if (!RemainingView.FindChar('\n', ChildSize))
				{
					ChildSize = RemainingView.Len();
				}
			
				UE::Private::FTokenizedStringNode NextChild(InParent);
				NextChild.StringView = FStringView(RemainingView.begin(), ChildSize);
				NextChild.LineIndex = LineIndex++;
				OutChildren.Add(NextChild);
			
				RemainingView = FStringView(RemainingView.begin() + (ChildSize + 1), FMath::Max(0,RemainingView.Len() - (ChildSize + 1)));
			}
		}
		else if (InParent.TokenIndex == INDEX_NONE)
		{
			// parent is a line. Each child should be a token
			int32 TokenIndex = 0;
		
			FRegexPattern TokenPattern(TEXT(R"(\w+|[^\w\s]+|[\s]+)"));
			// TODO: Constructing a string here is slow! FRegexMatcher should be using FStringView!
			FRegexMatcher TokenMatcher(TokenPattern, FString(InParent.StringView));

			while (TokenMatcher.FindNext())
			{
				const TCHAR* Begin = InParent.StringView.begin() + TokenMatcher.GetMatchBeginning();
				const TCHAR* End = InParent.StringView.begin() + TokenMatcher.GetMatchEnding();
			
				UE::Private::FTokenizedStringNode NextChild(InParent);
				NextChild.StringView = FStringView(Begin, End-Begin);
				NextChild.TokenIndex = TokenIndex++;
				OutChildren.Add(NextChild);
			}
		}
	}

	bool ShouldMatchByValue(const UE::Private::FTokenizedStringNode& TreeNodeA) const
	{
		// prioritize matching equivalent substrings
		return true;
	}

	bool ShouldInheritEqualFromChildren(const UE::Private::FTokenizedStringNode& TreeNodeA, const UE::Private::FTokenizedStringNode& TreeNodeB) const
	{
		return true;
	}
};

FString UKismetStringLibrary::DiffString(const FString& First, const FString& Second)
{
	using namespace UE::Private;

	class FStringDiffTree : public TAsyncTreeDifferences<FTokenizedStringNode>
	{
	public:
		FStringDiffTree(const FString& StringA, const FString& StringB)
			: TAsyncTreeDifferences<FTokenizedStringNode>(GetRootAttribute(StringA), GetRootAttribute(StringB))
		{}

		// generates a text based explanation of the diff
		// and calls PrintLine for each line of the result
		FString CollectDifferences() const
		{
			FString Result;
			ForEach (
				ETreeTraverseOrder::PreOrder,
				[&Result](const TUniquePtr<FStringDiffTree::DiffNodeType>& DiffNode)->ETreeTraverseControl
				{
					// the tree has multiple levels of granularity diffing the entire text, then lines, then tokens,
					// then individual characters.
					// Skip to the first diff of a line
					if (DiffNode->ValueA.LineIndex == INDEX_NONE && DiffNode->ValueB.LineIndex == INDEX_NONE)
					{
						return ETreeTraverseControl::Continue;
					}

					if (DiffNode->ValueA.TokenIndex == INDEX_NONE && DiffNode->ValueB.TokenIndex == INDEX_NONE)
					{
						// so that distances are consistent, replace tabs with spaces
						FString LeftString = FString(DiffNode->ValueA.StringView).Replace(TEXT("\t"), TEXT("  "));
						FString RightString = FString(DiffNode->ValueB.StringView).Replace(TEXT("\t"), TEXT("  "));
				
						switch (DiffNode->DiffResult)
						{
						case ETreeDiffResult::Invalid:
							break;
						case ETreeDiffResult::Identical:
							// empty string for no differences
							break;
						case ETreeDiffResult::MissingFromTree1:
							Result += FString::Printf(TEXT("+ [      | %04d ] %s\n"), DiffNode->ValueB.LineIndex + 1, *RightString);
							break;
						case ETreeDiffResult::MissingFromTree2:
							Result += FString::Printf(TEXT("- [ %04d |      ] %s\n"), DiffNode->ValueA.LineIndex + 1, *LeftString);
							break;
						case ETreeDiffResult::DifferentValues:
							{
								Result += FString::Printf(TEXT("~ [ %04d | %04d ]\n"), DiffNode->ValueA.LineIndex + 1, DiffNode->ValueB.LineIndex + 1);

								auto GrowString = [](FString& String, TCHAR Char, int32 NewLen)
								{
									while (String.Len() < NewLen) String.AppendChar(Char);
								};
							
								FString LeftPadded  = "                  ";
								FString Annotations = "                  ";
								FString RightPadded = "                  ";
								// the children of this node contain token diff info. Use it to generate two parallel diff strings that show what's changed
								for (TUniquePtr<FStringDiffTree::DiffNodeType>& TokenDiffNode : DiffNode->Children)
								{
									// so that distances are consistent, replace tabs with spaces
									LeftPadded += FString(TokenDiffNode->ValueA.StringView).Replace(TEXT("\t"), TEXT("  "));
									RightPadded += FString(TokenDiffNode->ValueB.StringView).Replace(TEXT("\t"), TEXT("  "));
							
									int32 NewLength = FMath::Max(RightPadded.Len(), LeftPadded.Len());
									GrowString(LeftPadded, ' ', NewLength);
									GrowString(RightPadded, ' ', NewLength);

									switch (TokenDiffNode->DiffResult)
									{
									case ETreeDiffResult::Invalid:
										check(false);
									case ETreeDiffResult::MissingFromTree1:
										GrowString(Annotations, 'v', NewLength);
										break;
									case ETreeDiffResult::MissingFromTree2:
										GrowString(Annotations, '^', NewLength);
										break;
									case ETreeDiffResult::DifferentValues:
										GrowString(Annotations, '~', NewLength);
										break;
									case ETreeDiffResult::Identical:
										GrowString(Annotations, ' ', NewLength);
										break;
									}
								}
						
								Result += LeftPadded + TEXT("\n");
								Result += Annotations + TEXT("\n");
								Result += RightPadded + TEXT("\n");
						
								break;
							}
						}
					}
					return ETreeTraverseControl::SkipChildren;
				}
			);
			return Result;
		}

	private:
		static TAttribute<TArray<FTokenizedStringNode>> GetRootAttribute(const FString& String)
		{
			return TAttribute<TArray<FTokenizedStringNode>>::CreateLambda([String]()->TArray<FTokenizedStringNode>
			{
				if(String.IsEmpty())
				{
					return {};
				}
				return {FTokenizedStringNode{String}};
			});
		}
	};

	FStringDiffTree Difference(First, Second);
	Difference.FlushQueue();
	return Difference.CollectDifferences();
}
#endif// WITH_EDITOR

FString UKismetStringLibrary::TimeSecondsToString(float InSeconds)
{
	// Determine whether to display this number as a negative
	const TCHAR* NegativeModifier = InSeconds < 0.f? TEXT("-") : TEXT("");
	InSeconds = FMath::Abs(InSeconds);
	
	// Get whole minutes
	const int32 NumMinutes = FMath::FloorToInt(InSeconds/60.f);
	// Get seconds not part of whole minutes
	const int32 NumSeconds = FMath::FloorToInt(InSeconds-(NumMinutes*60.f));
	// Get fraction of non-whole seconds, convert to 100th of a second, then floor to get whole 100ths
	const int32 NumCentiseconds = FMath::FloorToInt((InSeconds - FMath::FloorToFloat(InSeconds)) * 100.f);

	// Create string, including leading zeroes
	return FString::Printf(TEXT("%s%02d:%02d.%02d"), NegativeModifier, NumMinutes, NumSeconds, NumCentiseconds);
}

