// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetStringTableLibrary.h"
#include "Internationalization/StringTable.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/StringTableRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetStringTableLibrary)

#define LOCTEXT_NAMESPACE "Kismet"

FName UKismetStringTableLibrary::GetTableId(const UStringTable* StringTable)
{
	return StringTable ? StringTable->GetStringTableId() : FName();
}


bool UKismetStringTableLibrary::IsRegisteredTableId(const FName TableId)
{
	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	return StringTable.IsValid();
}


bool UKismetStringTableLibrary::IsRegisteredTableEntry(const FName TableId, const FString& Key)
{
	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	return StringTable.IsValid() && StringTable->FindEntry(Key).IsValid();
}


FString UKismetStringTableLibrary::GetTableNamespace(const FName TableId)
{
	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	if (StringTable.IsValid())
	{
		return StringTable->GetNamespace();
	}

	return FString();
}


FString UKismetStringTableLibrary::GetTableEntrySourceString(const FName TableId, const FString& Key)
{
	FString SourceString;

	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	if (StringTable.IsValid())
	{
		StringTable->GetSourceString(Key, SourceString);
	}

	return SourceString;
}


FString UKismetStringTableLibrary::GetTableEntryMetaData(const FName TableId, const FString& Key, const FName MetaDataId)
{
	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	if (StringTable.IsValid())
	{
		return StringTable->GetMetaData(Key, MetaDataId);
	}

	return FString();
}


TArray<FName> UKismetStringTableLibrary::GetRegisteredStringTables()
{
	TArray<FName> RegisteredStringTableIds;
	
	FStringTableRegistry::Get().EnumerateStringTables([&](const FName& InTableId, const FStringTableConstRef& InStringTable) -> bool
	{
		RegisteredStringTableIds.Add(InTableId);
		return true; // continue enumeration
	});
	
	return RegisteredStringTableIds;
}


TArray<FString> UKismetStringTableLibrary::GetKeysFromStringTable(const FName TableId)
{
	TArray<FString> KeysFromStringTable;

	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	if (StringTable.IsValid())
	{
		StringTable->EnumerateSourceStrings([&](const FString& InKey, const FString& InSourceString) -> bool
		{
			KeysFromStringTable.Add(InKey);
			return true; // continue enumeration
		});
	}

	return KeysFromStringTable;
}


TArray<FName> UKismetStringTableLibrary::GetMetaDataIdsFromStringTableEntry(const FName TableId, const FString& Key)
{
	TArray<FName> MetaDataIdsFromStringTable;

	FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId);
	if (StringTable.IsValid())
	{
		StringTable->EnumerateMetaData(Key, [&](FName InMetaDataId, const FString& InMetaData) -> bool
		{
			MetaDataIdsFromStringTable.Add(InMetaDataId);
			return true; // continue enumeration
		});
	}

	return MetaDataIdsFromStringTable;
}


bool UKismetStringTableLibrary::ExportTableToCSVString(const FName TableId, FString& OutCSVString)
{
	if (FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId))
	{
		OutCSVString = StringTable->ExportStringsToCSVString();
		return true;
	}

	return false;
}


bool UKismetStringTableLibrary::ExportTableToCSVFile(const FName TableId, const FString& Filename)
{
	if (FStringTableConstPtr StringTable = FStringTableRegistry::Get().FindStringTable(TableId))
	{
		return StringTable->ExportStringsToCSVFile(Filename);
	}

	return false;
}


bool UKismetStringTableLibrary::ImportTableFromCSVString(const FName TableId, const FString& CSVString)
{
	if (FStringTablePtr StringTable = FStringTableRegistry::Get().FindMutableStringTable(TableId))
	{
		return StringTable->ImportStringsFromCSVString(CSVString, *TableId.ToString());
	}

	return false;
}


bool UKismetStringTableLibrary::ImportTableFromCSVFile(const FName TableId, const FString& Filename)
{
	if (FStringTablePtr StringTable = FStringTableRegistry::Get().FindMutableStringTable(TableId))
	{
		return StringTable->ImportStringsFromCSVFile(Filename);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

