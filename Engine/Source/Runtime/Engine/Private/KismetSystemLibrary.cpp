// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetSystemLibrary.h"
#include "AssetRegistry/ARFilter.h"
#include "Blueprint/BlueprintSupport.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Engine/AssetManagerTypes.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/FileManager.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "Misc/EngineVersion.h"
#include "EngineLogs.h"
#include "Misc/PackageName.h"
#include "GameFramework/PlayerController.h"
#include "Misc/RuntimeErrors.h"
#include "Misc/URLRequestFilter.h"
#include "TimerManager.h"
#include "UObject/EnumProperty.h"
#include "UObject/Package.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/GameViewportClient.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/GameEngine.h"
#include "Engine/Console.h"
#include "DelayAction.h"
#include "InterpolateComponentToAction.h"
#include "Interfaces/IAdvertisingProvider.h"
#include "Advertising.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Engine/StreamableManager.h"
#include "Net/OnlineEngineInterface.h"
#include "UObject/FieldPath.h"
#include "UserActivityTracking.h"
#include "KismetTraceUtils.h"
#include "Engine/AssetManager.h"
#include "RHI.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UObject/FieldPathProperty.h"
#include "Commandlets/Commandlet.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/TextProperty.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetSystemLibrary)

//////////////////////////////////////////////////////////////////////////
// UKismetSystemLibrary

#define LOCTEXT_NAMESPACE "UKismetSystemLibrary"

namespace UE::Blueprint::Private
{
	const FName PropertyGetFailedWarning = FName("PropertyGetFailedWarning");
	const FName PropertySetFailedWarning = FName("PropertySetFailedWarning");

	bool bBlamePrintString = false;
	FAutoConsoleVariableRef CVarBlamePrintString(TEXT("bp.BlamePrintString"), 
		bBlamePrintString,
		TEXT("When true, prints the Blueprint Asset and Function that generated calls to Print String. Useful for tracking down screen message spam."));

	void Generic_SetStructurePropertyByName(UObject* OwnerObject, FName StructPropertyName, const FStructProperty* SrcStructProperty, const void* SrcStructAddr)
	{
		if (OwnerObject != nullptr)
		{
			const FStructProperty* DestStructProperty = FindFProperty<FStructProperty>(OwnerObject->GetClass(), StructPropertyName);

			// SrcStructAddr and SrcStructProperty can be null in certain scenarios.
			// For example, retrieving an element reference from an array of user structs in BP can result in a null source.
			// We'll report a BP exception in that case, but we also need to soft-fail here to prevent an assert in CopyValuesInternal.

			const bool bHasValidParameters =
				(DestStructProperty != nullptr) &&
				(SrcStructProperty != nullptr) &&
				(SrcStructAddr != nullptr)
			;

			if (bHasValidParameters)
			{
				const UStruct* SourceStruct = SrcStructProperty->Struct;
				const UStruct* DestStruct = DestStructProperty->Struct;

				if (ensure(SourceStruct && DestStruct))
				{
					// For derived structs, this can lead to object slicing in some contexts, but that's expected behavior.

					if (SourceStruct->IsChildOf(DestStruct))
					{
						void* DestStructAddr = DestStructProperty->ContainerPtrToValuePtr<void>(OwnerObject);
						DestStructProperty->CopyValuesInternal(DestStructAddr, SrcStructAddr, 1);
					}
				}
			}
		}
	}

	// Convert the BP-exposed Draw Debug version of Scene Depth Priority Group to the corresponding engine type enum value
	ESceneDepthPriorityGroup ConvertSceneDepthPriorityToEngineType(EDrawDebugSceneDepthPriorityGroup DepthPriority)
	{
		return (DepthPriority == EDrawDebugSceneDepthPriorityGroup::World)
			? ESceneDepthPriorityGroup::SDPG_World : ESceneDepthPriorityGroup::SDPG_Foreground;
	}
}

UKismetSystemLibrary::UKismetSystemLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			UE::Blueprint::Private::PropertyGetFailedWarning,
			LOCTEXT("UE::Blueprint::Private::PropertyGetFailedWarning", "Property Get Failed")
		)
	);

	FBlueprintSupport::RegisterBlueprintWarning(
		FBlueprintWarningDeclaration(
			UE::Blueprint::Private::PropertySetFailedWarning,
			LOCTEXT("UE::Blueprint::Private::PropertySetFailedWarning", "Property Set Failed")
		)
	);
}

void UKismetSystemLibrary::StackTraceImpl(const FFrame& StackFrame)
{
	const FString Trace = StackFrame.GetStackTrace();
	UE_LOGF(LogBlueprintUserMessages, Log, "\n%ls", *Trace);
}

void UKismetSystemLibrary::RaiseScriptError(const FString& ErrorMessage)
{
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING) || WITH_EDITOR
	FFrame* TopFrame = FFrame::GetThreadLocalTopStackFrame();
	if (TopFrame)
	{
	#if WITH_EDITOR
		const FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::UserRaisedError, FText::FromString(ErrorMessage));
		FBlueprintCoreDelegates::ThrowScriptException(TopFrame->Object, *TopFrame, ExceptionInfo);
	#else
		UE_LOGF(LogBlueprintUserMessages, Error, "%ls:\n%ls", *ErrorMessage, *TopFrame->GetStackTrace());
	#endif	// WITH_EDITOR
	}
#endif	// !(UE_BUILD_TEST || UE_BUILD_SHIPPING) || WITH_EDITOR
}

FString UKismetSystemLibrary::GetObjectName(const UObject* Object)
{
	return GetNameSafe(Object);
}

FString UKismetSystemLibrary::GetPathName(const UObject* Object)
{
	return GetPathNameSafe(Object);
}

FSoftObjectPath UKismetSystemLibrary::GetSoftObjectPath(const UObject* Object)
{
	return FSoftObjectPath(Object);
}

#if WITH_EDITOR
UObject* UKismetSystemLibrary::GetObjectFromSoftPath(const FSoftObjectPath& Path)
{
	UObject* Object = nullptr;
	Object = Path.ResolveObject();
	if (!Object)
	{
		Object = Path.TryLoad();
	}
	return Object;
}

UClass* UKismetSystemLibrary::GetClassFromSoftPath(const FSoftClassPath& Path)
{
	UClass* Class = nullptr;
	Class = Path.ResolveClass();
	if (!Class)
	{
		Class = Path.TryLoadClass<UObject>();
	}
	return Class;
}
#endif

FString UKismetSystemLibrary::GetSystemPath(const UObject* Object)
{
	if (Object && Object->IsAsset())
	{
		FString PackageFileName;
		FString PackageFile;
		if (FPackageName::TryConvertLongPackageNameToFilename(Object->GetPackage()->GetName(), PackageFileName) &&
			FPackageName::FindPackageFileWithoutExtension(PackageFileName, PackageFile))
		{
			return FPaths::ConvertRelativePathToFull(MoveTemp(PackageFile));
		}
	}
	return FString();
}

FString UKismetSystemLibrary::GetDisplayName(const UObject* Object)
{
	if (const AActor* Actor = Cast<const AActor>(Object))
	{
		return Actor->GetActorNameOrLabel();
	}
	else if (const UActorComponent* Component = Cast<const UActorComponent>(Object))
	{
		return Component->GetReadableName();
	}

	return Object ? Object->GetName() : FString();
}

FString UKismetSystemLibrary::GetClassDisplayName(const UClass* Class)
{
	return Class ? Class->GetName() : FString();
}

FSoftClassPath UKismetSystemLibrary::GetSoftClassPath(const UClass* Class)
{
	return FSoftClassPath(Class);
}

FTopLevelAssetPath UKismetSystemLibrary::GetClassTopLevelAssetPath(const UClass* Class)
{
	// This will succeed for all valid classes as they are never subobjects
	return FTopLevelAssetPath(Class);
}

FTopLevelAssetPath UKismetSystemLibrary::GetStructTopLevelAssetPath(const UScriptStruct* Struct)
{
	// This will succeed for all valid structs as they are never subobjects
	return FTopLevelAssetPath(Struct);
}

FTopLevelAssetPath UKismetSystemLibrary::GetEnumTopLevelAssetPath(const UEnum* Enum)
{
	// This will succeed for all valid enums as they are never subobjects
	return FTopLevelAssetPath(Enum);
}

UObject* UKismetSystemLibrary::GetOuterObject(const UObject* Object)
{
	return Object ? Object->GetOuter() : nullptr;
}

#if WITH_EDITOR
bool UKismetSystemLibrary::IsObjectCooked(const UObject* Object)
{
	return Object && Object->GetPackage()->HasAnyPackageFlags(PKG_Cooked);
}

bool UKismetSystemLibrary::ObjectHasEditorOnlyData(const UObject* Object)
{
	return Object && !Object->GetPackage()->HasAnyPackageFlags(PKG_FilterEditorOnly);
}

UObject* UKismetSystemLibrary::DuplicateObject(const UObject* SourceObject, UObject* Outer, const FName Name)
{
	return SourceObject ? ::DuplicateObject(SourceObject, Outer, Name) : nullptr;
}
#endif	// WITH_EDITOR

FString UKismetSystemLibrary::GetEngineVersion()
{
	return FEngineVersion::Current().ToString();
}

FString UKismetSystemLibrary::GetBuildVersion()
{
	return FApp::GetBuildVersion();
}

FString UKismetSystemLibrary::GetBuildConfiguration()
{
	return LexToString(FApp::GetBuildConfiguration());
}

FString UKismetSystemLibrary::GetGameName()
{
	return FString(FApp::GetProjectName());
}

FString UKismetSystemLibrary::GetProjectDirectory()
{
	return FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
}

FString UKismetSystemLibrary::GetProjectContentDirectory()
{
	return FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
}

FString UKismetSystemLibrary::GetProjectSavedDirectory()
{
	return FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir());
}

FString UKismetSystemLibrary::ConvertToRelativePath(const FString& InPath)
{
	return IFileManager::Get().ConvertToRelativePath(*InPath);
}

FString UKismetSystemLibrary::ConvertToAbsolutePath(const FString& Filename)
{
	return FPaths::ConvertRelativePathToFull(Filename);
}

FString UKismetSystemLibrary::NormalizeFilename(const FString& InPath)
{
	FString Normalized(InPath);
	FPaths::NormalizeFilename(Normalized);
	return Normalized;
}

FString UKismetSystemLibrary::GetGameBundleId()
{
	return FString(FPlatformProcess::GetGameBundleId());
}

FString UKismetSystemLibrary::GetPlatformUserName()
{
	return FString(FPlatformProcess::UserName());
}

FString UKismetSystemLibrary::GetPlatformUserDir()
{
	return FString(FPlatformProcess::UserDir());
}

bool UKismetSystemLibrary::DoesImplementInterface(const UObject* TestObject, TSubclassOf<UInterface> Interface)
{
	if (TestObject)
	{
		return UKismetSystemLibrary::DoesClassImplementInterface(TestObject->GetClass(), Interface);
	}

	return false;
}

bool UKismetSystemLibrary::DoesClassImplementInterface(const UClass* TestClass, TSubclassOf<UInterface> Interface)
{
	if (TestClass && Interface)
	{
		if (!Interface->IsChildOf(UInterface::StaticClass()))
		{
			LogRuntimeError(FText::Format(LOCTEXT("DoesClassImplementInterface.InvalidInterface", "Interface parameter {0} is not actually an interface."), FText::AsCultureInvariant(Interface->GetName())));
			return false;
		}

		return TestClass->ImplementsInterface(Interface);
	}

	return false;
}

double UKismetSystemLibrary::GetGameTimeInSeconds(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? World->GetTimeSeconds() : 0.0;
}

int64 UKismetSystemLibrary::GetFrameCount()
{
	return (int64) GFrameCounter;
}

#if WITH_EDITOR
double UKismetSystemLibrary::GetPlatformTime_Seconds()
{
	return FPlatformTime::Seconds();
}
#endif//WITH_EDITOR

bool UKismetSystemLibrary::IsServer(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? (World->GetNetMode() != NM_Client) : false;
}

bool UKismetSystemLibrary::IsDedicatedServer(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World)
	{
		return (World->GetNetMode() == NM_DedicatedServer);
	}
	return IsRunningDedicatedServer();
}

bool UKismetSystemLibrary::IsStandalone(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? (World->GetNetMode() == NM_Standalone) : false;
}

bool UKismetSystemLibrary::IsSplitScreen(const UObject* WorldContextObject)
{
	return HasMultipleLocalPlayers(WorldContextObject);
}

bool UKismetSystemLibrary::HasMultipleLocalPlayers(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	return World ? GEngine->HasMultipleLocalPlayers(World) : false;
}

bool UKismetSystemLibrary::IsPackagedForDistribution()
{
	return FPlatformMisc::IsPackagedForDistribution();
}

FString UKismetSystemLibrary::GetUniqueDeviceId()
{
	return FString();
}

FString UKismetSystemLibrary::GetDeviceId()
{
	return FPlatformMisc::GetDeviceId();
}

UClass* UKismetSystemLibrary::Conv_ObjectToClass(UObject* Object, TSubclassOf<UObject> Class)
{
	return Cast<UClass>(Object);
}

UObject* UKismetSystemLibrary::Conv_InterfaceToObject(const FScriptInterface& Interface)
{
	return Interface.GetObject();
}

bool UKismetSystemLibrary::IsValidInterface(const FScriptInterface& Interface)
{
	return IsValid(Interface.GetObject());
}

void UKismetSystemLibrary::LogString(const FString& InString, bool bPrintToLog)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING // Do not Print in Shipping or Test unless explictly enabled.
	if(bPrintToLog)
	{
		UE_LOGF(LogBlueprintUserMessages, Log, "%ls", *InString);
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Verbose, "%ls", *InString);
	}	
#endif
}

void UKismetSystemLibrary::PrintString(const UObject* WorldContextObject, const FString& InString, bool bPrintToScreen, bool bPrintToLog, FLinearColor TextColor, float Duration, const FName Key)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING // Do not Print in Shipping or Test unless explictly enabled.

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);
	FString Prefix;
	if (World)
	{
		if (World->WorldType == EWorldType::PIE)
		{
			switch(World->GetNetMode())
			{
				case NM_Client:
					// GPlayInEditorID 0 is always the server, so 1 will be first client.
					// You want to keep this logic in sync with GeneratePIEViewportWindowTitle and UpdatePlayInEditorWorldDebugString
					Prefix = FString::Printf(TEXT("Client %d: "), UE::GetPlayInEditorID());
					break;
				case NM_DedicatedServer:
				case NM_ListenServer:
					Prefix = FString::Printf(TEXT("Server: "));
					break;
				case NM_Standalone:
					break;
			}
		}
	}

#if DO_BLUEPRINT_GUARD
	if (UE::Blueprint::Private::bBlamePrintString && !FBlueprintContextTracker::Get().GetCurrentScriptStack().IsEmpty())
	{
		const TArrayView<const FFrame* const> ScriptStack = FBlueprintContextTracker::Get().GetCurrentScriptStack();
		Prefix = FString::Printf(TEXT("Blueprint Object: %s\nBlueprint Function: %s\n%s"), 
			*ScriptStack.Last()->Node->GetPackage()->GetPathName(),
			*ScriptStack.Last()->Node->GetName(),
			*Prefix);
	}
#endif

	const FString FinalDisplayString = Prefix + InString;
	FString FinalLogString = FinalDisplayString;

	static const FBoolConfigValueHelper DisplayPrintStringSource(TEXT("Kismet"), TEXT("bLogPrintStringSource"), GEngineIni);
	if (DisplayPrintStringSource)
	{
		const FString SourceObjectPrefix = FString::Printf(TEXT("[%s] "), *GetNameSafe(WorldContextObject));
		FinalLogString = SourceObjectPrefix + FinalLogString;
	}

	if (bPrintToLog)
	{
		UE_LOGF(LogBlueprintUserMessages, Log, "%ls", *FinalLogString);
		
		APlayerController* PC = (WorldContextObject ? UGameplayStatics::GetPlayerController(WorldContextObject, 0) : NULL);
		ULocalPlayer* LocalPlayer = (PC ? Cast<ULocalPlayer>(PC->Player) : NULL);
		if (LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->ViewportConsole)
		{
			LocalPlayer->ViewportClient->ViewportConsole->OutputText(FinalDisplayString);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Verbose, "%ls", *FinalLogString);
	}

	// Also output to the screen, if possible
	if (bPrintToScreen)
	{
		if (GAreScreenMessagesEnabled)
		{
			if (GConfig && Duration < 0)
			{
				GConfig->GetFloat( TEXT("Kismet"), TEXT("PrintStringDuration"), Duration, GEngineIni );
			}
			uint64 InnerKey = -1;
			if (Key != NAME_None)
			{
				InnerKey = GetTypeHash(Key);
			}
			GEngine->AddOnScreenDebugMessage(InnerKey, Duration, TextColor.ToFColor(true), FinalDisplayString);
		}
		else
		{
			UE_LOGF(LogBlueprint, VeryVerbose, "Screen messages disabled (!GAreScreenMessagesEnabled).  Cannot print to screen.");
		}
	}
#endif
}

void UKismetSystemLibrary::PrintText(const UObject* WorldContextObject, const FText InText, bool bPrintToScreen, bool bPrintToLog, FLinearColor TextColor, float Duration, const FName Key)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING // Do not Print in Shipping or Test unless explictly enabled.
	PrintString(WorldContextObject, InText.ToString(), bPrintToScreen, bPrintToLog, TextColor, Duration, Key);
#endif
}

void UKismetSystemLibrary::PrintWarning(const FString& InString)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || USE_LOGGING_IN_SHIPPING // Do not Print in Shipping or Test unless explictly enabled.
	PrintString(NULL, InString, true, true);
#endif
}

void UKismetSystemLibrary::SetWindowTitle(const FText& Title)
{
	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (GameEngine != nullptr)
	{
		TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin();
		if (GameViewportWindow.IsValid())
		{
			GameViewportWindow->SetTitle(Title);
		}
	}
}

void UKismetSystemLibrary::ExecuteConsoleCommand(const UObject* WorldContextObject, const FString& Command, APlayerController* Player)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);

	// First, try routing through the console manager directly.
	// This is needed in case the Exec commands have been compiled out, meaning that GEngine->Exec wouldn't route to anywhere.
	if (IConsoleManager::Get().ProcessUserConsoleInput(*Command, *GLog, World) == false)
	{
		APlayerController* TargetPC = Player || !World ? Player : World->GetFirstPlayerController();

		// Second, try routing through the primary player
		if (TargetPC)
		{
			TargetPC->ConsoleCommand(Command, true);
		}
		else
		{
			GEngine->Exec(World, *Command);
		}
	}
}

FString UKismetSystemLibrary::GetConsoleVariableStringValue(const FString& VariableName)
{
	FString Value;
	
	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		Value = Variable->GetString();
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Failed to find console variable '%ls'.", *VariableName);
	}
	
	return Value;
}

float UKismetSystemLibrary::GetConsoleVariableFloatValue(const FString& VariableName)
{
	float Value = 0.0f;

	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		Value = Variable->GetFloat();
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Failed to find console variable '%ls'.", *VariableName);
	}

	return Value;
}

int32 UKismetSystemLibrary::GetConsoleVariableIntValue(const FString& VariableName)
{
	int32 Value = 0;

	IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(*VariableName);
	if (Variable)
	{
		Value = Variable->GetInt();
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Failed to find console variable '%ls'.", *VariableName);
	}

	return Value;
}

bool UKismetSystemLibrary::GetConsoleVariableBoolValue(const FString& VariableName)
{
	return (GetConsoleVariableIntValue(VariableName) != 0);
}


void UKismetSystemLibrary::QuitGame(const UObject* WorldContextObject, class APlayerController* SpecificPlayer, TEnumAsByte<EQuitPreference::Type> QuitPreference, bool bIgnorePlatformRestrictions)
{
	APlayerController* TargetPC = SpecificPlayer ? SpecificPlayer : UGameplayStatics::GetPlayerController(WorldContextObject, 0);
	if( TargetPC )
	{
		if ( QuitPreference == EQuitPreference::Background)
		{
			TargetPC->ConsoleCommand("quit background");
		}
		else
		{
			if (bIgnorePlatformRestrictions)
			{
				TargetPC->ConsoleCommand("quit force");
			}
			else
			{
				TargetPC->ConsoleCommand("quit");
			}
		}
	}
}

#if WITH_EDITOR
void UKismetSystemLibrary::QuitEditor()
{
	GEngine->Exec(nullptr, TEXT("QUIT_EDITOR"), *GLog);
}
#endif	// WITH_EDITOR

bool UKismetSystemLibrary::K2_IsValidTimerHandle(FTimerHandle TimerHandle)
{
	return TimerHandle.IsValid();
}

FTimerHandle UKismetSystemLibrary::K2_InvalidateTimerHandle(FTimerHandle& TimerHandle)
{
	TimerHandle.Invalidate();
	return TimerHandle;
}

FTimerHandle UKismetSystemLibrary::K2_SetTimer(UObject* Object, FString FunctionName, float Time, bool bLooping, bool bMaxOncePerFrame, float InitialStartDelay, float InitialStartDelayVariance)
{
	FName const FunctionFName(*FunctionName);

	if (Object)
	{
		UFunction* const Func = Object->FindFunction(FunctionFName);
		if ( Func && (Func->ParmsSize > 0) )
		{
			// User passed in a valid function, but one that takes parameters
			// FTimerDynamicDelegate expects zero parameters and will choke on execution if it tries
			// to execute a mismatched function
			UE_LOGF(LogBlueprintUserMessages, Warning, "SetTimer passed a function (%ls) that expects parameters.", *FunctionName);
			return FTimerHandle();
		}
	}

	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, FunctionFName);
	return K2_SetTimerDelegate(Delegate, Time, bLooping, bMaxOncePerFrame, InitialStartDelay);
}

FTimerHandle UKismetSystemLibrary::K2_SetTimerForNextTick(UObject* Object, FString FunctionName)
{
	FName const FunctionFName(*FunctionName);

	if (Object)
	{
		UFunction* const Func = Object->FindFunction(FunctionFName);
		if (Func && (Func->ParmsSize > 0))
		{
			// User passed in a valid function, but one that takes parameters
			// FTimerDynamicDelegate expects zero parameters and will choke on execution if it tries
			// to execute a mismatched function
			UE_LOGF(LogBlueprintUserMessages, Warning, "SetTimerForNextTick passed a function (%ls) that expects parameters.", *FunctionName);
			return FTimerHandle();
		}
	}

	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, FunctionFName);
	return K2_SetTimerForNextTickDelegate(Delegate);
}

FTimerHandle UKismetSystemLibrary::K2_SetTimerDelegate(FTimerDynamicDelegate Delegate, float Time, bool bLooping, bool bMaxOncePerFrame, float InitialStartDelay, float InitialStartDelayVariance)
{
	FTimerHandle Handle;
	if (Delegate.IsBound())
	{
		const UWorld* const World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			InitialStartDelay += FMath::RandRange(-InitialStartDelayVariance, InitialStartDelayVariance);
			if (Time <= 0.f || (Time + InitialStartDelay) < 0.f)
			{
				FString ObjectName = GetNameSafe(Delegate.GetUObject());
				FString FunctionName = Delegate.GetFunctionName().ToString(); 
				FFrame::KismetExecutionMessage(*FString::Printf(TEXT("%s %s SetTimer passed a negative or zero time. The associated timer may fail to be created/fire! If using InitialStartDelayVariance, be sure it is smaller than (Time + InitialStartDelay)."), *ObjectName, *FunctionName), ELogVerbosity::Warning);
			}

			FTimerManager& TimerManager = World->GetTimerManager();
			Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			TimerManager.SetTimer(Handle, Delegate, Time, FTimerManagerTimerParameters { .bLoop = bLooping, .bMaxOncePerFrame = bMaxOncePerFrame, .FirstDelay = Time + InitialStartDelay });
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"SetTimer passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}

	return Handle;
}

FTimerHandle UKismetSystemLibrary::K2_SetTimerForNextTickDelegate(FTimerDynamicDelegate Delegate)
{
	FTimerHandle Handle;
	if (Delegate.IsBound())
	{
		const UWorld* const World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			Handle = TimerManager.SetTimerForNextTick(Delegate);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning,
			"SetTimerForNextTick passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}

	return Handle;
}

void UKismetSystemLibrary::K2_ClearTimer(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	K2_ClearTimerDelegate(Delegate);
}

void UKismetSystemLibrary::K2_ClearTimerDelegate(FTimerDynamicDelegate Delegate)
{
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			TimerManager.ClearTimer(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"ClearTimer passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
}

void UKismetSystemLibrary::K2_ClearTimerHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			World->GetTimerManager().ClearTimer(Handle);
		}
	}
}

void UKismetSystemLibrary::K2_ClearAndInvalidateTimerHandle(const UObject* WorldContextObject, FTimerHandle& Handle)
{
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			World->GetTimerManager().ClearTimer(Handle);
		}
	}
}

void UKismetSystemLibrary::K2_PauseTimer(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	K2_PauseTimerDelegate(Delegate);
}

void UKismetSystemLibrary::K2_PauseTimerDelegate(FTimerDynamicDelegate Delegate)
{
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			TimerManager.PauseTimer(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"PauseTimer passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
}

void UKismetSystemLibrary::K2_PauseTimerHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			World->GetTimerManager().PauseTimer(Handle);
		}
	}
}

void UKismetSystemLibrary::K2_UnPauseTimer(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	K2_UnPauseTimerDelegate(Delegate);
}

void UKismetSystemLibrary::K2_UnPauseTimerDelegate(FTimerDynamicDelegate Delegate)
{
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			TimerManager.UnPauseTimer(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning,
			"UnPauseTimer passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
}

void UKismetSystemLibrary::K2_UnPauseTimerHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			World->GetTimerManager().UnPauseTimer(Handle);
		}
	}
}

bool UKismetSystemLibrary::K2_IsTimerActive(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	return K2_IsTimerActiveDelegate(Delegate);
}

bool UKismetSystemLibrary::K2_IsTimerActiveDelegate(FTimerDynamicDelegate Delegate)
{
	bool bIsActive = false;
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			bIsActive = TimerManager.IsTimerActive(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"IsTimerActive passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}

	return bIsActive;
}

bool UKismetSystemLibrary::K2_IsTimerActiveHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	bool bIsActive = false;
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			bIsActive = World->GetTimerManager().IsTimerActive(Handle);
		}
	}

	return bIsActive;
}

bool UKismetSystemLibrary::K2_IsTimerPaused(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	return K2_IsTimerPausedDelegate(Delegate);
}

bool UKismetSystemLibrary::K2_IsTimerPausedDelegate(FTimerDynamicDelegate Delegate)
{
	bool bIsPaused = false;
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			bIsPaused = TimerManager.IsTimerPaused(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"IsTimerPaused passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
	return bIsPaused;
}

bool UKismetSystemLibrary::K2_IsTimerPausedHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	bool bIsPaused = false;
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			bIsPaused = World->GetTimerManager().IsTimerPaused(Handle);
		}
	}

	return bIsPaused;
}

bool UKismetSystemLibrary::K2_TimerExists(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	return K2_TimerExistsDelegate(Delegate);
}

bool UKismetSystemLibrary::K2_TimerExistsDelegate(FTimerDynamicDelegate Delegate)
{
	bool bTimerExists = false;
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			bTimerExists = TimerManager.TimerExists(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning,
			"TimerExists passed a bad function (%ls) or object (%ls)",
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
	return bTimerExists;
}

bool UKismetSystemLibrary::K2_TimerExistsHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	bool bTimerExists = false;
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			bTimerExists = World->GetTimerManager().TimerExists(Handle);
		}
	}

	return bTimerExists;
}

float UKismetSystemLibrary::K2_GetTimerElapsedTime(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	return K2_GetTimerElapsedTimeDelegate(Delegate);
}

float UKismetSystemLibrary::K2_GetTimerElapsedTimeDelegate(FTimerDynamicDelegate Delegate)
{
	float ElapsedTime = 0.f;
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			ElapsedTime = TimerManager.GetTimerElapsed(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"GetTimerElapsedTime passed a bad function (%ls) or object (%ls)", 
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
	return ElapsedTime;
}

float UKismetSystemLibrary::K2_GetTimerElapsedTimeHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	float ElapsedTime = 0.f;
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			ElapsedTime = World->GetTimerManager().GetTimerElapsed(Handle);
		}
	}

	return ElapsedTime;
}

float UKismetSystemLibrary::K2_GetTimerRemainingTime(UObject* Object, FString FunctionName)
{
	FTimerDynamicDelegate Delegate;
	Delegate.BindUFunction(Object, *FunctionName);

	return K2_GetTimerRemainingTimeDelegate(Delegate);
}

float UKismetSystemLibrary::K2_GetTimerRemainingTimeDelegate(FTimerDynamicDelegate Delegate)
{
	float RemainingTime = 0.f;
	if (Delegate.IsBound())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(Delegate.GetUObject(), EGetWorldErrorMode::LogAndReturnNull);
		if(World)
		{
			FTimerManager& TimerManager = World->GetTimerManager();
			FTimerHandle Handle = TimerManager.K2_FindDynamicTimerHandle(Delegate);
			RemainingTime = TimerManager.GetTimerRemaining(Handle);
		}
	}
	else
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, 
			"GetTimerRemainingTime passed a bad function (%ls) or object (%ls)", 
			*Delegate.GetFunctionName().ToString(), *GetNameSafe(Delegate.GetUObject()));
	}
	return RemainingTime;
}

float UKismetSystemLibrary::K2_GetTimerRemainingTimeHandle(const UObject* WorldContextObject, FTimerHandle Handle)
{
	float RemainingTime = 0.f;
	if (Handle.IsValid())
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World)
		{
			RemainingTime = World->GetTimerManager().GetTimerRemaining(Handle);
		}
	}

	return RemainingTime;
}

void UKismetSystemLibrary::SetIntPropertyByName(UObject* Object, FName PropertyName, int32 Value)
{
	if(Object != NULL)
	{
		FIntProperty* IntProp = FindFProperty<FIntProperty>(Object->GetClass(), PropertyName);
		if(IntProp != NULL)
		{
			IntProp->SetPropertyValue_InContainer(Object, Value);
		}		
	}
}

void UKismetSystemLibrary::SetInt64PropertyByName(UObject* Object, FName PropertyName, int64 Value)
{
	if (Object != NULL)
	{
		FInt64Property* IntProp = FindFProperty<FInt64Property>(Object->GetClass(), PropertyName);
		if (IntProp != NULL)
		{
			IntProp->SetPropertyValue_InContainer(Object, Value);
		}
	}
}

void UKismetSystemLibrary::SetBytePropertyByName(UObject* Object, FName PropertyName, uint8 Value)
{
	if(Object != NULL)
	{
		if(FByteProperty* ByteProp = FindFProperty<FByteProperty>(Object->GetClass(), PropertyName))
		{
			ByteProp->SetPropertyValue_InContainer(Object, Value);
		}
		else if(FEnumProperty* EnumProp = FindFProperty<FEnumProperty>(Object->GetClass(), PropertyName))
		{
			void* PropAddr = EnumProp->ContainerPtrToValuePtr<void>(Object);
			FNumericProperty* UnderlyingProp = EnumProp->GetUnderlyingProperty();
			UnderlyingProp->SetIntPropertyValue(PropAddr, (int64)Value);
		}
	}
}

void UKismetSystemLibrary::SetDoublePropertyByName(UObject* Object, FName PropertyName, double Value)
{
	if (Object != nullptr)
	{
		if (FDoubleProperty* DoubleProp = FindFProperty<FDoubleProperty>(Object->GetClass(), PropertyName))
		{
			DoubleProp->SetPropertyValue_InContainer(Object, Value);
		}
		// It's entirely possible that the property refers to a native float property,
		// so we need to make that check here.
		else if (FFloatProperty* FloatProp = FindFProperty<FFloatProperty>(Object->GetClass(), PropertyName))
		{
			float floatValue = static_cast<float>(Value);
			FloatProp->SetPropertyValue_InContainer(Object, floatValue);
		}
	}
}

void UKismetSystemLibrary::SetBoolPropertyByName(UObject* Object, FName PropertyName, bool Value)
{
	if(Object != NULL)
	{
		FBoolProperty* BoolProp = FindFProperty<FBoolProperty>(Object->GetClass(), PropertyName);
		if(BoolProp != NULL)
		{
			BoolProp->SetPropertyValue_InContainer(Object, Value );
		}
	}
}

void UKismetSystemLibrary::SetObjectPropertyByName(UObject* Object, FName PropertyName, UObject* Value)
{
	if (Object)
	{
		if (FObjectPropertyBase* ObjectProp = FindFProperty<FObjectPropertyBase>(Object->GetClass(), PropertyName))
		{
			if (!Value || Value->IsA(ObjectProp->PropertyClass)) // check it's the right type
			{
				ObjectProp->SetObjectPropertyValue_InContainer(Object, Value);
			}
		}
	}
}

void UKismetSystemLibrary::SetClassPropertyByName(UObject* Object, FName PropertyName, TSubclassOf<UObject> Value)
{
	if (Object)
	{
		if (FClassProperty* ClassProp = FindFProperty<FClassProperty>(Object->GetClass(), PropertyName))
		{
			if (!*Value || Value->IsChildOf(ClassProp->MetaClass)) // check it's the right type
			{
				ClassProp->SetObjectPropertyValue_InContainer(Object, *Value);
			}
		}
	}
}

void UKismetSystemLibrary::SetInterfacePropertyByName(UObject* Object, FName PropertyName, const FScriptInterface& Value)
{
	if (Object)
	{
		FInterfaceProperty* InterfaceProp = FindFProperty<FInterfaceProperty>(Object->GetClass(), PropertyName);
		if (InterfaceProp != NULL && Value.GetObject()->GetClass()->ImplementsInterface(InterfaceProp->InterfaceClass)) // check it's the right type
		{
			InterfaceProp->SetPropertyValue_InContainer(Object, Value);
		}
	}
}

void UKismetSystemLibrary::SetStringPropertyByName(UObject* Object, FName PropertyName, const FString& Value)
{
	if(Object != NULL)
	{
		FStrProperty* StringProp = FindFProperty<FStrProperty>(Object->GetClass(), PropertyName);
		if(StringProp != NULL)
		{
			StringProp->SetPropertyValue_InContainer(Object, Value);
		}		
	}
}

void UKismetSystemLibrary::SetNamePropertyByName(UObject* Object, FName PropertyName, const FName& Value)
{
	if(Object != NULL)
	{
		FNameProperty* NameProp = FindFProperty<FNameProperty>(Object->GetClass(), PropertyName);
		if(NameProp != NULL)
		{
			NameProp->SetPropertyValue_InContainer(Object, Value);
		}		
	}
}

void UKismetSystemLibrary::SetSoftObjectPropertyByName(UObject* Object, FName PropertyName, const TSoftObjectPtr<UObject>& Value)
{
	if (Object != NULL)
	{
		FSoftObjectProperty* ObjectProp = FindFProperty<FSoftObjectProperty>(Object->GetClass(), PropertyName);
		const FSoftObjectPtr* SoftObjectPtr = (const FSoftObjectPtr*)(&Value);
		ObjectProp->SetPropertyValue_InContainer(Object, *SoftObjectPtr);
	}
}

void UKismetSystemLibrary::SetFieldPathPropertyByName(UObject* Object, FName PropertyName, const TFieldPath<FField>& Value)
{
	if (Object != NULL)
	{
		FFieldPathProperty* FieldProp = FindFProperty<FFieldPathProperty>(Object->GetClass(), PropertyName);
		const FFieldPath* FieldPathPtr = (const FFieldPath*)(&Value);
		FieldProp->SetPropertyValue_InContainer(Object, *FieldPathPtr);
	}
}

DEFINE_FUNCTION(UKismetSystemLibrary::execSetCollisionProfileNameProperty)
{
	P_GET_OBJECT(UObject, OwnerObject);
	P_GET_PROPERTY(FNameProperty, StructPropertyName);

	Stack.StepCompiledIn<FStructProperty>(nullptr);
	FStructProperty* SrcStructProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* SrcStructAddr = Stack.MostRecentPropertyAddress;

	P_FINISH;
	P_NATIVE_BEGIN;
	UE::Blueprint::Private::Generic_SetStructurePropertyByName(OwnerObject, StructPropertyName, SrcStructProperty, SrcStructAddr);
	P_NATIVE_END;
}

DEFINE_FUNCTION(UKismetSystemLibrary::execSetStructurePropertyByName)
{
	P_GET_OBJECT(UObject, OwnerObject);
	P_GET_PROPERTY(FNameProperty, StructPropertyName);

	Stack.StepCompiledIn<FStructProperty>(nullptr);
	FStructProperty* SrcStructProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* SrcStructAddr = Stack.MostRecentPropertyAddress;

	P_FINISH;
	P_NATIVE_BEGIN;
	UE::Blueprint::Private::Generic_SetStructurePropertyByName(OwnerObject, StructPropertyName, SrcStructProperty, SrcStructAddr);
	P_NATIVE_END;
}

void UKismetSystemLibrary::SetSoftClassPropertyByName(UObject* Object, FName PropertyName, const TSoftClassPtr<UObject>& Value)
{
	if (Object != NULL)
	{
		FSoftClassProperty* ObjectProp = FindFProperty<FSoftClassProperty>(Object->GetClass(), PropertyName);
		const FSoftObjectPtr* SoftObjectPtr = (const FSoftObjectPtr*)(&Value);
		ObjectProp->SetPropertyValue_InContainer(Object, *SoftObjectPtr);
	}
}

FSoftObjectPath UKismetSystemLibrary::MakeSoftObjectPath(const FString& PathString)
{
	FSoftObjectPath SoftObjectPath(PathString);
	if (!PathString.IsEmpty() && !SoftObjectPath.IsValid())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PathString"), FText::AsCultureInvariant(PathString));
		LogRuntimeError(FText::Format(NSLOCTEXT("KismetSystemLibrary", "SoftObjectPath_PathStringInvalid", "Object path {PathString} not valid for MakeSoftObjectPath."), Args));
	}

	return SoftObjectPath;
}

void UKismetSystemLibrary::BreakSoftObjectPath(FSoftObjectPath InSoftObjectPath, FString& PathString)
{
	PathString = InSoftObjectPath.ToString();
}

TSoftObjectPtr<UObject> UKismetSystemLibrary::Conv_SoftObjPathToSoftObjRef(const FSoftObjectPath& SoftObjectPath)
{
	return TSoftObjectPtr<UObject>(SoftObjectPath);
}

FSoftObjectPath UKismetSystemLibrary::Conv_SoftObjRefToSoftObjPath(TSoftObjectPtr<UObject> SoftObjectReference)
{
	return SoftObjectReference.ToSoftObjectPath();
}

FTopLevelAssetPath UKismetSystemLibrary::MakeTopLevelAssetPath(const FString& FullPathOrPackageName, const FString& AssetName)
{
	if (!FullPathOrPackageName.StartsWith(TEXT("/")))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PathString"), FText::AsCultureInvariant(FullPathOrPackageName));
		LogRuntimeError(FText::Format(NSLOCTEXT("KismetSystemLibrary", "TopLevelAssetPath_UnrootedPath",
			"Short path \"{PathString}\" not valid for MakeTopLevelAssetPath. Path must be rooted with /, for example /Game."), Args));

		return FTopLevelAssetPath();
	}
	
	if (AssetName.IsEmpty())
	{
		FTopLevelAssetPath Result = FTopLevelAssetPath(FullPathOrPackageName);
		if (!Result.IsValid() && !FullPathOrPackageName.IsEmpty())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("PathString"), FText::AsCultureInvariant(FullPathOrPackageName));
			LogRuntimeError(FText::Format(NSLOCTEXT("KismetSystemLibrary", "TopLevelAssetPath_PathStringInvalid",
				"String \"{PathString}\" is invalid for MakeTopLevelAssetPath."), Args));
		}
		return Result;
	}
	else
	{
		FTopLevelAssetPath Result = FTopLevelAssetPath(*FullPathOrPackageName, *AssetName);
		if (!Result.IsValid())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("PackageName"), FText::AsCultureInvariant(FullPathOrPackageName));
			Args.Add(TEXT("AssetName"), FText::AsCultureInvariant(AssetName));
			LogRuntimeError(FText::Format(NSLOCTEXT("KismetSystemLibrary", "TopLevelAssetPath_PackageAndAssetStringsInvalid",
				"Strings (\"{PackageName}\", \"{AssetName}\") are invalid for MakeTopLevelAssetPath."), Args));
		}
		return Result;
	}
}

void UKismetSystemLibrary::BreakTopLevelAssetPath(const FTopLevelAssetPath& InTopLevelAssetPath, FString& PathString)
{
	InTopLevelAssetPath.ToString(PathString);
}

FSoftClassPath UKismetSystemLibrary::MakeSoftClassPath(const FString& PathString)
{
	FSoftClassPath SoftClassPath(PathString);
	if (!PathString.IsEmpty() && !SoftClassPath.IsValid())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PathString"), FText::AsCultureInvariant(PathString));
		LogRuntimeError(FText::Format(NSLOCTEXT("KismetSystemLibrary", "SoftClassPath_PathStringInvalid", "Object path {PathString} not valid for MakeSoftClassPath."), Args));
	}

	return SoftClassPath;
}

void UKismetSystemLibrary::BreakSoftClassPath(FSoftClassPath InSoftClassPath, FString& PathString)
{
	PathString = InSoftClassPath.ToString();
}

TSoftClassPtr<UObject> UKismetSystemLibrary::Conv_SoftClassPathToSoftClassRef(const FSoftClassPath& SoftClassPath)
{
	return TSoftClassPtr<UObject>(SoftClassPath);
}

FSoftClassPath UKismetSystemLibrary::Conv_SoftObjRefToSoftClassPath(TSoftClassPtr<UObject> SoftClassReference)
{
	// TSoftClassPtr and FSoftClassPath are not directly compatible
	return FSoftClassPath(SoftClassReference.ToString());
}

bool UKismetSystemLibrary::IsValidSoftObjectReference(const TSoftObjectPtr<UObject>& SoftObjectReference)
{
	return !SoftObjectReference.IsNull();
}

FString UKismetSystemLibrary::Conv_SoftObjectReferenceToString(const TSoftObjectPtr<UObject>& SoftObjectReference)
{
	return SoftObjectReference.ToString();
}

bool UKismetSystemLibrary::EqualEqual_SoftObjectReference(const TSoftObjectPtr<UObject>& A, const TSoftObjectPtr<UObject>& B)
{
	return A == B;
}

bool UKismetSystemLibrary::NotEqual_SoftObjectReference(const TSoftObjectPtr<UObject>& A, const TSoftObjectPtr<UObject>& B)
{
	return A != B;
}

UObject* UKismetSystemLibrary::LoadAsset_Blocking(TSoftObjectPtr<UObject> Asset)
{
	return Asset.LoadSynchronous();
}

bool UKismetSystemLibrary::IsValidSoftClassReference(const TSoftClassPtr<UObject>& SoftClassReference)
{
	return !SoftClassReference.IsNull();
}

FTopLevelAssetPath UKismetSystemLibrary::GetSoftClassTopLevelAssetPath(TSoftClassPtr<UObject> SoftClassReference)
{
	const FSoftObjectPath& ObjectPath = SoftClassReference.ToSoftObjectPath();

	// Class paths should never have a subpath
	ensure(ObjectPath.GetSubPathString().IsEmpty());
	return ObjectPath.GetAssetPath();
}

FString UKismetSystemLibrary::Conv_SoftClassReferenceToString(const TSoftClassPtr<UObject>& SoftClassReference)
{
	return SoftClassReference.ToString();
}

bool UKismetSystemLibrary::EqualEqual_SoftClassReference(const TSoftClassPtr<UObject>& A, const TSoftClassPtr<UObject>& B)
{
	return A == B;
}

bool UKismetSystemLibrary::NotEqual_SoftClassReference(const TSoftClassPtr<UObject>& A, const TSoftClassPtr<UObject>& B)
{
	return A != B;
}

UClass* UKismetSystemLibrary::LoadClassAsset_Blocking(TSoftClassPtr<UObject> AssetClass)
{
	return AssetClass.LoadSynchronous();
}

bool UKismetSystemLibrary::IsObjectOfSoftClass(const UObject* Object, TSoftClassPtr<UObject> SoftClass)
{
	if (!Object)
	{
		return false;
	}

	TSubclassOf<UObject> ObjectClass = SoftClass.Get();
	if (!ObjectClass)
	{
		return false;
	}

	TSubclassOf<UInterface> InterfaceClass = ObjectClass.Get();
	if (InterfaceClass)
	{
		check(Object->GetClass());
		return Object->GetClass()->ImplementsInterface(InterfaceClass);
	}

	return Object->IsA(ObjectClass);
}

UObject* UKismetSystemLibrary::Conv_SoftObjectReferenceToObject(const TSoftObjectPtr<UObject>& SoftObject)
{
	return SoftObject.Get();
}

TSubclassOf<UObject> UKismetSystemLibrary::Conv_SoftClassReferenceToClass(const TSoftClassPtr<UObject>& SoftClass)
{
	return SoftClass.Get();
}

TSoftObjectPtr<UObject> UKismetSystemLibrary::Conv_ObjectToSoftObjectReference(UObject *Object)
{
	return TSoftObjectPtr<UObject>(Object);
}

TSoftClassPtr<UObject> UKismetSystemLibrary::Conv_ClassToSoftClassReference(const TSubclassOf<UObject>& Class)
{
	return TSoftClassPtr<UObject>(*Class);
}

FSoftComponentReference UKismetSystemLibrary::Conv_ComponentReferenceToSoftComponentReference(const FComponentReference& ComponentReference)
{
	FSoftComponentReference SoftComponentReference;
	SoftComponentReference.ComponentProperty = ComponentReference.ComponentProperty;
	SoftComponentReference.PathToComponent = ComponentReference.PathToComponent;
	if (ComponentReference.OtherActor.IsValid())
	{
		SoftComponentReference.OtherActor = ComponentReference.OtherActor.Get();
	}
	return SoftComponentReference;
}

void UKismetSystemLibrary::SetTextPropertyByName(UObject* Object, FName PropertyName, const FText& Value)
{
	if(Object != nullptr)
	{
		FTextProperty* TextProp = FindFProperty<FTextProperty>(Object->GetClass(), PropertyName);
		if(TextProp != nullptr)
		{
			TextProp->SetPropertyValue_InContainer(Object, Value);
		}		
	}
}

void UKismetSystemLibrary::SetVectorPropertyByName(UObject* Object, FName PropertyName, const FVector& Value)
{
	if(Object != nullptr)
	{
		UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		FStructProperty* VectorProp = FindFProperty<FStructProperty>(Object->GetClass(), PropertyName);
		if(VectorProp != nullptr && VectorProp->Struct == VectorStruct)
		{
			*VectorProp->ContainerPtrToValuePtr<FVector>(Object) = Value;
		}		
	}
}

void UKismetSystemLibrary::SetVector3fPropertyByName(UObject* Object, FName PropertyName, const FVector3f& Value)
{
	if (Object != nullptr)
	{
		UScriptStruct* Vector3fStruct = TVariantStructure<FVector3f>::Get();
		FStructProperty* Vector3fProp = FindFProperty<FStructProperty>(Object->GetClass(), PropertyName);
		if (Vector3fProp != nullptr && Vector3fProp->Struct == Vector3fStruct)
		{
			*Vector3fProp->ContainerPtrToValuePtr<FVector3f>(Object) = Value;
		}
	}
}

void UKismetSystemLibrary::SetRotatorPropertyByName(UObject* Object, FName PropertyName, const FRotator& Value)
{
	if(Object != nullptr)
	{
		UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
		FStructProperty* RotatorProp = FindFProperty<FStructProperty>(Object->GetClass(), PropertyName);
		if(RotatorProp != nullptr && RotatorProp->Struct == RotatorStruct)
		{
			*RotatorProp->ContainerPtrToValuePtr<FRotator>(Object) = Value;
		}		
	}
}

void UKismetSystemLibrary::SetLinearColorPropertyByName(UObject* Object, FName PropertyName, const FLinearColor& Value)
{
	if(Object != nullptr)
	{
		UScriptStruct* LinearColorStruct = TBaseStructure<FLinearColor>::Get();
		FStructProperty* LinearColorProp = FindFProperty<FStructProperty>(Object->GetClass(), PropertyName);
		if(LinearColorProp != nullptr && LinearColorProp->Struct == LinearColorStruct)
		{
			*LinearColorProp->ContainerPtrToValuePtr<FLinearColor>(Object) = Value;
		}		
	}
}

void UKismetSystemLibrary::SetColorPropertyByName(UObject* Object, FName PropertyName, const FColor& Value)
{
	if (Object != nullptr)
	{
		UScriptStruct* ColorStruct = TBaseStructure<FColor>::Get();
		FStructProperty* ColorProp = FindFProperty<FStructProperty>(Object->GetClass(), PropertyName);
		if (ColorProp != nullptr && ColorProp->Struct == ColorStruct)
		{
			*ColorProp->ContainerPtrToValuePtr<FColor>(Object) = Value;
		}
	}
}

void UKismetSystemLibrary::SetTransformPropertyByName(UObject* Object, FName PropertyName, const FTransform& Value)
{
	if(Object != nullptr)
	{
		UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
		FStructProperty* TransformProp = FindFProperty<FStructProperty>(Object->GetClass(), PropertyName);
		if(TransformProp != nullptr && TransformProp->Struct == TransformStruct)
		{
			*TransformProp->ContainerPtrToValuePtr<FTransform>(Object) = Value;
		}		
	}
}

void UKismetSystemLibrary::SetCollisionProfileNameProperty(UObject* Object, FName PropertyName, const FCollisionProfileName& Value)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.
	check(0);
}

void UKismetSystemLibrary::GetActorListFromComponentList(const TArray<UPrimitiveComponent*>& ComponentList, UClass* ActorClassFilter, TArray<class AActor*>& OutActorList)
{
	OutActorList.Empty();
	for (int32 CompIdx=0; CompIdx<ComponentList.Num(); ++CompIdx)
	{
		UPrimitiveComponent* const C = ComponentList[CompIdx];
		if (C)
		{
			AActor* const Owner = C->GetOwner();
			if (Owner)
			{
				if ( !ActorClassFilter || Owner->IsA(ActorClassFilter) )
				{
					OutActorList.AddUnique(Owner);
				}
			}
		}
	}
}

bool UKismetSystemLibrary::SphereOverlapActors(const UObject* WorldContextObject, const FVector SpherePos, float SphereRadius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<AActor*>& OutActors)
{
	OutActors.Empty();

	TArray<UPrimitiveComponent*> OverlapComponents;
	bool bOverlapped = SphereOverlapComponents(WorldContextObject, SpherePos, SphereRadius, ObjectTypes, NULL, ActorsToIgnore, OverlapComponents);
	if (bOverlapped)
	{
		GetActorListFromComponentList(OverlapComponents, ActorClassFilter, OutActors);
	}

	return (OutActors.Num() > 0);
}

bool UKismetSystemLibrary::SphereOverlapComponents(const UObject* WorldContextObject, const FVector SpherePos, float SphereRadius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<UPrimitiveComponent*>& OutComponents)
{
	OutComponents.Empty();

	FCollisionQueryParams Params(SCENE_QUERY_STAT(SphereOverlapComponents), false);
	Params.AddIgnoredActors(ActorsToIgnore);
	TArray<FOverlapResult> Overlaps;

	FCollisionObjectQueryParams ObjectParams;
	for (auto Iter = ObjectTypes.CreateConstIterator(); Iter; ++Iter)
	{
		const ECollisionChannel & Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
		ObjectParams.AddObjectTypesToQuery(Channel);
	}


	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if(World != nullptr)
	{
		World->OverlapMultiByObjectType(Overlaps, SpherePos, FQuat::Identity, ObjectParams, FCollisionShape::MakeSphere(SphereRadius), Params);
	}

	for (int32 OverlapIdx=0; OverlapIdx<Overlaps.Num(); ++OverlapIdx)
	{
		FOverlapResult const& O = Overlaps[OverlapIdx];
		if (O.Component.IsValid())
		{ 
			if ( !ComponentClassFilter || O.Component.Get()->IsA(ComponentClassFilter) )
			{
				OutComponents.Add(O.Component.Get());
			}
		}
	}

	return (OutComponents.Num() > 0);
}



bool UKismetSystemLibrary::BoxOverlapActors(const UObject* WorldContextObject, const FVector BoxPos, FVector BoxExtent, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<AActor*>& OutActors)
{
	return BoxOverlapActorsWithOrientation(WorldContextObject, BoxPos, BoxExtent, FRotator::ZeroRotator, ObjectTypes, ActorClassFilter, ActorsToIgnore, OutActors);
}

bool UKismetSystemLibrary::BoxOverlapActorsWithOrientation(const UObject* WorldContextObject, const FVector BoxPos, FVector BoxExtent, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<AActor*>& OutActors)
{
	OutActors.Empty();

	TArray<UPrimitiveComponent*> OverlapComponents;
	bool bOverlapped = BoxOverlapComponentsWithOrientation(WorldContextObject, BoxPos, BoxExtent, Orientation, ObjectTypes, NULL, ActorsToIgnore, OverlapComponents);
	if (bOverlapped)
	{
		GetActorListFromComponentList(OverlapComponents, ActorClassFilter, OutActors);
	}

	return (OutActors.Num() > 0);
}

bool UKismetSystemLibrary::BoxOverlapComponents(const UObject* WorldContextObject, const FVector BoxPos, FVector BoxExtent, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<UPrimitiveComponent*>& OutComponents)
{
	return BoxOverlapComponentsWithOrientation(WorldContextObject, BoxPos, BoxExtent, FRotator::ZeroRotator, ObjectTypes, ComponentClassFilter, ActorsToIgnore, OutComponents);
}

bool UKismetSystemLibrary::BoxOverlapComponentsWithOrientation(const UObject* WorldContextObject, const FVector BoxPos, FVector BoxExtent, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> >& ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<UPrimitiveComponent*>& OutComponents)
{
	OutComponents.Empty();

	FCollisionQueryParams Params(SCENE_QUERY_STAT(BoxOverlapComponents), false);
	Params.AddIgnoredActors(ActorsToIgnore);

	TArray<FOverlapResult> Overlaps;

	FCollisionObjectQueryParams ObjectParams;
	for (auto Iter = ObjectTypes.CreateConstIterator(); Iter; ++Iter)
	{
		const ECollisionChannel& Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
		ObjectParams.AddObjectTypesToQuery(Channel);
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World != nullptr)
	{
		World->OverlapMultiByObjectType(Overlaps, BoxPos, Orientation.Quaternion(), ObjectParams, FCollisionShape::MakeBox(BoxExtent), Params);
	}

	for (int32 OverlapIdx = 0; OverlapIdx < Overlaps.Num(); ++OverlapIdx)
	{
		FOverlapResult const& O = Overlaps[OverlapIdx];
		if (O.Component.IsValid())
		{
			if (!ComponentClassFilter || O.Component.Get()->IsA(ComponentClassFilter))
			{
				OutComponents.Add(O.Component.Get());
			}
		}
	}

	return (OutComponents.Num() > 0);
}


bool UKismetSystemLibrary::CapsuleOverlapActors(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> >& ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<AActor*>& OutActors)
{
	return CapsuleOverlapActorsWithOrientation(WorldContextObject, CapsulePos, Radius, HalfHeight, FRotator::ZeroRotator, ObjectTypes, ActorClassFilter, ActorsToIgnore, OutActors);
}

bool UKismetSystemLibrary::CapsuleOverlapActorsWithOrientation(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<AActor*>& OutActors)
{
	OutActors.Empty();

	TArray<UPrimitiveComponent*> OverlapComponents;
	bool bOverlapped = CapsuleOverlapComponentsWithOrientation(WorldContextObject, CapsulePos, Radius, HalfHeight, Orientation, ObjectTypes, NULL, ActorsToIgnore, OverlapComponents);
	if (bOverlapped)
	{
		GetActorListFromComponentList(OverlapComponents, ActorClassFilter, OutActors);
	}

	return (OutActors.Num() > 0);
}

bool UKismetSystemLibrary::CapsuleOverlapComponents(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> >& ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<UPrimitiveComponent*>& OutComponents)
{
	return CapsuleOverlapComponentsWithOrientation(WorldContextObject, CapsulePos, Radius, HalfHeight, FRotator::ZeroRotator, ObjectTypes, ComponentClassFilter, ActorsToIgnore, OutComponents);
}

bool UKismetSystemLibrary::CapsuleOverlapComponentsWithOrientation(const UObject* WorldContextObject, const FVector CapsulePos, float Radius, float HalfHeight, FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<UPrimitiveComponent*>& OutComponents)
{
	OutComponents.Empty();

	FCollisionQueryParams Params(SCENE_QUERY_STAT(CapsuleOverlapComponents), false);
	Params.AddIgnoredActors(ActorsToIgnore);

	TArray<FOverlapResult> Overlaps;

	FCollisionObjectQueryParams ObjectParams;
	for (auto Iter = ObjectTypes.CreateConstIterator(); Iter; ++Iter)
	{
		const ECollisionChannel & Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
		ObjectParams.AddObjectTypesToQuery(Channel);
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World != nullptr)
	{
		World->OverlapMultiByObjectType(Overlaps, CapsulePos, Orientation.Quaternion(), ObjectParams, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params);
	}

	for (int32 OverlapIdx=0; OverlapIdx<Overlaps.Num(); ++OverlapIdx)
	{
		FOverlapResult const& O = Overlaps[OverlapIdx];
		if (O.Component.IsValid())
		{ 
			if ( !ComponentClassFilter || O.Component.Get()->IsA(ComponentClassFilter) )
			{
				OutComponents.Add(O.Component.Get());
			}
		}
	}

	return (OutComponents.Num() > 0);
}


bool UKismetSystemLibrary::ComponentOverlapActors(UPrimitiveComponent* Component, const FTransform& ComponentTransform, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ActorClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<AActor*>& OutActors)
{
	OutActors.Empty();

	TArray<UPrimitiveComponent*> OverlapComponents;
	bool bOverlapped = ComponentOverlapComponents(Component, ComponentTransform, ObjectTypes, NULL, ActorsToIgnore, OverlapComponents);
	if (bOverlapped)
	{
		GetActorListFromComponentList(OverlapComponents, ActorClassFilter, OutActors);
	}

	return (OutActors.Num() > 0);
}

bool UKismetSystemLibrary::ComponentOverlapComponents(UPrimitiveComponent* Component, const FTransform& ComponentTransform, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, UClass* ComponentClassFilter, const TArray<AActor*>& ActorsToIgnore, TArray<UPrimitiveComponent*>& OutComponents)
{
	OutComponents.Empty();

	if(Component != nullptr)
	{
		FComponentQueryParams Params(SCENE_QUERY_STAT(ComponentOverlapComponents));
		Params.AddIgnoredActors(ActorsToIgnore);

		TArray<FOverlapResult> Overlaps;

		FCollisionObjectQueryParams ObjectParams;
		for (auto Iter = ObjectTypes.CreateConstIterator(); Iter; ++Iter)
		{
			const ECollisionChannel & Channel = UCollisionProfile::Get()->ConvertToCollisionChannel(false, *Iter);
			ObjectParams.AddObjectTypesToQuery(Channel);
		}

		check( Component->GetWorld());
		Component->GetWorld()->ComponentOverlapMulti(Overlaps, Component, ComponentTransform.GetTranslation(), ComponentTransform.GetRotation(), Params, ObjectParams);

		for (int32 OverlapIdx=0; OverlapIdx<Overlaps.Num(); ++OverlapIdx)
		{
			FOverlapResult const& O = Overlaps[OverlapIdx];
			if (O.Component.IsValid())
			{ 
				if ( !ComponentClassFilter || O.Component.Get()->IsA(ComponentClassFilter) )
				{
					OutComponents.Add(O.Component.Get());
				}
			}
		}
	}

	return (OutComponents.Num() > 0);
}


bool UKismetSystemLibrary::LineTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);

	static const FName LineTraceSingleName("LineTraceSingle");
	FCollisionQueryParams Params = ConfigureCollisionParams(LineTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->LineTraceSingleByChannel(OutHit, Start, End, CollisionChannel, Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugLineTraceSingle(World, Start, End, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::LineTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);

	static const FName LineTraceMultiName("LineTraceMulti");
	FCollisionQueryParams Params = ConfigureCollisionParams(LineTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->LineTraceMultiByChannel(OutHits, Start, End, CollisionChannel, Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugLineTraceMulti(World, Start, End, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::BoxTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName BoxTraceSingleName("BoxTraceSingle");
	FCollisionQueryParams Params = ConfigureCollisionParams(BoxTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByChannel(OutHit, Start, End, Orientation.Quaternion(), UEngineTypes::ConvertToCollisionChannel(TraceChannel), FCollisionShape::MakeBox(HalfSize), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugBoxTraceSingle(World, Start, End, HalfSize, Orientation, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::BoxTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName BoxTraceMultiName("BoxTraceMulti");
	FCollisionQueryParams Params = ConfigureCollisionParams(BoxTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByChannel(OutHits, Start, End, Orientation.Quaternion(), UEngineTypes::ConvertToCollisionChannel(TraceChannel), FCollisionShape::MakeBox(HalfSize), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugBoxTraceMulti(World, Start, End, HalfSize, Orientation, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::SphereTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);

	static const FName SphereTraceSingleName("SphereTraceSingle");
	FCollisionQueryParams Params = ConfigureCollisionParams(SphereTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByChannel(OutHit, Start, End, FQuat::Identity, CollisionChannel, FCollisionShape::MakeSphere(Radius), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugSphereTraceSingle(World, Start, End, Radius, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::SphereTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);

	static const FName SphereTraceMultiName("SphereTraceMulti");
	FCollisionQueryParams Params = ConfigureCollisionParams(SphereTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByChannel(OutHits, Start, End, FQuat::Identity, CollisionChannel, FCollisionShape::MakeSphere(Radius), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugSphereTraceMulti(World, Start, End, Radius, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::CapsuleTraceSingle(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);

	static const FName CapsuleTraceSingleName("CapsuleTraceSingle");
	FCollisionQueryParams Params = ConfigureCollisionParams(CapsuleTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByChannel(OutHit, Start, End, FQuat::Identity, CollisionChannel, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugCapsuleTraceSingle(World, Start, End, Radius, HalfHeight, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}


bool UKismetSystemLibrary::CapsuleTraceMulti(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, ETraceTypeQuery TraceChannel, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel);

	static const FName CapsuleTraceMultiName("CapsuleTraceMulti");
	FCollisionQueryParams Params = ConfigureCollisionParams(CapsuleTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByChannel(OutHits, Start, End, FQuat::Identity, CollisionChannel, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugCapsuleTraceMulti(World, Start, End, Radius, HalfHeight, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;	
}

/** Object Query functions **/
bool UKismetSystemLibrary::LineTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName LineTraceSingleName("LineTraceSingleForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(LineTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->LineTraceSingleByObjectType(OutHit, Start, End, ObjectParams, Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugLineTraceSingle(World, Start, End, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::LineTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName LineTraceMultiName("LineTraceMultiForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(LineTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->LineTraceMultiByObjectType(OutHits, Start, End, ObjectParams, Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugLineTraceMulti(World, Start, End, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::SphereTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName SphereTraceSingleName("SphereTraceSingleForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(SphereTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByObjectType(OutHit, Start, End, FQuat::Identity, ObjectParams, FCollisionShape::MakeSphere(Radius), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugSphereTraceSingle(World, Start, End, Radius, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::SphereTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName SphereTraceMultiName("SphereTraceMultiForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(SphereTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByObjectType(OutHits, Start, End, FQuat::Identity, ObjectParams, FCollisionShape::MakeSphere(Radius), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugSphereTraceMulti(World, Start, End, Radius, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::BoxTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName BoxTraceSingleName("BoxTraceSingleForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(BoxTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	TArray<TEnumAsByte<ECollisionChannel>> CollisionObjectTraces;
	CollisionObjectTraces.AddUninitialized(ObjectTypes.Num());

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByObjectType(OutHit, Start, End, Orientation.Quaternion(), ObjectParams, FCollisionShape::MakeBox(HalfSize), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugBoxTraceSingle(World, Start, End, HalfSize, Orientation, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::BoxTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, const TArray<TEnumAsByte<	EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName BoxTraceMultiName("BoxTraceMultiForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(BoxTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByObjectType(OutHits, Start, End, Orientation.Quaternion(), ObjectParams, FCollisionShape::MakeBox(HalfSize), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugBoxTraceMulti(World, Start, End, HalfSize, Orientation, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::CapsuleTraceSingleForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName CapsuleTraceSingleName("CapsuleTraceSingleForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(CapsuleTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByObjectType(OutHit, Start, End, FQuat::Identity, ObjectParams, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugCapsuleTraceSingle(World, Start, End, Radius, HalfHeight, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::CapsuleTraceMultiForObjects(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName CapsuleTraceMultiName("CapsuleTraceMultiForObjects");
	FCollisionQueryParams Params = ConfigureCollisionParams(CapsuleTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	FCollisionObjectQueryParams ObjectParams = ConfigureCollisionObjectParams(ObjectTypes);
	if (ObjectParams.IsValid() == false)
	{
		UE_LOGF(LogBlueprintUserMessages, Warning, "Invalid object types");
		return false;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByObjectType(OutHits, Start, End, FQuat::Identity, ObjectParams, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugCapsuleTraceMulti(World, Start, End, Radius, HalfHeight, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}


bool UKismetSystemLibrary::LineTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName LineTraceSingleName("LineTraceSingleByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(LineTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->LineTraceSingleByProfile(OutHit, Start, End, ProfileName, Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugLineTraceSingle(World, Start, End, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::LineTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName LineTraceMultiName("LineTraceMultiByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(LineTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->LineTraceMultiByProfile(OutHits, Start, End, ProfileName, Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugLineTraceMulti(World, Start, End, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::BoxTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName BoxTraceSingleName("BoxTraceSingleByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(BoxTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	bool const bHit = World ? World->SweepSingleByProfile(OutHit, Start, End, Orientation.Quaternion(), ProfileName, FCollisionShape::MakeBox(HalfSize), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugBoxTraceSingle(World, Start, End, HalfSize, Orientation, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::BoxTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, const FVector HalfSize, const FRotator Orientation, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName BoxTraceMultiName("BoxTraceMultiByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(BoxTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByProfile(OutHits, Start, End, Orientation.Quaternion(), ProfileName, FCollisionShape::MakeBox(HalfSize), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugBoxTraceMulti(World, Start, End, HalfSize, Orientation, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::SphereTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName SphereTraceSingleName("SphereTraceSingleByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(SphereTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByProfile(OutHit, Start, End, FQuat::Identity, ProfileName, FCollisionShape::MakeSphere(Radius), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugSphereTraceSingle(World, Start, End, Radius, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::SphereTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName SphereTraceMultiName("SphereTraceMultiByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(SphereTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByProfile(OutHits, Start, End, FQuat::Identity, ProfileName, FCollisionShape::MakeSphere(Radius), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugSphereTraceMulti(World, Start, End, Radius, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

bool UKismetSystemLibrary::CapsuleTraceSingleByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, FHitResult& OutHit, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName CapsuleTraceSingleName("CapsuleTraceSingleByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(CapsuleTraceSingleName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepSingleByProfile(OutHit, Start, End, FQuat::Identity, ProfileName, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugCapsuleTraceSingle(World, Start, End, Radius, HalfHeight, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}


bool UKismetSystemLibrary::CapsuleTraceMultiByProfile(const UObject* WorldContextObject, const FVector Start, const FVector End, float Radius, float HalfHeight, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, EDrawDebugTrace::Type DrawDebugType, TArray<FHitResult>& OutHits, bool bIgnoreSelf, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	static const FName CapsuleTraceMultiName("CapsuleTraceMultiByProfile");
	FCollisionQueryParams Params = ConfigureCollisionParams(CapsuleTraceMultiName, bTraceComplex, ActorsToIgnore, bIgnoreSelf, WorldContextObject);

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	bool const bHit = World ? World->SweepMultiByProfile(OutHits, Start, End, FQuat::Identity, ProfileName, FCollisionShape::MakeCapsule(Radius, HalfHeight), Params) : false;

#if ENABLE_DRAW_DEBUG
	DrawDebugCapsuleTraceMulti(World, Start, End, Radius, HalfHeight, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
#endif

	return bHit;
}

#if WITH_EDITOR
TArray<FName> UKismetSystemLibrary::GetCollisionProfileNames()
{
	TArray<TSharedPtr<FName>> SharedNames;
	UCollisionProfile::GetProfileNames(SharedNames);

	TArray<FName> Names;
	Names.Reserve(SharedNames.Num());
	for (const TSharedPtr<FName>& SharedName : SharedNames)
	{
		if (const FName* Name = SharedName.Get())
		{
			Names.Add(*Name);
		}
	}

	return Names;
}
#endif

/** Draw a debug line */
void UKismetSystemLibrary::DrawDebugLine(const UObject* WorldContextObject, FVector const LineStart, FVector const LineEnd, FLinearColor Color, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugLine(World, LineStart, LineEnd, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug circle */
void UKismetSystemLibrary::DrawDebugCircle(const UObject* WorldContextObject,FVector Center, float Radius, int32 NumSegments, FLinearColor LineColor, float LifeTime, float Thickness, FVector YAxis, FVector ZAxis, bool bDrawAxis, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{ 
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugCircle(World, Center, Radius, NumSegments, LineColor.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness, YAxis, ZAxis, bDrawAxis);
	}
#endif
}

/** Draw a debug point */
void UKismetSystemLibrary::DrawDebugPoint(const UObject* WorldContextObject, FVector const Position, float Size, FLinearColor PointColor, float LifeTime, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugPoint(World, Position, Size, PointColor.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority));
	}
#endif
}

/** Draw directional arrow, pointing from LineStart to LineEnd. */
void UKismetSystemLibrary::DrawDebugArrow(const UObject* WorldContextObject, FVector const LineStart, FVector const LineEnd, float ArrowSize, FLinearColor Color, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World != nullptr)
	{
		::DrawDebugDirectionalArrow(World, LineStart, LineEnd, ArrowSize, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug box */
void UKismetSystemLibrary::DrawDebugBox(const UObject* WorldContextObject, FVector const Center, FVector Extent, FLinearColor Color, const FRotator Rotation, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (Rotation == FRotator::ZeroRotator)
		{
			::DrawDebugBox(World, Center, Extent, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
		}
		else
		{
			::DrawDebugBox(World, Center, Extent, Rotation.Quaternion(), Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
		}
	}
#endif
}

/** Draw a debug coordinate system. */
void UKismetSystemLibrary::DrawDebugCoordinateSystem(const UObject* WorldContextObject, FVector const AxisLoc, FRotator const AxisRot, float Scale, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugCoordinateSystem(World, AxisLoc, AxisRot, Scale, false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug sphere */
void UKismetSystemLibrary::DrawDebugSphere(const UObject* WorldContextObject, FVector const Center, float Radius, int32 Segments, FLinearColor Color, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugSphere(World, Center, Radius, Segments, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug cylinder */
void UKismetSystemLibrary::DrawDebugCylinder(const UObject* WorldContextObject, FVector const Start, FVector const End, float Radius, int32 Segments, FLinearColor Color, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugCylinder(World, Start, End, Radius, Segments, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug cone */
void UKismetSystemLibrary::DrawDebugCone(const UObject* WorldContextObject, FVector const Origin, FVector const Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FLinearColor Color, float Duration, float Thickness)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugCone(World, Origin, Direction, Length, AngleWidth, AngleHeight, NumSides, Color.ToFColor(true), false, Duration, SDPG_World, Thickness);
	}
#endif
}

void UKismetSystemLibrary::DrawDebugConeInDegrees(const UObject* WorldContextObject, FVector const Origin, FVector const Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FLinearColor Color, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugCone(World, Origin, Direction, Length, FMath::DegreesToRadians(AngleWidth), FMath::DegreesToRadians(AngleHeight), NumSides, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug capsule */
void UKismetSystemLibrary::DrawDebugCapsule(const UObject* WorldContextObject, FVector const Center, float HalfHeight, float Radius, const FRotator Rotation, FLinearColor Color, float LifeTime, float Thickness, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugCapsule(World, Center, HalfHeight, Radius, Rotation.Quaternion(), Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority), Thickness);
	}
#endif
}

/** Draw a debug string at a 3d world location. */
void UKismetSystemLibrary::DrawDebugString(const UObject* WorldContextObject, FVector const TextLocation, const FString& Text, class AActor* TestBaseActor, FLinearColor TextColor, float Duration)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugString(World, TextLocation, Text, TestBaseActor, TextColor.ToFColor(true), Duration);
	}
#endif
}

/** Removes all debug strings. */
void UKismetSystemLibrary::FlushDebugStrings(const UObject* WorldContextObject )
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::FlushDebugStrings( World );
	}
#endif
}

/** Draws a debug plane. */
void UKismetSystemLibrary::DrawDebugPlane(const UObject* WorldContextObject, FPlane const& P, FVector const Loc, float Size, FLinearColor Color, float LifeTime, EDrawDebugSceneDepthPriorityGroup DepthPriority)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugSolidPlane(World, P, Loc, Size, Color.ToFColor(true), false, LifeTime, UE::Blueprint::Private::ConvertSceneDepthPriorityToEngineType(DepthPriority));
	}
#endif
}

/** Flush all persistent debug lines and shapes */
void UKismetSystemLibrary::FlushPersistentDebugLines(const UObject* WorldContextObject)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::FlushPersistentDebugLines( World );
	}
#endif
}

/** Draws a debug frustum. */
void UKismetSystemLibrary::DrawDebugFrustum(const UObject* WorldContextObject, const FTransform& FrustumTransform, FLinearColor FrustumColor, float Duration, float Thickness)
{
#if ENABLE_DRAW_DEBUG
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World != nullptr && FrustumTransform.IsRotationNormalized())
	{
		FMatrix FrustumToWorld =  FrustumTransform.ToMatrixWithScale();
		::DrawDebugFrustum(World, FrustumToWorld, FrustumColor.ToFColor(true), false, Duration, SDPG_World, Thickness);
	}
#endif
}

/** Draw a debug camera shape. */
void UKismetSystemLibrary::DrawDebugCamera(const ACameraActor* CameraActor, FLinearColor CameraColor, float Duration)
{
#if ENABLE_DRAW_DEBUG
	if(CameraActor)
	{
		FVector CamLoc = CameraActor->GetActorLocation();
		FRotator CamRot = CameraActor->GetActorRotation();
		::DrawDebugCamera(CameraActor->GetWorld(), CameraActor->GetActorLocation(), CameraActor->GetActorRotation(), CameraActor->GetCameraComponent()->FieldOfView, 1.0f, CameraColor.ToFColor(true), false, Duration, SDPG_World);
	}
#endif
}

void UKismetSystemLibrary::DrawDebugFloatHistoryTransform(const UObject* WorldContextObject, const FDebugFloatHistory& FloatHistory, const FTransform& DrawTransform, FVector2D DrawSize, FLinearColor DrawColor, float LifeTime)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugFloatHistory(*World, FloatHistory, DrawTransform, DrawSize, DrawColor.ToFColor(true), false, LifeTime);
	}
#endif
}

void UKismetSystemLibrary::DrawDebugFloatHistoryLocation(const UObject* WorldContextObject, const FDebugFloatHistory& FloatHistory, FVector DrawLocation, FVector2D DrawSize, FLinearColor DrawColor, float LifeTime)
{
#if ENABLE_DRAW_DEBUG
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		::DrawDebugFloatHistory(*World, FloatHistory, DrawLocation, DrawSize, DrawColor.ToFColor(true), false, LifeTime);
	}
#endif
}

FDebugFloatHistory UKismetSystemLibrary::AddFloatHistorySample(float Value, const FDebugFloatHistory& FloatHistory)
{
	FDebugFloatHistory* const MutableFloatHistory = const_cast<FDebugFloatHistory*>(&FloatHistory);
	MutableFloatHistory->AddSample(Value);
	return FloatHistory;
}

/** Mark as modified. */
void UKismetSystemLibrary::CreateCopyForUndoBuffer(UObject* ObjectToModify)
{
	if (ObjectToModify != NULL)
	{
		ObjectToModify->Modify();
	}
}

void UKismetSystemLibrary::GetComponentBounds(const USceneComponent* Component, FVector& Origin, FVector& BoxExtent, float& SphereRadius)
{
	if (Component != NULL)
	{
		Origin = Component->Bounds.Origin;
		BoxExtent = Component->Bounds.BoxExtent;
		SphereRadius = Component->Bounds.SphereRadius;
	}
	else
	{
		Origin = FVector::ZeroVector;
		BoxExtent = FVector::ZeroVector;
		SphereRadius = 0.0f;
		UE_LOGF(LogBlueprintUserMessages, Verbose, "GetComponentBounds passed a bad component");
	}
}

void UKismetSystemLibrary::GetActorBounds(const AActor* Actor, FVector& Origin, FVector& BoxExtent)
{
	if (Actor != NULL)
	{
		const FBox Bounds = Actor->GetComponentsBoundingBox(true);

		// To keep consistency with the other GetBounds functions, transform our result into an origin / extent formatting
		Bounds.GetCenterAndExtents(Origin, BoxExtent);
	}
	else
	{
		Origin = FVector::ZeroVector;
		BoxExtent = FVector::ZeroVector;
		UE_LOGF(LogBlueprintUserMessages, Verbose, "GetActorBounds passed a bad actor");
	}
}


void UKismetSystemLibrary::Delay(const UObject* WorldContextObject, float Duration, FLatentActionInfo LatentInfo )
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FDelayAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == NULL)
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FDelayAction(Duration, LatentInfo));
		}
	}
}

void UKismetSystemLibrary::DelayUntilNextTick(const UObject* WorldContextObject, FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FDelayUntilNextTickAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == NULL)
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FDelayUntilNextTickAction(LatentInfo));
		}
	}
}

void UKismetSystemLibrary::DelayUntilNextFrame(const UObject* WorldContextObject, FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FDelayUntilNextFrameAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == NULL)
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FDelayUntilNextFrameAction(LatentInfo));
		}
	}
}


// Delay execution by Duration seconds; Calling again before the delay has expired will reset the countdown to Duration.
void UKismetSystemLibrary::RetriggerableDelay(const UObject* WorldContextObject, float Duration, FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		FDelayAction* Action = LatentActionManager.FindExistingAction<FDelayAction>(LatentInfo.CallbackTarget, LatentInfo.UUID);
		if (Action == NULL)
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FDelayAction(Duration, LatentInfo));
		}
		else
		{
			// Reset the existing delay to the new duration
			Action->TimeRemaining = Duration;
		}
	}
}

void UKismetSystemLibrary::MoveComponentTo(USceneComponent* Component, FVector TargetRelativeLocation, FRotator TargetRelativeRotation, bool bEaseOut, bool bEaseIn, float OverTime, bool bForceShortestRotationPath, TEnumAsByte<EMoveComponentAction::Type> MoveAction, FLatentActionInfo LatentInfo)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(Component, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		FInterpolateComponentToAction* Action = LatentActionManager.FindExistingAction<FInterpolateComponentToAction>(LatentInfo.CallbackTarget, LatentInfo.UUID);

		const FVector ComponentLocation = (Component != NULL) ? Component->GetRelativeLocation() : FVector::ZeroVector;
		const FRotator ComponentRotation = (Component != NULL) ? Component->GetRelativeRotation() : FRotator::ZeroRotator;

		// If not currently running
		if (Action == NULL)
		{
			if (MoveAction == EMoveComponentAction::Move)
			{
				// Only act on a 'move' input if not running
				Action = new FInterpolateComponentToAction(OverTime, LatentInfo, Component, bEaseOut, bEaseIn, bForceShortestRotationPath);

				Action->TargetLocation = TargetRelativeLocation;
				Action->TargetRotation = TargetRelativeRotation;

				Action->InitialLocation = ComponentLocation;
				Action->InitialRotation = ComponentRotation;

				LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, Action);
			}
		}
		else
		{
			if (MoveAction == EMoveComponentAction::Move)
			{
				// A 'Move' action while moving restarts interpolation
				Action->TotalTime = OverTime;
				Action->TimeElapsed = 0.f;

				Action->TargetLocation = TargetRelativeLocation;
				Action->TargetRotation = TargetRelativeRotation;

				Action->InitialLocation = ComponentLocation;
				Action->InitialRotation = ComponentRotation;
			}
			else if (MoveAction == EMoveComponentAction::Stop)
			{
				// 'Stop' just stops the interpolation where it is
				Action->bInterpolating = false;
			}
			else if (MoveAction == EMoveComponentAction::Return)
			{
				// Return moves back to the beginning
				Action->TotalTime = Action->TimeElapsed;
				Action->TimeElapsed = 0.f;

				// Set our target to be our initial, and set the new initial to be the current position
				Action->TargetLocation = Action->InitialLocation;
				Action->TargetRotation = Action->InitialRotation;

				Action->InitialLocation = ComponentLocation;
				Action->InitialRotation = ComponentRotation;
			}
		}
	}
}

int32 UKismetSystemLibrary::GetRenderingDetailMode()
{
	static const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DetailMode"));

	// clamp range
	int32 Ret = FMath::Clamp(CVar->GetInt(), 0, DM_MAX - 1);

	return Ret;
}

int32 UKismetSystemLibrary::GetRenderingMaterialQualityLevel()
{
	static const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MaterialQualityLevel"));

	// clamp range
	int32 Ret = FMath::Clamp(CVar->GetInt(), 0, (int32)EMaterialQualityLevel::Num - 1);

	return Ret;
}

bool UKismetSystemLibrary::GetSupportedFullscreenResolutions(TArray<FIntPoint>& Resolutions)
{
	uint32 MinYResolution = UKismetSystemLibrary::GetMinYResolutionForUI();

	FScreenResolutionArray SupportedResolutions;
	if ( RHIGetAvailableResolutions(SupportedResolutions, true) )
	{
		uint32 LargestY = 0;
		for ( const FScreenResolutionRHI& SupportedResolution : SupportedResolutions )
		{
			LargestY = FMath::Max(LargestY, SupportedResolution.Height);
			if(SupportedResolution.Height >= MinYResolution)
			{
				FIntPoint Resolution;
				Resolution.X = SupportedResolution.Width;
				Resolution.Y = SupportedResolution.Height;

				Resolutions.Add(Resolution);
			}
		}
		if(!Resolutions.Num())
		{
			// if we don't have any resolution, we take the largest one(s)
			for ( const FScreenResolutionRHI& SupportedResolution : SupportedResolutions )
			{
				if(SupportedResolution.Height == LargestY)
				{
					FIntPoint Resolution;
					Resolution.X = SupportedResolution.Width;
					Resolution.Y = SupportedResolution.Height;

					Resolutions.Add(Resolution);
				}
			}
		}

		return true;
	}

	return false;
}

bool UKismetSystemLibrary::GetConvenientWindowedResolutions(TArray<FIntPoint>& Resolutions)
{
	FDisplayMetrics DisplayMetrics;
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetInitialDisplayMetrics(DisplayMetrics);
	}
	else
	{
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
	}

	extern void GenerateConvenientWindowedResolutions(const struct FDisplayMetrics& InDisplayMetrics, TArray<FIntPoint>& OutResolutions);
	GenerateConvenientWindowedResolutions(DisplayMetrics, Resolutions);

	return true;
}

static TAutoConsoleVariable<int32> CVarMinYResolutionForUI(
	TEXT("r.MinYResolutionForUI"),
	720,
	TEXT("Defines the smallest Y resolution we want to support in the UI (default is 720)"),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarMinYResolutionFor3DView(
	TEXT("r.MinYResolutionFor3DView"),
	360,
	TEXT("Defines the smallest Y resolution we want to support in the 3D view"),
	ECVF_RenderThreadSafe
	);

int32 UKismetSystemLibrary::GetMinYResolutionForUI()
{
	int32 Value = FMath::Clamp(CVarMinYResolutionForUI.GetValueOnGameThread(), 200, 8 * 1024);

	return Value;
}

int32 UKismetSystemLibrary::GetMinYResolutionFor3DView()
{
	int32 Value = FMath::Clamp(CVarMinYResolutionFor3DView.GetValueOnGameThread(), 200, 8 * 1024);

	return Value;
}

void UKismetSystemLibrary::LaunchURL(const FString& URL)
{
	if (!URL.IsEmpty())
	{
		UE::Core::FURLRequestFilter Filter(TEXT("SystemLibrary.LaunchURLFilter"), GEngineIni);

		FPlatformProcess::LaunchURLFiltered(*URL, nullptr, nullptr, Filter);
	}
}

void UKismetSystemLibrary::LaunchExternalUrl(const TArray<FString>& DomainStrings, const FString& URL)
{
	if (!URL.IsEmpty())
	{
		UE::Core::FURLRequestFilter::FRequestMap AllowedDomains;
		AllowedDomains.Add(TEXT("http"), DomainStrings);
		UE::Core::FURLRequestFilter Filter(AllowedDomains);

		FPlatformProcess::LaunchURLFiltered(*URL, nullptr, nullptr, Filter);
	}
}

bool UKismetSystemLibrary::CanLaunchURL(const FString& URL)
{
	if (!URL.IsEmpty())
	{
		return FPlatformProcess::CanLaunchURL(*URL);
	}

	return false;
}

void UKismetSystemLibrary::CollectGarbage()
{
	GEngine->ForceGarbageCollection(true);
}

void UKismetSystemLibrary::ShowAdBanner(int32 AdIdIndex, bool bShowOnBottomOfScreen)
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		Provider->ShowAdBanner(bShowOnBottomOfScreen, AdIdIndex);
	}
}

int32 UKismetSystemLibrary::GetAdIDCount()
{
	uint32 AdIDCount = 0;
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		AdIDCount = Provider->GetAdIDCount();
	}

	return AdIDCount;
}

void UKismetSystemLibrary::HideAdBanner()
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		Provider->HideAdBanner();
	}
}

void UKismetSystemLibrary::ForceCloseAdBanner()
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		Provider->CloseAdBanner();
	}
}

void UKismetSystemLibrary::LoadInterstitialAd(int32 AdIdIndex)
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		Provider->LoadInterstitialAd(AdIdIndex);
	}
}

bool UKismetSystemLibrary::IsInterstitialAdAvailable()
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		return Provider->IsInterstitialAdAvailable();
	}
	return false;
}

bool UKismetSystemLibrary::IsInterstitialAdRequested()
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		return Provider->IsInterstitialAdRequested();
	}
	return false;
}

void UKismetSystemLibrary::ShowInterstitialAd()
{
	if (IAdvertisingProvider* Provider = FAdvertising::Get().GetDefaultProvider())
	{
		Provider->ShowInterstitialAd();
	}
}

void UKismetSystemLibrary::ShowPlatformSpecificLeaderboardScreen(const FString& CategoryName)
{
	// not PIE safe, doesn't have world context
	UOnlineEngineInterface::Get()->ShowLeaderboardUI(nullptr, CategoryName);
}

void UKismetSystemLibrary::ShowPlatformSpecificAchievementsScreen(const APlayerController* SpecificPlayer)
{
	UWorld* World = SpecificPlayer ? SpecificPlayer->GetWorld() : nullptr;

	// Get the controller id from the player
	int LocalUserNum = 0;
	if (SpecificPlayer)
	{
		ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(SpecificPlayer->Player);
		if (LocalPlayer)
		{
			LocalUserNum = LocalPlayer->GetControllerId();
		}
	}

	UOnlineEngineInterface::Get()->ShowAchievementsUI(World, LocalUserNum);
}

bool UKismetSystemLibrary::IsLoggedIn(const APlayerController* SpecificPlayer)
{
	UWorld* World = SpecificPlayer ? SpecificPlayer->GetWorld() : nullptr;

	int LocalUserNum = 0;
	if (SpecificPlayer != nullptr)
	{
		ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(SpecificPlayer->Player);
		if(LocalPlayer)
		{
			LocalUserNum = LocalPlayer->GetControllerId();
		}
	}

	return UOnlineEngineInterface::Get()->IsLoggedIn(World, LocalUserNum);
}

void UKismetSystemLibrary::SetStructurePropertyByName(UObject* Object, FName PropertyName, const FGenericStruct& Value)
{
	// We should never hit these!  They're stubs to avoid NoExport on the class.
	check(0);
}

bool UKismetSystemLibrary::IsScreensaverEnabled()
{
	return FPlatformApplicationMisc::IsScreensaverEnabled();
}

void UKismetSystemLibrary::ControlScreensaver(bool bAllowScreenSaver)
{
	FPlatformApplicationMisc::ControlScreensaver(bAllowScreenSaver ? FPlatformApplicationMisc::EScreenSaverAction::Enable : FPlatformApplicationMisc::EScreenSaverAction::Disable);
}

void UKismetSystemLibrary::SetVolumeButtonsHandledBySystem(bool bEnabled)
{
	FPlatformMisc::SetVolumeButtonsHandledBySystem(bEnabled);
}

bool UKismetSystemLibrary::GetVolumeButtonsHandledBySystem()
{
	return FPlatformMisc::GetVolumeButtonsHandledBySystem();
}

void UKismetSystemLibrary::SetGamepadsBlockDeviceFeedback(bool bBlock)
{
	FPlatformApplicationMisc::SetGamepadsBlockDeviceFeedback(bBlock);

}

void UKismetSystemLibrary::ResetGamepadAssignments()
{
	FPlatformApplicationMisc::ResetGamepadAssignments();
}

void UKismetSystemLibrary::ResetGamepadAssignmentToController(int32 ControllerId)
{
	FPlatformApplicationMisc::ResetGamepadAssignmentToController(ControllerId);
}

bool UKismetSystemLibrary::IsControllerAssignedToGamepad(int32 ControllerId)
{
	return FPlatformApplicationMisc::IsControllerAssignedToGamepad(ControllerId);
}

FString UKismetSystemLibrary::GetGamepadControllerName(int32 ControllerId)
{
	return FPlatformApplicationMisc::GetGamepadControllerName(ControllerId);
}

UTexture2D* UKismetSystemLibrary::GetGamepadButtonGlyph(const FString& ButtonKey, int32 ControllerIndex)
{
	return FPlatformApplicationMisc::GetGamepadButtonGlyph(FName(*ButtonKey), ControllerIndex);
}

void UKismetSystemLibrary::SetSuppressViewportTransitionMessage(const UObject* WorldContextObject, bool bState)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World && World->GetFirstLocalPlayerFromController() != nullptr && World->GetFirstLocalPlayerFromController()->ViewportClient != nullptr )
	{
		World->GetFirstLocalPlayerFromController()->ViewportClient->SetSuppressTransitionMessage(bState);
	}
}

TArray<FString> UKismetSystemLibrary::GetPreferredLanguages()
{
	return FPlatformMisc::GetPreferredLanguages();
}

FString UKismetSystemLibrary::GetDefaultLanguage()
{
	return FPlatformMisc::GetDefaultLanguage();
}

FString UKismetSystemLibrary::GetDefaultLocale()
{
	return FPlatformMisc::GetDefaultLocale();
}

FString UKismetSystemLibrary::GetLocalCurrencyCode()
{
	return FPlatformMisc::GetLocalCurrencyCode();
}

FString UKismetSystemLibrary::GetLocalCurrencySymbol()
{
	return FPlatformMisc::GetLocalCurrencySymbol();
}

struct FLoadAssetActionBase : public FPendingLatentAction
{
	// @TODO: it would be good to have static/global manager? 

public:
	FSoftObjectPath SoftObjectPath;
	FStreamableManager StreamableManager;
	TSharedPtr<FStreamableHandle> Handle;
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

	virtual void OnLoaded() PURE_VIRTUAL(FLoadAssetActionBase::OnLoaded, );

	FLoadAssetActionBase(const FSoftObjectPath& InSoftObjectPath, const FLatentActionInfo& InLatentInfo)
		: SoftObjectPath(InSoftObjectPath)
		, ExecutionFunction(InLatentInfo.ExecutionFunction)
		, OutputLink(InLatentInfo.Linkage)
		, CallbackTarget(InLatentInfo.CallbackTarget)
	{
		Handle = StreamableManager.RequestAsyncLoad(SoftObjectPath);
	}

	virtual ~FLoadAssetActionBase()
	{
		if (Handle.IsValid())
		{
			Handle->ReleaseHandle();
		}
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		const bool bLoaded = !Handle.IsValid() || Handle->HasLoadCompleted() || Handle->WasCanceled();
		if (bLoaded)
		{
			OnLoaded();
		}
		Response.FinishAndTriggerIf(bLoaded, ExecutionFunction, OutputLink, CallbackTarget);
	}

#if WITH_EDITOR
	virtual FString GetDescription() const override
	{
		return FString::Printf(TEXT("Load Asset Action Base: %s"), *SoftObjectPath.ToString());
	}
#endif
};

void UKismetSystemLibrary::LoadAsset(const UObject* WorldContextObject, TSoftObjectPtr<UObject> Asset, UKismetSystemLibrary::FOnAssetLoaded OnLoaded, FLatentActionInfo LatentInfo)
{
	struct FLoadAssetAction : public FLoadAssetActionBase
	{
	public:
		UKismetSystemLibrary::FOnAssetLoaded OnLoadedCallback;

		FLoadAssetAction(const FSoftObjectPath& InSoftObjectPath, UKismetSystemLibrary::FOnAssetLoaded InOnLoadedCallback, const FLatentActionInfo& InLatentInfo)
			: FLoadAssetActionBase(InSoftObjectPath, InLatentInfo)
			, OnLoadedCallback(InOnLoadedCallback)
		{}

		virtual void OnLoaded() override
		{
			UObject* LoadedObject = SoftObjectPath.ResolveObject();
			OnLoadedCallback.ExecuteIfBound(LoadedObject);
		}
	};

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();

		// We always spawn a new load even if this node already queued one, the outside node handles this case
		FLoadAssetAction* NewAction = new FLoadAssetAction(Asset.ToSoftObjectPath(), OnLoaded, LatentInfo);
		LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
	}
}

void UKismetSystemLibrary::LoadAssetClass(const UObject* WorldContextObject, TSoftClassPtr<UObject> AssetClass, UKismetSystemLibrary::FOnAssetClassLoaded OnLoaded, FLatentActionInfo LatentInfo)
{
	struct FLoadAssetClassAction : public FLoadAssetActionBase
	{
	public:
		UKismetSystemLibrary::FOnAssetClassLoaded OnLoadedCallback;

		FLoadAssetClassAction(const FSoftObjectPath& InSoftObjectPath, UKismetSystemLibrary::FOnAssetClassLoaded InOnLoadedCallback, const FLatentActionInfo& InLatentInfo)
			: FLoadAssetActionBase(InSoftObjectPath, InLatentInfo)
			, OnLoadedCallback(InOnLoadedCallback)
		{}

		virtual void OnLoaded() override
		{
			UClass* LoadedObject = Cast<UClass>(SoftObjectPath.ResolveObject());
			OnLoadedCallback.ExecuteIfBound(LoadedObject);
		}
	};

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();

		// We always spawn a new load even if this node already queued one, the outside node handles this case
		FLoadAssetClassAction* NewAction = new FLoadAssetClassAction(AssetClass.ToSoftObjectPath(), OnLoaded, LatentInfo);
		LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
	}
}

void UKismetSystemLibrary::LoadAssets(const UObject* WorldContextObject, TArray<TSoftObjectPtr<UObject>> Assets, FOnAssetsLoaded OnLoaded, FLatentActionInfo LatentInfo)
{
	struct FLoadAssetsAction : public FPendingLatentAction
	{
	public:

		TArray<FSoftObjectPath> SoftObjectPaths;
		FStreamableManager StreamableManager;
		TSharedPtr<FStreamableHandle> Handle;
		FName ExecutionFunction;
		int32 OutputLink;
		FWeakObjectPtr CallbackTarget;
		UKismetSystemLibrary::FOnAssetsLoaded OnLoadedCallback;

		FLoadAssetsAction(const TArray<FSoftObjectPath>& InSoftObjectPaths, UKismetSystemLibrary::FOnAssetsLoaded InOnLoadedCallback, const FLatentActionInfo& InLatentInfo)
			: SoftObjectPaths(InSoftObjectPaths)
			, ExecutionFunction(InLatentInfo.ExecutionFunction)
			, OutputLink(InLatentInfo.Linkage)
			, CallbackTarget(InLatentInfo.CallbackTarget)
			, OnLoadedCallback(InOnLoadedCallback)

		{
			Handle = StreamableManager.RequestAsyncLoad(SoftObjectPaths);
		}

		virtual ~FLoadAssetsAction()
		{
			if (Handle.IsValid())
			{
				Handle->ReleaseHandle();
			}
		}

		virtual void UpdateOperation(FLatentResponse& Response) override
		{
			const bool bLoaded = !Handle.IsValid() || Handle->HasLoadCompleted() || Handle->WasCanceled();
			if (bLoaded)
			{
				OnLoaded();
			}
			Response.FinishAndTriggerIf(bLoaded, ExecutionFunction, OutputLink, CallbackTarget);
		}

#if WITH_EDITOR
		virtual FString GetDescription() const override
		{
			FString AggregatedSoftObjectPaths;

			for (auto Path = SoftObjectPaths.CreateConstIterator(); Path; ++Path)
			{
				AggregatedSoftObjectPaths.Append(Path->ToString());

				if (*Path != SoftObjectPaths.Last())
				{
					AggregatedSoftObjectPaths.Append("\n");
				}
			}

			return FString::Printf(TEXT("Load Assets Action: %s"), *AggregatedSoftObjectPaths);
		}
#endif

		void OnLoaded() 
		{
			TArray<UObject*> LoadedObjects;

			for (FSoftObjectPath& Path : SoftObjectPaths)
			{
				LoadedObjects.Add(Path.ResolveObject());
			}

			OnLoadedCallback.ExecuteIfBound(LoadedObjects);
		}
	};

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();

		TArray<FSoftObjectPath> SoftObjectPathsToLoad;

		for (TSoftObjectPtr<UObject> SoftObjectPtr : Assets)
		{
			SoftObjectPathsToLoad.Add(SoftObjectPtr.ToSoftObjectPath());
		}

		// We always spawn a new load even if this node already queued one, the outside node handles this case
		FLoadAssetsAction* NewAction = new FLoadAssetsAction(SoftObjectPathsToLoad, OnLoaded, LatentInfo);
		LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
	}
}

void UKismetSystemLibrary::RegisterForRemoteNotifications()
{
	FPlatformMisc::RegisterForRemoteNotifications();
}

void UKismetSystemLibrary::UnregisterForRemoteNotifications()
{
	FPlatformMisc::UnregisterForRemoteNotifications();
}

void UKismetSystemLibrary::SetUserActivity(const FUserActivity& UserActivity)
{
	FUserActivityTracking::SetActivity(UserActivity);
}

FString UKismetSystemLibrary::GetCommandLine()
{
	return FString(FCommandLine::Get());
}

void UKismetSystemLibrary::ParseCommandLine(const FString& InCmdLine, TArray<FString>& OutTokens, TArray<FString>& OutSwitches, TMap<FString, FString>& OutParams)
{
	UCommandlet::ParseCommandLine(*InCmdLine, OutTokens, OutSwitches, OutParams);
}

bool UKismetSystemLibrary::ParseParam(const FString& InString, const FString& InParam)
{
	return FParse::Param(*InString, *InParam);
}


bool UKismetSystemLibrary::ParseParamValue(const FString& InString, const FString& InParam, FString& OutValue)
{
	return FParse::Value(*InString, *InParam, OutValue);
}

bool UKismetSystemLibrary::IsUnattended()
{
	return FApp::IsUnattended();
}

#if WITH_EDITOR

bool UKismetSystemLibrary::GetEditorProperty(UObject* Object, const FName PropertyName, int32& PropertyValue)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

bool UKismetSystemLibrary::Generic_GetEditorProperty(const UObject* Object, const FName PropertyName, void* ValuePtr, const FProperty* ValueProp)
{
	const FProperty* ObjectProp = PropertyAccessUtil::FindPropertyByName(PropertyName, Object->GetClass());
	TOptional<EPropertyAccessResultFlags> SparseDataAccessResult;
	if ((!ObjectProp || ObjectProp->HasAnyPropertyFlags(CPF_Deprecated)) && Object->HasAllFlags(RF_ClassDefaultObject))
	{
		// look for a sparse member of the same name - the sparse data is treated as an extension
		// of the class default object by the details panel:
		const UStruct* SparseDataStruct = Object->GetClass()->GetSparseClassDataStruct();
		if (SparseDataStruct)
		{
			const FProperty* SparseProp = PropertyAccessUtil::FindPropertyByName(PropertyName, SparseDataStruct);
			if (SparseProp)
			{
				void* SparseDest = Object->GetClass()->GetOrCreateSparseClassData();

				SparseDataAccessResult = PropertyAccessUtil::GetPropertyValue_InContainer(
					SparseProp,
					SparseDest,
					ValueProp,
					ValuePtr,
					INDEX_NONE);
			}
		}
	}

	if (!ObjectProp && !SparseDataAccessResult.IsSet())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) was missing"), *PropertyName.ToString(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
		return false;
	}

	const EPropertyAccessResultFlags AccessResult = SparseDataAccessResult.IsSet() ?
		SparseDataAccessResult.GetValue() : 
		PropertyAccessUtil::GetPropertyValue_Object(ObjectProp, Object, ValueProp, ValuePtr, INDEX_NONE);

	if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
	{
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::AccessProtected))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) is protected and cannot be read"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
			return false;
		}

		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) cannot be read"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
		return false;
	}

	if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::ConversionFailed))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' (%s) on '%s' (%s) tried to get to a property value of the incorrect type (%s)"), *ObjectProp->GetName(), *ObjectProp->GetClass()->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName(), *ValueProp->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
		return false;
	}

	return true;
}

DEFINE_FUNCTION(UKismetSystemLibrary::execGetEditorProperty)
{
	P_GET_OBJECT(UObject, Object);
	P_GET_PROPERTY(FNameProperty, PropertyName);

	Stack.StepCompiledIn<FProperty>(nullptr);
	const FProperty* ValueProp = Stack.MostRecentProperty;
	void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("GetEditorProperty_MissingOutputProperty", "Failed to resolve the output parameter for GetEditorProperty.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	if (!Object)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("GetEditorProperty_AccessNone", "Accessed None attempting to call GetEditorProperty.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult = false;

	if (Object)
	{
		P_NATIVE_BEGIN;
		bResult = Generic_GetEditorProperty(Object, PropertyName, ValuePtr, ValueProp);
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = bResult;
}

bool UKismetSystemLibrary::SetEditorProperty(UObject* Object, const FName PropertyName, const int32& PropertyValue, const EPropertyAccessChangeNotifyMode ChangeNotifyMode)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

bool UKismetSystemLibrary::Generic_SetEditorProperty(UObject* Object, const FName PropertyName, const void* ValuePtr, const FProperty* ValueProp, const EPropertyAccessChangeNotifyMode ChangeNotifyMode)
{
	TOptional<EPropertyAccessResultFlags> SparseDataAccessResult;
	const FProperty* ObjectProp = PropertyAccessUtil::FindPropertyByName(PropertyName, Object->GetClass());
	if ((!ObjectProp || ObjectProp->HasAnyPropertyFlags(CPF_Deprecated)) && Object->HasAllFlags(RF_ClassDefaultObject))
	{
		// look for a sparse member of the same name - the sparse data is treated as an extension
		// of the class default object by the details panel:
		const UStruct* SparseDataStruct = Object->GetClass()->GetSparseClassDataStruct();
		if (SparseDataStruct)
		{
			const FProperty* SparseProp = PropertyAccessUtil::FindPropertyByName(PropertyName, SparseDataStruct);
			if (SparseProp)
			{
				void* SparseDest = Object->GetClass()->GetOrCreateSparseClassData();

				// If the object is a template, instances of it may need to be updated as well depending on whether they are inheriting their
				// current value. Detect those instances which haven't overwritten the value. In the case of sparse data, this applies to
				// subclass CDOs.
				TArray<void*> ArchetypeInstances;
				const bool bObjectIsTemplate = PropertyAccessUtil::IsObjectTemplate(Object);
				if (bObjectIsTemplate)
				{
					PropertyAccessUtil::GetArchetypeInstancesInheritingPropertyValue_AsContainerData(SparseProp, Object, ArchetypeInstances);
				}

				SparseDataAccessResult = PropertyAccessUtil::SetPropertyValue_InContainer(
					SparseProp,
					SparseDest,
					ArchetypeInstances,
					ValueProp,
					ValuePtr,
					INDEX_NONE,
					PropertyAccessUtil::EditorReadOnlyFlags,
					bObjectIsTemplate,
					[SparseProp, Object, ChangeNotifyMode]()
					{
						return PropertyAccessUtil::BuildBasicChangeNotify(SparseProp, Object, ChangeNotifyMode, EPropertyChangeType::ValueSet);
					});
				if (*SparseDataAccessResult == EPropertyAccessResultFlags::Success)
				{
					if(UBlueprintGeneratedClass* AsBPGC = Cast<UBlueprintGeneratedClass>(Object->GetClass()))
					{
						AsBPGC->bIsSparseClassDataSerializable = true;
					}
					else
					{
						FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) is in sparse class data but not owned by a BlueprintGeneratedClass and may not be saved"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
					}
				}
			}
		}
	}

	if(!ObjectProp && !SparseDataAccessResult.IsSet())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) was missing"), *PropertyName.ToString(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
		return false;
	}

	const EPropertyAccessResultFlags AccessResult = SparseDataAccessResult.IsSet() ?
		SparseDataAccessResult.GetValue() :
		PropertyAccessUtil::SetPropertyValue_Object(ObjectProp, Object, ValueProp, ValuePtr, INDEX_NONE, PropertyAccessUtil::EditorReadOnlyFlags, ChangeNotifyMode);

	if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
	{
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::AccessProtected))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) is protected and cannot be set"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
			return false;
		}

		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::CannotEditTemplate))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) cannot be edited on templates"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
			return false;
		}

		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::CannotEditInstance))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) cannot be edited on instances"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
			return false;
		}

		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::ReadOnly))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) is read-only and cannot be set"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
			return false;
		}

		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) cannot be set"), *ObjectProp->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
		return false;
	}

	if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::ConversionFailed))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' (%s) on '%s' (%s) tried to set from a property value of the incorrect type (%s)"), *ObjectProp->GetName(), *ObjectProp->GetClass()->GetName(), *Object->GetPathName(), *Object->GetClass()->GetName(), *ValueProp->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
		return false;
	}

	return true;
}

DEFINE_FUNCTION(UKismetSystemLibrary::execSetEditorProperty)
{
	P_GET_OBJECT(UObject, Object);
	P_GET_PROPERTY(FNameProperty, PropertyName);

	Stack.StepCompiledIn<FProperty>(nullptr);
	const FProperty* ValueProp = Stack.MostRecentProperty;
	const void* ValuePtr = Stack.MostRecentPropertyAddress;

	P_GET_ENUM(EPropertyAccessChangeNotifyMode, ChangeNotifyMode);

	P_FINISH;

	if (!ValueProp || !ValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("SetEditorProperty_MissingOutputProperty", "Failed to resolve the output parameter for SetEditorProperty.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	if (!Object)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("SetEditorProperty_AccessNone", "Accessed None attempting to call SetEditorProperty.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult = false;

	if (Object)
	{
		P_NATIVE_BEGIN;
		bResult = Generic_SetEditorProperty(Object, PropertyName, ValuePtr, ValueProp, ChangeNotifyMode);
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = bResult;
}

namespace UE::Blueprint::Private
{
	void FindArchetypeValue(UObject* Object, const FName PropertyName, const FProperty*& OutArchetypeProperty, const void*& OutArchetypeValuePtr, bool& OutIsSparseData)
	{
		OutArchetypeProperty = nullptr;
		OutArchetypeValuePtr = nullptr;
		OutIsSparseData = false;

		const FProperty* ObjectProp = PropertyAccessUtil::FindPropertyByName(PropertyName, Object->GetClass());
		if ((!ObjectProp || ObjectProp->HasAnyPropertyFlags(CPF_Deprecated)) && Object->HasAllFlags(RF_ClassDefaultObject))
		{
			// look for a sparse member of the same name - the sparse data is treated as an extension
			// of the class default object by the details panel:
			if (const UScriptStruct* SparseDataStruct = Object->GetClass()->GetSparseClassDataStruct())
			{
				if (const FProperty* SparseProp = PropertyAccessUtil::FindPropertyByName(PropertyName, SparseDataStruct))
				{
					if (UScriptStruct* SparseDataArchetypeStruct = Object->GetClass()->GetSparseClassDataArchetypeStruct())
					{
						OutIsSparseData = true;
						OutArchetypeProperty = SparseProp;
						OutArchetypeValuePtr = SparseProp->ContainerPtrToValuePtrForDefaults<const void>(SparseDataArchetypeStruct, Object->GetClass()->GetArchetypeForSparseClassData());
					}
					return;
				}
			}
		}

		if (ObjectProp)
		{
			if (const UObject* ObjectArchetype = Object->GetArchetype())
			{
				OutArchetypeProperty = ObjectProp;
				OutArchetypeValuePtr = ObjectProp->ContainerPtrToValuePtrForDefaults<const void>(ObjectArchetype->GetClass(), ObjectArchetype);
			}
		}
	}
}

bool UKismetSystemLibrary::ResetEditorProperty(UObject* Object, const FName PropertyName, const EPropertyAccessChangeNotifyMode ChangeNotifyMode)
{
	if (!Object)
	{
		LogRuntimeError(NSLOCTEXT("KismetSystemLibrary", "ResetEditorProperty_AccessNone", "Accessed None attempting to call ResetEditorProperty."));
		return false;
	}

	const FProperty* ArchetypeProperty = nullptr;
	const void* ArchetypeValuePtr = nullptr;
	bool bIsSparseData = false;
	UE::Blueprint::Private::FindArchetypeValue(Object, PropertyName, ArchetypeProperty, ArchetypeValuePtr, bIsSparseData);
	if (!ArchetypeValuePtr)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) had no archetype value to reset to"), *PropertyName.ToString(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertySetFailedWarning);
		return false;
	}

	return Generic_SetEditorProperty(Object, PropertyName, ArchetypeValuePtr, ArchetypeProperty, ChangeNotifyMode);
}

EEditorPropertyValueState UKismetSystemLibrary::IsEditorPropertyOverridden(UObject* Object, const FName PropertyName)
{
	if (!Object)
	{
		LogRuntimeError(NSLOCTEXT("KismetSystemLibrary", "IsEditorPropertyOverridden_AccessNone", "Accessed None attempting to call IsEditorPropertyOverridden."));
		return EEditorPropertyValueState::NotFound;
	}

	const FProperty* ArchetypeProperty = nullptr;
	const void* ArchetypeValuePtr = nullptr;
	bool bIsSparseData = false;
	UE::Blueprint::Private::FindArchetypeValue(Object, PropertyName, ArchetypeProperty, ArchetypeValuePtr, bIsSparseData);
	if (!ArchetypeValuePtr)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) had no archetype value to query against"), *PropertyName.ToString(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
		return EEditorPropertyValueState::NotFound;
	}

	if (EPropertyAccessResultFlags AccessResult = PropertyAccessUtil::CanGetPropertyValue(ArchetypeProperty);
		EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
	{
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::AccessProtected))
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) is protected and cannot be read"), *PropertyName.ToString(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
			return EEditorPropertyValueState::AccessDenied;
		}

		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Property '%s' on '%s' (%s) cannot be read"), *PropertyName.ToString(), *Object->GetPathName(), *Object->GetClass()->GetName()), ELogVerbosity::Warning, UE::Blueprint::Private::PropertyGetFailedWarning);
		return EEditorPropertyValueState::AccessDenied;
	}

	auto GetPropertyValueForDiff = [Object, ArchetypeProperty](const void* PropertyValuePtr) -> FString
	{
		uint32 PortFlags = PPF_ForDiff;
		if (!Object->IsTemplate())
		{
			PortFlags |= PPF_ForDiffInstanceOnly;
		}
		if (ArchetypeProperty->ContainsInstancedObjectProperty())
		{
			PortFlags |= PPF_DeepComparison;
		}

		FString DefaultValue;
		ArchetypeProperty->ExportTextItem_Direct(DefaultValue, PropertyValuePtr, PropertyValuePtr, nullptr, PortFlags);
		return DefaultValue;
	};

	const void* ObjectValuePtr = bIsSparseData
		? ArchetypeProperty->ContainerPtrToValuePtr<const void>(Object->GetClass()->GetOrCreateSparseClassData())
		: ArchetypeProperty->ContainerPtrToValuePtr<const void>(Object);

	const FString ObjectValue = GetPropertyValueForDiff(ObjectValuePtr);
	const FString ArchetypeValue = GetPropertyValueForDiff(ArchetypeValuePtr);
	const bool bIsIdentical = ObjectValue.Equals(ArchetypeValue, ESearchCase::CaseSensitive);

	return bIsIdentical
		? EEditorPropertyValueState::Default
		: EEditorPropertyValueState::Overridden;
}

#endif	// WITH_EDITOR

int32 UKismetSystemLibrary::BeginTransaction(const FString& Context, FText Description, UObject* PrimaryObject)
{
#if WITH_EDITOR
	return GEngine->BeginTransaction(*Context, Description, PrimaryObject);
#else
	return INDEX_NONE;
#endif
}

int32 UKismetSystemLibrary::EndTransaction()
{
#if WITH_EDITOR
	return GEngine->EndTransaction();
#else
	return INDEX_NONE;
#endif
}

void UKismetSystemLibrary::CancelTransaction(const int32 Index)
{
#if WITH_EDITOR
	if (Index >= 0)
	{
		GEngine->CancelTransaction(Index);
	}
#endif
}

void UKismetSystemLibrary::TransactObject(UObject* Object)
{
#if WITH_EDITOR
	if (Object)
	{
		Object->Modify();
	}
#endif
}

void UKismetSystemLibrary::SnapshotObject(UObject* Object)
{
#if WITH_EDITOR
	if (Object)
	{
		SnapshotTransactionBuffer(Object);
	}
#endif
}

UObject* UKismetSystemLibrary::GetObjectFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId)
{
	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = UAssetManager::Get();
	FPrimaryAssetTypeInfo Info;
	if (Manager.GetPrimaryAssetTypeInfo(PrimaryAssetId.PrimaryAssetType, Info) && !Info.bHasBlueprintClasses)
	{
		return Manager.GetPrimaryAssetObject(PrimaryAssetId);
	}
	return nullptr;
}

TSubclassOf<UObject> UKismetSystemLibrary::GetClassFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId)
{
	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = UAssetManager::Get();
	FPrimaryAssetTypeInfo Info;
	if (Manager.GetPrimaryAssetTypeInfo(PrimaryAssetId.PrimaryAssetType, Info) && Info.bHasBlueprintClasses)
	{
		return Manager.GetPrimaryAssetObjectClass<UObject>(PrimaryAssetId);
	}
	return nullptr;
}

TSoftObjectPtr<UObject> UKismetSystemLibrary::GetSoftObjectReferenceFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId)
{
	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = UAssetManager::Get();
	FPrimaryAssetTypeInfo Info;
	if (Manager.GetPrimaryAssetTypeInfo(PrimaryAssetId.PrimaryAssetType, Info) && !Info.bHasBlueprintClasses)
	{
		return TSoftObjectPtr<UObject>(Manager.GetPrimaryAssetPath(PrimaryAssetId));
	}
	return nullptr;
}

TSoftClassPtr<UObject> UKismetSystemLibrary::GetSoftClassReferenceFromPrimaryAssetId(FPrimaryAssetId PrimaryAssetId)
{
	check(UAssetManager::IsInitialized());
	UAssetManager& Manager = UAssetManager::Get();
	FPrimaryAssetTypeInfo Info;
	if (Manager.GetPrimaryAssetTypeInfo(PrimaryAssetId.PrimaryAssetType, Info) && Info.bHasBlueprintClasses)
	{
		return TSoftClassPtr<UObject>(Manager.GetPrimaryAssetPath(PrimaryAssetId));
	}
	return nullptr;
}

FPrimaryAssetId UKismetSystemLibrary::GetPrimaryAssetIdFromObject(UObject* Object)
{
	if (Object)
	{
		check(UAssetManager::IsInitialized());
		return UAssetManager::Get().GetPrimaryAssetIdForObject(Object);
	}
	return FPrimaryAssetId();
}

FPrimaryAssetId UKismetSystemLibrary::GetPrimaryAssetIdFromClass(TSubclassOf<UObject> Class)
{
	if (Class)
	{
		check(UAssetManager::IsInitialized());
		return UAssetManager::Get().GetPrimaryAssetIdForObject(*Class);
	}
	return FPrimaryAssetId();
}

FPrimaryAssetId UKismetSystemLibrary::GetPrimaryAssetIdFromSoftObjectReference(TSoftObjectPtr<UObject> SoftObjectReference)
{
	check(UAssetManager::IsInitialized());
	return UAssetManager::Get().GetPrimaryAssetIdForPath(SoftObjectReference.ToSoftObjectPath());
}

FPrimaryAssetId UKismetSystemLibrary::GetPrimaryAssetIdFromSoftClassReference(TSoftClassPtr<UObject> SoftClassReference)
{
	check(UAssetManager::IsInitialized());
	return UAssetManager::Get().GetPrimaryAssetIdForPath(SoftClassReference.ToSoftObjectPath());
}

void UKismetSystemLibrary::GetPrimaryAssetIdList(FPrimaryAssetType PrimaryAssetType, TArray<FPrimaryAssetId>& OutPrimaryAssetIdList)
{
	check(UAssetManager::IsInitialized());
	UAssetManager::Get().GetPrimaryAssetIdList(PrimaryAssetType, OutPrimaryAssetIdList);
}

bool UKismetSystemLibrary::IsValidPrimaryAssetId(FPrimaryAssetId PrimaryAssetId)
{
	return PrimaryAssetId.IsValid();
}

FString UKismetSystemLibrary::Conv_PrimaryAssetIdToString(FPrimaryAssetId PrimaryAssetId)
{
	return PrimaryAssetId.ToString();
}

bool UKismetSystemLibrary::EqualEqual_PrimaryAssetId(FPrimaryAssetId A, FPrimaryAssetId B)
{
	return A == B;
}

bool UKismetSystemLibrary::NotEqual_PrimaryAssetId(FPrimaryAssetId A, FPrimaryAssetId B)
{
	return A != B;
}

bool UKismetSystemLibrary::IsValidPrimaryAssetType(FPrimaryAssetType PrimaryAssetType)
{
	return PrimaryAssetType.IsValid();
}

FString UKismetSystemLibrary::Conv_PrimaryAssetTypeToString(FPrimaryAssetType PrimaryAssetType)
{
	return PrimaryAssetType.ToString();
}

bool UKismetSystemLibrary::EqualEqual_PrimaryAssetType(FPrimaryAssetType A, FPrimaryAssetType B)
{
	return A == B;
}

bool UKismetSystemLibrary::NotEqual_PrimaryAssetType(FPrimaryAssetType A, FPrimaryAssetType B)
{
	return A != B;
}

void UKismetSystemLibrary::UnloadPrimaryAsset(FPrimaryAssetId PrimaryAssetId)
{
	check(UAssetManager::IsInitialized());
	UAssetManager::Get().UnloadPrimaryAsset(PrimaryAssetId);
}

void UKismetSystemLibrary::UnloadPrimaryAssetList(const TArray<FPrimaryAssetId>& PrimaryAssetIdList)
{
	check(UAssetManager::IsInitialized());
	UAssetManager::Get().UnloadPrimaryAssets(PrimaryAssetIdList);
}

bool UKismetSystemLibrary::GetCurrentBundleState(FPrimaryAssetId PrimaryAssetId, bool bForceCurrentState, TArray<FName>& OutBundles)
{
	check(UAssetManager::IsInitialized());
	return UAssetManager::Get().GetPrimaryAssetHandle(PrimaryAssetId, bForceCurrentState, &OutBundles).IsValid();
}

void UKismetSystemLibrary::GetPrimaryAssetsWithBundleState(const TArray<FName>& RequiredBundles, const TArray<FName>& ExcludedBundles, const TArray<FPrimaryAssetType>& ValidTypes, bool bForceCurrentState, TArray<FPrimaryAssetId>& OutPrimaryAssetIdList)
{
	check(UAssetManager::IsInitialized());
	UAssetManager::Get().GetPrimaryAssetsWithBundleState(OutPrimaryAssetIdList, ValidTypes, RequiredBundles, ExcludedBundles, bForceCurrentState);
}

FARFilter UKismetSystemLibrary::MakeARFilter(
	const TArray<FName>& PackageNames, 
	const TArray<FName>& PackagePaths, 
	const TArray<FSoftObjectPath>& SoftObjectPaths, 
	const TArray<FTopLevelAssetPath>& ClassPaths,
	const TSet<FTopLevelAssetPath>& RecursiveClassPathsExclusionSet, 
	const TArray<FName>& ClassNames, 
	const TSet<FName>& RecursiveClassesExclusionSet, 
	const bool bRecursivePaths, 
	const bool bRecursiveClasses, 
	const bool bIncludeOnlyOnDiskAssets
	)
{
	FARFilter NewFilter;
	NewFilter.PackageNames = PackageNames;
	NewFilter.PackagePaths = PackagePaths;
	NewFilter.SoftObjectPaths = SoftObjectPaths;
	NewFilter.bRecursivePaths = bRecursivePaths;
	NewFilter.bRecursiveClasses = bRecursiveClasses;
	NewFilter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;

	NewFilter.ClassPaths = ClassPaths;
	NewFilter.RecursiveClassPathsExclusionSet = RecursiveClassPathsExclusionSet;

	// Fixup to move to FTopLevelAssetPath
	for (const FName& ClassName : ClassNames)
	{
		FTopLevelAssetPath ClassPathName;
		if (!ClassName.IsNone())
		{
			FString ShortClassName = ClassName.ToString();
			ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(*ShortClassName, ELogVerbosity::Warning, TEXT("MakeARFilter should use ClassPaths, ClassNames is deprecated."));
			UE_CLOGF(ClassPathName.IsNull(), LogClass, Error, "Failed to convert short class name %ls to class path name.", *ShortClassName);
		}
		
		NewFilter.ClassPaths.Add(ClassPathName);
	}
	for (const FName& RecursiveClassToExclude : RecursiveClassesExclusionSet)
	{
		FTopLevelAssetPath ClassPathName;
		if (!RecursiveClassToExclude.IsNone())
		{
			FString ShortClassName = RecursiveClassToExclude.ToString();
			ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(*ShortClassName, ELogVerbosity::Warning, TEXT("MakeARFilter should use RecursiveClassPathsExclusionSet, RecursiveClassesExclusionSet is deprecated."));
			UE_CLOGF(ClassPathName.IsNull(), LogClass, Error, "Failed to convert short class name %ls to class path name.", *ShortClassName);
		}
		
		NewFilter.RecursiveClassPathsExclusionSet.Add(ClassPathName);
	}

	return NewFilter;
}

void UKismetSystemLibrary::BreakARFilter(
	FARFilter InARFilter,
	TArray<FName>& PackageNames,
	TArray<FName>& PackagePaths,
	TArray<FSoftObjectPath>& SoftObjectPaths,
	TArray<FTopLevelAssetPath>& ClassPaths,
	TSet<FTopLevelAssetPath>& RecursiveClassPathsExclusionSet,
	TArray<FName>& ClassNames,
	TSet<FName>& RecursiveClassesExclusionSet,
	bool& bRecursivePaths,
	bool& bRecursiveClasses,
	bool& bIncludeOnlyOnDiskAssets
	)
{
	PackageNames = InARFilter.PackageNames;
	PackagePaths = InARFilter.PackagePaths;
	SoftObjectPaths = InARFilter.SoftObjectPaths;
	ClassPaths = InARFilter.ClassPaths;
	RecursiveClassPathsExclusionSet = InARFilter.RecursiveClassPathsExclusionSet;
	bRecursivePaths = InARFilter.bRecursivePaths;
	bRecursiveClasses = InARFilter.bRecursiveClasses;
	bIncludeOnlyOnDiskAssets = InARFilter.bIncludeOnlyOnDiskAssets;

	// Fixup to move from FTopLevelAssetPath to legacy types
	for (const FTopLevelAssetPath& ClassPath : ClassPaths)
	{
		ClassNames.Add(ClassPath.GetAssetName());
	}
	for (const FTopLevelAssetPath& RecursiveClassPathToExclude : RecursiveClassPathsExclusionSet)
	{
		RecursiveClassesExclusionSet.Add(RecursiveClassPathToExclude.GetAssetName());
	}
}

#undef LOCTEXT_NAMESPACE

