// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetTextLibrary.h"

#include "Engine/World.h"
#include "Internationalization/PolyglotTextData.h"
#include "Internationalization/TextFormatter.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Misc/RuntimeErrors.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyAccessUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KismetTextLibrary)

#define LOCTEXT_NAMESPACE "Kismet"

UKismetTextLibrary::UKismetTextLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }

FText UKismetTextLibrary::Conv_VectorToText(FVector InVec)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("X"), InVec.X);
	Args.Add(TEXT("Y"), InVec.Y);
	Args.Add(TEXT("Z"), InVec.Z);

	return FText::Format(NSLOCTEXT("Core", "Vector3", "X={X} Y={Y} Z={Z}"), Args);
}

FText UKismetTextLibrary::Conv_Vector2dToText(FVector2D InVec)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("X"), InVec.X);
	Args.Add(TEXT("Y"), InVec.Y);

	return FText::Format(NSLOCTEXT("Core", "Vector2", "X={X} Y={Y}"), Args);
}

FText UKismetTextLibrary::Conv_RotatorToText(FRotator InRot)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("P"), InRot.Pitch);
	Args.Add(TEXT("Y"), InRot.Yaw);
	Args.Add(TEXT("R"), InRot.Roll);

	return FText::Format(NSLOCTEXT("Core", "Rotator", "P={P} Y={Y} R={R}"), Args);
}

FText UKismetTextLibrary::Conv_TransformToText(const FTransform& InTrans)
{
	const FVector T(InTrans.GetTranslation());
	const FRotator R(InTrans.Rotator());
	const FVector S(InTrans.GetScale3D());

	FFormatNamedArguments Args;
	Args.Add(TEXT("T"), Conv_VectorToText(T));
	Args.Add(TEXT("R"), Conv_RotatorToText(R));
	Args.Add(TEXT("S"), Conv_VectorToText(S));

	return FText::Format(NSLOCTEXT("Core", "Transform", "Translation: {T} Rotation: {R} Scale: {S}"), Args);
}

FText UKismetTextLibrary::Conv_ObjectToText(class UObject* InObj)
{
	if (InObj)
	{
		return FText::AsCultureInvariant(InObj->GetName());
	}

	return FCoreTexts::Get().None;
}

FText UKismetTextLibrary::Conv_ColorToText(FLinearColor InColor)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("R"), InColor.R);
	Args.Add(TEXT("G"), InColor.G);
	Args.Add(TEXT("B"), InColor.B);
	Args.Add(TEXT("A"), InColor.A);

	return FText::Format(NSLOCTEXT("Core", "LinearColor", "R={R} G={G} B={B} A={A}"), Args);
}

FString UKismetTextLibrary::Conv_TextToString(const FText& InText)
{
	return InText.ToString();
}

FText UKismetTextLibrary::Conv_StringToText(const FString& InString)
{	
	return FText::AsCultureInvariant(InString);
}

FText UKismetTextLibrary::Conv_NameToText(FName InName)
{
	return FText::AsCultureInvariant(InName.ToString());
}

FText UKismetTextLibrary::MakeInvariantText(const FString& InString)
{
	return FText::AsCultureInvariant(InString);
}

bool UKismetTextLibrary::TextIsEmpty(const FText& InText)
{
	return InText.IsEmpty();
}

bool UKismetTextLibrary::TextIsTransient(const FText& InText)
{
	return InText.IsTransient();
}

bool UKismetTextLibrary::TextIsCultureInvariant(const FText& InText)
{
	return InText.IsCultureInvariant();
}

FText UKismetTextLibrary::TextToLower(const FText& InText)
{
	return InText.ToLower();
}

FText UKismetTextLibrary::TextToUpper(const FText& InText)
{
	return InText.ToUpper();
}

FText UKismetTextLibrary::TextTrimPreceding(const FText& InText)
{
	return FText::TrimPreceding(InText);
}

FText UKismetTextLibrary::TextTrimTrailing(const FText& InText)
{
	return FText::TrimTrailing(InText);
}

FText UKismetTextLibrary::TextTrimPrecedingAndTrailing(const FText& InText)
{
	return FText::TrimPrecedingAndTrailing(InText);
}

FText UKismetTextLibrary::GetEmptyText()
{
	return FText::GetEmpty();
}

bool UKismetTextLibrary::FindTextInLocalizationTable(const FString& Namespace, const FString& Key, FText& OutText, const FString& SourceString)
{
	return FText::FindTextInLiveTable_Advanced(Namespace, Key, OutText, SourceString.IsEmpty() ? nullptr : &SourceString);
}

bool UKismetTextLibrary::EqualEqual_IgnoreCase_TextText(const FText& A, const FText& B)
{
	return A.EqualToCaseIgnored( B );
}

bool UKismetTextLibrary::EqualEqual_TextText(const FText& A, const FText& B)
{
	return A.EqualTo( B );
}

bool UKismetTextLibrary::NotEqual_IgnoreCase_TextText(const FText& A, const FText& B)
{
	return !A.EqualToCaseIgnored( B );
}

bool UKismetTextLibrary::NotEqual_TextText(const FText& A, const FText& B)
{
	return !A.EqualTo( B );
}

FText UKismetTextLibrary::Conv_BoolToText(bool InBool)
{
	return InBool ? LOCTEXT("True", "true") : LOCTEXT("False", "false");
}

FText UKismetTextLibrary::Conv_ByteToText(uint8 Value)
{
	return FText::AsNumber(Value, &FNumberFormattingOptions::DefaultNoGrouping());
}

FText UKismetTextLibrary::Conv_IntToText(int32 Value, bool bAlwaysSign/* = false*/, bool bUseGrouping/* = true*/, int32 MinimumIntegralDigits/* = 1*/, int32 MaximumIntegralDigits/* = 324*/)
{
	// Only update the values that need to be changed from the default FNumberFormattingOptions, 
	// as this lets us use the default formatter if possible (which is a performance win!)
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.AlwaysSign = bAlwaysSign;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;

	return FText::AsNumber(Value, &NumberFormatOptions);
}

FText UKismetTextLibrary::Conv_Int64ToText(int64 Value, bool bAlwaysSign /*= false*/, bool bUseGrouping /*= true*/, int32 MinimumIntegralDigits /*= 1*/, int32 MaximumIntegralDigits /*= 324*/)
{
	// Only update the values that need to be changed from the default FNumberFormattingOptions, 
	// as this lets us use the default formatter if possible (which is a performance win!)
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.AlwaysSign = bAlwaysSign;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;

	return FText::AsNumber(Value, &NumberFormatOptions);
}

FText UKismetTextLibrary::Conv_DoubleToText(double Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign, bool bUseGrouping, int32 MinimumIntegralDigits, int32 MaximumIntegralDigits, int32 MinimumFractionalDigits, int32 MaximumFractionalDigits)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.AlwaysSign = bAlwaysSign;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.RoundingMode = RoundingMode;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;
	NumberFormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;
	
	return FText::AsNumber(Value, &NumberFormatOptions);
}

FText UKismetTextLibrary::AsCurrencyBase(int32 BaseValue, const FString& CurrencyCode)
{
	return FText::AsCurrencyBase(BaseValue, CurrencyCode);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// FIXME: we need to deprecate this kismet api too
FText UKismetTextLibrary::AsCurrency_Integer(int32 Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign/* = false*/, bool bUseGrouping/* = true*/, int32 MinimumIntegralDigits/* = 1*/, int32 MaximumIntegralDigits/* = 324*/, int32 MinimumFractionalDigits/* = 0*/, int32 MaximumFractionalDigits/* = 3*/, const FString& CurrencyCode)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.AlwaysSign = bAlwaysSign;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.RoundingMode = RoundingMode;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;
	NumberFormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;
	return FText::AsCurrency(Value, CurrencyCode, &NumberFormatOptions);
}
FText UKismetTextLibrary::AsCurrency_Float(float Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign/* = false*/, bool bUseGrouping/* = true*/, int32 MinimumIntegralDigits/* = 1*/, int32 MaximumIntegralDigits/* = 324*/, int32 MinimumFractionalDigits/* = 0*/, int32 MaximumFractionalDigits/* = 3*/, const FString& CurrencyCode)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.AlwaysSign = bAlwaysSign;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.RoundingMode = RoundingMode;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;
	NumberFormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;
	return FText::AsCurrency(Value, CurrencyCode, &NumberFormatOptions);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FText UKismetTextLibrary::AsPercent_Float(float Value, TEnumAsByte<ERoundingMode> RoundingMode, bool bAlwaysSign/* = false*/, bool bUseGrouping/* = true*/, int32 MinimumIntegralDigits/* = 1*/, int32 MaximumIntegralDigits/* = 324*/, int32 MinimumFractionalDigits/* = 0*/, int32 MaximumFractionalDigits/* = 3*/)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.AlwaysSign = bAlwaysSign;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.RoundingMode = RoundingMode;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;
	NumberFormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;

	return FText::AsPercent(Value, &NumberFormatOptions);
}

FText UKismetTextLibrary::AsDate_DateTime(const FDateTime& InDateTime, TEnumAsByte<EDateTimeStyle::Type> InDateStyle)
{
	return FText::AsDate(InDateTime, InDateStyle, FText::GetInvariantTimeZone());
}

FText UKismetTextLibrary::AsTimeZoneDate_DateTime(const FDateTime& InDateTime, const FString& InTimeZone, TEnumAsByte<EDateTimeStyle::Type> InDateStyle)
{
	return FText::AsDate(InDateTime, InDateStyle, InTimeZone);
}

FText UKismetTextLibrary::AsDateTime_DateTime(const FDateTime& InDateTime, TEnumAsByte<EDateTimeStyle::Type> InDateStyle, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle)
{
	return FText::AsDateTime(InDateTime, InDateStyle, InTimeStyle, FText::GetInvariantTimeZone());
}

FText UKismetTextLibrary::AsTimeZoneDateTime_DateTime(const FDateTime& InDateTime, const FString& InTimeZone, TEnumAsByte<EDateTimeStyle::Type> InDateStyle, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle)
{
	return FText::AsDateTime(InDateTime, InDateStyle, InTimeStyle, InTimeZone);
}

FText UKismetTextLibrary::AsTime_DateTime(const FDateTime& InDateTime, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle)
{
	return FText::AsTime(InDateTime, InTimeStyle, FText::GetInvariantTimeZone());
}

FText UKismetTextLibrary::AsTimeZoneTime_DateTime(const FDateTime& InDateTime, const FString& InTimeZone, TEnumAsByte<EDateTimeStyle::Type> InTimeStyle)
{
	return FText::AsTime(InDateTime, InTimeStyle, InTimeZone);
}

FText UKismetTextLibrary::AsTimespan_Timespan(const FTimespan& InTimespan)
{
	return FText::AsTimespan(InTimespan);
}

FText UKismetTextLibrary::AsMemory(int64 NumBytes, TEnumAsByte<EMemoryUnitStandard> UnitStandard, bool bUseGrouping, int32 MinimumIntegralDigits, int32 MaximumIntegralDigits, int32 MinimumFractionalDigits, int32 MaximumFractionalDigits)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;
	NumberFormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;

	uint64 UnsignedNumBytes = static_cast<uint64>(FMath::Max(0, NumBytes));
	return FText::AsMemory(UnsignedNumBytes, &NumberFormatOptions, nullptr, UnitStandard);
}

FText UKismetTextLibrary::AsMemoryUnit(int64 NumBytes, EMemoryUnit Unit, bool bUseGrouping, int32 MinimumIntegralDigits, int32 MaximumIntegralDigits, int32 MinimumFractionalDigits, int32 MaximumFractionalDigits)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.UseGrouping = bUseGrouping;
	NumberFormatOptions.MinimumIntegralDigits = MinimumIntegralDigits;
	NumberFormatOptions.MaximumIntegralDigits = MaximumIntegralDigits;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;
	NumberFormatOptions.MaximumFractionalDigits = MaximumFractionalDigits;

	uint64 UnsignedNumBytes = static_cast<uint64>(FMath::Max(0, NumBytes));
	return FText::AsMemory(UnsignedNumBytes, Unit, &NumberFormatOptions, nullptr);
}

FText UKismetTextLibrary::Format(FText InPattern, TArray<FFormatArgumentData> InArgs)
{
	return FTextFormatter::Format(MoveTemp(InPattern), MoveTemp(InArgs), false, false);
}

bool UKismetTextLibrary::TextIsFromStringTable(const FText& Text)
{
	return Text.IsFromStringTable();
}

FText UKismetTextLibrary::TextFromStringTable(const FName TableId, const FString& Key)
{
	return FText::FromStringTable(TableId, Key);
}

bool UKismetTextLibrary::StringTableIdAndKeyFromText(FText Text, FName& OutTableId, FString& OutKey)
{
	return FTextInspector::GetTableIdAndKey(Text, OutTableId, OutKey);
}

bool UKismetTextLibrary::GetTextId(FText Text, FString& OutNamespace, FString& OutKey)
{
	const FTextId TextId = FTextInspector::GetTextId(Text);
	if (!TextId.IsEmpty())
	{
		TextId.GetNamespace().ToString(OutNamespace);
		TextId.GetKey().ToString(OutKey);
		return true;
	}
	return false;
}

FString UKismetTextLibrary::GetTextSourceString(FText Text)
{
	return Text.BuildSourceString();
}

void UKismetTextLibrary::IsPolyglotDataValid(const FPolyglotTextData& PolyglotData, bool& IsValid, FText& ErrorMessage)
{
	IsValid = PolyglotData.IsValid(&ErrorMessage);
}

FText UKismetTextLibrary::PolyglotDataToText(const FPolyglotTextData& PolyglotData)
{
	return PolyglotData.GetText();
}

bool UKismetTextLibrary::EditTextSourceString(UObject* TextOwner, FText& Text, const FString& SourceString, const bool bEmitChangeNotify)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

DEFINE_FUNCTION(UKismetTextLibrary::execEditTextSourceString)
{
	P_GET_OBJECT(UObject, TextOwner);

	P_GET_PROPERTY_REF(FTextProperty, Text);
	const FTextProperty* TextProperty = CastField<FTextProperty>(Stack.MostRecentProperty);

	P_GET_PROPERTY_REF(FStrProperty, SourceString);

	P_GET_PROPERTY(FBoolProperty, bEmitChangeNotify);

	P_FINISH;

	P_NATIVE_BEGIN;
	{
		*(bool*)RESULT_PARAM = false;

		if (!TextOwner)
		{
			LogRuntimeWarning(LOCTEXT("EditTextSourceString.Warning.NullTextOwner", "The given TextOwner was null!"));
			return;
		}

		if (!TextProperty)
		{
			LogRuntimeWarning(LOCTEXT("EditTextSourceString.Warning.NullTextProperty", "The given Text value was not a TextProperty!"));
			return;
		}

		if (Generic_EditTextPropertySourceString(TextOwner, TextProperty, SourceString, bEmitChangeNotify))
		{
			*(bool*)RESULT_PARAM = true;
			Text = TextProperty->GetPropertyValue_InContainer(TextOwner);
		}
	}
	P_NATIVE_END;
}

bool UKismetTextLibrary::EditTextPropertySourceString(UObject* TextOwner, const FName PropertyName, const FString& SourceString, const bool bEmitChangeNotify)
{
	if (!TextOwner)
	{
		LogRuntimeWarning(LOCTEXT("EditTextPropertySourceString.Warning.NullTextOwner", "The given TextOwner was null!"));
		return false;
	}

	const FTextProperty* TextProperty = CastField<FTextProperty>(TextOwner->GetClass()->FindPropertyByName(PropertyName));
	if (!TextProperty)
	{
		LogRuntimeWarning(FText::Format(LOCTEXT("EditTextPropertySourceString.Warning.NullTextProperty", "The given PropertyName ({0}) did not resolve to a TextProperty!"), FText::FromName(PropertyName)));
		return false;
	}

	return Generic_EditTextPropertySourceString(TextOwner, TextProperty, SourceString, bEmitChangeNotify);
}

bool UKismetTextLibrary::Generic_EditTextPropertySourceString(UObject* TextOwner, const FTextProperty* TextProperty, const FString& SourceString, const bool bEmitChangeNotify)
{
	checkf(TextOwner, TEXT("TextOwner is null!"));
	checkf(TextProperty, TEXT("TextProperty is null!"));

	if (!TextOwner->GetClass()->HasProperty(TextProperty))
	{
		LogRuntimeWarning(FText::Format(LOCTEXT("EditTextPropertySourceString.Warning.InvalidTextProperty", "The resolved TextProperty ({0}) doesn't belong to the given TextOwner ({1})!"), FText::AsCultureInvariant(TextProperty->GetPathName()), FText::AsCultureInvariant(TextOwner->GetPathName())));
		return false;
	}
		
	auto TextOwnerIsInEditorWorld = [TextOwner]()
	{
		const UWorld* WorldContext = TextOwner->GetWorld();
		return WorldContext && WorldContext->WorldType == EWorldType::Editor;
	};

	auto TextOwnerIsInAsset = [TextOwner]()
	{
		for (const UObject* Obj = TextOwner; Obj; Obj = Obj->GetOuter())
		{
			if (Obj->IsAsset())
			{
				return true;
			}
		}
		return false;
	};

	const bool bIsEditingEditorObject = GIsEditor && (TextOwnerIsInEditorWorld() || TextOwnerIsInAsset());
	const bool bIsEditingTemplateObject = PropertyAccessUtil::IsObjectTemplate(TextOwner);

	if (PropertyAccessUtil::CanSetPropertyValue(TextProperty, bIsEditingEditorObject ? PropertyAccessUtil::EditorReadOnlyFlags : PropertyAccessUtil::RuntimeReadOnlyFlags, bIsEditingTemplateObject) != EPropertyAccessResultFlags::Success)
	{
		LogRuntimeWarning(FText::Format(LOCTEXT("EditTextPropertySourceString.Warning.InvalidAccessPermissions", "The resolved TextProperty ({0}) cannot be edited as it is read-only on the given TextOwner ({1})!"), FText::AsCultureInvariant(TextProperty->GetPathName()), FText::AsCultureInvariant(TextOwner->GetPathName())));
		return false;
	}

	TArray<UObject*> InheritedInstances;
	TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify;
	if (GIsEditor)
	{
		if (TextOwner->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
		{
			PropertyAccessUtil::GetArchetypeInstancesInheritingPropertyValue(TextProperty, TextOwner, InheritedInstances);
		}

		if (bEmitChangeNotify)
		{
			ChangeNotify = PropertyAccessUtil::BuildBasicChangeNotify(TextProperty, TextOwner, EPropertyAccessChangeNotifyMode::Always, EPropertyChangeType::ValueSet);
			PropertyAccessUtil::EmitPreChangeNotify(ChangeNotify.Get(), false);
		}
	}

	auto DeterministicTextKeyGenerator = [TextOwner, TextProperty, bIsEditingEditorObject]()
	{
		return TextNamespaceUtil::GenerateDeterministicTextKey(TextOwner, TextProperty, bIsEditingEditorObject);
	};

	const bool bEditedValue = TextNamespaceUtil::EditTextProperty(TextOwner, TextProperty, TextNamespaceUtil::ETextEditAction::SourceString, SourceString, DeterministicTextKeyGenerator, bIsEditingEditorObject);

	if (GIsEditor)
	{
		if (bEditedValue && bIsEditingTemplateObject)
		{
			// Propagate to archetype instances that had the same value prior to the edit
			for (UObject* InheritedInstance : InheritedInstances)
			{
				PropertyAccessUtil::CopySinglePropertyValue(TextProperty, TextProperty->GetPropertyValuePtr_InContainer(TextOwner), TextProperty, TextProperty->GetPropertyValuePtr_InContainer(InheritedInstance));
			}
		}

		if (bEmitChangeNotify)
		{
			PropertyAccessUtil::EmitPostChangeNotify(ChangeNotify.Get(), false);
		}
	}

	return bEditedValue;
}

FText UKismetTextLibrary::Conv_NumericPropertyToText(const int32& Value)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return FText::GetEmpty();
}

DEFINE_FUNCTION(UKismetTextLibrary::execConv_NumericPropertyToText)
{
	Stack.StepCompiledIn<FProperty>(nullptr);
	const FProperty* SourceProperty = Stack.MostRecentProperty;
	void* SourceValuePtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	P_NATIVE_BEGIN;
	{
		*(FText*)RESULT_PARAM = FText::GetEmpty();

		if (SourceProperty == nullptr || SourceValuePtr == nullptr)
		{
			LogRuntimeWarning(LOCTEXT("GenericToText.Warning.NullProperty", "The property is invalid!"));
			return;
		}


		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(SourceProperty))
		{
			if (NumericProperty->IsFloatingPoint())
			{
				double Value = NumericProperty->GetFloatingPointPropertyValue(SourceValuePtr);
				*(FText*)RESULT_PARAM = FText::AsNumber(Value);
			}
			else if (UEnum* Enum = NumericProperty->GetIntPropertyEnum())
			{
				int64 Value = NumericProperty->GetSignedIntPropertyValue(SourceValuePtr);
				*(FText*)RESULT_PARAM = Enum->GetDisplayNameTextByValue(Value);
			}
			else if (NumericProperty->IsInteger())
			{
				// Value from BP are always signed.
				int64 Value = NumericProperty->GetSignedIntPropertyValue(SourceValuePtr);
				*(FText*)RESULT_PARAM = FText::AsNumber(Value);
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(SourceProperty))
		{
			const int64 Value = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(SourceValuePtr);
			*(FText*)RESULT_PARAM = EnumProperty->GetEnum()->GetDisplayNameTextByValue(Value);
		}
		else
		{
			LogRuntimeWarning(LOCTEXT("GenericToText.Warning.NotSupported", "The property not supported"));
		}
	}
	P_NATIVE_END;
}

#undef LOCTEXT_NAMESPACE
