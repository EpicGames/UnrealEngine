// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetTraceUtils.h"
#include "CollisionQueryParams.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Engine/HitResult.h"
#include "EngineLogs.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Math/RotationMatrix.h"

static const float KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE = 16.f;


FCollisionQueryParams ConfigureCollisionParams(FName TraceTag, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, bool bIgnoreSelf, const UObject* WorldContextObject)
{
	FCollisionQueryParams Params(TraceTag, SCENE_QUERY_STAT_ONLY(KismetTraceUtils), bTraceComplex);
	Params.bReturnPhysicalMaterial = true;
	Params.bReturnFaceIndex = !UPhysicsSettings::Get()->bSuppressFaceRemapTable; // Ask for face index, as long as we didn't disable globally
	Params.AddIgnoredActors(ActorsToIgnore);
	if (bIgnoreSelf)
	{
		const AActor* IgnoreActor = Cast<AActor>(WorldContextObject);
		if (IgnoreActor)
		{
			Params.AddIgnoredActor(IgnoreActor);
		}
		else
		{
			// find owner
			const UObject* CurrentObject = WorldContextObject;
			while (CurrentObject)
			{
				CurrentObject = CurrentObject->GetOuter();
				IgnoreActor = Cast<AActor>(CurrentObject);
				if (IgnoreActor)
				{
					Params.AddIgnoredActor(IgnoreActor);
					break;
				}
			}
		}
	}

	return Params;
}

FCollisionObjectQueryParams ConfigureCollisionObjectParams(const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes)
{
	TArray<TEnumAsByte<ECollisionChannel>> CollisionObjectTraces;
	CollisionObjectTraces.AddUninitialized(ObjectTypes.Num());

	for (auto Iter = ObjectTypes.CreateConstIterator(); Iter; ++Iter)
	{
		CollisionObjectTraces[Iter.GetIndex()] = UEngineTypes::ConvertToCollisionChannel(*Iter);
	}

	FCollisionObjectQueryParams ObjectParams;
	for (auto Iter = CollisionObjectTraces.CreateConstIterator(); Iter; ++Iter)
	{
		const ECollisionChannel & Channel = (*Iter);
		if (FCollisionObjectQueryParams::IsValidObjectQuery(Channel))
		{
			ObjectParams.AddObjectTypesToQuery(Channel);
		}
		else
		{
			UE_LOGF(LogBlueprintUserMessages, Warning, "%d isn't valid object type", (int32)Channel);
		}
	}

	return ObjectParams;
}

#if ENABLE_DRAW_DEBUG

void DrawDebugSweptSphere(const UWorld* InWorld, FVector const& Start, FVector const& End, float Radius, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority)
{
	FVector const TraceVec = End - Start;
	float const Dist = TraceVec.Size();

	FVector const Center = Start + TraceVec * 0.5f;
	float const HalfHeight = (Dist * 0.5f) + Radius;

	FQuat const CapsuleRot = FRotationMatrix::MakeFromZ(TraceVec).ToQuat();
	::DrawDebugCapsule(InWorld, Center, HalfHeight, Radius, CapsuleRot, Color, bPersistentLines, LifeTime, DepthPriority);
}

void DrawDebugSweptBox(const UWorld* InWorld, FVector const& Start, FVector const& End, FRotator const & Orientation, FVector const & HalfSize, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority)
{
	FVector const TraceVec = End - Start;
	float const Dist = TraceVec.Size();

	FVector const Center = Start + TraceVec * 0.5f;

	FQuat const CapsuleRot = Orientation.Quaternion();
	::DrawDebugBox(InWorld, Start, HalfSize, CapsuleRot, Color, bPersistentLines, LifeTime, DepthPriority);

	//now draw lines from vertices
	FVector Vertices[8];
	Vertices[0] = Start + CapsuleRot.RotateVector(FVector(-HalfSize.X, -HalfSize.Y, -HalfSize.Z));	//flt
	Vertices[1] = Start + CapsuleRot.RotateVector(FVector(-HalfSize.X, HalfSize.Y, -HalfSize.Z));	//frt
	Vertices[2] = Start + CapsuleRot.RotateVector(FVector(-HalfSize.X, -HalfSize.Y, HalfSize.Z));	//flb
	Vertices[3] = Start + CapsuleRot.RotateVector(FVector(-HalfSize.X, HalfSize.Y, HalfSize.Z));	//frb
	Vertices[4] = Start + CapsuleRot.RotateVector(FVector(HalfSize.X, -HalfSize.Y, -HalfSize.Z));	//blt
	Vertices[5] = Start + CapsuleRot.RotateVector(FVector(HalfSize.X, HalfSize.Y, -HalfSize.Z));	//brt
	Vertices[6] = Start + CapsuleRot.RotateVector(FVector(HalfSize.X, -HalfSize.Y, HalfSize.Z));	//blb
	Vertices[7] = Start + CapsuleRot.RotateVector(FVector(HalfSize.X, HalfSize.Y, HalfSize.Z));		//brb
	for (int32 VertexIdx = 0; VertexIdx < 8; ++VertexIdx)
	{
		::DrawDebugLine(InWorld, Vertices[VertexIdx], Vertices[VertexIdx] + TraceVec, Color, bPersistentLines, LifeTime, DepthPriority);
	}

	::DrawDebugBox(InWorld, End, HalfSize, CapsuleRot, Color, bPersistentLines, LifeTime, DepthPriority);
}

/** Util for drawing result of single line trace  */
void DrawDebugLineTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;

		// @fixme, draw line with thickness = 2.f?
		if (bHit && OutHit.bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			::DrawDebugLine(World, Start, OutHit.ImpactPoint, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, OutHit.ImpactPoint, End, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugPoint(World, OutHit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugLine(World, Start, End, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
	}
}

/** Util for drawing result of multi line trace  */
void DrawDebugLineTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;

		// @fixme, draw line with thickness = 2.f?
		if (bHit && OutHits.Last().bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			FVector const BlockingHitPoint = OutHits.Last().ImpactPoint;
			::DrawDebugLine(World, Start, BlockingHitPoint, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, BlockingHitPoint, End, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugLine(World, Start, End, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}

		// draw hits
		for (int32 HitIdx = 0; HitIdx < OutHits.Num(); ++HitIdx)
		{
			FHitResult const& Hit = OutHits[HitIdx];
			::DrawDebugPoint(World, Hit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, (Hit.bBlockingHit ? TraceColor.ToFColor(true) : TraceHitColor.ToFColor(true)), bPersistent, LifeTime);
		}
	}
}

void DrawDebugBoxTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, const FVector HalfSize, const FRotator Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None && (World != nullptr))
	{
		bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;

		if (bHit && OutHit.bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			::DrawDebugSweptBox(World, Start, OutHit.Location, Orientation, HalfSize, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugSweptBox(World, OutHit.Location, End, Orientation, HalfSize, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugPoint(World, OutHit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugSweptBox(World, Start, End, Orientation, HalfSize, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
	}
}

void DrawDebugBoxTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, const FVector HalfSize, const FRotator Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None && (World != nullptr))
	{
		bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;

		if (bHit && OutHits.Last().bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			FVector const BlockingHitPoint = OutHits.Last().Location;
			::DrawDebugSweptBox(World, Start, BlockingHitPoint, Orientation, HalfSize, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugSweptBox(World, BlockingHitPoint, End, Orientation, HalfSize, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugSweptBox(World, Start, End, Orientation, HalfSize, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}

		// draw hits
		for (int32 HitIdx = 0; HitIdx < OutHits.Num(); ++HitIdx)
		{
			FHitResult const& Hit = OutHits[HitIdx];
			::DrawDebugPoint(World, Hit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, (Hit.bBlockingHit ? TraceColor.ToFColor(true) : TraceHitColor.ToFColor(true)), bPersistent, LifeTime);
		}
	}
}

void DrawDebugSphereTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, float Radius, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;

		if (bHit && OutHit.bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			::DrawDebugSweptSphere(World, Start, OutHit.Location, Radius, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugSweptSphere(World, OutHit.Location, End, Radius, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugPoint(World, OutHit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugSweptSphere(World, Start, End, Radius, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
	}
}

void DrawDebugSphereTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, float Radius, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;

		if (bHit && OutHits.Last().bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			FVector const BlockingHitPoint = OutHits.Last().Location;
			::DrawDebugSweptSphere(World, Start, BlockingHitPoint, Radius, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugSweptSphere(World, BlockingHitPoint, End, Radius, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugSweptSphere(World, Start, End, Radius, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}

		// draw hits
		for (int32 HitIdx = 0; HitIdx < OutHits.Num(); ++HitIdx)
		{
			FHitResult const& Hit = OutHits[HitIdx];
			::DrawDebugPoint(World, Hit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, (Hit.bBlockingHit ? TraceColor.ToFColor(true) : TraceHitColor.ToFColor(true)), bPersistent, LifeTime);
		}
	}
}

void DrawDebugCapsuleTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	DrawDebugCapsuleTraceSingle(World, Start, End, Radius, HalfHeight, FRotator::ZeroRotator, DrawDebugType, bHit, OutHit, TraceColor, TraceHitColor, DrawTime);
}

void DrawDebugCapsuleTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, const FRotator& Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		const bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		const float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;
		const FQuat OrientationQuat = Orientation.Quaternion();

		if (bHit && OutHit.bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			::DrawDebugCapsule(World, Start, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugCapsule(World, OutHit.Location, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, Start, OutHit.Location, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugPoint(World, OutHit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, TraceColor.ToFColor(true), bPersistent, LifeTime);

			::DrawDebugCapsule(World, End, HalfHeight, Radius, OrientationQuat, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, OutHit.Location, End, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugCapsule(World, Start, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugCapsule(World, End, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, Start, End, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}
	}
}

void DrawDebugCapsuleTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	DrawDebugCapsuleTraceMulti(World, Start, End, Radius, HalfHeight, FRotator::ZeroRotator, DrawDebugType, bHit, OutHits, TraceColor, TraceHitColor, DrawTime);
}

void DrawDebugCapsuleTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, const FRotator& Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	if (DrawDebugType != EDrawDebugTrace::None)
	{
		const bool bPersistent = DrawDebugType == EDrawDebugTrace::Persistent;
		const float LifeTime = (DrawDebugType == EDrawDebugTrace::ForDuration) ? DrawTime : 0.f;
		const FQuat OrientationQuat = Orientation.Quaternion();

		if (bHit && OutHits.Last().bBlockingHit)
		{
			// Red up to the blocking hit, green thereafter
			FVector const BlockingHitPoint = OutHits.Last().Location;
			::DrawDebugCapsule(World, Start, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugCapsule(World, BlockingHitPoint, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, Start, BlockingHitPoint, TraceColor.ToFColor(true), bPersistent, LifeTime);

			::DrawDebugCapsule(World, End, HalfHeight, Radius, OrientationQuat, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, BlockingHitPoint, End, TraceHitColor.ToFColor(true), bPersistent, LifeTime);
		}
		else
		{
			// no hit means all red
			::DrawDebugCapsule(World, Start, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugCapsule(World, End, HalfHeight, Radius, OrientationQuat, TraceColor.ToFColor(true), bPersistent, LifeTime);
			::DrawDebugLine(World, Start, End, TraceColor.ToFColor(true), bPersistent, LifeTime);
		}

		// draw hits
		for (int32 HitIdx = 0; HitIdx < OutHits.Num(); ++HitIdx)
		{
			FHitResult const& Hit = OutHits[HitIdx];
			::DrawDebugPoint(World, Hit.ImpactPoint, KISMET_TRACE_DEBUG_IMPACTPOINT_SIZE, (Hit.bBlockingHit ? TraceColor.ToFColor(true) : TraceHitColor.ToFColor(true)), bPersistent, LifeTime);
		}
	}
}

#endif // ENABLE_DRAW_DEBUG
