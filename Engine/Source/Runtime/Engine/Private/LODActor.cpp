// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LODActorBase.cpp: Static mesh actor base class implementation.
=============================================================================*/

#include "Engine/LODActor.h"

#include "HLOD/HLODBatchingPolicy.h"
#include "UObject/UObjectIterator.h"
#include "Engine/CollisionProfile.h"
#include "HLOD/HLODProxyDesc.h"
#include "HLOD/HLODSetup.h"
#include "Misc/MapErrors.h"
#include "Logging/MessageLog.h"
#include "Materials/MaterialInterface.h"
#include "Misc/UObjectToken.h"

#include "Engine/StaticMesh.h"
#include "PrimitiveDrawingUtils.h"
#include "StaticMeshResources.h"
#include "EngineUtils.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Engine/HLODProxy.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LODActor)

#if WITH_EDITOR
#include "UObject/ObjectSaveContext.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogHLOD, Log, All);

#define LOCTEXT_NAMESPACE "LODActor"

int32 GMaximumAllowedHLODLevel = -1;

static FAutoConsoleVariableRef CVarMaximumAllowedHLODLevel(
	TEXT("r.HLOD.MaximumLevel"),
	GMaximumAllowedHLODLevel,
	TEXT("How far down the LOD hierarchy to allow showing (can be used to limit quality loss and streaming texture memory usage on high scalability settings)\n")
	TEXT("-1: No maximum level (default)\n")
	TEXT("0: Prevent ever showing a HLOD cluster instead of individual meshes\n")
	TEXT("1: Allow only the first level of HLOD clusters to be shown\n")
	TEXT("2+: Allow up to the Nth level of HLOD clusters to be shown"),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarHLODDitherPauseTime(
	TEXT("r.HLOD.DitherPauseTime"),
	0.5f,
	TEXT("HLOD dither pause time in seconds\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

ENGINE_API TAutoConsoleVariable<FString> CVarHLODDistanceOverride(
	TEXT("r.HLOD.DistanceOverride"),
	"0.0",
	TEXT("If non-zero, overrides the distance that HLOD transitions will take place for all objects at the HLOD level index, formatting is as follows:\n")
	TEXT("'r.HLOD.DistanceOverride 5000, 10000, 20000' would result in HLOD levels 0, 1 and 2 transitioning at 5000, 1000 and 20000 respectively."),
	ECVF_Scalability);

static TAutoConsoleVariable<FString> CVarHLODDistanceOverrideScale(
	TEXT("r.HLOD.DistanceOverrideScale"),
	"",
	TEXT("Scales the value in r.HLOD.DistanceOverride, Default off.\n")
	TEXT("This is an optional scale intended to allow game logic to dynamically modify without impacting scalability.\n")
);

static TAutoConsoleVariable<int32> CVarHLODForceDisableCastDynamicShadow(
	TEXT("r.HLOD.ForceDisableCastDynamicShadow"),
	0,
	TEXT("If non-zero, will set bCastDynamicShadow to false for all LODActors, regardless of the shadowing setting of their subactors."),
	ECVF_ReadOnly);

ENGINE_API TArray<float> ALODActor::HLODDistances;

#if !(UE_BUILD_SHIPPING)
static void HLODConsoleCommand(const TArray<FString>& Args, UWorld* World)
{
	if (Args.Num() == 1)
	{
		const int32 State = FCString::Atoi(*Args[0]);

		if (State == 0 || State == 1)
		{
			const bool bHLODEnabled = (State == 1) ? true : false;
			FlushRenderingCommands();
			const TArray<ULevel*>& Levels = World->GetLevels();
			for (ULevel* Level : Levels)
			{
				for (AActor* Actor : Level->Actors)
				{
					ALODActor* LODActor = Cast<ALODActor>(Actor);
					if (LODActor)
					{
						LODActor->SetActorHiddenInGame(!bHLODEnabled);
#if WITH_EDITOR
						LODActor->SetIsTemporarilyHiddenInEditor(!bHLODEnabled);
#endif // WITH_EDITOR
						LODActor->MarkComponentsRenderStateDirty();
					}
				}
			}
		}
	}
	else if (Args.Num() == 2)
	{
#if WITH_EDITOR
		if (Args[0] == "force")
		{
			const int32 ForcedLevel = FCString::Atoi(*Args[1]);

			if (ForcedLevel >= -1 && ForcedLevel < World->GetWorldSettings()->GetNumHierarchicalLODLevels())
			{
				const TArray<ULevel*>& Levels = World->GetLevels();
				for (ULevel* Level : Levels)
				{
					for (AActor* Actor : Level->Actors)
					{
						ALODActor* LODActor = Cast<ALODActor>(Actor);

						if (LODActor)
						{
							if (ForcedLevel != -1)
							{
								if (LODActor->LODLevel == ForcedLevel + 1)
								{
									LODActor->SetForcedView(true);
								}
								else
								{
									LODActor->SetHiddenFromEditorView(true, ForcedLevel + 1);
								}
							}
							else
							{
								LODActor->SetForcedView(false);
								LODActor->SetIsTemporarilyHiddenInEditor(false);
							}
						}
					}
				}
			}
		}		
#endif // WITH_EDITOR
	}
}

static FAutoConsoleCommandWithWorldAndArgs GHLODCmd(
	TEXT("r.HLOD"),
	TEXT("Single argument: 0 or 1 to Disable/Enable HLOD System\nMultiple arguments: force X where X is the HLOD level that should be forced into view"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(HLODConsoleCommand)
	);

static void ListUnbuiltHLODActors(const TArray<FString>& Args, UWorld* World)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	int32 NumUnbuilt = 0;
	for (TActorIterator<ALODActor> HLODIt(World); HLODIt; ++HLODIt)
	{
		ALODActor* Actor = *HLODIt;
		if (!Actor->IsBuilt() && Actor->HasValidLODChildren())
		{
			++NumUnbuilt;
			FString ActorPathName = Actor->GetPathName(World);
			UE_LOGF(LogHLOD, Warning, "HLOD %ls is unbuilt (HLOD level %i)", *ActorPathName, Actor->LODLevel);
		}
	}

	UE_LOGF(LogHLOD, Warning, "%d HLOD actor(s) were unbuilt", NumUnbuilt);
#endif	//  !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

static FAutoConsoleCommandWithWorldAndArgs GHLODListUnbuiltCmd(
	TEXT("r.HLOD.ListUnbuilt"),
	TEXT("Lists all unbuilt HLOD actors in the world"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(ListUnbuiltHLODActors)
);

#endif // !(UE_BUILD_SHIPPING)

//////////////////////////////////////////////////////////////////////////
// ALODActor

FAutoConsoleVariableSink ALODActor::CVarSink(FConsoleCommandDelegate::CreateStatic(&ALODActor::OnCVarsChanged));

ALODActor::ALODActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LODDrawDistance(5000)
	, bHasActorTriedToRegisterComponents(false)
{
	SetCanBeDamaged(false);

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = false;
	PrimaryActorTick.bAllowTickOnDedicatedServer = false;
	PrimaryActorTick.bTickEvenWhenPaused = true;

#if WITH_EDITORONLY_DATA
	
	bListedInSceneOutliner = false;

	NumTrianglesInSubActors = 0;
	NumTrianglesInMergedMesh = 0;
	
#endif // WITH_EDITORONLY_DATA

	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("StaticMeshComponent0"));
	SetupComponent(StaticMeshComponent);

	bNeedsDrawDistanceReset = false;
	bHasPatchedUpParent = false;
	ResetDrawDistanceTime = 0.0f;
	RootComponent = StaticMeshComponent;	
	CachedNumHLODLevels = 1;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bCachedIsBuilt = false;
	LastIsBuiltTime = 0.0;
#endif
}

void ALODActor::SetupComponent(UStaticMeshComponent* Component)
{
	// Cast shadows if any sub-actors do
	bool bCastsShadow = false;
	bool bCastsStaticShadow = false;
	bool bCastsDynamicShadow = false;

	Component->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	Component->Mobility = EComponentMobility::Static;
	Component->SetGenerateOverlapEvents(false);
	Component->CastShadow = bCastsShadow;
	Component->bCastStaticShadow = bCastsStaticShadow;
	Component->bCastDynamicShadow = bCastsDynamicShadow;
	Component->bAllowCullDistanceVolume = false;
	Component->bNeverDistanceCull = true;

	Component->MinDrawDistance = GetLODDrawDistanceWithOverride();
}

FString ALODActor::GetDetailedInfoInternal() const
{
	return StaticMeshComponent ? StaticMeshComponent->GetDetailedInfoInternal() : TEXT("No_StaticMeshComponent");
}

void ALODActor::PostLoad()
{
	Super::PostLoad();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Rebuild the InstancedStaticMeshComponents map
	ForEachComponent<UInstancedStaticMeshComponent>(false, [this](UInstancedStaticMeshComponent* ISMComponent)
	{
		FHLODInstancingKey InstancingKey;
		InstancingKey.StaticMesh = ISMComponent->GetStaticMesh();
		InstancingKey.Material = ISMComponent->GetMaterial(0);
		InstancedStaticMeshComponents.Emplace(InstancingKey, ISMComponent);
	});
#endif

	SetComponentsMinDrawDistance(LODDrawDistance, false);
	UpdateRegistrationToMatchMaximumLODLevel();

	// Force disabled dynamic shadow casting if requested from CVar
	if (CVarHLODForceDisableCastDynamicShadow.GetValueOnAnyThread() != 0)
	{
		StaticMeshComponent->bCastDynamicShadow = false;
	}

#if WITH_EDITOR
	if (bRequiresLODScreenSizeConversion)
	{
		if (TransitionScreenSize == 0.0f)
		{
			TransitionScreenSize = 1.0f;
		}
		else
		{
			const float HalfFOV = UE_PI * 0.25f;
			const float ScreenWidth = 1920.0f;
			const float ScreenHeight = 1080.0f;
			const FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);

			FBoxSphereBounds::Builder BoundsBuilder;
			ForEachComponent<UStaticMeshComponent>(false, [&BoundsBuilder](UStaticMeshComponent* SMComponent)
			{
				BoundsBuilder += SMComponent->CalcBounds(FTransform());
			});
			FBoxSphereBounds Bounds(BoundsBuilder);

			// legacy transition screen size was previously a screen AREA fraction using resolution-scaled values, so we need to convert to distance first to correctly calculate the threshold
			const float ScreenArea = TransitionScreenSize * (ScreenWidth * ScreenHeight);
			const float ScreenRadius = FMath::Sqrt(ScreenArea / UE_PI);
			const float ScreenDistance = FMath::Max(ScreenWidth / 2.0f * ProjMatrix.M[0][0], ScreenHeight / 2.0f * ProjMatrix.M[1][1]) * Bounds.SphereRadius / ScreenRadius;

			// Now convert using the query function
			TransitionScreenSize = ComputeBoundsScreenSize(FVector::ZeroVector, Bounds.SphereRadius, FVector(0.0f, 0.0f, ScreenDistance), ProjMatrix);
		}
	}

	CachedNumHLODLevels = GetLevel()->GetWorldSettings()->GetNumHierarchicalLODLevels();
#endif

#if !WITH_EDITOR
	// Invalid runtime LOD actor with null static mesh is invalid, look for a possible way to patch this up
	if (GetStaticMeshComponent() && GetStaticMeshComponent()->GetStaticMesh() == nullptr && GetStaticMeshComponent()->GetLODParentPrimitive())
		{
 			if (ALODActor* ParentLODActor = Cast<ALODActor>(GetStaticMeshComponent()->GetLODParentPrimitive()->GetOwner()))
			{
					// Make the parent HLOD
					ParentLODActor->SubActors.Remove(this);
					ParentLODActor->SubActors.Append(SubActors); // Don't register callbacks here, PostLoad should happen before PostRegisterAllComponents
					for (AActor* Actor : SubActors)
					{
						if (Actor)
						{
							Actor->SetLODParent(ParentLODActor->GetStaticMeshComponent(), ParentLODActor->GetDrawDistance());
						}
					}
  
					SubActors.Empty();
					bHasPatchedUpParent = true;
				}
			}
#endif // !WITH_EDITOR

	ParseOverrideDistancesCVar();
	UpdateOverrideTransitionDistance();
}

void ALODActor::SetComponentsMinDrawDistance(float InMinDrawDistance, bool bInMarkRenderStateDirty)
{
	float MinDrawDistance = FMath::Max(0.0f, InMinDrawDistance);

	ForEachComponent<UStaticMeshComponent>(false, [MinDrawDistance, bInMarkRenderStateDirty](UStaticMeshComponent* SMComponent)
	{
		SMComponent->MinDrawDistance = MinDrawDistance;
		if (bInMarkRenderStateDirty)
		{
			SMComponent->MarkRenderStateDirty();
		}
	});
}

/** Returns an array of distances that are used to override individual LOD actors min draw distances. */
const TArray<float>& ALODActor::GetHLODDistanceOverride()
{
	ParseOverrideDistancesCVar();
	return HLODDistances;
}

void ALODActor::UpdateOverrideTransitionDistance()
{
	const int32 NumDistances = ALODActor::HLODDistances.Num();
	// Determine correct distance index to apply to ensure combinations of different levels will work			
	const int32 DistanceIndex = [&]()
	{
		if (CachedNumHLODLevels == NumDistances)
		{
			return LODLevel - 1;
		}
		else if (CachedNumHLODLevels < NumDistances)
		{
			return (LODLevel + (NumDistances - CachedNumHLODLevels)) - 1;
		}
		else
		{
			// We've reached the end of the array, change nothing
			return (int32)INDEX_NONE;
		}
	}();

	if (DistanceIndex != INDEX_NONE)
	{
		float MinDrawDistance = (!HLODDistances.IsValidIndex(DistanceIndex) || FMath::IsNearlyZero(HLODDistances[DistanceIndex])) ? LODDrawDistance : ALODActor::HLODDistances[DistanceIndex];
		SetComponentsMinDrawDistance(MinDrawDistance, true);
	}
}

void ALODActor::ParseOverrideDistancesCVar()
{
	// Parse HLOD override distance cvar into array
	const FString DistanceOverrideValues = CVarHLODDistanceOverride.GetValueOnAnyThread();
	const FString DistanceOverrideScaleValues = CVarHLODDistanceOverrideScale.GetValueOnAnyThread();
	const TCHAR* Delimiters[] = { TEXT(","), TEXT(" ") };

	TArray<FString> Distances;
	DistanceOverrideValues.ParseIntoArray(/*out*/ Distances, Delimiters, UE_ARRAY_COUNT(Delimiters), true);

	TArray<FString> DistanceScales;
	if (!DistanceOverrideScaleValues.IsEmpty())
	{
		DistanceOverrideScaleValues.ParseIntoArray(/*out*/ DistanceScales, Delimiters, UE_ARRAY_COUNT(Delimiters), true);
	}	

	HLODDistances.Empty(Distances.Num());
	for (int32 Index = 0; Index < Distances.Num(); ++Index)
	{
		const float DistanceForThisLevel = FCString::Atof(*Distances[Index]);
		float DistanceScaleForThisLevel = 1.f;
		if (DistanceScales.IsValidIndex(Index))
		{
			DistanceScaleForThisLevel = FCString::Atof(*DistanceScales[Index]);
		}
		HLODDistances.Add(DistanceForThisLevel * DistanceScaleForThisLevel);
	}
}

float ALODActor::GetLODDrawDistanceWithOverride() const
{
	const int32 NumDistances = ALODActor::HLODDistances.Num();
	const int32 DistanceIndex = [&]()
	{
		if(CachedNumHLODLevels <= NumDistances)
		{
			return (LODLevel + (NumDistances - CachedNumHLODLevels)) - 1;
		}
		else
		{
			// We've reached the end of the array, change nothing
			return (int32)INDEX_NONE;
		}
	}();

	const float HLODDistanceOverride = (!ALODActor::HLODDistances.IsValidIndex(DistanceIndex)) ? 0.0f : ALODActor::HLODDistances[DistanceIndex];
	// Determine desired HLOD state
	float MinDrawDistance = LODDrawDistance;
	const bool bIsOverridingHLODDistance = HLODDistanceOverride != 0.0f;
	if(bIsOverridingHLODDistance)
	{
		MinDrawDistance = HLODDistanceOverride;
	}

	return MinDrawDistance;
}

void ALODActor::Tick(float DeltaSeconds)
{
	AActor::Tick(DeltaSeconds);
	if (bNeedsDrawDistanceReset)
	{		
		if (ResetDrawDistanceTime > CVarHLODDitherPauseTime.GetValueOnAnyThread())
		{
			// Determine desired HLOD state
			float MinDrawDistance = GetLODDrawDistanceWithOverride();

			SetComponentsMinDrawDistance(MinDrawDistance, true);
			bNeedsDrawDistanceReset = false;
			ResetDrawDistanceTime = 0.0f;
			PrimaryActorTick.SetTickFunctionEnable(false);
		}
		else
        {
			const float CurrentTimeDilation = FMath::Max(GetActorTimeDilation(), UE_SMALL_NUMBER);
			ResetDrawDistanceTime += DeltaSeconds / CurrentTimeDilation;
        }
	}
}

void ALODActor::SetLODParent(UPrimitiveComponent* InLODParent, float InParentDrawDistance, bool bApplyToImposters)
{
	if (bApplyToImposters)
	{
		AActor::SetLODParent(InLODParent, InParentDrawDistance);
	}
	else
	{
		if(InLODParent)
		{
			InLODParent->MinDrawDistance = InParentDrawDistance;
			InLODParent->MarkRenderStateDirty();
		}

		StaticMeshComponent->SetLODParentPrimitive(InLODParent);
	}
}

void ALODActor::PauseDitherTransition()
{
	SetComponentsMinDrawDistance(0.0f, true);
	bNeedsDrawDistanceReset = true;
	ResetDrawDistanceTime = 0.0f;
}

void ALODActor::StartDitherTransition()
{
	PrimaryActorTick.SetTickFunctionEnable(bNeedsDrawDistanceReset);
}

void ALODActor::UpdateRegistrationToMatchMaximumLODLevel()
{
	// Determine if we can show this HLOD level and allow or prevent the SMC from being registered
	// This doesn't save the memory of the static mesh or lowest mip levels, but it prevents the proxy from being created
	// or high mip textures from being streamed in
	const int32 MaximumAllowedHLODLevel = GMaximumAllowedHLODLevel;
	const bool bAllowShowingThisLevel = (MaximumAllowedHLODLevel < 0) || (LODLevel <= MaximumAllowedHLODLevel);

	check(StaticMeshComponent);
	if (StaticMeshComponent->bAutoRegister != bAllowShowingThisLevel)
	{
		StaticMeshComponent->bAutoRegister = bAllowShowingThisLevel;

		if (!bAllowShowingThisLevel && StaticMeshComponent->IsRegistered())
		{
			ensure(bHasActorTriedToRegisterComponents);
			UnregisterMeshComponents();
		}
		else if (bAllowShowingThisLevel && !StaticMeshComponent->IsRegistered())
		{
			// We should only register components if the actor had already tried to register before (otherwise it'll be taken care of in the normal flow)
			if (bHasActorTriedToRegisterComponents)
			{
				RegisterMeshComponents();
			}
		}
	}
}

void ALODActor::PostRegisterAllComponents() 
{
	Super::PostRegisterAllComponents();

	bHasActorTriedToRegisterComponents = true;

	// In case we patched up the subactors to a parent LOD actor, we can unregister this component as it's not used anymore
	if (bHasPatchedUpParent)
	{
		UnregisterMeshComponents();
	}

	if( UWorld* World = GetWorld(); World && World->IsGameWorld())
	{
		for (TObjectPtr<AActor>& ActorPtr : SubActors)
		{
			if (AActor* Actor = ActorPtr.Get())
			{
				Actor->OnEndPlay.AddUniqueDynamic(this, &ALODActor::OnSubActorEndPlay);
			}
		}
	}

#if WITH_EDITOR
	if(!GetWorld()->IsPlayInEditor())
	{
		// Clean up sub actor if assets were delete manually
		CleanSubActorArray();

		UpdateSubActorLODParents();
	}
#endif
}

void ALODActor::OnSubActorEndPlay(AActor* Actor, EEndPlayReason::Type Reason)
{
	// Other end play reasons will also be removing this actor from play so we don't need to touch our array
	if (Reason == EEndPlayReason::Destroyed)
	{
		SubActors.RemoveSwap(Actor);
	}
}

void ALODActor::RegisterMeshComponents()
{
	ForEachComponent<UStaticMeshComponent>(false, [](UStaticMeshComponent* SMComponent)
	{
		if (!SMComponent->IsRegistered())
		{
			SMComponent->RegisterComponent();
		}
	});
}

void ALODActor::UnregisterMeshComponents()
{
	ForEachComponent<UStaticMeshComponent>(false, [](UStaticMeshComponent* SMComponent)
	{
		if (SMComponent->IsRegistered())
		{
			SMComponent->UnregisterComponent();
		}
		else
		{
			SMComponent->bAutoRegister = false;
		}
	});
}

void ALODActor::SetDrawDistance(float InDistance)
{
	LODDrawDistance = InDistance;
	SetComponentsMinDrawDistance(GetLODDrawDistanceWithOverride(), false);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR

const bool ALODActor::IsBuilt(bool bInForce/*=false*/) const
{
	// Ignore if actor is being destroyed
	if (IsPendingKillPending())
	{
		return true;
	}

	auto IsBuiltHelper = [this]()
	{
		// Ensure all subactors are linked to a LOD static mesh component.
		for (AActor* SubActor : SubActors)
		{
			if(SubActor)
			{
				UStaticMeshComponent* SMComponent = SubActor->FindComponentByClass<UStaticMeshComponent>();
				UStaticMeshComponent* LODComponent = SMComponent ? Cast<UStaticMeshComponent>(SMComponent->GetLODParentPrimitive()) : nullptr;
				if (LODComponent == nullptr || LODComponent->GetOwner() != this || LODComponent->GetStaticMesh() == nullptr)
				{
					return false;
				}
			}
		}

		// No proxy mesh
		if (StaticMeshComponent->GetStaticMesh() && Proxy == nullptr)
		{
			return false;
		}

		// Mismatched key
		if (Proxy != nullptr && !Proxy->ContainsDataForActor(this))
		{
			return false;
		}

		// Unbuilt children
		for (AActor* SubActor : SubActors)
		{
			if (ALODActor* SubLODActor = Cast<ALODActor>(SubActor))
			{
				if (!SubLODActor->IsBuilt(true))
				{
					return false;
				}
			}
		}

		return true;
	};

	const double CurrentTime = FPlatformTime::Seconds();
	if (bInForce || (CurrentTime - LastIsBuiltTime > 0.5))
	{
		bCachedIsBuilt = IsBuiltHelper();
		LastIsBuiltTime = CurrentTime;
	}

	return bCachedIsBuilt;
}
#endif

const bool ALODActor::HasValidLODChildren() const
{
	if (SubActors.Num() > 0)
	{
		for (const AActor* Actor : SubActors)
		{
			if (Actor)
			{
				// Retrieve contained components for all sub-actors
				TArray<const UPrimitiveComponent*> Components;
				Actor->GetComponents(Components);

				// Try and find the parent primitive(s) and see if it matches this LODActor's static mesh component
				for (const UPrimitiveComponent* PrimitiveComponent : Components)
				{
					if (const UPrimitiveComponent* ParentPrimitiveComponent = PrimitiveComponent ? PrimitiveComponent->GetLODParentPrimitive() : nullptr)
					{
						if (ParentPrimitiveComponent && GetComponents().Contains(const_cast<UPrimitiveComponent*>(ParentPrimitiveComponent)))
						{
							return true;
						}
					}
				}
			}
		}		
	}

	return false;
}

#if WITH_EDITOR

void ALODActor::ForceUnbuilt()
{
	Key = NAME_None;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bCachedIsBuilt = false;
	LastIsBuiltTime = 0.0;
#endif
}

bool ALODActor::Modify(bool bAlwaysMarkDirty /*= true*/)
{
	// Avoid marking the package as dirty if this LODActor was spawned from an HLODDesc and is transient
	bAlwaysMarkDirty = bAlwaysMarkDirty && !WasBuiltFromHLODDesc();
	return Super::Modify(bAlwaysMarkDirty);
}

void ALODActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	// Flush all pending rendering commands.
	FlushRenderingCommands();
}

void ALODActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	FName PropertyName = PropertyThatChanged != NULL ? PropertyThatChanged->GetFName() : NAME_None;
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ALODActor, bOverrideTransitionScreenSize) || PropertyName == GET_MEMBER_NAME_CHECKED(ALODActor, TransitionScreenSize))
	{
		float CalculateScreenSize = 0.0f;

		if (bOverrideTransitionScreenSize)
		{
			CalculateScreenSize = TransitionScreenSize;
		}
		else
		{
			UWorld* World = GetWorld();
			check(World != nullptr);
			const TArray<struct FHierarchicalSimplification>& HierarchicalLODSetups = World->GetWorldSettings()->GetHierarchicalLODSetup();
			checkf(HierarchicalLODSetups.IsValidIndex(LODLevel - 1), TEXT("Out of range HLOD level (%i) found in LODActor (%s)"), LODLevel - 1, *GetName());
			CalculateScreenSize = HierarchicalLODSetups[LODLevel - 1].TransitionScreenSize;
		}

		RecalculateDrawingDistance(CalculateScreenSize);
	}

	UpdateRegistrationToMatchMaximumLODLevel();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ALODActor::CheckForErrors()
{
	FMessageLog MapCheck("MapCheck");

	// Only check when this is not a preview actor and actually has a static mesh	
	Super::CheckForErrors();
	if (!StaticMeshComponent)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
		MapCheck.Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_StaticMeshComponent", "{ActorName} : LODActor has no StaticMeshComponent. Please rebuild HLODs for this level."), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::StaticMeshComponent));
	}

	if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh() == nullptr)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidLODActorMissingMesh", "{ActorName} : LODActor has no static mesh. Please rebuild HLODs for this level."), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::LODActorMissingStaticMesh));
	}

	if (SubActors.Num() == 0)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidLODActorEmptyActor", "{ActorName} : No sub actors are assigned. Please rebuild HLODs for this level."), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::LODActorNoActorFound));
	}
	else
	{
		for (AActor* Actor : SubActors)
		{
			// see if it's null, if so it is not good
			if (Actor == nullptr)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ActorName"), FText::FromString(GetPathName()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidLODActorNullActor", "{ActorName} : Actor is missing. The actor might have been removed. Please rebuild HLODs for this level."), Arguments)))
					->AddToken(FMapErrorToken::Create(FMapErrors::LODActorMissingActor));
			}
		}
	}
}

void ALODActor::EditorApplyTranslation(const FVector& DeltaTranslation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
}

void ALODActor::EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
}

void ALODActor::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
}

void ALODActor::EditorApplyMirror(const FVector& MirrorScale, const FVector& PivotLocation)
{
}

void ALODActor::AddSubActor(AActor* InActor)
{
	AddSubActors({ InActor });
}

void ALODActor::AddSubActors(const TArray<AActor*>& InActors)
{
	if (UWorld* World = GetWorld(); World && World->IsGameWorld())
	{
		for (AActor* Actor : InActors)
		{
			if (Actor)
			{
				Actor->OnEndPlay.AddDynamic(this, &ALODActor::OnSubActorEndPlay);
			}
		}
	}
	SubActors.Append(InActors);

	float LODDrawDistanceWithOverride = GetLODDrawDistanceWithOverride();
	TArray<UStaticMeshComponent*> StaticMeshComponents;

	for(AActor* Actor : InActors)
	{
		check(Actor != this);
		UStaticMeshComponent* LODComponent = GetOrCreateLODComponentForActor(Actor);
		Actor->SetLODParent(LODComponent, LODDrawDistanceWithOverride);

		// Adding number of triangles
		ALODActor* LODActor = Cast<ALODActor>(Actor);
		if (!LODActor)
		{
			StaticMeshComponents.Reset();
			Actor->GetComponents(StaticMeshComponents);

			for (UStaticMeshComponent* Component : StaticMeshComponents)
			{
				const UStaticMesh* StaticMesh = (Component) ? ToRawPtr(Component->GetStaticMesh()) : nullptr;
				if (StaticMesh && StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->LODResources.Num() > 0)
				{
					NumTrianglesInSubActors += StaticMesh->GetRenderData()->LODResources[0].GetNumTriangles();
				}
				Component->MarkRenderStateDirty();
			}
		}
		else
		{
			NumTrianglesInSubActors += LODActor->GetNumTrianglesInSubActors();
		}
	}

	// Reset the shadowing flags and determine them according to our current sub actors
	DetermineShadowingFlags();
}

const bool ALODActor::RemoveSubActor(AActor* InActor)
{
	if ((InActor != nullptr) && SubActors.Contains(InActor))
	{
		InActor->OnEndPlay.RemoveAll(this);
		SubActors.Remove(InActor);
		InActor->SetLODParent(nullptr, 0);

		// Deducting number of triangles
		if (!InActor->IsA<ALODActor>())
		{
			TArray<UStaticMeshComponent*> StaticMeshComponents;
			InActor->GetComponents(StaticMeshComponents);
			for (UStaticMeshComponent* Component : StaticMeshComponents)
			{
				const UStaticMesh* StaticMesh = (Component) ? ToRawPtr(Component->GetStaticMesh()) : nullptr;
				if (StaticMesh && StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->LODResources.Num() > 0)
				{
					NumTrianglesInSubActors -= StaticMesh->GetRenderData()->LODResources[0].GetNumTriangles();
				}

				Component->MarkRenderStateDirty();
			}
		}
		else
		{
			ALODActor* LODActor = Cast<ALODActor>(InActor);
			NumTrianglesInSubActors -= LODActor->GetNumTrianglesInSubActors();
		}

		if (StaticMeshComponent)
		{
			StaticMeshComponent->MarkRenderStateDirty();
		}	
				
		// In case the user removes an actor while the HLOD system is force viewing one LOD level
		InActor->SetIsTemporarilyHiddenInEditor(false);

		// Reset the shadowing flags and determine them according to our current sub actors
		DetermineShadowingFlags();
				
		return true;
	}

	return false;
}

void ALODActor::DetermineShadowingFlags()
{
	// Cast shadows if any sub-actors do
	ForEachComponent<UStaticMeshComponent>(false, [=](UStaticMeshComponent* SMComponent)
	{
		SMComponent->CastShadow = false;
		SMComponent->bCastStaticShadow = false;
		SMComponent->bCastDynamicShadow = false;
		SMComponent->bCastFarShadow = false;
		SMComponent->MarkRenderStateDirty();
	});

	for (AActor* Actor : SubActors)
	{
		if (Actor)
		{
			UStaticMeshComponent* LODComponent = GetLODComponentForActor(Actor);
			Actor->ForEachComponent<UStaticMeshComponent>(false, [&](UStaticMeshComponent* SMComponent)
			{
				LODComponent->CastShadow |= SMComponent->CastShadow;
				LODComponent->bCastStaticShadow |= SMComponent->bCastStaticShadow;
				LODComponent->bCastDynamicShadow |= SMComponent->bCastDynamicShadow;
				LODComponent->bCastFarShadow |= SMComponent->bCastFarShadow;
			});
		}
	}
}

const bool ALODActor::HasValidSubActors() const
{
	TArray<UStaticMeshComponent*> Components;
	UHLODProxy::ExtractStaticMeshComponentsFromLODActor(this, Components);

	UStaticMeshComponent** ValidComponent = Components.FindByPredicate([&](const UStaticMeshComponent* Component)
				{
#if WITH_EDITOR
		return !Component->bHiddenInGame && Component->GetStaticMesh() != nullptr && Component->ShouldGenerateAutoLOD(LODLevel - 1);
#else
		return true;
#endif
	});

	return ValidComponent != nullptr;
}

const bool ALODActor::HasAnySubActors() const
{
	return (SubActors.Num() != 0);
}

void ALODActor::ToggleForceView()
{
	// Toggle the forced viewing of this LODActor, set drawing distance to 0.0f or LODDrawDistance
	SetComponentsMinDrawDistance((StaticMeshComponent->MinDrawDistance == 0.0f) ? LODDrawDistance : 0.0f, true);
}

void ALODActor::SetForcedView(const bool InState)
{
	// Set forced viewing state of this LODActor, set drawing distance to 0.0f or LODDrawDistance
	SetComponentsMinDrawDistance(InState ? 0.0f : LODDrawDistance, true);
}

void ALODActor::SetHiddenFromEditorView(const bool InState, const int32 ForceLODLevel )
{
	// If we are also subactor for a higher LOD level or this actor belongs to a higher HLOD level than is being forced hide the actor
	if (GetStaticMeshComponent()->GetLODParentPrimitive() || LODLevel > ForceLODLevel )
	{
		SetIsTemporarilyHiddenInEditor(InState);

		for (AActor* Actor : SubActors)
		{
			if (Actor)
			{
				// If this actor belongs to a lower HLOD level that is being forced hide the sub-actors
				if (LODLevel < ForceLODLevel)
				{
					Actor->SetIsTemporarilyHiddenInEditor(InState);
				}

				// Toggle/set the LOD parent to nullptr or this
				if (InState)
				{
					Actor->SetLODParent(nullptr, 0.0f);
				}
				else
				{
					Actor->SetLODParent(GetLODComponentForActor(Actor), LODDrawDistance);
				}
			}
		}
	}

	StaticMeshComponent->MarkRenderStateDirty();
}

const uint32 ALODActor::GetNumTrianglesInSubActors()
{
	return NumTrianglesInSubActors;
}

const uint32 ALODActor::GetNumTrianglesInMergedMesh()
{
	return NumTrianglesInMergedMesh;
}

void ALODActor::SetStaticMesh(class UStaticMesh* InStaticMesh)
{
	if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh() != InStaticMesh)
	{
		// Temporarily switch to movable in order to update the static mesh...
		StaticMeshComponent->SetMobility(EComponentMobility::Movable);
		StaticMeshComponent->SetStaticMesh(InStaticMesh);
		StaticMeshComponent->SetMobility(EComponentMobility::Static);

		ensure(StaticMeshComponent->GetStaticMesh() == InStaticMesh);
		if (InStaticMesh && InStaticMesh->GetRenderData() && InStaticMesh->GetRenderData()->LODResources.Num() > 0)
		{
			NumTrianglesInMergedMesh = InStaticMesh->GetRenderData()->LODResources[0].GetNumTriangles();
		}
	}
}

void ALODActor::ClearInstances()
{
	ForEachComponent<UInstancedStaticMeshComponent>(false, [this](UInstancedStaticMeshComponent* ISMComponent)
	{
		ISMComponent->ClearInstances();
	});
}

void ALODActor::AddInstances(const UStaticMesh* InStaticMesh, const UMaterialInterface* InMaterial, const TArray<FTransform>& InTransforms, const TArray<FCustomPrimitiveData>& InCustomPrimitiveData)
{
	check(InStaticMesh);
	check(InMaterial);
	check(!InTransforms.IsEmpty());
	check(InCustomPrimitiveData.IsEmpty() || InCustomPrimitiveData.Num() == InTransforms.Num());

	UInstancedStaticMeshComponent* Component = GetOrCreateISMComponent(FHLODInstancingKey(InStaticMesh, InMaterial));

	// Adjust number of custom data floats
	for (const FCustomPrimitiveData& CustomPrimData : InCustomPrimitiveData)
	{
		Component->NumCustomDataFloats = FMath::Max(Component->NumCustomDataFloats, CustomPrimData.Data.Num());
	}

	Component->PreAllocateInstancesMemory(InTransforms.Num());
	
	// Add all new instances
	for (int32 i = 0; i < InTransforms.Num(); i++)
	{
		int32 InstanceIndex = Component->AddInstance(InTransforms[i], /*bWorldSpace*/true);

		// Assign per instance custom data, if any
		if (!InCustomPrimitiveData.IsEmpty())
		{
			const FCustomPrimitiveData& CustomPrimData = InCustomPrimitiveData[i];
			Component->SetCustomData(InstanceIndex, CustomPrimData.Data);
		}
	}

	// Ensure parenting is up to date and take into account the newly created component.
	UpdateSubActorLODParents();
}

void ALODActor::AddInstances(const UStaticMesh* InStaticMesh, const UMaterialInterface* InMaterial, const TArray<FTransform>& InTransforms)
{
	AddInstances(InStaticMesh, InMaterial, InTransforms, {});
}

void ALODActor::UpdateSubActorLODParents()
{
	for (AActor* Actor : SubActors)
	{	
		if (Actor && !Actor->IsPendingKillPending())
		{
			UStaticMeshComponent* LODComponent = GetLODComponentForActor(Actor);
			Actor->SetLODParent(LODComponent, LODComponent->MinDrawDistance);
		}
	}
}

void ALODActor::CleanSubActorArray()
{
	for (int32 SubActorIndex = 0; SubActorIndex < SubActors.Num(); ++SubActorIndex)
	{
		AActor* Actor = SubActors[SubActorIndex];
		if (!IsValid(Actor))
		{
			SubActors.RemoveAtSwap(SubActorIndex);
			SubActorIndex--;
		}
	}
}

void ALODActor::RecalculateDrawingDistance(const float InTransitionScreenSize)
{
	// At the moment this assumes a fixed field of view of 90 degrees (horizontal and vertical axes)
	static const float FOVRad = 90.0f * (float)UE_PI / 360.0f;
	static const FMatrix ProjectionMatrix = FPerspectiveMatrix(FOVRad, 1920, 1080, 0.01f);
	FBoxSphereBounds::Builder BoundsBuilder;
	ForEachComponent<UStaticMeshComponent>(false, [&BoundsBuilder](UStaticMeshComponent* SMComponent)
	{
		BoundsBuilder += SMComponent->CalcBounds(FTransform());
	});
	FBoxSphereBounds Bounds(BoundsBuilder);

	float DrawDistance = ComputeBoundsDrawDistance(InTransitionScreenSize, Bounds.SphereRadius, ProjectionMatrix);
	SetDrawDistance(DrawDistance);

	UpdateSubActorLODParents();
}

bool ALODActor::UpdateProxyDesc()
{
	if (ProxyDesc)
	{
		return ProxyDesc->UpdateFromLODActor(this);
	}

	return false;
}

bool ALODActor::ShouldUseInstancing(const UStaticMeshComponent* InComponent)
{
	check(InComponent);

	if (InComponent->HLODBatchingPolicy != EHLODBatchingPolicy::Instancing)
	{
		return false;
	}

	const UStaticMesh* StaticMesh = InComponent->GetStaticMesh();
	if (StaticMesh == nullptr)
	{
		return false;
	}

	if (StaticMesh->GetNumLODs() == 0)
	{
		return false;
	}

	const int32 LODIndex = StaticMesh->GetNumLODs() - 1;
	const FStaticMeshLODResources& StaticMeshLOD = StaticMesh->GetRenderData()->LODResources[LODIndex];

	if (StaticMeshLOD.Sections.Num() != 1)
	{
		return false;
	}

	return true;
}

static UStaticMesh* GetImposterStaticMesh(const UStaticMeshComponent* InComponent)
{
	UStaticMesh* ImposterStaticMesh = nullptr;

	FString ImposterStaticMeshName = InComponent->GetStaticMesh()->GetPackage()->GetPathName() + TEXT("_ImposterMesh");
	ImposterStaticMesh = LoadObject<UStaticMesh>(nullptr, *ImposterStaticMeshName, nullptr, LOAD_Quiet | LOAD_NoWarn);

	return ImposterStaticMesh;
}

static UMaterialInterface* GetImposterMaterial(const UStaticMeshComponent* InComponent)
{
	UMaterialInterface* ImposterMaterial = nullptr;

	UStaticMesh* StaticMesh = InComponent->GetStaticMesh();
	check(StaticMesh);

	// Retrieve imposter LOD mesh and material
	const int32 LODIndex = StaticMesh->GetNumLODs() - 1;

	// Retrieve the sections, we're expect 1 for imposter meshes
	const FStaticMeshSectionArray& Sections = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections;
	if (Sections.Num() == 1)
	{
		// Retrieve material for this section
		ImposterMaterial = InComponent->GetMaterial(Sections[0].MaterialIndex);
	}
	else
	{
		UE_LOGF(LogHLOD, Warning, "Imposter's static mesh %ls has multiple mesh sections for its lowest LOD", *StaticMesh->GetName());
	}

	return ImposterMaterial;
}

static FHLODInstancingKey GetInstancingKey(const AActor* InActor, int32 InLODLevel)
{
	FHLODInstancingKey InstancingKey;

	TArray<UStaticMeshComponent*> Components;
	InActor->GetComponents(Components);
	Components.RemoveAll([&](UStaticMeshComponent* Val)
	{
		return Val->GetStaticMesh() == nullptr || !Val->ShouldGenerateAutoLOD(InLODLevel - 1);
	});

	if (Components.Num() == 1 && ALODActor::ShouldUseInstancing(Components[0]))
	{
		InstancingKey.StaticMesh = GetImposterStaticMesh(Components[0]);
		InstancingKey.Material = GetImposterMaterial(Components[0]);
	}

	return InstancingKey;
}

UInstancedStaticMeshComponent* ALODActor::GetISMComponent(const FHLODInstancingKey& InstancingKey) const
{
	return InstancedStaticMeshComponents.FindRef(InstancingKey);
}

UInstancedStaticMeshComponent* ALODActor::GetOrCreateISMComponent(const FHLODInstancingKey& InstancingKey)
{
	UInstancedStaticMeshComponent* LODComponent = GetISMComponent(InstancingKey);
	if (LODComponent == nullptr)
	{
		LODComponent = NewObject<UInstancedStaticMeshComponent>(this);
		SetupComponent(LODComponent);
		AddInstanceComponent(LODComponent);
		LODComponent->SetupAttachment(GetRootComponent());
		
		LODComponent->SetStaticMesh(const_cast<UStaticMesh*>(ToRawPtr(InstancingKey.StaticMesh)));
		LODComponent->SetMaterial(0, const_cast<UMaterialInterface*>(ToRawPtr(InstancingKey.Material)));

		if (StaticMeshComponent->IsRegistered())
		{
			LODComponent->RegisterComponent();
		}
		else
		{
			LODComponent->bAutoRegister = StaticMeshComponent->bAutoRegister;
		}

		InstancedStaticMeshComponents.Emplace(InstancingKey, LODComponent);
	}
	
	check(LODComponent->GetStaticMesh() == InstancingKey.StaticMesh);
	check(LODComponent->GetMaterial(0) == InstancingKey.Material);

	return LODComponent;
}

UStaticMeshComponent* ALODActor::GetLODComponentForActor(const AActor* InActor, bool bInFallbackToDefault) const
{
	UStaticMeshComponent* LODComponent = StaticMeshComponent;

	if (!InActor->IsA<ALODActor>())
	{
		FHLODInstancingKey InstancingKey = GetInstancingKey(InActor, LODLevel);
		if (InstancingKey.IsValid())
		{
			LODComponent = GetISMComponent(InstancingKey);
			if (LODComponent == nullptr && bInFallbackToDefault)
			{
				// Needs to be rebuilt... fallback to default component
				LODComponent = StaticMeshComponent;
			}
		}
	}

	return LODComponent;
}

UStaticMeshComponent* ALODActor::GetOrCreateLODComponentForActor(const AActor* InActor)
{
	UStaticMeshComponent* LODComponent = StaticMeshComponent;

	if (!InActor->IsA<ALODActor>())
	{
		FHLODInstancingKey InstancingKey = GetInstancingKey(InActor, LODLevel);
		if (InstancingKey.IsValid())
		{
			LODComponent = GetOrCreateISMComponent(InstancingKey);
		}
	}

	check(LODComponent != nullptr);
	return LODComponent;
}

#endif // WITH_EDITOR

FBox ALODActor::GetComponentsBoundingBox(bool bNonColliding, bool bIncludeFromChildActors) const
{
	FBox BoundBox = Super::GetComponentsBoundingBox(bNonColliding);

	// If BoundBox ends up to nothing create a new invalid one
	if (BoundBox.GetVolume() == 0.0f)
	{
		BoundBox = FBox(ForceInit);
	}

	if (bNonColliding)
	{
		bool bHasStaticMeshes = false;
		ForEachComponent<UStaticMeshComponent>(false, [&bHasStaticMeshes](UStaticMeshComponent* SMComponent)
		{
			bHasStaticMeshes |= SMComponent->GetStaticMesh() != nullptr;
		});

		// No valid static meshes found, use sub actors bounds instead.
		if (!bHasStaticMeshes)
		{
			FBox SMBoundBox(ForceInit);
			for (AActor* Actor : SubActors)
			{
				if (Actor)
				{
					BoundBox += Actor->GetComponentsBoundingBox(bNonColliding, bIncludeFromChildActors);
				}
			}
		}
	}

	return BoundBox;	
}

void ALODActor::OnCVarsChanged()
{
	// Initialized to MIN_int32 to make sure that we run this once at startup regardless of the CVar value (assuming it is valid)
	static int32 CachedMaximumAllowedHLODLevel = MIN_int32;
	const int32 MaximumAllowedHLODLevel = GMaximumAllowedHLODLevel;

	if (MaximumAllowedHLODLevel != CachedMaximumAllowedHLODLevel)
	{
		CachedMaximumAllowedHLODLevel = MaximumAllowedHLODLevel;

		for (ALODActor* Actor : TObjectRange<ALODActor>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
		{
			Actor->UpdateRegistrationToMatchMaximumLODLevel();
		}
	}
	
	static TArray<float> CachedDistances = HLODDistances;
	ParseOverrideDistancesCVar();

	const bool bInvalidatedCachedValues = [&]() -> bool
	{
		for (int32 Index = 0; Index < CachedDistances.Num(); ++Index)
		{
			const float CachedDistance = CachedDistances[Index];
			if (HLODDistances.IsValidIndex(Index))
			{
				const float NewDistance = HLODDistances[Index];
				if (NewDistance != CachedDistance)
				{
					return true;
				}
			}
			else
			{
				return true;
			}
		}

		return CachedDistances.Num() != HLODDistances.Num();
	}();

	if (bInvalidatedCachedValues)
	{
		CachedDistances = HLODDistances;
		const int32 NumDistances = CachedDistances.Num();
		for (ALODActor* Actor : TObjectRange<ALODActor>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
		{
			Actor->UpdateOverrideTransitionDistance();
		}
	}
}


void ALODActor::Serialize(FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE && Ar.IsSaving())
	{
		Key = UHLODProxy::GenerateKeyForActor(this);
	}
#endif

	Super::Serialize(Ar);
#if WITH_EDITOR
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	bRequiresLODScreenSizeConversion = Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::LODsUseResolutionIndependentScreenSize;

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CullDistanceRefactor_NeverCullALODActorsByDefault)
	{
		if (UStaticMeshComponent* SMComponent = GetStaticMeshComponent())
		{
			SMComponent->LDMaxDrawDistance = 0.f;
			SMComponent->bNeverDistanceCull = true;
		}
	}
#endif
}

#if WITH_EDITOR

void ALODActor::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Always rebuild key on save here.
	// We don't do this while cooking as keys rely on platform derived data which is context-dependent during cook
	if(!ObjectSaveContext.IsCooking())
	{
		const bool bMustUndoLevelTransform = false; // In the save process, the level transform is already removed
		Key = UHLODProxy::GenerateKeyForActor(this, bMustUndoLevelTransform);
	}

	// check & warn if we need building
	if(!IsBuilt(true))
	{
		UE_LOGF(LogHLOD, Log, "HLOD actor %ls in map %ls is not built. Meshes may not match.", *GetName(), *GetOutermost()->GetName());
	}
}


#endif	// #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE

