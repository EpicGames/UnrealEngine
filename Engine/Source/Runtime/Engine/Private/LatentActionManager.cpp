// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LatentActionManager.h"
#include "LatentActions.h"
#include "Stats/Stats.h"
#include "HAL/ConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LatentActionManager)

#ifndef LATENT_ACTION_PROFILING_ENABLED
#define LATENT_ACTION_PROFILING_ENABLED 0
#endif


FOnLatentActionsChanged FLatentActionManager::LatentActionsChangedDelegate;


namespace LatentActionCVars
{
	int32 GuaranteeEngineTickDelay = 0;
	static FAutoConsoleVariableRef CVarLatentActionGuaranteeEngineTickDelay(
		TEXT("LatentActions.GuaranteeNextTickDelay"),
		GuaranteeEngineTickDelay,
		TEXT("If true, latent actions delayed until next tick will guarantee the engine frame has advanced. If false, these would always run at the end of the same engine tick (default behavior prior to 5.5)."));
}

/////////////////////////////////////////////////////
// FPendingLatentAction

#if WITH_EDITOR
FString FPendingLatentAction::GetDescription() const
{
	return TEXT("Not implemented");
}
#endif

/////////////////////////////////////////////////////
// FLatentActionManager

void FLatentActionManager::AddNewAction(UObject* InActionObject, int32 UUID, FPendingLatentAction* NewAction)
{
	TSharedPtr<FObjectActions>& ObjectActions = ObjectToActionListMap.FindOrAdd(InActionObject);
	if (!ObjectActions.Get())
	{
		ObjectActions = MakeShareable(new FObjectActions());
	}
	ObjectActions->ActionList.Add(UUID, NewAction);

	LatentActionsChangedDelegate.Broadcast(InActionObject, ELatentActionChangeType::ActionsAdded);
}

void FLatentActionManager::RemoveActionsForObject(TWeakObjectPtr<UObject> InObject)
{
	FObjectActions* ObjectActions = GetActionsForObject(InObject);
	if (ObjectActions)
	{
		FWeakObjectAndActions* FoundEntry = ActionsToRemoveMap.FindByPredicate([InObject](const FWeakObjectAndActions& Entry) { return Entry.Key.HasSameIndexAndSerialNumber(InObject); });

		TSharedPtr<TArray<FUuidAndAction>> ActionToRemoveListPtr;
		if (FoundEntry)
		{
			ActionToRemoveListPtr = FoundEntry->Value;
		}
		else
		{
			ActionToRemoveListPtr = MakeShareable(new TArray<FUuidAndAction>());
			ActionsToRemoveMap.Emplace(FWeakObjectAndActions(InObject, ActionToRemoveListPtr));
		}
		ActionToRemoveListPtr->Reserve(ActionToRemoveListPtr->Num() + ObjectActions->ActionList.Num());
		for (FActionList::TConstIterator It(ObjectActions->ActionList); It; ++It)
		{
			ActionToRemoveListPtr->Add(*It);
		}
	}
}

int32 FLatentActionManager::GetNumActionsForObject(TWeakObjectPtr<UObject> InObject)
{
	FObjectActions* ObjectActions = GetActionsForObject(InObject);
	if (ObjectActions)
	{
		return ObjectActions->ActionList.Num();
	}

	return 0;
}


DECLARE_CYCLE_STAT(TEXT("Blueprint Latent Actions"), STAT_TickLatentActions, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Remove Latent Actions"), STAT_RemoveLatentActions, STATGROUP_Game);

void FLatentActionManager::BeginFrame()
{
	for (TPair<TWeakObjectPtr<UObject>, TSharedPtr<FObjectActions>>& ActionPair : ObjectToActionListMap)
	{
		FObjectActions* ObjectActions = ActionPair.Value.Get();
		check(ObjectActions);
		ObjectActions->bProcessedThisFrame = false;
	}
}

#if LATENT_ACTION_PROFILING_ENABLED
static float GLatentActionDurationLoggingThreshold = 0.005f;
static FAutoConsoleVariableRef CVarLatentActionDurationLoggingThreshold(
	TEXT("LatentActionDurationLoggingThreshold"),
	GLatentActionDurationLoggingThreshold,
	TEXT("Duration in seconds at which we will log information about what took time in FLatentActionManager::ProcessLatentActions()."),
	ECVF_Default
);

static float GLatentActionMinDurationToLog = 0.0001f;
static FAutoConsoleVariableRef CVarLatentActionMinDurationToLog(
	TEXT("LatentActionMinDurationToLog"),
	GLatentActionMinDurationToLog,
	TEXT("Min duration in seconds relevant to log when we exceed LatentActionDurationLoggingThreshold."),
	ECVF_Default
);

struct FLatentActionStat
{
	FLatentActionStat(FName InObjectName, FName InClassName, double InDuration)
		: ObjectName(InObjectName)
		, ClassName(InClassName)
		, Duration(InDuration)
	{
	}

	FName ObjectName = NAME_None;
	FName ClassName = NAME_None;
	double Duration = 0.0;
	// TODO: Is it possible to get the action name also?
};
static TArray<FLatentActionStat> GLatentActionStats;

struct FScopedLatentActionTimer
{
	FScopedLatentActionTimer(UObject* InObject)
	{
		UClass* ObjectClass = nullptr;
		if (InObject)
		{
			ObjectName = InObject->GetFName();
			ObjectClass = InObject->GetClass();
		}
		else
		{
			ObjectName = NAME_None;
		}

		ClassName = ObjectClass ? ObjectClass->GetFName() : NAME_None;

		StartTime = FPlatformTime::Seconds();
	}

	~FScopedLatentActionTimer()
	{
		const double EndTime = FPlatformTime::Seconds();
		const double Duration = EndTime - StartTime;
		GLatentActionStats.Emplace(ObjectName, ClassName, Duration);
	}

	FName ObjectName = NAME_None;
	FName ClassName = NAME_None;
	double StartTime = 0.0;
};
#endif // LATENT_ACTION_PROFILING_ENABLED

void FLatentActionManager::ProcessLatentActions(UObject* InObject, float DeltaTime)
{
#if LATENT_ACTION_PROFILING_ENABLED
	GLatentActionStats.Reset();
	const double StartTime = FPlatformTime::Seconds();
#endif // LATENT_ACTION_PROFILING_ENABLED

	if (!ActionsToRemoveMap.IsEmpty())
	{
		SCOPE_CYCLE_COUNTER(STAT_RemoveLatentActions);
		for (FActionsForObject::TIterator It(ActionsToRemoveMap); It; ++It)
		{
			auto LatentActionWork = [this, &It]()
			{
			FObjectActions* ObjectActions = GetActionsForObject(It->Key);
			TSharedPtr<TArray<FUuidAndAction>> ActionToRemoveListPtr = It->Value;
			if (ActionToRemoveListPtr.IsValid() && ObjectActions)
			{
				for (const FUuidAndAction& PendingActionToKill : *ActionToRemoveListPtr)
				{
					FPendingLatentAction* Action = PendingActionToKill.Value;
					const int32 RemovedNum = ObjectActions->ActionList.RemoveSingle(PendingActionToKill.Key, Action);
					if (RemovedNum && Action)
					{
						Action->NotifyActionAborted();
						delete Action;
					}
				}

				// Notify listeners that latent actions for this object were removed
				LatentActionsChangedDelegate.Broadcast(It->Key.Get(), ELatentActionChangeType::ActionsRemoved);
			}
			};

#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (!AutoRTFM::IsClosed())
			{
				static FName TransactionalWorkName("FLatentActionManager::ProcessLatentActions_Remove");
				UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [LatentActionWork]{
						LatentActionWork();
					});
			}
			else
#endif
			{
				LatentActionWork();
			}
		}
		ActionsToRemoveMap.Reset();
	}

	if (InObject)
	{
		if (FObjectActions* ObjectActions = GetActionsForObject(InObject))
		{
			if (!ObjectActions->bProcessedThisFrame)
			{
				SCOPE_CYCLE_COUNTER(STAT_TickLatentActions);
#if LATENT_ACTION_PROFILING_ENABLED
				FScopedLatentActionTimer Timer(InObject);
#endif // LATENT_ACTION_PROFILING_ENABLED

				TickLatentActionForObject(DeltaTime, ObjectActions->ActionList, InObject);
				ObjectActions->bProcessedThisFrame = true;
			}
		}
	}
	else if (!ObjectToActionListMap.IsEmpty())
	{
		SCOPE_CYCLE_COUNTER(STAT_TickLatentActions);
		for (FObjectToActionListMap::TIterator ObjIt(ObjectToActionListMap); ObjIt; ++ObjIt)
		{	
			auto LatentActionWork = [this, DeltaTime, &ObjIt]()
			{
			UObject* Object = ObjIt.Key().Get();
			FObjectActions* ObjectActions = ObjIt.Value().Get();
			check(ObjectActions);
			FActionList& ObjectActionList = ObjectActions->ActionList;

			if (Object)
			{
				// Tick all outstanding actions for this object
				if (!ObjectActions->bProcessedThisFrame && ObjectActionList.Num() > 0)
				{
#if LATENT_ACTION_PROFILING_ENABLED
					FScopedLatentActionTimer Timer(Object);
#endif // LATENT_ACTION_PROFILING_ENABLED
					TickLatentActionForObject(DeltaTime, ObjectActionList, Object);
					ensure(ObjectActions == ObjIt.Value().Get());
					ObjectActions->bProcessedThisFrame = true;
				}
			}
			else
			{
				// Terminate all outstanding actions for this object, which has been GCed
				for (TMultiMap<int32, FPendingLatentAction*>::TConstIterator It(ObjectActionList); It; ++It)
				{
					if (FPendingLatentAction* Action = It.Value())
					{
						Action->NotifyObjectDestroyed();
						delete Action;
					}
				}
				ObjectActionList.Reset();
			}

			// Remove the entry if there are no pending actions remaining for this object (or if the object was NULLed and cleaned up)
			if (ObjectActionList.Num() == 0)
			{
				ObjIt.RemoveCurrent();
			}
			};

#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (!AutoRTFM::IsClosed())
			{
				static FName TransactionalWorkName("FLatentActionManager::ProcessLatentActions");
				UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [LatentActionWork]{
						LatentActionWork();
					});
			}
			else
#endif
			{
				LatentActionWork();
			}
		}
	}

#if LATENT_ACTION_PROFILING_ENABLED
	const double EndTime = FPlatformTime::Seconds();
	const double Duration = EndTime - StartTime;
	if (GLatentActionDurationLoggingThreshold > 0.0 && Duration >= GLatentActionDurationLoggingThreshold)
	{

		UE_LOGF(LogScript, Warning, "%ls took longer than %f ms when updating %d latent actions!  Dumping all actions that took longer than %f ms.", ANSI_TO_TCHAR(__FUNCTION__), GLatentActionDurationLoggingThreshold * 1000.0, GLatentActionStats.Num(), GLatentActionMinDurationToLog * 1000.0);

		GLatentActionStats.Sort([](const auto& Lhs, const auto& Rhs) { return Lhs.Duration > Rhs.Duration; });

		for (int32 i = 0; i < GLatentActionStats.Num(); ++i)
		{
			if (GLatentActionStats[i].Duration > GLatentActionMinDurationToLog)
			{
				UE_LOGF(LogScript, Warning, "Class = %ls, Object = %ls, Duration = %f ms", *GLatentActionStats[i].ClassName.ToString(), *GLatentActionStats[i].ObjectName.ToString(), GLatentActionStats[i].Duration * 1000.0);
			}
		}
	}
#endif // LATENT_ACTION_PROFILING_ENABLED
}

void FLatentActionManager::TickLatentActionForObject(float DeltaTime, FActionList& ObjectActionList, UObject* InObject)
{
	// Operator on a copy of the action list since it might be modified while processing it
	using FActionElementArray = TArray<TPair<int32, class FPendingLatentAction*>, TInlineAllocator<32>>;
	FActionElementArray ItemsToRemove;
	for (const TPair<int32, class FPendingLatentAction*>& Element : ObjectActionList)
	{
		ItemsToRemove.Add(Element);
	}

	FLatentResponse Response(DeltaTime);
	for (FActionElementArray::TIterator It = ItemsToRemove.CreateIterator(); It; ++It)
	{
		FPendingLatentAction* Action = It->Value;

		Response.bRemoveAction = false;

		Action->UpdateOperation(Response);

		if (!Response.bRemoveAction)
		{
			It.RemoveCurrent();
		}
	}

	// Remove any items that were deleted
	for (const TPair<int32, class FPendingLatentAction*>& ItemPair : ItemsToRemove)
	{
		const int32 ItemIndex = ItemPair.Key;
		FPendingLatentAction* DyingAction = ItemPair.Value;
		ObjectActionList.Remove(ItemIndex, DyingAction);
		delete DyingAction;
	}

	if (ItemsToRemove.Num() > 0)
	{
		LatentActionsChangedDelegate.Broadcast(InObject, ELatentActionChangeType::ActionsRemoved);
	}

	// Trigger any pending execution links
	for (FLatentResponse::FExecutionInfo& LinkInfo : Response.LinksToExecute)
	{
		if (LinkInfo.LinkID != INDEX_NONE)
		{
			if (UObject* CallbackTarget = LinkInfo.CallbackTarget.Get())
			{
				check(CallbackTarget == InObject);

				if (UFunction* ExecutionFunction = CallbackTarget->FindFunction(LinkInfo.ExecutionFunction))
				{
					CallbackTarget->ProcessEvent(ExecutionFunction, &(LinkInfo.LinkID));
				}
				else
				{
					UE_LOGF(LogScript, Warning, "FLatentActionManager::ProcessLatentActions: Could not find latent action resume point named '%ls' on '%ls' called by '%ls'",
						*LinkInfo.ExecutionFunction.ToString(), *(CallbackTarget->GetPathName()), *(InObject->GetPathName()));
				}
			}
			else
			{
				UE_LOGF(LogScript, Warning, "FLatentActionManager::ProcessLatentActions: CallbackTarget is None.");
			}
		}
	}
}

#if WITH_EDITOR


FString FLatentActionManager::GetDescription(UObject* InObject, int32 UUID) const
{
	check(InObject);

	FString Description = *NSLOCTEXT("LatentActionManager", "NoPendingActions", "No Pending Actions").ToString();

	const FObjectActions* ObjectActions = GetActionsForObject(InObject);
	if (ObjectActions && ObjectActions->ActionList.Num() > 0)
	{	
		TArray<FPendingLatentAction*> Actions;
		ObjectActions->ActionList.MultiFind(UUID, Actions);

		const int32 PendingActions = Actions.Num();

		// See if there are pending actions
		if (PendingActions > 0)
		{
			FPendingLatentAction* PrimaryAction = Actions[0];
			FString ActionDesc = PrimaryAction->GetDescription();

			Description = (PendingActions > 1)
				? FText::Format(NSLOCTEXT("LatentActionManager", "NumPendingActionsFwd", "{0} Pending Actions: {1}"), PendingActions, FText::FromString(ActionDesc)).ToString()
				: ActionDesc;
		}
	}
	return Description;
}

void FLatentActionManager::GetActiveUUIDs(UObject* InObject, TSet<int32>& UUIDList) const
{
	check(InObject);

	const FObjectActions* ObjectActions = GetActionsForObject(InObject);
	if (ObjectActions && ObjectActions->ActionList.Num() > 0)
	{
		for (TMultiMap<int32, FPendingLatentAction*>::TConstIterator It(ObjectActions->ActionList); It; ++It)
		{
			UUIDList.Add(It.Key());
		}
	}
}

#endif

FLatentActionManager::~FLatentActionManager()
{
	for (TPair<TWeakObjectPtr<UObject>, TSharedPtr<FObjectActions>>& ObjectActionListIterator : ObjectToActionListMap)
	{
		TSharedPtr<FObjectActions>& ObjectActions = ObjectActionListIterator.Value;
		if (ObjectActions.IsValid())
		{
			for (auto& ActionIterator : ObjectActions->ActionList)
			{
				FPendingLatentAction* Action = ActionIterator.Value;
				ActionIterator.Value = nullptr;
				delete Action;
			}
			ObjectActions->ActionList.Reset();
		}
	}
}

