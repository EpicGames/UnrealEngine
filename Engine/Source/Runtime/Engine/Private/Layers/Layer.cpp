// Copyright Epic Games, Inc. All Rights Reserved.
#include "Layers/Layer.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Layer)

ULayer::ULayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LayerName( NAME_None )
	, bIsVisible( true )
{
}

void ULayer::SetLayerName(FName InLayerName)
{
	LayerName = InLayerName;
}

FName ULayer::GetLayerName() const
{
	return LayerName;
}

void ULayer::SetVisible(bool bInIsVisible)
{
	bIsVisible = bInIsVisible;
}

bool ULayer::IsVisible() const
{
	return bIsVisible;
}

void ULayer::ClearActorStats()
{
	ActorStats.Empty();
}

void ULayer::AddToStats(AActor* Actor)
{
	UClass* ActorClass = Actor->GetClass();

	bool bFoundClassStats = false;
	for (FLayerActorStats& Stats : ActorStats)
	{
		if (Stats.Type == ActorClass)
		{
			Stats.Total++;
			bFoundClassStats = true;
			break;
		}
	}

	if (!bFoundClassStats)
	{
		FLayerActorStats NewActorStats;
		NewActorStats.Total = 1;
		NewActorStats.Type = ActorClass;

		ActorStats.Add(NewActorStats);
	}
}

bool ULayer::RemoveFromStats(AActor* Actor)
{
	UClass* ActorClass = Actor->GetClass();

	bool bFoundClassStats = false;
	for (int StatsIndex = 0; StatsIndex < ActorStats.Num(); StatsIndex++)
	{
		FLayerActorStats& Stats = ActorStats[StatsIndex];

		if (Stats.Type == ActorClass)
		{
			bFoundClassStats = true;
			--Stats.Total;

			if (Stats.Total == 0)
			{
				ActorStats.RemoveAt(StatsIndex);
			}
			break;
		}
	}

	return bFoundClassStats;
}

const TArray<FLayerActorStats>& ULayer::GetActorStats() const
{
	return ActorStats;
}
