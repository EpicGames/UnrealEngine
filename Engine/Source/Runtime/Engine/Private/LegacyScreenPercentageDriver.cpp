// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyScreenPercentageDriver.h"
#include "UnrealEngine.h"
#include "Misc/ConfigCacheIni.h"
#include "DynamicResolutionState.h"
#include "Scalability.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LegacyScreenPercentageDriver)


static TAutoConsoleVariable<float> CVarScreenPercentageDefault(
	TEXT("r.ScreenPercentage.Default"), 100.f,
	TEXT(""),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<int32> CVarScreenPercentageDefaultDesktopMode(
	TEXT("r.ScreenPercentage.Default.Desktop.Mode"), int32(EScreenPercentageMode::BasedOnDisplayResolution),
	TEXT(""),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<int32> CVarScreenPercentageDefaultMobileMode(
	TEXT("r.ScreenPercentage.Default.Mobile.Mode"), int32(EScreenPercentageMode::Manual),
	TEXT(""),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<int32> CVarScreenPercentageDefaultVRMode(
	TEXT("r.ScreenPercentage.Default.VR.Mode"), int32(EScreenPercentageMode::Manual),
	TEXT(""),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<int32> CVarScreenPercentageDefaultPathTracerMode(
	TEXT("r.ScreenPercentage.Default.PathTracer.Mode"), int32(EScreenPercentageMode::Manual),
	TEXT(""),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<float> CVarScreenPercentage(
	TEXT("r.ScreenPercentage"), 0.0f,
	TEXT("To render in lower resolution and upscale for better performance (combined up with the blenable post process setting).\n")
	TEXT("70 is a good value for low aliasing and performance, can be verified with 'show TestImage'\n")
	TEXT("in percent, >0 and <=100, larger numbers are possible (supersampling) but the downsampling quality is improvable.")
	TEXT("<=0 compute the screen percentage is determined by r.ScreenPercentage.Default cvars."),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<float> CVarAutoPixelCountMultiplier(
	TEXT("r.ScreenPercentage.Auto.PixelCountMultiplier"), 1.0f,
	TEXT(""),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarScreenPercentageMinResolution(
	TEXT("r.ScreenPercentage.MinResolution"),
	0.0f,
	TEXT("Controls the absolute minimum number of rendered pixel."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarScreenPercentageMaxResolution(
	TEXT("r.ScreenPercentage.MaxResolution"),
	0.0f,
	TEXT("Controls the absolute maximum number of rendered pixel before any upscaling such that doesn't go higher than the specified 16:9 resolution ")
	TEXT("of this variable. For instance set this value to 1440 so that you are not rendering more than 2560x1440 = 3.6M pixels. ")
	TEXT("This is useful to set this on PC in your project's DefaultEditor.ini so you are not rendering more pixel on PC in PIE that you would ")
	TEXT("in average on console with your project specific dynamic resolution settings."),
	ECVF_Default);

#if !UE_BUILD_SHIPPING

void OnScreenPercentageChange(IConsoleVariable* Var)
{
	float SP = Var->GetFloat();

	if (!GEngine || !GEngine->GameViewport)
	{
		return;
	}

	// Check whether dynamic resolution is overriding r.ScreenPercentage.
	{
		FDynamicResolutionStateInfos DynResInfo;
		GEngine->GetDynamicResolutionCurrentStateInfos(/* out */ DynResInfo);

		if (DynResInfo.Status == EDynamicResolutionStatus::Enabled)
		{
			UE_LOGF(LogEngine, Display, "r.ScreenPercentage=%f is ignored because overriden by dynamic resolution settings (r.DynamicRes.OperationMode).", SP);
			return;
		}
		else if (DynResInfo.Status == EDynamicResolutionStatus::DebugForceEnabled)
		{
			UE_LOGF(LogEngine, Display, "r.ScreenPercentage=%f is ignored because overriden by dynamic resolution settings (r.DynamicRes.TestScreenPercentage).", SP);
			return;
		}
	}
}

#endif // !UE_BUILD_SHIPPING

void InitScreenPercentage()
{
#if !UE_BUILD_SHIPPING
	CVarScreenPercentage.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnScreenPercentageChange));
#endif
}

static float GetRenderingPixelCount(float Rendering169Height)
{
	return Rendering169Height * FMath::Abs(Rendering169Height) * (1920.0f / 1080.0f);
}

static float GetResolutionFraction(float ScreenPercentage)
{
	return ScreenPercentage / 100.0f;
}

FLegacyScreenPercentageDriver::FLegacyScreenPercentageDriver(
	const FSceneViewFamily& InViewFamily,
	float InGlobalResolutionFraction)
	: FLegacyScreenPercentageDriver(InViewFamily, InGlobalResolutionFraction, InGlobalResolutionFraction)
{ }

FLegacyScreenPercentageDriver::FLegacyScreenPercentageDriver(
	const FSceneViewFamily& InViewFamily,
	float InGlobalResolutionFraction,
	float InGlobalResolutionFractionUpperBound)
	: ViewFamily(InViewFamily)
	, GlobalResolutionFraction(InGlobalResolutionFraction)
	, GlobalResolutionFractionUpperBound(InGlobalResolutionFractionUpperBound)
{
	if (GlobalResolutionFraction != 1.0f)
	{
		check(ViewFamily.EngineShowFlags.ScreenPercentage);
	}
}

DynamicRenderScaling::TMap<float> FLegacyScreenPercentageDriver::GetResolutionFractionsUpperBound() const
{
	DynamicRenderScaling::TMap<float> UpperBounds;
	UpperBounds.SetAll(1.0f);

	if (ViewFamily.EngineShowFlags.ScreenPercentage)
	{
		UpperBounds[GDynamicPrimaryResolutionFraction] = FMath::Clamp(
			GlobalResolutionFractionUpperBound,
			ISceneViewFamilyScreenPercentage::kMinResolutionFraction,
			ISceneViewFamilyScreenPercentage::kMaxResolutionFraction);
	}

	return UpperBounds;
}

DynamicRenderScaling::TMap<float> FLegacyScreenPercentageDriver::GetResolutionFractions_RenderThread() const
{
	check(IsInParallelRenderingThread());

	DynamicRenderScaling::TMap<float> ResolutionFractions;
	ResolutionFractions.SetAll(1.0f);

	if (ViewFamily.EngineShowFlags.ScreenPercentage)
	{
		ResolutionFractions[GDynamicPrimaryResolutionFraction] = FMath::Clamp(
			GlobalResolutionFraction,
			ISceneViewFamilyScreenPercentage::kMinResolutionFraction,
			ISceneViewFamilyScreenPercentage::kMaxResolutionFraction);
	}

	return ResolutionFractions;
}

ISceneViewFamilyScreenPercentage* FLegacyScreenPercentageDriver::Fork_GameThread(
	const FSceneViewFamily& ForkedViewFamily) const
{
	check(IsInGameThread());

	return new FLegacyScreenPercentageDriver(
		ForkedViewFamily, GlobalResolutionFraction, GlobalResolutionFractionUpperBound);
}

// static
float FLegacyScreenPercentageDriver::GetCVarResolutionFraction()
{
	check(IsInGameThread());
	static const auto ScreenPercentageCVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.ScreenPercentage"));

	float GlobalFraction = GetResolutionFraction(CVarScreenPercentage.GetValueOnGameThread());
	if (GlobalFraction <= GetResolutionFraction(Scalability::MinResolutionScale))
	{
		GlobalFraction = 1.0f;
	}

	return GlobalFraction;
}

FStaticResolutionFractionHeuristic::FStaticResolutionFractionHeuristic(const FEngineShowFlags& EngineShowFlags)
{
	Settings.bAllowDisplayBasedScreenPercentageMode = (EngineShowFlags.StereoRendering == 0) && (EngineShowFlags.VREditing == 0);
}

#if WITH_EDITOR
// static
bool FStaticResolutionFractionHeuristic::FUserSettings::EditorOverridePIESettings()
{
	if (!GIsEditor)
	{
		return false;
	}

	static auto CVarEditorViewportOverrideGameScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.OverridePIEScreenPercentage"));
	if (!CVarEditorViewportOverrideGameScreenPercentage)
	{
		return false;
	}

	return CVarEditorViewportOverrideGameScreenPercentage->GetInt() != 0;
}
#endif

void FStaticResolutionFractionHeuristic::FUserSettings::PullRunTimeRenderingSettings(EViewStatusForScreenPercentage ViewStatus)
{
	float GlobalResolutionFractionOverride = GetResolutionFraction(CVarScreenPercentage.GetValueOnGameThread());
	MinRenderingResolution = CVarScreenPercentageMinResolution.GetValueOnGameThread();
	MaxRenderingResolution = CVarScreenPercentageMaxResolution.GetValueOnGameThread();

	if (GlobalResolutionFractionOverride > 0.0)
	{
		Mode = EScreenPercentageMode::Manual;
		GlobalResolutionFraction = GlobalResolutionFractionOverride;
		AutoPixelCountMultiplier = CVarAutoPixelCountMultiplier.GetValueOnGameThread();
	}
#if WITH_EDITOR
	else if (FStaticResolutionFractionHeuristic::FUserSettings::EditorOverridePIESettings())
	{
		PullEditorRenderingSettings(ViewStatus);
	}
#endif
	else
	{
		GlobalResolutionFraction = GetResolutionFraction(FMath::Max(CVarScreenPercentageDefault.GetValueOnGameThread(), Scalability::MinResolutionScale));

		if (ViewStatus == EViewStatusForScreenPercentage::PathTracer)
		{
			Mode = EScreenPercentageMode(FMath::Clamp(CVarScreenPercentageDefaultPathTracerMode.GetValueOnGameThread(), 0, 2));
		}
		else if (ViewStatus == EViewStatusForScreenPercentage::VR)
		{
			Mode = EScreenPercentageMode(FMath::Clamp(CVarScreenPercentageDefaultVRMode.GetValueOnGameThread(), 0, 2));
		}
		else if (ViewStatus == EViewStatusForScreenPercentage::Mobile)
		{
			Mode = EScreenPercentageMode(FMath::Clamp(CVarScreenPercentageDefaultMobileMode.GetValueOnGameThread(), 0, 2));
		}
		else if (ViewStatus == EViewStatusForScreenPercentage::Desktop)
		{
			Mode = EScreenPercentageMode(FMath::Clamp(CVarScreenPercentageDefaultDesktopMode.GetValueOnGameThread(), 0, 2));
		}
		else
		{
			unimplemented();
		}
	}
}

void FStaticResolutionFractionHeuristic::FUserSettings::PullRunTimeRenderingSettings()
{
	Mode = EScreenPercentageMode::Manual;
	GlobalResolutionFraction = GetResolutionFraction(CVarScreenPercentage.GetValueOnGameThread());
	MinRenderingResolution = CVarScreenPercentageMinResolution.GetValueOnGameThread();
	MaxRenderingResolution = CVarScreenPercentageMaxResolution.GetValueOnGameThread();
	AutoPixelCountMultiplier = CVarAutoPixelCountMultiplier.GetValueOnGameThread();

	if (GlobalResolutionFraction <= 0.0)
	{
		GlobalResolutionFraction = 1.0f;
		Mode = EScreenPercentageMode::Manual;
	}
}

void FStaticResolutionFractionHeuristic::FUserSettings::PullEditorRenderingSettings(EViewStatusForScreenPercentage ViewStatus)
#if WITH_EDITOR
{
	static auto CVarEditorViewportDefaultScreenPercentageRealTimeMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.RealTime"));
	static auto CVarEditorViewportDefaultScreenPercentageMobileMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.Mobile"));
	static auto CVarEditorViewportDefaultScreenPercentageVRMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.VR"));
	static auto CVarEditorViewportDefaultScreenPercentagePathTracerMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.PathTracer"));
	static auto CVarEditorViewportDefaultScreenPercentageMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentageMode.NonRealTime"));
	static auto CVarEditorViewportDefaultScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.ScreenPercentage"));
	static auto CVarEditorViewportDefaultMinRenderingResolution = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.MinRenderingResolution"));
	static auto CVarEditorViewportDefaultMaxRenderingResolution = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.MaxRenderingResolution"));

	if (ViewStatus == EViewStatusForScreenPercentage::PathTracer)
	{
		Mode = EScreenPercentageMode(FMath::Clamp(CVarEditorViewportDefaultScreenPercentagePathTracerMode->GetInt(), 0, 2));
	}
	else if (ViewStatus == EViewStatusForScreenPercentage::VR)
	{
		Mode = EScreenPercentageMode(FMath::Clamp(CVarEditorViewportDefaultScreenPercentageVRMode->GetInt(), 0, 2));
	}
	else if (ViewStatus == EViewStatusForScreenPercentage::Mobile)
	{
		Mode = EScreenPercentageMode(FMath::Clamp(CVarEditorViewportDefaultScreenPercentageMobileMode->GetInt(), 0, 2));
	}
	else if (ViewStatus == EViewStatusForScreenPercentage::Desktop)
	{
		Mode = EScreenPercentageMode(FMath::Clamp(CVarEditorViewportDefaultScreenPercentageRealTimeMode->GetInt(), 0, 2));
	}
	else if (ViewStatus == EViewStatusForScreenPercentage::NonRealtime)
	{
		Mode = EScreenPercentageMode(FMath::Clamp(CVarEditorViewportDefaultScreenPercentageMode->GetInt(), 0, 2));
	}
	else
	{
		unimplemented();
	}
	GlobalResolutionFraction = GetResolutionFraction(CVarEditorViewportDefaultScreenPercentage->GetInt());
	MinRenderingResolution = CVarEditorViewportDefaultMinRenderingResolution->GetInt();
	MaxRenderingResolution = CVarEditorViewportDefaultMaxRenderingResolution->GetInt();
}
#else
{
	unimplemented();
}
#endif

void FStaticResolutionFractionHeuristic::FUserSettings::PullEditorRenderingSettings(bool bIsRealTime, bool bIsPathTraced)
{
	if (bIsPathTraced)
	{
		PullEditorRenderingSettings(EViewStatusForScreenPercentage::PathTracer);
	}
	else if (bIsRealTime)
	{
		PullEditorRenderingSettings(EViewStatusForScreenPercentage::Desktop);
	}
	else
	{
		PullEditorRenderingSettings(EViewStatusForScreenPercentage::NonRealtime);
	}
}

void FStaticResolutionFractionHeuristic::PullViewFamilyRenderingSettings(const FSceneViewFamily& ViewFamily)
{
	check(ViewFamily.Views.Num() > 0);

	SecondaryViewFraction = ViewFamily.SecondaryViewFraction;

	TotalDisplayedPixelCount = 0;
	for (const FSceneView* View : ViewFamily.Views)
	{
		// Number of pixel drawn in ViewFamily.RenderTarget
		int32 DisplayPixelCount = View->CameraConstrainedViewRect.Area();
		
		TotalDisplayedPixelCount += DisplayPixelCount;
	}
}

float FStaticResolutionFractionHeuristic::ResolveResolutionFraction() const
{
	// Compute number of pixel being rendered.
	float LocalTotalDisplayedPixelCount = FMath::Max(TotalDisplayedPixelCount, 1) * SecondaryViewFraction * SecondaryViewFraction;

	float GlobalResolutionFraction = 1.0f;

	const EScreenPercentageMode EffectiveScreenPercentageMode = Settings.bAllowDisplayBasedScreenPercentageMode ? Settings.Mode : EScreenPercentageMode::Manual;

	if (EffectiveScreenPercentageMode == EScreenPercentageMode::BasedOnDisplayResolution)
	{
		static bool bInitPixelCount = false;
		static float AutoMinDisplayResolution;
		static float AutoMinRenderingResolution;
		static float AutoMidDisplayResolution;
		static float AutoMidRenderingResolution;
		static float AutoMaxDisplayResolution;
		static float AutoMaxRenderingResolution;

		if (!bInitPixelCount)
		{
			bool bSuccess = true;
			bSuccess = bSuccess && GConfig->GetFloat(TEXT("Rendering.AutoScreenPercentage"), TEXT("MinDisplayResolution"), AutoMinDisplayResolution, GEngineIni);
			bSuccess = bSuccess && GConfig->GetFloat(TEXT("Rendering.AutoScreenPercentage"), TEXT("MidDisplayResolution"), AutoMidDisplayResolution, GEngineIni);
			bSuccess = bSuccess && GConfig->GetFloat(TEXT("Rendering.AutoScreenPercentage"), TEXT("MaxDisplayResolution"), AutoMaxDisplayResolution, GEngineIni);
			bSuccess = bSuccess && GConfig->GetFloat(TEXT("Rendering.AutoScreenPercentage"), TEXT("MinRenderingResolution"), AutoMinRenderingResolution, GEngineIni);
			bSuccess = bSuccess && GConfig->GetFloat(TEXT("Rendering.AutoScreenPercentage"), TEXT("MidRenderingResolution"), AutoMidRenderingResolution, GEngineIni);
			bSuccess = bSuccess && GConfig->GetFloat(TEXT("Rendering.AutoScreenPercentage"), TEXT("MaxRenderingResolution"), AutoMaxRenderingResolution, GEngineIni);
			if (!bSuccess)
			{
				UE_LOGF(LogEngine, Fatal, "Failed to load Rendering.AutoScreenPercentage ini section.");
			}
			bInitPixelCount = true;
		}

		float LerpedRenderingPixelCount = 0.0f;
		if (LocalTotalDisplayedPixelCount < GetRenderingPixelCount(AutoMinDisplayResolution))
		{
			LerpedRenderingPixelCount = LocalTotalDisplayedPixelCount * GetRenderingPixelCount(AutoMinRenderingResolution) / GetRenderingPixelCount(AutoMinDisplayResolution);
		}
		else if (LocalTotalDisplayedPixelCount > GetRenderingPixelCount(AutoMaxDisplayResolution))
		{
			LerpedRenderingPixelCount = LocalTotalDisplayedPixelCount * GetRenderingPixelCount(AutoMaxRenderingResolution) / GetRenderingPixelCount(AutoMaxDisplayResolution);
		}
		else
		{
			float MinDisplayPixelCount   = GetRenderingPixelCount(AutoMinDisplayResolution);
			float MaxDisplayPixelCount   = GetRenderingPixelCount(AutoMidDisplayResolution);
			float MinRenderingPixelCount = GetRenderingPixelCount(AutoMinRenderingResolution);
			float MaxRenderingPixelCount = GetRenderingPixelCount(AutoMidRenderingResolution);
			if (LocalTotalDisplayedPixelCount > MaxDisplayPixelCount)
			{
				MinDisplayPixelCount   = GetRenderingPixelCount(AutoMidDisplayResolution);
				MaxDisplayPixelCount   = GetRenderingPixelCount(AutoMaxDisplayResolution);
				MinRenderingPixelCount = GetRenderingPixelCount(AutoMidRenderingResolution);
				MaxRenderingPixelCount = GetRenderingPixelCount(AutoMaxRenderingResolution);
			}

			float ReslutionFractionLerp = FMath::Clamp((LocalTotalDisplayedPixelCount - MinDisplayPixelCount) / (MaxDisplayPixelCount - MinDisplayPixelCount), 0.0f, 1.0f);
			LerpedRenderingPixelCount = FMath::Lerp(MinRenderingPixelCount, MaxRenderingPixelCount, ReslutionFractionLerp);
		}

		GlobalResolutionFraction = FMath::Sqrt(Settings.AutoPixelCountMultiplier * LerpedRenderingPixelCount / LocalTotalDisplayedPixelCount);
	}
	else if (EffectiveScreenPercentageMode == EScreenPercentageMode::BasedOnDPIScale)
	{
		GlobalResolutionFraction = 1.0f / DPIScale;
	}
	else
	{
		ensure(EffectiveScreenPercentageMode == EScreenPercentageMode::Manual);
		GlobalResolutionFraction = Settings.GlobalResolutionFraction;
	}

	// Max the rendering resolution to average target resolution used on platform that have dynamic resolution.
	const float MaxRenderingPixelCount = GetRenderingPixelCount(Settings.MaxRenderingResolution);
	if (MaxRenderingPixelCount > 0.0f)
	{
		// Compute max resolution fraction such that the total number of pixel doesn't go over the CVarScreenPercentageMaxResolution.
		float MaxGlobalResolutionFraction = FMath::Sqrt(float(MaxRenderingPixelCount) / float(LocalTotalDisplayedPixelCount));

		GlobalResolutionFraction = FMath::Min(GlobalResolutionFraction, MaxGlobalResolutionFraction);
	}

	// Min the rendering resolution to avoid any upscaling at very low resolutions.
	const float MinRenderingPixelCount = GetRenderingPixelCount(Settings.MinRenderingResolution);
	if (MinRenderingPixelCount > 0.0f)
	{
		float MinGlobalResolutionFraction = FMath::Min(FMath::Sqrt(float(MinRenderingPixelCount) / float(LocalTotalDisplayedPixelCount)), 1.0f);

		GlobalResolutionFraction = FMath::Max(GlobalResolutionFraction, MinGlobalResolutionFraction);
	}

	return GlobalResolutionFraction;
}
