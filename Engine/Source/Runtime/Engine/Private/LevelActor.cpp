// Copyright Epic Games, Inc. All Rights Reserved.


#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "EngineStats.h"
#include "Engine/Level.h"
#include "AI/NavigationSystemBase.h"
#include "RenderUtils.h"
#include "UObject/LinkerLoad.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Engine/OverlapResult.h"
#include "ContentStreaming.h"
#include "EditorSupportDelegates.h"
#include "GameFramework/GameModeBase.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/Player.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GameFramework/MovementInterface.h"
#include "Net/Core/PropertyConditions/PropertyConditions.h"

#include "Components/BoxComponent.h"
#include "GameFramework/MovementComponent.h"

#include "Misc/TimeGuard.h"

#define LOCTEXT_NAMESPACE "LevelActor"

DECLARE_CYCLE_STAT(TEXT("Destroy Actor"), STAT_DestroyActor, STATGROUP_Game);

// CVars
static TAutoConsoleVariable<float> CVarEncroachEpsilon(
	TEXT("p.EncroachEpsilon"),
	0.15f,
	TEXT("Epsilon value used during encroachment checking for shape components\n")
	TEXT("0: use full sized shape. > 0: shrink shape size by this amount (world units)"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarAllowDestroyNonNetworkActors(
	TEXT("p.AllowDestroyNonNetworkActors"),
	1,
	TEXT("When enabled, allows Clients in Networked Games to destroy non-networked actors (AActor::Role == ROLE_None). Does not change behavior on Servers or Standalone games.")
);

namespace UE::Gameplay::CVars
{
	bool bDelayOnActorSpawnedUntilFinishedSpawning = true;
	static FAutoConsoleVariableRef CVarDelayOnActorSpawnedUntilFinishedSpawning(
		TEXT("s.DelayOnActorSpawnedUntilFinishedSpawning"),
		bDelayOnActorSpawnedUntilFinishedSpawning,
		TEXT("Default: True (new/fixed). Introduced in UE5.6 that UWorld::OnActorSpawned will run only after the Actor has FinishedSpawning (is fully initialized).  False to keep old behavior.")
	);
}


#define LINE_CHECK_TRACING 0

#if LINE_CHECK_TRACING

/** Is tracking enabled */
int32 LineCheckTracker::bIsTrackingEnabled = false;
/** If this count is nonzero, dump the log when we exceed this number in any given frame */
int32 LineCheckTracker::TraceCountForSpikeDump = 0;
/** Number of traces recorded this frame */
int32 LineCheckTracker::CurrentCountForSpike = 0;

FStackTracker* LineCheckTracker::LineCheckStackTracker = NULL;
FScriptStackTracker* LineCheckTracker::LineCheckScriptStackTracker = NULL;

/** Updates an existing call stack trace with new data for this particular call*/
static void LineCheckUpdateFn(const FStackTracker::FCallStack& CallStack, void* UserData)
{
	if (UserData)
	{
		//Callstack has been called more than once, aggregate the data
		LineCheckTracker::FLineCheckData* NewLCData = static_cast<LineCheckTracker::FLineCheckData*>(UserData);
		LineCheckTracker::FLineCheckData* OldLCData = static_cast<LineCheckTracker::FLineCheckData*>(CallStack.UserData);

		OldLCData->Flags |= NewLCData->Flags;
		OldLCData->IsNonZeroExtent |= NewLCData->IsNonZeroExtent;

		if (NewLCData->LineCheckObjsMap.Num() > 0)
		{
			for (TMap<const FName, LineCheckTracker::FLineCheckData::LineCheckObj>::TConstIterator It(NewLCData->LineCheckObjsMap); It; ++It)
			{
				const LineCheckTracker::FLineCheckData::LineCheckObj &NewObj = It.Value();

				LineCheckTracker::FLineCheckData::LineCheckObj * OldObj = OldLCData->LineCheckObjsMap.Find(NewObj.ObjectName);
				if (OldObj)
				{
					OldObj->Count += NewObj.Count;
				}
				else
				{
					OldLCData->LineCheckObjsMap.Add(NewObj.ObjectName, NewObj);
				}
			}
		}
	}
}

/** After the stack tracker reports a given stack trace, it calls this function
*  which appends data particular to line checks
*/
static void LineCheckReportFn(const FStackTracker::FCallStack& CallStack, uint64 TotalStackCount, FOutputDevice& Ar)
{
	//Output to a csv file any relevant data
	LineCheckTracker::FLineCheckData* const LCData = static_cast<LineCheckTracker::FLineCheckData*>(CallStack.UserData);
	if (LCData)
	{
		FString UserOutput = LINE_TERMINATOR TEXT(",,,");
		UserOutput += (LCData->IsNonZeroExtent ? TEXT( "NonZeroExtent") : TEXT("ZeroExtent"));

		for (TMap<const FName, LineCheckTracker::FLineCheckData::LineCheckObj>::TConstIterator It(LCData->LineCheckObjsMap); It; ++It)
		{
			UserOutput += LINE_TERMINATOR TEXT(",,,");
			const LineCheckTracker::FLineCheckData::LineCheckObj &CurObj = It.Value();
			UserOutput += FString::Printf(TEXT("%s (%d) : %s"), *CurObj.ObjectName.ToString(), CurObj.Count, *CurObj.DetailedInfo);
		}

		UserOutput += LINE_TERMINATOR TEXT(",,,");
		
		Ar.Log(*UserOutput);
	}
}

/** Called at the beginning of each frame to check/reset spike count */
void LineCheckTracker::Tick()
{
	if(bIsTrackingEnabled && LineCheckStackTracker)
	{
		//Spike logging is enabled
		if (TraceCountForSpikeDump > 0)
		{
			//Dump if we exceeded the threshold this frame
			if (CurrentCountForSpike > TraceCountForSpikeDump)
			{
				DumpLineChecks(5);
			}
			//Reset for next frame
			ResetLineChecks();
		}

		CurrentCountForSpike = 0;
	}
}

/** Set the value which, if exceeded, will cause a dump of the line checks this frame */
void LineCheckTracker::SetSpikeMinTraceCount(int32 MinTraceCount)
{
	TraceCountForSpikeDump = FMath::Max(0, MinTraceCount);
	UE_LOGF(LogSpawn, Log, "Line trace spike count is %d.", TraceCountForSpikeDump);
}

/** Dump out the results of all line checks called in the game since the last call to ResetLineChecks() */
void LineCheckTracker::DumpLineChecks(int32 Threshold)
{
	if( LineCheckStackTracker )
	{
		const FString Filename = FString::Printf(TEXT("%sLineCheckLog-%s.csv"), *FPaths::ProjectLogDir(), *FDateTime::Now().ToString());
		FOutputDeviceFile OutputFile(*Filename);
		LineCheckStackTracker->DumpStackTraces( Threshold, OutputFile );
		OutputFile.TearDown();
	}

	if( LineCheckScriptStackTracker )
	{
		const FString Filename = FString::Printf(TEXT("%sScriptLineCheckLog-%s.csv"), *FPaths::ProjectLogDir(), *FDateTime::Now().ToString());
		FOutputDeviceFile OutputFile(*Filename);
		LineCheckScriptStackTracker->DumpStackTraces( Threshold, OutputFile );
		OutputFile.TearDown();
	}
}

/** Reset the line check stack tracker (calls FMemory::Free() on all user data pointers)*/
void LineCheckTracker::ResetLineChecks()
{
	if( LineCheckStackTracker )
	{
		LineCheckStackTracker->ResetTracking();
	}

	if( LineCheckScriptStackTracker )
	{
		LineCheckScriptStackTracker->ResetTracking();
	}
}

/** Turn line check stack traces on and off, does not reset the actual data */
void LineCheckTracker::ToggleLineChecks()
{
	bIsTrackingEnabled = !bIsTrackingEnabled;
	UE_LOGF(LogSpawn, Log, "Line tracing is now %ls.", bIsTrackingEnabled ? TEXT("enabled") : TEXT("disabled"));
	
	CurrentCountForSpike = 0;
	if (LineCheckStackTracker == NULL)
	{
		FPlatformStackWalk::InitStackWalking();
		LineCheckStackTracker = new FStackTracker(LineCheckUpdateFn, LineCheckReportFn);
	}

	if (LineCheckScriptStackTracker == NULL)
	{
		LineCheckScriptStackTracker = new FScriptStackTracker();
	}

	LineCheckStackTracker->ToggleTracking();
	LineCheckScriptStackTracker->ToggleTracking();
}

/** Captures a single stack trace for a line check */
void LineCheckTracker::CaptureLineCheck(int32 LineCheckFlags, const FVector* Extent, const FFrame* ScriptStackFrame, const UObject* Object)
{
	if (LineCheckStackTracker == NULL || LineCheckScriptStackTracker == NULL)
	{
		return;
	}

	if (ScriptStackFrame)
	{
		int32 EntriesToIgnore = 0;
		LineCheckScriptStackTracker->CaptureStackTrace(ScriptStackFrame, EntriesToIgnore);
	}
	else
	{		   
		FLineCheckData* const LCData = static_cast<FLineCheckData*>(FMemory::Malloc(sizeof(FLineCheckData)));
		FMemory::Memset(LCData, 0, sizeof(FLineCheckData));
		LCData->Flags = LineCheckFlags;
		LCData->IsNonZeroExtent = (Extent && !Extent->IsZero()) ? true : false;
		FLineCheckData::LineCheckObj LCObj;
		if (Object)
		{
			LCObj = FLineCheckData::LineCheckObj(Object->GetFName(), 1, Object->GetDetailedInfo());
		}
		else
		{
			LCObj = FLineCheckData::LineCheckObj(NAME_None, 1, TEXT("Unknown"));
		}
		
		LCData->LineCheckObjsMap.Add(LCObj.ObjectName, LCObj);		

		int32 EntriesToIgnore = 3;
		LineCheckStackTracker->CaptureStackTrace(EntriesToIgnore, static_cast<void*>(LCData));
		//Only increment here because execTrace() will lead to here also
		CurrentCountForSpike++;
	}
}
#endif //LINE_CHECK_TRACING

/*-----------------------------------------------------------------------------
	Level actor management.
-----------------------------------------------------------------------------*/

/** 
 * Generates a 102-bits actor GUID:
 *	- Bits 101-54 hold the user unique id.
 *	- Bits 53-0 hold a microseconds timestamp.
  *
 * Notes:
 *	- The timestamp is stored in microseconds for a total of 54 bits, enough to cover
 *	  the next 570 years.
 *	- The highest 72 bits are appended to the name in hexadecimal.
 *	- The lowest 30 bits of the timestamp are stored in the name number. This is to
 *	  minimize the total names generated for globally unique names (string part will
 *	  change every ~17 minutes for a specific actor class).
 *  - The name number bit 30 is reserved to know if this is a globally unique name. This
 *	  is not 100% safe, but should cover most cases.
 *  - The name number bit 31 is reserved to preserve the fast path name generation (see
 *	  GFastPathUniqueNameGeneration).
 *	- On some environments, cooking happens without network, so we can't retrieve the 
 *	  MAC address for spawned actors, assign a default one in this case.
 **/
class FActorGUIDGenerator
{
public:
	FActorGUIDGenerator()
		: Origin(FDateTime(2020, 1, 1))
		, Counter(0)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<uint8> MACAddress = FPlatformMisc::GetMacAddress();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (MACAddress.Num() == 6)
		{
			FMemory::Memcpy(UniqueID, MACAddress.GetData(), 6);
		}
		else if (IsRunningCommandlet())
		{
			FMemory::Memzero(UniqueID, 6);
		}
		else
		{
			const FString EpicAccountId = FPlatformMisc::GetEpicAccountId();
			const uint32 UniqueUserId = GetTypeHash(EpicAccountId) ^ FPlatformProcess::GetCurrentProcessId();
			const uint64 ElapsedUs = FDateTime::Now().GetTicks();

			UniqueID[0] = UniqueUserId & 0xff;
			UniqueID[1] = (UniqueUserId >> 8) & 0xff;
			UniqueID[2] = (UniqueUserId >> 16) & 0xff;
			UniqueID[3] = (UniqueUserId >> 24) & 0xff;
			UniqueID[4] = ElapsedUs & 0xff;
			UniqueID[5] = (ElapsedUs >> 8) & 0xff;
		}
	}

	FName NewActorGUID(FName BaseName)
	{
		uint8 HighPart[9];

		const FDateTime Now = FDateTime::Now();
		check(Now > Origin);

		const FTimespan Elapsed = FDateTime::Now() - Origin;
		const uint64 ElapsedUs = (uint64)Elapsed.GetTotalMilliseconds() * 1000 + (Counter++ % 1000);

		// Copy 48-bits unique id
		FMemory::Memcpy(HighPart, UniqueID, 6);
		
		// Append the high part of the timestamp (will change every ~17 minutes)
		const uint64 ElapsedUsHighPart = ElapsedUs >> 30;
		FMemory::Memcpy(HighPart + 6, &ElapsedUsHighPart, 3);

		// Make final name
		TStringBuilderWithBuffer<TCHAR, NAME_SIZE> StringBuilder;
		StringBuilder += BaseName.ToString();
		StringBuilder += TEXT("_UAID_");

		for (uint32 i=0; i<9; i++)
		{
			StringBuilder += NibbleToTChar(HighPart[i] >> 4);
			StringBuilder += NibbleToTChar(HighPart[i] & 15);
		}

		return FName(*StringBuilder, (ElapsedUs & 0x3fffffff) | (1 << 30));
	}
	
private:
	FDateTime Origin;
	uint8 UniqueID[6];
	uint32 Counter;
};

FName FActorSpawnUtils::MakeUniqueActorName(ULevel* Level, const UClass* Class, FName BaseName, bool bGloballyUnique)
{
	FName NewActorName;

	if (bGloballyUnique)
	{
		static FActorGUIDGenerator ActorGUIDGenerator;

		do
		{
			NewActorName = ActorGUIDGenerator.NewActorGUID(BaseName);
		}
		while (StaticFindObjectFast(nullptr, Level, NewActorName));
	}
	else
	{
		NewActorName = MakeUniqueObjectName(Level, Class, BaseName);
	}

	return NewActorName;
}

bool FActorSpawnUtils::IsGloballyUniqueName(FName Name)
{
	if (Name.GetNumber() & (1 << 30))
	{
		const FString PlainName = Name.GetPlainNameString();
		const int32 PlainNameLen = PlainName.Len();
		
		// Parse a name like this: StaticMeshActor_UAID_001122334455667788
		if (PlainNameLen >= 24)
		{
			if (!FCString::Strnicmp(*PlainName + PlainNameLen - 24, TEXT("_UAID_"), 6))
			{
				for (uint32 i=0; i<18; i++)
				{
					if (!CheckTCharIsHex(PlainName[PlainNameLen - i - 1]))
					{
						return false;
					}
				}

				return true;
			}
		}
	}

	return false;
}

FName FActorSpawnUtils::GetBaseName(FName Name)
{
	if (IsGloballyUniqueName(Name))
	{
		// Chop a name like this: StaticMeshActor_UAID_001122334455667788
		return *Name.GetPlainNameString().LeftChop(24);
	}

	return *Name.GetPlainNameString();
}

// LOOKING_FOR_PERF_ISSUES
#define PERF_SHOW_MULTI_PAWN_SPAWN_FRAMES (!(UE_BUILD_SHIPPING || UE_BUILD_TEST)) && (LOOKING_FOR_PERF_ISSUES || !WITH_EDITORONLY_DATA)

#if PERF_SHOW_MULTI_PAWN_SPAWN_FRAMES
	/** Array showing names of pawns spawned this frame. */
	TArray<FString>	ThisFramePawnSpawns;
#endif	//PERF_SHOW_MULTI_PAWN_SPAWN_FRAMES

AActor* UWorld::SpawnActorAbsolute( UClass* Class, FTransform const& AbsoluteTransform, const FActorSpawnParameters& SpawnParameters)
{
	AActor* Template = SpawnParameters.Template;

	if(!Template)
	{
		// Use class's default actor as a template.
		Template = Class->GetDefaultObject<AActor>();
	}

	FTransform NewTransform = AbsoluteTransform;
	USceneComponent* TemplateRootComponent = (Template)? Template->GetRootComponent() : NULL;
	if(TemplateRootComponent)
	{
		TemplateRootComponent->UpdateComponentToWorld();
		NewTransform = TemplateRootComponent->GetComponentToWorld().Inverse() * NewTransform;
	}

	return SpawnActor(Class, &NewTransform, SpawnParameters);
}

AActor* UWorld::SpawnActor( UClass* Class, FVector const* Location, FRotator const* Rotation, const FActorSpawnParameters& SpawnParameters )
{
	FTransform Transform;
	if (Location)
	{
		Transform.SetLocation(*Location);
	}
	if (Rotation)
	{
		Transform.SetRotation(FQuat(*Rotation));
	}

	return SpawnActor(Class, &Transform, SpawnParameters);
}

#include "GameFramework/SpawnActorTimer.h"

AActor* UWorld::SpawnActor( UClass* Class, FTransform const* UserTransformPtr, const FActorSpawnParameters& SpawnParameters )
{
	SCOPE_CYCLE_COUNTER(STAT_SpawnActorTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ActorSpawning);

#if WITH_EDITORONLY_DATA
	check( CurrentLevel ); 	
	check(GIsEditor || (CurrentLevel == PersistentLevel));
#else
	ULevel* CurrentLevel = PersistentLevel;
#endif

	// Make sure this class is spawnable.
	if( !Class )
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because no class was specified" );
		return NULL;
	}

	SCOPE_TIME_GUARD_NAMED_MS(TEXT("SpawnActor Of Type"), Class->GetFName(), 2);

#if ENABLE_SPAWNACTORTIMER
	FScopedSpawnActorTimer SpawnTimer(Class->GetFName(), SpawnParameters.bDeferConstruction ? ESpawnActorTimingType::SpawnActorDeferred : ESpawnActorTimingType::SpawnActorNonDeferred);
#endif

	if( Class->HasAnyClassFlags(CLASS_Deprecated) )
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because class %ls is deprecated", *Class->GetName() );
		return NULL;
	}
	if( Class->HasAnyClassFlags(CLASS_Abstract) )
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because class %ls is abstract", *Class->GetName() );
		return NULL;
	}
	else if( !Class->IsChildOf(AActor::StaticClass()) )
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because %ls is not an actor class", *Class->GetName() );
		return NULL;
	}
	else if (SpawnParameters.Template != NULL && SpawnParameters.Template->GetClass() != Class)
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because template class (%ls) does not match spawn class (%ls)", *SpawnParameters.Template->GetClass()->GetName(), *Class->GetName());
		if (!SpawnParameters.bNoFail)
		{
			return NULL;
		}
	}
	else if (bIsRunningConstructionScript && !SpawnParameters.bAllowDuringConstructionScript)
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because we are running a ConstructionScript (%ls)", *Class->GetName() );
		return NULL;
	}
	else if (bIsTearingDown)
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because we are in the process of tearing down the world");
		return NULL;
	}
	else if (UserTransformPtr && UserTransformPtr->ContainsNaN())
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because the given transform (%ls) is invalid", *(UserTransformPtr->ToString()));
		return NULL;
	}
#if !WITH_EDITOR
	else if (SpawnParameters.bForceGloballyUniqueName && !SpawnParameters.Name.IsNone())
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because a globally unique name was requested and an actor name was provided (%ls)", *SpawnParameters.Name.ToString());
		return NULL;
	}
#else
	if (SpawnParameters.OverridePackage && SpawnParameters.bCreateActorPackage)
	{
		UE_LOGF(LogSpawn, Warning, "SpawnActor failed because both the OverridePackage and bCreateActorPackage are set");
		return nullptr;
	}
#endif

	ULevel* LevelToSpawnIn = SpawnParameters.OverrideLevel;
	if (LevelToSpawnIn == NULL)
	{
		// Spawn in the same level as the owner if we have one.
		LevelToSpawnIn = (SpawnParameters.Owner != NULL) ? SpawnParameters.Owner->GetLevel() : ToRawPtr(CurrentLevel);
	}

	// Use class's default actor as a template if none provided.
	AActor* Template = SpawnParameters.Template ? SpawnParameters.Template : Class->GetDefaultObject<AActor>();
	check(Template);

	FName NewActorName = SpawnParameters.Name;
	UPackage* ExternalPackage = nullptr;
	bool bNeedGloballyUniqueName = false;

#if WITH_EDITOR
	// Generate and set the actor's external package if needed
	if (SpawnParameters.OverridePackage)
	{
		ExternalPackage = SpawnParameters.OverridePackage;
		bNeedGloballyUniqueName = true;
	}
	else if (LevelToSpawnIn->ShouldCreateNewExternalActors() && SpawnParameters.bCreateActorPackage && !(SpawnParameters.ObjectFlags & RF_Transient))
	{
		bNeedGloballyUniqueName = CastChecked<AActor>(Class->GetDefaultObject())->SupportsExternalPackaging();
	}

	if (!GIsEditor)
	{
		bNeedGloballyUniqueName = false;
	}
#else
	// Override all previous logic if the spawn request a globally unique name
	bNeedGloballyUniqueName |= SpawnParameters.bForceGloballyUniqueName;
#endif

	if (NewActorName.IsNone())
	{
		// If we are using a template object and haven't specified a name, create a name relative to the template, otherwise let the default object naming behavior in Stat
		const FName BaseName = Template->HasAnyFlags(RF_ClassDefaultObject) ? Class->GetFName() : *Template->GetFName().GetPlainNameString();

		NewActorName = FActorSpawnUtils::MakeUniqueActorName(LevelToSpawnIn, Template->GetClass(), BaseName, bNeedGloballyUniqueName);
	}
	else if (StaticFindObjectFast(nullptr, LevelToSpawnIn, NewActorName) || ((bNeedGloballyUniqueName != FActorSpawnUtils::IsGloballyUniqueName(NewActorName)) && (SpawnParameters.NameMode == FActorSpawnParameters::ESpawnActorNameMode::Requested)))
	{
		// If the supplied name is already in use or doesn't respect globally uniqueness, then either fail in the requested manner or determine a new name to use if the caller indicates that's ok
		switch(SpawnParameters.NameMode)
		{
		case FActorSpawnParameters::ESpawnActorNameMode::Requested:
			NewActorName = FActorSpawnUtils::MakeUniqueActorName(LevelToSpawnIn, Template->GetClass(), FActorSpawnUtils::GetBaseName(NewActorName), bNeedGloballyUniqueName);
			break;

		case FActorSpawnParameters::ESpawnActorNameMode::Required_Fatal:
			UE_LOGF(LogSpawn, Fatal, "Cannot generate unique name for '%ls' in level '%ls'.", *NewActorName.ToString(), *LevelToSpawnIn->GetFullName());
			return nullptr;

		case FActorSpawnParameters::ESpawnActorNameMode::Required_ErrorAndReturnNull:
			UE_LOGF(LogSpawn, Error, "Cannot generate unique name for '%ls' in level '%ls'.", *NewActorName.ToString(), *LevelToSpawnIn->GetFullName());
			return nullptr;

		case FActorSpawnParameters::ESpawnActorNameMode::Required_ReturnNull:
			return nullptr;

		default:
			check(0);
		}
	}

	// See if we can spawn on ded.server/client only etc (check NeedsLoadForClient & NeedsLoadForServer)
	if(!CanCreateInCurrentContext(Template))
	{
		UE_LOGF(LogSpawn, Warning, "Unable to spawn class '%ls' due to client/server context.", *Class->GetName() );
		return NULL;
	}

#if WITH_EDITOR
	if (bNeedGloballyUniqueName && !ExternalPackage)
	{
		TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ActorPath;
		ActorPath += LevelToSpawnIn->GetPathName();
		ActorPath += TEXT(".");
		ActorPath += NewActorName.ToString();

		// @todo FH: needs to handle mark package dirty and asset creation notification
		ExternalPackage = ULevel::CreateActorPackage(LevelToSpawnIn->GetPackage(), LevelToSpawnIn->GetActorPackagingScheme(), *ActorPath);
	}
#endif

	FTransform const UserTransform = UserTransformPtr ? *UserTransformPtr : FTransform::Identity;

	// Choose the collision handling method. In order of increasing priority: actor template setting (often class default), collision handling override, bNoFail
	ESpawnActorCollisionHandlingMethod CollisionHandlingMethod = Template->SpawnCollisionHandlingMethod;

	// Adopt collision handling override if any was set
	if (SpawnParameters.SpawnCollisionHandlingOverride != ESpawnActorCollisionHandlingMethod::Undefined)
	{
		CollisionHandlingMethod = SpawnParameters.SpawnCollisionHandlingOverride;
	}

	// If bNoFail is true, upgrade collision handling method to an always spawning one
	if (SpawnParameters.bNoFail)
	{
		if (CollisionHandlingMethod == ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding)
		{
			CollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;
		}
		else if (CollisionHandlingMethod == ESpawnActorCollisionHandlingMethod::DontSpawnIfColliding)
		{
			CollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		}
	}

	// see if we can avoid spawning altogether by checking native components
	// note: we can't handle all cases here, since we don't know the full component hierarchy until after the actor is spawned
	if (CollisionHandlingMethod == ESpawnActorCollisionHandlingMethod::DontSpawnIfColliding)
	{
		USceneComponent* const TemplateRootComponent = Template->GetRootComponent();

		// Note that we respect any initial transformation the root component may have from the CDO, so the final transform
		// might necessarily be exactly the passed-in UserTransform.
		FTransform const FinalRootComponentTransform =
			TemplateRootComponent
			? FTransform(TemplateRootComponent->GetRelativeRotation(), TemplateRootComponent->GetRelativeLocation(), TemplateRootComponent->GetRelativeScale3D()) * UserTransform
			: UserTransform;

		FVector const FinalRootLocation = FinalRootComponentTransform.GetLocation();
		FRotator const FinalRootRotation = FinalRootComponentTransform.Rotator();

		if (EncroachingBlockingGeometry(Template, FinalRootLocation, FinalRootRotation))
		{
			// a native component is colliding, that's enough to reject spawning
			UE_LOGF(LogSpawn, Log, "SpawnActor failed because of collision at the spawn location [%ls] for [%ls]", *FinalRootLocation.ToString(), *Class->GetName());
			return nullptr;
		}
	}

	EObjectFlags ActorFlags = SpawnParameters.ObjectFlags;

	// actually make the actor object
	AActor* const Actor = NewObject<AActor>(LevelToSpawnIn, Class, NewActorName, ActorFlags, Template, false/*bCopyTransientsFromClassDefaults*/, nullptr/*InInstanceGraph*/, ExternalPackage);
	
	check(Actor);
	check(Actor->GetLevel() == LevelToSpawnIn);

#if WITH_EDITOR
	// UE5-Release: bHideFromSceneOutliner must be set before anything tries
	// to create an FActorTreeItem in the Scene Outliner. Otherwise, the tree item will
	// be created and will be visible in the Scene Outliner. 
	// AActor::ClearActorLabel, called below, currently does that via FCoreDelegates::OnActorLabelChanged.
	// A better fix should prevent the FActorTreeItem creation before the end of the spawning sequence.
	if (SpawnParameters.bHideFromSceneOutliner)
	{
		FSetActorHiddenInSceneOutliner SetActorHidden(Actor);
	}
	Actor->bIsEditorPreviewActor = SpawnParameters.bTemporaryEditorActor;
#endif //WITH_EDITOR


#if ENABLE_SPAWNACTORTIMER
	SpawnTimer.SetActorName(Actor->GetFName());
#endif

#if WITH_EDITOR
	if (SpawnParameters.InitialActorLabel.Len() != 0)
	{
		Actor->SetInitialActorLabel(SpawnParameters.InitialActorLabel);
	}
	else
	{
		Actor->ClearActorLabel(); // Clear label on newly spawned actors
	}

	// Generate the actor's Guid
	FGuid ActorGuid;
	if (SpawnParameters.OverrideActorGuid.IsValid())
	{
		ActorGuid = SpawnParameters.OverrideActorGuid;
	}
	else if (IsRunningCookCommandlet())
	{
		// Generate the guid deterministically when creating actors at cook time
		ActorGuid = FGuid::NewDeterministicGuid(Actor->GetPathName());
	}
	else
	{
		ActorGuid = FGuid::NewGuid();
	}

	// Set the actor's guid
	FSetActorGuid SetActorGuid(Actor, ActorGuid);
#endif

	if (SpawnParameters.OverrideParentComponent)
	{
		FActorParentComponentSetter::Set(Actor, SpawnParameters.OverrideParentComponent);
	}

	if (SpawnParameters.CustomPreSpawnInitialization)
	{
		SpawnParameters.CustomPreSpawnInitialization(Actor);
	}

	if ( GUndo )
	{
		ModifyLevel( LevelToSpawnIn );
	}
	LevelToSpawnIn->TryAddActorToList( Actor, /*bAddUnique*/false);

#if PERF_SHOW_MULTI_PAWN_SPAWN_FRAMES
	if( Cast<APawn>(Actor) )
	{
		FString PawnName = FString::Printf(TEXT("%d: %s"), ThisFramePawnSpawns.Num(), *Actor->GetPathName());
		ThisFramePawnSpawns.Add(PawnName);
	}
#endif

	// tell the actor what method to use, in case it was overridden
	Actor->SpawnCollisionHandlingMethod = CollisionHandlingMethod;

	// Broadcast delegate before the actor and its contained components are initialized
	OnActorPreSpawnInitialization.Broadcast(Actor);

	Actor->PostSpawnInitialize(UserTransform, SpawnParameters.Owner, SpawnParameters.Instigator, SpawnParameters.IsRemoteOwned(), SpawnParameters.bNoFail, SpawnParameters.bDeferConstruction, SpawnParameters.TransformScaleMethod);
	
	// If we are spawning an external actor, mark this package dirty
	if (ExternalPackage)
	{
		ExternalPackage->MarkPackageDirty();
	}

	if (!IsValid(Actor) && !SpawnParameters.bNoFail)
	{
		UE_LOGF(LogSpawn, Log, "SpawnActor failed because the spawned actor %ls is invalid", *Actor->GetPathName());
		return NULL;
	}

	// This is too early, we were not guaranteed to call FinishSpawning in PostSpawnInitialize:
	// CVar in case some ugly case comes up
	if (!UE::Gameplay::CVars::bDelayOnActorSpawnedUntilFinishedSpawning)
	{
		// Broadcast notification of spawn
		OnActorSpawned.Broadcast(Actor);
	}

	Actor->CheckDefaultSubobjects();

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (Actor->IsAsset() && Actor->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
		{
			FAssetRegistryModule::AssetCreated(Actor);
		}

		GEngine->BroadcastLevelActorAdded(Actor);
	}
#endif

	// This is too early, we were not guaranteed to call FinishSpawning in PostSpawnInitialize:
	// CVar in case some ugly case comes up
	if (!UE::Gameplay::CVars::bDelayOnActorSpawnedUntilFinishedSpawning)
	{
		// Add this newly spawned actor to the network actor list. Do this after PostSpawnInitialize so that actor has "finished" spawning.
		AddNetworkActor( Actor );
	}

	return Actor;
}

ABrush* UWorld::SpawnBrush()
{
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	ABrush* const Result = SpawnActor<ABrush>( SpawnInfo );
	check(Result);
	return Result;
}

/**
 * Wrapper for DestroyActor() that should be called in the editor.
 *
 * @param	bShouldModifyLevel		If true, Modify() the level before removing the actor.
 */
bool UWorld::EditorDestroyActor( AActor* ThisActor, bool bShouldModifyLevel )
{
	FNavigationSystem::OnActorUnregistered(*ThisActor);

	bool bReturnValue = DestroyActor( ThisActor, false, bShouldModifyLevel );
	if (UWorld* World = ThisActor->GetWorld())
	{
		World->BroadcastLevelsChanged();
	}
	return bReturnValue;
}

/**
 * Removes the actor from its level's actor list and generally cleans up the engine's internal state.
 * What this function does not do, but is handled via garbage collection instead, is remove references
 * to this actor from all other actors, and kill the actor's resources.  This function is set up so that
 * no problems occur even if the actor is being destroyed inside its recursion stack.
 *
 * @param	ThisActor				Actor to remove.
 * @param	bNetForce				[opt] Ignored unless called during play.  Default is false.
 * @param	bShouldModifyLevel		[opt] If true, Modify() the level before removing the actor.  Default is true.
 * @return							true if destroy, false if actor couldn't be destroyed.
 */
bool UWorld::DestroyActor( AActor* ThisActor, bool bNetForce, bool bShouldModifyLevel )
{
	SCOPE_CYCLE_COUNTER(STAT_DestroyActor);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ActorDestroying);

	check(ThisActor);
	check(ThisActor->IsValidLowLevel());
	//UE_LOGF(LogSpawn, Log,  "Destroy %ls", *ThisActor->GetClass()->GetName() );

	SCOPE_CYCLE_UOBJECT(ThisActor, ThisActor);

	if (ThisActor->GetWorld() == NULL)
	{
		UE_LOGF(LogSpawn, Warning, "Destroying %ls, which doesn't have a valid world pointer", *ThisActor->GetPathName());
	}

	// If already on list to be deleted, pretend the call was successful.
	// We don't want recursive calls to trigger destruction notifications multiple times.
	if (ThisActor->IsPendingKillPending())
	{
		return true;
	}

	// Never destroy the world settings actor. This used to be enforced by bNoDelete and is actually needed for 
	// seamless travel and network games.
	if (GetWorldSettings() == ThisActor)
	{
		return false;
	}

	// In-game deletion rules.
	if( IsGameWorld() )
	{
		// Note, for Standalone games, Actors should have Authority == ROLE_Authority.
		// In that sense, they'll be treated as Network Actors here.
		const bool bIsNetworkedActor = ThisActor->GetLocalRole() != ROLE_None;

		// Can't kill if wrong role.
		const bool bCanDestroyNetworkActor = ThisActor->GetLocalRole() == ROLE_Authority || bNetForce || ThisActor->bNetTemporary;
		if (bIsNetworkedActor && !bCanDestroyNetworkActor)
		{
			return false;
		}

		const bool bCanDestroyNonNetworkActor = !!CVarAllowDestroyNonNetworkActors.GetValueOnAnyThread();
		if (!bIsNetworkedActor && !bCanDestroyNonNetworkActor)
		{
			return false;
		}

		if (ThisActor->DestroyNetworkActorHandled())
		{
			// Network actor short circuited the destroy (network will cleanup properly)
			// Don't destroy PlayerControllers and BeaconClients
			return false;
		}

		if (ThisActor->IsActorBeginningPlay()
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			&& IsActorTickAndUserCallbacksEnabled()
#endif
		)
		{
			FSetActorWantsDestroyDuringBeginPlay SetActorWantsDestroyDuringBeginPlay(ThisActor);
			return true; // while we didn't actually destroy it now, we are going to, so tell the calling code it succeeded
		}
	}
	else
	{
		ThisActor->Modify();
	}

	// Prevent recursion
	FMarkActorIsBeingDestroyed MarkActorIsBeingDestroyed(ThisActor);

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	// Skip over all of the logic below (this won't cause an indent, so look for the ifdef)
	if (IsActorTickAndUserCallbacksEnabled())
	{
#endif

	// Notify the texture streaming manager about the destruction of this actor.
	IStreamingManager::Get().NotifyActorDestroyed( ThisActor );

	OnActorDestroyed.Broadcast(ThisActor);

	// Tell this actor it's about to be destroyed.
	ThisActor->Destroyed();

	// Detach this actor's children
	TArray<AActor*> AttachedActors;
	ThisActor->GetAttachedActors(AttachedActors);

	if (AttachedActors.Num() > 0)
	{
		TInlineComponentArray<USceneComponent*> SceneComponents;
		ThisActor->GetComponents(SceneComponents);

		for (TArray< AActor* >::TConstIterator AttachedActorIt(AttachedActors); AttachedActorIt; ++AttachedActorIt)
		{
			AActor* ChildActor = *AttachedActorIt;
			if (ChildActor != NULL)
			{
				for (USceneComponent* SceneComponent : SceneComponents)
				{
					ChildActor->DetachAllSceneComponents(SceneComponent, FDetachmentTransformRules::KeepWorldTransform);
				}
#if WITH_EDITOR
				if( GIsEditor )
				{
					GEngine->BroadcastLevelActorDetached(ChildActor, ThisActor);
				}
#endif
			}
		}
	}

	// Detach from anything we were attached to
	USceneComponent* RootComp = ThisActor->GetRootComponent();
	if( RootComp != nullptr && RootComp->GetAttachParent() != nullptr)
	{
		AActor* OldParentActor = RootComp->GetAttachParent()->GetOwner();
		if (OldParentActor)
		{
			// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
			OldParentActor->Modify(/*bAlwaysMarkDirty=*/false);
		}

		ThisActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);

#if WITH_EDITOR
		if( GIsEditor )
		{
			GEngine->BroadcastLevelActorDetached(ThisActor, OldParentActor);
		}
#endif
	}

#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	} // if (bEnableActorTickAndUserCallbacks)
#endif

	ThisActor->ClearComponentOverlaps();

	// If this actor has an owner, notify it that it has lost a child.
	if( ThisActor->GetOwner() )
	{
		ThisActor->SetOwner(NULL);
	}

	if (ULevel* const ActorLevel = ThisActor->GetLevel())
	{
		UDemoNetDriver* DemoDriver = nullptr;

		if (const FLevelCollection* LevelCollection = ActorLevel->GetCachedLevelCollection())
		{
			DemoDriver = LevelCollection->GetDemoNetDriver();
		}

		if (!DemoDriver)
		{
			DemoDriver = GetDemoNetDriver();
		}

		if (!DemoDriver || !DemoDriver->IsPlaying())
		{
			ActorLevel->CreateReplicatedDestructionInfo(ThisActor);
		}
	}

	UE::Net::Private::FNetPropertyConditionManager::Get().NotifyObjectDestroyed(ThisActor);

	// Notify net drivers that this actor has been destroyed.
	if (FWorldContext* Context = GEngine->GetWorldContextFromWorld(this))
	{
		for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(ThisActor))
			{
				Driver.NetDriver->NotifyActorDestroyed(ThisActor);
			}
		}
	}
	else if (WorldType != EWorldType::Inactive && !IsRunningCommandlet())
	{
		// If we are preloading this world, it's normal that we don't have a valid context yet.
		if (!UWorld::WorldTypePreLoadMap.Find(GetOuter()->GetFName()))
		{
			// Inactive worlds do not have a world context, otherwise only worlds in the middle of seamless travel should have no context,
			// and in that case, we shouldn't be destroying actors on them until they have become the current world (i.e. CopyWorldData has been called)
			UE_LOGF(LogSpawn, Warning, "UWorld::DestroyActor: World has no context! World: %ls, Actor: %ls", *GetName(), *ThisActor->GetPathName());
		}
	}

	// Remove the actor from the actor list.
	RemoveActor( ThisActor, bShouldModifyLevel );

	// Invalidate the lighting cache in the Editor.  We need to check for GIsEditor as play has not begun in network game and objects get destroyed on switching levels
	if ( GIsEditor )
	{
		if (!IsGameWorld())
		{
			ThisActor->InvalidateLightingCache();
		}
		
#if WITH_EDITOR
		GEngine->BroadcastLevelActorDeleted(ThisActor);
#endif
	}

	OnActorRemovedFromWorld.Broadcast(ThisActor);

	// Clean up the actor's components.
	ThisActor->UnregisterAllComponents();

	// Mark the actor and its direct components as pending kill.
	ThisActor->MarkAsGarbage();
	ThisActor->MarkPackageDirty();
	ThisActor->MarkComponentsAsGarbage();

	// Unregister the actor's tick function
	const bool bRegisterTickFunctions = false;
	const bool bIncludeComponents = true;
	ThisActor->RegisterAllActorTickFunctions(bRegisterTickFunctions, bIncludeComponents);

	// Return success.
	return true;
}

/*-----------------------------------------------------------------------------
	Player spawning.
-----------------------------------------------------------------------------*/

APlayerController* UWorld::SpawnPlayActor(UPlayer* NewPlayer, ENetRole RemoteRole, const FURL& InURL, const FUniqueNetIdRepl& UniqueId, FString& Error, uint8 InNetPlayerIndex, int32 InLocalPlayerIdentifier)
{
	Error = TEXT("");

	// Make the option string.
	FString Options;
	for (int32 i = 0; i < InURL.Op.Num(); i++)
	{
		Options += TEXT('?');
		Options += InURL.Op[i];
	}

	if (AGameModeBase* const GameMode = GetAuthGameMode())
	{
		// Give the GameMode a chance to accept the login
		APlayerController* const NewPlayerController = GameMode->Login(NewPlayer, RemoteRole, *InURL.Portal, Options, UniqueId, Error);
		if (NewPlayerController == NULL)
		{
			UE_LOGF(LogSpawn, Warning, "Login failed: %ls", *Error);
			return NULL;
		}

		if (UNetConnection* Connection = Cast<UNetConnection>(NewPlayer))
		{
			NewPlayerController->SetClientHandshakeId(Connection->GetClientHandshakeId());
		}

		UE_LOGF(LogSpawn, Log, "%ls got player %ls [%ls]", *NewPlayerController->GetName(), *NewPlayer->GetName(), UniqueId.IsValid() ? *UniqueId->ToString() : TEXT("Invalid"));

		// Possess the newly-spawned player.
		NewPlayerController->NetPlayerIndex = InNetPlayerIndex;
		NewPlayerController->SetRole(ROLE_Authority);
		NewPlayerController->SetReplicates(RemoteRole != ROLE_None);
		NewPlayerController->SetLocalPlayerConnectionIdentifier(InLocalPlayerIdentifier);
		if (RemoteRole == ROLE_AutonomousProxy)
		{
			NewPlayerController->SetAutonomousProxy(true);
		}
		NewPlayerController->SetPlayer(NewPlayer);
		GameMode->PostLogin(NewPlayerController);
		return NewPlayerController;
	}

	UE_LOGF(LogSpawn, Warning, "Login failed: No game mode set.");
	return nullptr;
}

/*-----------------------------------------------------------------------------
	Level actor moving/placing.
-----------------------------------------------------------------------------*/

bool UWorld::FindTeleportSpot(const AActor* TestActor, FVector& TestLocation, FRotator TestRotation)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UWorld_FindTeleportSpot);

	if( !TestActor || !TestActor->GetRootComponent() )
	{
		return true;
	}
	FVector Adjust(0.f);

	const FVector OriginalTestLocation = TestLocation;

	// check if fits at desired location
	if( !EncroachingBlockingGeometry(TestActor, TestLocation, TestRotation, &Adjust) )
	{
		return true;
	}

	if ( Adjust.IsNearlyZero() )
	{
		// EncroachingBlockingGeometry fails to produce adjustment if movement component's target, or root component,
		// fallback to a box when calling GetCollisionShape (UPrimitiveComponent's behaviour).
		// This would occur if SkeletalMeshComponent was root, instead of capsule, for example.
		// Here we test for these cases and warn user why this function is failing.
		UMovementComponent* const MoveComponent = TestActor->FindComponentByClass<UMovementComponent>();
		if (MoveComponent && MoveComponent->UpdatedPrimitive)
		{
			UPrimitiveComponent* const PrimComponent = MoveComponent->UpdatedPrimitive;
			FCollisionShape shape = PrimComponent->GetCollisionShape();
			if (shape.IsBox() && (Cast<UBoxComponent>(PrimComponent) == nullptr))
			{
				UE_LOGF(LogPhysics, Warning, "UWorld::FindTeleportSpot called with an actor that is intersecting geometry. Failed to find new location likely due to "
					"movement component's 'UpdatedComponent' not being a collider component.");
			}
		}
		else
		{
			USceneComponent* const RootComponent = TestActor->GetRootComponent();
			UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(RootComponent);
			if(PrimComponent != nullptr)
			{
				FCollisionShape shape = PrimComponent->GetCollisionShape();
				if (shape.IsBox() && (Cast<UBoxComponent>(PrimComponent) == nullptr))
				{
					UE_LOGF(LogPhysics, Warning, "UWorld::FindTeleportSpot called with an actor that is intersecting geometry. Failed to find new location likely due to "
						" actor's root component not being a collider component.");
				}
			}
		}

		// Reset in case Adjust is not actually zero
		TestLocation = OriginalTestLocation;
		return false;
	}

	// first do only Z
	const FVector::FReal ZeroThreshold = UE_KINDA_SMALL_NUMBER;
	const bool bZeroZ = FMath::IsNearlyZero(Adjust.Z, ZeroThreshold);
	if (!bZeroZ)
	{
		TestLocation.Z += Adjust.Z;
		if( !EncroachingBlockingGeometry(TestActor, TestLocation, TestRotation) )
		{
			return true;
		}

		TestLocation = OriginalTestLocation;
	}

	// now try just XY
	const bool bZeroX = FMath::IsNearlyZero(Adjust.X, ZeroThreshold);
	const bool bZeroY = FMath::IsNearlyZero(Adjust.Y, ZeroThreshold);
	if (!bZeroX || !bZeroY)
	{
		const float X = bZeroX ? 0.f : Adjust.X;
		const float Y = bZeroY ? 0.f : Adjust.Y;
		FVector Adjustments[8];
		Adjustments[0] = FVector(X, Y, 0);

		// If initially spawning allow testing a few permutations (though this needs improvement).
		// During play only test the first adjustment, permuting axes could put the location on other sides of geometry.
		const int32 Iterations = (TestActor->HasActorBegunPlay() ? 1 : 8);

		if (Iterations > 1)
		{
			if (!bZeroX && !bZeroY)
			{
				Adjustments[1] = FVector(-X,  Y, 0);
				Adjustments[2] = FVector( X, -Y, 0);
				Adjustments[3] = FVector(-X, -Y, 0);
				Adjustments[4] = FVector( Y,  X, 0);
				Adjustments[5] = FVector(-Y,  X, 0);
				Adjustments[6] = FVector( Y, -X, 0);
				Adjustments[7] = FVector(-Y, -X, 0);
			}
			else
			{
				// If either X or Y was zero, the permutations above would result in only 4 unique attempts.
				Adjustments[1] = FVector(-X, -Y, 0);
				Adjustments[2] = FVector( Y,  X, 0);
				Adjustments[3] = FVector(-Y, -X, 0);
				// Mirror the dominant non-zero value
				const float D = bZeroY ? X : Y;
				Adjustments[4] = FVector( D,  D, 0);
				Adjustments[5] = FVector( D, -D, 0);
				Adjustments[6] = FVector(-D,  D, 0);
				Adjustments[7] = FVector(-D, -D, 0);
			}
		}

		for (int i = 0; i < Iterations; ++i)
		{
			TestLocation = OriginalTestLocation + Adjustments[i];
			if (!EncroachingBlockingGeometry(TestActor, TestLocation, TestRotation))
			{
				return true;
			}
		}

		// Try XY adjustment including Z. Note that even with only 1 iteration, this will still try the full proposed (X,Y,Z) adjustment.
		if (!bZeroZ)
		{
			for (int i = 0; i < Iterations; ++i)
			{
				TestLocation = OriginalTestLocation + Adjustments[i];
				TestLocation.Z += Adjust.Z;
				if (!EncroachingBlockingGeometry(TestActor, TestLocation, TestRotation))
				{
					return true;
				}
			}
		}
	}

	// Don't write out the last failed test location, we promised to only if we find a good spot, in case the caller re-uses the original input.
	TestLocation = OriginalTestLocation;
	return false;
}

/** Tests shape components more efficiently than the with-adjustment case, but does less-efficient ppr-poly collision for meshes. */
static bool ComponentEncroachesBlockingGeometry_NoAdjustment(UWorld const* World, AActor const* TestActor, UPrimitiveComponent const* PrimComp, FTransform const& TestWorldTransform, const TArray<AActor*>& IgnoreActors)
{	
	float const Epsilon = CVarEncroachEpsilon.GetValueOnGameThread();
	
	if (World && PrimComp)
	{
		bool bFoundBlockingHit = false;
		
		ECollisionChannel const BlockingChannel = PrimComp->GetCollisionObjectType();
		FCollisionShape const CollisionShape = PrimComp->GetCollisionShape(-Epsilon);

		if (CollisionShape.IsBox() && (Cast<UBoxComponent>(PrimComp) == nullptr))
		{
			// we have a bounding box not for a box component, which means this was the fallback aabb
			// since we don't need the penetration info, go ahead and test the component itself for overlaps, which is more accurate
			if (PrimComp->IsRegistered())
			{
				// must be registered
				TArray<FOverlapResult> Overlaps;
				FComponentQueryParams Params(SCENE_QUERY_STAT(ComponentEncroachesBlockingGeometry_NoAdjustment), TestActor);
				FCollisionResponseParams ResponseParams;
				PrimComp->InitSweepCollisionParams(Params, ResponseParams);
				Params.AddIgnoredActors(IgnoreActors);
				return World->ComponentOverlapMultiByChannel(Overlaps, PrimComp, TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, Params);
			}
			else
			{
				UE_LOGF(LogPhysics, Log, "Components must be registered in order to be used in a ComponentOverlapMulti call. PriComp: %ls TestActor: %ls", *PrimComp->GetName(), *TestActor->GetName());
				return false;
			}
		}
		else
		{
			FCollisionQueryParams Params(SCENE_QUERY_STAT(ComponentEncroachesBlockingGeometry_NoAdjustment), false, TestActor);
			FCollisionResponseParams ResponseParams;
			PrimComp->InitSweepCollisionParams(Params, ResponseParams);
			Params.AddIgnoredActors(IgnoreActors);
			return World->OverlapBlockingTestByChannel(TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, CollisionShape, Params, ResponseParams);
		}
	}

	return false;
}

static FVector CombineAdjustments(FVector CurrentAdjustment, FVector AdjustmentToAdd)
{
	// remove the part of the new adjustment that's parallel to the current adjustment
	if (CurrentAdjustment.IsZero())
	{
		return AdjustmentToAdd;
	}

	FVector Projection = AdjustmentToAdd.ProjectOnTo(CurrentAdjustment);
	Projection = Projection.GetClampedToMaxSize(CurrentAdjustment.Size());

	FVector OrthogalAdjustmentToAdd = AdjustmentToAdd - Projection;
	return CurrentAdjustment + OrthogalAdjustmentToAdd;
}

/** Tests shape components less efficiently than the no-adjustment case, but does quicker aabb collision for meshes. */
static bool ComponentEncroachesBlockingGeometry_WithAdjustment(UWorld const* World, AActor const* TestActor, UPrimitiveComponent const* PrimComp, FTransform const& TestWorldTransform, FVector& OutProposedAdjustment, const TArray<AActor*>& IgnoreActors)
{
	// init our output
	OutProposedAdjustment = FVector::ZeroVector;

	float const Epsilon = CVarEncroachEpsilon.GetValueOnGameThread();

	if (World && PrimComp)
	{
		bool bFoundBlockingHit = false;
		bool bComputePenetrationAdjustment = true;
		
		TArray<FOverlapResult> Overlaps;
		ECollisionChannel const BlockingChannel = PrimComp->GetCollisionObjectType();
		FCollisionShape const CollisionShape = PrimComp->GetCollisionShape(-Epsilon);

		if (CollisionShape.IsBox() && (Cast<UBoxComponent>(PrimComp) == nullptr))
		{
			// we have a bounding box not for a box component, which means this was the fallback aabb
			// so lets test the actual component instead of it's aabb
			// note we won't get penetration adjustment but that's ok
			if (PrimComp->IsRegistered())
			{
				// must be registered
				FComponentQueryParams Params(SCENE_QUERY_STAT(ComponentEncroachesBlockingGeometry_WithAdjustment), TestActor);
				FCollisionResponseParams ResponseParams;
				PrimComp->InitSweepCollisionParams(Params, ResponseParams);
				Params.AddIgnoredActors(IgnoreActors);
				bFoundBlockingHit = World->ComponentOverlapMultiByChannel(Overlaps, PrimComp, TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, Params);
				bComputePenetrationAdjustment = false;
			}
			else
			{
				UE_LOGF(LogPhysics, Log, "Components must be registered in order to be used in a ComponentOverlapMulti call. PriComp: %ls TestActor: %ls", *PrimComp->GetName(), *TestActor->GetName());
			}
		}
		else
		{
			// overlap our shape
			FCollisionQueryParams Params(SCENE_QUERY_STAT(ComponentEncroachesBlockingGeometry_WithAdjustment), false, TestActor);
			FCollisionResponseParams ResponseParams;
			PrimComp->InitSweepCollisionParams(Params, ResponseParams);
			Params.AddIgnoredActors(IgnoreActors);
			bFoundBlockingHit = World->OverlapMultiByChannel(Overlaps, TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, CollisionShape, Params, ResponseParams);
		}

		// compute adjustment
		if (bFoundBlockingHit && bComputePenetrationAdjustment)
		{
			// if encroaching, add up all the MTDs of overlapping shapes
			FMTDResult MTDResult;
			uint32 NumBlockingHits = 0;
			for (int32 HitIdx = 0; HitIdx < Overlaps.Num(); HitIdx++)
			{
				UPrimitiveComponent* const OverlapComponent = Overlaps[HitIdx].Component.Get();
				// first determine closest impact point along each axis
				if (OverlapComponent && OverlapComponent->GetCollisionResponseToChannel(BlockingChannel) == ECR_Block)
				{
					NumBlockingHits++;
					FCollisionShape const NonShrunkenCollisionShape = PrimComp->GetCollisionShape();
					const FBodyInstance* OverlapBodyInstance = OverlapComponent->GetBodyInstance(NAME_None, true, Overlaps[HitIdx].GetItemIndex());
					bool bSuccess = OverlapBodyInstance && OverlapBodyInstance->OverlapTest(TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), NonShrunkenCollisionShape, &MTDResult);
					if (bSuccess)
					{
						OutProposedAdjustment += MTDResult.Direction * MTDResult.Distance;
					}
					else
					{
						UE_LOGF(LogPhysics, Log, "OverlapTest says we are overlapping, yet MTD says we're not. Something is wrong");
						// It's not safe to use a partial result, that could push us out to an invalid location (like the other side of a wall).
						OutProposedAdjustment = FVector::ZeroVector;
						return true;
					}

					// #hack: sometimes for boxes, physx returns a 0 MTD even though it reports a contact (returns true)
					// to get around this, let's go ahead and test again with the epsilon-shrunken collision shape to see if we're really in 
					// the clear.
					if (bSuccess && FMath::IsNearlyZero(MTDResult.Distance))
					{
						FCollisionShape const ShrunkenCollisionShape = PrimComp->GetCollisionShape(-Epsilon);
						bSuccess = OverlapBodyInstance && OverlapBodyInstance->OverlapTest(TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), ShrunkenCollisionShape, &MTDResult);
						if (bSuccess)
						{
							OutProposedAdjustment += MTDResult.Direction * MTDResult.Distance;
						}
						else
						{
							// Ignore this overlap.
							UE_LOGF(LogPhysics, Log, "OverlapTest says we are overlapping, yet MTD says we're not (with smaller shape). Ignoring this overlap.");
							NumBlockingHits--;
							continue;
						}
					}
				}
			}

			// See if we chose to invalidate all of our supposed "blocking hits".
			if (NumBlockingHits == 0)
			{
				OutProposedAdjustment = FVector::ZeroVector;
				bFoundBlockingHit = false;
			}
		}

		return bFoundBlockingHit;
	}

	return false;
}

/** Tests if the given component overlaps any blocking geometry if it were placed at the given world transform, optionally returns a suggested translation to get the component away from its overlaps. */
static bool ComponentEncroachesBlockingGeometry(UWorld const* World, AActor const* TestActor, UPrimitiveComponent const* PrimComp, FTransform const& TestWorldTransform, FVector* OutProposedAdjustment, const TArray<AActor*>& IgnoreActors)
{
	return OutProposedAdjustment
		? ComponentEncroachesBlockingGeometry_WithAdjustment(World, TestActor, PrimComp, TestWorldTransform, *OutProposedAdjustment, IgnoreActors)
		: ComponentEncroachesBlockingGeometry_NoAdjustment(World, TestActor, PrimComp, TestWorldTransform, IgnoreActors);
}


// perf note: this is faster if ProposedAdjustment is null, since it can early out on first penetration
bool UWorld::EncroachingBlockingGeometry(const AActor* TestActor, FVector TestLocation, FRotator TestRotation, FVector* ProposedAdjustment)
{
	if (TestActor == nullptr)
	{
		return false;
	}

	USceneComponent* const RootComponent = TestActor->GetRootComponent();
	if (RootComponent == nullptr)
	{
		return false;
	}

	bool bFoundEncroacher = false;

	FVector TotalAdjustment(0.f);
	FTransform const TestRootToWorld = FTransform(TestRotation, TestLocation);
	FTransform const WorldToOldRoot = RootComponent->GetComponentToWorld().Inverse();

	UMovementComponent* const MoveComponent = TestActor->FindComponentByClass<UMovementComponent>();
	IMovementInterface* MovementInterface = nullptr;
	UPrimitiveComponent* MovedPrimComp = nullptr;
	
	if (MoveComponent)
	{
		MovedPrimComp = MoveComponent->UpdatedPrimitive;
	}
	else
	{
		MovementInterface = TestActor->FindComponentByInterface<IMovementInterface>();
		MovedPrimComp = MovementInterface ? Cast<UPrimitiveComponent>(MovementInterface->GetUpdatedObject()) : nullptr;
	}
	
	if (MovedPrimComp)
	{
		// This actor has a movement component, which we interpret to mean that this actor has a primary component being swept around
		// the world, and that component is the only one we care about encroaching (since the movement code will happily embedding
		// other components in the world during movement updates)
		if (MovedPrimComp->IsQueryCollisionEnabled())
		{
			// might not be the root, so we need to compute the transform
			FTransform const CompToRoot = MovedPrimComp->GetComponentToWorld() * WorldToOldRoot;
			FTransform const CompToNewWorld = CompToRoot * TestRootToWorld;

			TArray<AActor*> ChildActors;
			TestActor->GetAllChildActors(ChildActors);

			if (ComponentEncroachesBlockingGeometry(this, TestActor, MovedPrimComp, CompToNewWorld, ProposedAdjustment, ChildActors))
			{
				if (ProposedAdjustment == nullptr)
				{
					// don't need an adjustment and we know we are overlapping, so we can be done
					return true;
				}
				else
				{
					TotalAdjustment = *ProposedAdjustment;
				}

				bFoundEncroacher = true;
			}
		}
	}
	else
	{
		bool bFetchedChildActors = false;
		TArray<AActor*> ChildActors;

		// This actor does not have a movement component, so we'll assume all components are potentially important to keep out of the world
		UPrimitiveComponent* const RootPrimComp = Cast<UPrimitiveComponent>(RootComponent);
		if (RootPrimComp && RootPrimComp->IsQueryCollisionEnabled())
		{
			TestActor->GetAllChildActors(ChildActors);
			bFetchedChildActors = true;

			if (ComponentEncroachesBlockingGeometry(this, TestActor, RootPrimComp, TestRootToWorld, ProposedAdjustment, ChildActors))
			{
				if (ProposedAdjustment == nullptr)
				{
					// don't need an adjustment and we know we are overlapping, so we can be done
					return true;
				}
				else
				{
					TotalAdjustment = *ProposedAdjustment;
				}

				bFoundEncroacher = true;
			}
		}

		// now test all colliding children for encroachment
		TArray<USceneComponent*> Children;
		RootComponent->GetChildrenComponents(true, Children);

		for (USceneComponent* Child : Children)
		{
			if (Child->IsQueryCollisionEnabled())
			{
				UPrimitiveComponent* const PrimComp = Cast<UPrimitiveComponent>(Child);
				if (PrimComp)
				{
					FTransform const CompToRoot = Child->GetComponentToWorld() * WorldToOldRoot;
					FTransform const CompToNewWorld = CompToRoot * TestRootToWorld;

					if (!bFetchedChildActors)
					{
						TestActor->GetAllChildActors(ChildActors);
						bFetchedChildActors = true;
					}

					if (ComponentEncroachesBlockingGeometry(this, TestActor, PrimComp, CompToNewWorld, ProposedAdjustment, ChildActors))
					{
						if (ProposedAdjustment == nullptr)
						{
							// don't need an adjustment and we know we are overlapping, so we can be done
							return true;
						}

						TotalAdjustment = CombineAdjustments(TotalAdjustment, *ProposedAdjustment);
						bFoundEncroacher = true;
					}
				}
			}
		}
	}

	// copy over total adjustment
	if (ProposedAdjustment)
	{
		*ProposedAdjustment = TotalAdjustment;
	}

	return bFoundEncroacher;
}


void UWorld::LoadSecondaryLevels(bool bForce, TSet<FName>* PackageNamesToSkip)
{
	check( GIsEditor );

	// streamingServer
	// Only load secondary levels in the Editor, and not for commandlets.
	if( (!IsRunningCommandlet() || bForce)
	// Don't do any work for world info actors that are part of secondary levels being streamed in! 
	&& !IsAsyncLoading())
	{
		for( int32 LevelIndex=0; LevelIndex<StreamingLevels.Num(); LevelIndex++ )
		{
			ULevelStreaming* const StreamingLevel = StreamingLevels[LevelIndex];
			if( StreamingLevel )
			{
				bool bSkipFile = false;
				// If we are cooking don't cook sub levels multiple times if they've already been cooked
				FString PackageFilename;
				const FString StreamingLevelWorldAssetPackageName = StreamingLevel->GetWorldAssetPackageName();
				if (PackageNamesToSkip)
				{
					bSkipFile |= PackageNamesToSkip->Contains(StreamingLevel->GetWorldAssetPackageFName());
				}

				bool bAlreadyLoaded = false;
				UPackage* LevelPackage = FindObject<UPackage>(NULL, *StreamingLevelWorldAssetPackageName,EFindObjectFlags::ExactClass);
				// don't need to do any extra work if the level is already loaded
				if ( LevelPackage && LevelPackage->IsFullyLoaded() ) 
				{
					bAlreadyLoaded = true;
				}

				if ( !bSkipFile )
				{
					if ( !bAlreadyLoaded )
					{
						bool bLoadedLevelPackage = false;
						const FName StreamingLevelWorldAssetPackageFName = StreamingLevel->GetWorldAssetPackageFName();
						// Load the package and find the world object.
						if( FPackageName::IsShortPackageName(StreamingLevelWorldAssetPackageFName) == false )
						{
							ULevel::StreamedLevelsOwningWorld.Add(StreamingLevelWorldAssetPackageFName, this);
							LevelPackage = LoadPackage( NULL, *StreamingLevelWorldAssetPackageName, LOAD_None );
							ULevel::StreamedLevelsOwningWorld.Remove(StreamingLevelWorldAssetPackageFName);

							if( LevelPackage )
							{
								bLoadedLevelPackage = true;

								// Find the world object in the loaded package.
								UWorld* LoadedWorld	= UWorld::FindWorldInPackage(LevelPackage);
								// If the world was not found, it could be a redirector to a world. If so, follow it to the destination world.
								if (!LoadedWorld)
								{
									LoadedWorld = UWorld::FollowWorldRedirectorInPackage(LevelPackage);
								}
								check(LoadedWorld);

								if ( !LevelPackage->IsFullyLoaded() )
								{
									// LoadedWorld won't be serialized as there's a BeginLoad on the stack so we manually serialize it here.
									check( LoadedWorld->GetLinker() );
									LoadedWorld->GetLinker()->Preload( LoadedWorld );
								}


								// Keep reference to prevent garbage collection.
								check( LoadedWorld->PersistentLevel );

								LoadedWorld->PersistentLevel->HandleLegacyMapBuildData();

								ULevel* NewLoadedLevel = LoadedWorld->PersistentLevel;
								NewLoadedLevel->OwningWorld = this;

								FStreamingLevelPrivateAccessor::SetLoadedLevel(StreamingLevel, NewLoadedLevel);
							}
						}
						else
						{
							UE_LOGF(LogSpawn, Warning, "Streaming level uses short package name (%ls). Level will not be loaded.", *StreamingLevelWorldAssetPackageName);
						}

						// Remove this level object if the file couldn't be found.
						if ( !bLoadedLevelPackage )
						{
							RemoveStreamingLevelAt(LevelIndex--);
							MarkPackageDirty();
						}
					}
					else
					{
						UWorld* LoadedWorld = UWorld::FindWorldInPackage(LevelPackage);
						// If the world was not found, it could be a redirector to a world. If so, follow it to the destination world.
						if (!LoadedWorld)
						{
							LoadedWorld = UWorld::FollowWorldRedirectorInPackage(LevelPackage);
						}
						if (LoadedWorld)
						{
							if (ULevel* WorldLevel = LoadedWorld->PersistentLevel)
							{
								ULevel* LoadedLevel = StreamingLevel->GetLoadedLevel();
								if (!LoadedLevel)
								{
									FStreamingLevelPrivateAccessor::SetLoadedLevel(StreamingLevel, WorldLevel);
									if (WorldLevel->OwningWorld != this)
									{
										WorldLevel->OwningWorld = this;
									}
								}
								else if (LoadedLevel != WorldLevel)
								{
									UE_LOGF(LogSpawn, Warning, "Streaming level already has a loaded level, but it differs from the one in the associated package (%ls).", *StreamingLevelWorldAssetPackageName);
								}
							}
							else
							{
								UE_LOGF(LogSpawn, Warning, "Streaming level already has a loaded level, but the world in the associated package (%ls) does not have its persistent level set.", *StreamingLevelWorldAssetPackageName);
							}
						}
						else
						{
							UE_LOGF(LogSpawn, Warning, "Streaming level already has a loaded level, but the associated package (%ls) seems to contain no world.", *StreamingLevelWorldAssetPackageName);
						}
					}
				}
			}
		}
	}
}

/** Utility for returning the ULevelStreaming object for a particular sub-level, specified by package name */
ULevelStreaming* UWorld::GetLevelStreamingForPackageName(FName InPackageName)
{
	// iterate over each level streaming object
	for (ULevelStreaming* LevelStreaming : StreamingLevels)
	{
		// see if name matches
		if (LevelStreaming && LevelStreaming->GetWorldAssetPackageFName() == InPackageName)
		{
			// it doesn't, return this one
			return LevelStreaming;
		}
	}

	// failed to find one
	return nullptr;
}

#if WITH_EDITOR


void UWorld::RefreshStreamingLevels( const TArray<class ULevelStreaming*>& InLevelsToRefresh )
{
	// Reassociate levels in case we changed streaming behavior. Editor-only!
	if( GIsEditor )
	{
		// Load and associate levels if necessary.
		FlushLevelStreaming();

		bIsRefreshingStreamingLevels = true;
		bool bLevelRefreshed = false;
		// Remove all currently visible levels.
		for (ULevelStreaming* StreamingLevel : InLevelsToRefresh)
		{
			ULevel* LoadedLevel = StreamingLevel ? StreamingLevel->GetLoadedLevel() : nullptr;

			if( LoadedLevel && LoadedLevel->bIsVisible )
			{
				RemoveFromWorld( LoadedLevel );
				FStreamingLevelPrivateAccessor::OnLevelAdded(StreamingLevel); // Sketchy way to get the CurrentState correctly set to LoadedNotVisible
				StreamingLevelsToConsider.Add(StreamingLevel); // Need to ensure this level is reconsidered during the flush to get it made visible again
				bLevelRefreshed = true;
			}
		}

		if (bLevelRefreshed)
		{
			// Load and associate levels if necessary.
			FlushLevelStreaming();
		}

		// Update the level browser so it always contains valid data
		FEditorSupportDelegates::WorldChange.Broadcast();

		bIsRefreshingStreamingLevels = false;
	}
}

void UWorld::RefreshStreamingLevels()
{
	RefreshStreamingLevels( StreamingLevels );
}

void UWorld::IssueEditorLoadWarnings()
{
	float TotalLoadTimeFromFixups = 0;

	for (int32 LevelIndex = 0; LevelIndex < Levels.Num(); LevelIndex++)
	{
		ULevel* Level = Levels[LevelIndex];

		if (Level->FixupOverrideVertexColorsCount > 0)
		{
			const double LevelFixupTime  = ((double)Level->FixupOverrideVertexColorsTimeMS) / 1000.0;
			const uint32 LevelFixupCount = Level->FixupOverrideVertexColorsCount;

			TotalLoadTimeFromFixups += LevelFixupTime;
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("LoadTime"), FText::FromString(FString::Printf(TEXT("%.1fs"), LevelFixupTime)));
			Arguments.Add(TEXT("NumComponents"), FText::FromString(FString::Printf(TEXT("%u"), LevelFixupCount)));
			Arguments.Add(TEXT("LevelName"), FText::FromString(Level->GetOutermost()->GetName()));
			
			FMessageLog("MapCheck").Info()
				->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_RepairedPaintedVertexColors", "Repaired painted vertex colors in {LoadTime} for {NumComponents} components in {LevelName}.  Resave map to fix." ), Arguments ) ))
				->AddToken(FMapErrorToken::Create(FMapErrors::RepairedPaintedVertexColors));
		}
	}

	if (TotalLoadTimeFromFixups > 0)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("LoadTime"), FText::FromString(FString::Printf(TEXT("%.1fs"), TotalLoadTimeFromFixups)));
			
		FMessageLog("MapCheck").Warning()
			->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_SpentXRepairingPaintedVertexColors", "Spent {LoadTime} repairing painted vertex colors due to static mesh re-imports!  This will happen every load until the maps are resaved." ), Arguments ) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::RepairedPaintedVertexColors));
	}
}

#endif // WITH_EDITOR


AAudioVolume* UWorld::GetAudioSettings( const FVector& ViewLocation, FReverbSettings* OutReverbSettings, FInteriorSettings* OutInteriorSettings ) const
{
	// Find the highest priority volume encompassing the current view location.
	for (AAudioVolume* Volume : AudioVolumes)
	{
		// Volume encompasses, break out of loop.
		if (Volume->GetEnabled() && Volume->EncompassesPoint(ViewLocation))
		{
			if( OutReverbSettings )
			{
				*OutReverbSettings = Volume->GetReverbSettings();
			}

			if( OutInteriorSettings )
			{
				*OutInteriorSettings = Volume->GetInteriorSettings();
			}
			return Volume;
		}
	}

	// If first level is a FakePersistentLevel (see CommitMapChange for more info)
	// then use its world info for reverb settings
	AWorldSettings* CurrentWorldSettings = GetWorldSettings(true);

	if( OutReverbSettings )
	{
		*OutReverbSettings = CurrentWorldSettings->DefaultReverbSettings;
	}

	if( OutInteriorSettings )
	{
		*OutInteriorSettings = CurrentWorldSettings->DefaultAmbientZoneSettings;
	}

	return nullptr;
}


/**
 * Sets bMapNeedsLightingFullyRebuild to the specified value.  Marks the worldsettings package dirty if the value changed.
 *
 * @param	bInMapNeedsLightingFullyRebuild			The new value.
 */
void UWorld::SetMapNeedsLightingFullyRebuilt(int32 InNumLightingUnbuiltObjects, int32 InNumUnbuiltReflectionCaptures)
{
#if !UE_BUILD_SHIPPING
	AWorldSettings* WorldSettings = GetWorldSettings();
	if (IsStaticLightingAllowed() && WorldSettings && !WorldSettings->bForceNoPrecomputedLighting)
	{
		check(IsInGameThread());
		if ((NumLightingUnbuiltObjects != InNumLightingUnbuiltObjects && (NumLightingUnbuiltObjects == 0 || InNumLightingUnbuiltObjects == 0))
			|| (NumUnbuiltReflectionCaptures != InNumUnbuiltReflectionCaptures && (NumUnbuiltReflectionCaptures == 0 || InNumUnbuiltReflectionCaptures == 0)))
		{
			// Save the lighting invalidation for transactions.
			Modify(false);
		}

		NumLightingUnbuiltObjects = InNumLightingUnbuiltObjects;
		NumUnbuiltReflectionCaptures = InNumUnbuiltReflectionCaptures;

		// Update last time unbuilt lighting was encountered.
		if (NumLightingUnbuiltObjects > 0)
		{
			LastTimeUnbuiltLightingWasEncountered = FApp::GetCurrentTime();
		}
	}
#endif
}

#undef LOCTEXT_NAMESPACE
