// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Level.cpp: Level-related functions
=============================================================================*/

#include "Engine/LevelActorContainer.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/UObjectClusters.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelActorContainer)

DEFINE_LOG_CATEGORY_STATIC(LogLevelActorContainer, Log, All);

/**
* Handles UObject references found by TFastReferenceCollector
*/
class FActorClusterReferenceProcessor : public FSimpleReferenceProcessorBase
{
	int32 ClusterRootIndex;
	FUObjectCluster& Cluster;
	ULevel* ParentLevel;
	UPackage* ParentLevelPackage;

public:

	FActorClusterReferenceProcessor(int32 InClusterRootIndex, FUObjectCluster& InCluster, ULevel* InParentLevel)
		: ClusterRootIndex(InClusterRootIndex)
		, Cluster(InCluster)
		, ParentLevel(InParentLevel)
	{
		ParentLevelPackage = ParentLevel->GetOutermost();
	}

	FORCENOINLINE bool CanAddToCluster(UObject* Object)
	{
		if (!Object->IsIn(ParentLevelPackage))
		{
			// No external references are allowed in level clusters
			return false;
		}
		else if (!Object->IsIn(ParentLevel))
		{
			// If the object is in the same package but is not in the level we don't want it either.
			return false;
		}
		if (Object->IsA(ULevel::StaticClass()) || Object->IsA(UWorld::StaticClass()))
		{
			// And generally, no levels or worlds
			return false;
		}
		return Object->CanBeInCluster();
	}

	/**
	* Adds an object to cluster (if possible)
	*
	* @param ObjectIndex UObject index in GUObjectArray
	* @param ObjectItem UObject's entry in GUObjectArray
	* @param Obj The object to add to cluster
	* @param ObjectsToSerialize An array of remaining objects to serialize (Obj must be added to it if Obj can be added to cluster)
	* @param bOuterAndClass If true, the Obj's Outer and Class will also be added to the cluster
	*/
	void AddObjectToCluster(int32 ObjectIndex, FUObjectItem* ObjectItem, UObject* Obj, FWorkerContext& Context, bool bOuterAndClass)
	{
		// If we haven't finished loading, we can't be sure we know all the references
		check(!Obj->HasAnyFlags(RF_NeedLoad));
		check(ObjectItem->GetOwnerIndex() == 0 || ObjectItem->GetOwnerIndex() == ClusterRootIndex || ObjectIndex == ClusterRootIndex);
		check(Obj->CanBeInCluster());
		if (ObjectIndex != ClusterRootIndex && ObjectItem->GetOwnerIndex() == 0 && !GUObjectArray.IsDisregardForGC(Obj) && !Obj->IsRooted())
		{
			Context.ObjectsToSerialize.Add<Options>(Obj);
			check(!ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot));
			ObjectItem->SetOwnerIndex(ClusterRootIndex);
			Cluster.Objects.Add(ObjectIndex);

			if (bOuterAndClass)
			{
				UObject* ObjOuter = Obj->GetOuter();
				if (CanAddToCluster(ObjOuter))
				{
					HandleTokenStreamObjectReference(Context, Obj, ObjOuter, UE::GC::EMemberlessId::Outer, EOrigin::Other, true);
				}
				else
				{
					Cluster.MutableObjects.AddUnique(GUObjectArray.ObjectToIndex(ObjOuter));
				}
				if (!Obj->GetClass()->HasAllClassFlags(CLASS_Native))
				{
					UObject* ObjectClass = Obj->GetClass();
					HandleTokenStreamObjectReference(Context, Obj, ObjectClass, UE::GC::EMemberlessId::Class, EOrigin::Other, true);
					UObject* ObjectClassOuter = Obj->GetClass()->GetOuter();
					HandleTokenStreamObjectReference(Context, Obj, ObjectClassOuter, UE::GC::EMemberlessId::ClassOuter, EOrigin::Other, true);
				}
			}
		}
	}

	/**
	* Handles UObject reference from the token stream. Performance is critical here so we're FORCEINLINING this function.
	*
	* @param Context Context of the reference collection
	* @param ReferencingObject Object referencing the object to process.
	* @param Object Object being processed
	* @param MemberId Index to the token stream where the reference was found.
	* @param Origin Declares if a schema represents a blueprint generated type
	* @param bAllowReferenceElimination True if reference elimination is allowed (ignored when constructing clusters).
	*/
	FORCEINLINE void HandleTokenStreamObjectReference(FWorkerContext& Context, UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination)
	{
		if (Object)
		{
			// If we haven't finished loading, we can't be sure we know all the references
			check(!Object->HasAnyFlags(RF_NeedLoad));

			FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);

			// Add encountered object reference to list of to be serialized objects if it hasn't already been added.
			if (ObjectItem->GetOwnerIndex() != ClusterRootIndex)
			{
				if (ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot) || ObjectItem->GetOwnerIndex() != 0)
				{
					// Simply reference this cluster and all clusters it's referencing
					const int32 OtherClusterRootIndex = ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot) ? GUObjectArray.ObjectToIndex(Object) : ObjectItem->GetOwnerIndex();
					FUObjectItem* OtherClusterRootItem = GUObjectArray.IndexToObject(OtherClusterRootIndex);
					FUObjectCluster& OtherCluster = GUObjectClusters[OtherClusterRootItem->GetClusterIndex()];
					Cluster.ReferencedClusters.AddUnique(OtherClusterRootIndex);

					OtherCluster.ReferencedByClusters.AddUnique(ClusterRootIndex);

					for (int32 OtherClusterReferencedCluster : OtherCluster.ReferencedClusters)
					{
						if (OtherClusterReferencedCluster != ClusterRootIndex)
						{
							Cluster.ReferencedClusters.AddUnique(OtherClusterReferencedCluster);
						}
					}
					for (int32 OtherClusterReferencedMutableObjectIndex : OtherCluster.MutableObjects)
					{
						Cluster.MutableObjects.AddUnique(OtherClusterReferencedMutableObjectIndex);
					}
				}
				else if (!GUObjectArray.IsDisregardForGC(Object)) // Objects that are in disregard for GC set can be safely skipped
				{
					check(ObjectItem->GetOwnerIndex() == 0);

					// New object, add it to the cluster.
					if (CanAddToCluster(Object) && !Object->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad) && !Object->IsRooted())
					{
						AddObjectToCluster(GUObjectArray.ObjectToIndex(Object), ObjectItem, Object, Context, true);
					}
					else
					{
						UE_CLOGF(Object->HasAnyFlags(RF_NeedLoad), LogLevelActorContainer, Log, "%ls is being added to %ls's cluster but hasn't finished loading yet", *ParentLevel->GetFullName(), *Object->GetFullName());
						Cluster.MutableObjects.AddUnique(GUObjectArray.ObjectToIndex(Object));
					}
				}
			}
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	* Handles VCell reference from the token stream. Performance is critical here so we're FORCEINLINING this function.
	*
	* @param Context Context of the reference collection
	* @param ReferencingObject Object referencing the object to process.
	* @param Cell Cell being processed
	* @param MemberId Index to the token stream where the reference was found.
	* @param Origin Declares if a schema represents a blueprint generated type
	*/
	FORCEINLINE void HandleTokenStreamVerseCellReference(FWorkerContext& Context, UObject* ReferencingObject, Verse::VCell* Cell, FMemberId MemberId, EOrigin Origin)
	{
		// As with mutable objects, we assume that the cell is mutable but the pointer to the cell is not.  If the cell pointer needs to be modified,
		// then the underlying object must be mutable.
		if (Cell)
		{
			Cluster.MutableCells.AddUnique(Cell);
		}
	}
#endif
};

void ULevelActorContainer::CreateCluster()
{

	int32 ContainerInternalIndex = GUObjectArray.ObjectToIndex(this);
	FUObjectItem* RootItem = GUObjectArray.IndexToObject(ContainerInternalIndex);
	if (RootItem->GetOwnerIndex() != 0 || RootItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
	{
		return;
	}

	// If we haven't finished loading, we can't be sure we know all the references
	check(!HasAnyFlags(RF_NeedLoad));

	// Create a new cluster, reserve an arbitrary amount of memory for it.
	const int32 ClusterIndex = GUObjectClusters.AllocateCluster(GUObjectArray.ObjectToIndex(this));
	FUObjectCluster& Cluster = GUObjectClusters[ClusterIndex];
	Cluster.Objects.Reserve(64);

	// Collect all objects referenced by cluster root and by all objects it's referencing
	FActorClusterReferenceProcessor Processor(ContainerInternalIndex, Cluster, CastChecked<ULevel>(GetOuter()));
	TArray<UObject*> ObjectsToProcess = {static_cast<UObject*>(this)};
	FGCArrayStruct ArrayStruct;
	ArrayStruct.SetInitialObjectsUnpadded(ObjectsToProcess);
	CollectReferences(Processor, ArrayStruct);

	check(RootItem->GetOwnerIndex() == 0);
	RootItem->SetClusterIndex(ClusterIndex);
	RootItem->SetFlags(EInternalObjectFlags::ClusterRoot);

	if (Cluster.Objects.Num() >= GUObjectClusters.GetMinClusterSize())
	{
		// Sort all objects and set up the cluster root
		Cluster.Objects.Sort();
		Cluster.ReferencedClusters.Sort();
		Cluster.MutableObjects.Sort();

		if (UE_LOG_ACTIVE(LogLevelActorContainer, Verbose))
		{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
			FString ExtraDetail = FString::Printf(TEXT(", %d Verse cells"), Cluster.MutableCells.Num());
#else
			FString ExtraDetail;
#endif
			UE_LOGF(LogLevelActorContainer, Verbose, "Created LevelActorCluster (%d) for %ls with %d objects, %d referenced clusters%ls and %d mutable objects.",
				ClusterIndex, *GetOuter()->GetPathName(), Cluster.Objects.Num(), Cluster.ReferencedClusters.Num(), *ExtraDetail, Cluster.MutableObjects.Num());
		}

#if UE_GCCLUSTER_VERBOSE_LOGGING
		DumpClusterToLog(Cluster, true, false);
#endif
	}
	else
	{
		for (int32 ClusterObjectIndex : Cluster.Objects)
		{
			FUObjectItem* ClusterObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ClusterObjectIndex);
			ClusterObjectItem->SetOwnerIndex(0);
		}
		GUObjectClusters.FreeCluster(ClusterIndex);
		check(RootItem->GetOwnerIndex() == 0);
		check(!RootItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot));
	}
}

void ULevelActorContainer::OnClusterMarkedAsPendingKill()
{
	ULevel* Level = CastChecked<ULevel>(GetOuter());
	Level->ActorsForGC.Append(Actors);
	Actors.Reset();

	Super::OnClusterMarkedAsPendingKill();
}

