// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LevelBounds.h"
#include "Engine/CollisionProfile.h"
#include "Components/BoxComponent.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelBounds)

// Default size of the box (scale)
static const FVector DefaultLevelSize = FVector(1000.f);

ALevelBounds::ALevelBounds(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FTickableGameObject(ETickableTickType::Never)
{
	BoxComponent = CreateDefaultSubobject<UBoxComponent>(TEXT("BoxComponent0"));
	RootComponent = BoxComponent;
	BoxComponent->Mobility = EComponentMobility::Movable;
	BoxComponent->SetRelativeScale3D(DefaultLevelSize);
	BoxComponent->SetCanEverAffectNavigation(false);

	bAutoUpdateBounds = true;

	BoxComponent->bDrawOnlyIfSelected = true;
	BoxComponent->bUseAttachParentBound = false;
	BoxComponent->bUseEditorCompositing = true;
	BoxComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	BoxComponent->InitBoxExtent(FVector(0.5f, 0.5f, 0.5f));
	
	SetCanBeDamaged(false);
	
#if WITH_EDITOR
	bLevelBoundsDirty = true;
	bUsingDefaultBounds = false;
#endif

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif

}

void ALevelBounds::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		if (ULevel* Level = GetLevel())
		{
			Level->LevelBoundsActor = this;
		}
	}
}

FBox ALevelBounds::GetComponentsBoundingBox(bool bNonColliding, bool bIncludeFromChildActors) const
{
	checkf(RootComponent != nullptr, TEXT("LevelBounds actor with null root component: %s"), *GetPathNameSafe(this));
	FVector BoundsCenter = RootComponent->GetComponentLocation();
	FVector BoundsExtent = RootComponent->GetComponentTransform().GetScale3D() * 0.5f;
	return FBox(BoundsCenter - BoundsExtent, 
				BoundsCenter + BoundsExtent);
}

FBox ALevelBounds::CalculateLevelBounds(const ULevel* InLevel)
{
	FBox LevelBounds(ForceInit);
	
	if (InLevel)
	{
		// Iterate over all level actors
		for (int32 ActorIndex = 0; ActorIndex < InLevel->Actors.Num() ; ++ActorIndex)
		{
			AActor* Actor = InLevel->Actors[ActorIndex];
			if (Actor && Actor->IsLevelBoundsRelevant())
			{
				// Sum up components bounding boxes
				FBox ActorBox = Actor->GetComponentsBoundingBox(true);
				if (ActorBox.IsValid)
				{
					LevelBounds+= ActorBox;
				}
			}
		}
	}

	return LevelBounds;
}

#if WITH_EDITOR
void ALevelBounds::PostEditUndo()
{
	Super::PostEditUndo();
	
	MarkLevelBoundsDirty();
}

void ALevelBounds::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);
	
	MarkLevelBoundsDirty();
}

void ALevelBounds::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty( PropertyChangedEvent );
	
	MarkLevelBoundsDirty();
}

void ALevelBounds::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();
	
	if (!IsTemplate())
	{
		GetLevel()->LevelBoundsActor = this;
		SubscribeToUpdateEvents();
	}
}

void ALevelBounds::PostUnregisterAllComponents()
{
	if (!IsTemplate())
	{
		UnsubscribeFromUpdateEvents();
	}
	
	Super::PostUnregisterAllComponents();
}

void ALevelBounds::Tick(float DeltaTime)
{
	if (bLevelBoundsDirty)
	{
		UpdateLevelBounds();
		bLevelBoundsDirty = false;

		SetTickableTickType(ETickableTickType::Never);
	}
}

TStatId ALevelBounds::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(ALevelBounds, STATGROUP_Tickables);
}

bool ALevelBounds::IsTickable() const
{
	if (!IsValidChecked(this) || HasAnyFlags(RF_BeginDestroyed|RF_FinishDestroyed))
	{
		return false;
	}

	if (bAutoUpdateBounds)
	{
		UWorld* World = GetWorld();
		return (World && World->WorldType == EWorldType::Editor);
	}

	return false;
}

bool ALevelBounds::IsTickableInEditor() const
{
	return IsTickable();
}

void ALevelBounds::UpdateLevelBounds()
{
	FVector LevelCenter = FVector::ZeroVector;
	FVector LevelSize = DefaultLevelSize;
	FBox LevelBounds = CalculateLevelBounds(GetLevel());
	if (LevelBounds.IsValid)
	{
		LevelCenter = LevelBounds.GetCenter();
		LevelSize = LevelBounds.GetSize();
		
		bUsingDefaultBounds = false;
	}
	else
	{
		bUsingDefaultBounds = true;
	}

	// Avoid issue where the bounds size is zero and SetActorTransform complains
	if (LevelSize.X < 1.f)
	{
		LevelSize.X = 1.f;
	}

	if (LevelSize.Y < 1.f)
	{
		LevelSize.Y = 1.f;
	}

	if (LevelSize.Z < 1.f)
	{
		LevelSize.Z = 1.f;
	}

	SetActorTransform(FTransform(FQuat::Identity, LevelCenter, LevelSize));
	
	BroadcastLevelBoundsUpdated();
}

void ALevelBounds::MarkLevelBoundsDirty()
{
	bLevelBoundsDirty = true;

	if (GIsEditor)
	{
		SetTickableTickType(ETickableTickType::Conditional);
	}
}

bool ALevelBounds::IsUsingDefaultBounds() const
{
	return bUsingDefaultBounds;
}

void ALevelBounds::UpdateLevelBoundsImmediately()
{
	// This is used to get accurate bounds right when spawned.
	// This can't be done in PostActorCreated because the SpawnLocation interferes with the root component transform
	UpdateLevelBounds();
}

void ALevelBounds::OnLevelActorMoved(AActor* InActor)
{
	if (InActor->GetOuter() == GetOuter())
	{
		if (InActor == this)
		{
			BroadcastLevelBoundsUpdated();
		}
		else
		{
			MarkLevelBoundsDirty();
		}
	}
}
	
void ALevelBounds::OnLevelActorAddedRemoved(AActor* InActor)
{
	if (InActor->GetOuter() == GetOuter())
	{
		MarkLevelBoundsDirty();
	}
}

void ALevelBounds::BroadcastLevelBoundsUpdated()
{
	ULevel* Level = GetLevel();
	if (Level && 
		Level->LevelBoundsActor.Get() == this)
	{
		Level->BroadcastLevelBoundsActorUpdated();
	}
}

void ALevelBounds::SubscribeToUpdateEvents()
{
	// Subscribe only in editor worlds
	if (!GetWorld()->IsGameWorld())
	{
		UnsubscribeFromUpdateEvents();

		OnLevelActorMovedDelegateHandle   = GEngine->OnActorMoved       ().AddUObject(this, &ALevelBounds::OnLevelActorMoved);
		OnLevelActorDeletedDelegateHandle = GEngine->OnLevelActorDeleted().AddUObject(this, &ALevelBounds::OnLevelActorAddedRemoved);
		OnLevelActorAddedDelegateHandle   = GEngine->OnLevelActorAdded  ().AddUObject(this, &ALevelBounds::OnLevelActorAddedRemoved);
	}
}

void ALevelBounds::UnsubscribeFromUpdateEvents()
{
	GEngine->OnActorMoved       ().Remove(OnLevelActorMovedDelegateHandle);
	GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedDelegateHandle);
	GEngine->OnLevelActorAdded  ().Remove(OnLevelActorAddedDelegateHandle);
}


#endif // WITH_EDITOR



