// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "LevelInstancePrivate.h"
#include "Net/UnrealNetwork.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceActor)

#if WITH_EDITOR
#include "Cooker/CookDependency.h"
#include "Cooker/CookEvents.h"
#include "UObject/ObjectSaveContext.h"
#include "WorldPartition/LevelInstance/LevelInstanceActorDesc.h"
#include "LevelInstance/LevelInstanceEditorPivotActor.h"
#include "Misc/MessageDialog.h"
#endif

#define LOCTEXT_NAMESPACE "LevelInstanceActor"

ALevelInstance::ALevelInstance()
	: LevelInstanceActorGuid(this)
	, LevelInstanceActorImpl(this)
{
	RootComponent = LevelInstanceComponent = CreateDefaultSubobject<ULevelInstanceComponent>(TEXT("Root"));
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	SetDesiredRuntimeBehavior(ELevelInstanceRuntimeBehavior::Partitioned);
#endif
}

void ALevelInstance::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);
	Ar << LevelInstanceActorGuid;

#if WITH_EDITORONLY_DATA
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		// Remove PIE Prefix in case the LevelInstance was part of the DuplicateWorldForPIE
		// This can happen if the level (WorldAsset) is part of the world's levels.
		// ULevelStreaming::RenameForPIE will call FSoftObjectPath::AddPIEPackageName which will
		// force this softobjectpath to be processed by FSoftObjectPath::FixupForPIE (even one that comes from a level instance).
		WorldAsset = FSoftObjectPath(UWorld::RemovePIEPrefix(WorldAsset.ToString()));
	}

	// Level instances were previously always considered HLOD relevant.
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ResetLevelInstanceHLODRelevancy)
	{
		bEnableAutoLODGeneration = true;
	}
#endif
}

void ALevelInstance::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.Condition = COND_InitialOnly;
	DOREPLIFETIME_WITH_PARAMS_FAST(ALevelInstance, LevelInstanceSpawnGuid, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(ALevelInstance, CookedWorldAsset, Params);
}

void ALevelInstance::PostNetInit()
{
	Super::PostNetInit();
	
	if (ensure(!LevelInstanceActorGuid.IsValid()))
	{
		check(LevelInstanceSpawnGuid.IsValid());
		LevelInstanceActorGuid.ActorGuid = LevelInstanceSpawnGuid;
		LevelInstanceActorImpl.RegisterLevelInstance();
	}
}

void ALevelInstance::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (GetLocalRole() == ENetRole::ROLE_Authority && GetWorld()->IsGameWorld())
	{
#if !WITH_EDITOR
		// If the level instance was spawned, not loaded
		LevelInstanceActorGuid.AssignIfInvalid();
#endif
		LevelInstanceSpawnGuid = LevelInstanceActorGuid.GetGuid();
	}

	if (LevelInstanceActorGuid.IsValid())
	{
		LevelInstanceActorImpl.RegisterLevelInstance();
	}
}

void ALevelInstance::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	LevelInstanceActorImpl.UnregisterLevelInstance();
}

bool ALevelInstance::IsLoadingEnabled() const
{
	return LevelInstanceActorImpl.IsLoadingEnabled();
}

const TSoftObjectPtr<UWorld>& ALevelInstance::GetWorldAsset() const
{
#if WITH_EDITORONLY_DATA
	const TSoftObjectPtr<UWorld> &Result = GetPropertyOverrideAsset() ? GetPropertyOverrideAsset()->GetWorldAsset() : WorldAsset;
	if (!Result.IsNull())
	{
		return Result;
	}
#endif
	return CookedWorldAsset;
}

const FLevelInstanceID& ALevelInstance::GetLevelInstanceID() const
{
	return LevelInstanceActorImpl.GetLevelInstanceID();
}

bool ALevelInstance::HasValidLevelInstanceID() const
{
	return LevelInstanceActorImpl.HasValidLevelInstanceID();
}

const FGuid& ALevelInstance::GetLevelInstanceGuid() const
{
	return LevelInstanceActorGuid.GetGuid();
}

void ALevelInstance::OnLevelInstanceLoaded()
{
	LevelInstanceActorImpl.OnLevelInstanceLoaded();
}

#if WITH_EDITOR

void ALevelInstance::SetPropertyOverrideAsset(ULevelInstancePropertyOverrideAsset* InPropertyOverrideAsset)
{
	if (PropertyOverrides != InPropertyOverrideAsset)
	{
		Modify();
		PropertyOverrides = InPropertyOverrideAsset;
	}
}


ULevelInstanceComponent* ALevelInstance::GetLevelInstanceComponent() const
{
	return Cast<ULevelInstanceComponent>(RootComponent);
}

TSubclassOf<AActor> ALevelInstance::GetEditorPivotClass() const
{
	return ALevelInstancePivot::StaticClass();
}

bool ALevelInstance::SupportsPartialEditorLoading() const
{
	return ILevelInstanceInterface::SupportsPartialEditorLoading() && LevelInstanceActorImpl.SupportsPartialEditorLoading();
}

bool ALevelInstance::SupportsPropertyOverrides() const
{
	return LevelInstanceActorImpl.SupportsPropertyOverrides();
}

ULevelInstancePropertyOverrideAsset* ALevelInstance::GetPropertyOverrideAsset() const
{
	return SupportsPropertyOverrides() ? PropertyOverrides : nullptr;
}

TUniquePtr<FWorldPartitionActorDesc> ALevelInstance::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FLevelInstanceActorDesc());
}

ALevelInstance::FOnLevelInstanceActorPostLoad ALevelInstance::OnLevelInstanceActorPostLoad;

void ALevelInstance::PostLoad()
{
	Super::PostLoad();

	LevelInstanceComponent = Cast<ULevelInstanceComponent>(RootComponent);
	OnLevelInstanceActorPostLoad.Broadcast(this);

#if WITH_EDITORONLY_DATA
	if (IsRunningCookCommandlet() && ShouldCookWorldAsset())
	{
		CookedWorldAsset = GetWorldAsset();
	}
#endif
}

void ALevelInstance::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);

	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies)
	{
		FName WorldPackageName = FName(GetWorldAsset().GetLongPackageName());
		if (!WorldPackageName.IsNone())
		{
			CookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::Package(WorldPackageName));
		}
	}
}

bool ALevelInstance::ShouldCookWorldAsset() const
{
	// If ALevelInstance actor gets loaded it means it needs to Cook its WorldAsset (World Partition Embedded Level Instances don't get loaded as they aren't runtime relevant)
	// If ALevelInstnace is a template then we only need to Cook its WorldAsset if it's desired runtime behavior is to be Level Streamed
	return !IsTemplate() || GetDesiredRuntimeBehavior() == ELevelInstanceRuntimeBehavior::LevelStreaming;
}

void ALevelInstance::PreEditUndo()
{
	LevelInstanceActorImpl.PreEditUndo([this]() { Super::PreEditUndo(); });
}

void ALevelInstance::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	LevelInstanceActorImpl.PostEditUndo(TransactionAnnotation, [this](TSharedPtr<ITransactionObjectAnnotation> InTransactionAnnotation) { Super::PostEditUndo(InTransactionAnnotation); });
}

void ALevelInstance::PostEditUndo()
{
	LevelInstanceActorImpl.PostEditUndo([this]() { Super::PostEditUndo(); });
}

void ALevelInstance::PreEditChange(FProperty* PropertyThatWillChange)
{
	const bool bWorldAssetChange = PropertyThatWillChange && (PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ALevelInstance, WorldAsset));
	LevelInstanceActorImpl.PreEditChange(PropertyThatWillChange, bWorldAssetChange, [this](FProperty* Property) { Super::PreEditChange(Property); });
}

void ALevelInstance::CheckForErrors()
{
	Super::CheckForErrors();

	LevelInstanceActorImpl.CheckForErrors();
}

bool ALevelInstance::SetWorldAsset(TSoftObjectPtr<UWorld> InWorldAsset)
{
	FString Reason;
	if (!ULevelInstanceSubsystem::CanUseWorldAsset(this, InWorldAsset, &Reason))
	{
		UE_LOGF(LogLevelInstance, Warning, "%ls", *Reason);
		return false;
	}

	WorldAsset = InWorldAsset;
	return true;
}

void ALevelInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const bool bWorldAssetChange = PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ALevelInstance, WorldAsset);

	LevelInstanceActorImpl.PostEditChangeProperty(PropertyChangedEvent, bWorldAssetChange, [this](FPropertyChangedEvent& Event) { Super::PostEditChangeProperty(Event); });

#if WITH_EDITORONLY_DATA
	const bool bLIBehaviorChange = PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ALevelInstance, DesiredRuntimeBehavior);

	if (bLIBehaviorChange)
	{
		LevelInstanceComponent->ShadowDesiredRuntimeBehavior = DesiredRuntimeBehavior;
	}
#endif

}

bool ALevelInstance::CanEditChange(const FProperty* Property) const
{
	return Super::CanEditChange(Property) && LevelInstanceActorImpl.CanEditChange(Property);
}

bool ALevelInstance::CanEditChangeComponent(const UActorComponent* InComponent, const FProperty* InProperty) const
{
	return Super::CanEditChangeComponent(InComponent, InProperty) && LevelInstanceActorImpl.CanEditChangeComponent(InComponent, InProperty);
}

bool ALevelInstance::ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists)
{
	return LevelInstanceActorImpl.ResolveSubobject(SubObjectPath, OutObject, bLoadIfExists);
}

void ALevelInstance::PostEditImport()
{
	LevelInstanceActorImpl.PostEditImport([this]() { Super::PostEditImport(); });
}

bool ALevelInstance::CanDeleteSelectedActor(FText& OutReason) const
{
	return Super::CanDeleteSelectedActor(OutReason) && LevelInstanceActorImpl.CanDeleteSelectedActor(OutReason);
}

void ALevelInstance::SetIsTemporarilyHiddenInEditor(bool bIsHidden)
{
	LevelInstanceActorImpl.SetIsTemporarilyHiddenInEditor(bIsHidden, [this](bool bInHidden) { Super::SetIsTemporarilyHiddenInEditor(bInHidden); });
}

bool ALevelInstance::SetIsHiddenEdLayer(bool bIsHiddenEdLayer)
{
	return LevelInstanceActorImpl.SetIsHiddenEdLayer(bIsHiddenEdLayer, [this](bool bInHiddenEdLayer) { return Super::SetIsHiddenEdLayer(bInHiddenEdLayer); });
}

void ALevelInstance::EditorGetUnderlyingActors(TSet<AActor*>& OutUnderlyingActors) const
{
	Super::EditorGetUnderlyingActors(OutUnderlyingActors);
	LevelInstanceActorImpl.EditorGetUnderlyingActors(OutUnderlyingActors);
}

void ALevelInstance::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	Super::GetStreamingBounds(OutRuntimeBounds, OutEditorBounds);
	
	FBox LevelInstanceRuntimeBounds;
	if (LevelInstanceActorImpl.GetBounds(LevelInstanceRuntimeBounds))
	{
		OutRuntimeBounds = LevelInstanceRuntimeBounds;
	}

	FBox LevelInstanceEditorBounds;
	if (LevelInstanceActorImpl.GetEditorBounds(LevelInstanceEditorBounds))
	{
		OutEditorBounds = LevelInstanceEditorBounds;
	}
}

bool ALevelInstance::IsUserManaged() const
{
	return Super::IsUserManaged() || LevelInstanceActorImpl.IsUserManaged();
}

bool ALevelInstance::ShouldExport()
{
	return Super::ShouldExport() && LevelInstanceActorImpl.ShouldExport();
}

bool ALevelInstance::IsLockLocation() const
{
	return Super::IsLockLocation() || LevelInstanceActorImpl.IsLockLocation();
}

bool ALevelInstance::IsActorLabelEditable() const
{
	return Super::IsActorLabelEditable() && LevelInstanceActorImpl.IsActorLabelEditable();
}

bool ALevelInstance::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Objects.Add(const_cast<ALevelInstance*>(this));
	return true;
}

bool ALevelInstance::GetSoftReferencedContentObjects(TArray<FSoftObjectPath>& SoftObjects) const
{
	if (GetWorldAsset().ToSoftObjectPath().IsValid())
	{
		SoftObjects.Add(GetWorldAsset().ToSoftObjectPath());
		return true;
	}
	return false;
}

bool ALevelInstance::OpenAssetEditor()
{
	FText Reason;
	if (!CanEnterEdit(&Reason))
	{
		FMessageDialog::Open(EAppMsgType::Ok, Reason);
		return false;
	}
	return EnterEdit();
}

bool ALevelInstance::EditorCanAttachFrom(const AActor* InChild, FText& OutReason) const
{
	if (IsEditing())
	{
		return true;
	}

	return false;
}

FBox ALevelInstance::GetComponentsBoundingBox(bool bNonColliding, bool bIncludeFromChildActors) const
{
	FBox Box = Super::GetComponentsBoundingBox(bNonColliding, bIncludeFromChildActors);
	
	FBox LevelInstanceBounds;
	if (LevelInstanceActorImpl.GetBounds(LevelInstanceBounds))
	{
		Box += LevelInstanceBounds;
	}
	
	return Box;
}

void ALevelInstance::PushSelectionToProxies()
{
	Super::PushSelectionToProxies();

	LevelInstanceActorImpl.PushSelectionToProxies();
}

void ALevelInstance::PushLevelInstanceEditingStateToProxies(bool bInEditingState)
{
	Super::PushLevelInstanceEditingStateToProxies(bInEditingState);

	LevelInstanceActorImpl.PushLevelInstanceEditingStateToProxies(bInEditingState);
}

bool ALevelInstance::IsHLODRelevant() const
{
	if (!IsWorldAssetValid())
	{
		return false;
	}

	return Super::IsHLODRelevant();
}

bool ALevelInstance::HasHLODRelevantComponents() const
{
	// Assume LIs can provide HLOD relevant components
	return true;
}

void ALevelInstance::SetDesiredRuntimeBehavior(ELevelInstanceRuntimeBehavior NewBehavior)
{
	DesiredRuntimeBehavior = NewBehavior;
	LevelInstanceComponent->ShadowDesiredRuntimeBehavior = NewBehavior;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

