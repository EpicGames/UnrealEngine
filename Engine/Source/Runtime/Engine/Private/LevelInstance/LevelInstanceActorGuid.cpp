// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceActorGuid.h"
#include "Engine/World.h"
#include "UObject/PropertyPortFlags.h"

#if !WITH_EDITOR
void FLevelInstanceActorGuid::AssignIfInvalid()
{
	if (!ActorGuid.IsValid())
	{
		ActorGuid = FGuid::NewGuid();
	}
}
#endif

bool FLevelInstanceActorGuid::IsValid() const
{
	return GetGuid_Internal().IsValid();
}

const FGuid& FLevelInstanceActorGuid::GetGuid() const
{
	const FGuid& Guid = GetGuid_Internal();
	check(Actor->IsTemplate() || Guid.IsValid());
	return Guid;
}

const FGuid& FLevelInstanceActorGuid::GetGuid_Internal() const
{
	check(Actor);
#if WITH_EDITOR
	// It's possible for an actor not to have its world when duplicating unsaved actor when starting PIE (actor is outered to its UActorContainer)
	const FGuid& Guid = (!Actor->GetWorld() || !Actor->GetWorld()->IsGameWorld() || !Actor->GetIsReplicated() || Actor->HasAuthority()) ? Actor->GetActorGuid() : ActorGuid;
#else
	const FGuid& Guid = ActorGuid;
#endif
	return Guid;
}

FArchive& operator<<(FArchive& Ar, FLevelInstanceActorGuid& LevelInstanceActorGuid)
{
	check(LevelInstanceActorGuid.Actor);
#if WITH_EDITOR
	if (!LevelInstanceActorGuid.Actor->IsTemplate())
	{
		if (Ar.IsSaving() && Ar.IsCooking())
		{
			FGuid Guid = LevelInstanceActorGuid.GetGuid();
			Ar << Guid;
		}
		else if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
		{
			if (Ar.IsLoading())
			{
				Ar << LevelInstanceActorGuid.ActorGuid;
			}
			else if (Ar.IsSaving())
			{
				FGuid Guid = LevelInstanceActorGuid.GetGuid();
				Ar << Guid;
			}
		}
	}
#else
	if (Ar.IsLoading())
	{
		if (LevelInstanceActorGuid.Actor->IsTemplate())
		{
			check(!LevelInstanceActorGuid.ActorGuid.IsValid());
		}
		else if (Ar.GetPortFlags() & PPF_Duplicate)
		{
			LevelInstanceActorGuid.ActorGuid = FGuid::NewGuid();
		}
		else if (Ar.IsPersistent())
		{
			Ar << LevelInstanceActorGuid.ActorGuid;
			check(LevelInstanceActorGuid.ActorGuid.IsValid());
		}
	}
#endif
	return Ar;
}
