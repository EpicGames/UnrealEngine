// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceActorImpl.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Misc/PackageName.h"

#if WITH_EDITOR

#include "LevelInstance/LevelInstancePrivate.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"

#endif

#include "WorldPartition/ActorInstanceGuids.h"

#define LOCTEXT_NAMESPACE "LevelInstanceActor"

void FLevelInstanceActorImpl::RegisterLevelInstance()
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
	{
		LevelInstanceID = LevelInstanceSubsystem->RegisterLevelInstance(LevelInstance);

		LevelInstance->LoadLevelInstance();

#if WITH_EDITOR
		// Make sure transformation is up to date after registration as its possible LevelInstance actor can get unregistered when editing properties
		// through Details panel. In this case the ULevelInstanceComponent might not be able to update the ALevelInstanceEditorInstanceActor transform.
		LevelInstance->GetLevelInstanceComponent()->UpdateEditorInstanceActor();
#endif
	}
}

void FLevelInstanceActorImpl::UnregisterLevelInstance()
{
	// If LevelInstance has already been unregistered it will have an Invalid LevelInstanceID. Avoid processing it.
	if (!LevelInstanceID.IsValid())
	{
		return;
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
	{
		LevelInstanceSubsystem->UnregisterLevelInstance(LevelInstance);
		if (ULevel* LoadedLevel = LevelInstance->GetLoadedLevel())
		{
			FActorInstanceGuid::ReleaseLevelInstanceGuid(LoadedLevel);
		}

		LevelInstance->UnloadLevelInstance();

#if WITH_EDITOR
		// Make sure our Component doesn't keep a pointer to the Editor Actor once unloaded
		LevelInstance->GetLevelInstanceComponent()->CachedEditorInstanceActorPtr.Reset();
#endif

		// To avoid processing PostUnregisterAllComponents multiple times (BP Recompile is one use case)
		LevelInstanceID = FLevelInstanceID();
	}
}

const FLevelInstanceID& FLevelInstanceActorImpl::GetLevelInstanceID() const
{
	check(HasValidLevelInstanceID());
	return LevelInstanceID;
}

bool FLevelInstanceActorImpl::HasValidLevelInstanceID() const
{
	return LevelInstanceID.IsValid();
}

bool FLevelInstanceActorImpl::IsLoadingEnabled() const
{
#if WITH_EDITOR
	return !bGuardLoadUnload && !CastChecked<AActor>(LevelInstance)->bIsEditorPreviewActor;
#else
	return true;
#endif
}

void FLevelInstanceActorImpl::OnLevelInstanceLoaded()
{
	FActorInstanceGuid::SetLevelInstanceGuid(LevelInstance->GetLoadedLevel(), ULevelInstanceSubsystem::GetOwningLevel(LevelInstance->GetLoadedLevel()), LevelInstance->GetLevelInstanceGuid());

#if WITH_EDITOR
	AActor* Actor = CastChecked<AActor>(LevelInstance);
	if (!Actor->GetWorld()->IsGameWorld())
	{
		// Propagate bounds dirtyness up and check if we need to hide our LevelInstance because self or ancestor is hidden
		bool bHiddenInEditor = false;
		LevelInstance->GetLevelInstanceSubsystem()->ForEachLevelInstanceAncestorsAndSelf(Actor, [&bHiddenInEditor](const ILevelInstanceInterface* AncestorOrSelf)
		{
			const AActor* AncestorOrSelfActor = CastChecked<AActor>(AncestorOrSelf);
			AncestorOrSelfActor->GetLevel()->MarkLevelBoundsDirty();
			bHiddenInEditor |= AncestorOrSelfActor->IsTemporarilyHiddenInEditor();
			return true;
		});

		if (bHiddenInEditor)
		{
			Actor->SetIsTemporarilyHiddenInEditor(bHiddenInEditor);
		}
	}
#endif
}

#if WITH_EDITOR
bool FLevelInstanceActorImpl::ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists)
{
	if (ULevel* Level = LevelInstance->GetLoadedLevel())
	{
		return Level->ResolveSubobject(SubObjectPath, OutObject, bLoadIfExists);
	}

	return false;
}

bool FLevelInstanceActorImpl::SupportsPropertyOverrides() const
{
	return ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled();
}

bool FLevelInstanceActorImpl::SupportsPartialEditorLoading() const
{
	return bAllowPartialLoading;
}

void FLevelInstanceActorImpl::PreEditUndo(TFunctionRef<void()> SuperCall)
{
	CachedLevelInstanceID = LevelInstanceID;
	CachedWorldAsset = LevelInstance->GetWorldAsset();

	AActor* Actor = CastChecked<AActor>(LevelInstance);
	bCachedIsTemporarilyHiddenInEditor = Actor->IsTemporarilyHiddenInEditor(false);

	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall();
	}
}

void FLevelInstanceActorImpl::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation, TFunctionRef<void(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)> SuperCall)
{
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall(TransactionAnnotation);
	}

	PostEditUndoInternal();
}

void FLevelInstanceActorImpl::PostEditUndo(TFunctionRef<void()> SuperCall)
{
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall();
	}

	PostEditUndoInternal();
}

void FLevelInstanceActorImpl::PostEditUndoInternal()
{
	if (CachedWorldAsset != LevelInstance->GetWorldAsset())
	{
		LevelInstance->UpdateLevelInstanceFromWorldAsset();
	}

	AActor* Actor = CastChecked<AActor>(LevelInstance);
	if (bCachedIsTemporarilyHiddenInEditor != Actor->IsTemporarilyHiddenInEditor(false))
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
		{
			LevelInstanceSubsystem->SetIsTemporarilyHiddenInEditor(LevelInstance, !bCachedIsTemporarilyHiddenInEditor);
		}
	}

	// Here we want to load or unload based on our current state
	if (LevelInstance->HasValidLevelInstanceID() && !LevelInstance->IsLoaded())
	{
		LevelInstance->LoadLevelInstance();
	}
	else if (!IsValidChecked(Actor))
	{
		// Temp restore the ID so that we can unload
		TGuardValue<FLevelInstanceID> LevelInstanceIDGuard(LevelInstanceID, CachedLevelInstanceID);
		if (LevelInstance->IsLoaded())
		{
			LevelInstance->UnloadLevelInstance();
		}
	}

	CachedLevelInstanceID = FLevelInstanceID();
	CachedWorldAsset.Reset();

	if (ULevelInstanceComponent* LevelInstanceComponent = LevelInstance->GetLevelInstanceComponent())
	{
		// Order of operations when undoing may lead to the RootComponent being undone before our actor so we need to make sure we update here and in the component when undoing
		LevelInstanceComponent->UpdateEditorInstanceActor();
	}
}

void FLevelInstanceActorImpl::PostEditImport(TFunctionRef<void()> SuperCall)
{
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall();
	}
	LevelInstance->UpdateLevelInstanceFromWorldAsset();
}

void FLevelInstanceActorImpl::PreEditChange(FProperty* Property, bool bWorldAssetChange, TFunctionRef<void(FProperty*)> SuperCall)
{
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall(Property);
	}

	if (bWorldAssetChange)
	{
		CachedWorldAsset = LevelInstance->GetWorldAsset();
		bAllowPartialLoading = false;
		IWorldPartitionActorLoaderInterface::RefreshLoadedState(true);
	}	
}

void FLevelInstanceActorImpl::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent, bool bWorldAssetChange, TFunction<void(FPropertyChangedEvent&)> SuperCall)
{
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall(PropertyChangedEvent);
	}
		
	if(bWorldAssetChange)
	{
		FString Reason;
		if (!ULevelInstanceSubsystem::CanUseWorldAsset(LevelInstance, LevelInstance->GetWorldAsset(), &Reason))
		{
			UE_LOGF(LogLevelInstance, Warning, "%ls", *Reason);
			LevelInstance->SetWorldAsset(CachedWorldAsset);
		}
		else
		{
			LevelInstance->UpdateLevelInstanceFromWorldAsset();
		}
		CachedWorldAsset.Reset();
	}
}

bool FLevelInstanceActorImpl::CanDeleteSelectedActor(FText& OutReason) const
{
	if (LevelInstance->IsEditing())
	{
		OutReason = LOCTEXT("HasEditingLevel", "Can't delete LevelInstance because it is editing!");
		return false;
	}

	if (LevelInstance->HasChildEdit())
	{
		OutReason = LOCTEXT("HasEditingChildLevel", "Can't delete LevelInstance because it has editing child LevelInstances!");
		return false;
	}
	return true;
}

void FLevelInstanceActorImpl::SetIsTemporarilyHiddenInEditor(bool bIsHidden, TFunctionRef<void(bool)> SuperCall)
{
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		SuperCall(bIsHidden);
	}

	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
	{
		LevelInstanceSubsystem->SetIsTemporarilyHiddenInEditor(LevelInstance, bIsHidden);
	}
}

bool FLevelInstanceActorImpl::SetIsHiddenEdLayer(bool bIsHiddenEdLayer, TFunctionRef<bool(bool)> SuperCall)
{
	bool bHasChanged = false;
	{
		TGuardValue<bool> LoadUnloadGuard(bGuardLoadUnload, true);
		bHasChanged = SuperCall(bIsHiddenEdLayer);
	}

	if (bHasChanged)
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
		{
			LevelInstanceSubsystem->SetIsHiddenEdLayer(LevelInstance, bIsHiddenEdLayer);
		}
	}
	return bHasChanged;
}

void FLevelInstanceActorImpl::EditorGetUnderlyingActors(TSet<AActor*>& OutUnderlyingActors) const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
	{
		LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [&](AActor* LevelActor)
		{
			bool bAlreadySet = false;
			OutUnderlyingActors.Add(LevelActor, &bAlreadySet);
			if (!bAlreadySet)
			{
				LevelActor->EditorGetUnderlyingActors(OutUnderlyingActors);
			}
			return true;
		});
	}
}

bool FLevelInstanceActorImpl::IsLockedActor() const
{
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	if (LevelInstanceActor->IsInLevelInstance() && !LevelInstanceActor->IsInAnyEditLevelInstance())
	{
		return true;
	}

	if (LevelInstance->IsEditing() || LevelInstance->IsEditingPropertyOverrides())
	{
		return true;
	}

	if (LevelInstance->HasChildEdit())
	{
		return true;
	}

	return false;
}

bool FLevelInstanceActorImpl::ShouldExport() const
{
	return true;
}

bool FLevelInstanceActorImpl::IsUserManaged() const
{
	return !IsLockedActor();
}

bool FLevelInstanceActorImpl::IsLockLocation() const
{
	return IsLockedActor();
}

bool FLevelInstanceActorImpl::IsActorLabelEditable() const
{
	return !IsLockedActor();
}

bool FLevelInstanceActorImpl::CanEditChange(const FProperty* Property) const
{
	return !IsLockedActor();
}

bool FLevelInstanceActorImpl::CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const
{
	return !IsLockedActor();
}

bool FLevelInstanceActorImpl::GetBounds(FBox& OutBounds) const
{
	// Add Level Bounds
	if (LevelInstance->IsLoadingEnabled())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
		{
			if (LevelInstanceSubsystem->GetLevelInstanceBounds(LevelInstance, OutBounds))
			{
				return true;
			}
		}
	}

	return false;
}

bool FLevelInstanceActorImpl::GetEditorBounds(FBox& OutBounds) const
{
	if (LevelInstance->IsLoadingEnabled())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
		{
			if (LevelInstanceSubsystem->GetLevelInstanceEditorBounds(LevelInstance, OutBounds))
			{
				return true;
			}
		}
	}

	return false;
}

void FLevelInstanceActorImpl::PushSelectionToProxies()
{
	// Actors of the LevelInstance need to reflect the LevelInstance actor's selected state
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
	{
		LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [](AActor* LevelActor)
		{
			if (ALevelInstanceEditorInstanceActor* EditorInstanceActor = Cast<ALevelInstanceEditorInstanceActor>(LevelActor))
			{
				EditorInstanceActor->PushSelectionToProxies();
				return false;
			}
			return true;
		});
	}
}

void FLevelInstanceActorImpl::PushLevelInstanceEditingStateToProxies(bool bInEditingState)
{
	// Actors of the LevelInstance need to reflect the LevelInstance actor's Editing state
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
	{
		LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [bInEditingState](AActor* LevelActor)
		{
			LevelActor->PushLevelInstanceEditingStateToProxies(bInEditingState);
			return true;
		});
	}
}

void FLevelInstanceActorImpl::CheckForErrors()
{
	AActor* Actor = CastChecked<AActor>(LevelInstance);
	if (Actor->IsTemplate())
	{
		return;
	}

	TArray<TPair<FText, TSoftObjectPtr<UWorld>>> LoopInfo;
	const ILevelInstanceInterface* LoopStart = nullptr;
	if (!ULevelInstanceSubsystem::CheckForLoop(LevelInstance, LevelInstance->GetWorldAsset(), & LoopInfo, &LoopStart))
	{
		FMessageLog Message("MapCheck");
		TSharedRef<FTokenizedMessage> Error = Message.Error()->AddToken(FTextToken::Create(LOCTEXT("LevelInstanceActor_Loop_CheckForErrors", "LevelInstance level loop found!")));
		const AActor* LoopStartActor = CastChecked<AActor>(LoopStart);
		TSoftObjectPtr<UWorld> LoopStartAsset(LoopStartActor->GetLevel()->GetTypedOuter<UWorld>());
		Error->AddToken(FAssetNameToken::Create(LoopStartAsset.GetLongPackageName(), FText::FromString(LoopStartAsset.GetAssetName())));
		Error->AddToken(FTextToken::Create(FText::FromString(TEXT(":"))));
		Error->AddToken(FUObjectToken::Create(LoopStartActor));
		for (int32 i = LoopInfo.Num() - 1; i >= 0; --i)
		{
			Error->AddToken(FTextToken::Create(LoopInfo[i].Key));
			TSoftObjectPtr<UWorld> LevelInstancePtr = LoopInfo[i].Value;
			Error->AddToken(FAssetNameToken::Create(LevelInstancePtr.GetLongPackageName(), FText::FromString(LevelInstancePtr.GetAssetName())));
		}

		Error->AddToken(FMapErrorToken::Create(FName("LevelInstanceActor_Loop_CheckForErrors")));
	}

	FPackagePath WorldAssetPath;
	if (!FPackagePath::TryFromPackageName(LevelInstance->GetWorldAssetPackage(), WorldAssetPath) || !FPackageName::DoesPackageExist(WorldAssetPath))
	{
		FMessageLog("MapCheck").Error()
			->AddToken(FTextToken::Create(LOCTEXT("LevelInstanceActor_InvalidPackage", "LevelInstance actor")))
			->AddToken(FUObjectToken::Create(CastChecked<AActor>(LevelInstance)))
			->AddToken(FTextToken::Create(FText::FromString(TEXT("refers to an invalid asset:"))))
			->AddToken(FAssetNameToken::Create(LevelInstance->GetWorldAsset().GetLongPackageName(), FText::FromString(LevelInstance->GetWorldAsset().GetLongPackageName())))
			->AddToken(FMapErrorToken::Create(FName("LevelInstanceActor_InvalidPackage_CheckForErrors")));
	}
}

#endif

#undef LOCTEXT_NAMESPACE
