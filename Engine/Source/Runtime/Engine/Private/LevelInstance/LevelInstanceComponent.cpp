// Copyright Epic Games, Inc. All Rights Reserved.
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/ICookInfo.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceComponent)

#if WITH_EDITORONLY_DATA
static const TCHAR* GLevelSpriteAssetName = TEXT("/Engine/EditorResources/LevelInstance");
#endif

ULevelInstanceComponent::ULevelInstanceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bWasDirtyBeforeEditFilterChange(false)
#endif
{
#if WITH_EDITOR
	bWantsOnUpdateTransform = true;
#endif
}

#if WITH_EDITORONLY_DATA
void ULevelInstanceComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsSaving() && Ar.IsObjectReferenceCollector() && !Ar.IsCooking())
	{
		FSoftObjectPathSerializationScope EditorOnlyScope(ESoftObjectPathCollectType::EditorOnlyCollect);
		FSoftObjectPath SpriteAsset(GLevelSpriteAssetName);
		Ar << SpriteAsset;
	}
}
#endif

#if WITH_EDITOR
void ULevelInstanceComponent::OnRegister()
{
#if WITH_EDITORONLY_DATA
	// Prevents USceneComponent from creating the SpriteComponent in OnRegister because we want to provide a different texture and condition
	bVisualizeComponent = false;
#endif

	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	AActor* Owner = GetOwner();
	// Only show Sprite for non-instanced LevelInstances
	if (GetWorld() && !GetWorld()->IsGameWorld())
	{
		// Re-enable before calling CreateSpriteComponent
		bVisualizeComponent = true;
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		CreateSpriteComponent(LoadObject<UTexture2D>(nullptr, GLevelSpriteAssetName), false);
		if (SpriteComponent)
		{
			SpriteComponent->bShowLockedLocation = false;
			SpriteComponent->SetVisibility(ShouldShowSpriteComponent());
			SpriteComponent->RegisterComponent();
		}
	}

	//Initialize the Shadow variable to the parents value. This is for the cases where the ILevelInstanceInterface child classes use a constant behavior
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(GetOwner()))
	{
		ShadowDesiredRuntimeBehavior = LevelInstance->GetDesiredRuntimeBehavior();
	}
#endif //WITH_EDITORONLY_DATA
}

void ULevelInstanceComponent::SetFilter(const FWorldPartitionActorFilter& InFilter, bool bNotify)
{
	if (GetFilter() != InFilter)
	{
		Modify(!IsEditFilter());
		SetActiveFilter(InFilter);
		OnFilterChanged();

		if (bNotify)
		{
			FWorldPartitionActorFilter::GetOnWorldPartitionActorFilterChanged().Broadcast();
		}
	}
}

void ULevelInstanceComponent::SetActiveFilter(const FWorldPartitionActorFilter& InFilter)
{
	if (IsEditFilter())
	{
		EditFilter = InFilter;
	}
	else
	{
		Filter = InFilter;
	}
}

void ULevelInstanceComponent::OnFilterChanged()
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(GetOwner()))
	{
		LevelInstance->OnFilterChanged();
	}
}

bool ULevelInstanceComponent::ShouldShowSpriteComponent() const
{
	return GetOwner() && GetOwner()->GetLevel() && (GetOwner()->GetLevel()->IsPersistentLevel() || !GetOwner()->GetLevel()->IsInstancedLevel());
}

void ULevelInstanceComponent::PreEditUndo()
{
	UndoRedoCachedFilter = GetFilter();
}

void ULevelInstanceComponent::PostEditUndo()
{
	Super::PostEditUndo();
		
	UpdateComponentToWorld();
	UpdateEditorInstanceActor();

	if (GetFilter() != UndoRedoCachedFilter)
	{
		OnFilterChanged();
		FWorldPartitionActorFilter::RequestFilterRefresh(false);
		FWorldPartitionActorFilter::GetOnWorldPartitionActorFilterChanged().Broadcast();
	}
	UndoRedoCachedFilter = FWorldPartitionActorFilter();
}

void ULevelInstanceComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(ULevelInstanceComponent, EditFilter))
	{
		bWasDirtyBeforeEditFilterChange = GetPackage()->IsDirty();
	}

	Super::PreEditChange(PropertyAboutToChange);
}

void ULevelInstanceComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateEditorInstanceActor();

	const bool FilterProperty = PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULevelInstanceComponent, Filter);
	const bool EditFilterProperty = PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULevelInstanceComponent, EditFilter);

	if (FilterProperty || EditFilterProperty)
	{
		OnFilterChanged();
	}

	if (EditFilterProperty && !bWasDirtyBeforeEditFilterChange)
	{
		GetPackage()->SetDirtyFlag(false);
	}
}

void ULevelInstanceComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);

	UpdateEditorInstanceActor();
}

void ULevelInstanceComponent::UpdateEditorInstanceActor()
{
	if (!CachedEditorInstanceActorPtr.IsValid())
	{
		if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(GetOwner()))
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
			{
				if (LevelInstanceSubsystem->IsLoaded(LevelInstance))
				{
					LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [this, LevelInstance](AActor* LevelActor)
					{
						if (ALevelInstanceEditorInstanceActor* LevelInstanceEditorInstanceActor = Cast<ALevelInstanceEditorInstanceActor>(LevelActor))
						{
							check(LevelInstanceEditorInstanceActor->GetLevelInstanceID() == LevelInstance->GetLevelInstanceID());
							CachedEditorInstanceActorPtr = LevelInstanceEditorInstanceActor;
							return false;
						}
						return true;
					});
				}
			}
		}
	}

	if (ALevelInstanceEditorInstanceActor* EditorInstanceActor = Cast<ALevelInstanceEditorInstanceActor>(CachedEditorInstanceActorPtr.Get()))
	{
		EditorInstanceActor->UpdateWorldTransform(GetComponentTransform());
	}
}

void ULevelInstanceComponent::OnEdit()
{
	if (SpriteComponent)
	{
		SpriteComponent->SetVisibility(false);
	}
}

void ULevelInstanceComponent::OnEditOverrides()
{
	OnEdit();
}

void ULevelInstanceComponent::OnCommit()
{
	if (SpriteComponent)
	{
		SpriteComponent->SetVisibility(ShouldShowSpriteComponent());
	}
}

void ULevelInstanceComponent::OnCommitOverrides()
{
	OnCommit();
}

void ULevelInstanceComponent::UpdateEditFilter()
{
	// @todo_ow: for child level instances setup the proper filter based on parent hierarchy filter
	EditFilter = Filter;
}

void ULevelInstanceComponent::ClearCachedFilter()
{
	CachedFilteredActorsPerContainer.Reset();
}

bool ULevelInstanceComponent::IsEditFilter() const
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(GetOwner()))
	{
		return LevelInstance->IsEditing();
	}

	return false;
}

const TMap<FActorContainerID, TSet<FGuid>>& ULevelInstanceComponent::GetFilteredActorsPerContainer() const
{
	const FWorldPartitionActorFilter& CurrentFilter = GetFilter();
	if (CachedFilter != CurrentFilter)
	{
		CachedFilteredActorsPerContainer.Reset();
	}

	if (CachedFilteredActorsPerContainer.IsSet())
	{
		return CachedFilteredActorsPerContainer.GetValue();
	}

	TMap<FActorContainerID, TSet<FGuid>> FilteredActors;
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(GetOwner()))
	{
		// Fill Container Instance Filter
		UWorldPartitionSubsystem* WorldPartitionSubsystem = GetWorld()->GetSubsystem<UWorldPartitionSubsystem>();
		check(WorldPartitionSubsystem);
				
		FilteredActors = WorldPartitionSubsystem->GetFilteredActorsPerContainer(LevelInstance->GetLevelInstanceID().GetContainerID(), LevelInstance->GetWorldAssetPackage(), CurrentFilter, LevelInstance->GetLoadingFilterTypes());
	}
	
	CachedFilteredActorsPerContainer = MoveTemp(FilteredActors);
	CachedFilter = CurrentFilter;
	return CachedFilteredActorsPerContainer.GetValue();
}

#endif
