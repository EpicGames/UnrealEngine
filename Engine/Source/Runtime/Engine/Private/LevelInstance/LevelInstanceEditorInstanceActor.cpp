// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "GameFramework/WorldSettings.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorInstanceActor)

ALevelInstanceEditorInstanceActor::ALevelInstanceEditorInstanceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	RootComponent->Mobility = EComponentMobility::Static;
	
	// To keep the behavior of any calls to USceneComponent::GetActorPositionForRenderer() consistent between Editor and Game modes
	// we need to flag the root component so that it isn't considered as an AttachmentRoot.
	RootComponent->bIsNotRenderAttachmentRoot = true;
}

#if WITH_EDITOR
AActor* ALevelInstanceEditorInstanceActor::GetSelectionParent() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetLevelInstance(LevelInstanceID))
		{
			return CastChecked<AActor>(LevelInstance);
		}
	}

	return nullptr;
}

ALevelInstanceEditorInstanceActor* ALevelInstanceEditorInstanceActor::Create(ILevelInstanceInterface* LevelInstance, ULevel* LoadedLevel)
{
	FActorSpawnParameters SpawnParams;
	SpawnParams.OverrideLevel = LoadedLevel;
	SpawnParams.bHideFromSceneOutliner = true;
	SpawnParams.bCreateActorPackage = false;
	SpawnParams.ObjectFlags = RF_Transient;
	SpawnParams.bNoFail = true;

	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	ALevelInstanceEditorInstanceActor* InstanceActor = LevelInstanceActor->GetWorld()->SpawnActor<ALevelInstanceEditorInstanceActor>(LevelInstanceActor->GetActorLocation(), LevelInstanceActor->GetActorRotation(), SpawnParams);
	InstanceActor->SetActorScale3D(LevelInstanceActor->GetActorScale3D());
	InstanceActor->SetLevelInstanceID(LevelInstance->GetLevelInstanceID());
	
	for (AActor* LevelActor : LoadedLevel->Actors)
	{
		if (LevelActor && LevelActor->GetAttachParentActor() == nullptr && !LevelActor->IsChildActor() && LevelActor != InstanceActor)
		{
			LevelActor->AttachToActor(InstanceActor, FAttachmentTransformRules::KeepWorldTransform);
		}
	}

	InstanceActor->PushSelectionToProxies();

	return InstanceActor;
}

void ALevelInstanceEditorInstanceActor::UpdateWorldTransform(const FTransform& WorldTransform)
{
	GetRootComponent()->SetWorldTransform(WorldTransform);

	const ULevel* Level = GetLevel();

	ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level);
	check(LevelStreaming);

	AWorldSettings* WorldSettings = Level->GetWorldSettings();
	check(WorldSettings);

	// On the first move of the LI, build a cache to store the local-to-level transforms of attached actors with abs-flags
	// Since ALevelInstanceEditorInstanceActor is re-created after having an edit to the instanced level, this cache also gets dropped and re-created
	// Caching here avoids accessing the LI archetype and mapping archetype actors to streaming actors just for this purpose
	if (!bCacheBuilt)
	{
		bCacheBuilt = true;
		const FTransform LTInverse = LevelStreaming->LevelTransform.Inverse();
		for (AActor* Actor : Level->Actors)
		{
			if (!Actor || Actor == this)
			{
				continue;
			}
			USceneComponent* ActorRoot = Actor->GetRootComponent();
			if (!ActorRoot || !ActorRoot->GetAttachParent())
			{
				continue;
			}
			TInlineComponentArray<USceneComponent*> Components;
			Actor->GetComponents(Components);
			for (USceneComponent* Comp : Components)
			{
				if (!(Comp->IsUsingAbsoluteLocation() || Comp->IsUsingAbsoluteRotation() || Comp->IsUsingAbsoluteScale()))
				{
					continue;
				}
				// Cache the component's absolute-flagged transform in "level space", which is global space in the level archetype
				const FTransform LevelSpace(
					Comp->IsUsingAbsoluteRotation() ? LTInverse.TransformRotation(Comp->GetRelativeRotation().Quaternion()) : FQuat::Identity,
					Comp->IsUsingAbsoluteLocation() ? LTInverse.TransformPosition(Comp->GetRelativeLocation()) : FVector::ZeroVector,
					Comp->IsUsingAbsoluteScale()    ? LTInverse.GetScale3D() * Comp->GetRelativeScale3D() : FVector::OneVector
				);
				LevelSpaceTransformCache.Add(Comp, LevelSpace);
			}
		}
	}

	LevelStreaming->LevelTransform = FTransform(WorldSettings->LevelInstancePivotOffset) * WorldTransform;

	if (UWorldPartition* WorldPartition = WorldSettings->GetWorldPartition())
	{
		WorldPartition->SetInstanceTransform(LevelStreaming->LevelTransform);
	}

	// Actors with abs-flagged root components store position as a world-space value in
	// RelativeLocation rather than as a parent-relative offset, so moving the parent via
	// SetWorldTransform above leaves their world position unchanged. Recompute and write
	// the new world-space value from the cached level-space coordinates.
	const FTransform& LT = LevelStreaming->LevelTransform;
	for (const TPair<TObjectPtr<USceneComponent>, FTransform>& Pair : LevelSpaceTransformCache)
	{
		const TObjectPtr<USceneComponent>& Comp = Pair.Key;
		const FTransform& LevelSpace = Pair.Value;
		const FVector NewLoc = Comp->IsUsingAbsoluteLocation()
			? LT.TransformPosition(LevelSpace.GetLocation())
			: Comp->GetRelativeLocation();
		const FQuat NewRot = Comp->IsUsingAbsoluteRotation()
			? LT.TransformRotation(LevelSpace.GetRotation())
			: Comp->GetRelativeRotation().Quaternion();
		Comp->SetRelativeLocationAndRotation(NewLoc, NewRot);
		if (Comp->IsUsingAbsoluteScale())
		{
			Comp->SetRelativeScale3D(LT.GetScale3D() * LevelSpace.GetScale3D());
		}
		Comp->GetOwner()->MarkNeedsRecomputeBoundsOnceForGame();
	}
}

#endif
