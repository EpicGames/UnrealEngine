// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceEditorLevelStreaming.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorLevelStreaming)

#if WITH_EDITOR
#include "Engine/Engine.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Engine/Level.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Engine/LevelStreaming.h"
#include "LevelInstance/LevelInstanceEditorPivot.h"
#include "EditorLevelUtils.h"
#include "Engine/LevelBounds.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "PackageTools.h"
#include "WorldPartition/WorldPartition.h"
#endif

ULevelStreamingLevelInstanceEditor::ULevelStreamingLevelInstanceEditor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	SetShouldBeVisibleInEditor(true);

	if (!IsTemplate() && !GetWorld()->IsGameWorld())
	{
		GEngine->OnLevelActorAdded().AddUObject(this, &ULevelStreamingLevelInstanceEditor::OnLevelActorAdded);
	}
#endif
}

#if WITH_EDITOR
TOptional<FFolder::FRootObject> ULevelStreamingLevelInstanceEditor::GetFolderRootObject() const
{
	if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
	{
		if (AActor* Actor = CastChecked<AActor>(LevelInstance))
		{
			return FFolder::FRootObject(Actor);
		}
	}

	if (!LevelInstanceID.IsValid())
	{
		// When creating a new level instance, we have an invalid LevelInstanceID (until it gets loaded)
		// Return the world as root object.
		return FFolder::GetWorldRootFolder(GetWorld()).GetRootObject();
	}

	return TOptional<FFolder::FRootObject>();
}

ILevelInstanceInterface* ULevelStreamingLevelInstanceEditor::GetLevelInstance() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		return LevelInstanceSubsystem->GetLevelInstance(LevelInstanceID);
	}

	return nullptr;
}

ULevelStreamingLevelInstanceEditor* ULevelStreamingLevelInstanceEditor::Load(ILevelInstanceInterface* LevelInstance)
{
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	UWorld* CurrentWorld = LevelInstanceActor->GetWorld();

	// Make sure the level is not already loaded
	// For instance, drag and drop a Level into a Level Instance, then Edit the Level Instance will
	// result in the level at the wrong location (level transform wasn't applied)
	UPackage* LevelPackage = FindObject<UPackage>(nullptr, *LevelInstance->GetWorldAssetPackage());
	UWorld* LevelWorld = LevelPackage ? UWorld::FindWorldInPackage(LevelPackage) : nullptr;
	if (LevelWorld && LevelWorld->bIsWorldInitialized && !LevelWorld->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
	{
		UPackageTools::UnloadPackages({ LevelPackage });
	}

	ULevelStreamingLevelInstanceEditor* LevelStreaming = nullptr;

	// Set LevelInstanceID early
	auto OnLevelStreamingCreated = [LevelInstanceID = LevelInstance->GetLevelInstanceID()](ULevelStreaming* InLevelStreaming)
	{
		ULevelStreamingLevelInstanceEditor* LevelInstanceLevelStreaming = Cast<ULevelStreamingLevelInstanceEditor>(InLevelStreaming);
		check(LevelInstanceLevelStreaming);
		check(!LevelInstanceLevelStreaming->LevelInstanceID.IsValid());
		LevelInstanceLevelStreaming->LevelInstanceID = LevelInstanceID;
	};

	// If Asset is null we can create a level here
	if (LevelInstance->GetWorldAsset().IsNull())
	{
		EditorLevelUtils::FCreateNewStreamingLevelForWorldParams CreateNewStreamingLevelParams(ULevelStreamingLevelInstanceEditor::StaticClass(), TEXT(""));
		CreateNewStreamingLevelParams.Transform = LevelInstanceActor->GetTransform();
		CreateNewStreamingLevelParams.LevelStreamingCreatedCallback = OnLevelStreamingCreated;
		CreateNewStreamingLevelParams.bCreateWorldPartition = CurrentWorld->IsPartitionedWorld();

		LevelStreaming = LevelInstance->GetLevelInstanceSubsystem()->CreateNewStreamingLevelForWorld(*CurrentWorld, CreateNewStreamingLevelParams);
		if (LevelStreaming)
		{
			LevelInstanceActor->Modify();
			LevelInstance->SetWorldAsset(LevelStreaming->GetLoadedLevel()->GetTypedOuter<UWorld>());
		}
	}
	else
	{
		EditorLevelUtils::FAddLevelToWorldParams AddLevelToWorldParams(ULevelStreamingLevelInstanceEditor::StaticClass(), *LevelInstance->GetWorldAssetPackage());
		AddLevelToWorldParams.Transform = LevelInstanceActor->GetTransform();
		AddLevelToWorldParams.LevelStreamingCreatedCallback = OnLevelStreamingCreated;

		LevelStreaming = Cast<ULevelStreamingLevelInstanceEditor>(EditorLevelUtils::AddLevelToWorld(CurrentWorld, AddLevelToWorldParams));
	}
		
	if (LevelStreaming)
	{
		check(LevelStreaming);
		check(LevelStreaming->LevelInstanceID == LevelInstance->GetLevelInstanceID());

		GEngine->BlockTillLevelStreamingCompleted(LevelInstanceActor->GetWorld());

		if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
		{
			// Create special actor that will handle changing the pivot of this level
			FLevelInstanceEditorPivotHelper::Create(LevelInstance, LevelStreaming);
		}
				
		return LevelStreaming;
	}

	return nullptr;
}

void ULevelStreamingLevelInstanceEditor::Unload(ULevelStreamingLevelInstanceEditor* LevelStreaming)
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelStreaming->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
		{
			LoadedLevel->OnLoadedActorAddedToLevelPreEvent.RemoveAll(LevelStreaming);
			
			// Not needed if world is being cleaned up
			if (!LevelStreaming->GetWorld()->IsBeingCleanedUp())
			{
				LevelInstanceSubsystem->RemoveLevelsFromWorld({ LoadedLevel });
			}
		}
	}
}

void ULevelStreamingLevelInstanceEditor::OnLoadedActorsAddedToLevelPreEvent(const TArray<AActor*>& InActors)
{
	for (AActor* Actor : InActors)
	{
		OnLevelActorAdded(Actor);
	}
}

void ULevelStreamingLevelInstanceEditor::OnLevelActorAdded(AActor* InActor)
{
	if (InActor && InActor->GetLevel() == LoadedLevel)
	{
		FSetActorIsInLevelInstance SetIsInLevelInstance(InActor, ELevelInstanceType::LevelInstanceEdit);
		InActor->PushLevelInstanceEditingStateToProxies(true);
	}
}

void ULevelStreamingLevelInstanceEditor::OnPreInitializeContainerInstance(UActorDescContainerInstance::FInitializeParams& InInitParams, UActorDescContainerInstance* InContainerInstance)
{
	if (AActor* LevelInstanceActor = Cast<AActor>(GetLevelInstance()))
	{
		UWorldPartition* OwningWorldPartition = LevelInstanceActor->GetLevel()->GetWorldPartition();
		// Add parenting info to init param
		InInitParams.SetParent(OwningWorldPartition ? OwningWorldPartition->GetActorDescContainerInstance() : nullptr, LevelInstanceActor->GetActorGuid());
	}
	else
	{
		// When creating a new level instance the Level Instance actor doesn't exist yet
		check(ParentContainerInstance && ParentContainerGuid.IsValid());
		InInitParams.SetParent(ParentContainerInstance, ParentContainerGuid);
	}
}

void ULevelStreamingLevelInstanceEditor::OnLevelLoadedChanged(ULevel* InLevel)
{
	Super::OnLevelLoadedChanged(InLevel);

	if (ULevel* NewLoadedLevel = GetLoadedLevel())
	{
		check(InLevel == NewLoadedLevel);

		OnLoadedActorsAddedToLevelPreEvent(NewLoadedLevel->Actors);
		NewLoadedLevel->OnLoadedActorAddedToLevelPreEvent.AddUObject(this, &ULevelStreamingLevelInstanceEditor::OnLoadedActorsAddedToLevelPreEvent);

		// Avoid prompts for Level Instance editing
		NewLoadedLevel->bPromptWhenAddingToLevelBeforeCheckout = false;
		NewLoadedLevel->bPromptWhenAddingToLevelOutsideBounds = false;

		NewLoadedLevel->SetEditorPathOwner(Cast<AActor>(GetLevelInstance()));

		check(!NewLoadedLevel->bAlreadyMovedActors);
		if (AWorldSettings* WorldSettings = NewLoadedLevel->GetWorldSettings())
		{
			LevelTransform = FTransform(WorldSettings->LevelInstancePivotOffset) * LevelTransform;
		}

		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			if (UWorldPartition* OuterWorldPartition = NewLoadedLevel->GetWorldPartition())
			{
				check(!OuterWorldPartition->IsInitialized());
				OuterWorldPartition->OnActorDescContainerInstancePreInitialize.BindUObject(this, &ULevelStreamingLevelInstanceEditor::OnPreInitializeContainerInstance);
				OuterWorldPartition->bOverrideEnableStreamingInEditor = false;
			}

			LevelInstanceSubsystem->RegisterLoadedLevelStreamingLevelInstanceEditor(this);
		}
	}
}

FBox ULevelStreamingLevelInstanceEditor::GetBounds() const
{
	check(GetLoadedLevel());
	return ALevelBounds::CalculateLevelBounds(GetLoadedLevel());
}

#endif
