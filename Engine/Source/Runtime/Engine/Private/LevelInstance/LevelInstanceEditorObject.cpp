// Copyright Epic Games, Inc. All Rights Reserved.
#include "LevelInstance/LevelInstanceEditorObject.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorObject)

#if WITH_EDITOR
#include "Editor.h"
#include "EditorLevelUtils.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "LevelInstance/LevelInstanceEditorLevelStreaming.h"
#endif

#define LOCTEXT_NAMESPACE "LevelInstanceSubsystem"

ULevelInstanceEditorObject::ULevelInstanceEditorObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bCommittedChanges(false)
	, bCreatingChildLevelInstance(false)
	, EditWorld(nullptr)
#endif
{

}

#if WITH_EDITOR

bool ULevelInstanceEditorObject::CanDiscard(FText* OutReason /*= nullptr*/) const
{
	if (bMovedActors)
	{
		if(OutReason)
		{
			*OutReason = LOCTEXT("CantDiscardMovedActors", "Can't discard edit because some actors were moved from/to level while editing.");
		}

		return false;
	}
	
	return true;
}

void ULevelInstanceEditorObject::EnterEdit(UWorld* InEditWorld)
{
	check(!bMovedActors);
	check(!bCommittedChanges);
	check(!EditWorld);

	EditWorld = InEditWorld;
	EditorLevelUtils::OnMoveActorsToLevelEvent.AddUObject(this, &ULevelInstanceEditorObject::OnMoveActorsToLevel);
	FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &ULevelInstanceEditorObject::OnObjectPreSave);
	FEditorDelegates::OnPackageDeleted.AddUObject(this, &ULevelInstanceEditorObject::OnPackageDeleted);
}

void ULevelInstanceEditorObject::ExitEdit()
{
	EditorLevelUtils::OnMoveActorsToLevelEvent.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
	FEditorDelegates::OnPackageDeleted.RemoveAll(this);

	EditWorld = nullptr;
	bMovedActors = false;
	OtherPackagesToSave.Empty();
}

void ULevelInstanceEditorObject::OnPackageChanged(UPackage* Package)
{
	if (bCommittedChanges)
	{
		return;
	}

	check(EditWorld);
	if (EditWorld->GetPackage() == Package)
	{
		bCommittedChanges = true;
	}
	else
	{
		TSet<UPackage*> Packages;
		Packages.Append(EditWorld->GetPackage()->GetExternalPackages());
		if (Packages.Contains(Package))
		{
			bCommittedChanges = true;
		}
	}
}

void ULevelInstanceEditorObject::OnObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	if (!SaveContext.IsProceduralSave() && !SaveContext.IsFromAutoSave())
	{
		OnPackageChanged(Object->GetPackage());
	}
}

void ULevelInstanceEditorObject::OnPackageDeleted(UPackage* Package)
{
	OnPackageChanged(Package);
}

void ULevelInstanceEditorObject::OnMoveActorsToLevel(const TArray<AActor*>& ActorsToMove, const ULevel* DestLevel)
{	
	// When Child Level is being created we don't need to listen to this event because proper packages will be saved by the ULevelInstanceSubsystem code.
	if (bCreatingChildLevelInstance)
	{
		return;
	}

	auto DoesMoveInvolveEditingLevelInstance = [this](const ULevel* Level)
	{
		if (ULevelStreamingLevelInstanceEditor* LevelStreaming = Cast<ULevelStreamingLevelInstanceEditor>(FLevelUtils::FindStreamingLevel(Level)))
		{
			return true;
		}

		return false;
	};

	bool bNewlyMovedActors = false;
		
	if(DoesMoveInvolveEditingLevelInstance(DestLevel))
	{
		bNewlyMovedActors = true;
	}

	if (!bNewlyMovedActors)
	{
		for (const AActor* ActorToMove : ActorsToMove)
		{
			bNewlyMovedActors = DoesMoveInvolveEditingLevelInstance(ActorToMove->GetLevel());
			if (bNewlyMovedActors)
			{
				break;
			}
		}
	}

	if (bNewlyMovedActors)
	{
		Modify();
		bMovedActors = true;

		for (AActor* ActorToMove : ActorsToMove)
		{
			UPackage* PackageToSave = ActorToMove->GetPackage();
			if (!FPackageName::IsTempPackage(PackageToSave->GetName()))
			{
				OtherPackagesToSave.AddUnique(ActorToMove->GetPackage());
			}
		}
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
