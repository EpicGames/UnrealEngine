// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectSaveContext.h"
#include "LevelInstanceEditorObject.generated.h"

class AActor;
class ULevel;
class UWorld;
class UPackage;
class FText;

UCLASS()
class ULevelInstanceEditorObject : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
public:
	bool bCommittedChanges;
	bool bCreatingChildLevelInstance;
	TObjectPtr<UWorld> EditWorld;
	
	// If current Level edit has moved actors in/out of the level
	UPROPERTY()
	bool bMovedActors;

	// List of packages to save when committing level 
	UPROPERTY()
	TArray<TWeakObjectPtr<UPackage>> OtherPackagesToSave;
#endif

#if WITH_EDITOR

	bool CanDiscard(FText* OutReason = nullptr) const;

	void EnterEdit(UWorld* InEditWorld);
	void ExitEdit();

private:
	void OnPackageChanged(UPackage* Package);
	void OnObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext);
	void OnPackageDeleted(UPackage* Package);
	void OnMoveActorsToLevel(const TArray<AActor*>& ActorsToMove, const ULevel* DestLevel);
#endif
};