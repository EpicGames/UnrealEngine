// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceEditorPivot.h"
#include "Engine/Level.h"
#include "LevelInstance/LevelInstanceEditorPivotInterface.h"

#if WITH_EDITOR

#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Engine/LevelStreaming.h"
#include "GameFramework/WorldSettings.h"
#include "LevelUtils.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"

#endif

ULevelInstanceEditorPivotInterface::ULevelInstanceEditorPivotInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITOR

void ILevelInstanceEditorPivotInterface::UpdateOffset()
{
	if (FLevelUtils::IsApplyingLevelTransform())
	{
		return;
	}

	AActor* Actor = CastChecked<AActor>(this);
	// Offset Change is the relative transform of the pivot actor to its spawn transform (we only care about relative translation as we don't support rotation on pivot)
	FVector LocalToPivot = Actor->GetActorTransform().GetRelativeTransform(InitState.ActorTransform).GetTranslation();

	// We then apply that offset to the original pivot
	FVector NewPivotOffset = InitState.LevelOffset - LocalToPivot;

	AWorldSettings* WorldSettings = Actor->GetLevel()->GetWorldSettings();
	if (!NewPivotOffset.Equals(WorldSettings->LevelInstancePivotOffset))
	{
		WorldSettings->Modify();
		WorldSettings->LevelInstancePivotOffset = NewPivotOffset;
	}
}

ILevelInstanceEditorPivotInterface* FLevelInstanceEditorPivotHelper::Create(ILevelInstanceInterface* LevelInstance, ULevelStreaming* LevelStreaming)
{
	TSubclassOf<AActor> EditorPivotClass = LevelInstance->GetEditorPivotClass();
	if (!EditorPivotClass || !EditorPivotClass->ImplementsInterface(ULevelInstanceEditorPivotInterface::StaticClass()))
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.OverrideLevel = LevelStreaming->GetLoadedLevel();
	SpawnParams.bCreateActorPackage = false;
	SpawnParams.ObjectFlags |= RF_Transient;
	SpawnParams.bNoFail = true;

	// We place the pivot actor at the LevelInstance Transform so that it makes sense to the user (the pivot being the zero)
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	AActor* PivotActor = LevelInstanceActor->GetWorld()->SpawnActor<AActor>(EditorPivotClass, LevelInstanceActor->GetActorLocation(), LevelInstanceActor->GetActorRotation(), SpawnParams);
	ILevelInstanceEditorPivotInterface* PivotInterface = CastChecked<ILevelInstanceEditorPivotInterface>(PivotActor);

	AWorldSettings* WorldSettings = LevelStreaming->GetLoadedLevel()->GetWorldSettings();

	PivotInterface->SetInitState(ILevelInstanceEditorPivotInterface::FInitState { LevelInstance->GetLevelInstanceID(), PivotActor->GetActorTransform(), WorldSettings->LevelInstancePivotOffset });

	// Set Label last as this will call PostEditChangeProperty and update the offset so other fields need to be setup first.
	PivotActor->SetActorLabel(TEXT("Pivot"));

	return PivotInterface;
}

void FLevelInstanceEditorPivotHelper::ShowPivotLocation(const FVector& PivotLocation)
{
	if (GEditor)
	{
		// Show in viewport where the Pivot would be located
		GEditor->SetPivot(PivotLocation, false, true);
		GUnrealEd->SetPivotMovedIndependently(true);
		GEditor->RedrawLevelEditingViewports(false);
	}
}

void FLevelInstanceEditorPivotHelper::SetPivot(ILevelInstanceEditorPivotInterface* PivotInterface, ELevelInstancePivotType PivotType, AActor* PivotToActor)
{
	FVector Location = GetPivot(PivotInterface, PivotType, PivotToActor);
	AActor* PivotActor = CastChecked<AActor>(PivotInterface);
		
	PivotActor->Modify();
	PivotActor->SetActorLocation(Location);

	PivotInterface->UpdateOffset();

	ShowPivotLocation(Location);
}

FVector FLevelInstanceEditorPivotHelper::GetPivot(ILevelInstanceEditorPivotInterface* PivotInterface, ELevelInstancePivotType PivotType, AActor* PivotToActor)
{
	check(PivotType != ELevelInstancePivotType::Actor || PivotToActor != nullptr);
	AActor* PivotActor = CastChecked<AActor>(PivotInterface);

	if (PivotType == ELevelInstancePivotType::Actor)
	{
		return PivotToActor->GetActorLocation();
	}
	else if (PivotType == ELevelInstancePivotType::Center || PivotType == ELevelInstancePivotType::CenterMinZ)
	{
		ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(PivotActor->GetWorld());
		ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetLevelInstance(PivotInterface->GetLevelInstanceID());
		FBox OutBounds;
		LevelInstanceSubsystem->GetLevelInstanceBounds(LevelInstance, OutBounds);
		FVector Location = OutBounds.GetCenter();
		if (PivotType == ELevelInstancePivotType::CenterMinZ)
		{
			Location.Z = OutBounds.Min.Z;
		}
		return Location;
	}
	else
	{
		check(PivotType == ELevelInstancePivotType::WorldOrigin);
		return FVector::ZeroVector;
	}
}

#endif
