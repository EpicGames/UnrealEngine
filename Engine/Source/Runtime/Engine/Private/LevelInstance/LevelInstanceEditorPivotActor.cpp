// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceEditorPivotActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorPivotActor)

ALevelInstancePivot::ALevelInstancePivot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	RootComponent->Mobility = EComponentMobility::Static;
#if WITH_EDITORONLY_DATA
	bActorLabelEditable = false;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR

void ALevelInstancePivot::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	if (bFinished)
	{
		UpdateOffset();
	}
}

void ALevelInstancePivot::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateOffset();
}

void ALevelInstancePivot::PostEditUndo()
{
	Super::PostEditUndo();

	UpdateOffset();
}

#endif

