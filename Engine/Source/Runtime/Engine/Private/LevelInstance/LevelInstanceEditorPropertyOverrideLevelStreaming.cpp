// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstanceEditorPropertyOverrideLevelStreaming.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceEditorPropertyOverrideLevelStreaming)

#if WITH_EDITOR

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/LevelInstance/LevelInstanceContainerInstance.h"
#include "PackageTools.h"
#include "LevelUtils.h"
#include "ActorFolder.h"
#include "Misc/LazySingleton.h"
#include "Misc/ScopeExit.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerInstancingContext.h"
#include "Streaming/LevelStreamingDelegates.h"

#endif

ULevelStreamingLevelInstanceEditorPropertyOverride::ULevelStreamingLevelInstanceEditorPropertyOverride(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	SetShouldBeVisibleInEditor(true);
#endif
}

#if WITH_EDITOR

ILevelInstanceInterface* ULevelStreamingLevelInstanceEditorPropertyOverride::GetLevelInstance() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		return LevelInstanceSubsystem->GetLevelInstance(LevelInstanceID);
	}

	return nullptr;
}

UObject* ULevelStreamingLevelInstanceEditorPropertyOverride::GetArchetypeForObject(const UObject* InObject) const
{
	return EditorModule->GetArchetype(InObject);
}

ULevel* ULevelStreamingLevelInstanceEditorPropertyOverride::GetArchetypeLevel() const
{
	check(ArchetypeWorld);
	return ArchetypeWorld->PersistentLevel;
}

TOptional<FFolder::FRootObject> ULevelStreamingLevelInstanceEditorPropertyOverride::GetFolderRootObject() const
{
	if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
	{
		if (AActor* Actor = CastChecked<AActor>(LevelInstance))
		{
			return FFolder::FRootObject(Actor);
		}
	}
	// When LevelInstance is null, it's because the level instance is being streamed-out. 
	// Return the world as root object.
	return FFolder::GetWorldRootFolder(GetWorld()).GetRootObject();
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnActorReplacedEvent(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (AActor* Actor = InActorDescInstance->GetActor())
	{
		// If Archetype was replaced, find its corresponding instance actor to reapply properties on archetype only
		if (Actor->GetTypedOuter<UWorld>() == ArchetypeWorld)
		{
			if (AActor* FoundActor = FindObject<AActor>(GetLoadedLevel(), *Actor->GetName()))
			{
				ApplyPropertyOverrides({ FoundActor }, true, EApplyPropertyOverrideType::PreAndPostConstruction, EApplyActorType::Archetype);
			}
		}
		// If Actor was replaced reapply properties on it only
		else
		{
			ApplyPropertyOverrides({ Actor }, true, EApplyPropertyOverrideType::PreAndPostConstruction, EApplyActorType::Actor);
		}
	}

}

void ULevelStreamingLevelInstanceEditorPropertyOverride::ApplyPropertyOverrides(const TArray<AActor*>& InActors, bool bInAlreadyAppliedTransformOnActors, EApplyPropertyOverrideType ApplyPropertyOverrideType, EApplyActorType ApplyActorType)
{
	ILevelInstanceInterface* LevelInstance = GetLevelInstance();
	check(LevelInstance);

	ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem();
	check(LevelInstanceSubsystem);

	const FActorContainerID ContextContainerID = LevelInstanceSubsystem->GetLevelInstancePropertyOverridesContext(LevelInstance);

	const ILevelInstanceInterface* EditOwner = LevelInstanceSubsystem->GetLevelInstancePropertyOverridesEditOwner(LevelInstance);
	const FActorContainerID ArchetypeContextContainerID = EditOwner->GetLevelInstanceID().GetContainerID();

	const FTransform InverseTransform = LevelTransform.Inverse();

	for (AActor* Actor : InActors)
	{
		if (IsValid(Actor))
		{
			if(ApplyActorType == EApplyActorType::Archetype || ApplyActorType == EApplyActorType::ActorAndArchetype)
			{
				// Gather Archetype Contextual Property Overrides and apply them to the archetype actor (Archetype will get overrides applied up to the Property Edit Owner)
				TArray<FLevelInstanceActorPropertyOverride> LevelInstanceArchetypePropertyOverrides;
				if (LevelInstanceSubsystem->GetLevelInstancePropertyOverridesForActor(Actor, ArchetypeContextContainerID, LevelInstanceArchetypePropertyOverrides))
				{
					AActor* ArchetypeActor = CastChecked<AActor>(GetArchetypeForObject(Actor));

					// ArchetypeLevel is initialized at this point so we need to remove the level transform
					if (ArchetypeActor->GetRootComponent())
					{
						ApplyTransform(ArchetypeActor, InverseTransform, false);
					}
							
					// If we are applying Pre Construction Script overrides do it now
					bool bAppliedProperties = false;
					if (ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreConstructionScript || ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreAndPostConstruction)
					{
						for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : LevelInstanceArchetypePropertyOverrides)
						{
							bAppliedProperties |= ULevelInstancePropertyOverrideAsset::ApplyPropertyOverrides(LevelInstanceActorPropertyOverride.ActorPropertyOverride, ArchetypeActor, false);
						}
					}
					
					// If some Pre Construction Script properties were applied call ReRunConstructionScripts on Archetype
					if (bAppliedProperties)
					{
						ArchetypeActor->RerunConstructionScripts();
					}

					// If we are applying Post Construction Script overrides do it now
					if (ApplyPropertyOverrideType == EApplyPropertyOverrideType::PostConstructionScript || ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreAndPostConstruction)
					{
						for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : LevelInstanceArchetypePropertyOverrides)
						{
							ULevelInstancePropertyOverrideAsset::ApplyPropertyOverrides(LevelInstanceActorPropertyOverride.ActorPropertyOverride, ArchetypeActor, true);
						}
					}

					// Re-apply level transform
					if (ArchetypeActor->GetRootComponent())
					{
						ApplyTransform(ArchetypeActor, LevelTransform, true);
					}

					// Flag needed so we can track properly in OnObjectPropertyChanged
					FAddActorLevelInstanceFlags AddFlags(ArchetypeActor, ELevelInstanceFlags::HasPropertyOverrides);
				}
			}

			if(ApplyActorType == EApplyActorType::Actor || ApplyActorType == EApplyActorType::ActorAndArchetype)
			{	
				// Gather Contextual Property Overrides and apply them to the actor
				TArray<FLevelInstanceActorPropertyOverride> LevelInstanceActorPropertyOverrides;
				if (LevelInstanceSubsystem->GetLevelInstancePropertyOverridesForActor(Actor, ContextContainerID, LevelInstanceActorPropertyOverrides))
				{
					// If we have Property Overrides we need to Remove the level transform before applying them in case the Relative transform of the actors was modified
					if (bInAlreadyAppliedTransformOnActors && Actor->GetRootComponent())
					{
						ApplyTransform(Actor, InverseTransform, false);
					}
					
					bool bAppliedProperties = false;
					if (ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreConstructionScript || ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreAndPostConstruction)
					{
						for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : LevelInstanceActorPropertyOverrides)
						{
							bAppliedProperties |= ULevelInstancePropertyOverrideAsset::ApplyPropertyOverrides(LevelInstanceActorPropertyOverride.ActorPropertyOverride, Actor, false);
						}
					}

					// If some Pre Construction Script properties were applied and we are applying Post Construction Script properties, call ReRunConstructionScripts before
					if (bAppliedProperties && ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreAndPostConstruction)
					{
						Actor->RerunConstructionScripts();
					}

					if (ApplyPropertyOverrideType == EApplyPropertyOverrideType::PostConstructionScript || ApplyPropertyOverrideType == EApplyPropertyOverrideType::PreAndPostConstruction)
					{
						for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : LevelInstanceActorPropertyOverrides)
						{
							ULevelInstancePropertyOverrideAsset::ApplyPropertyOverrides(LevelInstanceActorPropertyOverride.ActorPropertyOverride, Actor, true);
						}
					}

					if (bInAlreadyAppliedTransformOnActors && Actor->GetRootComponent())
					{
						ApplyTransform(Actor, LevelTransform, true);
						Actor->GetRootComponent()->UpdateComponentToWorld();
						Actor->MarkComponentsRenderStateDirty();
					}

					// Flag actor as being overriden
					ELevelInstanceFlags FlagsToAdd = ELevelInstanceFlags::HasPropertyOverrides;
					if (LevelInstanceSubsystem->HasEditableLevelInstancePropertyOverrides(LevelInstanceActorPropertyOverrides))
					{
						EnumAddFlags(FlagsToAdd, ELevelInstanceFlags::HasEditablePropertyOverrides);
					}
					FAddActorLevelInstanceFlags AddFlags(Actor, FlagsToAdd);
				}
			}
		}
	}
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnLoadedActorsAddedToLevelPreEvent(const TArray<AActor*>& InActors)
{
	// This callback gets called prior to applying transform to new actors and calling ReRunConstructionScript on them
	const bool bAlreadyAppliedTransformOnActors = false;
	ApplyPropertyOverrides(InActors, bAlreadyAppliedTransformOnActors, EApplyPropertyOverrideType::PreConstructionScript);
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnLoadedActorsAddedToLevelPostEvent(const TArray<AActor*>& InActors)
{
	// This callback gets called after applying transform to new actors and calling ReRunConstructionScript on them
	const bool bAlreadyAppliedTransformOnActors = true;
	ApplyPropertyOverrides(InActors, bAlreadyAppliedTransformOnActors, EApplyPropertyOverrideType::PostConstructionScript);
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event)
{
	if (Event.ChangeType != EPropertyChangeType::Interactive)
	{
		AActor* Actor = Object->IsA<AActor>() ? Cast<AActor>(Object) : Object->GetTypedOuter<AActor>();
		if (Actor && Actor->GetLevel() == GetLoadedLevel())
		{
			FActorPropertyOverride ActorOverride;
			if (!ULevelInstancePropertyOverrideAsset::SerializeActorPropertyOverrides(this, Actor, /*bForReset=*/true, ActorOverride))
			{
				ELevelInstanceFlags FlagsToRemove = ELevelInstanceFlags::HasEditablePropertyOverrides;
				if (AActor* ArchetypeActor = Cast<AActor>(GetArchetypeForObject(Actor)); ArchetypeActor && !ArchetypeActor->HasLevelInstancePropertyOverrides())
				{
					EnumAddFlags(FlagsToRemove, ELevelInstanceFlags::HasPropertyOverrides);
				}
				FRemoveActorLevelInstanceFlags RemoveFlags(Actor, FlagsToRemove);
			}
			else
			{
				ELevelInstanceFlags FlagsToAdd = ELevelInstanceFlags::HasPropertyOverrides | ELevelInstanceFlags::HasEditablePropertyOverrides;
				FAddActorLevelInstanceFlags AddFlags(Actor, FlagsToAdd);
			}
		}
	}
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnPreInitializeContainerInstance(UActorDescContainerInstance::FInitializeParams& InInitParams, UActorDescContainerInstance* InContainerInstance)
{
	// Apply override container
	ULevelInstanceContainerInstance* LevelInstanceContainerInstance = CastChecked<ULevelInstanceContainerInstance>(InContainerInstance);
	AActor* LevelInstanceActor = Cast<AActor>(GetLevelInstance());

	// Property Overrides only support World Partition worlds
	UWorldPartition* OwningWorldPartition = LevelInstanceActor->GetLevel()->GetWorldPartition();
	check(OwningWorldPartition);

	// Add parenting info to init param
	InInitParams.SetParent(OwningWorldPartition->GetActorDescContainerInstance(), LevelInstanceActor->GetActorGuid());

	// New Level Instance actor won't have an ActorDescInstance yet
	if (FWorldPartitionActorDescInstance* ActorDescInstance = OwningWorldPartition->GetActorDescInstance(LevelInstanceActor->GetActorGuid()); ActorDescInstance && ActorDescInstance->IsChildContainerInstance())
	{
		// Override container 
		LevelInstanceContainerInstance->SetOverrideContainerAndAsset(ActorDescInstance->GetActorDesc()->GetChildContainer(), GetLevelInstance()->GetPropertyOverrideAsset());
	}
}

ULevelStreamingLevelInstanceEditorPropertyOverride* ULevelStreamingLevelInstanceEditorPropertyOverride::Load(ILevelInstanceInterface* LevelInstance)
{
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);

	bool bOutSuccess = false;
		
	const FString LongPackageName = LevelInstance->GetWorldAsset().GetLongPackageName();
	const FString ShortPackageName = FPackageName::GetShortName(LongPackageName);
	
	// Build a unique and deterministic LevelInstance level instance name by using LevelInstanceID. 
	const FString Suffix = FString::Printf(TEXT("%s_PropertyOverride_%016llx"), *ShortPackageName, LevelInstance->GetLevelInstanceID().GetHash());
			
	UWorld* LoadedArchetypeWorld = LoadArchetypeWorld(LongPackageName, Suffix);
	check(LoadedArchetypeWorld);

	FLoadLevelInstanceParams Params(LevelInstanceActor->GetWorld(), LevelInstance->GetWorldAssetPackage(), LevelInstanceActor->GetActorTransform());
	Params.OptionalLevelStreamingClass = ULevelStreamingLevelInstanceEditorPropertyOverride::StaticClass();
	Params.OptionalLevelNameOverride = &Suffix;
	Params.bLoadAsTempPackage = true;
	Params.EditorPathOwner = LevelInstanceActor;
	Params.LevelStreamingCreatedCallback = [LevelInstanceID = LevelInstance->GetLevelInstanceID(), LoadedArchetypeWorld](ULevelStreaming* NewLevelStreaming)
	{
		ULevelStreamingLevelInstanceEditorPropertyOverride* NewPropertyOverride = CastChecked<ULevelStreamingLevelInstanceEditorPropertyOverride>(NewLevelStreaming);
		NewPropertyOverride->ArchetypeWorld = LoadedArchetypeWorld;
		NewPropertyOverride->LevelInstanceID = LevelInstanceID;
	};
		
	ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreaming = Cast<ULevelStreamingLevelInstanceEditorPropertyOverride>(ULevelStreamingDynamic::LoadLevelInstance(Params, bOutSuccess));
	check(bOutSuccess);
	GEngine->BlockTillLevelStreamingCompleted(LevelInstanceActor->GetWorld());
	check(LevelStreaming->GetLoadedLevel() && LevelStreaming->GetLevelStreamingState() == ELevelStreamingState::LoadedVisible);
	return LevelStreaming;
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::Unload(ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreaming)
{
	if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
	{
		ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelStreaming->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
		check(LevelInstanceSubsystem);
			
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(LevelStreaming);

		LoadedLevel->OnLoadedActorAddedToLevelPreEvent.RemoveAll(LevelStreaming);
		LoadedLevel->OnLoadedActorAddedToLevelPostEvent.RemoveAll(LevelStreaming);
	
		UWorldPartition* OuterWorldPartition = LoadedLevel->GetWorldPartition();
		OuterWorldPartition->OnActorReplacedEvent.RemoveAll(LevelStreaming);

		UWorldPartition* ArchetypeWorldPartition = LevelStreaming->ArchetypeWorld->PersistentLevel->GetWorldPartition();
		ArchetypeWorldPartition->OnActorReplacedEvent.RemoveAll(LevelStreaming);

		check(LevelStreaming->EditorModule);
		LevelStreaming->EditorModule->SetPropertyOverridePolicy(nullptr);

		// ObjectReferences holds hard refs to archetype actors for GC protection during
		// the editing session. Release them now so the archetype world package can be fully GC'd
		// by UnloadArchetypeWorld; without this the package persists and the next call to
		// LoadArchetypeWorld would fire: check(!FindPackage(nullptr, *BasePackageName)).
		LevelStreaming->ObjectReferences.Reset();

		UnloadArchetypeWorld(LevelStreaming->ArchetypeWorld);
		LevelStreaming->ArchetypeWorld = nullptr;
				
		// Not needed if world is being cleaned up
		if (!LevelStreaming->GetWorld()->IsBeingCleanedUp())
		{
			// Reset Transactions because Property overriden actors support undo/redo and are about to be removed from world
			LevelStreaming->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>()->RemoveLevelsFromWorld({ LoadedLevel }, /* bResetTrans=*/ true);
		}
	}
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnCurrentStateChanged(ELevelStreamingState InPrevState, ELevelStreamingState InNewState)
{
	Super::OnCurrentStateChanged(InPrevState, InNewState);

	if (InNewState == ELevelStreamingState::LoadedVisible)
	{
		ILevelInstanceInterface* LevelInstance = GetLevelInstance();

		ULevel* Level = GetLoadedLevel();
		check(GetLevelStreamingState() == ELevelStreamingState::LoadedVisible);
		
		// Apply Post Construction Script Property Overrides for loaded actors and archetypes		
		check(Level->bAlreadyMovedActors);
		check(Level->bAreComponentsCurrentlyRegistered);

		ApplyPropertyOverrides(Level->Actors, Level->bAlreadyMovedActors, EApplyPropertyOverrideType::PostConstructionScript);
				
		// For now this class doesn't support partial loading, but if at some point it does this is needed to apply the property overrides when new actors get loaded in
		Level->OnLoadedActorAddedToLevelPostEvent.AddUObject(this, &ULevelStreamingLevelInstanceEditorPropertyOverride::OnLoadedActorsAddedToLevelPostEvent);

		FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &ULevelStreamingLevelInstanceEditorPropertyOverride::OnObjectPropertyChanged);

		// Push editing state to child actors
		for (AActor* Actor : Level->Actors)
		{
			if (IsValid(Actor))
			{
				FSetActorIsInLevelInstance SetIsInLevelInstance(Actor, ELevelInstanceType::LevelInstancePropertyOverride);
				Actor->PushLevelInstanceEditingStateToProxies(true);
			}
		}
	}
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::OnLevelLoadedChanged(ULevel* InLevel)
{
	Super::OnLevelLoadedChanged(InLevel);

	if (ULevel* NewLoadedLevel = GetLoadedLevel())
	{
		check(!GetWorld()->IsGameWorld());
		check(InLevel == NewLoadedLevel);

		ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
		check(LevelInstanceSubsystem);
		
		EditorModule = &FModuleManager::GetModuleChecked<ILevelInstanceEditorModule>("LevelInstanceEditor");
		check(EditorModule);

		// Setup Policy 
		PropertyOverridePolicy.Initialize(NewLoadedLevel, ArchetypeWorld->PersistentLevel, ULevelInstanceSettings::Get()->GetPropertyOverridePolicy());
		EditorModule->SetPropertyOverridePolicy(&PropertyOverridePolicy);

		// Compute LevelTransform
		check(!NewLoadedLevel->bAlreadyMovedActors);
		AWorldSettings* WorldSettings = NewLoadedLevel->GetWorldSettings();
		check(WorldSettings);
		LevelTransform = FTransform(WorldSettings->LevelInstancePivotOffset) * LevelTransform;

		// Apply Transform to Archetype so that reset to default properly goes back to transformed location
		FLevelUtils::FApplyLevelTransformParams TransformParams(ArchetypeWorld->PersistentLevel, LevelTransform);
		TransformParams.bSetRelativeTransformDirectly = true;
		TransformParams.bDoPostEditMove = true;
		FLevelUtils::ApplyLevelTransform(TransformParams);

		// Apply Pre Construction Script Property Overrides on Level Actors & Archetypes
		check(!NewLoadedLevel->bAreComponentsCurrentlyRegistered);
		ApplyPropertyOverrides(InLevel->Actors, NewLoadedLevel->bAlreadyMovedActors, EApplyPropertyOverrideType::PreConstructionScript);
										
		// Register of Loaded Actors Added Pre Event to apply Pre Construction Script Property Overrides on Actors that get loaded after (LoaderAdapters)
		NewLoadedLevel->OnLoadedActorAddedToLevelPreEvent.AddUObject(this, &ULevelStreamingLevelInstanceEditorPropertyOverride::OnLoadedActorsAddedToLevelPreEvent);
								
		// Register to Level Instance Subsystem
		LevelInstanceSubsystem->RegisterLoadedLevelStreamingPropertyOverride(this);

		// Setup Container Instance
		UWorldPartition* OuterWorldPartition = NewLoadedLevel->GetWorldPartition();
		check(OuterWorldPartition);
		check(!OuterWorldPartition->IsInitialized());
		OuterWorldPartition->SetContainerInstanceClass(ULevelInstanceContainerInstance::StaticClass());
		OuterWorldPartition->OnActorDescContainerInstancePreInitialize.BindUObject(this, &ULevelStreamingLevelInstanceEditorPropertyOverride::OnPreInitializeContainerInstance);
		OuterWorldPartition->OnActorReplacedEvent.AddUObject(this, &ULevelStreamingLevelInstanceEditorPropertyOverride::OnActorReplacedEvent);

		UWorldPartition* ArchetypeWorldPartition = ArchetypeWorld->PersistentLevel->GetWorldPartition();
		ArchetypeWorldPartition->OnActorReplacedEvent.AddUObject(this, &ULevelStreamingLevelInstanceEditorPropertyOverride::OnActorReplacedEvent);
		
		// Partial loading not supported for now while editing Property Overrides
		OuterWorldPartition->bOverrideEnableStreamingInEditor = false;
	}
}

UWorld* ULevelStreamingLevelInstanceEditorPropertyOverride::LoadArchetypeWorld(const FString& InWorldPackageName, const FString& InSuffix)
{
	// Load Archetype World (with all actors)
	FLinkerInstancingContext InstancingContext({ ULevel::LoadAllExternalObjectsTag });
	
	// Load as instanced world
	const FString BasePackageName = FString::Printf(TEXT("%s_%s_Archetype"), *FPackageName::GetLongPackagePath(InWorldPackageName), *InSuffix);
		
	check(!FindPackage(nullptr, *BasePackageName));

	UPackage* CreatedPackage = CreatePackage(*BasePackageName);
	UPackage* LoadedPackage = LoadPackage(CreatedPackage, *InWorldPackageName, LOAD_None, nullptr, &InstancingContext);
	check(CreatedPackage == LoadedPackage);
		
	UWorld* ArchetypeWorld = UWorld::FindWorldInPackage(LoadedPackage);
	check(ArchetypeWorld);	
	
	ArchetypeWorld->InitWorld(UWorld::InitializationValues()
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(false)
		.CreatePhysicsScene(false)
		.CreateFXSystem(false)
		.CreateAISystem(false)
		.CreateNavigation(false));
	ArchetypeWorld->UpdateWorldComponents(true, true);

	return ArchetypeWorld;
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::UnloadArchetypeWorld(UWorld* InWorld)
{
	InWorld->DestroyWorld(false);
	
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
}

void ULevelStreamingLevelInstanceEditorPropertyOverride::ApplyTransform(AActor* InActor, const FTransform& InTransform, bool bDoPostEditMove)
{
	FLevelUtils::FApplyLevelTransformParams TransformParams(InActor->GetLevel(), InTransform);
	TransformParams.Actor = InActor;
	TransformParams.bDoPostEditMove = bDoPostEditMove;
	TransformParams.bSetRelativeTransformDirectly = true;
	FLevelUtils::ApplyLevelTransform(TransformParams);
}

#endif
