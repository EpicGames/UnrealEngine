// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Level.h"
#include "Engine/LevelStreamingDynamic.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstance/LevelInstancePropertyOverridePolicy.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "LevelInstanceEditorPropertyOverrideLevelStreaming.generated.h"

class ILevelInstanceInterface;
class ILevelInstanceEditorModule;

UCLASS(Transient, MinimalAPI)
class ULevelStreamingLevelInstanceEditorPropertyOverride : public ULevelStreamingDynamic
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	ILevelInstanceInterface* GetLevelInstance() const;
		
	// Begin ULevelStreaming Interface
	virtual bool ShowInLevelCollection() const override { return false; }
	virtual bool IsUserManaged() const override { return false; }
	virtual TOptional<FFolder::FRootObject> GetFolderRootObject() const override;
	// End ULevelStreaming Interface

private:
	friend struct FLevelInstancePropertyOverrideUtils;
	friend class ULevelInstancePropertyOverrideAsset;

	UObject* GetArchetypeForObject(const UObject* InObject) const;
	ULevel* GetArchetypeLevel() const;
	
	const FLevelInstanceID& GetLevelInstanceID() const { return LevelInstanceID; }

	friend class ULevelInstanceSubsystem;

	static ULevelStreamingLevelInstanceEditorPropertyOverride* Load(ILevelInstanceInterface* LevelInstanceActor);
	static void Unload(ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreaming);

	// Begin ULevelStreaming Interface
	virtual void OnLevelLoadedChanged(ULevel* Level) override;
	// End ULevelStreaming Interface
			
	enum EApplyPropertyOverrideType
	{
		PreConstructionScript,
		PostConstructionScript,
		PreAndPostConstruction
	};

	enum EApplyActorType
	{
		Actor,
		Archetype,
		ActorAndArchetype
	};

	void ApplyPropertyOverrides(const TArray<AActor*>& InActors, bool bInAlreadyAppliedTransformOnActors, EApplyPropertyOverrideType ApplyPropertyOverrideType, EApplyActorType ApplyActorType = EApplyActorType::ActorAndArchetype);
	void OnLoadedActorsAddedToLevelPreEvent(const TArray<AActor*>& InActors);
	void OnLoadedActorsAddedToLevelPostEvent(const TArray<AActor*>& InActors);
	void OnActorReplacedEvent(FWorldPartitionActorDescInstance* InActorDescInstance);
	void OnPreInitializeContainerInstance(UActorDescContainerInstance::FInitializeParams& InInitParams, UActorDescContainerInstance* InContainerInstance);
	virtual void OnCurrentStateChanged(ELevelStreamingState InPrevState, ELevelStreamingState InNewState) override;
	void OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event);

	static UWorld* LoadArchetypeWorld(const FString& InWorldPackageName, const FString& InSuffix);
	static void UnloadArchetypeWorld(UWorld* InWorld);

	static void ApplyTransform(AActor* InActor, const FTransform& InTransform, bool bDoPostEditMove);
	
	FLevelInstanceID LevelInstanceID;
	
	class FPropertyOverridePolicy : public ILevelInstanceEditorModule::IPropertyOverridePolicy
	{
	public:
		virtual ~FPropertyOverridePolicy() = default;

		void Initialize(ULevel* InLevel, ULevel* InLevelArchetype, ULevelInstancePropertyOverridePolicy* InPolicy)
		{
			Level = InLevel;
			LevelArchetype = InLevelArchetype;
			Policy = InPolicy;
		}
				
		virtual UObject* GetArchetypeForObject(const UObject* InObject)  const override
		{
			return InObject == Level ? LevelArchetype : nullptr;
		}

		virtual bool CanEditProperty(const FEditPropertyChain& PropertyChain, const UObject* Object) const override
		{
			return CanEditProperty(PropertyChain.GetActiveNode()->GetValue(), Object);
		}

		virtual bool CanEditProperty(const FProperty* Property, const UObject* Object) const override
		{
			if (Object->GetTypedOuter<ULevel>() == Level)
			{
				return Policy && Policy->CanOverrideProperty(Property);
			}

			// Don't interfere with other levels
			return  true;
		}

	private:
		TObjectPtr<ULevel> Level = nullptr;
		TObjectPtr<ULevel> LevelArchetype = nullptr;
		TObjectPtr<ULevelInstancePropertyOverridePolicy> Policy = nullptr;
	};
		
	FPropertyOverridePolicy PropertyOverridePolicy;
	
	ILevelInstanceEditorModule* EditorModule = nullptr;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UWorld> ArchetypeWorld = nullptr;

	// Central GC-protection store for all UObject hard refs serialized during the editing session.
	// Populated by FWorldPartitionPropertyOverrideArchive and released in Unload().
	UPROPERTY(Transient)
	TSet<TObjectPtr<UObject>> ObjectReferences;
#endif
};
