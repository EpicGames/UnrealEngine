// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceLevelStreaming.h"
#include "WorldPartition/WorldPartition.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceInterface)

#if WITH_EDITOR
#include "LevelInstance/LevelInstanceComponent.h"
#endif

ULevelInstanceInterface::ULevelInstanceInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITOR
bool ILevelInstanceInterface::SupportsPartialEditorLoading() const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			if (LevelInstanceSubsystem->IsEditingLevelInstance(this) || LevelInstanceSubsystem->IsEditingLevelInstancePropertyOverrides(this) || LevelInstanceSubsystem->HasChildEdit(this))
			{
				return false;
			}

			const AActor* Actor = CastChecked<AActor>(this);

			// If the level instance actor (or any parent actor) is not spatially loaded, don't partially load.
			if (!Actor->GetIsSpatiallyLoaded())
			{
				return false;
			}

			// If the level instance actor (or any parent actor) is unsaved, don't partially load.
			if (Actor->GetPackage()->HasAllPackagesFlags(PKG_NewlyCreated))
			{
				return false;
			}

			// If the level is loaded
			if (ULevel* Level = GetLoadedLevel())
			{
				UWorldPartition* WorldPartition = Level->GetWorldPartition();

				// Disable partial loading for non WP levels
				if (!WorldPartition)
				{
					return false;
				}
				// Check that it has editor streaming enabled
				else if (WorldPartition->IsInitialized() && !WorldPartition->IsStreamingEnabledInEditor())
				{
					return false;
				}
			}

			if (ILevelInstanceInterface* Parent = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
			{
				if (!Parent->SupportsPartialEditorLoading())
				{
					return false;
				}
			}

			return true;
		}
	}

	return false;
}
#endif

ULevelInstanceSubsystem* ILevelInstanceInterface::GetLevelInstanceSubsystem() const
{
	const AActor* Actor = CastChecked<AActor>(this);
	return Actor->GetWorld() ? Actor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>() : nullptr;
}

bool ILevelInstanceInterface::IsLoaded() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->IsLoaded(this);
	}

	return false;
}

void ILevelInstanceInterface::LoadLevelInstance()
{
	if (IsLoadingEnabled())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			bool bForce = false;
#if WITH_EDITOR
			// When reinstancing or world wasn't ticked between changes. Avoid reloading the level but if the package changed, force the load
			bForce = IsLoaded() && LevelInstanceSubsystem->GetLevelInstanceLevel(this)->GetPackage()->GetLoadedPath() != FPackagePath::FromPackageNameChecked(GetWorldAssetPackage());
#endif
			LevelInstanceSubsystem->RequestLoadLevelInstance(this, bForce);
		}
	}
}

void ILevelInstanceInterface::UnloadLevelInstance()
{
	if (IsLoadingEnabled())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
#if WITH_EDITOR
			check(!HasDirtyChildren());
#endif
			LevelInstanceSubsystem->RequestUnloadLevelInstance(this);
		}
	}
}

ULevelStreamingLevelInstance* ILevelInstanceInterface::GetLevelStreaming() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->GetLevelInstanceLevelStreaming(this);
	}

	return nullptr;
}

TSubclassOf<ULevelStreamingLevelInstance> ILevelInstanceInterface::GetLevelStreamingClass() const
{
	return ULevelStreamingLevelInstance::StaticClass();
}

void ILevelInstanceInterface::UpdateLevelInstanceFromWorldAsset()
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
#if WITH_EDITOR
			LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(CastChecked<AActor>(this), [](const ILevelInstanceInterface* Ancestor)
			{
				if (ULevelInstanceComponent* LevelInstanceComponent = Ancestor->GetLevelInstanceComponent())
				{
					LevelInstanceComponent->ClearCachedFilter();
				}
				return true;
			});
#endif

			if (IsWorldAssetValid() && IsLoadingEnabled())
			{
				const bool bForceUpdate = true;
				LevelInstanceSubsystem->RequestLoadLevelInstance(this, bForceUpdate);
			}
			else if (IsLoaded())
			{
				UnloadLevelInstance();
			}
		}
	}
}

ULevel* ILevelInstanceInterface::GetLoadedLevel() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->GetLevelInstanceLevel(this);
	}

	return nullptr;
}

#if WITH_EDITOR

bool ILevelInstanceInterface::IsLevelInstanceHLODRelevant() const
{
	if (const ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		if (const AActor* Owner = LevelInstanceComponent->GetOwner())
		{
			return Owner->IsHLODRelevant();
		}
	}
	return false;
}

bool ILevelInstanceInterface::HasStandaloneHLOD() const
{
	if (ULevel* LoadedLevel = GetLoadedLevel())
	{
		if (UWorldPartition* WorldPartition = LoadedLevel->GetWorldPartition())
		{
			return WorldPartition->HasStandaloneHLOD();
		}
	}
	return false;
}

bool ILevelInstanceInterface::IsEditing() const
{
	// No need to check LevelInstanceID here since Editing Level Instance is referenced directly:
	// Level Instance ID can become invalid while Actor is being edited through AActor property change events causing component unregisters
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->IsEditingLevelInstance(this);
	}
	
	return false;
}

bool ILevelInstanceInterface::IsEditingPropertyOverrides() const
{
	// No need to check LevelInstanceID here since Override Level Instance is referenced directly:
	// Level Instance ID can become invalid while Actor is being edited through AActor property change events causing component unregisters
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->IsEditingLevelInstancePropertyOverrides(this);
	}

	return false;
}

bool ILevelInstanceInterface::HasChildEdit() const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->HasChildEdit(this);
		}
	}

	return false;
}

bool ILevelInstanceInterface::HasParentEdit() const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->HasParentEdit(this);
		}
	}

	return false;
}

bool ILevelInstanceInterface::IsDirty() const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->IsEditingLevelInstanceDirty(this);
		}
	}

	return false;
}

bool ILevelInstanceInterface::HasDirtyChildren() const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->HasDirtyChildrenLevelInstances(this);
		}
	}

	return false;
}

bool ILevelInstanceInterface::CanEnterEditPropertyOverrides(FText* OutReason) const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->CanEditLevelInstancePropertyOverrides(this, OutReason);
		}
	}

	return false;
}

bool ILevelInstanceInterface::CanEnterEdit(FText* OutReason) const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->CanEditLevelInstance(this, OutReason);
		}
	}

	return false;
}

bool ILevelInstanceInterface::EnterEditPropertyOverrides(AActor* ContextActor)
{
	ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem();
	check(LevelInstanceSubsystem);
	LevelInstanceSubsystem->EditLevelInstancePropertyOverrides(this, ContextActor);
	return true;
}

bool ILevelInstanceInterface::EnterEdit(AActor* ContextActor)
{
	ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem();
	check(LevelInstanceSubsystem);
	LevelInstanceSubsystem->EditLevelInstance(this, ContextActor);
	return true;
}

void ILevelInstanceInterface::OnEdit()
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		LevelInstanceComponent->OnEdit();
	}
}

void ILevelInstanceInterface::OnEditPropertyOverrides()
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		LevelInstanceComponent->OnEditOverrides();
	}
}

bool ILevelInstanceInterface::CanExitEditPropertyOverrides(bool bDiscardEdits, FText* OutReason) const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->CanCommitLevelInstancePropertyOverrides(this, bDiscardEdits, OutReason);
		}
	}

	return false;
}

bool ILevelInstanceInterface::CanExitEdit(bool bDiscardEdits, FText* OutReason) const
{
	if (HasValidLevelInstanceID())
	{
		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
		{
			return LevelInstanceSubsystem->CanCommitLevelInstance(this, bDiscardEdits, OutReason);
		}
	}

	return false;
}

bool ILevelInstanceInterface::ExitEditPropertyOverrides(bool bDiscardEdits)
{
	ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem();
	check(LevelInstanceSubsystem);
	return LevelInstanceSubsystem->CommitLevelInstancePropertyOverrides(this, bDiscardEdits);
}


bool ILevelInstanceInterface::ExitEdit(bool bDiscardEdits)
{
	ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem();
	check(LevelInstanceSubsystem);
	return LevelInstanceSubsystem->CommitLevelInstance(this, bDiscardEdits);
}

void ILevelInstanceInterface::OnCommitPropertyOverrides(bool bChanged)
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		LevelInstanceComponent->OnCommitOverrides();
	}
}

void ILevelInstanceInterface::OnCommit(bool bChanged)
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		LevelInstanceComponent->OnCommit();
	}
}

bool ILevelInstanceInterface::SetCurrent()
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->SetCurrent(this);
	}

	return false;
}

bool ILevelInstanceInterface::MoveActorsTo(const TArray<AActor*>& ActorsToMove)
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		return LevelInstanceSubsystem->MoveActorsTo(this, ActorsToMove);
	}

	return false;
}

const FWorldPartitionActorFilter& ILevelInstanceInterface::GetFilter() const
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		return LevelInstanceComponent->GetFilter();
	}

	static FWorldPartitionActorFilter NoFilter;
	return NoFilter;
}

const TMap<FActorContainerID, TSet<FGuid>>& ILevelInstanceInterface::GetFilteredActorsPerContainer() const
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		return LevelInstanceComponent->GetFilteredActorsPerContainer();
	}

	static TMap<FActorContainerID, TSet<FGuid>> NoFilteredActors;
	return NoFilteredActors;
}

void ILevelInstanceInterface::SetFilter(const FWorldPartitionActorFilter& InFilter, bool bNotify)
{
	if (ULevelInstanceComponent* LevelInstanceComponent = GetLevelInstanceComponent())
	{
		LevelInstanceComponent->SetFilter(InFilter, bNotify);
	}
}

EWorldPartitionActorFilterType ILevelInstanceInterface::GetLoadingFilterTypes() const
{
	const AActor* Actor = CastChecked<AActor>(this);
	if (Actor->GetWorld() && Actor->GetWorld()->IsPartitionedWorld())
	{
		return EWorldPartitionActorFilterType::All;
	}

	return EWorldPartitionActorFilterType::None;
}


EWorldPartitionActorFilterType ILevelInstanceInterface::GetDetailsFilterTypes() const
{
	const AActor* Actor = CastChecked<AActor>(this);
	if (Actor->IsTemplate() || (Actor->GetWorld() && Actor->GetWorld()->IsPartitionedWorld()))
	{
		return EWorldPartitionActorFilterType::Loading;
	}

	return EWorldPartitionActorFilterType::None;
}

#endif


