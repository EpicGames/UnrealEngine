// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceLevelStreaming.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstancePrivate.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Engine/LevelBounds.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceLevelStreaming)

#if WITH_EDITOR
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "WorldPartition/LevelInstance/LevelInstanceContainerInstance.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "LevelUtils.h"
#include "ActorFolder.h"
#include "Misc/LazySingleton.h"
#include "Misc/ScopeExit.h"
#include "UObject/LinkerLoad.h"
#include "Engine/GameEngine.h"
#include "Streaming/LevelStreamingDelegates.h"

static bool GDisableLevelInstanceEditorPartialLoading = false;
FAutoConsoleVariableRef CVarDisableLevelInstanceEditorPartialLoading(
	TEXT("wp.Editor.DisableLevelInstanceEditorPartialLoading"),
	GDisableLevelInstanceEditorPartialLoading,
	TEXT("Allow disabling partial loading of level instances in the editor."),
	ECVF_Default);

static bool GForceEditorWorldMode = false;
FAutoConsoleVariableRef CVarForceEditorWorldMode(
	TEXT("LevelInstance.ForceEditorWorldMode"),
	GForceEditorWorldMode,
	TEXT("Allow -game instances to behave like an editor with temporary root object attached to instance. This will prevent HLOD from working in -game. This feature is only supported on non WP worlds."),
	ECVF_Default);


namespace FLevelInstanceLevelStreamingUtils
{
	static void MarkObjectsInPackageAsTransientAndNonTransactional(UPackage* InPackage)
	{
		InPackage->ClearFlags(RF_Transactional);
		InPackage->SetFlags(RF_Transient);
		ForEachObjectWithPackage(InPackage, [](UObject* Obj)
			{
				Obj->SetFlags(RF_Transient);
				Obj->ClearFlags(RF_Transactional);
				return true;
			}, EGetObjectsFlags::IncludeNestedObjects);
	}
}

#endif

ULevelStreamingLevelInstance::ULevelStreamingLevelInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, CachedBounds(ForceInit)
	, bResetLoadersCalled(false)
#endif
{
#if WITH_EDITOR
	SetShouldBeVisibleInEditor(true);
#endif
}

ILevelInstanceInterface* ULevelStreamingLevelInstance::GetLevelInstance() const
{
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
	{
		return LevelInstanceSubsystem->GetLevelInstance(LevelInstanceID);
	}

	return nullptr;
}

#if WITH_EDITOR

bool ULevelStreamingLevelInstance::IsEditorWorldMode() const
{
	bool bCanSupportForceEditorWorldMode = !GIsEditor;
	if (GForceEditorWorldMode && !GIsEditor)
	{
		UWorld* OwningWorld = GetWorld();
		check(OwningWorld);
		// We do not support GForceEditorWorldMode in WP worlds.
		if (OwningWorld->GetWorldPartition() != nullptr)
		{
			bCanSupportForceEditorWorldMode = false;
		}
	}
	return (GForceEditorWorldMode && bCanSupportForceEditorWorldMode) || !GetWorld()->IsGameWorld();
}

TOptional<FFolder::FRootObject> ULevelStreamingLevelInstance::GetFolderRootObject() const
{
	if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
	{
		if (AActor* Actor = CastChecked<AActor>(LevelInstance))
		{
			return FFolder::FRootObject(Actor);
		}
	}
	// When LevelInstance is null, it's because the level instance is being streamed-out. 
	// Return the world as root object.
	return FFolder::GetWorldRootFolder(GetWorld()).GetRootObject();
}

FBox ULevelStreamingLevelInstance::GetBounds() const
{
	check(GetLoadedLevel());
	FTransform LevelInstanceTransform = Cast<AActor>(GetLevelInstance())->GetTransform();
	if (!CachedBounds.IsValid || !CachedTransform.Equals(LevelInstanceTransform))
	{
		CachedTransform = LevelInstanceTransform;
		CachedBounds = ALevelBounds::CalculateLevelBounds(GetLoadedLevel());
		
		// Possible if Level has no bounds relevant actors
		if (!CachedBounds.IsValid)
		{
			CachedBounds = FBox(CachedTransform.GetLocation(), CachedTransform.GetLocation());
		}
	}
	check(CachedBounds.IsValid);
	return CachedBounds;
}

void ULevelStreamingLevelInstance::OnActorReplacedEvent(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
	{
		ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem();
		check(LevelInstanceSubsystem);
		if (AActor* Actor = InActorDescInstance->GetActor())
		{
			ApplyPropertyOverrides({ Actor }, true, EApplyPropertieOverrideType::PreAndPostConstruction);
		}
	}
}

void ULevelStreamingLevelInstance::OnLoadedActorsAddedToLevelPreEvent(const TArray<AActor*>& InActors)
{
	InitializeActors(InActors);
	ApplyPropertyOverrides(InActors, false, EApplyPropertieOverrideType::PreConstruction);
}

void ULevelStreamingLevelInstance::ApplyPropertyOverrides(const TArray<AActor*>& InActors, bool bInAlreadyAppliedTransformOnActors, EApplyPropertieOverrideType InApplyPropertyOverrideType)
{
	if (IsEditorWorldMode())
	{
		if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
		{
			ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem();
			check(LevelInstanceSubsystem);

			const FActorContainerID ContainerID = LevelInstance->GetLevelInstanceID().GetContainerID();
			const FActorContainerID ContextContainerID = LevelInstanceSubsystem->GetLevelInstancePropertyOverridesContext(LevelInstance);

			for (AActor* Actor : InActors)
			{
				if (IsValid(Actor))
				{
					// Gather Contextual Property Overrides and apply them
					TArray<FLevelInstanceActorPropertyOverride> LevelInstanceActorPropertyOverrides;
					if (LevelInstanceSubsystem->GetLevelInstancePropertyOverridesForActor(Actor, ContextContainerID, LevelInstanceActorPropertyOverrides))
					{
						// If we have Property Overrides we need to Remove the level transform before applying them in case the Relative transform of the actors was modified
						if (bInAlreadyAppliedTransformOnActors && Actor->GetRootComponent())
						{
							FLevelUtils::RemoveEditorTransform(this, false, Actor);
						}

						bool bAppliedProperties = false;
						if (InApplyPropertyOverrideType == EApplyPropertieOverrideType::PreConstruction || InApplyPropertyOverrideType == EApplyPropertieOverrideType::PreAndPostConstruction)
						{
							for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : LevelInstanceActorPropertyOverrides)
							{
								bAppliedProperties |= ULevelInstancePropertyOverrideAsset::ApplyPropertyOverrides(LevelInstanceActorPropertyOverride.ActorPropertyOverride, Actor, false);
							}
						}

						// If we did apply some properties re-run construction script on actor
						if (bAppliedProperties && InApplyPropertyOverrideType == EApplyPropertieOverrideType::PreAndPostConstruction)
						{
							Actor->RerunConstructionScripts();
						}

						if (InApplyPropertyOverrideType == EApplyPropertieOverrideType::PostConstruction || InApplyPropertyOverrideType == EApplyPropertieOverrideType::PreAndPostConstruction)
						{
							for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : LevelInstanceActorPropertyOverrides)
							{
								ULevelInstancePropertyOverrideAsset::ApplyPropertyOverrides(LevelInstanceActorPropertyOverride.ActorPropertyOverride, Actor, true);
							}
						}
						
						if (bInAlreadyAppliedTransformOnActors && Actor->GetRootComponent())
						{
							FLevelUtils::ApplyEditorTransform(this, true, Actor);
							Actor->GetRootComponent()->UpdateComponentToWorld();
							Actor->MarkComponentsRenderStateDirty();
						}

						// Flag actor as being overriden
						ELevelInstanceFlags FlagsToAdd = ELevelInstanceFlags::HasPropertyOverrides;
						if (LevelInstanceSubsystem->HasEditableLevelInstancePropertyOverrides(LevelInstanceActorPropertyOverrides))
						{
							EnumAddFlags(FlagsToAdd, ELevelInstanceFlags::HasEditablePropertyOverrides);
						}
						FAddActorLevelInstanceFlags AddFlags(Actor, FlagsToAdd);
					}
				}
			}
		}
	}
}

void ULevelStreamingLevelInstance::InitializeActors(const TArray<AActor*>& InActors)
{
	if (IsEditorWorldMode())
	{
		if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
		{
			const AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
			const bool bIsHiddenEdLayer = LevelInstanceActor->bHiddenEdLayer;
			const bool bIsHiddenEdTemporary = LevelInstanceActor->IsTemporarilyHiddenInEditor();
			const FActorContainerID ContainerID = LevelInstance->GetLevelInstanceID().GetContainerID();
		
			for (AActor* Actor : InActors)
			{
				if (IsValid(Actor))
				{
					if (Actor->IsPackageExternal())
					{
						if (bResetLoadersCalled)
						{
							ResetLoaders(Actor->GetExternalPackage());
							ForEachObjectWithOuter(Actor, [](UObject* InObject)
							{
								if (InObject && InObject->IsPackageExternal())
								{
									ResetLoaders(InObject->GetExternalPackage());
								}
							}, EGetObjectsFlags::IncludeNestedObjects);
						}
					}

					Actor->SetIsHiddenEdLayer(bIsHiddenEdLayer);
					Actor->SetIsTemporarilyHiddenInEditor(bIsHiddenEdTemporary);

					// Must happen before the actors are registered with the world, which is the case for this delegate.
					FSetActorInstanceGuid SetActorInstanceGuid(Actor, ContainerID.GetActorGuid(Actor->GetActorGuid()));

					FSetActorIsInLevelInstance SetIsInLevelInstance(Actor, ELevelInstanceType::LevelInstance);
				}
			}
		}
	}
}

void ULevelStreamingLevelInstance::OnLoadedActorsAddedToLevelPostEvent(const TArray<AActor*>& InActors)
{
	if (IsEditorWorldMode())
	{
		if (GetLoadedLevel()->bAreComponentsCurrentlyRegistered)
		{
			if (ILevelInstanceInterface* LevelInstance = GetLevelInstance())
			{
				const AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
				const bool bIsInEditLevelInstanceHierarchy = LevelInstanceActor->IsInEditLevelInstanceHierarchy();

				for (AActor* Actor : InActors)
				{
					if (IsValid(Actor))
					{
						if (Actor->IsPackageExternal())
						{
							FLevelInstanceLevelStreamingUtils::MarkObjectsInPackageAsTransientAndNonTransactional(Actor->GetExternalPackage());
						}

						Actor->PushSelectionToProxies();
						Actor->PushLevelInstanceEditingStateToProxies(bIsInEditLevelInstanceHierarchy);

						if (LevelInstanceEditorInstanceActor.IsValid())
						{
							if (Actor->GetAttachParentActor() == nullptr && !Actor->IsChildActor())
							{
								Actor->AttachToActor(LevelInstanceEditorInstanceActor.Get(), FAttachmentTransformRules::KeepWorldTransform);
							}
						}
					}
				}
			}

			ApplyPropertyOverrides(InActors, true, EApplyPropertieOverrideType::PostConstruction);
		}
	}
}

void ULevelStreamingLevelInstance::ResetLevelInstanceLoaders()
{
	// @todo_ow: Ideally at some point it is no longer needed to ResetLoaders at all and Linker would not lock the package files preventing saves.
	if (bResetLoadersCalled)
	{
		return;
	}

	if (UWorld* OuterWorld = LoadedLevel ? LoadedLevel->GetTypedOuter<UWorld>() : nullptr)
	{
		UE_SCOPED_TIMER(*FString::Printf(TEXT("ULevelStreamingLevelInstance::ResetLevelInstanceLoaders(%s)"), *FPaths::GetBaseFilename(OuterWorld->GetPackage()->GetName())), LogLevelInstance, Log);
		
		// Reset loaders for objects outered to LoadedLevel before detaching the OuterWorld linker.
		// ResetLoaders might trigger async loads, which can in turn cause further loads that require the OuterWorld linker to complete.
		// This might happen, for example, when subobjects are replaced in PostLoad.
		// If GC occurs between that replacement and the ResetLoaders call, the original object will be reloaded.
		TArray<UObject*> PackagesToReset;
		ForEachObjectWithOuter(LoadedLevel, [&PackagesToReset](UObject* InObject)
		{
			if (InObject && InObject->IsPackageExternal())
			{
				PackagesToReset.Add(InObject->GetExternalPackage());
			}
		}, EGetObjectsFlags::IncludeNestedObjects);
		ResetLoaders(PackagesToReset);

		FName PackageName = OuterWorld->GetPackage()->GetLoadedPath().GetPackageFName();
		if (!ULevel::GetIsLevelPartitionedFromPackage(PackageName))
		{
			ResetLoaders(OuterWorld->GetPackage());
		}
		else if(FLinkerLoad* LinkerLoad = OuterWorld->GetPackage()->GetLinker())
		{
			// Resetting the loader will prevent any OFPA package to properly re-load since their import level will fail to resolve.
			// DetachLoader allows releasing the lock on the file handle so level package can be saved.
			LinkerLoad->DetachLoader();
		}

		bResetLoadersCalled = true;
	}
}

void ULevelStreamingLevelInstance::OnLoadedActorsRemovedFromLevelPostEvent(const TArray<AActor*>& InActors)
{
	// Detach actor or else it will keep it alive (attachement keeps a reference to the actor)
	check(LevelInstanceEditorInstanceActor.IsValid());

	for (AActor* Actor : InActors)
	{
		if (IsValid(Actor))
		{
			if (Actor->GetAttachParentActor() == LevelInstanceEditorInstanceActor.Get())
			{
				Actor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
			}
		}
	}
}

void ULevelStreamingLevelInstance::OnPreInitializeContainerInstance(UActorDescContainerInstance::FInitializeParams& InInitParams, UActorDescContainerInstance* InContainerInstance)
{
	AActor* LevelInstanceActor = Cast<AActor>(GetLevelInstance());
	UWorldPartition* OwningWorldPartition = LevelInstanceActor->GetLevel()->GetWorldPartition();
	
	// In Editor it is possible to have a non WP parent world in which case we pass in null to the SetParent method, this will ensure that in editor the Level Instance container ID won't be a IsMainContainer() and will properly handle IsMainWorldOnly actors
	InInitParams.SetParent(OwningWorldPartition ? OwningWorldPartition->GetActorDescContainerInstance() : nullptr, LevelInstanceActor->GetActorGuid());
		
	// Apply Override Container
	if (OwningWorldPartition)
	{
		if (FWorldPartitionActorDescInstance* LevelInstanceActorDescInstance = OwningWorldPartition->GetActorDescInstance(LevelInstanceActor->GetActorGuid()); LevelInstanceActorDescInstance && LevelInstanceActorDescInstance->IsChildContainerInstance())
		{
			ULevelInstanceContainerInstance* LevelInstanceContainerInstance = CastChecked<ULevelInstanceContainerInstance>(InContainerInstance);
			LevelInstanceContainerInstance->SetOverrideContainerAndAsset(LevelInstanceActorDescInstance->GetActorDesc()->GetChildContainer(), GetLevelInstance()->GetPropertyOverrideAsset());
		}
	}
}
#endif

ULevelStreamingLevelInstance* ULevelStreamingLevelInstance::LoadInstance(ILevelInstanceInterface* LevelInstance)
{
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
#if WITH_EDITOR
	if (!ULevelInstanceSubsystem::CheckForLoop(LevelInstance))
	{
		UE_LOGF(LogLevelInstance, Error, "Failed to load LevelInstance Actor '%ls' because that would cause a loop. Run Map Check for more details.", *LevelInstanceActor->GetPathName());
		return nullptr;
	}

	FPackagePath WorldAssetPath;
	if (!FPackagePath::TryFromPackageName(LevelInstance->GetWorldAssetPackage(), WorldAssetPath) || !FPackageName::DoesPackageExist(WorldAssetPath))
	{
		UE_LOGF(LogLevelInstance, Error, "Failed to load LevelInstance Actor '%ls' because it refers to an invalid package ('%ls'). Run Map Check for more details.", *LevelInstanceActor->GetPathName(), *LevelInstance->GetWorldAsset().GetLongPackageName());
		return nullptr;
	}
#endif

	bool bOutSuccess = false;

	const FString ShortPackageName = FPackageName::GetShortName(LevelInstance->GetWorldAsset().GetLongPackageName());
	// Build a unique and deterministic LevelInstance level instance name by using LevelInstanceID. 
	// Distinguish game from editor since we don't want to duplicate for PIE already loaded editor instances (not yet supported).
	const FString Suffix = FString::Printf(TEXT("%s_LevelInstance_%016llx_%d"), *ShortPackageName, LevelInstance->GetLevelInstanceID().GetHash(), LevelInstanceActor->GetWorld()->IsGameWorld() ? 1 : 0);
	
	UWorld* World = LevelInstanceActor->GetWorld();

	FLoadLevelInstanceParams Params(World, LevelInstance->GetWorldAssetPackage(), LevelInstanceActor->GetActorTransform());
	Params.OptionalLevelNameOverride = &Suffix;
	Params.OptionalLevelStreamingClass = LevelInstance->GetLevelStreamingClass();
	Params.bLoadAsTempPackage = true;
#if WITH_EDITOR
	Params.EditorPathOwner = LevelInstanceActor;
#endif
	
	if (World->IsGameWorld())
	{
		Params.bInitiallyVisible = LevelInstance->IsInitiallyVisible();
		Params.bAllowReuseExitingLevelStreaming = true;
	}

	const FString LevelInstancePackageName = ULevelStreamingDynamic::GetLevelInstancePackageName(Params);

	ULevelStreamingLevelInstance* LevelStreaming = Cast<ULevelStreamingLevelInstance>(ULevelStreamingDynamic::LoadLevelInstance(Params, bOutSuccess));
	if (bOutSuccess)
	{
		LevelStreaming->LevelInstanceID = LevelInstance->GetLevelInstanceID();
		LevelStreaming->LevelColor = FLinearColor::MakeRandomSeededColor(GetTypeHash(LevelInstance->GetLevelInstanceID()));

#if WITH_EDITOR
		if (!World->IsGameWorld())
		{
			GEngine->BlockTillLevelStreamingCompleted(LevelInstanceActor->GetWorld());
		}
#endif
		return LevelStreaming;
	}

	return nullptr;
}

#if WITH_EDITOR
void ULevelStreamingLevelInstance::OnCurrentStateChanged(ELevelStreamingState InPrevState, ELevelStreamingState InNewState)
{
	Super::OnCurrentStateChanged(InPrevState, InNewState);
	if (InNewState == ELevelStreamingState::LoadedVisible && IsEditorWorldMode())
	{
		ILevelInstanceInterface* LevelInstance = GetLevelInstance();

		ULevel* Level = GetLoadedLevel();
		check(GetLevelStreamingState() == ELevelStreamingState::LoadedVisible);

		// Flag all Level Objects as non RF_Transactional & RF_Transient so that they can't be added to the Transaction Buffer and will be allowed to be unloaded / reloaded without clearing the Transaction Buffer
		FLevelInstanceLevelStreamingUtils::MarkObjectsInPackageAsTransientAndNonTransactional(Level->GetPackage());

		ForEachObjectWithOuter(Level, [](UObject* InObject)
		{
			if (InObject && InObject->IsPackageExternal())
			{
				FLevelInstanceLevelStreamingUtils::MarkObjectsInPackageAsTransientAndNonTransactional(InObject->GetPackage());
			}
		}, EGetObjectsFlags::IncludeNestedObjects);

		// Initialize Deferred (Async Actors)s
		check(Level->bAlreadyMovedActors);
		ApplyPropertyOverrides(Level->Actors, true, EApplyPropertieOverrideType::PostConstruction);

		// Currently only used for Attachement which doesn't do anything before the ALevelInstanceEditorInstanceActor::Create call that follows
		Level->OnLoadedActorAddedToLevelPostEvent.AddUObject(this, &ULevelStreamingLevelInstance::OnLoadedActorsAddedToLevelPostEvent);
		Level->OnLoadedActorRemovedFromLevelPreEvent.AddUObject(this, &ULevelStreamingLevelInstance::OnLoadedActorsRemovedFromLevelPostEvent);

		// Create special actor that will handle selection and transform
		LevelInstanceEditorInstanceActor = ALevelInstanceEditorInstanceActor::Create(LevelInstance, Level);

		// Push editing state to child actors
		AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
		LevelInstanceActor->PushLevelInstanceEditingStateToProxies(LevelInstanceActor->IsInEditLevelInstanceHierarchy());
	}
}
#endif

void ULevelStreamingLevelInstance::UnloadInstance(ULevelStreamingLevelInstance* LevelStreaming)
{
#if WITH_EDITOR
	if (LevelStreaming->IsEditorWorldMode())
	{
		ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel();
		LoadedLevel->OnLoadedActorAddedToLevelPreEvent.RemoveAll(LevelStreaming);
		LoadedLevel->OnLoadedActorAddedToLevelPostEvent.RemoveAll(LevelStreaming);
		LoadedLevel->OnLoadedActorRemovedFromLevelPreEvent.RemoveAll(LevelStreaming);
		LevelStreaming->LevelInstanceEditorInstanceActor.Reset();

		if (UWorldPartition* OuterWorldPartition = LoadedLevel->GetWorldPartition())
		{
			OuterWorldPartition->OnActorReplacedEvent.RemoveAll(LevelStreaming);
		}

		// Check if we need to flush the Trans buffer...
		UWorld* OuterWorld = LoadedLevel->GetTypedOuter<UWorld>();
		bool bResetTrans = false;
		ForEachObjectWithOuterBreakable(OuterWorld, [&bResetTrans](UObject* Obj)
		{
			if(Obj->HasAnyFlags(RF_Transactional))
			{
				bResetTrans = true;
				UE_LOGF(LogLevelInstance, Warning, "Found RF_Transactional object '%ls' while unloading Level Instance.", *Obj->GetPathName());
				return false;
			}
			return true;
		}, EGetObjectsFlags::IncludeNestedObjects);

		LevelStreaming->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>()->RemoveLevelsFromWorld({ LevelStreaming->GetLoadedLevel() }, bResetTrans);
	}
	else
#endif
	if (LevelStreaming->GetWorld()->IsGameWorld())
	{
		LevelStreaming->SetShouldBeLoaded(false);
		LevelStreaming->SetShouldBeVisible(false);
		LevelStreaming->SetIsRequestingUnloadAndRemoval(true);
	}

}

void ULevelStreamingLevelInstance::OnLevelLoadedChanged(ULevel* InLevel)
{
	Super::OnLevelLoadedChanged(InLevel);

	if (ULevel* NewLoadedLevel = GetLoadedLevel())
	{
#if WITH_EDITOR
		if (IsEditorWorldMode())
		{	
			InitializeActors(InLevel->Actors);
			ApplyPropertyOverrides(InLevel->Actors, false, EApplyPropertieOverrideType::PreConstruction);
			NewLoadedLevel->OnLoadedActorAddedToLevelPreEvent.AddUObject(this, &ULevelStreamingLevelInstance::OnLoadedActorsAddedToLevelPreEvent);
		}
#endif
		check(InLevel == NewLoadedLevel);
		if (!NewLoadedLevel->bAlreadyMovedActors)
		{
			AWorldSettings* WorldSettings = NewLoadedLevel->GetWorldSettings();
			check(WorldSettings);
			LevelTransform = FTransform(WorldSettings->LevelInstancePivotOffset) * LevelTransform;
		}

		if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
		{
			LevelInstanceSubsystem->RegisterLoadedLevelStreamingLevelInstance(this);

#if WITH_EDITOR
			if (UWorldPartition* OuterWorldPartition = NewLoadedLevel->GetWorldPartition())
			{
				check(!OuterWorldPartition->IsInitialized());
				
				// In Non-Editor worlds we want the container of a Level Instance to be considered the main container, it will do its own generate streaming if it is a World Partition
				// so we don't need to be called on Pre init
				if (IsEditorWorldMode())
				{
					OuterWorldPartition->OnActorDescContainerInstancePreInitialize.BindUObject(this, &ULevelStreamingLevelInstance::OnPreInitializeContainerInstance);
					OuterWorldPartition->SetContainerInstanceClass(ULevelInstanceContainerInstance::StaticClass());
					OuterWorldPartition->OnActorReplacedEvent.AddUObject(this, &ULevelStreamingLevelInstance::OnActorReplacedEvent);
				}

				if (!GetWorld()->IsGameWorld())
				{
					if (UWorldPartition* OwningWorldPartition = GetWorld()->GetWorldPartition(); OwningWorldPartition && OwningWorldPartition->IsStreamingEnabled())
					{
						if (GDisableLevelInstanceEditorPartialLoading)
						{
							OuterWorldPartition->bOverrideEnableStreamingInEditor = false;
						}
						else if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetLevelInstance(LevelInstanceID))
						{
							OuterWorldPartition->bOverrideEnableStreamingInEditor = LevelInstance->SupportsPartialEditorLoading();
						}
					}
					else
					{
						// Do not enable Streaming in Editor if Level Instance is not part of a World Partition/Streaming world
						OuterWorldPartition->bOverrideEnableStreamingInEditor = false;
					}
				}
			}
#endif
		}
	}
}
