// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstancePropertyOverrideAsset)

#if WITH_EDITOR

#include "LevelInstance/LevelInstancePrivate.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceEditorPropertyOverrideLevelStreaming.h"
#include "WorldPartition/WorldPartitionPropertyOverrideSerialization.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideDesc.h"
#include "Misc/EditorPathHelper.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "FileHelpers.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "Editor.h"
#include "Algo/Reverse.h"

struct FLevelInstancePropertyOverrideUtils
{
	// Based on Construction Script Component Instance Data serialization
	static int32 GetEmptyArchiveSize()
	{
		// Cache off the length of an array that will come from SerializeTaggedProperties that had no properties saved in to it.
		auto GetSizeOfEmptyArchive = []() -> int32
			{
				const UObject* DummyObject = GetDefault<UObject>();
				TArray<uint8> Payload;
				FWorldPartitionPropertyOverrideWriter Writer(Payload);
				FPropertyOverrideReferenceTable ReferenceTable;
				FWorldPartitionPropertyOverrideArchive Archive(Writer, ReferenceTable);
				UClass* Class = DummyObject->GetClass();

				// By serializing the component with itself as its defaults we guarantee that no properties will be written out
				Class->SerializeTaggedProperties(Archive, (uint8*)DummyObject, Class, (uint8*)DummyObject);

				return Payload.Num();
			};
		static int32 SizeOfEmptyArchive = GetSizeOfEmptyArchive();
		return SizeOfEmptyArchive;
	}
};

void ULevelInstancePropertyOverrideAsset::PruneStaleActorOverrides(const FActorContainerPath& InContainerPath, const TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>& InActorDescs)
{
	// Remove actor overrides from the container when the GUID is no longer found in the actor descs
	if (FContainerPropertyOverride* ContainerOverride = PropertyOverridesPerContainer.Find(InContainerPath))
	{
		for (TMap<FGuid, FActorPropertyOverride>::TIterator It = ContainerOverride->ActorOverrides.CreateIterator(); It; ++It)
		{
			if (!InActorDescs.Contains(It.Key()))
			{
				It.RemoveCurrent();
			}
		}
		if (ContainerOverride->ActorOverrides.IsEmpty())
		{
			PropertyOverridesPerContainer.Remove(InContainerPath);
		}
	}
}

bool ULevelInstancePropertyOverrideAsset::SerializeActorPropertyOverrides(ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreaming, AActor* InActor, bool bForReset, FActorPropertyOverride& OutActorPropertyOverrides, const TMap<UObject*, UObject*>* InInstanceToArchetypeMap, const TMap<UObject*, UObject*>* InArchetypeToInstanceMap)
{
	AActor* ActorArchetype = Cast<AActor>(InLevelStreaming->GetArchetypeForObject(InActor));
	check(ActorArchetype->GetTypedOuter<ULevel>() == InLevelStreaming->GetArchetypeLevel());

	// Gather sub objects
	TArray<UObject*> Objects;
	GetObjectsWithOuter(InActor, Objects, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);

	Objects.Add(InActor);

	// Reset table
	OutActorPropertyOverrides.ReferenceTable = FPropertyOverrideReferenceTable();

	// When saving (not resetting), temporarily replace the instance actor's internal
	// references with their archetype counterparts so that FObjectProperty::Identical sees
	// matching pointers on both sides for unmodified refs (UE-356866).
	const bool bReplaceInternalReferences = !bForReset
		&& InInstanceToArchetypeMap && InInstanceToArchetypeMap->Num() > 0
		&& InArchetypeToInstanceMap && InArchetypeToInstanceMap->Num() > 0;

	FString ArchetypeSourceWorldPath, ArchetypeRemappedWorldPath;
	if (bReplaceInternalReferences)
	{
		FArchiveReplaceObjectRef<UObject> ReplaceAr(InActor, *InInstanceToArchetypeMap,
			EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);

		InLevelStreaming->ArchetypeWorld->GetSoftObjectPathMapping(ArchetypeSourceWorldPath, ArchetypeRemappedWorldPath);
	}
	ON_SCOPE_EXIT
	{
		if (bReplaceInternalReferences)
		{
			// Restore paths
			FArchiveReplaceObjectRef<UObject> RestoreAr(InActor, *InArchetypeToInstanceMap,
				EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
		}
	};

	TArray<uint8> Payload;
	// Serialize SubObjects
	const FString ActorName = InActor->GetName();
	for (UObject* Object : Objects)
	{
		Payload.Reset();
		FWorldPartitionPropertyOverrideWriter Writer(Payload);
		FWorldPartitionPropertyOverrideArchive Archive(Writer, OutActorPropertyOverrides.ReferenceTable, &InLevelStreaming->ObjectReferences);
		if (!ArchetypeRemappedWorldPath.IsEmpty())
		{
			Archive.SetInstancingContext(ArchetypeRemappedWorldPath, ArchetypeSourceWorldPath);
		}
		UObject* Archetype = InLevelStreaming->GetArchetypeForObject(Object);
		if (Archetype->GetTypedOuter<ULevel>() == InLevelStreaming->GetArchetypeLevel())
		{
			uint8* ToSerialize = bForReset ? (uint8*)Archetype : (uint8*)Object;
			uint8* Default = bForReset ? (uint8*)Object : (uint8*)Archetype;
			Object->GetClass()->SerializeTaggedProperties(Archive, ToSerialize, Object->GetClass(), Default);

			if (Payload.Num() != FLevelInstancePropertyOverrideUtils::GetEmptyArchiveSize())
			{
				const FString ObjectSubPathString = FSoftObjectPath(Object).GetSubPathString();
				const int32 IndexOfActorName = ObjectSubPathString.Find(ActorName);
				const FString SubObjectPathString = ObjectSubPathString.Mid(IndexOfActorName + ActorName.Len() + 1);

				FSubObjectPropertyOverride& SubObjectOverride = OutActorPropertyOverrides.SubObjectOverrides.Add(SubObjectPathString);
				SubObjectOverride.SerializedTaggedProperties = MoveTemp(Payload);
			}
		}
		else
		{
			UE_LOGF(LogLevelInstance, Warning, "Failed to find Property Override Archetype for: %ls", *Object->GetPathName());
		}
	}

	if (!OutActorPropertyOverrides.SubObjectOverrides.IsEmpty())
	{
		// Cache actor so that we can serialize its ActorDesc
		OutActorPropertyOverrides.Actor = InActor;
		return true;
	}

	return false;
}

TSoftObjectPtr<ULevelInstancePropertyOverrideAsset> ULevelInstancePropertyOverrideAsset::GetSourceAssetPtr() const
{
	// If we are in an instanced world return the source asset pointer
	if (UWorld* OuterWorld = GetTypedOuter<UWorld>())
	{
		FString SourceWorldPath;
		FString RemappedWorldPath;
		if (OuterWorld->GetSoftObjectPathMapping(SourceWorldPath, RemappedWorldPath))
		{
			FSoftObjectPath SoftObjectPath(this);
			SoftObjectPath.SetPath(FTopLevelAssetPath(SourceWorldPath), SoftObjectPath.GetSubPathString());
			return TSoftObjectPtr<ULevelInstancePropertyOverrideAsset>(SoftObjectPath);
		}
	}

	return TSoftObjectPtr<ULevelInstancePropertyOverrideAsset>(const_cast<ULevelInstancePropertyOverrideAsset*>(this));
}

FActorContainerPath ULevelInstancePropertyOverrideAsset::GetContainerPropertyOverridePath(ILevelInstanceInterface* InParent, ILevelInstanceInterface* InChild)
{
	if (InParent == InChild)
	{
		return FActorContainerPath();
	}

	AActor* ChildActor = CastChecked<AActor>(InChild);
	ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(ChildActor->GetWorld());
	check(LevelInstanceSubsystem);

	FActorContainerPath ContainerPath;
	LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(ChildActor, [InParent, &ContainerPath](const ILevelInstanceInterface* InLevelInstance)
	{
		// Ignore top level parent as the path needs to be relative to it
		if(InParent == InLevelInstance)
		{
			return false;
		}

		const AActor* LevelInstanceActor = CastChecked<AActor>(InLevelInstance);
		ContainerPath.ContainerGuids.Add(LevelInstanceActor->GetActorGuid());
		return true;
	});

	// Reverse so we get top parent guid first
	Algo::Reverse(ContainerPath.ContainerGuids);
		
	return ContainerPath;
}

void ULevelInstancePropertyOverrideAsset::ResetPropertyOverridesForActor(ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreaming, AActor* InActor)
{
	check(InActor->GetExternalPackage());
	check(InActor->GetLevel()->bAlreadyMovedActors);
	
	auto ApplyTransform = [](AActor* InActor, const FTransform& InTransform, bool bDoPostEditMove)
	{
		FLevelUtils::FApplyLevelTransformParams TransformParams(InActor->GetLevel(), InTransform);
		TransformParams.Actor = InActor;
		TransformParams.bDoPostEditMove = bDoPostEditMove;
		TransformParams.bSetRelativeTransformDirectly = true;
		FLevelUtils::ApplyLevelTransform(TransformParams);
	};

	const bool bPackageDirty = InActor->GetExternalPackage()->IsDirty();
	// In case we are in a transaction make sure to modify Actor/Root before removing transform
	InActor->Modify(false);
	if (USceneComponent* RootComponent = InActor->GetRootComponent())
	{
		RootComponent->Modify(false);
	}

	const FTransform InverseTransform = InLevelStreaming->LevelTransform.Inverse();
	ApplyTransform(InActor, InverseTransform, false);
	ApplyTransform(Cast<AActor>(InLevelStreaming->GetArchetypeForObject(InActor)), InverseTransform, false);
		
	bool bResetProperties = false;

	// Serialize Reset
	FActorPropertyOverride ActorOverride;
	if (SerializeActorPropertyOverrides(InLevelStreaming, InActor, /*bForReset=*/true, ActorOverride))
	{
		// Apply Reset
		ApplyPropertyOverrides(&ActorOverride, InActor, false);
		ApplyPropertyOverrides(&ActorOverride, InActor, true);
		bResetProperties = true;
	}

	// Do PostEditMove if properties have been reset
	ApplyTransform(InActor, InLevelStreaming->LevelTransform, bResetProperties);
	ApplyTransform(Cast<AActor>(InLevelStreaming->GetArchetypeForObject(InActor)), InLevelStreaming->LevelTransform, bResetProperties);

	if (bResetProperties)
	{
		InActor->PostEditChange();

		// If Package was dirty, resetting actor properties will mark it as not dirty anymore, and if it wasn't it will dirty it
		InActor->GetExternalPackage()->SetDirtyFlag(!bPackageDirty);

		// This will refresh selection and gizmo
		GEditor->NoteSelectionChange();
	}
}

void ULevelInstancePropertyOverrideAsset::SerializePropertyOverrides(ILevelInstanceInterface* InLevelInstanceOverrideOwner, ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreaming)
{
	check(InLevelInstanceOverrideOwner);
		
	FActorContainerPath ContainerPath = GetContainerPropertyOverridePath(InLevelInstanceOverrideOwner, InLevelStreaming->GetLevelInstance());
	
	if (ULevel* LoadedLevel = InLevelStreaming->GetLoadedLevel())
	{
		bool bFirstActor = true;
		FString ContainerString;

		auto ApplyTransform = [](AActor* InActor, const FTransform& InTransform)
		{
			FLevelUtils::FApplyLevelTransformParams TransformParams(InActor->GetLevel(), InTransform);
			TransformParams.Actor = InActor;
			TransformParams.bDoPostEditMove = false;
			TransformParams.bSetRelativeTransformDirectly = true;
			FLevelUtils::ApplyLevelTransform(TransformParams);
		};
		
		PropertyOverridesPerContainer.Remove(ContainerPath);

		const FTransform InverseTransform = InLevelStreaming->LevelTransform.Inverse();

		// Build a full edit namespace -> source namespace mapping, so that different pointers representing the same object in the edited override level
		// and in the archetype level are not interpreted as being different in the diff-serialization that will occur inside SerializeActorPropertyOverrides
		// SerializeActorPropertyOverrides will temporarily switch these pointers.
		TMap<UObject*, UObject*> InstanceToArchetypeMap;
		TMap<UObject*, UObject*> ArchetypeToInstanceMap;
		ULevel* ArchetypeLevel = InLevelStreaming->GetArchetypeLevel();
		for (AActor* Actor : LoadedLevel->Actors)
		{
			if (IsValid(Actor))
			{
				if (UObject* ActorArchetype = InLevelStreaming->GetArchetypeForObject(Actor))
				{
					if (ActorArchetype->GetTypedOuter<ULevel>() == ArchetypeLevel)
					{
						InstanceToArchetypeMap.Add(Actor, ActorArchetype);
						ArchetypeToInstanceMap.Add(ActorArchetype, Actor);

						TArray<UObject*> SubObjects;
						GetObjectsWithOuter(Actor, SubObjects, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);
						for (UObject* SubObj : SubObjects)
						{
							if (UObject* SubArchetype = InLevelStreaming->GetArchetypeForObject(SubObj))
							{
								if (SubArchetype->GetTypedOuter<ULevel>() == ArchetypeLevel)
								{
									InstanceToArchetypeMap.Add(SubObj, SubArchetype);
									ArchetypeToInstanceMap.Add(SubArchetype, SubObj);
								}
							}
						}
					}
				}
			}
		}

		for (AActor* Actor : LoadedLevel->Actors)
		{
			if (IsValid(Actor))
			{
				if (AActor* ArchetypeActor = Cast<AActor>(InLevelStreaming->GetArchetypeForObject(Actor)); ArchetypeActor && ArchetypeActor->GetLevel() == InLevelStreaming->GetArchetypeLevel())
				{
					check(Actor->GetLevel()->bAlreadyMovedActors);

					// Remove LevelStreaming Transform from Instance and Archetype
					ApplyTransform(Actor, InverseTransform);
					ApplyTransform(ArchetypeActor, InverseTransform);

					// Serialize Overrides
					FActorPropertyOverride OutActorPropertyOverride;
					if (SerializeActorPropertyOverrides(InLevelStreaming, Actor, /*bForReset=*/false, OutActorPropertyOverride, &InstanceToArchetypeMap, &ArchetypeToInstanceMap))
					{
						FContainerPropertyOverride& ContainerOverride = PropertyOverridesPerContainer.FindOrAdd(ContainerPath);
						FActorPropertyOverride& ActorOverride = ContainerOverride.ActorOverrides.Add(Actor->GetActorGuid(), MoveTemp(OutActorPropertyOverride));
					}

					// Add LevelStreaming Transform to Instance and Archetype
					ApplyTransform(Actor, InLevelStreaming->LevelTransform);
					ApplyTransform(ArchetypeActor, InLevelStreaming->LevelTransform);
				}
				else
				{
					UE_LOGF(LogLevelInstance, Warning, "Failed to find Property Override Archetype for: %ls", *Actor->GetPathName());
				}
			}
		}
	}
}

#endif
