// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstancePropertyOverridePolicy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstancePropertyOverridePolicy)

ULevelInstancePropertyOverridePolicy::ULevelInstancePropertyOverridePolicy(FVTableHelper& Helper)
	: Super(Helper)
{
}

#if WITH_EDITOR

#include "UObject/PropertyOptional.h"

bool ULevelInstancePropertyOverridePolicy::CanOverrideProperty(const FProperty* InProperty) const
{
	// If this is property is inside a container, get the owning field
	const FProperty* PropertyToCheck = InProperty;
	const bool bIsInsideContainerProperty = PropertyToCheck->GetOwner<FArrayProperty>() || PropertyToCheck->GetOwner<FSetProperty>() || PropertyToCheck->GetOwner<FMapProperty>() || PropertyToCheck->GetOwner<FOptionalProperty>();
	if (bIsInsideContainerProperty)
	{
		PropertyToCheck = CastField<FProperty>(InProperty->Owner.ToField());
	}

	// Disabled on the UPROPERTY declaration
	if (PropertyToCheck->HasMetaData(TEXT("DisableLevelInstancePropertyOverride")))
	{
		return false;
	}
		
	// Property needs to be editable and non transient
	// We also do not support instanced UObject references properties
	if (PropertyToCheck->HasAnyPropertyFlags(CPF_Transient | CPF_EditConst) ||
		PropertyToCheck->ContainsInstancedObjectProperty() ||
		!PropertyToCheck->HasAnyPropertyFlags(CPF_Edit))
	{
		return false;
	}

	return CanOverridePropertyImpl(InProperty);
}
#endif
