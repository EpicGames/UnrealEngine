// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceSettings.h"
#include "WorldPartition/WorldPartitionSettings.h"

#if WITH_EDITOR
#include "Settings/EditorExperimentalSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceSettings)

ULevelInstanceSettings::ULevelInstanceSettings()
{
}

#if WITH_EDITOR
bool ULevelInstanceSettings::IsLevelInstancePropertyOverrideSettingEnabled() const
{
	return GetDefault<UEditorExperimentalSettings>()->bEnableLevelInstancePropertyOverrides;
}

bool ULevelInstanceSettings::IsPropertyOverridePolicyDefined() const
{
	return !!PropertyOverridePolicy;
}

bool ULevelInstanceSettings::IsPropertyOverrideEnabled() const
{
	return IsLevelInstancePropertyOverrideSettingEnabled() && IsPropertyOverridePolicyDefined();
}

bool ULevelInstanceSettings::IsLevelInstanceEditCompatibleWithLandscapeEdit() const
{
	return GetDefault<UEditorExperimentalSettings>()->bEnableLevelInstanceLandscapeEdit;
}

void ULevelInstanceSettings::UpdatePropertyOverridePolicy()
{
	// Policy hasn't changed
	if (PropertyOverridePolicy && PropertyOverridePolicy->GetClass()->GetPathName() == PropertyOverridePolicyClass)
	{
		return;
	}

	// Policy set to null
	if (PropertyOverridePolicyClass.IsEmpty())
	{
		PropertyOverridePolicy = nullptr;
		UWorldPartitionSettings::Get()->SetPropertyOverridePolicy(nullptr);
	}
	else
	{
		UClass* PropertyOverrideClassPtr = LoadClass<ULevelInstancePropertyOverridePolicy>(nullptr, *PropertyOverridePolicyClass, nullptr, LOAD_NoWarn);
		if (PropertyOverrideClassPtr)
		{
			PropertyOverridePolicy = NewObject<ULevelInstancePropertyOverridePolicy>(GetTransientPackage(), PropertyOverrideClassPtr, NAME_None);
			UWorldPartitionSettings::Get()->SetPropertyOverridePolicy(PropertyOverridePolicy);
		}
		else
		{
			PropertyOverridePolicy = nullptr;
			UWorldPartitionSettings::Get()->SetPropertyOverridePolicy(nullptr);
		}
	}
}

#endif
