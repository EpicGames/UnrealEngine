// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Engine/LevelStreaming.h"
#include "LevelInstance/LevelInstanceLevelStreaming.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "Misc/StringFormatArg.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "EngineUtils.h"
#include "LevelInstancePrivate.h"
#include "LevelUtils.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "LevelInstance/LevelInstanceEditorLevelStreaming.h"
#include "LevelInstance/LevelInstanceEditorPropertyOverrideLevelStreaming.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceEditorObject.h"
#include "LevelInstance/LevelInstanceEditorPivotActor.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "WorldPartition/LevelInstance/LevelInstanceContainerInstance.h"
#include "WorldPartition/LevelInstance/LevelInstanceActorDesc.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/WorldPartitionMiniMap.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "FileHelpers.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorLevelUtils.h"
#include "Modules/ModuleManager.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "Selection.h"
#include "Engine/LevelScriptBlueprint.h"
#include "EdGraph/EdGraph.h"
#include "Misc/MessageDialog.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#else
#include "LevelInstance/LevelInstanceInterface.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceSubsystem)

#define LOCTEXT_NAMESPACE "LevelInstanceSubsystem"

DEFINE_LOG_CATEGORY(LogLevelInstance);

#if WITH_EDITOR
bool ULevelInstanceSubsystem::bPrimitiveColorHandlerRegistered = false;

int32 GLevelInstanceShouldAutoUpdatePackedBlueprintsOnCommit = 0;
static FAutoConsoleVariableRef CVarLevelInstanceShouldAutoUpdatePackedBlueprintsOnCommit(
	TEXT("levelinstance.ShouldAutoUpdatePackedBlueprintsOnCommit"),
	GLevelInstanceShouldAutoUpdatePackedBlueprintsOnCommit,
	TEXT("Enable to automatically update packed blueprints associated with the world asset when committing changes to a level instance."));
#endif

ULevelInstanceSubsystem::ULevelInstanceSubsystem()
	: UWorldSubsystem()
#if WITH_EDITOR
	, bIsCreatingLevelInstance(false)
	, bIsCommittingLevelInstance(false)
#endif
{}

ULevelInstanceSubsystem::~ULevelInstanceSubsystem()
{}

void ULevelInstanceSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	ULevelInstanceSubsystem* This = CastChecked<ULevelInstanceSubsystem>(InThis);

#if WITH_EDITORONLY_DATA
	if (This->LevelInstanceEdit)
	{
		This->LevelInstanceEdit->AddReferencedObjects(Collector);
	}
#endif
}

void ULevelInstanceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

#if WITH_EDITOR
	// Make sure Policy is initialized
	ULevelInstanceSettings::Get()->UpdatePropertyOverridePolicy();
	
	if (GEditor)
	{
		ILevelInstanceEditorModule& EditorModule = FModuleManager::LoadModuleChecked<ILevelInstanceEditorModule>("LevelInstanceEditor");
		EditorModule.OnExitEditorMode().AddUObject(this, &ULevelInstanceSubsystem::OnExitEditorMode);
		EditorModule.OnTryExitEditorMode().AddUObject(this, &ULevelInstanceSubsystem::OnTryExitEditorMode);
		
		FEditorDelegates::OnAssetsPreDelete.AddUObject(this, &ULevelInstanceSubsystem::OnAssetsPreDelete);
		FEditorDelegates::PreSaveWorldWithContext.AddUObject(this, &ULevelInstanceSubsystem::OnPreSaveWorldWithContext);
		FWorldDelegates::OnPreWorldRename.AddUObject(this, &ULevelInstanceSubsystem::OnPreWorldRename);
		FWorldDelegates::OnWorldCleanup.AddUObject(this, &ULevelInstanceSubsystem::OnWorldCleanup);
	}
#endif
}

void ULevelInstanceSubsystem::Deinitialize()
{
#if WITH_EDITOR
	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	FWorldDelegates::OnPreWorldRename.RemoveAll(this);
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);

	if (ILevelInstanceEditorModule* EditorModule = FModuleManager::GetModulePtr<ILevelInstanceEditorModule>("LevelInstanceEditor"))
	{
		EditorModule->OnExitEditorMode().RemoveAll(this);
		EditorModule->OnTryExitEditorMode().RemoveAll(this);
	}
#endif
}

bool ULevelInstanceSubsystem::DoesSupportWorldType(EWorldType::Type WorldType) const
{
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::EditorPreview || WorldType == EWorldType::Inactive;
}

ILevelInstanceInterface* ULevelInstanceSubsystem::GetLevelInstance(const FLevelInstanceID& LevelInstanceID) const
{
	if (ILevelInstanceInterface*const* LevelInstance = RegisteredLevelInstances.Find(LevelInstanceID))
	{
		return *LevelInstance;
	}

	return nullptr;
}

FLevelInstanceID::FLevelInstanceID(ULevelInstanceSubsystem* LevelInstanceSubsystem, ILevelInstanceInterface* LevelInstance)
{
	TArray<FGuid> Guids;
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(LevelInstanceActor, [&Guids](const ILevelInstanceInterface* AncestorOrSelf)
	{
		Guids.Add(AncestorOrSelf->GetLevelInstanceGuid());
		return true;
	});
	check(!Guids.IsEmpty());
	
	uint64 TmpHash = 0;
		
	if (LevelInstanceActor->IsNameStableForNetworking())
	{
		ActorName = LevelInstanceActor->GetFName();
		FString NameStr = ActorName.ToString();
		TmpHash = CityHash64((const char*)*NameStr, NameStr.Len() * sizeof(TCHAR));

		// Only include OuterWorld shortpackage name in GameWorlds. In Editor this would cause the FLevelInstanceID to change for an actor within another LevelInstance
		// based off if the parent LevelInstance was in Edit (non instanced) or not (instanced)
		if (UWorld* OuterWorld = LevelInstanceActor->GetTypedOuter<UWorld>(); OuterWorld && LevelInstanceActor->GetWorld()->IsGameWorld())
		{
			PackageShortName = UWorld::RemovePIEPrefix(FPackageName::GetShortName(OuterWorld->GetPackage()));
			TmpHash = CityHash64WithSeed((const char*)*PackageShortName, PackageShortName.Len() * sizeof(TCHAR), TmpHash);
		}
	}
	
	// Make sure to start main container id
	ContainerID = FActorContainerID();
	for (int32 GuidIndex = Guids.Num() - 1; GuidIndex >= 0; GuidIndex--)
	{
		ContainerID = FActorContainerID(ContainerID, Guids[GuidIndex]);
	}

	Hash = CityHash64WithSeed((const char*)&ContainerID, sizeof(ContainerID), TmpHash);	
}

FLevelInstanceID ULevelInstanceSubsystem::RegisterLevelInstance(ILevelInstanceInterface* LevelInstance)
{
	FLevelInstanceID LevelInstanceID(this, LevelInstance);
	check(LevelInstanceID.IsValid());
	ILevelInstanceInterface*& Value = RegisteredLevelInstances.FindOrAdd(LevelInstanceID);
	check(GIsReinstancing || Value == nullptr || Value == LevelInstance);
	Value = LevelInstance;

	return LevelInstanceID;
}

void ULevelInstanceSubsystem::UnregisterLevelInstance(ILevelInstanceInterface* LevelInstance)
{
	RegisteredLevelInstances.Remove(LevelInstance->GetLevelInstanceID());
}

void ULevelInstanceSubsystem::RequestLoadLevelInstance(ILevelInstanceInterface* LevelInstance, bool bForce /* = false */)
{
	check(LevelInstance && IsValidChecked(CastChecked<AActor>(LevelInstance)) && !CastChecked<AActor>(LevelInstance)->IsUnreachable());
	if (LevelInstance->IsWorldAssetValid())
	{
#if WITH_EDITOR
		if (!IsEditingLevelInstance(LevelInstance) && !IsEditingLevelInstancePropertyOverrides(LevelInstance))
#endif
		{
			LevelInstancesToUnload.Remove(LevelInstance->GetLevelInstanceID());

			if (IsLoading(LevelInstance))
			{
				return;
			}

			bool* bForcePtr = LevelInstancesToLoadOrUpdate.Find(LevelInstance);

			// Avoid loading if already loaded. Can happen if actor requests unload/load in same frame. Without the force it means its not necessary.
			if (IsLoaded(LevelInstance) && !bForce && (bForcePtr == nullptr || !(*bForcePtr)))
			{
				return;
			}

			if (bForcePtr != nullptr)
			{
				*bForcePtr |= bForce;
			}
			else
			{
				LevelInstancesToLoadOrUpdate.Add(LevelInstance, bForce);
			}
		}
	}
}

void ULevelInstanceSubsystem::RequestUnloadLevelInstance(ILevelInstanceInterface* LevelInstance)
{
	const FLevelInstanceID& LevelInstanceID = LevelInstance->GetLevelInstanceID();
	// Test whether level instance is loaded or is still loading
	if (LoadedLevelInstances.Contains(LevelInstanceID) || LoadingLevelInstances.Contains(LevelInstanceID))
	{
		// LevelInstancesToUnload uses FLevelInstanceID because LevelInstance* can be destroyed in later Tick and we don't need it.
		LevelInstancesToUnload.Add(LevelInstanceID);
	}
	LevelInstancesToLoadOrUpdate.Remove(LevelInstance);
}

bool ULevelInstanceSubsystem::IsLoaded(const ILevelInstanceInterface* LevelInstance) const
{
	return LevelInstance->HasValidLevelInstanceID() && LoadedLevelInstances.Contains(LevelInstance->GetLevelInstanceID());
}

bool ULevelInstanceSubsystem::IsLoading(const ILevelInstanceInterface* LevelInstance) const
{
	return LevelInstance->HasValidLevelInstanceID() && LoadingLevelInstances.Contains(LevelInstance->GetLevelInstanceID());
}

void ULevelInstanceSubsystem::OnUpdateStreamingState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelInstanceSubsystem::UpdateStreamingState);

	if (!LevelInstancesToUnload.Num() && !LevelInstancesToLoadOrUpdate.Num())
	{
		return;
	}

#if WITH_EDITOR
	// Do not update during transaction
	if (GUndo)
	{
		return;
	}
#endif

	UpdateStreamingStateInternal();

#if WITH_EDITOR
	if (!GetWorld()->IsGameWorld())
	{
		// For Editor Worlds make sure UpdateStreamingState completes all recursive loading/unloading
		while (LevelInstancesToLoadOrUpdate.Num() || LevelInstancesToUnload.Num())
		{
			UpdateStreamingStateInternal();
		}
	}
#endif
}

void ULevelInstanceSubsystem::UpdateStreamingStateInternal()
{
#if WITH_EDITOR
	FScopedSlowTask SlowTask(LevelInstancesToUnload.Num() + LevelInstancesToLoadOrUpdate.Num() * 2, LOCTEXT("UpdatingLevelInstances", "Updating Level Instances..."), !GetWorld()->IsGameWorld() && !GetWorld()->GetIsInBlockTillLevelStreamingCompleted());
	SlowTask.MakeDialogDelayed(1.0f);

	check(!LevelsToRemoveScope);
	LevelsToRemoveScope.Reset(new FLevelsToRemoveScope(this));
#endif

	if (LevelInstancesToUnload.Num())
	{
		TSet<FLevelInstanceID> LevelInstancesToUnloadCopy(MoveTemp(LevelInstancesToUnload));
		for (const FLevelInstanceID& LevelInstanceID : LevelInstancesToUnloadCopy)
		{
#if WITH_EDITOR
			SlowTask.EnterProgressFrame(1.f, LOCTEXT("UnloadingLevelInstance", "Unloading Level Instance"));
#endif
			if (LoadingLevelInstances.Contains(LevelInstanceID))
			{
				LevelInstancesToUnload.Add(LevelInstanceID);
			}
			else
			{
				UnloadLevelInstance(LevelInstanceID);
			}
		}
	}

	if (LevelInstancesToLoadOrUpdate.Num())
	{
		// Unload levels before doing any loading
		TMap<ILevelInstanceInterface*, bool> LevelInstancesToLoadOrUpdateCopy(MoveTemp(LevelInstancesToLoadOrUpdate));
		for (const TPair<ILevelInstanceInterface*, bool>& Pair : LevelInstancesToLoadOrUpdateCopy)
		{
#if WITH_EDITOR
			SlowTask.EnterProgressFrame(1.f, LOCTEXT("UnloadingLevelInstance", "Unloading Level Instance"));
#endif
			ILevelInstanceInterface* LevelInstance = Pair.Key;
			if (Pair.Value)
			{
				UnloadLevelInstance(LevelInstance->GetLevelInstanceID());
			}
		}

#if WITH_EDITOR
		LevelsToRemoveScope.Reset();
		double StartTime = FPlatformTime::Seconds();
#endif
		for (const TPair<ILevelInstanceInterface*, bool>& Pair : LevelInstancesToLoadOrUpdateCopy)
		{
#if WITH_EDITOR
			SlowTask.EnterProgressFrame(1.f, FText::Format(
				LOCTEXT("LoadingLevelInstance", "Loading Level Instance {0}"),
				FText::FromString(FAssetToolsModule::GetModule().Get().GetUserFacingObjectPath(Pair.Key->GetWorldAsset().ToSoftObjectPath()))));
#endif
			LoadLevelInstance(Pair.Key);
		}
#if WITH_EDITOR
		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogLevelInstance, Log, "Loaded %ls levels in %ls seconds", *FText::AsNumber(LevelInstancesToLoadOrUpdateCopy.Num()).ToString(), *FText::AsNumber(ElapsedTime).ToString());
#endif
	}

#if WITH_EDITOR
	LevelsToRemoveScope.Reset();
	IWorldPartitionActorLoaderInterface::RefreshLoadedState(true);
#endif
}

void ULevelInstanceSubsystem::RegisterLoadedLevelStreamingLevelInstance(ULevelStreamingLevelInstance* LevelStreaming)
{
	const FLevelInstanceID LevelInstanceID = LevelStreaming->GetLevelInstanceID();
	check(LoadingLevelInstances.Contains(LevelInstanceID));
	LoadingLevelInstances.Remove(LevelInstanceID);
	check(!LoadedLevelInstances.Contains(LevelInstanceID));
	FLevelInstance& LevelInstanceEntry = LoadedLevelInstances.Add(LevelInstanceID);
	LevelInstanceEntry.LevelStreaming = LevelStreaming;

	// LevelInstanceID might not be registered anymore in the case where the level instance 
	// was unloaded while still being load.
	if (ILevelInstanceInterface* LevelInstance = LevelStreaming->GetLevelInstance())
	{
		check(LevelInstance->GetLevelInstanceID() == LevelInstanceID);
		LevelInstance->OnLevelInstanceLoaded();
	}
	else
	{
		// Validate that the LevelInstanceID is requested to be unloaded
		check(LevelInstancesToUnload.Contains(LevelInstanceID));
	}
}

ULevel* ULevelInstanceSubsystem::GetLevelInstanceLevel(const ILevelInstanceInterface* LevelInstance) const
{
	if (LevelInstance->HasValidLevelInstanceID())
	{
#if WITH_EDITOR
		if (const FLevelInstanceEdit* CurrentEdit = GetLevelInstanceEdit(LevelInstance))
		{
			return CurrentEdit->LevelStreaming->GetLoadedLevel();
		}
		else if (const FPropertyOverrideEdit* CurrentPropertyOverrideEdit = GetLevelInstancePropertyOverrideEdit(LevelInstance))
		{
			return CurrentPropertyOverrideEdit->LevelStreaming->GetLoadedLevel();
		}
		else 
#endif // WITH_EDITOR
			if (const FLevelInstance* LevelInstanceEntry = LoadedLevelInstances.Find(LevelInstance->GetLevelInstanceID()))
		{
			return LevelInstanceEntry->LevelStreaming->GetLoadedLevel();
		}
	}

	return nullptr;
}

#if WITH_EDITOR

void ULevelInstanceSubsystem::OnAssetsPreDelete(const TArray<UObject*>& Objects)
{
	for (UObject* Object : Objects)
	{
		if (IsValid(Object))
		{
			if (UPackage* Package = Object->GetPackage())
			{
				TArray<ILevelInstanceInterface*> LevelInstances = GetLevelInstances(Package->GetLoadedPath().GetPackageName());
				for (ILevelInstanceInterface* LevelInstancePtr : LevelInstances)
				{
					UnloadLevelInstance(LevelInstancePtr->GetLevelInstanceID());
				}
			}
		}
	}
}

void ULevelInstanceSubsystem::OnPreSaveWorldWithContext(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext)
{
	if (!ObjectSaveContext.IsFromAutoSave() && !ObjectSaveContext.IsProceduralSave())
	{
		if (const UPackage* WorldPackage = InWorld->GetPackage())
		{
			ResetLoadersForWorldAssetInternal(WorldPackage->GetName());
		}
	}
}

void ULevelInstanceSubsystem::OnPreWorldRename(UWorld* InWorld, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename)
{
	const bool bTestRename = (Flags & REN_Test) != 0;
	if (!bTestRename)
	{
		if (const UPackage* WorldPackage = InWorld->GetPackage())
		{
			ResetLoadersForWorldAssetInternal(WorldPackage->GetName());
		}
	}
}

void ULevelInstanceSubsystem::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	if (InWorld == GetWorld() && !InWorld->IsGameWorld() && bCleanupResources)
	{
		// LevelInstanceSubsystem doesn't support being Deinitialized and then Initialized without doing the following cleanup code (which happens with UWorld::ReInitWorld())
		// because UWorld::CleanupWorldInternal doesn't do a clean Streaming out of StreamingLevels which is fine for regular StreamingLevels but since LevelInstance StreamingLevels are tied to Actors being registered/unregistered
		// We need to do a cleanup here to make sure a call to UWorld::ReInitWorld() can properly reinitialize/stream in those levels instances again.
		TArray<ULevelStreaming*> StreamingLevels;
		ForEachLevelStreaming([&StreamingLevels](ULevelStreaming* LevelStreaming)
		{
			if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
			{
				// Avoid GC Leak by restoring OwningWorld to its original value (stop pointing to World being cleaned up)
				LoadedLevel->OwningWorld = LoadedLevel->GetTypedOuter<UWorld>();
				// Level Streaming isn't going to be properly RemovedFromWorld so here we remove the annotation
				ULevelStreaming::RemoveLevelAnnotation(LoadedLevel);
				// Make sure Level can't be reused if world gets re-initialized through UWorld::ReInitWorld()
				// This will make sure that the Level Package and its OFPA actor packages get trashed so they can't get reused (ULevel::CleanupLevel)
				LoadedLevel->SetForceCantReuseUnloadedButStillAround(true);
			}
			StreamingLevels.Add(LevelStreaming);
			return true;
		});

		if (StreamingLevels.Num())
		{
			InWorld->RemoveStreamingLevels(StreamingLevels);
		}

		if (LevelInstanceEdit.IsValid())
		{
			ILevelInstanceInterface* LI = LevelInstanceEdit->GetLevelInstance();
			BroadcastOnLevelInstanceEditModeChanged(LI, ILevelInstanceEditorModule::ELevelInstanceEditMode::None);
		}

		LoadedLevelInstances.Empty();
		LevelInstanceEdit.Reset();
		PropertyOverrideEdit.Reset();
	}
}

void ULevelInstanceSubsystem::ForEachLevelStreaming(TFunctionRef<bool(ULevelStreaming*)> Operation) const
{
	// Make sure Levels are properly trashed when cleaning up world (can't be reused)
	for (auto& [LevelInstanceID, LoadedLevelInstance] : LoadedLevelInstances)
	{
		if (!Operation(LoadedLevelInstance.LevelStreaming))
		{
			return;
		}
	}

	if (LevelInstanceEdit)
	{
		Operation(LevelInstanceEdit->LevelStreaming);
	}

	if (PropertyOverrideEdit)
	{
		Operation(PropertyOverrideEdit->LevelStreaming);
	}
}

FString ULevelInstanceSubsystem::GetActorNameToSelectFromContext(const ILevelInstanceInterface* LevelInstance, const AActor* ContextActor, const FString& DefaultActorNameToSelect) const
{
	FString ActorNameToSelect = DefaultActorNameToSelect;
	if (ContextActor)
	{
		ActorNameToSelect = ContextActor->GetName();
		ForEachLevelInstanceAncestorsAndSelf(ContextActor, [&ActorNameToSelect, LevelInstance](const ILevelInstanceInterface* AncestorLevelInstance)
		{
			// stop when we hit the LevelInstance we are about to edit
			if (AncestorLevelInstance == LevelInstance)
			{
				return false;
			}

			ActorNameToSelect = CastChecked<AActor>(AncestorLevelInstance)->GetName();
			return true;
		});
	}

	return ActorNameToSelect;
}

void ULevelInstanceSubsystem::SelectActorFromActorName(ILevelInstanceInterface* LevelInstance, const FString& ActorName) const
{
	// Try and select something meaningful
	AActor* ActorToSelect = nullptr;
	if (!ActorName.IsEmpty())
	{
		ActorToSelect = FindObject<AActor>(LevelInstance->GetLoadedLevel(), *ActorName);
	}

	// default to LevelInstance
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	if (!ActorToSelect)
	{
		ActorToSelect = LevelInstanceActor;
	}
		
	GEditor->SelectActor(ActorToSelect, true, true);
}


void ULevelInstanceSubsystem::RegisterLoadedLevelStreamingLevelInstanceEditor(ULevelStreamingLevelInstanceEditor* LevelStreaming)
{
	if (!bIsCreatingLevelInstance)
	{
		check(!LevelInstanceEdit.IsValid());
		ILevelInstanceInterface* LevelInstance = LevelStreaming->GetLevelInstance();
		LevelInstanceEdit = MakeUnique<FLevelInstanceEdit>(LevelStreaming, LevelInstance);

		BroadcastOnLevelInstanceEditModeChanged(LevelInstance, ILevelInstanceEditorModule::ELevelInstanceEditMode::Original);
	}
}

#endif

void ULevelInstanceSubsystem::LoadLevelInstance(ILevelInstanceInterface* LevelInstance)
{
	check(LevelInstance);
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	if (IsLoaded(LevelInstance) || !IsValidChecked(LevelInstanceActor) || LevelInstanceActor->IsUnreachable() || !LevelInstance->IsWorldAssetValid())
	{
		return;
	}

	if (!ensure(LevelInstance->HasValidLevelInstanceID()))
	{
		return;
	}

	const FLevelInstanceID& LevelInstanceID = LevelInstance->GetLevelInstanceID();
	check(!LoadedLevelInstances.Contains(LevelInstanceID));
	check(!LoadingLevelInstances.Contains(LevelInstanceID));
	LoadingLevelInstances.Add(LevelInstanceID);

	if (ULevelStreamingLevelInstance* LevelStreaming = ULevelStreamingLevelInstance::LoadInstance(LevelInstance))
	{
#if WITH_EDITOR
		check(LevelInstanceActor->GetWorld()->IsGameWorld() || LoadedLevelInstances.Contains(LevelInstanceID));
#endif
		// If still considered loading but level streaming was reused and its level is loaded
		if (GetWorld()->IsGameWorld() && IsLoading(LevelInstance) && LevelStreaming->GetLoadedLevel())
		{
			// Register the loaded level instance
			RegisterLoadedLevelStreamingLevelInstance(LevelStreaming);
			check(!IsLoading(LevelInstance));
			check(IsLoaded(LevelInstance));
		}
	}
	else
	{
		LoadingLevelInstances.Remove(LevelInstanceID);
	}
}

void ULevelInstanceSubsystem::UnloadLevelInstance(const FLevelInstanceID& LevelInstanceID)
{
	if (GetWorld()->IsGameWorld())
	{
		FLevelInstance LoadedLevelInstance;
		if (LoadedLevelInstances.RemoveAndCopyValue(LevelInstanceID, LoadedLevelInstance))
		{
			ULevelStreamingLevelInstance::UnloadInstance(LoadedLevelInstance.LevelStreaming);
		}
	}
#if WITH_EDITOR
	else
	{
		// Create scope if it doesn't exist
		bool bReleaseScope = false;
		if (!LevelsToRemoveScope)
		{
			bReleaseScope = true;
			LevelsToRemoveScope.Reset(new FLevelsToRemoveScope(this));
		}

		FLevelInstance LoadedLevelInstance;
		if (LoadedLevelInstances.RemoveAndCopyValue(LevelInstanceID, LoadedLevelInstance))
		{
			if (ULevel* LoadedLevel = LoadedLevelInstance.LevelStreaming->GetLoadedLevel())
			{
				ForEachActorInLevel(LoadedLevel, [this](AActor* LevelActor)
				{
					if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(LevelActor))
					{
						// Make sure to remove from pending loads if we are unloading child can't be loaded
						LevelInstancesToLoadOrUpdate.Remove(LevelInstance);

						UnloadLevelInstance(LevelInstance->GetLevelInstanceID());
					}
					return true;
				});
			}

			ULevelStreamingLevelInstance::UnloadInstance(LoadedLevelInstance.LevelStreaming);
		}

		if (bReleaseScope)
		{
			LevelsToRemoveScope.Reset();
		}
	}
#endif
}

void ULevelInstanceSubsystem::ForEachActorInLevel(ULevel* Level, TFunctionRef<bool(AActor * LevelActor)> Operation) const
{
	for (AActor* LevelActor : Level->Actors)
	{
		if (IsValid(LevelActor))
		{
			if (!Operation(LevelActor))
			{
				return;
			}
		}
	}
}

void ULevelInstanceSubsystem::ForEachLevelInstanceAncestorsAndSelf(AActor* Actor, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
	{
		if (!Operation(LevelInstance))
		{
			return;
		}
	}

	ForEachLevelInstanceAncestors(Actor, Operation);
}

void ULevelInstanceSubsystem::ForEachLevelInstanceAncestorsAndSelf(const AActor* Actor, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const
{
	if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
	{
		if (!Operation(LevelInstance))
		{
			return;
		}
	}

	ForEachLevelInstanceAncestors(Actor, Operation);
}

ULevelStreamingLevelInstance* ULevelInstanceSubsystem::GetLevelInstanceLevelStreaming(const ILevelInstanceInterface* LevelInstance) const
{
	if (LevelInstance->HasValidLevelInstanceID())
	{
		if (const FLevelInstance* LevelInstanceEntry = LoadedLevelInstances.Find(LevelInstance->GetLevelInstanceID()))
		{
			return LevelInstanceEntry->LevelStreaming;
		}
	}

	return nullptr;
}

void ULevelInstanceSubsystem::ForEachLevelInstanceAncestors(AActor* Actor, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const
{
	ILevelInstanceInterface* ParentLevelInstance = nullptr;
	do
	{
		ParentLevelInstance = GetOwningLevelInstance(Actor->GetLevel());
		Actor = Cast<AActor>(ParentLevelInstance);

	} while (ParentLevelInstance != nullptr && Operation(ParentLevelInstance));
}

void ULevelInstanceSubsystem::ForEachLevelInstanceAncestors(const AActor* Actor, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const
{
	const ILevelInstanceInterface* ParentLevelInstance = nullptr;
	if(Actor)
	{ 
		do
		{
			ParentLevelInstance = GetOwningLevelInstance(Actor->GetLevel());
			Actor = Cast<AActor>(ParentLevelInstance);
		} while (Actor != nullptr && Operation(ParentLevelInstance));
	}
}

ILevelInstanceInterface* ULevelInstanceSubsystem::GetOwningLevelInstance(const ULevel* Level) const
{
	if (ULevelStreaming* BaseLevelStreaming = FLevelUtils::FindStreamingLevel(Level))
	{
#if WITH_EDITOR
		if (ULevelStreamingLevelInstanceEditor* LevelStreamingEditor = Cast<ULevelStreamingLevelInstanceEditor>(BaseLevelStreaming))
		{
			return LevelStreamingEditor->GetLevelInstance();
		}
		else if (ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreamingPropertyOverride = Cast<ULevelStreamingLevelInstanceEditorPropertyOverride>(BaseLevelStreaming))
		{
			return LevelStreamingPropertyOverride->GetLevelInstance();
		}
#endif
		if (ULevelStreamingLevelInstance* LevelStreaming = Cast<ULevelStreamingLevelInstance>(BaseLevelStreaming))
		{
			return LevelStreaming->GetLevelInstance();
		}
		else if (UWorldPartitionLevelStreamingDynamic* WorldPartitionLevelStreaming = Cast<UWorldPartitionLevelStreamingDynamic>(BaseLevelStreaming))
		{
			// Instanced world partition might be uninitialized here, can't resolved if it's the case.
			if (UWorld* StreamingWorld = WorldPartitionLevelStreaming->GetStreamingWorld())
			{
				return GetOwningLevelInstance(StreamingWorld->PersistentLevel);
			}
		}
	}

	return nullptr;
}

ULevel* ULevelInstanceSubsystem::GetOwningLevel(const ULevel* Level, bool bFollowChainToNonLevelInstanceOwningLevel /*= false*/)
{	
	auto GetOwningLevelInternal = [] (const ULevel* ForLevel) -> ULevel*
	{
		if (ForLevel->GetWorld() != ForLevel->GetTypedOuter<UWorld>())
		{	
			if (ULevelStreaming* LevelStreaming = ULevelStreaming::FindStreamingLevel(ForLevel))
			{
				if (ULevelStreamingLevelInstance* LevelInstanceStreaming = Cast<ULevelStreamingLevelInstance>(LevelStreaming))
				{
					if (AActor* LevelActor = Cast<AActor>(LevelInstanceStreaming->GetLevelInstance()))
					{
						return LevelActor->GetLevel();
					}
				}
				if (UWorldPartitionLevelStreamingDynamic* WPLevelStreaming = Cast<UWorldPartitionLevelStreamingDynamic>(LevelStreaming))
				{
					if (UObject* WorldPartition =  WPLevelStreaming->GetOuterWorldPartition().ResolveObject())
					{						
						return WorldPartition->GetTypedOuter<ULevel>();
					}
				}
			}
		}

		return nullptr;
	};

	// locate the non instanced level responsible for getting us in the world
	const ULevel* OwningLevel = bFollowChainToNonLevelInstanceOwningLevel ? Level : nullptr;
	const ULevel* CurrentLevel = Level;

	do
	{
    	CurrentLevel = GetOwningLevelInternal(CurrentLevel);
		if (CurrentLevel)
		{
			OwningLevel = CurrentLevel;
		}
	} while(CurrentLevel && bFollowChainToNonLevelInstanceOwningLevel);
	
	return const_cast<ULevel*>(OwningLevel);
}


void ULevelInstanceSubsystem::ForEachActorInLevelInstance(const ILevelInstanceInterface* LevelInstance, TFunctionRef<bool(AActor* LevelActor)> Operation) const
{
	if (ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance))
	{
		ForEachActorInLevel(LevelInstanceLevel, Operation);
	}
}

bool ULevelInstanceSubsystem::CanUseWorldAsset(const ILevelInstanceInterface* LevelInstance, TSoftObjectPtr<UWorld> WorldAsset, FString* OutReason)
{
	if (OutReason)
	{
		*OutReason = FString();
	}

	#if WITH_EDITOR
	// Do not validate when running convert commandlet as package might not exist yet.
	if (UWorldPartitionSubsystem::IsRunningConvertWorldPartitionCommandlet())
	{
		return true;
	}
	#endif

	if (WorldAsset.IsNull())
	{
		return true;
	}

	FString PackageName;
	if (!FPackageName::DoesPackageExist(WorldAsset.GetLongPackageName()))
	{
		if (OutReason)
		{
			*OutReason = FString::Format(TEXT("Attempting to set Level Instance to package {0} which does not exist. Ensure the level was saved before attepting to set the level instance world asset."), { WorldAsset.GetLongPackageName() });
		}
		return false;
	}

	TArray<TPair<FText, TSoftObjectPtr<UWorld>>> LoopInfo;
	const ILevelInstanceInterface* LoopStart = nullptr;

	// Check if the current LevelInstance is being set to a WorldAsset that loads the current LevelInstance or any ancestor that owns the current level's package.
	if (!CheckForLoop(LevelInstance, WorldAsset, OutReason ? &LoopInfo : nullptr, OutReason ? &LoopStart : nullptr))
	{
		if (OutReason)
		{
			if (ensure(LoopStart))
			{
				const AActor* LoopStartActor = CastChecked<AActor>(LoopStart);
				TSoftObjectPtr<UWorld> LoopStartAsset(LoopStartActor->GetLevel()->GetTypedOuter<UWorld>());
				*OutReason = FString::Format(TEXT("Setting LevelInstance to {0} would cause loop {1}:{2}\n"), { WorldAsset.GetLongPackageName(), LoopStartActor->GetName(), LoopStartAsset.GetLongPackageName() });
				for (int32 i = LoopInfo.Num() - 1; i >= 0; --i)
				{
					OutReason->Append(FString::Format(TEXT("{0} {1}\n"), 
						{ *LoopInfo[i].Key.ToString(), 
						*LoopInfo[i].Value.GetLongPackageName() }));
				}
			}
		}

		return false;
	}

	return true;
}

bool ULevelInstanceSubsystem::CheckForLoop(const ILevelInstanceInterface* LevelInstance, TSoftObjectPtr<UWorld> WorldAsset, TArray<TPair<FText, TSoftObjectPtr<UWorld>>>* LoopInfo, const ILevelInstanceInterface** OutLoopStart)
{
	bool bValid = true;

	if (LevelInstance)
	{
		if(ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem())
		{
			LevelInstanceSubsystem->ForEachLevelInstanceAncestorsAndSelf(CastChecked<AActor>(LevelInstance), [&bValid, &WorldAsset, LevelInstance, LoopInfo, OutLoopStart](const ILevelInstanceInterface* CurrentLevelInstance)
			{
				FName PackageToTest(*WorldAsset.GetLongPackageName());
				// Check to exclude NAME_None since Preview Levels are in the transient package
				// Check the level we are spawned in to detect the loop (this will handle loops caused by LevelInstances and by regular level streaming)
				const AActor* CurrentActor = CastChecked<AActor>(CurrentLevelInstance);
				if (PackageToTest != NAME_None)
				{
					if (ULevel* LevelTheActorBelongsTo = CurrentActor->GetLevel())
					{
						if (UPackage* PackageTheLevelBelongsTo = LevelTheActorBelongsTo->GetPackage())
						{
							if (PackageTheLevelBelongsTo->GetLoadedPath() == FPackagePath::FromPackageNameChecked(PackageToTest))
							{
								bValid = false;
								if (OutLoopStart)
								{
									*OutLoopStart = CurrentLevelInstance;
								}
							}
						}
					}
				}
	
				if (LoopInfo)
				{
					TSoftObjectPtr<UWorld> CurrentAsset = (CurrentLevelInstance == LevelInstance)? WorldAsset : (CurrentLevelInstance)? CurrentLevelInstance->GetWorldAsset() : nullptr;
					FText LevelInstanceName = FText::FromString(CurrentActor->GetPathName());
					FText Description = FText::Format(LOCTEXT("LevelInstanceLoopLink", "-> Actor: {0} loads"), LevelInstanceName);
					LoopInfo->Emplace(Description, CurrentAsset);
				}
	
				return bValid;
			});
		}
	}

	return bValid;
}

#if WITH_EDITOR

void ULevelInstanceSubsystem::Tick()
{
	if (GetWorld()->WorldType == EWorldType::Inactive)
	{
		return;
	}

	// For non-game world, Tick is responsible of processing LevelInstances to update/load/unload
	if (!GetWorld()->IsGameWorld())
	{
		OnUpdateStreamingState();

		// Update Editor Mode if we are the GEditor world's Subsystem
		if (GEditor && GEditor->GetEditorWorldContext().World() == GetWorld())
		{
			if (ILevelInstanceEditorModule* EditorModule = (ILevelInstanceEditorModule*)FModuleManager::Get().GetModule("LevelInstanceEditor"))
			{
				const bool bActivated = LevelInstanceEdit || PropertyOverrideEdit;
				EditorModule->UpdateEditorMode(bActivated);
			}
		}
	}
}

void ULevelInstanceSubsystem::OnExitEditorMode()
{
	if (LevelInstanceEdit || PropertyOverrideEdit)
	{
		OnExitEditorModeInternal(/*bForceExit=*/true);
	}
}

void ULevelInstanceSubsystem::OnTryExitEditorMode()
{
	if (LevelInstanceEdit || PropertyOverrideEdit)
	{
		OnExitEditorModeInternal(/*bForceExit=*/false);
	}
}

bool ULevelInstanceSubsystem::TryCommitLevelInstanceEdit(bool bForceExit)
{
	if (LevelInstanceEdit)
	{
		TGuardValue<bool> CommitScope(bIsCommittingLevelInstance, true);
		bool bDiscard = false;
		if (!PromptUserForCommit(LevelInstanceEdit.Get(), bDiscard, bForceExit))
		{
			return false;
		}

		return CommitLevelInstanceInternal(LevelInstanceEdit, bDiscard, /*bDiscardOnFailure=*/bForceExit);
	}

	return true;
}

bool ULevelInstanceSubsystem::TryCommitLevelInstancePropertyOverrideEdit(bool bForceExit)
{
	if (PropertyOverrideEdit)
	{
		TGuardValue<bool> CommitScope(bIsCommittingLevelInstance, true);
		bool bDiscard = false;
		if (!PromptUserForCommitPropertyOverrides(PropertyOverrideEdit.Get(), bDiscard, bForceExit))
		{
			return false;
		}

		return CommitLevelInstancePropertyOverridesInternal(PropertyOverrideEdit, bDiscard);
	}

	return true;
}

void ULevelInstanceSubsystem::OnExitEditorModeInternal(bool bForceExit)
{
	if (bIsCommittingLevelInstance || bIsCreatingLevelInstance)
	{
		return;
	}

	if (!TryCommitLevelInstancePropertyOverrideEdit(bForceExit))
	{
		return;
	}

	TryCommitLevelInstanceEdit(bForceExit);
}

bool ULevelInstanceSubsystem::GetLevelInstanceBounds(const ILevelInstanceInterface* LevelInstance, FBox& OutBounds) const
{
	return GetLevelInstanceBoundsInternal(LevelInstance, false, OutBounds);
}

bool ULevelInstanceSubsystem::GetLevelInstanceEditorBounds(const ILevelInstanceInterface* LevelInstance, FBox& OutBounds) const
{
	return GetLevelInstanceBoundsInternal(LevelInstance, true, OutBounds);
}

bool ULevelInstanceSubsystem::GetLevelInstanceBoundsInternal(const ILevelInstanceInterface* LevelInstance, bool bIsEditorBounds, FBox& OutBounds) const
{
	if (IsLoaded(LevelInstance))
	{
		const FLevelInstance& LevelInstanceEntry = LoadedLevelInstances.FindChecked(LevelInstance->GetLevelInstanceID());
		OutBounds = LevelInstanceEntry.LevelStreaming->GetBounds();
		return true;
	}
	
	if (LevelInstance->HasValidLevelInstanceID()) // Check ID to make sure GetLevelInstancesBounds is called on a registered LevelInstance that can retrieve its Edit.
	{
		if (const FLevelInstanceEdit* CurrentEdit = GetLevelInstanceEdit(LevelInstance))
		{
			OutBounds = CurrentEdit->LevelStreaming->GetBounds();
			return true;
		}
	}
	
	if(LevelInstance->IsWorldAssetValid())
	{
		// @todo_ow: remove this temp fix once it is again safe to call the asset registry while saving.
		// Currently it can lead to a FindObject which is illegal while saving.
		if (UE::IsSavingPackage(nullptr))
		{
			OutBounds = FBox(ForceInit);
			return true;
		}

		FString LevelPackage = LevelInstance->GetWorldAssetPackage();

		if (FBox ContainerBounds = UActorDescContainerSubsystem::GetChecked().GetContainerBounds(*LevelPackage, bIsEditorBounds); ContainerBounds.IsValid)
		{
			FTransform LevelInstancePivotOffsetTransform = FTransform(ULevel::GetLevelInstancePivotOffsetFromPackage(*LevelPackage));
			FTransform LevelTransform = LevelInstancePivotOffsetTransform * CastChecked<AActor>(LevelInstance)->GetActorTransform();
			OutBounds = ContainerBounds.TransformBy(LevelTransform);
			return true;
		}

		return GetLevelInstanceBoundsFromPackage(CastChecked<AActor>(LevelInstance)->GetActorTransform(), *LevelInstance->GetWorldAssetPackage(), OutBounds);
	}

	return false;
}

bool ULevelInstanceSubsystem::GetLevelInstanceBoundsFromPackage(const FTransform& InstanceTransform, FName LevelPackage, FBox& OutBounds)
{
	FBox LevelBounds;
	if (ULevel::GetLevelBoundsFromPackage(LevelPackage, LevelBounds))
	{
		FVector BoundsLocation;
		FVector BoundsExtent;
		LevelBounds.GetCenterAndExtents(BoundsLocation, BoundsExtent);

		//@todo_ow: This will result in a new BoundsExtent that is larger than it should. To fix this, we would need the Object Oriented BoundingBox of the actor (the BV of the actor without rotation)
		const FVector BoundsMin = BoundsLocation - BoundsExtent;
		const FVector BoundsMax = BoundsLocation + BoundsExtent;
		OutBounds = FBox(BoundsMin, BoundsMax).TransformBy(InstanceTransform);
		return true;
	}

	return false;
}

void ULevelInstanceSubsystem::ForEachLevelInstanceChild(const ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const
{
	ForEachLevelInstanceChildImpl(LevelInstance, bRecursive, Operation);
}

bool ULevelInstanceSubsystem::ForEachLevelInstanceChildImpl(const ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const
{
	bool bContinue = true;
	if (ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance))
	{
		ForEachActorInLevel(LevelInstanceLevel, [&bContinue, this, Operation,bRecursive](AActor* LevelActor)
		{
			if (const ILevelInstanceInterface* ChildLevelInstance = Cast<ILevelInstanceInterface>(LevelActor))
			{
				bContinue = Operation(ChildLevelInstance);
				
				if (bContinue && bRecursive)
				{
					bContinue = ForEachLevelInstanceChildImpl(ChildLevelInstance, bRecursive, Operation);
				}
			}
			return bContinue;
		});
	}

	return bContinue;
}

void ULevelInstanceSubsystem::ForEachLevelInstanceChild(ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const
{
	ForEachLevelInstanceChildImpl(LevelInstance, bRecursive, Operation);
}

bool ULevelInstanceSubsystem::ForEachLevelInstanceChildImpl(ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const
{
	bool bContinue = true;
	if (ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance))
	{
		ForEachActorInLevel(LevelInstanceLevel, [&bContinue, this, Operation, bRecursive](AActor* LevelActor)
		{
			if (ILevelInstanceInterface* ChildLevelInstance = Cast<ILevelInstanceInterface>(LevelActor))
			{
				bContinue = Operation(ChildLevelInstance);

				if (bContinue && bRecursive)
				{
					bContinue = ForEachLevelInstanceChildImpl(ChildLevelInstance, bRecursive, Operation);
				}
			}
			return bContinue;
		});
	}

	return bContinue;
}

bool ULevelInstanceSubsystem::HasDirtyChildrenLevelInstances(const ILevelInstanceInterface* LevelInstance) const
{
	bool bDirtyChildren = false;
	ForEachLevelInstanceChild(LevelInstance, /*bRecursive=*/true, [this, &bDirtyChildren](const ILevelInstanceInterface* ChildLevelInstance)
	{
		if (IsEditingLevelInstanceDirty(ChildLevelInstance))
		{
			bDirtyChildren = true;
			return false;
		}
		return true;
	});
	return bDirtyChildren;
}

void ULevelInstanceSubsystem::SetIsHiddenEdLayer(ILevelInstanceInterface* LevelInstance, bool bIsHiddenEdLayer)
{
	if (ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance))
	{
		ForEachActorInLevel(LevelInstanceLevel, [bIsHiddenEdLayer](AActor* LevelActor)
		{
			LevelActor->SetIsHiddenEdLayer(bIsHiddenEdLayer);
			return true;
		});
	}
}

void ULevelInstanceSubsystem::SetIsTemporarilyHiddenInEditor(ILevelInstanceInterface* LevelInstance, bool bIsHidden)
{
	if (ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance))
	{
		ForEachActorInLevel(LevelInstanceLevel, [bIsHidden](AActor* LevelActor)
		{
			LevelActor->SetIsTemporarilyHiddenInEditor(bIsHidden);
			return true;
		});
	}
}

bool ULevelInstanceSubsystem::SetCurrent(ILevelInstanceInterface* LevelInstance) const
{
	if (IsEditingLevelInstance(LevelInstance))
	{
		return GetWorld()->SetCurrentLevel(GetLevelInstanceLevel(LevelInstance));
	}

	return false;
}

bool ULevelInstanceSubsystem::IsCurrent(const ILevelInstanceInterface* LevelInstance) const
{
	if (IsEditingLevelInstance(LevelInstance))
	{
		return GetLevelInstanceLevel(LevelInstance) == GetWorld()->GetCurrentLevel();
	}

	return false;
}

bool ULevelInstanceSubsystem::MoveActorsToLevel(const TArray<AActor*>& ActorsToRemove, ULevel* DestinationLevel, TArray<AActor*>* OutActors /*= nullptr*/) const
{
	check(DestinationLevel);

	const bool bWarnAboutReferences = true;
	const bool bWarnAboutRenaming = true;
	const bool bMoveAllOrFail = true;
	if (!EditorLevelUtils::MoveActorsToLevel(ActorsToRemove, DestinationLevel, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, OutActors))
	{
		UE_LOGF(LogLevelInstance, Warning, "Failed to move actors out of Level Instance because not all actors could be moved");
		return false;
	}

	ILevelInstanceInterface* OwningInstance = GetOwningLevelInstance(DestinationLevel);
	if (!OwningInstance || !OwningInstance->IsEditing())
	{
		for (const auto& Actor : ActorsToRemove)
		{
			const bool bEditing = false;
			Actor->PushLevelInstanceEditingStateToProxies(bEditing);
		}
	}

	return true;
}

bool ULevelInstanceSubsystem::MoveActorsTo(ILevelInstanceInterface* LevelInstance, const TArray<AActor*>& ActorsToMove, TArray<AActor*>* OutActors /*= nullptr*/)
{
	check(IsEditingLevelInstance(LevelInstance));
	ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance);
	check(LevelInstanceLevel);

	return MoveActorsToLevel(ActorsToMove, LevelInstanceLevel, OutActors);
}

ULevelStreamingLevelInstanceEditor* ULevelInstanceSubsystem::CreateNewStreamingLevelForWorld(UWorld& InWorld, const EditorLevelUtils::FCreateNewStreamingLevelForWorldParams& InParams)
{
	EditorLevelUtils::FCreateNewStreamingLevelForWorldParams CreateNewStreamingLevelParamsCopy(InParams);
	check(CreateNewStreamingLevelParamsCopy.LevelStreamingClass && CreateNewStreamingLevelParamsCopy.LevelStreamingClass->IsChildOf<ULevelStreamingLevelInstanceEditor>());
		
	CreateNewStreamingLevelParamsCopy.PreSaveLevelCallback = [ActorsToMove = InParams.ActorsToMove, PreSaveLevelCallback = InParams.PreSaveLevelCallback](ULevel* InLevel)
	{
		if (InLevel->IsUsingExternalActors())
		{
			// UWorldFactory::FactoryCreateNew will modify the default brush to be in global space (see GEditor->InitBuilderBrush(NewWorld)).
			// The level is about to be saved, it doesn't have a transform and it is not yet added to the world levels.
			// Since, no logic will remove the transform on the actor, force its transform to identity here.
			if (ABrush* Brush = InLevel->GetDefaultBrush())
			{
				Brush->GetRootComponent()->SetRelativeTransform(FTransform::Identity);
			}
		}

		if (UWorldPartition* WorldPartition = InLevel->GetWorldPartition())
		{
			// Validations
			check(InLevel->IsUsingActorFolders());

			// Reset HLOD Layer (no defaults needed for Level Instances)
			WorldPartition->SetDefaultHLODLayer(nullptr);

			// Make sure new level's AWorldDataLayers contains all the necessary Data Layer Instances before moving actors
			TSet<TObjectPtr<const UDataLayerAsset>> SourceDataLayerAssets;
			if (ActorsToMove)
			{
				for (AActor* ActorToMove : *ActorsToMove)
				{
					if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(ActorToMove))
					{
						// Use the raw asset list as we don't want parent DataLayers
						for (const UDataLayerAsset* DataLayerAsset : ActorToMove->GetDataLayerAssets())
						{
							if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromAsset(DataLayerAsset))
							{
								// Validate that there's a valid Data Layer Instance for this asset in the source level and that this isn't a private Data Layer
								if (!DataLayerAsset->IsPrivate())
								{
									// @todo_ow: Add LevelInstance EDL support (For now skip all Data Layer Instances part of an EDL)
									if (!DataLayerInstance->GetRootExternalDataLayerInstance())
									{
										SourceDataLayerAssets.Add(DataLayerAsset);
									}
								}
							}
						}
					}
				}
			}

			if (!SourceDataLayerAssets.IsEmpty())
			{
				AWorldDataLayers* WorldDataLayers = InLevel->GetWorldDataLayers();
				check(WorldDataLayers);
				for (const UDataLayerAsset* SourceDataLayerAsset : SourceDataLayerAssets)
				{
					WorldDataLayers->CreateDataLayer<UDataLayerInstanceWithAsset>(SourceDataLayerAsset);
				}
			}
		}

		if (PreSaveLevelCallback)
		{
			PreSaveLevelCallback(InLevel);
		}
	};
	return Cast<ULevelStreamingLevelInstanceEditor>(EditorLevelUtils::CreateNewStreamingLevelForWorld(*GetWorld(), CreateNewStreamingLevelParamsCopy));
}

bool ULevelInstanceSubsystem::CanCreateLevelInstanceFrom(const TArray<AActor*>& ActorsToMove, FText* OutReason)
{
	if (ActorsToMove.IsEmpty())
	{
		if (OutReason != nullptr)
		{
			*OutReason = LOCTEXT("CanCreateLevelInstanceFromEmptyActorArray", "Failed to create Level Instance from actor array");
		}
		return false;
	}

	for (AActor* ActorToMove : ActorsToMove)
	{
		if (!CanMoveActorToLevel(ActorToMove, OutReason))
		{
			return false;
		}
	}

	return true;
}

ILevelInstanceInterface* ULevelInstanceSubsystem::CreateLevelInstanceFrom(const TArray<AActor*>& ActorsToMove, const FNewLevelInstanceParams& CreationParams)
{
	FText Reason;
	if (!CanCreateLevelInstanceFrom(ActorsToMove, &Reason))
	{
		UE_LOGF(LogLevelInstance, Warning, "Failed to create Level Instance : %ls", *Reason.ToString());
		return nullptr;
	}

	check(!bIsCreatingLevelInstance);
	TGuardValue<bool> CreateLevelInstanceGuard(bIsCreatingLevelInstance, true);
	ULevel* CurrentLevel = GetWorld()->GetCurrentLevel();
	
	TOptional<const UExternalDataLayerAsset*> CommonExternalDataLayerAsset;
	FBox ActorLocationBox(ForceInit);
	for (const AActor* ActorToMove : ActorsToMove)
	{
		const bool bNonColliding = true;
		const bool bIncludeChildren = true;
		FBox LocalActorLocationBox = ActorToMove->GetComponentsBoundingBox(bNonColliding, bIncludeChildren);
		// In the case where we receive an invalid bounding box, use actor's location if it has a root component
		if (!LocalActorLocationBox.IsValid && ActorToMove->GetRootComponent())
		{
			LocalActorLocationBox = FBox({ ActorToMove->GetActorLocation() });
		}
		ActorLocationBox += LocalActorLocationBox;

		if (!CommonExternalDataLayerAsset.IsSet())
		{
			CommonExternalDataLayerAsset = ActorToMove->GetExternalDataLayerAsset();
		}
		else if (CommonExternalDataLayerAsset.GetValue() != ActorToMove->GetExternalDataLayerAsset())
		{
			CommonExternalDataLayerAsset = nullptr;
		}
	}

	FVector LevelInstanceLocation;
	if (CreationParams.PivotType == ELevelInstancePivotType::Actor)
	{
		check(CreationParams.PivotActor);
		LevelInstanceLocation = CreationParams.PivotActor->GetActorLocation();
	}
	else if (CreationParams.PivotType == ELevelInstancePivotType::WorldOrigin)
	{
		LevelInstanceLocation = FVector(0.f, 0.f, 0.f);
	}
	else
	{
		LevelInstanceLocation = ActorLocationBox.GetCenter();
		if (CreationParams.PivotType == ELevelInstancePivotType::CenterMinZ)
		{
			LevelInstanceLocation.Z = ActorLocationBox.Min.Z;
		}
	}
		
	FString LevelFilename;
	if (!CreationParams.LevelPackageName.IsEmpty())
	{
		LevelFilename = FPackageName::LongPackageNameToFilename(CreationParams.LevelPackageName, FPackageName::GetMapPackageExtension());
	}

	// Tell current level edit to stop listening because management of packages to save is done here (operation is atomic and can't be undone)
	if (LevelInstanceEdit)
	{
		LevelInstanceEdit->EditorObject->bCreatingChildLevelInstance = true;
	}
	ON_SCOPE_EXIT
	{
		if (LevelInstanceEdit)
		{
			LevelInstanceEdit->EditorObject->bCreatingChildLevelInstance = false;
		}
	};
	
	TSet<FName> DirtyPackages;

	// Capture Packages before Moving actors as they can get GCed in the process
	for (AActor* ActorToMove : ActorsToMove)
	{
		// Don't force saving of unsaved/temp packages onto the user.
		if (!FPackageName::IsTempPackage(ActorToMove->GetPackage()->GetName()))
		{
			DirtyPackages.Add(ActorToMove->GetPackage()->GetFName());
		}
	}

	// Predetermine New Level Instance Actor Guid here and ContainerInstance so that we can feed them to the LevelStreaming object
	FGuid LevelInstanceActorGuid = FGuid::NewGuid();
	UWorldPartition* CurrentWorldPartition = CurrentLevel->GetWorldPartition();
	UActorDescContainerInstance* ParentContainerInstance = CurrentWorldPartition ? CurrentWorldPartition->GetActorDescContainerInstance() : nullptr;

	ULevelStreamingLevelInstanceEditor* LevelStreaming = nullptr;
	{
		EditorLevelUtils::FCreateNewStreamingLevelForWorldParams CreateNewStreamingLevelParams(ULevelStreamingLevelInstanceEditor::StaticClass(), LevelFilename);
		CreateNewStreamingLevelParams.bUseExternalActors = CreationParams.UseExternalActors();
		CreateNewStreamingLevelParams.bUseSaveAs = true;
		CreateNewStreamingLevelParams.bCreateWorldPartition = GetWorld()->IsPartitionedWorld();
		CreateNewStreamingLevelParams.bEnableWorldPartitionStreaming = CreationParams.bEnableStreaming;
		CreateNewStreamingLevelParams.ActorsToMove = &ActorsToMove;
		CreateNewStreamingLevelParams.TemplateWorld = CreationParams.TemplateWorld;
		CreateNewStreamingLevelParams.LevelStreamingCreatedCallback = [LevelInstanceActorGuid, ParentContainerInstance](ULevelStreaming* InLevelStreaming)
		{
			ULevelStreamingLevelInstanceEditor* LevelInstanceLevelStreaming = CastChecked<ULevelStreamingLevelInstanceEditor>(InLevelStreaming);
			LevelInstanceLevelStreaming->ParentContainerInstance = ParentContainerInstance;
			LevelInstanceLevelStreaming->ParentContainerGuid = LevelInstanceActorGuid;
		};

		LevelStreaming = CreateNewStreamingLevelForWorld(*GetWorld(), CreateNewStreamingLevelParams);
	}

	if (!LevelStreaming)
	{
		UE_LOGF(LogLevelInstance, Warning, "Failed to create new Level");
		return nullptr;
	}
		
	ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel();
	check(LoadedLevel);
		
	for (AActor* Actor : LoadedLevel->Actors)
	{
		if (Actor)
		{
			// @todo_ow : Decide if we want to re-create the same hierarchy as the source level.
			Actor->SetFolderPath_Recursively(NAME_None);

			// @todo_ow: Add LevelInstance EDL support (For now, remove all Data Layers part of an EDL)
			for (const UDataLayerInstance* DataLayerInstance : Actor->GetDataLayerInstances())
			{
				if (DataLayerInstance->GetRootExternalDataLayerInstance())
				{
					DataLayerInstance->RemoveActor(Actor);
				}
			}
		}
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.OverrideActorGuid = LevelInstanceActorGuid;
	SpawnParams.OverrideLevel = CurrentLevel;
	AActor* NewLevelInstanceActor = nullptr;
	TSoftObjectPtr<UWorld> WorldPtr(LoadedLevel->GetTypedOuter<UWorld>());
			
	// Make sure newly created level asset gets scanned
	ULevel::ScanLevelAssets(LoadedLevel->GetPackage()->GetName());
	
	// Use CreationParams class if provided
	UClass* ActorClass = CreationParams.LevelInstanceClass;
	if (!ActorClass)
	{
		ActorClass = CreationParams.Type == ELevelInstanceCreationType::LevelInstance ? ALevelInstance::StaticClass() : APackedLevelActor::StaticClass();
	}

	check(ActorClass->ImplementsInterface(ULevelInstanceInterface::StaticClass()));

	const UExternalDataLayerAsset* ExternalDataLayerAsset = CommonExternalDataLayerAsset.Get(nullptr);
	UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(GetWorld());
	UExternalDataLayerInstance* ExternalDataLayerInstance = ExternalDataLayerManager ? ExternalDataLayerManager->GetExternalDataLayerInstance(ExternalDataLayerAsset) : nullptr;
	// @todo_ow: We temporarily allow adding the ExternalDataLayerInstance to the ActorEditorContext or else it wouldn't allow it to be added since the current level is not the persistent level.
	if (ExternalDataLayerInstance)
	{
		ExternalDataLayerInstance->bSkipCheckReadOnlyForSubLevels = true;
	}
	FScopedOverrideSpawningLevelMountPointObject EDLScope(ExternalDataLayerAsset);

	if (!ActorClass->IsChildOf<APackedLevelActor>())
	{
		NewLevelInstanceActor = GetWorld()->SpawnActor<AActor>(ActorClass, SpawnParams);
	}
	else
	{
		FString PackageDir = FPaths::GetPath(WorldPtr.GetLongPackageName());
		FString AssetName = FPackedLevelActorBuilder::GetPackedBPPrefix() + WorldPtr.GetAssetName();
		FString BPAssetPath = FString::Format(TEXT("{0}/{1}.{1}"), { PackageDir , AssetName });
		const bool bCompile = true;

		UBlueprint* NewBP = nullptr;
		if (CreationParams.LevelPackageName.IsEmpty())
		{
			NewBP = FPackedLevelActorBuilder::CreatePackedLevelActorBlueprintWithDialog(TSoftObjectPtr<UBlueprint>(FSoftObjectPath(BPAssetPath)), WorldPtr, bCompile);
		}
		else
		{
			NewBP = FPackedLevelActorBuilder::CreatePackedLevelActorBlueprint(TSoftObjectPtr<UBlueprint>(FSoftObjectPath(BPAssetPath)), WorldPtr, bCompile);
		}
				
		if (NewBP)
		{
			NewLevelInstanceActor = GetWorld()->SpawnActor<APackedLevelActor>(NewBP->GeneratedClass, SpawnParams);
		}

		if (!NewLevelInstanceActor)
		{
			UE_LOGF(LogLevelInstance, Warning, "Failed to create packed level blueprint. Creating non blueprint packed level instance instead.");
			NewLevelInstanceActor = GetWorld()->SpawnActor<APackedLevelActor>(APackedLevelActor::StaticClass(), SpawnParams);
		}
	}
	if (ExternalDataLayerInstance)
	{
		ExternalDataLayerInstance->bSkipCheckReadOnlyForSubLevels = false;
	}
	
	check(NewLevelInstanceActor);
	check(NewLevelInstanceActor->GetActorGuid() == LevelInstanceActorGuid);

	ILevelInstanceInterface* NewLevelInstance = CastChecked<ILevelInstanceInterface>(NewLevelInstanceActor);
	NewLevelInstance->SetWorldAsset(WorldPtr);
	NewLevelInstanceActor->SetActorLocation(LevelInstanceLocation);
	NewLevelInstance->AdjustPivotOnCreation();
	NewLevelInstanceActor->SetActorLabel(WorldPtr.GetAssetName());

	// Actors were moved and kept their World positions so when saving we want their positions to actually be relative to the LevelInstance Actor
	// so we set the LevelTransform and we mark the level as having moved its actors. 
	// On Level save FLevelUtils::RemoveEditorTransform will fixup actor transforms to make them relative to the LevelTransform.
	LevelStreaming->LevelTransform = NewLevelInstanceActor->GetActorTransform();
	LoadedLevel->bAlreadyMovedActors = true;

	GEditor->SelectNone(false, true);
	GEditor->SelectActor(NewLevelInstanceActor, true, true);

	NewLevelInstance->OnEdit();

	// Notify parents of edit
	TArray<FLevelInstanceID> AncestorIDs;
	ForEachLevelInstanceAncestors(NewLevelInstanceActor, [&AncestorIDs](ILevelInstanceInterface* InAncestor)
	{
		AncestorIDs.Add(InAncestor->GetLevelInstanceID());
		return true;
	});

	for (const FLevelInstanceID& AncestorID : AncestorIDs)
	{
		OnEditChild(AncestorID);
	}
	
	// New level instance
	const FLevelInstanceID NewLevelInstanceID = NewLevelInstance->GetLevelInstanceID();

	struct FStackLevelInstanceEdit : public FGCObject
	{
		TUniquePtr<FLevelInstanceEdit> LevelInstanceEdit;

		virtual ~FStackLevelInstanceEdit() {}

		//~ FGCObject
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			if (LevelInstanceEdit)
			{
				LevelInstanceEdit->AddReferencedObjects(Collector);
			}
		}
		virtual FString GetReferencerName() const override
		{
			return TEXT("FStackLevelInstanceEdit");
		}
		//~ FGCObject
	};

	FStackLevelInstanceEdit StackLevelInstanceEdit;
	LevelStreaming->LevelInstanceID = NewLevelInstanceID;
	StackLevelInstanceEdit.LevelInstanceEdit = MakeUnique<FLevelInstanceEdit>(LevelStreaming, NewLevelInstance);
	// Force mark it as changed
	StackLevelInstanceEdit.LevelInstanceEdit->MarkCommittedChanges();

	GetWorld()->SetCurrentLevel(LoadedLevel);

	// Commit will always pop the actor editor context, make sure to push one here
	UActorEditorContextSubsystem::Get()->PushContext();
	bool bCommitted = CommitLevelInstanceInternal(StackLevelInstanceEdit.LevelInstanceEdit, /*bDiscardEdits=*/false, /*bDiscardOnFailure=*/true, &DirtyPackages);
	check(bCommitted);
	check(!StackLevelInstanceEdit.LevelInstanceEdit);

	// In case Commit caused actor to be GCed (Commit can cause BP reinstancing)
	NewLevelInstanceActor = Cast<AActor>(GetLevelInstance(NewLevelInstanceID));

	// Don't force saving of unsaved/temp packages onto the user. 
	if (!FPackageName::IsTempPackage(NewLevelInstanceActor->GetPackage()->GetName()))
	{
		FEditorFileUtils::PromptForCheckoutAndSave({ NewLevelInstanceActor->GetPackage() }, /*bCheckDirty*/false, /*bPromptToSave*/false);
	}

	// After commit, CurrentLevel goes back to world's PersistentLevel. Set it back to the current editing level instance (if any).
	if (ILevelInstanceInterface* EditingLevelInstance = GetEditingLevelInstance())
	{
		SetCurrent(EditingLevelInstance);
	}

	return GetLevelInstance(NewLevelInstanceID);
}

bool ULevelInstanceSubsystem::CanBreakLevelInstance(const ILevelInstanceInterface* LevelInstance) const
{
	return !IsEditingLevelInstance(LevelInstance) && !IsEditingLevelInstancePropertyOverrides(LevelInstance) && !HasParentPropertyOverridesEdit(LevelInstance) && !LevelInstanceHasLevelScriptBlueprint(LevelInstance);
}

bool ULevelInstanceSubsystem::BreakLevelInstance(ILevelInstanceInterface* LevelInstance, uint32 Levels /* = 1 */,
	TArray<AActor*>* OutMovedActors /* = nullptr */, ELevelInstanceBreakFlags Flags /* = None */)
{
	if (!CanBreakLevelInstance(LevelInstance))
	{
		return false;
	}

	const double StartTime = FPlatformTime::Seconds();

	const uint32 bAvoidRelabelOnPasteSelected = GetMutableDefault<ULevelEditorMiscSettings>()->bAvoidRelabelOnPasteSelected;
	ON_SCOPE_EXIT { GetMutableDefault<ULevelEditorMiscSettings>()->bAvoidRelabelOnPasteSelected = bAvoidRelabelOnPasteSelected; };
	GetMutableDefault<ULevelEditorMiscSettings>()->bAvoidRelabelOnPasteSelected = 1;

	ULevel* OldCurrentLevel = GetWorld()->GetCurrentLevel();
	if (AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance))
	{
		GetWorld()->SetCurrentLevel(LevelInstanceActor->GetLevel());
	}

	TArray<AActor*> MovedActors;
	BreakLevelInstance_Impl(LevelInstance, Levels, MovedActors, Flags);

	GetWorld()->SetCurrentLevel(OldCurrentLevel);

	USelection* ActorSelection = GEditor->GetSelectedActors();
	ActorSelection->BeginBatchSelectOperation();
	for (AActor* MovedActor : MovedActors)
	{
		GEditor->SelectActor(MovedActor, true, false);
	}
	ActorSelection->EndBatchSelectOperation(false);

	bool bStatus = MovedActors.Num() > 0;

	const double ElapsedTime = FPlatformTime::Seconds() - StartTime;
	UE_LOGF(LogLevelInstance, Log, "Break took %ls seconds (%ls actors)", *FText::AsNumber(ElapsedTime).ToString(), *FText::AsNumber(MovedActors.Num()).ToString());
	
	if (OutMovedActors)
	{
		*OutMovedActors = MoveTemp(MovedActors);
	}

	return bStatus;
}

void ULevelInstanceSubsystem::BreakLevelInstance_Impl(ILevelInstanceInterface* LevelInstance, uint32 Levels,
	TArray<AActor*>& OutMovedActors, ELevelInstanceBreakFlags Flags)
{
	if (Levels > 0)
	{
		AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
		// Can only break the top level LevelInstance
		check(LevelInstanceActor->GetLevel() == GetWorld()->GetCurrentLevel());

		// Actors in a packed level actor will not be streamed in unless they are editing. Must force this before moving.
		if (LevelInstanceActor->IsA<APackedLevelActor>())
		{
			BlockLoadLevelInstance(LevelInstance);
		}

		// need to ensure that LevelInstance has been streamed in fully
		GEngine->BlockTillLevelStreamingCompleted(LevelInstanceActor->GetWorld());

		// Cannot break a level instance which has a level script
		if (LevelInstanceHasLevelScriptBlueprint(LevelInstance))
		{
			UE_LOGF(LogLevelInstance, Warning, "Failed to completely break Level Instance because some children have Level Scripts.");

			if (LevelInstanceActor->IsA<APackedLevelActor>())
			{
				BlockUnloadLevelInstance(LevelInstance);
			}
			return;
		}

		TArray<const UDataLayerInstance*> LevelInstanceDataLayerInstances = LevelInstanceActor->GetDataLayerInstances();

		TSet<AActor*> ActorsToMove;
		TFunction<bool(AActor*)> AddActorToMove = [this, &ActorsToMove, &AddActorToMove, &LevelInstanceDataLayerInstances](AActor* Actor)
		{
			if (ActorsToMove.Contains(Actor))
			{
				return true;
			}

			// Skip some actor types
			if ((Actor != Actor->GetLevel()->GetDefaultBrush()) &&
				!Actor->IsA<AWorldSettings>() &&
				!Actor->IsMainWorldOnly())
			{
				if (CanMoveActorToLevel(Actor))
				{
					FSetActorHiddenInSceneOutliner Show(Actor, false);

					// Detach if Parent Actor can't be moved
					if (AActor* ParentActor = Actor->GetAttachParentActor())
					{
						if (!AddActorToMove(ParentActor))
						{
							Actor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
						}
					}

					// Apply the same data layer settings to the actors to move out
					for (const UDataLayerInstance* DataLayerInstance : LevelInstanceDataLayerInstances)
					{
						if (Actor->SupportsDataLayerType(DataLayerInstance->GetClass()))
						{
							if (const UDataLayerAsset* DataLayerAsset = DataLayerInstance->GetAsset())
							{
								// For DataLayerInstanceAsset we add the Asset to the actor instead of the DataLayerInstance because Actor hasn't moved yet and this
								// will fail on a UDataLayerInstanceWithAsset when comparing the Actor's outer to the DataLayerInstance outer.
								// todo_ow : Assign the actor to the DataLayerInstance in the destination level via UDataLayerInstance::AddActor to go through the DataLayerInstance process.
								FAssignActorDataLayer::AddDataLayerAsset(Actor, DataLayerAsset);
							}
							else
							{
								Actor->AddDataLayer(DataLayerInstance);
							}
						}
					}

					ActorsToMove.Add(Actor);
					return true;
				}
			}

			return false;
		};

		ForEachActorInLevelInstance(LevelInstance, [this, &ActorsToMove, &AddActorToMove](AActor* Actor)
		{
			AddActorToMove(Actor);
			return true;
		});

		ULevel* DestinationLevel = GetWorld()->GetCurrentLevel();
		check(DestinationLevel);

		const bool bWarnAboutReferences = true;
		const bool bWarnAboutRenaming = false;
		const bool bMoveAllOrFail = true;

		TArray<AActor*> ActorsMovedThisStage;
		if (!EditorLevelUtils::CopyActorsToLevel(ActorsToMove.Array(), DestinationLevel, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, &ActorsMovedThisStage))
		{
			UE_LOGF(LogLevelInstance, Warning, "Failed to break Level Instance because not all actors could be moved");
			return;
		}

		OutMovedActors.Append(ActorsMovedThisStage);

		const bool bKeepFolders = EnumHasAnyFlags(Flags, ELevelInstanceBreakFlags::KeepFolders);
		FString LevelInstanceFolder;
		if (bKeepFolders)
		{
			// Build the folder name to move the actors into
			const FName LevelInstancePath = LevelInstanceActor->GetFolderPath();
			if (!LevelInstancePath.IsNone())
			{
				LevelInstanceFolder = LevelInstancePath.ToString();
				LevelInstanceFolder += TEXT('/');
			}
			LevelInstanceFolder += LevelInstanceActor->GetActorNameOrLabel();
		}

		TArray<ILevelInstanceInterface*> ChildLevelInstances;
		for (AActor* Actor : ActorsMovedThisStage)
		{
			if (bKeepFolders)
			{
				// Update the folder path of the moved actor, combining LI's path + LI's name + actor's path
				TStringBuilder<128> NewActorPath;
				NewActorPath += LevelInstanceFolder;

				const FName OldActorPath = Actor->GetFolderPath();
				if (!OldActorPath.IsNone())
				{
					NewActorPath += TEXT('/');
					NewActorPath += Actor->GetFolderPath().ToString();
				}

				Actor->SetFolderPath(FName(NewActorPath));
			}

			// Break up any sub LevelInstances if more levels are requested
			if (Levels > 1)
			{
				if (ILevelInstanceInterface* ChildLevelInstance = Cast<ILevelInstanceInterface>(Actor))
				{
					OutMovedActors.RemoveSingleSwap(Actor);

					ChildLevelInstances.Add(ChildLevelInstance);
				}
			}
		}

		// Clear undo buffer here because operation of Breaking a level instance is not undoable.
		// - Do it before Unload and Destroy calls because those calls will try and unload the level.
		//	 The unloading of the level might cause a call to UEngine::FindAndPrintStaleReferencesToObject
		//	 which will slow down the unloading because the Trans buffer has some references to the moved actors.
		if (GEditor->Trans)
		{
			GEditor->Trans->Reset(LOCTEXT("BreakLevelInstance", "Break Level Instance"));
		}

		if (LevelInstanceActor->IsA<APackedLevelActor>())
		{
			BlockUnloadLevelInstance(LevelInstance);
		}

		// Destroy the old LevelInstance instance actor
		GetWorld()->DestroyActor(LevelInstanceActor);

		for (auto& Child : ChildLevelInstances)
		{
			BreakLevelInstance_Impl(Child, Levels - 1, OutMovedActors, Flags);
		}
	}
}

void ULevelInstanceSubsystem::BroadcastOnLevelInstanceEditModeChanged(ILevelInstanceInterface* LevelInstance, ILevelInstanceEditorModule::ELevelInstanceEditMode NewMode)
{
	if (LevelInstance == nullptr)
	{
		return;
	}

	if (ILevelInstanceEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ILevelInstanceEditorModule>("LevelInstanceEditor"))
	{
		EditorModule->BroadcastOnLevelInstanceEditModeChanged(LevelInstance, NewMode);
	}
}

bool ULevelInstanceSubsystem::LevelInstanceHasLevelScriptBlueprint(const ILevelInstanceInterface* LevelInstance) const
{
	if (LevelInstance)
	{
		if (ULevel* LevelInstanceLevel = GetLevelInstanceLevel(LevelInstance))
		{
			if (ULevelScriptBlueprint* LevelScriptBP = LevelInstanceLevel->GetLevelScriptBlueprint(true))
			{
				TArray<UEdGraph*> AllGraphs;
				LevelScriptBP->GetAllGraphs(AllGraphs);
				for (UEdGraph* CurrentGraph : AllGraphs)
				{
					for (UEdGraphNode* Node : CurrentGraph->Nodes)
					{
						if (!Node->IsAutomaticallyPlacedGhostNode())
						{
							return true;
						}
					}
				}
			}
		}
	}

	return false;
}

void ULevelInstanceSubsystem::RemoveLevelsFromWorld(const TArray<ULevel*>& InLevels, bool bResetTrans)
{
	if (LevelsToRemoveScope && LevelsToRemoveScope->IsValid())
	{
		for (ULevel* Level : InLevels)
		{
			LevelsToRemoveScope->Levels.AddUnique(Level);
		}
		LevelsToRemoveScope->bResetTrans |= bResetTrans;
	}
	else
	{
		// No need to clear the whole editor selection since actor of this level will be removed from the selection by: UEditorEngine::OnLevelRemovedFromWorld
		EditorLevelUtils::RemoveLevelsFromWorld(InLevels, /*bClearSelection*/false, bResetTrans);
	}
}

ULevelInstanceSubsystem::FLevelsToRemoveScope::FLevelsToRemoveScope(ULevelInstanceSubsystem* InOwner)
	: Owner(InOwner)
	, bIsBeingDestroyed(false)
{}

ULevelInstanceSubsystem::FLevelsToRemoveScope::~FLevelsToRemoveScope()
{
	if (Levels.Num() > 0)
	{
		bIsBeingDestroyed = true;
		double StartTime = FPlatformTime::Seconds();
		ULevelInstanceSubsystem* LevelInstanceSubsystem = Owner.Get();
		check(LevelInstanceSubsystem);
		LevelInstanceSubsystem->RemoveLevelsFromWorld(Levels, bResetTrans);
		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogLevelInstance, Log, "Unloaded %ls levels in %ls seconds", *FText::AsNumber(Levels.Num()).ToString(), *FText::AsNumber(ElapsedTime).ToString());
	}
}

bool ULevelInstanceSubsystem::CanMoveActorToLevel(const AActor* Actor, FText* OutReason) const
{
	if (Actor->IsA<ALevelInstancePivot>())
	{
		return false;
	}

	if (Actor->GetWorld() == GetWorld())
	{
		if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
		{
			if (Actor->GetExternalDataLayerAsset())
			{
				if (OutReason != nullptr)
				{
					*OutReason = LOCTEXT("CantMoveActorUsingExternalDataLayer", "Can't move Level Instance actor using External Data Layer");
				}
				return false;
			}

			if (IsEditingLevelInstance(LevelInstance))
			{
				if (OutReason != nullptr)
				{
					*OutReason = LOCTEXT("CantMoveActorLevelEditing", "Can't move Level Instance actor while it is being edited");
				}
				return false;
			}

			if (IsEditingLevelInstancePropertyOverrides(LevelInstance))
			{
				if (OutReason != nullptr)
				{
					*OutReason = LOCTEXT("CanMoveActorLevelPropertyOverries", "Can't move Level Instance actor while it is in property override edit");
				}
				return false;
			}

			bool bEditingChildren = false;
			ForEachLevelInstanceChild(LevelInstance, true, [this, &bEditingChildren](const ILevelInstanceInterface* ChildLevelInstance)
			{
				if (IsEditingLevelInstance(ChildLevelInstance))
				{
					bEditingChildren = true;
					return false;
				}
				return true;
			});

			if (bEditingChildren)
			{
				if (OutReason != nullptr)
				{
					*OutReason = LOCTEXT("CantMoveActorToLevelChildEditing", "Can't move Level Instance actor while one of its child Level Instance is being edited");
				}
				return false;
			}
		}

		if (ILevelInstanceInterface* ParentLevelInstance = GetParentLevelInstance(Actor))
		{
			if (ParentLevelInstance->IsEditingPropertyOverrides())
			{
				if (OutReason != nullptr)
				{
					*OutReason = LOCTEXT("CanMoveActorToLevelParentPropertyOverrides", "Can't move actor while its parent is in property override edit");
				}
				return false;
			}
		}
	}

	return true;
}

void ULevelInstanceSubsystem::OnActorDeleted(AActor* Actor)
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
	{
		if (Actor->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			// We are receiving this event when destroying the old actor after BP reinstantiation. In this case,
			// the newly created actor was already added to the list, so we can safely ignore this case.
			check(GIsReinstancing);
			return;
		}

		// Unregistered Level Instance Actor nothing to do.
		if (!LevelInstance->HasValidLevelInstanceID())
		{
			return;
		}

		const bool bAlreadyRooted = Actor->IsRooted();
		// Unloading LevelInstances leads to GC and Actor can be collected. Add to root temp. It will get collected after the OnActorDeleted callbacks
		if (!bAlreadyRooted)
		{
			Actor->AddToRoot();
		}

		const bool bIsEditingLevelInstance = IsEditingLevelInstance(LevelInstance);

		FScopedSlowTask SlowTask(0, LOCTEXT("UnloadingLevelInstances", "Unloading Level Instances..."), !GetWorld()->IsGameWorld());
		SlowTask.MakeDialogDelayed(1.0f);
		check(!IsEditingLevelInstanceDirty(LevelInstance) && !HasDirtyChildrenLevelInstances(LevelInstance));
		if (bIsEditingLevelInstance)
		{
			CommitLevelInstance(LevelInstance);
		}
		
		RequestUnloadLevelInstance(LevelInstance);
		
		// Remove from root so it gets collected on the next GC if it can be.
		if (!bAlreadyRooted)
		{
			Actor->RemoveFromRoot();
		}
	}
}

bool ULevelInstanceSubsystem::ShouldIgnoreDirtyPackage(UPackage* DirtyPackage, const UWorld* EditingWorld)
{
	if (DirtyPackage == EditingWorld->GetOutermost())
	{
		return false;
	}

	bool bIgnore = true;
	ForEachObjectWithPackage(DirtyPackage, [&bIgnore, EditingWorld](UObject* Object)
	{
		if (Object->GetOutermostObject() == EditingWorld)
		{
			bIgnore = false;
		}

		return bIgnore;
	});

	return bIgnore;
}

ULevelInstanceSubsystem::FLevelInstanceEdit::FLevelInstanceEdit(ULevelStreamingLevelInstanceEditor* InLevelStreaming, ILevelInstanceInterface* InLevelInstance)
	: LevelStreaming(InLevelStreaming), LevelInstanceActor(CastChecked<AActor>(InLevelInstance))
{
	check(LevelStreaming->LevelInstanceID == InLevelInstance->GetLevelInstanceID());
	// Update Edit Filter before actors are added to world
	InLevelInstance->GetLevelInstanceComponent()->UpdateEditFilter();
	EditorObject = NewObject<ULevelInstanceEditorObject>(GetTransientPackage(), NAME_None, RF_Transactional);
	EditorObject->EnterEdit(GetEditWorld());
}

ULevelInstanceSubsystem::FLevelInstanceEdit::~FLevelInstanceEdit()
{
	EditorObject->ExitEdit();
	if (LevelStreaming)
	{
		ULevelStreamingLevelInstanceEditor::Unload(LevelStreaming);
	}
}

UWorld* ULevelInstanceSubsystem::FLevelInstanceEdit::GetEditWorld() const
{
	if (LevelStreaming && LevelStreaming->GetLoadedLevel())
	{
		return LevelStreaming->GetLoadedLevel()->GetTypedOuter<UWorld>();
	}

	return nullptr;
}

ILevelInstanceInterface* ULevelInstanceSubsystem::FLevelInstanceEdit::GetLevelInstance() const
{
	return Cast<ILevelInstanceInterface>(LevelInstanceActor);
}

void ULevelInstanceSubsystem::FLevelInstanceEdit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EditorObject);
	Collector.AddReferencedObject(LevelStreaming);
	Collector.AddReferencedObject(LevelInstanceActor);
}

bool ULevelInstanceSubsystem::FLevelInstanceEdit::CanDiscard(FText* OutReason) const
{
	return EditorObject->CanDiscard(OutReason);
}

bool ULevelInstanceSubsystem::FLevelInstanceEdit::HasCommittedChanges() const
{
	return EditorObject->bCommittedChanges;
}

void ULevelInstanceSubsystem::FLevelInstanceEdit::MarkCommittedChanges()
{
	EditorObject->bCommittedChanges = true;
}

void ULevelInstanceSubsystem::FLevelInstanceEdit::GetPackagesToSave(TArray<UPackage*>& OutPackagesToSave) const
{
	const UWorld* EditingWorld = GetEditWorld();
	check(EditingWorld);

	FEditorFileUtils::GetDirtyPackages(OutPackagesToSave, [EditingWorld](UPackage* DirtyPackage)
	{
		return ULevelInstanceSubsystem::ShouldIgnoreDirtyPackage(DirtyPackage, EditingWorld);
	});

	for (const TWeakObjectPtr<UPackage>& WeakPackageToSave : EditorObject->OtherPackagesToSave)
	{
		if (UPackage* PackageToSave = WeakPackageToSave.Get())
		{
			OutPackagesToSave.Add(PackageToSave);
		}
	}
}

const ULevelInstanceSubsystem::FLevelInstanceEdit* ULevelInstanceSubsystem::GetLevelInstanceEdit(const ILevelInstanceInterface* LevelInstance) const
{
	if (LevelInstance && LevelInstanceEdit && LevelInstanceEdit->GetLevelInstance() == LevelInstance)
	{
		return LevelInstanceEdit.Get();
	}

	return nullptr;
}

const ULevelInstanceSubsystem::FPropertyOverrideEdit* ULevelInstanceSubsystem::GetLevelInstancePropertyOverrideEdit(const ILevelInstanceInterface* LevelInstance) const
{
	if (LevelInstance && PropertyOverrideEdit && PropertyOverrideEdit->GetLevelInstance() == LevelInstance)
	{
		return PropertyOverrideEdit.Get();
	}

	return nullptr;
}

bool ULevelInstanceSubsystem::IsEditingLevelInstanceDirty(const ILevelInstanceInterface* LevelInstance) const
{
	const FLevelInstanceEdit* CurrentEdit = GetLevelInstanceEdit(LevelInstance);
	if (!CurrentEdit)
	{
		return false;
	}

	return IsLevelInstanceEditDirty(CurrentEdit);
}

bool ULevelInstanceSubsystem::IsLevelInstanceEditDirty(const FLevelInstanceEdit* InLevelInstanceEdit) const
{
	TArray<UPackage*> OutPackagesToSave;
	InLevelInstanceEdit->GetPackagesToSave(OutPackagesToSave);
	return OutPackagesToSave.Num() > 0;
}

ILevelInstanceInterface* ULevelInstanceSubsystem::GetEditingLevelInstance() const
{
	if (LevelInstanceEdit)
	{
		return LevelInstanceEdit->GetLevelInstance();
	}

	return nullptr;
}

bool ULevelInstanceSubsystem::PromptUserForCommit(const FLevelInstanceEdit* InLevelInstanceEdit, bool& bOutDiscard, bool bForceCommit) const
{
	bOutDiscard = false;
	// Can commit no pending changes
	if (!IsLevelInstanceEditDirty(InLevelInstanceEdit)) 
	{
		return true;
	}

	// If changes can be discarded prompt user
	if (CanCommitLevelInstance(InLevelInstanceEdit->GetLevelInstance(), /*bDiscardEdits=*/true))
	{
		// if bForceExit we can't cancel the exiting of the mode so the user needs to decide between saving or discarding
		EAppReturnType::Type Ret = FMessageDialog::Open(
			bForceCommit ? EAppMsgType::YesNo : EAppMsgType::YesNoCancel, LOCTEXT("CommitOrDiscardChangesMsg", "Unsaved Level changes will get discarded. Do you want to save them now?"),
			LOCTEXT("CommitOrDiscardChangesTitle", "Save changes?"));
		if (Ret == EAppReturnType::Cancel && !bForceCommit)
		{
			return false;
		}

		bOutDiscard = (Ret != EAppReturnType::Yes);
	}

	// Can commit but can't discard changes
	return true;
}

bool ULevelInstanceSubsystem::PromptUserForCommitPropertyOverrides(const FPropertyOverrideEdit* InPropertyOverrideEdit, bool& bOutDiscard, bool bForceCommit) const
{
	bOutDiscard = false;
	// Can commit no pending changes
	if (!InPropertyOverrideEdit->IsDirty())
	{
		return true;
	}

	// If changes can be discarded prompt user
	if (CanCommitLevelInstancePropertyOverrides(InPropertyOverrideEdit->GetLevelInstance(), /*bDiscardEdits=*/true))
	{
		// if bForceExit we can't cancel the exiting of the mode so the user needs to decide between saving or discarding
		EAppReturnType::Type Ret = FMessageDialog::Open(
			bForceCommit ? EAppMsgType::YesNo : EAppMsgType::YesNoCancel, LOCTEXT("CommitOrDiscardPropertyOverrideChangesMsg", "Unsaved Property override changes will get discarded. Do you want to save them now?"),
			LOCTEXT("CommitOrDiscardPropertyOverrideChangesTitle", "Save changes?"));
		if (Ret == EAppReturnType::Cancel && !bForceCommit)
		{
			return false;
		}

		bOutDiscard = (Ret != EAppReturnType::Yes);
	}

	// Can commit but can't discard changes
	return true;
}

bool ULevelInstanceSubsystem::CanEditLevelInstanceCommon(const ILevelInstanceInterface* LevelInstance, FText* OutReason) const
{
	// Only allow Editing in Editor World
	if (GetWorld()->WorldType != EWorldType::Editor)
	{
		return false;
	}

	if (IsEditingLevelInstance(LevelInstance))
	{
		if (OutReason)
		{
			*OutReason = FText::Format(LOCTEXT("CanEditLevelInstanceAlreadyBeingEdited", "This level instance is already being edited.\n\nAsset path: {0}"), FText::FromString(LevelInstance->GetWorldAssetPackage()));
		}
		return false;
	}

	if (IsEditingLevelInstancePropertyOverrides(LevelInstance))
	{
		if (OutReason)
		{
			*OutReason = FText::Format(LOCTEXT("CanEditLevelInstanceAlreadyBeingOverriden", "This level instance is already being overridden.\n\nAsset path: {0}"), FText::FromString(LevelInstance->GetWorldAssetPackage()));
		}
		return false;
	}

	return true;
}

bool ULevelInstanceSubsystem::CanEditLevelInstance(const ILevelInstanceInterface* LevelInstance, FText* OutReason) const
{
	if (!CanEditLevelInstanceCommon(LevelInstance, OutReason))
	{
		return false;
	}
	
	if (LevelInstance->IsWorldAssetValid())
	{
		const FString WorldAssetPackage = LevelInstance->GetWorldAssetPackage();

		if (GetWorld()->PersistentLevel->GetPackage()->GetName() == WorldAssetPackage)
		{
			if (OutReason)
			{
				*OutReason = FText::Format(LOCTEXT("CanEditLevelInstancePersistentLevel", "The Persistent level and the Level Instance are the same ({0})."), FText::FromString(WorldAssetPackage));
			}
			return false;
		}

		if (FLevelUtils::FindStreamingLevel(GetWorld(), *WorldAssetPackage))
		{
			if (OutReason)
			{
				*OutReason = FText::Format(LOCTEXT("CanEditLevelInstanceAlreadyExists", "The same level was added to world outside of Level Instances ({0})."), FText::FromString(WorldAssetPackage));
			}
			return false;
		}

		FPackagePath WorldAssetPath;
		if (!FPackagePath::TryFromPackageName(WorldAssetPackage, WorldAssetPath) || !FPackageName::DoesPackageExist(WorldAssetPath))
		{
			if (OutReason)
			{
				*OutReason = FText::Format(LOCTEXT("CanEditLevelInstanceInvalidAsset", "Level Instance asset is invalid ({0})."), FText::FromString(WorldAssetPackage));
			}
			return false;
		}

		if (!Cast<APackedLevelActor>(LevelInstance) && 
		ULevel::GetIsLevelPartitionedFromPackage(*WorldAssetPackage) &&
		!ULevel::GetIsStreamingDisabledFromPackage(*WorldAssetPackage))
		{
			ILevelInstanceEditorModule& EditorModule = FModuleManager::GetModuleChecked<ILevelInstanceEditorModule>("LevelInstanceEditor");
			if (!EditorModule.IsEditInPlaceStreamingEnabled())
			{
				if (OutReason)
				{
					*OutReason = FText::Format(LOCTEXT("CanEditLevelInstanceWithStreamingEnabled", "Level Instance can't be edited in place because it has streaming enabled ({0})"), FText::FromString(WorldAssetPackage));
				}
				return false;
			}
		}
	}

	return true;
}

bool ULevelInstanceSubsystem::IsSubSelectionEnabled() const
{
	ILevelInstanceEditorModule& EditorModule = FModuleManager::GetModuleChecked<ILevelInstanceEditorModule>("LevelInstanceEditor");
	return EditorModule.IsSubSelectionEnabled();
}

bool ULevelInstanceSubsystem::CanEditLevelInstancePropertyOverrides(const ILevelInstanceInterface* LevelInstance, FText* OutReason) const
{
	if (!CanEditLevelInstanceCommon(LevelInstance, OutReason))
	{
		return false;
	}

	if (!ULevelInstanceSettings::Get()->IsLevelInstancePropertyOverrideSettingEnabled())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("LevelInstanceNotEnabledPropertyOverrides", "Check the \"Enable Level Instance Property Overrides\" Editor Setting");
		}
		return false;
	}

	if (!ULevelInstanceSettings::Get()->IsPropertyOverridePolicyDefined())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("LevelInstanceProvidePolicyClass", "Provide an override policy class in your DefaultEngine.ini file.\nFor ex. under [/Script/Engine.LevelInstanceSettings], add PropertyOverridePolicyClass=/Script/Engine.LevelInstancePropertyOverrideSamplePolicy");
		}
		return false;
	}

	if (!LevelInstance->SupportsPropertyOverrides())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("LevelInstanceNoSupportPropertyOverrides", "Level Instance does not support property overrides");
		}
		return false;
	}
		
	const ILevelInstanceInterface* TopLevelAncestor = nullptr;
	bool bAllWorldPartitions = true;
	ForEachLevelInstanceAncestorsAndSelf(CastChecked<AActor>(LevelInstance), [this, &bAllWorldPartitions, &TopLevelAncestor](const ILevelInstanceInterface* AncestorOrSelf)
	{
		// Get the level loaded by this Level Instance
		ULevel* LoadedLevel = GetLevelInstanceLevel(AncestorOrSelf);
		// Check that it is World Partitioned
		bAllWorldPartitions &= LoadedLevel && !!LoadedLevel->GetWorldPartition();

		TopLevelAncestor = AncestorOrSelf;
		return bAllWorldPartitions;
	});
		
	// If any of the levels in the edit hierarchy are non-partitioned, we can't edit
	if (!bAllWorldPartitions || !GetWorld()->IsPartitionedWorld())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("LevelInstancePropertyOverridesWorldPartitionOnly", "Overrides are only supported for levels that use World Partition.");
		}
		return false;
	}

	if (GetWorld()->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("LevelInstancePropertyOverridesNewlyCreated", "Overrides are only supported for saved levels. Save the level first.");
		}
		return false;
	}

	return true;
}

bool ULevelInstanceSubsystem::CanCommitLevelInstance(const ILevelInstanceInterface* LevelInstance, bool bDiscardEdits, FText* OutReason) const
{
	if (const FLevelInstanceEdit* CurrentEdit = GetLevelInstanceEdit(LevelInstance))
	{
		return !bDiscardEdits || CurrentEdit->CanDiscard(OutReason);
	}

	if (OutReason)
	{
		*OutReason = LOCTEXT("CanCommitLevelInstanceNotEditing", "Level Instance is not currently being edited");
	}
	return false;
}

bool ULevelInstanceSubsystem::CanCommitLevelInstancePropertyOverrides(const ILevelInstanceInterface* LevelInstance, bool bDiscardEdits, FText* OutReason) const
{
	if (const FPropertyOverrideEdit* CurrentEdit = GetLevelInstancePropertyOverrideEdit(LevelInstance))
	{
		return !bDiscardEdits || CurrentEdit->CanDiscard(OutReason);
	}

	if (OutReason)
	{
		*OutReason = LOCTEXT("CanCommitLevelInstanceNotOverriding", "Level Instance is not currently in property override edit");
	}
	return false;
}

void ULevelInstanceSubsystem::EditLevelInstance(ILevelInstanceInterface* LevelInstance, TWeakObjectPtr<AActor> ContextActorPtr)
{
	EditLevelInstanceInternal(LevelInstance, ContextActorPtr, FString(), false);
}

bool ULevelInstanceSubsystem::EditLevelInstanceInternal(ILevelInstanceInterface* LevelInstance, TWeakObjectPtr<AActor> ContextActorPtr, const FString& InActorNameToSelect, bool bRecursive)
{
	check(CanEditLevelInstance(LevelInstance));
		
	const FLevelInstanceID EditLevelInstanceID = LevelInstance->GetLevelInstanceID();

	bool bDiscard = false;
	bool bDiscardPropertyOverride = false;

	// If there is a current property override edit and it is dirty, offer the user a chance to Save/Discard/Cancel
	if (PropertyOverrideEdit && !PromptUserForCommitPropertyOverrides(PropertyOverrideEdit.Get(), bDiscardPropertyOverride))
	{
		return false;
	}

	// If there is a current edit and it is dirty, offer the user a chance to Save/Discard/Cancel
	if (LevelInstanceEdit && !PromptUserForCommit(LevelInstanceEdit.Get(), bDiscard))
	{
		return false;
	}

	// Once we've promted the user and he accepted, if we do have a PropertyOverride edit, commit it first
	if (PropertyOverrideEdit && !CommitLevelInstancePropertyOverridesInternal(PropertyOverrideEdit, bDiscardPropertyOverride))
	{
		return false;
	}
	check(!PropertyOverrideEdit);

	// In case we committed some overrides
	LevelInstance = GetLevelInstance(EditLevelInstanceID);
		
	FScopedSlowTask SlowTask(0, LOCTEXT("BeginEditLevelInstance", "Loading Level Instance for edit..."), !GetWorld()->IsGameWorld());
	SlowTask.MakeDialog();

	// Gather information from the context actor to try and select something meaningful after the loading
	FString ActorNameToSelect = GetActorNameToSelectFromContext(LevelInstance, ContextActorPtr.Get(), InActorNameToSelect);

	// Make sure selection is refreshed (Edit can have impact on details view)
	GEditor->SelectNone(true, true);
	
	// Avoid calling OnEditChild twice  on ancestors when EditLevelInstance calls itself
	if (!bRecursive)
	{
		TArray<FLevelInstanceID> AncestorIDs;
		ForEachLevelInstanceAncestors(CastChecked<AActor>(LevelInstance), [&AncestorIDs](ILevelInstanceInterface* InAncestor)
		{
			AncestorIDs.Add(InAncestor->GetLevelInstanceID());
			return true;
		});

		for (const FLevelInstanceID& AncestorID : AncestorIDs)
		{
			OnEditChild(AncestorID);
		}
	}

	// Check if there is an open (but clean) ancestor unload it before opening the LevelInstance for editing
	if (LevelInstanceEdit)
	{	
		// Only support one level of recursion to commit current edit
		check(!bRecursive);

		// Make sure to keep the top level instance actor loaded when we commit the current one
		FWorldPartitionReference CurrentEditLevelInstanceActorRef = CurrentEditLevelInstanceActor;
		
		CommitLevelInstanceInternal(LevelInstanceEdit, bDiscard);

		ILevelInstanceInterface* LevelInstanceToEdit = GetLevelInstance(EditLevelInstanceID);
		check(LevelInstanceToEdit);

		return EditLevelInstanceInternal(LevelInstanceToEdit, nullptr, ActorNameToSelect, /*bRecursive=*/true);
	}

	// Cleanup async requests in case
	LevelInstancesToUnload.Remove(EditLevelInstanceID);
	LevelInstancesToLoadOrUpdate.Remove(LevelInstance);
	// Unload right away
	UnloadLevelInstance(EditLevelInstanceID);
	
	// When editing a Level Instance, push a new empty actor editor context
	UActorEditorContextSubsystem::Get()->PushContext();

	// Load Edit LevelInstance level
	ULevelStreamingLevelInstanceEditor* LevelStreaming = ULevelStreamingLevelInstanceEditor::Load(LevelInstance);
	if (!LevelStreaming)
	{
		UActorEditorContextSubsystem::Get()->PopContext();
		LevelInstance->LoadLevelInstance();
		return false;
	}

	check(LevelInstanceEdit.IsValid());
	check(LevelInstanceEdit->GetLevelInstance() == LevelInstance);
	check(LevelInstanceEdit->LevelStreaming == LevelStreaming);
		
	// Try and select something meaningful
	SelectActorFromActorName(LevelInstance, ActorNameToSelect);
	
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	LevelInstanceActor->SetIsTemporarilyHiddenInEditor(false);

	// Notify
	LevelInstance->OnEdit();

	for (const auto& Actor : LevelStreaming->LoadedLevel->Actors)
	{
		const bool bEditing = true;
		if (Actor)
		{
			Actor->PushLevelInstanceEditingStateToProxies(bEditing);
		}
	}
	
	// Edit can't be undone
	GEditor->ResetTransaction(LOCTEXT("LevelInstanceEditResetTrans", "Edit Level Instance"));

	ResetLoadersForWorldAsset(LevelInstance->GetWorldAsset().GetLongPackageName());

	if (UWorldPartition* WorldPartition = LevelInstanceActor->GetWorld()->GetWorldPartition(); WorldPartition && WorldPartition->IsMainWorldPartition())
	{
		AActor* TopLevelInstanceActor = LevelInstanceActor;
		while (AActor* CurrentTopLevelInstanceActor = Cast<AActor>(GetParentLevelInstance(TopLevelInstanceActor)))
		{
			TopLevelInstanceActor = CurrentTopLevelInstanceActor;
		}

		if (FWorldPartitionActorDescInstance* TopLevelInstanceActorDescInstance = WorldPartition->GetActorDescInstance(TopLevelInstanceActor->GetActorGuid()))
		{
			check(!CurrentEditLevelInstanceActor.IsValid());
			CurrentEditLevelInstanceActor = FWorldPartitionReference(TopLevelInstanceActorDescInstance->GetContainerInstance(), TopLevelInstanceActorDescInstance->GetGuid());
		}
	}

	return true;
}

void ULevelInstanceSubsystem::ResetLoadersForWorldAsset(const FString& WorldAsset)
{
	for (TObjectIterator<UWorld> It(RF_ClassDefaultObject | RF_ArchetypeObject, true); It; ++It)
	{
		UWorld* CurrentWorld = *It;
		if (IsValid(CurrentWorld))
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = CurrentWorld->GetSubsystem<ULevelInstanceSubsystem>())
			{
				LevelInstanceSubsystem->ResetLoadersForWorldAssetInternal(WorldAsset);
			}
		}
	}
}

void ULevelInstanceSubsystem::ResetLoadersForWorldAssetInternal(const FString& WorldAsset)
{
	for (const auto& [LevelInstanceID, LoadedLevelInstance] : LoadedLevelInstances)
	{
		if (LoadedLevelInstance.LevelStreaming && LoadedLevelInstance.LevelStreaming->PackageNameToLoad.ToString() == WorldAsset)
		{
			LoadedLevelInstance.LevelStreaming->ResetLevelInstanceLoaders();
		}
	}
}

bool ULevelInstanceSubsystem::CommitLevelInstance(ILevelInstanceInterface* LevelInstance, bool bDiscardEdits, TSet<FName>* DirtyPackages)
{
	if (GetEditingLevelInstance() == LevelInstance)
	{
		check(LevelInstanceEdit.Get() == GetLevelInstanceEdit(LevelInstance));
		check(CanCommitLevelInstance(LevelInstance));
		return CommitLevelInstanceInternal(LevelInstanceEdit, bDiscardEdits, /*bDiscardOnFailure=*/false, DirtyPackages);
	}
	
	return false;
}

bool ULevelInstanceSubsystem::CommitLevelInstanceInternal(TUniquePtr<FLevelInstanceEdit>& InLevelInstanceEdit, bool bDiscardEdits, bool bDiscardOnFailure, TSet<FName>* DirtyPackages)
{
	TGuardValue<bool> CommitScope(bIsCommittingLevelInstance, true);
	ILevelInstanceInterface* LevelInstance = InLevelInstanceEdit->GetLevelInstance();
	check(InLevelInstanceEdit);
	UWorld* EditingWorld = InLevelInstanceEdit->GetEditWorld();
	check(EditingWorld);
			
	// Check with EditorObject if Discard is possible
	if (!InLevelInstanceEdit->CanDiscard())
	{
		bDiscardEdits = false;
	}

	// Check if we need to Commit a Property Override Edit before
	bool bChangesCommitted = false;
	if (PropertyOverrideEdit)
	{
		bChangesCommitted |= CommitLevelInstancePropertyOverridesInternal(PropertyOverrideEdit, bDiscardEdits);
		check(!PropertyOverrideEdit);
	}
		
	// Build list of Packages to save
	TSet<FName> PackagesToSave;

	// First dirty packages belonging to the edit Level or external level actors that were moved into the level
	TArray<UPackage*> EditPackagesToSave;
	InLevelInstanceEdit->GetPackagesToSave(EditPackagesToSave);
	for (UPackage* PackageToSave : EditPackagesToSave)
	{
		PackagesToSave.Add(PackageToSave->GetFName());
	}

	// Second dirty packages passed in to the Commit method
	if (DirtyPackages)
	{
		PackagesToSave.Append(*DirtyPackages);
	}
	
	const FString WorldAssetPackageStr = LevelInstance->GetWorldAssetPackage();
	const FName WorldAssetPackage(*WorldAssetPackageStr);

	// Backup ID on Commit in case Actor gets recreated
	const FLevelInstanceID LevelInstanceID = LevelInstance->GetLevelInstanceID();

	// Did some change get saved outside of the commit (regular saving in editor while editing)
	bChangesCommitted |= InLevelInstanceEdit->HasCommittedChanges();
	if (PackagesToSave.Num() > 0 && !bDiscardEdits)
	{
		const bool bPromptUserToSave = false;
		const bool bSaveMapPackages = true;
		const bool bSaveContentPackages = true;
		const bool bFastSave = false;
		const bool bNotifyNoPackagesSaved = false;
		const bool bCanBeDeclined = true;

		bool bSaveSucceeded = FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, nullptr,
			[&PackagesToSave, EditingWorld](UPackage* DirtyPackage)
			{
				if (PackagesToSave.Contains(DirtyPackage->GetFName()))
				{
					return false;
				}
				return ShouldIgnoreDirtyPackage(DirtyPackage, EditingWorld);
			});
				
		if (!bSaveSucceeded && !bDiscardOnFailure)
		{
			return false;
		}

		// Consider changes committed is was already set to true because of outside saves or if the save succeeded.
		bChangesCommitted |= bSaveSucceeded;
	}

	FScopedSlowTask SlowTask(0, LOCTEXT("EndEditLevelInstance", "Unloading Level..."), !GetWorld()->IsGameWorld());
	SlowTask.MakeDialog();

	// Make sure selection is refreshed (Commit can have impact on details view)
	GEditor->SelectNone(true, true);

	// Remove from streaming level...
	BroadcastOnLevelInstanceEditModeChanged(LevelInstance, ILevelInstanceEditorModule::ELevelInstanceEditMode::None);
	InLevelInstanceEdit.Reset();

	if (bChangesCommitted)
	{
		ULevel::ScanLevelAssets(WorldAssetPackageStr);
	}

	// Notify (Actor might get destroyed by this call if its a packed bp)
	LevelInstance->OnCommit(bChangesCommitted);

	if (GLevelInstanceShouldAutoUpdatePackedBlueprintsOnCommit && bChangesCommitted)
	{
		ILevelInstanceEditorModule& EditorModule = FModuleManager::LoadModuleChecked<ILevelInstanceEditorModule>("LevelInstanceEditor");
		EditorModule.UpdateAllPackedLevelActorsForWorldAsset(LevelInstance->GetWorldAsset());
	}

	// Update pointer since BP Compilation might have invalidated LevelInstance
	LevelInstance = GetLevelInstance(LevelInstanceID);

	// Update Registered Container Bounds
	UActorDescContainerSubsystem::GetChecked().NotifyContainerUpdated(WorldAssetPackage);

	TMap<ULevelInstanceSubsystem*, TArray<FLevelInstanceID>> LevelInstancesToUpdate;
	// Gather list to update
	for (TObjectIterator<UWorld> It(RF_ClassDefaultObject | RF_ArchetypeObject, true); It; ++It)
	{
		UWorld* CurrentWorld = *It;
		if (IsValid(CurrentWorld))
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = CurrentWorld->GetSubsystem<ULevelInstanceSubsystem>())
			{
				TArray<ILevelInstanceInterface*> WorldLevelInstances = LevelInstanceSubsystem->GetLevelInstances(WorldAssetPackageStr);
				for (ILevelInstanceInterface* CurrentLevelInstance : WorldLevelInstances)
				{
					if (CurrentLevelInstance == LevelInstance || bChangesCommitted)
					{
						LevelInstancesToUpdate.FindOrAdd(LevelInstanceSubsystem).Add(CurrentLevelInstance->GetLevelInstanceID());
					}
				}
			}
		}
	}

	// Do update
	for (const auto& KeyValuePair : LevelInstancesToUpdate)
	{
		for (const FLevelInstanceID& LevelInstanceIDToUpdate : KeyValuePair.Value)
		{
			if (ILevelInstanceInterface* LevelInstanceToUpdate = KeyValuePair.Key->GetLevelInstance(LevelInstanceIDToUpdate))
			{
				LevelInstanceToUpdate->UpdateLevelInstanceFromWorldAsset();
			}
		}
	}

	LevelInstance = GetLevelInstance(LevelInstanceID);
	
	// Notify Ancestors
	FLevelInstanceID LevelInstanceToSelectID = LevelInstanceID;
	TArray<FLevelInstanceID> AncestorIDs;
	ForEachLevelInstanceAncestors(CastChecked<AActor>(LevelInstance), [&LevelInstanceToSelectID, &AncestorIDs](ILevelInstanceInterface* AncestorLevelInstance)
	{
		LevelInstanceToSelectID = AncestorLevelInstance->GetLevelInstanceID();
		AncestorIDs.Add(AncestorLevelInstance->GetLevelInstanceID());
		return true;
	});

	for (const FLevelInstanceID& AncestorID : AncestorIDs)
	{
		OnCommitChild(AncestorID, bChangesCommitted);
	}

	CurrentEditLevelInstanceActor.Reset();
		
	if (ILevelInstanceInterface* LevelInstanceToSelect = GetLevelInstance(LevelInstanceToSelectID))
	{
		GEditor->SelectActor(CastChecked<AActor>(LevelInstanceToSelect), true, true);
	}
				
	// Wait for Level Instances to be loaded
	BlockOnLoading();

	// Send out Event if changes were committed
	if (bChangesCommitted)
	{
		LevelInstanceChangedEvent.Broadcast(WorldAssetPackage);

		// Send event per World (per LevelInstanceSubsystem)
		for (const auto& KeyValuePair : LevelInstancesToUpdate)
		{
			TArray<ILevelInstanceInterface*> UpdatedLevelInstances;
			for (const FLevelInstanceID& UpdatedLevelInstanceID : KeyValuePair.Value)
			{
				if (ILevelInstanceInterface* UpdatedLevelInstance = KeyValuePair.Key->GetLevelInstance(UpdatedLevelInstanceID))
				{
					UpdatedLevelInstances.Add(UpdatedLevelInstance);
				}
			}

			if (UpdatedLevelInstances.Num() > 0)
			{
				KeyValuePair.Key->LevelInstancesUpdatedEvent.Broadcast(UpdatedLevelInstances);
			}
		}
	}
	else
	{
		LevelInstance = GetLevelInstance(LevelInstanceID);
		LevelInstanceEditCancelled.Broadcast(LevelInstance, /*bHasDiscardedChanges=*/!PackagesToSave.IsEmpty());
	}

	GEngine->BroadcastLevelActorListChanged();

	// Restore actor editor context
	UActorEditorContextSubsystem::Get()->PopContext();

	return true;
}

ILevelInstanceInterface* ULevelInstanceSubsystem::GetParentLevelInstance(const AActor* Actor) const
{
	check(Actor);
	const ULevel* OwningLevel = Actor->GetLevel();
	check(OwningLevel);
	return GetOwningLevelInstance(OwningLevel);
}

void ULevelInstanceSubsystem::BlockOnLoading()
{
	// Make sure blocking loads can happen and are not part of transaction
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);

	// Blocking until LevelInstance is loaded and all its child LevelInstances
	while (LevelInstancesToLoadOrUpdate.Num())
	{
		OnUpdateStreamingState();
	}
}

void ULevelInstanceSubsystem::BlockLoadLevelInstance(ILevelInstanceInterface* LevelInstance)
{
	check(!LevelInstance->IsEditing());
	RequestLoadLevelInstance(LevelInstance, true);

	BlockOnLoading();
}

void ULevelInstanceSubsystem::BlockUnloadLevelInstance(ILevelInstanceInterface* LevelInstance)
{
	check(!LevelInstance->IsEditing());
	RequestUnloadLevelInstance(LevelInstance);

	BlockOnLoading();
}

bool ULevelInstanceSubsystem::HasChildEdit(const ILevelInstanceInterface* LevelInstance) const
{
	const int32* ChildEditCountPtr = ChildEdits.Find(LevelInstance->GetLevelInstanceID());
	return ChildEditCountPtr && *ChildEditCountPtr;
}

bool ULevelInstanceSubsystem::HasParentEdit(const ILevelInstanceInterface* LevelInstance) const
{
	bool bResult = false;
	
	const AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	ForEachLevelInstanceAncestors(LevelInstanceActor, [LevelInstance, &bResult](const ILevelInstanceInterface* Ancestor)
	{
		bResult = Ancestor->IsEditing();
		return !bResult;
	});

	return bResult;
}

bool ULevelInstanceSubsystem::HasParentPropertyOverridesEdit(const ILevelInstanceInterface* LevelInstance) const
{
	bool bResult = false;

	const AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	ForEachLevelInstanceAncestors(LevelInstanceActor, [LevelInstance, &bResult](const ILevelInstanceInterface* Ancestor)
	{
		bResult = Ancestor->IsEditingPropertyOverrides();
		return !bResult;
	});

	return bResult;
}

void ULevelInstanceSubsystem::OnCommitChild(const FLevelInstanceID& LevelInstanceID, bool bChildChanged)
{
	int32& ChildEditCount = ChildEdits.FindChecked(LevelInstanceID);
	check(ChildEditCount > 0);
	ChildEditCount--;

	if (ILevelInstanceInterface* LevelInstance = GetLevelInstance(LevelInstanceID))
	{
		LevelInstance->OnCommitChild(bChildChanged);
	}
}

void ULevelInstanceSubsystem::OnEditChild(const FLevelInstanceID& LevelInstanceID)
{
	int32& ChildEditCount = ChildEdits.FindOrAdd(LevelInstanceID, 0);
	// Child edit count can reach 2 maximum in the Context of creating a LevelInstance inside an already editing child level instance
	// through CreateLevelInstanceFrom
	check(ChildEditCount < 2);
	ChildEditCount++;

	if (ILevelInstanceInterface* LevelInstance = GetLevelInstance(LevelInstanceID))
	{
		LevelInstance->OnEditChild();
	}
}

TArray<ILevelInstanceInterface*> ULevelInstanceSubsystem::GetLevelInstances(const FString& WorldAssetPackage) const
{
	TArray<ILevelInstanceInterface*> MatchingLevelInstances;
	for (const auto& KeyValuePair : RegisteredLevelInstances)
	{
		if (KeyValuePair.Value->GetWorldAssetPackage() == WorldAssetPackage)
		{
			MatchingLevelInstances.Add(KeyValuePair.Value);
		}
	}

	return MatchingLevelInstances;
}

TArray<ILevelInstanceInterface*> ULevelInstanceSubsystem::GetLevelInstances(const TSoftObjectPtr<ULevelInstancePropertyOverrideAsset>& PropertyOverrideAsset) const
{
	TArray<ILevelInstanceInterface*> MatchingLevelInstances;
	for (const auto& KeyValuePair : RegisteredLevelInstances)
	{
		if (ULevelInstancePropertyOverrideAsset* PropertyOverride = KeyValuePair.Value->GetPropertyOverrideAsset(); PropertyOverride && PropertyOverride->GetSourceAssetPtr() == PropertyOverrideAsset)
		{
			MatchingLevelInstances.Add(KeyValuePair.Value);
		}
	}

	return MatchingLevelInstances;
}

void ULevelInstanceSubsystem::ForEachLevelInstanceActorAncestors(const ULevel* Level, TFunctionRef<bool(AActor*)> Operation) const
{
	AActor* CurrentActor = Cast<AActor>(GetOwningLevelInstance(Level));
	while (CurrentActor)
	{
		if (!Operation(CurrentActor))
		{
			break;
		}
		CurrentActor = Cast<AActor>(GetParentLevelInstance(CurrentActor));
	};
}

TArray<AActor*> ULevelInstanceSubsystem::GetParentLevelInstanceActors(const ULevel* Level) const
{
	TArray<AActor*> ParentActors;
	ForEachLevelInstanceActorAncestors(Level, [&ParentActors](AActor* Parent)
	{
		ParentActors.Add(Parent);
		return true;
	});
	return ParentActors;
}

// Builds and returns a string based the LevelInstance hierarchy using actor labels.
// Ex: ParentLevelInstanceActorLabel.ChildLevelInstanceActorLabel.ActorLabel
FString ULevelInstanceSubsystem::PrefixWithParentLevelInstanceActorLabels(const FString& ActorLabel, const ULevel* Level) const
{
	TStringBuilder<128> Builder;
	Builder = ActorLabel;
	ForEachLevelInstanceActorAncestors(Level, [&Builder](AActor* Parent)
	{
		if (Builder.Len())
		{
			Builder.Prepend(TEXT("."));
		}
		Builder.Prepend(Parent->GetActorLabel());
		return true;
	});
	return Builder.ToString();
}

bool ULevelInstanceSubsystem::CheckForLoop(const ILevelInstanceInterface* LevelInstance, TArray<TPair<FText, TSoftObjectPtr<UWorld>>>* LoopInfo, const ILevelInstanceInterface** LoopStart)
{
	return CheckForLoop(LevelInstance, LevelInstance->GetWorldAsset(), LoopInfo, LoopStart);
}

bool ULevelInstanceSubsystem::PassLevelInstanceFilter(UWorld* World, const FWorldPartitionHandle& ActorHandle) const
{
	UWorld* ContainerOuterWorld = ActorHandle->GetContainerInstance()->GetOuterWorldPartition()->GetTypedOuter<UWorld>();
	check(ContainerOuterWorld);
	if (const ILevelInstanceInterface* TopAncestor = GetOwningLevelInstance(ContainerOuterWorld->PersistentLevel))
	{
		FActorContainerID ContainerID = TopAncestor->GetLevelInstanceID().GetContainerID();
		ForEachLevelInstanceAncestors(Cast<AActor>(TopAncestor), [&TopAncestor](const ILevelInstanceInterface* Ancestor)
		{
			TopAncestor = Ancestor;
			return true;
		});

		const TMap<FActorContainerID, TSet<FGuid>>& FilteredActors = TopAncestor->GetFilteredActorsPerContainer();
		if (const TSet<FGuid>* FilteredActorsForContainer = FilteredActors.Find(ContainerID))
		{
			if (FilteredActorsForContainer->Contains(ActorHandle->GetGuid()))
			{
				return false;
			}
		}
	}
	
	return true;
}

void ULevelInstanceSubsystem::EditLevelInstancePropertyOverrides(ILevelInstanceInterface* LevelInstance, AActor* ContextActor)
{
	if (!CanEditLevelInstancePropertyOverrides(LevelInstance))
	{
		return;
	}

	const FLevelInstanceID LevelInstanceID = LevelInstance->GetLevelInstanceID();

	bool bDiscard = false;
	bool bCommitEdit = false;
	// Not in same hierarchy, prompt user to commit edit first
	if (LevelInstanceEdit && !LevelInstance->HasParentEdit())
	{
		if (!PromptUserForCommit(LevelInstanceEdit.Get(), bDiscard))
		{
			return;
		}
		// Edit isn't our parent so commit it
		bCommitEdit = true;
	}

	// Gather information from the context actor to try and select something meaningful after the loading
	FString ActorNameToSelect = GetActorNameToSelectFromContext(LevelInstance, ContextActor);

	bool bDiscardPropertyOverride = false;
	if (PropertyOverrideEdit && !PromptUserForCommitPropertyOverrides(PropertyOverrideEdit.Get(), bDiscardPropertyOverride))
	{
		return;
	}

	if (PropertyOverrideEdit && !CommitLevelInstancePropertyOverridesInternal(PropertyOverrideEdit, bDiscardPropertyOverride))
	{
		return;
	}

	if (bCommitEdit && LevelInstanceEdit && !CommitLevelInstanceInternal(LevelInstanceEdit, bDiscard))
	{
		return;
	}

	// Make sure selection is refreshed (Edit can have impact on details view)
	GEditor->SelectNone(true, true);

	// Cleanup async requests in case
	LevelInstancesToUnload.Remove(LevelInstanceID);
	LevelInstancesToLoadOrUpdate.Remove(GetLevelInstance(LevelInstanceID));
	// Unload right away
	UnloadLevelInstance(LevelInstanceID);

	// When editing a Level Instance, push a new empty actor editor context
	UActorEditorContextSubsystem::Get()->PushContext();

	// Load Edit LevelInstance level
	LevelInstance = GetLevelInstance(LevelInstanceID);
	BroadcastOnLevelInstanceEditModeChanged(LevelInstance, ILevelInstanceEditorModule::ELevelInstanceEditMode::Override);

	ULevelStreamingLevelInstanceEditorPropertyOverride::Load(LevelInstance);

	// Select Context Actor
	SelectActorFromActorName(GetLevelInstance(LevelInstanceID), ActorNameToSelect);
}

bool ULevelInstanceSubsystem::CanResetPropertyOverridesForActor(AActor* Actor) const
{
	// Resetting individual Actor overrides requires that we are currently in Property Override Edit
	if (!PropertyOverrideEdit)
	{
		return false;
	}

	// And that the Actor's parent level instance is the current property override edit
	ILevelInstanceInterface* ParentLevelInstance = GetParentLevelInstance(Actor);
	if (ParentLevelInstance != PropertyOverrideEdit->GetLevelInstance())
	{
		return false;
	}

	return true;
}

void ULevelInstanceSubsystem::ResetPropertyOverridesForActor(AActor* Actor)
{
	if (!CanResetPropertyOverridesForActor(Actor))
	{
		return;
	}
	PropertyOverrideEdit->GetLevelInstance()->GetPropertyOverrideAsset()->ResetPropertyOverridesForActor(PropertyOverrideEdit->LevelStreaming, Actor);
}

bool ULevelInstanceSubsystem::IsEditingLevelInstancePropertyOverrides(const ILevelInstanceInterface* LevelInstance) const
{
	return PropertyOverrideEdit && PropertyOverrideEdit->GetLevelInstance() == LevelInstance;
}

bool ULevelInstanceSubsystem::CommitLevelInstancePropertyOverrides(ILevelInstanceInterface* LevelInstance, bool bDiscardEdits)
{
	if (GetEditingPropertyOverridesLevelInstance() == LevelInstance)
	{
		check(PropertyOverrideEdit.Get() == GetLevelInstancePropertyOverrideEdit(LevelInstance));
		check(CanCommitLevelInstancePropertyOverrides(LevelInstance));
		return CommitLevelInstancePropertyOverridesInternal(PropertyOverrideEdit, bDiscardEdits);
	}

	return false;
}

bool ULevelInstanceSubsystem::CommitLevelInstancePropertyOverridesInternal(TUniquePtr<FPropertyOverrideEdit>& InPropertyOverrideEdit, bool bDiscardEdits)
{
	ILevelInstanceInterface* LevelInstance = InPropertyOverrideEdit->GetLevelInstance();
	ILevelInstanceInterface* LevelInstanceWithOverrides = GetLevelInstancePropertyOverridesEditOwner(LevelInstance);
	TSoftObjectPtr<ULevelInstancePropertyOverrideAsset> PreviousOverrideAsset = LevelInstanceWithOverrides->GetPropertyOverrideAsset() ? LevelInstanceWithOverrides->GetPropertyOverrideAsset()->GetSourceAssetPtr() : nullptr;
	const FLevelInstanceID LevelInstanceWithOverridesID = LevelInstanceWithOverrides->GetLevelInstanceID();

	bool bSaved = false;
	if (InPropertyOverrideEdit->IsDirty() && !bDiscardEdits)
	{
		bSaved = InPropertyOverrideEdit->Save(LevelInstanceWithOverrides);
	}

	// Make sure selection is refreshed (Commit can have impact on details view)
	GEditor->SelectNone(true, true);

	// Restore actor editor context
	UActorEditorContextSubsystem::Get()->PopContext();
		
	InPropertyOverrideEdit.Reset();

	// Update Level Instance (saved or not it needs to reload)
	LevelInstanceWithOverrides->UpdateLevelInstanceFromWorldAsset();

	// If it was saved reload other affected level instances
	if (bSaved)
	{
		UpdateLevelInstancesFromPropertyOverrideAsset(PreviousOverrideAsset, LevelInstanceWithOverrides->GetPropertyOverrideAsset());
	}

	BroadcastOnLevelInstanceEditModeChanged(LevelInstanceWithOverrides, ILevelInstanceEditorModule::ELevelInstanceEditMode::None);

	BlockOnLoading();

	if (AActor* LevelInstanceToSelect = Cast<AActor>(GetLevelInstance(LevelInstanceWithOverridesID)))
	{
		GEditor->SelectActor(LevelInstanceToSelect, true, true);
	}
	return true;
}

bool ULevelInstanceSubsystem::CanResetPropertyOverrides(ILevelInstanceInterface* LevelInstance) const
{
	// Don't allow reset of the level instances property overrides if there is a current Property Override Edit in progress
	if (PropertyOverrideEdit)
	{
		return false;
	}

	if (!LevelInstance->GetPropertyOverrideAsset())
	{
		return false;
	}
		
	// Allow reset of the overrides if this level instance is not parented or if it's parent is currently in edit
	ILevelInstanceInterface* ParentLevelInstance = GetParentLevelInstance(CastChecked<AActor>(LevelInstance));
	return !ParentLevelInstance || ParentLevelInstance->IsEditing();
}

void ULevelInstanceSubsystem::ResetPropertyOverrides(ILevelInstanceInterface* LevelInstance)
{
	if (!CanResetPropertyOverrides(LevelInstance))
	{
		return;
	}

	const FLevelInstanceID LevelInstanceWithOverridesID = LevelInstance->GetLevelInstanceID();
	const TSoftObjectPtr<ULevelInstancePropertyOverrideAsset> AssetPath = LevelInstance->GetPropertyOverrideAsset()->GetSourceAssetPtr();
	ULevelInstancePropertyOverrideAsset* PropertyOverrideAsset = LevelInstance->GetPropertyOverrideAsset();
	
	AActor* LevelInstanceActor = CastChecked<AActor>(LevelInstance);
	const bool bWasDirty = LevelInstanceActor->GetPackage()->IsDirty();

	LevelInstance->SetPropertyOverrideAsset(nullptr);

	if (FEditorFileUtils::PromptForCheckoutAndSave({ LevelInstanceActor->GetPackage() }, /*bCheckDirty*/false, /*bPromptToSave*/false) != FEditorFileUtils::PR_Success)
	{
		LevelInstance->SetPropertyOverrideAsset(PropertyOverrideAsset);
		LevelInstanceActor->GetPackage()->SetDirtyFlag(bWasDirty);
		return;
	}

	GEditor->SelectNone(true, true);

	LevelInstance->UpdateLevelInstanceFromWorldAsset();
	UpdateLevelInstancesFromPropertyOverrideAsset(AssetPath, nullptr);
	BlockOnLoading();
	
	// Make sure selection is refreshed (Reset can have impact on details view)
	if (AActor* LevelInstanceToSelect = Cast<AActor>(GetLevelInstance(LevelInstanceWithOverridesID)))
	{
		GEditor->SelectActor(LevelInstanceToSelect, true, true);
	}
}

void ULevelInstanceSubsystem::RegisterPrimitiveColorHandler()
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (!bPrimitiveColorHandlerRegistered && GetDefault<ULevelInstanceSettings>()->IsPropertyOverrideEnabled())
	{
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(TEXT("LevelInstancePropertyOverride"), LOCTEXT("LevelInstancePropertyOverrideColorHandler", "Level Instance Property Override"), [](const UPrimitiveComponent* InPrimitiveComponent)
		{
			if (AActor* Actor = InPrimitiveComponent ? InPrimitiveComponent->GetOwner() : nullptr; Actor && Actor->IsInLevelInstance())
			{
				return Actor->HasLevelInstancePropertyOverrides() ? FLinearColor::Green : FLinearColor::Red;
			}
			return FLinearColor::White;
		},
		[]() {},
		LOCTEXT("LevelInstancePropertyOverride_ToolTip", "Colorize actor with his level instance color, Green means the level has some property overrides, otherwise it will be Red. Rest is White."));
		bPrimitiveColorHandlerRegistered = true;
	}
#endif
}

void ULevelInstanceSubsystem::UnregisterPrimitiveColorHandler()
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (bPrimitiveColorHandlerRegistered)
	{
		FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(TEXT("LevelInstancePropertyOverride"));
		bPrimitiveColorHandlerRegistered = false;
	}
#endif
}

void ULevelInstanceSubsystem::UpdateLevelInstancesFromPropertyOverrideAsset(const TSoftObjectPtr<ULevelInstancePropertyOverrideAsset>& PreviousAssetPath, ULevelInstancePropertyOverrideAsset* NewAsset)
{
	if (PreviousAssetPath.IsNull())
	{
		return;
	}

	TArray<TPair<ULevelInstanceSubsystem*, FLevelInstanceID>> LevelInstancesToUpdate;
	// Gather list to update
	for (TObjectIterator<UWorld> It(RF_ClassDefaultObject | RF_ArchetypeObject, true); It; ++It)
	{
		UWorld* CurrentWorld = *It;
		if (IsValid(CurrentWorld))
		{
			if (ULevelInstanceSubsystem* LevelInstanceSubsystem = CurrentWorld->GetSubsystem<ULevelInstanceSubsystem>())
			{
				TArray<ILevelInstanceInterface*> WorldLevelInstances = LevelInstanceSubsystem->GetLevelInstances(PreviousAssetPath);
				for (ILevelInstanceInterface* CurrentLevelInstance : WorldLevelInstances)
				{
					AActor* CurrentActor = CastChecked<AActor>(CurrentLevelInstance);

					// We are updating a instanced property override so we need to patchup the other instance's property override assets so that reload can be done properly
					ULevelInstancePropertyOverrideAsset* CurrentAsset = CurrentLevelInstance->GetPropertyOverrideAsset();
					check(CurrentAsset && CurrentAsset->GetSourceAssetPtr() == PreviousAssetPath);

					// This is always the case for now but check anyways in case we do end up supporting public override assets
					// Public assets wouldn't need to be patched up
					if (CurrentAsset->IsInOuter(CurrentActor))
					{
						// If there is a new property override duplicate it into the level instance
						ULevelInstancePropertyOverrideAsset* NewAssetCopy = nullptr;
						if (NewAsset)
						{	
							NewAssetCopy = CastChecked<ULevelInstancePropertyOverrideAsset>(StaticDuplicateObject(NewAsset, CurrentActor, NewAsset->GetFName(), CurrentActor->GetFlags()));
						}
						// Set new duplicated asset or nullptr
						CurrentLevelInstance->SetPropertyOverrideAsset(NewAssetCopy);
					}

					// Request update for level instance now that its property override asset has been patched up
					LevelInstancesToUpdate.Add({ LevelInstanceSubsystem, CurrentLevelInstance->GetLevelInstanceID() });
				}
			}
		}
	}

	// Do update
	for (const auto& KeyValuePair : LevelInstancesToUpdate)
	{
		if (ILevelInstanceInterface* LevelInstanceToUpdate = KeyValuePair.Key->GetLevelInstance(KeyValuePair.Value))
		{
			LevelInstanceToUpdate->UpdateLevelInstanceFromWorldAsset();
		}
	}
}

ILevelInstanceInterface* ULevelInstanceSubsystem::GetEditingPropertyOverridesLevelInstance() const
{
	if (PropertyOverrideEdit)
	{
		return PropertyOverrideEdit->GetLevelInstance();
	}

	return nullptr;
}

FActorContainerID ULevelInstanceSubsystem::GetLevelInstancePropertyOverridesContext(ILevelInstanceInterface* LevelInstance) const
{
	// Default to Main container
	FActorContainerID Context;
	check(Context.IsMainContainer());

	// Return the top level ContainerID for which we want to apply properties from.
	// By default we apply everything (main container) but if an ancestor is being edited we want to apply up to the edited ancestor (same as opening that ancestor level)
	ForEachLevelInstanceAncestors(CastChecked<AActor>(LevelInstance), [&Context](ILevelInstanceInterface* Ancestor)
	{
		// If Ancestor of current LevelInstanceOverrideEditOwner is being edited then we stop because we've found the override owner
		if (Ancestor->IsEditing())
		{
			Context = Ancestor->GetLevelInstanceID().GetContainerID();
			return false;
		}
		return true;
	});

	return Context;
}

ILevelInstanceInterface* ULevelInstanceSubsystem::GetLevelInstancePropertyOverridesEditOwner(ILevelInstanceInterface* LevelInstance) const
{
	ILevelInstanceInterface* LevelInstanceOverrideEditOwner = LevelInstance;

	// Find where to save the overrides
	ForEachLevelInstanceAncestorsAndSelf(CastChecked<AActor>(LevelInstance), [&LevelInstanceOverrideEditOwner](ILevelInstanceInterface* Ancestor)
	{
		// If Ancestor of current LevelInstanceOverrideEditOwner is being edited then we stop because we've found the override owner
		if (Ancestor->IsEditing())
		{
			return false;
		}

		LevelInstanceOverrideEditOwner = Ancestor;
		return true;
	});

	return LevelInstanceOverrideEditOwner;
}

const ILevelInstanceInterface* ULevelInstanceSubsystem::GetLevelInstancePropertyOverridesEditOwner(const ILevelInstanceInterface* LevelInstance) const
{
	return GetLevelInstancePropertyOverridesEditOwner(const_cast<ILevelInstanceInterface*>(LevelInstance));
}

bool ULevelInstanceSubsystem::HasEditableLevelInstancePropertyOverrides(TArray<FLevelInstanceActorPropertyOverride>& InPropertyOverrides) const
{
	// Return true if one of the property overrides is from an editable level instance
	for (const FLevelInstanceActorPropertyOverride& LevelInstanceActorPropertyOverride : InPropertyOverrides)
	{
		if (LevelInstanceActorPropertyOverride.LevelInstanceID.IsValid())
		{
			if (AActor* LevelInstanceActor = Cast<AActor>(GetLevelInstance(LevelInstanceActorPropertyOverride.LevelInstanceID)))
			{
				if (!LevelInstanceActor->IsInLevelInstance() || LevelInstanceActor->IsInEditLevelInstance())
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool ULevelInstanceSubsystem::GetLevelInstancePropertyOverridesForActor(const AActor* Actor, FActorContainerID PropertyOverrideContext, TArray<FLevelInstanceActorPropertyOverride>& OutPropertyOverrides) const
{
	if (ILevelInstanceInterface* OwningLevelInstance = GetParentLevelInstance(Actor))
	{
		if (ULevel* Level = GetLevelInstanceLevel(OwningLevelInstance))
		{
			if (UWorldPartition* WorldPartition = Level->GetWorldPartition())
			{
				if (ULevelInstanceContainerInstance* LevelInstanceContainerInstance = Cast<ULevelInstanceContainerInstance>(WorldPartition->FActorDescContainerInstanceCollection::GetActorDescContainerInstance(Actor->GetActorGuid())))
				{
					LevelInstanceContainerInstance->GetPropertyOverridesForActor(OwningLevelInstance->GetLevelInstanceID().GetContainerID(), PropertyOverrideContext, Actor->GetActorGuid(), OutPropertyOverrides);
					return !OutPropertyOverrides.IsEmpty();
				}
			}
		}
	}

	return false;
}

ULevelInstanceSubsystem::FPropertyOverrideEdit::FPropertyOverrideEdit(ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreaming)
	: LevelStreaming(InLevelStreaming)
{
}

ULevelInstanceSubsystem::FPropertyOverrideEdit::~FPropertyOverrideEdit()
{
	ULevelStreamingLevelInstanceEditorPropertyOverride::Unload(LevelStreaming);
}

ILevelInstanceInterface* ULevelInstanceSubsystem::FPropertyOverrideEdit::GetLevelInstance() const
{
	return LevelStreaming->GetLevelInstance();
}


void ULevelInstanceSubsystem::RegisterLoadedLevelStreamingPropertyOverride(ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreaming)
{
	check(!PropertyOverrideEdit.IsValid());
	PropertyOverrideEdit = MakeUnique<FPropertyOverrideEdit>(LevelStreaming);
}

bool ULevelInstanceSubsystem::FPropertyOverrideEdit::IsDirty() const
{
	if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
	{
		for (AActor* Actor : LoadedLevel->Actors)
		{
			if (IsValid(Actor) && Actor->GetPackage()->IsDirty())
			{
				return true;
			}
		}
	}
	
	return false;
}

bool ULevelInstanceSubsystem::FPropertyOverrideEdit::Save(ILevelInstanceInterface* LevelInstanceOverrideOwner) const
{
	check(LevelInstanceOverrideOwner);
			
	AActor* OverrideOwnerActor = CastChecked<AActor>(LevelInstanceOverrideOwner);
	check(OverrideOwnerActor->IsPackageExternal());
	const bool bWasDirty = OverrideOwnerActor->GetPackage()->IsDirty();

	// Create a unique name so that we can update other instances and be sure that their patched PropertyOverrides can have same name
	const FGuid PropertyOverrideGuid = FGuid::NewGuid();
	const FName PropertyOverrideName(*FString::Format(TEXT("PropertyOverride_{0}"), { PropertyOverrideGuid.ToString() }));
	
	ULevelInstancePropertyOverrideAsset* NewPropertyOverride = nullptr;
	ULevelInstancePropertyOverrideAsset* ExistingPropertyOverride = LevelInstanceOverrideOwner->GetPropertyOverrideAsset();
	if (ExistingPropertyOverride)
	{
		// Duplicate previous override object as it contains the overrides that we aren't currently editing that we don't want to lose
		NewPropertyOverride = CastChecked<ULevelInstancePropertyOverrideAsset>(StaticDuplicateObject(ExistingPropertyOverride, OverrideOwnerActor, PropertyOverrideName, OverrideOwnerActor->GetFlags()));
	}
	else
	{
		NewPropertyOverride = NewObject<ULevelInstancePropertyOverrideAsset>(OverrideOwnerActor, PropertyOverrideName, OverrideOwnerActor->GetFlags());
		NewPropertyOverride->Initialize(LevelInstanceOverrideOwner->GetWorldAsset());
	}
	check(NewPropertyOverride->GetWorldAsset() == LevelInstanceOverrideOwner->GetWorldAsset());
			
	// Serialize current edit overrides
	NewPropertyOverride->SerializePropertyOverrides(LevelInstanceOverrideOwner, LevelStreaming);
	
	// Reset to nullptr if override is empty
	LevelInstanceOverrideOwner->SetPropertyOverrideAsset(!NewPropertyOverride->GetPropertyOverridesPerContainer().IsEmpty() ? NewPropertyOverride : nullptr);
	
	// This will be used when initializing the ActorDesc to distinguish between saving the LevelInstance actor normally or through Property override save
	TGuardValue<bool> SavingGuard(NewPropertyOverride->bSavingOverrideEdit, true);
	if (FEditorFileUtils::PromptForCheckoutAndSave({ OverrideOwnerActor->GetPackage() }, /*bCheckDirty*/false, /*bPromptToSave*/false) != FEditorFileUtils::PR_Success)
	{
		// Save failed, set property override to previous (could be null)
		LevelInstanceOverrideOwner->SetPropertyOverrideAsset(ExistingPropertyOverride);
		OverrideOwnerActor->GetPackage()->SetDirtyFlag(bWasDirty);
		return false;
	}

	return true;
}

#endif

#undef LOCTEXT_NAMESPACE

