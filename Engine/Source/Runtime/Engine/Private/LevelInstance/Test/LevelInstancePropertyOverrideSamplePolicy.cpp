// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelInstancePropertyOverrideSamplePolicy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstancePropertyOverrideSamplePolicy)

#if WITH_EDITOR
bool ULevelInstancePropertyOverrideSamplePolicy::CanOverridePropertyImpl(const FProperty* InProperty) const
{
	// Add code here to prevent edit on properties
	return Super::CanOverridePropertyImpl(InProperty);
}
#endif
