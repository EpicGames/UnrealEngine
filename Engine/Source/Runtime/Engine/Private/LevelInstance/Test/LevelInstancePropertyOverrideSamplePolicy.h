// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "LevelInstance/LevelInstancePropertyOverridePolicy.h"
#include "LevelInstancePropertyOverrideSamplePolicy.generated.h"

UCLASS(MinimalAPI)
class ULevelInstancePropertyOverrideSamplePolicy : public ULevelInstancePropertyOverridePolicy
{
	GENERATED_BODY()
public:
	ULevelInstancePropertyOverrideSamplePolicy() {}
	virtual ~ULevelInstancePropertyOverrideSamplePolicy() = default;

#if WITH_EDITOR
	virtual bool CanOverridePropertyImpl(const FProperty* InProperty) const override;
#endif
};
