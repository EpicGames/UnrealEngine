// Copyright Epic Games, Inc. All Rights Reserved.


#include "Engine/LevelScriptActor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineLogs.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerController.h"
#include "Engine/InputDelegateBinding.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelScriptActor)


//////////////////////////////////////////////////////////////////////////
// ALevelScriptActor

ALevelScriptActor::ALevelScriptActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

#if WITH_EDITORONLY_DATA
	bActorLabelEditable = false;
	bEditable = false;
#endif // WITH_EDITORONLY_DATA

	SetCanBeDamaged(false);
	bInputEnabled = true;
 
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	bAlwaysRelevant = true;
	bReplayRewindable = true;

#if WITH_EDITOR
	// this is intended as an early detection of cases where more than one LevelScriptActor is introduced into a single map
	TArray<ALevelScriptActor*> Siblings = FindSiblingLevelScriptActors();

	for (const ALevelScriptActor* Sibling : Siblings)
	{
		ensureMsgf(false,
			TEXT("Detected the creation of more than one LevelScriptActor (%s, %s) within the same outer. This can lead to duplicate level blueprint operations during play."),
			*GetPathName(), *Sibling->GetPathName());
	}
#endif // WITH_EDITOR
}

void ALevelScriptActor::PreInitializeComponents()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (UInputDelegateBinding::SupportsInputDelegate(GetClass()) && !InputComponent)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// create an InputComponent object so that the level script actor can bind key events
		InputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass());
		InputComponent->RegisterComponent();

		UInputDelegateBinding::BindInputDelegates(GetClass(), InputComponent);
	}
	Super::PreInitializeComponents();
}

bool ALevelScriptActor::RemoteEvent(FName EventName)
{
	bool bFoundEvent = false;

	// Iterate over all levels, and try to find a matching function on the level's script actor
	for( TArray<ULevel*>::TConstIterator it = GetWorld()->GetLevels().CreateConstIterator(); it; ++it )
	{
		ULevel* CurLevel = *it;
		if( CurLevel && CurLevel->bIsVisible )
		{
			ALevelScriptActor* LSA = CurLevel->GetLevelScriptActor();
			if( LSA )
			{
				// Find an event with no parameters
				UFunction* EventTarget = LSA->FindFunction(EventName);
				if( EventTarget && EventTarget->NumParms == 0)
				{
					LSA->ProcessEvent(EventTarget, NULL);
					bFoundEvent = true;
				}
			}
		}
	}

	return bFoundEvent;
}

void ALevelScriptActor::SetCinematicMode(bool bCinematicMode, bool bHidePlayer, bool bAffectsHUD, bool bAffectsMovement, bool bAffectsTurning)
{
	// Loop through all player controllers and call their version of SetCinematicMode which is where all the magic happens and replication occurs
	for( FConstPlayerControllerIterator Iterator = GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator )
	{
		if (APlayerController* PC = Iterator->Get())
		{
			PC->SetCinematicMode(bCinematicMode, bHidePlayer, bAffectsHUD, bAffectsMovement, bAffectsTurning);
		}
	}
}

void ALevelScriptActor::EnableInput(class APlayerController* PlayerController)
{
	if (PlayerController != NULL)
	{
		UE_LOGF(LogLevel, Warning, "EnableInput on a LevelScript actor can not be specified for only one PlayerController.  Enabling for all PlayerControllers.");
	}
	bInputEnabled = true;
}

void ALevelScriptActor::DisableInput(class APlayerController* PlayerController)
{
	if (PlayerController != NULL)
	{
		UE_LOGF(LogLevel, Warning, "DisableInput on a LevelScript actor can not be specified for only one PlayerController.  Disabling for all PlayerControllers.");
	}
	bInputEnabled = false;
}


#if WITH_EDITOR

TArray<ALevelScriptActor*> ALevelScriptActor::FindSiblingLevelScriptActors() const
{
	TArray<ALevelScriptActor*> SiblingLSAs;

	const UObject* ThisOuter = GetOuter();
	if (!ThisOuter->IsA<UPackage>())
	{
		TArray<UObject*> AllSiblingObjects;
		GetObjectsWithOuter(ThisOuter, AllSiblingObjects, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);

		for (UObject* Sibling : AllSiblingObjects)
		{
			ALevelScriptActor* SiblingAsLSA = Cast<ALevelScriptActor>(Sibling);
			const bool bIsNotAnLSA = SiblingAsLSA == nullptr;
			const bool bIsTheSameObject = Sibling == this;
			const bool bHasNewerClassVersion = Sibling->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists);

			if (!bIsNotAnLSA && !bIsTheSameObject && !bHasNewerClassVersion)
			{
				SiblingLSAs.Add(SiblingAsLSA);
			}
		}
	}

	return SiblingLSAs;
}
#endif // WITH_EDITOR
