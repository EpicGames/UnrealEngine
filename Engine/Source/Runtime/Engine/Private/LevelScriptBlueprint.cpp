// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LevelScriptBlueprint.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelScriptBlueprint)

//////////////////////////////////////////////////////////////////////////
// ULevelScriptBlueprint

ULevelScriptBlueprint::ULevelScriptBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

FString ULevelScriptBlueprint::GetFriendlyName() const
{
#if WITH_EDITORONLY_DATA
	return FriendlyName;
#else
	return UBlueprint::GetFriendlyName();
#endif
}

FString ULevelScriptBlueprint::CreateLevelScriptNameFromLevel(const ULevel* Level)
{
	// Since all maps are named "PersistentLevel" the level script name is based on the LevelPackage
	check(Level);
	UObject* LevelPackage = Level->GetOutermost();
	return FPackageName::GetShortName(LevelPackage->GetFName().GetPlainNameString());

}

#endif	//#if WITH_EDITOR


