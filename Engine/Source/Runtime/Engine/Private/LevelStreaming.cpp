// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "ContentStreaming.h"
#include "Math/ColorList.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectRedirector.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "Engine/LevelStreamingPersistent.h"
#include "Engine/LevelStreamingVolume.h"
#include "LevelUtils.h"
#include "EngineUtils.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/ReferenceChainSearch.h"
#if WITH_EDITOR
	#include "Framework/Notifications/NotificationManager.h"
	#include "Widgets/Notifications/SNotificationList.h"
#endif
#include "Engine/LevelStreamingDynamic.h"
#include "Components/BrushComponent.h"
#include "Engine/CoreSettings.h"
#include "PhysicsEngine/BodySetup.h"
#include "SceneInterface.h"
#include "Engine/NetConnection.h"
#include "Engine/PackageMapClient.h"
#include "Serialization/LoadTimeTrace.h"
#include "Streaming/LevelStreamingDelegates.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "WorldPartition/WorldPartition.h"
#include "Misc/ScopeRWLock.h"
#include "UObject/UObjectGlobalsInternal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelStreaming)

DEFINE_LOG_CATEGORY(LogLevelStreaming);

#define LOCTEXT_NAMESPACE "World"

// CVars
namespace LevelStreamingCVars
{
	static void OnVisibilityTransactionCVarChanged(IConsoleVariable* CVar)
	{
		if (GEngine)
		{
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				UWorld* World = Context.World();
				if (World && World->IsGameWorld())
				{
					World->InvalidateVisiblityTransactionRequestCachedVariables();
				}
			}
		}
	}

	// There are cases where we might have multiple visibility requests (and data) in flight leading to the server 
	// starting to replicate data based on an older visibility/streamingstatus update which can lead to broken channels
	// to mitigate this problem we assign a TransactionId to each request/update to make sure that we are acting on the correct data
	static bool bDefaultAllowClientUseMakingInvisibleTransactionRequests = true;
	FAutoConsoleVariableRef CVarDefaultAllowClientUseMakingInvisibleTransactionRequests(
		TEXT("LevelStreaming.DefaultAllowClientUseMakingInvisibleTransactionRequests"),
		bDefaultAllowClientUseMakingInvisibleTransactionRequests,
		TEXT("Flag combined with world support to use making invisible transaction requests to the server\n")
		TEXT("that determines whether the client should wait for the server to acknowledge visibility update before making streaming levels invisible.\n")
		TEXT("0: Disable, 1: Enable"),
		FConsoleVariableDelegate::CreateStatic(&*LevelStreamingCVars::OnVisibilityTransactionCVarChanged),
		ECVF_Default);

	static bool bDefaultAllowClientUseMakingVisibleTransactionRequests = false;
	FAutoConsoleVariableRef CVarDefaultAllowClientUseMakingVisibleTransactionRequests(
		TEXT("LevelStreaming.DefaultAllowClientUseMakingVisibleTransactionRequests"),
		bDefaultAllowClientUseMakingVisibleTransactionRequests,
		TEXT("Flag combined with world support to use making visible transaction requests to the server\n")
		TEXT("that determines whether the client should wait for the server to acknowledge visibility update before making streaming levels visible.\n")
		TEXT("0: Disable, 1: Enable"),
		FConsoleVariableDelegate::CreateStatic(&*LevelStreamingCVars::OnVisibilityTransactionCVarChanged),
		ECVF_Default);

	static bool bShouldServerUseMakingVisibleTransactionRequest = true;
	FAutoConsoleVariableRef CVarShouldServerUseMakingVisibleTransactionRequest(
		TEXT("LevelStreaming.ShouldServerUseMakingVisibleTransactionRequest"),
		bShouldServerUseMakingVisibleTransactionRequest,
		TEXT("Whether server should wait for client to acknowledge visibility update before treating streaming levels as visible by the client.\n")
		TEXT("0: Disable, 1: Enable"),
		FConsoleVariableDelegate::CreateStatic(&*LevelStreamingCVars::OnVisibilityTransactionCVarChanged),
		ECVF_Default);

	static bool bShouldReuseUnloadedButStillAroundLevels = true;
	FAutoConsoleVariableRef CVarShouldReuseUnloadedButStillAroundLevels(
		TEXT("LevelStreaming.ShouldReuseUnloadedButStillAroundLevels"),
		bShouldReuseUnloadedButStillAroundLevels,
		TEXT("Whether level streaming will reuse the unloaded levels that aren't GC'd yet.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_ReadOnly);

	static bool bAllowIncrementalRemovalWhilePendingVisibility = true;
	FAutoConsoleVariableRef CVarAllowIncrementalRemovalWhilePendingVisibility(
		TEXT("LevelStreaming.AllowIncrementalRemovalWhilePendingVisibility"),
		bAllowIncrementalRemovalWhilePendingVisibility,
		TEXT("Whether incremental removal of a streaming level can be done while there's a pending visible streaming level being processed.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);

	int32 GLevelStreamingPriorityBias = 0;
	FAutoConsoleVariableRef CVarLevelStreamingPriorityBias(
		TEXT("LevelStreaming.LevelStreamingPriorityBias"),
		GLevelStreamingPriorityBias,
		TEXT("PriorityBias added to level streaming priority when making async load requests"),
		ECVF_Default
	);

	static bool bForceFlushLoadsBeforeAddToWorld = false;
	FAutoConsoleVariableRef CVarForceFlushLoadsBeforeAddToWorld(
		TEXT("LevelStreaming.ForceFlushLoadsBeforeAddToWorld"),
		bForceFlushLoadsBeforeAddToWorld,
		TEXT("Force a full FlushAsyncLoads() call before AddToWorld in MakingVisible.\n")
		TEXT("0: Disable, 1: Enable"),
		ECVF_Default);
}

bool ULevelStreaming::DefaultAllowClientUseMakingInvisibleTransactionRequests()
{
	return LevelStreamingCVars::bDefaultAllowClientUseMakingInvisibleTransactionRequests;
}

bool ULevelStreaming::DefaultAllowClientUseMakingVisibleTransactionRequests()
{
	return LevelStreamingCVars::bDefaultAllowClientUseMakingVisibleTransactionRequests;
}

bool ULevelStreaming::ShouldClientUseMakingInvisibleTransactionRequest() const
{
	if (!bSkipClientUseMakingInvisibleTransactionRequest)
	{
		// Rely on the world to decide whether the client should wait for the server to acknowledge
		// visibility before making streaming levels invisible on the client.
		UWorld* World = GetWorld();
		return World && World->SupportsMakingInvisibleTransactionRequests();
	}
	return false;
}

bool ULevelStreaming::ShouldClientUseMakingVisibleTransactionRequest() const
{
	if (!bSkipClientUseMakingVisibleTransactionRequest)
	{
		// Rely on the world to decide whether the client should wait for the server to acknowledge
		// visibility before making streaming levels visible on the client.
		UWorld* World = GetWorld();
		return World && World->SupportsMakingVisibleTransactionRequests();
	}
	return false;
}

bool ULevelStreaming::ShouldServerUseMakingVisibleTransactionRequest()
{
	return LevelStreamingCVars::bShouldServerUseMakingVisibleTransactionRequest;
}

bool ULevelStreaming::AllowIncrementalRemovalWhilePendingVisibility()
{
	return LevelStreamingCVars::bAllowIncrementalRemovalWhilePendingVisibility;
}

bool ULevelStreaming::ShouldReuseUnloadedButStillAroundLevels(const ULevel* InLevel)
{
#if WITH_EDITOR
	if (InLevel && InLevel->GetForceCantReuseUnloadedButStillAround())
	{
		return false;
	}
#endif
	UWorld* OuterWorld = InLevel ? InLevel->GetTypedOuter<UWorld>() : nullptr;
	if (OuterWorld && OuterWorld->IsGameWorld() && !LevelStreamingCVars::bShouldReuseUnloadedButStillAroundLevels)
	{
		return false;
	}
	return true;
}

int32 ULevelStreamingDynamic::UniqueLevelInstanceId = 0;

/**
 * This helper function is defined here so that it can go into the 4.18.1 hotfix (for UE-51791),
 * even though it would make more logical sense to have this logic in a member function of UNetDriver.
 * We're getting away with this because UNetDriver::GuidCache is (unfortunately) public.
 * 
 * Renames any package entries in the GuidCache with a path matching UnPrefixedName to have a PIE prefix.
 * This is needed because a client may receive an export for a level package before it's loaded and
 * its name registered with FSoftObjectPath::AddPIEPackageName. In this case, the entry in the GuidCache
 * will not be PIE-prefixed, but when the level is actually loaded, its package will be renamed with the
 * prefix. Any subsequent references to this package won't resolve unless the name is fixed up.
 *
 * @param World the world whose NetDriver will be used for the rename
 * @param UnPrefixedPackageName the path of the package to rename
 */
static void NetDriverRenameStreamingLevelPackageForPIE(const UWorld* World, FName UnPrefixedPackageName)
{
	FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);
	if (!WorldContext || WorldContext->WorldType != EWorldType::PIE)
	{
		return;
	}

	for (FNamedNetDriver& Driver : WorldContext->ActiveNetDrivers)
	{
		if (Driver.NetDriver && Driver.NetDriver->GetNetGuidCache().IsValid())
		{
			for (TPair<FNetworkGUID, FNetGuidCacheObject>& GuidPair : Driver.NetDriver->GetNetGuidCache()->ObjectLookup)
			{
				// Only look for packages, which will have a static GUID and an invalid OuterGUID.
				const bool bIsPackage = GuidPair.Key.IsStatic() && !GuidPair.Value.OuterGUID.IsValid();
				if (bIsPackage && GuidPair.Value.PathName == UnPrefixedPackageName)
				{
					GuidPair.Value.PathName = *UWorld::ConvertToPIEPackageName(GuidPair.Value.PathName.ToString(), WorldContext->PIEInstance);
				}
			}
		}
	}
}

FStreamLevelAction::FStreamLevelAction(bool bIsLoading, const FName& InLevelName, bool bIsMakeVisibleAfterLoad, bool bInShouldBlock, const FLatentActionInfo& InLatentInfo, UWorld* World)
	: bLoading(bIsLoading)
	, bMakeVisibleAfterLoad(bIsMakeVisibleAfterLoad)
	, bShouldBlock(bInShouldBlock)
	, LevelName(InLevelName)
	, LatentInfo(InLatentInfo)
{
	ULevelStreaming* LocalLevel = FindAndCacheLevelStreamingObject( LevelName, World );
	Level = LocalLevel;
	ActivateLevel( LocalLevel );
}

void FStreamLevelAction::UpdateOperation(FLatentResponse& Response)
{
	ULevelStreaming* LevelStreamingObject = Level.Get(); // to avoid confusion.
	const bool bIsLevelValid = LevelStreamingObject != nullptr;
	UE_LOGF(LogLevelStreaming, Verbose, "FStreamLevelAction::UpdateOperation() LevelName %ls, bIsLevelValid %d", *LevelName.ToString(), (int32)bIsLevelValid);
	if (bIsLevelValid)
	{
		bool bIsOperationFinished = UpdateLevel(LevelStreamingObject);
	Response.FinishAndTriggerIf(bIsOperationFinished, LatentInfo.ExecutionFunction, LatentInfo.Linkage, LatentInfo.CallbackTarget);
	}
	else
	{
		Response.FinishAndTriggerIf(true, LatentInfo.ExecutionFunction, LatentInfo.Linkage, LatentInfo.CallbackTarget);
	}
}

#if WITH_EDITOR
FString FStreamLevelAction::GetDescription() const
{
	return FString::Printf(TEXT("Streaming Level in progress...(%s)"), *LevelName.ToString());
}
#endif

/**
* Helper function to potentially find a level streaming object by name
*
* @param	LevelName							Name of level to search streaming object for in case Level is NULL
* @return	level streaming object or NULL if none was found
*/
ULevelStreaming* FStreamLevelAction::FindAndCacheLevelStreamingObject( const FName LevelName, UWorld* InWorld )
{
	// Search for the level object by name.
	if( LevelName != NAME_None )
	{
		FString SearchPackageName = MakeSafeLevelName( LevelName, InWorld );
		if (FPackageName::IsShortPackageName(SearchPackageName))
		{
			// Make sure MyMap1 and Map1 names do not resolve to a same streaming level
			SearchPackageName = TEXT("/") + SearchPackageName;
		}

		for (ULevelStreaming* LevelStreaming : InWorld->GetStreamingLevels())
		{
			// We check only suffix of package name, to handle situations when packages were saved for play into a temporary folder
			// Like Saved/Autosaves/PackageName
			if (LevelStreaming && 
				LevelStreaming->GetWorldAssetPackageName().EndsWith(SearchPackageName, ESearchCase::IgnoreCase))
			{
				return LevelStreaming;
			}
		}
	}

	return NULL;
}

/**
 * Given a level name, returns a level name that will work with Play on Editor or Play on Console
 *
 * @param	InLevelName		Raw level name (no UEDPIE or UED<console> prefix)
 * @param	InWorld			World in which to check for other instances of the name
 */
FString FStreamLevelAction::MakeSafeLevelName( const FName& InLevelName, UWorld* InWorld )
{
	// Special case for PIE, the PackageName gets mangled.
	if (!InWorld->StreamingLevelsPrefix.IsEmpty())
	{
		FString PackageName = FPackageName::GetShortName(InLevelName);
		if (!PackageName.StartsWith(InWorld->StreamingLevelsPrefix))
		{
			PackageName = InWorld->StreamingLevelsPrefix + PackageName;
		}

		if (!FPackageName::IsShortPackageName(InLevelName))
		{
			PackageName = FPackageName::GetLongPackagePath(InLevelName.ToString()) + TEXT("/") + PackageName;
		}
		
		return PackageName;
	}
	
	return InLevelName.ToString();
}
/**
* Handles "Activated" for single ULevelStreaming object.
*
* @param	LevelStreamingObject	LevelStreaming object to handle "Activated" for.
*/
void FStreamLevelAction::ActivateLevel( ULevelStreaming* LevelStreamingObject )
{	
	if (LevelStreamingObject)
	{
		// Loading.
		if (bLoading)
		{
			UE_LOGF(LogStreaming, Log, "Streaming in level %ls (%ls)...",*LevelStreamingObject->GetName(),*LevelStreamingObject->GetWorldAssetPackageName());
			LevelStreamingObject->SetShouldBeLoaded(true);
			LevelStreamingObject->SetShouldBeVisible(LevelStreamingObject->GetShouldBeVisibleFlag()	|| bMakeVisibleAfterLoad);
			LevelStreamingObject->bShouldBlockOnLoad = bShouldBlock;
		}
		// Unloading.
		else 
		{
			UE_LOGF(LogStreaming, Log, "Streaming out level %ls (%ls)...",*LevelStreamingObject->GetName(),*LevelStreamingObject->GetWorldAssetPackageName());
			LevelStreamingObject->SetShouldBeLoaded(false);
			LevelStreamingObject->SetShouldBeVisible(false);
			LevelStreamingObject->bShouldBlockOnUnload = bShouldBlock;
		}

		// If we have a valid world
		if (UWorld* LevelWorld = LevelStreamingObject->GetWorld())
		{
				const bool bShouldBeLoaded = LevelStreamingObject->ShouldBeLoaded();
				const bool bShouldBeVisible = LevelStreamingObject->ShouldBeVisible();

			UE_LOGF(LogLevel, Log, "ActivateLevel %ls %i %i %i",
				*LevelStreamingObject->GetWorldAssetPackageName(),
				bShouldBeLoaded,
				bShouldBeVisible,
				bShouldBlock);

			// Notify players of the change
			for (FConstPlayerControllerIterator Iterator = LevelWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
			{
				if (APlayerController* PlayerController = Iterator->Get())
				{
					PlayerController->LevelStreamingStatusChanged(
						LevelStreamingObject,
						bShouldBeLoaded,
						bShouldBeVisible,
						bShouldBlock,
						bShouldBlock,
						INDEX_NONE);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogLevel, Warning, "Failed to find streaming level object associated with '%ls'", *LevelName.ToString() );
	}
}

/**
* Handles "UpdateOp" for single ULevelStreaming object.
*
* @param	LevelStreamingObject	LevelStreaming object to handle "UpdateOp" for.
*
* @return true if operation has completed, false if still in progress
*/
bool FStreamLevelAction::UpdateLevel( ULevelStreaming* LevelStreamingObject )
{
	// No level streaming object associated with this sequence.
	if (LevelStreamingObject == nullptr)
	{
		return true;
	}
	// Level is neither loaded nor should it be so we finished (in the sense that we have a pending GC request) unloading.
	else if ((LevelStreamingObject->GetLoadedLevel() == nullptr) && !LevelStreamingObject->ShouldBeLoaded() )
	{
		return true;
	}
	// Level shouldn't be loaded but is as background level streaming is enabled so we need to fire finished event regardless.
	else if (LevelStreamingObject->GetLoadedLevel() && !LevelStreamingObject->ShouldBeLoaded() && !GUseBackgroundLevelStreaming)
	{
		return true;
	}
	// Level is both loaded and wanted so we finished loading.
	else if (LevelStreamingObject->GetLoadedLevel() && LevelStreamingObject->ShouldBeLoaded() 
		// Make sure we are visible if we are required to be so.
		&& (!bMakeVisibleAfterLoad || LevelStreamingObject->GetLoadedLevel()->bIsVisible) )
	{
		return true;
	}

	// Loading/ unloading in progress.
	return false;
}

/*-----------------------------------------------------------------------------
	ULevelStreaming* implementation.
-----------------------------------------------------------------------------*/

ULevelStreaming::ULevelStreaming(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsStatic(false)
{
# if WITH_EDITORONLY_DATA
	bShouldBeVisibleInEditor = true;
#endif
	LevelColor = FLinearColor::White;
	LevelTransform = FTransform::Identity;
	MinTimeBetweenVolumeUnloadRequests = 2.0f;
	bDrawOnLevelStatusMap = true;
	LevelLODIndex = INDEX_NONE;
	CurrentState = ELevelStreamingState::Removed;
	bSkipClientUseMakingInvisibleTransactionRequest = false;
	bSkipClientUseMakingVisibleTransactionRequest = false;
	bGarbageCollectionClusteringEnabled = true;

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_ImmutableDefaultObject) && ExactCast<ULevelStreaming>(this))
	{
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(TEXT("LevelColor"), LOCTEXT("LevelColor", "Level Color"), [](const UPrimitiveComponent* InPrimitiveComponent)
		{
			if (ULevel* Level = InPrimitiveComponent ? InPrimitiveComponent->GetComponentLevel() : nullptr)
			{
				if (ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level))
				{
					return LevelStreaming->LevelColor;
				}
			}
			return FLinearColor::White;
		},
		[]() {},
		LOCTEXT("LevelColor_ToopTip", "Colorize actor with its level color, otherwise the color is White."));
	}
#endif
}

void ULevelStreaming::PostLoad()
{
	Super::PostLoad();

	const bool PIESession = GetWorld()->WorldType == EWorldType::PIE || GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);

#if WITH_EDITOR
	// If this streaming level was saved with a short package name, try to convert it to a long package name
	if ( !PIESession && PackageName_DEPRECATED != NAME_None )
	{
		const FString DeprecatedPackageNameString = PackageName_DEPRECATED.ToString();
		if ( FPackageName::IsShortPackageName(PackageName_DEPRECATED) == false )
		{
			// Convert the FName reference to a TSoftObjectPtr, then broadcast that we loaded a reference
			// so this reference is gathered by the cooker without having to resave the package.
			SetWorldAssetByPackageName(PackageName_DEPRECATED);
			WorldAsset.GetUniqueID().PostLoadPath(GetLinker());
		}
		else
		{
			UE_LOGF(LogLevelStreaming, Display, "Invalid streaming level package name (%ls). Only long package names are supported. This streaming level may not load or save properly.", *DeprecatedPackageNameString);
		}
	}
#endif

	if (!IsValidStreamingLevel())
	{
		const FString WorldPackageName = GetWorldAssetPackageName();
		UE_LOGF(LogLevelStreaming, Display, "Failed to find streaming level package file: %ls. This streaming level may not load or save properly.", *WorldPackageName);
#if WITH_EDITOR
		if (GIsEditor)
		{
			// Launch notification to inform user of default change
			FFormatNamedArguments Args;
			Args.Add(TEXT("PackageName"), FText::FromString(WorldPackageName));
			FNotificationInfo Info(FText::Format(LOCTEXT("LevelStreamingFailToStreamLevel", "Failed to find streamed level {PackageName}, please fix the reference to it in the Level Browser"), Args));
			Info.ExpireDuration = 7.0f;

			FSlateNotificationManager::Get().AddNotification(Info);
		}
#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	if (GetLinkerUEVersion() < VER_UE4_LEVEL_STREAMING_DRAW_COLOR_TYPE_CHANGE)
	{
		LevelColor = DrawColor_DEPRECATED;
	}
#endif
}

UWorld* ULevelStreaming::GetWorld() const
{
	// Fail gracefully if a CDO
	if(IsTemplate())
	{
		return nullptr;
	}
	// Otherwise 
	else
	{
		return CastChecked<UWorld>(GetOuter());
	}
}

UWorld* ULevelStreaming::GetStreamingWorld() const
{
	check(!IsTemplate());
	return GetWorld();
}

bool ULevelStreaming::IsLevelVisible() const
{
	return LoadedLevel && LoadedLevel->bIsVisible;
}

void ULevelStreaming::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);
	
	if (Ar.IsLoading())
	{
		const bool bIsRunningPIE =
			GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor) && 
			GetOutermost()->GetPIEInstanceID() != INDEX_NONE
		;

		if (bIsRunningPIE)
		{
			RenameForPIE(GetOutermost()->GetPIEInstanceID());
		}

#if WITH_EDITOR
		// If PackageNameToLoad doesn't match WorldAsset, we can potentially create a corrupt UPackage where the name and file path don't match.
		// In this case, we have no option but to clear out PackageNameToLoad, since WorldAsset is the source of truth.
		// Note that RenameForPIE intentionally creates a mismatch, but that's acceptable since PIE effectively creates transient duplicates.
		if (Ar.IsPersistent() && !bIsRunningPIE)
		{
			const bool bHasMismatchedPackageName =
				(PackageNameToLoad != NAME_None) &&
				(PackageNameToLoad != WorldAsset.GetLongPackageFName())
			;

			if (bHasMismatchedPackageName)
			{
				const UPackage* Package = GetPackage();
				ensure(Package);
				const FString& PackageName = Package ? Package->GetName() : TEXT("Unknown package");

				UE_LOGF(LogLevelStreaming, Warning, "WorldAsset (%ls) and PackageNameToLoad (%ls) point to different streaming levels. PackageNameToLoad will be set to none. Resave this map (%ls) to remove warnings.",
					*WorldAsset.GetLongPackageName(),
					*PackageNameToLoad.ToString(),
					*PackageName);

				PackageNameToLoad = NAME_None;
			}
		}
#endif
	}
}

void ULevelStreaming::OnLevelAdded()
{
	if (LoadedLevel)
	{
		if (LoadedLevel->bIsVisible)
		{
			SetCurrentState(ELevelStreamingState::LoadedVisible);
		}
		else
		{
			SetCurrentState(ELevelStreamingState::LoadedNotVisible);
		}
	}
	else
	{
		SetCurrentState(ELevelStreamingState::Unloaded);
	}
}

void ULevelStreaming::OnLevelRemoved()
{
	// If in one of the transitional states removing the level will be highly problematic
	ensure(CurrentState != ELevelStreamingState::Loading);
	ensure(CurrentState != ELevelStreamingState::MakingInvisible);
	ensure(CurrentState != ELevelStreamingState::MakingVisible);

	SetCurrentState(ELevelStreamingState::Removed);
}

ELevelStreamingState ULevelStreaming::GetLevelStreamingState() const
{
	UE::TReadScopeLock ReadLock(CurrentStateLock);
	return CurrentState;
}

void ULevelStreaming::SetCurrentState(ELevelStreamingState NewState)
{
	ELevelStreamingState OldState;
	{
		UE::TWriteScopeLock WriteLock(CurrentStateLock);
		OldState = CurrentState;
		CurrentState = NewState;
	}

	if (OldState != NewState)
	{
		OnCurrentStateChanged(OldState, NewState);
	}

	// TODO: We should only fire the delegate when the current state has changed, but first AsyncLevelLoadComplete needs to be fixed to 
	// only set the new state once the LoadedLevel is assigned. Clients currently rely on getting a repeated notification after the loaded
	// level is available.
	FLevelStreamingDelegates::OnLevelStreamingStateChanged.Broadcast(GetWorld(), this, GetLoadedLevel(), OldState, NewState);
}

bool ULevelStreaming::UpdateTargetState()
{
	FScopeCycleCounterUObject ContextScope(this);
	if (CurrentState == ELevelStreamingState::FailedToLoad || CurrentState == ELevelStreamingState::Removed)
	{
		return false;
	}

	UWorld* World = GetWorld();

#if WITH_EDITOR
	// Don't bother loading sub-levels in PIE for levels that aren't visible in editor
	if (World->IsPlayInEditor() && GEngine->OnlyLoadEditorVisibleLevelsInPIE())
	{
		if (!GetShouldBeVisibleInEditor())
		{
			return false;
		}
	}
#endif

	ELevelStreamingTargetState NewTarget = DetermineTargetState();

	if (TargetState != NewTarget)
	{
		ELevelStreamingTargetState OldTarget = TargetState;
		TargetState = NewTarget;
		FLevelStreamingDelegates::OnLevelStreamingTargetStateChanged.Broadcast(
			GetWorld(), 
			this, 
			GetLoadedLevel(), 
			CurrentState,
			OldTarget,
			NewTarget
			);
	}

	// Return whether the level should continue to be considered
	switch (CurrentState)
	{
	case ELevelStreamingState::MakingVisible:
	case ELevelStreamingState::MakingInvisible:
	case ELevelStreamingState::Loading:
		// Always continue to consider if we are actively working on a state change 
		return true;
	case ELevelStreamingState::Unloaded:
		return NewTarget != ELevelStreamingTargetState::Unloaded;
	case ELevelStreamingState::LoadedNotVisible:
		return NewTarget != ELevelStreamingTargetState::LoadedNotVisible || !IsDesiredLevelLoaded();
	case ELevelStreamingState::LoadedVisible:
		return NewTarget != ELevelStreamingTargetState::LoadedVisible || !IsDesiredLevelLoaded();
	case ELevelStreamingState::FailedToLoad:
	case ELevelStreamingState::Removed:
	default:
		return false;
	}
}

ELevelStreamingTargetState ULevelStreaming::DetermineTargetState() const
{
	UWorld* World = GetWorld();
	switch(CurrentState)
	{
	case ELevelStreamingState::MakingVisible:
		ensure(LoadedLevel);
		if (!ShouldBeVisible() && !World->IsLevelMakingVisible(LoadedLevel))
		{
			// Since level doesn't need to be visible anymore, change TargetState to ELevelStreamingTargetState::LoadedNotVisible.
			// Next UpdateStreamingState will handle switching CurrentState to ELevelStreamingState::LoadedNotVisible.
			// From there, regular flow will properly handle TargetState.
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else
		{
			return ELevelStreamingTargetState::LoadedVisible;
		}

	case ELevelStreamingState::MakingInvisible:
		ensure(LoadedLevel);
		return ELevelStreamingTargetState::LoadedNotVisible;

	case ELevelStreamingState::Loading:
		return ELevelStreamingTargetState::LoadedNotVisible;

	case ELevelStreamingState::Unloaded:
		if (bIsRequestingUnloadAndRemoval)
		{
			return ELevelStreamingTargetState::UnloadedAndRemoved;
		}
		else if (World->GetShouldForceUnloadStreamingLevels())
		{
			return ELevelStreamingTargetState::Unloaded;
		}
		else if (!World->IsGameWorld())
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else if (ShouldBeLoaded())
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else
		{
			return ELevelStreamingTargetState::Unloaded;
		}

	case ELevelStreamingState::LoadedNotVisible:
		if (bIsRequestingUnloadAndRemoval || World->GetShouldForceUnloadStreamingLevels())
		{
			return ELevelStreamingTargetState::Unloaded;
		}
		else if (World->IsGameWorld() && !ShouldBeLoaded())
		{
			return ELevelStreamingTargetState::Unloaded;
		}
		else if (!IsDesiredLevelLoaded())
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else if (ShouldBeVisible())
		{
			return ELevelStreamingTargetState::LoadedVisible;
		}
		else
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}

	case ELevelStreamingState::LoadedVisible:
		if (bIsRequestingUnloadAndRemoval || World->GetShouldForceUnloadStreamingLevels())
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else if (World->IsGameWorld() && !ShouldBeLoaded())
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else if (!ShouldBeVisible())
		{
			return ELevelStreamingTargetState::LoadedNotVisible;
		}
		else if (!IsDesiredLevelLoaded())
		{
			return ELevelStreamingTargetState::LoadedVisible;
		}
		else
		{
			return ELevelStreamingTargetState::LoadedVisible;
		}

	case ELevelStreamingState::FailedToLoad:
		return ELevelStreamingTargetState::Unloaded;

	case ELevelStreamingState::Removed:
		return ELevelStreamingTargetState::Unloaded;

	default:
		ensure(false);
		return ELevelStreamingTargetState::Unloaded;
	}
}

bool ULevelStreaming::IsConcernedByNetVisibilityTransactionAck() const
{
	UWorld* World = GetWorld();
	return LoadedLevel && !LoadedLevel->bClientOnlyVisible && World && World->IsGameWorld() && World->IsNetMode(NM_Client) && (World->NetDriver && World->NetDriver->ServerConnection->GetConnectionState() == USOCK_Open);
}

bool ULevelStreaming::IsWaitingForNetVisibilityTransactionAck(ENetLevelVisibilityRequest InRequestType) const
{
	if (NetVisibilityState.PendingRequestType.IsSet() && (NetVisibilityState.PendingRequestType == InRequestType) && IsConcernedByNetVisibilityTransactionAck())
	{
		check(((NetVisibilityState.PendingRequestType == ENetLevelVisibilityRequest::MakingInvisible) && ShouldClientUseMakingInvisibleTransactionRequest()) ||
			  ((NetVisibilityState.PendingRequestType == ENetLevelVisibilityRequest::MakingVisible) && ShouldClientUseMakingVisibleTransactionRequest()));

		return (NetVisibilityState.ClientPendingRequestIndex != NetVisibilityState.ClientAckedRequestIndex) || NetVisibilityState.bHasClientPendingRequest;
	}

	return false;
}

void ULevelStreaming::ServerUpdateLevelVisibility(bool bIsVisible, bool bTryMakeVisible, FNetLevelVisibilityTransactionId TransactionId)
{
	if (IsConcernedByNetVisibilityTransactionAck())
	{
		UWorld* World = GetWorld();
		for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get(); 
			UNetConnection* NetConnection = IsValid(PlayerController) && !PlayerController->IsPendingKillPending() ? PlayerController->GetNetConnection() : nullptr;
			if (NetConnection && NetConnection->GetUChildConnection() == nullptr && NetConnection->GetConnectionState() == USOCK_Open)
			{
				FUpdateLevelVisibilityLevelInfo LevelVisibility(LoadedLevel, bIsVisible, bTryMakeVisible);
				LevelVisibility.PackageName = PlayerController->NetworkRemapPath(LevelVisibility.PackageName, false);
				LevelVisibility.VisibilityRequestId = TransactionId;
				PlayerController->ServerUpdateLevelVisibility(LevelVisibility);
				// No need to loop on all player controllers as ServerUpdateLevelVisibility only needs to be done once per connection
				break;
			}
		}
	}
}

bool ULevelStreaming::ShouldWaitForServerAckBeforeChangingVisibilityState(ENetLevelVisibilityRequest InRequestType)
{
	if (IsWaitingForNetVisibilityTransactionAck(InRequestType))
	{
		if (NetVisibilityState.bHasClientPendingRequest)
		{
			// We have a pending request, IncrementTransactionIndex and send ServerUpdateLevelVisibility request to server
			FNetLevelVisibilityTransactionId TransactionId;
			TransactionId.SetIsClientInstigator(true);
			TransactionId.SetTransactionIndex(NetVisibilityState.ClientPendingRequestIndex);
			NetVisibilityState.ClientAckedRequestIndex = NetVisibilityState.ClientPendingRequestIndex;
			NetVisibilityState.ClientPendingRequestIndex = TransactionId.IncrementTransactionIndex();
			NetVisibilityState.ClientAckedRequestCanMakeVisible.Reset();
			NetVisibilityState.bHasClientPendingRequest = false;

			const bool bIsVisible = false;
			const bool bTryMakeVisible = (InRequestType == ENetLevelVisibilityRequest::MakingVisible);
			ServerUpdateLevelVisibility(bIsVisible, bTryMakeVisible, TransactionId);
			return true;
		}
		else if (NetVisibilityState.ClientPendingRequestIndex != NetVisibilityState.ClientAckedRequestIndex)
		{
			// Wait for server to acknowledge the visibility change
			return true;
		}

		// Invalidate request
		NetVisibilityState.InvalidateClientPendingRequest();
	}
	return false;
};

bool ULevelStreaming::RequestVisibilityChange(bool bVisible)
{
	return true;
}

bool ULevelStreaming::CanMakeInvisible()
{
	// Once the Level becomes the current pending invisibility level, this function must return true or else it could
	// block indefinitely if called inside UWorld::BlockTillLevelStreamingCompleted
	//
	// Here's one example of why this could go wrong :
	//      - The first call to CanMakeInvisible for a level returns true (when normally it would return false)
	//			ShouldWaitForServerAckBeforeChangingVisibilityState would normally return true but
	//			IsWaitingForNetVisibilityTransactionAck returns false only because ServerConnection is different than USOCK_Open 
	//			and IsConcernedByNetVisibilityTransactionAck returns false
	//		- RemoveFromWorld is called on this level and sets CurrentLevelPendingInvisibility
	//		- Because time limit is exceeded, RemoveFromWorld exists with World's CurrentLevelPendingInvisibility 
	//		  still set to this level streaming loaded level
	//		- Since we are inside UWorld::BlockTillLevelStreamingCompleted, IsVisibilityRequestPending returns true because CurrentLevelPendingInvisibility is valid
	//		- UpdateStreamingState is called again
	//		- All future calls to CanMakeInvisible return false (ServerConnection is USOCK_Open) and we are waiting for server ack
	//
	// This is one hypothetical example. Detecting this will prevent any future case that could trigger an infinte loop in BlockTillLevelStreamingCompleted.
	//
	UWorld* World = GetWorld();
	if (World && World->IsGameWorld() && LoadedLevel && World->IsLevelMakingInvisible(LoadedLevel))
	{
		return true;
	}

	const bool bCanMakeInvisible = RequestVisibilityChange(false);

	if (ShouldClientUseMakingInvisibleTransactionRequest())
	{
		if (ShouldWaitForServerAckBeforeChangingVisibilityState(ENetLevelVisibilityRequest::MakingInvisible))
		{
			return false;
		}
	}

	return bCanMakeInvisible;
}

bool ULevelStreaming::CanMakeVisible()
{
	UWorld* World = GetWorld();
	if (World && World->IsGameWorld() && LoadedLevel)
	{
		// Once the Level becomes the current pending visibility level, this function must return true or else it could
		// block indefinitely if called inside UWorld::BlockTillLevelStreamingCompleted (same reason as CanMakeInvisible but with World's CurrentLevelPendingVisibility)
		if (World->IsLevelMakingVisible(LoadedLevel))
		{
			return true;
		}
		// Delay AddToWorld of a partition world if this same partitioned world hasn't finished removing it's sub-levels triggered by a prior RemoveFromWorld
		else if (const UWorldPartition* WorldPartition = LoadedLevel->GetWorldPartition(); WorldPartition && !WorldPartition->CanInitialize(World))
		{
			return false;
		}
	}

	const bool bCanMakeVisible = RequestVisibilityChange(true);

	if (ShouldClientUseMakingVisibleTransactionRequest())
	{
		if (ShouldWaitForServerAckBeforeChangingVisibilityState(ENetLevelVisibilityRequest::MakingVisible))
		{
			return false;
		}
		else if (NetVisibilityState.ClientAckedRequestCanMakeVisible.IsSet() && !NetVisibilityState.ClientAckedRequestCanMakeVisible.GetValue())
		{
			// Server response was negative
			// Until client and server streaming level state matches, client starts another visibilily request to try to make level visible
			check(!IsWaitingForNetVisibilityTransactionAck(ENetLevelVisibilityRequest::MakingVisible));
			BeginClientNetVisibilityRequest(true);
			return false;
		}
	}

	return bCanMakeVisible;
}

void ULevelStreaming::UpdateNetVisibilityTransactionState(bool bInShouldBeVisible, FNetLevelVisibilityTransactionId TransactionId)
{
	UWorld* World = GetWorld();
	if (World && World->IsGameWorld())
	{
		const bool bIsClientTransaction = TransactionId.IsClientTransaction();
		const ENetLevelVisibilityRequest Target = bInShouldBeVisible ? ENetLevelVisibilityRequest::MakingVisible : ENetLevelVisibilityRequest::MakingInvisible;
		// Check if client is already waiting
		if (bIsClientTransaction && IsWaitingForNetVisibilityTransactionAck(Target))
		{
			return;
		}

		NetVisibilityState.InvalidateClientPendingRequest();
		NetVisibilityState.ServerRequestIndex = bIsClientTransaction ? FNetLevelVisibilityTransactionId::InvalidTransactionIndex : TransactionId.GetTransactionIndex();

		if (bIsClientTransaction)
		{
			if (!bInShouldBeVisible && ShouldClientUseMakingInvisibleTransactionRequest())
			{
				// If this is a client request to make invisible, we will conditionally send a notification to the server before we make the level invisible
				if (!LoadedLevel || !World->IsLevelMakingInvisible(LoadedLevel))
				{
					NetVisibilityState.bHasClientPendingRequest = true;
					NetVisibilityState.PendingRequestType = ENetLevelVisibilityRequest::MakingInvisible;
				}
			}
			else if (bInShouldBeVisible && ShouldClientUseMakingVisibleTransactionRequest())
			{
				// If this is a client request to make visible, we will conditionally send a notification to the server before we make the level visible
				if (!LoadedLevel || !World->IsLevelMakingVisible(LoadedLevel))
				{
					NetVisibilityState.bHasClientPendingRequest = true;
					NetVisibilityState.PendingRequestType = ENetLevelVisibilityRequest::MakingVisible;
				}
			}
		}
	}
}

void ULevelStreaming::BeginClientNetVisibilityRequest(bool bInShouldBeVisible)
{
	UWorld* World = GetWorld();
	if (World && World->IsGameWorld() && World->IsNetMode(NM_Client))
	{
		FNetLevelVisibilityTransactionId TransactionId;
		TransactionId.SetIsClientInstigator(true);

		UpdateNetVisibilityTransactionState(bInShouldBeVisible, TransactionId);
	}
}

void ULevelStreaming::AckNetVisibilityTransaction(FNetLevelVisibilityTransactionId InAckedClientTransactionId, bool bInClientAckCanMakeVisible)
{
	if (ensure(NetVisibilityState.ClientPendingRequestIndex != NetVisibilityState.ClientAckedRequestIndex))
	{
		NetVisibilityState.ClientAckedRequestIndex = InAckedClientTransactionId.GetTransactionIndex();

		// If received an ack for MakingVisible, store the server response in ClientAckedRequestCanMakeVisible
		if ((NetVisibilityState.PendingRequestType == ENetLevelVisibilityRequest::MakingVisible) &&
			(NetVisibilityState.ClientPendingRequestIndex == NetVisibilityState.ClientAckedRequestIndex))
		{
			check(ShouldClientUseMakingVisibleTransactionRequest());
			NetVisibilityState.ClientAckedRequestCanMakeVisible = bInClientAckCanMakeVisible;
		}
	}
}

void ULevelStreaming::UpdateStreamingState(bool& bOutUpdateAgain, bool& bOutRedetermineTarget, const TOptional<const UE::FTimeout>& InExternalTimeout)
{
	FScopeCycleCounterUObject ContextScope(this);

	UWorld* World = GetWorld();

	bOutUpdateAgain = false;
	bOutRedetermineTarget = false;

	auto UpdateStreamingState_RequestLevel = [&]()
	{
		bool bBlockOnLoad = (bShouldBlockOnLoad || ShouldBeAlwaysLoaded());
		const bool bAllowLevelLoadRequests = (bBlockOnLoad || World->AllowLevelLoadRequests());
		bBlockOnLoad |= (!GUseBackgroundLevelStreaming || !World->IsGameWorld());

		const ELevelStreamingState PreviousState = CurrentState;

		RequestLevel(World, bAllowLevelLoadRequests, (bBlockOnLoad ? ULevelStreaming::AlwaysBlock : ULevelStreaming::BlockAlwaysLoadedLevelsOnly));

		if (CurrentState != ELevelStreamingState::Loading)
		{
			bOutRedetermineTarget = true;

			if (CurrentState != PreviousState)
			{
				bOutUpdateAgain = true;
			}
		}

		if (LoadedLevel == nullptr)
		{
			DiscardPendingUnloadLevel(World);
		}
	};

	switch(CurrentState)
	{
	case ELevelStreamingState::MakingVisible:
		if (ensure(LoadedLevel))
		{
			// Handle case where MakingVisible is not needed anymore
			if (TargetState == ELevelStreamingTargetState::LoadedNotVisible)
			{
				SetCurrentState(ELevelStreamingState::LoadedNotVisible);
				bOutUpdateAgain = true;
				bOutRedetermineTarget = true;
				// Make sure to update level visibility state (in case the server already acknowledged a Making Visible request)
				if (ShouldClientUseMakingVisibleTransactionRequest())
				{
					ServerUpdateLevelVisibility(false);
				}
			}
			else
			{
				// Only respond with ServerTransactionId if the is the target visibility state is supposed to be visible
				FNetLevelVisibilityTransactionId TransactionId;
				TransactionId.SetTransactionIndex(NetVisibilityState.ServerRequestIndex);

				// Calling CanMakeVisible will trigger a request for visibility if necessary
				if (!CanMakeVisible())
				{
					check(!World->IsLevelMakingVisible(LoadedLevel))
					break;
				}

				if (LevelStreamingCVars::bForceFlushLoadsBeforeAddToWorld)
				{
					// This is intended as a temporary workaround (CORO-8867). A proper fix is being tracked via CORO-8868.
					FlushAsyncLoading();
				}

				World->AddToWorld(LoadedLevel, LevelTransform, !bShouldBlockOnLoad, InExternalTimeout, TransactionId, this);

				if (LoadedLevel->bIsVisible)
				{
					// immediately discard previous level
					DiscardPendingUnloadLevel(World);

					if (World->Scene)
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateLevelStreamingInner_OnLevelAddedToWorld);
						// Notify the new level has been added after the old has been discarded
						World->Scene->OnLevelAddedToWorld(LoadedLevel->GetOutermost()->GetFName(), World, LoadedLevel->bIsLightingScenario);
					}

					SetCurrentState(ELevelStreamingState::LoadedVisible);
					bOutUpdateAgain = true;
					bOutRedetermineTarget = true;
				}
			}
		}
		break;

	case ELevelStreamingState::MakingInvisible:
		if (ensure(LoadedLevel))
		{
			auto RemoveLevelFromScene = [World, this]()
			{
				if (World->Scene)
				{
					World->Scene->OnLevelRemovedFromWorld(LoadedLevel->GetOutermost()->GetFName(), World, LoadedLevel->bIsLightingScenario);
				}
			};

			const bool bWasVisible = LoadedLevel->bIsVisible;

			// We do not want to have any changes in flights when ending play, so before making invisible we wait for server acknowledgment
			if (!CanMakeInvisible())
			{
				check(!World->IsLevelMakingInvisible(LoadedLevel));
				break;
			}

			FNetLevelVisibilityTransactionId TransactionId;
			TransactionId.SetTransactionIndex(NetVisibilityState.ServerRequestIndex);

			// Hide loaded level, incrementally if necessary
			World->RemoveFromWorld(LoadedLevel, !ShouldBlockOnUnload() && World->IsGameWorld(), InExternalTimeout, TransactionId, this);

			// Hide loaded level immediately if bRequireFullVisibilityToRender is set
			const bool LevelBecameInvisible = bWasVisible && !LoadedLevel->bIsVisible;
			if (LoadedLevel->bRequireFullVisibilityToRender && LevelBecameInvisible)
			{
				RemoveLevelFromScene();
			}

			// If the level is now hidden & all components have been removed from the world
			const bool LevelWasRemovedFromWorld = !LoadedLevel->bIsVisible && !LoadedLevel->bIsBeingRemoved;
			if (LevelWasRemovedFromWorld)
			{
				// Remove level from scene if we haven't done it already
				if (!LoadedLevel->bRequireFullVisibilityToRender)
				{
					RemoveLevelFromScene();
				}

				SetCurrentState(ELevelStreamingState::LoadedNotVisible);
				bOutUpdateAgain = true;
				bOutRedetermineTarget = true;
			}
		}
		break;

	case ELevelStreamingState::Loading:
		// Just waiting
		break;

	case ELevelStreamingState::Unloaded:
		
		switch (TargetState)
		{
			case ELevelStreamingTargetState::LoadedNotVisible:
			{
				UpdateStreamingState_RequestLevel();
			}
			break;

			case ELevelStreamingTargetState::UnloadedAndRemoved:
				World->RemoveStreamingLevel(this);
				bOutRedetermineTarget = true;
				break;

			case ELevelStreamingTargetState::Unloaded:
				// This is to support the case where a request to load is followed by another to unload the same level
				// We set bOutRedetermineTarget to true so that FStreamingLevelPrivateAccessor::UpdateTargetState 
				// gets called by UWorld::UpdateLevelStreaming which will return false so that the streaming level
				// gets removed from StreamingLevelsToConsider.
				bOutRedetermineTarget = true;
				break;

			default:
				ensure(false);
		}
		break;

	case ELevelStreamingState::LoadedNotVisible:
		switch (TargetState)
		{
		case ELevelStreamingTargetState::LoadedVisible:
			SetCurrentState(ELevelStreamingState::MakingVisible);
			// Make sure client pending visibility request (if any) matches current state
			NetVisibilityState.InvalidateClientPendingRequest();
			BeginClientNetVisibilityRequest(true);
			bOutUpdateAgain = true;
			break;

		case ELevelStreamingTargetState::Unloaded:
			DiscardPendingUnloadLevel(World);
			ClearLoadedLevel();
			DiscardPendingUnloadLevel(World);

			bOutUpdateAgain = true;
			bOutRedetermineTarget = true;
			break;

		case ELevelStreamingTargetState::LoadedNotVisible:
			if (LoadedLevel && !IsDesiredLevelLoaded())
			{
				// Process PendingUnloadLevel to unblock level streaming state machine (no new request will start while there's a pending level to unload) 
				// This rare case can happen if desired level (typically LODPackage) changed between last RequestLevel call and AsyncLevelLoadComplete completion callback.
				DiscardPendingUnloadLevel(World);
			}

			UpdateStreamingState_RequestLevel();
			
			// This is to fix the Blocking load on a redirected world package
			// When loading a redirected streaming level the state will go from: Unloaded -> LoadedNotVisible
			// This state change will generate a new RequestLevel call which will load the redirected package.
			// In blocking load, loading will be done after the UpdateStreamingState_RequestLevel call and leave us in the LoadedNotVisible (with a loadedlevel) state which would prevent the bOutUpdateAgain from being set to true.
			// So this condition here makes sure that we aren't loading (async) and that we should be visible (LoadedNotVisible isn't our final target).
			// If that is the case we request another update.
			if (CurrentState != ELevelStreamingState::Loading)
			{
				bOutUpdateAgain |= ShouldBeVisible();
			}
			break;

		default:
			ensure(false);
		}

		break;

	case ELevelStreamingState::LoadedVisible:
		switch (TargetState)
		{
		case ELevelStreamingTargetState::LoadedNotVisible:
			SetCurrentState(ELevelStreamingState::MakingInvisible);
			// Make sure client pending visibility request (if any) matches current state
			NetVisibilityState.InvalidateClientPendingRequest();
			BeginClientNetVisibilityRequest(false);
			bOutUpdateAgain = true;
			break;

		case ELevelStreamingTargetState::LoadedVisible:
			UpdateStreamingState_RequestLevel();
			break;

		default:
			ensure(false);
		}

		break;

	case ELevelStreamingState::FailedToLoad:
		bOutRedetermineTarget = true;
		break;

	default:
		ensureMsgf(false, TEXT("Unexpected state in ULevelStreaming::UpdateStreamingState for '%s'. CurrentState='%s' TargetState='%s'"), *GetPathName(), ::EnumToString(CurrentState), ::EnumToString(TargetState));
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const TCHAR* ULevelStreaming::EnumToString(ULevelStreaming::ECurrentState InCurrentState)
{
	return ::EnumToString((ELevelStreamingState)InCurrentState);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const TCHAR* EnumToString(ELevelStreamingState InCurrentState)
{
	switch (InCurrentState)
	{
	case ELevelStreamingState::Removed:
		return TEXT("Removed");
	case ELevelStreamingState::Unloaded:
		return TEXT("Unloaded");
	case ELevelStreamingState::FailedToLoad:
		return TEXT("FailedToLoad");
	case ELevelStreamingState::Loading:
		return TEXT("Loading");
	case ELevelStreamingState::LoadedNotVisible:
		return TEXT("LoadedNotVisible");
	case ELevelStreamingState::MakingVisible:
		return TEXT("MakingVisible");
	case ELevelStreamingState::LoadedVisible:
		return TEXT("LoadedVisible");
	case ELevelStreamingState::MakingInvisible:
		return TEXT("MakingInvisible");
	}
	ensure(false);
	return TEXT("Unknown");
}

const TCHAR* EnumToString(ELevelStreamingTargetState InTargetState)
{
	switch (InTargetState)
	{
	case ELevelStreamingTargetState::Unloaded:
		return TEXT("Unloaded");
	case ELevelStreamingTargetState::UnloadedAndRemoved:
		return TEXT("UnloadedAndRemoved");
	case ELevelStreamingTargetState::LoadedNotVisible:
		return TEXT("LoadedNotVisible");
	case ELevelStreamingTargetState::LoadedVisible:
		return TEXT("LoadedVisible");
	}
	ensure(false);
	return TEXT("Unknown");
}

FName ULevelStreaming::GetLODPackageName() const
{
	if (LODPackageNames.IsValidIndex(LevelLODIndex))
	{
		return LODPackageNames[LevelLODIndex];
	}
	else
	{
		return GetWorldAssetPackageFName();
	}
}

FName ULevelStreaming::GetLODPackageNameToLoad() const
{
	if (LODPackageNames.IsValidIndex(LevelLODIndex))
	{
		return LODPackageNamesToLoad.IsValidIndex(LevelLODIndex) ? LODPackageNamesToLoad[LevelLODIndex] : NAME_None;
	}
	else
	{
		return PackageNameToLoad;
	}
}

#if WITH_EDITOR
void ULevelStreaming::RemoveLevelFromCollectionForReload()
{
	if (LoadedLevel)
	{
		// Remove the loaded level from its current collection, if any.
		if (LoadedLevel->GetCachedLevelCollection())
		{
			LoadedLevel->GetCachedLevelCollection()->RemoveLevel(LoadedLevel);
		}
	}
}

void ULevelStreaming::AddLevelToCollectionAfterReload()
{
	if (LoadedLevel)
	{
		// Remove the loaded level from its current collection, if any.
		if (LoadedLevel->GetCachedLevelCollection())
		{
			LoadedLevel->GetCachedLevelCollection()->RemoveLevel(LoadedLevel);
		}
		// Add this level to the correct collection
		FLevelCollection& LC = GetWorld()->FindOrAddCollectionForLevelStreaming(this);
		LC.AddLevel(LoadedLevel);
	}
}
#endif

FUObjectAnnotationSparse<ULevelStreaming::FLevelAnnotation, false> ULevelStreaming::LevelAnnotations;

void ULevelStreaming::RemoveLevelAnnotation(const ULevel* Level)
{
	ULevelStreaming::LevelAnnotations.RemoveAnnotation(Level);
}

ULevelStreaming* ULevelStreaming::FindStreamingLevel(const ULevel* Level)
{
	ULevelStreaming* FoundLevelStreaming = nullptr;
	if (Level && ensure(IsValid(Level)))
	{
		if (Level->OwningWorld && !Level->IsPersistentLevel())
		{
			FLevelAnnotation LevelAnnotation = LevelAnnotations.GetAnnotation(Level);
			if (LevelAnnotation.LevelStreaming)
			{
				check(LevelAnnotation.LevelStreaming->GetLoadedLevel() == Level);
				FoundLevelStreaming = LevelAnnotation.LevelStreaming;
			}
		}
	}

	return FoundLevelStreaming;
}

void ULevelStreaming::SetLoadedLevel(ULevel* Level)
{ 
	// Pending level should be unloaded at this point
	check(PendingUnloadLevel == nullptr);
	PendingUnloadLevel = LoadedLevel;
	LoadedLevel = Level;
	bHasCachedLoadedLevelPackageName = false;

	// Cancel unloading for this level, in case it was queued for it
	FLevelStreamingGCHelper::CancelUnloadRequest(LoadedLevel);

	// Add this level to the correct collection
	UWorld* World = GetWorld();
	FLevelCollection& LC = World->FindOrAddCollectionForLevelStreaming(this);
	LC.RemoveLevel(PendingUnloadLevel);

	if (PendingUnloadLevel)
	{
		RemoveLevelAnnotation(PendingUnloadLevel);
	}

	if (LoadedLevel)
	{
		LoadedLevel->OwningWorld = World;
		ULevelStreaming::LevelAnnotations.AddAnnotation(LoadedLevel, FLevelAnnotation(this));

		// Remove the loaded level from its current collection, if any.
		if (LoadedLevel->GetCachedLevelCollection())
		{
			LoadedLevel->GetCachedLevelCollection()->RemoveLevel(LoadedLevel);
		}
		LC.AddLevel(LoadedLevel);

		SetCurrentState((LoadedLevel->bIsVisible ? ELevelStreamingState::LoadedVisible : ELevelStreamingState::LoadedNotVisible));
	}
	else
	{
		SetCurrentState(ELevelStreamingState::Unloaded);
	}

	World->UpdateStreamingLevelShouldBeConsidered(this);

	// Virtual call for derived classes to add their logic
	OnLevelLoadedChanged(LoadedLevel);

	if (LoadedLevel)
	{
		LoadedLevel->OnLevelLoaded();
	}
}

void ULevelStreaming::DiscardPendingUnloadLevel(UWorld* PersistentWorld)
{
	if (PendingUnloadLevel)
	{
		if (PendingUnloadLevel->bIsVisible)
		{
			PersistentWorld->RemoveFromWorld(PendingUnloadLevel);
		}

		if (!PendingUnloadLevel->bIsVisible)
		{
			FLevelStreamingGCHelper::RequestUnload(PendingUnloadLevel);
			PendingUnloadLevel = nullptr;
		}
	}
}

bool ULevelStreaming::IsDesiredLevelLoaded() const
{
	if (LoadedLevel)
	{
		const bool bIsGameWorld = GetWorld()->IsGameWorld();
		const FName DesiredPackageName = bIsGameWorld ? GetLODPackageName() : GetWorldAssetPackageFName();
		const FName LoadedLevelPackageName = GetLoadedLevelPackageName();

		return (LoadedLevelPackageName == DesiredPackageName);
	}

	return false;
}

void ULevelStreaming::PrepareLoadedLevel(ULevel* InLevel, UPackage* InLevelPackage, int32 InPIEInstanceID)
{
	check(InLevel);
	UWorld* LevelOwningWorld = InLevel->OwningWorld;

	InLevel->bGarbageCollectionClusteringEnabled = bGarbageCollectionClusteringEnabled;
#if WITH_EDITOR
	InLevel->SetEditorPathOwner(EditorPathOwner.Get());
#endif

	if (ensure(LevelOwningWorld))
	{
		if (LoadedLevel && (LevelOwningWorld->IsLevelMakingVisible(LoadedLevel) || LevelOwningWorld->IsLevelMakingInvisible(LoadedLevel)))
		{
			// We can't change current loaded level if it's still processing visibility request
			// On next UpdateLevelStreaming call this loaded package will be found in memory by RequestLevel function in case visibility request has finished
			UE_LOGF(LogLevelStreaming, Verbose, "Delaying setting result of async load new level %ls, because current loaded level still processing visibility request", *InLevelPackage->GetName());
		}
		else
		{
			check(PendingUnloadLevel == nullptr);

#if WITH_EDITOR
			if (InPIEInstanceID != INDEX_NONE)
			{
				InLevel->FixupForPIE(InPIEInstanceID);
			}
#endif
			SetLoadedLevel(InLevel);
			// Broadcast level loaded event to blueprints
			OnLevelLoaded.Broadcast();
		}
	}

	InLevel->HandleLegacyMapBuildData();

	// Notify the streamer to start building incrementally the level streaming data.
	IStreamingManager::Get().AddLevel(InLevel);

	// Make sure this level will start to render only when it will be fully added to the world
	if (ShouldRequireFullVisibilityToRender())
	{
		InLevel->bRequireFullVisibilityToRender = true;
		// LOD levels should not be visible on server
		if (LODPackageNames.Num() > 0)
		{
			InLevel->bClientOnlyVisible = LODPackageNames.Contains(InLevelPackage->GetFName());
		}
	}

	// Apply streaming level property to level
	InLevel->bClientOnlyVisible |= bClientOnlyVisible;

	// In the editor levels must be in the levels array regardless of whether they are visible or not
	if (ensure(LevelOwningWorld) && LevelOwningWorld->WorldType == EWorldType::Editor)
	{
		LevelOwningWorld->AddLevel(InLevel);
#if WITH_EDITOR
		// We should also at this point, apply the level's editor transform
		if (!InLevel->bAlreadyMovedActors)
		{
			FLevelUtils::ApplyEditorTransform(this, false);
			InLevel->bAlreadyMovedActors = true;
		}
#endif // WITH_EDITOR
	}
}

bool ULevelStreaming::ValidateUniqueWorldAsset(UWorld* PersistentWorld)
{
	// Validate that the streaming level is unique, check for clash with currently loaded streaming levels
	for (ULevelStreaming* OtherLevel : PersistentWorld->GetStreamingLevels())
	{
		if (OtherLevel == nullptr || OtherLevel == this)
		{
			continue;
		}

		const ELevelStreamingState OtherState = OtherLevel->CurrentState;
		if (OtherState == ELevelStreamingState::FailedToLoad || OtherState == ELevelStreamingState::Removed || (OtherState == ELevelStreamingState::Unloaded && (OtherLevel->TargetState == ELevelStreamingTargetState::Unloaded || OtherLevel->TargetState == ELevelStreamingTargetState::UnloadedAndRemoved)))
		{
			// If the other level is neither loaded nor in the process of being loaded, we don't need to consider it
			continue;
		}

		if (OtherLevel->WorldAsset == WorldAsset)
		{
			if (OtherLevel->GetIsRequestingUnloadAndRemoval())
			{
				return false; // Cannot load now, retry until the OtherLevel is done unloading
			}
			else
			{
				UE_LOGF(LogLevelStreaming, Warning, "Streaming Level '%ls' uses same destination for level ('%ls') as '%ls'. Level cannot be loaded again and this StreamingLevel will be flagged as failed to load.", *GetPathName(), *WorldAsset.GetLongPackageName(), *OtherLevel->GetPathName());
				SetCurrentState(ELevelStreamingState::FailedToLoad);
				return false;
			}
		}
	}
	return true;
}

bool ULevelStreaming::RequestLevel(UWorld* PersistentWorld, bool bAllowLevelLoadRequests, EReqLevelBlock BlockPolicy)
{
	// Quit early in case load request already issued
	if (CurrentState == ELevelStreamingState::Loading)
	{
		return true;
	}

	// Previous attempts have failed, no reason to try again
	if (CurrentState == ELevelStreamingState::FailedToLoad)
	{
		return false;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ULevelStreaming_RequestLevel);
	FScopeCycleCounterUObject Context(PersistentWorld);

	// Package name we want to load
	const bool bIsGameWorld = PersistentWorld->IsGameWorld();
	const FName DesiredPackageName = bIsGameWorld ? GetLODPackageName() : GetWorldAssetPackageFName();
	const FName LoadedLevelPackageName = GetLoadedLevelPackageName();

	// Check if currently loaded level is what we want right now
	if (LoadedLevel && LoadedLevelPackageName == DesiredPackageName)
	{
		return true;
	}

	// Can not load new level now, there is still level pending unload
	if (PendingUnloadLevel)
	{
		return false;
	}

	// Can not load new level now either, we're still processing visibility for this one
    if (LoadedLevel && (PersistentWorld->IsLevelMakingVisible(LoadedLevel) || PersistentWorld->IsLevelMakingInvisible(LoadedLevel)))
    {
		UE_LOGF(LogLevelStreaming, Verbose, "Delaying load of new level %ls, because %ls still processing visibility request.", *DesiredPackageName.ToString(), *CachedLoadedLevelPackageName.ToString());
		return false;
	}

	// Validate that our new streaming level is unique, check for clash with currently loaded streaming levels
	if (!ValidateUniqueWorldAsset(PersistentWorld))
	{
		return false;
	}

	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("LevelStreaming - %s"), *GetPathName());

	EPackageFlags PackageFlags = PKG_ContainsMap;
	int32 PIEInstanceID = INDEX_NONE;

	// Try to find the [to be] loaded package.
	UWorld* World = nullptr;
	UPackage* LevelPackage = (UPackage*)StaticFindObjectFast(UPackage::StaticClass(), nullptr, DesiredPackageName, EFindObjectFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);
	
	if (LevelPackage)
	{
		// Find world object and use its PersistentLevel pointer.
		World = UWorld::FindWorldInPackage(LevelPackage);

		// Check for a redirector. Follow it, if found.
		if (!World)
		{
			World = UWorld::FollowWorldRedirectorInPackage(LevelPackage);
			LevelPackage = World ? World->GetOutermost() : nullptr;
		}
	}

	// copy streaming level on demand if we are in PIE
	// (the world is already loaded for the editor, just find it and copy it)
	if ( LevelPackage == nullptr && PersistentWorld->IsPlayInEditor() )
	{
		if (PersistentWorld->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			PackageFlags |= PKG_PlayInEditor;
		}
		PIEInstanceID = PersistentWorld->GetOutermost()->GetPIEInstanceID();

		const FString NonPrefixedLevelName = UWorld::StripPIEPrefixFromPackageName(DesiredPackageName.ToString(), PersistentWorld->StreamingLevelsPrefix);
		UPackage* EditorLevelPackage = FindObjectFast<UPackage>(nullptr, FName(*NonPrefixedLevelName));

		bool bShouldDuplicate = EditorLevelPackage && (BlockPolicy == AlwaysBlock || EditorLevelPackage->IsDirty() || !GEngine->PreferToStreamLevelsInPIE());
		if (bShouldDuplicate)
		{
			// Do the duplication
			UWorld* PIELevelWorld = UWorld::DuplicateWorldForPIE(NonPrefixedLevelName, PersistentWorld);
			if (PIELevelWorld)
			{
				check(PendingUnloadLevel == NULL);
				SetLoadedLevel(PIELevelWorld->PersistentLevel);

				// Broadcast level loaded event to blueprints
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_OnLevelLoaded_Broadcast);
					OnLevelLoaded.Broadcast();
				}

				return true;
			}
			else if (PersistentWorld->WorldComposition == NULL) // In world composition streaming levels are not loaded by default
			{
				if ( bAllowLevelLoadRequests )
				{
					UE_LOGF(LogLevelStreaming, Log, "World to duplicate for PIE '%ls' not found. Attempting load.", *NonPrefixedLevelName);
				}
				else
				{
					UE_LOGF(LogLevelStreaming, Warning, "Unable to duplicate PIE World: '%ls'", *NonPrefixedLevelName);
				}
			}
		}
	}

	// Package is already or still loaded.
	if (LevelPackage)
	{
		if (World != nullptr)
		{
			if (!IsValid(World))
			{
				// We're trying to reload a level that has very recently been marked for garbage collection, it might not have been cleaned up yet
				// So continue attempting to reload the package if possible
				UE_LOGF(LogLevelStreaming, Verbose, "RequestLevel: World is pending kill %ls", *DesiredPackageName.ToString());
				return false;
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (World->PersistentLevel == nullptr)
			{
				UE_LOGF(LogLevelStreaming, Error, "World exists but PersistentLevel doesn't for %ls, most likely caused by reference to world of unloaded level and GC setting reference to null while keeping world object", *World->GetOutermost()->GetName());
				UE_LOGF(LogLevelStreaming, Error, "Most likely caused by reference to world of unloaded level and GC setting reference to null while keeping world object. Referenced by:");

				FReferenceChainSearch::FindAndPrintStaleReferencesToObject(World, UObjectBaseUtility::IsGarbageEliminationEnabled() ? EPrintStaleReferencesOptions::Fatal : (EPrintStaleReferencesOptions::Error | EPrintStaleReferencesOptions::Ensure));

				return false;
			}
#endif
			// Here, if the level should not be re-used, trash the package.
			// This can happen in the case where a streaming level is first removed inside the World Tick,
			// FLevelStreamingGCHelper::RequestUnload will not call PrepareStreamedOutLevelForGC right away 
			// and will process the level in FLevelStreamingGCHelper::OnWorldTickEnd.
			// If we are insde a BlockTillLevelStreamingCompleted (still in the World Tick), the target
			// state of the Level can change to be loaded by a subsystem like UWorldPartitionSubsystem::OnUpdateStreamingState.
			// In this scenario, a request to load the level will happen and the level package will be found in memory.
			if (!ULevelStreaming::ShouldReuseUnloadedButStillAroundLevels(World->PersistentLevel))
			{
				UE_LOGF(LogLevelStreaming, Verbose, "Trying to reuse level %ls but ShouldReuseUnloadedButStillAroundLevels returns false. Trash the current package and reload.", *LevelPackage->GetName());
				TrashObject(LevelPackage);
				return false;
			}

			if (World->PersistentLevel != LoadedLevel)
			{
				// Level already exists but may have the wrong type due to being inactive before, so copy data over
				World->WorldType = PersistentWorld->WorldType;
				World->PersistentLevel->OwningWorld = PersistentWorld;

				PrepareLoadedLevel(World->PersistentLevel, LevelPackage, PIEInstanceID);
			}
			
			return true;
		}
	}

	// Async load package if world object couldn't be found and we are allowed to request a load.
	if (bAllowLevelLoadRequests)
	{
		const FName DesiredPackageNameToLoad = bIsGameWorld ? GetLODPackageNameToLoad() : PackageNameToLoad;
		FString NormalizedPackageName = (DesiredPackageNameToLoad.IsNone() ? DesiredPackageName : DesiredPackageNameToLoad).ToString();
		// The PackageName might be an objectpath; convert it to a packagename if it is not one already
		NormalizedPackageName = FPackageName::ObjectPathToPackageName(NormalizedPackageName);
		FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(NormalizedPackageName);

		SetCurrentState(ELevelStreamingState::Loading);
		OnLoadingStarted();
			
		ULevel::StreamedLevelsOwningWorld.Add(DesiredPackageName, PersistentWorld);
		UWorld::WorldTypePreLoadMap.FindOrAdd(DesiredPackageName) = PersistentWorld->WorldType;

		// Kick off async load request.
		STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, *(FString( TEXT( "RequestLevel - " ) + DesiredPackageName.ToString() )) );
		TRACE_BEGIN_REGION(*WriteToString<256>(TEXT("Load: "), FNameBuilder(DesiredPackageName)), TEXT("LevelStreaming"));

		FLinkerInstancingContext* InstancingContextPtr = nullptr;
#if WITH_EDITOR
		FLinkerInstancingContext InstancingContext;
		if (DesiredPackageName != NAME_None && PackagePath.GetPackageFName() != DesiredPackageName)
		{
			// When loading an instanced package we want to avoid it being processed as an asset so we make sure to set it RF_Transient.
			// If the package is not created here, it will get created by the LoadPackageAsync call. 
			// Gameworld packages (PIE) are already ignored so we can let LoadPackageAsync do its job.
			if (!bIsGameWorld)
			{
				UPackage* NewPackage = CreatePackage(*DesiredPackageName.ToString());
				NewPackage->SetFlags(RF_Transient);
			}

			// When loading an instanced package we need to invoke an instancing context function in case non external actors part of the level are 
			// pulling on external actors.
			const FString ExternalActorsPathStr = ULevel::GetExternalActorsPath(PackagePath.GetPackageName());
			const FString DesiredPackageNameStr = DesiredPackageName.ToString();

			InstancingContext.AddPackageMappingFunc([ExternalActorsPathStr, DesiredPackageNameStr](FName Original)
			{
				const FString OriginalStr = Original.ToString();
				if (OriginalStr.StartsWith(ExternalActorsPathStr))
				{
					return FName(*ULevel::GetExternalActorPackageInstanceName(DesiredPackageNameStr, OriginalStr));
				}
				return Original;
			});

			InstancingContextPtr = &InstancingContext;
		}
#endif
		LastAsyncLoadPackagePath = PackagePath;

		int32 OutAsyncLoadPriority;
		if (!FMath::AddAndCheckForOverflow(LevelStreamingCVars::GLevelStreamingPriorityBias, GetPriority(), OutAsyncLoadPriority))
		{
			OutAsyncLoadPriority = MAX_int32;
		}
		AsyncRequestIDs.Add(LoadPackageAsync(PackagePath, DesiredPackageName, FLoadPackageAsyncDelegate::CreateUObject(this, &ULevelStreaming::AsyncLevelLoadComplete), PackageFlags, PIEInstanceID, OutAsyncLoadPriority, InstancingContextPtr));

		// streamingServer: server loads everything?
		// Editor immediately blocks on load and we also block if background level streaming is disabled.
		if (BlockPolicy == AlwaysBlock || (ShouldBeAlwaysLoaded() && BlockPolicy != NeverBlock))
		{
			if (IsAsyncLoading())
			{
				UE_LOGF(LogStreaming, Display, "ULevelStreaming::RequestLevel(%ls) is flushing async loading", *DesiredPackageName.ToString());
			}

			// Finish all async loading. Since we will clear our requests upon completion of all loads, 
			// we take a copy so FlushAsyncLoading won't touch an invalidated array view
			TArray<int32> LocalRequestIDs = AsyncRequestIDs;
			FlushAsyncLoading(LocalRequestIDs);
		}
	}

	return true;
}

void ULevelStreaming::AsyncLevelLoadComplete(const FName& InPackageName, UPackage* InLoadedPackage, EAsyncLoadingResult::Type Result)
{
	// TODO: Should not set state here so that observers will have access to LoadedLevel
	SetCurrentState(ELevelStreamingState::LoadedNotVisible);
	OnLoadingFinished();
	FPackagePath PackagePath = MoveTemp(LastAsyncLoadPackagePath);

	if (InLoadedPackage)
	{
		UPackage* LevelPackage = InLoadedPackage;
		
		// Try to find a UWorld object in the level package.
		UWorld* World = UWorld::FindWorldInPackage(LevelPackage);

		if (World)
		{
			if (ULevel* Level = World->PersistentLevel)
			{
				PrepareLoadedLevel(Level, LevelPackage, GetOutermost()->GetPIEInstanceID());
			}
			else
			{
				UE_LOGF(LogLevelStreaming, Warning, "Couldn't find ULevel object in package '%ls'", *InPackageName.ToString() );
			}
		}
		else
		{
			// No world in this package
			LevelPackage->ClearPackageFlags(PKG_ContainsMap);

			// There could have been a redirector in the package. Attempt to follow it.
			UObjectRedirector* WorldRedirector = nullptr;
			UWorld* DestinationWorld = UWorld::FollowWorldRedirectorInPackage(LevelPackage, &WorldRedirector);
			if (DestinationWorld)
			{
				// To follow the world redirector for level streaming...
				// 1) Update all globals that refer to the redirector package by name
				// 2) Update the PackageNameToLoad to refer to the new package location
				// 3) If the package name to load was the same as the destination package name...
				//         ... update the package name to the new package and let the next RequestLevel try this process again.
				//    If the package name to load was different...
				//         ... it means the specified package name was explicit and we will just load from another file.

				FName OldDesiredPackageName = InPackageName;
				TWeakObjectPtr<UWorld>* OwningWorldPtr = ULevel::StreamedLevelsOwningWorld.Find(OldDesiredPackageName);
				UWorld* OwningWorld = OwningWorldPtr ? OwningWorldPtr->Get() : NULL;
				ULevel::StreamedLevelsOwningWorld.Remove(OldDesiredPackageName);

				// Try again with the destination package to load.
				// IMPORTANT: check this BEFORE changing PackageNameToLoad, otherwise you wont know if the package name was supposed to be different.
				const bool bLoadingIntoDifferentPackage = (GetWorldAssetPackageFName() != PackageNameToLoad) && (PackageNameToLoad != NAME_None);

				// ... now set PackageNameToLoad
				PackageNameToLoad = DestinationWorld->GetOutermost()->GetFName();

				if ( PackageNameToLoad != OldDesiredPackageName )
				{
					EWorldType::Type* OldPackageWorldType = UWorld::WorldTypePreLoadMap.Find(OldDesiredPackageName);
					if ( OldPackageWorldType )
					{
						UWorld::WorldTypePreLoadMap.FindOrAdd(PackageNameToLoad) = *OldPackageWorldType;
						UWorld::WorldTypePreLoadMap.Remove(OldDesiredPackageName);
					}
				}

				// Now determine if we are loading into the package explicitly or if it is okay to just load the other package.
				if ( bLoadingIntoDifferentPackage )
				{
					// Loading into a new custom package explicitly. Load the destination world directly into the package.
					// Detach the linker to load from a new file into the same package.
					FLinkerLoad* PackageLinker = FLinkerLoad::FindExistingLinkerForPackage(LevelPackage);
					if (PackageLinker)
					{
						PackageLinker->Detach();
						DeleteLoader(PackageLinker);
						PackageLinker = nullptr;
					}

					// Make sure the redirector is not in the way of the new world.
					// Pass NULL as the name to make a new unique name and GetTransientPackage() for the outer to remove it from the package.
					WorldRedirector->Rename(NULL, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);

					// Change the loaded world's type back to inactive since it won't be used.
					DestinationWorld->WorldType = EWorldType::Inactive;
				}
				else
				{
					// Loading the requested package normally. Fix up the destination world then update the requested package to the destination.
					if (OwningWorld)
					{
						if (DestinationWorld->PersistentLevel)
						{
							DestinationWorld->PersistentLevel->OwningWorld = OwningWorld;
						}

						// In some cases, BSP render data is not created because the OwningWorld was not set correctly.
						// Regenerate that render data here
						DestinationWorld->PersistentLevel->InvalidateModelSurface();
						DestinationWorld->PersistentLevel->CommitModelSurfaces();
					}
					
					SetWorldAsset(DestinationWorld);
				}
			}
			else
			{
				UE_LOGF(LogLevelStreaming, Warning, "Failed to find a world in package '%ls'", *InPackageName.ToString());
				SetCurrentState(ELevelStreamingState::FailedToLoad);
				SetShouldBeLoaded(false);
			}
		}
	}
	else if (Result == EAsyncLoadingResult::Canceled)
	{
		// Cancel level streaming
		SetCurrentState(ELevelStreamingState::Unloaded);
		SetShouldBeLoaded(false);
	}
	else
	{
		UE_LOGF(LogLevelStreaming, Warning, "Failed to load package '%ls'", *InPackageName.ToString());
		if (!FPackageName::DoesPackageExist(PackagePath, &PackagePath))
		{
			UE_LOGF(LogLevelStreaming, Error, "Couldn't find file for package %ls.", *PackagePath.GetDebugName());
		}
		SetCurrentState(ELevelStreamingState::FailedToLoad);
 		SetShouldBeLoaded(false);
	}

	// Clean up the world type list and owning world list now that PostLoad has occurred
	UWorld::WorldTypePreLoadMap.Remove(InPackageName);
	ULevel::StreamedLevelsOwningWorld.Remove(InPackageName);

	STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, *(FString( TEXT( "RequestLevelComplete - " ) + InPackageName.ToString() )) );
	TRACE_END_REGION(*WriteToString<256>(TEXT("Load: "), FNameBuilder(InPackageName)));
}

bool ULevelStreaming::IsStreamingStatePending() const
{
	UWorld* PersistentWorld = GetWorld();
	if (PersistentWorld)
	{
		if (IsLevelLoaded() == ShouldBeLoaded() && 
			(IsLevelVisible() == ShouldBeVisible() || !ShouldBeLoaded())) // visibility state does not matter if sub-level set to be unloaded
		{
			const FName DesiredPackageName = PersistentWorld->IsGameWorld() ? GetLODPackageName() : GetWorldAssetPackageFName();
			if (!LoadedLevel || CachedLoadedLevelPackageName == DesiredPackageName)
			{
				return false;
			}
		}
		
		return true;
	}
	
	return false;
}

void ULevelStreaming::SetIsRequestingUnloadAndRemoval(const bool bInIsRequestingUnloadAndRemoval)
{
	if (bInIsRequestingUnloadAndRemoval != bIsRequestingUnloadAndRemoval)
	{
		bIsRequestingUnloadAndRemoval = bInIsRequestingUnloadAndRemoval;
		// Only need to do this if setting to true because if we weren't already being considered and in a transitional state
		// we would have already been removed so it would be irrelevant
		if (bInIsRequestingUnloadAndRemoval)
		{
			if (UWorld* World = GetWorld())
			{
				World->UpdateStreamingLevelShouldBeConsidered(this);
			}
		}
	}
}

#if WITH_EDITOR
void ULevelStreaming::SetShouldBeVisibleInEditor(const bool bInShouldBeVisibleInEditor)
{
	if (bInShouldBeVisibleInEditor != bShouldBeVisibleInEditor)
	{
		bShouldBeVisibleInEditor = bInShouldBeVisibleInEditor;
		if (UWorld* World = GetWorld())
		{
			World->UpdateStreamingLevelShouldBeConsidered(this);
		}
	}
}
#endif

ULevelStreaming* ULevelStreaming::CreateInstance(const FString& InstanceUniqueName)
{
	ULevelStreaming* StreamingLevelInstance = nullptr;
	
	UWorld* InWorld = GetWorld();
	if (InWorld)
	{
		// Create instance long package name 
		FString InstanceShortPackageName = InWorld->StreamingLevelsPrefix + FPackageName::GetShortName(InstanceUniqueName);
		FString InstancePackagePath = FPackageName::GetLongPackagePath(GetWorldAssetPackageName()) + TEXT("/");
		FName	InstanceUniquePackageName = FName(*(InstancePackagePath + InstanceShortPackageName));

		// check if instance name is unique among existing streaming level objects
		const bool bUniqueName = (InWorld->GetStreamingLevels().IndexOfByPredicate(ULevelStreaming::FPackageNameMatcher(InstanceUniquePackageName)) == INDEX_NONE);
				
		if (bUniqueName)
		{
			StreamingLevelInstance = NewObject<ULevelStreaming>(InWorld, GetClass(), NAME_None, RF_Transient, NULL);
			// new level streaming instance will load the same map package as this object
			StreamingLevelInstance->PackageNameToLoad = ((PackageNameToLoad == NAME_None) ? GetWorldAssetPackageFName() : PackageNameToLoad);
			// under a provided unique name

			FSoftObjectPath WorldAssetPath(*WriteToString<512>(InstanceUniquePackageName, TEXT("."), FPackageName::GetShortName(StreamingLevelInstance->PackageNameToLoad)));
			StreamingLevelInstance->SetWorldAsset(TSoftObjectPtr<UWorld>(WorldAssetPath));
			StreamingLevelInstance->SetShouldBeLoaded(false);
			StreamingLevelInstance->SetShouldBeVisible(false);
			StreamingLevelInstance->LevelTransform = LevelTransform;

			// add a new instance to streaming level list
			InWorld->AddStreamingLevel(StreamingLevelInstance);
		}
		else
		{
			UE_LOGF(LogStreaming, Warning, "Provided streaming level instance name is not unique: %ls", *InstanceUniquePackageName.ToString());
		}
	}
	
	return StreamingLevelInstance;
}

void ULevelStreaming::BroadcastLevelLoadedStatus(UWorld* PersistentWorld, FName LevelPackageName, bool bLoaded)
{
	for (ULevelStreaming* StreamingLevel : PersistentWorld->GetStreamingLevels())
	{
		if (StreamingLevel->GetWorldAssetPackageFName() == LevelPackageName)
		{
			if (bLoaded)
			{
				StreamingLevel->OnLevelLoaded.Broadcast();
			}
			else
			{
				StreamingLevel->OnLevelUnloaded.Broadcast();
			}
		}
	}
}
	
void ULevelStreaming::BroadcastLevelVisibleStatus(UWorld* PersistentWorld, FName LevelPackageName, bool bVisible)
{
	TArray<ULevelStreaming*, TInlineAllocator<1>> LevelsToBroadcast;

	for (ULevelStreaming* StreamingLevel : PersistentWorld->GetStreamingLevels())
	{
		if (StreamingLevel->GetWorldAssetPackageFName() == LevelPackageName)
		{
			LevelsToBroadcast.Add(StreamingLevel);
		}
	}

	for (ULevelStreaming* StreamingLevel : LevelsToBroadcast)
	{
		if (bVisible)
		{
			StreamingLevel->OnLevelShown.Broadcast();
		}
		else
		{
			StreamingLevel->OnLevelHidden.Broadcast();
		}
	}
}

void ULevelStreaming::SetWorldAsset(const TSoftObjectPtr<UWorld>& NewWorldAsset)
{
	if (WorldAsset != NewWorldAsset)
	{
		WorldAsset = NewWorldAsset;
		bHasCachedWorldAssetPackageFName = false;
		bHasCachedLoadedLevelPackageName = false;

		if (CurrentState == ELevelStreamingState::FailedToLoad)
		{
			SetCurrentState(ELevelStreamingState::Unloaded);
		}

		if (UWorld* World = GetWorld())
		{
			World->UpdateStreamingLevelShouldBeConsidered(this);
		}
	}
}

FString ULevelStreaming::GetWorldAssetPackageName() const
{
	return GetWorldAssetPackageFName().ToString();
}

FName ULevelStreaming::GetWorldAssetPackageFName() const
{
	if (!bHasCachedWorldAssetPackageFName)
	{
		CachedWorldAssetPackageFName = WorldAsset.ToSoftObjectPath().GetLongPackageFName();
		bHasCachedWorldAssetPackageFName = true;
	}
	return CachedWorldAssetPackageFName;
}

FName ULevelStreaming::GetLoadedLevelPackageName() const
{
	if( !bHasCachedLoadedLevelPackageName )
	{
		CachedLoadedLevelPackageName = (LoadedLevel ? LoadedLevel->GetOutermost()->GetFName() : NAME_None);
		bHasCachedLoadedLevelPackageName = true;
	}

	return CachedLoadedLevelPackageName;
}

void ULevelStreaming::OnLoadingStarted()
{
	UWorld* World = GetWorld();
	if (World && World->IsGameWorld())
	{
		FWorldNotifyStreamingLevelLoading::Started(World);
	}
}

void ULevelStreaming::OnLoadingFinished()
{
	// Clear our request ids as we are done loading
	AsyncRequestIDs.Empty();

	UWorld* World = GetWorld();
	if (World && World->IsGameWorld())
	{
		if (World->GetStreamingLevels().Contains(this))
		{
			FWorldNotifyStreamingLevelLoading::Finished(World);
		}
	}
}

void ULevelStreaming::SetWorldAssetByPackageName(FName InPackageName)
{
	// Need to strip PIE prefix from object name, only the package has it
	const FString TargetWorldPackageName = InPackageName.ToString();
	const FString TargetWorldObjectName = UWorld::RemovePIEPrefix(FPackageName::GetLongPackageAssetName(TargetWorldPackageName));
	TSoftObjectPtr<UWorld> NewWorld;
	NewWorld = FString::Printf(TEXT("%s.%s"), *TargetWorldPackageName, *TargetWorldObjectName);
	SetWorldAsset(NewWorld);
}

void ULevelStreaming::RenameForPIE(int32 PIEInstanceID, bool bKeepWorldAssetName)
{
	const UWorld* const World = GetWorld();

	// Apply PIE prefix so this level references
	if (!WorldAsset.IsNull())
	{
		FName NonPrefixedName = *UWorld::StripPIEPrefixFromPackageName(GetWorldAssetPackageName(), UWorld::BuildPIEPackagePrefix(PIEInstanceID));
		NetDriverRenameStreamingLevelPackageForPIE(World, NonPrefixedName);
		
		// Store original name 
		if (PackageNameToLoad == NAME_None)
		{
			PackageNameToLoad = NonPrefixedName;
		}
		FName PlayWorldStreamingPackageName = FName(*UWorld::ConvertToPIEPackageName(GetWorldAssetPackageName(), PIEInstanceID));
		FSoftObjectPath::AddPIEPackageName(PlayWorldStreamingPackageName);
		if (bKeepWorldAssetName)
		{
			SetWorldAsset(TSoftObjectPtr<UWorld>(FSoftObjectPath(FString::Printf(TEXT("%s.%s"), *PlayWorldStreamingPackageName.ToString(), *FPackageName::ObjectPathToObjectName(WorldAsset.ToString())))));
		}
		else
		{
			SetWorldAssetByPackageName(PlayWorldStreamingPackageName);
		}
	}
	
	// Rename LOD levels if any
	if (LODPackageNames.Num() > 0)
	{
		LODPackageNamesToLoad.Reset(LODPackageNames.Num());
		for (FName& LODPackageName : LODPackageNames)
		{
			// Store LOD level original package name
			LODPackageNamesToLoad.Add(LODPackageName); 
			// Apply PIE prefix to package name			
			const FName NonPrefixedLODPackageName = LODPackageName;
			LODPackageName = FName(*UWorld::ConvertToPIEPackageName(LODPackageName.ToString(), PIEInstanceID));
			FSoftObjectPath::AddPIEPackageName(LODPackageName);

			NetDriverRenameStreamingLevelPackageForPIE(World, NonPrefixedLODPackageName);
		}
	}
}

void ULevelStreaming::SetPriority(const int32 NewPriority)
{
	if (NewPriority != StreamingPriority)
	{
		StreamingPriority = NewPriority;

		if (CurrentState != ELevelStreamingState::Removed && CurrentState != ELevelStreamingState::FailedToLoad)
		{
			if (UWorld* World = GetWorld())
			{
				World->UpdateStreamingLevelPriority(this);
			}
		}
	}
}

void ULevelStreaming::SetPriorityOverride(const int32 PriorityOverride)
{
	if (!StreamingPriorityOverrideBackup.IsSet())
	{
		StreamingPriorityOverrideBackup = StreamingPriority;
	}

	SetPriority(PriorityOverride);
}

void ULevelStreaming::ResetPriorityOverride()
{
	if (StreamingPriorityOverrideBackup.IsSet())
	{
		SetPriority(StreamingPriorityOverrideBackup.GetValue());
		StreamingPriorityOverrideBackup.Reset();
	}
}

void ULevelStreaming::SetLevelLODIndex(const int32 LODIndex)
{
	if (LODIndex != LevelLODIndex)
	{
		LevelLODIndex = LODIndex;

		if (CurrentState == ELevelStreamingState::FailedToLoad)
		{
			SetCurrentState(ELevelStreamingState::Unloaded);
		}

		if (UWorld* World = GetWorld())
		{
			World->UpdateStreamingLevelShouldBeConsidered(this);
		}
	}
}

void ULevelStreaming::SetShouldBeVisible(const bool bInShouldBeVisible)
{
	if (bInShouldBeVisible != bShouldBeVisible)
	{
		bShouldBeVisible = bInShouldBeVisible;
		if (UWorld* World = GetWorld())
		{
			World->UpdateStreamingLevelShouldBeConsidered(this);
		}
	}
}

void ULevelStreaming::SetShouldBeLoaded(const bool bInShouldBeLoaded)
{
}

bool ULevelStreaming::ShouldBeVisible() const
{
	if( GetWorld()->IsGameWorld() )
	{
		// Game and play in editor viewport codepath.
		return bShouldBeVisible && ShouldBeLoaded();
	}
#if WITH_EDITORONLY_DATA
	// Editor viewport codepath.
	return bShouldBeVisibleInEditor;
#else
	return false;
#endif
}

FBox ULevelStreaming::GetStreamingVolumeBounds()
{
	FBox Bounds(ForceInit);

	// Iterate over each volume associated with this LevelStreaming object
	for(int32 VolIdx=0; VolIdx<EditorStreamingVolumes.Num(); VolIdx++)
	{
		ALevelStreamingVolume* StreamingVol = EditorStreamingVolumes[VolIdx];
		if(StreamingVol && StreamingVol->GetBrushComponent())
		{
			Bounds += StreamingVol->GetBrushComponent()->BrushBodySetup->AggGeom.CalcAABB(StreamingVol->GetBrushComponent()->GetComponentTransform());
		}
	}

	return Bounds;
}

#if WITH_EDITOR
void ULevelStreaming::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* OutermostProperty = PropertyChangedEvent.Property;
	if ( OutermostProperty != NULL )
	{
		const FName PropertyName = OutermostProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelStreaming, LevelTransform))
		{
			GetWorld()->UpdateLevelStreaming();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelStreaming, EditorStreamingVolumes))
		{
			RemoveStreamingVolumeDuplicates();

			// Update levels references in each streaming volume 
			for (TActorIterator<ALevelStreamingVolume> It(GetWorld()); It; ++It)
			{
				(*It)->UpdateStreamingLevelsRefs();
			}
		}

		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelStreaming, LevelColor))
		{
			// Make sure the level's Level Color change is applied immediately by reregistering the
			// components of the actor's in the level
			if (LoadedLevel != nullptr)
			{
				LoadedLevel->MarkLevelComponentsRenderStateDirty();
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelStreaming, WorldAsset))
		{
			bHasCachedWorldAssetPackageFName = false;
			bHasCachedLoadedLevelPackageName = false;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULevelStreaming, bIsStatic))
		{
			UWorld* World = GetWorld();
			if (LoadedLevel && World)
			{
				FLevelCollection* PreviousCollection = LoadedLevel->GetCachedLevelCollection();
				FLevelCollection& LC = World->FindOrAddCollectionForLevelStreaming(this);

				if (PreviousCollection && PreviousCollection != &LC)
				{
					PreviousCollection->RemoveLevel(LoadedLevel);
					LC.AddLevel(LoadedLevel);
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ULevelStreaming::RemoveStreamingVolumeDuplicates()
{
	for (int32 VolumeIdx = EditorStreamingVolumes.Num()-1; VolumeIdx >= 0; VolumeIdx--)
	{
		ALevelStreamingVolume* Volume = EditorStreamingVolumes[VolumeIdx];
		if (Volume) // Allow duplicate null entries, for array editor convenience
		{
			int32 DuplicateIdx = EditorStreamingVolumes.Find(Volume);
			check(DuplicateIdx != INDEX_NONE);
			if (DuplicateIdx != VolumeIdx)
			{
				EditorStreamingVolumes.RemoveAt(VolumeIdx);
			}
		}
	}
}

#endif // WITH_EDITOR

ALevelScriptActor* ULevelStreaming::GetLevelScriptActor()
{
	if (LoadedLevel)
	{
		return LoadedLevel->GetLevelScriptActor();
	}
	return nullptr;
}

bool ULevelStreaming::IsValidStreamingLevel() const
{
	const bool PIESession = GetWorld()->WorldType == EWorldType::PIE || GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
	if (!PIESession && !WorldAsset.IsNull())
	{
		FName WorldPackageName = GetWorldAssetPackageFName();

		if (UPackage* WorldPackage = FindObjectFast<UPackage>(nullptr, WorldPackageName))
		{
			if (FLinkerLoad* Linker = WorldPackage->GetLinker())
			{
				/**
				 * This support packages that were or will be instanced on load.
				 * This might be redundant but it avoid changing the behavior of this function where a loaded package can still fail 
				 * if it doesn't have on disk file associated to it.
				 */
				return FPackageName::DoesPackageExist(Linker->GetPackagePath());
			}
		} 
		
		// Handle unloaded instanced package
		if (PackageNameToLoad != NAME_None && WorldPackageName != PackageNameToLoad)
		{
			WorldPackageName = PackageNameToLoad;
		}

		FPackagePath WorldPackagePath;
		if (!FPackagePath::TryFromPackageName(WorldPackageName, /* Out*/ WorldPackagePath))
		{
			return false;
		}
				
		return FPackageName::DoesPackageExist(WorldPackagePath);
	}
	return true;
}

EStreamingStatus ULevelStreaming::GetLevelStreamingStatus() const
{
	// This function can be called from an async task (when wp.runtime.UpdateStreaming.EnableAsyncUpdate = true)
	ELevelStreamingState LocalState = GetLevelStreamingState();
	if (LocalState == ELevelStreamingState::FailedToLoad)
	{
		return LEVEL_FailedToLoad;
	}
	else if (LocalState == ELevelStreamingState::MakingInvisible)
	{
		return LEVEL_MakingInvisible;
	}
	else if (LoadedLevel)
	{
		if (LocalState == ELevelStreamingState::LoadedVisible)
		{
			return LEVEL_Visible;
		}
		else if ((LocalState == ELevelStreamingState::MakingVisible) && (GetWorld()->IsLevelMakingVisible(LoadedLevel)))
		{
			return LEVEL_MakingVisible;
		}
		return LEVEL_Loaded;
	}
	else
	{
		if (LocalState == ELevelStreamingState::Loading)
		{
			return LEVEL_Loading;
		}

		check(LocalState == ELevelStreamingState::Removed || LocalState == ELevelStreamingState::Unloaded);

		// We can't use FindObjectFast() while the GC is running... Let's treat the level as unloaded at this point.
		if (IsGarbageCollecting())
		{
			return LEVEL_Unloaded;
		}

		// See whether the level's world object is still around.
		UPackage* LevelPackage = FindObjectFast<UPackage>(nullptr, GetWorldAssetPackageFName());
		UWorld* LevelWorld = LevelPackage ? UWorld::FindWorldInPackage(LevelPackage) : nullptr;
		return LevelWorld ? LEVEL_UnloadedButStillAround : LEVEL_Unloaded;
	}
}

/** Utility that gets a color for a particular level status */
FColor ULevelStreaming::GetLevelStreamingStatusColor(EStreamingStatus Status)
{
	switch (Status)
	{
	case LEVEL_Unloaded: return FColor::Red;
	case LEVEL_UnloadedButStillAround: return FColor::Purple;
	case LEVEL_Loading: return FColor::Yellow;
	case LEVEL_Loaded: return FColor::Cyan;
	case LEVEL_MakingVisible: return FColor::Blue;
	case LEVEL_Visible: return FColor::Green;
	case LEVEL_Preloading: return FColor::Magenta;
	case LEVEL_FailedToLoad: return FColorList::Maroon;
	case LEVEL_MakingInvisible: return FColorList::Orange;
	default: return FColor::White;
	};
}

const TCHAR* ULevelStreaming::GetLevelStreamingStatusDisplayName(EStreamingStatus Status)
{
	switch (Status)
	{
	case LEVEL_Unloaded: return TEXT("Unloaded");
	case LEVEL_UnloadedButStillAround: return TEXT("Unloaded Still Around");
	case LEVEL_Loading: return TEXT("Loading");
	case LEVEL_Loaded: return TEXT("Loaded Not Visible");
	case LEVEL_MakingVisible: return TEXT("Making Visible");
	case LEVEL_Visible: return TEXT("Loaded Visible");
	case LEVEL_Preloading: return TEXT("Preloading");
	case LEVEL_FailedToLoad: return TEXT("Failed to Load");
	case LEVEL_MakingInvisible: return TEXT("Making Invisible");
	default: return TEXT("Unknown");
	};
}

#if WITH_EDITOR
void ULevelStreaming::PostEditUndo()
{
	Super::PostEditUndo();

	if (UWorld* World = GetWorld())
	{
		World->UpdateStreamingLevelShouldBeConsidered(this);
	}
}

TOptional<FFolder::FRootObject> ULevelStreaming::GetFolderRootObject() const
{ 
	// We consider that if either the loaded level or its world persistent level uses actor folder objects, the loaded level is the folder root object.
	if (LoadedLevel)
	{
		if (LoadedLevel->IsUsingActorFolders() || LoadedLevel->GetWorld()->PersistentLevel->IsUsingActorFolders())
		{
			return FFolder::FRootObject(LoadedLevel);
		}
		else
		{
			return FFolder::GetWorldRootFolder(LoadedLevel->GetWorld()).GetRootObject();
		}
	}
	return TOptional<FFolder::FRootObject>();
}

const FName& ULevelStreaming::GetFolderPath() const
{
	return FolderPath;
}

void ULevelStreaming::SetFolderPath(const FName& InFolderPath)
{
	if (FolderPath != InFolderPath)
	{
		Modify();

		FolderPath = InFolderPath;

		// @TODO: Should this be broadcasted through the editor, similar to BroadcastLevelActorFolderChanged?
	}
}
#endif	// WITH_EDITOR

/*-----------------------------------------------------------------------------
	ULevelStreamingPersistent implementation.
-----------------------------------------------------------------------------*/
ULevelStreamingPersistent::ULevelStreamingPersistent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	ULevelStreamingDynamic implementation.
-----------------------------------------------------------------------------*/
ULevelStreamingDynamic::ULevelStreamingDynamic(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULevelStreamingDynamic::PostLoad()
{
	Super::PostLoad();

	// Initialize startup state of the streaming level
	if ( GetWorld()->IsGameWorld() )
	{
		SetShouldBeLoaded(bInitiallyLoaded);
		SetShouldBeVisible(bInitiallyVisible);
	}
}

void ULevelStreamingDynamic::SetShouldBeLoaded(const bool bInShouldBeLoaded)
{
	if (bInShouldBeLoaded != bShouldBeLoaded)
	{
		bShouldBeLoaded = bInShouldBeLoaded;
		if (UWorld* World = GetWorld())
		{
			World->UpdateStreamingLevelShouldBeConsidered(this);
		}
	}
}

ULevelStreamingDynamic* ULevelStreamingDynamic::LoadLevelInstance(UObject* WorldContextObject, const FString LevelObjectPath, const FVector Location, const FRotator Rotation, bool& bOutSuccess, const FString& OptionalLevelNameOverride, TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass, bool bLoadAsTempPackage)
{
	bOutSuccess = false;
	UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return nullptr;
	}

	// Check whether requested map exists
	// LevelObjectPath may be either an objectpath, a package name, or a file path; and packagename/object path may be a LongPackageName or ShortPackageName
	// convert that flexible input to ObjectPath with a LongPackageName
	FString PackageName;
	FString ObjectRelativePath;
	if (FPackageName::IsShortPackageName(LevelObjectPath) || FPackageName::IsValidObjectPath(LevelObjectPath))
	{
		FString UnusedClassName;
		FString ObjectName;
		FString SubObjectName;
		FPackageName::SplitFullObjectPath(LevelObjectPath, UnusedClassName, PackageName, ObjectName, SubObjectName);
	}
	else if (!FPackageName::TryConvertFilenameToLongPackageName(LevelObjectPath, PackageName))
	{
		// An unrecognized path or format
		return nullptr;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	FName ExistingPackageName = AssetRegistry.GetFirstPackageByName(PackageName);
	bOutSuccess = !ExistingPackageName.IsNone();
	if (!bOutSuccess)
	{
		return nullptr;
	}
	const FString LongPackageName = ExistingPackageName.ToString();
	FLoadLevelInstanceParams Params(World, LongPackageName, FTransform(Rotation, Location));
	Params.OptionalLevelNameOverride = OptionalLevelNameOverride.IsEmpty() ? nullptr : &OptionalLevelNameOverride;
	Params.OptionalLevelStreamingClass = OptionalLevelStreamingClass;
	Params.bLoadAsTempPackage = bLoadAsTempPackage;
	return LoadLevelInstance(Params, bOutSuccess);
}

ULevelStreamingDynamic* ULevelStreamingDynamic::LoadLevelInstanceBySoftObjectPtr(UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, const FVector Location, const FRotator Rotation, bool& bOutSuccess, const FString& OptionalLevelNameOverride, TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass, bool bLoadAsTempPackage)
{
	return LoadLevelInstanceBySoftObjectPtr(WorldContextObject, Level, FTransform(Rotation, Location), bOutSuccess, OptionalLevelNameOverride, OptionalLevelStreamingClass, bLoadAsTempPackage);
}

ULevelStreamingDynamic* ULevelStreamingDynamic::LoadLevelInstanceBySoftObjectPtr(UObject* WorldContextObject, const TSoftObjectPtr<UWorld> Level, const FTransform LevelTransform, bool& bOutSuccess, const FString& OptionalLevelNameOverride, TSubclassOf<ULevelStreamingDynamic> OptionalLevelStreamingClass, bool bLoadAsTempPackage)
{
	bOutSuccess = false;
	if (Level.IsNull())
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	FLoadLevelInstanceParams Params(World, Level.GetLongPackageName(), LevelTransform);
	Params.OptionalLevelNameOverride = OptionalLevelNameOverride.IsEmpty() ? nullptr : &OptionalLevelNameOverride;
	Params.OptionalLevelStreamingClass = OptionalLevelStreamingClass;
	Params.bLoadAsTempPackage = bLoadAsTempPackage;
	return LoadLevelInstance(Params, bOutSuccess);
}

ULevelStreamingDynamic* ULevelStreamingDynamic::LoadLevelInstance(const FLoadLevelInstanceParams& Params, bool& bOutSuccess)
{
	bOutSuccess = false;
	if (!Params.World)
	{
		return nullptr;
	}

	if (Params.LongPackageName.IsEmpty())
	{
		return nullptr;
	}

	return LoadLevelInstance_Internal(Params, bOutSuccess);
}

ULevelStreamingDynamic* ULevelStreamingDynamic::LoadLevelInstance_Internal(const FLoadLevelInstanceParams& Params, bool& bOutSuccess)
{
	const FString PackagePath = FPackageName::GetLongPackagePath(Params.LongPackageName);
	FString ShortPackageName = FPackageName::GetShortName(Params.LongPackageName);
	UClass* LevelStreamingClass = Params.OptionalLevelStreamingClass != nullptr ? Params.OptionalLevelStreamingClass.Get() : ULevelStreamingDynamic::StaticClass();

	if (ShortPackageName.StartsWith(Params.World->StreamingLevelsPrefix))
	{
		ShortPackageName.RightChopInline(Params.World->StreamingLevelsPrefix.Len(), EAllowShrinking::No);
	}

	// Remove PIE prefix if it's there before we actually load the level
	const FString OnDiskPackageName = PackagePath + TEXT("/") + ShortPackageName;

	// Determine loaded package name
	const FString LevelPackageNameStr(GetLevelInstancePackageName(Params));

	const bool bNeedsUniqueTest = Params.OptionalLevelNameOverride != nullptr;

	const FName UnmodifiedLevelPackageName = FName(*LevelPackageNameStr);
#if WITH_EDITOR
	const bool bIsPlayInEditor = Params.World->IsPlayInEditor();
	int32 PIEInstance = INDEX_NONE;
	if (bIsPlayInEditor)
	{
		const FWorldContext& WorldContext = GEngine->GetWorldContextFromWorldChecked(Params.World);
		PIEInstance = WorldContext.PIEInstance;
	}
#endif
	if (bNeedsUniqueTest)
	{
		FName ModifiedLevelPackageName = UnmodifiedLevelPackageName;
#if WITH_EDITOR
		if (bIsPlayInEditor)
		{
			ModifiedLevelPackageName = FName(*UWorld::ConvertToPIEPackageName(LevelPackageNameStr, PIEInstance));
		}
#endif
		// Test if the streaming level already exists
		if (ULevelStreaming* const* ExistingLevelStreaming = Params.World->GetStreamingLevels().FindByPredicate([&ModifiedLevelPackageName](ULevelStreaming* LS) { return LS && LS->GetWorldAssetPackageFName() == ModifiedLevelPackageName; }))
		{
			// Allow reusing a streaming level only if :
			// - Params.bAllowReuseExitingLevelStreaming is true
			// - Params.World is a game world
			// - Existing LevelStreaming has the same Class
			// - Existing LevelStreaming has the same Level Transform
			ULevelStreamingDynamic* StreamingLevel = Params.bAllowReuseExitingLevelStreaming && Params.World->IsGameWorld() ? Cast<ULevelStreamingDynamic>(*ExistingLevelStreaming) : nullptr;
			if (StreamingLevel && 
				StreamingLevel->GetClass() == LevelStreamingClass &&
				StreamingLevel->LevelTransform.Equals(Params.LevelTransform))
			{
				bOutSuccess = true;
				StreamingLevel->SetShouldBeLoaded(true);
				StreamingLevel->SetShouldBeVisible(Params.bInitiallyVisible);
				StreamingLevel->SetIsRequestingUnloadAndRemoval(false);
				UE_LOGF(LogLevelStreaming, Verbose, "LoadLevelInstance found existing StreamingLevel for LevelPackageName:%ls", *ModifiedLevelPackageName.ToString());
				return StreamingLevel;
			}
			else
			{
				UE_LOGF(LogLevelStreaming, Error, "LoadLevelInstance called with a name that already exists, returning nullptr. LevelPackageName:%ls", *ModifiedLevelPackageName.ToString());
				return nullptr;
			}
		}
	}
    
	// Setup streaming level object that will load specified map
	ULevelStreamingDynamic* StreamingLevel = NewObject<ULevelStreamingDynamic>(Params.World, LevelStreamingClass, NAME_None, RF_Transient, NULL);
	if (Params.LevelStreamingCreatedCallback)
	{
		Params.LevelStreamingCreatedCallback(StreamingLevel);
	}

	FSoftObjectPath WorldAssetPath(*WriteToString<512>(UnmodifiedLevelPackageName, TEXT("."), ShortPackageName));
    StreamingLevel->SetWorldAsset(TSoftObjectPtr<UWorld>(WorldAssetPath));
#if WITH_EDITOR
	if (bIsPlayInEditor)
	{
		// Necessary for networking in PIE
		StreamingLevel->RenameForPIE(PIEInstance);
	}
#endif // WITH_EDITOR
    StreamingLevel->LevelColor = FColor::MakeRandomColor();
    StreamingLevel->SetShouldBeLoaded(true);
    StreamingLevel->SetShouldBeVisible(Params.bInitiallyVisible);
    StreamingLevel->bShouldBlockOnLoad = false;
    StreamingLevel->bInitiallyLoaded = true;
    StreamingLevel->bInitiallyVisible = Params.bInitiallyVisible;
	// Transform
    StreamingLevel->LevelTransform = Params.LevelTransform;
	// Map to Load
    StreamingLevel->PackageNameToLoad = FName(*OnDiskPackageName);
#if WITH_EDITOR
	StreamingLevel->EditorPathOwner = Params.EditorPathOwner;
#endif
    // Add the new level to world.
    Params.World->AddStreamingLevel(StreamingLevel);
      
	bOutSuccess = true;
    return StreamingLevel;
}	

FString ULevelStreamingDynamic::GetLevelInstancePackageName(const FLoadLevelInstanceParams& Params)
{
	const FString PackagePath = FPackageName::GetLongPackagePath(Params.LongPackageName);
	FString ShortPackageName = FPackageName::GetShortName(Params.LongPackageName);

	if (ShortPackageName.StartsWith(Params.World->StreamingLevelsPrefix))
	{
		ShortPackageName.RightChopInline(Params.World->StreamingLevelsPrefix.Len(), EAllowShrinking::No);
	}

	// Remove PIE prefix if it's there before we actually load the level
	const FString OnDiskPackageName = PackagePath + TEXT("/") + ShortPackageName;

	// Determine loaded package name
	TStringBuilder<512> LevelPackageNameStrBuilder;
	if (Params.bLoadAsTempPackage)
	{
		LevelPackageNameStrBuilder.Append(TEXT("/Temp"));
	}
	LevelPackageNameStrBuilder.Append(PackagePath);
	LevelPackageNameStrBuilder.Append(TEXT("/"));
		
	if (Params.OptionalLevelNameOverride)
	{
		// Use the supplied suffix, which is expected to result in a unique package name but we have to check if it is not.
		LevelPackageNameStrBuilder.Append(*Params.OptionalLevelNameOverride);
	}
	else
	{
		LevelPackageNameStrBuilder.Append(ShortPackageName);
		LevelPackageNameStrBuilder.Append(TEXT("_LevelInstance_"));
		LevelPackageNameStrBuilder.Append(FString::FromInt(++UniqueLevelInstanceId));
	}

	return LevelPackageNameStrBuilder.ToString();
}

/*-----------------------------------------------------------------------------
	ULevelStreamingAlwaysLoaded implementation.
-----------------------------------------------------------------------------*/

ULevelStreamingAlwaysLoaded::ULevelStreamingAlwaysLoaded(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetShouldBeVisible(true);
}

void ULevelStreamingAlwaysLoaded::GetPrestreamPackages(TArray<UObject*>& OutPrestream)
{
	OutPrestream.Add(GetLoadedLevel()); // Nulls will be ignored later
}

#undef LOCTEXT_NAMESPACE

