// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LevelStreamingGCHelper.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/CoreSettings.h"
#include "Engine/NetDriver.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobalsInternal.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/UObjectIterator.h"

/************************************************************************/
/* FLevelStreamingGCHelper implementation                               */
/************************************************************************/
FLevelStreamingGCHelper::FOnGCStreamedOutLevelsEvent FLevelStreamingGCHelper::OnGCStreamedOutLevels;
TArray<TWeakObjectPtr<ULevel> > FLevelStreamingGCHelper::LevelsPendingUnload;
int32 FLevelStreamingGCHelper::NumberOfPreparedStreamedOutActorsForGC = 0;
int32 FLevelStreamingGCHelper::ActorsPendingUnload = 0;

TSet<FName> FLevelStreamingGCHelper::LevelPackageNames;
bool FLevelStreamingGCHelper::bEnabledForCommandlet = false;
bool FLevelStreamingGCHelper::bIsPrepareStreamedOutLevelForGCDelayedToWorldTickEnd = false;
int32 FLevelStreamingGCHelper::NumberOfPreparedStreamedOutLevelsForGC = 0;

void FLevelStreamingGCHelper::AddGarbageCollectorCallback()
{
	// Only register for garbage collection once
	static bool GarbageCollectAdded = false;
	if ( GarbageCollectAdded == false )
	{
		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddStatic(FLevelStreamingGCHelper::OnPreGarbageCollect);
		FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(FLevelStreamingGCHelper::VerifyLevelsGotRemovedByGC);
		FWorldDelegates::OnWorldTickEnd.AddStatic(FLevelStreamingGCHelper::OnWorldTickEnd);
		GarbageCollectAdded = true;
	}
}

void FLevelStreamingGCHelper::EnableForCommandlet()
{
	check(IsRunningCommandlet());
	bEnabledForCommandlet = true;
}

void FLevelStreamingGCHelper::RequestUnload( ULevel* InLevel )
{
	auto IsInsideWorldTick = []()
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->bInTick)
			{
				return true;
			}
		}
		return false;
	};

	if (!IsRunningCommandlet() || bEnabledForCommandlet)
	{
		check( InLevel );
		check( InLevel->bIsVisible == false );
		
		if (!ULevelStreaming::ShouldReuseUnloadedButStillAroundLevels(InLevel))
		{
			if (!IsInsideWorldTick())
			{
				// Prepare unloaded level
				PrepareStreamedOutLevelForGC(InLevel);
			}
			else
			{
				// Since PrepareStreamedOutLevelForGC cannot be called inside a world tick, delay it to the next OnWorldTickEnd
				// TODO: Investigate why NotifyStreamingLevelUnload can't be called inside world's tick
				bIsPrepareStreamedOutLevelForGCDelayedToWorldTickEnd = true;
				if (LevelsPendingUnload.Find(InLevel) == INDEX_NONE)
				{
					LevelsPendingUnload.Add(InLevel);
					InLevel->NumActorsMarkedForPendingUnloadByGC = InLevel->Actors.Num();
					ActorsPendingUnload += InLevel->NumActorsMarkedForPendingUnloadByGC;
				}
			}
		}
		else
		{
			if (LevelsPendingUnload.Find(InLevel) == INDEX_NONE)
			{
				LevelsPendingUnload.Add(InLevel);
				InLevel->NumActorsMarkedForPendingUnloadByGC = InLevel->Actors.Num();
				ActorsPendingUnload += InLevel->NumActorsMarkedForPendingUnloadByGC;
			}
		}
	}
}

void FLevelStreamingGCHelper::CancelUnloadRequest( ULevel* InLevel )
{
	if (InLevel && LevelsPendingUnload.Remove(InLevel) && InLevel->NumActorsMarkedForPendingUnloadByGC > 0)
	{
		ActorsPendingUnload -= InLevel->NumActorsMarkedForPendingUnloadByGC;
		check(ActorsPendingUnload >= 0);
		InLevel->NumActorsMarkedForPendingUnloadByGC = 0;
	}
}

void FLevelStreamingGCHelper::OnWorldTickEnd(UWorld* InWorld, ELevelTick InTickType, float InDeltaSeconds)
{
	if (bIsPrepareStreamedOutLevelForGCDelayedToWorldTickEnd)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLevelStreamingGCHelper::OnWorldTickEnd);
		bIsPrepareStreamedOutLevelForGCDelayedToWorldTickEnd = false;
		PrepareStreamedOutLevelsForGC();
	}
}

void FLevelStreamingGCHelper::OnPreGarbageCollect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLevelStreamingGCHelper::OnPreGarbageCollect);
	if (GetNumLevelsPendingPurge() > 0)
	{
		OnGCStreamedOutLevels.Broadcast();
	}

	PrepareStreamedOutLevelsForGC();
}

void FLevelStreamingGCHelper::PrepareStreamedOutLevelsForGC()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLevelStreamingGCHelper::PrepareStreamedOutLevelsForGC);

	// Iterate over all level objects that want to be unloaded.
	for( int32 LevelIndex=0; LevelIndex < LevelsPendingUnload.Num(); LevelIndex++ )
	{
		ULevel* Level = LevelsPendingUnload[LevelIndex].Get();
		if (Level && Level->NumActorsMarkedForPendingUnloadByGC > 0)
		{
			// Reset cached NumActorsMarkedForPendingUnloadByGC for level.
			Level->NumActorsMarkedForPendingUnloadByGC = 0;
		}
		PrepareStreamedOutLevelForGC(Level);
	}

	LevelsPendingUnload.Empty();
	NumberOfPreparedStreamedOutLevelsForGC = 0;
	NumberOfPreparedStreamedOutActorsForGC = 0;
	ActorsPendingUnload = 0;
}

void FLevelStreamingGCHelper::PrepareStreamedOutLevelForGC(ULevel* InLevel)
{
	if (InLevel && ((!GIsEditor || bEnabledForCommandlet) || InLevel->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor)))
	{
		++NumberOfPreparedStreamedOutLevelsForGC;
		NumberOfPreparedStreamedOutActorsForGC += InLevel->Actors.Num();

		UPackage* LevelPackage = InLevel->GetOutermost();
		UE_LOGF(LogStreaming, Log, "PrepareStreamedOutLevelsForGC called on '%ls'", *LevelPackage->GetName());

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World)
			{
				// This can never ever be called during tick; same goes for GC in general.
				check(!World->bInTick);

				FWorldContext& MutableContext = GEngine->GetWorldContextFromWorldChecked(World);
				for (FNamedNetDriver& Driver : MutableContext.ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr)
					{
						// The net driver must remove this level and its actors from the packagemap, or else
						// the client package map will keep hard refs to them and prevent GC
						Driver.NetDriver->NotifyStreamingLevelUnload(InLevel);
					}
				}

				// Broadcast level unloaded event to blueprints through level streaming objects
				ULevelStreaming::BroadcastLevelLoadedStatus(World, LevelPackage->GetFName(), false);
			}
		}

		InLevel->CleanupLevel();

		// Make sure that this package has been unloaded after GC pass.
		LevelPackageNames.Add(InLevel->GetPackage()->GetFName());

		// Mark world and all other package subobjects as pending kill
		// This will destroy metadata objects and any other objects left behind
		TSet<UPackage*> Packages;
		ForEachObjectWithOuter(InLevel->GetOutermostObject(), [&Packages](UObject* Object)
		{
			bool bIsAlreadyInSet = false;
			UPackage* Package = Object->GetPackage();
			Packages.Add(Package, &bIsAlreadyInSet);
			if (!bIsAlreadyInSet)
			{
				ForEachObjectWithPackage(Package, [](UObject* PackageObject)
				{
					PackageObject->ClearFlags(RF_Standalone);
					PackageObject->MarkAsGarbage();
					return true;
				}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);

				Package->MarkAsGarbage();
			}
		}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);

		if (!UObjectBaseUtility::IsGarbageEliminationEnabled())
		{
			// Rename the packages that we are streaming out so that we can possibly reload another copy of them
			for (UPackage* Package : Packages)
			{
				FCoreUObjectInternalDelegates::GetOnLeakedPackageRenameDelegate().Broadcast(Package);
				TrashObject(Package);
			}

#if !WITH_EDITOR
			// Clear the level actors array to maximize the memory savings from GC if there are outstanding references to some actors.
			// Don't do this outside the editor now til we can validate it works properly with external packages.
			InLevel->Actors.Empty();
			InLevel->ActorsForGC.Empty();
			InLevel->ActorCluster = nullptr; // The actor cluster has been marked garbage so will be dissolved, also drop a reference to it here since it has an internal array of pointers
#endif
		}
	}
}

void FLevelStreamingGCHelper::VerifyLevelsGotRemovedByGC()
{
	if (LevelPackageNames.Num())
	{
#if DO_GUARD_SLOW
		const bool bShouldVerify = !GIsEditor;
#else
		const bool bShouldVerify = false;
#endif

#if !UE_BUILD_SHIPPING
		if (bShouldVerify || GLevelStreamingForceVerifyLevelsGotRemovedByGC)
		{
			int32 FailCount = 0;
			const bool bIsAsyncLoading = IsAsyncLoading();
			// Iterate over all objects and find out whether they reside in a GC'ed level package.
			for (FThreadSafeObjectIterator It; It; ++It)
			{
				UObject* Object = *It;
				// Check whether object's outermost is in the list.
				if (LevelPackageNames.Contains(Object->GetOutermost()->GetFName())
					// But disregard package object itself.
					&& !Object->IsA(UPackage::StaticClass()))
				{
					if (bIsAsyncLoading && Object->HasAnyInternalFlags(EInternalObjectFlags::Async | EInternalObjectFlags_AsyncLoading))
					{
						UE_LOGF(LogGarbage, Display, "Level object %ls isn't released by async loading yet, "
							"it will get garbage collected next time instead.",
							*Object->GetFullName());
					}
					else
					{
						UE_LOGF(LogGarbage, Warning, "Level object %ls didn't get garbage collected! Trying to find culprit, though this might crash. Try increasing stack size if it does. Referenced by:", *Object->GetFullName());
						FReferenceChainSearch RefChainSearch(Object, EReferenceChainSearchMode::Shortest | EReferenceChainSearchMode::PrintResults);
						FailCount++;
					}
				}
			}
			if (FailCount > 0)
			{
				UE_LOGF(LogGarbage, Error, "Streamed out levels were not completely garbage collected! Please see previous log entries.");
				// If not forcing verify, consider errors as fatal
				check(GLevelStreamingForceVerifyLevelsGotRemovedByGC);
			}
		}
#endif
		LevelPackageNames.Empty();
	}
}

int32 FLevelStreamingGCHelper::GetNumLevelsPendingPurge()
{
	return LevelsPendingUnload.Num() + NumberOfPreparedStreamedOutLevelsForGC;
}

int32 FLevelStreamingGCHelper::GetNumActorsPendingPurge()
{
	return ActorsPendingUnload + NumberOfPreparedStreamedOutActorsForGC;
}

void FLevelStreamingGCHelper::TrashPackage(UPackage* InPackage)
{
	TrashObject(InPackage);
}
