// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LevelStreamingVolume.h"
#include "Engine/CollisionProfile.h"
#include "Engine/World.h"
#include "Components/BrushComponent.h"
#include "Engine/LevelStreaming.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelStreamingVolume)

ALevelStreamingVolume::ALevelStreamingVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	GetBrushComponent()->bAlwaysCreatePhysicsState = true;

	bColored = true;
	BrushColor.R = 255;
	BrushColor.G = 165;
	BrushColor.B = 0;
	BrushColor.A = 255;

	StreamingUsage = SVB_LoadingAndVisibility;
}

void ALevelStreamingVolume::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor)
	{
		UpdateStreamingLevelsRefs();
	}
#endif//WITH_EDITOR
}

#if WITH_EDITOR

void ALevelStreamingVolume::UpdateStreamingLevelsRefs()
{
	StreamingLevelNames.Reset();
	
	UWorld* OwningWorld = GetWorld();
	if (OwningWorld)
	{
		for (ULevelStreaming* LevelStreaming : OwningWorld->GetStreamingLevels())
		{
			if (LevelStreaming && LevelStreaming->EditorStreamingVolumes.Find(this) != INDEX_NONE)
			{
				StreamingLevelNames.Add(LevelStreaming->GetWorldAssetPackageFName());
			}
		}
	}
}

#endif// WITH_EDITOR
