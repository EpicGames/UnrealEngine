// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LevelTick.cpp: Level timer tick function
=============================================================================*/

#include "Engine/Level.h"
#include "Async/ParallelFor.h"
#include "Misc/TimeGuard.h"
#include "GameFramework/WorldSettings.h"
#include "StateStreamManager.h"
#include "UObject/UObjectStats.h"
#include "EngineStats.h"
#include "RenderingThread.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "ProfilingDebugging/LevelStreamingProfilingSubsystem.h"
#include "AI/NavigationSystemBase.h"
#include "GameFramework/PlayerController.h"
#include "ParticleHelper.h"
#include "Engine/LevelStreaming.h"
#include "Engine/NetConnection.h"
#include "SceneInterface.h"
#include "UnrealEngine.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "Engine/LevelStreamingVolume.h"
#include "IXRTrackingSystem.h"
#include "Camera/CameraPhotography.h"
#include "UObject/Stack.h"
#include "PhysicsEngine/CollisionAnalyzerCapture.h"
#include "Rendering/RenderCommandPipes.h"
#include "Stats/ThreadIdleStats.h"
#include "SkeletalMeshUpdater.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#if !UE_SERVER
#include "IMediaModule.h"
#include "Modules/ModuleManager.h"
#endif

//#include "SoundDefinitions.h"
#include "FXSystem.h"
#include "TickTaskManagerInterface.h"
#include "Engine/CoreSettings.h"

#include "InGamePerformanceTracker.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "GPUSkinCache.h"
#include "ComputeWorkerInterface.h"
#include "RenderGraphBuilder.h"
#include "StaticMeshResources.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "ObjectCacheEventSink.h"
#else
#include "TimerManager.h"
#endif

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

CSV_DEFINE_CATEGORY_MODULE(ENGINE_API, Ticks, true);
CSV_DEFINE_CATEGORY_MODULE(ENGINE_API, ActorCount, true);

#if CSV_PROFILER_STATS && CSV_TRACK_UOBJECT_COUNT
CSV_DEFINE_CATEGORY_MODULE(ENGINE_API, ObjectCount, true);
#endif

// this will log out all of the objects that were ticked in the FDetailedTickStats struct so you can isolate what is expensive
#define LOG_DETAILED_DUMPSTATS 0

#define LOG_DETAILED_PATHFINDING_STATS 0

/** Global boolean to toggle the log of detailed tick stats. */
/** Needs LOG_DETAILED_DUMPSTATS to be 1 **/
bool GLogDetailedDumpStats = true; 

/** Game stats */

// DECLARE_CYCLE_STAT is the reverse of what will be displayed in the game's stat game

DEFINE_STAT(STAT_AsyncWorkWaitTime);
DEFINE_STAT(STAT_PhysicsTime);

DEFINE_STAT(STAT_SpawnActorTime);
DEFINE_STAT(STAT_ActorBeginPlay);

DEFINE_STAT(STAT_GCSweepTime);
DEFINE_STAT(STAT_GCMarkTime);

DEFINE_STAT(STAT_TeleportToTime);
DEFINE_STAT(STAT_MoveComponentTime);
DEFINE_STAT(STAT_MoveComponentSceneComponentTime);
DEFINE_STAT(STAT_UpdateOverlaps);
DEFINE_STAT(STAT_PerformOverlapQuery);
DEFINE_STAT(STAT_UpdatePhysicsVolume);
DEFINE_STAT(STAT_EndScopedMovementUpdate);

DEFINE_STAT(STAT_PostTickComponentLW);
DEFINE_STAT(STAT_PostTickComponentRecreate);
DEFINE_STAT(STAT_PostTickComponentUpdate);
DEFINE_STAT(STAT_PostTickComponentUpdateWait);

DECLARE_CYCLE_STAT(TEXT("OnEndOfFrameUpdateDuringTick"), STAT_OnEndOfFrameUpdateDuringTick, STATGROUP_Game);
TRACE_DECLARE_INT_COUNTER(UObjectCount, TEXT("UObject Count"));

DEFINE_STAT(STAT_TickTime);
DEFINE_STAT(STAT_WorldTickTime);
DEFINE_STAT(STAT_UpdateCameraTime);
DEFINE_STAT(STAT_CharacterMovement);
DEFINE_STAT(STAT_PlayerControllerTick);

DEFINE_STAT(STAT_VolumeStreamingTickTime);
DEFINE_STAT(STAT_VolumeStreamingChecks);

DEFINE_STAT(STAT_NetWorldTickTime);
DEFINE_STAT(STAT_NavWorldTickTime);
DEFINE_STAT(STAT_ResetAsyncTraceTickTime);
DEFINE_STAT(STAT_TickableTickTime);
DEFINE_STAT(STAT_RuntimeMovieSceneTickTime);
DEFINE_STAT(STAT_FinishAsyncTraceTickTime);
DEFINE_STAT(STAT_NetBroadcastTickTime);
DEFINE_STAT(STAT_NetServerRepActorsTime);
DEFINE_STAT(STAT_NetServerGatherPrioritizeRepActorsTime);
DEFINE_STAT(STAT_NetConsiderActorsTime);
DEFINE_STAT(STAT_NetUpdateUnmappedObjectsTime);
DEFINE_STAT(STAT_NetInitialDormantCheckTime);
DEFINE_STAT(STAT_NetPrioritizeActorsTime);
DEFINE_STAT(STAT_NetReplicateActorTime);
DEFINE_STAT(STAT_NetReplicateDynamicPropTime);
DEFINE_STAT(STAT_NetReplicateDynamicPropCompareTime);
DEFINE_STAT(STAT_NetReplicateDynamicPropSendTime);
DEFINE_STAT(STAT_NetReplicateDynamicPropSendBackCompatTime);
DEFINE_STAT(STAT_NetSkippedDynamicProps);
DEFINE_STAT(STAT_NetSerializeItemDeltaTime);
DEFINE_STAT(STAT_NetUpdateGuidToReplicatorMap);
DEFINE_STAT(STAT_NetReplicateStaticPropTime);
DEFINE_STAT(STAT_NetBroadcastPostTickTime);
DEFINE_STAT(STAT_NetRebuildConditionalTime);
DEFINE_STAT(STAT_PackageMap_SerializeObjectTime);


/*-----------------------------------------------------------------------------
	Externs.
-----------------------------------------------------------------------------*/

extern bool GShouldLogOutAFrameOfMoveComponent;
extern bool GShouldLogOutAFrameOfSetBodyTransform;

#if LOG_DETAILED_PATHFINDING_STATS
/** Global detailed pathfinding stats. */
FDetailedTickStats GDetailedPathFindingStats(30, 10, 1, 20, TEXT("pathfinding"));
#endif

/*-----------------------------------------------------------------------------
	Detailed tick stats helper classes.
-----------------------------------------------------------------------------*/

/** Constructor, private on purpose and initializing all members. */
FDetailedTickStats::FDetailedTickStats( int32 InNumObjectsToReport, float InTimeBetweenLogDumps, float InMinTimeBetweenLogDumps, float InTimesToReport, const TCHAR* InOperationPerformed )
:	GCIndex( 0 )
,   GCCallBackRegistered( false )
,	NumObjectsToReport( InNumObjectsToReport )
,	TimeBetweenLogDumps( InTimeBetweenLogDumps )
,	MinTimeBetweenLogDumps( InMinTimeBetweenLogDumps )
,	LastTimeOfLogDump( 0 )
,	TimesToReport( InTimesToReport )
,	OperationPerformed( InOperationPerformed )
{
}

/**  Destructor, unregisters the GC callback */
FDetailedTickStats::~FDetailedTickStats()
{
	// remove callback as we are dead
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(OnPreGarbageCollectDelegateHandle);
}

/**
 * Starts tracking an object and returns whether it's a recursive call or not. If it is recursive
 * the function will return false and EndObject should not be called on the object.
 *
 * @param	Object		Object to track
 * @return	false if object is already tracked and EndObject should NOT be called, true otherwise
 */
bool FDetailedTickStats::BeginObject( UObject* Object )
{
	// If object is already tracked, tell calling code to not track again.
	if( ObjectsInFlight.Contains( Object ) )
	{
		return false;
	}
	// Keep track of the fact that this object is being tracked.
	else
	{
		ObjectsInFlight.Add( Object );
		return true;
	}
}

/**
 * Add instance of object to stats
 *
 * @param Object	Object instance
 * @param DeltaTime	Time operation took this instance
 * @param   bForSummary Object should be used for high level summary
 */
void FDetailedTickStats::EndObject( UObject* Object, float DeltaTime, bool bForSummary )
{
	// Find existing entry and update it if found.
	int32* TickStatIndex = ObjectToStatsMap.Find( Object );
	bool bCreateNewEntry = true;
	if( TickStatIndex )
	{
		FTickStats* TickStats = &AllStats[*TickStatIndex];
		// If GC has occurred since we last checked, we need to validate that this is still the correct object
		if (TickStats->GCIndex == GCIndex || // was checked since last GC
			(Object->GetPathName() == TickStats->ObjectPathName && Object->GetClass()->GetFName() == TickStats->ObjectClassFName)) // still refers to the same object
		{
			TickStats->GCIndex = GCIndex;
			TickStats->TotalTime += DeltaTime;
			TickStats->Count++;
			bCreateNewEntry = false;
		}
		// else this mapping is stale and the memory has been reused for a new object
	}
	// Create new entry.
	if (bCreateNewEntry)		
	{	
		// The GC callback cannot usually be registered at construction because this comes from a static data structure 
		// do it now if need be and it is ready
		if (!GCCallBackRegistered)
		{
			GCCallBackRegistered = true;
			// register callback so that we can avoid finding the wrong stats for new objects reusing memory that used to be associated with a different object
			OnPreGarbageCollectDelegateHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FDetailedTickStats::OnPreGarbageCollect);
		}

		FTickStats NewTickStats;
		NewTickStats.GCIndex			= GCIndex;
		NewTickStats.ObjectPathName		= Object->GetPathName();
		NewTickStats.ObjectDetailedInfo	= Object->GetDetailedInfo();
		NewTickStats.ObjectClassFName	= Object->GetClass()->GetFName();
		if (NewTickStats.ObjectDetailedInfo == TEXT("No_Detailed_Info_Specified"))
		{
			NewTickStats.ObjectDetailedInfo = TEXT(""); // This is a common, useless, case; save memory and clean up report by avoiding storing it
		}

		NewTickStats.Count			= 1;
		NewTickStats.TotalTime		= DeltaTime;
		NewTickStats.bForSummary	= bForSummary;
		int32 Index = AllStats.Add(NewTickStats);
		ObjectToStatsMap.Add( Object, Index );
	}
	// Object no longer is in flight at this point.
	ObjectsInFlight.Remove(Object);
}

/** Reset stats to clean slate. */
void FDetailedTickStats::Reset()
{
	AllStats.Empty();
	ObjectToStatsMap.Empty();
}

/** Dump gathered stats information to the log. */
void FDetailedTickStats::DumpStats()
{
	// Determine whether we should dump to the log.
	bool bShouldDump = false;
	
	// Dump request due to interval.
	if( FApp::GetCurrentTime() > (LastTimeOfLogDump + TimeBetweenLogDumps) )
	{
		bShouldDump = true;
	}
	
	// Dump request due to low framerate.
	float TotalTime = 0;
	for( TArray<FTickStats>::TIterator It(AllStats); It; ++It )
	{
		const FTickStats& TickStat = *It;
		if( TickStat.bForSummary == true )
		{
			TotalTime += TickStat.TotalTime;
		}
	}
	if( TotalTime * 1000 > TimesToReport )
	{
		bShouldDump = true;
	}

	// Only dump every TimeBetweenLogDumps seconds.
	if( bShouldDump 
	&& ((FApp::GetCurrentTime() - LastTimeOfLogDump) > MinTimeBetweenLogDumps) )
	{
		LastTimeOfLogDump = FApp::GetCurrentTime();

		// Array of stats, used for sorting.
		TArray<FTickStats> SortedTickStats;
		TArray<FTickStats> SortedTickStatsDetailed;
		// Populate from TArray in unsorted fashion.
		for( TArray<FTickStats>::TIterator It(AllStats); It; ++It )
		{
			const FTickStats& TickStat = *It;
			if(TickStat.bForSummary == true )
			{
				SortedTickStats.Add( TickStat );
			}
			else
			{
				SortedTickStatsDetailed.Add( TickStat );
			}
		}
		// Sort stats by total time spent.
		SortedTickStats.Sort( FTickStats() );
		SortedTickStatsDetailed.Sort( FTickStats() );

		// Keep track of totals.
		FTickStats Totals;
		Totals.TotalTime	= 0;
		Totals.Count		= 0;

		// Dump tick stats sorted by total time.
		UE_LOGF(LogLevel, Log, "Per object stats, frame # %llu", (uint64)GFrameCounter);
		for( int32 i=0; i<SortedTickStats.Num(); i++ )
		{
			const FTickStats& TickStats = SortedTickStats[i];
			if( i<NumObjectsToReport )
			{
				UE_LOGF(LogLevel, Log, "%5.2f ms, %4i instances, avg cost %5.3f, %ls", 1000 * TickStats.TotalTime, TickStats.Count, (TickStats.TotalTime/TickStats.Count) * 1000, *TickStats.ObjectPathName ); 
			}
			Totals.TotalTime += TickStats.TotalTime;
			Totals.Count	 += TickStats.Count;
		}
		UE_LOGF(LogLevel, Log, "Total time spent %ls %4i instances: %5.2f", *OperationPerformed, Totals.Count, Totals.TotalTime * 1000 );

#if LOG_DETAILED_DUMPSTATS
		if (GLogDetailedDumpStats)
		{
			Totals.TotalTime	= 0;
			Totals.Count		= 0;

			UE_LOGF(LogLevel, Log, "Detailed object stats, frame # %i", GFrameCounter);
			for( int32 i=0; i<SortedTickStatsDetailed.Num(); i++ )
			{
				const FTickStats& TickStats = SortedTickStatsDetailed(i);
				if( i<NumObjectsToReport*10 )
				{
					UE_LOGF(LogLevel, Log, "avg cost %5.3f, %ls %ls",(TickStats.TotalTime/TickStats.Count) * 1000, *TickStats.ObjectPathName, *TickStats.ObjectDetailedInfo ); 
				}
				Totals.TotalTime += TickStats.TotalTime;
				Totals.Count	 += TickStats.Count;
			}
			UE_LOGF(LogLevel, Log, "Total time spent %ls %4i instances: %5.2f", *OperationPerformed, Totals.Count, Totals.TotalTime * 1000 );
		}
#endif // LOG_DETAILED_DUMPSTATS

	}
}


/**
 * Constructor, keeping track of object's class and start time.
 */
FScopedDetailTickStats::FScopedDetailTickStats( FDetailedTickStats& InDetailedTickStats, UObject* InObject )
:	DetailedTickStats( InDetailedTickStats )
,	Object( InObject )
,	StartCycles( FPlatformTime::Cycles() )
{
	bShouldTrackObjectClass = DetailedTickStats.BeginObject( Object->GetClass() );
	bShouldTrackObject = DetailedTickStats.BeginObject( Object );
}

/**
 * Destructor, calculating delta time and updating global helper.
 */
FScopedDetailTickStats::~FScopedDetailTickStats()
{
	const float DeltaTime = FPlatformTime::ToSeconds(FPlatformTime::Cycles() - StartCycles);	
	if( bShouldTrackObject )
	{
		DetailedTickStats.EndObject( Object, DeltaTime, false );
	}
	if( bShouldTrackObjectClass )
	{
		DetailedTickStats.EndObject( Object->GetClass(), DeltaTime, true );
	}
}




/* Controller Tick
Controllers are never animated, and do not look for an owner to be ticked before them
Non-player controllers don't support being an autonomous proxy
*/
void AController::TickActor( float DeltaSeconds, ELevelTick TickType, FActorTickFunction& ThisTickFunction )
{
	//root of tick hierarchy

	if (TickType == LEVELTICK_ViewportsOnly)
	{
		return;
	}

	if(IsValidChecked(this))
	{
		Tick(DeltaSeconds);	// perform any tick functions unique to an actor subclass
	}
}

////////////
// Timing //
////////////


/*-----------------------------------------------------------------------------
	Network client tick.
-----------------------------------------------------------------------------*/

void UWorld::TickNetClient( float DeltaSeconds )
{
	SCOPE_TIME_GUARD(TEXT("UWorld::TickNetClient"));

	// If our net driver has lost connection to the server,
	// and there isn't a PendingNetGame, throw a network failure error.
	if( NetDriver->ServerConnection->GetConnectionState() == USOCK_Closed )
	{
		if (GEngine->PendingNetGameFromWorld(this) == nullptr)
		{
			const FString Error = NSLOCTEXT("Engine", "ConnectionFailed", "Your connection to the host has been lost.").ToString();
			GEngine->BroadcastNetworkFailure(this, NetDriver, ENetworkFailure::ConnectionLost, Error);
		}
	}
}

/*-----------------------------------------------------------------------------
	Main level timer tick handler.
-----------------------------------------------------------------------------*/


bool UWorld::IsPaused() const
{
	// pause if specifically set or if we're waiting for the end of the tick to perform streaming level loads (so actors don't fall through the world in the meantime, etc)
	const AWorldSettings* Info = GetWorldSettings(/*bCheckStreamingPersistent=*/false, /*bChecked=*/false);
	return ( (Info && Info->GetPauserPlayerState() != nullptr && TimeSeconds >= PauseDelay) ||
				(bRequestedBlockOnAsyncLoading && GetNetMode() == NM_Client) ||
				(GEngine->ShouldCommitPendingMapChange(this)) ||
				(IsPlayInEditor() && bDebugPauseExecution) );
}


bool UWorld::IsCameraMoveable() const
{
	bool bIsCameraMoveable = (!IsPaused() || bIsCameraMoveableWhenPaused || IsPlayingReplay());
#if WITH_EDITOR
	// to fix UE-17047 Motion Blur exaggeration when Paused in Simulate:
	// Simulate is excluded as the camera can move which invalidates motionblur
	bIsCameraMoveable = bIsCameraMoveable || (GEditor && GEditor->bIsSimulatingInEditor);
#endif
	return bIsCameraMoveable;
}

/**
 * Streaming settings for levels which are determined visible by level streaming volumes.
 */
class FVisibleLevelStreamingSettings
{
public:
	FVisibleLevelStreamingSettings()
	{
		bShouldBeVisible		= false;
		bShouldBlockOnLoad		= false;
		bShouldChangeVisibility	= false;
	}

	FVisibleLevelStreamingSettings( EStreamingVolumeUsage Usage )
	{
		switch( Usage )
		{
		case SVB_Loading:
			bShouldBeVisible		= false;
			bShouldBlockOnLoad		= false;
			bShouldChangeVisibility	= false;
			break;
		case SVB_LoadingNotVisible:
			bShouldBeVisible		= false;
			bShouldBlockOnLoad		= false;
			bShouldChangeVisibility	= true;
			break;
		case SVB_LoadingAndVisibility:
			bShouldBeVisible		= true;
			bShouldBlockOnLoad		= false;
			bShouldChangeVisibility	= true;
			break;
		case SVB_VisibilityBlockingOnLoad:
			bShouldBeVisible		= true;
			bShouldBlockOnLoad		= true;
			bShouldChangeVisibility	= true;
			break;
		case SVB_BlockingOnLoad:
			bShouldBeVisible		= false;
			bShouldBlockOnLoad		= true;
			bShouldChangeVisibility	= false;
			break;
		default:
			UE_LOGF(LogLevel, Fatal,"Unsupported usage %i",(int32)Usage);
		}
	}

	FVisibleLevelStreamingSettings& operator|=(const FVisibleLevelStreamingSettings& B)
	{
		bShouldBeVisible		|= B.bShouldBeVisible;
		bShouldBlockOnLoad		|= B.bShouldBlockOnLoad;
		bShouldChangeVisibility	|= B.bShouldChangeVisibility;
		return *this;
	}

	bool AllSettingsEnabled() const
	{
		return bShouldBeVisible && bShouldBlockOnLoad;
	}

	bool ShouldBeVisible( bool bCurrentShouldBeVisible ) const
	{
		if( bShouldChangeVisibility )
		{
			return bShouldBeVisible;
		}
		else
		{
			return bCurrentShouldBeVisible;
		}
	}

	bool ShouldBlockOnLoad() const
	{
		return bShouldBlockOnLoad;
	}

private:
	/** Whether level should be visible.						*/
	bool bShouldBeVisible;
	/** Whether level should block on load.						*/
	bool bShouldBlockOnLoad;
	/** Whether existing visibility settings should be changed. */
	bool bShouldChangeVisibility;
};

/**
 * Issues level streaming load/unload requests based on whether
 * players are inside/outside level streaming volumes.
 */
void UWorld::ProcessLevelStreamingVolumes(FVector* OverrideViewLocation)
{
	if (GetWorldSettings()->bUseClientSideLevelStreamingVolumes != (GetNetMode() == NM_Client))
	{
		return;
	}

	// if we are delaying using streaming volumes, return now
	if( StreamingVolumeUpdateDelay > 0 )
	{
		StreamingVolumeUpdateDelay--;
		return;
	}
	// Option to skip indefinitely.
	else if( StreamingVolumeUpdateDelay == INDEX_NONE )
	{
		return;
	}

	SCOPE_CYCLE_COUNTER( STAT_VolumeStreamingTickTime );

	bool bStreamingVolumesAreRelevant = false;
	for (FConstPlayerControllerIterator Iterator = GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		if (Iterator->GetResidence() != EResidence::Remote)
		{
			APlayerController* PlayerActor = Iterator->Get();
			if (PlayerActor && PlayerActor->bIsUsingStreamingVolumes)
			{
				bStreamingVolumesAreRelevant = true;
				break;
			}
		}
	}

	if (bStreamingVolumesAreRelevant)
	{
		// Begin by assembling a list of kismet streaming objects that have non-EditorPreVisOnly volumes associated with them.
		// @todo DB: Cache this, e.g. level startup.
		TArray<ULevelStreaming*> LevelStreamingObjectsWithVolumes;
		TSet<ULevelStreaming*> LevelStreamingObjectsWithVolumesOtherThanBlockingLoad;
		for (ULevelStreaming* LevelStreamingObject : StreamingLevels)
		{
			if( LevelStreamingObject )
			{
				for (ALevelStreamingVolume* StreamingVolume : LevelStreamingObject->EditorStreamingVolumes)
				{
					if( StreamingVolume 
					&& !StreamingVolume->bEditorPreVisOnly 
					&& !StreamingVolume->bDisabled )
					{
						LevelStreamingObjectsWithVolumes.Add(LevelStreamingObject);
						if( StreamingVolume->StreamingUsage != SVB_BlockingOnLoad )
						{
							LevelStreamingObjectsWithVolumesOtherThanBlockingLoad.Add(LevelStreamingObject);
						}
						break;
					}
				}
			}
		}

		// The set of levels with volumes whose volumes current contain player viewpoints.
		TMap<ULevelStreaming*,FVisibleLevelStreamingSettings> VisibleLevelStreamingObjects;

		// Iterate over all players and build a list of level streaming objects with
		// volumes that contain player viewpoints.
		for( FConstPlayerControllerIterator Iterator = GetPlayerControllerIterator(); Iterator; ++Iterator )
		{
			if (Iterator->GetResidence() == EResidence::Remote)
			{
				continue;
			}
			
			APlayerController* PlayerActor = Iterator->Get();
			if (PlayerActor && PlayerActor->bIsUsingStreamingVolumes)
			{
				FVector ViewLocation(0,0,0);
				// let the caller override the location to check for volumes
				if (OverrideViewLocation)
				{
					ViewLocation = *OverrideViewLocation;
				}
				else
				{
					FRotator ViewRotation(0,0,0);
					PlayerActor->GetPlayerViewPoint( ViewLocation, ViewRotation );
				}

				TMap<AVolume*,bool> VolumeMap;

				// Iterate over streaming levels with volumes and compute whether the
				// player's ViewLocation is in any of their volumes.
				for( int32 LevelIndex = 0 ; LevelIndex < LevelStreamingObjectsWithVolumes.Num() ; ++LevelIndex )
				{
					ULevelStreaming* LevelStreamingObject = LevelStreamingObjectsWithVolumes[ LevelIndex ];

					// StreamingSettings is an OR of all level streaming settings of volumes containing player viewpoints.
					FVisibleLevelStreamingSettings StreamingSettings;

					// See if level streaming settings were computed for other players.
					FVisibleLevelStreamingSettings* ExistingStreamingSettings = VisibleLevelStreamingObjects.Find( LevelStreamingObject );
					if ( ExistingStreamingSettings )
					{
						// Stop looking for viewpoint-containing volumes once all streaming settings have been enabled for the level.
						if ( ExistingStreamingSettings->AllSettingsEnabled() )
						{
							continue;
						}

						// Initialize the level's streaming settings with settings that were computed for other players.
						StreamingSettings = *ExistingStreamingSettings;
					}

					// For each streaming volume associated with this level . . .
					for ( int32 i = 0 ; i < LevelStreamingObject->EditorStreamingVolumes.Num() ; ++i )
					{
						ALevelStreamingVolume* StreamingVolume = LevelStreamingObject->EditorStreamingVolumes[i];
						if ( StreamingVolume && !StreamingVolume->bEditorPreVisOnly && !StreamingVolume->bDisabled )
						{
							bool bViewpointInVolume;
							bool* bResult = VolumeMap.Find(StreamingVolume);
							if ( bResult )
							{
								// This volume has already been considered for another level.
								bViewpointInVolume = *bResult;
							}
							else
							{						
								// Compute whether the viewpoint is inside the volume and cache the result.
								bViewpointInVolume = StreamingVolume->EncompassesPoint( ViewLocation );								
						
								VolumeMap.Add( StreamingVolume, bViewpointInVolume );
								INC_DWORD_STAT( STAT_VolumeStreamingChecks );
							}

							if ( bViewpointInVolume )
							{
								// Copy off the streaming settings for this volume.
								StreamingSettings |= FVisibleLevelStreamingSettings( (EStreamingVolumeUsage) StreamingVolume->StreamingUsage );

								// Update the streaming settings for the level.
								// This also marks the level as "should be loaded".
								VisibleLevelStreamingObjects.Add( LevelStreamingObject, StreamingSettings );

								// Stop looking for viewpoint-containing volumes once all streaming settings have been enabled.
								if ( StreamingSettings.AllSettingsEnabled() )
								{
									break;
								}
							}
						}
					}
				} // for each streaming level 
			} // bIsUsingStreamingVolumes
		} // for each PlayerController

		// Iterate over all streaming levels and set the level's loading status based
		// on whether it was found to be visible by a level streaming volume.
		for( int32 LevelIndex = 0 ; LevelIndex < LevelStreamingObjectsWithVolumes.Num() ; ++LevelIndex )
		{
			ULevelStreaming* LevelStreamingObject = LevelStreamingObjectsWithVolumes[LevelIndex];

			// Figure out whether level should be loaded and keep track of original state for notifications on change.
			FVisibleLevelStreamingSettings* NewStreamingSettings= VisibleLevelStreamingObjects.Find( LevelStreamingObject );
			bool bShouldAffectLoading							= LevelStreamingObjectsWithVolumesOtherThanBlockingLoad.Find( LevelStreamingObject ) != nullptr;
			bool bShouldBeLoaded								= (NewStreamingSettings != nullptr);
			bool bOriginalShouldBeLoaded						= LevelStreamingObject->ShouldBeLoaded();
			bool bOriginalShouldBeVisible						= LevelStreamingObject->ShouldBeVisible();
			bool bOriginalShouldBlockOnLoad						= LevelStreamingObject->bShouldBlockOnLoad;
			int32 OriginalLODIndex								= LevelStreamingObject->GetLevelLODIndex();

			if( bShouldBeLoaded || bShouldAffectLoading )
			{
				if( bShouldBeLoaded )
				{
					// Loading.
					LevelStreamingObject->SetShouldBeLoaded(true);
					LevelStreamingObject->SetShouldBeVisible(NewStreamingSettings->ShouldBeVisible(bOriginalShouldBeVisible));
					LevelStreamingObject->bShouldBlockOnLoad	= NewStreamingSettings->ShouldBlockOnLoad();
				}
				else if (LevelStreamingObject->ShouldBeLoaded())
				{
					// Prevent unload request flood.  The additional check ensures that unload requests can still be issued in the first UnloadCooldownTime seconds of play.
					if (TimeSeconds - LevelStreamingObject->LastVolumeUnloadRequestTime > LevelStreamingObject->MinTimeBetweenVolumeUnloadRequests ||  LevelStreamingObject->LastVolumeUnloadRequestTime < 0.1f)
					{
						if (GetPlayerControllerIterator())
						{
							LevelStreamingObject->LastVolumeUnloadRequestTime = TimeSeconds;
							LevelStreamingObject->SetShouldBeLoaded(false);
							LevelStreamingObject->SetShouldBeVisible(false);
						}
					}
				}

				const bool bNewShouldBeLoaded = LevelStreamingObject->ShouldBeLoaded();
				const bool bNewShouldBeVisible = LevelStreamingObject->ShouldBeVisible();

				// Notify players of the change.
				if( bOriginalShouldBeLoaded		!= bNewShouldBeLoaded
				||	bOriginalShouldBeVisible	!= bNewShouldBeVisible
				||	bOriginalShouldBlockOnLoad	!= LevelStreamingObject->bShouldBlockOnLoad
				||  OriginalLODIndex			!= LevelStreamingObject->GetLevelLODIndex())
				{
					for( FConstPlayerControllerIterator Iterator = GetPlayerControllerIterator(); Iterator; ++Iterator )
					{
						if (APlayerController* PlayerController = Iterator->Get())
						{
							PlayerController->LevelStreamingStatusChanged( 
									LevelStreamingObject, 
									bNewShouldBeLoaded, 
									bNewShouldBeVisible,
									LevelStreamingObject->bShouldBlockOnLoad,
									LevelStreamingObject->bShouldBlockOnUnload,
									LevelStreamingObject->GetLevelLODIndex());
						}
					}
				}
			}
		}
	}
}

/**
	* Run a tick group, ticking all actors and components
	* @param Group - Ticking group to run
	* @param bBlockTillComplete - if true, do not return until all ticks are complete
	*/
void UWorld::RunTickGroup(ETickingGroup Group, bool bBlockTillComplete = true)
{
	check(TickGroup == Group); // this should already be at the correct value, but we want to make sure things are happening in the right order
	FTickTaskManagerInterface::Get().RunTickGroup(Group, bBlockTillComplete);

	// Flush any grouped updates (TODO replace this with a more generic end of tick group callback)
	ProcessPendingGroupMoves(true);

	TickGroup = ETickingGroup(TickGroup + 1); // new actors go into the next tick group because this one is already gone
}

static TAutoConsoleVariable<int32> CVarAllowAsyncRenderThreadUpdates(
	TEXT("AllowAsyncRenderThreadUpdates"),
	1,
	TEXT("Used to control async renderthread updates. Also gated on FApp::ShouldUseThreadingForPerformance().")
	TEXT("0: GameThread")
	TEXT("1: GameThread with ParallelFor")
	TEXT("2: Async task with ParallelFor"));

static TAutoConsoleVariable<int32> CVarAllowAsyncRenderThreadUpdatesEditorGameWorld(
	TEXT("AllowAsyncRenderThreadUpdatesEditorGameWorld"),
	0,
	TEXT("Used to control async renderthread updates in an editor game world."));

static TAutoConsoleVariable<int32> CVarAllowAsyncRenderThreadUpdatesEditor(
	TEXT("AllowAsyncRenderThreadUpdatesEditor"),
	0,
	TEXT("Used to control async renderthread updates in the editor."));

enum class EComponentEndOfFrameUpdateMode : uint8
{
	// Game thread only
	GameThread,

	// Parallel tasks with the game thread helping out
	Parallel,

	// Fully async tasks
	Async
};

EComponentEndOfFrameUpdateMode GetComponentEndOfFrameUpdateMode(const UWorld* World)
{
	const auto GetMode = [] (int32 Value)
	{
		return FMath::Clamp<EComponentEndOfFrameUpdateMode>((EComponentEndOfFrameUpdateMode)Value, EComponentEndOfFrameUpdateMode::GameThread, EComponentEndOfFrameUpdateMode::Async);
	};

#if WITH_EDITOR
	if (World->IsGameWorld())
	{
		return GetMode(CVarAllowAsyncRenderThreadUpdatesEditorGameWorld.GetValueOnAnyThread());
	}
	else
	{
		return GetMode(CVarAllowAsyncRenderThreadUpdatesEditor.GetValueOnAnyThread());
	}
#else
	return GetMode(CVarAllowAsyncRenderThreadUpdates.GetValueOnAnyThread());
#endif
}

namespace EComponentMarkedForEndOfFrameUpdateState
{
	enum Type
	{
		Unmarked,
		Marked,
		MarkedForGameThread,
	};
}

// Utility struct to allow world direct access to UActorComponent::MarkedForEndOfFrameUpdateState without friending all of UActorComponent
struct FMarkComponentEndOfFrameUpdateState
{
	friend class UWorld;

private:
	FORCEINLINE static void Set(UActorComponent* Component, int32 ArrayIndex, const EComponentMarkedForEndOfFrameUpdateState::Type UpdateState)
	{
		checkSlow(UpdateState < 4); // Only 2 bits are allocated to store this value
		Component->MarkedForEndOfFrameUpdateState = UpdateState;
		Component->MarkedForEndOfFrameUpdateArrayIndex = ArrayIndex;
	}

	FORCEINLINE static void ClearReadyForEarlyEndOfFrameUpdate(UActorComponent* Component)
	{
		Component->bReadyForEarlyEndOfFrameUpdate = false;
	}

	FORCEINLINE static int32 GetArrayIndex(UActorComponent* Component)
	{
		return Component->MarkedForEndOfFrameUpdateArrayIndex;
	}

	FORCEINLINE static void SetMarkedForPreEndOfFrameSync(UActorComponent* Component)
	{
		Component->bMarkedForPreEndOfFrameSync = true;
	}

	FORCEINLINE static void ClearMarkedForPreEndOfFrameSync(UActorComponent* Component)
	{
		Component->bMarkedForPreEndOfFrameSync = false;
	}

	FORCEINLINE static void SetUpdating(UActorComponent* Component, bool bUpdating)
	{
		Component->bRenderStateUpdating = bUpdating;
	}
};

#if WITH_EDITOR
void UWorld::UpdateActorComponentEndOfFrameUpdateState(UActorComponent* Component) const
{
	int32 MarkedIndex = GetComponentsThatNeedEndOfFrameUpdate(Component).IndexOfByKey(Component);

	if (MarkedIndex != INDEX_NONE)
	{
		FMarkComponentEndOfFrameUpdateState::Set(Component, MarkedIndex, EComponentMarkedForEndOfFrameUpdateState::Marked);
	}
	else
	{
		int32 MarkedForGameThreadIndex = ComponentsThatNeedEndOfFrameUpdate_OnGameThread.IndexOfByKey(Component);
		if(MarkedForGameThreadIndex != INDEX_NONE)
		{
			FMarkComponentEndOfFrameUpdateState::Set(Component, MarkedForGameThreadIndex, EComponentMarkedForEndOfFrameUpdateState::MarkedForGameThread);
		}
		else
		{
			FMarkComponentEndOfFrameUpdateState::Set(Component, INDEX_NONE, EComponentMarkedForEndOfFrameUpdateState::Unmarked);
		}
	}
}
#endif

#if UE_WITH_PSO_PRECACHING
void UWorld::AddPrimitiveComponentPSOPrecacheCompleted(UPrimitiveComponent* Component)
{
	if (Component->IsRenderStateUpdating())
	{
		ComponentsThatNeedEndOfFrameMarkRenderStateDirty.Emplace(Component);
	}
	else
	{
		Component->MarkRenderStateDirty();
	}
}
#endif

void UWorld::ClearActorComponentEndOfFrameUpdate(UActorComponent* Component)
{
	check(!bPostTickComponentUpdate); // can't call this while we are doing the updates

	if (Component->IsRenderStateUpdating())
	{
		UE_LOGF(LogLevel, Fatal, "Calling ClearActorComponentEndOfFrameUpdate on component %ls but it is currently being updated. The component was marked for early update with MarkForNeededEndOfFrameUpdate but is being modified after the early update task was kicked. This is a race condition. Use AllowAsyncRenderThreadUpdates{Editor, EditorGameWorld} 1 to disable async updates to work around this issue.", *Component->GetReadableName());
	}

	const uint32 CurrentState = Component->GetMarkedForEndOfFrameUpdateState();

	auto& ComponentsToUpdate = GetComponentsThatNeedEndOfFrameUpdate(Component);
	auto& ComponentsToSync   = GetComponentsThatNeedEndOfFrameSync(Component);

	if (CurrentState == EComponentMarkedForEndOfFrameUpdateState::Marked)
	{
		const int32 ArrayIndex = FMarkComponentEndOfFrameUpdateState::GetArrayIndex(Component);
		check(ComponentsToUpdate.IsValidIndex(ArrayIndex));
		check(ComponentsToUpdate[ArrayIndex] == Component);
		ComponentsToUpdate[ArrayIndex] = nullptr;
		FMarkComponentEndOfFrameUpdateState::ClearReadyForEarlyEndOfFrameUpdate(Component);
	}
	else if (CurrentState == EComponentMarkedForEndOfFrameUpdateState::MarkedForGameThread)
	{
		const int32 ArrayIndex = FMarkComponentEndOfFrameUpdateState::GetArrayIndex(Component);
		check(ComponentsThatNeedEndOfFrameUpdate_OnGameThread.IsValidIndex(ArrayIndex));
		check(ComponentsThatNeedEndOfFrameUpdate_OnGameThread[ArrayIndex] == Component);
		ComponentsThatNeedEndOfFrameUpdate_OnGameThread[ArrayIndex] = nullptr;
	}
	FMarkComponentEndOfFrameUpdateState::Set(Component, INDEX_NONE, EComponentMarkedForEndOfFrameUpdateState::Unmarked);

	if (Component->GetMarkedForPreEndOfFrameSync())
	{
		ComponentsToSync.Remove(Component);
		FMarkComponentEndOfFrameUpdateState::ClearMarkedForPreEndOfFrameSync(Component);
	}
}

void UWorld::MarkActorComponentForNeededEndOfFrameUpdate(UActorComponent* Component, bool bForceGameThread)
{
	check(!bPostTickComponentUpdate); // can't call this while we are doing the updates

	if (Component->IsRenderStateUpdating())
	{
		UE_LOGF(LogLevel, Fatal, "Calling ClearActorComponentEndOfFrameUpdate on component %ls but it is currently being updated. The component was marked for early update with MarkForNeededEndOfFrameUpdate but is being modified after the early update task was kicked. This is a race condition. Use AllowAsyncRenderThreadUpdates{Editor, EditorGameWorld} 1 to disable async updates to work around this issue.", *Component->GetReadableName());
	}

	uint32 CurrentState = Component->GetMarkedForEndOfFrameUpdateState();

	auto& ComponentsToUpdate = GetComponentsThatNeedEndOfFrameUpdate(Component);
	auto& ComponentsToSync   = GetComponentsThatNeedEndOfFrameSync(Component);

	// force game thread can be turned on later, but we are not concerned about that, those are only cvars and constants; if those are changed during a frame, they won't fully kick in till next frame.
	if (CurrentState == EComponentMarkedForEndOfFrameUpdateState::Marked && bForceGameThread)
	{
		const int32 ArrayIndex = FMarkComponentEndOfFrameUpdateState::GetArrayIndex(Component);
		check(ComponentsToUpdate.IsValidIndex(ArrayIndex));
		check(ComponentsToUpdate[ArrayIndex] == Component);
		ComponentsToUpdate[ArrayIndex] = nullptr;
		CurrentState = EComponentMarkedForEndOfFrameUpdateState::Unmarked;
	}
	// it is totally ok if it is currently marked for the gamethread but now they are not forcing game thread. It will run on the game thread this frame.

	if (CurrentState == EComponentMarkedForEndOfFrameUpdateState::Unmarked)
	{
		// When there is no rendering thread force all updates on game thread,
		// to avoid modifying scene structures from multiple task threads
		bForceGameThread = bForceGameThread || !GIsThreadedRendering || !FApp::ShouldUseThreadingForPerformance();
		if (!bForceGameThread)
		{
			bForceGameThread = GetComponentEndOfFrameUpdateMode(this) == EComponentEndOfFrameUpdateMode::GameThread;
		}

		if (bForceGameThread)
		{
			FMarkComponentEndOfFrameUpdateState::Set(Component, ComponentsThatNeedEndOfFrameUpdate_OnGameThread.Num(), EComponentMarkedForEndOfFrameUpdateState::MarkedForGameThread);
			ComponentsThatNeedEndOfFrameUpdate_OnGameThread.Add(Component);
		}
		else
		{
			FMarkComponentEndOfFrameUpdateState::Set(Component, ComponentsToUpdate.Num(), EComponentMarkedForEndOfFrameUpdateState::Marked);
			ComponentsToUpdate.Add(Component);
		}

		// If the component might have outstanding tasks when we get to EOF updates, we will need to call the sync function
		if (Component->RequiresPreEndOfFrameSync())
		{
			FMarkComponentEndOfFrameUpdateState::SetMarkedForPreEndOfFrameSync(Component);
			ComponentsToSync.Add(Component);
		}
	}
}

void UWorld::SetMaterialParameterCollectionInstanceNeedsUpdate()
{
	bMaterialParameterCollectionInstanceNeedsDeferredUpdate = true;
}

bool UWorld::HasEndOfFrameUpdates() const
{
	return ComponentsThatNeedEndOfFrameUpdate_OnGameThread.Num() > 0 || ComponentsThatNeedEndOfFrameUpdate_AsyncTick.Num() > 0 || ComponentsThatNeedEndOfFrameUpdate.Num() > 0;
}

DECLARE_GPU_STAT(EndOfFrameUpdates);
DECLARE_GPU_STAT(GPUSkinCacheRayTracingGeometry);

struct UWorld::FEndOfFrameUpdateContext
{
	TArray<TObjectPtr<UActorComponent>> Components;
	TArray<TObjectPtr<UActorComponent>> Components_OnGameThread;
	FSceneInterface::IPrimitiveTransformUpdater* PrimitiveTransformUpdater = nullptr;

	void SetUpdating(bool bUpdating)
	{
		for (UActorComponent* Component : Components)
		{
			if (Component)
			{
				FMarkComponentEndOfFrameUpdateState::SetUpdating(Component, bUpdating);
			}
		}
	}

	void UpdateComponent_Parallel(FRenderCommandList* RenderCommandList, int32 Index)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DeferredRenderUpdates);
		FTaskTagScope Scope(ETaskTag::EParallelGameThread);
		FRenderCommandList::FRecordScope RecordScope(RenderCommandList);
		FSceneInterface::FPrimitiveTransformUpdateScope TransformScope(PrimitiveTransformUpdater);

#if WITH_EDITOR
		if (!IsInParallelGameThread() && IsInGameThread())
		{
			FObjectCacheEventSink::ProcessQueuedNotifyEvents();
		}
#endif

		UActorComponent* NextComponent = Components[Index];
		if (NextComponent)
		{
			if (NextComponent->IsRegistered() && !NextComponent->IsTemplate() && IsValid(NextComponent))
			{
				NextComponent->DoDeferredRenderUpdates_Concurrent();
			}

			check(!IsValid(NextComponent) || NextComponent->GetMarkedForEndOfFrameUpdateState() == EComponentMarkedForEndOfFrameUpdateState::Marked);
			FMarkComponentEndOfFrameUpdateState::Set(NextComponent, INDEX_NONE, EComponentMarkedForEndOfFrameUpdateState::Unmarked);
		}
	};

	void Update_GameThread(FSceneInterface* InScene, FRenderCommandList* RenderCommandList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DeferredRenderUpdates_GameThread);
		FSceneInterface::FPrimitiveTransformUpdateScope TransformScope(PrimitiveTransformUpdater);

		// To avoid any problems in case of reentrancy during the deferred update pass, we gather everything and clears the buffers first
		// Reentrancy can occur if a render update need to force wait on an async resource and a progress bar ticks the game-thread during that time.
		TArray<UActorComponent*> DeferredUpdates;
		DeferredUpdates.Reserve(Components_OnGameThread.Num());

		for (UActorComponent* Component : Components_OnGameThread)
		{
			if (Component)
			{
				if (Component->IsRegistered() && !Component->IsTemplate() && IsValid(Component))
				{
					DeferredUpdates.Add(Component);
				}

				check(!IsValid(Component) || Component->GetMarkedForEndOfFrameUpdateState() == EComponentMarkedForEndOfFrameUpdateState::MarkedForGameThread);
				FMarkComponentEndOfFrameUpdateState::Set(Component, INDEX_NONE, EComponentMarkedForEndOfFrameUpdateState::Unmarked);
			}
		}

		// We are only regenerating render state here, not components
		FStaticMeshComponentBulkReregisterContext ReregisterContext(InScene, DeferredUpdates, EBulkReregister::RenderState);

		for (UActorComponent* Component : DeferredUpdates)
		{
			FMarkComponentEndOfFrameUpdateState::SetUpdating(Component, true);
			Component->DoDeferredRenderUpdates_Concurrent();
			FMarkComponentEndOfFrameUpdateState::SetUpdating(Component, false);
		}
	}
};

#define UE_END_OF_FRAME_UPDATE_REGION TEXT("EndOfFrameUpdates")

void UWorld::SendAllEndOfFrameUpdatesInternal(EEndOfFrameUpdateLocation Location)
{
	SCOPED_NAMED_EVENT(UWorld_SendAllEndOfFrameUpdates, FColor::Yellow);
	SCOPE_CYCLE_COUNTER(STAT_PostTickComponentUpdate);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EndOfFrameUpdates);
	CSV_SCOPED_SET_WAIT_STAT(EndOfFrameUpdates);

#if WITH_STATE_STREAM
	FStateStreamLaneScope LaneScope(StateStreamManager, LaneId);
#endif

	const bool bAsyncInTick = Location == EEndOfFrameUpdateLocation::Async && bInTick;

	if (!bAsyncInTick)
	{
		// Allow systems to complete async work that could introduce additional components to end of frame updates
		FWorldDelegates::OnWorldPreSendAllEndOfFrameUpdates.Broadcast(this);
	}

	if ((bAsyncInTick && ComponentsThatNeedEndOfFrameUpdate_AsyncTick.IsEmpty()) || !HasEndOfFrameUpdates())
	{
		return;
	}

	const auto MergeArray = [](TArray<TObjectPtr<UActorComponent>>& Dst, TArray<TObjectPtr<UActorComponent>>& Src)
	{
		if (Dst.IsEmpty())
		{
			Dst = MoveTemp(Src);
			Src.Empty(Dst.Num());
		}
		else
		{
			Dst.Append(Src);
			Src.Reset();
		}
	};

	// Process OnPreEndOfFrameSync BEFORE moving the component arrays into the context.
	// OnPreEndOfFrameSync can trigger WaitUntilTaskCompletes (e.g. cloth simulation), which pumps
	// game-thread tasks. Those tasks can call MarkActorComponentForNeededEndOfFrameUpdate, which
	// validates that already-marked components still exist at their stored array index. If we moved
	// the arrays first, those indices would point into the moved-away (context-owned) data, causing
	// a check failure. Processing sync first keeps the arrays in place and indices valid.
	//
	// We use a draining loop because OnPreEndOfFrameSync (or tasks pumped during it) may mark
	// additional components for sync. The TSet is copied to a local TArray for safe iteration,
	// then drained until no new sync components remain.
	{
		auto DrainSyncSet = [](TSet<TObjectPtr<UActorComponent>>& SyncSet)
		{
			while (!SyncSet.IsEmpty())
			{
				TSet<TObjectPtr<UActorComponent>> LocalSync = MoveTemp(SyncSet);
				SyncSet.Reset();

				for (UActorComponent* Component : LocalSync)
				{
					if (Component)
					{
						check(!IsValid(Component) || Component->GetMarkedForPreEndOfFrameSync());
						if (IsValid(Component))
						{
							Component->OnPreEndOfFrameSync();
						}
						FMarkComponentEndOfFrameUpdateState::ClearMarkedForPreEndOfFrameSync(Component);
					}
				}
			}
		};

		DrainSyncSet(ComponentsThatNeedPreEndOfFrameSync_AsyncTick);

		if (!bAsyncInTick)
		{
			DrainSyncSet(ComponentsThatNeedPreEndOfFrameSync);
		}
	}

	FEndOfFrameUpdateContext* Context = new FEndOfFrameUpdateContext();
	MergeArray(Context->Components, ComponentsThatNeedEndOfFrameUpdate_AsyncTick);

	if (!bAsyncInTick)
	{
		MergeArray(Context->Components, ComponentsThatNeedEndOfFrameUpdate);
		Context->Components_OnGameThread = MoveTemp(ComponentsThatNeedEndOfFrameUpdate_OnGameThread);
		ComponentsThatNeedEndOfFrameUpdate_OnGameThread.Empty(Context->Components_OnGameThread.Num());
	}

	// If we have a previous context active, then force this one to run asynchronously as well. They are not allowed to touch
	// the same components (validated via IsInAsyncEndOfFrameUpdate) so it is safe to run the tasks concurrently.
	const bool bLaunchAsyncTask = Scene && (!EndOfFrameUpdateContexts.IsEmpty() || Location == EEndOfFrameUpdateLocation::Async);

	//If we call SendAllEndOfFrameUpdates during a tick, we must ensure that all marked objects have completed any async work etc before doing the updates.
	if (bInTick)
	{
		SCOPE_CYCLE_COUNTER(STAT_OnEndOfFrameUpdateDuringTick);

		//If this proves too slow we can possibly have the mark set a complimentary bit array that marks which components need this call? Or just another component array?
		for (UActorComponent* Component : Context->Components)
		{
			if (Component)
			{
				Component->OnEndOfFrameUpdateDuringTick();
			}
		}

		for (UActorComponent* Component : Context->Components_OnGameThread)
		{
			if (Component)
			{
				Component->OnEndOfFrameUpdateDuringTick();
			}
		}
	}

	bPostTickComponentUpdate = true;

	const auto ParallelWork = [Context](FRenderCommandList* InRenderCommandList, int32 Index)
	{
		Context->UpdateComponent_Parallel(InRenderCommandList, Index);
	};

	const EParallelForFlags ParallelForFlags = EParallelForFlags::Unbalanced;

	FRenderCommandList* RootRenderCommandList = FRenderCommandList::Create(ERenderCommandListFlags::CloseOnSubmit);
	FRenderCommandList::FRecordScope RecordScope(RootRenderCommandList, FRenderCommandList::EStopRecordingAction::Submit);

#if WANTS_DRAW_MESH_EVENTS
	RHI_BREADCRUMB_EVENT_GAMETHREAD("SendAllEndOfFrameUpdates");
#endif

	FSceneInterface::IPrimitiveTransformUpdater* PrimitiveTransformUpdater = nullptr;

	if (Scene)
	{
		PrimitiveTransformUpdater = Scene->CreatePrimitiveTransformUpdater(Context->Components.Num() + Context->Components_OnGameThread.Num());
		Context->PrimitiveTransformUpdater = PrimitiveTransformUpdater;
	}

	if (bLaunchAsyncTask)
	{
		const int32 NumContexts = ParallelForImpl::GetNumberOfThreadTasks(Context->Components.Num(), 1, ParallelForFlags);

		if (EndOfFrameUpdateContexts.IsEmpty())
		{
			Scene->GetSkeletalMeshUpdater()->BeginAsyncPushCommands();

			TRACE_BEGIN_REGION(UE_END_OF_FRAME_UPDATE_REGION);
#if WITH_EDITOR
			FObjectCacheEventSink::BeginQueueNotifyEvents();
#endif
		}

		EndOfFrameUpdateContexts.Emplace(Context);
		Context->SetUpdating(true);

		FRenderCommandList* TaskRenderCommandList = FRenderCommandList::Create();

		EndOfFrameUpdateTasks.Emplace(UE::Tasks::Launch(UE_SOURCE_LOCATION, [Context, ParallelWork, TaskRenderCommandList, NumContexts, ParallelForFlags]
		{
			FRenderCommandList::FRecordScope RecordScope(TaskRenderCommandList, FRenderCommandList::EStopRecordingAction::Close);
			FRenderCommandList::FParallelForContext ParallelForContext(TaskRenderCommandList, NumContexts);
			ParallelForWithExistingTaskContext(TEXT("EndOfFrameUpdate"), ParallelForContext.GetCommandLists(), Context->Components.Num(), 1, ParallelWork, ParallelForFlags);

		}, EndOfFrameUpdatePrerequisiteTask, UE::Tasks::ETaskPriority::High));

		FRenderCommandDispatcher::Submit(TaskRenderCommandList, RootRenderCommandList);

		Context->Update_GameThread(Scene, RootRenderCommandList);
	}
	else
	{
#if WITH_EDITOR
		FObjectCacheEventSink::BeginQueueNotifyEvents();
#endif

		const auto GameThreadWork = [Context, Scene = Scene, RootRenderCommandList]
		{
			Context->Update_GameThread(Scene, RootRenderCommandList);
		};

		FRenderCommandList::FParallelForContext ParallelForContext(RootRenderCommandList, Context->Components.Num(), 1, ParallelForFlags);
		ParallelForWithPreWorkWithExistingTaskContext(TEXT("EndOfFrameUpdate"), ParallelForContext.GetCommandLists(), Context->Components.Num(), 1, ParallelWork, GameThreadWork, ParallelForFlags);

#if WITH_EDITOR
		// Any remaining events will be flushed with this call
		FObjectCacheEventSink::EndQueueNotifyEvents();
#endif

		delete Context;
		Context = nullptr;
	}

	if (Scene)
	{
		Scene->UpdatePrimitiveTransforms(PrimitiveTransformUpdater);
	}

	bPostTickComponentUpdate = false;
}

void UWorld::StartAsyncSendAllEndOfFrameUpdates()
{
	if (GetComponentEndOfFrameUpdateMode(this) != EComponentEndOfFrameUpdateMode::Async)
	{
		return;
	}

	SendAllEndOfFrameUpdatesInternal(EEndOfFrameUpdateLocation::Async);
}

void UWorld::FinishAsyncSendAllEndOfFrameUpdates()
{
#if WITH_STATE_STREAM
	FStateStreamLaneScope LaneScope(StateStreamManager, LaneId);
#endif

	if (!EndOfFrameUpdateContexts.IsEmpty())
	{
		Scene->GetSkeletalMeshUpdater()->EndAsyncPushCommands();

		EndOfFrameUpdateTasks.Emplace(MoveTemp(EndOfFrameUpdatePrerequisiteTask));
		UE::Tasks::Wait(EndOfFrameUpdateTasks);
		EndOfFrameUpdateTasks.Reset();

		for (FEndOfFrameUpdateContext* Context : EndOfFrameUpdateContexts)
		{
			Context->SetUpdating(false);
			delete Context;
		}
		EndOfFrameUpdateContexts.Reset();

#if WITH_EDITOR
		// Any remaining events will be flushed with this call
		FObjectCacheEventSink::EndQueueNotifyEvents();
#endif

		TRACE_END_REGION(UE_END_OF_FRAME_UPDATE_REGION);
	}

	for (UActorComponent* Component : ComponentsThatNeedEndOfFrameMarkRenderStateDirty)
	{
		Component->MarkRenderStateDirty();
	}
	ComponentsThatNeedEndOfFrameMarkRenderStateDirty.Empty();
}

/**
 * Send all render updates to the rendering thread.
 */
void UWorld::SendAllEndOfFrameUpdates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SendAllEndOfFrameUpdates);
	SendAllEndOfFrameUpdatesInternal(EEndOfFrameUpdateLocation::Immediate);
	FlushDeferredParameterCollectionInstanceUpdates();

	if (!EndOfFrameUpdateTasks.IsEmpty())
	{
		UE::Tasks::FTaskEvent TaskEvent{ UE_SOURCE_LOCATION };
		TaskEvent.AddPrerequisites(EndOfFrameUpdateTasks);
		TaskEvent.Trigger();

		EndOfFrameUpdateTasks.Reset();
		EndOfFrameUpdatePrerequisiteTask = Scene->GetSkeletalMeshUpdater()->AddPushCommandsTask(TaskEvent);
	}
}

/**
 * Flush any pending parameter collection updates to the render thrad.
 */
void UWorld::FlushDeferredParameterCollectionInstanceUpdates()
{
	if ( bMaterialParameterCollectionInstanceNeedsDeferredUpdate )
	{
		for (UMaterialParameterCollectionInstance* ParameterCollectionInstance : ParameterCollectionInstances)
		{
			if (ParameterCollectionInstance)
			{
				ParameterCollectionInstance->DeferredUpdateRenderState(false);
			}
		}

		bMaterialParameterCollectionInstanceNeedsDeferredUpdate = false;
	}
}

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
static TAutoConsoleVariable<int32> CVarRecordTickCountsToCSV(
	TEXT("csv.RecordTickCounts"),
	1,
	TEXT("Record tick counts by context when performing CSV capture"));

static TAutoConsoleVariable<int32> CVarDetailedTickContextForCSV(
	TEXT("csv.DetailedTickContext"),
	0,
	TEXT("Gives more detailed info for Tick counts in CSV"));

static TAutoConsoleVariable<int32> CVarRecordActorCountsToCSV(
	TEXT("csv.RecordActorCounts"),
	1,
	TEXT("Record actor counts by class when performing CSV capture"));

static TAutoConsoleVariable<int32> CVarRecordActorCountsToCSVThreshold(
	TEXT("csv.RecordActorCountsThreshold"),
	5,
	TEXT("Number of instances of an native Actor class required before recording to CSV stat"));

extern FCriticalSection CSVActorClassNameToCountMapLock;
extern TMap<FName, int32> CSVActorClassNameToCountMap;
extern TSAN_ATOMIC(int32) CSVActorTotalCount;

/** Add additional stats to CSV profile for tick and actor counts */
static void RecordWorldCountsToCSV(UWorld* World, bool bDoingActorTicks)
{
	if(FCsvProfiler::Get()->IsCapturing())
	{
		if (bDoingActorTicks && CVarRecordTickCountsToCSV.GetValueOnGameThread())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RecordTickCountsToCSV);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RecordTickCountsToCSV);

			bool bDetailed = (CVarDetailedTickContextForCSV.GetValueOnGameThread() != 0);

			TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess> TickContextToCountMap;
			int32 EnabledCount = 0;
			FTickTaskManagerInterface::Get().GetEnabledTickFunctionCounts(World, TickContextToCountMap, EnabledCount, bDetailed, true);

			for (auto It = TickContextToCountMap.CreateConstIterator(); It; ++It)
			{
				FCsvProfiler::Get()->RecordCustomStat(It->Key, CSV_CATEGORY_INDEX(Ticks), It->Value, ECsvCustomStatOp::Accumulate); // use accumulate in case we have more than one world ticking
			}

			// By default, Ticks/Total equals Basic/TicksQueued.
			// Ticks/Total can be used to accumulate tick counts from game systems that don't run in the task graph.
			CSV_CUSTOM_STAT(Ticks, Total, EnabledCount, ECsvCustomStatOp::Accumulate);
		}

		if (CVarRecordActorCountsToCSV.GetValueOnAnyThread())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RecordActorCountsToCSV);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RecordActorCountsToCSV);

			const int32 Threshold = CVarRecordActorCountsToCSVThreshold.GetValueOnAnyThread();

			{
				FScopeLock Lock(&CSVActorClassNameToCountMapLock);
				for (auto It = CSVActorClassNameToCountMap.CreateConstIterator(); It; ++It)
				{
					if (It->Value > Threshold)
					{
						FCsvProfiler::Get()->RecordCustomStat(It->Key, CSV_CATEGORY_INDEX(ActorCount), It->Value, ECsvCustomStatOp::Set);
					}
				}
			}

			static FName TotalActorCountStatName("TotalActorCount");
			FCsvProfiler::Get()->RecordCustomStat(TotalActorCountStatName, CSV_CATEGORY_INDEX(ActorCount), CSVActorTotalCount, ECsvCustomStatOp::Set);
		}

#if CSV_TRACK_UOBJECT_COUNT
		static const FName TotalObjectCountStatName("Total");
		FCsvProfiler::Get()->RecordCustomStat(TotalObjectCountStatName, CSV_CATEGORY_INDEX(ObjectCount), UObjectStats::GetUObjectCount(), ECsvCustomStatOp::Set);
#endif
	}
}
#endif // (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

DECLARE_CYCLE_STAT(TEXT("TG_PrePhysics"), STAT_TG_PrePhysics, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("TG_StartPhysics"), STAT_TG_StartPhysics, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("Start TG_DuringPhysics"), STAT_TG_DuringPhysics, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("TG_EndPhysics"), STAT_TG_EndPhysics, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("TG_PostPhysics"), STAT_TG_PostPhysics, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("TG_PostUpdateWork"), STAT_TG_PostUpdateWork, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("TG_LastDemotable"), STAT_TG_LastDemotable, STATGROUP_TickGroups);

#include "GameFramework/SpawnActorTimer.h"

/**
 * Update the level after a variable amount of time, DeltaSeconds, has passed.
 * All child actors are ticked after their owners have been ticked.
 */
void UWorld::Tick( ELevelTick TickType, float DeltaSeconds )
{
	SCOPE_TIME_GUARD(TEXT("UWorld::Tick"));
	SCOPED_NAMED_EVENT(UWorld_Tick, FColor::Orange);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(WorldTickMisc);
	CSV_SCOPED_SET_WAIT_STAT(WorldTickMisc);

	if (GIntraFrameDebuggingGameThread)
	{
		return;
	}

#if WITH_STATE_STREAM
	FStateStreamLaneScope LaneScope(StateStreamManager, LaneId);
#endif

	UE::Stats::FThreadIdleStats::BeginCriticalPath();

	RHI_BREADCRUMB_EVENT_GAMETHREAD("WorldTick");

	FWorldDelegates::OnWorldTickStart.Broadcast(this, TickType, DeltaSeconds);

	//Tick game and other thread trackers.
	if (PerfTrackers)
	{
		for (int32 Tracker = 0; Tracker < (int32)EInGamePerfTrackers::Num; ++Tracker)
		{
			PerfTrackers->GetInGamePerformanceTracker((EInGamePerfTrackers)Tracker, EInGamePerfTrackerThreads::GameThread).Tick();
			PerfTrackers->GetInGamePerformanceTracker((EInGamePerfTrackers)Tracker, EInGamePerfTrackerThreads::OtherThread).Tick();
		}
	}

#if LOG_DETAILED_PATHFINDING_STATS
	GDetailedPathFindingStats.Reset();
#endif

	SCOPE_CYCLE_COUNTER(STAT_WorldTickTime);

	// @todo vreditor: In the VREditor, this isn't actually wrapping the whole frame.  That would have to happen in EditorEngine.cpp's Tick.  However, it didn't seem to affect anything when I tried that.
	if (GEngine->XRSystem.IsValid())
	{
		GEngine->XRSystem->OnStartGameFrame( GEngine->GetWorldContextFromWorldChecked( this ) );
	}

#if ENABLE_SPAWNACTORTIMER
	FSpawnActorTimer& SpawnTimer = FSpawnActorTimer::Get();
	SpawnTimer.IncrementFrameCount();
#endif

#if ENABLE_COLLISION_ANALYZER
	// Tick collision analyzer (only if level is really ticking)
	if(TickType == LEVELTICK_All || TickType == LEVELTICK_ViewportsOnly)
	{
		ICollisionAnalyzer* Analyzer = FCollisionAnalyzerModule::Get();
		Analyzer->TickAnalyzer(this);
		GCollisionAnalyzerIsRecording = Analyzer->IsRecording();
	}
#endif // ENABLE_COLLISION_ANALYZER

	AWorldSettings* Info = GetWorldSettings();
	FMemMark Mark(FMemStack::Get());
	GInitRunaway();
	bInTick=true;
	bool bIsPaused = IsPaused();

	{
		SCOPE_CYCLE_COUNTER(STAT_NetWorldTickTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NetworkIncoming);
		CSV_SCOPED_SET_WAIT_STAT(NetworkTick);
		SCOPE_TIME_GUARD(TEXT("UWorld::Tick - NetTick"));
		LLM_SCOPE(ELLMTag::Networking);
		// Update the net code and fetch all incoming packets.
		BroadcastTickDispatch(DeltaSeconds);
		BroadcastPostTickDispatch();

		if( NetDriver && NetDriver->ServerConnection )
		{
			TickNetClient( DeltaSeconds );
		}
	}

	// Update time.
	RealTimeSeconds += DeltaSeconds;

	// Audio always plays at real-time regardless of time dilation, but only when NOT paused
	if( !bIsPaused )
	{
		AudioTimeSeconds += DeltaSeconds;
	}

	// Save off actual delta
	float RealDeltaSeconds = DeltaSeconds;

	// apply time multipliers
	DeltaSeconds *= Info->GetEffectiveTimeDilation();

	// Handle clamping of time to an acceptable value
	const float GameDeltaSeconds = Info->FixupDeltaSeconds(DeltaSeconds, RealDeltaSeconds);
	check(GameDeltaSeconds >= 0.0f);

	DeltaSeconds = GameDeltaSeconds;
	DeltaTimeSeconds = DeltaSeconds;
	DeltaRealTimeSeconds = RealDeltaSeconds;

	UnpausedTimeSeconds += DeltaSeconds;

	if ( !bIsPaused )
	{
		TimeSeconds += DeltaSeconds;
	}

	if( bPlayersOnly )
	{
		TickType = LEVELTICK_ViewportsOnly;
	}

	// give the async loading code more time if we're performing a high priority load or are in seamless travel
	if (Info->bHighPriorityLoading || Info->bHighPriorityLoadingLocal || IsInSeamlessTravel())
	{
		// GUseUnifiedTimeBudgetForStreaming handles this in HandleUnifiedStreaming
		if (!GUseUnifiedTimeBudgetForStreaming)
		{
			CSV_SCOPED_SET_WAIT_STAT(AsyncLoading);
			TRACE_CPUPROFILER_EVENT_SCOPE(HighPriorityAsyncLoading)
			// Force it to use the entire time slice, even if blocked on I/O
			ProcessAsyncLoading(true, true, GPriorityAsyncLoadingExtraTime / 1000.0f);
		}
	}

	// Translate world origin if requested
	if (OriginLocation != RequestedOriginLocation)
	{
		SetNewWorldOrigin(RequestedOriginLocation);
	}
	else
	{
		OriginOffsetThisFrame = FVector::ZeroVector;
	}
	
	// update world's subsystems (NavigationSystem for now)
	if (NavigationSystem != nullptr)
	{
		SCOPE_CYCLE_COUNTER(STAT_NavWorldTickTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NavigationBuild);

		NavigationSystem->Tick(DeltaSeconds);
	}

	bool bDoingActorTicks = 
		(TickType!=LEVELTICK_TimeOnly)
		&&	!bIsPaused
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
		&&  IsActorTickAndUserCallbacksEnabled()
#endif
		&&	(!NetDriver || !NetDriver->ServerConnection || NetDriver->ServerConnection->GetConnectionState()==USOCK_Open);

	FLatentActionManager& CurrentLatentActionManager = GetLatentActionManager();

	// Reset the list of objects the LatentActionManager has processed this frame
	CurrentLatentActionManager.BeginFrame();
	
	if (bDoingActorTicks)
	{
		{
			// Reset Async Trace before Tick starts 
			SCOPE_CYCLE_COUNTER(STAT_ResetAsyncTraceTickTime);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ResetAsyncTraceTickTime);
			ResetAsyncTrace();
		}
		{
			// Run pre-actor tick delegates that want clamped/dilated time
			SCOPE_CYCLE_COUNTER(STAT_TickTime);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(WorldPreActorTick);
			FWorldDelegates::OnWorldPreActorTick.Broadcast(this, TickType, DeltaSeconds);
		}
	}

	// Tick level sequence actors first
	MovieSceneSequenceTick.Broadcast(DeltaSeconds);

#if !UE_SERVER
	if (MovieSceneSequenceTick.IsBound())
	{
		// tick media framework pre-engine
		// (needs to be delayed post movie scene tick delegates handling, sif they are used - otherwise is triggered elsewhere)
		static const FName MediaModuleName("Media");
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
		if (MediaModule != nullptr)
		{
			MediaModule->TickPreEngine();
		}
	}
#endif

	// If only the DynamicLevel collection has entries, we can skip the validation and tick all levels.
	bool bValidateLevelList = false;
	bool bHasTickedACollection = false;

	for (const FLevelCollection& LevelCollection : LevelCollections)
	{
		if (LevelCollection.GetType() != ELevelCollectionType::DynamicSourceLevels)
		{
			const int32 NumLevels = LevelCollection.GetLevels().Num();
			if (NumLevels != 0)
			{
				bValidateLevelList = true;
				break;
			}
		}
	}

	for (int32 i = 0; i < LevelCollections.Num(); ++i)
	{
		// Build a list of levels from the collection that are also in the world's Levels array.
		// Collections may contain levels that aren't loaded in the world at the moment.
		TArray<ULevel*> LevelsToTick;
		for (ULevel* CollectionLevel : LevelCollections[i].GetLevels())
		{
			const bool bAddToTickList = (bValidateLevelList == false) || Levels.Contains(CollectionLevel);
			if (bAddToTickList && CollectionLevel)
			{
				LevelsToTick.Add(CollectionLevel);
			}
		}

		if (LevelsToTick.Num() == 0)
		{
			// Nothing to do, early out before creating the slow scope
			continue;
		}

		// Set up context on the world for this level collection
		FScopedLevelCollectionContextSwitch LevelContext(i, this);

		// If caller wants time update only, or we are paused, skip the rest.
		if (bDoingActorTicks)
		{
			// Actually tick actors now that context is set up
			SetupPhysicsTickFunctions(DeltaSeconds);
			TickGroup = TG_PrePhysics; // reset this to the start tick group
			FTickTaskManagerInterface::Get().StartFrame(this, DeltaSeconds, TickType, LevelsToTick);

			SCOPE_CYCLE_COUNTER(STAT_TickTime);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(TickActors);
			{
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TG_PrePhysics"), 10);
				SCOPE_CYCLE_COUNTER(STAT_TG_PrePhysics);
				CSV_SCOPED_SET_WAIT_STAT(PrePhysics);
				RunTickGroup(TG_PrePhysics);
			}
			bInTick = false;
			EnsureCollisionTreeIsBuilt();
			bInTick = true;
			{
				SCOPE_CYCLE_COUNTER(STAT_TG_StartPhysics);
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TG_StartPhysics"), 10);
				CSV_SCOPED_SET_WAIT_STAT(StartPhysics);
				RunTickGroup(TG_StartPhysics);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_TG_DuringPhysics);
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TG_DuringPhysics"), 10);
				CSV_SCOPED_SET_WAIT_STAT(DuringPhysics);
				RunTickGroup(TG_DuringPhysics, false); // No wait here, we should run until idle though. We don't care if all of the async ticks are done before we start running post-phys stuff
			}
			TickGroup = TG_EndPhysics; // set this here so the current tick group is correct during collision notifies, though I am not sure it matters. 'cause of the false up there^^^
			{
				SCOPE_CYCLE_COUNTER(STAT_TG_EndPhysics);
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TG_EndPhysics"), 10);
				CSV_SCOPED_SET_WAIT_STAT(EndPhysics);
				RunTickGroup(TG_EndPhysics);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_TG_PostPhysics);
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TG_PostPhysics"), 10);
				CSV_SCOPED_SET_WAIT_STAT(PostPhysics);
				RunTickGroup(TG_PostPhysics);
			}
	
		}
		else if( bIsPaused )
		{
			FTickTaskManagerInterface::Get().RunPauseFrame(this, DeltaSeconds, LEVELTICK_PauseTick, LevelsToTick);
		}
		
		// Run this on the first collection with levels, which will be DynamicSourceLevels by default
		if (!bHasTickedACollection)
		{
			bHasTickedACollection = true;

			// Process any remaining latent actions
			if( !bIsPaused )
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FlushLatentActions);
				// This will process any latent actions that have not been processed already
				CurrentLatentActionManager.ProcessLatentActions(nullptr, DeltaSeconds);
			}
#if 0 // if you need to debug physics drawing in editor, use this. If you type pxvis collision, it will work. 
			else
			{
				// Tick our async work (physics, etc.) and tick with no elapsed time for playersonly
				TickAsyncWork(0.f);
				// Wait for async work to come back
				WaitForAsyncWork();
			}
#endif

			{
				SCOPE_CYCLE_COUNTER(STAT_TickableTickTime);

				if (TickType != LEVELTICK_TimeOnly && !bIsPaused)
				{
					SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TimerManager"), 5);
					STAT(FScopeCycleCounter Context(GetTimerManager().GetStatId());)
					GetTimerManager().Tick(DeltaSeconds);
				}

				{
					SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TickObjects"), 5);
					FTickableGameObject::TickObjects(this, TickType, bIsPaused, DeltaSeconds);
				}
			}

			// Kick early end of frame updates after the major tick managers run, as they are the most likely to contain systems that touch components.
			StartAsyncSendAllEndOfFrameUpdates();

			// Update cameras
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			if (IsActorTickAndUserCallbacksEnabled())
#endif
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateCameraTime);
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Camera);
				// Update cameras last. This needs to be done before NetUpdates, and after all actors have been ticked.
				for( FConstPlayerControllerIterator Iterator = GetPlayerControllerIterator(); Iterator; ++Iterator )
				{
					if (Iterator->GetResidence() == EResidence::Remote)
					{
						continue;
					}
					
					if (APlayerController* PlayerController = Iterator->Get())
					{
						if (!bIsPaused || PlayerController->ShouldPerformFullTickWhenPaused())
						{
							PlayerController->UpdateCameraManager(DeltaSeconds);
						}
						else if (PlayerController->PlayerCameraManager && FCameraPhotographyManager::IsSupported(this))
						{
							PlayerController->PlayerCameraManager->UpdateCameraPhotographyOnly();
						}
					}
				}
			}

			// Update streaming volumes
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
			if (IsActorTickAndUserCallbacksEnabled())
#endif
			{
				if (!bIsPaused && IsGameWorld())
				{
					// Update world's required streaming levels
					InternalUpdateStreamingState();
				}
			}
		}

		if (bDoingActorTicks)
		{
			SCOPE_CYCLE_COUNTER(STAT_TickTime);
			{
				SCOPE_CYCLE_COUNTER(STAT_TG_PostUpdateWork);
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - PostUpdateWork"), 5);
				CSV_SCOPED_SET_WAIT_STAT(PostUpdateWork);
				RunTickGroup(TG_PostUpdateWork);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_TG_LastDemotable);
				SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - TG_LastDemotable"), 5);
				CSV_SCOPED_SET_WAIT_STAT(LastDemotable);
				RunTickGroup(TG_LastDemotable);
			}

			FTickTaskManagerInterface::Get().EndFrame();
		}
	}

#if WITH_EDITOR
	// Tick LevelInstanceSubsystem outside of FTickTaskManagerInterface::StartFrame/EndFrame because it can cause levels to be deleted and invalidate its LevelList
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetSubsystem<ULevelInstanceSubsystem>())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LevelInstanceSubsystem);
		LevelInstanceSubsystem->Tick();
	}
#endif

	if (bDoingActorTicks)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(bDoingActorTicks);
		SCOPE_CYCLE_COUNTER(STAT_TickTime);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FWorldDelegates::OnWorldPostActorTick)
			FWorldDelegates::OnWorldPostActorTick.Broadcast(this, TickType, DeltaSeconds);
		}

		// All tick is done, execute async trace
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FinishAsyncTrace);
			SCOPE_CYCLE_COUNTER(STAT_FinishAsyncTraceTickTime);
			SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - FinishAsyncTrace"), 5);
			FinishAsyncTrace();
		}
	}

	{
		STAT(FParticleMemoryStatManager::UpdateStats());
	}

	// Update net and flush networking.
    // Tick all net drivers
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NetBroadcastTickTime);
		SCOPE_CYCLE_COUNTER(STAT_NetBroadcastTickTime);
		LLM_SCOPE(ELLMTag::Networking);
		BroadcastPreTickFlush(RealDeltaSeconds);
		BroadcastTickFlush(RealDeltaSeconds); // note: undilated time is being used here
	}
	
     // PostTick all net drivers
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BroadcastPostTickFlush);
		LLM_SCOPE(ELLMTag::Networking);
		BroadcastPostTickFlush(RealDeltaSeconds); // note: undilated time is being used here
	}

	if( Scene )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateSpeedTreeWind);
		// Update SpeedTree wind objects.
		Scene->UpdateSpeedTreeWind(TimeSeconds);
	}

	// Tick the FX system.
	if (!bIsPaused && FXSystem != nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FXSystem);
		SCOPE_TIME_GUARD_MS(TEXT("UWorld::Tick - FX"), 5);
		FXSystem->Tick(this, DeltaSeconds);
	}

#if WITH_EDITOR
	// Finish up.
	bDebugFrameStepExecutedThisFrame = bDebugFrameStepExecution;
	if(bDebugFrameStepExecution)
	{
		bDebugPauseExecution = true;
		bDebugFrameStepExecution = false;
	}
#endif


	bInTick = false;
	Mark.Pop();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalCollectGarbage);
		GEngine->ConditionalCollectGarbage();
	}
	
	TRACE_COUNTER_SET(UObjectCount, GUObjectArray.GetObjectArrayNumMinusAvailable());

	// players only request from last frame
	if (bPlayersOnlyPending)
	{
		bPlayersOnly = bPlayersOnlyPending;
		bPlayersOnlyPending = false;
	}

#if LOG_DETAILED_PATHFINDING_STATS
	GDetailedPathFindingStats.DumpStats();
#endif

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)
	RecordWorldCountsToCSV(this, bDoingActorTicks);

	if (IsGameWorld() && FCsvProfiler::Get()->IsCapturing())
	{
		if (ULevelStreamingProfilingSubsystem* LevelStreamingProfilingSubsystem = GetSubsystem<ULevelStreamingProfilingSubsystem>())
		{
			LevelStreamingProfilingSubsystem->RecordLevelStreamingCountsToCSV();
		}
	}
#endif // (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	GShouldLogOutAFrameOfMoveComponent = false;
	GShouldLogOutAFrameOfSetBodyTransform = false;

#if LOOKING_FOR_PERF_ISSUES || !WITH_EDITORONLY_DATA
	extern TArray<FString>	ThisFramePawnSpawns;
	if(ThisFramePawnSpawns.Num() > 1 && IsGameWorld() && !GIsServer && GEngine->bCheckForMultiplePawnsSpawnedInAFrame )
	{
		const FString WarningMessage = FString::Printf( TEXT("%d PAWN SPAWNS THIS FRAME! "), ThisFramePawnSpawns.Num() );

		UE_LOGF(LogLevel, Warning, "%ls", *WarningMessage );
		// print out the pawns that were spawned
		for(int32 i=0; i<ThisFramePawnSpawns.Num(); i++)
		{
			UE_LOGF(LogLevel, Warning, "%ls", *ThisFramePawnSpawns[i]);
		}

		if( IsGameWorld() && GAreScreenMessagesEnabled && ThisFramePawnSpawns.Num() > GEngine->NumPawnsAllowedToBeSpawnedInAFrame )
		{
			GEngine->AddOnScreenDebugMessage((uint64)((PTRINT)this), 5.f, FColor::Red, *WarningMessage);

			for(int32 i=0; i<ThisFramePawnSpawns.Num(); i++)
			{
				GEngine->AddOnScreenDebugMessage((uint64)((PTRINT)+i), 5.f, FColor::Red, *ThisFramePawnSpawns[i]);
			}
		}
	}
	ThisFramePawnSpawns.Empty();
#endif
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if WITH_EDITOR
	if(GIsEditor && bDoDelayedUpdateCullDistanceVolumes)
	{
		bDoDelayedUpdateCullDistanceVolumes = false;
		UpdateCullDistanceVolumes();
	}
#endif // WITH_EDITOR

	// Dump the viewpoints with which we were rendered last frame. They will be updated when the world is next rendered.
	ViewLocationsRenderedLastFrame.Reset();
	CachedViewInfoRenderedLastFrame.Reset();

	if (GEngine->XRSystem.IsValid())
	{
		GEngine->XRSystem->OnEndGameFrame( GEngine->GetWorldContextFromWorldChecked( this ) );
	}

	UWorld* WorldParam = this;
	ENQUEUE_RENDER_COMMAND(TickInGamePerfTrackersRT)(
		[WorldParam](FRHICommandList& RHICmdList)
		{
			//Tick game and other thread trackers.
			if (WorldParam->PerfTrackers)
			{
				for (int32 Tracker = 0; Tracker < (int32)EInGamePerfTrackers::Num; ++Tracker)
				{
					WorldParam->PerfTrackers->GetInGamePerformanceTracker((EInGamePerfTrackers)Tracker, EInGamePerfTrackerThreads::RenderThread).Tick();
				}
			}
		});

	FWorldDelegates::OnWorldTickEnd.Broadcast(this, TickType, DeltaSeconds);

#if WITH_STATE_STREAM
	StateStreamManager->Game_EndTick(RealTimeSeconds, LaneId);
	StateStreamManager->Game_BeginTick(LaneId);
#endif

	UE::Stats::FThreadIdleStats::EndCriticalPath();
}

void UWorld::CleanupActors()
{
	// Remove NULL entries from actor list. Only does so for dynamic actors to avoid resorting; in theory static 
	// actors shouldn't be deleted during gameplay.
	for (ULevel* Level : Levels)
	{
		// Don't compact actors array for levels that are currently in the process of being made visible or invisible
		// as the code that spreads this work across several frames relies on the actor count not changing as it keeps
		// an index into the array.
		if( ensure(Level != nullptr) && !IsLevelMakingVisible(Level) && !IsLevelMakingInvisible(Level))
		{
			// Actor 0 (world info) and 1 (default brush) are special and should never be removed from the actor array even if NULL
			const int32 FirstDynamicIndex = 2;
			int32 NumActorsToRemove = 0;
			// Remove NULL entries from array, we're iterating backwards to avoid unnecessary memcpys during removal.
			for( int32 ActorIndex=Level->Actors.Num()-1; ActorIndex>=FirstDynamicIndex; ActorIndex-- )
			{
				// To avoid shuffling things down repeatedly when not necessary count nulls and then remove in bunches
				if (Level->Actors[ActorIndex] == nullptr)
				{
					++NumActorsToRemove;
				}
				else if (NumActorsToRemove > 0)
				{
					Level->Actors.RemoveAt(ActorIndex+1, NumActorsToRemove, EAllowShrinking::No);
					NumActorsToRemove = 0;
				}
			}
			if (NumActorsToRemove > 0)
			{
				// If our FirstDynamicIndex (and any immediately following it) were null it won't get caught in the loop, so do a cleanup pass here
				Level->Actors.RemoveAt(FirstDynamicIndex, NumActorsToRemove, EAllowShrinking::No);
			}
		}
	}
}
