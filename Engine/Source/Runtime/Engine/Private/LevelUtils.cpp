// Copyright Epic Games, Inc. All Rights Reserved.


#include "LevelUtils.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "EditorSupportDelegates.h"
#include "Misc/FeedbackContext.h"
#include "GameFramework/WorldSettings.h"
#include "Components/ModelComponent.h"
#include "Streaming/ServerStreamingLevelsVisibility.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#endif

#define LOCTEXT_NAMESPACE "LevelUtils"


#if WITH_EDITOR
// Structure to hold the state of the Level file on disk, the goal is to query it only one time per frame.
struct FLevelReadOnlyData
{
	FLevelReadOnlyData()
		:IsReadOnly(false)
		,LastUpdateTime(-1.0f)
	{}
	/** the current level file state */
	bool IsReadOnly;
	/** Last time when the level file state was update */
	float LastUpdateTime;
};
// Map to link the level data with a level
static TMap<ULevel*, FLevelReadOnlyData> LevelReadOnlyCache;

namespace LevelUtilsInternal
{
	static void ApplyEditorTransform(ULevel* LoadedLevel, bool bDoPostEditMove, AActor* Actor, const FTransform& Transform)
	{
		if (LoadedLevel)
		{
			FLevelUtils::FApplyLevelTransformParams TransformParams(LoadedLevel, Transform);

			while (AActor* AttachParent = Actor ? Actor->GetAttachParentActor() : nullptr)
			{
				Actor = AttachParent;
			}

			TransformParams.bDoPostEditMove = bDoPostEditMove;
			TransformParams.Actor = Actor;

			FLevelUtils::ApplyLevelTransform(TransformParams);
		}
	}
}

#endif

/////////////////////////////////////////////////////////////////////////////////////////
//
//	FindStreamingLevel methods.
//
/////////////////////////////////////////////////////////////////////////////////////////


#if WITH_EDITOR
bool FLevelUtils::bMovingLevel = false;
bool FLevelUtils::bApplyingLevelTransform = false;
#endif

ULevelStreaming* FLevelUtils::FindStreamingLevel(const ULevel* Level)
{
	return ULevelStreaming::FindStreamingLevel(Level);
}

ULevelStreaming* FLevelUtils::FindStreamingLevel(UWorld* InWorld, const FName PackageName)
{
	ULevelStreaming* MatchingLevel = NULL;
	if (InWorld && !PackageName.IsNone())
	{
		for (ULevelStreaming* CurStreamingLevel : InWorld->GetStreamingLevels())
		{
			if (CurStreamingLevel && CurStreamingLevel->GetWorldAssetPackageFName() == PackageName)
			{
				MatchingLevel = CurStreamingLevel;
				break;
			}
		}
	}
	return MatchingLevel;
}

ULevelStreaming* FLevelUtils::FindStreamingLevel(UWorld* InWorld, const TCHAR* InPackageName)
{
	return FindStreamingLevel(InWorld, FName(InPackageName));
}

bool FLevelUtils::IsValidStreamingLevel(UWorld* InWorld, const TCHAR* InPackageName)
{
	if (FindStreamingLevel(InWorld, InPackageName))
	{
		return true;
	}

#if WITH_EDITOR
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = InWorld ? InWorld->GetSubsystem<UWorldPartitionSubsystem>() : nullptr)
	{
		bool bIsValidStreamingLevel = false;
		WorldPartitionSubsystem->ForEachWorldPartition([&bIsValidStreamingLevel, InPackageName](UWorldPartition* WorldPartition)
		{
			bIsValidStreamingLevel = WorldPartition->IsValidPackageName(InPackageName);
			return !bIsValidStreamingLevel;
		});
		return bIsValidStreamingLevel;
	}
#endif

	return false;
}

bool FLevelUtils::SupportsMakingVisibleTransactionRequests(UWorld* InWorld)
{
	return InWorld && InWorld->SupportsMakingVisibleTransactionRequests();
}

bool FLevelUtils::SupportsMakingInvisibleTransactionRequests(UWorld* InWorld)
{
	return InWorld && InWorld->SupportsMakingInvisibleTransactionRequests();
}

bool FLevelUtils::IsServerStreamingLevelVisible(UWorld* InWorld, const FName& InPackageName)
{
	// If there's no implementation for this world to query the server visible streaming levels, return true
	if (!SupportsMakingVisibleTransactionRequests(InWorld))
	{
		return true;
	}

	const AServerStreamingLevelsVisibility* ServerStreamingLevelsVisibility = InWorld->GetServerStreamingLevelsVisibility();
	return ServerStreamingLevelsVisibility && ServerStreamingLevelsVisibility->Contains(InPackageName);
}

ULevelStreaming* FLevelUtils::GetServerVisibleStreamingLevel(UWorld* InWorld, const FName& InPackageName)
{
	if (SupportsMakingVisibleTransactionRequests(InWorld))
	{
		if (const AServerStreamingLevelsVisibility* ServerStreamingLevelsVisibility = InWorld->GetServerStreamingLevelsVisibility())
		{
			return ServerStreamingLevelsVisibility->GetVisibleStreamingLevel(InPackageName);
		}
	}
	return nullptr;
}

/////////////////////////////////////////////////////////////////////////////////////////
//
//	Level locking/unlocking.
//
/////////////////////////////////////////////////////////////////////////////////////////

/**
 * Returns true if the specified level is locked for edit, false otherwise.
 *
 * @param	Level		The level to query.
 * @return				true if the level is locked, false otherwise.
 */
#if WITH_EDITOR
bool FLevelUtils::IsLevelLocked(ULevel* Level)
{
	//We should not check file status on disk if we are not running the editor
	// Don't permit spawning in read only levels if they are locked
	if ( GIsEditor && !UE::GetIsEditorLoadingPackage() )
	{
		if ( GEngine && GEngine->bLockReadOnlyLevels )
		{
			if (!LevelReadOnlyCache.Contains(Level))
			{
				LevelReadOnlyCache.Add(Level, FLevelReadOnlyData());
			}
			check(LevelReadOnlyCache.Contains(Level));
			FLevelReadOnlyData &LevelData = LevelReadOnlyCache[Level];
			//Make sure we test if the level file on disk is readonly only once a frame,
			//when the frame time get updated.
			if (LevelData.LastUpdateTime < Level->OwningWorld->GetRealTimeSeconds())
			{
				LevelData.LastUpdateTime = Level->OwningWorld->GetRealTimeSeconds();
				//If we dont find package we dont consider it as readonly
				LevelData.IsReadOnly = false;
				const UPackage* pPackage = Level->GetOutermost();
				if (pPackage)
				{
					FString PackageFileName;
					if (FPackageName::DoesPackageExist(pPackage->GetName(), &PackageFileName))
					{
						LevelData.IsReadOnly = IFileManager::Get().IsReadOnly(*PackageFileName);
					}
				}
			}

			if (LevelData.IsReadOnly)
			{
				return true;
			}
		}
	}

	// PIE levels and transient move levels are usually never locked.
	if ( Level->RootPackageHasAnyFlags(PKG_PlayInEditor) || Level->GetName() == TEXT("TransLevelMoveBuffer") )
	{
		return false;
	}

	ULevelStreaming* StreamingLevel = FindStreamingLevel( Level );
	if ( StreamingLevel != NULL )
	{
		return StreamingLevel->bLocked;
	}
	else
	{
		return Level->bLocked;
	}
}
bool FLevelUtils::IsLevelLocked( AActor* Actor )
{
	return Actor != NULL && !Actor->IsTemplate() && Actor->GetLevel() != NULL && IsLevelLocked(Actor->GetLevel());
}

/**
 * Sets a level's edit lock.
 *
 * @param	Level		The level to modify.
 */
void FLevelUtils::ToggleLevelLock(ULevel* Level)
{
	if ( !Level )
	{
		return;
	}

	ULevelStreaming* StreamingLevel = FindStreamingLevel( Level );
	if ( StreamingLevel != NULL )
	{
		// We need to set the RF_Transactional to make a streaming level serialize itself. so store the original ones, set the flag, and put the original flags back when done
		EObjectFlags cachedFlags = StreamingLevel->GetFlags();
		StreamingLevel->SetFlags( RF_Transactional );
		StreamingLevel->Modify();			
		StreamingLevel->SetFlags( cachedFlags );

		StreamingLevel->bLocked = !StreamingLevel->bLocked;
	}
	else
	{
		Level->Modify();
		Level->bLocked = !Level->bLocked;	
	}
}
#endif //#if WITH_EDITOR

/////////////////////////////////////////////////////////////////////////////////////////
//
//	Level loading/unloading.
//
/////////////////////////////////////////////////////////////////////////////////////////

/**
 * Returns true if the level is currently loaded in the editor, false otherwise.
 *
 * @param	Level		The level to query.
 * @return				true if the level is loaded, false otherwise.
 */
bool FLevelUtils::IsLevelLoaded(ULevel* Level)
{
	if ( Level && Level->IsPersistentLevel() )
	{
		// The persistent level is always loaded.
		return true;
	}

	ULevelStreaming* StreamingLevel = FindStreamingLevel( Level );
	return (StreamingLevel != nullptr);
}


/////////////////////////////////////////////////////////////////////////////////////////
//
//	Level visibility.
//
/////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA
/**
 * Returns true if the specified level is visible in the editor, false otherwise.
 *
 * @param	StreamingLevel		The level to query.
 */
bool FLevelUtils::IsStreamingLevelVisibleInEditor(const ULevelStreaming* StreamingLevel)
{
	const bool bVisible = StreamingLevel && StreamingLevel->GetShouldBeVisibleInEditor();
	return bVisible;
}
#endif

/**
 * Returns true if the specified level is visible in the editor, false otherwise.
 *
 * @param	Level		The level to query.
 */
bool FLevelUtils::IsLevelVisible(const ULevel* Level)
{
	if (!Level)
	{
		return false;
	}

	// P-level is specially handled
	if ( Level->IsPersistentLevel() )
	{
#if WITH_EDITORONLY_DATA
		return !( Level->OwningWorld->PersistentLevel->GetWorldSettings()->bHiddenEdLevel );
#else
		return true;
#endif
	}

	static const FName NAME_TransLevelMoveBuffer("TransLevelMoveBuffer");
	if (Level->GetFName() == NAME_TransLevelMoveBuffer)
	{
		// The TransLevelMoveBuffer does not exist in the streaming list and is never visible
		return false;
	}

	return Level->bIsVisible;
}

#if WITH_EDITOR
/////////////////////////////////////////////////////////////////////////////////////////
//
//	Level editor transforms.
//
/////////////////////////////////////////////////////////////////////////////////////////

void FLevelUtils::SetEditorTransform(ULevelStreaming* StreamingLevel, const FTransform& Transform, bool bDoPostEditMove )
{
	check(StreamingLevel);

	// Check we are actually changing the value
	if(StreamingLevel->LevelTransform.Equals(Transform))
	{
		return;
	}

	// Setup an Undo transaction
	const FScopedTransaction LevelOffsetTransaction( LOCTEXT( "ChangeEditorLevelTransform", "Edit Level Transform" ) );
	StreamingLevel->Modify();

	bool bNeedsToRebuildLevelBSP = false;
	if (ULevel* LoadedLevel = StreamingLevel->GetLoadedLevel())
	{
		for (AActor* Actor : LoadedLevel->Actors)
		{
			// Ensure that all Actors are in the transaction so that their location is restored and any construction script behaviors 
			// based on being at a different location are correctly applied on undo/redo
			if (Actor)
			{
				Actor->Modify();
			}

			// Iterate over all actors in the level to see if we need to rebuild the Level BSP(i.e., moving any Brush actor affecting BSP)
			if (!bNeedsToRebuildLevelBSP && GEditor)
			{
				if (ABrush* Brush = Cast<ABrush>(Actor))
				{
					bNeedsToRebuildLevelBSP = Brush->CanEverAffectBSP();
				}
			}
		}
	}

	// Apply new transform
	RemoveEditorTransform(StreamingLevel, false );
	StreamingLevel->LevelTransform = Transform;
	ApplyEditorTransform(StreamingLevel, bDoPostEditMove);

	if (bNeedsToRebuildLevelBSP)
	{
		check(GEditor);
		check(StreamingLevel->GetLoadedLevel());

		// RebuildAlteredBSP() only tracks the levels of currently selected actors if they are brushes
		// which is not applicable when we are moving the whole level
		// So we need to build the specific level BSP
		GEditor->RebuildLevel(*StreamingLevel->GetLoadedLevel());
	}

	// Redraw the viewports to see this change
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void FLevelUtils::ApplyEditorTransform(const ULevelStreaming* StreamingLevel, bool bDoPostEditMove, AActor* Actor)
{
	check(StreamingLevel);
	LevelUtilsInternal::ApplyEditorTransform(StreamingLevel->GetLoadedLevel(), bDoPostEditMove, Actor, StreamingLevel->LevelTransform);
}

void FLevelUtils::RemoveEditorTransform(const ULevelStreaming* StreamingLevel, bool bDoPostEditMove, AActor* Actor)
{
	check(StreamingLevel);
	LevelUtilsInternal::ApplyEditorTransform(StreamingLevel->GetLoadedLevel(), bDoPostEditMove, Actor, StreamingLevel->LevelTransform.Inverse());
}

void FLevelUtils::ApplyPostEditMove( ULevel* Level )
{	
	check(Level)	
	GWarn->BeginSlowTask( LOCTEXT( "ApplyPostEditMove", "Updating all actors in level after move" ), true);

	const int32 NumActors = Level->Actors.Num();

	// Iterate over all actors in the level and transform them
	bMovingLevel = true;
	for( int32 ActorIndex=0; ActorIndex < NumActors; ++ActorIndex )
	{
		GWarn->UpdateProgress( ActorIndex, NumActors );
		AActor* Actor = Level->Actors[ActorIndex];
		if( Actor )
		{
			if (!Actor->GetWorld()->IsGameWorld() )
			{
				Actor->PostEditMove(true);				
			}
		}
	}
	bMovingLevel = false;
	GWarn->EndSlowTask();	
}


bool FLevelUtils::IsMovingLevel()
{
	return bMovingLevel;
}

bool FLevelUtils::IsApplyingLevelTransform()
{
	return bApplyingLevelTransform;
}

#endif // WITH_EDITOR

void FLevelUtils::ApplyLevelTransform(const FLevelUtils::FApplyLevelTransformParams& TransformParams)
{
	const bool bTransformActors =  !TransformParams.LevelTransform.Equals(FTransform::Identity);
	if (bTransformActors)
	{
#if WITH_EDITOR
		TGuardValue<bool> ApplyingLevelTransformGuard(bApplyingLevelTransform, true);
#endif
		
		const auto ApplyLevelTransformToCompDirect = [&TransformParams](AActor* Actor, USceneComponent* Comp)
		{
			if (Comp->GetAttachParent() == nullptr)
			{
				Comp->SetRelativeLocation_Direct(TransformParams.LevelTransform.TransformPosition(Comp->GetRelativeLocation()));
				Comp->SetRelativeRotation_Direct(TransformParams.LevelTransform.TransformRotation(Comp->GetRelativeRotation().Quaternion()).Rotator());
				Comp->SetRelativeScale3D_Direct(TransformParams.LevelTransform.GetScale3D() * Comp->GetRelativeScale3D());
			}
			else if (Comp->IsUsingAbsoluteLocation() || Comp->IsUsingAbsoluteRotation() || Comp->IsUsingAbsoluteScale())
			{
				if (Comp->IsUsingAbsoluteLocation())
				{
					Comp->SetRelativeLocation_Direct(TransformParams.LevelTransform.TransformPosition(Comp->GetRelativeLocation()));
				}
				if (Comp->IsUsingAbsoluteRotation())
				{
					Comp->SetRelativeRotation_Direct(TransformParams.LevelTransform.TransformRotation(Comp->GetRelativeRotation().Quaternion()).Rotator());
				}
				if (Comp->IsUsingAbsoluteScale())
				{
					Comp->SetRelativeScale3D_Direct(TransformParams.LevelTransform.GetScale3D() * Comp->GetRelativeScale3D());
				}
			}
			else
			{
				return;
			}
			Actor->MarkNeedsRecomputeBoundsOnceForGame();
		};
		const auto ApplyLevelTransformToCompRelative = [&TransformParams](AActor* Actor, USceneComponent* Comp)
		{
			if (Comp->GetAttachParent() == nullptr)
			{
				Comp->SetRelativeLocationAndRotation(TransformParams.LevelTransform.TransformPosition(Comp->GetRelativeLocation()), TransformParams.LevelTransform.TransformRotation(Comp->GetRelativeRotation().Quaternion()));
				Comp->SetRelativeScale3D(TransformParams.LevelTransform.GetScale3D() * Comp->GetRelativeScale3D());
			}
			else if (Comp->IsUsingAbsoluteLocation() || Comp->IsUsingAbsoluteRotation() || Comp->IsUsingAbsoluteScale())
			{
				if (Comp->IsUsingAbsoluteLocation() || Comp->IsUsingAbsoluteRotation())
				{
					const FVector NewLoc = Comp->IsUsingAbsoluteLocation()
						? TransformParams.LevelTransform.TransformPosition(Comp->GetRelativeLocation())
						: Comp->GetRelativeLocation();
					const FQuat NewRot = Comp->IsUsingAbsoluteRotation()
						? TransformParams.LevelTransform.TransformRotation(Comp->GetRelativeRotation().Quaternion())
						: Comp->GetRelativeRotation().Quaternion();
					Comp->SetRelativeLocationAndRotation(NewLoc, NewRot);
				}
				if (Comp->IsUsingAbsoluteScale())
				{
					Comp->SetRelativeScale3D(TransformParams.LevelTransform.GetScale3D() * Comp->GetRelativeScale3D());
				}
			}
			else
			{
				return;
			}
			Actor->MarkNeedsRecomputeBoundsOnceForGame();
		};
		
		// Apply the transform only to the specified actor
		if (TransformParams.Actor)
		{
			if (TransformParams.bSetRelativeTransformDirectly)
			{
				if (TransformParams.Actor->GetRootComponent())
				{
					TInlineComponentArray<USceneComponent*> Components;
					TransformParams.Actor->GetComponents(Components);
					for (USceneComponent* Comp : Components)
					{
						ApplyLevelTransformToCompDirect(TransformParams.Actor, Comp);
					}
				}
			}
			else
			{
				if (TransformParams.Actor->GetRootComponent())
				{
					TInlineComponentArray<USceneComponent*> Components;
					TransformParams.Actor->GetComponents(Components);
					for (USceneComponent* Comp : Components)
					{
						ApplyLevelTransformToCompRelative(TransformParams.Actor, Comp);
					}
				}
			}
#if WITH_EDITOR
			if (TransformParams.bDoPostEditMove && !TransformParams.Actor->GetWorld()->IsGameWorld())
			{
				bMovingLevel = true;
				TransformParams.Actor->PostEditMove(true);
				bMovingLevel = false;
			}
#endif
			return;
		}
		// Otherwise do the usual

		if (!TransformParams.LevelTransform.GetRotation().IsIdentity())
		{
			// If there is a rotation applied, then the relative precomputed bounds become invalid.
			TransformParams.Level->bTextureStreamingRotationChanged = true;
		}

		if (TransformParams.bSetRelativeTransformDirectly)
		{
			// Iterate over all model components to transform BSP geometry accordingly
			for (UModelComponent* ModelComponent : TransformParams.Level->ModelComponents)
			{
				if (ModelComponent)
				{
					ModelComponent->SetRelativeLocation_Direct(TransformParams.LevelTransform.TransformPosition(ModelComponent->GetRelativeLocation()));
					ModelComponent->SetRelativeRotation_Direct(TransformParams.LevelTransform.TransformRotation(ModelComponent->GetRelativeRotation().Quaternion()).Rotator());
					ModelComponent->SetRelativeScale3D_Direct(TransformParams.LevelTransform.GetScale3D() * ModelComponent->GetRelativeScale3D());
				}
			}

			// Iterate over all actors in the level and transform them
			for (AActor* Actor : TransformParams.Level->Actors)
			{
				if (Actor)
				{
					if (Actor->GetRootComponent())
					{
						TInlineComponentArray<USceneComponent*> Components;
						Actor->GetComponents(Components);
						for (USceneComponent* Comp : Components)
						{
							ApplyLevelTransformToCompDirect(Actor, Comp);
						}
					}
				}
			}
		}
		else
		{
			// Iterate over all model components to transform BSP geometry accordingly
			for (UModelComponent* ModelComponent : TransformParams.Level->ModelComponents)
			{
				if (ModelComponent)
				{
					ModelComponent->SetRelativeLocationAndRotation(TransformParams.LevelTransform.TransformPosition(ModelComponent->GetRelativeLocation()), TransformParams.LevelTransform.TransformRotation(ModelComponent->GetRelativeRotation().Quaternion()));
					ModelComponent->SetRelativeScale3D(TransformParams.LevelTransform.GetScale3D() * ModelComponent->GetRelativeScale3D());
				}
			}

			// Iterate over all actors in the level and transform them
			for (AActor* Actor : TransformParams.Level->Actors)
			{
				if (Actor)
				{
					if (Actor->GetRootComponent())
					{
						TInlineComponentArray<USceneComponent*> Components;
						Actor->GetComponents(Components);
						for (USceneComponent* Comp : Components)
						{
							ApplyLevelTransformToCompRelative(Actor, Comp);
						}
					}
				}
			}
		}

#if WITH_EDITOR
		if (TransformParams.bDoPostEditMove)
		{
			ApplyPostEditMove(TransformParams.Level);
		}
#endif // WITH_EDITOR

		TransformParams.Level->OnApplyLevelTransform.Broadcast(TransformParams.LevelTransform);
		FWorldDelegates::PostApplyLevelTransform.Broadcast(TransformParams.Level, TransformParams.LevelTransform);
	}
}

#undef LOCTEXT_NAMESPACE
