// Copyright Epic Games, Inc. All Rights Reserved.


#include "Engine/Light.h"
#include "Engine/Level.h"
#include "Engine/SpotLight.h"
#include "Engine/World.h"
#include "Components/ArrowComponent.h"
#include "Engine/PointLight.h"
#include "Engine/DirectionalLight.h"
#include "Engine/GeneratedMeshAreaLight.h"
#include "LightingBuildOptions.h"
#include "Net/UnrealNetwork.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/PointLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Light)

ALight::ALight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LightComponent = CreateDefaultSubobject<ULightComponent>(TEXT("LightComponent0"));

	bCollideWhenPlacing = true;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;
}

/** The quality level to use for half-resolution lightmaps (not exposed)		*/
ELightingBuildQuality FLightingBuildOptions::HalfResolutionLightmapQualityLevel = Quality_Medium;

/**
 * @return true if the lighting should be built for the level, given the current set of lighting build options.
 */
bool FLightingBuildOptions::ShouldBuildLightingForLevel(ULevel* Level) const
{
	// Reject NULL levels.
	if (Level == NULL)
	{
		return false;
	}

	if ( bOnlyBuildCurrentLevel )
	{
		// Reject non-current levels.
		if ( Level != Level->OwningWorld->GetCurrentLevel() )
		{
			return false;
		}
	}
	else if ( bOnlyBuildSelectedLevels )
	{
		// Reject unselected levels.
		if ( !SelectedLevels.Contains( Level ) )
		{
			return false;
		}
	}

	return true;
}

void ALight::Destroyed()
{
	if (LightComponent)
	{
		// Mark the light as not affecting the world before updating the shadowmap channel allocation
		LightComponent->bAffectsWorld = false;

		UWorld* World = GetWorld();

		if (World && !World->IsGameWorld())
		{
			// Force stationary light channel preview to be updated on editor delete
			LightComponent->InvalidateLightingCache();
		}
	}

	Super::Destroyed();
}

void ALight::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );
	
	DOREPLIFETIME( ALight, bEnabled );
}

void ALight::OnRep_bEnabled()
{
	LightComponent->SetVisibility(bEnabled);
}

void ALight::SetMobility(EComponentMobility::Type InMobility)
{
	if(LightComponent)
	{
		LightComponent->SetMobility(InMobility);
	}
}


void ALight::SetEnabled(bool bSetEnabled)
{
	if(LightComponent)
	{
		LightComponent->SetVisibility(bSetEnabled);
	}
}

bool ALight::IsEnabled() const
{
	return LightComponent ? LightComponent->GetVisibleFlag() : false;
}

void ALight::ToggleEnabled()
{
	if(LightComponent)
	{
		LightComponent->ToggleVisibility();
	}
}

void ALight::SetBrightness(float NewBrightness)
{
	if(LightComponent)
	{
		LightComponent->SetIntensity(NewBrightness);
	}
}

float ALight::GetBrightness() const
{
	return LightComponent ? LightComponent->Intensity : 0.f;
}

void ALight::SetLightColor(FLinearColor NewLightColor)
{
	if(LightComponent)
	{
		LightComponent->SetLightColor(NewLightColor);
	}
}

FLinearColor ALight::GetLightColor() const
{
	return LightComponent ? FLinearColor(LightComponent->LightColor) : FLinearColor::Black;
}

void ALight::SetLightFunctionMaterial(UMaterialInterface* NewLightFunctionMaterial)
{
	LightComponent->SetLightFunctionMaterial(NewLightFunctionMaterial);
}

void ALight::SetLightFunctionScale(FVector NewLightFunctionScale)
{
	LightComponent->SetLightFunctionScale(NewLightFunctionScale);
}

void ALight::SetLightFunctionFadeDistance(float NewLightFunctionFadeDistance)
{
	LightComponent->SetLightFunctionFadeDistance(NewLightFunctionFadeDistance);
}

void ALight::SetCastShadows(bool bNewValue)
{
	LightComponent->SetCastShadows(bNewValue);
}

void ALight::SetAffectTranslucentLighting(bool bNewValue)
{
	LightComponent->SetAffectTranslucentLighting(bNewValue);
}

void ALight::PostLoad()
{
	Super::PostLoad();

	if (LightComponent && LightComponent->Mobility == EComponentMobility::Static)
	{
		LightComponent->ClearLightFunctionMaterial();
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

APointLight::APointLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UPointLightComponent>(TEXT("LightComponent0")))
{
	PointLightComponent = CastChecked<UPointLightComponent>(GetLightComponent());
	PointLightComponent->Mobility = EComponentMobility::Stationary;

	RootComponent = PointLightComponent;
}

#if WITH_EDITOR
void APointLight::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_LIGHT_MOBILITY_CLASSES)
	{
		static FName PointLightStatic_NAME("PointLightStatic");
		static FName PointLightMovable_NAME("PointLightMovable");
		static FName PointLightStationary_NAME("PointLightStationary");

		check(GetLightComponent() != NULL);

		if(OldClassName == PointLightStatic_NAME)
		{
			GetLightComponent()->Mobility = EComponentMobility::Static;
		}
		else if(OldClassName == PointLightMovable_NAME)
		{
			GetLightComponent()->Mobility = EComponentMobility::Movable;
		}
		else if(OldClassName == PointLightStationary_NAME)
		{
			GetLightComponent()->Mobility = EComponentMobility::Stationary;
		}
	}
}
#endif // WITH_EDITOR

ADirectionalLight::ADirectionalLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UDirectionalLightComponent>(TEXT("LightComponent0")))
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName ID_Lighting;
		FText NAME_Lighting;
		FConstructorStatics()
			: ID_Lighting(TEXT("Lighting"))
			, NAME_Lighting(NSLOCTEXT( "SpriteCategory", "Lighting", "Lighting" ))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if !WITH_EDITORONLY_DATA
	UDirectionalLightComponent* DirectionalLightComponent;
#endif
	DirectionalLightComponent = CastChecked<UDirectionalLightComponent>(GetLightComponent());
	DirectionalLightComponent->Mobility = EComponentMobility::Stationary;
	DirectionalLightComponent->SetRelativeRotation(FRotator(-46.0f, 0.0f, 0.0f));
	// Make directional light icons big since they tend to be important
	// This is the root component so its scale affects all other components
	DirectionalLightComponent->SetRelativeScale3D(FVector(2.5f, 2.5f, 2.5f));

	RootComponent = DirectionalLightComponent;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));
	if (ArrowComponent)
	{
		ArrowComponent->ArrowColor = FColor(150, 200, 255);

		ArrowComponent->bTreatAsASprite = true;
		ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Lighting;
		ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Lighting;
		ArrowComponent->SetupAttachment(DirectionalLightComponent);
		ArrowComponent->bLightAttachment = true;
		ArrowComponent->bIsScreenSizeScaled = true;
	}

	bIsSpatiallyLoaded = false;
#endif // WITH_EDITORONLY_DATA

}

void ADirectionalLight::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if(ArrowComponent != nullptr)
	{
		ArrowComponent->ArrowColor = GetLightColor().ToFColor(true);
	}
#endif
}

#if WITH_EDITOR
void ADirectionalLight::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_LIGHT_MOBILITY_CLASSES)
	{
		static FName DirectionalLightStatic_NAME("DirectionalLightStatic");
		static FName DirectionalLightMovable_NAME("DirectionalLightMovable");
		static FName DirectionalLightStationary_NAME("DirectionalLightStationary");

		UDirectionalLightComponent* DirLightComp = CastChecked<UDirectionalLightComponent>(GetLightComponent());

		if(OldClassName == DirectionalLightStatic_NAME)
		{
			DirLightComp->Mobility = EComponentMobility::Static;
		}
		else if(OldClassName == DirectionalLightMovable_NAME)
		{
			DirLightComp->Mobility = EComponentMobility::Movable;
			DirLightComp->DynamicShadowDistanceMovableLight = DirLightComp->WholeSceneDynamicShadowRadius_DEPRECATED; 
		}
		else if(OldClassName == DirectionalLightStationary_NAME)
		{
			DirLightComp->Mobility = EComponentMobility::Stationary;

			// copy radius to correct var, if it had been changed
			if(DirLightComp->WholeSceneDynamicShadowRadius_DEPRECATED != 20000.f)
			{
				DirLightComp->DynamicShadowDistanceStationaryLight = DirLightComp->WholeSceneDynamicShadowRadius_DEPRECATED;  // copy radius to correct var
			}
		}
	}
}

void ADirectionalLight::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(ArrowComponent != nullptr)
	{
		ArrowComponent->ArrowColor = GetLightColor().ToFColor(true);
	}
}
#endif // WITH_EDITOR

void APointLight::SetRadius(float NewRadius)
{
	PointLightComponent->SetAttenuationRadius(NewRadius);
}

void APointLight::SetLightFalloffExponent(float NewLightFalloffExponent)
{
	PointLightComponent->SetLightFalloffExponent(NewLightFalloffExponent);
}

#if WITH_EDITOR
void APointLight::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 10000.0f : 100.0f );

	FMath::ApplyScaleToFloat( PointLightComponent->AttenuationRadius, ModifiedScale, 1.0f );
	PostEditChange();
}
#endif

AGeneratedMeshAreaLight::AGeneratedMeshAreaLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bEditable = false;
#endif // WITH_EDITORONLY_DATA

	GetLightComponent()->CastStaticShadows = false;
}

bool ALight::IsToggleable() const
{
	return !LightComponent->HasStaticLighting();
}


