// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightComponentId.h"
#include "HAL/ThreadSafeCounter.h"
#include "AutoRTFM.h"

FThreadSafeCounter FLightComponentId::NextLightId = 0;

FLightComponentId FLightComponentId::GetNextId()
{
	return FLightComponentId{ AutoRTFM::Open([]() -> uint32 { return NextLightId.Increment(); }) };
}