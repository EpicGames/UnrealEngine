// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightMap.cpp: Light-map implementation.
=============================================================================*/

#include "LightMap.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Level.h"
#include "Misc/QueuedThreadPool.h"
#include "ShadowMap.h"
#include "Engine/ShadowMapTexture2D.h"
#include "UnrealEngine.h"
#include "Interfaces/ITargetPlatform.h"
#include "RenderUtils.h"
#include "StaticLighting.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/LightComponent.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "Misc/FeedbackContext.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "VT/LightmapVirtualTexture.h"
#include "VT/VirtualTexture.h"
#include "EngineModule.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "TextureResource.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "StaticMeshLight.h"
#include "Engine/InstancedStaticMesh.h"
#include "Templates/Greater.h"

#define VISUALIZE_PACKING 0

DEFINE_LOG_CATEGORY_STATIC(LogLightMap, Log, All);

FLightmassDebugOptions GLightmassDebugOptions;

/** Whether to use bilinear filtering on lightmaps */
bool GUseBilinearLightmaps = true;

/** Whether to allow padding around mappings. */
bool GAllowLightmapPadding = true;

/** Counts the number of lightmap textures generated each lighting build. */
ENGINE_API int32 GLightmapCounter = 0;
/** Whether to compress lightmaps. Reloaded from ini each lighting build. */
ENGINE_API bool GCompressLightmaps = true;

/** Whether to allow lighting builds to generate streaming lightmaps. */
ENGINE_API bool GAllowStreamingLightmaps = false;

/** Largest boundingsphere radius to use when packing lightmaps into a texture atlas. */
ENGINE_API float GMaxLightmapRadius = 5000.0f;	//10000.0;	//2000.0f;

/** The quality level of the current lighting build */
ELightingBuildQuality GLightingBuildQuality = Quality_Preview;

#if WITH_EDITOR
	/** Information about the lightmap sample that is selected */
	UNREALED_API extern FSelectedLightmapSample GCurrentSelectedLightmapSample;
#endif

/** The color to set selected texels to */
ENGINE_API FColor GTexelSelectionColor(255, 50, 0);

#if WITH_EDITOR
	// NOTE: We're only counting the top-level mip-map for the following variables.
	/** Total number of texels allocated for all lightmap textures. */
	ENGINE_API uint64 GNumLightmapTotalTexels = 0;
	/** Total number of texels used if the texture was non-power-of-two. */
	ENGINE_API uint64 GNumLightmapTotalTexelsNonPow2 = 0;
	/** Number of lightmap textures generated. */
	ENGINE_API int32 GNumLightmapTextures = 0;
	/** Total number of mapped texels. */
	ENGINE_API uint64 GNumLightmapMappedTexels = 0;
	/** Total number of unmapped texels. */
	ENGINE_API uint64 GNumLightmapUnmappedTexels = 0;
	/** Whether to allow cropping of unmapped borders in lightmaps and shadowmaps. Controlled by BaseEngine.ini setting. */
	ENGINE_API bool GAllowLightmapCropping = false;
	/** Total lightmap texture memory size (in bytes), including GLightmapTotalStreamingSize. */
	ENGINE_API uint64 GLightmapTotalSize = 0;
	/** Total memory size for streaming lightmaps (in bytes). */
	ENGINE_API uint64 GLightmapTotalStreamingSize = 0;
#endif

static TAutoConsoleVariable<int32> CVarTexelDebugging(
	TEXT("r.TexelDebugging"),
	0,	
	TEXT("Whether T + Left mouse click in the editor selects lightmap texels for debugging Lightmass.  Lightmass must be recompiled with ALLOW_LIGHTMAP_SAMPLE_DEBUGGING enabled for this to work."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarVirtualTexturedLightMaps(
	TEXT("r.VirtualTexturedLightmaps"),
	0,
	TEXT("Controls wether to stream the lightmaps using virtual texturing.\n") \
	TEXT(" 0: Disabled.\n") \
	TEXT(" 1: Enabled."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarIncludeNonVirtualTexturedLightMaps(
	TEXT("r.IncludeNonVirtualTexturedLightmaps"),
	0,
	TEXT("If 'r.VirtualTexturedLightmaps' is enabled, controls whether non-VT lightmaps are generated/saved as well.\n") \
	TEXT("Including non-VT lightmaps will constrain lightmap atlas size, which removes some of the benefit of VT lightmaps.\n") \
	TEXT(" 0: Not included.\n") \
	TEXT(" 1: Included."));

static TAutoConsoleVariable<int32> CVarVTEnableLossyCompressLightmaps(
	TEXT("r.VT.EnableLossyCompressLightmaps"),
	0,
	TEXT("Enables lossy compression on virtual texture lightmaps. Lossy compression tends to have lower quality on lightmap textures, vs regular color textures."));

bool IsTexelDebuggingEnabled()
{
	return CVarTexelDebugging.GetValueOnGameThread() != 0;
}

#include "TextureLayout.h"

FLightMap::FLightMap()
	: bAllowHighQualityLightMaps(true)
	, NumRefs(0)
{
	bAllowHighQualityLightMaps = AllowHighQualityLightmaps(GMaxRHIFeatureLevel);
#if !PLATFORM_DESKTOP 
	checkf(bAllowHighQualityLightMaps || IsMobilePlatform(GMaxRHIShaderPlatform), TEXT("Low quality lightmaps are not currently supported on consoles. Make sure console variable r.HighQualityLightMaps is true for this platform"));
#endif
}

void FLightMap::Serialize(FArchive& Ar)
{
	Ar << LightGuids;
}

void FLightMap::Cleanup()
{
	BeginCleanup(this);
}

ULightMapTexture2D::ULightMapTexture2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LODGroup = TEXTUREGROUP_Lightmap;
}

void ULightMapTexture2D::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	uint32 Flags = LightmapFlags;
	Ar << Flags;
	LightmapFlags = ELightMapFlags( Flags );
}

/** 
 * Returns a one line description of an object for viewing in the generic browser
 */
FString ULightMapTexture2D::GetDesc()
{
	return FString::Printf( TEXT("Lightmap: %dx%d [%s]"), GetSizeX(), GetSizeY(), GPixelFormats[GetPixelFormat()].Name );
}

#if WITH_EDITOR
static void DumpLightmapSizeOnDisk()
{
	UE_LOGF(LogLightMap,Log,"Lightmap size on disk");
	UE_LOGF(LogLightMap,Log,"Source (KB),Source is Compressed,Platform Data (KB),Lightmap");
	for (TObjectIterator<ULightMapTexture2D> It; It; ++It)
	{
		ULightMapTexture2D* Lightmap = *It;
		UE_LOGF(LogLightMap,Log,"%f,%d,%f,%ls",
			Lightmap->Source.GetSizeOnDisk() / 1024.0f,
			Lightmap->Source.IsSourceCompressed(),
			Lightmap->CalcTextureMemorySizeEnum(TMC_AllMips) / 1024.0f,
			*Lightmap->GetPathName()
			);
	}
}

static FAutoConsoleCommand CmdDumpLightmapSizeOnDisk(
	TEXT("DumpLightmapSizeOnDisk"),
	TEXT("Dumps the size of all loaded lightmaps on disk (source and platform data)"),
	FConsoleCommandDelegate::CreateStatic(DumpLightmapSizeOnDisk)
	);
#endif // #if WITH_EDITOR

/** Lightmap resolution scaling factors for debugging.  The defaults are to use the original resolution unchanged. */
float TextureMappingDownsampleFactor0 = 1.0f;
int32 TextureMappingMinDownsampleSize0 = 16;
float TextureMappingDownsampleFactor1 = 1.0f;
int32 TextureMappingMinDownsampleSize1 = 128;
float TextureMappingDownsampleFactor2 = 1.0f;
int32 TextureMappingMinDownsampleSize2 = 256;

static int32 AdjustTextureMappingSize(int32 InSize)
{
	int32 NewSize = InSize;
	if (InSize > TextureMappingMinDownsampleSize0 && InSize <= TextureMappingMinDownsampleSize1)
	{
		NewSize = FMath::TruncToInt(InSize * TextureMappingDownsampleFactor0);
	}
	else if (InSize > TextureMappingMinDownsampleSize1 && InSize <= TextureMappingMinDownsampleSize2)
	{
		NewSize = FMath::TruncToInt(InSize * TextureMappingDownsampleFactor1);
	}
	else if (InSize > TextureMappingMinDownsampleSize2)
	{
		NewSize = FMath::TruncToInt(InSize * TextureMappingDownsampleFactor2);
	}
	return NewSize;
}

FStaticLightingMesh::FStaticLightingMesh(
	int32 InNumTriangles,
	int32 InNumShadingTriangles,
	int32 InNumVertices,
	int32 InNumShadingVertices,
	int32 InTextureCoordinateIndex,
	bool bInCastShadow,
	bool bInTwoSidedMaterial,
	const TArray<ULightComponent*>& InRelevantLights,
	const UPrimitiveComponent* const InComponent,
	const FBox& InBoundingBox,
	const FGuid& InGuid,
	const FGuid& InComponentGuid
	):
	NumTriangles(InNumTriangles),
	NumShadingTriangles(InNumShadingTriangles),
	NumVertices(InNumVertices),
	NumShadingVertices(InNumShadingVertices),
	TextureCoordinateIndex(InTextureCoordinateIndex),
	bCastShadow(bInCastShadow && InComponent->bCastStaticShadow),
	bTwoSidedMaterial(bInTwoSidedMaterial),
	RelevantLights(InRelevantLights),
	Component(InComponent),
	ComponentGuid(InComponentGuid),
	BoundingBox(InBoundingBox),
	Guid(FGuid::NewGuid()),
	SourceMeshGuid(InGuid),
	HLODTreeIndex(0),
	HLODChildStartIndex(0),
	HLODChildEndIndex(0)
{
	for (const ULightComponent* LightComponent : InRelevantLights)
	{
		RelevantLightsGuid.Add(LightComponent->LightGuid);
	}
}

FStaticLightingMesh::FStaticLightingMesh() : 
	NumTriangles(0),
	NumShadingTriangles(0),
	NumVertices(0),
	NumShadingVertices(0),
	TextureCoordinateIndex(0),
	bCastShadow(0),
	bTwoSidedMaterial(0),	
	Component(0)
{		
}

void FStaticLightingMesh::Serialize(FArchive& Ar)
{
	Ar << BoundingBox;
	Ar << Guid;
	Ar << ComponentGuid;
	Ar << RelevantLightsGuid;
	Ar << SourceMeshGuid;	
}

FStaticLightingTextureMapping::FStaticLightingTextureMapping(FStaticLightingMesh* InMesh,UObject* InOwner,int32 InSizeX,int32 InSizeY,int32 InLightmapTextureCoordinateIndex,bool bInBilinearFilter):
	FStaticLightingMapping(InMesh,InOwner),
	SizeX(AdjustTextureMappingSize(InSizeX)),
	SizeY(AdjustTextureMappingSize(InSizeY)),
	LightmapTextureCoordinateIndex(InLightmapTextureCoordinateIndex),
	bBilinearFilter(bInBilinearFilter)
{}

FStaticLightingGlobalVolumeMapping::FStaticLightingGlobalVolumeMapping(FStaticLightingMesh* InMesh, UObject* InOwner, int32 InSizeX, int32 InSizeY, int32 InLightmapTextureCoordinateIndex) :
	FStaticLightingTextureMapping(InMesh, InOwner, InSizeX, InSizeY, InLightmapTextureCoordinateIndex)
{}

#if WITH_EDITOR
void FStaticLightingMapping::Serialize(FArchive& Ar)
{	
	bool bIsInstancedMesh = Ar.IsLoading() ? false : Mesh->IsInstancedMesh();
	Ar << bIsInstancedMesh;

	/** The mesh associated with the mapping. */
	if (Ar.IsLoading())
	{
		if (bIsInstancedMesh)
		{
			Mesh = new FStaticLightingMesh_InstancedStaticMesh();
		}
		else
		{
			Mesh = new FStaticMeshStaticLightingMesh();
		}		
	}
				
	Mesh->Serialize(Ar);

	/** The object which owns the mapping. */
	FSoftObjectPath OwnerPath;
	if (Owner)
	{
		OwnerPath = FSoftObjectPath(Owner);
	}
	Ar << OwnerPath;
	Owner = OwnerPath.ResolveObject();

	/** true if the mapping should be processed by Lightmass. */
	bool bLocalProcessMapping = bProcessMapping;
	Ar << bLocalProcessMapping;
	bProcessMapping = bLocalProcessMapping;
}

void FStaticLightingTextureMapping::Serialize(FArchive& Ar)
{
	FStaticLightingMapping::Serialize(Ar);

	Ar << SizeX;
	Ar << SizeY;
	Ar << LightmapTextureCoordinateIndex;
	Ar << bBilinearFilter;
}

/**
 * An allocation of a region of light-map texture to a specific light-map.
 */
struct FLightMapAllocation
{
	/** 
	 * Basic constructor
	 */
	FLightMapAllocation()
	{
		MappedRect.Min.X = 0;
		MappedRect.Min.Y = 0;
		MappedRect.Max.X = 0;
		MappedRect.Max.Y = 0;
		Primitive = nullptr;
		Registry = NULL;
		InstanceIndex = INDEX_NONE;
		bSkipEncoding = false;
	}

	/**
	 * Copy construct from FQuantizedLightmapData
	 */
	FLightMapAllocation(FQuantizedLightmapData&& QuantizedData)
		: TotalSizeX(QuantizedData.SizeX)
		, TotalSizeY(QuantizedData.SizeY)
		, bHasSkyShadowing(QuantizedData.bHasSkyShadowing)
		, RawData(MoveTemp(QuantizedData.Data))
	{
		FMemory::Memcpy(Scale, QuantizedData.Scale, sizeof(Scale));
		FMemory::Memcpy(Add, QuantizedData.Add, sizeof(Add));
		PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
		MappedRect.Min.X = 0;
		MappedRect.Min.Y = 0;
		MappedRect.Max.X = TotalSizeX;
		MappedRect.Max.Y = TotalSizeY;
		Primitive = nullptr;
		InstanceIndex = INDEX_NONE;
		bSkipEncoding = false;
	}

	// Called after the lightmap is encoded
	void PostEncode()
	{
		if (InstanceIndex >= 0 && Registry)
		{
			FMeshMapBuildData* MeshBuildData = Registry->GetMeshBuildDataDuringBuild(MapBuildDataId);
			check(MeshBuildData);

			UInstancedStaticMeshComponent* Component = CastChecked<UInstancedStaticMeshComponent>(Primitive);

			// Instances may have been removed since LM allocation.
			// Instances may have also been shuffled from removes. We do not handle this case.
			if( InstanceIndex < MeshBuildData->PerInstanceLightmapData.Num() )
			{
				// TODO: We currently only support one LOD of static lighting in foliage
				// Need to create per-LOD instance data to fix that
				MeshBuildData->PerInstanceLightmapData[InstanceIndex].LightmapUVBias = FVector2f(LightMap->GetCoordinateBias());
				Component->SetBakedLightingDataChanged(InstanceIndex);
			}
		}
	}

	TRefCountPtr<FLightMap2D> LightMap;

	UPrimitiveComponent* Primitive;
	UMapBuildDataRegistry* Registry;
	FGuid MapBuildDataId;
	int32			InstanceIndex;

	/** Upper-left X-coordinate in the texture atlas. */
	int32				OffsetX;
	/** Upper-left Y-coordinate in the texture atlas. */
	int32				OffsetY;
	/** Total number of texels along the X-axis. */
	int32				TotalSizeX;
	/** Total number of texels along the Y-axis. */
	int32				TotalSizeY;
	/** The rectangle of mapped texels within this mapping that is placed in the texture atlas. */
	FIntRect		MappedRect;
	bool			bDebug;
	bool			bHasSkyShadowing;
	ELightMapPaddingType			PaddingType;
	TArray<FLightMapCoefficients>	RawData;
	float							Scale[NUM_STORED_LIGHTMAP_COEF][4];
	float							Add[NUM_STORED_LIGHTMAP_COEF][4];

	TMap<ULightComponent*, TArray<FQuantizedSignedDistanceFieldShadowSample> > ShadowMapData;

	/** True if we can skip encoding this allocation because it's similar enough to an existing
	    allocation at the same offset */
	bool bSkipEncoding;
};

struct FLightMapAllocationGroup
{
	FLightMapAllocationGroup()
		: Outer(nullptr)
		, LightmapFlags(LMF_None)
		, Bounds(ForceInit)
		, TotalTexels(0)
	{
	}

	FLightMapAllocationGroup(FLightMapAllocationGroup&&) = default;
	FLightMapAllocationGroup& operator=(FLightMapAllocationGroup&&) = default;

	FLightMapAllocationGroup(const FLightMapAllocationGroup&) = delete; // non-copy-able
	FLightMapAllocationGroup& operator=(const FLightMapAllocationGroup&) = delete;

	TArray<TUniquePtr<FLightMapAllocation>, TInlineAllocator<1>> Allocations;

	UObject*		Outer;
	ELightMapFlags	LightmapFlags;

	// Bounds of the primitive that the mapping is applied to
	// Used to group nearby allocations into the same lightmap texture
	FBoxSphereBounds Bounds;

	int32			TotalTexels;
};

/**
 * A light-map texture which has been partially allocated, but not yet encoded.
 */
struct FLightMapPendingTexture : public FTextureLayout
{

	/** Helper data to keep track of the asynchronous tasks for the 4 lightmap textures. */
	ULightMapTexture2D*				Textures[NUM_STORED_LIGHTMAP_COEF];
	ULightMapTexture2D*				SkyOcclusionTexture;
	ULightMapTexture2D*				AOMaterialMaskTexture;
	UShadowMapTexture2D*            ShadowMapTexture;

	ULightMapVirtualTexture2D*		VirtualTextures[NUM_STORED_LIGHTMAP_COEF];


	TArray<TUniquePtr<FLightMapAllocation>> Allocations;
	UObject*						Outer;
	TWeakObjectPtr<UWorld>			OwningWorld;
	/** Bounding volume for all mappings within this texture.							*/
	FBoxSphereBounds				Bounds;

	/** Lightmap streaming flags that must match in order to be stored in this texture.	*/
	ELightMapFlags					LightmapFlags;
	// Optimization to quickly test if a new allocation won't fit
	// Primarily of benefit to instanced mesh lightmaps
	int64							UnallocatedTexels;
	int32							NumOutstandingAsyncTasks;
	bool							bUObjectsCreated;
	int32							NumNonPower2Texels;
	int32							NumVirtualTextureLayers[NUM_STORED_LIGHTMAP_COEF];
	uint64							NumLightmapMappedTexels;
	uint64							NumLightmapUnmappedTexels;
	volatile bool					bIsFinishedEncoding; // has the encoding thread finished encoding (not the AsyncCache)
	bool							bHasRunPostEncode;
	bool							bTexelDebuggingEnabled;

	FLightMapPendingTexture(UWorld* InWorld, uint32 InSizeX,uint32 InSizeY, ETextureLayoutAspectRatio Aspect)
		: FTextureLayout(4, 4, InSizeX, InSizeY, /* PowerOfTwo */ true, Aspect, /* AlignByFour */ true) // Min size is 4x4 in case of block compression.
		, SkyOcclusionTexture(nullptr)
		, AOMaterialMaskTexture(nullptr)
		, ShadowMapTexture(nullptr)
		, OwningWorld(InWorld)
		, Bounds(FBox(ForceInit))
		, LightmapFlags(LMF_None)
		, UnallocatedTexels(static_cast<int64>(InSizeX) * InSizeY)
		, NumOutstandingAsyncTasks(0)
		, bUObjectsCreated(false)
		, NumNonPower2Texels(0)
		, NumLightmapMappedTexels(0)
		, NumLightmapUnmappedTexels(0)
		, bIsFinishedEncoding(false)
		, bHasRunPostEncode(false)
		, bTexelDebuggingEnabled(IsTexelDebuggingEnabled())
	{
		FMemory::Memzero(Textures);
		FMemory::Memzero(VirtualTextures);
		FMemory::Memzero(NumVirtualTextureLayers);
	}

	~FLightMapPendingTexture()
	{
	}

	void CreateUObjects();

	/**
	 * Processes the textures and starts asynchronous compression tasks for all mip-levels.
	 */
	void StartEncoding(const FStaticLightingBuildContext* LightingContext, ITextureCompressorModule* UnusedCompressor);

	void EncodeCoefficientTexture(int32 CoefficientIndex, UTexture* Texture, uint32 LayerIndex, const FColor& TextureColor, bool bEncodeVirtualTexture);
	void EncodeSkyOcclusionTexture(UTexture* Texture, uint32 LayerIndex, const FColor& TextureColor);
	void EncodeAOMaskTexture(UTexture* Texture, uint32 LayerIndex, const FColor& TextureColor);
	void EncodeShadowMapTexture(const TArray<TArray<FFourDistanceFieldSamples>>& MipData, UTexture* Texture, uint32 LayerIndex);

	/**
	 * Call this function after the IsFinishedEncoding function returns true
	 */
	void PostEncode();

	/**
	 * IsFinishedCoding
	 * Are we ready to call PostEncode
	 * encode is run in a separate thread
	 * @return are we finished with the StartEncoding function yet
	 */
	bool IsFinishedEncoding() const;

	/**
	 * IsAsyncCacheComplete
	 * checks if any of our texture async caches are still running
	 */
	bool IsAsyncCacheComplete() const;

	/**
	 * Call this function after IsAscynCacheComplete returns true
	 */
	void FinishCachingTextures();

	
	
	/**
	 * Finds a free area in the texture large enough to contain a surface with the given size.
	 * If a large enough area is found, it is marked as in use, the output parameters OutBaseX and OutBaseY are
	 * set to the coordinates of the upper left corner of the free area and the function return true.
	 * Otherwise, the function returns false and OutBaseX and OutBaseY remain uninitialized.
	 *
	 * If the allocation succeeded, Allocation.OffsetX and Allocation.OffsetY will be set to the upper-left corner
	 * of the area allocated.
	 *
	 * @param Allocation	Lightmap allocation to try to fit
	 * @param bForceIntoThisTexture	True if we should ignore distance and other factors when considering whether the mapping should be packed into this texture
	 *
	 * @return	True if succeeded, false otherwise.
	 */
	bool AddElement(FLightMapAllocationGroup& AllocationGroup, const bool bForceIntoThisTexture = false);

private:
	/**
	* Finish caching the texture
	*/
	void FinishCacheTexture(UTexture2D* Texture);

	void PostEncode(UTexture2D* Texture);

	

	FName GetLightmapName(int32 TextureIndex, int32 CoefficientIndex);
	FName GetSkyOcclusionTextureName(int32 TextureIndex);
	FName GetAOMaterialMaskTextureName(int32 TextureIndex);
	FName GetShadowTextureName(int32 TextureIndex);
	FName GetVirtualTextureName(int32 TextureIndex, int32 CoefficientIndex);

	bool NeedsSkyOcclusionTexture() const;
	bool NeedsAOMaterialMaskTexture() const;
	bool NeedsStaticShadowTexture() const;
};

/**
 * IsAsyncCacheComplete
 * checks if any of our texture async caches are still running
 */
bool FLightMapPendingTexture::IsAsyncCacheComplete() const
{
	check(IsInGameThread()); //updates global variables and accesses shared UObjects
	if (SkyOcclusionTexture && !SkyOcclusionTexture->IsAsyncCacheComplete())
	{
		return false;
	}

	if (AOMaterialMaskTexture && !AOMaterialMaskTexture->IsAsyncCacheComplete())
	{
		return false;
	}

	if (ShadowMapTexture && !ShadowMapTexture->IsAsyncCacheComplete())
	{
		return false;
	}

	// Encode and compress the coefficient textures.
	for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto Texture = Textures[CoefficientIndex];
		if (Texture == nullptr)
		{
			continue;
		}
		if (!Texture->IsAsyncCacheComplete())
		{
			return false;
		}
	}

	for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto VirtualTexture = VirtualTextures[CoefficientIndex];
		if (VirtualTexture == nullptr)
		{
			continue;
		}
		if (!VirtualTexture->IsAsyncCacheComplete())
		{
			return false;
		}
	}

	return true;
}

/**
 * Called once the compression tasks for all mip-levels of a texture has finished.
 * Copies the compressed data into each of the mip-levels of the texture and deletes the tasks.
 *
 * @param CoefficientIndex	Texture coefficient index, identifying the specific texture with this FLightMapPendingTexture.
 */


void FLightMapPendingTexture::FinishCacheTexture(UTexture2D* Texture)
{
	check(IsInGameThread()); // updating global variables needs to be done in main thread
	check(Texture != nullptr);

	Texture->FinishCachePlatformData();
	Texture->UpdateResource();

	int32 TextureSize = Texture->CalcTextureMemorySizeEnum(TMC_AllMips);
	GLightmapTotalSize += TextureSize;
	GLightmapTotalStreamingSize += (LightmapFlags & LMF_Streamed) ? TextureSize : 0;
}

void FLightMapPendingTexture::PostEncode(UTexture2D* Texture)
{
	check(IsInGameThread());
	check(Texture != nullptr);
	Texture->CachePlatformData(true, true);
}


bool FLightMapPendingTexture::IsFinishedEncoding() const
{
	return bIsFinishedEncoding;
}

void FLightMapPendingTexture::PostEncode()
{
	check(IsInGameThread()); 
	check(bIsFinishedEncoding);

	if (bHasRunPostEncode)
	{
		return;
	}
	bHasRunPostEncode = true;

	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		auto& Allocation = Allocations[AllocationIndex];

		int32 PaddedSizeX = Allocation->TotalSizeX;
		int32 PaddedSizeY = Allocation->TotalSizeY;
		int32 BaseX = Allocation->OffsetX - Allocation->MappedRect.Min.X;
		int32 BaseY = Allocation->OffsetY - Allocation->MappedRect.Min.Y;
		if (FPlatformProperties::HasEditorOnlyData() && GLightmassDebugOptions.bPadMappings && (Allocation->PaddingType == LMPT_NormalPadding))
		{
			if ((PaddedSizeX - 2 > 0) && ((PaddedSizeY - 2) > 0))
			{
				PaddedSizeX -= 2;
				PaddedSizeY -= 2;
				BaseX += 1;
				BaseY += 1;
			}
		}

		// Calculate the coordinate scale/biases this light-map.
		FVector2D Scale((float)PaddedSizeX / (float)GetSizeX(), (float)PaddedSizeY / (float)GetSizeY());
		FVector2D Bias((float)BaseX / (float)GetSizeX(), (float)BaseY / (float)GetSizeY());

		// Set the scale/bias of the lightmap
		check(Allocation->LightMap);
		Allocation->LightMap->CoordinateScale = Scale;
		Allocation->LightMap->CoordinateBias = Bias;
		Allocation->PostEncode();

		// Free the light-map's raw data.
		Allocation->RawData.Empty();
	}


	if (SkyOcclusionTexture!=nullptr)
	{
		PostEncode(SkyOcclusionTexture);
	}

	if (AOMaterialMaskTexture != nullptr)
	{
		PostEncode(AOMaterialMaskTexture);
	}

	if (ShadowMapTexture != nullptr)
	{
		PostEncode(ShadowMapTexture);
	}

	// update all the global stats
	GNumLightmapMappedTexels += NumLightmapMappedTexels;
	GNumLightmapUnmappedTexels += NumLightmapUnmappedTexels;
	GNumLightmapTotalTexelsNonPow2 += NumNonPower2Texels;

	// Encode and compress the coefficient textures.
	for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto Texture = Textures[CoefficientIndex];
		if (Texture == nullptr)
		{
			continue;
		}
		
		PostEncode(Texture);

		GNumLightmapTotalTexels += Texture->Source.GetSizeX() * Texture->Source.GetSizeY();
		GNumLightmapTextures++;


		UPackage* TexturePackage = Texture->GetOutermost();
		if (OwningWorld.IsValid())
		{
			for (int32 LevelIndex = 0; TexturePackage && LevelIndex < OwningWorld->GetNumLevels(); LevelIndex++)
			{
				ULevel* Level = OwningWorld->GetLevel(LevelIndex);
				UPackage* LevelPackage = Level->GetOutermost();
				if (TexturePackage == LevelPackage)
				{
					Level->LightmapTotalSize += float(Texture->CalcTextureMemorySizeEnum(TMC_AllMips)) / 1024.0f;
					break;
				}
			}
		}
	}

	// Rebuild the virtual texture layers
	/*if (VirtualTexture != nullptr)
	{
		GWarn->StatusUpdate(0, 0, NSLOCTEXT("LightMap2D", "BeginEncodingVTLightMapsTask", "Encoding VT light-maps"));

		if (NeedsStaticShadowTexture())
		{
			const uint32 Index = VirtualTexture->GetLayerForType(ELightMapVirtualTextureType::ShadowMask);

			// Update VT shadow map format now, since it will change based on number of valid shadow channels
			VirtualTexture->LayerSettings[Index].Format = ShadowMapTexture->Source.GetFormat();
		}

		VirtualTexture->BuildLightmapData(true);
	}*/
	for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto VirtualTexture = VirtualTextures[CoefficientIndex];
		if (VirtualTexture == nullptr)
		{
			continue;
		}

		PostEncode(VirtualTexture);
	}
}


void FLightMapPendingTexture::FinishCachingTextures()
{
	check(IsInGameThread()); //updates global variables and accesses shared UObjects
	if (SkyOcclusionTexture)
	{
		FinishCacheTexture(SkyOcclusionTexture);
	}

	if (AOMaterialMaskTexture)
	{
		FinishCacheTexture(AOMaterialMaskTexture);
	}

	if (ShadowMapTexture)
	{
		FinishCacheTexture(ShadowMapTexture);
	}

	// Encode and compress the coefficient textures.
	for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto& Texture = Textures[CoefficientIndex];
		if (Texture)
		{
			FinishCacheTexture(Texture);
		}
	}

	for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto& VirtualTexture = VirtualTextures[CoefficientIndex];
		if (VirtualTexture)
		{
			FinishCacheTexture(VirtualTexture);
		}
	}
}

/**
 * Finds a free area in the texture large enough to contain a surface with the given size.
 * If a large enough area is found, it is marked as in use, the output parameters OutBaseX and OutBaseY are
 * set to the coordinates of the upper left corner of the free area and the function return true.
 * Otherwise, the function returns false and OutBaseX and OutBaseY remain uninitialized.
 *
 * If the allocation succeeded, Allocation.OffsetX and Allocation.OffsetY will be set to the upper-left corner
 * of the allocated area.
 *
 * @param Allocation	Lightmap allocation group to try to fit
 * @param bForceIntoThisTexture	True if we should ignore distance and other factors when considering whether the mapping should be packed into this texture
 *
 * @return	True if succeeded, false otherwise.
 */
bool FLightMapPendingTexture::AddElement(FLightMapAllocationGroup& AllocationGroup, bool bForceIntoThisTexture)
{
	if (!bForceIntoThisTexture)
	{
		// Don't pack lightmaps from different packages into the same texture.
		if (Outer != AllocationGroup.Outer)
		{
			return false;
		}
	}

	// This is a rough test, passing it doesn't guarantee it'll fit
	// But failing it does guarantee that it _won't_ fit
	if (UnallocatedTexels < AllocationGroup.TotalTexels)
	{
		return false;
	}

	const bool bEmptyTexture = Allocations.Num() == 0;
	const FBoxSphereBounds NewBounds = bEmptyTexture ? AllocationGroup.Bounds : Bounds + AllocationGroup.Bounds;

	if (!bEmptyTexture && !bForceIntoThisTexture)
	{
		// Don't mix streaming lightmaps with non-streaming lightmaps.
		if ((LightmapFlags & LMF_Streamed) != (AllocationGroup.LightmapFlags & LMF_Streamed))
		{
			return false;
		}

		// Is this a streaming lightmap?
		if (LightmapFlags & LMF_Streamed)
		{
			bool bPerformDistanceCheck = true;

			// Don't pack together lightmaps that are too far apart
			if (bPerformDistanceCheck && NewBounds.SphereRadius > GMaxLightmapRadius && NewBounds.SphereRadius > (Bounds.SphereRadius + UE_SMALL_NUMBER))
			{
				return false;
			}
		}
	}

	int32 NewUnallocatedTexels = UnallocatedTexels;

	int32 iAllocation = 0;
	for (; iAllocation < AllocationGroup.Allocations.Num(); ++iAllocation)
	{
		auto& Allocation = AllocationGroup.Allocations[iAllocation];
		uint32 BaseX, BaseY;
		const uint32 MappedRectWidth = Allocation->MappedRect.Width();
		const uint32 MappedRectHeight = Allocation->MappedRect.Height();
		if (FTextureLayout::AddElement(BaseX, BaseY, MappedRectWidth, MappedRectHeight))
		{
			Allocation->OffsetX = BaseX;
			Allocation->OffsetY = BaseY;

			// Assumes bAlignByFour
			NewUnallocatedTexels -= ((MappedRectWidth + 3) & ~3) * ((MappedRectHeight + 3) & ~3);
		}
		else
		{
			// failed to add all elements to the texture
			break;
		}
	}
	if (iAllocation < AllocationGroup.Allocations.Num())
	{
		// failed to add all elements to the texture
		// remove the ones added so far to restore our original state
		for (--iAllocation; iAllocation >= 0; --iAllocation)
		{
			auto& Allocation = AllocationGroup.Allocations[iAllocation];
			const uint32 MappedRectWidth = Allocation->MappedRect.Width();
			const uint32 MappedRectHeight = Allocation->MappedRect.Height();
			verify(FTextureLayout::RemoveElement(Allocation->OffsetX, Allocation->OffsetY, MappedRectWidth, MappedRectHeight));
		}
		return false;
	}

	Bounds = NewBounds;
	UnallocatedTexels = NewUnallocatedTexels;

	return true;
}

/** Whether to color each lightmap texture with a different (random) color. */
bool GVisualizeLightmapTextures = false;

static void GenerateLightmapMipsAndDilateColor(int32 NumMips, int32 TextureSizeX, int32 TextureSizeY, FColor TextureColor, FColor** MipData, int8** MipCoverageData)
{
	for(int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
	{
		const int32 SourceMipSizeX = FMath::Max(1, TextureSizeX >> (MipIndex - 1));
		const int32 SourceMipSizeY = FMath::Max(1, TextureSizeY >> (MipIndex - 1));
		const int32 DestMipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const int32 DestMipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);

		// Downsample the previous mip-level, taking into account which texels are mapped.
		FColor* NextMipData = MipData[MipIndex];
		FColor* LastMipData = MipData[MipIndex - 1];

		int8* NextMipCoverageData = MipCoverageData[ MipIndex ];
		int8* LastMipCoverageData = MipCoverageData[ MipIndex - 1 ];

		const int32 MipFactorX = SourceMipSizeX / DestMipSizeX;
		const int32 MipFactorY = SourceMipSizeY / DestMipSizeY;

		//@todo - generate mips before encoding lightmaps!  
		// Currently we are filtering in the encoded space, similar to generating mips of sRGB textures in sRGB space
		for(int32 Y = 0; Y < DestMipSizeY; Y++)
		{
			for(int32 X = 0; X < DestMipSizeX; X++)
			{
				FLinearColor AccumulatedColor = FLinearColor::Black;
				uint32 Coverage = 0;

				const uint32 MinSourceY = (Y + 0) * MipFactorY;
				const uint32 MaxSourceY = (Y + 1) * MipFactorY;
				for(uint32 SourceY = MinSourceY; SourceY < MaxSourceY; SourceY++)
				{
					const uint32 MinSourceX = (X + 0) * MipFactorX;
					const uint32 MaxSourceX = (X + 1) * MipFactorX;
					for(uint32 SourceX = MinSourceX; SourceX < MaxSourceX; SourceX++)
					{
						const FColor& SourceColor = LastMipData[SourceY * SourceMipSizeX + SourceX];
						int8 SourceCoverage = LastMipCoverageData[ SourceY * SourceMipSizeX + SourceX ];
						if( SourceCoverage )
						{
							AccumulatedColor += SourceColor.ReinterpretAsLinear() * SourceCoverage;
							Coverage += SourceCoverage;
						}
					}
				}
				FColor& DestColor = NextMipData[Y * DestMipSizeX + X];
				int8& DestCoverage = NextMipCoverageData[Y * DestMipSizeX + X];
				if ( GVisualizeLightmapTextures )
				{
					DestColor = TextureColor;
					DestCoverage = 127;
				}
				else if(Coverage)
				{
					DestColor = ( AccumulatedColor / Coverage ).QuantizeRound();
					DestCoverage = Coverage / (MipFactorX * MipFactorY);
				}
				else
				{
					DestColor = FColor(0,0,0);
					DestCoverage = 0;
				}
			}
		}
	}

	// Expand texels which are mapped into adjacent texels which are not mapped to avoid artifacts when using texture filtering.
	for(int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
	{
		FColor* MipLevelData = MipData[MipIndex];
		int8* MipLevelCoverageData = MipCoverageData[MipIndex];

		uint32 MipSizeX = FMath::Max(1,TextureSizeX >> MipIndex);
		uint32 MipSizeY = FMath::Max(1,TextureSizeY >> MipIndex);
		for(uint32 DestY = 0;DestY < MipSizeY;DestY++)
		{
			for(uint32 DestX = 0; DestX < MipSizeX; DestX++)
			{
				FColor& DestColor = MipLevelData[DestY * MipSizeX + DestX];
				int8& DestCoverage = MipLevelCoverageData[DestY * MipSizeX + DestX];
				if(DestCoverage == 0)
				{
					FLinearColor AccumulatedColor = FLinearColor::Black;
					uint32 Coverage = 0;

					const int32 MinSourceY = FMath::Max((int32)DestY - 1, (int32)0);
					const int32 MaxSourceY = FMath::Min((int32)DestY + 1, (int32)MipSizeY - 1);
					for(int32 SourceY = MinSourceY; SourceY <= MaxSourceY; SourceY++)
					{
						const int32 MinSourceX = FMath::Max((int32)DestX - 1, (int32)0);
						const int32 MaxSourceX = FMath::Min((int32)DestX + 1, (int32)MipSizeX - 1);
						for(int32 SourceX = MinSourceX; SourceX <= MaxSourceX; SourceX++)
						{
							FColor& SourceColor = MipLevelData[SourceY * MipSizeX + SourceX];
							int8 SourceCoverage = MipLevelCoverageData[SourceY * MipSizeX + SourceX];
							if( SourceCoverage > 0 )
							{
								static const uint32 Weights[3][3] =
								{
									{ 1, 255, 1 },
									{ 255, 0, 255 },
									{ 1, 255, 1 },
								};
								AccumulatedColor += SourceColor.ReinterpretAsLinear() * SourceCoverage * Weights[SourceX - DestX + 1][SourceY - DestY + 1];
								Coverage += SourceCoverage * Weights[SourceX - DestX + 1][SourceY - DestY + 1];
							}
						}
					}

					if(Coverage)
					{
						DestColor = (AccumulatedColor / Coverage).QuantizeRound();
						DestCoverage = -1;
					}
				}
			}
		}
	}

	// Fill zero coverage texels with closest colors using mips
	for(int32 MipIndex = NumMips - 2; MipIndex >= 0; MipIndex--)
	{
		const int32 DstMipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const int32 DstMipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);
		const int32 SrcMipSizeX = FMath::Max(1, TextureSizeX >> (MipIndex + 1));
		const int32 SrcMipSizeY = FMath::Max(1, TextureSizeY >> (MipIndex + 1));

		// Source from higher mip, taking into account which texels are mapped.
		FColor* DstMipData = MipData[MipIndex];
		FColor* SrcMipData = MipData[MipIndex + 1];

		int8* DstMipCoverageData = MipCoverageData[ MipIndex ];
		int8* SrcMipCoverageData = MipCoverageData[ MipIndex + 1 ];

		for(int32 DstY = 0; DstY < DstMipSizeY; DstY++)
		{
			for(int32 DstX = 0; DstX < DstMipSizeX; DstX++)
			{
				const uint32 SrcX = DstX / 2;
				const uint32 SrcY = DstY / 2;

				const FColor& SrcColor = SrcMipData[ SrcY * SrcMipSizeX + SrcX ];
				int8 SrcCoverage = SrcMipCoverageData[ SrcY * SrcMipSizeX + SrcX ];

				FColor& DstColor = DstMipData[ DstY * DstMipSizeX + DstX ];
				int8& DstCoverage = DstMipCoverageData[ DstY * DstMipSizeX + DstX ];

				// Point upsample mip data for zero coverage texels
				// TODO bilinear upsample
				if( SrcCoverage != 0 && DstCoverage == 0 )
				{
					DstColor = SrcColor;
					DstCoverage = SrcCoverage;
				}
			}
		}
	}
}

static void GenerateLightmapMipsAndDilateByte(int32 NumMips, int32 TextureSizeX, int32 TextureSizeY, uint8 TextureColor, uint8** MipData, int8** MipCoverageData)
{
	for(int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
	{
		const int32 SourceMipSizeX = FMath::Max(1, TextureSizeX >> (MipIndex - 1));
		const int32 SourceMipSizeY = FMath::Max(1, TextureSizeY >> (MipIndex - 1));
		const int32 DestMipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const int32 DestMipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);

		// Downsample the previous mip-level, taking into account which texels are mapped.
		uint8* NextMipData = MipData[MipIndex];
		uint8* LastMipData = MipData[MipIndex - 1];

		int8* NextMipCoverageData = MipCoverageData[ MipIndex ];
		int8* LastMipCoverageData = MipCoverageData[ MipIndex - 1 ];

		const int32 MipFactorX = SourceMipSizeX / DestMipSizeX;
		const int32 MipFactorY = SourceMipSizeY / DestMipSizeY;

		//@todo - generate mips before encoding lightmaps!  
		// Currently we are filtering in the encoded space, similar to generating mips of sRGB textures in sRGB space
		for(int32 Y = 0; Y < DestMipSizeY; Y++)
		{
			for(int32 X = 0; X < DestMipSizeX; X++)
			{
				float AccumulatedColor = 0;
				uint32 Coverage = 0;

				const uint32 MinSourceY = (Y + 0) * MipFactorY;
				const uint32 MaxSourceY = (Y + 1) * MipFactorY;
				for(uint32 SourceY = MinSourceY; SourceY < MaxSourceY; SourceY++)
				{
					const uint32 MinSourceX = (X + 0) * MipFactorX;
					const uint32 MaxSourceX = (X + 1) * MipFactorX;
					for(uint32 SourceX = MinSourceX; SourceX < MaxSourceX; SourceX++)
					{
						const uint8& SourceColor = LastMipData[SourceY * SourceMipSizeX + SourceX];
						int8 SourceCoverage = LastMipCoverageData[ SourceY * SourceMipSizeX + SourceX ];
						if( SourceCoverage )
						{
							AccumulatedColor += SourceColor / 255.0f * SourceCoverage;
							Coverage += SourceCoverage;
						}
					}
				}
				uint8& DestColor = NextMipData[Y * DestMipSizeX + X];
				int8& DestCoverage = NextMipCoverageData[Y * DestMipSizeX + X];
				if ( GVisualizeLightmapTextures )
				{
					DestColor = TextureColor;
					DestCoverage = 127;
				}
				else if(Coverage)
				{
					DestColor = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(AccumulatedColor / Coverage * 255.f), 0, 255);
					DestCoverage = Coverage / (MipFactorX * MipFactorY);
				}
				else
				{
					DestColor = 0;
					DestCoverage = 0;
				}
			}
		}
	}

	// Expand texels which are mapped into adjacent texels which are not mapped to avoid artifacts when using texture filtering.
	for(int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
	{
		uint8* MipLevelData = MipData[MipIndex];
		int8* MipLevelCoverageData = MipCoverageData[MipIndex];

		uint32 MipSizeX = FMath::Max(1,TextureSizeX >> MipIndex);
		uint32 MipSizeY = FMath::Max(1,TextureSizeY >> MipIndex);
		for(uint32 DestY = 0;DestY < MipSizeY;DestY++)
		{
			for(uint32 DestX = 0; DestX < MipSizeX; DestX++)
			{
				uint8& DestColor = MipLevelData[DestY * MipSizeX + DestX];
				int8& DestCoverage = MipLevelCoverageData[DestY * MipSizeX + DestX];
				if(DestCoverage == 0)
				{
					float AccumulatedColor = 0;
					uint32 Coverage = 0;

					const int32 MinSourceY = FMath::Max((int32)DestY - 1, (int32)0);
					const int32 MaxSourceY = FMath::Min((int32)DestY + 1, (int32)MipSizeY - 1);
					for(int32 SourceY = MinSourceY; SourceY <= MaxSourceY; SourceY++)
					{
						const int32 MinSourceX = FMath::Max((int32)DestX - 1, (int32)0);
						const int32 MaxSourceX = FMath::Min((int32)DestX + 1, (int32)MipSizeX - 1);
						for(int32 SourceX = MinSourceX; SourceX <= MaxSourceX; SourceX++)
						{
							uint8& SourceColor = MipLevelData[SourceY * MipSizeX + SourceX];
							int8 SourceCoverage = MipLevelCoverageData[SourceY * MipSizeX + SourceX];
							if( SourceCoverage > 0 )
							{
								static const uint32 Weights[3][3] =
								{
									{ 1, 255, 1 },
									{ 255, 0, 255 },
									{ 1, 255, 1 },
								};
								AccumulatedColor += SourceColor / 255.0f * SourceCoverage * Weights[SourceX - DestX + 1][SourceY - DestY + 1];
								Coverage += SourceCoverage * Weights[SourceX - DestX + 1][SourceY - DestY + 1];
							}
						}
					}

					if(Coverage)
					{
						DestColor = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(AccumulatedColor / Coverage * 255.f), 0, 255);
						DestCoverage = -1;
					}
				}
			}
		}
	}

	// Fill zero coverage texels with closest colors using mips
	for(int32 MipIndex = NumMips - 2; MipIndex >= 0; MipIndex--)
	{
		const int32 DstMipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const int32 DstMipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);
		const int32 SrcMipSizeX = FMath::Max(1, TextureSizeX >> (MipIndex + 1));
		const int32 SrcMipSizeY = FMath::Max(1, TextureSizeY >> (MipIndex + 1));

		// Source from higher mip, taking into account which texels are mapped.
		uint8* DstMipData = MipData[MipIndex];
		uint8* SrcMipData = MipData[MipIndex + 1];

		int8* DstMipCoverageData = MipCoverageData[ MipIndex ];
		int8* SrcMipCoverageData = MipCoverageData[ MipIndex + 1 ];

		for(int32 DstY = 0; DstY < DstMipSizeY; DstY++)
		{
			for(int32 DstX = 0; DstX < DstMipSizeX; DstX++)
			{
				const uint32 SrcX = DstX / 2;
				const uint32 SrcY = DstY / 2;

				const uint8& SrcColor = SrcMipData[ SrcY * SrcMipSizeX + SrcX ];
				int8 SrcCoverage = SrcMipCoverageData[ SrcY * SrcMipSizeX + SrcX ];

				uint8& DstColor = DstMipData[ DstY * DstMipSizeX + DstX ];
				int8& DstCoverage = DstMipCoverageData[ DstY * DstMipSizeX + DstX ];

				// Point upsample mip data for zero coverage texels
				// TODO bilinear upsample
				if( SrcCoverage != 0 && DstCoverage == 0 )
				{
					DstColor = SrcColor;
					DstCoverage = SrcCoverage;
				}
			}
		}
	}
}

void FLightMapPendingTexture::CreateUObjects()
{
	check(IsInGameThread());
	++GLightmapCounter;
	
	// Only build VT lightmaps if they are enabled
	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GMaxRHIShaderPlatform);
	const bool bIncludeNonVirtualTextures = !bUseVirtualTextures || (CVarIncludeNonVirtualTexturedLightMaps.GetValueOnAnyThread() != 0);
	
	if (bIncludeNonVirtualTextures)
	{
		if (NeedsSkyOcclusionTexture())
		{
			SkyOcclusionTexture = NewObject<ULightMapTexture2D>(Outer, GetSkyOcclusionTextureName(GLightmapCounter));
		}

		if (NeedsAOMaterialMaskTexture())
		{
			AOMaterialMaskTexture = NewObject<ULightMapTexture2D>(Outer, GetAOMaterialMaskTextureName(GLightmapCounter));
		}

		if (NeedsStaticShadowTexture())
		{
			ShadowMapTexture = NewObject<UShadowMapTexture2D>(Outer, GetShadowTextureName(GLightmapCounter));
		}

		// Encode and compress the coefficient textures.
		for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
		{
			Textures[CoefficientIndex] = nullptr;
			// Skip generating simple lightmaps if wanted.
			static const auto CVarSupportLowQualityLightmaps = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
			const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmaps) || (CVarSupportLowQualityLightmaps->GetValueOnAnyThread() != 0);

			if ((!bAllowLowQualityLightMaps) && CoefficientIndex >= LQ_LIGHTMAP_COEF_INDEX)
			{
				continue;
			}

			// Create the light-map texture for this coefficient.
			auto Texture = NewObject<ULightMapTexture2D>(Outer, GetLightmapName(GLightmapCounter, CoefficientIndex));
			Textures[CoefficientIndex] = Texture;
		}
	}

	if (bUseVirtualTextures)
	{
		for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
		{
			VirtualTextures[CoefficientIndex] = nullptr;
			NumVirtualTextureLayers[CoefficientIndex] = 0;
			// Skip generating simple lightmaps if wanted.
			static const auto CVarSupportLowQualityLightmaps = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
			const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmaps) || (CVarSupportLowQualityLightmaps->GetValueOnAnyThread() != 0);

			if ((!bAllowLowQualityLightMaps) && CoefficientIndex >= LQ_LIGHTMAP_COEF_INDEX)
			{
				continue;
			}

			auto VirtualTexture = NewObject<ULightMapVirtualTexture2D>(Outer, GetVirtualTextureName(GLightmapCounter, CoefficientIndex));
			VirtualTexture->VirtualTextureStreaming = true;

			VirtualTexture->SetLayerForType(ELightMapVirtualTextureType::LightmapLayer0, NumVirtualTextureLayers[CoefficientIndex]++);
			VirtualTexture->SetLayerForType(ELightMapVirtualTextureType::LightmapLayer1, NumVirtualTextureLayers[CoefficientIndex]++);

			if (CoefficientIndex < LQ_LIGHTMAP_COEF_INDEX)
			{

				if (NeedsAOMaterialMaskTexture())
				{
					VirtualTexture->SetLayerForType(ELightMapVirtualTextureType::AOMaterialMask, NumVirtualTextureLayers[CoefficientIndex]++);
				}
				if (NeedsSkyOcclusionTexture())
				{
					VirtualTexture->SetLayerForType(ELightMapVirtualTextureType::SkyOcclusion, NumVirtualTextureLayers[CoefficientIndex]++);
				}
			}

			if (NeedsStaticShadowTexture())
			{
				VirtualTexture->SetLayerForType(ELightMapVirtualTextureType::ShadowMask, NumVirtualTextureLayers[CoefficientIndex]++);
			}

			VirtualTextures[CoefficientIndex] = VirtualTexture;
		}
	}
	else
	{
		FMemory::Memzero(VirtualTextures);
	}
	
	check(bUObjectsCreated == false);
	bUObjectsCreated = true;
}

bool FLightMapPendingTexture::NeedsSkyOcclusionTexture() const
{
	if (bUObjectsCreated)
	{
		if (VirtualTextures[0])
		{
			return VirtualTextures[0]->HasLayerForType(ELightMapVirtualTextureType::SkyOcclusion);
		}
		return SkyOcclusionTexture != nullptr;
	}

	bool bNeedsSkyOcclusionTexture = false;

	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		auto& Allocation = Allocations[AllocationIndex];

		if (Allocation->bHasSkyShadowing)
		{
			bNeedsSkyOcclusionTexture = true;
			break;
		}
	}
	return bNeedsSkyOcclusionTexture;
}

bool FLightMapPendingTexture::NeedsAOMaterialMaskTexture() const
{
	if (bUObjectsCreated)
	{
		if (VirtualTextures[0])
		{
			return VirtualTextures[0]->HasLayerForType(ELightMapVirtualTextureType::AOMaterialMask);
		}
		return AOMaterialMaskTexture != nullptr;
	}

	const FLightmassWorldInfoSettings* LightmassWorldSettings = OwningWorld.IsValid() ? &(OwningWorld->GetWorldSettings()->LightmassSettings) : NULL;
	if (LightmassWorldSettings && LightmassWorldSettings->bUseAmbientOcclusion && LightmassWorldSettings->bGenerateAmbientOcclusionMaterialMask)
	{
		return true;
	}

	return false;
}

bool FLightMapPendingTexture::NeedsStaticShadowTexture() const
{
	if (bUObjectsCreated)
	{
		if (VirtualTextures[0])
		{
			return VirtualTextures[0]->HasLayerForType(ELightMapVirtualTextureType::ShadowMask);
		}
		return ShadowMapTexture != nullptr;
	}

	bool bResult = false;
	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		auto& Allocation = Allocations[AllocationIndex];

		if (Allocation->ShadowMapData.Num() > 0)
		{
			bResult = true;
			break;
		}
	}
	return bResult;
}

void FLightMapPendingTexture::EncodeSkyOcclusionTexture(UTexture* Texture, uint32 LayerIndex, const FColor& TextureColor)
{
	const int32 NumMips = Texture->Source.GetNumMips();
	check(NumMips > 0);

	FTextureFormatSettings FormatSettings;
	FormatSettings.SRGB = false;
	FormatSettings.CompressionNoAlpha = false;
	FormatSettings.CompressionNone = !GCompressLightmaps;
	Texture->SetLayerFormatSettings(LayerIndex, FormatSettings);

	int32 TextureSizeX = Texture->Source.GetSizeX();
	int32 TextureSizeY = Texture->Source.GetSizeY();

	// Lock all mip levels.
	FColor* MipData[MAX_TEXTURE_MIP_COUNT] = { 0 };
	int8* MipCoverageData[MAX_TEXTURE_MIP_COUNT] = { 0 };
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		MipData[MipIndex] = (FColor*)Texture->Source.LockMip(0, LayerIndex, MipIndex);

		const int32 MipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const int32 MipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);
		MipCoverageData[MipIndex] = (int8*)FMemory::Malloc(MipSizeX * MipSizeY);
	}

	// Create the uncompressed top mip-level.
	FColor* TopMipData = MipData[0];
	FMemory::Memzero(TopMipData, TextureSizeX * TextureSizeY * sizeof(FColor));
	FMemory::Memzero(MipCoverageData[0], TextureSizeX * TextureSizeY);

	FIntRect TextureRect(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		auto& Allocation = Allocations[AllocationIndex];

		// Skip encoding of this texture if we were asked not to bother
		if (!Allocation->bSkipEncoding)
		{
			TextureRect.Min.X = FMath::Min<int32>(TextureRect.Min.X, Allocation->OffsetX);
			TextureRect.Min.Y = FMath::Min<int32>(TextureRect.Min.Y, Allocation->OffsetY);
			TextureRect.Max.X = FMath::Max<int32>(TextureRect.Max.X, Allocation->OffsetX + Allocation->MappedRect.Width());
			TextureRect.Max.Y = FMath::Max<int32>(TextureRect.Max.Y, Allocation->OffsetY + Allocation->MappedRect.Height());

			// Copy the raw data for this light-map into the raw texture data array.
			for (int32 Y = Allocation->MappedRect.Min.Y; Y < Allocation->MappedRect.Max.Y; ++Y)
			{
				for (int32 X = Allocation->MappedRect.Min.X; X < Allocation->MappedRect.Max.X; ++X)
				{
					const FLightMapCoefficients& SourceCoefficients = Allocation->RawData[Y * Allocation->TotalSizeX + X];

					int32 DestY = Y - Allocation->MappedRect.Min.Y + Allocation->OffsetY;
					int32 DestX = X - Allocation->MappedRect.Min.X + Allocation->OffsetX;

					FColor&	DestColor = TopMipData[DestY * TextureSizeX + DestX];

					DestColor.R = SourceCoefficients.SkyOcclusion[0];
					DestColor.G = SourceCoefficients.SkyOcclusion[1];
					DestColor.B = SourceCoefficients.SkyOcclusion[2];
					DestColor.A = SourceCoefficients.SkyOcclusion[3];

					int8& DestCoverage = MipCoverageData[0][DestY * TextureSizeX + DestX];
					DestCoverage = SourceCoefficients.Coverage / 2;
				}
			}
		}
	}

	GenerateLightmapMipsAndDilateColor(NumMips, TextureSizeX, TextureSizeY, TextureColor, MipData, MipCoverageData);

	// Unlock all mip levels.
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		Texture->Source.UnlockMip(0, LayerIndex, MipIndex);
		FMemory::Free(MipCoverageData[MipIndex]);
	}
}

void FLightMapPendingTexture::EncodeAOMaskTexture(UTexture* Texture, uint32 LayerIndex, const FColor& TextureColor)
{
	const int32 NumMips = Texture->Source.GetNumMips();
	check(NumMips > 0);

	FTextureFormatSettings FormatSettings;
	FormatSettings.SRGB = false;
	FormatSettings.CompressionNoAlpha = false;
	FormatSettings.CompressionNone = !GCompressLightmaps;
	FormatSettings.CompressionSettings = TC_Alpha; // BC4
	Texture->SetLayerFormatSettings(LayerIndex, FormatSettings);

	int32 TextureSizeX = Texture->Source.GetSizeX();
	int32 TextureSizeY = Texture->Source.GetSizeY();

	// Lock all mip levels.
	uint8* MipData[MAX_TEXTURE_MIP_COUNT] = { 0 };
	int8* MipCoverageData[MAX_TEXTURE_MIP_COUNT] = { 0 };
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		MipData[MipIndex] = (uint8*)Texture->Source.LockMip(0, LayerIndex, MipIndex);

		const int32 MipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const int32 MipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);
		MipCoverageData[MipIndex] = (int8*)FMemory::Malloc(MipSizeX * MipSizeY);
	}

	// Create the uncompressed top mip-level.
	uint8* TopMipData = MipData[0];
	FMemory::Memzero(TopMipData, TextureSizeX * TextureSizeY * sizeof(uint8));
	FMemory::Memzero(MipCoverageData[0], TextureSizeX * TextureSizeY);

	FIntRect TextureRect(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		auto& Allocation = Allocations[AllocationIndex];

		// Skip encoding of this texture if we were asked not to bother
		if (!Allocation->bSkipEncoding)
		{
			TextureRect.Min.X = FMath::Min<int32>(TextureRect.Min.X, Allocation->OffsetX);
			TextureRect.Min.Y = FMath::Min<int32>(TextureRect.Min.Y, Allocation->OffsetY);
			TextureRect.Max.X = FMath::Max<int32>(TextureRect.Max.X, Allocation->OffsetX + Allocation->MappedRect.Width());
			TextureRect.Max.Y = FMath::Max<int32>(TextureRect.Max.Y, Allocation->OffsetY + Allocation->MappedRect.Height());

			// Copy the raw data for this light-map into the raw texture data array.
			for (int32 Y = Allocation->MappedRect.Min.Y; Y < Allocation->MappedRect.Max.Y; ++Y)
			{
				for (int32 X = Allocation->MappedRect.Min.X; X < Allocation->MappedRect.Max.X; ++X)
				{
					const FLightMapCoefficients& SourceCoefficients = Allocation->RawData[Y * Allocation->TotalSizeX + X];

					int32 DestY = Y - Allocation->MappedRect.Min.Y + Allocation->OffsetY;
					int32 DestX = X - Allocation->MappedRect.Min.X + Allocation->OffsetX;

					uint8& DestValue = TopMipData[DestY * TextureSizeX + DestX];

					DestValue = SourceCoefficients.AOMaterialMask;

					int8& DestCoverage = MipCoverageData[0][DestY * TextureSizeX + DestX];
					DestCoverage = SourceCoefficients.Coverage / 2;
				}
			}
		}
	}

	GenerateLightmapMipsAndDilateByte(NumMips, TextureSizeX, TextureSizeY, TextureColor.R, MipData, MipCoverageData);

	// Unlock all mip levels.
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		Texture->Source.UnlockMip(0, LayerIndex, MipIndex);
		FMemory::Free(MipCoverageData[MipIndex]);
	}
}

void FLightMapPendingTexture::EncodeShadowMapTexture(const TArray<TArray<FFourDistanceFieldSamples>>& MipData, UTexture* Texture, uint32 LayerIndex)
{
	const int32 NumMips = Texture->Source.GetNumMips();
	check(NumMips > 0);
	check(NumMips == MipData.Num());

	FTextureFormatSettings FormatSettings;
	FormatSettings.SRGB = false;
	FormatSettings.CompressionNone = true;
	Texture->SetLayerFormatSettings(LayerIndex, FormatSettings);

	const ETextureSourceFormat SourceFormat = Texture->Source.GetFormat(LayerIndex);
	check(SourceFormat == TSF_G8 || SourceFormat == TSF_BGRA8);

	const int32 TextureSizeX = Texture->Source.GetSizeX();
	const int32 TextureSizeY = Texture->Source.GetSizeY();

	// Copy the mip-map data into the UShadowMapTexture2D's mip-map array.
	for (int32 MipIndex = 0; MipIndex < MipData.Num(); MipIndex++)
	{
		uint8* DestMipData = (uint8*)Texture->Source.LockMip(0, LayerIndex, MipIndex);
		uint32 MipSizeX = FMath::Max<uint32>(1, GetSizeX() >> MipIndex);
		uint32 MipSizeY = FMath::Max<uint32>(1, GetSizeY() >> MipIndex);

		for (uint32 Y = 0; Y < MipSizeY; Y++)
		{
			for (uint32 X = 0; X < MipSizeX; X++)
			{
				const FFourDistanceFieldSamples& SourceSample = MipData[MipIndex][Y * MipSizeX + X];

				if (SourceFormat == TSF_G8)
				{
					DestMipData[Y * MipSizeX + X] = SourceSample.Samples[0].Distance;
				}
				else
				{
					((FColor*)DestMipData)[Y * MipSizeX + X] = FColor(SourceSample.Samples[0].Distance, SourceSample.Samples[1].Distance, SourceSample.Samples[2].Distance, SourceSample.Samples[3].Distance);
				}
			}
		}

		Texture->Source.UnlockMip(0, LayerIndex, MipIndex);
	}
}

void FLightMapPendingTexture::EncodeCoefficientTexture(int32 CoefficientIndex, UTexture* Texture, uint32 LayerIndex, const FColor& TextureColor, bool bEncodeVirtualTexture)
{
	const int32 NumMips = Texture->Source.GetNumMips();
	check(NumMips > 0);

	FTextureFormatSettings FormatSettings;
	FormatSettings.SRGB = false;
	FormatSettings.CompressionNoAlpha = CoefficientIndex >= LQ_LIGHTMAP_COEF_INDEX;
	FormatSettings.CompressionNone = !GCompressLightmaps;
	Texture->SetLayerFormatSettings(LayerIndex, FormatSettings);

	if (bEncodeVirtualTexture)
	{
		Texture->SetLayerFormatSettings(LayerIndex + 1, FormatSettings);
	}

	const int32 TextureSizeX = Texture->Source.GetSizeX();
	const int32 TextureSizeY = Texture->Source.GetSizeY();

	// Lock all mip levels.
	FColor* MipData0[MAX_TEXTURE_MIP_COUNT] = { 0 };
	FColor* MipData1[MAX_TEXTURE_MIP_COUNT] = { 0 };
	int8* MipCoverageData0[MAX_TEXTURE_MIP_COUNT] = { 0 };
	int8* MipCoverageData1[MAX_TEXTURE_MIP_COUNT] = { 0 };
	{
		const int32 StartBottom = GetSizeX() * GetSizeY();
		for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
		{
			const int32 MipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
			const int32 MipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);

			MipData0[MipIndex] = (FColor*)Texture->Source.LockMip(0, LayerIndex, MipIndex);
			MipCoverageData0[MipIndex] = (int8*)FMemory::Malloc(MipSizeX * MipSizeY);

			if (bEncodeVirtualTexture)
			{
				// 2 coefficients are stored on adjacent VT layers
				MipData1[MipIndex] = (FColor*)Texture->Source.LockMip(0, LayerIndex + 1, MipIndex);
				MipCoverageData1[MipIndex] = (int8*)FMemory::Malloc(MipSizeX * MipSizeY);
			}
			else
			{
				// 2 coefficients are stored on the top/bottom halves of the same destination texture
				MipData1[MipIndex] = MipData0[MipIndex] + StartBottom;
				MipCoverageData1[MipIndex] = MipCoverageData0[MipIndex] + StartBottom;
			}
		}
	}

	// Create the uncompressed top mip-level.
	FMemory::Memzero(MipData0[0], TextureSizeX * TextureSizeY * sizeof(FColor));
	FMemory::Memzero(MipCoverageData0[0], TextureSizeX * TextureSizeY);
	if (bEncodeVirtualTexture)
	{
		FMemory::Memzero(MipData1[0], TextureSizeX * TextureSizeY * sizeof(FColor));
		FMemory::Memzero(MipCoverageData1[0], TextureSizeX * TextureSizeY);
	}

	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		auto& Allocation = Allocations[AllocationIndex];
		for (int k = 0; k < 2; k++)
		{
			Allocation->LightMap->ScaleVectors[CoefficientIndex + k] = FVector4f(
				Allocation->Scale[CoefficientIndex + k][0],
				Allocation->Scale[CoefficientIndex + k][1],
				Allocation->Scale[CoefficientIndex + k][2],
				Allocation->Scale[CoefficientIndex + k][3]
			);

			Allocation->LightMap->AddVectors[CoefficientIndex + k] = FVector4f(
				Allocation->Add[CoefficientIndex + k][0],
				Allocation->Add[CoefficientIndex + k][1],
				Allocation->Add[CoefficientIndex + k][2],
				Allocation->Add[CoefficientIndex + k][3]
			);
		}

		// Skip encoding of this texture if we were asked not to bother
		if (!Allocation->bSkipEncoding)
		{
			FIntRect TextureRect(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
			TextureRect.Min.X = FMath::Min<int32>(TextureRect.Min.X, Allocation->OffsetX);
			TextureRect.Min.Y = FMath::Min<int32>(TextureRect.Min.Y, Allocation->OffsetY);
			TextureRect.Max.X = FMath::Max<int32>(TextureRect.Max.X, Allocation->OffsetX + Allocation->MappedRect.Width());
			TextureRect.Max.Y = FMath::Max<int32>(TextureRect.Max.Y, Allocation->OffsetY + Allocation->MappedRect.Height());

			NumNonPower2Texels += TextureRect.Width() * TextureRect.Height();

			// Copy the raw data for this light-map into the raw texture data array.
			for (int32 Y = Allocation->MappedRect.Min.Y; Y < Allocation->MappedRect.Max.Y; ++Y)
			{
				for (int32 X = Allocation->MappedRect.Min.X; X < Allocation->MappedRect.Max.X; ++X)
				{
					const FLightMapCoefficients& SourceCoefficients = Allocation->RawData[Y * Allocation->TotalSizeX + X];

					int32 DestY = Y - Allocation->MappedRect.Min.Y + Allocation->OffsetY;
					int32 DestX = X - Allocation->MappedRect.Min.X + Allocation->OffsetX;

					FColor&	DestColor = MipData0[0][DestY * TextureSizeX + DestX];
					int8&	DestCoverage = MipCoverageData0[0][DestY * TextureSizeX + DestX];

					FColor&	DestBottomColor = MipData1[0][DestX + DestY * TextureSizeX];
					int8&	DestBottomCoverage = MipCoverageData1[0][DestX + DestY * TextureSizeX];

#if VISUALIZE_PACKING
					if (X == Allocation->MappedRect.Min.X || Y == Allocation->MappedRect.Min.Y ||
						X == Allocation->MappedRect.Max.X - 1 || Y == Allocation->MappedRect.Max.Y - 1 ||
						X == Allocation->MappedRect.Min.X + 1 || Y == Allocation->MappedRect.Min.Y + 1 ||
						X == Allocation->MappedRect.Max.X - 2 || Y == Allocation->MappedRect.Max.Y - 2)
					{
						DestColor = FColor::Red;
					}
					else
					{
						DestColor = FColor::Green;
					}
#else // VISUALIZE_PACKING
					DestColor.R = SourceCoefficients.Coefficients[CoefficientIndex][0];
					DestColor.G = SourceCoefficients.Coefficients[CoefficientIndex][1];
					DestColor.B = SourceCoefficients.Coefficients[CoefficientIndex][2];
					DestColor.A = SourceCoefficients.Coefficients[CoefficientIndex][3];

					DestBottomColor.R = SourceCoefficients.Coefficients[CoefficientIndex + 1][0];
					DestBottomColor.G = SourceCoefficients.Coefficients[CoefficientIndex + 1][1];
					DestBottomColor.B = SourceCoefficients.Coefficients[CoefficientIndex + 1][2];
					DestBottomColor.A = SourceCoefficients.Coefficients[CoefficientIndex + 1][3];

					if (GVisualizeLightmapTextures)
					{
						DestColor = TextureColor;
					}

					// uint8 -> int8
					DestCoverage = DestBottomCoverage = SourceCoefficients.Coverage / 2;
					if (SourceCoefficients.Coverage > 0)
					{
						NumLightmapMappedTexels++;
					}
					else
					{
						NumLightmapUnmappedTexels++;
					}

#if WITH_EDITOR
					if (bTexelDebuggingEnabled)
					{
						int32 PaddedX = X;
						int32 PaddedY = Y;
						if (GLightmassDebugOptions.bPadMappings && (Allocation->PaddingType == LMPT_NormalPadding))
						{
							if (Allocation->TotalSizeX - 2 > 0 && Allocation->TotalSizeY - 2 > 0)
							{
								PaddedX -= 1;
								PaddedY -= 1;
							}
						}

						if (Allocation->bDebug
							&& PaddedX == GCurrentSelectedLightmapSample.LocalX
							&& PaddedY == GCurrentSelectedLightmapSample.LocalY)
						{
							extern FColor GTexelSelectionColor;
							DestColor = GTexelSelectionColor;
						}
					}
#endif // WITH_EDITOR
#endif // !VISUALIZE_PACKING
				}
			}


		}
	}

	GenerateLightmapMipsAndDilateColor(NumMips, TextureSizeX, TextureSizeY, TextureColor, MipData0, MipCoverageData0);
	if (bEncodeVirtualTexture)
	{
		GenerateLightmapMipsAndDilateColor(NumMips, TextureSizeX, TextureSizeY, TextureColor, MipData1, MipCoverageData1);
	}

	// Unlock all mip levels.
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		Texture->Source.UnlockMip(0, LayerIndex, MipIndex);
		FMemory::Free(MipCoverageData0[MipIndex]);
		if (bEncodeVirtualTexture)
		{
			Texture->Source.UnlockMip(0, LayerIndex + 1, MipIndex);
			FMemory::Free(MipCoverageData1[MipIndex]);
		}
	}
}

void FLightMapPendingTexture::StartEncoding(const FStaticLightingBuildContext* LightingContext, ITextureCompressorModule* UnusedCompressor)
{
	if (!bUObjectsCreated)
	{
		check(IsInGameThread());
		CreateUObjects();
	}

	// Create the uncompressed top mip-level.
	TArray< TArray<FFourDistanceFieldSamples> > ShadowMipData;
	int32 NumShadowChannelsUsed = 0;
	if (NeedsStaticShadowTexture())
	{
		NumShadowChannelsUsed = FLightMap2D::EncodeShadowTexture(LightingContext, *this, ShadowMipData);
	}

	FColor TextureColor;
	if ( GVisualizeLightmapTextures )
	{
		TextureColor = FColor::MakeRandomColor();
	}

	const ETextureSourceFormat SkyOcclusionFormat = TSF_BGRA8;
	const ETextureSourceFormat AOMaskFormat = TSF_G8;
	const ETextureSourceFormat ShadowMapFormat = NumShadowChannelsUsed == 1 ? TSF_G8 : TSF_BGRA8;
	const ETextureSourceFormat BaseFormat = TSF_BGRA8;

	if (SkyOcclusionTexture != nullptr)
	{
		auto Texture = SkyOcclusionTexture;
		Texture->Source.Init2DWithMipChain(GetSizeX(), GetSizeY(), SkyOcclusionFormat);
		Texture->MipGenSettings = TMGS_LeaveExistingMips;
		Texture->Filter	= GUseBilinearLightmaps ? TF_Default : TF_Nearest;
		Texture->LODGroup = TEXTUREGROUP_Lightmap;
		Texture->LightmapFlags = ELightMapFlags( LightmapFlags );
		EncodeSkyOcclusionTexture(Texture, 0u, TextureColor);
	}
	
	if (AOMaterialMaskTexture != nullptr)
	{
		auto Texture = AOMaterialMaskTexture;
		Texture->Source.Init2DWithMipChain(GetSizeX(), GetSizeY(), AOMaskFormat);
		Texture->MipGenSettings = TMGS_LeaveExistingMips;
		Texture->Filter	= GUseBilinearLightmaps ? TF_Default : TF_Nearest;
		Texture->LODGroup = TEXTUREGROUP_Lightmap;
		Texture->LightmapFlags = ELightMapFlags( LightmapFlags );
		EncodeAOMaskTexture(Texture, 0u, TextureColor);
	}

	if (ShadowMapTexture != nullptr)
	{
		ShadowMapTexture->Filter = GUseBilinearLightmaps ? TF_Default : TF_Nearest;
		ShadowMapTexture->LODGroup = TEXTUREGROUP_Shadowmap;

		// TODO - Unify these?  Currently the values match
		ShadowMapTexture->ShadowmapFlags = EShadowMapFlags(LightmapFlags);

		ShadowMapTexture->Source.Init2DWithMipChain(GetSizeX(), GetSizeY(), ShadowMapFormat);
		ShadowMapTexture->MipGenSettings = TMGS_LeaveExistingMips;

		EncodeShadowMapTexture(ShadowMipData, ShadowMapTexture, 0u);
	}

	// Encode and compress the coefficient textures.
	for(uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		auto Texture = Textures[CoefficientIndex];
		if (Texture == nullptr)
		{
			continue;
		}

		Texture->Source.Init2DWithMipChain(GetSizeX(), GetSizeY() * 2, BaseFormat);	// Top/bottom atlased
		Texture->MipGenSettings = TMGS_LeaveExistingMips;
		Texture->Filter	= GUseBilinearLightmaps ? TF_Default : TF_Nearest;
		Texture->LODGroup = TEXTUREGROUP_Lightmap;
		Texture->LightmapFlags = ELightMapFlags( LightmapFlags );

		EncodeCoefficientTexture(CoefficientIndex, Texture, 0u, TextureColor, false);
	}

	// Encode virtual texture light maps
	{
		const uint32 InvalidLayerId = ~0u;

		for (uint32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
		{
			auto VirtualTexture = VirtualTextures[CoefficientIndex];
			if (VirtualTexture == nullptr)
			{
				continue;
			}

			// Copy data from all the separate Lightmap textures into the proper layers of the VT source
			const uint32 SkyOcclusionLayer = VirtualTexture->GetLayerForType(ELightMapVirtualTextureType::SkyOcclusion);
			const uint32 AOMaterialMaskLayer = VirtualTexture->GetLayerForType(ELightMapVirtualTextureType::AOMaterialMask);
			const uint32 ShadowMaskLayer = VirtualTexture->GetLayerForType(ELightMapVirtualTextureType::ShadowMask);

			TArray<ETextureSourceFormat> LayerFormat;
			LayerFormat.Init(TSF_Invalid, NumVirtualTextureLayers[CoefficientIndex]);
			LayerFormat[0] = BaseFormat;
			LayerFormat[1] = BaseFormat;

			if (SkyOcclusionLayer != InvalidLayerId)
			{
				LayerFormat[SkyOcclusionLayer] = SkyOcclusionFormat;
			}
			if (AOMaterialMaskLayer != InvalidLayerId)
			{
				LayerFormat[AOMaterialMaskLayer] = AOMaskFormat;
			}
			if (ShadowMaskLayer != InvalidLayerId)
			{
				LayerFormat[ShadowMaskLayer] = CoefficientIndex < LQ_LIGHTMAP_COEF_INDEX ? ShadowMapFormat : TSF_G8;
			}

			VirtualTexture->Source.InitLayered2DWithMipChain(GetSizeX(), GetSizeY(), NumVirtualTextureLayers[CoefficientIndex], LayerFormat.GetData());
			VirtualTexture->MipGenSettings = TMGS_LeaveExistingMips;
			VirtualTexture->SRGB = 0;
			VirtualTexture->Filter = GUseBilinearLightmaps ? TF_Default : TF_Nearest;
			VirtualTexture->LODGroup = TEXTUREGROUP_Lightmap;
			VirtualTexture->CompressionNoAlpha = false;
			VirtualTexture->CompressionNone = !GCompressLightmaps;
			VirtualTexture->LossyCompressionAmount = CVarVTEnableLossyCompressLightmaps.GetValueOnAnyThread() ? TLCA_Default : TLCA_None;

			// VirtualTexture->OodleTextureSdkVersion will be set to latest by default constructor
			//  dynamic/generated textures use latest OodleTextureSdkVersion

			FTextureFormatSettings DefaultFormatSettings;
			VirtualTexture->GetDefaultFormatSettings(DefaultFormatSettings);
			VirtualTexture->LayerFormatSettings.Init(DefaultFormatSettings, NumVirtualTextureLayers[CoefficientIndex]);

			EncodeCoefficientTexture(CoefficientIndex, VirtualTexture, 0u, TextureColor, true);

			if (SkyOcclusionLayer != InvalidLayerId)
			{
				EncodeSkyOcclusionTexture(VirtualTexture, SkyOcclusionLayer, TextureColor);
			}
			if (AOMaterialMaskLayer != InvalidLayerId)
			{
				EncodeAOMaskTexture(VirtualTexture, AOMaterialMaskLayer, TextureColor);
			}
			if (ShadowMaskLayer != InvalidLayerId)
			{
				EncodeShadowMapTexture(ShadowMipData, VirtualTexture, ShadowMaskLayer);
			}
		}
	}

	// Link textures to allocations
	for (int32 AllocationIndex = 0; AllocationIndex < Allocations.Num(); AllocationIndex++)
	{
		FLightMapAllocation& Allocation = *Allocations[AllocationIndex];
		Allocation.LightMap->SkyOcclusionTexture = SkyOcclusionTexture;
		Allocation.LightMap->AOMaterialMaskTexture = AOMaterialMaskTexture;
		Allocation.LightMap->ShadowMapTexture = ShadowMapTexture;
		Allocation.LightMap->Textures[0] = Textures[0];
		Allocation.LightMap->Textures[1] = Textures[2];
		Allocation.LightMap->VirtualTextures[0] = VirtualTextures[0];
		Allocation.LightMap->VirtualTextures[1] = VirtualTextures[2];
	}

	bIsFinishedEncoding = true;
}

FName FLightMapPendingTexture::GetLightmapName(int32 TextureIndex, int32 CoefficientIndex)
{
	check(CoefficientIndex >= 0 && CoefficientIndex < NUM_STORED_LIGHTMAP_COEF);
	FString PotentialName;
	UObject* ExistingObject = NULL;
	int32 LightmapIndex = 0;
	// Search for an unused name
	do
	{
		if (CoefficientIndex < NUM_HQ_LIGHTMAP_COEF)
		{
			PotentialName = FString("HQ_Lightmap") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex);
		}
		else
		{
			PotentialName = FString("LQ_Lightmap") + TEXT("_") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex);
		}
		ExistingObject = FindObject<UObject>(Outer, *PotentialName);
		LightmapIndex++;
	}
	while (ExistingObject != NULL);
	return FName(*PotentialName);
}

FName FLightMapPendingTexture::GetSkyOcclusionTextureName(int32 TextureIndex)
{
	FString PotentialName;
	UObject* ExistingObject = NULL;
	int32 LightmapIndex = 0;
	// Search for an unused name
	do
	{
		PotentialName = FString("SkyOcclusion") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex);

		ExistingObject = FindObject<UObject>(Outer, *PotentialName);
		LightmapIndex++;
	}
	while (ExistingObject != NULL);
	return FName(*PotentialName);
}

FName FLightMapPendingTexture::GetAOMaterialMaskTextureName(int32 TextureIndex)
{
	FString PotentialName;
	UObject* ExistingObject = NULL;
	int32 LightmapIndex = 0;
	// Search for an unused name
	do
	{
		PotentialName = FString("AOMaterialMask") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex);

		ExistingObject = FindObject<UObject>(Outer, *PotentialName);
		LightmapIndex++;
	}
	while (ExistingObject != NULL);
	return FName(*PotentialName);
}

FName FLightMapPendingTexture::GetShadowTextureName(int32 TextureIndex)
{
	FString PotentialName;
	UObject* ExistingObject = NULL;
	int32 LightmapIndex = 0;
	// Search for an unused name
	do
	{
		PotentialName = FString("StaticShadow") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex);

		ExistingObject = FindObject<UObject>(Outer, *PotentialName);
		LightmapIndex++;
	} while (ExistingObject != NULL);
	return FName(*PotentialName);
}

FName FLightMapPendingTexture::GetVirtualTextureName(int32 TextureIndex, int32 CoefficientIndex)
{
	FString PotentialName;
	UObject* ExistingObject = NULL;
	int32 LightmapIndex = 0;
	// Search for an unused name
	do
	{
		if (CoefficientIndex < NUM_HQ_LIGHTMAP_COEF)
		{
			PotentialName = FString("VirtualTexture") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex) + FString("_HQ");
		}
		else
		{
			PotentialName = FString("VirtualTexture") + FString::FromInt(LightmapIndex) + TEXT("_") + FString::FromInt(TextureIndex) + FString("_LQ");
		}

		ExistingObject = FindObject<UObject>(Outer, *PotentialName);
		LightmapIndex++;
	} while (ExistingObject != NULL);
	return FName(*PotentialName);
}

/** The light-maps which have not yet been encoded into textures. */
static TArray<FLightMapAllocationGroup> PendingLightMaps;
static uint32 PendingLightMapSize = 0;

#endif // WITH_EDITOR

/** If true, update the status when encoding light maps */
bool FLightMap2D::bUpdateStatus = true;

TRefCountPtr<FLightMap2D> FLightMap2D::AllocateLightMap(UObject* LightMapOuter,
	FQuantizedLightmapData*& SourceQuantizedData,
	const TMap<ULightComponent*, FShadowMapData2D*>& SourceShadowMapData,
	const FBoxSphereBounds& Bounds, ELightMapPaddingType InPaddingType, ELightMapFlags InLightmapFlags)
{
	// If the light-map has no lights in it, return NULL.
	if (!SourceQuantizedData && SourceShadowMapData.Num() == 0)
	{
		return NULL;
	}

#if WITH_EDITOR
	FLightMapAllocationGroup AllocationGroup;
	AllocationGroup.Outer = LightMapOuter;
	AllocationGroup.LightmapFlags = InLightmapFlags;
	AllocationGroup.Bounds = Bounds;
	if (!GAllowStreamingLightmaps)
	{
		AllocationGroup.LightmapFlags = ELightMapFlags(AllocationGroup.LightmapFlags & ~LMF_Streamed);
	}

	// Create a new light-map.
	TRefCountPtr<FLightMap2D> LightMap = TRefCountPtr<FLightMap2D>(new FLightMap2D());
	uint32 SizeX = 0u;
	uint32 SizeY = 0u;
	if (SourceQuantizedData)
	{
		LightMap->LightGuids = SourceQuantizedData->LightGuids;
		SizeX = SourceQuantizedData->SizeX;
		SizeY = SourceQuantizedData->SizeY;
	}
	for (const auto& It : SourceShadowMapData)
	{
		const FShadowMapData2D* ShadowMapData = It.Value;
		LightMap->LightGuids.AddUnique(It.Key->LightGuid);
		check(SizeX == 0u || SizeX == ShadowMapData->GetSizeX());
		check(SizeY == 0u || SizeY == ShadowMapData->GetSizeY());
		SizeX = ShadowMapData->GetSizeX();
		SizeY = ShadowMapData->GetSizeY();
	}

	// Create allocation and add it to the group
	{
		TUniquePtr<FLightMapAllocation> Allocation = MakeUnique<FLightMapAllocation>();
		Allocation->TotalSizeX = SizeX;
		Allocation->TotalSizeY = SizeY;
		Allocation->MappedRect.Max.X = SizeX;
		Allocation->MappedRect.Max.Y = SizeY;
		Allocation->PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
		if (SourceQuantizedData)
		{
			FMemory::Memcpy(Allocation->Scale, SourceQuantizedData->Scale, sizeof(Allocation->Scale));
			FMemory::Memcpy(Allocation->Add, SourceQuantizedData->Add, sizeof(Allocation->Add));
			Allocation->RawData = MoveTemp(SourceQuantizedData->Data);
			Allocation->bHasSkyShadowing = SourceQuantizedData->bHasSkyShadowing;

			// SourceQuantizedData is no longer needed now that FLightMapAllocation has what it needs
			delete SourceQuantizedData;
			SourceQuantizedData = NULL;

			// Track the size of pending light-maps.
			PendingLightMapSize += ((Allocation->TotalSizeX + 3) & ~3) * ((Allocation->TotalSizeY + 3) & ~3);
		}

		for (const auto& It : SourceShadowMapData)
		{
			const FShadowMapData2D* RawData = It.Value;
			TArray<FQuantizedSignedDistanceFieldShadowSample>& DistanceFieldShadowData = Allocation->ShadowMapData.Add(It.Key, TArray<FQuantizedSignedDistanceFieldShadowSample>());

			switch (RawData->GetType())
			{
			case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA:
			case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED:
				// If the data is already quantized, this will just copy the data
				RawData->Quantize(DistanceFieldShadowData);
				break;
			default:
				check(0);
			}

			delete RawData;

			// Track the size of pending light-maps.
			PendingLightMapSize += Allocation->TotalSizeX * Allocation->TotalSizeY;
		}

		Allocation->PaddingType = InPaddingType;
		Allocation->LightMap = LightMap;

#if WITH_EDITOR
		if (IsTexelDebuggingEnabled())
		{
			// Detect if this allocation belongs to the texture mapping that was being debugged
			//@todo - this only works for mappings that can be uniquely identified by a single component, BSP for example does not work.
			if (GCurrentSelectedLightmapSample.Component && GCurrentSelectedLightmapSample.Component == LightMapOuter)
			{
				GCurrentSelectedLightmapSample.Lightmap = LightMap;
				Allocation->bDebug = true;
			}
			else
			{
				Allocation->bDebug = false;
			}
		}
#endif

		AllocationGroup.Allocations.Add(MoveTemp(Allocation));
	}

	PendingLightMaps.Add(MoveTemp(AllocationGroup));

	return LightMap;
#else
	return NULL;
#endif // WITH_EDITOR
}

TRefCountPtr<FLightMap2D> FLightMap2D::AllocateInstancedLightMap(UObject* LightMapOuter, UInstancedStaticMeshComponent* Component,
	TArray<TUniquePtr<FQuantizedLightmapData>> InstancedSourceQuantizedData,
	TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>>&& InstancedShadowMapData,
	UMapBuildDataRegistry* Registry, FGuid MapBuildDataId, const FBoxSphereBounds& Bounds, ELightMapPaddingType InPaddingType, ELightMapFlags InLightmapFlags)
{
#if WITH_EDITOR
	check(InstancedSourceQuantizedData.Num() > 0);
	check(InstancedShadowMapData.Num() == 0 || InstancedShadowMapData.Num() == InstancedSourceQuantizedData.Num());

	// Verify all instance lightmaps are the same size
	const uint32 SizeX = InstancedSourceQuantizedData[0]->SizeX;
	const uint32 SizeY = InstancedSourceQuantizedData[0]->SizeY;
	for (int32 InstanceIndex = 1; InstanceIndex < InstancedSourceQuantizedData.Num(); ++InstanceIndex)
	{
		auto& SourceQuantizedData = InstancedSourceQuantizedData[InstanceIndex];
		check(SourceQuantizedData->SizeX == SizeX);
		check(SourceQuantizedData->SizeY == SizeY);
	}

	TSet<ULightComponent*> AllLights;
	for (int32 InstanceIndex = 0; InstanceIndex < InstancedShadowMapData.Num(); ++InstanceIndex)
	{
		for (const auto& It : InstancedShadowMapData[InstanceIndex])
		{
			const FShadowMapData2D* ShadowData = It.Value.Get();
			check(ShadowData->GetSizeX() == SizeX);
			check(ShadowData->GetSizeY() == SizeY);
			AllLights.Add(It.Key);
		}
	}

	// Unify all the shadow map data to contain the same lights in the same order
	for (auto& ShadowMapData : InstancedShadowMapData)
	{
		for (ULightComponent* Light : AllLights)
		{
			if (!ShadowMapData.Contains(Light))
			{
				ShadowMapData.Add(Light, MakeUnique<FQuantizedShadowSignedDistanceFieldData2D>(SizeX, SizeY));
			}
		}
	}

	// Requantize source data to the same quantization
	// Most of the following code is cloned from UModel::ApplyStaticLighting(), possibly it can be shared in future?
	// or removed, if instanced mesh components can be given per-instance lightmap unpack coefficients
	float MinCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
	float MaxCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
	for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2)
	{
		for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
		{
			// Color
			MinCoefficient[CoefficientIndex][ColorIndex] = FLT_MAX;
			MaxCoefficient[CoefficientIndex][ColorIndex] = 0.0f;

			// Direction
			MinCoefficient[CoefficientIndex + 1][ColorIndex] = FLT_MAX;
			MaxCoefficient[CoefficientIndex + 1][ColorIndex] = -FLT_MAX;
		}
	}

	// first, we need to find the max scale for all mappings, and that will be the scale across all instances of this component
	for (auto& SourceQuantizedData : InstancedSourceQuantizedData)
	{
		for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
		{
			for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
			{
				// The lightmap data for directional coefficients was packed in lightmass with
				// Pack: y = (x - Min) / (Max - Min)
				// We need to solve for Max and Min in order to combine BSP mappings into a lighting group.
				// QuantizedData->Scale and QuantizedData->Add were calculated in lightmass in order to unpack the lightmap data like so
				// Unpack: x = y * UnpackScale + UnpackAdd
				// Which means
				// Scale = Max - Min
				// Add = Min
				// Therefore we can solve for min and max using substitution

				float Scale = SourceQuantizedData->Scale[CoefficientIndex][ColorIndex];
				float Add = SourceQuantizedData->Add[CoefficientIndex][ColorIndex];
				float Min = Add;
				float Max = Scale + Add;

				MinCoefficient[CoefficientIndex][ColorIndex] = FMath::Min(MinCoefficient[CoefficientIndex][ColorIndex], Min);
				MaxCoefficient[CoefficientIndex][ColorIndex] = FMath::Max(MaxCoefficient[CoefficientIndex][ColorIndex], Max);
			}
		}
	}

	// Now calculate the new unpack scale and add based on the composite min and max
	float Scale[NUM_STORED_LIGHTMAP_COEF][4];
	float Add[NUM_STORED_LIGHTMAP_COEF][4];
	for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
	{
		for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
		{
			Scale[CoefficientIndex][ColorIndex] = FMath::Max(MaxCoefficient[CoefficientIndex][ColorIndex] - MinCoefficient[CoefficientIndex][ColorIndex], UE_DELTA);
			Add[CoefficientIndex][ColorIndex] = MinCoefficient[CoefficientIndex][ColorIndex];
		}
	}

	// perform requantization
	for (auto& SourceQuantizedData : InstancedSourceQuantizedData)
	{
		for (uint32 Y = 0; Y < SourceQuantizedData->SizeY; Y++)
		{
			for (uint32 X = 0; X < SourceQuantizedData->SizeX; X++)
			{
				// get source from input, dest from the rectangular offset in the group
				FLightMapCoefficients& LightmapSample = SourceQuantizedData->Data[Y * SourceQuantizedData->SizeX + X];

				// Treat alpha special because of residual
				{
					// Decode LogL
					float LogL = (float)LightmapSample.Coefficients[0][3] / 255.0f;
					float Residual = (float)LightmapSample.Coefficients[1][3] / 255.0f;
					LogL += (Residual - 0.5f) / 255.0f;
					LogL = LogL * SourceQuantizedData->Scale[0][3] + SourceQuantizedData->Add[0][3];

					// Encode LogL
					LogL = (LogL - Add[0][3]) / Scale[0][3];
					Residual = LogL * 255.0f - FMath::RoundToFloat(LogL * 255.0f) + 0.5f;

					LightmapSample.Coefficients[0][3] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(LogL * 255.0f), 0, 255);
					LightmapSample.Coefficients[1][3] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Residual * 255.0f), 0, 255);
				}

				// go over each color coefficient and dequantize and requantize with new Scale/Add
				for (int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
				{
					// Don't touch alpha here
					for (int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++)
					{
						// dequantize it
						float Dequantized = (float)LightmapSample.Coefficients[CoefficientIndex][ColorIndex] / 255.0f;
						const float Exponent = CoefficientIndex == 0 ? 2.0f : 1.0f;
						Dequantized = FMath::Pow(Dequantized, Exponent);

						const float Unpacked = Dequantized * SourceQuantizedData->Scale[CoefficientIndex][ColorIndex] + SourceQuantizedData->Add[CoefficientIndex][ColorIndex];
						const float Repacked = (Unpacked - Add[CoefficientIndex][ColorIndex]) / Scale[CoefficientIndex][ColorIndex];

						// requantize it
						LightmapSample.Coefficients[CoefficientIndex][ColorIndex] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(FMath::Pow(Repacked, 1.0f / Exponent) * 255.0f), 0, 255);
					}
				}
			}
		}

		// Save new requantized Scale/Add
		FMemory::Memcpy(SourceQuantizedData->Scale, Scale, sizeof(Scale));
		FMemory::Memcpy(SourceQuantizedData->Add, Add, sizeof(Add));
	}

	FLightMapAllocationGroup AllocationGroup = FLightMapAllocationGroup();
	AllocationGroup.Outer = LightMapOuter;
	AllocationGroup.LightmapFlags = InLightmapFlags;
	AllocationGroup.Bounds = Bounds;
	if (!GAllowStreamingLightmaps)
	{
		AllocationGroup.LightmapFlags = ELightMapFlags(AllocationGroup.LightmapFlags & ~LMF_Streamed);
	}

	TRefCountPtr<FLightMap2D> BaseLightmap = nullptr;

	for (int32 InstanceIndex = 0; InstanceIndex < InstancedSourceQuantizedData.Num(); ++InstanceIndex)
	{
		auto& SourceQuantizedData = InstancedSourceQuantizedData[InstanceIndex];

		// Create a new light-map.
		TRefCountPtr<FLightMap2D> LightMap = TRefCountPtr<FLightMap2D>(new FLightMap2D(SourceQuantizedData->LightGuids));

		if (InstancedShadowMapData.Num() != 0)
		{
			// Include light guids from shadowmap lights, if present
			for (auto& It : InstancedShadowMapData[InstanceIndex])
			{
				LightMap->LightGuids.AddUnique(It.Key->LightGuid);
				if (BaseLightmap)
				{
					// Base lightmap contains guids for all lights used by group
					BaseLightmap->LightGuids.AddUnique(It.Key->LightGuid);
				}
			}
		}

		if (InstanceIndex == 0)
		{
			BaseLightmap = LightMap;
		}
		else
		{
			// we need the base lightmap to contain all of the lights used by all lightmaps in the group
			for (auto& LightGuid : SourceQuantizedData->LightGuids)
			{
				BaseLightmap->LightGuids.AddUnique(LightGuid);
			}
		}

		TUniquePtr<FLightMapAllocation> Allocation = MakeUnique<FLightMapAllocation>(MoveTemp(*SourceQuantizedData));
		Allocation->PaddingType = InPaddingType;
		Allocation->LightMap = MoveTemp(LightMap);
		Allocation->Primitive = Component;
		Allocation->Registry = Registry;
		Allocation->MapBuildDataId = MapBuildDataId;
		Allocation->InstanceIndex = InstanceIndex;

		if (InstancedShadowMapData.Num() != 0)
		{
			for (auto& It : InstancedShadowMapData[InstanceIndex])
			{
				TUniquePtr<FShadowMapData2D>& RawData = It.Value;
				TArray<FQuantizedSignedDistanceFieldShadowSample>& DistanceFieldShadowData = Allocation->ShadowMapData.Add(It.Key, TArray<FQuantizedSignedDistanceFieldShadowSample>());

				switch (RawData->GetType())
				{
				case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA:
				case FShadowMapData2D::SHADOW_SIGNED_DISTANCE_FIELD_DATA_QUANTIZED:
					// If the data is already quantized, this will just copy the data
					RawData->Quantize(DistanceFieldShadowData);
					break;
				default:
					check(0);
				}

				RawData.Reset();

				// Track the size of pending light-maps.
				PendingLightMapSize += Allocation->TotalSizeX * Allocation->TotalSizeY;
			}
		}

		// SourceQuantizedData is no longer needed now that FLightMapAllocation has what it needs
		SourceQuantizedData.Reset();

		// Track the size of pending light-maps.
		PendingLightMapSize += ((Allocation->TotalSizeX + 3) & ~3) * ((Allocation->TotalSizeY + 3) & ~3);

		AllocationGroup.Allocations.Add(MoveTemp(Allocation));
	}

	PendingLightMaps.Add(MoveTemp(AllocationGroup));

	return BaseLightmap;
#else
	return nullptr;
#endif //WITH_EDITOR
}

/**
 * Executes all pending light-map encoding requests.
 * @param	bLightingSuccessful	Whether the lighting build was successful or not.
 * @param	bMultithreadedEncode encode textures on different threads ;)
 */
void FLightMap2D::EncodeTextures(const FStaticLightingBuildContext* LightingContext, bool bLightingSuccessful, bool bMultithreadedEncode)
{
#if WITH_EDITOR
	if (bLightingSuccessful)
	{
		const bool bUseVirtualTextures = UseVirtualTextureLightmap(GMaxRHIShaderPlatform);
		const bool bIncludeNonVirtualTextures = !bUseVirtualTextures || (CVarIncludeNonVirtualTexturedLightMaps.GetValueOnAnyThread() != 0);

		GWarn->BeginSlowTask( NSLOCTEXT("LightMap2D", "BeginEncodingLightMapsTask", "Encoding light-maps"), false );
		int32 PackedLightAndShadowMapTextureSizeX = LightingContext->World->GetWorldSettings()->PackedLightAndShadowMapTextureSize;
		int32 PackedLightAndShadowMapTextureSizeY = PackedLightAndShadowMapTextureSizeX / 2;

		if (!bIncludeNonVirtualTextures)
		{
			// If we exclusively using VT lightmaps, just make square textures, don't need 2x1 aspect ratio
			// YW: while it's possible to create huge VT lightmaps (like 32k), it hurts multithread lightmap encoding performance seriously.
			// Packed lightmaps with 1k - 2k resolution should be enough to reduce drawcalls
			PackedLightAndShadowMapTextureSizeY = PackedLightAndShadowMapTextureSizeX;
		}

		// Reset the pending light-map size.
		PendingLightMapSize = 0;

		// Crop lightmaps if allowed
		if (GAllowLightmapCropping)
		{
			for (FLightMapAllocationGroup& PendingGroup : PendingLightMaps)
			{
				if (!ensure(PendingGroup.Allocations.Num() >= 1))
				{
					continue;
				}

				// TODO: Crop all allocations in a group to the same size for instanced meshes
				if (PendingGroup.Allocations.Num() == 1)
				{
					for (auto& Allocation : PendingGroup.Allocations)
					{
						CropUnmappedTexels(Allocation->RawData, Allocation->TotalSizeX, Allocation->TotalSizeY, Allocation->MappedRect);
					}
				}
			}
		}

		// Calculate size of pending allocations for sorting
		for (FLightMapAllocationGroup& PendingGroup : PendingLightMaps)
		{
			PendingGroup.TotalTexels = 0;
			for (auto& Allocation : PendingGroup.Allocations)
			{
				// Assumes bAlignByFour
				PendingGroup.TotalTexels += ((Allocation->MappedRect.Width() + 3) & ~3) * ((Allocation->MappedRect.Height() + 3) & ~3);
			}
		}

		// Sort the light-maps in descending order by size.
		Algo::SortBy(PendingLightMaps, &FLightMapAllocationGroup::TotalTexels, TGreater<>());

		// Allocate texture space for each light-map.
		TArray<FLightMapPendingTexture*> PendingTextures;

		for (FLightMapAllocationGroup& PendingGroup : PendingLightMaps)
		{
			if (!ensure(PendingGroup.Allocations.Num() >= 1))
			{
				continue;
			}

			int32 MaxWidth = 0;
			int32 MaxHeight = 0;
			for (auto& Allocation : PendingGroup.Allocations)
			{
				MaxWidth = FMath::Max(MaxWidth, Allocation->MappedRect.Width());
				MaxHeight = FMath::Max(MaxHeight, Allocation->MappedRect.Height());
			}

			FLightMapPendingTexture* Texture = nullptr;

			// Find an existing texture which the light-map can be stored in.
			// Lightmaps will always be 4-pixel aligned...
			for (FLightMapPendingTexture* ExistingTexture : PendingTextures)
			{
				if (ExistingTexture->AddElement(PendingGroup))
				{
					Texture = ExistingTexture;
					break;
				}
			}

			if (!Texture)
			{
				int32 NewTextureSizeX = PackedLightAndShadowMapTextureSizeX;
				int32 NewTextureSizeY = PackedLightAndShadowMapTextureSizeY;

				// Assumes identically-sized allocations, fit into the smallest 2x1 rectangle
				const int32 AllocationCountX = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(FMath::DivideAndRoundUp(PendingGroup.Allocations.Num() * 2 * MaxHeight, MaxWidth))));
				const int32 AllocationCountY = FMath::DivideAndRoundUp(PendingGroup.Allocations.Num(), AllocationCountX);
				const int32 AllocationSizeX = AllocationCountX * MaxWidth;
				const int32 AllocationSizeY = AllocationCountY * MaxHeight;

				if (AllocationSizeX > NewTextureSizeX || AllocationSizeY > NewTextureSizeY)
				{
					NewTextureSizeX = FMath::RoundUpToPowerOfTwo(AllocationSizeX);
					NewTextureSizeY = FMath::RoundUpToPowerOfTwo(AllocationSizeY);

					// Force 2:1 aspect
					if (bIncludeNonVirtualTextures)
					{
						NewTextureSizeX = FMath::Max(NewTextureSizeX, NewTextureSizeY * 2);
						NewTextureSizeY = FMath::Max(NewTextureSizeY, NewTextureSizeX / 2);
					}
				}

				// If there is no existing appropriate texture, create a new one.
				// If we have non-VT, need 2to1 aspect ratio to handle packing lightmap into top/bottom texture region
				// With only VT, better to use square lightmaps (better page table usage)
				Texture = new FLightMapPendingTexture(LightingContext->World, NewTextureSizeX, NewTextureSizeY, bIncludeNonVirtualTextures ? ETextureLayoutAspectRatio::Force2To1 : ETextureLayoutAspectRatio::ForceSquare);
				PendingTextures.Add(Texture);
				Texture->Outer = PendingGroup.Outer;
				Texture->Bounds = PendingGroup.Bounds;
				Texture->LightmapFlags = PendingGroup.LightmapFlags;
				verify(Texture->AddElement(PendingGroup));
			}

			// Give the texture ownership of the allocations
			for (auto& Allocation : PendingGroup.Allocations)
			{
				Texture->Allocations.Add(MoveTemp(Allocation));
			}
		}
		PendingLightMaps.Empty();
		if (bMultithreadedEncode)
		{
			FThreadSafeCounter Counter(PendingTextures.Num());
			// allocate memory for all the async encode tasks
			TArray<FAsyncEncode<FLightMapPendingTexture>> AsyncEncodeTasks;
			AsyncEncodeTasks.Empty(PendingTextures.Num());
			for (auto& Texture :PendingTextures)
			{
				// precreate the UObjects then give them to some threads to process
				// need to precreate Uobjects 
				Texture->CreateUObjects();
				auto AsyncEncodeTask = new (AsyncEncodeTasks)FAsyncEncode<FLightMapPendingTexture>(Texture, LightingContext, Counter, nullptr);
				GLargeThreadPool->AddQueuedWork(AsyncEncodeTask);
			}

			while (Counter.GetValue() > 0)
			{
				GWarn->UpdateProgress(Counter.GetValue(), PendingTextures.Num());
				FPlatformProcess::Sleep(0.0001f);
			}
		}
		else
		{
			// Encode all the pending textures.
			for (int32 TextureIndex = 0; TextureIndex < PendingTextures.Num(); TextureIndex++)
			{
				if (bUpdateStatus && (TextureIndex % 20) == 0)
				{
					GWarn->UpdateProgress(TextureIndex, PendingTextures.Num());
				}
				FLightMapPendingTexture* PendingTexture = PendingTextures[TextureIndex];
				PendingTexture->StartEncoding(LightingContext,nullptr);
			}
		}

		// finish the encode (separate from waiting for the cache to complete)
		while (true)
		{
			bool bIsFinishedPostEncode = true;
			for (auto& PendingTexture : PendingTextures)
			{
				if (PendingTexture->IsFinishedEncoding())
				{
					PendingTexture->PostEncode();
				}
				else
				{
					// call post encode in order
					bIsFinishedPostEncode = false;
					break;
				}
			}
			if (bIsFinishedPostEncode)
				break;
		}

		for (auto& PendingTexture : PendingTextures)
		{
			PendingTexture->FinishCachingTextures();
			delete PendingTexture;
		}

		PendingTextures.Empty();


		// End the encoding lighmaps slow task
		GWarn->EndSlowTask();
		
	}
	else
	{
		PendingLightMaps.Empty();
	}
#endif //WITH_EDITOR
}

#if WITH_EDITOR
int32 FLightMap2D::EncodeShadowTexture(const FStaticLightingBuildContext* LightingContext, struct FLightMapPendingTexture& PendingTexture, TArray< TArray<FFourDistanceFieldSamples>>& MipData)
{
	TArray<FFourDistanceFieldSamples>* TopMipData = new(MipData) TArray<FFourDistanceFieldSamples>();
	TopMipData->Empty(PendingTexture.GetSizeX() * PendingTexture.GetSizeY());
	TopMipData->AddZeroed(PendingTexture.GetSizeX() * PendingTexture.GetSizeY());
	int32 TextureSizeX = PendingTexture.GetSizeX();
	int32 TextureSizeY = PendingTexture.GetSizeY();
	int32 MaxChannelsUsed = 0;

	for (int32 AllocationIndex = 0; AllocationIndex < PendingTexture.Allocations.Num(); AllocationIndex++)
	{
		FLightMapAllocation& Allocation = *PendingTexture.Allocations[AllocationIndex];
		bool bChannelUsed[4] = { 0 };
		FVector4f InvUniformPenumbraSize(0, 0, 0, 0);

		for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
		{
			for (const auto& ShadowMapPair : Allocation.ShadowMapData)
			{
				ULightComponent* CurrentLight = ShadowMapPair.Key;
				const FLightComponentMapBuildData* LightBuildData = LightingContext->GetOrCreateRegistryForActor(CurrentLight->GetOwner())->GetLightBuildData(CurrentLight->LightGuid);

				// Should have been setup by ReassignStationaryLightChannels
				check(LightBuildData);

				if (LightBuildData->ShadowMapChannel == ChannelIndex)
				{
					MaxChannelsUsed = FMath::Max(MaxChannelsUsed, ChannelIndex + 1);
					bChannelUsed[ChannelIndex] = true;
					const TArray<FQuantizedSignedDistanceFieldShadowSample>& SourceSamples = ShadowMapPair.Value;

					// Warning - storing one penumbra size for the whole shadowmap even though multiple lights can share a channel
					InvUniformPenumbraSize[ChannelIndex] = 1.0f / ShadowMapPair.Key->GetUniformPenumbraSize();

					// Copy the raw data for this light-map into the raw texture data array.
					for (int32 Y = Allocation.MappedRect.Min.Y; Y < Allocation.MappedRect.Max.Y; ++Y)
					{
						for (int32 X = Allocation.MappedRect.Min.X; X < Allocation.MappedRect.Max.X; ++X)
						{
							int32 DestY = Y - Allocation.MappedRect.Min.Y + Allocation.OffsetY;
							int32 DestX = X - Allocation.MappedRect.Min.X + Allocation.OffsetX;

							FFourDistanceFieldSamples& DestSample = (*TopMipData)[DestY * TextureSizeX + DestX];
							const FQuantizedSignedDistanceFieldShadowSample& SourceSample = SourceSamples[Y * Allocation.TotalSizeX + X];

							if (SourceSample.Coverage > 0)
							{
								// Note: multiple lights can write to different parts of the destination due to channel assignment
								DestSample.Samples[ChannelIndex] = SourceSample;
							}
							/*(if (SourceSample.Coverage > 0)
							{
								GNumShadowmapMappedTexels++;
							}
							else
							{
								GNumShadowmapUnmappedTexels++;
							}*/
						}
					}
				}
			}
		}

		// Free the shadow-map's raw data.
		for (auto& ShadowMapPair : Allocation.ShadowMapData)
		{
			ShadowMapPair.Value.Empty();
		}

		int32 PaddedSizeX = Allocation.TotalSizeX;
		int32 PaddedSizeY = Allocation.TotalSizeY;
		int32 BaseX = Allocation.OffsetX - Allocation.MappedRect.Min.X;
		int32 BaseY = Allocation.OffsetY - Allocation.MappedRect.Min.Y;

		if (GLightmassDebugOptions.bPadMappings && (Allocation.PaddingType == LMPT_NormalPadding))
		{
			if ((PaddedSizeX - 2 > 0) && ((PaddedSizeY - 2) > 0))
			{
				PaddedSizeX -= 2;
				PaddedSizeY -= 2;
				BaseX += 1;
				BaseY += 1;
			}
		}

		for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
		{
			Allocation.LightMap->bShadowChannelValid[ChannelIndex] = bChannelUsed[ChannelIndex];
		}

		Allocation.LightMap->InvUniformPenumbraSize = InvUniformPenumbraSize;
	}

	const uint32 NumMips = FMath::Max(FMath::CeilLogTwo(TextureSizeX), FMath::CeilLogTwo(TextureSizeY)) + 1;

	for (uint32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
	{
		const uint32 SourceMipSizeX = FMath::Max(1, TextureSizeX >> (MipIndex - 1));
		const uint32 SourceMipSizeY = FMath::Max(1, TextureSizeY >> (MipIndex - 1));
		const uint32 DestMipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		const uint32 DestMipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);

		// Downsample the previous mip-level, taking into account which texels are mapped.
		TArray<FFourDistanceFieldSamples>* NextMipData = new(MipData) TArray<FFourDistanceFieldSamples>();
		NextMipData->Empty(DestMipSizeX * DestMipSizeY);
		NextMipData->AddZeroed(DestMipSizeX * DestMipSizeY);
		const uint32 MipFactorX = SourceMipSizeX / DestMipSizeX;
		const uint32 MipFactorY = SourceMipSizeY / DestMipSizeY;

		for (uint32 Y = 0; Y < DestMipSizeY; Y++)
		{
			for (uint32 X = 0; X < DestMipSizeX; X++)
			{
				float AccumulatedFilterableComponents[4][FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents];

				for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
				{
					for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
					{
						AccumulatedFilterableComponents[ChannelIndex][i] = 0;
					}
				}
				uint32 Coverage[4] = { 0 };

				for (uint32 SourceY = Y * MipFactorY; SourceY < (Y + 1) * MipFactorY; SourceY++)
				{
					for (uint32 SourceX = X * MipFactorX; SourceX < (X + 1) * MipFactorX; SourceX++)
					{
						for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
						{
							const FFourDistanceFieldSamples& FourSourceSamples = MipData[MipIndex - 1][SourceY * SourceMipSizeX + SourceX];
							const FQuantizedSignedDistanceFieldShadowSample& SourceSample = FourSourceSamples.Samples[ChannelIndex];

							if (SourceSample.Coverage)
							{
								for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
								{
									AccumulatedFilterableComponents[ChannelIndex][i] += SourceSample.GetFilterableComponent(i) * SourceSample.Coverage;
								}

								Coverage[ChannelIndex] += SourceSample.Coverage;
							}
						}
					}
				}

				FFourDistanceFieldSamples& FourDestSamples = (*NextMipData)[Y * DestMipSizeX + X];

				for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
				{
					FQuantizedSignedDistanceFieldShadowSample& DestSample = FourDestSamples.Samples[ChannelIndex];

					if (Coverage[ChannelIndex])
					{
						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							DestSample.SetFilterableComponent(AccumulatedFilterableComponents[ChannelIndex][i] / (float)Coverage[ChannelIndex], i);
						}

						DestSample.Coverage = (uint8)(Coverage[ChannelIndex] / (MipFactorX * MipFactorY));
					}
					else
					{
						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							AccumulatedFilterableComponents[ChannelIndex][i] = 0;
						}
						DestSample.Coverage = 0;
					}
				}
			}
		}
	}

	const FIntPoint Neighbors[] =
	{
		// Check immediate neighbors first
		FIntPoint(1,0),
		FIntPoint(0,1),
		FIntPoint(-1,0),
		FIntPoint(0,-1),
		// Check diagonal neighbors if no immediate neighbors are found
		FIntPoint(1,1),
		FIntPoint(-1,1),
		FIntPoint(-1,-1),
		FIntPoint(1,-1)
	};

	// Extrapolate texels which are mapped onto adjacent texels which are not mapped to avoid artifacts when using texture filtering.
	for (int32 MipIndex = 0; MipIndex < MipData.Num(); MipIndex++)
	{
		uint32 MipSizeX = FMath::Max(1, TextureSizeX >> MipIndex);
		uint32 MipSizeY = FMath::Max(1, TextureSizeY >> MipIndex);

		for (uint32 DestY = 0; DestY < MipSizeY; DestY++)
		{
			for (uint32 DestX = 0; DestX < MipSizeX; DestX++)
			{
				FFourDistanceFieldSamples& FourDestSamples = MipData[MipIndex][DestY * MipSizeX + DestX];

				for (int32 ChannelIndex = 0; ChannelIndex < 4; ChannelIndex++)
				{
					FQuantizedSignedDistanceFieldShadowSample& DestSample = FourDestSamples.Samples[ChannelIndex];

					if (DestSample.Coverage == 0)
					{
						float ExtrapolatedFilterableComponents[FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents];

						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							ExtrapolatedFilterableComponents[i] = 0;
						}

						for (int32 NeighborIndex = 0; NeighborIndex < UE_ARRAY_COUNT(Neighbors); NeighborIndex++)
						{
							if (static_cast<int32>(DestY) + Neighbors[NeighborIndex].Y >= 0
								&& DestY + Neighbors[NeighborIndex].Y < MipSizeY
								&& static_cast<int32>(DestX) + Neighbors[NeighborIndex].X >= 0
								&& DestX + Neighbors[NeighborIndex].X < MipSizeX)
							{
								const FFourDistanceFieldSamples& FourNeighborSamples = MipData[MipIndex][(DestY + Neighbors[NeighborIndex].Y) * MipSizeX + DestX + Neighbors[NeighborIndex].X];
								const FQuantizedSignedDistanceFieldShadowSample& NeighborSample = FourNeighborSamples.Samples[ChannelIndex];

								if (NeighborSample.Coverage > 0)
								{
									if (static_cast<int32>(DestY) + Neighbors[NeighborIndex].Y * 2 >= 0
										&& DestY + Neighbors[NeighborIndex].Y * 2 < MipSizeY
										&& static_cast<int32>(DestX) + Neighbors[NeighborIndex].X * 2 >= 0
										&& DestX + Neighbors[NeighborIndex].X * 2 < MipSizeX)
									{
										// Lookup the second neighbor in the first neighbor's direction
										//@todo - check the second neighbor's coverage?
										const FFourDistanceFieldSamples& SecondFourNeighborSamples = MipData[MipIndex][(DestY + Neighbors[NeighborIndex].Y * 2) * MipSizeX + DestX + Neighbors[NeighborIndex].X * 2];
										const FQuantizedSignedDistanceFieldShadowSample& SecondNeighborSample = FourNeighborSamples.Samples[ChannelIndex];

										for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
										{
											// Extrapolate while maintaining the first derivative, which is especially important for signed distance fields
											ExtrapolatedFilterableComponents[i] = NeighborSample.GetFilterableComponent(i) * 2.0f - SecondNeighborSample.GetFilterableComponent(i);
										}
									}
									else
									{
										// Couldn't find a second neighbor to use for extrapolating, just copy the neighbor's values
										for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
										{
											ExtrapolatedFilterableComponents[i] = NeighborSample.GetFilterableComponent(i);
										}
									}
									break;
								}
							}
						}
						for (int32 i = 0; i < FQuantizedSignedDistanceFieldShadowSample::NumFilterableComponents; i++)
						{
							DestSample.SetFilterableComponent(ExtrapolatedFilterableComponents[i], i);
						}
					}
				}
			}
		}
	}

	return MaxChannelsUsed;
}
#endif // WITH_EDITOR

FLightMap2D::FLightMap2D()
{
	FMemory::Memzero(bShadowChannelValid);

	Textures[0] = NULL;
	Textures[1] = NULL;
	SkyOcclusionTexture = NULL;
	AOMaterialMaskTexture = NULL;
	ShadowMapTexture = NULL;
	VirtualTextures[0] = NULL;
	VirtualTextures[1] = NULL;
}

FLightMap2D::FLightMap2D(const TArray<FGuid>& InLightGuids)
{
	FMemory::Memzero(bShadowChannelValid);

	LightGuids = InLightGuids;
	Textures[0] = NULL;
	Textures[1] = NULL;
	SkyOcclusionTexture = NULL;
	AOMaterialMaskTexture = NULL;
	ShadowMapTexture = NULL;
	VirtualTextures[0] = NULL;
	VirtualTextures[1] = NULL;
}

const UTexture2D* FLightMap2D::GetTexture(uint32 BasisIndex) const
{
	return Textures[BasisIndex];
}

UTexture2D* FLightMap2D::GetTexture(uint32 BasisIndex)
{
	return Textures[BasisIndex];
}

UTexture2D* FLightMap2D::GetSkyOcclusionTexture() const
{
	return SkyOcclusionTexture;
}

UTexture2D* FLightMap2D::GetAOMaterialMaskTexture() const
{
	return AOMaterialMaskTexture;
}

bool FLightMap2D::IsVirtualTextureValid() const
{
#if WITH_EDITOR
	static const auto CVarSupportLowQualityLightmap = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
	const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmap) || (CVarSupportLowQualityLightmap->GetValueOnAnyThread() != 0);

	if (VirtualTextures[0] && (!bAllowLowQualityLightMaps || (bAllowLowQualityLightMaps && VirtualTextures[1])))
#else
	if (VirtualTextures[0] || VirtualTextures[1])
#endif
	{
		return true;
	}
	else
	{
		return false;
	}
}

/**
 * Returns whether the specified basis has a valid lightmap texture or not.
 * @param	BasisIndex - The basis index.
 * @return	true if the specified basis has a valid lightmap texture, otherwise false
 */
bool FLightMap2D::IsValid(uint32 BasisIndex) const
{
	return Textures[BasisIndex] != nullptr;
}

struct FLegacyLightMapTextureInfo
{
	ULightMapTexture2D* Texture;
	FLinearColor Scale;
	FLinearColor Bias;

	friend FArchive& operator<<(FArchive& Ar,FLegacyLightMapTextureInfo& I)
	{
		return Ar << I.Texture << I.Scale << I.Bias;
	}
};

void FLightMap2D::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(Textures[0]);
	Collector.AddReferencedObject(Textures[1]);
	Collector.AddReferencedObject(SkyOcclusionTexture);
	Collector.AddReferencedObject(AOMaterialMaskTexture);
	Collector.AddReferencedObject(ShadowMapTexture);
	Collector.AddReferencedObject(VirtualTextures[0]);
	Collector.AddReferencedObject(VirtualTextures[1]);
}

void FLightMap2D::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	const int32 RenderCustomVersion = Ar.CustomVer(FRenderingObjectVersion::GUID);

	FLightMap::Serialize(Ar);

	const bool bUsingVTLightmaps = UseVirtualTextureLightmap(GMaxRHIShaderPlatform, Ar.CookingTarget());

	if( Ar.IsLoading() && Ar.UEVer() < VER_UE4_LOW_QUALITY_DIRECTIONAL_LIGHTMAPS )
	{
		for(uint32 CoefficientIndex = 0;CoefficientIndex < 3;CoefficientIndex++)
		{
			ULightMapTexture2D* Dummy = NULL;
			Ar << Dummy;
			FVector4 Dummy2;
			Ar << Dummy2;
			Ar << Dummy2;
		}
	}
	else if( Ar.IsLoading() && Ar.UEVer() < VER_UE4_COMBINED_LIGHTMAP_TEXTURES )
	{
		for( uint32 CoefficientIndex = 0; CoefficientIndex < 4; CoefficientIndex++ )
		{
			ULightMapTexture2D* Dummy = NULL;
			Ar << Dummy;
			FVector4 Dummy2;
			Ar << Dummy2;
			Ar << Dummy2;
		}
	}
	else
	{
		if (Ar.IsCooking())
		{
			bool bStripLQLightmaps = !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::LowQualityLightmaps) || bUsingVTLightmaps;
			bool bStripHQLightmaps = !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::HighQualityLightmaps) || bUsingVTLightmaps;

			TObjectPtr<ULightMapTexture2D> Dummy;
			auto& Texture1 = bStripHQLightmaps ? Dummy : Textures[0];
			auto& Texture2 = bStripLQLightmaps ? Dummy : Textures[1];
			Ar << Texture1;
			Ar << Texture2;
		}
		else
		{
			Ar << Textures[0];
			Ar << Textures[1];
		}

		if (Ar.UEVer() >= VER_UE4_SKY_LIGHT_COMPONENT)
		{
			if (Ar.IsCooking())
			{
				bool bStripHQLightmaps = !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::HighQualityLightmaps) || bUsingVTLightmaps;

				TObjectPtr<ULightMapTexture2D> Dummy;
				auto& SkyTexture = bStripHQLightmaps ? Dummy : SkyOcclusionTexture;
				Ar << SkyTexture;

				if (Ar.UEVer() >= VER_UE4_AO_MATERIAL_MASK)
				{
					auto& MaskTexture = bStripHQLightmaps ? Dummy : AOMaterialMaskTexture;
					Ar << MaskTexture;
				}
			}
			else
			{
				Ar << SkyOcclusionTexture;

				if (Ar.UEVer() >= VER_UE4_AO_MATERIAL_MASK)
				{
					Ar << AOMaterialMaskTexture;
				}
			}
		}
		
		for(uint32 CoefficientIndex = 0;CoefficientIndex < NUM_STORED_LIGHTMAP_COEF;CoefficientIndex++)
		{
			Ar << ScaleVectors[CoefficientIndex];
			Ar << AddVectors[CoefficientIndex];
		}
	}

	Ar << CoordinateScale << CoordinateBias;

	if (RenderCustomVersion >= FRenderingObjectVersion::LightmapHasShadowmapData)
	{
		for (int Channel = 0; Channel < UE_ARRAY_COUNT(bShadowChannelValid); Channel++)
		{
			Ar << bShadowChannelValid[Channel];
		}

		Ar << InvUniformPenumbraSize;
	}

	if (RenderCustomVersion >= FRenderingObjectVersion::VirtualTexturedLightmaps)
	{
		if (RenderCustomVersion >= FRenderingObjectVersion::VirtualTexturedLightmapsV2)
		{
			if (RenderCustomVersion >= FRenderingObjectVersion::VirtualTexturedLightmapsV3)
			{
				// Don't save VT's if they are disabled for rendering
				if (bUsingVTLightmaps)
				{
					if (Ar.IsCooking())
					{
						bool bStripLQLightmaps = !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::LowQualityLightmaps);
						bool bStripHQLightmaps = !Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::HighQualityLightmaps);

						TObjectPtr<ULightMapVirtualTexture2D> Dummy;
						auto& Texture1 = bStripHQLightmaps ? Dummy : VirtualTextures[0];
						auto& Texture2 = bStripLQLightmaps ? Dummy : VirtualTextures[1];
						Ar << Texture1;
						Ar << Texture2;
					}
					else
					{
						Ar << VirtualTextures[0];
						Ar << VirtualTextures[1];
					}
				}
				else
				{
					ULightMapVirtualTexture2D* Dummy = NULL;
					Ar << Dummy;
					Ar << Dummy;
					if (Ar.IsLoading())
					{
						VirtualTextures[0] = nullptr;
						VirtualTextures[1] = nullptr;
					}
				}
			}
			else
			{
				// Don't save VT's if they are disabled for rendering
				if (bUsingVTLightmaps)
				{
					Ar << VirtualTextures[0];
					if (Ar.IsLoading())
					{
						VirtualTextures[1] = nullptr;
					}
				}
				else
				{
					ULightMapVirtualTexture2D* Dummy = NULL;
					Ar << Dummy;
					if (Ar.IsLoading())
					{
						VirtualTextures[0] = nullptr;
						VirtualTextures[1] = nullptr;
					}
				}
			}
		}
		else
		{
			// Older version using older lightmap texture type
			ULightMapVirtualTexture* Dummy = NULL;
			Ar << Dummy;
			if (Ar.IsLoading())
			{
				VirtualTextures[0] = nullptr;
				VirtualTextures[1] = nullptr;
			}
		}
	}
	else
	{
		VirtualTextures[0] = nullptr;
		VirtualTextures[1] = nullptr;
	}

	
	// Force no divide by zeros even with low precision. This should be fixed during build but for some reason isn't.
	if( Ar.IsLoading() )
	{
		for( int k = 0; k < 3; k++ )
		{
			ScaleVectors[2][k] = FMath::Max( 0.0f, ScaleVectors[2][k] );
			AddVectors[2][k] = FMath::Max( 0.01f, AddVectors[2][k] );
		}
	}

	//Release unneeded texture references on load so they will be garbage collected.
	//In the editor we need to keep these references since they will need to be saved.
	if (Ar.IsLoading() && !GIsEditor)
	{
		if (bUsingVTLightmaps)
		{
			Textures[0] = NULL;
			Textures[1] = NULL;
			SkyOcclusionTexture = NULL;
			AOMaterialMaskTexture = NULL;
		}
		else
		{
			Textures[bAllowHighQualityLightMaps ? 1 : 0] = NULL;

			if (!bAllowHighQualityLightMaps)
			{
				SkyOcclusionTexture = NULL;
				AOMaterialMaskTexture = NULL;
			}

			VirtualTextures[0] = NULL;
			VirtualTextures[1] = NULL;
		}
	}
}

FLightMapInteraction FLightMap2D::GetInteraction(ERHIFeatureLevel::Type InFeatureLevel) const
{
	bool bHighQuality = AllowHighQualityLightmaps(InFeatureLevel);

	int32 LightmapIndex = bHighQuality ? 0 : 1;

	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(InFeatureLevel));
	if (!bUseVirtualTextures)
	{
		bool bValidTextures = Textures[LightmapIndex] && Textures[LightmapIndex]->GetResource();

		// When the FLightMap2D is first created, the textures aren't set, so that case needs to be handled.
		if (bValidTextures)
		{
			return FLightMapInteraction::Texture(ToRawPtrArray(Textures), SkyOcclusionTexture, AOMaterialMaskTexture, ScaleVectors, AddVectors, CoordinateScale, CoordinateBias, bHighQuality);
		}
	}
	else
	{
		// Preview lightmaps don't stream from disk, thus no FVirtualTexture2DResource
		bool bValidVirtualTexture = VirtualTextures[LightmapIndex] && (VirtualTextures[LightmapIndex]->GetResource() != nullptr || VirtualTextures[LightmapIndex]->bPreviewLightmap);
		if (bValidVirtualTexture)
		{
			return FLightMapInteraction::InitVirtualTexture(VirtualTextures[LightmapIndex], ScaleVectors, AddVectors, CoordinateScale, CoordinateBias, bHighQuality);
		}
	}

	return FLightMapInteraction::None();
}

FShadowMapInteraction FLightMap2D::GetShadowInteraction(ERHIFeatureLevel::Type InFeatureLevel) const
{
	bool bHighQuality = AllowHighQualityLightmaps(InFeatureLevel);

	int32 LightmapIndex = bHighQuality ? 0 : 1;

	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(InFeatureLevel));
	if (bUseVirtualTextures)
	{
		// Preview lightmaps don't stream from disk, thus no FVirtualTexture2DResource
		const bool bValidVirtualTexture = VirtualTextures[LightmapIndex] && (VirtualTextures[LightmapIndex]->GetResource() != nullptr || VirtualTextures[LightmapIndex]->bPreviewLightmap);
		if (bValidVirtualTexture)
		{
			return FShadowMapInteraction::InitVirtualTexture(VirtualTextures[LightmapIndex], CoordinateScale, CoordinateBias, bShadowChannelValid, InvUniformPenumbraSize);
		}
	}
	return FShadowMapInteraction::None();
}

void FLegacyLightMap1D::Serialize(FArchive& Ar)
{
	FLightMap::Serialize(Ar);

	check(Ar.IsLoading());

	UObject* Owner;

	TQuantizedLightSampleBulkData<FQuantizedDirectionalLightSample> DirectionalSamples;
	TQuantizedLightSampleBulkData<FQuantizedSimpleLightSample> SimpleSamples;

	Ar << Owner;

	DirectionalSamples.Serialize( Ar, Owner, INDEX_NONE, false );

	for (int32 ElementIndex = 0; ElementIndex < 5; ElementIndex++)
	{
		FVector Dummy;
		Ar << Dummy;
	}

	SimpleSamples.Serialize( Ar, Owner, INDEX_NONE, false );
}

FArchive& operator<<(FArchive& Ar, FLightMap*& R)
{
	uint32 LightMapType = FLightMap::LMT_None;

	if (Ar.IsSaving())
	{
		if (R != nullptr)
		{
			if (R->GetLightMap2D())
			{
				LightMapType = FLightMap::LMT_2D;
			}
		}
	}

	Ar << LightMapType;

	if (Ar.IsLoading())
	{
		// explicitly don't call "delete R;",
		// we expect the calling code to handle that
		switch (LightMapType)
		{
		case FLightMap::LMT_None:
			R = nullptr;
			break;
		case FLightMap::LMT_1D:
			R = new FLegacyLightMap1D();
			break;
		case FLightMap::LMT_2D:
			R = new FLightMap2D();
			break;
		default:
			check(0);
		}
	}

	if (R != nullptr)
	{
		R->Serialize(Ar);

		if (Ar.IsLoading())
		{
			// Toss legacy vertex lightmaps
			if (LightMapType == FLightMap::LMT_1D)
			{
				delete R;
				R = nullptr;
			}

			// Dump old lightmaps
			if (Ar.UEVer() < VER_UE4_COMBINED_LIGHTMAP_TEXTURES)
			{
				delete R; // safe because if we're loading we new'd this above
				R = nullptr;
			}
		}
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FLightMapCoefficients& Coefs) 
{ 
	Ar.Serialize(&Coefs, sizeof(FLightMapCoefficients)); 
	return Ar;
}

void FQuantizedLightmapData::Serialize(FArchive& Ar)
{
	Ar << SizeX;
	Ar << SizeY;
	Ar << Data;

	Ar.Serialize(Scale, sizeof(Scale));
	Ar.Serialize(Add, sizeof(Add));
	Ar << LightGuids;
	
	Ar << bHasSkyShadowing;
}

bool FQuantizedLightmapData::HasNonZeroData() const
{
	// 1D lightmaps don't have a valid coverage amount, so they shouldn't be discarded if the coverage is 0
	uint8 MinCoverageThreshold = (SizeY == 1) ? 0 : 1;

	// Check all of the samples for a non-zero coverage (if valid) and at least one non-zero coefficient
	for (int32 SampleIndex = 0; SampleIndex < Data.Num(); SampleIndex++)
	{
		const FLightMapCoefficients& LightmapSample = Data[SampleIndex];

		if (LightmapSample.Coverage >= MinCoverageThreshold)
		{
			// Don't look at simple lightmap coefficients if we're not building them.
			static const auto CVarSupportLowQualityLightmaps = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
			const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmaps) || (CVarSupportLowQualityLightmaps->GetValueOnAnyThread() != 0);
			const int32 NumCoefficients = bAllowLowQualityLightMaps ? NUM_STORED_LIGHTMAP_COEF : NUM_HQ_LIGHTMAP_COEF;

			for (int32 CoefficentIndex = 0; CoefficentIndex < NumCoefficients; CoefficentIndex++)
			{
				if ((LightmapSample.Coefficients[CoefficentIndex][0] != 0) || (LightmapSample.Coefficients[CoefficentIndex][1] != 0) || (LightmapSample.Coefficients[CoefficentIndex][2] != 0))
				{
					return true;
				}
			}

			if (bHasSkyShadowing)
			{
				for (int32 Index = 0; Index < UE_ARRAY_COUNT(LightmapSample.SkyOcclusion); Index++)
				{
					if (LightmapSample.SkyOcclusion[Index] != 0)
					{
						return true;
					}
				}
			}

			if (LightmapSample.AOMaterialMask != 0)
			{
				return true;
			}
		}
	}

	return false;
}

FLightmapResourceCluster::~FLightmapResourceCluster()
{
	check(AllocatedVT == nullptr);
}

bool FLightmapResourceCluster::GetUseVirtualTexturing() const
{
	return UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(GetFeatureLevel()));
}

// Two stage initialization of FLightmapResourceCluster
// 1. when UMapBuildDataRegistry is post-loaded and render resource is initialized
// 2. when the level is made visible (ULevel::InitializeRenderingResources()), which calls UMapBuildDataRegistry::InitializeClusterRenderingResources() and fills FeatureLevel
// When both parts are provided, TryInitialize() creates the final UB with actual content
// Otherwise UniformBuffer is created with empty parameters
void FLightmapResourceCluster::TryInitializeUniformBuffer()
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	FLightmapResourceClusterShaderParameters Parameters;

	if (HasValidFeatureLevel())
	{
		ConditionalCreateAllocatedVT();
		GetLightmapClusterResourceParameters(GetFeatureLevel(), Input, GetUseVirtualTexturing() ? GetAllocatedVT() : nullptr, Parameters);
	}
	else
	{
		GetLightmapClusterResourceParameters(GMaxRHIFeatureLevel, FLightmapClusterResourceInput(), nullptr, Parameters);
	}

	if (!UniformBuffer.IsValid())
	{
		UniformBuffer = FLightmapResourceClusterShaderParameters::CreateUniformBuffer(Parameters, UniformBuffer_MultiFrame);
	}
	else
	{
		RHICmdList.UpdateUniformBuffer(UniformBuffer, &Parameters);
	}
}

void FLightmapResourceCluster::SetFeatureLevelAndInitialize(const FStaticFeatureLevel InFeatureLevel)
{
	check(IsInRenderingThread());
	SetFeatureLevel(InFeatureLevel);
	if(IsInitialized() && GIsRHIInitialized)
	{
		TryInitializeUniformBuffer();
	}
}

void FLightmapResourceCluster::UpdateUniformBuffer()
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	if (UniformBuffer.IsValid())
	{
		check(HasValidFeatureLevel());
	
		FLightmapResourceClusterShaderParameters Parameters;
		GetLightmapClusterResourceParameters(GetFeatureLevel(), Input, GetUseVirtualTexturing() ? GetAllocatedVT() : nullptr, Parameters);

		RHICmdList.UpdateUniformBuffer(UniformBuffer, &Parameters);
	}
}

static void OnVirtualTextureDestroyed(const FVirtualTextureProducerHandle& InHandle, void* Baton)
{
	FLightmapResourceCluster* Cluster = static_cast<FLightmapResourceCluster*>(Baton);
	Cluster->ReleaseAllocatedVT();
	Cluster->ConditionalCreateAllocatedVT();
	Cluster->UpdateUniformBuffer();
}

const IAllocatedVirtualTexture* FLightmapResourceCluster::GetAllocatedVT() const
{
	check(IsInParallelRenderingThread());
	return AllocatedVT;
}

void FLightmapResourceCluster::ConditionalCreateAllocatedVT()
{
	check(IsInRenderingThread());
	check(HasValidFeatureLevel());
	
	const ULightMapVirtualTexture2D* VirtualTexture = Input.LightMapVirtualTextures[AllowHighQualityLightmaps(GetFeatureLevel()) ? 0 : 1];
	
#if WITH_EDITOR
	// Compilation is still pending, this function will be called back once compilation finishes.
	if (VirtualTexture && VirtualTexture->IsCompiling())
	{
		return;
	}
#endif

	if (!AllocatedVT && VirtualTexture && VirtualTexture->GetResource())
	{
		check(VirtualTexture->VirtualTextureStreaming);
		const FVirtualTexture2DResource* Resource = (FVirtualTexture2DResource*)VirtualTexture->GetResource();
		const FVirtualTextureProducerHandle ProducerHandle = Resource->GetProducerHandle();

		GetRendererModule().AddVirtualTextureProducerDestroyedCallback(ProducerHandle, &OnVirtualTextureDestroyed, const_cast<FLightmapResourceCluster*>(this));

		FAllocatedVTDescription VTDesc;
		VTDesc.Dimensions = 2;
		VTDesc.TileSize = Resource->GetTileSize();
		VTDesc.TileBorderSize = Resource->GetBorderSize();
		VTDesc.NumTextureLayers = 0u;

		for (uint32 TypeIndex = 0u; TypeIndex < (uint32)ELightMapVirtualTextureType::Count; ++TypeIndex)
		{
			const uint32 LayerIndex = VirtualTexture->GetLayerForType((ELightMapVirtualTextureType)TypeIndex);
			if (LayerIndex != ~0u)
			{
				VTDesc.NumTextureLayers = TypeIndex + 1u;
				VTDesc.ProducerHandle[TypeIndex] = ProducerHandle; // use the same producer for each layer
				VTDesc.ProducerLayerIndex[TypeIndex] = LayerIndex;
			}
			else
			{
				VTDesc.ProducerHandle[TypeIndex] = FVirtualTextureProducerHandle();
			}
		}

		check(VTDesc.NumTextureLayers > 0u);
		for (uint32 LayerIndex = 0u; LayerIndex < VTDesc.NumTextureLayers; ++LayerIndex)
		{
			if (VTDesc.ProducerHandle[LayerIndex].PackedValue == 0u)
			{
				// if there are any layer 'holes' in our allocated VT, point the empty layer to layer0
				// this isn't strictly necessary, but without this VT feedback analysis will see an unmapped page each frame for the empty layer,
				// and attempt to do some extra work before determining there's nothing else to do...this wastes CPU time
				// By mapping to layer0, we ensure that every layer has a valid mapping, and the overhead of mapping the empty layer to layer0 is very small, since layer0 will already be resident
				VTDesc.ProducerHandle[LayerIndex] = ProducerHandle;
				VTDesc.ProducerLayerIndex[LayerIndex] = 0u;
			}
		}

		AllocatedVT = GetRendererModule().AllocateVirtualTexture(VTDesc);
	}
}

void FLightmapResourceCluster::ReleaseAllocatedVT()
{
	if (AllocatedVT)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		GetRendererModule().DestroyVirtualTexture(AllocatedVT);
		AllocatedVT = nullptr;
	}
}

void FLightmapResourceCluster::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FLightmapResourceCluster_InitRHI);
	TryInitializeUniformBuffer();
}

void FLightmapResourceCluster::ReleaseRHI()
{
	ReleaseAllocatedVT();
	UniformBuffer = nullptr;
}
