// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightSceneProxy.h"

#include "Components/LightComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/TextureLightProfile.h"
#include "EngineDefines.h"
#include "LightSceneProxyDesc.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "TextureResource.h"
#include "UObject/Package.h"

FLightSceneProxy::FLightSceneProxy(const ULightComponent* InLightComponent)
	: FLightSceneProxy(ULightComponent::BuildSceneProxyDesc(*InLightComponent))
{
}

FLightSceneProxy::FLightSceneProxy(const FLightSceneProxyDesc& InLightDesc)
	: LightComponent(InLightDesc.LightComponent)
	, LightComponentId(InLightDesc.LightComponentId)
	, SceneInterface(InLightDesc.SceneInterface)
	, Color(InLightDesc.Color)
	, IndirectLightingScale(InLightDesc.IndirectLightingScale)
	, VolumetricScatteringIntensity(FMath::Max(InLightDesc.VolumetricScatteringIntensity, 0.0f))
	, RayEndBias(InLightDesc.RayEndBias)
	, ShadowResolutionScale(InLightDesc.ShadowResolutionScale)
	, ShadowBias(InLightDesc.ShadowBias)
	, ShadowSlopeBias(InLightDesc.ShadowSlopeBias)
	, ShadowSharpen(InLightDesc.ShadowSharpen)
	, ContactShadowLength(InLightDesc.ContactShadowLength)
	, ContactShadowCastingIntensity(InLightDesc.ContactShadowCastingIntensity)
	, ContactShadowNonCastingIntensity(InLightDesc.ContactShadowNonCastingIntensity)
	, SpecularScale(InLightDesc.SpecularScale)
	, DiffuseScale(InLightDesc.DiffuseScale)
	, LightGuid(InLightDesc.LightGuid)
	, ShadowMapChannel(INDEX_NONE) // Initialized below
	, PreviewShadowMapChannel(INDEX_NONE) // Initialized below
	, RayStartOffsetDepthScale(InLightDesc.RayStartOffsetDepthScale)
	, StaticShadowDepthMap(InLightDesc.GetStaticShadowDepthMap())
	, LightFunctionScale(InLightDesc.LightFunctionScale)
	, LightFunctionFadeDistance(InLightDesc.LightFunctionFadeDistance)
	, LightFunctionDisabledBrightness(InLightDesc.LightFunctionDisabledBrightness)
	, LightFunctionMaterial(nullptr) // Initialized below
	, IESTexture(InLightDesc.IESTexture)
	, bContactShadowLengthInWS(InLightDesc.bContactShadowLengthInWS)
	, bMovable(InLightDesc.bMovable)
	, bStaticLighting(InLightDesc.bStaticLighting)
	, bStaticShadowing(InLightDesc.bStaticShadowing)
	, bCastDynamicShadow(InLightDesc.bCastDynamicShadow)
	, bCastStaticShadow(InLightDesc.bCastStaticShadow)
	, bCastTranslucentShadows(InLightDesc.bCastTranslucentShadows)
	, bTransmission(InLightDesc.bTransmission)
	, bCastVolumetricShadow(InLightDesc.bCastVolumetricShadow)
	, bCastHairStrandsDeepShadow(InLightDesc.bCastHairStrandsDeepShadow)
	, bCastShadowsFromCinematicObjectsOnly(InLightDesc.bCastShadowsFromCinematicObjectsOnly)
	, bForceCachedShadowsForMovablePrimitives(InLightDesc.bForceCachedShadowsForMovablePrimitives)
	, CastRaytracedShadow(InLightDesc.CastRaytracedShadow)
	, bAffectReflection(InLightDesc.bAffectReflection)
	, bAffectGlobalIllumination(InLightDesc.bAffectGlobalIllumination)
	, bAffectTranslucentLighting(InLightDesc.bAffectTranslucentLighting)
	, bUsedAsAtmosphereSunLight(InLightDesc.bUsedAsAtmosphereSunLight)
	, bUseRayTracedDistanceFieldShadows(InLightDesc.bUseRayTracedDistanceFieldShadows)
	, bUseVirtualShadowMaps(false) // Initialized below
	, bCastModulatedShadows(false)
	, bUseWholeSceneCSMForMovableObjects(false)
	, bSelected(InLightDesc.bSelected)
	, bAllowMegaLights(InLightDesc.bAllowMegaLights)
	, bIsPrecomputedLightingValid(InLightDesc.bIsPrecomputedLightingValid)
	, bEnableLightShaftBloom(InLightDesc.bEnableLightShaftBloom)
	, BloomScale(InLightDesc.BloomScale)
	, BloomThreshold(InLightDesc.BloomThreshold)
	, BloomMaxBrightness(InLightDesc.BloomMaxBrightness)
	, BloomTint(InLightDesc.BloomTint)
	, MegaLightsShadowMethod(InLightDesc.MegaLightsShadowMethod)
	, LightFunctionAtlasLightIndex(0)
	, AtmosphereSunLightIndex(InLightDesc.AtmosphereSunLightIndex)
	, AtmosphereSunDiskColorScale(InLightDesc.AtmosphereSunDiskColorScale)
	, LightType(InLightDesc.LightType)
	, LightingChannelMask(InLightDesc.LightingChannelMask)
	, ViewLightingChannelMask(InLightDesc.ViewLightingChannelMask)
	, StatId(InLightDesc.GetStatID())
	, ComponentName(InLightDesc.ComponentName)
	, LevelName(InLightDesc.LevelName)
	, VSMTexelDitherScale(1.0f)
	, VSMResolutionLodBias(0.0f)
	, FarShadowDistance(0)
	, FarShadowCascadeCount(0)
	, ModulatedShadowColor(EForceInit::ForceInitToZero)
	, ShadowAmount(1.0f)
	, SamplesPerPixel(InLightDesc.SamplesPerPixel)
	, DeepShadowLayerDistribution(InLightDesc.DeepShadowLayerDistribution)
	, IESAtlasId(~0u)
	, TimesliceAtlasIndex{0}
	, TimesliceAtlasFrame(0)
#if ACTOR_HAS_LABELS
	, OwnerNameOrLabel(InLightDesc.OwnerNameOrLabel)
#endif
{
	check(SceneInterface);
	check(LightComponentId.IsValid());

	// Currently we use virtual shadows maps for all lights when the global setting is enabled
	bUseVirtualShadowMaps = ::UseVirtualShadowMaps(SceneInterface->GetShaderPlatform(), SceneInterface->GetFeatureLevel());
	
	// Treat stationary lights as movable when VSMs are enabled
	if (bUseVirtualShadowMaps || !IsStaticLightingAllowed())
	{
		bStaticShadowing = bStaticLighting;
	}

	if ((InLightDesc.MapBuildDataShadowMapChannel != INDEX_NONE) && bStaticShadowing && !bStaticLighting)
	{
		ShadowMapChannel = InLightDesc.MapBuildDataShadowMapChannel;
	}
	else
	{
		ShadowMapChannel = INDEX_NONE;
	}

	// Use the preview channel if valid, otherwise fallback to the lighting build channel
	PreviewShadowMapChannel = InLightDesc.PreviewShadowMapChannel != INDEX_NONE ? InLightDesc.PreviewShadowMapChannel : ShadowMapChannel;

	if (InLightDesc.LightFunctionMaterial &&
		InLightDesc.LightFunctionMaterial->GetMaterial()->MaterialDomain == MD_LightFunction)
	{
		LightFunctionMaterial = InLightDesc.LightFunctionMaterial->GetRenderProxy();
	}

	if (bCastDynamicShadow && IsMobilePlatform(SceneInterface->GetShaderPlatform()))
	{
		if ((GetLightType() == LightType_Point && !DoesRuntimeSupportOnePassPointLightShadows(SceneInterface->GetShaderPlatform()))
			|| GetLightType() == LightType_Rect
			|| (GetLightType() == LightType_Spot && !IsMobileMovableSpotlightShadowsEnabled(SceneInterface->GetShaderPlatform())))
		{
			bCastDynamicShadow = false;
		}

		bTransmission = false;
	}
}

FLightSceneProxy::~FLightSceneProxy() = default;

bool FLightSceneProxy::ShouldCreatePerObjectShadowsForDynamicObjects() const
{
	// Only create per-object shadows for Stationary lights, which use static shadowing from the world and therefore need a way to integrate dynamic objects
	return HasStaticShadowing() && !HasStaticLighting();
}

/** Whether this light should create CSM for dynamic objects only (mobile renderer) */
bool FLightSceneProxy::UseCSMForDynamicObjects() const
{
	return false;
}

bool FLightSceneProxy::GetScissorRect(FIntRect& ScissorRect, const FSceneView& View, const FIntRect& ViewRect) const
{
	ScissorRect = ViewRect;
	return false;
}

void FLightSceneProxy::SetTransform(const FMatrix& InLightToWorld, const FVector4& InPosition)
{
	LightToWorld = InLightToWorld;
	WorldToLight = InLightToWorld.InverseFast();
	Position = InPosition;
}

void FLightSceneProxy::SetColor(const FLinearColor& InColor)
{
	Color = InColor;
}

void FLightSceneProxy::ApplyWorldOffset(FVector InOffset)
{
	FMatrix NewLightToWorld = LightToWorld.ConcatTranslation(InOffset);
	FVector4 NewPosition = Position + InOffset;
	SetTransform(NewLightToWorld, NewPosition);
}

FSphere FLightSceneProxy::GetBoundingSphere() const
{
	// Directional lights will have a radius of WORLD_MAX,
	// but we use UE_OLD_WORLD_MAX which is smaller, because WORLD_MAX is SUPER larger when set to UE_LARGE_WORLD_MAX,
	// and in this case some GPUs clipper can then fail for camera with a narrow field of view.
	return FSphere(FVector::ZeroVector, UE_OLD_WORLD_MAX);
}

FTexture* FLightSceneProxy::GetIESTextureResource() const
{
	return IESTexture ? IESTexture->GetResource() : nullptr;
}

void FLightSceneProxy::SetTransmission_GameThread(const ULightComponent* Component)
{
	ENQUEUE_RENDER_COMMAND(FSetTransmissionCommand)(
		[Proxy = this, ComponentTransmission = Component->bTransmission, ComponentCastsShadow = Component->CastShadows && Component->CastDynamicShadows](FRHICommandList& RHICmdList)
	{
		if (ComponentCastsShadow && IsMobilePlatform(Proxy->SceneInterface->GetShaderPlatform()))
		{
			Proxy->bTransmission = false;
		}
		else
		{
			Proxy->bTransmission = ComponentTransmission && Proxy->bCastDynamicShadow && !Proxy->bStaticShadowing;
		}
	});
}

void FLightSceneProxy::SetLightFunctionMaterial_GameThread(const ULightComponent* Component)
{
	const FMaterialRenderProxy* NewLightFunctionMaterial = nullptr;

	if (Component->LightFunctionMaterial &&
		Component->LightFunctionMaterial->GetMaterial()->MaterialDomain == MD_LightFunction)
	{
		NewLightFunctionMaterial = Component->LightFunctionMaterial->GetRenderProxy();
	}

	ENQUEUE_RENDER_COMMAND(FUpdateTransmissionCommand)(
		[Proxy = this, NewLightFunctionMaterial](FRHICommandList& RHICmdList)
	{
		Proxy->LightFunctionMaterial = NewLightFunctionMaterial;
	});
}