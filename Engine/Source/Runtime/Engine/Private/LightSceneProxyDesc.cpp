// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightSceneProxyDesc.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Components/LightComponent.h"
#include "UObject/Package.h"

void FLightSceneProxyDesc::Serialize(FArchive& Ar)
{
	FArchive_Serialize_BitfieldBool(Ar, bContactShadowLengthInWS);
	FArchive_Serialize_BitfieldBool(Ar, bMovable);
	FArchive_Serialize_BitfieldBool(Ar, bStaticLighting);
	FArchive_Serialize_BitfieldBool(Ar, bStaticShadowing);
	FArchive_Serialize_BitfieldBool(Ar, bCastDynamicShadow);
	FArchive_Serialize_BitfieldBool(Ar, bCastStaticShadow);
	FArchive_Serialize_BitfieldBool(Ar, bCastTranslucentShadows);
	FArchive_Serialize_BitfieldBool(Ar, bTransmission);
	FArchive_Serialize_BitfieldBool(Ar, bCastVolumetricShadow);
	FArchive_Serialize_BitfieldBool(Ar, bCastHairStrandsDeepShadow);
	FArchive_Serialize_BitfieldBool(Ar, bCastShadowsFromCinematicObjectsOnly);
	FArchive_Serialize_BitfieldBool(Ar, bForceCachedShadowsForMovablePrimitives);
	FArchive_Serialize_BitfieldBool(Ar, bAffectReflection);
	FArchive_Serialize_BitfieldBool(Ar, bAffectGlobalIllumination);
	FArchive_Serialize_BitfieldBool(Ar, bAffectTranslucentLighting);
	FArchive_Serialize_BitfieldBool(Ar, bUsedAsAtmosphereSunLight);
	FArchive_Serialize_BitfieldBool(Ar, bUseRayTracedDistanceFieldShadows);
	FArchive_Serialize_BitfieldBool(Ar, bAllowMegaLights);
	FArchive_Serialize_BitfieldBool(Ar, bIsPrecomputedLightingValid);
	FArchive_Serialize_BitfieldBool(Ar, bEnableLightShaftBloom);
	Ar << Color;
	Ar << IndirectLightingScale;
	Ar << VolumetricScatteringIntensity;
	Ar << RayEndBias;
	Ar << ShadowResolutionScale;
	Ar << ShadowBias;
	Ar << ShadowSlopeBias;
	Ar << ShadowSharpen;
	Ar << ContactShadowLength;
	Ar << ContactShadowCastingIntensity;
	Ar << ContactShadowNonCastingIntensity;
	Ar << SpecularScale;
	Ar << DiffuseScale;
	Ar << LightGuid;
	Ar << MapBuildDataShadowMapChannel;
	Ar << PreviewShadowMapChannel;
	Ar << RayStartOffsetDepthScale;
	Ar << LightFunctionScale;
	Ar << LightFunctionFadeDistance;
	Ar << LightFunctionDisabledBrightness;
	Ar << LightFunctionMaterial;
	Ar << IESTexture;
	Ar << CastRaytracedShadow;
	Ar << BloomScale;
	Ar << BloomThreshold;
	Ar << BloomMaxBrightness;
	Ar << BloomTint;
	Ar << MegaLightsShadowMethod;
	Ar << AtmosphereSunLightIndex;
	Ar << AtmosphereSunDiskColorScale;
	Ar << LightType;
	Ar << LightingChannelMask;
	Ar << ViewLightingChannelMask;
	Ar << ComponentName;
	Ar << LevelName;
	Ar << SamplesPerPixel;
	Ar << DeepShadowLayerDistribution;
}

const FStaticShadowDepthMap* FLightSceneProxyDesc::GetStaticShadowDepthMap() const
{
	return LightComponent ? LightComponent->GetStaticShadowDepthMap() : nullptr;
}

TStatId FLightSceneProxyDesc::GetStatID() const
{
	return LightComponent ? LightComponent->GetStatID(true) : TStatId();
}