// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/LightWeightInstanceBlueprintFunctionLibrary.h"
#include "GameFramework/Actor.h"
#include "GameFramework/LightWeightInstanceSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightWeightInstanceBlueprintFunctionLibrary)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FActorInstanceHandle UDEPRECATED_LightWeightInstanceBlueprintFunctionLibrary::CreateNewLightWeightInstance(UClass* InActorClass, FTransform InTransform, UDataLayerInstance* InLayer, UWorld* World)
{
	// set up initialization data
	FLWIData PerInstanceData;
	PerInstanceData.Transform = InTransform;

	return FLightWeightInstanceSubsystem::Get().CreateNewLightWeightInstance(InActorClass, &PerInstanceData, InLayer, World);
}

FActorInstanceHandle UDEPRECATED_LightWeightInstanceBlueprintFunctionLibrary::ConvertActorToLightWeightInstance(AActor* InActor)
{
	if (!ensure(InActor))
	{
		return FActorInstanceHandle();
	}

	// Get or create a light weight instance for this class and layer
	// use the first layer the actor is in if it's in multiple layers
#if WITH_EDITOR
	TArray<const UDataLayerInstance*> DataLayerInstances = InActor->GetDataLayerInstances();
	const UDataLayerInstance* DataLayerInstance = DataLayerInstances.Num() > 0 ? DataLayerInstances[0] : nullptr;
#else
	const UDataLayerInstance* DataLayerInstance = nullptr;
#endif // WITH_EDITOR
	if (ADEPRECATED_LightWeightInstanceManager* LWIManager = FLightWeightInstanceSubsystem::Get().FindOrAddLightWeightInstanceManager(*InActor->GetClass(), *InActor->GetWorld(), InActor->GetActorLocation(), DataLayerInstance))
	{
		return LWIManager->ConvertActorToLightWeightInstance(InActor);
	}

	return FActorInstanceHandle(InActor);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
