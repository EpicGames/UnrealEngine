// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/LightWeightInstanceManager.h"
#include "GameFramework/LightWeightInstanceSubsystem.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Misc/ScopeExit.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightWeightInstanceManager)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

// CVar variable that is the size of the grid managers are placed into.
static int32 LWIGridSize = -1;
static FAutoConsoleVariableRef CVarLWIGridSize
(
	TEXT("LWI.Editor.GridSize"),
	LWIGridSize,
	TEXT("Sets the size of a grid that LWI managers will be generated with."),
	ECVF_Default
);


ADEPRECATED_LightWeightInstanceManager::ADEPRECATED_LightWeightInstanceManager(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{
	bReplicates = true;
	AcceptedClass = AActor::StaticClass();
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FLightWeightInstanceSubsystem::Get().AddManager(this);
	}
}

ADEPRECATED_LightWeightInstanceManager::~ADEPRECATED_LightWeightInstanceManager()
{
	FLightWeightInstanceSubsystem::Get().RemoveManager(this);
}

void ADEPRECATED_LightWeightInstanceManager::SetRepresentedClass(UClass* ActorClass)
{
	RepresentedClass = ActorClass;
}

void ADEPRECATED_LightWeightInstanceManager::Tick(float DeltaSeconds)
{
	// do nothing
}

AActor* ADEPRECATED_LightWeightInstanceManager::FindActor(const FActorInstanceHandle& Handle)
{
	TObjectPtr<AActor>* FoundActor = Actors.Find(Handle.GetInstanceIndex());

	return FoundActor ? *FoundActor : nullptr;
}

AActor* ADEPRECATED_LightWeightInstanceManager::FindOrCreateActor(const FActorInstanceHandle& Handle)
{
	// make sure the handle doesn't have an actor already
	if (ensure(!Handle.GetCachedActor()))
	{
		// check if we already have an actor for this handle
		TObjectPtr<AActor>* FoundActor = nullptr;
		if ((FoundActor = Actors.Find(Handle.GetInstanceIndex())) == nullptr)
		{
			// spawn a new actor
			ConvertInstanceToActor(Handle);
		}
		else
		{
#if !UE_BUILD_SHIPPING
			const bool bSpawnInProgress = InstanceToActorConversionsInProgress.Contains(Handle.GetInstanceIndex());
			if (!ensure(!bSpawnInProgress))
			{
				UE_LOGF(LogLightWeightInstance, Error, 
					"Calling FetchActorFromHandle on ActorInstance [ %ls ] index [ %d ] - actor is spawned but may not be fully setup",
					*BaseInstanceName,
					Handle.GetInstanceIndex());
			}
#endif //!UE_BUILD_SHIPPING

			Handle.SetCachedActor(*FoundActor);
		}
	}

	// Unless we are on the server or the actor has been spawned, this ensure will fail.
	// Commented out until there is more robust replication.
	// ensure(Handle.Actor.IsValid());
	return Handle.GetCachedActor();
}

AActor* ADEPRECATED_LightWeightInstanceManager::ConvertInstanceToActor(const FActorInstanceHandle& Handle)
{
#if !UE_BUILD_SHIPPING
	check(!InstanceToActorConversionsInProgress.Contains(Handle.GetInstanceIndex()));
	InstanceToActorConversionsInProgress.Add(Handle.GetInstanceIndex());
	ON_SCOPE_EXIT
	{
		InstanceToActorConversionsInProgress.RemoveSingleSwap(Handle.GetInstanceIndex());
	};
#endif //!UE_BUILD_SHIPPING

	// we shouldn't be calling this on indices that already have an actor representing them
	if (Actors.Contains(Handle.GetInstanceIndex()) && Actors[Handle.GetInstanceIndex()] != nullptr)
	{
		return Actors[Handle.GetInstanceIndex()];
	}
	// if we're pointing at an invalid index we can't return an actor
	if (ValidIndices.Num() <= Handle.GetInstanceIndex() || ValidIndices[Handle.GetInstanceIndex()] == false)
	{
		return nullptr;
	}

	//if we have previously spawned an actor for this index, don't spawn again even though we have no current actor entry.
	//this may occur if an FActorInstanceHandle was cached before a spawned actor was destroyed and fetched after.
	if (DestroyedActorIndices.Contains(Handle.GetInstanceIndex()))
	{
		UE_LOGF(LogLightWeightInstance, Verbose,
			"ADEPRECATED_LightWeightInstanceManager::ConvertInstanceToActor - manager [ %ls ] had already spawned an actor for index [ %d ] that was destroyed",
			*GetActorNameOrLabel(),
			Handle.GetInstanceIndex());
		return nullptr;
	}

	AActor* NewActor = nullptr;
	// Only spawn actors on the server so they are replicated to the clients. Otherwise we'll end up with multiples.
	if (HasAuthority())
	{
		FActorSpawnParameters SpawnParams;
		SetSpawnParameters(SpawnParams);
		SpawnParams.CustomPreSpawnInitialization = [this, &Handle](AActor* SpawnedActor)
		{
			PreSpawnInitialization(Handle, SpawnedActor);
		};

		NewActor = GetLevel()->GetWorld()->SpawnActor<AActor>(GetActorClassToSpawn(Handle), GetTransform(Handle), SpawnParams);
		check(NewActor);

		//should have been assigned in CustomPreSpawnInitialization
		check(Handle.GetCachedActor() == NewActor);
		check(NewActor == Actors.FindRef(Handle.GetInstanceIndex()));

		NewActor->OnDestroyed.AddUniqueDynamic(this, &ADEPRECATED_LightWeightInstanceManager::OnSpawnedActorDestroyed);

		PostActorSpawn(Handle);
	}

	return NewActor;
}

void ADEPRECATED_LightWeightInstanceManager::OnSpawnedActorDestroyed(AActor* DestroyedActor)
{
	OnSpawnedActorDestroyed(DestroyedActor, FindIndexForActor(DestroyedActor));
}

void ADEPRECATED_LightWeightInstanceManager::OnSpawnedActorDestroyed(AActor* DestroyedActor, const int32 DestroyedActorInstanceIndex)
{
	check(DestroyedActor);
	if (!ensure(DestroyedActorInstanceIndex != INDEX_NONE))
	{
		UE_LOGF(LogLightWeightInstance, Error,
			"OnSpawnedActorDestroyed - actor [ %ls ] is not being tracked by manager [ %ls ]",
			*DestroyedActor->GetName(),
			*BaseInstanceName);
	}

	Actors.Remove(DestroyedActorInstanceIndex);
	DestroyedActor->OnDestroyed.RemoveAll(this);

	if (ensure(!DestroyedActorIndices.Contains(DestroyedActorInstanceIndex)))
	{
		DestroyedActorIndices.Add(DestroyedActorInstanceIndex);
	}
	else
	{
		UE_LOGF(LogLightWeightInstance, Error,
			"ADEPRECATED_LightWeightInstanceManager::OnSpawnedActorDestroyed - manager [ %ls ] DestroyedActorIndices already has an entry for index [ %d ]",
			*GetActorNameOrLabel(),
			DestroyedActorInstanceIndex);
	}
}

int32 ADEPRECATED_LightWeightInstanceManager::FindIndexForActor(const AActor* InActor) const
{
	for (const TPair<int32, TObjectPtr<AActor>>& ActorPair : Actors)
	{
		if (ActorPair.Value == InActor)
		{
			return ActorPair.Key;
		}
	}

	return INDEX_NONE;
}

FActorInstanceHandle ADEPRECATED_LightWeightInstanceManager::ConvertActorToLightWeightInstance(AActor* InActor)
{
	if (!InActor)
	{
		return FActorInstanceHandle();
	}

	if (FLWIData* Data = AllocateInitData())
	{
		SetDataFromActor(Data, InActor);
		// If we used to manage this actor as a light weight instance then we will already have entries associated with it. Check for these so we can use them again instead of allocating new entries
		int32 Idx = INDEX_NONE;
		for (auto& IndexActorPair : Actors)
		{
			if (IndexActorPair.Value == InActor)
			{
				Idx = IndexActorPair.Key;
				break;
			}
		}

		if (Idx == INDEX_NONE)
		{
			Idx = AddNewInstance(Data);
			Idx = ConvertInternalIndexToHandleIndex(Idx);
		}
		else
		{
			UpdateDataAtIndex(Data, Idx);
		}

		// cleanup
		InActor->Destroy();
		delete Data;

		return FActorInstanceHandle::MakeDehydratedActorHandle(*this, Idx);
	}
	else
	{
		// something went wrong and we can't manage this actor
		// just return a handle to the actor
		return FActorInstanceHandle(InActor);
	}
}

FLWIData* ADEPRECATED_LightWeightInstanceManager::AllocateInitData() const
{
	return new FLWIData;
}


bool ADEPRECATED_LightWeightInstanceManager::SetDataFromActor(FLWIData* InData, AActor* InActor) const
{
	if (!InData || !InActor)
	{
		return false;
	}

	InData->Transform = InActor->GetActorTransform();

	return true;
}

int32 ADEPRECATED_LightWeightInstanceManager::ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const
{
	return InIndex;
}

int32 ADEPRECATED_LightWeightInstanceManager::GetGridSize() const
{
	return LWIGridSize;	
}

FInt32Vector3 ADEPRECATED_LightWeightInstanceManager::ConvertPositionToCoord(const FVector& InPosition) const
{
	float GridSize = GetGridSize();
	if (GridSize > 0)
	{
		return FInt32Vector3(
			FMath::FloorToInt(InPosition.X / GridSize),
			FMath::FloorToInt(InPosition.Y / GridSize),
			FMath::FloorToInt(InPosition.Z / GridSize)
		);
	}

	return FInt32Vector3::ZeroValue;
}

FBox ADEPRECATED_LightWeightInstanceManager::ConvertPositionToGridBounds(const FVector& InPosition) const
{
	float GridSize = GetGridSize();
	if (GridSize > 0)
	{
		const FInt32Vector3 GridCoord = ConvertPositionToCoord(InPosition);

		FBox GridBounds;
		GridBounds.Min = FVector(GridCoord * GridSize);
		GridBounds.Max = FVector(GridCoord * GridSize + FInt32Vector3(GridSize));
		return GridBounds;
	}

	return FBox(InPosition, InPosition);
}

void ADEPRECATED_LightWeightInstanceManager::SetSpawnParameters(FActorSpawnParameters& SpawnParams)
{
	SpawnParams.OverrideLevel = GetLevel();
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.ObjectFlags = RF_Transactional;
}

UClass* ADEPRECATED_LightWeightInstanceManager::GetActorClassToSpawn(const FActorInstanceHandle& Handle) const
{
	return RepresentedClass;
}

void ADEPRECATED_LightWeightInstanceManager::PreSpawnInitialization(const FActorInstanceHandle& Handle, AActor* SpawnedActor)
{
	Handle.SetCachedActor(SpawnedActor);
	Actors.Add(Handle.GetInstanceIndex(), SpawnedActor);
}

void ADEPRECATED_LightWeightInstanceManager::PostActorSpawn(const FActorInstanceHandle& Handle)
{
	// do nothing
}

bool ADEPRECATED_LightWeightInstanceManager::IsIndexValid(int32 Index) const
{
	if (ValidIndices.IsValidIndex(Index))
	{
		return ValidIndices[Index];
	}

	return false;
}

bool ADEPRECATED_LightWeightInstanceManager::FindActorForHandle(const FActorInstanceHandle& Handle) const
{
	if (Handle.GetCachedActor())
	{
		return true;
	}

	const TObjectPtr<AActor>* FoundActor = Actors.Find(Handle.GetInstanceIndex());
	AActor* ResolvedActor = FoundActor ? FoundActor->Get() : nullptr;
	Handle.SetCachedActor(ResolvedActor);
	return ResolvedActor != nullptr;
}

FVector ADEPRECATED_LightWeightInstanceManager::GetLocation(const FActorInstanceHandle& Handle) const
{
	return GetTransform(Handle).GetTranslation();
}

FRotator ADEPRECATED_LightWeightInstanceManager::GetRotation(const FActorInstanceHandle& Handle) const
{
	return GetTransform(Handle).Rotator();
}

FTransform ADEPRECATED_LightWeightInstanceManager::GetTransform(const FActorInstanceHandle& Handle) const
{
	if (FindActorForHandle(Handle))
	{
		check(Handle.GetCachedActor());
		return Handle.GetCachedActor()->GetActorTransform();
	}

	if (ensure(IsIndexValid(Handle.GetInstanceIndex())))
	{
		// Return in world space.
		return InstanceTransforms[Handle.GetInstanceIndex()] * GetActorTransform();
	}

	return FTransform();
}

FString ADEPRECATED_LightWeightInstanceManager::GetName(const FActorInstanceHandle& Handle) const
{
	if (FindActorForHandle(Handle))
	{
		check(Handle.GetCachedActor());
		return Handle.GetCachedActor()->GetName();
	}

	return FString::Printf(TEXT("%s_%u"), *BaseInstanceName, Handle.GetInstanceIndex());
}

bool ADEPRECATED_LightWeightInstanceManager::DoesRepresentClass(const UClass* OtherClass) const
{
	return OtherClass ? OtherClass->IsChildOf(GetRepresentedClassInternal()) : false;
}

bool ADEPRECATED_LightWeightInstanceManager::DoesAcceptClass(const UClass* OtherClass) const
{
	return OtherClass ? OtherClass->IsChildOf(GetAcceptedClass()) : false;
}

UClass* ADEPRECATED_LightWeightInstanceManager::GetRepresentedClassInternal() const
{
	return IsValid(RepresentedClass) ? RepresentedClass : nullptr;
}

UClass* ADEPRECATED_LightWeightInstanceManager::GetAcceptedClass() const
{
	return IsValid(AcceptedClass) ? AcceptedClass : nullptr;
}

int32 ADEPRECATED_LightWeightInstanceManager::AddNewInstance(FLWIData* InitData)
{
	if (!InitData)
	{
		return INDEX_NONE;
	}

	// use one of the free indices if any are available; otherwise grow the size of the array
	const int32 DataIdx = FreeIndices.Num() > 0 ? FreeIndices.Pop(EAllowShrinking::No) : ValidIndices.Num();
	
	// Update the rest of our per instance data
	AddNewInstanceAt(InitData, DataIdx);

	return DataIdx;
}

void ADEPRECATED_LightWeightInstanceManager::AddNewInstanceAt(FLWIData* InitData, int32 Index)
{
	// allocate space on the end of the array if we need to
	if (Index >= ValidIndices.Num())
	{
		GrowDataArrays();
	}
	ensure(Index < ValidIndices.Num());

	// update our data
	UpdateDataAtIndex(InitData, Index);
	ValidIndices[Index] = true;
}

void ADEPRECATED_LightWeightInstanceManager::GrowDataArrays()
{
	InstanceTransforms.AddUninitialized();
	ValidIndices.AddUninitialized();

	check(InstanceTransforms.Num() == ValidIndices.Num());
}

void ADEPRECATED_LightWeightInstanceManager::UpdateDataAtIndex(FLWIData* InData, int32 Index)
{
	// Convert to local space.
	InstanceTransforms[Index] = InData->Transform * GetActorTransform().Inverse();
}

void ADEPRECATED_LightWeightInstanceManager::RemoveInstance(const int32 Index)
{
	if (ensure(IsIndexValid(Index)))
	{
		// mark the index as no longer in use
		FreeIndices.Add(Index);
		ValidIndices[Index] = false;

		// destroy the associated actor if one existed
		if (TObjectPtr<AActor>* FoundActor = Actors.Find(Index))
		{
			AActor* ActorToDestroy = *FoundActor;
			ActorToDestroy->Destroy();
		}
	}
}

bool ADEPRECATED_LightWeightInstanceManager::HasAnyValidInstancesOrManagedActors() const
{
	bool bHasAnyValidIndex = false;
	for (const bool bIsIndexValid : ValidIndices)
	{
		if (bIsIndexValid)
		{
			bHasAnyValidIndex = true;
			break;
		}
	}
	return bHasAnyValidIndex;
}

void ADEPRECATED_LightWeightInstanceManager::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceManager, RepresentedClass);
	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceManager, InstanceTransforms);
	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceManager, FreeIndices);
	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceManager, ValidIndices);
}

void ADEPRECATED_LightWeightInstanceManager::OnRep_Transforms()
{
	// do nothing
}
#if WITH_EDITOR
void ADEPRECATED_LightWeightInstanceManager::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (Name == GET_MEMBER_NAME_CHECKED(ADEPRECATED_LightWeightInstanceManager, RepresentedClass))
	{
		SetRepresentedClass(RepresentedClass);
	}
}
#endif

PRAGMA_ENABLE_DEPRECATION_WARNINGS
