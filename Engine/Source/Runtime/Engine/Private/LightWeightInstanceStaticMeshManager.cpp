// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/LightWeightInstanceStaticMeshManager.h"

#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "GameFramework/LightWeightInstanceSubsystem.h"
#include "Net/UnrealNetwork.h"
#include "Templates/Greater.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightWeightInstanceStaticMeshManager)

#if WITH_EDITOR
#include "Editor.h"
#else
#include "Engine/World.h"
#include "TimerManager.h"
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS

ADEPRECATED_LightWeightInstanceStaticMeshManager::ADEPRECATED_LightWeightInstanceStaticMeshManager(const FObjectInitializer& ObjectInitializer)
	:Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Still create the deprecated InstancedStaticMeshComponent so the old data can be serialized during loading
	InstancedStaticMeshComponent_DEPRECATED = CreateDefaultSubobject<UHierarchicalInstancedStaticMeshComponent>(TEXT("InstancedStaticMeshComponent0"));
#endif

	ISMComponent = CreateDefaultSubobject<UInstancedStaticMeshComponent>(TEXT("ISMComponent0"));
	ISMComponent->SetRemoveSwap();
	SetRootComponent(ISMComponent);
	AddInstanceComponent(ISMComponent);

	if (StaticMesh.IsValid())
	{
		OnStaticMeshSet();
	}

	SetInstancedStaticMeshParams();
}

#if WITH_EDITORONLY_DATA
void ADEPRECATED_LightWeightInstanceStaticMeshManager::PostLoad()
{
	Super::PostLoad();

	if (InstancedStaticMeshComponent_DEPRECATED != nullptr && InstancedStaticMeshComponent_DEPRECATED->PerInstanceSMData.Num() > 0)
	{
		// assume new ISM component is still empty
		check(ISMComponent->PerInstanceSMData.Num() == 0);

		// copy over serialized instance data to ISM from HISM
		ISMComponent->SetRelativeTransform(InstancedStaticMeshComponent_DEPRECATED->GetRelativeTransform());
		ISMComponent->SetMobility(InstancedStaticMeshComponent_DEPRECATED->Mobility);
		ISMComponent->SetStaticMesh(InstancedStaticMeshComponent_DEPRECATED->GetStaticMesh());
		ISMComponent->InstancingRandomSeed = InstancedStaticMeshComponent_DEPRECATED->InstancingRandomSeed;

		int32 StartCullDistance, EndCullDistance;
		InstancedStaticMeshComponent_DEPRECATED->GetCullDistances(StartCullDistance, EndCullDistance);
		ISMComponent->SetCullDistances(StartCullDistance, EndCullDistance);

		for (int32 Idx = 0; Idx < InstancedStaticMeshComponent_DEPRECATED->GetNumOverrideMaterials(); ++Idx)
		{
			ISMComponent->SetMaterial(Idx, InstancedStaticMeshComponent_DEPRECATED->GetMaterial(Idx));
		}
		for (FTransform& InstanceTransform : InstanceTransforms)
		{
			ISMComponent->AddInstance(InstanceTransform, /*bWorldSpace*/false);
		}

		ISMComponent->OnPostPopulatePerInstanceData();

		// release all instances in the HISM
		InstancedStaticMeshComponent_DEPRECATED->ClearInstances();

		// restore the root component after destroying the old root component)
		SetRootComponent(ISMComponent);

		// all instances have been added in order of the transforms so reset the indirection lookup tables
		DataIndicesToRenderingIndices.SetNum(InstanceTransforms.Num());
		RenderingIndicesToDataIndices.SetNum(InstanceTransforms.Num());
		for (int32 DataIndex = 0; DataIndex < InstanceTransforms.Num(); ++DataIndex)
		{
			DataIndicesToRenderingIndices[DataIndex] = DataIndex;
			RenderingIndicesToDataIndices[DataIndex] = DataIndex;
		}
	}
}
#endif

void ADEPRECATED_LightWeightInstanceStaticMeshManager::SetRepresentedClass(UClass* ActorClass)
{
	Super::SetRepresentedClass(ActorClass);

	AActor* ActorCDO = RepresentedClass ? Cast<AActor>(RepresentedClass->GetDefaultObject()) : nullptr;
	if (ActorCDO)
	{
		BaseInstanceName = ActorCDO->GetName();
		SetStaticMeshFromActor(ActorCDO);
	}
	else
	{
		BaseInstanceName.Reset();
		ClearStaticMesh();
	}

	if (ISMComponent)
	{
		ISMComponent->OnPostPopulatePerInstanceData();
	}
}

int32 ADEPRECATED_LightWeightInstanceStaticMeshManager::ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const
{
	if (ensureMsgf(RenderingIndicesToDataIndices.IsValidIndex(InIndex), TEXT("Invalid index [ %d ]"), InIndex))
	{
		return RenderingIndicesToDataIndices[InIndex];
	}

	return InIndex;
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::AddNewInstanceAt(FLWIData* InitData, int32 Index)
{
	Super::AddNewInstanceAt(InitData, Index);
	AddInstanceToRendering(Index);
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::RemoveInstance(const int32 Index)
{
#if WITH_EDITOR
	Modify();
#endif

	RemoveInstanceFromRendering(Index);
	Super::RemoveInstance(Index);
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::AddInstanceToRendering(int32 DataIndex)
{
	if (!ensureMsgf(RenderingIndicesToDataIndices.Contains(DataIndex) == false, TEXT("LWI rendering instance added more than once. Index: %d"), DataIndex))
	{
		return;
	}

	//cancel any pending deletes
	const bool bPendingDeleteCancelled = DataIndicesToBeDeleted.RemoveSingle(DataIndex) > 0;

	// The rendering indices are tightly packed so we know it's going on the end of the array
	const int32 RenderingIdx = RenderingIndicesToDataIndices.Add(DataIndex);

	UE_LOGF(LogLightWeightInstance, Verbose,
		"ADEPRECATED_LightWeightInstanceStaticMeshManager::AddInstanceToRendering - manager [ %ls ] adding instance at data index [ %d ] rendering index [ %d ], did cancel pending delete? [ %ls ]",
		*GetActorNameOrLabel(),
		DataIndex,
		RenderingIdx,
		bPendingDeleteCancelled ? TEXT("true") : TEXT("false"));

	// Now that we know the rendering index we can fill in the other side of the map
	if (DataIndex >= DataIndicesToRenderingIndices.Num())
	{
		ensure(DataIndex == DataIndicesToRenderingIndices.Add(RenderingIdx));
	}
	else
	{
		DataIndicesToRenderingIndices[DataIndex] = RenderingIdx;
	}

	// Update the ISMC
	if (ISMComponent)
	{
		ensure(RenderingIdx == ISMComponent->AddInstance(InstanceTransforms[DataIndex], /*bWorldSpace*/false));
	}
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::RemoveInstanceFromRendering(int32 DataIndex)
{
	if (IsIndexValid(DataIndex))
	{
		if (DataIndicesToBeDeleted.IsEmpty())
		{
			// We can't remove the indices right away because we could have a situation where we receive multiple requests to convert an instance to an actor in the same frame	
#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->GetTimerManager()->SetTimerForNextTick(this, &ADEPRECATED_LightWeightInstanceStaticMeshManager::PostRemoveInstanceFromRendering);
			}
			else
#endif // WITH_EDITOR
			{
				GetWorld()->GetTimerManager().SetTimerForNextTick(this, &ADEPRECATED_LightWeightInstanceStaticMeshManager::PostRemoveInstanceFromRendering);
			}
		}

		DataIndicesToBeDeleted.AddUnique(DataIndex);

		UE_LOGF(LogLightWeightInstance, Verbose,
			"ADEPRECATED_LightWeightInstanceStaticMeshManager::RemoveInstanceFromRendering - manager [ %ls ] removing instance at data index [ %d ]",
			*GetActorNameOrLabel(),
			DataIndex);
	}
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::PostRemoveInstanceFromRendering()
{
	TArray<int32> RenderingIndicesToBeDeleted;
	for (int32 DataIndex : DataIndicesToBeDeleted)
	{
		RenderingIndicesToBeDeleted.Add(DataIndicesToRenderingIndices[DataIndex]);

		UE_LOGF(LogLightWeightInstance, Verbose,
			"ADEPRECATED_LightWeightInstanceStaticMeshManager::PostRemoveInstanceFromRendering - manager [ %ls ] removing instance at data index [ %d ] rendering index [ %d ]",
			*GetActorNameOrLabel(),
			DataIndex,
			DataIndicesToRenderingIndices[DataIndex]);
	}

	RenderingIndicesToBeDeleted.Sort();
	int32 Count = RenderingIndicesToBeDeleted.Num();
	for (int32 Idx = Count-1; Idx >= 0; Idx--)
	{
		int32 RenderingIndex = RenderingIndicesToBeDeleted[Idx];
		if (RenderingIndex != INDEX_NONE)
		{
			if (ISMComponent)
			{
				ISMComponent->RemoveInstance(RenderingIndex);
			}
			RenderingIndicesToDataIndices.RemoveAtSwap(RenderingIndex);

			// fix up the other side of the map to match the change we just made
			// if we removed the last element than nothing was moved so we're done
			if (RenderingIndex < RenderingIndicesToDataIndices.Num())
			{
				// find the data index that corresponds with the changed rendering index
				const int32 ShiftedDataIndex = RenderingIndicesToDataIndices[RenderingIndex];

				DataIndicesToRenderingIndices[ShiftedDataIndex] = RenderingIndex;
			}
		}
	}

	DataIndicesToBeDeleted.Reset();
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceStaticMeshManager, StaticMesh);
	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceStaticMeshManager, RenderingIndicesToDataIndices);
	DOREPLIFETIME(ADEPRECATED_LightWeightInstanceStaticMeshManager, DataIndicesToRenderingIndices);
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::OnRep_StaticMesh()
{
	OnStaticMeshSet();
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::OnRep_Transforms()
{
	Super::OnRep_Transforms();

	if (ISMComponent)
	{
		ISMComponent->AddInstance(InstanceTransforms.Last(), /*bWorldSpace*/false);
	}
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::PostActorSpawn(const FActorInstanceHandle& Handle)
{
	Super::PostActorSpawn(Handle);

	// remove the rendered instance from the HISMC
	RemoveInstanceFromRendering(Handle.GetInstanceIndex());
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::SetInstancedStaticMeshParams()
{
	FName CollisionProfileName("LightWeightInstancedStaticMeshPhysics");
	ISMComponent->SetCollisionProfileName(CollisionProfileName);

	ISMComponent->CanCharacterStepUpOn = ECB_Owner;
	ISMComponent->CastShadow = true;
	ISMComponent->bCastDynamicShadow = true;
	ISMComponent->bCastStaticShadow = true;
	ISMComponent->PrimaryComponentTick.bCanEverTick = false;
	// Allows updating in game, while optimizing rendering for the case that it is not modified
	ISMComponent->Mobility = EComponentMobility::Movable;
	// Allows per-instance selection in the editor
	ISMComponent->bHasPerInstanceHitProxies = true;

	if(!IsTemplate())
	{
		while(ISMComponent->InstancingRandomSeed == 0)
		{
			ISMComponent->InstancingRandomSeed = FMath::Rand();
		}
	}
} 

void ADEPRECATED_LightWeightInstanceStaticMeshManager::SetStaticMeshFromActor(AActor* InActor)
{
	ensureMsgf(false, TEXT("ALightWeightInstanceManager::SetStaticMeshFromActor was called on %s. Projects should override this function in subclasses."), *GetNameSafe(InActor));
	ClearStaticMesh();
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::OnStaticMeshSet()
{
#if WITH_EDITOR
	Modify();
#endif
	
	if (ISMComponent)
	{
		EComponentMobility::Type Mobility = ISMComponent->Mobility;
		if (Mobility == EComponentMobility::Static)
		{
			ISMComponent->SetMobility(EComponentMobility::Stationary);
			ISMComponent->SetStaticMesh(StaticMesh.Get());
			ISMComponent->SetMobility(Mobility);
		}
		else
		{
			ISMComponent->SetStaticMesh(StaticMesh.Get());
		}

		if (UStaticMesh* Mesh = StaticMesh.Get())
		{
			for (int32 Idx = 0; Idx < Mesh->GetStaticMaterials().Num(); ++Idx)
			{
				ISMComponent->SetMaterial(Idx, Mesh->GetMaterial(Idx));
			}
		}
	}
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::ClearStaticMesh()
{
#if WITH_EDITOR
	Modify();
#endif
	
	StaticMesh = nullptr;
	OnStaticMeshSet();
}

FText ADEPRECATED_LightWeightInstanceStaticMeshManager::GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const
{
	check(InstanceId.ISMComponent == ISMComponent);

	const FText OwnerDisplayName = FText::FromString(RepresentedClass ? RepresentedClass->GetName() : AActor::GetName());
	return FText::Format(NSLOCTEXT("LightWeightInstanceStaticMeshManager", "DisplayNameFmt", "{0} - Instance {1}"), OwnerDisplayName, InstanceId.InstanceIndex);
}

FText ADEPRECATED_LightWeightInstanceStaticMeshManager::GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const
{
	check(InstanceId.ISMComponent == ISMComponent);
	
	const FText OwnerDisplayPath = FText::FromString(GetPathName(GetWorld())); // stops the path at the level of the world the object is in
	return FText::Format(NSLOCTEXT("LightWeightInstanceStaticMeshManager", "TooltipFmt", "Instance {0} on {1}"), InstanceId.InstanceIndex, OwnerDisplayPath);
}

bool ADEPRECATED_LightWeightInstanceStaticMeshManager::CanEditSMInstance(const FSMInstanceId& InstanceId) const
{
	check(InstanceId.ISMComponent == ISMComponent);

	ISMInstanceManager* InstanceManager = ISMComponent;
	return InstanceManager->CanEditSMInstance(InstanceId);
}

bool ADEPRECATED_LightWeightInstanceStaticMeshManager::CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const
{
	check(InstanceId.ISMComponent == ISMComponent);

	ISMInstanceManager* InstanceManager = ISMComponent;
	return InstanceManager->CanMoveSMInstance(InstanceId, InWorldType);
}

bool ADEPRECATED_LightWeightInstanceStaticMeshManager::GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
	check(InstanceId.ISMComponent == ISMComponent);

	if (RenderingIndicesToDataIndices.IsValidIndex(InstanceId.InstanceIndex))
	{
		const int32 DataIndex = RenderingIndicesToDataIndices[InstanceId.InstanceIndex];
		OutInstanceTransform = bWorldSpace ? InstanceTransforms[DataIndex] * GetActorTransform() : InstanceTransforms[DataIndex];
		return true;
	}
	
	return false;
}

bool ADEPRECATED_LightWeightInstanceStaticMeshManager::SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	check(InstanceId.ISMComponent == ISMComponent);

	if (RenderingIndicesToDataIndices.IsValidIndex(InstanceId.InstanceIndex) && ISMComponent->UpdateInstanceTransform(InstanceId.InstanceIndex, InstanceTransform, bWorldSpace, bMarkRenderStateDirty, bTeleport))
	{
		const int32 DataIndex = RenderingIndicesToDataIndices[InstanceId.InstanceIndex];
		ISMComponent->GetInstanceTransform(InstanceId.InstanceIndex, InstanceTransforms[DataIndex], /*bWorldSpace*/false);
		return true;
	}

	return false;
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId)
{
	check(InstanceId.ISMComponent == ISMComponent);

	ISMInstanceManager* InstanceManager = ISMComponent;
	InstanceManager->NotifySMInstanceMovementStarted(InstanceId);
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId)
{
	check(InstanceId.ISMComponent == ISMComponent);

	ISMInstanceManager* InstanceManager = ISMComponent;
	InstanceManager->NotifySMInstanceMovementOngoing(InstanceId);
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId)
{
	check(InstanceId.ISMComponent == ISMComponent);

	ISMInstanceManager* InstanceManager = ISMComponent;
	InstanceManager->NotifySMInstanceMovementEnded(InstanceId);
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected)
{
	check(InstanceId.ISMComponent == ISMComponent);

	ISMInstanceManager* InstanceManager = ISMComponent;
	InstanceManager->NotifySMInstanceSelectionChanged(InstanceId, bIsSelected);
}

bool ADEPRECATED_LightWeightInstanceStaticMeshManager::DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds)
{
	TArray<int32> DataIndices;
	GetLWIDataIndices(InstanceIds, DataIndices);

	// Sort so Remove doesn't alter the indices of items still to remove
	DataIndices.Sort(TGreater<int32>());

	Modify();
	ISMComponent->Modify();

	for (int32 DataIndex : DataIndices)
	{
		RemoveInstance(DataIndex);
	}

	return true;
}

bool ADEPRECATED_LightWeightInstanceStaticMeshManager::DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds)
{
	TArray<int32> DataIndices;
	GetLWIDataIndices(InstanceIds, DataIndices);

	Modify();
	ISMComponent->Modify();

	TArray<int32> NewDataIndices;
	DuplicateLWIInstances(DataIndices, NewDataIndices);

	OutNewInstanceIds.Reset(NewDataIndices.Num());
	for (int32 NewDataIndex : NewDataIndices)
	{
		OutNewInstanceIds.Add(FSMInstanceId{ ISMComponent, DataIndicesToRenderingIndices[NewDataIndex] });
	}

	return true;
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::DuplicateLWIInstances(TArrayView<const int32> DataIndices, TArray<int32>& OutNewDataIndices)
{
	OutNewDataIndices.Reset(DataIndices.Num());
	for (int32 DataIndex : DataIndices)
	{
		FLWIData InitData;
		InitData.Transform = InstanceTransforms[DataIndex];

		const int32 NewDataIndex = AddNewInstance(&InitData);
		if (DataIndex != INDEX_NONE)
		{
			OutNewDataIndices.Add(NewDataIndex);
		}
	}
}

void ADEPRECATED_LightWeightInstanceStaticMeshManager::GetLWIDataIndices(TArrayView<const FSMInstanceId> InstanceIds, TArray<int32>& OutDataIndices) const
{
	OutDataIndices.Reset(InstanceIds.Num());
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		check(InstanceId.ISMComponent == ISMComponent);
		if (RenderingIndicesToDataIndices.IsValidIndex(InstanceId.InstanceIndex))
		{
			OutDataIndices.Add(RenderingIndicesToDataIndices[InstanceId.InstanceIndex]);
		}
	}
}

int32 ADEPRECATED_LightWeightInstanceStaticMeshManager::ConvertInternalIndexToHandleIndex(int32 InInternalIndex) const
{
	return DataIndicesToRenderingIndices[InInternalIndex];
}

int32 ADEPRECATED_LightWeightInstanceStaticMeshManager::ConvertHandleIndexToInternalIndex(int32 InHandleIndex) const
{
	return RenderingIndicesToDataIndices[InHandleIndex];
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
