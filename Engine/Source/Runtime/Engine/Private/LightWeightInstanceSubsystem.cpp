// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/LightWeightInstanceSubsystem.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

DEFINE_LOG_CATEGORY(LogLightWeightInstance);

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FLightWeightInstanceSubsystem::FLightWeightInstanceSubsystem()
{
#if WITH_EDITOR
	if (GEngine)
	{
		OnLevelActorAddedHandle = GEngine->OnLevelActorAdded().AddLambda([this](AActor* Actor)
			{
				if (ADEPRECATED_LightWeightInstanceManager* LWIManager = Cast<ADEPRECATED_LightWeightInstanceManager>(Actor))
				{
					Get().AddManager(LWIManager);
				}
			});
		OnLevelActorDeletedHandle = GEngine->OnLevelActorDeleted().AddLambda([this](AActor* Actor)
			{
				if (ADEPRECATED_LightWeightInstanceManager* LWIManager = Cast<ADEPRECATED_LightWeightInstanceManager>(Actor))
				{
					Get().RemoveManager(LWIManager);
				}
			});
	}
#endif // WITH_EDITOR
}

FLightWeightInstanceSubsystem::~FLightWeightInstanceSubsystem()
{
#if WITH_EDITOR
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedHandle);
		GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
	}
#endif // WITH_EDITOR
}

int32 FLightWeightInstanceSubsystem::GetManagerIndex(const ADEPRECATED_LightWeightInstanceManager* Manager) const
{
	UE::TReadScopeLock Lock(LWIManagersRWLock);
	for (int32 Idx = 0; Idx < LWInstanceManagers.Num(); ++Idx)
	{
		if (LWInstanceManagers[Idx] == Manager)
		{
			return Idx;
		}
	}

	return INDEX_NONE;
}

const ADEPRECATED_LightWeightInstanceManager* FLightWeightInstanceSubsystem::GetManagerAt(int32 Index) const
{
	UE::TReadScopeLock Lock(LWIManagersRWLock);
	return LWInstanceManagers.IsValidIndex(Index) ? LWInstanceManagers[Index] : nullptr;
}

bool FLightWeightInstanceSubsystem::AddManager(ADEPRECATED_LightWeightInstanceManager* Manager)
{
	UE::TWriteScopeLock Lock(LWIManagersRWLock);
	return Get().LWInstanceManagers.Add(Manager) ? true : false;
}

bool FLightWeightInstanceSubsystem::RemoveManager(ADEPRECATED_LightWeightInstanceManager* Manager)
{
	UE::TWriteScopeLock Lock(LWIManagersRWLock);
	return Get().LWInstanceManagers.Remove(Manager) ? true : false;
}

ADEPRECATED_LightWeightInstanceManager* FLightWeightInstanceSubsystem::FindLightWeightInstanceManager(const FActorInstanceHandle& Handle) const
{
	if (Handle.GetManagerInterface().IsValid())
	{
		return Handle.GetManager<ADEPRECATED_LightWeightInstanceManager>();
	}

	if (AActor* CachedActor = Handle.GetCachedActor())
	{
		const UClass* CachedActorClass = CachedActor->GetClass();
		CA_ASSUME(CachedActorClass);

		UE::TReadScopeLock Lock(LWIManagersRWLock);
		// see if we already have a match
		for (ADEPRECATED_LightWeightInstanceManager* LWInstance : LWInstanceManagers)
		{
			if (CachedActorClass == LWInstance->GetRepresentedClass(Handle.GetInstanceIndex()))
			{
				const FInt32Vector3 GridCoord = LWInstance->ConvertPositionToCoord(Handle.GetLocation());
				const FInt32Vector3 ManagerGridCoord = LWInstance->ConvertPositionToCoord(LWInstance->GetActorLocation());
				if (ManagerGridCoord == GridCoord)
				{
#if WITH_EDITOR
					// make sure the data layers match
					TArray<const UDataLayerInstance*> ManagerLayers = LWInstance->GetDataLayerInstances();
					const UDataLayerInstance* ManagerLayer = ManagerLayers.Num() > 0 ? ManagerLayers[0] : nullptr;

					TArray<const UDataLayerInstance*> ActorLayers = LWInstance->GetDataLayerInstances();
					const UDataLayerInstance* ActorLayer = ActorLayers.Num() > 0 ? ActorLayers[0] : nullptr;

					if (ManagerLayer == ActorLayer)
#endif // WITH_EDITOR
					{
						return LWInstance;
					}
				}
			}
		}
	}

	return nullptr;
}

ADEPRECATED_LightWeightInstanceManager* FLightWeightInstanceSubsystem::FindLightWeightInstanceManager(UClass& ActorClass, UWorld& World, const FVector& InPos, const UDataLayerInstance* DataLayer) const
{
	UE::TReadScopeLock Lock(LWIManagersRWLock);

	for (ADEPRECATED_LightWeightInstanceManager* InstanceManager : LWInstanceManagers)
	{
		if (IsValid(InstanceManager))
		{
			if (InstanceManager->GetRepresentedClassInternal() == &ActorClass)
			{
				const FInt32Vector3 GridCoord = InstanceManager->ConvertPositionToCoord(InPos);
				const FInt32Vector3 ManagerGridCoord = InstanceManager->ConvertPositionToCoord(InstanceManager->GetActorLocation());

				if (ManagerGridCoord == GridCoord)
				{
#if WITH_EDITOR
					if (!DataLayer || (InstanceManager->SupportsDataLayerType(UDataLayerInstance::StaticClass()) && InstanceManager->ContainsDataLayer(DataLayer)))
#endif // WITH_EDITOR
					{
						return InstanceManager;
					}
				}
			}
		}
	}

	return nullptr;
}

ADEPRECATED_LightWeightInstanceManager* FLightWeightInstanceSubsystem::FindOrAddLightWeightInstanceManager(UClass& ActorClass, UWorld& World, const FVector& InPos, const UDataLayerInstance* DataLayer)
{
	if (ADEPRECATED_LightWeightInstanceManager* FoundManager = FindLightWeightInstanceManager(ActorClass, World, InPos, DataLayer))
	{
		return FoundManager;
	}

	// we didn't find a match so we should add one.
	// Find the best base class to start from
	UClass* BestMatchingClass = FindBestInstanceManagerClass(&ActorClass);
	if (BestMatchingClass == nullptr)
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.ObjectFlags = RF_Transactional;

	FTransform ManagerTransform(FTransform::Identity);

	ADEPRECATED_LightWeightInstanceManager* InstanceManagerCDO = CastChecked<ADEPRECATED_LightWeightInstanceManager>(BestMatchingClass->GetDefaultObject(true));
	int32 GridSize = InstanceManagerCDO->GetGridSize();
	if (GridSize > 0)
	{
		FVector ManagerLocation(InstanceManagerCDO->ConvertPositionToCoord(InPos));
		ManagerLocation *= GridSize;
		ManagerLocation += FVector((float)GridSize * 0.5f);
		ManagerTransform.SetLocation(ManagerLocation);
	}

	if (ADEPRECATED_LightWeightInstanceManager* NewInstanceManager = World.SpawnActor<ADEPRECATED_LightWeightInstanceManager>(BestMatchingClass, ManagerTransform, SpawnParams))
	{
		NewInstanceManager->SetRepresentedClass(&ActorClass);

#if WITH_EDITOR
		// Add the new manager to the DataLayer
		if (DataLayer)
		{
			ensure(NewInstanceManager->SupportsDataLayerType(UDataLayerInstance::StaticClass()));
			NewInstanceManager->AddDataLayer(DataLayer);
		}
#endif // WITH_EDITOR

		check(LWInstanceManagers.Find(NewInstanceManager) != INDEX_NONE);

		return NewInstanceManager;
	}
	else
	{
		return nullptr;
	}
}

UClass* FLightWeightInstanceSubsystem::FindBestInstanceManagerClass(const UClass* InActorClass)
{
	// Get every light weight instance class
	// FRED_TODO: we need to search through unloaded blueprints as well
	// FRED_TODO: this should be done once and cached. In the editor we can add a listener for when new BP classes are added
	TArray<UClass*> ManagerClasses;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(ADEPRECATED_LightWeightInstanceManager::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract))
		{
			// Skip SKEL and REINST classes.
			if (It->GetName().StartsWith(TEXT("SKEL_")) || It->GetName().StartsWith(TEXT("REINST_")))
			{
				continue;
			}
			ManagerClasses.Add(*It);
		}
	}

	// Figure out which one is the closest fit for InActorClass
	UClass* BestManagerClass = nullptr;
	int32 BestDistance = INT_MAX;
	for (UClass* ManagerClass : ManagerClasses)
	{
		if (ManagerClass->GetDefaultObject<ADEPRECATED_LightWeightInstanceManager>()->DoesAcceptClass(InActorClass))
		{
			const UClass* HandledClass = ManagerClass->GetDefaultObject<ADEPRECATED_LightWeightInstanceManager>()->GetRepresentedClassInternal();
			if (!HandledClass)
			{
				HandledClass = ManagerClass->GetDefaultObject<ADEPRECATED_LightWeightInstanceManager>()->GetAcceptedClass();
			}
			if (InActorClass == HandledClass)
			{
				BestManagerClass = ManagerClass;
				break;
			}
			const UClass* ActorClass = InActorClass;
			int32 Distance = 0;
			while (ActorClass && ActorClass != HandledClass)
			{
				++Distance;
				ActorClass = ActorClass->GetSuperClass();
			}
			if (ActorClass && Distance < BestDistance)
			{
				BestDistance = Distance;
				BestManagerClass = ManagerClass;
			}
		}
	}

	return BestManagerClass;
}

AActor* FLightWeightInstanceSubsystem::GetActor_NoCreate(const FActorInstanceHandle& Handle) const
{
	return Handle.GetCachedActor();
}

UClass* FLightWeightInstanceSubsystem::GetActorClass(const FActorInstanceHandle& Handle)
{
	if (const AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor->StaticClass();
	}

	if (ADEPRECATED_LightWeightInstanceManager* LWIManager = FindLightWeightInstanceManager(Handle))
	{
		return LWIManager->GetRepresentedClass(Handle.GetInstanceIndex());
	}

	return nullptr;
}

FVector FLightWeightInstanceSubsystem::GetLocation(const FActorInstanceHandle& Handle)
{
	ensure(Handle.IsValid());

	if (const AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor->GetActorLocation();
	}

	if (ADEPRECATED_LightWeightInstanceManager* InstanceManager = FindLightWeightInstanceManager(Handle))
	{
		return InstanceManager->GetLocation(Handle);
	}

	return FVector::ZeroVector;
}

FString FLightWeightInstanceSubsystem::GetName(const FActorInstanceHandle& Handle)
{
	ensure(Handle.IsValid());

	if (const AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor->GetName();
	}

	if (ADEPRECATED_LightWeightInstanceManager* InstanceManager = FindLightWeightInstanceManager(Handle))
	{
		return InstanceManager->GetName(Handle);
	}

	return TEXT("None");
}

ULevel* FLightWeightInstanceSubsystem::GetLevel(const FActorInstanceHandle& Handle)
{
	ensure(Handle.IsValid());

	if (AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor->GetLevel();
	}

	if (ADEPRECATED_LightWeightInstanceManager* InstanceManager = FindLightWeightInstanceManager(Handle))
	{
		return InstanceManager->GetLevel();
	}

	return nullptr;
}

bool FLightWeightInstanceSubsystem::IsInLevel(const FActorInstanceHandle& Handle, const ULevel* InLevel)
{
	ensure(Handle.IsValid());

	if (AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor->IsInLevel(InLevel);
	}

	if (ADEPRECATED_LightWeightInstanceManager* InstanceManager = FindLightWeightInstanceManager(Handle))
	{
		return InstanceManager->GetLevel() == InLevel;
	}

	return false;
}

FActorInstanceHandle FLightWeightInstanceSubsystem::CreateNewLightWeightInstance(UClass* InActorClass, FLWIData* InitData, UDataLayerInstance* InLayer, UWorld* World)
{
	// Get or create a light weight instance for this class and data layer
	if (InitData)
	{
		if (ADEPRECATED_LightWeightInstanceManager* LWIManager = FindOrAddLightWeightInstanceManager(*InActorClass, *World, InitData->Transform.GetLocation(), InLayer))
		{
			// create an instance with the given data
			int32 InstanceIdx = LWIManager->AddNewInstance(InitData);
			InstanceIdx = LWIManager->ConvertInternalIndexToHandleIndex(InstanceIdx);
			return FActorInstanceHandle(FActorInstanceManagerInterface(LWIManager), InstanceIdx);
		}
	}

	return FActorInstanceHandle();
}

void FLightWeightInstanceSubsystem::DeleteInstance(const FActorInstanceHandle& Handle)
{
	if (ADEPRECATED_LightWeightInstanceManager* LWIManager = FindLightWeightInstanceManager(Handle))
	{
		LWIManager->RemoveInstance(Handle.GetInstanceIndex());
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
