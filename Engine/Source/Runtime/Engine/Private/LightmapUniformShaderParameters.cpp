// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapUniformShaderParameters.h"
#include "LightMap.h"
#include "RenderUtils.h"
#include "UnrealEngine.h"
#include "VirtualTexturing.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FPrecomputedLightingUniformParameters, "PrecomputedLightingBuffer");

FLightmapSceneShaderData::FLightmapSceneShaderData(const class FLightCacheInterface* LCI, ERHIFeatureLevel::Type FeatureLevel)
	: Data(InPlace, NoInit)
{
	FPrecomputedLightingUniformParameters Parameters;
	GetPrecomputedLightingParameters(FeatureLevel, Parameters, LCI);
	Setup(Parameters);
}

void FLightmapSceneShaderData::Setup(const FPrecomputedLightingUniformParameters& ShaderParameters)
{
	static_assert(sizeof(FPrecomputedLightingUniformParameters) == sizeof(FVector4f) * 15, "The FLightmapSceneShaderData manual layout below and in usf must match FPrecomputedLightingUniformParameters.  Update this assert when adding a new member.");
	// Note: layout must match GetLightmapData in usf

	Data[0] = ShaderParameters.StaticShadowMapMasks;
	Data[1] = ShaderParameters.InvUniformPenumbraSizes;
	Data[2] = ShaderParameters.LightMapCoordinateScaleBias;
	Data[3] = ShaderParameters.ShadowMapCoordinateScaleBias;
	Data[4] = ShaderParameters.LightMapScale[0];
	Data[5] = ShaderParameters.LightMapScale[1];
	Data[6] = ShaderParameters.LightMapAdd[0];
	Data[7] = ShaderParameters.LightMapAdd[1];
	
	// bitcast FUintVector4 -> FVector4f
	FMemory::Memcpy(&Data[8], &ShaderParameters.LightmapVTPackedPageTableUniform[0], sizeof(FVector4f) * 2u);

	// bitcast FUintVector4 -> FVector4f
	FMemory::Memcpy(&Data[10], &ShaderParameters.LightmapVTPackedUniform[0], sizeof(FVector4f) * 5u);
}

void GetDefaultPrecomputedLightingParameters(FPrecomputedLightingUniformParameters& Parameters)
{
	Parameters.StaticShadowMapMasks = FVector4f(1, 1, 1, 1);
	Parameters.InvUniformPenumbraSizes = FVector4f(0, 0, 0, 0);
	Parameters.LightMapCoordinateScaleBias = FVector4f(1, 1, 0, 0);
	Parameters.ShadowMapCoordinateScaleBias = FVector4f(1, 1, 0, 0);
	 
	const uint32 NumCoef = FMath::Max<uint32>(NUM_HQ_LIGHTMAP_COEF, NUM_LQ_LIGHTMAP_COEF);
	for (uint32 CoefIndex = 0; CoefIndex < NumCoef; ++CoefIndex)
	{
		Parameters.LightMapScale[CoefIndex] = FVector4f(1, 1, 1, 1);
		Parameters.LightMapAdd[CoefIndex] = FVector4f(0, 0, 0, 0);
	}

	FMemory::Memzero(Parameters.LightmapVTPackedPageTableUniform);

	for (uint32 LayerIndex = 0u; LayerIndex < 5u; ++LayerIndex)
	{
		Parameters.LightmapVTPackedUniform[LayerIndex] = FUintVector4(ForceInitToZero);
	}
}

void GetPrecomputedLightingParameters(
	ERHIFeatureLevel::Type FeatureLevel,
	FPrecomputedLightingUniformParameters& Parameters,
	const FLightCacheInterface* LCI
)
{
	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(FeatureLevel));

	// TDistanceFieldShadowsAndLightMapPolicy
	const FShadowMapInteraction ShadowMapInteraction = LCI ? LCI->GetShadowMapInteraction(FeatureLevel) : FShadowMapInteraction();
	if (ShadowMapInteraction.GetType() == SMIT_Texture)
	{
		Parameters.ShadowMapCoordinateScaleBias = FVector4f(FVector2f(ShadowMapInteraction.GetCoordinateScale()), FVector2f(ShadowMapInteraction.GetCoordinateBias()));
		Parameters.StaticShadowMapMasks = FVector4f(ShadowMapInteraction.GetChannelValid(0), ShadowMapInteraction.GetChannelValid(1), ShadowMapInteraction.GetChannelValid(2), ShadowMapInteraction.GetChannelValid(3));
		Parameters.InvUniformPenumbraSizes = ShadowMapInteraction.GetInvUniformPenumbraSize();
	}
	else
	{
		Parameters.ShadowMapCoordinateScaleBias = FVector4f(1, 1, 0, 0);
		Parameters.StaticShadowMapMasks = FVector4f(1, 1, 1, 1);
		Parameters.InvUniformPenumbraSizes = FVector4f(0, 0, 0, 0);
	}


	// TLightMapPolicy
	const FLightMapInteraction LightMapInteraction = LCI ? LCI->GetLightMapInteraction(FeatureLevel) : FLightMapInteraction();
	if (LightMapInteraction.GetType() == LMIT_Texture)
	{
		const bool bAllowHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel) && LightMapInteraction.AllowsHighQualityLightmaps();

		// Vertex Shader
		const FVector2D LightmapCoordinateScale = LightMapInteraction.GetCoordinateScale();
		const FVector2D LightmapCoordinateBias = LightMapInteraction.GetCoordinateBias();
		Parameters.LightMapCoordinateScaleBias = FVector4f(LightmapCoordinateScale.X, LightmapCoordinateScale.Y, LightmapCoordinateBias.X, LightmapCoordinateBias.Y);

		uint32 NumLightmapVTLayers = 0u;
		if (bUseVirtualTextures)
		{
			check(LCI); // If LCI was nullptr, LightMapInteraction.GetType() would be LMIT_None, not LMIT_Texture
			const FLightmapResourceCluster* ResourceCluster = LCI->GetResourceCluster();
			check(ResourceCluster);
			const IAllocatedVirtualTexture* AllocatedVT = ResourceCluster->GetAllocatedVT();
			if (AllocatedVT)
			{
				AllocatedVT->GetPackedPageTableUniform(&Parameters.LightmapVTPackedPageTableUniform[0]);
				NumLightmapVTLayers = AllocatedVT->GetNumTextureLayers();
				for (uint32 LayerIndex = 0u; LayerIndex < NumLightmapVTLayers; ++LayerIndex)
				{
					AllocatedVT->GetPackedUniform(&Parameters.LightmapVTPackedUniform[LayerIndex], LayerIndex);
				}
			}
		}
		else
		{
			FMemory::Memzero(Parameters.LightmapVTPackedPageTableUniform);
		}

		for (uint32 LayerIndex = NumLightmapVTLayers; LayerIndex < 5u; ++LayerIndex)
		{
			Parameters.LightmapVTPackedUniform[LayerIndex] = FUintVector4(ForceInitToZero);
		}

		const uint32 NumCoef = bAllowHighQualityLightMaps ? NUM_HQ_LIGHTMAP_COEF : NUM_LQ_LIGHTMAP_COEF;
		const FVector4f* Scales = LightMapInteraction.GetScaleArray();
		const FVector4f* Adds = LightMapInteraction.GetAddArray();
		for (uint32 CoefIndex = 0; CoefIndex < NumCoef; ++CoefIndex)
		{
			Parameters.LightMapScale[CoefIndex] = Scales[CoefIndex];
			Parameters.LightMapAdd[CoefIndex] = Adds[CoefIndex];
		}
	}
	else
	{
		// Vertex Shader
		Parameters.LightMapCoordinateScaleBias = FVector4f(1, 1, 0, 0);

		// Pixel Shader
		const uint32 NumCoef = FMath::Max<uint32>(NUM_HQ_LIGHTMAP_COEF, NUM_LQ_LIGHTMAP_COEF);
		for (uint32 CoefIndex = 0; CoefIndex < NumCoef; ++CoefIndex)
		{
			Parameters.LightMapScale[CoefIndex] = FVector4f(1, 1, 1, 1);
			Parameters.LightMapAdd[CoefIndex] = FVector4f(0, 0, 0, 0);
		}
	}
}
