// Copyright Epic Games, Inc. All Rights Reserved.

#include "Lightmass/LightmappedSurfaceCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightmappedSurfaceCollection)

ULightmappedSurfaceCollection::ULightmappedSurfaceCollection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

