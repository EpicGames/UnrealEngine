// Copyright Epic Games, Inc. All Rights Reserved.

#include "Lightmass/LightmassCharacterIndirectDetailVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightmassCharacterIndirectDetailVolume)


ALightmassCharacterIndirectDetailVolume::ALightmassCharacterIndirectDetailVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bColored = true;
	BrushColor.R = 155;
	BrushColor.G = 185;
	BrushColor.B = 25;
	BrushColor.A = 255;

}

