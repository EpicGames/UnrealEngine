// Copyright Epic Games, Inc. All Rights Reserved.

#include "Lightmass/LightmassImportanceVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightmassImportanceVolume)

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif

ALightmassImportanceVolume::ALightmassImportanceVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bColored = true;
	BrushColor.R = 255;
	BrushColor.G = 255;
	BrushColor.B = 25;
	BrushColor.A = 255;

}

#if WITH_EDITOR
void ALightmassImportanceVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FStaticLightingSystemInterface::OnLightmassImportanceVolumeModified.Broadcast();
}

void ALightmassImportanceVolume::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	FStaticLightingSystemInterface::OnLightmassImportanceVolumeModified.Broadcast();
}
#endif
