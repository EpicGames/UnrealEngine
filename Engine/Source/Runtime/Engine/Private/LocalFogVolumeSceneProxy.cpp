// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FVolumetricCloudSceneProxy implementation.
=============================================================================*/

#include "LocalFogVolumeSceneProxy.h"
#include "Components/LocalFogVolumeComponent.h"



FLocalFogVolumeSceneProxy::FLocalFogVolumeSceneProxy(const ULocalFogVolumeComponent* InComponent)
	: RadialFogExtinction(InComponent->RadialFogExtinction)
	, HeightFogExtinction(InComponent->HeightFogExtinction)
	, HeightFogFalloff(InComponent->HeightFogFalloff)
	, HeightFogOffset(InComponent->HeightFogOffset)
	, FogUniformScale(1.0f)
	, FogSortPriority(uint8(127 - int8(InComponent->FogSortPriority))) // FogSortPriority on the component is in [-127,127] and needs to be negated to match expected priority behavior.
	, FogStartDistance(InComponent->FogStartDistance)
	, FogPhaseG(InComponent->FogPhaseG)
	, FogAlbedo(InComponent->FogAlbedo)
	, FogEmissive(InComponent->FogEmissive)
{
	UpdateComponentTransform(InComponent->GetComponentTransform());
}

FLocalFogVolumeSceneProxy::~FLocalFogVolumeSceneProxy()
{
}

void FLocalFogVolumeSceneProxy::UpdateComponentTransform(const FTransform& Transform)
{
	FogTransform = Transform;
	const float MaximumAxisScale = FogTransform.GetMaximumAxisScale() * ULocalFogVolumeComponent::GetBaseVolumeSize();
	FogTransform.SetScale3D(FVector(MaximumAxisScale, MaximumAxisScale, MaximumAxisScale));

	FogUniformScale = MaximumAxisScale;
}

