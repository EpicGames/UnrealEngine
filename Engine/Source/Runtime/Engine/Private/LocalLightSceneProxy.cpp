// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LocalLightSceneProxy.cpp: Local light scene info implementation.
=============================================================================*/

#include "LocalLightSceneProxy.h"
#include "LocalLightSceneProxyDesc.h"
#include "Components/LocalLightComponent.h"
#include "Math/InverseRotationMatrix.h"
#include "SceneManagement.h"
#include "SceneView.h"

/** Initialization constructor. */
FLocalLightSceneProxy::FLocalLightSceneProxy(const ULocalLightComponent* Component)
	: FLocalLightSceneProxy(ULocalLightComponent::BuildSceneProxyDesc(*Component))
{
}

FLocalLightSceneProxy::FLocalLightSceneProxy(const FLocalLightSceneProxyDesc& LightDesc)
	: FLightSceneProxy(LightDesc)
	, MaxDrawDistance(LightDesc.MaxDrawDistance)
	, FadeRange(LightDesc.MaxDistanceFadeRange)
	, InverseExposureBlend(LightDesc.InverseExposureBlend)
{
	UpdateRadius(LightDesc.AttenuationRadius);
}

void FLocalLightSceneProxy::UpdateRadius_GameThread(float InRadius)
{
	FLocalLightSceneProxy* InLightSceneInfo = this;
	ENQUEUE_RENDER_COMMAND(UpdateRadius)(
		[InLightSceneInfo, InRadius](FRHICommandList& RHICmdList)
		{
			InLightSceneInfo->UpdateRadius(InRadius);
		});
}

float FLocalLightSceneProxy::GetMaxDrawDistance() const
{
	return MaxDrawDistance;
}

float FLocalLightSceneProxy::GetFadeRange() const
{
	return FadeRange;
}

/** @return radius of the light or 0 if no radius */
float FLocalLightSceneProxy::GetRadius() const
{
	return Radius;
}

bool FLocalLightSceneProxy::AffectsBounds(const FBoxSphereBounds& Bounds) const
{
	if ((Bounds.Origin - GetLightToWorld().GetOrigin()).SizeSquared() > FMath::Square(Radius + Bounds.SphereRadius))
	{
		return false;
	}

	if (!FLightSceneProxy::AffectsBounds(Bounds))
	{
		return false;
	}

	return true;
}

bool FLocalLightSceneProxy::GetScissorRect(FIntRect& ScissorRect, const FSceneView& View, const FIntRect& ViewRect) const
{
	ScissorRect = ViewRect;
	return FMath::ComputeProjectedSphereScissorRect(ScissorRect, GetLightToWorld().GetOrigin(), Radius, View.ViewMatrices.GetViewOrigin(), View.ViewMatrices.GetWorldToView(), View.ViewMatrices.GetViewToClip()) == 1;
}

bool FLocalLightSceneProxy::SetScissorRect(FRHICommandList& RHICmdList, const FSceneView& View, const FIntRect& ViewRect, FIntRect* OutScissorRect) const
{
	FIntRect ScissorRect;

	if (GetScissorRect(ScissorRect, View, ViewRect))
	{
		RHICmdList.SetScissorRect(true, ScissorRect.Min.X, ScissorRect.Min.Y, ScissorRect.Max.X, ScissorRect.Max.Y);
		if (OutScissorRect)
		{
			*OutScissorRect = ScissorRect;
		}
		return true;
	}
	else
	{
		RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
		return false;
	}
}

FSphere FLocalLightSceneProxy::GetBoundingSphere() const
{
	return FSphere(GetPosition(), GetRadius());
}

float FLocalLightSceneProxy::GetEffectiveScreenRadius(const FViewMatrices& ShadowViewMatrices, const FIntPoint& CameraViewRectSize) const
{
	const FVector2D& ProjectionScale = ShadowViewMatrices.GetProjectionScale();
	const float ScreenScale = FMath::Max(CameraViewRectSize.X * 0.5f * ProjectionScale.X, CameraViewRectSize.Y * 0.5f * ProjectionScale.Y);

	const float LightDistance = (GetOrigin() - ShadowViewMatrices.GetViewOrigin()).Size();
	return ScreenScale * GetRadius() / FMath::Max(LightDistance, 1.0f);
}

FVector FLocalLightSceneProxy::GetPerObjectProjectedShadowProjectionPoint(const FBoxSphereBounds& SubjectBounds) const
{
	return GetOrigin();
}

bool FLocalLightSceneProxy::GetPerObjectProjectedShadowInitializer(const FBoxSphereBounds& SubjectBounds, class FPerObjectProjectedShadowInitializer& OutInitializer) const
{
	// Use a perspective projection looking at the primitive from the light position.
	FVector LightPosition = GetPerObjectProjectedShadowProjectionPoint(SubjectBounds);
	FVector LightVector = SubjectBounds.Origin - LightPosition;
	float LightDistance = LightVector.Size();
	float SilhouetteRadius = 1.0f;
	const float SubjectRadius = SubjectBounds.BoxExtent.Size();
	const float ShadowRadiusMultiplier = 1.1f;

	if (LightDistance > SubjectRadius)
	{
		SilhouetteRadius = FMath::Min(SubjectRadius * FMath::InvSqrt((LightDistance - SubjectRadius) * (LightDistance + SubjectRadius)), 1.0f);
	}

	if (LightDistance <= SubjectRadius * ShadowRadiusMultiplier)
	{
		// Make the primitive fit in a single < 90 degree FOV projection.
		LightVector = SubjectRadius * LightVector.GetSafeNormal() * ShadowRadiusMultiplier;
		LightPosition = (SubjectBounds.Origin - LightVector);
		LightDistance = SubjectRadius * ShadowRadiusMultiplier;
		SilhouetteRadius = 1.0f;
	}

	OutInitializer.PreShadowTranslation = -LightPosition;
	OutInitializer.WorldToLight = FInverseRotationMatrix((LightVector / LightDistance).Rotation());
	OutInitializer.Scales = FVector2D(1.0f / SilhouetteRadius, 1.0f / SilhouetteRadius);
	OutInitializer.SubjectBounds = FBoxSphereBounds(SubjectBounds.Origin - LightPosition, SubjectBounds.BoxExtent, SubjectBounds.SphereRadius);
	OutInitializer.WAxis = FVector4(0, 0, 1, 0);
	OutInitializer.MinLightW = 0.1f;
	OutInitializer.MaxDistanceToCastInLightW = Radius;
	return true;
}

/** Updates the light scene info's radius from the component. */
void FLocalLightSceneProxy::UpdateRadius(float ComponentRadius)
{
	Radius = ComponentRadius;

	// Min to avoid div by 0 (NaN in InvRadius)
	InvRadius = 1.0f / FMath::Max(0.00001f, ComponentRadius);
}

bool FLocalLightSceneProxy::IsLocalLight() const
{
	return true;
}
