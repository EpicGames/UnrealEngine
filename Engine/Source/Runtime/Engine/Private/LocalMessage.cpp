// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/LocalMessage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalMessage)

FClientReceiveData::FClientReceiveData()
	: LocalPC(NULL)
	, MessageType(FName("None"))
	, MessageIndex(-1)
	, MessageString(TEXT(""))
	, RelatedPlayerState_1(NULL)
	, RelatedPlayerState_2(NULL)
	, OptionalObject(NULL)
{
	
}

ULocalMessage::ULocalMessage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

