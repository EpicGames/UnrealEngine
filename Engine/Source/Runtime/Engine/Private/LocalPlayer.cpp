// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/LocalPlayer.h"
#include "Engine/ChildConnection.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Engine/ScriptViewportClient.h"
#include "Misc/FileHelper.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/WorldSettings.h"
#include "UObject/UObjectAnnotation.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Math/InverseRotationMatrix.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/PlayerState.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "UnrealEngine.h"

#include "Net/OnlineEngineInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "HAL/PlatformApplicationMisc.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Framework/Application/SlateApplication.h"

#include "IXRTrackingSystem.h"
#include "IXRCamera.h"
#include "Camera/CameraComponent.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "Net/DataChannel.h"

#include "GameDelegates.h"
#include "UnrealClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalPlayer)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Engine/DebugCameraController.h"
#endif

#if WITH_EDITOR
#include "Editor.h"
#include "Settings/LevelEditorPlaySettings.h"
#endif

DEFINE_LOG_CATEGORY(LogPlayerManagement);

#if !UE_BUILD_SHIPPING

static TAutoConsoleVariable<int32> CVarViewportTest(
	TEXT("r.Test.ConstrainedView"),
	0,
	TEXT("Allows to test different viewport rectangle configuations (in game only) as they can happen when using cinematics/Editor.\n")
	TEXT("0: off(default)\n")
	TEXT("1..7: Various Configuations"),
	ECVF_RenderThreadSafe);

#endif // !UE_BUILD_SHIPPING

int32 GCalcLocalPlayerCachedLODDistanceFactor = 1;
static FAutoConsoleVariableRef CVarCalcLocalPlayerCachedLODDistanceFactor(
	TEXT("r.CalcLocalPlayerCachedLODDistanceFactor"),
	GCalcLocalPlayerCachedLODDistanceFactor,
	TEXT("Should we calculate a LOD Distance Factor based on the current FOV.  Should not be necessary since LOD is already based on screen size.\n")
	);

DECLARE_CYCLE_STAT(TEXT("CalcSceneView"), STAT_CalcSceneView, STATGROUP_Engine);

//////////////////////////////////////////////////////////////////////////
// Things used by ULocalPlayer::Exec
//@TODO: EXEC

bool GShouldLogOutAFrameOfMoveComponent = false;
bool GShouldLogOutAFrameOfSetBodyTransform = false;

//////////////////////////////////////////////////////////////////////////
// ULocalPlayer

FLocalPlayerContext::FLocalPlayerContext()
{

}

FLocalPlayerContext::FLocalPlayerContext( const class ULocalPlayer* InLocalPlayer, UWorld* InWorld )
	: World(InWorld)
{
	SetLocalPlayer( InLocalPlayer );
}

FLocalPlayerContext::FLocalPlayerContext( const class APlayerController* InPlayerController )
{
	SetPlayerController( InPlayerController );
}

bool FLocalPlayerContext::IsValid() const
{
	if (ULocalPlayer* LocalPlayerPtr = LocalPlayer.Get())
	{
		if (UWorld* WorldPtr = GetWorld())
		{
			if (APlayerController* PC = (WorldPtr ? LocalPlayerPtr->GetPlayerController(WorldPtr) : ToRawPtr(LocalPlayerPtr->PlayerController)))
			{
				return (PC->Player != nullptr);
			}
		}
	}

	return false;
}

bool FLocalPlayerContext::IsInitialized() const
{
	return LocalPlayer.IsValid();
}

UWorld* FLocalPlayerContext::GetWorld() const
{
	UWorld* WorldPtr = World.Get();
	if (WorldPtr != nullptr)
	{
		return WorldPtr;
	}

	return GetLocalPlayer()->GetWorld();
}

UGameInstance* FLocalPlayerContext::GetGameInstance() const
{
	if (UWorld* WorldPtr = GetWorld())
	{
		return WorldPtr->GetGameInstance();
	}

	return nullptr;
}

ULocalPlayer* FLocalPlayerContext::GetLocalPlayer() const
{
	ULocalPlayer* LocalPlayerPtr = LocalPlayer.Get();
	check(LocalPlayerPtr);
	return LocalPlayerPtr;
}

APlayerController* FLocalPlayerContext::GetPlayerController() const
{
	ULocalPlayer* LocalPlayerPtr = GetLocalPlayer();
	UWorld* WorldPtr = World.Get();
	return (WorldPtr ? LocalPlayerPtr->GetPlayerController(WorldPtr) : ToRawPtr(LocalPlayerPtr->PlayerController));
}

class AGameStateBase* FLocalPlayerContext::GetGameState() const
{
	AGameStateBase* GameState = nullptr;

	if (UWorld* WorldPtr = World.Get())
	{
		GameState = WorldPtr->GetGameState();
	}
	else
	{
		ULocalPlayer* LocalPlayerPtr = GetLocalPlayer();
		if (UWorld* LocalPlayerWorld = LocalPlayerPtr ? LocalPlayerPtr->GetWorld() : nullptr)
		{
			GameState = LocalPlayerWorld->GetGameState();
		}
	}

	return GameState;
}

APlayerState* FLocalPlayerContext::GetPlayerState() const
{
	APlayerController* PC = GetPlayerController();
	return PC ? PC->PlayerState : nullptr;
}

AHUD* FLocalPlayerContext::GetHUD() const
{
	APlayerController* PC = GetPlayerController();
	return PC ? PC->MyHUD : nullptr;
}

class APawn* FLocalPlayerContext::GetPawn() const
{
	APlayerController* PC = GetPlayerController();
	return PC ? PC->GetPawn() : nullptr;
}

void FLocalPlayerContext::SetLocalPlayer( const ULocalPlayer* InLocalPlayer )
{
	LocalPlayer = MakeWeakObjectPtr(const_cast<ULocalPlayer*>(InLocalPlayer));
}

void FLocalPlayerContext::SetPlayerController( const APlayerController* InPlayerController )
{
	check( InPlayerController->IsLocalPlayerController() );
	LocalPlayer = CastChecked<ULocalPlayer>(InPlayerController->Player);
	World = InPlayerController->GetWorld();
}

bool FLocalPlayerContext::IsFromLocalPlayer(const AActor* ActorToTest) const
{
	if (ActorToTest)
	{
		if (ULocalPlayer* LocalPlayerPtr = LocalPlayer.Get())
		{
			if (UWorld* WorldPtr = GetWorld())
			{
				if (APlayerController* PC = (WorldPtr ? LocalPlayerPtr->GetPlayerController(WorldPtr) : ToRawPtr(LocalPlayerPtr->PlayerController)))
				{
					if (   ActorToTest == PC
					    || ActorToTest == PC->GetPawn()
					    || ActorToTest == PC->PlayerState)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}


//////////////////////////////////////////////////////////////////////////
// ULocalPlayer

ULocalPlayer::ULocalPlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SlateOperations( FReply::Unhandled() )
{
	PendingLevelPlayerControllerClass = APlayerController::StaticClass();
}

ULocalPlayer::ULocalPlayer(FVTableHelper& Helper)
	: Super(Helper)
	, SlateOperations(FReply::Unhandled())
{
}

UScriptViewportClient* ULocalPlayer::GetViewportClient() const
{
	return ViewportClientOverride
		? ViewportClientOverride.Get()
		: ViewportClient.Get();
}

void ULocalPlayer::OverrideViewportClient(UScriptViewportClient* InViewportClient)
{
	ViewportClientOverride = InViewportClient;
}

void ULocalPlayer::PlayerAdded(UGameViewportClient* InViewportClient, int32 InControllerID)
{
	ViewportClient = InViewportClient;
	SetControllerId(InControllerID);

	SubsystemCollection.Initialize(this);
}

void ULocalPlayer::PlayerAdded(UGameViewportClient* InViewportClient, FPlatformUserId InUserId)
{
	ViewportClient = InViewportClient;
	SetPlatformUserId(InUserId);

	SubsystemCollection.Initialize(this);
}

void ULocalPlayer::InitOnlineSession()
{
	// FIXME: This may be obsolete, still here to support a few straggler cases that do stuff in child classes
}

void ULocalPlayer::PlayerRemoved()
{
	SubsystemCollection.Deinitialize();

	if (!IsTemplate())
	{
		for (FSceneViewStateReference& ViewState : ViewStates)
		{
			ViewState.Destroy();
		}
	}
}

bool ULocalPlayer::SpawnPlayActor(const FString& URL,FString& OutError, UWorld* InWorld)
{
	check(InWorld);
	if (!InWorld->IsNetMode(NM_Client))
	{
		FURL PlayerURL(NULL, *URL, TRAVEL_Absolute);

		// Get player nickname
		FString PlayerName = GetNickname();
		if (PlayerName.Len() > 0)
		{
			PlayerURL.AddOption(*FString::Printf(TEXT("Name=%s"), *PlayerName));
		}

		// Send any game-specific url options for this player
		FString GameUrlOptions = GetGameLoginOptions();
		if (GameUrlOptions.Len() > 0)
		{
			PlayerURL.AddOption(*FString::Printf(TEXT("%s"), *GameUrlOptions));
		}

		// Get player unique id
		FUniqueNetIdRepl UniqueId(GetPreferredUniqueNetId());

		PlayerController = InWorld->SpawnPlayActor(this, ROLE_SimulatedProxy, PlayerURL, UniqueId, OutError, GEngine->GetGamePlayers(InWorld).Find(this));
	}
	else
	{
		// Statically bind to the specified player controller
		UClass* PCClass = PendingLevelPlayerControllerClass;
		// The PlayerController gets replicated from the client though the engine assumes that every Player always has
		// a valid PlayerController so we spawn a dummy one that is going to be replaced later.

		//
		// Look at APlayerController::OnActorChannelOpen + UNetConnection::HandleClientPlayer for the code the
		// replaces this fake player controller with the real replicated one from the server
		//

		FActorSpawnParameters SpawnInfo;
		SpawnInfo.ObjectFlags |= RF_Transient;	// We never want to save player controllers into a map
		PlayerController = InWorld->SpawnActor<APlayerController>(PCClass, SpawnInfo);
		const int32 PlayerIndex = GEngine->GetGamePlayers(InWorld).Find(this);
		PlayerController->NetPlayerIndex = PlayerIndex;
		PlayerController->Player = this;
	}
	return PlayerController != NULL;
}

void ULocalPlayer::SendSplitJoin(TArray<FString>& Options)
{
	UNetDriver* NetDriver = NULL;

	UWorld* World = GetWorld();
	if (World)
	{
		NetDriver = World->GetNetDriver();
	}

	SendSplitJoin(Options, NetDriver, UE::Net::EJoinFlags::None);
}

void ULocalPlayer::SendSplitJoin(TArray<FString>& Options, UNetDriver* NetDriver, UE::Net::EJoinFlags Flags)
{
	if (NetDriver == NULL || NetDriver->ServerConnection == NULL || NetDriver->ServerConnection->GetConnectionState() != USOCK_Open)
	{
		UE_LOGF(LogPlayerManagement, Warning, "SendSplitJoin(): Not connected to a server");
	}
	else if (!bSentSplitJoin)
	{
		// make sure we don't already have a connection
		bool bNeedToSendJoin = false;
		if (PlayerController == NULL)
		{
			bNeedToSendJoin = true;
		}
		else if (NetDriver->ServerConnection->PlayerController != PlayerController)
		{
			bNeedToSendJoin = true;
			for (int32 i = 0; i < NetDriver->ServerConnection->Children.Num(); i++)
			{
				if (NetDriver->ServerConnection->Children[i]->PlayerController == PlayerController)
				{
					bNeedToSendJoin = false;
					break;
				}
			}
		}

		if (bNeedToSendJoin)
		{
			// use the default URL except for player name for splitscreen players
			FURL URL;
			URL.LoadURLConfig(TEXT("DefaultPlayer"), GGameIni);

			// Send the player nickname at login
			FString PlayerName = GetNickname();
			if (PlayerName.Len() > 0)
			{
				URL.AddOption(*FString::Printf(TEXT("Name=%s"), *PlayerName));
			}

			URL.AddOption(*FString::Printf(TEXT("LocalPlayerId=%d"), ConnectionIdentifier));

			// Send any game-specific url options for this player
			FString GameUrlOptions = GetGameLoginOptions();
			if (GameUrlOptions.Len() > 0)
			{
				URL.AddOption(*FString::Printf(TEXT("%s"), *GameUrlOptions));
			}

			for (FString& Option : Options)
			{
				URL.AddOption(*FString::Printf(TEXT("%s"), *Option));
			}

			// Send the player unique Id at login
			FUniqueNetIdRepl UniqueIdRepl(GetPreferredUniqueNetId());

			FString URLString = URL.ToString();

			if (EnumHasAllFlags(Flags, UE::Net::EJoinFlags::NoPawn))
			{
				FNetControlMessage<NMT_JoinNoPawnSplit>::Send(NetDriver->ServerConnection, URLString, UniqueIdRepl);
			}
			else
			{
				FNetControlMessage<NMT_JoinSplit>::Send(NetDriver->ServerConnection, URLString, UniqueIdRepl);
			}
			
			bSentSplitJoin = true;
		}
	}
}

/**
 * Singleton managing saved locked views and the current per-player state.
 */
class FLockedViewState
{
public:
	/** Singleton accessor. */
	static FLockedViewState& Get()
	{
		static FLockedViewState State;
		return State;
	}

	/**
	 * Retrieves the locked view point for the given player.
	 * @param Player			The player for which to retrieve the locked view point.
	 * @param OutViewLocation	The location at which the view was locked.
	 * @param OutViewRotation	The rotation at which the view was locked.
	 * @param OutFOV			The FOV at which the view was locked.
	 * @returns true if the view is locked, false if it is not.
	 */
	bool GetViewPoint(ULocalPlayer const* Player, FVector& OutViewLocation, FRotator& OutViewRotation, float& OutFOV)
	{
		FPlayerState PlayerState = PlayerStates.GetAnnotation(Player);
		if (PlayerState.bLocked)
		{
			OutViewLocation = PlayerState.ViewPoint.Location;
			OutViewRotation = PlayerState.ViewPoint.Rotation;
			OutFOV = PlayerState.ViewPoint.FOV;
			return true;
		}
		return false;
	}

	/**
	 * Returns true if the player's viewpoint is locked.
	 */
	bool IsViewLocked(ULocalPlayer const* Player)
	{
		FPlayerState PlayerState = PlayerStates.GetAnnotation(Player);
		return PlayerState.bLocked;
	}

	/**
	 * Forces the player's view to be unlocked.
	 */
	void UnlockView(ULocalPlayer* Player)
	{
		PlayerStates.RemoveAnnotation(Player);
	}

	/**
	 * Processes a LockView console command.
	 * @param Player	The player for which the command was given.
	 * @param Args		Arguments to the LockView command.
	 */
	void LockView(ULocalPlayer* Player, const TArray<FString>& Args)
	{
		bool bPrintHelp = false;
		bool bShouldLockView = false;
		FPlayerState PlayerState = PlayerStates.GetAnnotation(Player);

		// ? as only arg == display help.
		if (Args.Num() == 1 && Args[0] == TEXT("?"))
		{
			bPrintHelp = true;
		}
		// No args == toggle view locking.
		else if (Args.Num() == 0)
		{
			if (PlayerState.bLocked)
			{
				PlayerStates.RemoveAnnotation(Player);
			}
			else
			{
				FMinimalViewInfo MinViewInfo;
				Player->GetViewPoint(MinViewInfo);
				PlayerState.ViewPoint.Location = MinViewInfo.Location;
				PlayerState.ViewPoint.Rotation = MinViewInfo.Rotation;
				PlayerState.ViewPoint.FOV = MinViewInfo.FOV;
				bShouldLockView = true;
			}
		}
		// One arg == lock view at named location.
		else if (Args.Num() == 1)
		{
			FName ViewName(*Args[0]);
			if (Viewpoints.Contains(ViewName))
			{
				PlayerState.ViewPoint = Viewpoints.FindRef(ViewName);
			}
			else
			{
				FMinimalViewInfo MinViewInfo;
				Player->GetViewPoint(MinViewInfo);
				PlayerState.ViewPoint.Location = MinViewInfo.Location;
				PlayerState.ViewPoint.Rotation = MinViewInfo.Rotation;
				PlayerState.ViewPoint.FOV = MinViewInfo.FOV;
				Viewpoints.Add(ViewName,PlayerState.ViewPoint);
			}
			bShouldLockView = true;
		}
		// Six args == specify explicit location
		else if (Args.Num() == 6)
		{
			bool bAnyEmpty = false;
			for (int32 i = 0; i < Args.Num(); ++i)
			{
				bAnyEmpty |= Args[i].Len() == 0;
			}
			if (bAnyEmpty)
			{
				bPrintHelp = true;
			}
			else
			{
				PlayerState.ViewPoint = GetViewPointFromStrings(&Args[0],Args.Num());
				bShouldLockView = true;
			}
		}
		// Seven args == specify an explicit location and store it.
		else if (Args.Num() == 7)
		{
			bool bAnyEmpty = false;
			for (int32 i = 0; i < Args.Num(); ++i)
			{
				bAnyEmpty |= Args[i].Len() == 0;
			}
			if (bAnyEmpty)
			{
				bPrintHelp = true;
			}
			else
			{
				FName ViewName(*Args[0]);
				PlayerState.ViewPoint = GetViewPointFromStrings(&Args[1],Args.Num() - 1);
				Viewpoints.Add(ViewName,PlayerState.ViewPoint);
				bShouldLockView = true;
			}
		}
		// Anything else: unrecognized. Print help.
		else
		{
			bPrintHelp = true;
		}

		if (bShouldLockView)
		{
			PlayerState.bLocked = true;

			// Also copy to the clipboard.
			FString ViewPointString = ViewPointToString(PlayerState.ViewPoint);
			FPlatformApplicationMisc::ClipboardCopy(*ViewPointString);

			PlayerStates.AddAnnotation(Player, MoveTemp(PlayerState));
		}

		if (bPrintHelp)
		{
			UE_LOGF(LogConsoleResponse,Display,
				"Locks the player view and rendering time.\n" "r.LockView ?\n" "   Displays this message.\n" "r.LockView\n" "   Toggles whether the view is currently locked.\n" "r.LockView <name>\n" "   Locks the view at the named location. If there is no stored view with that name the current view is stored with that name.\n" "r.LockView x y z pitch yaw roll\n" "   Locks the view at the specified location and rotation.\n" "r.LockView <name> x y z pitch yaw roll\n" "   Locks the view at the specified location and rotation and stores it with the specified name.\n"
				);
		}
	}

private:
	/**
	 * Information stored for a given viewpoint.
	 */
	struct FViewPoint
	{
		FVector Location;
		float FOV;
		FRotator Rotation;
	};

	/** Viewpoints stored by name. */
	TMap<FName,FViewPoint> Viewpoints;

	/**
	 * Per-player state attached to ULocalPlayer objects via a sparse UObject
	 * annotation.
	 */
	struct FPlayerState
	{
		FViewPoint ViewPoint;
		bool bLocked;

		FPlayerState()
		{
			ViewPoint.Location = FVector::ZeroVector;
			ViewPoint.FOV = 90.0f;
			ViewPoint.Rotation = FRotator::ZeroRotator;
			bLocked = false;
		}

		bool IsDefault() const
		{
			return bLocked == false
				&& ViewPoint.Location == FVector::ZeroVector
				&& ViewPoint.FOV == 90.0f
				&& ViewPoint.Rotation == FRotator::ZeroRotator;
		}
	};
	FUObjectAnnotationSparse<FPlayerState,/*bAutoRemove=*/true> PlayerStates;

	/** Default constructor. */
	FLockedViewState()
	{
	}

	/**
	 * Parses a viewpoint from an array of strings.
	 *   WARNING: It is expected that the array has six entries!
	 */
	static FViewPoint GetViewPointFromStrings(const FString* Strings, int32 NumStrings)
	{
		FViewPoint ViewPoint = {};
		if (NumStrings == 6)
		{
			ViewPoint.Location.X = FCString::Atof(*Strings[0]);
			ViewPoint.Location.Y = FCString::Atof(*Strings[1]);
			ViewPoint.Location.Z = FCString::Atof(*Strings[2]);
			ViewPoint.Rotation.Pitch = FCString::Atof(*Strings[3]);
			ViewPoint.Rotation.Yaw = FCString::Atof(*Strings[4]);
			ViewPoint.Rotation.Roll = FCString::Atof(*Strings[5]);
			ViewPoint.FOV = 90.0f;
		}
		return ViewPoint;
	}

	/**
	 * Constructs a string from the view point.
	 */
	static FString ViewPointToString(const FViewPoint& ViewPoint)
	{
		return FString::Printf(TEXT("%f %f %f %f %f %f"),
			ViewPoint.Location.X,
			ViewPoint.Location.Y,
			ViewPoint.Location.Z,
			ViewPoint.Rotation.Pitch,
			ViewPoint.Rotation.Yaw,
			ViewPoint.Rotation.Roll
			);
	}

	/**
	 * Constructs a string representing all locked views and copies it to the
	 * clipboard. Passing this string to r.LockViews will restore the state of
	 * those locked views.
	 */
	static void CopyLockedViews()
	{
		FString LockedViewsStr;
		FLockedViewState& This = FLockedViewState::Get();
		bool bFirst = true;

		for (TMap<FName,FViewPoint>::TConstIterator It(This.Viewpoints); It; ++It)
		{
			LockedViewsStr += FString::Printf(
				TEXT("%s%s %s"),
				bFirst ? TEXT("") : TEXT(";\n"),
				*It.Key().ToString(),
				*ViewPointToString(It.Value())
				);
			bFirst = false;
		}
		FPlatformApplicationMisc::ClipboardCopy(*LockedViewsStr);
		UE_LOGF(LogConsoleResponse,Display,"%ls",*LockedViewsStr);
	}

	static FAutoConsoleCommand CmdCopyLockedViews;
};

/** Console command to copy all named locked views to the clipboard. */
FAutoConsoleCommand FLockedViewState::CmdCopyLockedViews(
	TEXT("r.CopyLockedViews"),
	TEXT("Copies all locked views in to a string that r.LockView will accept to reload them."),
	FConsoleCommandDelegate::CreateStatic(FLockedViewState::CopyLockedViews)
	);

void ULocalPlayer::GetViewPoint(FMinimalViewInfo& OutViewInfo) const
{
	if (FLockedViewState::Get().GetViewPoint(this, OutViewInfo.Location, OutViewInfo.Rotation, OutViewInfo.FOV) == false
		&& PlayerController != NULL)
	{
		if (PlayerController->PlayerCameraManager != NULL)
		{
			OutViewInfo = PlayerController->PlayerCameraManager->GetCameraCacheView();
			OutViewInfo.FOV = PlayerController->PlayerCameraManager->GetFOVAngle();
			PlayerController->GetPlayerViewPoint(/*out*/ OutViewInfo.Location, /*out*/ OutViewInfo.Rotation);
		}
		else
		{
			PlayerController->GetPlayerViewPoint(/*out*/ OutViewInfo.Location, /*out*/ OutViewInfo.Rotation);
		}
	}

	if (ViewportClient != nullptr && ViewportClientOverride == nullptr)
	{
		FSceneViewExtensionContext SceneViewExtensionContext(ViewportClient->Viewport);
		SceneViewExtensionContext.bStereoEnabled = true;
		for (const FSceneViewExtensionRef& ViewExt : GEngine->ViewExtensions->GatherActiveExtensions(SceneViewExtensionContext))
		{
			ViewExt->SetupViewPoint(PlayerController, OutViewInfo);
		};
	}

	// We store the originally desired FOV as other classes may adjust to account for ultra-wide aspect ratios
	OutViewInfo.DesiredFOV = OutViewInfo.FOV;
}

void ULocalPlayer::ReceivedPlayerController(APlayerController* NewController)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_LocalPlayer_HandlePlayerControllerChanged);
	
	Super::ReceivedPlayerController(NewController);
	
	// Broadcast an event for anyone that may be listening
	OnPlayerControllerChanged().Broadcast(NewController);

	// Tell any local player subsystems
	SubsystemCollection.ForEachSubsystem([NewController](ULocalPlayerSubsystem* Subsystem){
		Subsystem->PlayerControllerChanged(NewController);
	});
}

bool ULocalPlayer::CalcSceneViewInitOptions(
	struct FSceneViewInitOptions& ViewInitOptions,
	FViewport* Viewport,
	class FViewElementDrawer* ViewDrawer,
	int32 StereoViewIndex)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CalcSceneViewInitOptions);
	if ((PlayerController == nullptr) || (Size.X <= 0.f) || (Size.Y <= 0.f) || (Viewport == nullptr))
	{
		return false;
	}

	// If we are emulating splitscreen rendering for a screenshot functional test, override the layout set in UGameViewportClient::LayoutPlayers()
	if (bEmulateSplitscreen)
	{
		static const TConsoleVariableData<int32>* CVarEmulateSplitscreenMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ScreenshotTest.EmulateSplitscreenMode"));
		const int32 EmulateSplitscreenMode = CVarEmulateSplitscreenMode ? CVarEmulateSplitscreenMode->GetValueOnAnyThread() : 0;

		checkf(EmulateSplitscreenMode >= 1 && EmulateSplitscreenMode <= 3, TEXT("bEmulateSplitscreen should only be set by screenshot functional tests via r.ScreenshotTest.EmulateSplitscreenMode"));
		checkf((StereoViewIndex == 0 || StereoViewIndex == 1), TEXT("bEmulateSplitscreen only supports two splitscreen views"));

		const bool bBaseView = (StereoViewIndex == 0);
		switch (EmulateSplitscreenMode)
		{
			case 1:	// Horizontal
				Origin = bBaseView ? FVector2D(0.0f, 0.0f) : FVector2D(0.5f, 0.0f);
				Size = FVector2D(0.5f, 1.0f);
				break;
			case 2:	// Vertical
				Origin = bBaseView ? FVector2D(0.0f, 0.0f) : FVector2D(0.0f, 0.5f);
				Size = FVector2D(1.0f, 0.5f);
				break;
			case 3:	// Diagonal
				Origin = bBaseView ? FVector2D(0.0f, 0.0f) : FVector2D(0.5f, 0.5f);
				Size = FVector2D(0.5f, 0.5f);
				break;
			default:
				checkNoEntry();
				break;
		}
	}

	// get the projection data
	if (GetProjectionData(Viewport, /*inout*/ ViewInitOptions, StereoViewIndex) == false)
	{
		// Return NULL if this we didn't get back the info we needed
		return false;
	}

	// return if we have an invalid view rect
	if (!ViewInitOptions.IsValidViewRectangle())
	{
		return false;
	}

	if (PlayerController->PlayerCameraManager != NULL)
	{
		// Apply screen fade effect to screen.
		if (PlayerController->PlayerCameraManager->bEnableFading)
		{
			ViewInitOptions.OverlayColor = PlayerController->PlayerCameraManager->FadeColor;
			ViewInitOptions.OverlayColor.A = FMath::Clamp(PlayerController->PlayerCameraManager->FadeAmount, 0.0f, 1.0f);
		}

		// Do color scaling if desired.
		if (PlayerController->PlayerCameraManager->bEnableColorScaling)
		{
			ViewInitOptions.ColorScale = FLinearColor(
				PlayerController->PlayerCameraManager->ColorScale.X,
				PlayerController->PlayerCameraManager->ColorScale.Y,
				PlayerController->PlayerCameraManager->ColorScale.Z
				);
		}

		// Was there a camera cut this frame?
		ViewInitOptions.bInCameraCut = PlayerController->PlayerCameraManager->bGameCameraCutThisFrame;
	}

	if (GEngine->IsStereoscopic3D(Viewport))
	{
		ViewInitOptions.StereoPass = GEngine->StereoRenderingDevice->GetViewPassForIndex(StereoViewIndex != INDEX_NONE, StereoViewIndex);
	}

	check(PlayerController && PlayerController->GetWorld());

	// True when this view should use a device-managed ViewState.
	const bool bUseExternalViewState = GEngine->IsStereoscopic3D(Viewport)
		&& GEngine->StereoRenderingDevice->HasExternalViewState(StereoViewIndex);

	if (!bUseExternalViewState)
	{
		const uint32 ViewIndex = StereoViewIndex != INDEX_NONE ? StereoViewIndex : 0;

		// Make sure the ViewStates array has enough elements for the given ViewIndex.
		{
			const int32 RequiredViewStates = (ViewIndex + 1) - ViewStates.Num();

			if (RequiredViewStates > 0)
			{
				ViewStates.AddDefaulted(RequiredViewStates);
			}
		}

		// Allocate the current ViewState if necessary
		if (ViewStates[ViewIndex].GetReference() == nullptr)
		{
			const UWorld* CurrentWorld = GetWorld();
			const ERHIFeatureLevel::Type FeatureLevel = CurrentWorld ? CurrentWorld->GetFeatureLevel() : GMaxRHIFeatureLevel;

			ViewStates[ViewIndex].Allocate(FeatureLevel);
		}

		// Use ViewState from ULocalPlayer
		ViewInitOptions.SceneViewStateInterface = ViewStates[ViewIndex].GetReference();
	}
	else
	{
		// Use ViewState from IStereoRendering
		ViewInitOptions.SceneViewStateInterface = GEngine->StereoRenderingDevice->GetExternalViewState(StereoViewIndex);
	}

	ViewInitOptions.ViewActor = PlayerController->GetViewTarget();

	// TODO: Switch to GetLocalPlayerIndex during GetControllerId deprecation, this is only used by MotionControllerComponent
	ViewInitOptions.PlayerIndex = GetControllerId();
	ViewInitOptions.ViewElementDrawer = ViewDrawer;
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.LODDistanceFactor = PlayerController->LocalPlayerCachedLODDistanceFactor;
	ViewInitOptions.StereoViewIndex = StereoViewIndex;
	ViewInitOptions.WorldToMetersScale = PlayerController->GetWorldSettings()->WorldToMeters;
	ViewInitOptions.CursorPos = Viewport->HasMouseCapture() ? FIntPoint(-1, -1) : FIntPoint(Viewport->GetMouseX(), Viewport->GetMouseY());
	ViewInitOptions.OriginOffsetThisFrame = PlayerController->GetWorld()->OriginOffsetThisFrame;

	return true;
}

FSceneView* ULocalPlayer::CalcSceneView( class FSceneViewFamily* ViewFamily,
	FVector& OutViewLocation,
	FRotator& OutViewRotation,
	FViewport* Viewport,
	class FViewElementDrawer* ViewDrawer,
	int32 StereoViewIndex)
{
	SCOPE_CYCLE_COUNTER(STAT_CalcSceneView);

	FSceneViewInitOptions ViewInitOptions;

	if (!CalcSceneViewInitOptions(ViewInitOptions, Viewport, ViewDrawer, StereoViewIndex))
	{
		return nullptr;
	}

	// Get the viewpoint...technically doing this twice
	// but it makes GetProjectionData better
	FMinimalViewInfo ViewInfo;
	GetViewPoint(ViewInfo);
	ViewInitOptions.ViewLocation = ViewInfo.Location;
	ViewInitOptions.ViewRotation = ViewInfo.Rotation;
	ViewInitOptions.bUseFieldOfViewForLOD = ViewInfo.bUseFieldOfViewForLOD;
	ViewInitOptions.FOV = ViewInfo.FOV;
	ViewInitOptions.DesiredFOV = ViewInfo.DesiredFOV;
	ViewInitOptions.FirstPersonParams = FFirstPersonParameters(ViewInfo.CalculateFirstPersonFOVCorrectionFactor(), ViewInfo.FirstPersonScale, ViewInfo.bUseFirstPersonParameters);
	ViewInitOptions.OverscanResolutionFraction = ViewInfo.OverscanResolutionFraction;
	ViewInitOptions.CropFraction = ViewInfo.CropFraction;
	ViewInitOptions.AsymmetricCropFraction = ViewInfo.AsymmetricCropFraction;
	
	// Fill out the rest of the view init options
	ViewInitOptions.ViewFamily = ViewFamily;

	if (!PlayerController->bRenderPrimitiveComponents)
	{
		// Emplaces an empty show only primitive list.
		ViewInitOptions.ShowOnlyPrimitives.Emplace();
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BuildHiddenComponentList);
		PlayerController->BuildHiddenComponentList(ViewInfo.Location, /*out*/ ViewInitOptions.HiddenPrimitives);
	}

	//@TODO: SPLITSCREEN: This call will have an issue with splitscreen, as the show flags are shared across the view family
	EngineShowFlagOrthographicOverride( ViewInitOptions.IsPerspectiveProjection(), ViewFamily->EngineShowFlags );

	FSceneView* const View = new FSceneView(ViewInitOptions);

	OutViewLocation = View->ViewLocation;
	OutViewRotation = View->ViewRotation;
	// Pass on the previous view transform from the view info (probably provided by the camera if set)
	View->PreviousViewTransform = ViewInfo.PreviousViewTransform;

	ViewFamily->Views.Add(View);

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PostprocessSettings);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(PostProcessSettings);
		View->StartFinalPostprocessSettings(ViewInfo.Location);

		TArray<FPostProcessSettings> const* CameraAnimPPSettings = nullptr;
		TArray<float> const* CameraAnimPPBlendWeights = nullptr;
		TArray<EViewTargetBlendOrder> const* CameraAnimPPBlendOrders = nullptr;

		// Base overrides (post process volumes, etc)
		if (PlayerController->PlayerCameraManager)
		{
			PlayerController->PlayerCameraManager->GetCachedPostProcessBlends(CameraAnimPPSettings, CameraAnimPPBlendWeights, CameraAnimPPBlendOrders);
			checkSlow(CameraAnimPPBlendWeights && CameraAnimPPBlendOrders && CameraAnimPPSettings);

			for (int32 PPIdx = 0; PPIdx < CameraAnimPPBlendWeights->Num(); ++PPIdx)
			{
				if ((*CameraAnimPPBlendOrders)[PPIdx] == VTBlendOrder_Base)
				{
					View->OverridePostProcessSettings( (*CameraAnimPPSettings)[PPIdx], (*CameraAnimPPBlendWeights)[PPIdx]);
				}
			}
		}

		// Main camera
		View->OverridePostProcessSettings(ViewInfo.PostProcessSettings, ViewInfo.PostProcessBlendWeight);

		// Camera overrides (cameras blending in, camera modifiers, etc)
		if (PlayerController->PlayerCameraManager)
		{
			checkSlow(CameraAnimPPBlendWeights && CameraAnimPPBlendOrders && CameraAnimPPSettings);
			for (int32 PPIdx = 0; PPIdx < CameraAnimPPBlendWeights->Num(); ++PPIdx)
			{
				if ((*CameraAnimPPBlendOrders)[PPIdx] == VTBlendOrder_Override)
				{
					View->OverridePostProcessSettings( (*CameraAnimPPSettings)[PPIdx], (*CameraAnimPPBlendWeights)[PPIdx]);
				}
			}

			PlayerController->PlayerCameraManager->UpdatePhotographyPostProcessing(View->FinalPostProcessSettings);
		}

		if (GEngine->IsStereoscopic3D(Viewport))
		{
			FPostProcessSettings StereoDeviceOverridePostProcessinSettings;
			float BlendWeight = 1.0f;
			bool StereoSettingsAvailable = GEngine->StereoRenderingDevice->OverrideFinalPostprocessSettings(&StereoDeviceOverridePostProcessinSettings, View->StereoPass, View->StereoViewIndex, BlendWeight);
			if (StereoSettingsAvailable)
			{
				View->OverridePostProcessSettings(StereoDeviceOverridePostProcessinSettings, BlendWeight);
			}
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		ADebugCameraController* DebugCameraController = Cast<ADebugCameraController>(PlayerController);
		if (DebugCameraController != nullptr)
		{
			DebugCameraController->UpdateVisualizeBufferPostProcessing(View->FinalPostProcessSettings);
		}
#endif

		View->EndFinalPostprocessSettings(ViewInitOptions);
	}

	for (int ViewExt = 0; ViewExt < ViewFamily->ViewExtensions.Num(); ViewExt++)
	{
		ViewFamily->ViewExtensions[ViewExt]->SetupView(*ViewFamily, *View);
	}

	return View;
}

bool ULocalPlayer::GetPixelBoundingBox(const FBox& ActorBox, FVector2D& OutLowerLeft, FVector2D& OutUpperRight, const FVector2f* OptionalAllotedSize)
{
	//@TODO: CAMERA: This has issues with aspect-ratio constrained cameras
	if ((ViewportClient != NULL) && (ViewportClient->Viewport != NULL) && (PlayerController != NULL))
	{
		// get the projection data
		FSceneViewProjectionData ProjectionData;
		if (GetProjectionData(ViewportClient->Viewport, /*out*/ ProjectionData) == false)
		{
			return false;
		}

		return ULocalPlayer::GetPixelBoundingBox(ProjectionData, ActorBox, OutLowerLeft, OutUpperRight, OptionalAllotedSize);
	}
	else
	{
		return false;
	}
}

bool ULocalPlayer::GetPixelBoundingBox(const FSceneViewProjectionData& ProjectionData, const FBox& ActorBox, FVector2D& OutLowerLeft, FVector2D& OutUpperRight, const FVector2f* OptionalAllotedSize)
{
	// if we passed in an optional size, use it for the viewrect
	FIntRect ViewRect = ProjectionData.GetConstrainedViewRect();
	if (OptionalAllotedSize != NULL)
	{
		ViewRect.Min = FIntPoint(0, 0);
		ViewRect.Max = FIntPoint(OptionalAllotedSize->X, OptionalAllotedSize->Y);
	}

	// transform the box
	const int32 NumOfVerts = 8;
	FVector Vertices[NumOfVerts] =
	{
		FVector(ActorBox.Min),
		FVector(ActorBox.Min.X, ActorBox.Min.Y, ActorBox.Max.Z),
		FVector(ActorBox.Min.X, ActorBox.Max.Y, ActorBox.Min.Z),
		FVector(ActorBox.Max.X, ActorBox.Min.Y, ActorBox.Min.Z),
		FVector(ActorBox.Max.X, ActorBox.Max.Y, ActorBox.Min.Z),
		FVector(ActorBox.Max.X, ActorBox.Min.Y, ActorBox.Max.Z),
		FVector(ActorBox.Min.X, ActorBox.Max.Y, ActorBox.Max.Z),
		FVector(ActorBox.Max)
	};

	// create the view projection matrix
	const FMatrix ViewProjectionMatrix = ProjectionData.ComputeViewProjectionMatrix();

	int SuccessCount = 0;
	OutLowerLeft = FVector2D(FLT_MAX, FLT_MAX);
	OutUpperRight = FVector2D(FLT_MIN, FLT_MIN);
	for (int i = 0; i < NumOfVerts; ++i)
	{
		//grab the point in screen space
		const FVector4 ScreenPoint = ViewProjectionMatrix.TransformFVector4(FVector4(Vertices[i], 1.0f));

		if (ScreenPoint.W > 0.0f)
		{
			float InvW = 1.0f / ScreenPoint.W;
			FVector2D PixelPoint = FVector2D(ViewRect.Min.X + (0.5f + ScreenPoint.X * 0.5f * InvW) * ViewRect.Width(),
				ViewRect.Min.Y + (0.5f - ScreenPoint.Y * 0.5f * InvW) * ViewRect.Height());

			PixelPoint.X = FMath::Clamp<float>(PixelPoint.X, 0, ViewRect.Width());
			PixelPoint.Y = FMath::Clamp<float>(PixelPoint.Y, 0, ViewRect.Height());

			OutLowerLeft.X = FMath::Min(OutLowerLeft.X, PixelPoint.X);
			OutLowerLeft.Y = FMath::Min(OutLowerLeft.Y, PixelPoint.Y);

			OutUpperRight.X = FMath::Max(OutUpperRight.X, PixelPoint.X);
			OutUpperRight.Y = FMath::Max(OutUpperRight.Y, PixelPoint.Y);

			++SuccessCount;
		}
	}

	// make sure we are calculating with more than one point;
	return SuccessCount >= 2;
}

bool ULocalPlayer::GetPixelPoint(const FVector& InPoint, FVector2D& OutPoint, const FVector2f* OptionalAllotedSize)
{
	//@TODO: CAMERA: This has issues with aspect-ratio constrained cameras
	if ((ViewportClient != NULL) && (ViewportClient->Viewport != NULL) && (PlayerController != NULL))
	{
		// get the projection data
		FSceneViewProjectionData ProjectionData;
		if (GetProjectionData(ViewportClient->Viewport, /*inout*/ ProjectionData) == false)
		{
			return false;
		}

		return ULocalPlayer::GetPixelPoint(ProjectionData, InPoint, OutPoint, OptionalAllotedSize);
	}

	return false;
}

bool ULocalPlayer::GetPixelPoint(const FSceneViewProjectionData& ProjectionData, const FVector& InPoint, FVector2D& OutPoint, const FVector2f* OptionalAllotedSize)
{
	bool bInFrontOfCamera = true;

	// if we passed in an optional size, use it for the viewrect
	FIntRect ViewRect = ProjectionData.GetConstrainedViewRect();
	if (OptionalAllotedSize != NULL)
	{
		ViewRect.Min = FIntPoint(0, 0);
		ViewRect.Max = FIntPoint(OptionalAllotedSize->X, OptionalAllotedSize->Y);
	}

	// create the view projection matrix
	const FMatrix ViewProjectionMatrix = ProjectionData.ComputeViewProjectionMatrix();

	//@TODO: CAMERA: Validate this code!
	// grab the point in screen space
	FVector4 ScreenPoint = ViewProjectionMatrix.TransformFVector4(FVector4(InPoint, 1.0f));

	ScreenPoint.W = (ScreenPoint.W == 0) ? UE_KINDA_SMALL_NUMBER : ScreenPoint.W;

	float InvW = 1.0f / ScreenPoint.W;
	OutPoint = FVector2D(ViewRect.Min.X + (0.5f + ScreenPoint.X * 0.5f * InvW) * ViewRect.Width(),
		ViewRect.Min.Y + (0.5f - ScreenPoint.Y * 0.5f * InvW) * ViewRect.Height());

	if (ScreenPoint.W < 0.0f)
	{
		bInFrontOfCamera = false;
		OutPoint = FVector2D(ViewRect.Max) - OutPoint;
	}

	return bInFrontOfCamera;
}

bool ULocalPlayer::GetProjectionData(FViewport* Viewport, FSceneViewProjectionData& ProjectionData, int32 StereoViewIndex) const
{
	// If the actor
	if ((Viewport == nullptr) || (PlayerController == nullptr) || (Viewport->GetSizeXY().X == 0) || (Viewport->GetSizeXY().Y == 0) || (Size.X == 0) || (Size.Y == 0))
	{
		return false;
	}

	int32 X = FMath::TruncToInt(Origin.X * Viewport->GetSizeXY().X);
	int32 Y = FMath::TruncToInt(Origin.Y * Viewport->GetSizeXY().Y);

	X += Viewport->GetInitialPositionXY().X;
	Y += Viewport->GetInitialPositionXY().Y;

	uint32 SizeX = FMath::TruncToInt(Size.X * Viewport->GetSizeXY().X);
	uint32 SizeY = FMath::TruncToInt(Size.Y * Viewport->GetSizeXY().Y);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// We expect some size to avoid problems with the view rect manipulation
	if(SizeX > 50 && SizeY > 50)
	{
		int32 Value = CVarViewportTest.GetValueOnGameThread();

		if(Value)
		{
			int InsetX = SizeX / 4;
			int InsetY = SizeY / 4;

			// this allows to test various typical view port situations (todo: split screen)
			switch(Value)
			{
				case 1: X += InsetX; Y += InsetY; SizeX -= InsetX * 2; SizeY -= InsetY * 2;break;
				case 2: Y += InsetY; SizeY -= InsetY * 2; break;
				case 3: X += InsetX; SizeX -= InsetX * 2; break;
				case 4: SizeX /= 2; SizeY /= 2; break;
				case 5: SizeX /= 2; SizeY /= 2; X += SizeX;	break;
				case 6: SizeX /= 2; SizeY /= 2; Y += SizeY; break;
				case 7: SizeX /= 2; SizeY /= 2; X += SizeX; Y += SizeY; break;
			}
		}
	}
#endif

	FIntRect UnconstrainedRectangle = FIntRect(X, Y, X+SizeX, Y+SizeY);

	ProjectionData.SetViewRectangle(UnconstrainedRectangle);

	// Get the viewpoint.
	FMinimalViewInfo ViewInfo;
	GetViewPoint(/*out*/ ViewInfo);
	



	// If stereo rendering is enabled, update the size and offset appropriately for this pass
	const bool bNeedStereo = StereoViewIndex != INDEX_NONE && GEngine->IsStereoscopic3D();
	const bool bIsHeadTrackingAllowed =
		GEngine->XRSystem.IsValid() &&
		(GetWorld() != nullptr ? GEngine->XRSystem->IsHeadTrackingAllowedForWorld(*GetWorld()) : GEngine->XRSystem->IsHeadTrackingAllowed());
	if (bNeedStereo)
	{
		GEngine->StereoRenderingDevice->AdjustViewRect(StereoViewIndex, X, Y, SizeX, SizeY);
	}

	// scale distances for cull distance purposes by the ratio of our current FOV to the default FOV
	if (GCalcLocalPlayerCachedLODDistanceFactor != 0)
	{
		PlayerController->LocalPlayerCachedLODDistanceFactor = ViewInfo.FOV / FMath::Max<float>(0.01f, (PlayerController->PlayerCameraManager != NULL) ? PlayerController->PlayerCameraManager->DefaultFOV : 90.f);
	}
	else // This should be removed in the final version. Leaving in so this can be toggled on and off in order to evaluate it.
	{
		PlayerController->LocalPlayerCachedLODDistanceFactor = 1.f;
	}

    FVector StereoViewLocation = ViewInfo.Location;
    if (bNeedStereo || bIsHeadTrackingAllowed)
    {
		auto XRCamera = GEngine->XRSystem.IsValid() ? GEngine->XRSystem->GetXRCamera() : nullptr;
		if (XRCamera.IsValid())
		{
			AActor* ViewTarget = PlayerController->GetViewTarget();
			const bool bHasActiveCamera = ViewTarget && ViewTarget->HasActiveCameraComponent();
			XRCamera->UseImplicitHMDPosition(bHasActiveCamera);
		}

		if (GEngine->IsStereoscopic3D(Viewport))
		{
			GEngine->StereoRenderingDevice->CalculateStereoViewOffset(StereoViewIndex, ViewInfo.Rotation, GetWorld()->GetWorldSettings()->WorldToMeters, StereoViewLocation);
		}
    }

	// Create the view matrix
	ProjectionData.ViewOrigin = StereoViewLocation;
	ProjectionData.ViewRotationMatrix = FInverseRotationMatrix(ViewInfo.Rotation) * FMatrix(
		FPlane(0,	0,	1,	0),
		FPlane(1,	0,	0,	0),
		FPlane(0,	1,	0,	0),
		FPlane(0,	0,	0,	1));

#if WITH_EDITOR
	ULevelEditorPlaySettings* PlaySettingsConfig = GetMutableDefault<ULevelEditorPlaySettings>();
	if (PlaySettingsConfig != nullptr && PlaySettingsConfig->DeviceToEmulate.IsEmpty())
	{
		if (GEditor != nullptr && GEditor->IsPreviewPlatformActive() && GEditor->GetAllowConstrainedAspectRatioInPreview())
		{
			float PlatformAspectRatio = GEditor->PreviewPlatform.GetConstrainedAspectRatio();
			if (GEditor->GetFlipConstrainedAspectRatioInPreview() && PlatformAspectRatio != 0.f)
			{
				PlatformAspectRatio = 1 / PlatformAspectRatio;
			}

			FSlateApplication::Get().OnConstrainedAspectRatioChanged.Broadcast(PlatformAspectRatio);

			FMargin Safezones(GEditor->PreviewPlatform.GetSafeZones());
			if (Safezones != FSlateApplication::Get().GetCustomSafeZone())
			{
				FSlateApplication::Get().SetCustomSafeZone(Safezones);
				FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(Safezones, true);
			}
		}
		else
		{
			FSlateApplication::Get().OnConstrainedAspectRatioChanged.Broadcast(0.f);

			if (FMargin() != FSlateApplication::Get().GetCustomSafeZone())
			{
				FSlateApplication::Get().ResetCustomSafeZone();
				FSlateApplication::Get().OnDebugSafeZoneChanged.Broadcast(FMargin(), true);
			}
		}
	}
#endif

	// @todo viewext this use case needs to be revisited
	if (!bNeedStereo)
	{
		// Create the projection matrix (and possibly constrain the view rectangle)
		FMinimalViewInfo::CalculateProjectionMatrixGivenView(ViewInfo, AspectRatioAxisConstraint, Viewport, /*inout*/ ProjectionData);

		for (auto& ViewExt : GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(Viewport)))
        {
			ViewExt->SetupViewProjectionMatrix(ProjectionData);
		};
	}
	else
	{
		// Let the stereoscopic rendering device handle creating its own projection matrix, as needed
		ProjectionData.ProjectionMatrix = GEngine->StereoRenderingDevice->GetStereoProjectionMatrix(StereoViewIndex);

		// calculate the out rect
		ProjectionData.SetViewRectangle(FIntRect(X, Y, X + SizeX, Y + SizeY));
	}

	return true;
}

bool ULocalPlayer::HandleDNCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Create a pending Note actor (only in PIE)
	if( PlayerController )
	{
		FString Comment = FString(Cmd);
		int32 NewNoteIndex = GEngine->PendingDroppedNotes.AddZeroed();
		FDropNoteInfo& NewNote = GEngine->PendingDroppedNotes[NewNoteIndex];

		// Use the pawn's location if we have one
		if( PlayerController->GetPawnOrSpectator() != NULL )
		{
			NewNote.Location = PlayerController->GetPawnOrSpectator()->GetActorLocation();
		}
		else
		{
			// No pawn, so just use the camera's location
			FRotator CameraRotation;
			PlayerController->GetPlayerViewPoint(NewNote.Location, CameraRotation);
		}

		NewNote.Rotation = PlayerController->GetControlRotation();
		NewNote.Comment = Comment;
		UE_LOGF(LogPlayerManagement, Log, "Note Dropped: (%3.2f,%3.2f,%3.2f) - '%ls'", NewNote.Location.X, NewNote.Location.Y, NewNote.Location.Z, *NewNote.Comment);
	}
	return true;
}


bool ULocalPlayer::HandleExitCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// If there is no viewport it was already closed.
	if ( ViewportClient && ViewportClient->Viewport )
	{
		ViewportClient->CloseRequested(ViewportClient->Viewport);
	}

	FGameDelegates::Get().GetExitCommandDelegate().Broadcast();

	return true;
}

bool ULocalPlayer::HandleListMoveBodyCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GShouldLogOutAFrameOfSetBodyTransform = true;
	return true;
}

bool ULocalPlayer::HandleListAwakeBodiesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GetWorld()->GetPhysicsScene()->ListAwakeRigidBodies(true);
	return true;
}

bool ULocalPlayer::HandleListSimBodiesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GetWorld()->GetPhysicsScene()->ListAwakeRigidBodies(false);
	return true;
}

bool ULocalPlayer::HandleMoveComponentTimesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GShouldLogOutAFrameOfMoveComponent = true;
	return true;
}

bool ULocalPlayer::HandleListSkelMeshesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Iterate over all skeletal mesh components and create mapping from skeletal mesh to instance.
	TMultiMap<USkeletalMesh*,USkeletalMeshComponent*> SkeletalMeshToInstancesMultiMap;
	for( TObjectIterator<USkeletalMeshComponent> It; It; ++It )
	{
		USkeletalMeshComponent* SkeletalMeshComponent = *It;
		USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();

		if( !SkeletalMeshComponent->IsTemplate() )
		{
			SkeletalMeshToInstancesMultiMap.Add( SkeletalMesh, SkeletalMeshComponent );
		}
	}

	// Retrieve player location for distance checks.
	FVector PlayerLocation = FVector::ZeroVector;
	if( PlayerController && PlayerController->GetPawn() )
	{
		PlayerLocation = PlayerController->GetPawn()->GetActorLocation();
	}

	// Iterate over multi-map and dump information sorted by skeletal mesh.
	for( TObjectIterator<USkeletalMesh> It; It; ++It )
	{
		// Look up array of instances associated with this key/ skeletal mesh.
		USkeletalMesh* SkeletalMesh = *It;
		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		SkeletalMeshToInstancesMultiMap.MultiFind( SkeletalMesh, SkeletalMeshComponents );

		if( SkeletalMesh && SkeletalMeshComponents.Num() )
		{
			// Dump information about skeletal mesh.
			FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalMesh->GetResourceForRendering();
			check(SkelMeshRenderData->LODRenderData.Num());
			UE_LOGF(LogPlayerManagement, Log, "%5i Vertices for LOD 0 of %ls", SkelMeshRenderData->LODRenderData[0].GetNumVertices(),*SkeletalMesh->GetFullName());

			// Dump all instances.
			for( int32 InstanceIndex=0; InstanceIndex<SkeletalMeshComponents.Num(); InstanceIndex++ )
			{
				USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshComponents[InstanceIndex];
				check(SkeletalMeshComponent);
				UWorld* World = SkeletalMeshComponent->GetWorld();
				check(World);
				float TimeSinceLastRender = World->GetTimeSeconds() - SkeletalMeshComponent->GetLastRenderTime();

				UE_LOGF(LogPlayerManagement, Log, "%ls%2i  Component    : %ls",
					(TimeSinceLastRender > 0.5) ? TEXT(" ") : TEXT("*"),
					InstanceIndex,
					*SkeletalMeshComponent->GetFullName() );
				if( SkeletalMeshComponent->GetOwner() )
				{
					UE_LOGF(LogPlayerManagement, Log, "     Owner        : %ls",*SkeletalMeshComponent->GetOwner()->GetFullName());
				}
				UE_LOGF(LogPlayerManagement, Log, "     LastRender   : %f", TimeSinceLastRender);
				UE_LOGF(LogPlayerManagement, Log, "     CullDistance : %f   Distance: %f   Location: (%7.1f,%7.1f,%7.1f)",
					SkeletalMeshComponent->CachedMaxDrawDistance,
					FVector::Dist( PlayerLocation, SkeletalMeshComponent->Bounds.Origin ),
					SkeletalMeshComponent->Bounds.Origin.X,
					SkeletalMeshComponent->Bounds.Origin.Y,
					SkeletalMeshComponent->Bounds.Origin.Z );
			}
		}
	}
	return true;
}

bool ULocalPlayer::HandleListPawnComponentsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	for( TObjectIterator<APawn> It; It; ++It )
	{
		APawn *Pawn = *It;
		UE_LOGF(LogPlayerManagement, Log, "Components for pawn: %ls (collision component: %ls)",*Pawn->GetName(),*Pawn->GetRootComponent()->GetName());

		int32 CompIdx = 0;
		for (UActorComponent* Comp : Pawn->GetComponents())
		{
			if (Comp && Comp->IsRegistered())
			{
				UE_LOGF(LogPlayerManagement, Log, "  %d: %ls",CompIdx,*Comp->GetName());
			}
			++CompIdx;
		}
	}
	return true;
}


bool ULocalPlayer::HandleExecCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TCHAR Filename[512];
	if( FParse::Token( Cmd, Filename, UE_ARRAY_COUNT(Filename), 0 ) )
	{
		ExecMacro( Filename, Ar );
	}
	return true;
}

bool ULocalPlayer::HandleToggleDrawEventsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
//Added ability to toggle this on during a test build if needed
#if WITH_PROFILEGPU || UE_BUILD_TEST
	if( GetEmitDrawEvents() )
	{
		SetEmitDrawEvents(false);
		UE_LOGF(LogEngine, Warning, "Draw events are now DISABLED");
	}
	else
	{
		SetEmitDrawEvents(true);
		UE_LOGF(LogEngine, Warning, "Draw events are now ENABLED");
	}
#endif
	return true;
}

bool ULocalPlayer::HandleToggleStreamingVolumesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (FParse::Command(&Cmd, TEXT("ON")))
	{
		GetWorld()->DelayStreamingVolumeUpdates( 0 );
	}
	else if (FParse::Command(&Cmd, TEXT("OFF")))
	{
		GetWorld()->DelayStreamingVolumeUpdates( INDEX_NONE );
	}
	else
	{
		if( GetWorld()->StreamingVolumeUpdateDelay == INDEX_NONE )
		{
			GetWorld()->DelayStreamingVolumeUpdates( 0 );
		}
		else
		{
			GetWorld()->DelayStreamingVolumeUpdates( INDEX_NONE );
		}
	}
	return true;
}

#if UE_ALLOW_EXEC_COMMANDS
bool ULocalPlayer::Exec(UWorld* InWorld, const TCHAR* Cmd,FOutputDevice& Ar)
{
// NOTE: all of these can probably be #if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) out

	if( FParse::Command(&Cmd,TEXT("LISTMOVEBODY")) )
	{
		return HandleListMoveBodyCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("r.LockView")) )
	{
		TArray<FString> Lines;
		FString CmdString(Cmd);
		int32 NewLineIndex;
		if (CmdString.FindChar(TEXT(';'),NewLineIndex))
		{
			CmdString.ParseIntoArray(Lines,TEXT(";"),true);
		}
		else
		{
			Lines.Add(CmdString);
		}

		for (int32 LineIndex = 0; LineIndex < Lines.Num(); ++LineIndex)
		{
			TArray<FString> Args;
			Lines[LineIndex].ParseIntoArrayWS(Args);
			FLockedViewState::Get().LockView(this,Args);
		}
		if (Lines.Num() > 1)
		{
			FLockedViewState::Get().UnlockView(this);
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("r.ResetViewState")))
	{
		// Reset states (e.g. TemporalAA index) to make rendering more deterministic (for automated screenshot verification)
		for (auto& State : ViewStates)
		{
			if (FSceneViewStateInterface* Ref = State.GetReference())
			{
				Ref->ResetViewState();
			}
		}

		return true;
	}
	// This will list all awake rigid bodies
	else if( FParse::Command(&Cmd,TEXT("LISTAWAKEBODIES")) )
	{
		return HandleListAwakeBodiesCommand( Cmd, Ar );
	}
	// This will list all simulating rigid bodies
	else if( FParse::Command(&Cmd,TEXT("LISTSIMBODIES")) )
	{
		return HandleListSimBodiesCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd, TEXT("MOVECOMPTIMES")) )
	{
		return HandleMoveComponentTimesCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("LISTSKELMESHES")) )
	{
		return HandleListSkelMeshesCommand( Cmd, Ar );
	}
	else if ( FParse::Command(&Cmd,TEXT("LISTPAWNCOMPONENTS")) )
	{
		return HandleListPawnComponentsCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("EXEC")) )
	{
		return HandleExecCommand( Cmd, Ar );
	}
#if WITH_PROFILEGPU
	else if( FParse::Command(&Cmd,TEXT("TOGGLEDRAWEVENTS")) )
	{
		return HandleToggleDrawEventsCommand( Cmd, Ar );
	}
#endif
	else if( FParse::Command(&Cmd,TEXT("TOGGLESTREAMINGVOLUMES")) )
	{
		return HandleToggleStreamingVolumesCommand( Cmd, Ar );
	}
	else if(ViewportClient && ViewportClient->Exec( InWorld, Cmd,Ar))
	{
		return true;
	}
	else if ( Super::Exec( InWorld, Cmd, Ar ) )
	{
		return true;
	}
	else
	{
		return false;
	}
}
#endif // UE_ALLOW_EXEC_COMMANDS

bool ULocalPlayer::Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		// Override a few commands in PIE
		if (FParse::Command(&Cmd, TEXT("DN")))
		{
			return HandleDNCommand(Cmd, Ar);
		}

		if( FParse::Command(&Cmd,TEXT("Exit"))
		||	FParse::Command(&Cmd,TEXT("Quit")))
		{
			return HandleExitCommand( Cmd, Ar );
		}

		if (FParse::Command(&Cmd, TEXT("FocusNextPIEWindow")))
		{
			GEngine->FocusNextPIEWorld(InWorld);
			return true;
		}
		if (FParse::Command(&Cmd, TEXT("FocusLastPIEWindow")))
		{
			GEngine->FocusNextPIEWorld(InWorld, true);
			return true;
		}

		if (Super::Exec_Editor(InWorld, Cmd, Ar))
		{
			return true;
		}
	}
#endif // WITH_EDITOR

	return false;
}

void ULocalPlayer::ExecMacro( const TCHAR* Filename, FOutputDevice& Ar )
{
	// make sure Binaries is specified in the filename
	FString FixedFilename;
	if (!FCString::Stristr(Filename, TEXT("Binaries")))
	{
		FixedFilename = FString("../../Binaries/") + Filename;
		Filename = *FixedFilename;
	}

	FString Text;
	if (FFileHelper::LoadFileToString(Text, Filename))
	{
		UE_LOGF(LogPlayerManagement, Log, "Execing %ls", Filename);
		const TCHAR* Data = *Text;
		FString Line;
		while( FParse::Line(&Data, Line) )
		{
			Exec(GetWorld(), *Line, Ar);
		}
	}
	else
	{
		UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Can't find file '%s'"), Filename));
	}
}

void ULocalPlayer::SetControllerId( int32 NewControllerId )
{
	if ( ControllerId != NewControllerId )
	{
		UE_LOGF(LogPlayerManagement, Log, "%ls changing ControllerId from %i to %i", *GetFName().ToString(), ControllerId, NewControllerId);

		const int32 CurrentControllerId = ControllerId;

		// set this player's ControllerId to -1 so that if we need to swap controllerIds with another player we don't
		// re-enter the function for this player.
		ControllerId = -1;

		// see if another player is already using this ControllerId; if so, swap controllerIds with them
		// TODO: Re-evaluate if this swap logic makes sense during controller id deprecation
		GEngine->SwapControllerId(this, CurrentControllerId, NewControllerId);
		ControllerId = NewControllerId;

		OnControllerIdChanged().Broadcast(NewControllerId, CurrentControllerId);

		if (GEngine->IsControllerIdUsingPlatformUserId())
		{
			// This won't recurse back because we've already modified ControllerId
			SetPlatformUserId(IPlatformInputDeviceMapper::Get().GetPlatformUserForUserIndex(NewControllerId));
		}
	}
}

void ULocalPlayer::SetPlatformUserId(FPlatformUserId NewPlatformUserId)
{
	if (NewPlatformUserId != PlatformUserId)
	{
		const FPlatformUserId CurrentPlatformUserId = PlatformUserId;

		// set this player's CurrentPlatformUserId to PLATFORMUSERID_NONE so that if we need to swap
		// platform users with another player we don't re-enter the function for this player.
		PlatformUserId = PLATFORMUSERID_NONE;

		// see if another player is already using this PlatformUserID; if so, swap PlatformUserIDs with them
		GEngine->SwapPlatformUserId(this, CurrentPlatformUserId, NewPlatformUserId);
		PlatformUserId = NewPlatformUserId;
		
		OnPlatformUserIdChanged().Broadcast(NewPlatformUserId, CurrentPlatformUserId);

		if (GEngine->IsControllerIdUsingPlatformUserId())
		{
			SetControllerId(IPlatformInputDeviceMapper::Get().GetUserIndexForPlatformUser(PlatformUserId));
		}
	}
}

int32 ULocalPlayer::GetPlatformUserIndex() const
{
	FPlatformUserId UserId = GetPlatformUserId();
	return IPlatformInputDeviceMapper::Get().GetUserIndexForPlatformUser(UserId);
}

int32 ULocalPlayer::GetLocalPlayerIndex() const
{
	return GetIndexInGameInstance();
}

FString ULocalPlayer::GetNickname() const
{
	UWorld* World = GetWorld();
	if (World != NULL)
	{
		// Try to get platform identity first
		FString PlatformNickname;
		if (UOnlineEngineInterface::Get()->GetPlayerPlatformNickname(World, PlatformUserId, PlatformNickname))
		{
			return PlatformNickname;
		}

		FUniqueNetIdRepl UniqueId = GetPreferredUniqueNetId();
		if (UniqueId.IsValid())
		{
			return UOnlineEngineInterface::Get()->GetPlayerNickname(World, UniqueId);
		}
	}

	return TEXT("");
}

FUniqueNetIdRepl ULocalPlayer::GetUniqueNetIdFromCachedControllerId() const
{
	UWorld* World = GetWorld();
	if (World != nullptr)
	{
		return FUniqueNetIdRepl(UOnlineEngineInterface::Get()->GetUniquePlayerIdWrapper(World, ControllerId));
	}

	return FUniqueNetIdRepl();
}

FUniqueNetIdRepl ULocalPlayer::GetUniqueNetIdForPlatformUser() const
{
	UWorld* World = GetWorld();
	if (World != nullptr)
	{
		return FUniqueNetIdRepl(UOnlineEngineInterface::Get()->GetUniquePlayerIdWrapper(World, PlatformUserId));
	}

	return FUniqueNetIdRepl();
}

FUniqueNetIdRepl ULocalPlayer::GetCachedUniqueNetId() const
{
	return CachedUniqueNetId;
}

void ULocalPlayer::SetCachedUniqueNetId(const FUniqueNetIdRepl& NewUniqueNetId)
{
	CachedUniqueNetId = NewUniqueNetId;
}

FUniqueNetIdRepl ULocalPlayer::GetPreferredUniqueNetId() const
{
	// Prefer the cached unique net id (only if it's valid)
	// This is for backwards compatibility for games that don't yet cache the unique id properly
	if (GetCachedUniqueNetId().IsValid())
	{
		return GetCachedUniqueNetId();
	}

	// If the cached unique net id is not valid, then use the platfomr user
	return GetUniqueNetIdForPlatformUser();
}

TSharedPtr<FSlateUser> ULocalPlayer::GetSlateUser()
{
	return FSlateApplication::Get().GetUserFromControllerId(ControllerId);
}

TSharedPtr<const FSlateUser> ULocalPlayer::GetSlateUser() const
{
	return FSlateApplication::Get().GetUserFromControllerId(ControllerId);
}

UWorld* ULocalPlayer::GetWorld() const
{
	return ViewportClient ? ViewportClient->GetWorld() : nullptr;
}

UGameInstance* ULocalPlayer::GetGameInstance() const
{
	return ViewportClient ? ViewportClient->GetGameInstance() : nullptr;
}

int32 ULocalPlayer::GetIndexInGameInstance() const
{
	int32 FoundIndex = INDEX_NONE;
	UGameInstance* GameInstance = GetGameInstance();
	if (GameInstance)
	{
		const TArray<ULocalPlayer*>& LocalPlayers = GameInstance->GetLocalPlayers();
		LocalPlayers.Find(const_cast<ULocalPlayer*>(this), FoundIndex);
	}

	return FoundIndex;
}

void ULocalPlayer::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	ULocalPlayer* This = CastChecked<ULocalPlayer>(InThis);

	for (FSceneViewStateReference& ViewState : This->ViewStates)
	{
		FSceneViewStateInterface* Ref = ViewState.GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}

	This->SubsystemCollection.AddReferencedObjects(This, Collector);

	UPlayer::AddReferencedObjects(This, Collector);
}

bool ULocalPlayer::IsPrimaryPlayer() const
{
	return GetLocalPlayerIndex() == 0;
}

void ULocalPlayer::CleanupViewState(FStringView MidParentRootPath /*= {}*/)
{
	for (FSceneViewStateReference& State : ViewStates)
	{
		FSceneViewStateInterface* Ref = State.GetReference();
		if (Ref)
		{
			Ref->ClearMIDPool(MidParentRootPath);
		}
	}
}

void ULocalPlayer::SetConnectionIdentifier(const int32 Id)
{
	ConnectionIdentifier = Id;
}
	
int32 ULocalPlayer::GetConnectionIdentifier() const
{
	return ConnectionIdentifier;
}