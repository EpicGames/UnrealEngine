// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LocalVertexFactory.cpp: Local vertex factory implementation
=============================================================================*/

#include "LocalVertexFactory.h"
#include "MeshBatch.h"
#include "MeshDrawShaderBindings.h"
#include "SkeletalRenderPublic.h"
#include "SpeedTreeWind.h"
#include "Misc/DelayedAutoRegister.h"
#include "Rendering/ColorVertexBuffer.h"
#include "MaterialDomain.h"
#include "MeshMaterialShader.h"
#include "PrimitiveUniformShaderParameters.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "GPUSkinCache.h"
#include "GPUSkinVertexFactory.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "Rendering/StaticMeshVertexBuffer.h"

IMPLEMENT_TYPE_LAYOUT(FLocalVertexFactoryShaderParametersBase);
IMPLEMENT_TYPE_LAYOUT(FLocalVertexFactoryShaderParameters);

class FSpeedTreeWindNullUniformBuffer : public TUniformBuffer<FSpeedTreeUniformParameters>
{
	typedef TUniformBuffer< FSpeedTreeUniformParameters > Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

void FSpeedTreeWindNullUniformBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FSpeedTreeUniformParameters Parameters;
	FMemory::Memzero(Parameters);
	SetContentsNoUpdate(Parameters);
	
	Super::InitRHI(RHICmdList);
}

static TGlobalResource< FSpeedTreeWindNullUniformBuffer > GSpeedTreeWindNullUniformBuffer;

class FGPUSkinPassThroughFactoryNullUniformBuffer : public TUniformBuffer<FGPUSkinPassThroughFactoryLooseParameters>
{
	typedef TUniformBuffer<FGPUSkinPassThroughFactoryLooseParameters> Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FGPUSkinPassThroughFactoryLooseParameters Parameters;
		Parameters.FrameNumber = -1;
		Parameters.PositionBuffer = GNullVertexBuffer.VertexBufferSRV;
		Parameters.PreviousPositionBuffer = GNullVertexBuffer.VertexBufferSRV;
		Parameters.PreSkinnedTangentBuffer = GNullVertexBuffer.VertexBufferSRV;
		SetContentsNoUpdate(Parameters);
		Super::InitRHI(RHICmdList);
	}
};

static TGlobalResource<FGPUSkinPassThroughFactoryNullUniformBuffer> GGPUSkinPassThroughFactoryNullUniformBuffer;

void FLocalVertexFactoryShaderParametersBase::Bind(const FShaderParameterMap& ParameterMap)
{
	LODParameter.Bind(ParameterMap, TEXT("SpeedTreeLODInfo"));
	bAnySpeedTreeParamIsBound = LODParameter.IsBound() || ParameterMap.ContainsParameterAllocation(TEXT("SpeedTreeData"));
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLocalVertexFactoryUniformShaderParameters, "LocalVF");

FLocalVertexFactoryInstanceDitherParameters MakeNullLocalVertexFactoryInstanceDitherParameters()
{
	FLocalVertexFactoryInstanceDitherParameters InstanceDitherParameters;
	InstanceDitherParameters.InstancingViewZCompareZero = 				FVector4f(MIN_flt, MIN_flt, MAX_flt, 1.0f);
	InstanceDitherParameters.InstancingViewZCompareOne = 				FVector4f(MIN_flt, MIN_flt, MAX_flt, 0.0f);
	InstanceDitherParameters.InstancingViewZConstant = 					FVector4f(ForceInit);
	InstanceDitherParameters.InstancingTranslatedWorldViewOriginZero =	FVector4f(ForceInit);
	InstanceDitherParameters.InstancingTranslatedWorldViewOriginOne = 	FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
	return InstanceDitherParameters;
}

void GetLocalVFUniformShaderParameters(
	FLocalVertexFactoryUniformShaderParameters& UniformParameters,
	const FLocalVertexFactory* LocalVertexFactory, 
	uint32 LODLightmapDataIndex, 
	FColorVertexBuffer* OverrideColorVertexBuffer, 
	int32 BaseVertexIndex,
	int32 PreSkinBaseVertexIndex
	)
{
	UniformParameters.LODLightmapDataIndex = LODLightmapDataIndex;
	int32 ColorIndexMask = 0;

	if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
	{
		UniformParameters.VertexFetch_PositionBuffer = LocalVertexFactory->GetPositionsSRV();
		UniformParameters.VertexFetch_PreSkinPositionBuffer = LocalVertexFactory->GetPreSkinPositionSRV();

		UniformParameters.VertexFetch_PackedTangentsBuffer = LocalVertexFactory->GetTangentsSRV();
		UniformParameters.VertexFetch_TexCoordBuffer = LocalVertexFactory->GetTextureCoordinatesSRV();

		if (OverrideColorVertexBuffer)
		{
			UniformParameters.VertexFetch_ColorComponentsBuffer = OverrideColorVertexBuffer->GetColorComponentsSRV();
			ColorIndexMask = OverrideColorVertexBuffer->GetNumVertices() > 1 ? ~0 : 0;
		}
		else
		{
			UniformParameters.VertexFetch_ColorComponentsBuffer = LocalVertexFactory->GetColorComponentsSRV();
			ColorIndexMask = (int32)LocalVertexFactory->GetColorIndexMask();
		}
	}
	else
	{
        UniformParameters.VertexFetch_PositionBuffer = GNullColorVertexBuffer.VertexBufferSRV;
		UniformParameters.VertexFetch_PreSkinPositionBuffer = GNullColorVertexBuffer.VertexBufferSRV;
		UniformParameters.VertexFetch_PackedTangentsBuffer = GNullColorVertexBuffer.VertexBufferSRV;
		UniformParameters.VertexFetch_TexCoordBuffer = GNullColorVertexBuffer.VertexBufferSRV;
	}

	if (!UniformParameters.VertexFetch_ColorComponentsBuffer)
	{
		UniformParameters.VertexFetch_ColorComponentsBuffer = GNullColorVertexBuffer.VertexBufferSRV;
	}

	const int32 NumTexCoords = LocalVertexFactory->GetNumTexcoords();
	const int32 LightMapCoordinateIndex = LocalVertexFactory->GetLightMapCoordinateIndex();
	const int32 EffectiveBaseVertexIndex = RHISupportsAbsoluteVertexID(GMaxRHIShaderPlatform) ? 0 : BaseVertexIndex;
	const int32 EffectivePreSkinBaseVertexIndex = RHISupportsAbsoluteVertexID(GMaxRHIShaderPlatform) ? 0 : PreSkinBaseVertexIndex;

	UniformParameters.VertexFetch_Parameters = {ColorIndexMask, NumTexCoords, LightMapCoordinateIndex, EffectiveBaseVertexIndex};
	UniformParameters.PreSkinBaseVertexIndex = EffectivePreSkinBaseVertexIndex;
	UniformParameters.bIsGPUSkinPassThrough = LocalVertexFactory->IsGPUSkinPassThrough() ? 1 : 0;
	UniformParameters.InstanceDitherParameters = MakeNullLocalVertexFactoryInstanceDitherParameters();
}

TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters> CreateLocalVFUniformBuffer(
	const FLocalVertexFactory* LocalVertexFactory, 
	uint32 LODLightmapDataIndex, 
	FColorVertexBuffer* OverrideColorVertexBuffer, 
	int32 BaseVertexIndex,
	int32 PreSkinBaseVertexIndex,
	const FLocalVertexFactoryInstanceDitherParameters* OptionalInstanceDitherParameters,
	EUniformBufferUsage UniformBufferUsage 
	)
{
	FLocalVertexFactoryUniformShaderParameters UniformParameters;
	GetLocalVFUniformShaderParameters(UniformParameters, LocalVertexFactory, LODLightmapDataIndex, OverrideColorVertexBuffer, BaseVertexIndex, PreSkinBaseVertexIndex);
	if (OptionalInstanceDitherParameters)
	{
		UniformParameters.InstanceDitherParameters = *OptionalInstanceDitherParameters;
	}
	return TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBufferUsage);
}

void FLocalVertexFactoryShaderParametersBase::GetElementShaderBindingsBase(
	const FSceneInterface* Scene,
	const FSceneView* View,
	const FMeshMaterialShader* Shader, 
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory, 
	const FMeshBatchElement& BatchElement,
	FRHIUniformBuffer* VertexFactoryUniformBuffer,
	FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams
	) const
{
	const auto* LocalVertexFactory = static_cast<const FLocalVertexFactory*>(VertexFactory);
	{
		if (!VertexFactoryUniformBuffer)
		{
			// No batch element override
			VertexFactoryUniformBuffer = LocalVertexFactory->GetUniformBuffer();
		}

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLocalVertexFactoryUniformShaderParameters>(), VertexFactoryUniformBuffer);
	}

	//@todo - allow FMeshBatch to supply vertex streams (instead of requiring that they come from the vertex factory), and this userdata hack will no longer be needed for override vertex color
	if (BatchElement.bUserDataIsColorVertexBuffer)
	{
		FColorVertexBuffer* OverrideColorVertexBuffer = (FColorVertexBuffer*)BatchElement.UserData;
		check(OverrideColorVertexBuffer);

		if (!LocalVertexFactory->SupportsManualVertexFetch(FeatureLevel))
		{
			LocalVertexFactory->GetColorOverrideStream(OverrideColorVertexBuffer, VertexStreams);
		}	
	}

	if (bAnySpeedTreeParamIsBound)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FLocalVertexFactoryShaderParameters_SetMesh_SpeedTree);
		FRHIUniformBuffer* SpeedTreeUniformBuffer = Scene? Scene->GetSpeedTreeUniformBuffer(VertexFactory) : nullptr;
		if (SpeedTreeUniformBuffer == nullptr)
		{
			SpeedTreeUniformBuffer = GSpeedTreeWindNullUniformBuffer.GetUniformBufferRHI();
		}
		check(SpeedTreeUniformBuffer != nullptr);

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FSpeedTreeUniformParameters>(), SpeedTreeUniformBuffer);

		if (LODParameter.IsBound())
		{
			FVector3f LODData(BatchElement.MinScreenSize, BatchElement.MaxScreenSize, BatchElement.MaxScreenSize - BatchElement.MinScreenSize);
			ShaderBindings.Add(LODParameter, LODData);
		}
	}
}

void FLocalVertexFactoryShaderParameters::GetElementShaderBindings(
	const FSceneInterface* Scene,
	const FSceneView* View,
	const FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams
) const
{
	FLocalVertexFactory const* LocalVertexFactory = static_cast<FLocalVertexFactory const*>(VertexFactory);
	if (LocalVertexFactory->bGPUSkinPassThrough)
	{
		const FGPUSkinPassthroughVertexFactory* GPUSkinPassthroughVF = static_cast<const FGPUSkinPassthroughVertexFactory*>(LocalVertexFactory);
		// Bind vertex streams.
		GPUSkinPassthroughVF->GetOverrideVertexStreams(VertexStreams);

		// Bind the vertex factory uniform buffer.
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLocalVertexFactoryUniformShaderParameters>(), LocalVertexFactory->GetUniformBuffer());
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGPUSkinPassThroughFactoryLooseParameters>(), GPUSkinPassthroughVF->LooseParametersUniformBuffer);
	}
	else
	{
		// Decode VertexFactoryUserData as VertexFactoryUniformBuffer
		FRHIUniformBuffer* VertexFactoryUniformBuffer = static_cast<FRHIUniformBuffer*>(BatchElement.VertexFactoryUserData);

		FLocalVertexFactoryShaderParametersBase::GetElementShaderBindingsBase(
			Scene,
			View,
			Shader,
			InputStreamType,
			FeatureLevel,
			VertexFactory,
			BatchElement,
			VertexFactoryUniformBuffer,
			ShaderBindings,
			VertexStreams);

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FGPUSkinPassThroughFactoryLooseParameters>(), GGPUSkinPassThroughFactoryNullUniformBuffer);
	}
}

// CVarIgnoreLVFUsageMode settings:
//     0: never ignore (Default)
//     1: ignored on PC >= SM5
//     2: ignored always
TAutoConsoleVariable<int32> CVarIgnoreLVFUsageMode(
	TEXT("r.Material.IgnoreLVFUsageMode"), 0,
	TEXT("Setting to adjust how UsedWithStaticMesh material flag is interpreted.\n"),
	ECVF_ReadOnly);

static bool ShouldIgnore(const EShaderPlatform Platform)
{
	switch (CVarIgnoreLVFUsageMode.GetValueOnAnyThread())
	{
	case 0:
		return false;
	case 1:
		return IsPCPlatform(Platform) && GetMaxSupportedFeatureLevel(Platform) >= ERHIFeatureLevel::SM5;
	case 2:
		return true;
	}

	return false;
}

FLocalVertexFactory::FLocalVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, const char* InDebugName)
	: FVertexFactory(InFeatureLevel)
	, ColorStreamIndex(INDEX_NONE)
	, DebugName(InDebugName)
{
}

FLocalVertexFactory::~FLocalVertexFactory() = default;

static bool GUseInstancedStaticMeshVertexFactory = false;
FAutoConsoleVariableRef CVarUseInstancedStaticMeshVertexFactory(
	TEXT("r.InstancedStaticMeshes.UseInstancedStaticMeshVertexFactory"),
	GUseInstancedStaticMeshVertexFactory,
	TEXT("If enabled (default is off), all platforms use a separate vertex factory for the instanced static meshes which increases the number of shader permutations significantly.\n")
	TEXT("  When disabled some platforms - notably those without full GPU scene - will still enable the separate ISM VF."),
	ECVF_ReadOnly);

bool DoesPlatformUseInstancedStaticMeshVertexFactory(const FStaticShaderPlatform ShaderPlatform)
{
	const bool bPlatformSupportsFullGPUScene = UseGPUScene(ShaderPlatform) && !PlatformGPUSceneUsesUniformBufferView(ShaderPlatform);
	// If the platform doesn't support full GPU-Scene it must use the separate ISM VF
	return GUseInstancedStaticMeshVertexFactory
		|| !bPlatformSupportsFullGPUScene;
}

/**
 * Should we cache the material's shadertype on this platform with this vertex factory? 
 */
bool FLocalVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	// Allow config forcing of compilation without StaticMesh usage being set.
	if (ShouldIgnore(Parameters.Platform))
	{
		return true;
	}

	// We use the local vertex factory in passthrough mode when using MeshDeformers or when using SkeletalMesh through the SkinCache.
	if ((Parameters.MaterialParameters.bIsUsedWithSkeletalMesh && IsGPUSkinCacheAvailable(Parameters.Platform)) ||
		(Parameters.MaterialParameters.bIsUsedWithMeshDeformer && AreMeshDeformersAvailable(Parameters.Platform)))
	{
		return true;
	}

	// We use the local vertex factory in instanced mode for materials that select InstancedStaticMesh usage when that is supported.
	if (Parameters.MaterialParameters.bIsUsedWithInstancedStaticMeshes && DoesPlatformUseInstancedStaticMeshVertexFactory(Parameters.Platform))
	{
		return true;
	}

	// Otherwise compile when the material has selected StaticMesh usage.
	return Parameters.MaterialParameters.bIsUsedWithStaticMesh == 1;
}

void FLocalVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	// Don't override e.g. SplineMesh's opt-out
	OutEnvironment.SetDefineIfUnset(TEXT("VF_SUPPORTS_SPEEDTREE_WIND"), TEXT("1"));

	if (RHISupportsManualVertexFetch(Parameters.Platform))
	{
		OutEnvironment.SetDefineIfUnset(TEXT("MANUAL_VERTEX_FETCH"), TEXT("1"));
	}

	const bool bVFSupportsPrimtiveSceneData = Parameters.VertexFactoryType->SupportsPrimitiveIdStream() && UseGPUScene(Parameters.Platform);
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bVFSupportsPrimtiveSceneData);

	if (!DoesPlatformUseInstancedStaticMeshVertexFactory(Parameters.Platform))
	{
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), TEXT("1"));
		// Note: is defaulted off, but set explicitly here anyway for clarity
		OutEnvironment.SetDefine(TEXT("IS_INSTANCED_STATIC_MESH_VF"), TEXT("0"));
	}

	// When combining ray tracing and WPO, leave the mesh in local space for consistency with how shading normals are calculated.
	// See UE-139634 for the case that lead to this.
	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_LOCAL_SPACE"), TEXT("1"));

	if (Parameters.VertexFactoryType->SupportsGPUSkinPassThrough())
	{
		OutEnvironment.SetDefine(TEXT("SUPPORT_GPUSKIN_PASSTHROUGH"), IsGPUSkinPassThroughSupported(Parameters.Platform));
	}

	OutEnvironment.SetDefine(TEXT("ALWAYS_EVALUATE_WORLD_POSITION_OFFSET"),
		Parameters.MaterialParameters.bAlwaysEvaluateWorldPositionOffset ? 1 : 0);
}

void FLocalVertexFactory::ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors)
{
	if (Type->SupportsPrimitiveIdStream() 
		&& UseGPUScene(Platform) 
		&& !IsMobilePlatform(Platform) // On mobile VS may use PrimtiveUB while GPUScene is enabled
		&& ParameterMap.ContainsParameterAllocation(FPrimitiveUniformShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
	{
		OutErrors.AddUnique(*FString::Printf(TEXT("Shader attempted to bind the Primitive uniform buffer even though Vertex Factory %s computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(Parameters).Member instead of Primitive.Member."), Type->GetName()));
	}
}

/**
* Return the vertex elements when no vertex declaration is provided during PSO precaching
*/
void FLocalVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	uint8 StreamIndex = 0;
	Elements.Add(FVertexElement(StreamIndex++, 0, VET_Float3, 0, 0, false));

	switch (VertexInputStreamType)
	{
		case EVertexInputStreamType::Default:
		{
			if (!RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
			{
				// Normals
				Elements.Add(FVertexElement(StreamIndex, 0, VET_PackedNormal, 1, 0, false));
				Elements.Add(FVertexElement(StreamIndex++, 4, VET_PackedNormal, 2, 0, false));

				// Color
				Elements.Add(FVertexElement(StreamIndex++, 0, VET_Color, 3, 0, false));

				// TexCoords
				int8 UVStreamIndex = StreamIndex;
				Elements.Add(FVertexElement(StreamIndex, 0, VET_Half4, 4, 0, false));
				Elements.Add(FVertexElement(StreamIndex, 8, VET_Half4, 5, 0, false));
				Elements.Add(FVertexElement(StreamIndex, 16, VET_Half4, 6, 0, false));
				Elements.Add(FVertexElement(StreamIndex++, 24, VET_Half4, 7, 0, false));

				// Pre skin data
				if (IsGPUSkinPassThroughSupported(GMaxRHIShaderPlatform))
				{
					Elements.Add(FVertexElement(StreamIndex++, 0, VET_Float3, 14, 0, false));
				}

				// Light map data
				if (IsStaticLightingAllowed())
				{
					Elements.Add(FVertexElement(UVStreamIndex, 0, VET_Half2, 15, 0, false));
				}
			}
			break;
		}
		case EVertexInputStreamType::PositionOnly:
		{
			break;
		}
		case EVertexInputStreamType::PositionAndNormalOnly:
		{
			// 2-axis TangentBasis components in a single buffer, hence *2u
			Elements.Add(FVertexElement(StreamIndex++, 4, VET_PackedNormal, 2, 0, false));
			break;
		}
		default:
			checkNoEntry();
	}
	
	// Primitive ID stream
	if (UseGPUScene(GMaxRHIShaderPlatform)
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		if (VertexInputStreamType == EVertexInputStreamType::Default)
		{
			Elements.Add(FVertexElement(StreamIndex, 0, VET_UInt, 13, 0, true));
		}
		else
		{
			Elements.Add(FVertexElement(StreamIndex, 0, VET_UInt, 1, 0, true));
		}
	}
}

void FLocalVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, bool bSupportsManualVertexFetch, FDataType& Data, FVertexDeclarationElementList& Elements)
{
	FVertexStreamList VertexStreams;
	int32 ColorStreamIndex;
	GetVertexElements(FeatureLevel, InputStreamType, bSupportsManualVertexFetch, Data, Elements, VertexStreams, ColorStreamIndex);

	if (UseGPUScene(GMaxRHIShaderPlatform) 
		&& !PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{
		if (InputStreamType == EVertexInputStreamType::Default)
		{
			Elements.Add(FVertexElement(VertexStreams.Num(), 0, VET_UInt, 13, 0, true));
		}
		else
		{
			Elements.Add(FVertexElement(VertexStreams.Num(), 0, VET_UInt, 1, 0, true));
		}
	}
}

void FLocalVertexFactory::SetData(const FDataType& InData)
{
	SetData(FRHICommandListImmediate::Get(), InData);
}

void FLocalVertexFactory::SetData(FRHICommandListBase& RHICmdList, const FDataType& InData)
{
	// The shader code makes assumptions that the color component is a FColor, performing swizzles on ES3 and Metal platforms as necessary
	// If the color is sent down as anything other than VET_Color then you'll get an undesired swizzle on those platforms
	check((InData.ColorComponent.Type == VET_None) || (InData.ColorComponent.Type == VET_Color));

	Data = InData;
	UpdateRHI(RHICmdList);
}

/**
* Copy the data from another vertex factory
* @param Other - factory to copy from
*/
void FLocalVertexFactory::Copy(const FLocalVertexFactory& Other)
{
	FLocalVertexFactory* VertexFactory = this;
	const FDataType* DataCopy = &Other.Data;
	ENQUEUE_RENDER_COMMAND(FLocalVertexFactoryCopyData)(
		[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
		{
			VertexFactory->Data = *DataCopy;
		});
	BeginUpdateResourceRHI(this);
}

void FLocalVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FLocalVertexFactory_InitRHI);

	// We create different streams based on feature level
	check(HasValidFeatureLevel());

	// VertexFactory needs to be able to support max possible shader platform and feature level
	// in case if we switch feature level at runtime.
	const bool bCanUseGPUScene = UseGPUScene(GShaderPlatformForFeatureLevel[GetFeatureLevel()]);
	const bool bUseManualVertexFetch = SupportsManualVertexFetch(GetFeatureLevel());

	// If the vertex buffer containing position is not the same vertex buffer containing the rest of the data,
	// then initialize PositionStream and PositionDeclaration.
	if (Data.PositionComponent.VertexBuffer != Data.TangentBasisComponents[0].VertexBuffer)
	{
		auto AddDeclaration = [this](EVertexInputStreamType InputStreamType, bool bAddNormal)
		{
			FVertexDeclarationElementList StreamElements;
			StreamElements.Add(AccessStreamComponent(Data.PositionComponent, 0, InputStreamType));

			bAddNormal = bAddNormal && Data.TangentBasisComponents[1].VertexBuffer != NULL;
			if (bAddNormal)
			{
				StreamElements.Add(AccessStreamComponent(Data.TangentBasisComponents[1], 2, InputStreamType));
			}

			AddPrimitiveIdStreamElement(InputStreamType, StreamElements, 1, 1);

			InitDeclaration(StreamElements, InputStreamType);
		};

		AddDeclaration(EVertexInputStreamType::PositionOnly, false);
		AddDeclaration(EVertexInputStreamType::PositionAndNormalOnly, true);
	}

	FVertexDeclarationElementList Elements;
	GetVertexElements(GetFeatureLevel(), EVertexInputStreamType::Default, bUseManualVertexFetch, Data, Elements, Streams, ColorStreamIndex);
	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, 13, 13);
	check(Streams.Num() > 0);

	InitDeclaration(Elements);
	check(IsValidRef(GetDeclaration()));

	const int32 DefaultBaseVertexIndex = 0;
	const int32 DefaultPreSkinBaseVertexIndex = 0;
	{
		SCOPED_LOADTIMER(FLocalVertexFactory_InitRHI_CreateLocalVFUniformBuffer);
		UniformBuffer = CreateLocalVFUniformBuffer(this, Data.LODLightmapDataIndex, nullptr, DefaultBaseVertexIndex, DefaultPreSkinBaseVertexIndex);
	}

	check(IsValidRef(GetDeclaration()));
}

void AddStaticMeshTextureCoordinateElements(
	int32 BaseTexCoordAttribute, 
	const TArrayView<FVertexStreamComponent>& TextureCoordinates, 
	FVertexDeclarationElementList& Elements,
	FVertexFactory::FVertexStreamList& InOutStreams)
{
	for (int32 CoordinateIndex = 0; CoordinateIndex < TextureCoordinates.Num(); ++CoordinateIndex)
	{
		Elements.Add(FVertexFactory::AccessStreamComponent(
			TextureCoordinates[CoordinateIndex],
			BaseTexCoordAttribute + CoordinateIndex,
			InOutStreams
		));
	}

	for (int32 CoordinateIndex = TextureCoordinates.Num(); CoordinateIndex < MAX_STATIC_TEXCOORDS / 2; ++CoordinateIndex)
	{
		Elements.Add(FVertexFactory::AccessStreamComponent(
			TextureCoordinates[TextureCoordinates.Num() - 1],
			BaseTexCoordAttribute + CoordinateIndex,
			InOutStreams
		));
	}
}

void FLocalVertexFactory::GetVertexElements(
	ERHIFeatureLevel::Type FeatureLevel, 
	EVertexInputStreamType InputStreamType,
	bool bSupportsManualVertexFetch,
	FDataType& Data, 
	FVertexDeclarationElementList& Elements, 
	FVertexStreamList& InOutStreams, 
	int32& OutColorStreamIndex)
{
	check(InputStreamType == EVertexInputStreamType::Default);
	
	if (Data.PositionComponent.VertexBuffer != nullptr)
	{
		Elements.Add(AccessStreamComponent(Data.PositionComponent, 0, InOutStreams));
	}

#if !WITH_EDITOR
	// Can't rely on manual vertex fetch in the editor to not add the unused elements because vertex factories created
	// with manual vertex fetch support can somehow still be used when booting up in for example ES3.1 preview mode
	// The vertex factories are then used during mobile rendering and will cause PSO creation failure.
	// First need to fix invalid usage of these vertex factories before this can be enabled again. (UE-165187)
	if (!bSupportsManualVertexFetch)
#endif // WITH_EDITOR
	{
		// Only the tangent and normal are used by the stream; the bitangent is derived in the shader.
		uint8 TangentBasisAttributes[2] = { 1, 2 };
		for (int32 AxisIndex = 0; AxisIndex < 2; AxisIndex++)
		{
			if (Data.TangentBasisComponents[AxisIndex].VertexBuffer != nullptr)
			{
				Elements.Add(AccessStreamComponent(Data.TangentBasisComponents[AxisIndex], TangentBasisAttributes[AxisIndex], InOutStreams));
			}
		}

		if (Data.ColorComponentsSRV == nullptr)
		{
			Data.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
			Data.ColorIndexMask = 0;
		}

		if (Data.ColorComponent.VertexBuffer)
		{
			Elements.Add(AccessStreamComponent(Data.ColorComponent, 3, InOutStreams));
		}
		else
		{
			// If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
			// This wastes 4 bytes per vertex, but prevents having to compile out twice the number of vertex factories.
			FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
			Elements.Add(AccessStreamComponent(NullColorComponent, 3, InOutStreams));
		}
		OutColorStreamIndex = Elements.Last().StreamIndex;

		if (Data.TextureCoordinates.Num())
		{
			const int32 BaseTexCoordAttribute = 4;
			AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, Data.TextureCoordinates, Elements, InOutStreams);
		}

		// TODO: should also check if VFType supports 'SupportsGPUSkinPassThrough'
		if (IsGPUSkinPassThroughSupported(GMaxRHIShaderPlatform))
		{
			// Fill PreSkinPosition slot for GPUSkinPassThrough vertex factory, or else use a dummy buffer.
			FVertexStreamComponent NullComponent(&GNullColorVertexBuffer, 0, 0, VET_Float3);
			Elements.Add(AccessStreamComponent(Data.PreSkinPositionComponent.VertexBuffer ? Data.PreSkinPositionComponent : NullComponent, 14, InOutStreams));
		}

		if (IsStaticLightingAllowed())
		{
			if (Data.LightMapCoordinateComponent.VertexBuffer)
			{
				Elements.Add(AccessStreamComponent(Data.LightMapCoordinateComponent, 15, InOutStreams));
			}
			else if (Data.TextureCoordinates.Num())
			{
				Elements.Add(AccessStreamComponent(Data.TextureCoordinates[0], 15, InOutStreams));
			}
		}
	}
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLocalVertexFactory, SF_Vertex, FLocalVertexFactoryShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLocalVertexFactory, SF_RayHitGroup, FLocalVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLocalVertexFactory, SF_Compute, FLocalVertexFactoryShaderParameters);
#endif // RHI_RAYTRACING

IMPLEMENT_VERTEX_FACTORY_TYPE(FLocalVertexFactory,"/Engine/Private/LocalVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsPositionOnly
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsLightmapBaking
	| EVertexFactoryFlags::SupportsManualVertexFetch
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsGPUSkinPassThrough
	| EVertexFactoryFlags::SupportsLumenMeshCards
	| EVertexFactoryFlags::SupportsTriangleSorting
);
