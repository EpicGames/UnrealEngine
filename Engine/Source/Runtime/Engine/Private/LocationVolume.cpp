// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationVolume.h"
#include "Engine/World.h"
#include "Components/BrushComponent.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocationVolume)

#if WITH_EDITOR
class ENGINE_API FLoaderAdapterLocationVolumeActor : public FLoaderAdapterActor
{
public:
	FLoaderAdapterLocationVolumeActor(AActor* InActor)
		: FLoaderAdapterActor(InActor)
	{}

	//~ Begin IWorldPartitionActorLoaderInterface::ILoader interface
	virtual TOptional<FColor> GetColor() const override
	{
		return CastChecked<ALocationVolume>(Actor)->DebugColor;
	}
	//~ End IWorldPartitionActorLoaderInterface::ILoader interface
};
#endif

ALocationVolume::ALocationVolume(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	GetBrushComponent()->SetGenerateOverlapEvents(false);
	SetActorEnableCollision(false);

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
	DebugColor = FColor::White;
	bIsAutoLoad = false;	
#endif
}

#if WITH_EDITOR
void ALocationVolume::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (!GetWorld()->IsGameWorld() && GetWorld()->IsPartitionedWorld())
	{
		if (!WorldPartitionActorLoader)
		{
			WorldPartitionActorLoader = new FLoaderAdapterLocationVolumeActor(this);
		}

		if (bIsAutoLoad)
		{
			bIsAutoLoad = false;
			WorldPartitionActorLoader->Load();
		}
	}
}

void ALocationVolume::UnregisterAllComponents(bool bForReregister)
{
	Super::UnregisterAllComponents(bForReregister);

	if (HasActorRegisteredAllComponents() && !bForReregister && WorldPartitionActorLoader)
	{
		bIsAutoLoad = WorldPartitionActorLoader->IsLoaded();
		WorldPartitionActorLoader->Unload();
	}
}

void ALocationVolume::BeginDestroy()
{
	if (WorldPartitionActorLoader)
	{
		delete WorldPartitionActorLoader;
		WorldPartitionActorLoader = nullptr;
	}

	Super::BeginDestroy();
}

IWorldPartitionActorLoaderInterface::ILoaderAdapter* ALocationVolume::GetLoaderAdapter()
{
	return WorldPartitionActorLoader;
}
#endif

void ALocationVolume::Load()
{
#if WITH_EDITOR
	if (WorldPartitionActorLoader)
	{
		WorldPartitionActorLoader->Load();
	}
#endif
}

void ALocationVolume::Unload()
{
#if WITH_EDITOR
	if (WorldPartitionActorLoader)
	{
		WorldPartitionActorLoader->Unload();
	}
#endif
}

bool ALocationVolume::IsLoaded() const
{
#if WITH_EDITOR
	if (WorldPartitionActorLoader)
	{
		return WorldPartitionActorLoader->IsLoaded();
	}
#endif

	return false;
}
