// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
MapBuildData.cpp
=============================================================================*/

#include "LightMap.h"
#include "RenderUtils.h"
#include "UObject/UObjectAnnotation.h"
#include "PrecomputedLightVolume.h"
#include "PrecomputedVolumetricLightmap.h"
#include "StaticMeshComponentLODInfo.h"
#include "Engine/MapBuildDataRegistry.h"
#include "ShadowMap.h"
#include "Stats/StatsTrace.h"
#include "UObject/Package.h"
#include "EngineUtils.h"
#include "Components/ModelComponent.h"
#include "ComponentRecreateRenderStateContext.h"
#include "UObject/MobileObjectVersion.h"
#include "UObject/ReflectionCaptureObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "ContentStreaming.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Interfaces/ITargetPlatform.h"
#if WITH_EDITOR
#include "LandscapeComponent.h"
#include "Components/StaticMeshComponent.h"
#include "UObject/UObjectIterator.h"
#include "VT/LightmapVirtualTexture.h"
#include "AssetCompilingManager.h"
#endif
#include "Engine/TextureCube.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "UnrealEngine.h"
#include "WorldPartition/StaticLightingData/VolumetricLightmapGrid.h"
#include "WorldPartition/ActorInstanceGuids.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "HAL/IConsoleManager.h"

DECLARE_MEMORY_STAT(TEXT("Stationary Light Static Shadowmap"),STAT_StationaryLightBuildData,STATGROUP_MapBuildData);
DECLARE_MEMORY_STAT(TEXT("Reflection Captures"),STAT_ReflectionCaptureBuildData,STATGROUP_MapBuildData);

DEFINE_LOG_CATEGORY(LogMapBuildDataRegistry);

static TAutoConsoleVariable<bool> CVarReflectionCaptureCookBakedCaptures(
	TEXT("r.ReflectionCapture.CookBakedCaptures"),
	true,
	TEXT("When false, disables baked capture data from being added to cooked packages.  Useful with r.ReflectionCapture.Runtime 1")
	TEXT("(which disables rendering of baked captures) to avoid data being inadvertently added to packages (say if an artist mis-tags")
	TEXT("a capture).  For A/B testing between baked and runtime without needing to re-cook, you may want to include the baked data")
	TEXT("anyway, so this is a separate CVar from the runtime toggle."),
	ECVF_ReadOnly);

FArchive& operator<<(FArchive& Ar, FMeshMapBuildData& MeshMapBuildData)
{
	Ar << MeshMapBuildData.LightMap;
	Ar << MeshMapBuildData.ShadowMap;
	Ar << MeshMapBuildData.IrrelevantLights;
	MeshMapBuildData.PerInstanceLightmapData.BulkSerialize(Ar);

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FSkyAtmosphereMapBuildData& Data)
{
	//Ar << Data.Dummy; // No serialisation needed
	return Ar;
}

ULevel* UWorld::GetActiveLightingScenario() const
{
	if (PersistentLevel && PersistentLevel->bIsPartitioned)
	{
		if (PersistentLevel->bIsLightingScenario)
		{
			return PersistentLevel;
		}
	}
	else
	{
		for (int32 LevelIndex = 0; LevelIndex < Levels.Num(); LevelIndex++)
		{
			ULevel* LocalLevel = Levels[LevelIndex];

			if (LocalLevel->bIsVisible && LocalLevel->bIsLightingScenario)
			{
				return LocalLevel;
			}
		}
	}

	return nullptr;
}

void UWorld::PropagateLightingScenarioChange()
{
	for (ULevel* Level : GetLevels())
	{
		Level->ReleaseRenderingResources();
		Level->InitializeRenderingResources();

		for (UModelComponent* ModelComponent : Level->ModelComponents)
		{
			ModelComponent->PropagateLightingScenarioChange();
		}
	}

	TArray<UActorComponent*> WorldComponents;
	for (USceneComponent* Component : TObjectRange<USceneComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		if (Component->GetWorld() == this)
		{
			WorldComponents.Emplace(Component);
		}
	}

	{
		// Use a global context so UpdateAllPrimitiveSceneInfos only runs once, rather than for each component.  Can save minutes of time.
		FGlobalComponentRecreateRenderStateContext Context(WorldComponents);
		
		for (UActorComponent* Component : WorldComponents)
		{
			((USceneComponent*)Component)->PropagateLightingScenarioChange();
		}
	}

	IStreamingManager::Get().PropagateLightingScenarioChange();
}

UMapBuildDataRegistry* CreateRegistryForLegacyMap(ULevel* Level)
{
	static FName RegistryName("MapBuildDataRegistry");
	// Create a new registry for legacy map build data, but put it in the level's package.  
	// This avoids creating a new package during cooking which the cooker won't know about.
	Level->MapBuildData = NewObject<UMapBuildDataRegistry>(Level->GetOutermost(), RegistryName, RF_NoFlags);
	return Level->MapBuildData;
}

void ULevel::HandleLegacyMapBuildData()
{
	if (GComponentsWithLegacyLightmaps.GetAnnotationMap().Num() > 0 
		|| GLevelsWithLegacyBuildData.GetAnnotationMap().Num() > 0
		|| GLightComponentsWithLegacyBuildData.GetAnnotationMap().Num() > 0)
	{
		FLevelLegacyMapBuildData LegacyLevelData = GLevelsWithLegacyBuildData.GetAndRemoveAnnotation(this);

		UMapBuildDataRegistry* Registry = NULL;
		if (LegacyLevelData.Id != FGuid())
		{
			Registry = CreateRegistryForLegacyMap(this);
			Registry->AddLevelPrecomputedLightVolumeBuildData(LegacyLevelData.Id, LegacyLevelData.Data);
		}

		for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ActorIndex++)
		{
			if (!Actors[ActorIndex])
			{
				continue;
			}

			TInlineComponentArray<UActorComponent*> Components;
			Actors[ActorIndex]->GetComponents(Components);

			for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
			{
				UActorComponent* CurrentComponent = Components[ComponentIndex];
				FMeshMapBuildLegacyData LegacyMeshData = GComponentsWithLegacyLightmaps.GetAndRemoveAnnotation(CurrentComponent);

				for (int32 EntryIndex = 0; EntryIndex < LegacyMeshData.Data.Num(); EntryIndex++)
				{
					if (!Registry)
					{
						Registry = CreateRegistryForLegacyMap(this);
					}

					FMeshMapBuildData& DestData = Registry->AllocateMeshBuildData(LegacyMeshData.Data[EntryIndex].Key, false);
					DestData = *LegacyMeshData.Data[EntryIndex].Value;
					delete LegacyMeshData.Data[EntryIndex].Value;
				}

				FLightComponentLegacyMapBuildData LegacyLightData = GLightComponentsWithLegacyBuildData.GetAndRemoveAnnotation(CurrentComponent);

				if (LegacyLightData.Id != FGuid())
				{
					FLightComponentMapBuildData& DestData = Registry->FindOrAllocateLightBuildData(LegacyLightData.Id, false);
					DestData = *LegacyLightData.Data;
					delete LegacyLightData.Data;
				}
			}
		}

		for (UModelComponent* ModelComponent : ModelComponents)
		{
			ModelComponent->PropagateLightingScenarioChange();
			FMeshMapBuildLegacyData LegacyData = GComponentsWithLegacyLightmaps.GetAndRemoveAnnotation(ModelComponent);

			for (int32 EntryIndex = 0; EntryIndex < LegacyData.Data.Num(); EntryIndex++)
			{
				if (!Registry)
				{
					Registry = CreateRegistryForLegacyMap(this);
				}

				FMeshMapBuildData& DestData = Registry->AllocateMeshBuildData(LegacyData.Data[EntryIndex].Key, false);
				DestData = *LegacyData.Data[EntryIndex].Value;
				delete LegacyData.Data[EntryIndex].Value;
			}
		}

		if (MapBuildData)
		{
			MapBuildData->SetupLightmapResourceClusters();
		}
	}

	if (GReflectionCapturesWithLegacyBuildData.GetAnnotationMap().Num() > 0)
	{
		UMapBuildDataRegistry* Registry = MapBuildData;

		for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ActorIndex++)
		{
			if (!Actors[ActorIndex])
			{
				continue;
			}

			TInlineComponentArray<UActorComponent*> Components;
			Actors[ActorIndex]->GetComponents(Components);

			for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
			{
				UActorComponent* CurrentComponent = Components[ComponentIndex];
				UReflectionCaptureComponent* ReflectionCapture = Cast<UReflectionCaptureComponent>(CurrentComponent);

				if (ReflectionCapture)
				{
					FReflectionCaptureMapBuildLegacyData LegacyReflectionData = GReflectionCapturesWithLegacyBuildData.GetAndRemoveAnnotation(ReflectionCapture);

					if (!LegacyReflectionData.IsDefault())
					{
						if (!Registry)
						{
							Registry = CreateRegistryForLegacyMap(this);
						}

						FReflectionCaptureMapBuildData& DestData = Registry->AllocateReflectionCaptureBuildData(LegacyReflectionData.Id, false);
						DestData = *LegacyReflectionData.MapBuildData;
						delete LegacyReflectionData.MapBuildData;
					}
				}
			}
		}

		if (Registry)
		{
			Registry->HandleLegacyEncodedCubemapData();
		}
	}
}

FMeshMapBuildData::FMeshMapBuildData()
{
	ResourceCluster = nullptr;
}

FMeshMapBuildData::~FMeshMapBuildData()
{}

void FMeshMapBuildData::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (LightMap)
	{
		LightMap->AddReferencedObjects(Collector);
	}

	if (ShadowMap)
	{
		ShadowMap->AddReferencedObjects(Collector);
	}
}

void FStaticShadowDepthMapData::Empty()
{
	ShadowMapSizeX = 0;
	ShadowMapSizeY = 0;
	DepthSamples.Empty();
}

FArchive& operator<<(FArchive& Ar, FStaticShadowDepthMapData& ShadowMapData)
{
	Ar << ShadowMapData.WorldToLight;
	Ar << ShadowMapData.ShadowMapSizeX;
	Ar << ShadowMapData.ShadowMapSizeY;
	Ar << ShadowMapData.DepthSamples;

	return Ar;
}

FLightComponentMapBuildData::~FLightComponentMapBuildData()
{
	DEC_DWORD_STAT_BY(STAT_StationaryLightBuildData, DepthMap.GetAllocatedSize());
}

void FLightComponentMapBuildData::FinalizeLoad()
{
	INC_DWORD_STAT_BY(STAT_StationaryLightBuildData, DepthMap.GetAllocatedSize());
}

FArchive& operator<<(FArchive& Ar, FLightComponentMapBuildData& LightBuildData)
{
	Ar << LightBuildData.ShadowMapChannel;
	Ar << LightBuildData.DepthMap;

	if (Ar.IsLoading())
	{
		LightBuildData.FinalizeLoad();
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FReflectionCaptureMapBuildData& ReflectionCaptureMapBuildData)
{
	Ar << ReflectionCaptureMapBuildData.CubemapSize;
	Ar << ReflectionCaptureMapBuildData.AverageBrightness;

	float Brightness = 1.0f;
	if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::StoreReflectionCaptureBrightnessForCooking 
		&& Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ExcludeBrightnessFromEncodedHDRCubemap)
	{
		Ar << Brightness;
	}

	static FName FullHDR("FullHDR");
	static FName EncodedHDR("EncodedHDR");

	TArray<FName> Formats;

	if (Ar.IsCooking())
	{
		// Get all the reflection capture formats that the target platform wants
		Ar.CookingTarget()->GetReflectionCaptureFormats(Formats);
	}

	if (Formats.Num() == 0 || Formats.Contains(FullHDR))
	{
		Ar << ReflectionCaptureMapBuildData.FullHDRCapturedData;
	}
	else
	{
		TArray<uint8> StrippedData;
		Ar << StrippedData;
	}

	if (Ar.CustomVer(FMobileObjectVersion::GUID) >= FMobileObjectVersion::StoreReflectionCaptureCompressedMobile
		&& Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::StoreReflectionCaptureEncodedHDRDataInRG11B10Format)
	{
		UTextureCube* EncodedCaptureData = nullptr;
		Ar << EncodedCaptureData;
	}
	else
	{
		if ((Formats.Num() == 0 || Formats.Contains(EncodedHDR))
			&& Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::StoreReflectionCaptureEncodedHDRDataInRG11B10Format)
		{
			Ar << ReflectionCaptureMapBuildData.EncodedHDRCapturedData;
		}
		else
		{
			TArray<uint8> StrippedData;
			Ar << StrippedData;
		}
	}

	if (Ar.IsLoading())
	{
		ReflectionCaptureMapBuildData.FinalizeLoad();
	}

	return Ar;
}

FReflectionCaptureMapBuildData::~FReflectionCaptureMapBuildData()
{
	DEC_DWORD_STAT_BY(STAT_ReflectionCaptureBuildData, AllocatedSize);
}

void FReflectionCaptureMapBuildData::FinalizeLoad()
{
	AllocatedSize = FullHDRCapturedData.GetAllocatedSize() + EncodedHDRCapturedData.GetAllocatedSize();
	INC_DWORD_STAT_BY(STAT_ReflectionCaptureBuildData, AllocatedSize);

	bool bMobileEnableClusteredReflections = MobileForwardEnableClusteredReflections(GMaxRHIShaderPlatform) || IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform);
	bool bEncodedDataRequired = (GIsEditor || (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1 && !bMobileEnableClusteredReflections));
	// If the RG11B10 format is not really supported, decode it to RGBA16F 
	if (GPixelFormats[PF_FloatR11G11B10].BlockBytes == 8 && bEncodedDataRequired && EncodedHDRCapturedData.Num() > 0)
	{
		const int32 NumMips = FMath::CeilLogTwo(CubemapSize) + 1;

		int32 SourceMipBaseIndex = 0;
		int32 DestMipBaseIndex = 0;

		TArray<uint8> DecodedHDRData;

		int32 DecodedDataSize = EncodedHDRCapturedData.Num() * sizeof(FFloat16Color) / sizeof(FFloat3Packed);

		DecodedHDRData.Empty(DecodedDataSize);
		DecodedHDRData.AddZeroed(DecodedDataSize);

		for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
		{
			const int32 MipSize = 1 << (NumMips - MipIndex - 1);
			const int32 SourceCubeFaceBytes = MipSize * MipSize * sizeof(FFloat3Packed);
			const int32 DestCubeFaceBytes = MipSize * MipSize * sizeof(FFloat16Color);

			// Decode rest of texels
			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				const int32 FaceSourceIndex = SourceMipBaseIndex + CubeFace * SourceCubeFaceBytes;
				const int32 FaceDestIndex = DestMipBaseIndex + CubeFace * DestCubeFaceBytes;
				const FFloat3Packed* FaceSourceData = (const FFloat3Packed*)&EncodedHDRCapturedData[FaceSourceIndex];
				FFloat16Color* FaceDestData = (FFloat16Color*)&DecodedHDRData[FaceDestIndex];

				// Convert each texel from RG11B10 to linear space FP16 FColor
				for (int32 y = 0; y < MipSize; y++)
				{
					for (int32 x = 0; x < MipSize; x++)
					{
						int32 TexelIndex = x + y * MipSize;
						FaceDestData[TexelIndex] = FFloat16Color(FaceSourceData[TexelIndex].ToLinearColor());
					}
				}
			}

			SourceMipBaseIndex += SourceCubeFaceBytes * CubeFace_MAX;
			DestMipBaseIndex += DestCubeFaceBytes * CubeFace_MAX;
		}

		EncodedHDRCapturedData = MoveTemp(DecodedHDRData);
	}
}

void FReflectionCaptureMapBuildData::AddReferencedObjects(FReferenceCollector& Collector)
{

}

UMapBuildDataRegistry::UMapBuildDataRegistry(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LevelLightingQuality = Quality_MAX;
	bSetupResourceClusters = false;
	VolumetricLightMapGridDesc = nullptr;


#if WITH_EDITOR
	FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddUObject(this, &ThisClass::HandleAssetPostCompileEvent);
#endif
}

#if WITH_EDITOR
void UMapBuildDataRegistry::HandleAssetPostCompileEvent(const TArray<FAssetCompileData>& CompiledAssets)
{
	TSet<FLightmapResourceCluster*> ClustersToUpdate;
	for (const FAssetCompileData& CompileData : CompiledAssets)
	{
		if (ULightMapVirtualTexture2D* LightMapVirtualTexture2D = Cast<ULightMapVirtualTexture2D>(CompileData.Asset.Get()))
		{
			// If our lightmap clusters are affected by the virtual textures that just finished compiling, 
			// we need to update their uniform buffer.
			for (FLightmapResourceCluster& Cluster : LightmapResourceClusters)
			{
				if (Cluster.Input.LightMapVirtualTextures[0] == LightMapVirtualTexture2D ||
					Cluster.Input.LightMapVirtualTextures[1] == LightMapVirtualTexture2D)
				{
					ClustersToUpdate.Add(&Cluster);
				}
			}
		}
	}

	if (ClustersToUpdate.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UMapBuildDataRegistry::HandleAssetPostCompileEvent);

		ENQUEUE_RENDER_COMMAND(UpdateClusterUniformBuffer)(
			[ClustersToUpdate = ClustersToUpdate](FRHICommandList& RHICmdList)
			{
				for (FLightmapResourceCluster* Cluster : ClustersToUpdate)
				{
					Cluster->UpdateRHI(RHICmdList);
				}
			});

		for (TObjectIterator<ULandscapeComponent> It; It; ++It)
		{
			if (It->IsRenderStateCreated() && It->SceneProxy != nullptr)
			{
				if (FMeshMapBuildData* BuildData = MeshBuildData.Find(It->MapBuildDataId))
				{
					if (ClustersToUpdate.Contains(BuildData->ResourceCluster))
					{
						It->MarkRenderStateDirty();
					}
				}
			}
		}

		for (TObjectIterator<UStaticMeshComponent> It; It; ++It)
		{
			if (It->IsRenderStateCreated() && It->SceneProxy != nullptr)
			{
				for (const FStaticMeshComponentLODInfo& LODInfo : It->LODData)
				{
					if (FMeshMapBuildData* BuildData = MeshBuildData.Find(LODInfo.MapBuildDataId))
					{
						if (ClustersToUpdate.Contains(BuildData->ResourceCluster))
						{
							It->MarkRenderStateDirty();
							break;
						}
					}
				}
			}
		}
	}

}
#endif // #if WITH_EDITOR

void UMapBuildDataRegistry::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	FStripDataFlags StripFlags(Ar, 0);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FMobileObjectVersion::GUID);
	Ar.UsingCustomVersion(FReflectionCaptureObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (!StripFlags.IsAudioVisualDataStripped())
	{
		Ar << MeshBuildData;
		Ar << LevelPrecomputedLightVolumeBuildData;

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::VolumetricLightmaps)
		{
			Ar << LevelPrecomputedVolumetricLightmapBuildData;
		}

		Ar << LightBuildData;

		// When cooking, optionally strip baked cubemap data.  When CVarReflectionCaptureCookBakedCaptures is false, all baked
		// data is stripped, otherwise baked data is stripped only from runtime-flagged captures (where it wouldn't be used).
		// Data is stashed rather than dropped so in-memory state can be preserved for the editor.
		TMap<FGuid, FReflectionCaptureMapBuildData> StashedCaptureData;
		if (Ar.IsCooking() && Ar.IsSaving())
		{
			bool bAllowBakedCaptures = CVarReflectionCaptureCookBakedCaptures.GetValueOnAnyThread();

			if (!bAllowBakedCaptures)
			{
				// Don't include any baked capture data, assuming runtime captures are to be used.  Stash everything.
				StashedCaptureData = MoveTemp(ReflectionCaptureBuildData);
				ReflectionCaptureBuildData.Empty();
			}
			else
			{
				// Stash data only for runtime-flagged captures.
				for (TObjectIterator<UReflectionCaptureComponent> It; It; ++It)
				{
					if (It->IsRuntimeCapture())
					{
						const FGuid& CaptureId = It->MapBuildDataId;
						if (FReflectionCaptureMapBuildData* Data = ReflectionCaptureBuildData.Find(CaptureId))
						{
							StashedCaptureData.Add(CaptureId, MoveTemp(*Data));
							ReflectionCaptureBuildData.Remove(CaptureId);
						}
					}
				}
			}
		}

		if (Ar.IsSaving())
		{
			for (TMap<FGuid, FReflectionCaptureMapBuildData>::TIterator It(ReflectionCaptureBuildData); It; ++It)
			{
				const FReflectionCaptureMapBuildData& CaptureBuildData = It.Value();
				// Sanity check that every reflection capture entry has valid data for at least one format
				check(CaptureBuildData.FullHDRCapturedData.Num() > 0 || CaptureBuildData.EncodedHDRCapturedData.Num() > 0);
			}
		}

		if (Ar.CustomVer(FReflectionCaptureObjectVersion::GUID) >= FReflectionCaptureObjectVersion::MoveReflectionCaptureDataToMapBuildData)
		{
			Ar << ReflectionCaptureBuildData;
		}

		// Restore stashed capture data so in-memory state is preserved for the editor.
		for (auto& Pair : StashedCaptureData)
		{
			ReflectionCaptureBuildData.Add(Pair.Key, MoveTemp(Pair.Value));
		}
		
		if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::SkyAtmosphereStaticLightingVersioning)
		{
			Ar << SkyAtmosphereBuildData;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::VolumetricLightMapGridDescSupport)
		{			
			bool bHasGrid = VolumetricLightMapGridDesc != nullptr;
			Ar << bHasGrid;

			if (bHasGrid)
			{
				// Create the grid when loading for the 1st time
				if (!VolumetricLightMapGridDesc)
				{
					VolumetricLightMapGridDesc = new FVolumetricLightMapGridDesc();
				}

				FVolumetricLightMapGridDesc::StaticStruct()->SerializeItem(Ar, VolumetricLightMapGridDesc, nullptr);
				VolumetricLightMapGridDesc->SerializeBulkData(Ar, this);
			}
		}
	}

#if UE_LOG_MAPBUILDATA_ENABLED 

	UE_LOG_MAPBUILDDATA(Log, TEXT("Loaded Registry %s"), *GetFullName());
	for (auto& It : MeshBuildData)
	{
		UE_LOG_MAPBUILDDATA(Log, TEXT("    => Mesh GUID : %s"), *It.Key.ToString());
	}

	for (auto& It : LightBuildData)
	{
		UE_LOG_MAPBUILDDATA(Log, TEXT("    => Light GUID : %s"), *It.Key.ToString());
	}
#endif
}

void UMapBuildDataRegistry::PostLoad()
{
	Super::PostLoad();
	bool bMobileEnableClusteredReflections = MobileForwardEnableClusteredReflections(GMaxRHIShaderPlatform) || IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform);
	bool bFullDataRequired = GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5 || bMobileEnableClusteredReflections;
	bool bEncodedDataRequired = (GIsEditor || (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1 && !bMobileEnableClusteredReflections));

	HandleLegacyEncodedCubemapData();

	if (ReflectionCaptureBuildData.Num() > 0
		// Only strip in PostLoad for cooked platforms.  Uncooked may need to generate encoded HDR data in UReflectionCaptureComponent::OnRegister().
		&& FPlatformProperties::RequiresCookedData())
	{
		// We expect to use only one type of data at cooked runtime
		check(bFullDataRequired != bEncodedDataRequired);

		for (TMap<FGuid, FReflectionCaptureMapBuildData>::TIterator It(ReflectionCaptureBuildData); It; ++It)
		{
			FReflectionCaptureMapBuildData& CaptureBuildData = It.Value();

			if (!bFullDataRequired)
			{
				CaptureBuildData.FullHDRCapturedData.Empty();
			}

			if (!bEncodedDataRequired)
			{
				CaptureBuildData.EncodedHDRCapturedData.Empty();
			}

			check(CaptureBuildData.EncodedHDRCapturedData.Num() > 0 || CaptureBuildData.FullHDRCapturedData.Num() > 0 || FApp::CanEverRender() == false);
		}
	}

	SetupLightmapResourceClusters();
}

#if WITH_EDITORONLY_DATA
void UMapBuildDataRegistry::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UTextureCube::StaticClass()));
}
#endif

void UMapBuildDataRegistry::HandleLegacyEncodedCubemapData()
{
#if WITH_EDITOR
	bool bUsesMobileDeferredShading = IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform);
	bool bEncodedDataRequired = (GIsEditor || (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1 && !bUsesMobileDeferredShading));

	if (ReflectionCaptureBuildData.Num() > 0 && bEncodedDataRequired)
	{
		for (TMap<FGuid, FReflectionCaptureMapBuildData>::TIterator It(ReflectionCaptureBuildData); It; ++It)
		{
			FReflectionCaptureMapBuildData& CaptureBuildData = It.Value();
			if (CaptureBuildData.EncodedHDRCapturedData.Num() == 0 && CaptureBuildData.FullHDRCapturedData.Num() != 0)
			{
				GenerateEncodedHDRData(CaptureBuildData.FullHDRCapturedData, CaptureBuildData.CubemapSize, CaptureBuildData.EncodedHDRCapturedData);
			}
		}
	}
#endif
}

void UMapBuildDataRegistry::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMapBuildDataRegistry* TypedThis = Cast<UMapBuildDataRegistry>(InThis);
	check(TypedThis);

	for (TMap<FGuid, FMeshMapBuildData>::TIterator It(TypedThis->MeshBuildData); It; ++It)
	{
		It.Value().AddReferencedObjects(Collector);
	}

	for (TMap<FGuid, FReflectionCaptureMapBuildData>::TIterator It(TypedThis->ReflectionCaptureBuildData); It; ++It)
	{
		It.Value().AddReferencedObjects(Collector);
	}
}

void UMapBuildDataRegistry::BeginDestroy()
{
	Super::BeginDestroy();

	ReleaseResources();

	// Start a fence to track when BeginReleaseResource has completed
	DestroyFence.BeginFence();
}

bool UMapBuildDataRegistry::IsReadyForFinishDestroy()
{
	return Super::IsReadyForFinishDestroy() && DestroyFence.IsFenceComplete();
}

void UMapBuildDataRegistry::FinishDestroy()
{
	Super::FinishDestroy();

	EmptyLevelData();
}

FMeshMapBuildData& UMapBuildDataRegistry::AllocateMeshBuildData(const FGuid& MeshId, bool bMarkDirty)
{
	check(MeshId.IsValid());
	check(!bSetupResourceClusters);

	UE_LOG_MAPBUILDDATA(Log, TEXT("Allocating MeshBuildData in Registry %s for Guid: %s"), *GetFullName(), *MeshId.ToString());

	if (bMarkDirty)
	{
		MarkPackageDirty();
	}

	return MeshBuildData.Add(MeshId, FMeshMapBuildData());
}

const FMeshMapBuildData* UMapBuildDataRegistry::GetMeshBuildData(FGuid MeshId) const
{
	const FMeshMapBuildData* FoundData = MeshBuildData.Find(MeshId);

	if (FoundData && !FoundData->ResourceCluster)
	{
		// Don't expose a FMeshMapBuildData to the renderer which hasn't had its ResourceCluster setup yet
		// This can happen during lighting build completion, before the clusters have been assigned.
		return nullptr;
	}

	UE_LOG_MAPBUILDDATA(Log, TEXT("Finding MeshBuildData (%p) in Registry %s for Guid: %s"), FoundData, *GetFullName(), *MeshId.ToString());
	return FoundData;
}

FMeshMapBuildData* UMapBuildDataRegistry::GetMeshBuildData(FGuid MeshId)
{
	FMeshMapBuildData* FoundData = MeshBuildData.Find(MeshId);

	if (FoundData && !FoundData->ResourceCluster)
	{
		return nullptr;
	}

	UE_LOG_MAPBUILDDATA(Log, TEXT("Finding MeshBuildData (%p) in Registry %s for Guid: %s"), FoundData, *GetFullName(), *MeshId.ToString());
	return FoundData;
}

FMeshMapBuildData* UMapBuildDataRegistry::GetMeshBuildDataDuringBuild(FGuid MeshId)
{
	return MeshBuildData.Find(MeshId);
}

FPrecomputedLightVolumeData& UMapBuildDataRegistry::AllocateLevelPrecomputedLightVolumeBuildData(const FGuid& LevelId)
{
	check(LevelId.IsValid());
	MarkPackageDirty();
	return *LevelPrecomputedLightVolumeBuildData.Add(LevelId, new FPrecomputedLightVolumeData());
}

void UMapBuildDataRegistry::AddLevelPrecomputedLightVolumeBuildData(const FGuid& LevelId, FPrecomputedLightVolumeData* InData)
{
	check(LevelId.IsValid());
	LevelPrecomputedLightVolumeBuildData.Add(LevelId, InData);
}

const FPrecomputedLightVolumeData* UMapBuildDataRegistry::GetLevelPrecomputedLightVolumeBuildData(FGuid LevelId) const
{
	const FPrecomputedLightVolumeData* const * DataPtr = LevelPrecomputedLightVolumeBuildData.Find(LevelId);

	if (DataPtr)
	{
		return *DataPtr;
	}

	return NULL;
}

FPrecomputedLightVolumeData* UMapBuildDataRegistry::GetLevelPrecomputedLightVolumeBuildData(FGuid LevelId)
{
	FPrecomputedLightVolumeData** DataPtr = LevelPrecomputedLightVolumeBuildData.Find(LevelId);

	if (DataPtr)
	{
		return *DataPtr;
	}

	return NULL;
}

FPrecomputedVolumetricLightmapData& UMapBuildDataRegistry::AllocateLevelPrecomputedVolumetricLightmapBuildData(const FGuid& LevelId)
{
	if (VolumetricLightMapGridDesc && VolumetricLightMapGridDesc->GetCell(LevelId) )
	{
		FPrecomputedVolumetricLightmapData* DataPtr = VolumetricLightMapGridDesc->GetOrCreatePrecomputedVolumetricLightmapBuildData(LevelId);		
		return *DataPtr;	
	}
	
	check(LevelId.IsValid());
	MarkPackageDirty();
	return *LevelPrecomputedVolumetricLightmapBuildData.Add(LevelId, new FPrecomputedVolumetricLightmapData());
}

void UMapBuildDataRegistry::AddLevelPrecomputedVolumetricLightmapBuildData(const FGuid& LevelId, FPrecomputedVolumetricLightmapData* InData)
{
	check(LevelId.IsValid());
	LevelPrecomputedVolumetricLightmapBuildData.Add(LevelId, InData);
}

const FPrecomputedVolumetricLightmapData* UMapBuildDataRegistry::GetLevelPrecomputedVolumetricLightmapBuildData(FGuid LevelId) const
{
	if (VolumetricLightMapGridDesc)
	{
		const FPrecomputedVolumetricLightmapData* DataPtr = VolumetricLightMapGridDesc->GetPrecomputedVolumetricLightmapBuildData(LevelId);		
		return DataPtr;
	}

	const FPrecomputedVolumetricLightmapData* const * DataPtr = LevelPrecomputedVolumetricLightmapBuildData.Find(LevelId);

	if (DataPtr)
	{
		return *DataPtr;
	}

	return NULL;
}

FPrecomputedVolumetricLightmapData* UMapBuildDataRegistry::GetLevelPrecomputedVolumetricLightmapBuildData(FGuid LevelId)
{
	if (VolumetricLightMapGridDesc)
	{
		FPrecomputedVolumetricLightmapData* DataPtr = VolumetricLightMapGridDesc->GetPrecomputedVolumetricLightmapBuildData(LevelId);		
		
		if (DataPtr)
		{
			return DataPtr;
		}
	}

	FPrecomputedVolumetricLightmapData** DataPtr = LevelPrecomputedVolumetricLightmapBuildData.Find(LevelId);

	if (DataPtr)
	{
		return *DataPtr;
	}

	return NULL;
}

FLightComponentMapBuildData& UMapBuildDataRegistry::FindOrAllocateLightBuildData(FGuid LightId, bool bMarkDirty)
{
	check(LightId.IsValid());

	if (bMarkDirty)
	{
		MarkPackageDirty();
	}

	UE_LOG_MAPBUILDDATA(Log, TEXT("Allocating LightBuildData in Registry %s for Guid: %s"), *GetFullName(), *LightId.ToString());

	return LightBuildData.FindOrAdd(LightId);
}

const FLightComponentMapBuildData* UMapBuildDataRegistry::GetLightBuildData(FGuid LightId) const
{
	UE_LOG_MAPBUILDDATA(Log, TEXT("Finding LightBuildData (%p) in Registry %s for Guid: %s"), LightBuildData.Find(LightId), *GetFullName(), *LightId.ToString());
	return LightBuildData.Find(LightId);
}

FLightComponentMapBuildData* UMapBuildDataRegistry::GetLightBuildData(FGuid LightId)
{
	UE_LOG_MAPBUILDDATA(Log, TEXT("Finding LightBuildData (%p) in Registry %s for Guid: %s"), LightBuildData.Find(LightId), *GetFullName(), *LightId.ToString());
	return LightBuildData.Find(LightId);
}

FReflectionCaptureMapBuildData& UMapBuildDataRegistry::AllocateReflectionCaptureBuildData(const FGuid& CaptureId, bool bMarkDirty)
{
	check(CaptureId.IsValid());

	if (bMarkDirty)
	{
		MarkPackageDirty();
	}

	return ReflectionCaptureBuildData.Add(CaptureId, FReflectionCaptureMapBuildData());
}

const FReflectionCaptureMapBuildData* UMapBuildDataRegistry::GetReflectionCaptureBuildData(FGuid CaptureId) const
{
	return ReflectionCaptureBuildData.Find(CaptureId);
}

FReflectionCaptureMapBuildData* UMapBuildDataRegistry::GetReflectionCaptureBuildData(FGuid CaptureId)
{
	return ReflectionCaptureBuildData.Find(CaptureId);
}

FSkyAtmosphereMapBuildData& UMapBuildDataRegistry::FindOrAllocateSkyAtmosphereBuildData(const FGuid& Guid)
{
	check(Guid.IsValid());
	return SkyAtmosphereBuildData.FindOrAdd(Guid);
}

const FSkyAtmosphereMapBuildData* UMapBuildDataRegistry::GetSkyAtmosphereBuildData(const FGuid& Guid) const
{
	check(Guid.IsValid());
	return SkyAtmosphereBuildData.Find(Guid);
}

void UMapBuildDataRegistry::ClearSkyAtmosphereBuildData()
{
	SkyAtmosphereBuildData.Empty();
}

void UMapBuildDataRegistry::InvalidateStaticLighting(UWorld* World, bool bRecreateRenderState, const TSet<FGuid>* ResourcesToKeep)
{
	TUniquePtr<FGlobalComponentRecreateRenderStateContext> RecreateContext;

	if (bRecreateRenderState)
	{
		// Warning: if skipping this, caller is responsible for unregistering any components potentially referencing this UMapBuildDataRegistry before we change its contents!
		RecreateContext = TUniquePtr<FGlobalComponentRecreateRenderStateContext>(new FGlobalComponentRecreateRenderStateContext);
	}

	InvalidateSurfaceLightmaps(World, false, ResourcesToKeep);

	if (LevelPrecomputedLightVolumeBuildData.Num() > 0 || LevelPrecomputedVolumetricLightmapBuildData.Num() > 0 || LightmapResourceClusters.Num() > 0)
	{
		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			World->GetLevel(LevelIndex)->ReleaseRenderingResources();
		}

		ReleaseResources(ResourcesToKeep);

		// Make sure the RT has processed the release command before we delete any FPrecomputedLightVolume's
		FlushRenderingCommands();

		EmptyLevelData(ResourcesToKeep);

		MarkPackageDirty();
	}

	// Clear all the atmosphere guids from the MapBuildData when starting a new build.
	ClearSkyAtmosphereBuildData();

	bSetupResourceClusters = false;
}

void UMapBuildDataRegistry::InvalidateSurfaceLightmaps(UWorld* World, bool bRecreateRenderState, const TSet<FGuid>* ResourcesToKeep)
{
	TUniquePtr<FGlobalComponentRecreateRenderStateContext> RecreateContext;

	if (bRecreateRenderState)
	{
		// Warning: if skipping this, caller is responsible for unregistering any components potentially referencing this UMapBuildDataRegistry before we change its contents!
		RecreateContext = TUniquePtr<FGlobalComponentRecreateRenderStateContext>(new FGlobalComponentRecreateRenderStateContext);
	}

	if (MeshBuildData.Num() > 0 || LightBuildData.Num() > 0)
	{
		if (!ResourcesToKeep || !ResourcesToKeep->Num())
		{
			MeshBuildData.Empty();
			LightBuildData.Empty();
		}
		else // Otherwise keep any resource if it's guid is in ResourcesToKeep.
		{
			TMap<FGuid, FMeshMapBuildData> PrevMeshData;
			TMap<FGuid, FLightComponentMapBuildData> PrevLightData;
			Swap(MeshBuildData, PrevMeshData);
			Swap(LightBuildData, PrevLightData);

			for (const FGuid& Guid : *ResourcesToKeep)
			{
				const FMeshMapBuildData* MeshData = PrevMeshData.Find(Guid);
				if (MeshData)
				{
					MeshBuildData.Add(Guid, *MeshData);
					continue;
				}

				const FLightComponentMapBuildData* LightData = PrevLightData.Find(Guid);
				if (LightData)
				{
					LightBuildData.Add(Guid, *LightData);
					continue;
				}
			}
		}

		// Invalidate the LightmapResourceClusters
		{
			// LightmapResourceClusters needs to be cleared at RT to avoid a flush in GT because the RenderResource in a ResourceCluster needs to be released before the destructor of FLightmapResourceCluster
			ENQUEUE_RENDER_COMMAND(FReleaseLightmapResourceClustersCmd)(
				[LocalLightmapResourceClusters = MoveTemp(LightmapResourceClusters)](FRHICommandListImmediate& RHICmdList) mutable
			{
				for (auto& ResourceCluster : LocalLightmapResourceClusters)
				{
					ResourceCluster.ReleaseResource();
				}
			});

			LightmapResourceClusters.Empty();
		}

		MarkPackageDirty();
	}
}

void UMapBuildDataRegistry::InvalidateReflectionCaptures(const TSet<FGuid>* ResourcesToKeep)
{
	if (ReflectionCaptureBuildData.Num() > 0)
	{
		// Warning: caller is responsible for unregistering any components potentially referencing this UMapBuildDataRegistry before we change its contents!

		TMap<FGuid, FReflectionCaptureMapBuildData> PrevReflectionCapturedData;
		Swap(ReflectionCaptureBuildData, PrevReflectionCapturedData);

		for (TMap<FGuid, FReflectionCaptureMapBuildData>::TIterator It(PrevReflectionCapturedData); It; ++It)
		{
			// Keep any resource if it's guid is in ResourcesToKeep.
			if (ResourcesToKeep && ResourcesToKeep->Contains(It.Key()))
			{
				ReflectionCaptureBuildData.Add(It.Key(), It.Value());
			}
		}

		MarkPackageDirty();
	}
}

bool UMapBuildDataRegistry::IsLegacyBuildData() const
{
	return GetOutermost()->ContainsMap();
}

bool UMapBuildDataRegistry::IsLightingValid(ERHIFeatureLevel::Type InFeatureLevel) const
{
	if (MeshBuildData.Num() == 0)
	{
		return LevelPrecomputedLightVolumeBuildData.Num() > 0 || LevelPrecomputedVolumetricLightmapBuildData.Num() > 0;
	}
	else
	{
		const bool bUsingVTLightmaps = UseVirtualTextureLightmap(GetFeatureLevelShaderPlatform(InFeatureLevel));

		// this code checks if AT LEAST 1 virtual textures is valid. 
		for (auto MeshBuildDataPair : MeshBuildData)
		{
			const FMeshMapBuildData& Data = MeshBuildDataPair.Value;
			if (/*Data.IsDefault() == false &&*/ Data.LightMap.IsValid())
			{
				const FLightMap2D* Lightmap2D = Data.LightMap->GetLightMap2D();
				if (Lightmap2D)
				{
					if ((bUsingVTLightmaps && Lightmap2D->IsVirtualTextureValid()) || (!bUsingVTLightmaps && (Lightmap2D->IsValid(0) || Lightmap2D->IsValid(1))))
					{
						return true;
					}
				}
			}
		}
		return false;
	}
}

FLightmapClusterResourceInput GetClusterInput(const FMeshMapBuildData& MeshBuildData)
{
	FLightmapClusterResourceInput ClusterInput;

	FLightMap2D* LightMap2D = MeshBuildData.LightMap ? MeshBuildData.LightMap->GetLightMap2D() : nullptr;

	if (LightMap2D)
	{
		ClusterInput.LightMapTextures[0] = LightMap2D->GetTexture(0);
		ClusterInput.LightMapTextures[1] = LightMap2D->GetTexture(1);
		ClusterInput.SkyOcclusionTexture = LightMap2D->GetSkyOcclusionTexture();
		ClusterInput.AOMaterialMaskTexture = LightMap2D->GetAOMaterialMaskTexture();
		ClusterInput.LightMapVirtualTextures[0] = LightMap2D->GetVirtualTexture(0);
		ClusterInput.LightMapVirtualTextures[1] = LightMap2D->GetVirtualTexture(1);
	}

	FShadowMap2D* ShadowMap2D = MeshBuildData.ShadowMap ? MeshBuildData.ShadowMap->GetShadowMap2D() : nullptr;
		
	if (ShadowMap2D)
	{
		ClusterInput.ShadowMapTexture = ShadowMap2D->GetTexture();
	}

	return ClusterInput;
}

void UMapBuildDataRegistry::SetupLightmapResourceClusters()
{
	if (!bSetupResourceClusters)
	{
		bSetupResourceClusters = true;

		QUICK_SCOPE_CYCLE_COUNTER(STAT_UMapBuildDataRegistry_SetupLightmapResourceClusters);

		TSet<FLightmapClusterResourceInput> LightmapClusters;
		LightmapClusters.Empty(1 + MeshBuildData.Num() / 30);

		// Build resource clusters from MeshBuildData
		for (TMap<FGuid, FMeshMapBuildData>::TIterator It(MeshBuildData); It; ++It)
		{
			const FMeshMapBuildData& Data = It.Value();
			LightmapClusters.Add(GetClusterInput(Data));
		}

		LightmapResourceClusters.Empty(LightmapClusters.Num());
		LightmapResourceClusters.AddDefaulted(LightmapClusters.Num());

		// Assign ResourceCluster to FMeshMapBuildData
		for (TMap<FGuid, FMeshMapBuildData>::TIterator It(MeshBuildData); It; ++It)
		{
			FMeshMapBuildData& Data = It.Value();
			const FLightmapClusterResourceInput ClusterInput = GetClusterInput(Data);
			const FSetElementId ClusterId = LightmapClusters.FindId(ClusterInput);
			check(ClusterId.IsValidId());
			const int32 ClusterIndex = ClusterId.AsInteger();
			LightmapResourceClusters[ClusterIndex].Input = ClusterInput;
			Data.ResourceCluster = &LightmapResourceClusters[ClusterIndex];
		}

		// Init empty cluster uniform buffers so they can be referenced by cached mesh draw commands.
		// Can't create final uniform buffers as feature level is unknown at this point.
		for (FLightmapResourceCluster& Cluster : LightmapResourceClusters)
		{
			BeginInitResource(&Cluster);
		}
	}
}

void UMapBuildDataRegistry::GetLightmapResourceClusterStats(int32& NumMeshes, int32& NumClusters) const
{
	check(bSetupResourceClusters);
	NumMeshes = MeshBuildData.Num();
	NumClusters = LightmapResourceClusters.Num();
}

void UMapBuildDataRegistry::InitializeClusterRenderingResources(ERHIFeatureLevel::Type InFeatureLevel)
{
	// Resource clusters should have been setup during PostLoad, however the cooker makes a dummy level for InitializePhysicsSceneForSaveIfNecessary which is not PostLoaded and contains no build data, ignore it.
	check(bSetupResourceClusters || MeshBuildData.Num() == 0);
	// If we have any mesh build data, we must have at least one resource cluster, otherwise clusters have not been setup properly.
	check(LightmapResourceClusters.Num() > 0 || MeshBuildData.Num() == 0);
	
	ENQUEUE_RENDER_COMMAND(SetFeatureLevelAndInitialize)(
		[&LightmapResourceClusters = LightmapResourceClusters, InFeatureLevel](FRHICommandList& RHICmdList)
		{
			// At this point all lightmap cluster resources are initialized and we can update cluster uniform buffers.
			for (FLightmapResourceCluster& Cluster : LightmapResourceClusters)
			{
				Cluster.SetFeatureLevelAndInitialize(InFeatureLevel);
			}
		});
}

void UMapBuildDataRegistry::ReleaseResources(const TSet<FGuid>* ResourcesToKeep)
{
	CleanupTransientOverrideMapBuildData();

	for (TMap<FGuid, FPrecomputedVolumetricLightmapData*>::TIterator It(LevelPrecomputedVolumetricLightmapBuildData); It; ++It)
	{
		if (!ResourcesToKeep || !ResourcesToKeep->Contains(It.Key()))
		{
			BeginReleaseResource(It.Value());
		}
	}

	for (FLightmapResourceCluster& ResourceCluster : LightmapResourceClusters)
	{
		BeginReleaseResource(&ResourceCluster);
	}
}

void UMapBuildDataRegistry::EmptyLevelData(const TSet<FGuid>* ResourcesToKeep)
{
	TMap<FGuid, FPrecomputedLightVolumeData*> PrevPrecomputedLightVolumeData;
	TMap<FGuid, FPrecomputedVolumetricLightmapData*> PrevPrecomputedVolumetricLightmapData;
	Swap(LevelPrecomputedLightVolumeBuildData, PrevPrecomputedLightVolumeData);
	Swap(LevelPrecomputedVolumetricLightmapBuildData, PrevPrecomputedVolumetricLightmapData);

	for (TMap<FGuid, FPrecomputedLightVolumeData*>::TIterator It(PrevPrecomputedLightVolumeData); It; ++It)
	{
		// Keep any resource if it's guid is in ResourcesToKeep.
		if (!ResourcesToKeep || !ResourcesToKeep->Contains(It.Key()))
		{
			delete It.Value();
		}
		else
		{
			LevelPrecomputedLightVolumeBuildData.Add(It.Key(), It.Value());
		}
	}

	for (TMap<FGuid, FPrecomputedVolumetricLightmapData*>::TIterator It(PrevPrecomputedVolumetricLightmapData); It; ++It)
	{
		// Keep any resource if it's guid is in ResourcesToKeep.
		if (!ResourcesToKeep || !ResourcesToKeep->Contains(It.Key()))
		{
			delete It.Value();
		}
		else
		{
			LevelPrecomputedVolumetricLightmapBuildData.Add(It.Key(), It.Value());
		}
	}

	// keep the VLM grid if we kept the VLM data
	if (!LevelPrecomputedVolumetricLightmapBuildData.Num())
	{	
		delete VolumetricLightMapGridDesc;
		VolumetricLightMapGridDesc = nullptr;
	}

	LightmapResourceClusters.Empty();
}

void UMapBuildDataRegistry::CleanupTransientOverrideMapBuildData()
{
	for (UHierarchicalInstancedStaticMeshComponent* Component : TObjectRange<UHierarchicalInstancedStaticMeshComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		for (auto& LOD : Component->LODData)
		{
			LOD.OverrideMapBuildData.Reset();
		}
	}
}

UMapBuildDataRegistry* UMapBuildDataRegistry::Get(const UActorComponent* Component)
{
	AActor* Owner = Component->GetOwner();

	if (Owner)
	{
		return Get(Owner);
	}

	return nullptr;
}

#if WITH_EDITOR
void UMapBuildDataRegistry::RedirectToRegistry(TArray<FGuid>& ActorInstances, UMapBuildDataRegistry* Registry)
{	
	// In PIE multiple worlds will reuse the same global UMapBuildDataRegistry so we make sure to refcount the add/remove of the redirects
	int32& CurrentRefCount = RedirectedRegistriesRefcount.FindOrAdd(Registry->GetFName());
	if (CurrentRefCount == 0)
	{
		for (const FGuid& ActorInstanceGuid : ActorInstances)
		{
			ensureMsgf(!Redirects.Find(ActorInstanceGuid), TEXT("Adding redundant mapping for ActorInstance %s, New registry: %s, Previous registry: %s"), *ActorInstanceGuid.ToString(), *Registry->GetName(), *Redirects.FindChecked(ActorInstanceGuid)->GetName());
			Redirects.Add(ActorInstanceGuid, Registry);
		}
	}
	
	CurrentRefCount++;
}

void UMapBuildDataRegistry::RemoveRedirect(TArray<FGuid>& ActorInstances, UMapBuildDataRegistry* Registry)
{
	// In PIE multiple worlds will reuse the same global UMapBuildDataRegistry so we make sure to refcount the add/remove of the redirects
	int32& CurrentRefCount = RedirectedRegistriesRefcount.FindChecked(Registry->GetFName());	
	CurrentRefCount--;

	check(CurrentRefCount >= 0);

	if (CurrentRefCount == 0)
	{
		for (const FGuid& ActorInstanceGuid : ActorInstances)
		{
			Redirects.Remove(ActorInstanceGuid);
		}

		RedirectedRegistriesRefcount.Remove(Registry->GetFName());
	}
}
#else
void UMapBuildDataRegistry::RemoveRegistry(UMapBuildDataRegistry* Registry)
{
	FScopeLock AutoLock(&PackagesToMapBuildDataLock);
	PackagesToMapBuildData.Remove(Registry->GetPackage());
}
#endif

UMapBuildDataRegistry* UMapBuildDataRegistry::FindRegistryWorldPartition(const AActor* Actor)
{	
	UMapBuildDataRegistry* Registry = nullptr;

	// Finding the correct registry
	//  In editor & PIE : loaded registries will insert a redirect from the ActorInstanceGuids they provide data for so that we can find the proper registry
	//  In runtime : registry will live inside the same package as the actor so we can find the correct registry through the actor package
#if WITH_EDITOR
	check(IsInGameThread());

	if (!Registry)
	{
		FGuid ActorInstanceGuid = FActorInstanceGuid::GetActorInstanceGuid(*(const_cast<AActor*>(Actor)));
		if (UMapBuildDataRegistry** FoundRegistry = Redirects.Find(ActorInstanceGuid))
		{
			Registry = *FoundRegistry;
		}
	}
#else	
	FScopeLock AutoLock(&PackagesToMapBuildDataLock);

	UPackage* ObjectPackage = Actor->GetPackage();

	auto GetRegistryFromPackage = [ObjectPackage](const UObject* Object) -> UMapBuildDataRegistry*
	{
		UMapBuildDataRegistry* Registry = nullptr;
		ForEachObjectWithPackage(ObjectPackage, [&Registry](UObject* ObjInPackage) -> bool
		{
			Registry = Cast<UMapBuildDataRegistry>(ObjInPackage);
			if (Registry)
			{
				// stop enumeration
				return false;
			}

			return true;
		});

		return Registry;
	};


	if (UMapBuildDataRegistry** RegistryPtr = PackagesToMapBuildData.Find(ObjectPackage))
	{
		Registry = *RegistryPtr;
	}
	else
	{
		Registry = GetRegistryFromPackage(Actor);
		
		if (!Registry)
		{
			 Registry = GetRegistryFromPackage(ULevelInstanceSubsystem::GetOwningLevel(Actor->GetLevel(), false));
		}

		check(!PackagesToMapBuildData.Find(ObjectPackage));
		PackagesToMapBuildData.Add(ObjectPackage, Registry);
	}
#endif

	return Registry;
}

UMapBuildDataRegistry* UMapBuildDataRegistry::Get(const AActor* Actor)
{
	ULevel* OwnerLevel = Actor->GetLevel();
	UWorld* World = OwnerLevel ? OwnerLevel->GetWorld() : nullptr;	
	UMapBuildDataRegistry* Registry = nullptr;

	if (World && World->IsPartitionedWorld() && World->PersistentLevel->MapBuildData)
	{
		Registry = World->PersistentLevel->MapBuildData->FindRegistryWorldPartition(Actor);
	}

	if (!Registry)
	{
		Registry = Get(OwnerLevel, World);
	}

	UE_LOG_MAPBUILDDATA(Log, TEXT("Returning Registry %s for Actor %s, %s"), *Registry->GetFullName(), *Actor->GetActorNameOrLabel(), *Actor->GetFullName());
	return Registry;
}

UMapBuildDataRegistry* UMapBuildDataRegistry::Get(ULevel* OwnerLevel, UWorld* World)
{
	UMapBuildDataRegistry* MapBuildData = nullptr;

	if (OwnerLevel && World)
	{
		ULevel* ActiveLightingScenario = World->GetActiveLightingScenario();
		
		if (ActiveLightingScenario && ActiveLightingScenario->MapBuildData)
		{
			MapBuildData = ActiveLightingScenario->MapBuildData;
		}
		else if (OwnerLevel->MapBuildData)
		{
			MapBuildData = OwnerLevel->MapBuildData;
		}
	}

	return MapBuildData;
}

void UMapBuildDataRegistry::SetVolumetricLightMapGridDesc(FVolumetricLightMapGridDesc* GridDesc)
{	 
	delete VolumetricLightMapGridDesc;
	VolumetricLightMapGridDesc = GridDesc;
}

FUObjectAnnotationSparse<FMeshMapBuildLegacyData, true> GComponentsWithLegacyLightmaps;
FUObjectAnnotationSparse<FLevelLegacyMapBuildData, true> GLevelsWithLegacyBuildData;
FUObjectAnnotationSparse<FLightComponentLegacyMapBuildData, true> GLightComponentsWithLegacyBuildData;
FUObjectAnnotationSparse<FReflectionCaptureMapBuildLegacyData, true> GReflectionCapturesWithLegacyBuildData;
