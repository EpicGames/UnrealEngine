// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCache.h"
#include "ComponentRecreateRenderStateContext.h"
#include "VT/VirtualTextureBuildSettings.h"
#include "HAL/IConsoleManager.h"
#include "RenderUtils.h"

static TAutoConsoleVariable CVarMaterialCacheSupported(
	TEXT("r.MaterialCache.Support"),
	false,
	TEXT("Enable material cache support"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable CVarMaterialCacheEnabled(
	TEXT("r.MaterialCache.Enabled"),
	true,
	TEXT("Enable runtime material cache"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
	{
		// Recreate all proxies with new descriptor data
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable CVarMaterialCacheTileWidth(
	TEXT("r.MaterialCache.TileWidth"),
	128,
	TEXT("Tile width (per axis) of each tile"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable CVarMaterialCacheTileBorderWidth(
	TEXT("r.MaterialCache.TileBorderWidth"),
	4,
	TEXT("Tile border width (per axis) of each tile"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable CVarMaterialCacheAllocationMaxTileCountX(
	TEXT("r.MaterialCache.AllocationMaxTileCountWidth"),
	1024,
	TEXT("Maximum number of tiles (per axis) of each allocation"),
	ECVF_ReadOnly
);

uint32 GetMaterialCacheTileWidth()
{
	static uint32 Width = FVirtualTextureBuildSettings::ClampAndAlignTileSize(CVarMaterialCacheTileWidth->GetInt());
	return Width;
}

uint32 GetMaterialCacheTileBorderWidth()
{
	static uint32 Width = FVirtualTextureBuildSettings::ClampAndAlignTileBorderSize(CVarMaterialCacheTileBorderWidth->GetInt());
	return Width;
}

bool IsMaterialCacheSupported(FStaticShaderPlatform Platform)
{
	return CVarMaterialCacheSupported->GetBool() && UseVirtualTexturing(Platform);
}

bool IsMaterialCacheEnabled(FStaticShaderPlatform Platform)
{
	return IsMaterialCacheSupported(Platform) && CVarMaterialCacheEnabled->GetBool();
}

FIntPoint GetMaterialCacheTileCount(float TriangleArea, float TexelSize, TOptional<FIntPoint> MaxTileCount)
{	
	float TexelCountUniform = TriangleArea / TexelSize;
	float TileCountUniform  = TexelCountUniform / FMath::Square(static_cast<float>(GetMaterialCacheTileWidth()));
	float TileCountX        = FMath::Max(1u, FMath::Sqrt(TileCountUniform));
	
	TileCountX = FMath::RoundUpToPowerOfTwo(TileCountX);
	
	if (!MaxTileCount)
	{
		MaxTileCount = FIntPoint(CVarMaterialCacheAllocationMaxTileCountX->GetInt());
	}
	
	return FIntPoint(TileCountX, TileCountX).ComponentMin(MaxTileCount.GetValue());
}
