// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheDescriptor.h"

FMaterialCacheDescriptor PackMaterialCacheDescriptor(uint32 TagOffset, uint32 UVChannel)
{
	FMaterialCacheDescriptor Out{};
	Out.TagOffset = TagOffset;
	Out.UVChannel = UVChannel;
	return Out;
}
