// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheMaterial.h"

#include "Materials/MaterialExpressionLocalPosition.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"
#include "Materials/MaterialExpressionPreSkinnedNormal.h"
#include "Materials/MaterialExpressionTangent.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionVertexColor.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionVertexTangentWS.h"
#include "Materials/MaterialExpressionWorldPosition.h"

bool MaterialCacheIsExpressionNonUVDerived(const UMaterialExpression* Expression, uint64& UVChannelsUsedMask)
{
	// This is a fickle way of summarizing graph behaviour, and is certainly going to fail at some point with the translator as is.
	// Luckily, MIR will greatly help, as the lowered op-codes are an ideal form to summarize this. Until then, we do the below.

	// Collect coordinate indices used
	if (const UMaterialExpressionTextureCoordinate* TexCoord = Cast<UMaterialExpressionTextureCoordinate>(Expression))
	{
		verify(TexCoord->CoordinateIndex < 64);
		UVChannelsUsedMask |= (1ull << TexCoord->CoordinateIndex);
	}
	else if (const UMaterialExpressionTextureSample* Sample = Cast<UMaterialExpressionTextureSample>(Expression))
	{
		verify(Sample->ConstCoordinate < 64);
		UVChannelsUsedMask |= (1ull << Sample->ConstCoordinate);
	}

	// Any non-UV derived vertex data?
	const bool bExpressionClassTest =
		Expression->IsA(UMaterialExpressionWorldPosition::StaticClass()) ||
		Expression->IsA(UMaterialExpressionLocalPosition::StaticClass()) ||
		Expression->IsA(UMaterialExpressionVertexColor::StaticClass()) ||
		Expression->IsA(UMaterialExpressionVertexNormalWS::StaticClass()) ||
		Expression->IsA(UMaterialExpressionVertexTangentWS::StaticClass()) ||
		Expression->IsA(UMaterialExpressionPixelNormalWS::StaticClass()) ||
		Expression->IsA(UMaterialExpressionPreSkinnedNormal::StaticClass()) ||
		Expression->IsA(UMaterialExpressionTangent::StaticClass());

	return bExpressionClassTest;
}
