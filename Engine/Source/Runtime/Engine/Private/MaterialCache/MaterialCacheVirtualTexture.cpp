// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "EngineModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "RendererInterface.h"
#include "TextureResource.h"
#include "VirtualTextureEnum.h"
#include "VirtualTexturing.h"
#include "MaterialCache/IMaterialCacheTagProvider.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocation.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocator.h"
#include "MaterialCache/MaterialCache.h"
#include "MaterialCache/MaterialCacheAttribute.h"
#include "MaterialCache/MaterialCacheMeshProcessor.h"
#include "MaterialCache/MaterialCacheStackProvider.h"
#include "MaterialCache/MaterialCacheVirtualTextureRenderProxy.h"
#include "VT/VirtualTextureBuildSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialCacheVirtualTexture)

class FMaterialCacheVirtualTextureResource : public FTextureResource
{
public:
	FMaterialCacheVirtualTextureResource(FSceneInterface* Scene, const FMaterialCacheTagLayout& TagLayout, FIntPoint InTileCount, int32 InTileSize, int32 InTileBorderSize, bool bSharedShading)
		: Scene(Scene)
		, TagLayout(TagLayout)
		, TileCount(InTileCount)
		, TileSize(InTileSize)
		, TileBorderSize(InTileBorderSize)
		, bSharedShading(bSharedShading)
	{
		MaxLevel = FMath::CeilLogTwo(FMath::Max(InTileCount.X, InTileCount.Y));
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		static FName TextureName = TEXT("MaterialCacheVirtualTexture");
		
		FSamplerStateInitializerRHI SamplerStateInitializer;
		SamplerStateInitializer.Filter = SF_Bilinear;
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		// Create underlying producer
		FVTProducerDescription ProducerDesc;
		ProducerDesc.Name               = TextureName;
		ProducerDesc.FullNameHash       = GetTypeHash(TextureName);
		ProducerDesc.bContinuousUpdate  = false;
		ProducerDesc.Dimensions         = 2;
		ProducerDesc.TileSize           = TileSize;
		ProducerDesc.TileBorderSize     = TileBorderSize;
		ProducerDesc.BlockWidthInTiles  = TileCount.X;
		ProducerDesc.BlockHeightInTiles = TileCount.Y;
		ProducerDesc.DepthInTiles       = 1u;
		ProducerDesc.MaxLevel           = MaxLevel;
		ProducerDesc.NumTextureLayers   = TagLayout.Layers.Num();
		ProducerDesc.NumPhysicalGroups  = 1;
		ProducerDesc.Priority           = EVTProducerPriority::Normal;
		
		// Always put it in a single pool so we can sample it globally
		ProducerDesc.bRequiresSinglePhysicalPool = true;

		for (int32 LayerIndex = 0; LayerIndex < TagLayout.Layers.Num(); LayerIndex++)
		{
			ProducerDesc.LayerFormat[LayerIndex]        = TagLayout.Layers[LayerIndex].StorageFormat;
			ProducerDesc.bIsLayerSRGB[LayerIndex]       = TagLayout.Layers[LayerIndex].bIsSRGB;
			ProducerDesc.PhysicalGroupIndex[LayerIndex] = 0;
		}
		
		FMaterialCacheVirtualTextureDescription Desc;
		Desc.ProducerDesc = ProducerDesc;
		Desc.bSharedShading = bSharedShading;
		Desc.TagLayout = TagLayout;

		IMaterialCacheVirtualTextureAllocator* Allocator = GetRendererModule().GetMaterialCacheVirtualTextureAllocator();
		Allocation = Allocator->Allocate(RHICmdList, Scene, Desc);
	}

	virtual void ReleaseRHI() override
	{
		IMaterialCacheVirtualTextureAllocator* Allocator = GetRendererModule().GetMaterialCacheVirtualTextureAllocator();
		Allocator->Deallocate(Allocation);
	}
	
	FMaterialCacheVirtualTextureAllocation* GetAllocationChecked()
	{
		check(IsInRenderingThread());
		check(Allocation);
		return Allocation;
	}
	
	FMaterialCacheVirtualTextureAllocation* GetAllocation()
	{
		check(IsInRenderingThread());
		return Allocation;
	}

private:
	/** Owning scene, lifetime tied to the parent game virtual texture */
	FSceneInterface* Scene = nullptr;
	
	/** Physical formats */
	FMaterialCacheTagLayout TagLayout;
	
	/** Backing allocation */
	FMaterialCacheVirtualTextureAllocation* Allocation = nullptr;
	
	/** Tiled properties */
	FIntPoint TileCount;
	uint32    TileSize       = 0;
	uint32    TileBorderSize = 0;
	uint32    MaxLevel       = 0;
	uint32    NumSourceMips  = 1;
	bool      bSharedShading = false;
};

UMaterialCacheVirtualTexture::UMaterialCacheVirtualTexture(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	VirtualTextureStreaming = false;

#if WITH_EDITORONLY_DATA
	CompressionNone = true;
	CompressionForceAlpha = true;
#endif // WITH_EDITORONLY_DATA
}

UMaterialCacheVirtualTexture::~UMaterialCacheVirtualTexture()
{
	
}

FMaterialCacheVirtualTextureRenderProxy* UMaterialCacheVirtualTexture::CreateRenderProxy(const FBox2f& UVRegion, uint32 UVCoordinateIndex)
{
	FMaterialCacheVirtualTextureResource* Resource = GetPrivateResource();
	if (!Resource)
	{
		return nullptr;
	}
	
	FMaterialCacheVirtualTextureRenderProxy* Proxy = new FMaterialCacheVirtualTextureRenderProxy();
	Proxy->UVRegion = UVRegion;
	Proxy->UVCoordinateIndex = UVCoordinateIndex;
	
	// Stack providers are optional
	if (UMaterialCacheStackProvider* StackProvider = MaterialStackProvider.Get())
	{
		Proxy->StackProviderRenderProxy = TUniquePtr<FMaterialCacheStackProviderRenderProxy>(StackProvider->CreateRenderProxy());
	}

	// Render thread initialization
	ENQUEUE_RENDER_COMMAND(GetBackbufferFormatCmd)([Proxy, Resource](FRHICommandListImmediate&)
	{
		Proxy->Allocation = Resource->GetAllocation();
	});

	return Proxy;
}

FMaterialCacheTagLayout UMaterialCacheVirtualTexture::GetRuntimeLayout() const
{
	FMaterialCacheTagLayout Layout;

	// Tags are optional
	if (UMaterialCacheVirtualTextureTag* TagHandle = Tag.Get())
	{
		Layout = TagHandle->GetRuntimeLayout();
	}

	// If there's no valid layers, or its invalid, assume defaults
	if (!Layout.Layers.Num())
	{
		PackMaterialCacheAttributeLayers(DefaultMaterialCacheAttributes, Layout.Layers);
	}

	return Layout;
}

void UMaterialCacheVirtualTexture::Flush()
{
	// Get the resource on the game thread
	FMaterialCacheVirtualTextureResource* Resource = GetPrivateResource();
	if (!Resource)
	{
		return;
	}

	// Flush the full UV-range
	ENQUEUE_RENDER_COMMAND(MaterialCacheFlush)([Resource](FRHICommandListBase&)
	{
		if (FMaterialCacheVirtualTextureAllocation* Allocation = Resource->GetAllocation())
		{
			if (Allocation->VirtualTexture)
			{
				GetRendererModule().FlushVirtualTextureCache(Allocation->VirtualTexture, FVector2f(0, 0), FVector2f(1, 1));
			}
		}
	});
}

void UMaterialCacheVirtualTexture::Unregister()
{		
	// May not exist if headless
	FSceneInterface* Scene = GetScene();
	if (!Scene)
	{
		return;
	}

	// Get the resource on the game thread
	FMaterialCacheVirtualTextureResource* Resource = GetPrivateResource();
	if (!Resource)
	{
		return;
	}
	
	ENQUEUE_RENDER_COMMAND(ReleaseVT)([this, Resource](FRHICommandListImmediate&)
	{
		FMaterialCacheVirtualTextureAllocation* Allocation = Resource->GetAllocation();
		if (!ensure(Allocation))
		{
			return;
		}
			
		IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
		TagProvider->Unregister(Allocation);
	});
}

void UMaterialCacheVirtualTexture::RecreateAllocation()
{
	// Get the resource on the game thread
	FMaterialCacheVirtualTextureResource* Resource = GetPrivateResource();
	if (!Resource)
	{
		return;
	}
	
	ENQUEUE_RENDER_COMMAND(ReleaseVT)([this, Resource](FRHICommandListImmediate& RHICmdList)
	{
		FMaterialCacheVirtualTextureAllocation* Allocation = Resource->GetAllocation();
		if (!ensure(Allocation))
		{
			return;
		}

		IMaterialCacheVirtualTextureAllocator* Allocator = GetRendererModule().GetMaterialCacheVirtualTextureAllocator();
		Allocator->Reallocate(RHICmdList, Allocation);
	});
}

FIntPoint UMaterialCacheVirtualTexture::GetRuntimeTileCount() const
{
	FIntPoint TaggedTileCount;
	
	if (UMaterialCacheVirtualTextureTag* TagHandle = Tag.Get())
	{
		TaggedTileCount = TileCount * TagHandle->TileCountMultiplier;
	}
	else
	{
		TaggedTileCount = TileCount;
	}

	return TaggedTileCount.ComponentMax(FIntPoint(1, 1));
}

void UMaterialCacheVirtualTexture::GetVirtualTextureBuildSettings(FVirtualTextureBuildSettings& OutSettings) const
{
	OutSettings.TileSize       = GetMaterialCacheTileWidth();
	OutSettings.TileBorderSize = GetMaterialCacheTileBorderWidth();
}

EMaterialValueType UMaterialCacheVirtualTexture::GetMaterialType() const
{
	return MCT_TextureVirtual;
}

float UMaterialCacheVirtualTexture::GetSurfaceWidth() const
{
	return GetMaterialCacheTileWidth() * GetRuntimeTileCount().X;
}

float UMaterialCacheVirtualTexture::GetSurfaceHeight() const
{
	return GetMaterialCacheTileWidth() * GetRuntimeTileCount().Y;
}

uint32 UMaterialCacheVirtualTexture::GetSurfaceArraySize() const
{
	return 1;
}

float UMaterialCacheVirtualTexture::GetSurfaceDepth() const
{
	return 1;
}

ETextureClass UMaterialCacheVirtualTexture::GetTextureClass() const
{
	return ETextureClass::TwoD;
}

FTextureResource* UMaterialCacheVirtualTexture::CreateResource()
{
	check(IsInGameThread());

	// Owning component may have released this texture
	if (!OwningComponent.Get())
	{
		return nullptr;
	}

	if (!OwningComponent->GetScene())
	{
		return nullptr;
	}
	
	FVirtualTextureBuildSettings DefaultSettings;
	DefaultSettings.Init();
	GetVirtualTextureBuildSettings(DefaultSettings);

	return new FMaterialCacheVirtualTextureResource(
		OwningComponent->GetScene(),
		GetRuntimeLayout(),
		GetRuntimeTileCount(),
		DefaultSettings.TileSize,
		DefaultSettings.TileBorderSize,
		bSharedShading
	);
}

FMaterialCacheVirtualTextureResource* UMaterialCacheVirtualTexture::GetPrivateResource()
{
	FTextureResource* Resource = GetResource();
	if (!Resource)
	{
		return nullptr;
	}
	
	return static_cast<FMaterialCacheVirtualTextureResource*>(Resource);
}

FSceneInterface* UMaterialCacheVirtualTexture::GetScene()
{
	USceneComponent* Component = OwningComponent.Get();
	if (!Component)
	{
		return nullptr;
	}
	
	return Component->GetScene();
}

void FMaterialCacheVirtualTextureRenderProxy::Flush(FSceneInterface* Scene)
{
	IMaterialCacheVirtualTextureAllocator* Allocator = GetRendererModule().GetMaterialCacheVirtualTextureAllocator();
	Allocator->Flush(Allocation, FVector2f::Zero(), FVector2f::One());
}
