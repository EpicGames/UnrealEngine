// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheVirtualTextureTag.h"
#include "VirtualTexturing.h"
#include "ComponentRecreateRenderStateContext.h"
#include "MaterialCache/IMaterialCacheTagProvider.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialCacheVirtualTextureTag)

UMaterialCacheVirtualTextureTag::UMaterialCacheVirtualTextureTag(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	// Pack default layers
	Attributes = TArrayView<EMaterialCacheAttribute>(DefaultMaterialCacheAttributes);
	PackRuntimeLayers();
}

FMaterialCacheTagLayout UMaterialCacheVirtualTextureTag::GetRuntimeLayout()
{
	FMaterialCacheTagLayout Out;
	Out.Guid = Guid;
	Out.Layers = RuntimeLayers;
	return Out;
}

#if WITH_EDITOR
void UMaterialCacheVirtualTextureTag::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Treat this as an entirely new tag type
	Guid = FGuid::NewGuid();
	
	// Repack
	PackRuntimeLayers();

	// TODO: Mark dependent materials as out of date!
	UObject::PostEditChangeProperty(PropertyChangedEvent);
	
	// Since the tag layout has changed, recreating the allocation isn't enough
	// We need to invalidate the entire resource, and recreate the proxy associations
	for (TObjectIterator<UMaterialCacheVirtualTexture> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		It->UpdateResource();
	}
	
	// Reassociate to the new resources
	FGlobalComponentRecreateRenderStateContext Context;
}
#endif // WITH_EDITOR

void UMaterialCacheVirtualTextureTag::PackRuntimeLayers()
{
	FMaterialCachePackInfo PackInfo;
	PackInfo.bEnableCompression = bEnableCompression;
	PackInfo.bEnableBaseColorHQ = bEnableCompression && bEnableBaseColorHQ;
	PackInfo.bEnableNormalHQ    = bEnableCompression && bEnableNormalHQ;
	
	// Repack with new attributes
	RuntimeLayers.Reset();
	PackMaterialCacheAttributeLayers(TArrayView<EMaterialCacheAttribute>(Attributes), PackInfo,RuntimeLayers);

	// Must have at least one attribute
	if (RuntimeLayers.IsEmpty())
	{
		UE_LOGF(LogVirtualTexturing, Error, "Invalid material cache tag, must have at least one layer.");
	}

	// Validate against physical limits
	if (RuntimeLayers.Num() > MaterialCacheMaxRuntimeLayers)
	{
		UE_LOGF(LogVirtualTexturing, Error, "Invalid material cache tag, too many layers (max %u). Consider removing attributes or splitting it into separate tags.", MaterialCacheMaxRuntimeLayers);
		RuntimeLayers.Empty();
	}
}
