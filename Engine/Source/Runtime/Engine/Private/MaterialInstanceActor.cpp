// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialInstanceActor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Misc/MessageDialog.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInstanceActor)

/**
 * Construct a list of static actor names.
 * @param OutString - The string containing the list of actor's names.
 * @param Actors - The list of actors to check.
 */
static void GetListOfStaticActors(FString& OutString, const TArray<AActor*>& Actors)
{
	for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex)
	{
		AActor* Actor = Actors[ActorIndex];
		if (Actor && Actor->IsRootComponentStatic())
		{
			OutString += FString::Printf(TEXT("\n%s"), *Actor->GetFullName());
		}
	}
}

AMaterialInstanceActor::AMaterialInstanceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (!IsRunningCommandlet() && (SpriteComponent != NULL))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> MaterialInstanceSpriteObject;
			FName ID_Materials;
			FText NAME_Materials;
			FConstructorStatics()
				: MaterialInstanceSpriteObject(TEXT("/Engine/EditorResources/MatInstActSprite"))
				, ID_Materials(TEXT("Materials"))
				, NAME_Materials(NSLOCTEXT("SpriteCategory", "Materials", "Materials"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.MaterialInstanceSpriteObject.Get();
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Materials;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Materials;
		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->SetupAttachment(SceneComponent);
		SpriteComponent->bIsScreenSizeScaled = true;
	}
#endif // WITH_EDITORONLY_DATA
}

void AMaterialInstanceActor::PostLoad()
{
	Super::PostLoad();

	// Warn the user if any static actors exist in the list.
	FString StaticActors;
	GetListOfStaticActors(StaticActors, TargetActors);
	if (StaticActors.Len() > 0)
	{
		UE_LOGF(LogTemp, Log, "Static actors may not be referenced by a material instance actor:%ls", *StaticActors);
	}
}

#if WITH_EDITOR
void AMaterialInstanceActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Warn the user if any static actors exist in the list.
	FString StaticActors;
	GetListOfStaticActors(StaticActors, TargetActors);
	if (StaticActors.Len() > 0)
	{
		const FText WarningMsg = FText::Format( NSLOCTEXT("Engine", "MaterialInstanceActor_NonStaticActorRef", "Static actors may not be referenced by a material instance actor:{0}"), FText::FromString( StaticActors ) );
		UE_LOGF(LogTemp, Log, "%ls", *WarningMsg.ToString());
		FMessageDialog::Open( EAppMsgType::Ok, WarningMsg);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
/** Returns SpriteComponent subobject **/
UBillboardComponent* AMaterialInstanceActor::GetSpriteComponent() const { return SpriteComponent; }
#endif


