// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/MaterialMerging.h"

#if WITH_EDITOR
#include "Components/PrimitiveComponent.h"
#include "MaterialUtilities.h"
#include "MeshDescription.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialMerging)

DEFINE_LOG_CATEGORY_STATIC(LogMaterialMerging, Log, All);

FMaterialProxySettings::FMaterialProxySettings()
	: TextureSizingType(TextureSizingType_UseSingleTextureSize)
	, TextureSize(1024, 1024)
	, TargetTexelDensityPerMeter(5.0f)
	, MeshMaxScreenSizePercent(0.5f)
	, MeshMinDrawDistance(10000.0f)
	, GutterSpace(4.0f)
	, MetallicConstant(0.0f)
	, RoughnessConstant(0.5f)
	, AnisotropyConstant(0.0f)
	, SpecularConstant(0.5f)
	, OpacityConstant(1.0f)
	, OpacityMaskConstant(1.0f)
	, AmbientOcclusionConstant(1.0f)
	, MaterialMergeType(EMaterialMergeType::MaterialMergeType_Default)
	, BlendMode(BLEND_Opaque)
	, bAllowTwoSidedMaterial(true)
	, bNormalMap(true)
	, bTangentMap(false)
	, bMetallicMap(false)
	, bRoughnessMap(false)
	, bAnisotropyMap(false)
	, bSpecularMap(false)
	, bEmissiveMap(false)
	, bOpacityMap(false)
	, bOpacityMaskMap(false)
	, bAmbientOcclusionMap(false)
	, DiffuseTextureSize(1024, 1024)
	, NormalTextureSize(1024, 1024)
	, TangentTextureSize(1024, 1024)
	, MetallicTextureSize(1024, 1024)
	, RoughnessTextureSize(1024, 1024)
	, AnisotropyTextureSize(1024, 1024)
	, SpecularTextureSize(1024, 1024)
	, EmissiveTextureSize(1024, 1024)
	, OpacityTextureSize(1024, 1024)
	, OpacityMaskTextureSize(1024, 1024)
	, AmbientOcclusionTextureSize(1024, 1024)
{
}

bool FMaterialProxySettings::operator == (const FMaterialProxySettings& Other) const
{
	return TextureSize == Other.TextureSize
		&& TextureSizingType == Other.TextureSizingType
		&& TargetTexelDensityPerMeter == Other.TargetTexelDensityPerMeter
		&& MeshMaxScreenSizePercent == Other.MeshMaxScreenSizePercent
		&& MeshMinDrawDistance == Other.MeshMinDrawDistance
		&& GutterSpace == Other.GutterSpace
		&& MetallicConstant == Other.MetallicConstant
		&& RoughnessConstant == Other.RoughnessConstant
		&& AnisotropyConstant == Other.AnisotropyConstant
		&& SpecularConstant == Other.SpecularConstant
		&& OpacityConstant == Other.OpacityConstant
		&& OpacityMaskConstant == Other.OpacityMaskConstant
		&& AmbientOcclusionConstant == Other.AmbientOcclusionConstant
		&& MaterialMergeType == Other.MaterialMergeType
		&& BlendMode == Other.BlendMode
		&& bAllowTwoSidedMaterial == Other.bAllowTwoSidedMaterial
		&& bNormalMap == Other.bNormalMap
		&& bTangentMap == Other.bTangentMap
		&& bMetallicMap == Other.bMetallicMap
		&& bRoughnessMap == Other.bRoughnessMap
		&& bAnisotropyMap == Other.bAnisotropyMap
		&& bSpecularMap == Other.bSpecularMap
		&& bEmissiveMap == Other.bEmissiveMap
		&& bOpacityMap == Other.bOpacityMap
		&& bOpacityMaskMap == Other.bOpacityMaskMap
		&& bAmbientOcclusionMap == Other.bAmbientOcclusionMap
		&& DiffuseTextureSize == Other.DiffuseTextureSize
		&& NormalTextureSize == Other.NormalTextureSize
		&& TangentTextureSize == Other.TangentTextureSize
		&& MetallicTextureSize == Other.MetallicTextureSize
		&& RoughnessTextureSize == Other.RoughnessTextureSize
		&& AnisotropyTextureSize == Other.AnisotropyTextureSize
		&& SpecularTextureSize == Other.SpecularTextureSize
		&& EmissiveTextureSize == Other.EmissiveTextureSize
		&& OpacityTextureSize == Other.OpacityTextureSize
		&& OpacityMaskTextureSize == Other.OpacityMaskTextureSize
		&& AmbientOcclusionTextureSize == Other.AmbientOcclusionTextureSize;
}

bool FMaterialProxySettings::operator != (const FMaterialProxySettings& Other) const
{
	return !(*this == Other);
}

FIntPoint FMaterialProxySettings::GetMaxTextureSize() const
{
	auto MaxBBox = [](const FIntPoint& A, const FIntPoint B)
	{
		return FIntPoint(FMath::Max(A.X, B.X), FMath::Max(A.Y, B.Y));
	};

	FIntPoint MaxTextureSize = FIntPoint(1, 1);

	const bool bUseTextureSize =
		(TextureSizingType == ETextureSizingType::TextureSizingType_UseSingleTextureSize) ||
		(TextureSizingType == ETextureSizingType::TextureSizingType_UseAutomaticBiasedSizes);

	switch (TextureSizingType)
	{
	case ETextureSizingType::TextureSizingType_UseSingleTextureSize:
	case ETextureSizingType::TextureSizingType_UseAutomaticBiasedSizes:
		MaxTextureSize = MaxBBox(MaxTextureSize, TextureSize);
		break;
	
	case ETextureSizingType::TextureSizingType_UseManualOverrideTextureSize:
		MaxTextureSize = MaxBBox(MaxTextureSize, DiffuseTextureSize);
		if (bNormalMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, NormalTextureSize);
		}
		if (bTangentMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, TangentTextureSize);			
		}
		if (bMetallicMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, MetallicTextureSize);
		}
		if (bRoughnessMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, RoughnessTextureSize);
		}
		if (bAnisotropyMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, AnisotropyTextureSize);
		}
		if (bSpecularMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, SpecularTextureSize);
		}
		if (bEmissiveMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, EmissiveTextureSize);
		}
		if (bOpacityMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, OpacityTextureSize);
		}
		if (bOpacityMaskMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, OpacityMaskTextureSize);
		}
		if (bAmbientOcclusionMap)
		{
			MaxTextureSize = MaxBBox(MaxTextureSize, AmbientOcclusionTextureSize);
		}
		break;

	case TextureSizingType_UseSimplygonAutomaticSizing:
	case TextureSizingType_AutomaticFromTexelDensity:
	case TextureSizingType_AutomaticFromMeshScreenSize:
	case TextureSizingType_AutomaticFromMeshDrawDistance:
	default:
		UE_LOGF(LogMaterialMerging, Error, "Unsupported TextureSizingType value. You should resolve the material texture size first with ResolveTextureSize()");
		break;
	}

	return MaxTextureSize;
}

#if WITH_EDITOR

bool FMaterialProxySettings::ResolveTexelDensity(const TArray<UPrimitiveComponent*>& InComponents)
{
	if (ResolveTexelDensity(InComponents, TargetTexelDensityPerMeter))
	{
		TextureSizingType = ETextureSizingType::TextureSizingType_AutomaticFromTexelDensity;
		return true;
	}

	return false;
}

bool FMaterialProxySettings::ResolveTexelDensity(const TArray<UPrimitiveComponent*>& InComponents, float& OutTexelDensity) const
{
	// Gather bounds of the input components
	auto GetActorsBounds = [&]() -> FBoxSphereBounds
	{
		FBoxSphereBounds::Builder BoundsBuilder;

		for (UPrimitiveComponent* Component : InComponents)
		{
			if (Component)
			{
				BoundsBuilder += Component->Bounds;
			}
		}

		return BoundsBuilder;
	};

	if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshScreenSize)
	{
		OutTexelDensity = FMaterialUtilities::ComputeRequiredTexelDensityFromScreenSize(MeshMaxScreenSizePercent, GetActorsBounds().SphereRadius);
		return true;
	}
	else if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		OutTexelDensity = FMaterialUtilities::ComputeRequiredTexelDensityFromDrawDistance(MeshMinDrawDistance, GetActorsBounds().SphereRadius);
		return true;
	}
	else if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromTexelDensity)
	{
		OutTexelDensity = TargetTexelDensityPerMeter;
		return true;
	}

	return false;
}

void FMaterialProxySettings::ResolveTextureSize(const FMeshDescription& InMesh)
{
	if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromTexelDensity ||
		TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshScreenSize ||
		TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshScreenSize)
		{
			TargetTexelDensityPerMeter = FMaterialUtilities::ComputeRequiredTexelDensityFromScreenSize(MeshMaxScreenSizePercent, InMesh.GetBounds().SphereRadius);
		}
		else if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshDrawDistance)
		{
			TargetTexelDensityPerMeter = FMaterialUtilities::ComputeRequiredTexelDensityFromDrawDistance(MeshMinDrawDistance, InMesh.GetBounds().SphereRadius);
		}

		TextureSize = FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(InMesh, TargetTexelDensityPerMeter);
		TextureSizingType = ETextureSizingType::TextureSizingType_UseSingleTextureSize;
	}
}

void FMaterialProxySettings::ResolveTextureSize(const float InWorldSpaceRadius, const double InWorldSpaceArea, const double InUVSpaceArea)
{
	if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromTexelDensity ||
		TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshScreenSize ||
		TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshScreenSize)
		{
			TargetTexelDensityPerMeter = FMaterialUtilities::ComputeRequiredTexelDensityFromScreenSize(MeshMaxScreenSizePercent, InWorldSpaceRadius);
		}
		else if (TextureSizingType == ETextureSizingType::TextureSizingType_AutomaticFromMeshDrawDistance)
		{
			TargetTexelDensityPerMeter = FMaterialUtilities::ComputeRequiredTexelDensityFromDrawDistance(MeshMinDrawDistance, InWorldSpaceRadius);
		}

		TextureSize = FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(InWorldSpaceArea, InUVSpaceArea, TargetTexelDensityPerMeter);
		TextureSizingType = ETextureSizingType::TextureSizingType_UseSingleTextureSize;
	}
}

#endif // WITH_EDITOR
