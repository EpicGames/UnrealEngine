// Copyright Epic Games, Inc. All Rights Reserved.
#include "HLSLMaterialDerivativeAutogen.h"
#include "HLSLMaterialTranslator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLSLMaterialDerivativeAutogen)

#if WITH_EDITORONLY_DATA

static int32 GDebugGenerateAllFunctionsEnabled = 0;
static FAutoConsoleVariableRef CVarAnalyticDerivDebugGenerateAllFunctions(
	TEXT("r.MaterialEditor.AnalyticDeriv.DebugGenerateAllFunctions"),
	GDebugGenerateAllFunctionsEnabled,
	TEXT("Debug: Generate all derivative functions.")
);

static inline bool IsDebugGenerateAllFunctionsEnabled()
{
	return GDebugGenerateAllFunctionsEnabled != 0;
}

static inline const TCHAR * GetBoolVectorName(EDerivativeType Type)
{
	switch(Type)
	{
	case EDerivativeType::Float1:
		return TEXT("bool");
	case EDerivativeType::Float2:
		return TEXT("bool2");
	case EDerivativeType::Float3:
		return TEXT("bool3");
	case EDerivativeType::Float4:
		return TEXT("bool4");
	case EDerivativeType::LWCScalar:
		return TEXT("bool");
	case EDerivativeType::LWCVector2:
		return TEXT("bool2");
	case EDerivativeType::LWCVector3:
		return TEXT("bool3");
	case EDerivativeType::LWCVector4:
		return TEXT("bool4");
	default:
		ensureMsgf(false, TEXT("Unhandled EDerivativeType value: %d for GetBoolVectorName"), static_cast<int32>(Type));
		return TEXT("");
	}
}

static inline const TCHAR * GetFloatVectorName(EDerivativeType Type)
{
	switch(Type)
	{
	case EDerivativeType::Float1:
		return TEXT("float");
	case EDerivativeType::Float2:
		return TEXT("float2");
	case EDerivativeType::Float3:
		return TEXT("float3");
	case EDerivativeType::Float4:
		return TEXT("float4");
	case EDerivativeType::LWCScalar:
		return TEXT("FWSScalar");
	case EDerivativeType::LWCVector2:
		return TEXT("FWSVector2");
	case EDerivativeType::LWCVector3:
		return TEXT("FWSVector3");
	case EDerivativeType::LWCVector4:
		return TEXT("FWSVector4");
	default:
		ensureMsgf(false, TEXT("Unhandled EDerivativeType value: %d for GetFloatVectorName"), static_cast<int32>(Type));
		return TEXT("");
	}
}

static inline uint32 GetNumComponents(EDerivativeType Type)
{
	switch (Type)
	{
	case EDerivativeType::Float1: return 1u;
	case EDerivativeType::Float2: return 2u;
	case EDerivativeType::Float3: return 3u;
	case EDerivativeType::Float4: return 4u;
	case EDerivativeType::LWCScalar: return 1u;
	case EDerivativeType::LWCVector2: return 2u;
	case EDerivativeType::LWCVector3: return 3u;
	case EDerivativeType::LWCVector4: return 4u;
	default: ensureMsgf(false, TEXT("Unhandled EDerivativeType value: %d for GetNumComponents"), static_cast<int32>(Type)); return 0u;
	}
}

static inline const TCHAR* GetFloatVectorDDXYName(EDerivativeType Type)
{
	return GetFloatVectorName(MakeNonLWCType(Type));
}

static inline const TCHAR * GetDerivVectorName(EDerivativeType Type)
{
	switch(Type)
	{
	case EDerivativeType::Float1:
		return TEXT("FloatDeriv");
	case EDerivativeType::Float2:
		return TEXT("FloatDeriv2");
	case EDerivativeType::Float3:
		return TEXT("FloatDeriv3");
	case EDerivativeType::Float4:
		return TEXT("FloatDeriv4");
	case EDerivativeType::LWCScalar:
		return TEXT("FWSScalarDeriv");
	case EDerivativeType::LWCVector2:
		return TEXT("FWSVector2Deriv");
	case EDerivativeType::LWCVector3:
		return TEXT("FWSVector3Deriv");
	case EDerivativeType::LWCVector4:
		return TEXT("FWSVector4Deriv");
	default:
		ensureMsgf(false, TEXT("Unhandled EDerivativeType value: %d for GetDerivVectorName"), static_cast<int32>(Type));
		return TEXT("");
	}
}

EDerivativeType GetDerivType(EMaterialValueType ValueType, bool bAllowNonFloat)
{
	switch (ValueType)
	{
	case MCT_StaticBool:
	case MCT_Float:
	case MCT_Float1:
	case MCT_UInt1:
		return EDerivativeType::Float1;
	case MCT_Float2:
		return EDerivativeType::Float2;
	case MCT_Float3:
		return EDerivativeType::Float3;
	case MCT_Float4:
		return EDerivativeType::Float4;
	case MCT_LWCScalar:
		return EDerivativeType::LWCScalar;
	case MCT_LWCVector2:
		return EDerivativeType::LWCVector2;
	case MCT_LWCVector3:
		return EDerivativeType::LWCVector3;
	case MCT_LWCVector4:
		return EDerivativeType::LWCVector4;
	default:
		ensureMsgf(bAllowNonFloat, TEXT("Invalid EDerivativeType value: %d for GetDerivType"), static_cast<int32>(ValueType));
		return EDerivativeType::None;
	}
}

static EMaterialValueType GetMaterialTypeFromDerivType(EDerivativeType Type)
{
	switch (Type)
	{
	case EDerivativeType::Float1:
		return MCT_Float;
	case EDerivativeType::Float2:
		return MCT_Float2;
	case EDerivativeType::Float3:
		return MCT_Float3;
	case EDerivativeType::Float4:
		return MCT_Float4;
	case EDerivativeType::LWCScalar:
		return MCT_LWCScalar;
	case EDerivativeType::LWCVector2:
		return MCT_LWCVector2;
	case EDerivativeType::LWCVector3:
		return MCT_LWCVector3;
	case EDerivativeType::LWCVector4:
		return MCT_LWCVector4;
	default:
		ensureMsgf(false, TEXT("Invalid EDerivativeType value: %d for GetMaterialTypeFromDerivType"), static_cast<int32>(Type));
		return (EMaterialValueType)0;
	}
}

static FString CoerceFloat(FHLSLMaterialTranslator& Translator, const TCHAR* Value, EDerivativeType DstType, EDerivativeType SrcType)
{
	const EMaterialCastFlags CastFlags = EMaterialCastFlags::ReplicateScalar | EMaterialCastFlags::AllowAppendZeroes | EMaterialCastFlags::AllowTruncate;
	return Translator.CastValue(Value, GetMaterialTypeFromDerivType(SrcType), GetMaterialTypeFromDerivType(DstType), CastFlags);
}

void FMaterialDerivativeAutogen::EnableGeneratedDepencencies()
{
	for (int32 IndexLHS = 0; IndexLHS < NumDerivativeTypes; IndexLHS++)
	{
		for (int32 IndexRHS = 0; IndexRHS < NumDerivativeTypes; IndexRHS++)
		{
			// PowPositiveClamped requires Pow
			if (bFunc2OpIsEnabled[(int32)EFunc2::PowPositiveClamped][IndexLHS][IndexRHS])
			{
				bFunc2OpIsEnabled[(int32)EFunc2::Pow][IndexLHS][IndexRHS] = true;
			}
		}
	}
	
	for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
	{
		const EDerivativeType Type = (EDerivativeType)Index;
		const EDerivativeType ScalarType = IsLWCType(Type) ? EDerivativeType::LWCScalar : EDerivativeType::Float1;

		// normalize requires rsqrt1, dot, expand, and mul
		if (bFunc1OpIsEnabled[(int32)EFunc1::Normalize][Index])
		{
			bConvertDerivEnabled[Index][0] = true;
			if (IsLWCType(Type))
			{
				// Convert from LWC->NonLWC
				bConvertDerivEnabled[(int32)MakeNonLWCType(Type)][Index] = true;
			}
			bFunc2OpIsEnabled[(int32)EFunc2::Dot][Index][Index] = true;
			bFunc1OpIsEnabled[(int32)EFunc1::Rsqrt][(int32)ScalarType] = true;
			bFunc2OpIsEnabled[(int32)EFunc2::Mul][Index][Index] = true;
		}

		// length requires sqrt1 and dot, dot requires a few other things, but those are handled below
		if (bFunc1OpIsEnabled[(int32)EFunc1::Length][Index])
		{
			bFunc2OpIsEnabled[(int32)EFunc2::Dot][Index][Index] = true;
			bFunc1OpIsEnabled[(int32)EFunc1::Sqrt][(int32)ScalarType] = true;
		}

		// inv length requires rsqrt1 (instead of sqrt1) and dot
		if (bFunc1OpIsEnabled[(int32)EFunc1::InvLength][Index])
		{
			bFunc2OpIsEnabled[(int32)EFunc2::Dot][Index][Index] = true;
			bFunc1OpIsEnabled[(int32)EFunc1::Rsqrt][(int32)ScalarType] = true;
		}
	}

	// Dot requires extract, mul1, add1 and FloatDeriv constructor
	for (int32 IndexLHS = 0; IndexLHS < NumDerivativeTypes; IndexLHS++)
	{
		const EDerivativeType LHSType = (EDerivativeType)IndexLHS;
		for (int32 IndexRHS = 0; IndexRHS < NumDerivativeTypes; IndexRHS++)
		{
			const EDerivativeType RHSType = (EDerivativeType)IndexRHS;
			if (bFunc2OpIsEnabled[(int32)EFunc2::Dot][IndexLHS][IndexRHS])
			{
				bool bEnableLWCVariant = IsLWCType(LHSType) || IsLWCType(RHSType);
				bool bEnableNonLWCVariant = !IsLWCType(LHSType) || !IsLWCType(RHSType);

				bExtractIndexEnabled[IndexLHS] = true;
				if (bEnableLWCVariant)
				{
					bConstructConstantDerivEnabled[(int32)EDerivativeType::LWCScalar] = true;
					bFunc2OpIsEnabled[(int32)EFunc2::Add][(int32)EDerivativeType::LWCScalar][(int32)EDerivativeType::LWCScalar] = true;
					bFunc2OpIsEnabled[(int32)EFunc2::Mul][(int32)EDerivativeType::LWCScalar][(int32)EDerivativeType::LWCScalar] = true;
				}
				if(bEnableNonLWCVariant)
				{
					bConstructConstantDerivEnabled[(int32)EDerivativeType::Float1] = true;
					bFunc2OpIsEnabled[(int32)EFunc2::Add][(int32)EDerivativeType::Float1][(int32)EDerivativeType::Float1] = true;
					bFunc2OpIsEnabled[(int32)EFunc2::Mul][(int32)EDerivativeType::Float1][(int32)EDerivativeType::Float1] = true;
				}
			}
		}
	}

	if (bRotateScaleOffsetTexCoords)
	{
		bFunc2OpIsEnabled[(int32)EFunc2::Add][1][1] = true;
		bFunc2OpIsEnabled[(int32)EFunc2::Mul][1][1] = true;
		bConstructDerivEnabled[1] = true;
	}
}

// given a string, convert it from type to type
FString FMaterialDerivativeAutogen::CoerceValueRaw(FHLSLMaterialTranslator& Translator, const FString& Token, const FDerivInfo& SrcInfo, EDerivativeType DstType)
{
	FString Ret = Token;

	// If the original value is a derivative, grab the raw value
	if (SrcInfo.DerivativeStatus == EDerivativeStatus::Valid)
	{
		Ret = TEXT("DERIV_BASE_VALUE(") + Ret + TEXT(")");
	}

	Ret = CoerceFloat(Translator, *Ret, DstType, SrcInfo.DerivType);
	return Ret;
}

// given a string, convert it from type to type
FString FMaterialDerivativeAutogen::CoerceValueDeriv(const FString& Token, const FDerivInfo& SrcInfo, EDerivativeType DstType)
{
	FString Ret = Token;

	check(IsDerivativeValid(SrcInfo.DerivativeStatus));

	// If it's valid, then it's already a type. Otherwise, we need to convert it from raw to deriv
	if (SrcInfo.DerivativeStatus == EDerivativeStatus::Zero)
	{
		FString SrcDerivType = GetDerivVectorName(SrcInfo.DerivType);
		bConstructConstantDerivEnabled[(int32)SrcInfo.DerivType] = true;
		Ret = TEXT("ConstructConstant") + SrcDerivType + TEXT("(") + Ret + TEXT(")");
	}

	return ConvertDeriv(Ret, DstType, SrcInfo.DerivType);
}

FString FMaterialDerivativeAutogen::ConstructDeriv(const FString& Value, const FString& Ddx, const FString& Ddy, EDerivativeType DstType)
{
	bConstructDerivEnabled[(int32)DstType] = true;
	FString TypeName = GetDerivVectorName(DstType);
	FString Ret = TEXT("Construct") + TypeName + TEXT("(") + Value + TEXT(",") + Ddx + TEXT(",") + Ddy + TEXT(")");
	return Ret;
}

FString FMaterialDerivativeAutogen::ConstructDerivFinite(const FString& Value, EDerivativeType DstType)
{
	bConstructFiniteDerivEnabled[(int32)DstType] = true;

	FString TypeName = GetDerivVectorName(DstType);
	FString Ret = TEXT("ConstructFinite") + TypeName + TEXT("(") + Value + TEXT(")");
	return Ret;
}

FString FMaterialDerivativeAutogen::ConvertDeriv(const FString& Value, EDerivativeType DstType, EDerivativeType SrcType)
{
	if (DstType == SrcType)
		return Value;
	
	bConvertDerivEnabled[(int32)DstType][(int32)SrcType] = true;

	FString SrctTypeName = GetDerivVectorName(SrcType);
	FString DstTypeName = GetDerivVectorName(DstType);
	return FString::Printf(TEXT("Convert_%s_%s(%s)"), *DstTypeName, *SrctTypeName, *Value);
}

FMaterialDerivativeAutogen::FOperationType1 FMaterialDerivativeAutogen::GetFunc1ReturnType(EDerivativeType SrcType, EFunc1 Op)
{
	const EDerivativeType SrcNonLWCType = MakeNonLWCType(SrcType);
	switch(Op)
	{
	case EFunc1::Abs:
		// Operate on LWCType
		return SrcType;
	case EFunc1::Saturate:
	case EFunc1::Frac:
	case EFunc1::Sqrt:
	case EFunc1::Rsqrt:
	case EFunc1::Rcp:
	case EFunc1::Normalize:
	case EFunc1::Sin:
	case EFunc1::Cos:
	case EFunc1::Tan:
		// Operate on LWCType, but return a non-LWCType
		return FOperationType1(SrcNonLWCType, SrcType);
	case EFunc1::Length:
		// Operate on LWCType, return LWCScalar
		return FOperationType1(IsLWCType(SrcType) ? EDerivativeType::LWCScalar : EDerivativeType::Float1, SrcType);
	case EFunc1::InvLength:
		// Operate on LWCType, but return a float1
		return FOperationType1(EDerivativeType::Float1, SrcType);
	case EFunc1::Log:
	case EFunc1::Log2:
	case EFunc1::Log10:
	case EFunc1::Exp:
	case EFunc1::Exp2:
	case EFunc1::Asin:
	case EFunc1::AsinFast:
	case EFunc1::Acos:
	case EFunc1::AcosFast:
	case EFunc1::Atan:
	case EFunc1::AtanFast:
		// No support for LWC
		return SrcNonLWCType;
	default:
		check(0);
		break;
	}

	return EDerivativeType::None;
}

FMaterialDerivativeAutogen::FOperationType2 FMaterialDerivativeAutogen::GetFunc2ReturnType(EDerivativeType LhsType, EDerivativeType RhsType, EFunc2 Op)
{
	const EDerivativeType LhsNonLWCType = MakeNonLWCType(LhsType);
	const EDerivativeType RhsNonLWCType = MakeNonLWCType(RhsType);
	const EDerivativeType NonLWCType = (EDerivativeType)FMath::Max<int32>((int32)LhsNonLWCType, (int32)RhsNonLWCType);

	switch(Op)
	{
	case EFunc2::Add:
	case EFunc2::Sub:
	case EFunc2::Mul:
	case EFunc2::Div:
	case EFunc2::Dot:
	case EFunc2::Max:
	case EFunc2::Min:
	case EFunc2::Fmod:
	case EFunc2::Modulo:
		// Operations that support LWC
		// Any LWC type promotes result to LWC
		if (IsLWCType(LhsType) || (IsLWCType(RhsType) && Op != EFunc2::Fmod && Op != EFunc2::Modulo))
		{
			const EDerivativeType LhsLWCType = MakeLWCType(LhsType);
			const EDerivativeType RhsLWCType = MakeLWCType(RhsType);
			if (LhsLWCType == RhsLWCType ||
				LhsLWCType == EDerivativeType::LWCScalar ||
				RhsLWCType == EDerivativeType::LWCScalar ||
				// Dot allows mixing different numbers of components
				// Legacy implementation truncated non-scalar types, this will instead promote to the maximum number of components
				// But because promotions are filled with 0s, this will effectively be the same thing (and simplifies the logic here)
				Op == EFunc2::Dot)
			{
				const EDerivativeType LWCType = (EDerivativeType)FMath::Max<int32>((int32)LhsLWCType, (int32)RhsLWCType);
				switch (Op)
				{
				case EFunc2::Dot:
					// Result is always LWCScalar
					return FOperationType2(EDerivativeType::LWCScalar, LWCType, LWCType);
				case EFunc2::Div:
					// Make sure we generate LWCDivide(LWC, float) if possible, since have a LWC denominator will result in worse approximation
					return FOperationType2(LWCType, LWCType, IsLWCType(RhsType) ? LWCType : NonLWCType);
				case EFunc2::Fmod:
				case EFunc2::Modulo:
					// FMOD Rhs is non-LWC
					return FOperationType2(NonLWCType, LWCType, NonLWCType);
				case EFunc2::Min:
				case EFunc2::Max:
				case EFunc2::PowPositiveClamped:
					// See note about Min/Max below
					return LhsLWCType;
				default:
					return FOperationType2(LWCType, IsLWCType(LhsType) ? LWCType : NonLWCType, IsLWCType(RhsType) ? LWCType : NonLWCType);
				}
			}
		}
		// Fallthrough to non-LWC case
	case EFunc2::Pow:
	case EFunc2::PowPositiveClamped:
	case EFunc2::Atan2:
	case EFunc2::Atan2Fast:
		if (LhsNonLWCType == RhsNonLWCType ||
			LhsNonLWCType == EDerivativeType::Float1 ||
			RhsNonLWCType == EDerivativeType::Float1 ||
			Op == EFunc2::Dot)
		{
			// if the initial type is different from the output type, then it's only valid if type is 0 (float). We can convert
			// a float to a type with more components, but for example, we can't implicitly convert a float2 to a float3/float4.
			switch (Op)
			{
			case EFunc2::Dot:
				// Result is always float
				return FOperationType2(EDerivativeType::Float1, NonLWCType, NonLWCType);
			case EFunc2::Min:
			case EFunc2::Max:
			case EFunc2::PowPositiveClamped:
				// Min/Max should probably use the same logic as other binary operations
				// However, legacy HLSLMaterialTranslator::Min/Max simply coerce the Rhs value to the type of the Lhs value
				// Need to preserve this behavior here (may want to wrap this behind a legacy flag/CVAR at some point)
				// Legacy PowPositiveClamped behavior is implemented differently, but end result is the same
				return LhsNonLWCType;
			default:
				return NonLWCType;
			}
		}
		break;
	case EFunc2::Cross:
		return EDerivativeType::Float3;
	default:
		check(0);
		break;
	}

	return EDerivativeType::None;
}


int32 FMaterialDerivativeAutogen::GenerateExpressionFunc1(FHLSLMaterialTranslator& Translator, EFunc1 Op, int32 SrcCode)
{
	if (SrcCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const FDerivInfo SrcDerivInfo = Translator.GetDerivInfo(SrcCode);
	const FOperationType1 OperationType = GetFunc1ReturnType(SrcDerivInfo.DerivType, Op);
	if (OperationType.ReturnType == EDerivativeType::None)
	{
		return Translator.Errorf(TEXT("Invalid input type: %s"), Translator.DescribeType(SrcDerivInfo.Type));
	}
	const bool bIsLWC = IsLWCType(OperationType.IntermediateType);

	EDerivativeStatus DstStatus = IsDerivativeValid(SrcDerivInfo.DerivativeStatus) ? EDerivativeStatus::Valid : EDerivativeStatus::NotValid;
	bool bUseScalarVersion = (DstStatus != EDerivativeStatus::Valid);

	// make initial tokens
	FString DstTokens[CompiledPDV_MAX];

	for (int32 Index = 0; Index < CompiledPDV_MAX; Index++)
	{
		ECompiledPartialDerivativeVariation Variation = (ECompiledPartialDerivativeVariation)Index;

		FString SrcToken = Translator.GetParameterCodeDeriv(SrcCode,Variation);
		SrcToken = CoerceValueRaw(Translator, SrcToken, SrcDerivInfo, OperationType.IntermediateType);

		FString DstToken;
		// just generate a type
		switch(Op)
		{
		case EFunc1::Abs:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSAbs(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("abs(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Log:
			DstToken = TEXT("log(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Log2:
			DstToken = TEXT("log2(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Log10:
			DstToken = TEXT("log10(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Exp:
			DstToken = TEXT("exp(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Exp2:
			DstToken = TEXT("exp2(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Sin:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSSin(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("sin(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Cos:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSCos(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("cos(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Tan:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSTan(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("tan(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Asin:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSASin(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("asin(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::AsinFast:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSASin(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("asinFast(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Acos:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSACos(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("acos(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::AcosFast:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSACos(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("acosFast(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Atan:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSATan(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("atan(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::AtanFast:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSATan(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("atanFast(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Sqrt:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSSqrtDemote(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("sqrt(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Rcp:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSRcpDemote(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("rcp(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Rsqrt:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSRsqrtDemote(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("rsqrt(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Saturate:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSSaturateDemote(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("saturate(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Frac:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSFracDemote(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("frac(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::Length:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSLength(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("length(") + SrcToken + TEXT(")");
			}
			break;
		case EFunc1::InvLength:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSRcpLengthDemote(") + SrcToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("rcp(length(") + SrcToken + TEXT("))");
			}
			break;
		case EFunc1::Normalize:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSNormalizeDemote(") + SrcToken + TEXT(")");
			}
			else if (OperationType.IntermediateType == EDerivativeType::Float1)
			{
				// Normalizing a scalar is always just 1.0f
				// Avoid generating a call to normalize(float), as this overload is not defined on all shader compilers
				DstToken = TEXT("1.0f");
			}
			else
			{
				DstToken = TEXT("normalize(") + SrcToken + TEXT(")");
			}
			break;
		default:
			check(0);
			break;
		}

		DstTokens[Index] = DstToken;
	}

	if (!bUseScalarVersion)
	{
		FString SrcToken = Translator.GetParameterCodeDeriv(SrcCode, CompiledPDV_Analytic);

		// Add LWC suffix to function names for LWC operations, avoids problems with HLSL overloading rules that just count number of 'floats' in each type
		const FString Suffix = bIsLWC ? TEXT("LWC") : TEXT("");

		SrcToken = CoerceValueDeriv(SrcToken, SrcDerivInfo, OperationType.IntermediateType);

		check(Op < EFunc1::Num);
		bFunc1OpIsEnabled[(int32)Op][(int32)OperationType.IntermediateType] = true;
		
		FString DstToken;
		switch(Op)
		{
		case EFunc1::Abs:
			DstToken = TEXT("AbsDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Log:
			DstToken = TEXT("LogDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Log2:
			DstToken = TEXT("Log2Deriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Log10:
			DstToken = TEXT("Log10Deriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Exp:
			DstToken = TEXT("ExpDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Exp2:
			DstToken = TEXT("Exp2Deriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Sin:
			DstToken = TEXT("SinDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Cos:
			DstToken = TEXT("CosDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Tan:
			DstToken = TEXT("TanDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Asin:
			DstToken = TEXT("AsinDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::AsinFast:
			DstToken = TEXT("AsinFastDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Acos:
			DstToken = TEXT("AcosDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::AcosFast:
			DstToken = TEXT("AcosFastDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Atan:
			DstToken = TEXT("AtanDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::AtanFast:
			DstToken = TEXT("AtanFastDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Sqrt:
			DstToken = TEXT("SqrtDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Rcp:
			DstToken = TEXT("RcpDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Rsqrt:
			DstToken = TEXT("RsqrtDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Saturate:
			DstToken = TEXT("SaturateDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Frac:
			DstToken = TEXT("FracDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Length:
			DstToken = TEXT("LengthDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::InvLength:
			DstToken = TEXT("InvLengthDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		case EFunc1::Normalize:
			DstToken = TEXT("NormalizeDeriv") + Suffix + TEXT("(") + SrcToken + TEXT(")");
			break;
		default:
			check(0);
			break;
		}

		DstTokens[CompiledPDV_Analytic] = DstToken;
	}

	EMaterialValueType DstMatType = GetMaterialTypeFromDerivType(OperationType.ReturnType);
	int32 Ret = Translator.AddCodeChunkInnerDeriv(*DstTokens[CompiledPDV_FiniteDifferences],*DstTokens[CompiledPDV_Analytic],DstMatType,false /*?*/, DstStatus);
	return Ret;
}

int32 FMaterialDerivativeAutogen::GenerateExpressionFunc2(FHLSLMaterialTranslator& Translator, EFunc2 Op, int32 LhsCode, int32 RhsCode)
{
	if (LhsCode == INDEX_NONE || RhsCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const FDerivInfo LhsDerivInfo = Translator.GetDerivInfo(LhsCode);
	const FDerivInfo RhsDerivInfo = Translator.GetDerivInfo(RhsCode);

	if (Op == EFunc2::Fmod || Op == EFunc2::Modulo)
	{
		check(RhsDerivInfo.DerivativeStatus == EDerivativeStatus::Zero);
	}

	const FOperationType2 OperationType = GetFunc2ReturnType(LhsDerivInfo.DerivType, RhsDerivInfo.DerivType, Op);
	if (OperationType.ReturnType == EDerivativeType::None)
	{
		const FString Param1 = LhsDerivInfo.DerivType != EDerivativeType::None ? GetFloatVectorName(LhsDerivInfo.DerivType) : Translator.DescribeType(LhsDerivInfo.Type);
		const FString Param2 = RhsDerivInfo.DerivType != EDerivativeType::None ? GetFloatVectorName(LhsDerivInfo.DerivType) : Translator.DescribeType(RhsDerivInfo.Type);

		return Translator.Errorf(TEXT("Invalid input types: %s, %s"), *Param1, *Param2);
	}
			
	const bool bIsLWC = IsLWCType(OperationType.LhsIntermediateType) || IsLWCType(OperationType.RhsIntermediateType);

	EDerivativeStatus DstStatus = EDerivativeStatus::NotValid;

	// Rules for derivatives:
	// 1. If either the LHS or RHS is Not Valid or Not Aware, then the derivative is not valid. Run scalar route.
	// 2. If both LHS and RHS are known to be Zero, then run raw code, and specify a known zero status.
	// 3. If both LHS and RHS are Valid derivatives, then run deriv path.
	// 4. If one is Valid and the other is known Zero, then promote the Zero to Valid, and run deriv path.

	bool bUseScalarVersion = false;
	bool bIsDerivValidZero = true;
	bool bMakeDerivLhs = false;
	bool bMakeDerivRhs = false;
	if (!IsDerivativeValid(LhsDerivInfo.DerivativeStatus) || !IsDerivativeValid(RhsDerivInfo.DerivativeStatus))
	{
		// use scalar version as a fallback
		bUseScalarVersion = true;
		// derivative is not valid
		bIsDerivValidZero = false;

		// We output status as invalid, since one of the parameters is either not aware or not valid
		DstStatus = EDerivativeStatus::NotValid;

	}
	else if (LhsDerivInfo.DerivativeStatus == EDerivativeStatus::Zero && RhsDerivInfo.DerivativeStatus == EDerivativeStatus::Zero)
	{
		// use scalar version
		bUseScalarVersion = true;
		// since we know both incoming values have derivatives of zero, we know the output is zero
		bIsDerivValidZero = true;
		// we know that the value is zero
		DstStatus = EDerivativeStatus::Zero;
	}
	else
	{
		check(IsDerivativeValid(LhsDerivInfo.DerivativeStatus));
		check(IsDerivativeValid(RhsDerivInfo.DerivativeStatus));

		// use deriv version
		bUseScalarVersion = false;
		// we will be calculating a valid derivative, so this value doesn't matter, but make it false for consistency
		bIsDerivValidZero = false;

		// if the lhs has a derivative of zero, and rhs is non-zero, convert lhs from a scalar type to deriv type
		if (LhsDerivInfo.DerivativeStatus == EDerivativeStatus::Zero)
		{
			bMakeDerivLhs = true;
		}

		// if the rhs has a derivative of zero, and lhs is non-zero, convert rhs from a scalar type to deriv type
		if (RhsDerivInfo.DerivativeStatus == EDerivativeStatus::Zero)
		{
			bMakeDerivRhs = true;
		}

		// derivative results will be valid
		DstStatus = EDerivativeStatus::Valid;
	}

	FString DstTokens[CompiledPDV_MAX];

	for (int32 Index = 0; Index < CompiledPDV_MAX; Index++)
	{
		ECompiledPartialDerivativeVariation Variation = (ECompiledPartialDerivativeVariation)Index;

		FString LhsToken = Translator.GetParameterCodeDeriv(LhsCode,Variation);
		FString RhsToken = Translator.GetParameterCodeDeriv(RhsCode,Variation);

		LhsToken = CoerceValueRaw(Translator, LhsToken, LhsDerivInfo, OperationType.LhsIntermediateType);
		RhsToken = CoerceValueRaw(Translator, RhsToken, RhsDerivInfo, OperationType.RhsIntermediateType);

		FString DstToken;
		// just generate a type
		switch(Op)
		{
		case EFunc2::Add:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Add);
				DstToken = TEXT("WSAdd(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("(") + LhsToken + TEXT(" + ") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Sub:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Subtract);
				DstToken = TEXT("WSSubtract(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("(") + LhsToken + TEXT(" - ") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Mul:
			if (bIsLWC)
			{
				if (Translator.IsConstFloatOfPow2Expression(RhsCode))
				{
					//Translator.AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorVector);
					DstToken = TEXT("WSMultiplyByPow2(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
				}
				else
				{
					Translator.AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorVector);
					DstToken = TEXT("WSMultiply(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
				}
			}
			else
			{
				DstToken = TEXT("(") + LhsToken + TEXT(" * ") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Div:
			if (bIsLWC)
			{
				if (Translator.IsConstFloatOfPow2Expression(RhsCode))
				{
					//Translator.AddLWCFuncUsage(ELWCFunctionKind::Divide);
					DstToken = TEXT("WSDivideByPow2(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
				}
				else
				{
					Translator.AddLWCFuncUsage(ELWCFunctionKind::Divide);
					DstToken = TEXT("WSDivide(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
				}
			}
			else
			{
				DstToken = TEXT("(") + LhsToken + TEXT(" / ") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Fmod:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSFmodDemote(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("fmod(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Modulo:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSModuloDemote(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("(") + LhsToken + TEXT("%") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Min:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSMin(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("min(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Max:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSMax(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("max(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Dot:
			if (bIsLWC)
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				DstToken = TEXT("WSDot(") + LhsToken + TEXT(", ") + RhsToken + TEXT(")");
			}
			else
			{
				DstToken = TEXT("dot(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			}
			break;
		case EFunc2::Pow:
			DstToken = TEXT("pow(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::PowPositiveClamped:
			DstToken = TEXT("PositiveClampedPow(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Atan2:
			DstToken = TEXT("atan2(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Atan2Fast:
			DstToken = TEXT("atan2Fast(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Cross:
			DstToken = TEXT("cross(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		default:
			check(0);
			break;
		}

		DstTokens[Index] = DstToken;
	}

	if (!bUseScalarVersion)
	{
		FString LhsToken = Translator.GetParameterCodeDeriv(LhsCode,CompiledPDV_Analytic);
		FString RhsToken = Translator.GetParameterCodeDeriv(RhsCode,CompiledPDV_Analytic);
		const FString Suffix = bIsLWC ? TEXT("LWC") : TEXT("");

		LhsToken = CoerceValueDeriv(LhsToken, LhsDerivInfo, OperationType.LhsIntermediateType);
		RhsToken = CoerceValueDeriv(RhsToken, RhsDerivInfo, OperationType.RhsIntermediateType);

		check(Op < EFunc2::Num);
		bFunc2OpIsEnabled[(int32)Op][(int32)OperationType.LhsIntermediateType][(int32)OperationType.RhsIntermediateType] = true;

		FString DstToken;
		switch(Op)
		{
		case EFunc2::Add:
			DstToken = TEXT("AddDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Sub:
			DstToken = TEXT("SubDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Mul:
			DstToken = TEXT("MulDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Div:
			DstToken = TEXT("DivDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Fmod:
			DstToken = TEXT("FmodDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Modulo:
			DstToken = TEXT("ModuloDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Dot:
			DstToken = TEXT("DotDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Min:
			DstToken = TEXT("MinDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Max:
			DstToken = TEXT("MaxDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Pow:
			DstToken = TEXT("PowDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::PowPositiveClamped:
			DstToken = TEXT("PowPositiveClampedDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Cross:
			DstToken = TEXT("CrossDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Atan2:
			DstToken = TEXT("Atan2Deriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		case EFunc2::Atan2Fast:
			DstToken = TEXT("Atan2FastDeriv") + Suffix + TEXT("(") + LhsToken + TEXT(",") + RhsToken + TEXT(")");
			break;
		default:
			check(0);
			break;
		}

		DstTokens[CompiledPDV_Analytic] = DstToken;
		DstStatus = EDerivativeStatus::Valid;
	}

	const EMaterialValueType DstMatType = GetMaterialTypeFromDerivType(OperationType.ReturnType);
	int32 Ret = Translator.AddCodeChunkInnerDeriv(*DstTokens[CompiledPDV_FiniteDifferences],*DstTokens[CompiledPDV_Analytic],DstMatType,false /*?*/, DstStatus);
	return Ret;
}

int32 FMaterialDerivativeAutogen::GenerateLerpFunc(FHLSLMaterialTranslator& Translator, int32 A, int32 B, int32 S)
{
	// TODO: generalize to Func3?
	if (A == INDEX_NONE || B == INDEX_NONE || S == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const FDerivInfo ADerivInfo = Translator.GetDerivInfo(A);
	const FDerivInfo BDerivInfo = Translator.GetDerivInfo(B);
	const FDerivInfo SDerivInfo = Translator.GetDerivInfo(S);
	
	const EMaterialValueType ResultType = Translator.GetArithmeticResultType(A, B);
	const EMaterialValueType AlphaType = (MakeNonLWCType(ResultType) == MakeNonLWCType(SDerivInfo.Type)) ? MakeNonLWCType(ResultType) : MCT_Float1;

	// Early out if the result type determined by input types is invalid.
	if (ResultType == EMaterialValueType::MCT_Unknown || AlphaType == EMaterialValueType::MCT_Unknown)
	{
		return INDEX_NONE;
	}

	const uint32 NumResultComponents = GetNumComponents(ResultType);

	const bool bAllZeroDeriv = (ADerivInfo.DerivativeStatus == EDerivativeStatus::Zero && BDerivInfo.DerivativeStatus == EDerivativeStatus::Zero && SDerivInfo.DerivativeStatus == EDerivativeStatus::Zero);
	const bool bIsLWC = ResultType & MCT_LWCType;
	if (bIsLWC) { Translator.AddLWCFuncUsage(ELWCFunctionKind::Other); }
	const TCHAR* FunctionName = bIsLWC ? TEXT("WSLerp") : TEXT("lerp");
	FString FiniteString = FString::Printf(TEXT("%s(%s,%s,%s)"), FunctionName, *Translator.CoerceParameter(A, ResultType), *Translator.CoerceParameter(B, ResultType), *Translator.CoerceParameter(S, AlphaType));
	
	if (!bAllZeroDeriv && IsDerivativeValid(ADerivInfo.DerivativeStatus) && IsDerivativeValid(BDerivInfo.DerivativeStatus) && IsDerivativeValid(SDerivInfo.DerivativeStatus))
	{
		const EDerivativeType ResultDerivType = GetDerivType(ResultType);
		const EDerivativeType AlphaDerivType = MakeNonLWCType(ResultDerivType);
		FString ADeriv = Translator.GetParameterCodeDeriv(A, CompiledPDV_Analytic);
		FString BDeriv = Translator.GetParameterCodeDeriv(B, CompiledPDV_Analytic);
		FString SDeriv = Translator.GetParameterCodeDeriv(S, CompiledPDV_Analytic);

		ADeriv = CoerceValueDeriv(ADeriv, ADerivInfo, ResultDerivType);
		BDeriv = CoerceValueDeriv(BDeriv, BDerivInfo, ResultDerivType);
		SDeriv = CoerceValueDeriv(SDeriv, SDerivInfo, AlphaDerivType);

		FString AnalyticString = FString::Printf(TEXT("LerpDeriv(%s, %s, %s)"), *ADeriv, *BDeriv, *SDeriv);

		check(NumResultComponents <= 4);
		bLerpEnabled[(int32)ResultDerivType] = true;

		return Translator.AddCodeChunkInnerDeriv(*FiniteString, *AnalyticString, ResultType, false, EDerivativeStatus::Valid);
	}
	else
	{
		return Translator.AddCodeChunkInnerDeriv(*FiniteString, *FiniteString, ResultType, false, bAllZeroDeriv ? EDerivativeStatus::Zero : EDerivativeStatus::NotValid);
	}
}

int32 FMaterialDerivativeAutogen::GenerateRotateScaleOffsetTexCoordsFunc(FHLSLMaterialTranslator& Translator, int32 TexCoord, int32 RotationScale, int32 Offset)
{
	// TODO: generalize to Func3?
	if (TexCoord == INDEX_NONE || RotationScale == INDEX_NONE || Offset == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	const FDerivInfo TexCoordDerivInfo		= Translator.GetDerivInfo(TexCoord);
	const FDerivInfo RotationScaleDerivInfo	= Translator.GetDerivInfo(RotationScale);
	const FDerivInfo OffsetDerivInfo		= Translator.GetDerivInfo(Offset);

	const EMaterialValueType ResultType = IsLWCType(TexCoordDerivInfo.Type) ? MCT_LWCVector2 : MCT_Float2;
	const uint32 NumResultComponents = 2;

	const bool bAllZeroDeriv = (TexCoordDerivInfo.DerivativeStatus == EDerivativeStatus::Zero && RotationScaleDerivInfo.DerivativeStatus == EDerivativeStatus::Zero && OffsetDerivInfo.DerivativeStatus == EDerivativeStatus::Zero);
	FString FiniteString = FString::Printf(TEXT("RotateScaleOffsetTexCoords(%s, %s, %s.xy)"),	*Translator.CoerceParameter(TexCoord, ResultType),
																								*Translator.CoerceParameter(RotationScale, MCT_Float4),
																								*Translator.CoerceParameter(Offset, MCT_Float2));

	if (!bAllZeroDeriv && IsDerivativeValid(TexCoordDerivInfo.DerivativeStatus) && IsDerivativeValid(RotationScaleDerivInfo.DerivativeStatus) && IsDerivativeValid(OffsetDerivInfo.DerivativeStatus))
	{
		FString TexCoordDeriv		= Translator.GetParameterCodeDeriv(TexCoord, CompiledPDV_Analytic);
		FString RotationScaleDeriv	= Translator.GetParameterCodeDeriv(RotationScale, CompiledPDV_Analytic);
		FString OffsetDeriv			= Translator.GetParameterCodeDeriv(Offset, CompiledPDV_Analytic);

		TexCoordDeriv		= CoerceValueDeriv(TexCoordDeriv,		TexCoordDerivInfo,			EDerivativeType::Float2);
		RotationScaleDeriv	= CoerceValueDeriv(RotationScaleDeriv,	RotationScaleDerivInfo,		EDerivativeType::Float4);
		OffsetDeriv			= CoerceValueDeriv(OffsetDeriv,			OffsetDerivInfo,			EDerivativeType::Float2);

		FString AnalyticString = FString::Printf(TEXT("RotateScaleOffsetTexCoordsDeriv(%s, %s, %s)"), *TexCoordDeriv, *RotationScaleDeriv, *OffsetDeriv);

		check(NumResultComponents <= 4);
		bRotateScaleOffsetTexCoords = true;

		return Translator.AddCodeChunkInnerDeriv(*FiniteString, *AnalyticString, ResultType, false, EDerivativeStatus::Valid);
	}
	else
	{
		return Translator.AddCodeChunkInnerDeriv(*FiniteString, *FiniteString, ResultType, false, bAllZeroDeriv ? EDerivativeStatus::Zero : EDerivativeStatus::NotValid);
	}
}

int32 FMaterialDerivativeAutogen::GenerateIfFunc(FHLSLMaterialTranslator& Translator, int32 A, int32 B, int32 Greater, int32 Equal, int32 Less, int32 Threshold)
{
	if (A == INDEX_NONE || B == INDEX_NONE || Greater == INDEX_NONE || Less == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const FString AFinite = Translator.GetParameterCode(A);
	const FString BFinite = Translator.GetParameterCode(B);

	const bool bEqual = (Equal != INDEX_NONE);

	if (bEqual && Threshold == INDEX_NONE)
		return INDEX_NONE;

	EMaterialValueType CompareType = Translator.GetArithmeticResultType(A, B);
	EMaterialValueType ResultType = Translator.GetArithmeticResultType(Greater, Less);

	if (bEqual)
		ResultType = Translator.GetArithmeticResultType(ResultType, Translator.GetParameterType(Greater));
	
	Greater = Translator.ForceCast(Greater,		ResultType);
	Less	= Translator.ForceCast(Less,		ResultType);
	if (Greater == INDEX_NONE || Less == INDEX_NONE)
		return INDEX_NONE;

	const FString GreaterFinite = Translator.GetParameterCode(Greater);
	const FString LessFinite	= Translator.GetParameterCode(Less);

	if (bEqual)
	{
		Equal = Translator.ForceCast(Equal, ResultType);
		if (Equal == INDEX_NONE)
			return INDEX_NONE;
	}
	
	const FDerivInfo GreaterDerivInfo	= Translator.GetDerivInfo(Greater, true);
	const FDerivInfo LessDerivInfo		= Translator.GetDerivInfo(Less, true);

	FString CodeFinite;
	FString ThresholdFinite;
	FString CompareGreaterEqual;
	FString CompareNotEqual;
	if (IsLWCType(CompareType))
	{
		Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
		CompareGreaterEqual = FString::Printf(TEXT("WSGreaterEqual(%s, %s)"), *AFinite, *BFinite);
	}
	else
	{
		CompareGreaterEqual = FString::Printf(TEXT("(%s >= %s)"), *AFinite, *BFinite);
	}

	if (bEqual)
	{
		ThresholdFinite = *Translator.GetParameterCode(Threshold);
		if (IsLWCType(CompareType))
		{
			Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
			CompareNotEqual = FString::Printf(TEXT("(!WSEqualsApprox(%s, %s, %s))"), *AFinite, *BFinite, *ThresholdFinite);
		}
		else
		{
			CompareNotEqual = FString::Printf(TEXT("(abs(%s - %s) > %s)"), *AFinite, *BFinite, *ThresholdFinite);
		}

		if (IsLWCType(ResultType))
		{
			Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
			CodeFinite = FString::Printf(
				TEXT("WSSelect(%s, WSSelect(%s, %s, %s), %s)"),
				*CompareNotEqual, *CompareGreaterEqual,
				*GreaterFinite, *LessFinite, *Translator.GetParameterCode(Equal));
		}
		else
		{
			if (ResultType == MCT_ShadingModel)
			{
				// @lh-todo: Workaround SPIR-V bug in DXC: Wrong literal type is deduced during implicit type deduction from int to uint
				// GitHub PR: https://github.com/microsoft/DirectXShaderCompiler/pull/4626
				CodeFinite = FString::Printf(
					TEXT("select(%s, select(%s, (uint)%s, (uint)%s), (uint)%s)"),
					*CompareNotEqual, *CompareGreaterEqual,
					*GreaterFinite, *LessFinite, *Translator.GetParameterCode(Equal));
			}
			else
			{
				CodeFinite = FString::Printf(
					TEXT("select(%s, select(%s, %s, %s), %s)"),
					*CompareNotEqual, *CompareGreaterEqual,
					*GreaterFinite, *LessFinite, *Translator.GetParameterCode(Equal));
			}
		}
	}
	else
	{
		if (IsLWCType(ResultType))
		{
			Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
			CodeFinite = FString::Printf(
				TEXT("WSSelect(%s, %s, %s)"),
				*CompareGreaterEqual,
				*GreaterFinite, *LessFinite
			);
		}
		else
		{
			if (ResultType == MCT_ShadingModel)
			{
				// @lh-todo: Workaround SPIR-V bug in DXC: Wrong literal type is deduced during implicit type deduction from int to uint
				// GitHub PR: https://github.com/microsoft/DirectXShaderCompiler/pull/4626
				CodeFinite = FString::Printf(
					TEXT("select(%s, (uint)%s, (uint)%s)"),
					*CompareGreaterEqual,
					*GreaterFinite, *LessFinite
				);
			}
			else
			{
				CodeFinite = FString::Printf(
					TEXT("select(%s, %s, %s)"),
					*CompareGreaterEqual,
					*GreaterFinite, *LessFinite
				);
			}
		}
	}

	const bool bAllDerivValid = IsDerivativeValid(GreaterDerivInfo.DerivativeStatus) && IsDerivativeValid(LessDerivInfo.DerivativeStatus) && (!bEqual || IsDerivativeValid(Translator.GetDerivativeStatus(Equal)));
	const bool bAllDerivZero = GreaterDerivInfo.DerivativeStatus == EDerivativeStatus::Zero && LessDerivInfo.DerivativeStatus == EDerivativeStatus::Zero && (!bEqual || Translator.GetDerivativeStatus(Equal) == EDerivativeStatus::Zero);
	if (bAllDerivValid && !bAllDerivZero)
	{
		const EDerivativeType ResultDerivType = GetDerivType(ResultType);

		FString GreaterDeriv	= Translator.GetParameterCodeDeriv(Greater, CompiledPDV_Analytic);
		FString LessDeriv		= Translator.GetParameterCodeDeriv(Less,	CompiledPDV_Analytic);

		// TODO - replace GreaterDerivInfo/LessDerivInfo.TypeIndex with ResultDerivType??
		GreaterDeriv	= CoerceValueDeriv(GreaterDeriv,	GreaterDerivInfo,	ResultDerivType);
		LessDeriv		= CoerceValueDeriv(LessDeriv,		LessDerivInfo,		ResultDerivType);

		FString CodeAnalytic;
		if (bEqual)
		{
			const FDerivInfo EqualDerivInfo = Translator.GetDerivInfo(Equal, true);
			FString EqualDeriv = Translator.GetParameterCodeDeriv(Equal, CompiledPDV_Analytic);
			EqualDeriv = CoerceValueDeriv(EqualDeriv, EqualDerivInfo, ResultDerivType); // TODO - see above

			CodeAnalytic = FString::Printf(TEXT("IfDeriv(%s, %s, %s, %s, %s)"), *CompareNotEqual, *CompareGreaterEqual, *GreaterDeriv, *LessDeriv, *EqualDeriv);
			bIf2Enabled[(int32)ResultDerivType] = true;
		}
		else
		{
			CodeAnalytic = FString::Printf(TEXT("IfDeriv(%s, %s, %s)"), *CompareGreaterEqual, *GreaterDeriv, *LessDeriv);
			bIfEnabled[(int32)ResultDerivType] = true;
		}

		return Translator.AddCodeChunkInnerDeriv(*CodeFinite, *CodeAnalytic, ResultType, false, EDerivativeStatus::Valid);

	}
	else
	{
		return Translator.AddCodeChunkInnerDeriv(*CodeFinite, *CodeFinite, ResultType, false, bAllDerivZero ? EDerivativeStatus::Zero : EDerivativeStatus::NotValid);
	}
}

FString FMaterialDerivativeAutogen::GenerateUsedFunctions(FHLSLMaterialTranslator& Translator)
{
	// Certain derivative functions rely on other derivative functions. For example, Dot() requires Mul() and Add(). So if (for example) dot is enabled, then enable mul1 and add1.
	EnableGeneratedDepencencies();

	FString Ret;

	// Full FloatDerivX constructors with explicit derivatives.
	for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
	{
		if (bConstructDerivEnabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			EDerivativeType DerivType = (EDerivativeType)Index;
			if (!IsLWCType(DerivType))
			{
				continue;				// Non-LWC constructors defined in common.ush
			}

			FString BaseName = GetDerivVectorName(DerivType);
			FString FieldName = GetFloatVectorName(DerivType);
			FString FieldNameDDXY = GetFloatVectorDDXYName(DerivType);

			Ret += BaseName + TEXT(" Construct") + BaseName + TEXT("(") + FieldName + TEXT(" InValue,") + FieldNameDDXY + TEXT(" InDdx,") + FieldNameDDXY + TEXT(" InDdy)") HLSL_LINE_TERMINATOR;
			Ret += TEXT("{") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Value = InValue;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Ddx = InDdx;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Ddy = InDdy;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("}") HLSL_LINE_TERMINATOR;
			Ret += TEXT("") HLSL_LINE_TERMINATOR;
		}
	}

	// FloatDerivX constructors from constant floatX.
	for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
	{
		if (bConstructConstantDerivEnabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			EDerivativeType DerivType = (EDerivativeType)Index;
			FString BaseName = GetDerivVectorName(DerivType);
			FString FieldName = GetFloatVectorName(DerivType);

			Ret += BaseName + TEXT(" ConstructConstant") + BaseName + TEXT("(") + FieldName + TEXT(" Value)") HLSL_LINE_TERMINATOR;
			Ret += TEXT("{") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Value = Value;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Ddx = 0;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Ddy = 0;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("}") HLSL_LINE_TERMINATOR;
			Ret += TEXT("") HLSL_LINE_TERMINATOR;
		}
	}

	// FloatDerivX constructor from floatX with implicit derivatives.
	for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
	{
		if (bConstructFiniteDerivEnabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			EDerivativeType DerivType = (EDerivativeType)Index;
			FString BaseName = GetDerivVectorName(DerivType);
			FString FieldName = GetFloatVectorName(DerivType);

			Ret += BaseName + TEXT(" ConstructFinite") + BaseName + TEXT("(") + FieldName + TEXT(" InValue)") HLSL_LINE_TERMINATOR;
			Ret += TEXT("{") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tRet.Value = InValue;") HLSL_LINE_TERMINATOR;
			if (Index == 4)
			{
				// LWC_TODO: Better way to do this??
				Ret += TEXT("\tRet.Ddx = ddx(WSHackToFloat(InValue));") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddy = ddy(WSHackToFloat(InValue));") HLSL_LINE_TERMINATOR;
			}
			else
			{
				Ret += TEXT("\tRet.Ddx = ddx(InValue);") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddy = ddy(InValue);") HLSL_LINE_TERMINATOR;
			}
			Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("}") HLSL_LINE_TERMINATOR;
			Ret += TEXT("") HLSL_LINE_TERMINATOR;
		}
	}

	// Convert between FloatDeriv types
	for (int32 DstIndex = 0; DstIndex < NumDerivativeTypes; DstIndex++)
	{
		for (int32 SrcIndex = 0; SrcIndex < NumDerivativeTypes; SrcIndex++)
		{
			if (SrcIndex == DstIndex)
				continue;

			if (bConvertDerivEnabled[DstIndex][SrcIndex] || IsDebugGenerateAllFunctionsEnabled())
			{
				EDerivativeType SrcDerivType = (EDerivativeType)SrcIndex;
				EDerivativeType DstDerivType = (EDerivativeType)DstIndex;
				FString DstBaseName = GetDerivVectorName(DstDerivType);
				FString SrcBaseName = GetDerivVectorName(SrcDerivType);
				FString ScalarName = GetDerivVectorName(EDerivativeType::Float1);
				const EDerivativeType SrcTypeDDXY = MakeNonLWCType(SrcDerivType);
				const EDerivativeType DstTypeDDXY = MakeNonLWCType(DstDerivType);

				Ret += DstBaseName + TEXT(" Convert_") + DstBaseName + TEXT("_") + SrcBaseName + TEXT("(") + SrcBaseName + TEXT(" Src)") HLSL_LINE_TERMINATOR;
				Ret += TEXT("{") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\t") + DstBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Value = ") + CoerceFloat(Translator, TEXT("Src.Value"), DstDerivType, SrcDerivType) + TEXT(";") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddx = ") + CoerceFloat(Translator, TEXT("Src.Ddx"), DstTypeDDXY, SrcTypeDDXY) + TEXT(";") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddy = ") + CoerceFloat(Translator, TEXT("Src.Ddy"), DstTypeDDXY, SrcTypeDDXY) + TEXT(";") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
				Ret += TEXT("}") HLSL_LINE_TERMINATOR;
				Ret += TEXT("") HLSL_LINE_TERMINATOR;
			}
		}
	}

	const TCHAR* SwizzleList[4] =
	{
		TEXT("x"),
		TEXT("y"),
		TEXT("z"),
		TEXT("w")
	};

	// Extract single FloatDeriv element from FloatDerivX
	for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
	{
		if (bExtractIndexEnabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			EDerivativeType DerivType = (EDerivativeType)Index;
			EDerivativeType ScalarDerivType = IsLWCType(DerivType) ? EDerivativeType::LWCScalar : EDerivativeType::Float1;
			FString BaseName = GetDerivVectorName(DerivType);
			FString ScalarName = GetDerivVectorName(ScalarDerivType);
			const uint32 NumComponents = GetNumComponents(DerivType);

			for (uint32 ElemIndex = 0; ElemIndex < NumComponents; ElemIndex++)
			{
				FString ElemStr = FString::Printf(TEXT("%d"), ElemIndex + 1);

				Ret += ScalarName + TEXT(" Extract") + BaseName + TEXT("_") + ElemStr + TEXT("(") + BaseName + TEXT(" InValue)") HLSL_LINE_TERMINATOR;
				Ret += TEXT("{") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\t") + ScalarName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
				if (IsLWCType(DerivType))
				{
					Ret += FString::Printf(TEXT("\tRet.Value = WSGetComponent(InValue.Value, %d);") HLSL_LINE_TERMINATOR, ElemIndex);
				}
				else
				{
					Ret += TEXT("\tRet.Value = InValue.Value.") + FString(SwizzleList[ElemIndex]) + TEXT(";") HLSL_LINE_TERMINATOR;
				}
				Ret += TEXT("\tRet.Ddx = InValue.Ddx.") + FString(SwizzleList[ElemIndex]) + TEXT(";") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddy = InValue.Ddy.") + FString(SwizzleList[ElemIndex]) + TEXT(";") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
				Ret += TEXT("}") HLSL_LINE_TERMINATOR;
				Ret += TEXT("") HLSL_LINE_TERMINATOR;
			}
		}
	}

	// Func2s
	for (int32 Op = 0; Op < (int32)EFunc2::Num; Op++)
	{
		for (int32 LHSIndex = 0; LHSIndex < NumDerivativeTypes; LHSIndex++)
		{
			for (int32 RHSIndex = 0; RHSIndex < NumDerivativeTypes; RHSIndex++)
			{
				if (bFunc2OpIsEnabled[Op][LHSIndex][RHSIndex] || IsDebugGenerateAllFunctionsEnabled())
				{
					EDerivativeType LHSDerivType = (EDerivativeType)LHSIndex;
					EDerivativeType RHSDerivType = (EDerivativeType)RHSIndex;
					bool bIsLWCOp = IsLWCType(LHSDerivType) || IsLWCType(RHSDerivType);
					EDerivativeType DerivType = bIsLWCOp ? MakeLWCType(LHSDerivType) : MakeNonLWCType(LHSDerivType);

					EDerivativeType ScalarDerivType = bIsLWCOp ? EDerivativeType::LWCScalar : EDerivativeType::Float1;
					FString BaseName = GetDerivVectorName(DerivType);
					FString NonLWCBaseName = GetDerivVectorName(MakeNonLWCType(DerivType));
					FString ScalarName = GetDerivVectorName(ScalarDerivType);
					FString FieldName = GetFloatVectorName(MakeNonLWCType(DerivType));
					FString BoolName = GetBoolVectorName(DerivType);

					FString LHSBaseName = GetDerivVectorName(LHSDerivType);
					FString RHSBaseName = GetDerivVectorName(RHSDerivType);

					const uint32 NumComponents = GetNumComponents(DerivType);
					const FString Suffix = bIsLWCOp ? TEXT("LWC") : TEXT("");

					switch ((EFunc2)Op)
					{
					case EFunc2::Add:
						Ret += BaseName + TEXT(" AddDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Add);
							Ret += TEXT("\tRet.Value = WSAdd(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\tRet.Value = A.Value + B.Value;") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\tRet.Ddx = A.Ddx + B.Ddx;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = A.Ddy + B.Ddy;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Sub:
						Ret += BaseName + TEXT(" SubDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Subtract);
							Ret += TEXT("\tRet.Value = WSSubtract(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\tRet.Value = A.Value - B.Value;") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\tRet.Ddx = A.Ddx - B.Ddx;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = A.Ddy - B.Ddy;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Mul:
						Ret += BaseName + TEXT(" MulDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorVector);
							Ret += TEXT("\tRet.Value = WSMultiply(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddx = A.Ddx * WSDemote(B.Value) + WSDemote(A.Value) * B.Ddx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddy = A.Ddy * WSDemote(B.Value) + WSDemote(A.Value) * B.Ddy;") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\tRet.Value = A.Value * B.Value;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddx = A.Ddx * B.Value + A.Value * B.Ddx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddy = A.Ddy * B.Value + A.Value * B.Ddy;") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Div:
						Ret += BaseName + TEXT(" DivDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(LHSDerivType) && IsLWCType(RHSDerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Divide);
							Ret += TEXT("\tRet.Value = WSDivide(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Demote, 2);
							Translator.AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorVector, 3);
							Ret += TEXT("\t") + FieldName + TEXT(" Denom = WSRcpDemote(WSMultiply(B.Value, B.Value));") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" dFdA =  WSDemote(WSMultiply(B.Value, Denom));") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" dFdB = -WSDemote(WSMultiply(A.Value, Denom));") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx + dFdB * B.Ddx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy + dFdB * B.Ddy;") HLSL_LINE_TERMINATOR;
						}
						else if (IsLWCType(LHSDerivType) && !IsLWCType(RHSDerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Divide);
							Ret += TEXT("\tRet.Value = WSDivide(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" Denom = rcp(B.Value * B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" dFdA =  B.Value * Denom;") HLSL_LINE_TERMINATOR;
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Demote);
							Translator.AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorVector);
							Ret += TEXT("\t") + FieldName + TEXT(" dFdB = -WSDemote(WSMultiply(A.Value, Denom));") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx + dFdB * B.Ddx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy + dFdB * B.Ddy;") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\tRet.Value = A.Value / B.Value;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" Denom = rcp(B.Value * B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" dFdA =  B.Value * Denom;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + FieldName + TEXT(" dFdB = -A.Value * Denom;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx + dFdB * B.Ddx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy + dFdB * B.Ddy;") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;

						break;
					case EFunc2::Fmod:
						// Only valid when B derivatives are zero.
						// We can't really do anything meaningful in the non-zero case.
						Ret += NonLWCBaseName + TEXT(" FmodDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + NonLWCBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
							Ret += TEXT("\tRet.Value = WSFmodDemote(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\tRet.Value = fmod(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\tRet.Ddx = A.Ddx;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = A.Ddy;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Modulo:
						// Only valid when B derivatives are zero.
						// We can't really do anything meaningful in the non-zero case.
						Ret += NonLWCBaseName + TEXT(" ModuloDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + NonLWCBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
							Ret += TEXT("\tRet.Value = WSModuloDemote(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\tRet.Value = (A.Value % B.Value);") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\tRet.Ddx = A.Ddx;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = A.Ddy;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Min:
						Ret += BaseName + TEXT(" MinDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
							Ret += TEXT("\t") + BoolName + TEXT(" Cmp = WSLess(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Value = WSSelect(Cmp, A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\t") + BoolName + TEXT(" Cmp = A.Value < B.Value;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Value = select(Cmp, A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\tRet.Ddx = select(Cmp, A.Ddx, B.Ddx);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = select(Cmp, A.Ddy, B.Ddy);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Max:
						Ret += BaseName + TEXT(" MaxDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
							Ret += TEXT("\t") + BoolName + TEXT(" Cmp = WSGreater(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Value = WSSelect(Cmp, A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\t") + BoolName + TEXT(" Cmp = A.Value > B.Value;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Value = select(Cmp, A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\tRet.Ddx = select(Cmp, A.Ddx, B.Ddx);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = select(Cmp, A.Ddy, B.Ddy);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Dot:
						Ret += ScalarName + TEXT(" DotDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						if (IsLWCType(DerivType))
						{
							Ret += TEXT("\t") + ScalarName + TEXT(" Ret = ConstructConstant") + ScalarName + TEXT("(WSPromote(0.0f));") HLSL_LINE_TERMINATOR;
						}
						else
						{
							Ret += TEXT("\t") + ScalarName + TEXT(" Ret = ConstructConstant") + ScalarName + TEXT("(0);") HLSL_LINE_TERMINATOR;
						}
						for (uint32 Component = 0; Component < NumComponents; Component++)
						{
							Ret += FString::Printf(TEXT("\tRet = AddDeriv%s(Ret,MulDeriv%s(Extract%s_%d(A),Extract%s_%d(B)));"), *Suffix, *Suffix, *BaseName, Component + 1, *BaseName, Component + 1) + HLSL_LINE_TERMINATOR;
						}
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Pow:
						// pow(A,B) = exp(B*log(A))
						//     pow'(A,B) = exp(B*log(A)) * (B'*log(A) + (B/A)*A')
						//     pow'(A,B) = pow(A,B) * (B'*log(A) + (B/A)*A')
						// sanity check when B is constant and A is a linear function (B'=0,A'=1)
						//     pow'(A,B) = pow(A,B) * (0*log(A) + (B/A)*1)
						//     pow'(A,B) = B * pow(A,B-1)
						Ret += BaseName + TEXT(" PowDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Value = pow(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddx = Ret.Value * (B.Ddx * log(A.Value) + (B.Value/A.Value)*A.Ddx);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = Ret.Value * (B.Ddy * log(A.Value) + (B.Value/A.Value)*A.Ddy);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::PowPositiveClamped:
						Ret += BaseName + TEXT(" PowPositiveClampedDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BoolName + TEXT(" InRange = (0.0 < B.Value);") HLSL_LINE_TERMINATOR; // should we check for A as well?
						Ret += TEXT("\t") + FieldName + TEXT(" Zero = 0.0;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Value = PositiveClampedPow(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddx = Ret.Value * (B.Ddx * log(A.Value) + (B.Value/A.Value)*A.Ddx);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = Ret.Value * (B.Ddy * log(A.Value) + (B.Value/A.Value)*A.Ddy);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddx = select(InRange, Ret.Ddx, Zero);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = select(InRange, Ret.Ddy, Zero);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Atan2:
						Ret += BaseName + TEXT(" Atan2Deriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Value = atan2(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" Denom = rcp(A.Value * A.Value + B.Value * B.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA =  B.Value * Denom;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdB = -A.Value * Denom;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx + dFdB * B.Ddx;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy + dFdB * B.Ddy;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Atan2Fast:
						Ret += BaseName + TEXT(" Atan2FastDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
						Ret += TEXT("{") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Value = atan2Fast(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" Denom = rcp(A.Value * A.Value + B.Value * B.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA =  B.Value * Denom;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdB = -A.Value * Denom;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx + dFdB * B.Ddx;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy + dFdB * B.Ddy;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
						Ret += TEXT("}") HLSL_LINE_TERMINATOR;
						Ret += TEXT("") HLSL_LINE_TERMINATOR;
						break;
					case EFunc2::Cross:
						if (DerivType == EDerivativeType::Float3)
						{
							// (A*B)' = A' * B + A * B'
							// Cross(A, B) = A.yzx * B.zxy - A.zxy * B.yzx;
							// Cross(A, B)' = A.yzx' * B.zxy + A.yzx * B.zxy' - A.zxy' * B.yzx - A.zxy * B.yzx';
							Ret += BaseName + TEXT(" CrossDeriv") + Suffix + TEXT("(") + LHSBaseName + TEXT(" A, ") + RHSBaseName + TEXT(" B)") HLSL_LINE_TERMINATOR;
							Ret += TEXT("{") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Value = cross(A.Value, B.Value);") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddx = A.Ddx.yzx * B.Value.zxy + A.Value.yzx * B.Ddx.zxy - A.Ddx.zxy * B.Value.yzx - A.Value.zxy * B.Ddx.yzx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\tRet.Ddy = A.Ddy.yzx * B.Value.zxy + A.Value.yzx * B.Ddy.zxy - A.Ddy.zxy * B.Value.yzx - A.Value.zxy * B.Ddy.yzx;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
							Ret += TEXT("}") HLSL_LINE_TERMINATOR;
							Ret += TEXT("") HLSL_LINE_TERMINATOR;
						}
						break;
					default:
						check(0);
						break;
					}
				}
			}
		}
	}

	for (int32 Op = 0; Op < (int32)EFunc1::Num; Op++)
	{
		for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
		{
			if (bFunc1OpIsEnabled[Op][Index] || IsDebugGenerateAllFunctionsEnabled())
			{
				EDerivativeType DerivType = (EDerivativeType)Index;
				EDerivativeType ScalarDerivType = IsLWCType(DerivType) ? EDerivativeType::LWCScalar : EDerivativeType::Float1;
				FString BaseName	= GetDerivVectorName(DerivType);
				FString NonLWCBaseName = GetDerivVectorName(MakeNonLWCType(DerivType));
				FString ScalarName = GetDerivVectorName(ScalarDerivType);
				FString FieldName	= GetFloatVectorName(MakeNonLWCType(DerivType));
				FString NonLWCFieldName = GetFloatVectorName(MakeNonLWCType(DerivType));
				FString BoolName	= GetBoolVectorName(DerivType);
				const FString Suffix = IsLWCType(DerivType) ? TEXT("LWC") : TEXT("");

				switch((EFunc1)Op)
				{
				case EFunc1::Abs:
					Ret += BaseName + TEXT(" AbsDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCFieldName + TEXT(" One = 1.0f;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSAbs(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = select(WSGreaterEqual(A.Value, 0.0f), One, -One);") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = abs(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = select(A.Value >= 0.0f, One, -One);") HLSL_LINE_TERMINATOR;
					}

					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Sin:
					Ret += NonLWCBaseName + TEXT(" SinDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSSin(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA = WSCos(A.Value);");
					}
					else
					{
						Ret += TEXT("\tRet.Value = sin(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA = cos(A.Value);");
					}
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Cos:
					Ret += NonLWCBaseName + TEXT(" CosDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSCos(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA = -WSSin(A.Value);");
					}
					else
					{
						Ret += TEXT("\tRet.Value = cos(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA = -sin(A.Value);");
					}
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Tan:
					Ret += NonLWCBaseName + TEXT(" TanDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSTan(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(Pow2(WSCos(A.Value)));");
					}
					else
					{
						Ret += TEXT("\tRet.Value = tan(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(Pow2(cos(A.Value)));");
					}
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Asin:
					Ret += BaseName + TEXT(" AsinDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = asin(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rsqrt(max(1.0f - A.Value * A.Value, 0.00001f));");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::AsinFast:
					Ret += BaseName + TEXT(" AsinFastDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = asinFast(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rsqrt(max(1.0f - A.Value * A.Value, 0.00001f));");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Acos:
					Ret += BaseName + TEXT(" AcosDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = acos(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = -rsqrt(max(1.0f - A.Value * A.Value, 0.00001f));");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::AcosFast:
					Ret += BaseName + TEXT(" AcosFastDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = acosFast(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = -rsqrt(max(1.0f - A.Value * A.Value, 0.00001f));");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Atan:
					Ret += BaseName + TEXT(" AtanDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = atan(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(A.Value * A.Value + 1.0f);");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::AtanFast:
					Ret += BaseName + TEXT(" AtanFastDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = atanFast(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(A.Value * A.Value + 1.0f);");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Sqrt:
					Ret += NonLWCBaseName + TEXT(" SqrtDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSSqrtDemote(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = 0.5f * WSRsqrtDemote(WSMax(A.Value, 0.00001f));") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = sqrt(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = 0.5f * rsqrt(max(A.Value, 0.00001f));") HLSL_LINE_TERMINATOR;
					}
					
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Rcp:
					Ret += NonLWCBaseName + TEXT(" RcpDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSRcpDemote(A.Value);") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = rcp(A.Value);") HLSL_LINE_TERMINATOR;
					}
					Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = -Ret.Value * Ret.Value;");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Rsqrt:
					Ret += NonLWCBaseName + TEXT(" RsqrtDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSRsqrtDemote(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = -0.5f * WSRsqrtDemote(A.Value) * WSRcpDemote(A.Value);");
					}
					else
					{
						Ret += TEXT("\tRet.Value = rsqrt(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + NonLWCFieldName + TEXT(" dFdA = -0.5f * rsqrt(A.Value) * rcp(A.Value);");
					}
					
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Saturate:
					Ret += NonLWCBaseName + TEXT(" SaturateDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCFieldName + TEXT(" Zero = 0.0f;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSSaturateDemote(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BoolName + TEXT(" InRange = WSEquals(Ret.Value, A.Value);") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = saturate(A.Value);") HLSL_LINE_TERMINATOR;
						Ret += TEXT("\t") + BoolName + TEXT(" InRange = and(0.0 < A.Value, A.Value < 1.0);") HLSL_LINE_TERMINATOR;
					}
					Ret += TEXT("\tRet.Ddx = select(InRange, A.Ddx, Zero);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = select(InRange, A.Ddy, Zero);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Frac:
					Ret += NonLWCBaseName + TEXT(" FracDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + NonLWCBaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSFracDemote(A.Value);") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = frac(A.Value);") HLSL_LINE_TERMINATOR;
					}
					Ret += TEXT("\tRet.Ddx = A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Log:
					Ret += BaseName + TEXT(" LogDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = log(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(A.Value);");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx ;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Log2:
					Ret += BaseName + TEXT(" Log2Deriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = log2(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(A.Value) * 1.442695f;");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx ;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Log10:
					Ret += BaseName + TEXT(" Log10Deriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = log10(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = rcp(A.Value) * 0.4342945f;");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Exp:
					Ret += BaseName + TEXT(" ExpDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = exp(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddx = exp(A.Value) * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = exp(A.Value) * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Exp2:
					Ret += BaseName + TEXT(" Exp2Deriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Value = exp2(A.Value);") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + FieldName + TEXT(" dFdA = exp2(A.Value) * 0.693147f;");
					Ret += TEXT("\tRet.Ddx = dFdA * A.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = dFdA * A.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Length:
					Ret += ScalarName + TEXT(" LengthDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\t") + ScalarName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSLength(A.Value);") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = length(A.Value);") HLSL_LINE_TERMINATOR;
					}
					Ret += FString::Printf(TEXT("\tFloatDeriv Deriv = SqrtDeriv%s(DotDeriv%s(A,A));") HLSL_LINE_TERMINATOR, * Suffix, * Suffix);
					Ret += TEXT("\tRet.Ddx = Deriv.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = Deriv.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::InvLength:
					Ret += TEXT("FloatDeriv InvLengthDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tFloatDeriv Ret;") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
						Ret += TEXT("\tRet.Value = WSRcpLengthDemote(A.Value);") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\tRet.Value = rcp(length(A.Value));") HLSL_LINE_TERMINATOR;
					}
					Ret += FString::Printf(TEXT("\tFloatDeriv Deriv = RsqrtDeriv%s(DotDeriv%s(A,A));") HLSL_LINE_TERMINATOR, *Suffix, *Suffix);
					Ret += TEXT("\tRet.Ddx = Deriv.Ddx;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\tRet.Ddy = Deriv.Ddy;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				case EFunc1::Normalize:
					Ret += NonLWCBaseName + TEXT(" NormalizeDeriv") + Suffix + TEXT("(") + BaseName + TEXT(" A)") HLSL_LINE_TERMINATOR;
					Ret += TEXT("{") HLSL_LINE_TERMINATOR;
					Ret += FString::Printf(TEXT("\tFloatDeriv InvLen = RsqrtDeriv%s(DotDeriv%s(A,A));") HLSL_LINE_TERMINATOR, *Suffix, *Suffix);
					Ret += TEXT("\t") + BaseName + TEXT(" Ret = MulDeriv") + Suffix + TEXT("(") + ConvertDeriv(TEXT("InvLen"), DerivType, EDerivativeType::Float1) + TEXT(", A);") HLSL_LINE_TERMINATOR;
					if (IsLWCType(DerivType))
					{
						// Convert the result to non-LWC
						Ret += TEXT("\treturn ") + ConvertDeriv(TEXT("Ret"), MakeNonLWCType(DerivType), DerivType) + TEXT(";") HLSL_LINE_TERMINATOR;
					}
					else
					{
						Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
					}
					Ret += TEXT("}") HLSL_LINE_TERMINATOR;
					Ret += TEXT("") HLSL_LINE_TERMINATOR;
					break;
				default:
					check(0);
					break;
				}
			}
		}
	}

	for (int32 Index = 0; Index < NumDerivativeTypes; Index++)
	{
		EDerivativeType DerivType = (EDerivativeType)Index;
		const FString BaseName = GetDerivVectorName(DerivType);
		const FString NonLWCBaseName = GetDerivVectorName(MakeNonLWCType(DerivType));
		const FString FieldName = GetFloatVectorName(DerivType);
		const FString BoolName = GetBoolVectorName(DerivType);

		if (bLerpEnabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			
			// lerp(a,b,s) = a*(1-s) + b*s
			// lerp(a,b,s)' = a' * (1 - s') + b' * s + s' * (b - a)
			Ret += FString::Printf(TEXT("%s LerpDeriv(%s A, %s B, %s S)"), *BaseName, *BaseName, *BaseName, *NonLWCBaseName) + HLSL_LINE_TERMINATOR;
			Ret += TEXT("{") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t") + BaseName + TEXT(" Ret;") HLSL_LINE_TERMINATOR;
			if (IsLWCType(DerivType))
			{
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Other);
				Ret += TEXT("\tRet.Value = WSLerp(A.Value, B.Value, S.Value);") HLSL_LINE_TERMINATOR;
				Translator.AddLWCFuncUsage(ELWCFunctionKind::Subtract, 2);
				Ret += TEXT("\tRet.Ddx = lerp(A.Ddx, B.Ddx, S.Value) + S.Ddx * WSSubtractDemote(B.Value, A.Value);") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddy = lerp(A.Ddy, B.Ddy, S.Value) + S.Ddy * WSSubtractDemote(B.Value, A.Value);") HLSL_LINE_TERMINATOR;
			}
			else
			{
				Ret += TEXT("\tRet.Value = lerp(A.Value, B.Value, S.Value);") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddx = lerp(A.Ddx, B.Ddx, S.Value) + S.Ddx * (B.Value - A.Value);") HLSL_LINE_TERMINATOR;
				Ret += TEXT("\tRet.Ddy = lerp(A.Ddy, B.Ddy, S.Value) + S.Ddy * (B.Value - A.Value);") HLSL_LINE_TERMINATOR;
			}
			Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("}") HLSL_LINE_TERMINATOR;
			Ret += TEXT("") HLSL_LINE_TERMINATOR;
		}

		if (bIfEnabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			// TODO - do we need to make these bool-vectors?
			Ret += FString::Printf(TEXT("%s IfDeriv(bool bGreaterEqual, %s Greater, %s Less)"), *BaseName, *BaseName, *BaseName) + HLSL_LINE_TERMINATOR;
			Ret += TEXT("{") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tif(bGreaterEqual)") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t\treturn Greater;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\telse") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t\treturn Less;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("}") HLSL_LINE_TERMINATOR;
			Ret += TEXT("") HLSL_LINE_TERMINATOR;
		}

		if (bIf2Enabled[Index] || IsDebugGenerateAllFunctionsEnabled())
		{
			Ret += FString::Printf(TEXT("%s IfDeriv(bool bNotEqual, bool bGreaterEqual, %s Greater, %s Less, %s Equal)"), *BaseName, *BaseName, *BaseName, *BaseName) + HLSL_LINE_TERMINATOR;
			Ret += TEXT("{") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tif(!bNotEqual)") HLSL_LINE_TERMINATOR;	// Written like this to preserve NaN behavior of original code.
			Ret += TEXT("\t\treturn Equal;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\tif(bGreaterEqual)") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t\treturn Greater;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\telse") HLSL_LINE_TERMINATOR;
			Ret += TEXT("\t\treturn Less;") HLSL_LINE_TERMINATOR;
			Ret += TEXT("}") HLSL_LINE_TERMINATOR;
			Ret += TEXT("") HLSL_LINE_TERMINATOR;
		}
	}

	if (bRotateScaleOffsetTexCoords || IsDebugGenerateAllFunctionsEnabled())
	{
		// float2(dot(InTexCoords, InRotationScale.xy), dot(InTexCoords, InRotationScale.zw)) + InOffset;
		// InTexCoords.xy * InRotationScale.xw + InTexCoords.yx * InRotationScale.yz + InOffset;
		Ret += TEXT("FloatDeriv2 RotateScaleOffsetTexCoordsDeriv(FloatDeriv2 TexCoord, FloatDeriv4 RotationScale, FloatDeriv2 Offset)") HLSL_LINE_TERMINATOR;
		Ret += TEXT("{") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tFloatDeriv2 Ret = Offset;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tRet = AddDeriv(Ret, MulDeriv(TexCoord, SwizzleDeriv2(RotationScale, xw)));") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tRet = AddDeriv(Ret, MulDeriv(SwizzleDeriv2(TexCoord, yx), SwizzleDeriv2(RotationScale, yz)));") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\treturn Ret;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("}") HLSL_LINE_TERMINATOR;
		Ret += TEXT("") HLSL_LINE_TERMINATOR;
	}
	
	if (bUnMirrorEnabled[1][1] || IsDebugGenerateAllFunctionsEnabled())
	{
		// UnMirrorUV
		Ret += TEXT("FloatDeriv2 UnMirrorUV(FloatDeriv2 UV, FMaterialPixelParameters Parameters)") HLSL_LINE_TERMINATOR;
		Ret += TEXT("{") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tconst MaterialFloat Scale = (Parameters.UnMirrored * 0.5f);") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Value = UV.Value * Scale + 0.5f;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Ddx *= Scale;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Ddy *= Scale;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\treturn UV;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("}") HLSL_LINE_TERMINATOR;
		Ret += TEXT("") HLSL_LINE_TERMINATOR;
	}
	
	if(bUnMirrorEnabled[1][0] || IsDebugGenerateAllFunctionsEnabled())
	{
		// UnMirrorU
		Ret += TEXT("FloatDeriv2 UnMirrorU(FloatDeriv2 UV, FMaterialPixelParameters Parameters)") HLSL_LINE_TERMINATOR;
		Ret += TEXT("{") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tconst MaterialFloat Scale = (Parameters.UnMirrored * 0.5f);") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Value.x = UV.Value.x * Scale + 0.5f;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Ddx.x *= Scale;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Ddy.x *= Scale;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\treturn UV;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("}") HLSL_LINE_TERMINATOR;
		Ret += TEXT("") HLSL_LINE_TERMINATOR;
	}
	
	if (bUnMirrorEnabled[0][1] || IsDebugGenerateAllFunctionsEnabled())
	{
		// UnMirrorV
		Ret += TEXT("FloatDeriv2 UnMirrorV(FloatDeriv2 UV, FMaterialPixelParameters Parameters)") HLSL_LINE_TERMINATOR;
		Ret += TEXT("{") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tconst MaterialFloat Scale = (Parameters.UnMirrored * 0.5f);") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Value.y = UV.Value.y * Scale + 0.5f;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Ddx.y *= Scale;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\tUV.Ddy.y *= Scale;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("\treturn UV;") HLSL_LINE_TERMINATOR;
		Ret += TEXT("}") HLSL_LINE_TERMINATOR;
		Ret += TEXT("") HLSL_LINE_TERMINATOR;
	}

	return Ret;
}


FString FMaterialDerivativeAutogen::ApplyUnMirror(FString Value, bool bUnMirrorU, bool bUnMirrorV)
{
	if (bUnMirrorU && bUnMirrorV)
	{
		Value = FString::Printf(TEXT("UnMirrorUV(%s, Parameters)"), *Value);
	}
	else if (bUnMirrorU)
	{
		Value = FString::Printf(TEXT("UnMirrorU(%s, Parameters)"), *Value);
	}
	else if (bUnMirrorV)
	{
		Value = FString::Printf(TEXT("UnMirrorV(%s, Parameters)"), *Value);
	}

	bUnMirrorEnabled[bUnMirrorU][bUnMirrorV] = true;
	
	return Value;	
}

#endif // WITH_EDITORONLY_DATA
