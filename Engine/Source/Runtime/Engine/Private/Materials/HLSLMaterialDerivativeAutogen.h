// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialShared.h"

#include "HLSLMaterialDerivativeAutogen.generated.h"

class FHLSLMaterialTranslator;

/**
* For a node, the known information of the partial derivatives.
* NotAware	- This node is made by a function that has no knowledge of analytic partial derivatives.
* NotValid	- This node is aware of partial derivatives, and knows that one of its source inputs is not partial derivative aware, and therefore its value is not to be used.
* Zero		- This node is aware of partial derivatives, and knows that it's value is zero, as is the case for uniform parameters.
* Valid		- This node is aware of partial derivatives, and knows that it has a calculated value.
**/
UENUM()
enum class EDerivativeStatus : uint8
{
	NotAware,
	NotValid,
	Zero,
	Valid,
};

#if WITH_EDITORONLY_DATA

inline bool IsDerivativeValid(EDerivativeStatus Status)
{
	return Status == EDerivativeStatus::Valid || Status == EDerivativeStatus::Zero;
}

enum class EDerivativeType
{
	None = -1,
	Float1 = 0,
	Float2,
	Float3,
	Float4,
	LWCScalar,
	LWCVector2,
	LWCVector3,
	LWCVector4,
	Num,
};
static const int32 NumDerivativeTypes = (int32)EDerivativeType::Num;

inline bool IsLWCType(EDerivativeType Type)
{
	return (int32)Type >= (int32)EDerivativeType::LWCScalar;
}

inline EDerivativeType MakeNonLWCType(EDerivativeType Type)
{
	switch (Type)
	{
	case EDerivativeType::LWCScalar: return EDerivativeType::Float1;
	case EDerivativeType::LWCVector2: return EDerivativeType::Float2;
	case EDerivativeType::LWCVector3: return EDerivativeType::Float3;
	case EDerivativeType::LWCVector4: return EDerivativeType::Float4;
	default: return Type;
	}
}

inline EDerivativeType MakeLWCType(EDerivativeType Type)
{
	switch (Type)
	{
	case EDerivativeType::Float1: return EDerivativeType::LWCScalar;
	case EDerivativeType::Float2: return EDerivativeType::LWCVector2;
	case EDerivativeType::Float3: return EDerivativeType::LWCVector3;
	case EDerivativeType::Float4: return EDerivativeType::LWCVector4;
	default: return Type;
	}
}

EDerivativeType GetDerivType(EMaterialValueType ValueType, bool bAllowNonFloat = false);


struct FDerivInfo
{
	EMaterialValueType	Type;
	EDerivativeType		DerivType;
	EDerivativeStatus	DerivativeStatus;
};

class FMaterialDerivativeAutogen
{
public:
	// Unary functions
	enum class EFunc1
	{
		Abs,
		Log,
		Log2,
		Log10,
		Exp,
		Exp2,
		Sin,
		Cos,
		Tan,
		Asin,
		AsinFast,
		Acos,
		AcosFast,
		Atan,
		AtanFast,
		Sqrt,
		Rcp,
		Rsqrt,
		Saturate,
		Frac,
		Length,
		InvLength,
		Normalize,
		Num
	};

	// Binary functions
	enum class EFunc2
	{
		Add,
		Sub,
		Mul,
		Div,
		Fmod,
		Modulo,
		Max,
		Min,
		Dot,	// Depends on Add/Mul, so it must come after them
		Pow,
		PowPositiveClamped,
		Cross,
		Atan2,
		Atan2Fast,
		Num
	};

	int32 GenerateExpressionFunc1(FHLSLMaterialTranslator& Translator, EFunc1 Op, int32 SrcCode);
	int32 GenerateExpressionFunc2(FHLSLMaterialTranslator& Translator, EFunc2 Op, int32 LhsCode, int32 RhsCode);

	int32 GenerateLerpFunc(FHLSLMaterialTranslator& Translator, int32 A, int32 B, int32 S);
	int32 GenerateRotateScaleOffsetTexCoordsFunc(FHLSLMaterialTranslator& Translator, int32 TexCoord, int32 RotationScale, int32 Offset);
	int32 GenerateIfFunc(FHLSLMaterialTranslator& Translator, int32 A, int32 B, int32 Greater, int32 Equal, int32 Less, int32 Threshold);
	
	FString GenerateUsedFunctions(FHLSLMaterialTranslator& Translator);

	FString ApplyUnMirror(FString Value, bool bUnMirrorU, bool bUnMirrorV);

	FString ConstructDeriv(const FString& Value, const FString& Ddx, const FString& Ddy, EDerivativeType DstType);
	FString ConstructDerivFinite(const FString& Value, EDerivativeType DstType);

	FString ConvertDeriv(const FString& Value, EDerivativeType DstType, EDerivativeType SrcType);

private:
	struct FOperationType1
	{
		FOperationType1(EDerivativeType InType) : ReturnType(InType), IntermediateType(InType) {}
		FOperationType1(EDerivativeType InReturnType, EDerivativeType InIntermediateType) : ReturnType(InReturnType), IntermediateType(InIntermediateType) {}

		EDerivativeType ReturnType; // type of the result
		EDerivativeType IntermediateType; // type of the inputs (may need to implicitly convert to this type before the operation)
	};

	struct FOperationType2
	{
		FOperationType2(EDerivativeType InType) : ReturnType(InType), LhsIntermediateType(InType), RhsIntermediateType(InType) {}
		FOperationType2(EDerivativeType InReturnType, EDerivativeType InIntermediateType) : ReturnType(InReturnType), LhsIntermediateType(InIntermediateType), RhsIntermediateType(InIntermediateType) {}
		FOperationType2(EDerivativeType InReturnType, EDerivativeType InLhsIntermediateType, EDerivativeType InRhsIntermediateType) : ReturnType(InReturnType), LhsIntermediateType(InLhsIntermediateType), RhsIntermediateType(InRhsIntermediateType) {}

		EDerivativeType ReturnType; // type of the result
		EDerivativeType LhsIntermediateType; // type of the inputs (may need to implicitly convert to this type before the operation)
		EDerivativeType RhsIntermediateType; // type of the inputs (may need to implicitly convert to this type before the operation)
	};

	static FOperationType1 GetFunc1ReturnType(EDerivativeType SrcType, EFunc1 Op);
	static FOperationType2 GetFunc2ReturnType(EDerivativeType LhsType, EDerivativeType RhsType, EFunc2 Op);

	FString CoerceValueRaw(FHLSLMaterialTranslator& Translator, const FString& Token, const FDerivInfo& SrcInfo, EDerivativeType DstType);
	FString CoerceValueDeriv(const FString& Token, const FDerivInfo& SrcInfo, EDerivativeType DstType);

	void EnableGeneratedDepencencies();

	// State to keep track of which derivative functions have been used and need to be generated.
	bool bFunc1OpIsEnabled[(int32)EFunc1::Num][NumDerivativeTypes] = {};
	bool bFunc2OpIsEnabled[(int32)EFunc2::Num][NumDerivativeTypes][NumDerivativeTypes] = {};

	bool bConstructDerivEnabled[NumDerivativeTypes] = {};
	bool bConstructConstantDerivEnabled[NumDerivativeTypes] = {};
	bool bConstructFiniteDerivEnabled[NumDerivativeTypes] = {};

	bool bConvertDerivEnabled[NumDerivativeTypes][NumDerivativeTypes] = {};

	bool bExtractIndexEnabled[NumDerivativeTypes] = {};

	bool bIfEnabled[NumDerivativeTypes] = {};
	bool bIf2Enabled[NumDerivativeTypes] = {};
	bool bLerpEnabled[NumDerivativeTypes] = {};
	bool bRotateScaleOffsetTexCoords = false;
	bool bUnMirrorEnabled[2][2] = {};
	
};

#endif // WITH_EDITORONLY_DATA
