// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HLSLMaterialTranslator.cpp: Translates material expressions into HLSL code.
=============================================================================*/

#include "HLSLMaterialTranslator.h"
#include "VT/VirtualTextureScalability.h"
#include "Engine/Engine.h"
#include "MaterialDomain.h"
#include "MaterialSharedPrivate.h"
#include "Engine/Texture.h"
#include "Engine/TextureCollection.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Field/FieldSystemTypes.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/HLSLMaterialDerivativeAutogen.h"
#include "Materials/MaterialFunction.h"
#include "MaterialExpressionSettings.h"
#include "ProfilingDebugging/CookStats.h"
#include "Engine/RendererSettings.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionAbsorptionMediumMaterialOutput.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h" 
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionNoise.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionUtils.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionViewProperty.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionFirstPersonOutput.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialSourceTemplate.h"
#include "Materials/SubstrateMaterial.h"
#include "Materials/MaterialInsights.h"
#include "StringTemplate.h"
#include "ParameterCollection.h"
#include "RenderUtils.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "Stats/StatsMisc.h"
#include "Stats/StatsTrace.h"
#include "SubstrateDefinitions.h"
#include "UObject/ObjectRedirector.h"
#include "VT/RuntimeVirtualTexture.h"
#include <memory>
#include <tuple>
#include "ShaderCompiler.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Templates/GuardValueAccessors.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataCache.h"
#include "MaterialCachedData.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Engine/SubsurfaceProfile.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionTangentOutput.h"
#include "MaterialCache/MaterialCacheAttribute.h"

#if WITH_EDITORONLY_DATA
#include "Materials/MaterialExpressionSubstrate.h"
#include "ShaderPlatformCachedIniValue.h"
#endif

#include <functional>

TAutoConsoleVariable<bool> CVarShadersSDCEEnabled(
	TEXT("r.Shaders.SDCE"),
	true,
	TEXT("Strip dead code using symbolic dead code elimination on structured member stores"));

#if ENABLE_COOK_STATS && STATS

static bool GEnableMaterialTranslationLogFile = false;
static FAutoConsoleVariableRef CVarEnableMaterialTranslationLogFile(
	TEXT("r.Material.EnableTranslationLogFile"),
	GEnableMaterialTranslationLogFile,
	TEXT("Enables material translation log file generation for tracking materials translation time (written to  \"ShaderDebugInfo/MaterialTranslationLog-X.csv\")."),
	ECVF_ReadOnly);
/**
 * Utility used to create a MaterialTranslationLog.txt file during cooks that contains the list of all translated materials
 * and other info such as how long the translation took.
 */
struct FCsvLogFile
{
	FCriticalSection CriticalSection;
	FString LogContent;

	static FCsvLogFile& Get()
	{
		static FCsvLogFile Instance;
		return Instance;
	}

	FCsvLogFile()
	{
		LogContent = TEXT("Date,MaterialName,TranslationTime\n");
	}

	void AddEntry(FStringView MaterialName, FDateTime DateTime, float TranslationTime)
	{
		if (GEnableMaterialTranslationLogFile)
		{
			FScopeLock Lock{ &CriticalSection };
			FString DateTimeString = DateTime.ToString(TEXT("%Y-%m-%d %H:%M:%S"));
			LogContent.Appendf(TEXT("%s,%s,%f\n"), *DateTimeString, MaterialName.GetData(), TranslationTime);
		}
	}

	void Save()
	{
		if (GEnableMaterialTranslationLogFile)
		{
			uint32 MultiprocessId = UE::GetMultiprocessId();
			FString FilePath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / FString::Printf(TEXT("MaterialTranslationLog-%d.csv"), MultiprocessId);
			if (!FFileHelper::SaveStringToFile(LogContent, *FilePath))
			{
				UE_LOGF(LogMaterial, Display, "Cannot open MaterialTranslation log file '%ls'", *FilePath);
			}
		}
	}

};

static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		FCsvLogFile::Get().Save();
	});

#endif

#if WITH_EDITORONLY_DATA

using namespace Substrate;

static int32 GAnalyticDerivEnabled = 1;
static FAutoConsoleVariableRef CVarAnalyticDerivEnabled(
	TEXT("r.MaterialEditor.AnalyticDeriv"),
	GAnalyticDerivEnabled,
	TEXT("Enable analytic derivative code generation.")
	);

// Debugging options
static int32 GDebugTextureSampleEnabled = 0;
static FAutoConsoleVariableRef CVarAnalyticDerivDebugTextureSample(
	TEXT("r.MaterialEditor.AnalyticDeriv.DebugTextureSample"),
	GDebugTextureSampleEnabled,
	TEXT("Debug: Instrument texture sampling with modes that can be controlled with r.GeneralPurposeTweak/r.GeneralPurposeTweak2.")
);

static int32 GDebugEmitInvalidDerivTokensEnabled = 0;
static FAutoConsoleVariableRef CVarAnalyticDerivDebugEmitInvalidDerivTokens(
	TEXT("r.MaterialEditor.AnalyticDeriv.DebugEmitInvalidDerivTokens"),
	GDebugEmitInvalidDerivTokensEnabled,
	TEXT("Debug: Emit '$' tokens to mark expressions with invalid derivatives.\n")
);

static int32 GLWCEnabled = 1;
static FAutoConsoleVariableRef CVarLWCEnabled(
	TEXT("r.MaterialEditor.LWCEnabled"),
	GLWCEnabled,
	TEXT("Enable generation of LWC values in materials. If disabled, materials will perform all operations at float-precision")
);

static bool GPedanticErrorChecksEnabled = false;
static FAutoConsoleVariableRef CVarPedanticErrorChecksEnabled(
	TEXT("r.Material.PedanticErrorChecksEnabled"),
	GPedanticErrorChecksEnabled,
	TEXT("Enables material compilation pedantic error checking"));

static bool GCullIntermediateUniformExpressions = true;
static FAutoConsoleVariableRef CVarCullIntermediateUniformExpressions(
	TEXT("r.Material.CullIntermediateUniformExpressions"),
	GCullIntermediateUniformExpressions,
	TEXT("Enables culling of intermediate uniform expressions, reducing preshader count, saving performance"));

static int32 GPreshaderGapInterval = 32;
static FAutoConsoleVariableRef CVarPreshaderGapInterval(
	TEXT("r.Material.PreshaderGapInterval"),
	GPreshaderGapInterval,
	TEXT("Insert an empty element in the preshader buffer every specified number of elements in the buffer.  Workaround for a shader compiler register overflow bug."));

/* Controls whether DDC caching of material translation results should be forcefully disabled. */
#define FORCE_DISABLE_MATERIAL_TRANSLATION_DDC false

/* Helper macro to check whether the DDC material translation data has arrived and
 * therefore quit material translation. */
#if FORCE_DISABLE_MATERIAL_TRANSLATION_DDC
#define CHECK_DDC_QUERY_FINISHED_ELSE_RETURN()
#else
#define CHECK_DDC_QUERY_FINISHED_ELSE_RETURN() if (DDCQueryHit) { return; }
#endif

UE::DerivedData::FCacheBucket MaterialTranslationDDCBucket = UE::DerivedData::FCacheBucket(TEXT("MaterialTranslation"));
UE::DerivedData::FValueId MaterialCompilationOutputId = UE::DerivedData::FValueId::FromName("FHLSLMaterialTranslator_MaterialCompilationOutput");
UE::DerivedData::FValueId MaterialResultsOutputId = UE::DerivedData::FValueId::FromName("FHLSLMaterialTranslator_Results");
UE::DerivedData::FValueId EnvironmentDefinesId = UE::DerivedData::FValueId::FromName("FHLSLMaterialTranslator_EnvironmentDefines");

/** Data structure used to cache a part of material translation results. It contains all the generated
 *  defines that will be declared during the compilation of the generated material shader.
 *  Note: if you make a change to this structure, remember to bump FDevSystemGuids::MaterialTranslationDDCVersion.
 */
struct FHLSLMaterialTranslator::FEnvironmentDefines
{
	enum EVirtualPageType
	{
		TABLE_MESH_PAINT           = (1 << 0),
		TABLE_MATERIAL_CACHE       = (1 << 1),
		TABLE_COLLECTION           = (1 << 2),
		LOCAL_TABLE0               = (1 << 3),
		LOCAL_TABLE1               = (1 << 4),
		LOCAL_ADAPTIVE_INDIRECTION = (1 << 5)
	};

	bool bNeedsParticlePosition;
	bool bNeedsParticleVelocity;
	bool bUseDynamicParameters;
	uint32 DynamicParametersMask;
	bool bNeedsParticleTime;
	bool bUsesParticleMotionBlur;
	bool bNeedsParticleRandom;
	bool bSphericalParticleOpacity;
	bool bUseParticleSubUVs;
	bool bLightmapUVAccess;
	bool bUsesAOMaterialMask;
	bool bUsesSpeedtree;
	bool bNeedsWorldPositionExcludingShaderOffsets;
	bool bNeedsParticleSize;
	bool bNeedsParticleSpriteRotation;
	bool bNeedsSceneTextures;
	bool bAlphaPropagatePostProcessInput0;
	bool bAlphaPropagateUserSceneTexture;
	uint32 UsedSceneTextures;
	bool bUsesEyeAdaptation;
	bool bVirtualTextureOutput;
	bool bUsesPerInstanceCustomData;
	bool bUsesPerInstanceRandomPS;
	bool bUsesPerInstanceFadeAmount;
	bool bUsesVertexInterpolator;
	bool bUsesSkyAtmosphere;
	bool bUsesVertexColor;
	bool bUsesParticleColor;
	bool bUsesParticleLocalToWorld;
	bool bUsesParticleWorldToLocal;
	bool bUsesInstanceLocalToWorldPS;
	bool bUsesInstanceWorldToLocalPS;
	bool bUsesTransformVector;
	bool bUsesPixelDepthOffset;
	bool bUsesWorldPositionOffset;
	bool bUsesDisplacement;
	bool bUsesEmissiveColor;
	bool bUsesDistortion;
	bool bUsesExplicitDerivatives;
	bool bUsesFirstPersonInterpolation;
	bool bMaterialEnableTranslucencyFogging;
	bool bMaterialEnableTranslucencyCloudFogging;
	bool bMaterialIsSky;
	bool bMaterialComputeFogPerPixel;
	bool bMaterialFullyRough;
	bool bMaterialUsesAnisotropy;
	bool bMaterialUsesSpecularProfile;
	uint8 MaterialDecalReadMask;
	int8 PixelDepthOffsetMode;
	bool bMaterialUsesDecalLookup;
	uint8 MaterialPathTracingBufferRead;
	bool MaterialNeuralPostProcess;
	uint32 NumVirtualTextureSamples;
	uint32 NumVirtualTextureFeedbackRequests;
	bool bMaterialVirtualTextureFeedback;
	TArray<uint8> VirtualPageTypes;
	bool bSingleLayerWaterShadingQuality;
	bool bShadingModelsIsLit;
	uint32 MaterialShadingModelEnabled;
	bool bMaterialSubsurfaceProfileUseCurvature;
	bool bMaterialShadingModelEyeUseCurvature;
	bool bDisableForwardLocalLights;
	bool bSingleLayerWaterSeparatedMainLight;
	bool bMaterialSingleShadingModel;
	bool bMaterialVolumetricAdvanced;
	bool bMaterialVolumetricAdvancedPhasePerSample;
	bool bMaterialVolumetricAdvancedGreyscaleMaterial;
	bool bMaterialVolumetricAdvancedRaymarchVolumeShadow;
	bool bMaterialVolumetricAdvancedClampMultiscatteringContribution;
	uint32 MaterialVolumetricAdvancedMultiscatteringOctaveCount;
	bool bMaterialVolumetricAdvancedConservativeDensity;
	bool bMaterialVolumetricAdvancedOverrideAmbientOcclusion;
	bool bMaterialVolumetricAdvancedAdvancedGroundContribution;
	bool bMaterialVolumetricCloudEmptySpaceSkippingOutput;
	bool bDualSourceColorBlendingEnabled;
	TArray<TPair<FString, int32>> SubstrateDefines;
	uint8 NumCustomizedUVs;
	bool bTextureSampleDebug;
	bool bMaterialEnableTranslucentLocalLightShadow;
	bool bMaterialEnableTranslucentHighQualityLocalLightShadow;
	bool bMaterialEnableTranslucentHighQualityDirectionalLightShadow;
	TArray<TObjectPtr<UMaterialParameterCollection>> ParameterCollections;

	bool HasShadingModel(EMaterialShadingModel model) const
	{
		return (MaterialShadingModelEnabled & (1 << model)) != 0;
	}

	void Serialize(FArchive& Ar)
	{
		Ar << bNeedsParticlePosition;
		Ar << bNeedsParticleVelocity;
		Ar << bUseDynamicParameters;
		Ar << DynamicParametersMask;
		Ar << bNeedsParticleTime;
		Ar << bUsesParticleMotionBlur;
		Ar << bNeedsParticleRandom;
		Ar << bSphericalParticleOpacity;
		Ar << bUseParticleSubUVs;
		Ar << bLightmapUVAccess;
		Ar << bUsesAOMaterialMask;
		Ar << bUsesSpeedtree;
		Ar << bNeedsWorldPositionExcludingShaderOffsets;
		Ar << bNeedsParticleSize;
		Ar << bNeedsParticleSpriteRotation;
		Ar << bNeedsSceneTextures;
		Ar << bAlphaPropagatePostProcessInput0;
		Ar << bAlphaPropagateUserSceneTexture;
		Ar << UsedSceneTextures;
		Ar << bUsesEyeAdaptation;
		Ar << bVirtualTextureOutput;
		Ar << bUsesPerInstanceCustomData;
		Ar << bUsesPerInstanceRandomPS;
		Ar << bUsesPerInstanceFadeAmount;
		Ar << bUsesVertexInterpolator;
		Ar << bUsesSkyAtmosphere;
		Ar << bUsesVertexColor;
		Ar << bUsesParticleColor;
		Ar << bUsesParticleLocalToWorld;
		Ar << bUsesParticleWorldToLocal;
		Ar << bUsesInstanceLocalToWorldPS;
		Ar << bUsesInstanceWorldToLocalPS;
		Ar << bUsesTransformVector;
		Ar << bUsesPixelDepthOffset;
		Ar << bUsesWorldPositionOffset;
		Ar << bUsesDisplacement;
		Ar << bUsesEmissiveColor;
		Ar << bUsesDistortion;
		Ar << bUsesExplicitDerivatives;
		Ar << bUsesFirstPersonInterpolation;
		Ar << bMaterialEnableTranslucencyFogging;
		Ar << bMaterialEnableTranslucencyCloudFogging;
		Ar << bMaterialIsSky;
		Ar << bMaterialComputeFogPerPixel;
		Ar << bMaterialFullyRough;
		Ar << bMaterialUsesAnisotropy;
		Ar << bMaterialUsesSpecularProfile;
		Ar << MaterialDecalReadMask;
		Ar << PixelDepthOffsetMode;
		Ar << bMaterialUsesDecalLookup;
		Ar << MaterialPathTracingBufferRead;
		Ar << MaterialNeuralPostProcess;
		Ar << NumVirtualTextureSamples;
		Ar << NumVirtualTextureFeedbackRequests;
		Ar << bMaterialVirtualTextureFeedback;
		Ar << VirtualPageTypes;
		Ar << bSingleLayerWaterShadingQuality;
		Ar << bShadingModelsIsLit;
		Ar << MaterialShadingModelEnabled;
		Ar << bMaterialSubsurfaceProfileUseCurvature;
		Ar << bMaterialShadingModelEyeUseCurvature;
		Ar << bDisableForwardLocalLights;
		Ar << bSingleLayerWaterSeparatedMainLight;
		Ar << bMaterialSingleShadingModel;
		Ar << bMaterialVolumetricAdvanced;
		Ar << bMaterialVolumetricAdvancedPhasePerSample;
		Ar << bMaterialVolumetricAdvancedGreyscaleMaterial;
		Ar << bMaterialVolumetricAdvancedRaymarchVolumeShadow;
		Ar << bMaterialVolumetricAdvancedClampMultiscatteringContribution;
		Ar << MaterialVolumetricAdvancedMultiscatteringOctaveCount;
		Ar << bMaterialVolumetricAdvancedConservativeDensity;
		Ar << bMaterialVolumetricAdvancedOverrideAmbientOcclusion;
		Ar << bMaterialVolumetricAdvancedAdvancedGroundContribution;
		Ar << bMaterialVolumetricCloudEmptySpaceSkippingOutput;
		Ar << bDualSourceColorBlendingEnabled;
		Ar << SubstrateDefines;
		Ar << NumCustomizedUVs;
		Ar << bTextureSampleDebug;
		Ar << bMaterialEnableTranslucentLocalLightShadow;
		Ar << bMaterialEnableTranslucentHighQualityLocalLightShadow;
		Ar << bMaterialEnableTranslucentHighQualityDirectionalLightShadow;
		TArray<UObject*> ParameterCollectionObjects;
		if (!Ar.IsLoading())
		{
			ParameterCollectionObjects.Reserve(ParameterCollections.Num());
			for (const TObjectPtr<UMaterialParameterCollection>& Object : ParameterCollections)
			{
				ParameterCollectionObjects.Add(Object);
			}
		}

		Ar << ParameterCollectionObjects;

		if (Ar.IsLoading())
		{
			ParameterCollections.Reset(ParameterCollectionObjects.Num());
			for (UObject* Object : ParameterCollectionObjects)
			{
				while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
				{
					Object = Redirector->DestinationObject;
				}

				UMaterialParameterCollection* Collection = Cast<UMaterialParameterCollection>(Object);
				
				if (!Collection && Object)
				{
					UE_LOGF(LogMaterial, Error,
						"FEnvironmentDefines.ParameterCollections had object %ls which is not a UMaterialParameterCollection; setting it to null.",
						*Object->GetFullName());
				}

				ParameterCollections.Add(Collection);
			}
		}
	}
};


static inline bool IsAnalyticDerivEnabled()
{
	return GAnalyticDerivEnabled != 0;
}

static inline bool IsDebugTextureSampleEnabled()
{
	return IsAnalyticDerivEnabled() && (GDebugTextureSampleEnabled != 0);
}

/** @return the vector type containing a given number of components. */
static inline EMaterialValueType GetVectorType(uint32 NumComponents)
{
	switch(NumComponents)
	{
		case 1: return MCT_Float;
		case 2: return MCT_Float2;
		case 3: return MCT_Float3;
		case 4: return MCT_Float4;
		default: return MCT_Unknown;
	};
}

static inline EMaterialValueType GetLWCVectorType(uint32 NumComponents)
{
	switch (NumComponents)
	{
	case 1: return MCT_LWCScalar;
	case 2: return MCT_LWCVector2;
	case 3: return MCT_LWCVector3;
	case 4: return MCT_LWCVector4;
	default: return MCT_Unknown;
	};
}

static inline EMaterialValueType GetMaterialValueType(UE::Shader::EValueType Type)
{
	switch (Type)
	{
	case UE::Shader::EValueType::Void: return MCT_VoidStatement;

	case UE::Shader::EValueType::Float1: return MCT_Float;
	case UE::Shader::EValueType::Float2: return MCT_Float2;
	case UE::Shader::EValueType::Float3: return MCT_Float3;
	case UE::Shader::EValueType::Float4: return MCT_Float4;

	case UE::Shader::EValueType::Double1: return MCT_LWCScalar;
	case UE::Shader::EValueType::Double2: return MCT_LWCVector2;
	case UE::Shader::EValueType::Double3: return MCT_LWCVector3;
	case UE::Shader::EValueType::Double4: return MCT_LWCVector4;

	case UE::Shader::EValueType::Int1: return MCT_Float;
	case UE::Shader::EValueType::Int2: return MCT_Float2;
	case UE::Shader::EValueType::Int3: return MCT_Float3;
	case UE::Shader::EValueType::Int4: return MCT_Float4;

	case UE::Shader::EValueType::Bool1: return MCT_Float;
	case UE::Shader::EValueType::Bool2: return MCT_Float2;
	case UE::Shader::EValueType::Bool3: return MCT_Float3;
	case UE::Shader::EValueType::Bool4: return MCT_Float4;

	case UE::Shader::EValueType::Struct: return MCT_MaterialAttributes;
	default: checkNoEntry(); return MCT_Unknown;
	}
}

static inline EMaterialValueType GetMaterialValueType(EMaterialParameterType Type)
{
	switch (Type)
	{
	case EMaterialParameterType::Scalar: return MCT_Float;
	case EMaterialParameterType::Vector: return MCT_Float4;
	case EMaterialParameterType::DoubleVector: return MCT_LWCVector4;
	case EMaterialParameterType::StaticSwitch: return MCT_Bool;
	default: checkNoEntry(); return MCT_Unknown;
	}
}

static inline int32 SwizzleComponentToIndex(TCHAR Component)
{
	switch (Component)
	{
	case TCHAR('x'): case TCHAR('X'): case TCHAR('r'): case TCHAR('R'): return 0;
	case TCHAR('y'): case TCHAR('Y'): case TCHAR('g'): case TCHAR('G'): return 1;
	case TCHAR('z'): case TCHAR('Z'): case TCHAR('b'): case TCHAR('B'): return 2;
	case TCHAR('w'): case TCHAR('W'): case TCHAR('a'): case TCHAR('A'): return 3;
	default:
		return -1;
	}
}

static inline const TCHAR * GetFloatZeroVector(uint32 NumComponents)
{
	switch (NumComponents)
	{
	case 1:
		return TEXT("0.0f");
	case 2:
		return TEXT("float2(0.0f, 0.0f)");
	case 3:
		return TEXT("float3(0.0f, 0.0f, 0.0f)");
	case 4:
		return TEXT("float4(0.0f, 0.0f, 0.0f, 0.0f)");
	default:
		check(0);
		return TEXT("");
	}
}

static FString FormatUnsupportedMaterialDomainError(const FMaterialExternalCodeDeclaration& InExternalCode, const FName& AssetPathName)
{
	TStringBuilder<1024> SupportedMaterialDomainsString;
	for (EMaterialDomain Domain : InExternalCode.Domains)
	{
		if (SupportedMaterialDomainsString.Len() > 0)
		{
			SupportedMaterialDomainsString.Append(TEXT(", "));
		}
		SupportedMaterialDomainsString.Append(*MaterialDomainString(Domain));
	}
	return FString::Printf(TEXT("External code '%s' (asset: %s) is only available in the following material domains: %s"), *InExternalCode.Name.ToString(), *AssetPathName.ToString(), *SupportedMaterialDomainsString);
}

void FHLSLMaterialTranslator::AppendVersion(FShaderKeyGenerator& KeyGen, EShaderPlatform Platform)
{
	static const FGuid DDCVersion = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().MaterialTranslationDDCVersion);
	KeyGen.AppendDebugText(TEXT("_MatTransl_"));
	KeyGen.AppendDebugText(FApp::GetProjectName());
	KeyGen.Append(FMaterialSourceTemplate::Get().GetTemplateHashString(Platform));
	KeyGen.AppendSeparator();
	KeyGen.Append(DDCVersion);
	KeyGen.AppendSeparator();
}

FHLSLMaterialTranslator::FHLSLMaterialTranslator(FMaterial* InMaterial,
	FMaterialCompilationOutput& InMaterialCompilationOutput,
	const FStaticParameterSet& InStaticParameters,
	EShaderPlatform InPlatform,
	EMaterialQualityLevel::Type InQualityLevel,
	ERHIFeatureLevel::Type InFeatureLevel,
	const ITargetPlatform* InTargetPlatform, //if InTargetPlatform is nullptr, we use the current active
	const FSubstrateCompilationConfig* InSubstrateCompilationConfig,
	const FString& MaterialTranslationDDCKeyString)
:	ShaderFrequency(SF_Pixel)
,	MaterialProperty(MP_EmissiveColor)
,	CurrentScopeChunks(nullptr)
,	CurrentScopeID(0u)
,	NextTempScopeID(SF_NumFrequencies)
,	Material(InMaterial)
,	StaticParameters(InStaticParameters)
,	Platform(InPlatform)
,	QualityLevel(InQualityLevel)
,	FeatureLevel(InFeatureLevel)
,	NextSymbolIndex(INDEX_NONE)
,	NextVertexInterpolatorIndex(0)
,	CurrentCustomVertexInterpolatorOffset(0)
,	CompileErrorsSink(nullptr)
,	CompileErrorExpressionsSink(nullptr)
,	TranslationResult(EHLSLMaterialTranslatorResult::Success)
,	bCompileForComputeShader(false)
,	bUsesSceneDepth(false)
,	bNeedsParticlePosition(false)
,	bNeedsParticleVelocity(false)
,	bNeedsParticleTime(false)
,	bUsesParticleMotionBlur(false)
,	bNeedsParticleRandom(false)
,	bUsesSphericalParticleOpacity(false)
,   bUsesParticleSubUVs(false)
,	bUsesLightmapUVs(false)
,	bUsesAOMaterialMask(false)
,	bUsesSpeedTree(false)
,	bNeedsWorldPositionExcludingShaderOffsets(false)
,	bNeedsParticleSize(false)
,	bNeedsParticleSpriteRotation(false)
,	bNeedsSceneTexturePostProcessInputs(false)
,	bUsesSkyAtmosphere(false)
,	bUsesVertexColor(false)
,	bUsesParticleColor(false)
,	bUsesParticleLocalToWorld(false)
,	bUsesParticleWorldToLocal(false)
,	bUsesInstanceLocalToWorldPS(false)
,	bUsesInstanceWorldToLocalPS(false)
,	bUsesPerInstanceRandomPS(false)
,	bUsesVertexPosition(false)
,	bPotentiallyManipulateTexCoords(false)
,	bUsesTransformVector(false)
,	bCompilingPreviousFrame(false)
,	bOutputsBasePassVelocities(true)
,	bUsesPixelDepthOffset(false)
,	bUsesWorldPositionOffset(false)
,	bUsesDisplacement(false)
,	bUsesEmissiveColor(false)
,	bUsesDistanceCullFade(false)
,	bIsFullyRough(0)
,	bAllowCodeChunkGeneration(true)
,	bUsesAnisotropy(false)
,	bUsesCurvature(false)
,	bUsesPerInstanceFadeAmount(false)
,	bCullIntermediateUniformExpressions(GCullIntermediateUniformExpressions)
,	bUsesExplicitDerivatives(false)
,	bUsesFirstPersonInterpolation(false)
,	bUsesVirtualTextureSampleForNormalProperty(false)
,	bIsInRuntimeVirtualTextureOutput(false)
,	AddingUniformExpression(0)
,	AllocatedUserTexCoords()
,	AllocatedUserVertexTexCoords()
,	DynamicParticleParameterMask(0)
,	NumVtSamples(0)
,	TargetPlatform(InTargetPlatform)
,	MaterialCompilationOutput(InMaterialCompilationOutput)
{
	check((sizeof(SubstrateTranslatorData.SubstrateCompilationContext)/sizeof(FSubstrateCompilationContext)) == ESubstrateCompilationContext::SCC_MAX);

	FMemory::Memzero(SharedPixelProperties);

	FMemory::Memzero(NumForLoops);

	FMemory::Memzero(MaterialAttributesReturned);

	InStaticParameters.GetMaterialLayers(CachedMaterialLayers);

	SharedPixelProperties[MP_Normal] = true;
	SharedPixelProperties[MP_Tangent] = true;
	SharedPixelProperties[MP_EmissiveColor] = true;
	SharedPixelProperties[MP_Opacity] = true;
	SharedPixelProperties[MP_OpacityMask] = true;
	SharedPixelProperties[MP_BaseColor] = true;
	SharedPixelProperties[MP_Metallic] = true;
	SharedPixelProperties[MP_Specular] = true;
	SharedPixelProperties[MP_Roughness] = true;
	SharedPixelProperties[MP_Anisotropy] = true;
	SharedPixelProperties[MP_AmbientOcclusion] = true;
	SharedPixelProperties[MP_Refraction] = true;
	SharedPixelProperties[MP_PixelDepthOffset] = true;
	SharedPixelProperties[MP_SubsurfaceColor] = true;
	SharedPixelProperties[MP_ShadingModel] = true;
	SharedPixelProperties[MP_SurfaceThickness] = true;
	SharedPixelProperties[MP_FrontMaterial] = true;
	SharedPixelProperties[MP_Displacement] = true;
	SharedPixelProperties[MP_CustomData0] = true;
	SharedPixelProperties[MP_CustomData1] = true;

	for (int32 Frequency = 0; Frequency < SF_NumFrequencies; ++Frequency)
	{
		FunctionStacks[Frequency].Add(new FMaterialFunctionCompileState(nullptr));
	}

	// Default value for attribute stack added to simplify code when compiling new attributes, see SetMaterialProperty.
	const FGuid& MissingAttribute = FMaterialAttributeDefinitionMap::GetID(MP_MAX);
	MaterialAttributesStack.Add(MissingAttribute);

	// Default owner for parameters
	ParameterOwnerStack.Add(FMaterialParameterInfo());

	if (TargetPlatform == nullptr)
	{
		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		if (TPM)
		{
			TargetPlatform = TPM->GetRunningTargetPlatform();
		}
	}

	if (InSubstrateCompilationConfig)
	{
		SubstrateTranslatorData.SubstrateCompilationConfig = *InSubstrateCompilationConfig;
	}
	SubstrateTranslatorData.Initialize(Material->GetMaterialInterface(), MaterialCompilationOutput, this, nullptr);

	EnvironmentDefines.Reset(new FEnvironmentDefines);

	// Hash the DDC key string and store the result for DDC retrieval.
	DDCKeyHash = FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(MaterialTranslationDDCKeyString)));
}

FHLSLMaterialTranslator::~FHLSLMaterialTranslator()
{
	ClearAllFunctionStacks();
}

bool FHLSLMaterialTranslator::ShouldStopTranslating() const
{
	return DDCQueryHit;
}

int32 FHLSLMaterialTranslator::GetNumUserTexCoords() const
{
	return AllocatedUserTexCoords.FindLast(true) + 1;
}

int32 FHLSLMaterialTranslator::GetNumUserVertexTexCoords() const
{
	return AllocatedUserVertexTexCoords.FindLast(true) + 1;
}

void FHLSLMaterialTranslator::ClearAllFunctionStacks()
{
	for (uint32 Frequency = 0; Frequency < SF_NumFrequencies; ++Frequency)
	{
		ClearFunctionStack(Frequency);
	}
}

void FHLSLMaterialTranslator::ClearFunctionStack(uint32 Frequency)
{
	check(Frequency < SF_NumFrequencies);

	if (FunctionStacks[Frequency].Num() == 0)
	{
		return;  // Already cleared (at the end of Translate(), for example)
	}

	check(FunctionStacks[Frequency].Num() == 1);  // All states should be popped off, leaving only the null state
	delete FunctionStacks[Frequency][0];
	FunctionStacks[Frequency].Empty();
}

void FHLSLMaterialTranslator::AssignTempScope(TArray<FShaderCodeChunk>& InScope)
{
	CurrentScopeChunks = &InScope;
	CurrentScopeID = NextTempScopeID++;
}

void FHLSLMaterialTranslator::AssignShaderFrequencyScope(EShaderFrequency InShaderFrequency)
{
	check(InShaderFrequency < SF_NumFrequencies);
	check(InShaderFrequency < NextTempScopeID);
	CurrentScopeChunks = &SharedPropertyCodeChunks[InShaderFrequency];
	CurrentScopeID = (uint64)InShaderFrequency;
}

template<typename ExpressionsArrayType>
void FHLSLMaterialTranslator::GatherCustomVertexInterpolators(const ExpressionsArrayType& Expressions)
{
	for (UMaterialExpression* Expression : Expressions)
	{
		if (UMaterialExpressionVertexInterpolator* Interpolator = Cast<UMaterialExpressionVertexInterpolator>(Expression))
		{
			TArray<FShaderCodeChunk> CustomExpressionChunks;
			AssignTempScope(CustomExpressionChunks);

			// Errors are appended to a temporary pool as it's not known at this stage which interpolators are required
			CompileErrorsSink = &Interpolator->CompileErrors;
			CompileErrorExpressionsSink = &Interpolator->CompileErrorExpressions;

			// Compile node and store those successfully translated
			int32 Ret = Interpolator->CompileInput(this, NextVertexInterpolatorIndex);
			if (Ret != INDEX_NONE)
			{
				CustomVertexInterpolators.AddUnique(Interpolator);
				NextVertexInterpolatorIndex++;

				for (FShaderCodeChunk& ShaderCodeChunk : CustomExpressionChunks)
				{
					if (ShaderCodeChunk.UniformExpression)
					{
						ShaderCodeChunk.UniformExpression->ShaderFrequencyMask |= (1 << SF_Vertex);
					}
				}
			}

			// Restore error handling
			CompileErrorsSink = nullptr;
			CompileErrorExpressionsSink = nullptr;

			// Each interpolator chain must be handled as an independent compile
			for (FMaterialFunctionCompileState* FunctionStack : FunctionStacks[SF_Vertex])
			{
				FunctionStack->Reset();
			}
		}
		else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (FunctionCall->MaterialFunction)
			{
				FMaterialFunctionCompileState LocalState(FunctionCall);
				FunctionCall->LinkFunctionIntoCaller(this);
				PushFunction(&LocalState);

				GatherCustomVertexInterpolators(FunctionCall->MaterialFunction->GetExpressions());

				FMaterialFunctionCompileState* CompileState = PopFunction();
				check(CompileState->ExpressionStack.Num() == 0);
				FunctionCall->UnlinkFunctionFromCaller(this);
			}
		}
		else if (UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
		{
			const FMaterialLayersFunctions* OverrideLayers = GetMaterialLayers();
			if (OverrideLayers)
			{
				LayersExpression->OverrideLayerGraph(OverrideLayers);
			}

			if (LayersExpression->bIsLayerGraphBuilt)
			{
				for (auto& Layer : LayersExpression->LayerCallers)
				{
					if (Layer && Layer->MaterialFunction)
					{
						FMaterialFunctionCompileState LocalState(Layer);
						Layer->LinkFunctionIntoCaller(this);
						PushFunction(&LocalState);
						
						GatherCustomVertexInterpolators(Layer->MaterialFunction->GetExpressions());

						FMaterialFunctionCompileState* CompileState = PopFunction();
						check(CompileState->ExpressionStack.Num() == 0);
						Layer->UnlinkFunctionFromCaller(this);
					}
				}

				for (auto& Blend : LayersExpression->BlendCallers)
				{
					if (Blend && Blend->MaterialFunction)
					{
						FMaterialFunctionCompileState LocalState(Blend);
						Blend->LinkFunctionIntoCaller(this);
						PushFunction(&LocalState);

						GatherCustomVertexInterpolators(Blend->MaterialFunction->GetExpressions());

						FMaterialFunctionCompileState* CompileState = PopFunction();
						check(CompileState->ExpressionStack.Num() == 0);
						Blend->UnlinkFunctionFromCaller(this);
					}
				}
			}

			if (OverrideLayers)
			{
				LayersExpression->OverrideLayerGraph(nullptr);
			}
		}
	}
}

void FHLSLMaterialTranslator::CompileCustomOutputs(TArray<UMaterialExpressionCustomOutput*>& CustomOutputExpressions, TSet<UClass*>& SeenCustomOutputExpressionsClasses, bool bIsBeforeAttributes)
{
	for (UMaterialExpressionCustomOutput* CustomOutput : CustomOutputExpressions)
	{
		if (CustomOutput->HasCustomSourceOutput() || CustomOutput->ShouldCompileBeforeAttributes() != bIsBeforeAttributes)
		{
			continue;
		}

		if (!CustomOutput->AllowMultipleCustomOutputs() && SeenCustomOutputExpressionsClasses.Contains(CustomOutput->GetClass()))
		{
			Errorf(TEXT("The material can contain only one %s node"), *CustomOutput->GetDescription());
		}
		else
		{
			TopCustomOutput = CustomOutput;
			
			SeenCustomOutputExpressionsClasses.Add(CustomOutput->GetClass());

			int32 MaxOutputs = CustomOutput->GetMaxOutputs();
			int32 NumOutputs = CustomOutput->GetNumOutputs();
			if ((MaxOutputs > 0) && (NumOutputs > MaxOutputs))
			{
				Errorf(TEXT("The material can only contain up to %i output %s nodes (current number: %i)"), MaxOutputs, *CustomOutput->GetDescription(), NumOutputs);
			}
			else
			{
				if (CustomOutput->NeedsCustomOutputDefines())
				{
					ResourcesString += FString::Printf(TEXT("#define NUM_MATERIAL_OUTPUTS_%s %d\n"), *CustomOutput->GetFunctionName().ToUpper(), NumOutputs);
				}

				if (NumOutputs > 0)
				{
					const bool bNeedsPreviousFrameEvaluation = CustomOutput->NeedsPreviousFrameEvaluation();
					const int32 NumEvaluations = bNeedsPreviousFrameEvaluation ? 2 : 1;
					const bool bCachedCompilingPreviousFrame = bCompilingPreviousFrame;

					TArray<EShaderFrequency, TInlineAllocator<4u>> OutputFrequencies;

					for (int32 PreviousFrameEvaluation = 0; PreviousFrameEvaluation < NumEvaluations; PreviousFrameEvaluation++)
					{
						for (int32 Index = 0; Index < NumOutputs; Index++)
						{
							EShaderFrequency CustomOutputShaderFrequency = CustomOutput->GetShaderFrequency(Index);
							OutputFrequencies.AddUnique(CustomOutputShaderFrequency);
							
							{
								ClearFunctionStack(CustomOutputShaderFrequency);
								FunctionStacks[CustomOutputShaderFrequency].Add(new FMaterialFunctionCompileState(nullptr));
							}
							MaterialProperty = MP_MAX; // Indicates we're not compiling any material property.
							ShaderFrequency = CustomOutputShaderFrequency;
							bCompilingPreviousFrame = PreviousFrameEvaluation != 0;
							TArray<FShaderCodeChunk> CustomExpressionChunks;
							AssignTempScope(CustomExpressionChunks);
							CustomOutput->Compile(this, Index);

							if (ShaderFrequency == SF_Vertex)
							{
								for (FShaderCodeChunk& ShaderCodeChunk : CustomExpressionChunks)
								{
									if (ShaderCodeChunk.UniformExpression)
									{
										ShaderCodeChunk.UniformExpression->ShaderFrequencyMask |= (1 << SF_Vertex);
									}
								}
							}

						}
					}

					bCompilingPreviousFrame = bCachedCompilingPreviousFrame; // Restore the cached value of bCompilingPreviousFrame

					for (EShaderFrequency CustomOutputShaderFrequency : OutputFrequencies)
					{
						ClearFunctionStack(CustomOutputShaderFrequency);
						FunctionStacks[CustomOutputShaderFrequency].Add(new FMaterialFunctionCompileState(nullptr));
					}
				}
			}
			
			TopCustomOutput = nullptr;
		}
	}
}

uint64 FHLSLMaterialTranslator::CompileMaterialAttributesCustomOutputs(uint64 CustomOutputNodeBitmask)
{
	// Some proxies return null for this. But the main one we are interested in doesn't.
	UMaterialInterface* MatIf = Material->GetMaterialInterface();
	if (!IsValid(MatIf))
	{
		return 0;
	}

	UMaterial* BaseMaterial = MatIf->GetMaterial();
	if (!BaseMaterial->bUseMaterialAttributes)
	{
		return 0;
	}

	// Get all expressions in the input chain of the MaterialAttributes output and create a bitmask of all connected attributes, including custom outputs.
	TArray<UMaterialExpression*> InputExpressions;
	BaseMaterial->GetExpressionsInPropertyChain(MP_MaterialAttributes, InputExpressions, &StaticParameters, ERHIFeatureLevel::Num, EMaterialQualityLevel::Num, ERHIShadingPath::Num, true /*bInRecurseIntoMaterialFunctions*/);
	const uint64 AttributesBitmask = FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(InputExpressions);

	// Get a list of all custom output attribute definitions.
	static TArray<FMaterialCustomOutputAttributeDefintion> CustomAttributes;
	if (CustomAttributes.IsEmpty())
	{
		FMaterialAttributeDefinitionMap::GetCustomAttributeList(CustomAttributes);
	}

	uint64 ProcessedCustomOutputsMask = 0;

	// Iterate over all supported custom outputs and compile them if they're set in the bitmask. Based on the logic in CompileCustomOutputs().
	for (const FMaterialCustomOutputAttributeDefintion& CustomAttribute : CustomAttributes)
	{
		const uint64 CustomAttributeBit = FMaterialAttributeDefinitionMap::GetBitmask(CustomAttribute.AttributeID);
		if ((CustomAttributeBit & AttributesBitmask) == 0)
		{
			continue;
		}

		if ((CustomOutputNodeBitmask & FMaterialAttributeDefinitionMap::GetBitmask(CustomAttribute.AttributeID)) != 0)
		{
			UE_LOGF(LogMaterial, Display, "The material sets custom output '%ls' using both MaterialAttributes and a custom output node! The value set via MaterialAttributes will be ignored.", *CustomAttribute.AttributeName);
			continue;
		}

		const int32 NumOutputs = 1; // TODO: Properly support custom output nodes with more than one output. Currently all of the supported custom outputs just require the define to have a value > 0.
		const FString FunctionNameUpper = CustomAttribute.FunctionName.ToUpper();
		ResourcesString += FString::Printf(TEXT("#ifndef NUM_MATERIAL_OUTPUTS_%s\n#define NUM_MATERIAL_OUTPUTS_%s %d\n#endif\n"), *FunctionNameUpper, *FunctionNameUpper, NumOutputs);

		const int32 NumEvaluations = CustomAttribute.bNeedsPreviousFrameEvaluation ? 2 : 1;
		const bool bCachedCompilingPreviousFrame = bCompilingPreviousFrame;

		for (int32 PreviousFrameEvaluation = 0; PreviousFrameEvaluation < NumEvaluations; PreviousFrameEvaluation++)
		{
			{
				ClearFunctionStack(CustomAttribute.ShaderFrequency);
				FunctionStacks[CustomAttribute.ShaderFrequency].Add(new FMaterialFunctionCompileState(nullptr));
			}
			MaterialProperty = MP_MAX; // Indicates we're not compiling any material property.
			ShaderFrequency = CustomAttribute.ShaderFrequency;
			bCompilingPreviousFrame = PreviousFrameEvaluation != 0;
			TArray<FShaderCodeChunk> CustomExpressionChunks;
			AssignTempScope(CustomExpressionChunks);
			int32 CodeChunk = Material->CompileCustomAttribute(CustomAttribute.AttributeID, this);
			if (CodeChunk != INDEX_NONE)
			{
				if (ShaderFrequency == SF_Vertex)
				{
					for (FShaderCodeChunk& ShaderCodeChunk : CustomExpressionChunks)
					{
						if (ShaderCodeChunk.UniformExpression)
						{
							ShaderCodeChunk.UniformExpression->ShaderFrequencyMask |= (1 << SF_Vertex);
						}
					}
				}

				CustomOutput(CustomAttribute.AttributeName, HLSLTypeString(GetParameterType(CodeChunk)), CustomAttribute.FunctionName, CustomAttribute.OutputIndex, CodeChunk, EMaterialCustomOutputFlags::None);
			}
		}

		bCompilingPreviousFrame = bCachedCompilingPreviousFrame; // Restore the cached value of bCompilingPreviousFrame

		ClearFunctionStack(CustomAttribute.ShaderFrequency);
		FunctionStacks[CustomAttribute.ShaderFrequency].Add(new FMaterialFunctionCompileState(nullptr));

		ProcessedCustomOutputsMask |= CustomAttributeBit;
	}

	return ProcessedCustomOutputsMask;
}

template<typename ExpressionsArrayType>
EMaterialExpressionVisitResult FHLSLMaterialTranslator::VisitExpressionsRecursive(const ExpressionsArrayType& Expressions, IMaterialExpressionVisitor& InVisitor)
{
	EMaterialExpressionVisitResult VisitResult = MVR_CONTINUE;
	for (UMaterialExpression* Expression : Expressions)
	{
		VisitResult = InVisitor.Visit(Expression);
		if (VisitResult == MVR_STOP)
		{
			break;
		}

		if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (FunctionCall->MaterialFunction)
			{
				FMaterialFunctionCompileState LocalState(FunctionCall);
				FunctionCall->LinkFunctionIntoCaller(this);
				PushFunction(&LocalState);

				VisitResult = VisitExpressionsRecursive(FunctionCall->MaterialFunction->GetExpressions(), InVisitor);

				FMaterialFunctionCompileState* CompileState = PopFunction();
				check(CompileState->ExpressionStack.Num() == 0);
				FunctionCall->UnlinkFunctionFromCaller(this);

				if (VisitResult == MVR_STOP)
				{
					break;
				}
			}
		}
		else if (UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
		{
			const FMaterialLayersFunctions* OverrideLayers = GetMaterialLayers();
			if (OverrideLayers)
			{
				LayersExpression->OverrideLayerGraph(OverrideLayers);
			}

			if (LayersExpression->bIsLayerGraphBuilt)
			{
				for (auto& Layer : LayersExpression->LayerCallers)
				{
					if (Layer && Layer->MaterialFunction)
					{
						FMaterialFunctionCompileState LocalState(Layer);
						Layer->LinkFunctionIntoCaller(this);
						PushFunction(&LocalState);

						VisitResult = VisitExpressionsRecursive(Layer->MaterialFunction->GetExpressions(), InVisitor);

						FMaterialFunctionCompileState* CompileState = PopFunction();
						check(CompileState->ExpressionStack.Num() == 0);
						Layer->UnlinkFunctionFromCaller(this);

						if (VisitResult == MVR_STOP)
						{
							break;
						}
					}
				}

				for (auto& Blend : LayersExpression->BlendCallers)
				{
					if (Blend && Blend->MaterialFunction)
					{
						FMaterialFunctionCompileState LocalState(Blend);
						Blend->LinkFunctionIntoCaller(this);
						PushFunction(&LocalState);

						VisitResult = VisitExpressionsRecursive(Blend->MaterialFunction->GetExpressions(), InVisitor);

						FMaterialFunctionCompileState* CompileState = PopFunction();
						check(CompileState->ExpressionStack.Num() == 0);
						Blend->UnlinkFunctionFromCaller(this);

						if (VisitResult == MVR_STOP)
						{
							break;
						}
					}
				}
			}

			if (OverrideLayers)
			{
				LayersExpression->OverrideLayerGraph(nullptr);
			}

			if (VisitResult == MVR_STOP)
			{
				break;
			}
		}
	}

	return VisitResult;
}

EMaterialExpressionVisitResult FHLSLMaterialTranslator::VisitExpressionsForProperty(EMaterialProperty InProperty, IMaterialExpressionVisitor& InVisitor)
{
	UMaterialInterface *MatIf = Material->GetMaterialInterface();
	// Some proxies return null for this. But the main one we are interested in doesn't
	if (MatIf)
	{
		TArray<UMaterialExpression*> InputExpressions;
		MatIf->GetMaterial()->GetExpressionsInPropertyChain(InProperty, InputExpressions, &StaticParameters);
		return VisitExpressionsRecursive(InputExpressions, InVisitor);
	}
	return MVR_STOP;
}


UE_TRACE_EVENT_BEGIN(Cpu, FHLSLMaterialTranslatorTranslate, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, MaterialName)
UE_TRACE_EVENT_END()
 

EHLSLMaterialTranslatorResult FHLSLMaterialTranslator::Translate(bool bForceDisableDDCQuery)
{
	STAT(FDateTime TranslationDateTime = FDateTime::Now());
	STAT(double TotalTime = FPlatformTime::Seconds());
	STAT(double TranslationOnlyTime = 0);

#if CPUPROFILERTRACE_ENABLED
	FString TraceMaterialName;
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		TraceMaterialName = Material->GetMaterialInterface()->GetFullName();
	}
	UE_TRACE_LOG_SCOPED_T(Cpu, FHLSLMaterialTranslatorTranslate, CpuChannel) << FHLSLMaterialTranslatorTranslate.MaterialName(*TraceMaterialName);
#endif

	static const bool bVerbose = FParse::Param(FCommandLine::Get(), TEXT("verbosematerialtranslation"));
	if (bVerbose)
	{
		UE_LOGF(LogMaterial, Display, "Translating material '%ls'", *Material->GetMaterialInterface()->GetFullName());
	}

	// We call FindObject to serialize the array of Parameter Collections used by this material in EnvironmentDefines,
	// but this can happen during save. FindObject is illegal during save because if the discovered objects
	// are serialized into the package it will cause a crash on package load. But we are not storing the results
	// of FindObject into the package so it is okay to remove the restriction.
	TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);

	static const bool bNoMaterialTranslationDDC = FParse::Param(FCommandLine::Get(), TEXT("nomaterialtranslationddc"));

	// Disable translation DDC if material is previw, we manually disabled it (globally or through
	// the -nomaterialtranslationddc flag or if GUseMaterialTranslationResultsGrouping is disabled
	// as it's a requirement.
#if FORCE_DISABLE_MATERIAL_TRANSLATION_DDC
	const bool bDisableTranslationDDC = true;
#else
	const bool bDisableTranslationDDC = bNoMaterialTranslationDDC || Material->IsPreview() || !Material->IsPersistent();
#endif

	// DDC query local data
	DDCQueryCompleted = false;
	DDCQueryHit = false;

	UE::DerivedData::FRequestOwner DDCRequestOwner{ UE::DerivedData::EPriority::Highest };
	FSharedBuffer MaterialCompilationOutputBuffer;
	FSharedBuffer TranslationResultsBuffer;
	FSharedBuffer EnvironmentDefinesBuffer;

	// Asynchronously query the DDC for results
	if (!bForceDisableDDCQuery && !bDisableTranslationDDC)
	{
		AsyncQueryDDC(DDCRequestOwner, EnvironmentDefinesBuffer);
	}

	// Synchronously begin translating the material
	{
		SCOPE_SECONDS_COUNTER(TranslationOnlyTime);
		TranslateMaterial();
	}

	// One of the two has terminated. This will be a NOOP if DDC query task has completed.
	DDCRequestOwner.Cancel();

	// If we've hit the cache, deserialize the buffer read from the DDC into mater	ial translation results.
	if (DDCQueryHit)
	{
		if (bVerbose)
		{
			UE_LOGF(LogMaterial, Display, "Material '%ls' translation results retrieved from DDC.", *Material->GetMaterialInterface()->GetFullName());
		}

		STAT(double SerializeTime = FPlatformTime::Seconds());

		// Serialize the environment defines from the buffer retrieved from the DDC.
		FMemoryReaderView EnvironmentDefinesBufferReader{ TArrayView<uint8>{ (uint8*)EnvironmentDefinesBuffer.GetData(), (int)EnvironmentDefinesBuffer.GetSize() } };
		FObjectAndNameAsStringProxyArchive EnvironmentDefinesBufferReaderProxy{ EnvironmentDefinesBufferReader, true };
		EnvironmentDefinesBufferReaderProxy.bResolveRedirectors = true;
		EnvironmentDefines->Serialize(EnvironmentDefinesBufferReaderProxy);

		// If a parameter collection failed to load, the DDC entry is invalid. Caller needs
		// to invoke translate again forcing DDC querying off in order to re-translate the
		// material from scratch. We can't do this ourselves because this class was designed
		// to be single-use. The caller needs to create a new instance and invoke Translate
		// again.
		if (EnvironmentDefines->ParameterCollections.Contains(nullptr))
		{
			TranslationResult = EHLSLMaterialTranslatorResult::RetryWithoutDDC;
		}
		else
		{
			MaterialCompilationOutput = DDCMaterialCompilationOutput;
		
			STAT(DDCRequestSerializeTime += SerializeTime - FPlatformTime::Seconds());
			STAT(GShaderCompilerStats->IncrementMaterialCacheHit());
			TranslationResult = EHLSLMaterialTranslatorResult::Success;
		}
	}
	else if (TranslationResult == EHLSLMaterialTranslatorResult::Success)
	{
		if (bVerbose)
		{
			UE_LOGF(LogMaterial, Display, "Material '%ls' completed full translation.", *Material->GetMaterialInterface()->GetFullName());
		}

		// Material not in cache. If translation was succesful, finalize the results and push them to the DDC
		PrepareEnvironmentDefines();
		PrepareMaterialSourceStringParameters();
		GenerateMaterialCacheSource();

		if (!bDisableTranslationDDC && DDCQueryCompleted)
		{
			if (bVerbose)
			{
				UE_LOGF(LogMaterial, Display, "Pushing material '%ls' translation results to DDC.", *Material->GetMaterialInterface()->GetFullName());
			}

			PushResultsToDDC();
		}
	}

	ClearAllFunctionStacks();

	// Report timings to Material Cook Stats
#if STATS
	TotalTime = FPlatformTime::Seconds() - TotalTime;
	GShaderCompilerStats->IncrementMaterialTranslated(TotalTime, TranslationOnlyTime, DDCRequestSerializeTime);
	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_HLSLTranslation, (float)TotalTime);

#if ENABLE_COOK_STATS
	// Write out a CSV file MaterialTranslationLog.txt containing info about all material translations.
	FCsvLogFile::Get().AddEntry(Material->GetMaterialInterface()->GetFullName(), TranslationDateTime, TotalTime);
#endif // ENABLE_COOK_STATS

#endif // STATS

	return TranslationResult;
}

uint64 HasShaderFrequency(uint64 ShaderFrequencyMask, EShaderFrequency ShaderFrequency)
{
	return (ShaderFrequencyMask & (1ULL << ((uint64)ShaderFrequency))) ? 1 : 0;
}

static inline const TCHAR* GetCurrentMaterialUBNameInternal(EShaderFrequency ShaderFrequency)
{
	if (ShaderFrequency == SF_Vertex)
	{
		return TEXT("__Material_UB_VS__");
	}
	return TEXT("__Material_UB__");
}

void FHLSLMaterialTranslator::TranslateMaterial()
{
	TranslationResult = EHLSLMaterialTranslatorResult::Success;
	
	// No cache hit, continue translating the material
	check(ScopeStack.Num() == 0);

	// WARNING: No compile outputs should be stored on the UMaterial / FMaterial / FMaterialResource, unless they are transient editor-only data (like error expressions)
	// Compile outputs that need to be saved must be stored in MaterialCompilationOutput, which will be saved to the DDC.

	bCompileForComputeShader = Material->IsLightFunction();

	Material->CompileErrors.Empty();
	Material->ErrorExpressions.Empty();

	// If pedantic error checking is enabled, log out the pre-compilation errors
	if (GPedanticErrorChecksEnabled)
	{
		TranslationResult = Material->CheckInValidStateForCompilation(this) ? EHLSLMaterialTranslatorResult::Success : EHLSLMaterialTranslatorResult::Failure;
		if (TranslationResult != EHLSLMaterialTranslatorResult::Success)
		{
			return;
		}
	}

	// Verify for the absence of loops.
	UMaterialInterface* Interface = Material->GetMaterialInterface();
	if (!Interface)
	{
		TranslationResult = EHLSLMaterialTranslatorResult::Failure;
		return;
	}

	if (UMaterial* UMaterial = Interface->GetMaterial())
	{
		TSet<UMaterialExpression*> VisitedExpressions;
		for (UMaterialExpression* Expression : UMaterial->GetExpressions())
		{
			CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
			if (Expression && Expression->ContainsInputLoop(VisitedExpressions))
			{
				AppendExpressionError(Expression, TEXT("Expression is part of a cycle. Please make sure the material graph is acyclic."));
				TranslationResult = EHLSLMaterialTranslatorResult::Failure;
				return;
			}
		}
	}

	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
	//
	// Process the Substrate tree representing the material topology.
	//

	// Get shading models from compilation (or material).
	FMaterialShadingModelField MaterialShadingModels = GetCompiledShadingModels();

	{
		// Temp code chunk scope (e.g.needed for the creation of static booleans from static switch parameter node, see UMaterialExpressionStaticSwitch::GetEffectiveInput).
		TArray<FShaderCodeChunk> TempChunks;
		AssignTempScope(TempChunks);

		bool bDoClearFunctionStack = false;
		SubstrateTranslatorData.ProcessSubstrateTreeAndTopology(MaterialShadingModels, Material->GetMaterialGraphNodePreviewExpression(), bDoClearFunctionStack);

		if (bDoClearFunctionStack)
		{
			//Clear the stacks as they may be referencing scoped chunks that are lost before the compile step.
			//Skips the Vertex shader because this is handled whilst resetting custom vertex interpolation below.
			for (int iFrequency = EShaderFrequency::SF_Vertex + 1; iFrequency < EShaderFrequency::SF_NumFrequencies; iFrequency++)
			{
				while (FunctionStacks[iFrequency].Num() > 1)
				{
					FMaterialFunctionCompileState* Stack = FunctionStacks[iFrequency].Pop(EAllowShrinking::No);
					delete Stack;
				}
				FunctionStacks[iFrequency][0]->Reset();
			}
		}
	}

	// Generate code:
	// Normally one would expect the generator to emit something like
	//		float Local0 = ...
	//		...
	//		float Local3= ...
	//		...
	//		float Localn= ...
	//		PixelMaterialInputs.EmissiveColor = Local0 + ...
	//		PixelMaterialInputs.Normal = Local3 * ...
	// However because the Normal can be used in the middle of generating other Locals (which happens when using a node like PixelNormalWS)
	// instead we generate this:
	//		float Local0 = ...
	//		...
	//		float Local3= ...
	//		PixelMaterialInputs.Normal = Local3 * ...
	//		...
	//		float Localn= ...
	//		PixelMaterialInputs.EmissiveColor = Local0 + ...
	// in other words, compile Normal first, then emit all the expressions up to the last one Normal requires;
	// assign the normal into the shared struct, then emit the remaining expressions; finally assign the rest of the shared struct inputs.
	// Inputs that are not shared, have false in the SharedPixelProperties array, and those ones will emit the full code.

	int32 NormalCodeChunkEnd = -1;
	int32 Chunk[CompiledMP_MAX];
	int32 VertexAttributesChunk = INDEX_NONE;
	int32 PixelAttributesChunk = INDEX_NONE;

	memset(Chunk, INDEX_NONE, sizeof(Chunk));

	// Translate all custom vertex interpolators before main attributes so type information is available
	{
		CustomVertexInterpolators.Empty();
		CurrentCustomVertexInterpolatorOffset = 0;
		NextVertexInterpolatorIndex = 0;
		MaterialProperty = MP_MAX;
		ShaderFrequency = SF_Vertex;

		TArray<UMaterialExpression*> Expressions;
		Material->GatherExpressionsForCustomInterpolators(Expressions);
		GatherCustomVertexInterpolators(Expressions);

		// Reset shared stack data
		while (FunctionStacks[SF_Vertex].Num() > 1)
		{
			FMaterialFunctionCompileState* Stack = FunctionStacks[SF_Vertex].Pop(EAllowShrinking::No);
			delete Stack;
		}
		FunctionStacks[SF_Vertex][0]->Reset();

		// Whilst expression list is available, apply node count limits
		int32 NumMaterialLayersAttributes = 0;
		for (UMaterialExpression* Expression : Expressions)
		{
			CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
			if (UMaterialExpressionMaterialAttributeLayers* Layers = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
			{
				++NumMaterialLayersAttributes;

				if (NumMaterialLayersAttributes > 1)
				{
					Errorf(TEXT("Materials can contain only one Material Attribute Layers node."));
					break;
				}
			}
		}
	}

	const EShaderFrequency NormalShaderFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(MP_Normal);
	const EMaterialDomain Domain = Material->GetMaterialDomain();
	const EBlendMode BlendMode = Material->GetBlendMode();

	const EShaderFrequency FrontMaterialShaderFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(MP_FrontMaterial);
	int32 FullySimplifiedFrontMaterialCodeChunkStart = -1;
	int32 FullySimplifiedFrontMaterialCodeChunkEnd = -1;

	// Gather the implementation for any custom output expressions
	TArray<UMaterialExpressionCustomOutput*> CustomOutputExpressions;
	Material->GatherCustomOutputExpressions(CustomOutputExpressions);
	TSet<UClass*> SeenCustomOutputExpressionsClasses;

	// Some custom outputs must be pre-compiled so they can be re-used as shared inputs
	CompileCustomOutputs(CustomOutputExpressions, SeenCustomOutputExpressionsClasses, true);

	// Normal must always be compiled first; this will ensure its chunk calculations are the first to be added
	{
		// Verify that start chunk is 0
		check(SharedPropertyCodeChunks[NormalShaderFrequency].Num() == 0);
		Chunk[MP_Normal] = Material->CompilePropertyAndSetMaterialProperty(MP_Normal					,this);
		NormalCodeChunkEnd = SharedPropertyCodeChunks[NormalShaderFrequency].Num();
	}

	// Output all chunks types for legacy materials (in case shader code wants to work with legacy attributes directly)
	{
		// Rest of properties
		Chunk[MP_EmissiveColor]	= Material->CompilePropertyAndSetMaterialProperty(MP_EmissiveColor,	this);
		Chunk[MP_DiffuseColor]	= Material->CompilePropertyAndSetMaterialProperty(MP_DiffuseColor,	this);
		Chunk[MP_SpecularColor]	= Material->CompilePropertyAndSetMaterialProperty(MP_SpecularColor,	this);
		Chunk[MP_BaseColor]		= Material->CompilePropertyAndSetMaterialProperty(MP_BaseColor,		this);
		Chunk[MP_Metallic]		= Material->CompilePropertyAndSetMaterialProperty(MP_Metallic,		this);
		Chunk[MP_Specular]		= Material->CompilePropertyAndSetMaterialProperty(MP_Specular,		this);
		Chunk[MP_Roughness]		= Material->CompilePropertyAndSetMaterialProperty(MP_Roughness,		this);
		Chunk[MP_Tangent]		= Material->CompilePropertyAndSetMaterialProperty(MP_Tangent,		this);

		// Make sure to compile this property before using ShadingModelsFromCompilation
		Chunk[MP_ShadingModel]	= Material->CompilePropertyAndSetMaterialProperty(MP_ShadingModel,	this);
	}

	// This needs to be evaluated here for substrate hidden conversion just to check if we need anisotropy through attribute, code chunk of constant.
	Chunk[MP_Anisotropy]		= Material->CompilePropertyAndSetMaterialProperty(MP_Anisotropy, this);

	// Other root node inputs always evaluated.
	Chunk[MP_Opacity]			= Material->CompilePropertyAndSetMaterialProperty(MP_Opacity,		this);
	Chunk[MP_OpacityMask]		= Material->CompilePropertyAndSetMaterialProperty(MP_OpacityMask,	this);
	Chunk[MP_WorldPositionOffset]= Material->CompilePropertyAndSetMaterialProperty(MP_WorldPositionOffset, this);

	Chunk[MP_SurfaceThickness] = Material->CompilePropertyAndSetMaterialProperty(MP_SurfaceThickness		,this);

	// This is what is run when hidden conversion is not executed
	if (!SubstrateTranslatorData.bMaterialUsesRootNodeToSubstrateHiddenConversion)
	{
		Chunk[MP_FrontMaterial] = Material->CompilePropertyAndSetMaterialProperty(MP_FrontMaterial, this);

		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();

		// Now generate the code for the fully simplified MP_FrontMaterial right after MP_FrontMaterial
		SubstrateTranslatorData.FullySimplifiedSubstrateFrontMaterialCodeChunk = SubstrateCreateAndRegisterNullMaterial();
		{																							// This causes issues, material look different
			// Then generate the code
			FullySimplifiedFrontMaterialCodeChunkStart = SharedPropertyCodeChunks[FrontMaterialShaderFrequency].Num();
			SubstrateTranslatorData.CurrentSubstrateCompilationContext = ESubstrateCompilationContext::SCC_FullySimplified;
			SubstrateTranslatorData.FullySimplifiedSubstrateFrontMaterialCodeChunk = Material->CompilePropertyAndSetMaterialProperty(MP_FrontMaterial, this);
			SubstrateTranslatorData.CurrentSubstrateCompilationContext = ESubstrateCompilationContext::SCC_Default;
			FullySimplifiedFrontMaterialCodeChunkEnd = SharedPropertyCodeChunks[FrontMaterialShaderFrequency].Num();
		}
	}
	else // if (bMaterialUsesRootNodeToSubstrateHiddenConversion)
	{
		auto CompileFrontMaterial = [&]()
			{
				UMaterialInterface* MaterialInterface = Material->GetMaterialInterface();
				UMaterial* BaseMaterial = MaterialInterface ? MaterialInterface->GetMaterial() : nullptr;
				UMaterialEditorOnlyData* EditorOnlyData = BaseMaterial ? BaseMaterial->GetEditorOnlyData() : nullptr;

				if (BaseMaterial && BaseMaterial->bUseMaterialAttributes)
				{
					// Need to root subsurface profile for the uniform scalar numeric parameter name to be correct.
					USubsurfaceProfile* SSSProfile = MaterialInterface ? MaterialInterface->GetSubsurfaceProfileRoot_Internal() : nullptr;
					const bool bHasSSS = SSSProfile != nullptr;
					
					return UMaterialExpressionSubstrateConvertMaterialAttributes::CompileCommon(this, 0,
						FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(BaseMaterial->GetExpressions()), EditorOnlyData->MaterialAttributes, MaterialInterface->GetShadingModels().GetFirstShadingModel(),
						*SubstrateTranslatorData.GetWaterScatteringCoefficients(), *SubstrateTranslatorData.GetWaterAbsorptionCoefficients(), *SubstrateTranslatorData.GetWaterPhaseG(), *SubstrateTranslatorData.GetColorScaleBehindWater(),
						bHasSSS, SSSProfile,
						*SubstrateTranslatorData.GetClearCoatNormal(), *SubstrateTranslatorData.GetCustomTangent());

				}
				else if ((Material->GetMaterialDomain() == MD_Surface || Material->GetMaterialDomain() == MD_RuntimeVirtualTexture || Material->GetMaterialDomain() == MD_DeferredDecal) && EditorOnlyData)
				{
					USubsurfaceProfile* SSSProfile = MaterialInterface ? MaterialInterface->GetSubsurfaceProfileRoot_Internal() : nullptr;
					const bool bHasSSS = SSSProfile != nullptr;
					const bool bHasAnisotropy = EditorOnlyData->Anisotropy.IsConnected();

					return UMaterialExpressionSubstrateShadingModels::CompileCommon(this,
						EditorOnlyData->BaseColor, EditorOnlyData->Specular, EditorOnlyData->Metallic, EditorOnlyData->Roughness, EditorOnlyData->EmissiveColor,
						EditorOnlyData->Opacity, EditorOnlyData->SubsurfaceColor, EditorOnlyData->ClearCoat, EditorOnlyData->ClearCoatRoughness,
						EditorOnlyData->ShadingModelFromMaterialExpression, MaterialInterface->GetShadingModels().GetFirstShadingModel(),
						*SubstrateTranslatorData.GetThinTranslucentTransmittanceColor(), *SubstrateTranslatorData.GetThinTranslucentSurfaceCoverage(),
						*SubstrateTranslatorData.GetWaterScatteringCoefficients(), *SubstrateTranslatorData.GetWaterAbsorptionCoefficients(), *SubstrateTranslatorData.GetWaterPhaseG(), *SubstrateTranslatorData.GetColorScaleBehindWater(),
						bHasAnisotropy, EditorOnlyData->Anisotropy,
						EditorOnlyData->Normal, EditorOnlyData->Tangent,
						*SubstrateTranslatorData.GetClearCoatNormal(), *SubstrateTranslatorData.GetCustomTangent(),
						bHasSSS, SSSProfile,
						EditorOnlyData);
				}
				else if (Material->GetMaterialDomain() == MD_Volume && EditorOnlyData)
				{
					return UMaterialExpressionSubstrateVolumetricFogCloudBSDF::CompileCommon(
						this,
						EditorOnlyData->BaseColor,
						EditorOnlyData->SubsurfaceColor,
						EditorOnlyData->EmissiveColor,
						EditorOnlyData->AmbientOcclusion,
						MaterialInterface->GetShadingModels().HasOnlyShadingModel(MSM_Unlit),
						EditorOnlyData);
				}
				else if (Material->GetMaterialDomain() == MD_LightFunction && EditorOnlyData)
				{
					return UMaterialExpressionSubstrateLightFunction::CompileCommon(this, EditorOnlyData->EmissiveColor, EditorOnlyData);
				}
				else if (Material->GetMaterialDomain() == MD_PostProcess && EditorOnlyData)
				{
					return UMaterialExpressionSubstratePostProcess::CompileCommon(this, EditorOnlyData->EmissiveColor, EditorOnlyData->Opacity, EditorOnlyData);
				}
				else if (Material->GetMaterialDomain() == MD_UI && EditorOnlyData)
				{
					return UMaterialExpressionSubstrateUI::CompileCommon(this, EditorOnlyData->EmissiveColor, EditorOnlyData->Opacity, EditorOnlyData);
				}

				return SubstrateCreateAndRegisterNullMaterial();
			};

		// Need to assign a valid scope for Front material to be correct (and avoid nullptr CurrentScopeChunks in case no other properties have been compiled before).
		SetMaterialProperty(MP_FrontMaterial, FrontMaterialShaderFrequency, false/*bUsePreviousFrameTime*/);

		Chunk[MP_FrontMaterial] = CompileFrontMaterial();

		FullySimplifiedFrontMaterialCodeChunkStart = SharedPropertyCodeChunks[FrontMaterialShaderFrequency].Num();
		SubstrateTranslatorData.CurrentSubstrateCompilationContext = ESubstrateCompilationContext::SCC_FullySimplified;
		SubstrateTranslatorData.FullySimplifiedSubstrateFrontMaterialCodeChunk = CompileFrontMaterial();
		SubstrateTranslatorData.CurrentSubstrateCompilationContext = ESubstrateCompilationContext::SCC_Default;
		FullySimplifiedFrontMaterialCodeChunkEnd = SharedPropertyCodeChunks[FrontMaterialShaderFrequency].Num();
	}

	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();

	ValidateShadingModelsForFeatureLevel(MaterialShadingModels);

	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();

	if (Domain == MD_Volume || (Domain == MD_Surface && IsSubsurfaceShadingModel(MaterialShadingModels)))
	{
		// Note we don't test for the blend mode as you can have a translucent material using the subsurface shading model

		// another ForceCast as CompilePropertyAndSetMaterialProperty() can return MCT_Float which we don't want here
		int32 SubsurfaceColor = Material->CompilePropertyAndSetMaterialProperty(MP_SubsurfaceColor, this);
		SubsurfaceColor = ForceCast(SubsurfaceColor, FMaterialAttributeDefinitionMap::GetValueType(MP_SubsurfaceColor), MFCF_ExactMatch | MFCF_ReplicateValue);

		static FName NameSubsurfaceProfile = SubsurfaceProfile::GetSubsurfaceProfileParameterName();

		// 1.0f is is a not used profile - later this gets replaced with the actual profile
		int32 CodeSubsurfaceProfile = ForceCast(ScalarParameter(NameSubsurfaceProfile, 1.0f), MCT_Float1);

		Chunk[MP_SubsurfaceColor] = AppendVector(SubsurfaceColor, CodeSubsurfaceProfile);
			
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
	}

	{
		Chunk[MP_CustomData0] = Material->CompilePropertyAndSetMaterialProperty(MP_CustomData0, this);
		Chunk[MP_CustomData1] = Material->CompilePropertyAndSetMaterialProperty(MP_CustomData1, this);
	}
	Chunk[MP_AmbientOcclusion] = Material->CompilePropertyAndSetMaterialProperty(MP_AmbientOcclusion, this);

	if (IsTranslucentBlendMode(BlendMode) || MaterialShadingModels.HasShadingModel(MSM_SingleLayerWater))
	{
		// Cast to exact match is needed for float parameter to be correctly cast to float2.
		int32 UserRefraction = ForceCast(Material->CompilePropertyAndSetMaterialProperty(MP_Refraction, this), MCT_Float2, MFCF_ExactMatch);
			
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
		int32 RefractionDepthBias = ForceCast(ScalarParameter(FName("RefractionDepthBias"), Material->GetRefractionDepthBiasValue()), MCT_Float1);

		Chunk[MP_Refraction] = AppendVector(UserRefraction, RefractionDepthBias);
	}

	if (bCompileForComputeShader)
	{
		Chunk[CompiledMP_EmissiveColorCS] = Material->CompilePropertyAndSetMaterialProperty(MP_EmissiveColor, this, SF_Compute);
	}

	if (Chunk[MP_WorldPositionOffset] != INDEX_NONE)
	{
		// Only calculate previous WPO if there is a current WPO
		Chunk[CompiledMP_PrevWorldPositionOffset] = Material->CompilePropertyAndSetMaterialProperty(MP_WorldPositionOffset, this, SF_Vertex, true);
	}

	Chunk[MP_PixelDepthOffset] = Material->CompilePropertyAndSetMaterialProperty(MP_PixelDepthOffset, this);

	Chunk[MP_Displacement] = Material->CompilePropertyAndSetMaterialProperty(MP_Displacement, this);

	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
	ResourcesString = TEXT("");

	// Compile all custom output nodes that were not compiled in the first batch.
	CompileCustomOutputs(CustomOutputExpressions, SeenCustomOutputExpressionsClasses, false);

	// Compile custom outputs that are set via MaterialAttributes. Nodes take precedence, so any outputs set via both MaterialAttributes and a node will use the value set on the node.
	const uint64 CustomOutputNodesBitmask = FMaterialAttributeDefinitionMap::GetCustomOutputNodesBitmask(CustomOutputExpressions);
	const uint64 ProcessedMaterialAttributesCustomOutputs = CompileMaterialAttributesCustomOutputs(CustomOutputNodesBitmask);
	const uint64 CustomOutputsBitmask = CustomOutputNodesBitmask | ProcessedMaterialAttributesCustomOutputs;

	// No more calls to non-vertex shader CompilePropertyAndSetMaterialProperty beyond this point
	const uint32 SavedNumUserTexCoords = GetNumUserTexCoords();

	for (uint32 CustomUVIndex = MP_CustomizedUVs0; CustomUVIndex <= MP_CustomizedUVs7; CustomUVIndex++)
	{
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
		// Only compile custom UV inputs for UV channels requested by the pixel shader inputs
		// Any unconnected inputs will have a texcoord generated for them in Material->CompileProperty, which will pass through the vertex (uncustomized) texture coordinates
		// Note: this is using NumUserTexCoords, which is set by translating all the pixel properties above
		if (CustomUVIndex - MP_CustomizedUVs0 < SavedNumUserTexCoords)
		{
			Chunk[CustomUVIndex] = Material->CompilePropertyAndSetMaterialProperty((EMaterialProperty)CustomUVIndex, this);
		}
	}

	// Output the implementation for any custom expressions we will call below.
	for (int32 ExpressionIndex = 0; ExpressionIndex < CustomExpressions.Num(); ExpressionIndex++)
	{
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
		ResourcesString += CustomExpressions[ExpressionIndex].Implementation + "\n\n";
	}

	// Translation is designed to have a code chunk generation phase followed by several passes that only has readonly access to the code chunks.
	// At this point we mark the code chunk generation complete.
	bAllowCodeChunkGeneration = false;

	bUsesEmissiveColor = SubstrateTranslatorData.bSubstrateWritesEmissive || IsMaterialPropertyUsed(MP_EmissiveColor, Chunk[MP_EmissiveColor], FLinearColor(0, 0, 0, 0), 3);
	bUsesPixelDepthOffset = (AllowPixelDepthOffset(Platform) && Material->AllowPixelDepthOffset() && IsMaterialPropertyUsed(MP_PixelDepthOffset, Chunk[MP_PixelDepthOffset], FLinearColor(0, 0, 0, 0), 1));

	bool bUsesWorldPositionOffsetCurrent = IsMaterialPropertyUsed(MP_WorldPositionOffset, Chunk[MP_WorldPositionOffset], FLinearColor(0, 0, 0, 0), 3);
	bool bUsesWorldPositionOffsetPrevious = IsMaterialPropertyUsed(MP_WorldPositionOffset, Chunk[CompiledMP_PrevWorldPositionOffset], FLinearColor(0, 0, 0, 0), 3);
	bUsesWorldPositionOffset = bUsesWorldPositionOffsetCurrent || bUsesWorldPositionOffsetPrevious;

	// Check if displacement is actually used (disable if it was left at the default, invalid constant value of -1)
	bUsesDisplacement = DoesPlatformSupportNanite(Platform) &&
		Material->IsTessellationEnabled() &&
		IsMaterialPropertyUsed(MP_Displacement, Chunk[MP_Displacement], FLinearColor(-1, 0, 0, 0), 1);

	// Check if opacity mask is used. This is used by the FMaterialShaderParameters to determine whether to generate masked shaders.
	const bool bUsesOpacityMask = IsMaterialPropertyUsed(MP_OpacityMask, Chunk[MP_OpacityMask], FLinearColor(1, 0, 0, 0), 1);
	// Check if opacity is used. This is can also generate masked shaders when casting shadows as masked from a translucent material.
	const bool bUsesOpacity = IsMaterialPropertyUsed(MP_Opacity, Chunk[MP_Opacity], FLinearColor(1, 0, 0, 0), 1);

	static const FGuid FirstPersonInterpolationAlphaGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("FirstPersonInterpolationAlpha"));
	const bool bHasFirstPersonOutput = (CustomOutputsBitmask & FMaterialAttributeDefinitionMap::GetBitmask(FirstPersonInterpolationAlphaGuid)) != 0;
	bUsesFirstPersonInterpolation = bHasFirstPersonOutput;
	MaterialCompilationOutput.bModifiesMeshPosition = bUsesPixelDepthOffset || bUsesWorldPositionOffset || bUsesDisplacement || bHasFirstPersonOutput;
	MaterialCompilationOutput.bUsesWorldPositionOffset = bUsesWorldPositionOffset;
	MaterialCompilationOutput.bUsesPixelDepthOffset = bUsesPixelDepthOffset;
	MaterialCompilationOutput.bUsesDisplacement = bUsesDisplacement;
	MaterialCompilationOutput.bUsesCustomizedUVs = Material->GetNumCustomizedUVs() > 0;
	MaterialCompilationOutput.bUsesOpacityMask = bUsesOpacityMask;
	MaterialCompilationOutput.bUsesOpacity = bUsesOpacity;

	static const FGuid TemporalResponsivenessGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("TemporalResponsiveness"));
	static const FGuid MotionVectorWorldOffsetGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("MotionVectorWorldOffset"));
	const bool bUsesTemporalResponsivenessOutput =  (CustomOutputsBitmask & FMaterialAttributeDefinitionMap::GetBitmask(TemporalResponsivenessGuid)) != 0;
	const bool bUsesMotionVectorWorldOffsetOutput = (CustomOutputsBitmask & FMaterialAttributeDefinitionMap::GetBitmask(MotionVectorWorldOffsetGuid)) != 0;
	MaterialCompilationOutput.bUsesTemporalResponsiveness = bUsesTemporalResponsivenessOutput;
	MaterialCompilationOutput.bUsesMotionVectorWorldOffset = bUsesMotionVectorWorldOffsetOutput;

	// Fully rough if we have a roughness code chunk and it's constant and evaluates to 1.
	bIsFullyRough = Chunk[MP_Roughness] != INDEX_NONE && IsMaterialPropertyUsed(MP_Roughness, Chunk[MP_Roughness], FLinearColor(1, 0, 0, 0), 1) == false;

	bUsesAnisotropy = IsMaterialPropertyUsed(MP_Anisotropy, Chunk[MP_Anisotropy], FLinearColor(0, 0, 0, 0), 1);
	MaterialCompilationOutput.bUsesAnisotropy = bUsesAnisotropy;
	if (SubstrateTranslatorData.bSubstrateMaterialUsesLegacyMaterialCompilation && bUsesAnisotropy)
	{
		MaterialCompilationOutput.SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_COMPLEX;
		MaterialCompilationOutput.SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature::Anisotropy;
	}
	if (uint16(MaterialCompilationOutput.SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures & (ESubstrateBsdfFeature::Anisotropy)) == uint16(ESubstrateBsdfFeature::Anisotropy))
	{
		MaterialCompilationOutput.bUsesAnisotropy = true;
	}

	/**
	 * A material is not compatible with the atlas if 
	 *  - it samples world position of depth since we do not have such data when rendering an atlas tile.
	 *  - it manipulates TexCoords for the following reasons:
	 *		- Clamped textures will look different if texcoords are manipulated
	 *		- Wrapped textures look good with any texcoords offsets (since tiles are repeatable and have correct HW filtering at edges)
	 *		- Wrapped textures look different however if texcoords are scaled (looks correct for 1, 2, 3 but not for 1.5 or 1.1 for instance as UV space [0,1] will no longer align with the atlas tile edges)
	 * But, an artist can specify and override the fact that a material is compatible with the light function atlas.
	 */
	MaterialCompilationOutput.bIsLightFunctionAtlasCompatible = (!bUsesVertexPosition && !bUsesSceneDepth && !MaterialCompilationOutput.bNeedsSceneTextures && !bPotentiallyManipulateTexCoords) || Material->GetForceCompatibleWithLightFunctionAtlas();

	EMaterialDecalResponse MDR = (EMaterialDecalResponse)Material->GetMaterialDecalResponse();
	if (MDR == MDR_Color || MDR == MDR_ColorNormal || MDR == MDR_ColorRoughness || MDR == MDR_ColorNormalRoughness)
	{
		MaterialCompilationOutput.SetIsDBufferTextureUsed(0);
		AddEstimatedTextureSample(1);
	}
	if (MDR == MDR_Normal || MDR == MDR_ColorNormal || MDR == MDR_NormalRoughness || MDR == MDR_ColorNormalRoughness)
	{
		MaterialCompilationOutput.SetIsDBufferTextureUsed(1);
		AddEstimatedTextureSample(1);
	}
	if (MDR == MDR_Roughness || MDR == MDR_ColorRoughness || MDR == MDR_NormalRoughness || MDR == MDR_ColorNormalRoughness)
	{
		MaterialCompilationOutput.SetIsDBufferTextureUsed(2);
		AddEstimatedTextureSample(1);
	}

	bOpacityPropertyIsUsed = IsMaterialPropertyUsed(MP_Opacity, Chunk[MP_Opacity], FLinearColor(1, 0, 0, 0), 1);

	bUsesCurvature = FeatureLevel == ERHIFeatureLevel::ES3_1 &&
					((MaterialShadingModels.HasShadingModel(MSM_SubsurfaceProfile) && IsMaterialPropertyUsed(MP_CustomData0, Chunk[MP_CustomData0], FLinearColor(1, 0, 0, 0), 1))
					|| (MaterialShadingModels.HasShadingModel(MSM_Eye) && bOpacityPropertyIsUsed));

	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();

	// Don't allow opaque and masked materials to scene depth as the results are undefined
	if (bUsesSceneDepth && Domain != MD_PostProcess && !IsTranslucentBlendMode(BlendMode))
	{
		Errorf(TEXT("Only transparent or postprocess materials can read from scene depth."));
	}

	if (bUsesSceneDepth)
	{
		MaterialCompilationOutput.SetIsSceneTextureUsed(PPI_SceneDepth);
	}

	MaterialCompilationOutput.bUsesDistanceCullFade = bUsesDistanceCullFade;

	if (MaterialCompilationOutput.bNeedsSceneTextures)
	{
		if (Domain != MD_DeferredDecal && Domain != MD_PostProcess)
		{
			if (!MaterialShadingModels.HasShadingModel(MSM_SingleLayerWater) && IsOpaqueOrMaskedBlendMode(BlendMode))
			{
				// In opaque pass, none of the textures are available
				Errorf(TEXT("SceneTexture expressions cannot be used in opaque materials except if used with the Single Layer Water shading model."));
			}
			else if (bNeedsSceneTexturePostProcessInputs)
			{
				Errorf(TEXT("SceneTexture expressions cannot use post process inputs or scene color in non post process domain materials"));
			}
		}
	}

	// Final validation logic shared between old and new translator.
	TArray<FString> ValidationErrors;
	UE::MaterialTranslatorUtils::FinalCompileValidation(
		Material->GetMaterialInterface()->GetMaterial(),
		MaterialCompilationOutput,
		MaterialShadingModels,
		BlendMode,
		SubstrateTranslatorData.FrontMaterialExpr != nullptr,
		Platform,
		ValidationErrors);

	for (const FString& ValidationError : ValidationErrors)
	{
		Errorf(*ValidationError);
	}

	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();

	// Catch any modifications to NumUserTexCoords that will not seen by customized UVs
	check(SavedNumUserTexCoords == GetNumUserTexCoords());

	FString InterpolatorsOffsetsDefinitionCode;
	TArray<uint32> Dummmy;
	TBitArray<> FinalAllocatedCoords = GetVertexInterpolatorsOffsets(InterpolatorsOffsetsDefinitionCode, Dummmy);

	// Finished compilation, verify final interpolator count restrictions
	if (CurrentCustomVertexInterpolatorOffset > 0)
	{
		const int32 MaxNumScalars = 8 * 2;
		const int32 TotalUsedScalars = FinalAllocatedCoords.FindLast(true) + 1;

		if (TotalUsedScalars > MaxNumScalars)
		{
			Errorf(TEXT("Maximum number of custom vertex interpolators exceeded. (%i / %i scalar values) (TexCoord: %i scalars, Custom: %i scalars)"),
				TotalUsedScalars, MaxNumScalars, GetNumUserTexCoords() * 2, CurrentCustomVertexInterpolatorOffset);
		}
	}

	MaterialCompilationOutput.NumUsedUVScalars = GetNumUserTexCoords() * 2;
	MaterialCompilationOutput.NumUsedCustomInterpolatorScalars = CurrentCustomVertexInterpolatorOffset;

	for (int32 VariationIter = 0; VariationIter < CompiledPDV_MAX; VariationIter++)
	{
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
		ECompiledPartialDerivativeVariation Variation = (ECompiledPartialDerivativeVariation)VariationIter;

		// Do Normal Chunk first
		{
			GetFixedParameterCode(
				0,
				NormalCodeChunkEnd,
				Chunk[MP_Normal],
				SharedPropertyCodeChunks[NormalShaderFrequency],
						DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[MP_Normal],
						DerivativeVariations[Variation].TranslatedCodeChunks[MP_Normal],
						Variation);

			// Always gather MP_Normal definitions as they can be shared by other properties
			if (DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[MP_Normal].IsEmpty())
			{
				DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[MP_Normal] = GetDefinitions(SharedPropertyCodeChunks[NormalShaderFrequency], 0, NormalCodeChunkEnd, Variation);
			}
		}

		// Now the rest, skipping Normal
		for (uint32 PropertyId = 0; PropertyId < MP_MAX; ++PropertyId)
		{
			CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
			if (PropertyId == MP_MaterialAttributes || PropertyId == MP_Normal || PropertyId == MP_CustomOutput)
			{
				continue;
			}

			const EShaderFrequency PropertyShaderFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency((EMaterialProperty)PropertyId);

			int32 StartChunk = 0;
			if (PropertyShaderFrequency == NormalShaderFrequency && SharedPixelProperties[PropertyId])
			{
				// When processing shared properties, do not generate the code before the Normal was generated as those are already handled
				StartChunk = NormalCodeChunkEnd;
			}

			// Reduce definition statements that don't contribute to the function's return value.
			// @todo-lh: This should be expanded to a general reduction, but is currently only intended to fix an FXC internal compiler error reported in UE-117831
			// When Substrate is enabled, we do not reduce MP_Displacement as it causes shader compilation error, due to FrontMaterial expression being removed, causing local variables to be removed while still used.
			const bool bReduceAfterReturnValue = Substrate::IsSubstrateEnabled() ? 
				(PropertyId == MP_WorldPositionOffset || PropertyId == CompiledMP_PrevWorldPositionOffset) : 
				(PropertyId == MP_WorldPositionOffset || PropertyId == CompiledMP_PrevWorldPositionOffset || PropertyId == MP_Displacement);

			if (SubstrateTranslatorData.bSubstrateEnabled && PropertyId >= MP_FrontMaterial && PropertyShaderFrequency == FrontMaterialShaderFrequency)
			{
				int32 PropertyIdCodeChunk = Chunk[PropertyId];
				bool bPropertyIsConstant = false;
				if (PropertyIdCodeChunk != INDEX_NONE)
				{
					TArray<FShaderCodeChunk>& CodeChunks = SharedPropertyCodeChunks[PropertyShaderFrequency];
					bPropertyIsConstant = CodeChunks[PropertyIdCodeChunk].UniformExpression && CodeChunks[PropertyIdCodeChunk].UniformExpression->IsConstant();
				}

				// We must have all properties generated the same way since it is the last one that is used to append to MaterialTemplate.ush
				FString Temp;

				// Initialise definitions to TranslatedCodeChunkDefinitions or constant value directly assigned to PixelMaterialInputs to TranslatedCodeChunks.
				GetFixedParameterCode(
					StartChunk,
					FullySimplifiedFrontMaterialCodeChunkStart,
					Chunk[PropertyId],
					SharedPropertyCodeChunks[PropertyShaderFrequency],
					DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId],
					DerivativeVariations[Variation].TranslatedCodeChunks[PropertyId],
					Variation,
					bReduceAfterReturnValue);

				// If bPropertyIsConstant, then this means that the property have not generated any code definition in TranslatedCodeChunkDefinitions 
				// and instead will be assigned as a constant on PixelMaterialInputs using TranslatedCodeChunks.
				// We need to not add any definitions such as SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL to the TranslatedCodeChunkDefinitions string so we need to skip the remaining, now useless, operations.
				// With that, TranslatedCodeChunkDefinitions[PropertyId] will be empty and this property will be skipped to not end up being the "LastProperty" representing the final code definition.
				if (!bPropertyIsConstant)
				{
					DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId] += TEXT("\t#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1\n");

					GetFixedParameterCode(
						FullySimplifiedFrontMaterialCodeChunkStart,
						FullySimplifiedFrontMaterialCodeChunkEnd,
						Chunk[PropertyId],
						SharedPropertyCodeChunks[PropertyShaderFrequency],
						Temp,
						DerivativeVariations[Variation].TranslatedCodeChunks[PropertyId],
						Variation,
						bReduceAfterReturnValue);

					DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId] += Temp + TEXT("\t#endif // SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL\n");

					GetFixedParameterCode(
						FullySimplifiedFrontMaterialCodeChunkEnd,
						SharedPropertyCodeChunks[PropertyShaderFrequency].Num(),
						Chunk[PropertyId],
						SharedPropertyCodeChunks[PropertyShaderFrequency],
						Temp,
						DerivativeVariations[Variation].TranslatedCodeChunks[PropertyId],
						Variation,
						bReduceAfterReturnValue);

					DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId] += Temp;
				}
			}
			else
			{
				GetFixedParameterCode(
					StartChunk,
					SharedPropertyCodeChunks[PropertyShaderFrequency].Num(),
					Chunk[PropertyId],
					SharedPropertyCodeChunks[PropertyShaderFrequency],
					DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId],
					DerivativeVariations[Variation].TranslatedCodeChunks[PropertyId],
					Variation,
					bReduceAfterReturnValue);
			}
		}

		// The code chunk corresponding to FullySimplifiedSubstrateFrontMaterialCodeChunk have already been written as part of MP_FrontMaterial.
		// Here we get the FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunks representing the variable storing the final fully simplified SubstrateData.
		if (SubstrateTranslatorData.bSubstrateEnabled)
		{
			uint32 PropertyId = MP_FrontMaterial;

			if (PropertyId == MP_MaterialAttributes || PropertyId == MP_Normal || PropertyId == MP_CustomOutput)
			{
				continue;
			}

			const EShaderFrequency PropertyShaderFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency((EMaterialProperty)PropertyId);

			int32 StartChunk = 0;
			if (PropertyShaderFrequency == NormalShaderFrequency && SharedPixelProperties[PropertyId])
			{
				// When processing shared properties, do not generate the code before the Normal was generated as those are already handled
				StartChunk = NormalCodeChunkEnd;
			}

			// Reduce definition statements that don't contribute to the function's return value.
			// @todo-lh: This should be expanded to a general reduction, but is currently only intended to fix an FXC internal compiler error reported in UE-117831
			const bool bReduceAfterReturnValue = (PropertyId == MP_WorldPositionOffset || PropertyId == CompiledMP_PrevWorldPositionOffset || PropertyId == MP_Displacement);

			GetFixedParameterCode(
				StartChunk,
				SharedPropertyCodeChunks[PropertyShaderFrequency].Num(),
				SubstrateTranslatorData.FullySimplifiedSubstrateFrontMaterialCodeChunk,								//Chunk[PropertyId],
				SharedPropertyCodeChunks[PropertyShaderFrequency],
				SubstrateTranslatorData.FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunkDefinitions,
				SubstrateTranslatorData.FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunks,
				Variation,
				bReduceAfterReturnValue);
		}

		for (uint32 PropertyId = MP_MAX; PropertyId < CompiledMP_MAX; ++PropertyId)
		{
			CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
			switch (PropertyId)
			{
			case CompiledMP_EmissiveColorCS:
				if (bCompileForComputeShader)
				{
							GetFixedParameterCode(Chunk[PropertyId], SharedPropertyCodeChunks[SF_Compute], DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId], DerivativeVariations[Variation].TranslatedCodeChunks[PropertyId], Variation);
				}
				break;
			case CompiledMP_PrevWorldPositionOffset:
				{
						GetFixedParameterCode(Chunk[PropertyId], SharedPropertyCodeChunks[SF_Vertex], DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[PropertyId], DerivativeVariations[Variation].TranslatedCodeChunks[PropertyId], Variation);
				}
				break;
			default: check(0);
				break;
			}
		}

		// Output the implementation for any custom output expressions
		for (int32 ExpressionIndex = 0; ExpressionIndex < DerivativeVariations[Variation].CustomOutputImplementations.Num(); ExpressionIndex++)
		{
			CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
			ResourcesString += DerivativeVariations[Variation].CustomOutputImplementations[ExpressionIndex] + "\n\n";
		}
	}

	// Make sure every uniform expression is marked as accessed by the VS. This is needed below to send the information to the uniform expression set to create the compact version of the UB
	{
		static_assert(sizeof(EShaderFrequencyMask)*8 >= (1 << SF_NumBits));
		const EShaderFrequencyMask VSFrequencyMask = (1 << SF_Vertex);
		for (FShaderCodeChunk& ShaderCodeChunk : SharedPropertyCodeChunks[SF_Vertex])
		{
			if (ShaderCodeChunk.UniformExpression)
			{
				ShaderCodeChunk.UniformExpression->ShaderFrequencyMask |= VSFrequencyMask;
				FMaterialUniformExpressionTexture* MaterialUniformExpressionTexture = ShaderCodeChunk.UniformExpression->GetTextureUniformExpression();
				if (MaterialUniformExpressionTexture)
				{
					int32 MaterialCacheTagIndex = MaterialUniformExpressionTexture->GetMaterialCacheTagIndex();
					if (MaterialCompilationOutput.UniformExpressionSet.MaterialCacheTagStacks.IsValidIndex(MaterialCacheTagIndex))
					{
						MaterialCompilationOutput.UniformExpressionSet.MaterialCacheTagStacks[MaterialCacheTagIndex].ShaderFrequencyMask |= VSFrequencyMask;
					}
				}
			}
		}
	}

	FString ResourcesStringPrefix = TEXT("#define __Material_UB__ Material" HLSL_LINE_TERMINATOR);

	if (UseMaterialVSUniformBuffer())
	{
		ResourcesStringPrefix += TEXT(
			"#if (VERTEXSHADER || MESHSHADER || (SHADER_USES_ONLY_VERTEX_DATA && SHADER_SUPPORTS_MATERIAL_VS_UB))" HLSL_LINE_TERMINATOR
			"#define __Material_UB_VS__ MaterialVS" HLSL_LINE_TERMINATOR
			"#define USE_MATERIAL_VS_UB 1" HLSL_LINE_TERMINATOR
			"#else" HLSL_LINE_TERMINATOR
			"#define __Material_UB_VS__ Material" HLSL_LINE_TERMINATOR
			"#define USE_MATERIAL_VS_UB 0" HLSL_LINE_TERMINATOR
			"#endif" HLSL_LINE_TERMINATOR);
	}
	else
	{
		// When we want to compile a Nanite compute shader which is also calling SF_Vertex level 
		// functions, we actually remap __Material_UB_VS__ to the generic Material UB. 
		// Since the indices are the same for VS-accessed uniforms, we only need to replace the UB name
		ResourcesStringPrefix += TEXT("#define __Material_UB_VS__ Material" HLSL_LINE_TERMINATOR);
		ResourcesStringPrefix += TEXT("#define USE_MATERIAL_VS_UB 0" HLSL_LINE_TERMINATOR);
	}

	ResourcesStringPrefix += GenerateUniformExpressionMacroDefinitions(); 
	ResourcesString = ResourcesStringPrefix + ResourcesString;

	// Store the number of float4s
	MaterialCompilationOutput.UniformExpressionSet.UniformPreshaderBufferSize = (UniformPreshaderOffset + 3u) / 4u;

	FCompactUniformExpressionSet* CompactUniformsVS = nullptr;
	if (UseMaterialVSUniformBuffer())
	{
		CompactUniformsVS = MaterialCompilationOutput.UniformExpressionSet.CreateCompactUniformsVS();
		check(CompactUniformsVS);
	}
	else
	{
		MaterialCompilationOutput.UniformExpressionSet.CompactUniformsVSOptional.Empty();
		check(UniformPreshaderOffsetVS == 0);
	}

	if (CompactUniformsVS)
	{
		CompactUniformsVS->UniformPreshaderBufferSize = (UniformPreshaderOffsetVS + 3u) / 4u;
	}

	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		MaterialCompilationOutput.UniformExpressionSet.UniformTextureParameters[TypeIndex].Empty(UniformTextureExpressions[TypeIndex].Num());
		for (FMaterialUniformExpressionTexture* TextureExpression : UniformTextureExpressions[TypeIndex])
		{
			CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
			FMaterialTextureParameterInfo& TextureParameter = MaterialCompilationOutput.UniformExpressionSet.UniformTextureParameters[TypeIndex].AddDefaulted_GetRef();
			TextureExpression->GetTextureParameterInfo(TextureParameter);
			TextureParameter.ShaderFrequencyMask = TextureExpression->ShaderFrequencyMask;
		}
	}

	MaterialCompilationOutput.UniformExpressionSet.UniformExternalTextureParameters.Empty(UniformExternalTextureExpressions.Num());
	for (FMaterialUniformExpressionExternalTexture* TextureExpression : UniformExternalTextureExpressions)
	{
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
		FMaterialExternalTextureParameterInfo& TextureParameter = MaterialCompilationOutput.UniformExpressionSet.UniformExternalTextureParameters.AddDefaulted_GetRef();
		TextureExpression->GetExternalTextureParameterInfo(TextureParameter);
		TextureParameter.ShaderFrequencyMask = TextureExpression->ShaderFrequencyMask;
	}

	MaterialCompilationOutput.UniformExpressionSet.UniformTextureCollectionParameters.Empty(UniformTextureCollectionExpressions.Num());
	for (const FMaterialUniformExpressionTextureCollection* TextureExpression : UniformTextureCollectionExpressions)
	{
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();
		FMaterialTextureCollectionParameterInfo& TextureParameter = MaterialCompilationOutput.UniformExpressionSet.UniformTextureCollectionParameters.AddDefaulted_GetRef();
		TextureExpression->GetTextureCollectionParameterInfo(TextureParameter);
		TextureParameter.ShaderFrequencyMask = TextureExpression->ShaderFrequencyMask;
	}

	// Write substrate tree specific code.
	if (SubstrateTranslatorData.bSubstrateEnabled)
	{
		CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();

		const bool bSubstrateFrontMaterialProvided = Chunk[MP_FrontMaterial] != INDEX_NONE;
		const bool bValid = SubstrateTranslatorData.GenerateSubstrateTreeHLSLCode(bSubstrateFrontMaterialProvided);
		if(!bValid)
		{
			TranslationResult = EHLSLMaterialTranslatorResult::Failure;
			return;
		}

		// We conveniently append that code to the resources string.
		ResourcesString += SubstrateTranslatorData.SubstrateTreeHLSL; // 
	}

	MaterialCompilationOutput.UniformExpressionSet.SetParameterCollections(ParameterCollections);

	check(VTStacks.Num() == MaterialCompilationOutput.UniformExpressionSet.VTStacks.Num());
	for (int VTStackIndex = 0; VTStackIndex < MaterialCompilationOutput.UniformExpressionSet.VTStacks.Num(); ++VTStackIndex)
	{
		MaterialCompilationOutput.UniformExpressionSet.VTStacks[VTStackIndex].ShaderFrequencyMask = (1 << VTStacks[VTStackIndex].ShaderFrequency);
	}

	// Store the number of unique VT samples
	MaterialCompilationOutput.EstimatedNumVirtualTextureLookups = NumVtSamples;
}

void FHLSLMaterialTranslator::ValidateShadingModelsForFeatureLevel(const FMaterialShadingModelField& ShadingModels)
{
#if 0
	if (FeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		const TArray<EMaterialShadingModel>& InvalidShadingModels = {};
		for (EMaterialShadingModel InvalidShadingModel : InvalidShadingModels)
		{
			if (ShadingModels.HasShadingModel(InvalidShadingModel))
			{
				FString FeatureLevelName;
				GetFeatureLevelName(FeatureLevel, FeatureLevelName);

				FString ShadingModelName;
				if (const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/Engine.EMaterialShadingModel"), EFindObjectFlags::ExactClass))
				{
					ShadingModelName = EnumPtr->GetNameStringByValue(InvalidShadingModel);
				}

				Errorf(TEXT("ShadingModel %s not supported in feature level %s"), *ShadingModelName, *FeatureLevelName);
			}
		}
	}
#endif
}

const TCHAR* FHLSLMaterialTranslator::GetCurrentMaterialUniformBufferName() const
{
	return GetCurrentMaterialUBNameInternal(ShaderFrequency);
}

void FHLSLMaterialTranslator::GetMaterialEnvironment(EShaderPlatform InPlatform, FShaderCompilerEnvironment& OutEnvironment)
{
	check(InPlatform == GetShaderPlatform());

	bool bMaterialRequestsDualSourceBlending = false;

	if (EnvironmentDefines->bNeedsParticlePosition)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_POSITION"), 1);
	}

	if (EnvironmentDefines->bNeedsParticleVelocity)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_VELOCITY"), 1);
	}

	if (EnvironmentDefines->bUseDynamicParameters)
	{
		OutEnvironment.SetDefine(TEXT("USE_DYNAMIC_PARAMETERS"), 1);
		OutEnvironment.SetDefine(TEXT("DYNAMIC_PARAMETERS_MASK"), EnvironmentDefines->DynamicParametersMask);
	}

	if (EnvironmentDefines->bNeedsParticleTime)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_TIME"), 1);
	}

	if (EnvironmentDefines->bUsesParticleMotionBlur)
	{
		OutEnvironment.SetDefine(TEXT("USES_PARTICLE_MOTION_BLUR"), 1);
	}

	if (EnvironmentDefines->bNeedsParticleRandom)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_RANDOM"), 1);
	}

	if (EnvironmentDefines->bSphericalParticleOpacity)
	{
		OutEnvironment.SetDefine(TEXT("SPHERICAL_PARTICLE_OPACITY"), TEXT("1"));
	}

	if (EnvironmentDefines->bUseParticleSubUVs)
	{
		OutEnvironment.SetDefine(TEXT("USE_PARTICLE_SUBUVS"), TEXT("1"));
	}

	if (EnvironmentDefines->bLightmapUVAccess)
	{
		OutEnvironment.SetDefine(TEXT("LIGHTMAP_UV_ACCESS"), TEXT("1"));
	}

	if (EnvironmentDefines->bUsesAOMaterialMask)
	{
		OutEnvironment.SetDefine(TEXT("USES_AO_MATERIAL_MASK"), TEXT("1"));
	}

	if (EnvironmentDefines->bUsesSpeedtree)
	{
		OutEnvironment.SetDefine(TEXT("USES_SPEEDTREE"), TEXT("1"));
	}

	if (EnvironmentDefines->bNeedsWorldPositionExcludingShaderOffsets)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS"), TEXT("1"));
	}

	if (EnvironmentDefines->bNeedsParticleSize)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_SIZE"), TEXT("1"));
	}

	if (EnvironmentDefines->bNeedsParticleSpriteRotation)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_SPRITE_ROTATION"), TEXT("1"));
	}

	if (EnvironmentDefines->bNeedsSceneTextures)
	{
		OutEnvironment.SetDefine(TEXT("NEEDS_SCENE_TEXTURES"), TEXT("1"));
	}

	if (EnvironmentDefines->bAlphaPropagatePostProcessInput0 || EnvironmentDefines->bAlphaPropagateUserSceneTexture)
	{
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_PROPAGATE_ALPHA_INPUT"), EnvironmentDefines->bAlphaPropagatePostProcessInput0 ? TEXT("PPI_PostProcessInput0") : TEXT("UserSceneTextureSceneColorInput"));
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_USED_SCENE_TEXTURES"), EnvironmentDefines->UsedSceneTextures);
	}

	if (EnvironmentDefines->bUsesEyeAdaptation)
	{
		OutEnvironment.SetDefine(TEXT("USES_EYE_ADAPTATION"), TEXT("1"));
	}

	if (EnvironmentDefines->bVirtualTextureOutput)
	{
		OutEnvironment.SetDefine(TEXT("VIRTUAL_TEXTURE_OUTPUT"), 1);
	}

	OutEnvironment.SetDefine(TEXT("USES_PER_INSTANCE_CUSTOM_DATA"), EnvironmentDefines->bUsesPerInstanceCustomData);
	OutEnvironment.SetDefine(TEXT("USES_PER_INSTANCE_FADE_AMOUNT"), EnvironmentDefines->bUsesPerInstanceFadeAmount);

	OutEnvironment.SetDefine(TEXT("USES_VERTEX_INTERPOLATOR"), EnvironmentDefines->bUsesVertexInterpolator);
	OutEnvironment.SetDefine(TEXT("NUM_CUSTOMIZED_UVS"), EnvironmentDefines->NumCustomizedUVs);

	OutEnvironment.SetDefine(TEXT("MATERIAL_SKY_ATMOSPHERE"), EnvironmentDefines->bUsesSkyAtmosphere);
	OutEnvironment.SetDefine(TEXT("INTERPOLATE_VERTEX_COLOR"), EnvironmentDefines->bUsesVertexColor);
	OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_COLOR"), EnvironmentDefines->bUsesParticleColor);
	OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_LOCAL_TO_WORLD"), EnvironmentDefines->bUsesParticleLocalToWorld);
	OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_WORLD_TO_LOCAL"), EnvironmentDefines->bUsesParticleWorldToLocal);
	OutEnvironment.SetDefine(TEXT("NEEDS_INSTANCE_LOCAL_TO_WORLD_PS"), EnvironmentDefines->bUsesInstanceLocalToWorldPS);
	OutEnvironment.SetDefine(TEXT("NEEDS_INSTANCE_WORLD_TO_LOCAL_PS"), EnvironmentDefines->bUsesInstanceWorldToLocalPS);
	OutEnvironment.SetDefine(TEXT("NEEDS_PER_INSTANCE_RANDOM_PS"), EnvironmentDefines->bUsesPerInstanceRandomPS);
	OutEnvironment.SetDefine(TEXT("USES_TRANSFORM_VECTOR"), EnvironmentDefines->bUsesTransformVector);
	OutEnvironment.SetDefine(TEXT("WANT_PIXEL_DEPTH_OFFSET"), EnvironmentDefines->bUsesPixelDepthOffset);
	OutEnvironment.SetDefine(TEXT("PIXEL_DEPTH_OFFSET_MODE"), EnvironmentDefines->PixelDepthOffsetMode);

	// we want USES_WORLD_POSITION_OFFSET to be readable as a bool compile argument, hence the != 0 comparison 
	// (bUsesWorldPositionOffset is actually a 1-bit uint32 bitfield member)
	OutEnvironment.SetDefineAndCompileArgument(TEXT("USES_WORLD_POSITION_OFFSET"), EnvironmentDefines->bUsesWorldPositionOffset);
	OutEnvironment.SetDefineAndCompileArgument(TEXT("USES_DISPLACEMENT"), EnvironmentDefines->bUsesDisplacement);
	OutEnvironment.SetDefineAndCompileArgument(TEXT("SHADER_SUPPORTS_MATERIAL_VS_UB"), !EnvironmentDefines->bUsesDisplacement);

	OutEnvironment.SetDefine(TEXT("USES_EXPLICIT_DERIVATIVES"), EnvironmentDefines->bUsesExplicitDerivatives);
	OutEnvironment.SetDefineAndCompileArgument(TEXT("USES_FIRST_PERSON_INTERPOLATION"), EnvironmentDefines->bUsesFirstPersonInterpolation);

	OutEnvironment.SetDefine(TEXT("USES_EMISSIVE_COLOR"), EnvironmentDefines->bUsesEmissiveColor);
	// Distortion uses tangent space transform 
	OutEnvironment.SetDefine(TEXT("USES_DISTORTION"), EnvironmentDefines->bUsesDistortion);

	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENCY_FOGGING"), EnvironmentDefines->bMaterialEnableTranslucencyFogging);
	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENCY_CLOUD_FOGGING"), EnvironmentDefines->bMaterialEnableTranslucencyCloudFogging);
	OutEnvironment.SetDefine(TEXT("MATERIAL_IS_SKY"), EnvironmentDefines->bMaterialIsSky);
	OutEnvironment.SetDefine(TEXT("MATERIAL_COMPUTE_FOG_PER_PIXEL"), EnvironmentDefines->bMaterialComputeFogPerPixel);
	OutEnvironment.SetDefine(TEXT("MATERIAL_FULLY_ROUGH"), EnvironmentDefines->bMaterialFullyRough);

	OutEnvironment.SetDefine(TEXT("MATERIAL_USES_ANISOTROPY"), EnvironmentDefines->bMaterialUsesAnisotropy);
	OutEnvironment.SetDefine(TEXT("MATERIAL_USES_SPECULAR_PROFILE"), EnvironmentDefines->bMaterialUsesSpecularProfile);

	OutEnvironment.SetDefine(TEXT("MATERIAL_DECAL_READ_MASK"), EnvironmentDefines->MaterialDecalReadMask);
	OutEnvironment.SetDefine(TEXT("MATERIAL_USES_DECAL_LOOKUP"), EnvironmentDefines->bMaterialUsesDecalLookup);
	OutEnvironment.SetDefine(TEXT("MATERIAL_PATH_TRACING_BUFFER_READ"), EnvironmentDefines->MaterialPathTracingBufferRead);
	OutEnvironment.SetDefine(TEXT("MATERIAL_NEURAL_POST_PROCESS"), EnvironmentDefines->MaterialNeuralPostProcess);

	OutEnvironment.SetDefine(TEXT("NUM_VIRTUALTEXTURE_SAMPLES"), EnvironmentDefines->NumVirtualTextureSamples);
	OutEnvironment.SetDefine(TEXT("NUM_VIRTUALTEXTURE_FEEDBACK_REQUESTS"), EnvironmentDefines->NumVirtualTextureFeedbackRequests);
	OutEnvironment.SetDefine(TEXT("MATERIAL_VIRTUALTEXTURE_FEEDBACK"), EnvironmentDefines->bMaterialVirtualTextureFeedback);

	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENT_LOCAL_LIGHT_SHADOW"), EnvironmentDefines->bMaterialEnableTranslucentLocalLightShadow);
	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_LOCAL_LIGHT_SHADOW"), EnvironmentDefines->bMaterialEnableTranslucentHighQualityLocalLightShadow);
	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_DIRECTIONAL_LIGHT_SHADOW"), EnvironmentDefines->bMaterialEnableTranslucentHighQualityDirectionalLightShadow);

	OutEnvironment.SetDefine(TEXT("MATERIAL_SAMPLES_CACHE"), Material->SamplesMaterialCache() || Material->IsDefaultMaterial());

	check(MaterialCompilationOutput.UniformExpressionSet.VTStacks.Num() == EnvironmentDefines->VirtualPageTypes.Num());
	for (int i = 0; i < EnvironmentDefines->VirtualPageTypes.Num(); ++i)
	{
		if (EnvironmentDefines->VirtualPageTypes[i] & FEnvironmentDefines::TABLE_MESH_PAINT)
		{
			OutEnvironment.SetDefine(
				*FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_%d"), i), 
				TEXT("Scene.MeshPaint.PageTableTexture"));

			OutEnvironment.SetDefine(
				*FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_UNIFORM_%d"), i), 
				TEXT("GetMeshPaintTextureDescriptor(GetPrimitiveData(Parameters))"));
		}
		else if (EnvironmentDefines->VirtualPageTypes[i] & FEnvironmentDefines::TABLE_MATERIAL_CACHE)
		{
			// Driven manually
		}
		else if (EnvironmentDefines->VirtualPageTypes[i] & FEnvironmentDefines::TABLE_COLLECTION)
		{
			/** Page table fetched during sampling */
		}
		else
		{
			FMaterialVirtualTextureStack& CurrentVTStack = MaterialCompilationOutput.UniformExpressionSet.VTStacks[i];
			EShaderFrequency UniformExpressionVSOnlyFrequency = (HasShaderFrequency(CurrentVTStack.ShaderFrequencyMask, SF_Vertex)) ? SF_Vertex : SF_NumFrequencies;

			const TCHAR* MaterialVSName = GetCurrentMaterialUBNameInternal(UniformExpressionVSOnlyFrequency);

			// Setup page table defines to map each VT stack to either 1 or 2 page table textures, depending on how many layers it uses
			FString PageTableValue = FString::Printf(TEXT("%s.VirtualTexturePageTable0_%d"), MaterialVSName, i);
			if (EnvironmentDefines->VirtualPageTypes[i] & FEnvironmentDefines::LOCAL_TABLE1)
			{
				PageTableValue += FString::Printf(TEXT(", %s.VirtualTexturePageTable1_%d"), MaterialVSName, i);
			}
			if (EnvironmentDefines->VirtualPageTypes[i] & FEnvironmentDefines::LOCAL_ADAPTIVE_INDIRECTION)
			{
				PageTableValue += FString::Printf(TEXT(", %s.VirtualTexturePageTableIndirection_%d"), MaterialVSName, i);
			}
			OutEnvironment.SetDefine(*FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_%d"), i), *PageTableValue);

			// Setup page table uniform defines.
			FString PageTableUniformValue = FString::Printf(TEXT("%s.VTPackedPageTableUniform[%d*2], %s.VTPackedPageTableUniform[%d*2+1]"), MaterialVSName, i, MaterialVSName, i);
			OutEnvironment.SetDefine(*FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_UNIFORM_%d"), i), *PageTableUniformValue);
		}
	}

	for (int32 CollectionIndex = 0; CollectionIndex < EnvironmentDefines->ParameterCollections.Num(); CollectionIndex++)
	{
		// Add uniform buffer declarations for any parameter collections referenced
		const FString CollectionName = FString::Printf(TEXT("MaterialCollection%u"), CollectionIndex);

		// Check that the parameter collection loaded successfully.
		UMaterialParameterCollection* ParameterCollection = EnvironmentDefines->ParameterCollections[CollectionIndex];
		if (!ParameterCollection)
		{
			UE_LOGF(LogMaterial, Warning, "Null parameter collection found in environment defines while translating material.");
			continue;
		}

		// Ensure PostLoad is called so the uniform buffers are created in case the parameter collection was loaded async 
		ParameterCollection->ConditionalPostLoad();

		// Check that the parameter collection uniform buffer structure is valid
		if (!ParameterCollection->HasValidUniformBufferStruct())
		{
			UE_LOGF(LogMaterial, Warning, "Invalid parameter collection uniform buffer struct found in environment defines while translating material.");
			continue;
		}

		// This can potentially become an issue for MaterialCollection Uniform Buffers if they ever get non-numeric resources (eg Textures), as
		// OutEnvironment.ResourceTableMap has a map by name, and the N ParameterCollection Uniform Buffers ALL are names "MaterialCollection"
		// (and the hlsl cbuffers are named MaterialCollection0, etc, so the names don't match the layout)
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(*CollectionName, ParameterCollection->GetUniformBufferStruct(), InPlatform, OutEnvironment);
	}

	OutEnvironment.SetDefine(TEXT("IS_MATERIAL_SHADER"), TEXT("1"));

	if (EnvironmentDefines->bSingleLayerWaterShadingQuality)
	{
		// Value must match SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE in SingleLayerWaterCommon.ush!
		OutEnvironment.SetDefine(TEXT("SINGLE_LAYER_WATER_SHADING_QUALITY"), TEXT("1"));
	}

	if (EnvironmentDefines->bShadingModelsIsLit)
	{
		if (EnvironmentDefines->HasShadingModel(MSM_DefaultLit))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_DEFAULT_LIT"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_Subsurface) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SSS))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_SUBSURFACE"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_PreintegratedSkin) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SSS))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_SubsurfaceProfile) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SSS))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE"), TEXT("1"));
		}

		if (EnvironmentDefines->bMaterialSubsurfaceProfileUseCurvature)
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SUBSURFACE_PROFILE_USE_CURVATURE"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_ClearCoat) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_CLEAR_COAT"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_TwoSidedFoliage) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SSS))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_Hair) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Hair))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_HAIR"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_Cloth) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Fuzz))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_CLOTH"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_Eye) || EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Eye))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_EYE"), TEXT("1"));
		}

		if (EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Toon))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_TOON"), TEXT("1"));
		}

		if (EnvironmentDefines->bMaterialShadingModelEyeUseCurvature)
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_EYE_USE_CURVATURE"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_SingleLayerWater))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_SINGLELAYERWATER"), TEXT("1"));
		}

		if (EnvironmentDefines->HasShadingModel(MSM_ThinTranslucent))
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT"), TEXT("1"));
			bMaterialRequestsDualSourceBlending = true;
		}

		if (EnvironmentDefines->bDisableForwardLocalLights)
		{
			OutEnvironment.SetDefine(TEXT("DISABLE_FORWARD_LOCAL_LIGHTS"), TEXT("1"));
		}

		if (EnvironmentDefines->bSingleLayerWaterSeparatedMainLight)
		{
			OutEnvironment.SetDefine(TEXT("SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT"), TEXT("1"));
		}

		if (EnvironmentDefines->bMaterialSingleShadingModel)
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SINGLE_SHADINGMODEL"), TEXT("1"));
		}
	}
	else
	{
		OutEnvironment.SetDefine(TEXT("MATERIAL_SINGLE_SHADINGMODEL"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_UNLIT"), TEXT("1"));
	}

	OutEnvironment.SetDefine(TEXT("MATERIAL_LWC_ENABLED"), GLWCEnabled ? TEXT("1") : TEXT("0"));
	OutEnvironment.SetDefine(TEXT("WSVECTOR_IS_TILEOFFSET"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("WSVECTOR_IS_DOUBLEFLOAT"), TEXT("0"));

	if (EnvironmentDefines->bMaterialVolumetricAdvanced)
	{
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED"), TEXT("1"));

		if (EnvironmentDefines->bMaterialVolumetricAdvancedPhasePerSample)
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERSAMPLE"), TEXT("1"));
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERPIXEL"), TEXT("1"));
		}

		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_GRAYSCALE_MATERIAL"), EnvironmentDefines->bMaterialVolumetricAdvancedGreyscaleMaterial);
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_RAYMARCH_VOLUME_SHADOW"), EnvironmentDefines->bMaterialVolumetricAdvancedRaymarchVolumeShadow);
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_CLAMP_MULTISCATTERING_CONTRIBUTION"), EnvironmentDefines->bMaterialVolumetricAdvancedClampMultiscatteringContribution);
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT"), EnvironmentDefines->MaterialVolumetricAdvancedMultiscatteringOctaveCount);
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY"), EnvironmentDefines->bMaterialVolumetricAdvancedConservativeDensity);
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_OVERRIDE_AMBIENT_OCCLUSION"), EnvironmentDefines->bMaterialVolumetricAdvancedOverrideAmbientOcclusion);
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_GROUND_CONTRIBUTION"), EnvironmentDefines->bMaterialVolumetricAdvancedAdvancedGroundContribution);
	}

	if (EnvironmentDefines->bMaterialVolumetricCloudEmptySpaceSkippingOutput)
	{
		OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT"), TEXT("1"));
	}

	OutEnvironment.SetDefine(TEXT("DUAL_SOURCE_COLOR_BLENDING_ENABLED"), EnvironmentDefines->bDualSourceColorBlendingEnabled);

	if (SubstrateTranslatorData.bMaterialIsSubstrate)
	{
		if (!Substrate::IsSubstrateBlendableGBufferEnabled(InPlatform))
		{
			static const TCHAR* SDCESymbols[] = {
				TEXT("FMaterialPixelParameters"),
				TEXT("FMaterialLWCData"),
				TEXT("FPrimitiveSceneData"),
				TEXT("FPixelMaterialInputs"),
				TEXT("FCluster"),
				TEXT("FNaniteView"),
				TEXT("FInstanceDynamicData"),
				TEXT("FNanitePostDeformVertex"),
				TEXT("FMaterialVertexParameters"),
				TEXT("FNanitePixelAttributes"),
				TEXT("ViewState"),
				TEXT("FSceneData"),
				TEXT("FPrimitiveSceneData")
			};
		
			OutEnvironment.SetCompileArgument(TEXT("UESHADERMETADATA_SDCE"), FString::Join(SDCESymbols, TEXT(";")));
		}
	}

	// Append all substrate defines
	for (int i = 0; i < EnvironmentDefines->SubstrateDefines.Num(); ++i)
	{
		OutEnvironment.SetDefine(*EnvironmentDefines->SubstrateDefines[i].Get<0>(), EnvironmentDefines->SubstrateDefines[i].Get<1>());
	}

	OutEnvironment.SetDefine(TEXT("TEXTURE_SAMPLE_DEBUG"), EnvironmentDefines->bTextureSampleDebug);

	for (const TRefCountPtr<FMaterialUniformExpressionTextureCollection>& TextureCollection : UniformTextureCollectionExpressions)
	{
		if (TextureCollection->IsVirtualCollection())
		{
			OutEnvironment.SetDefine(TEXT("TEXTURE_COLLECTION_PACKED_UNIFORMS"), 1u);
			break;
		}
	}
}

// Assign custom interpolators to slots, packing them as much as possible in unused slots.
TBitArray<> FHLSLMaterialTranslator::GetVertexInterpolatorsOffsets(FString& VertexInterpolatorsOffsetsDefinitionCode, TArray<uint32>& UsedTextureCoordsIndices) const
{
	TBitArray<> AllocatedCoords = AllocatedUserTexCoords; // Don't mess with the already assigned sets of UV coords

	int32 CurrentSlot = INDEX_NONE;
	int32 EndAllocatedSlot = INDEX_NONE;

	auto GetNextUVSlot = [&CurrentSlot, &EndAllocatedSlot, &AllocatedCoords, &UsedTextureCoordsIndices]() -> int32
	{
		if (CurrentSlot == EndAllocatedSlot)
		{
			CurrentSlot = AllocatedCoords.FindAndSetFirstZeroBit();
			if (CurrentSlot == INDEX_NONE)
			{
				CurrentSlot = AllocatedCoords.Add(true);
			}

			// Track one slot per component (u,v)
			const int32 NUM_COMPONENTS = 2;
			CurrentSlot *= NUM_COMPONENTS;
			EndAllocatedSlot = CurrentSlot + NUM_COMPONENTS;
		}

		int32 ResultUVSlot = CurrentSlot / 2;
		CurrentSlot++;

		UsedTextureCoordsIndices.AddUnique(ResultUVSlot);

		return ResultUVSlot;
	};

	TArray<UMaterialExpressionVertexInterpolator*> SortedInterpolators;
	Algo::TransformIf(CustomVertexInterpolators, 
						SortedInterpolators, 
						[](const UMaterialExpressionVertexInterpolator* Interpolator) { return Interpolator && Interpolator->InterpolatorIndex != INDEX_NONE && Interpolator->InterpolatorOffset != INDEX_NONE; },
						[](UMaterialExpressionVertexInterpolator* Interpolator) { return Interpolator; });
						
	SortedInterpolators.Sort([](const UMaterialExpressionVertexInterpolator& LHS, const UMaterialExpressionVertexInterpolator& RHS)  { return LHS.InterpolatorOffset < RHS.InterpolatorOffset; });
		
	for (UMaterialExpressionVertexInterpolator* Interpolator : SortedInterpolators)
	{
		int32 Index = Interpolator->InterpolatorIndex;

		const EMaterialValueType Type = Interpolator->InterpolatedType == MCT_Float ? MCT_Float1 : Interpolator->InterpolatedType;

		VertexInterpolatorsOffsetsDefinitionCode += HLSL_LINE_TERMINATOR;
		VertexInterpolatorsOffsetsDefinitionCode += FString::Printf(TEXT("#define VERTEX_INTERPOLATOR_%i_TEXCOORDS_X\t%i") HLSL_LINE_TERMINATOR, Index, GetNextUVSlot());

		if (Type >= MCT_Float2)
		{
			VertexInterpolatorsOffsetsDefinitionCode += FString::Printf(TEXT("#define VERTEX_INTERPOLATOR_%i_TEXCOORDS_Y\t%i") HLSL_LINE_TERMINATOR, Index, GetNextUVSlot());

			if (Type >= MCT_Float3)
			{
				VertexInterpolatorsOffsetsDefinitionCode += FString::Printf(TEXT("#define VERTEX_INTERPOLATOR_%i_TEXCOORDS_Z\t%i") HLSL_LINE_TERMINATOR, Index, GetNextUVSlot());

				if (Type == MCT_Float4)
				{
					VertexInterpolatorsOffsetsDefinitionCode += FString::Printf(TEXT("#define VERTEX_INTERPOLATOR_%i_TEXCOORDS_W\t%i") HLSL_LINE_TERMINATOR, Index, GetNextUVSlot());
				}
			}
		}
			
		VertexInterpolatorsOffsetsDefinitionCode += HLSL_LINE_TERMINATOR;
	}

	return AllocatedCoords;
}

void FHLSLMaterialTranslator::GetSharedInputsMaterialCode(FString& PixelMembersDeclaration, FString& NormalAssignment, FString& PixelMembersInitializationEpilog, ECompiledPartialDerivativeVariation DerivativeVariation)
{
	int32 LastProperty = -1;
	int32 TranslatedCodeChunkDefinitionsLargestSize = 0;
	FString PixelInputInitializerValues;
	FString NormalInitializerValue;

	for (int32 PropertyIndex = 0; PropertyIndex < MP_MAX; ++PropertyIndex)
	{
		// Skip non-shared properties
		if (!SharedPixelProperties[PropertyIndex])
		{
			continue;
		}

		const EMaterialProperty Property = (EMaterialProperty)PropertyIndex;
		check(FMaterialAttributeDefinitionMap::GetShaderFrequency(Property) == SF_Pixel);
		// Special case MP_SubsurfaceColor as the actual property is a combination of the color and the profile but we don't want to expose the profile
		const FString PropertyName = Property == MP_SubsurfaceColor ? "Subsurface" : FMaterialAttributeDefinitionMap::GetAttributeName(Property);
		check(PropertyName.Len() > 0);				
		const EMaterialValueType Type = Property == MP_SubsurfaceColor ? MCT_Float4 : FMaterialAttributeDefinitionMap::GetValueType(Property);

		// Normal requires its own separate initializer
		if (Property == MP_Normal)
		{
			NormalInitializerValue = FString::Printf(TEXT("\tPixelMaterialInputs.%s = %s;\n"), *PropertyName, *DerivativeVariations[DerivativeVariation].TranslatedCodeChunks[Property]);
		}
		else
		{
			const FString& TranslatedCodeChunkDefinitions = DerivativeVariations[DerivativeVariation].TranslatedCodeChunkDefinitions[Property];
			if (TranslatedCodeChunkDefinitions.Len() >= TranslatedCodeChunkDefinitionsLargestSize)
			{
				TranslatedCodeChunkDefinitionsLargestSize = TranslatedCodeChunkDefinitions.Len();
				LastProperty = Property;
			}
		}

		PixelInputInitializerValues += FString::Printf(TEXT("\tPixelMaterialInputs.%s = %s;\n"), *PropertyName, *DerivativeVariations[DerivativeVariation].TranslatedCodeChunks[Property]);

		PixelMembersDeclaration += FString::Printf(TEXT("\t%s %s;\n"), HLSLTypeString(Type), *PropertyName);
	}

	NormalAssignment = NormalInitializerValue;
	if (LastProperty != -1)
	{
		PixelMembersInitializationEpilog += DerivativeVariations[DerivativeVariation].TranslatedCodeChunkDefinitions[LastProperty] + TEXT("\n");
	}

	// Generate some more substrate specific HLSL code.
	if (Substrate::IsSubstrateEnabled())
	{
		SubstrateTranslatorData.GeneratePixelMemberDeclarationHLSLCode(PixelMembersDeclaration, HLSLTypeString(FMaterialAttributeDefinitionMap::GetValueType(MP_FrontMaterial)));
		SubstrateTranslatorData.GeneratePixelInputInitializerHLSLCode(PixelInputInitializerValues); 
		SubstrateTranslatorData.GeneratePixelNormalInitializerHLSLCode(PixelInputInitializerValues);
	}

	PixelMembersInitializationEpilog += PixelInputInitializerValues;
}

FString FHLSLMaterialTranslator::GetMaterialShaderCode()
{
	// use "/Engine/Private/MaterialTemplate.ush" to create the functions to get data (e.g. material attributes) and code (e.g. material expressions to create specular color) from C++
	int32 MaterialTemplateLineNumberNew;
	FStringTemplateResolver Resolver = FMaterialSourceTemplate::Get().BeginResolve(GetShaderPlatform(), &MaterialTemplateLineNumberNew);

	Resolver.SetParameterMap(&MaterialSourceTemplateParams);
	MaterialSourceTemplateParams.Add({ TEXT("line_number"), FString::Printf(TEXT("%u"), MaterialTemplateLineNumberNew) });

	// Copy generate code chunks to the insights object.
	Material->GetMaterialInterface()->MaterialInsight->Legacy_ShaderStringParameters = MaterialSourceTemplateParams;

	return Resolver.Finalize();
}

// ========== PROTECTED: ========== //

bool FHLSLMaterialTranslator::IsMaterialPropertyUsed(EMaterialProperty Property, int32 PropertyChunkIndex, const FLinearColor& ReferenceValue, int32 NumComponents) const
{
	const int32 Frequency = (int32)FMaterialAttributeDefinitionMap::GetShaderFrequency(Property);
	bool bPropertyUsed = false;

	if ((MaterialAttributesReturned[Frequency] & (1ull << Property)) != 0u)
	{
		// Property was set via a 'Return Material Attributes' expression
		bPropertyUsed = true;
	}
	else if (PropertyChunkIndex == -1)
	{
		bPropertyUsed = false;
	}
	else
	{
		const FShaderCodeChunk& PropertyChunk = SharedPropertyCodeChunks[Frequency][PropertyChunkIndex];

		// Determine whether the property is used. 
		// If the output chunk has a uniform expression, it is constant, and GetNumberValue returns the default property value then property isn't used.
		bPropertyUsed = true;

		if( PropertyChunk.UniformExpression && PropertyChunk.UniformExpression->IsConstant() )
		{
			FLinearColor Value;
			FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
			PropertyChunk.UniformExpression->GetNumberValue(DummyContext, Value);

			if ((NumComponents < 1 || Value.R == ReferenceValue.R)
				&& (NumComponents < 2 || Value.G == ReferenceValue.G)
				&& (NumComponents < 3 || Value.B == ReferenceValue.B)
				&& (NumComponents < 4 || Value.A == ReferenceValue.A))
			{
				bPropertyUsed = false;
			}
		}
	}

	return bPropertyUsed;
}

// only used by GetMaterialShaderCode()
// @param Index ECompiledMaterialProperty or EMaterialProperty
FString FHLSLMaterialTranslator::GenerateFunctionCode(uint32 Index, ECompiledPartialDerivativeVariation Variation) const
{
	check(Index < CompiledMP_MAX);
	return DerivativeVariations[Variation].TranslatedCodeChunkDefinitions[Index] + TEXT("	return ") + DerivativeVariations[Variation].TranslatedCodeChunks[Index] + TEXT(";");
}

const FShaderCodeChunk& FHLSLMaterialTranslator::AtParameterCodeChunk(int32 Index) const
{
	check(Index != INDEX_NONE);
	checkf(Index >= 0 && Index < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Index, CurrentScopeChunks->Num(), (int)Platform);
	const FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Index];
	return CodeChunk;
}

EDerivativeStatus FHLSLMaterialTranslator::GetDerivativeStatus(int32 Index) const
{
	return AtParameterCodeChunk(Index).DerivativeStatus;
}

FDerivInfo FHLSLMaterialTranslator::GetDerivInfo(int32 Index, bool bAllowNonFloat)
{
	const FShaderCodeChunk& CodeChunk = AtParameterCodeChunk(Index);
	if (!bAllowNonFloat && GetDerivType(CodeChunk.Type, true) == EDerivativeType::None)
	{
		Errorf(TEXT("Internal error: unexpected non-numeric derivative argument type found (%s)."), DescribeType(CodeChunk.Type));
		return FDerivInfo { CodeChunk.Type, EDerivativeType::None, EDerivativeStatus::NotValid };
	}
	return FDerivInfo { CodeChunk.Type, GetDerivType(CodeChunk.Type, bAllowNonFloat), CodeChunk.DerivativeStatus };
}

// Similar to GetParameterCode, but has no default, and is derivative aware. Making it a separate function in case it needs to diverge,
// but after looking at it, it has the same logic as GetParameterCode() for now.
FString FHLSLMaterialTranslator::GetParameterCodeDeriv(int32 Index, ECompiledPartialDerivativeVariation Variation)
{
	// In the case of a uniform expression, both finite and deriv versions are the same (raw floats, known zero)
	checkf(Index >= 0 && Index < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Index, CurrentScopeChunks->Num(), (int)Platform);
	const FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Index];

	FString Result;
	if((CodeChunk.UniformExpression && CodeChunk.UniformExpression->IsConstant()) || CodeChunk.bInline)
	{
		// Constant uniform expressions and code chunks which are marked to be inlined are accessed via Definition
		Result = CodeChunk.AtDefinition(Variation);
	}
	else
	{
		if (CodeChunk.UniformExpression && !CodeChunk.bIntermediate)
		{
			// If the code chunk has a uniform expression, create a new code chunk to access it
			const int32 AccessedIndex = AccessUniformExpression(Index);
			const FShaderCodeChunk& AccessedCodeChunk = (*CurrentScopeChunks)[AccessedIndex];
			if(AccessedCodeChunk.bInline)
			{
				// Handle the accessed code chunk being inlined
				Result = AccessedCodeChunk.AtDefinition(Variation);
			}
			else
			{
				// Return the symbol used to reference this code chunk
				check(AccessedCodeChunk.SymbolName.Len() > 0);
				Result = AccessedCodeChunk.SymbolName;
			}
		}
		else
		{
			// Return the symbol used to reference this code chunk
			check(CodeChunk.SymbolName.Len() > 0);
			Result = CodeChunk.SymbolName;
		}
	}

	return Result;
}

FString FHLSLMaterialTranslator::GetParameterCode(int32 Index, const TCHAR* Default)
{
	FString Ret = GetParameterCodeRaw(Index,Default);

	if (GetDerivativeStatus(Index) == EDerivativeStatus::Valid)
	{
		Ret = TEXT("DERIV_BASE_VALUE(") + Ret + TEXT(")");
	}

	return Ret;
}

// GetParameterCode
FString FHLSLMaterialTranslator::GetParameterCodeRaw(int32 Index, const TCHAR* Default)
{
	if(Index == INDEX_NONE)
	{
		checkf(Default != nullptr, TEXT("Invalid material expression and no default value provided"));
		return Default;
	}

	checkf(Index >= 0 && Index < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Index, CurrentScopeChunks->Num(), (int)Platform);
	const FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Index];
	if((CodeChunk.UniformExpression && CodeChunk.UniformExpression->IsConstant()) || CodeChunk.bInline)
	{
		// Constant uniform expressions and code chunks which are marked to be inlined are accessed via Definition
		return CodeChunk.DefinitionFinite;
	}
	else
	{
		if (CodeChunk.UniformExpression && !CodeChunk.bIntermediate)
		{
			// If the code chunk has a uniform expression, create a new code chunk to access it
			const int32 AccessedIndex = AccessUniformExpression(Index);
			const FShaderCodeChunk& AccessedCodeChunk = (*CurrentScopeChunks)[AccessedIndex];
			if(AccessedCodeChunk.bInline)
			{
				// Handle the accessed code chunk being inlined
				return AccessedCodeChunk.DefinitionFinite;
			}
			// Return the symbol used to reference this code chunk
			check(AccessedCodeChunk.SymbolName.Len() > 0);
			return AccessedCodeChunk.SymbolName;
		}
			
		ReferencedCodeChunks.AddUnique(Index);
			
		// Return the symbol used to reference this code chunk
		check(CodeChunk.SymbolName.Len() > 0);
		return CodeChunk.SymbolName;
	}
}

uint64 FHLSLMaterialTranslator::GetParameterHash(int32 Index)
{
	if (Index == INDEX_NONE)
	{
		return 0u;
	}

	checkf(Index >= 0 && Index < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Index, CurrentScopeChunks->Num(), (int)Platform);
	const FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Index];

	if (CodeChunk.UniformExpression && !CodeChunk.UniformExpression->IsConstant())
	{
		// Non-constant uniform expressions are accessed through a separate code chunk...need to give the hash of that
		const int32 AccessedIndex = AccessUniformExpression(Index);
		const FShaderCodeChunk& AccessedCodeChunk = (*CurrentScopeChunks)[AccessedIndex];
		return AccessedCodeChunk.Hash;
	}

	return CodeChunk.Hash;
}

uint64 FHLSLMaterialTranslator::GetParameterMaterialAttributeMask(int32 Index)
{
	if (Index == INDEX_NONE)
	{
		return 0u;
	}

	checkf(Index >= 0 && Index < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Index, CurrentScopeChunks->Num(), (int)Platform);
	const FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Index];
	check(CodeChunk.Type == MCT_MaterialAttributes);
	check(!CodeChunk.UniformExpression);

	return CodeChunk.MaterialAttributeMask;
}

void FHLSLMaterialTranslator::SetParameterMaterialAttributes(int32 Index, uint64 Mask)
{
	checkf(Index >= 0 && Index < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Index, CurrentScopeChunks->Num(), (int)Platform);
	FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Index];
	check(CodeChunk.Type == MCT_MaterialAttributes);
	CodeChunk.MaterialAttributeMask |= Mask;
}


/** Creates a string of all definitions needed for the given material input. */
FString FHLSLMaterialTranslator::GetDefinitions(const TArray<FShaderCodeChunk>& CodeChunks, int32 StartChunk, int32 EndChunk, ECompiledPartialDerivativeVariation Variation, const TCHAR* ReturnValueSymbolName) const
{
	FString Definitions;
	for (int32 ChunkIndex = StartChunk; ChunkIndex < EndChunk; ChunkIndex++)
	{
		const FShaderCodeChunk& CodeChunk = CodeChunks[ChunkIndex];
		// Uniform expressions (both constant and variable) and inline expressions don't have definitions.
		if ((!CodeChunk.UniformExpression || CodeChunk.bIntermediate) &&
			(!CodeChunk.bInline || CodeChunk.Type == MCT_VoidStatement))
		{
			Definitions += CodeChunk.AtDefinition(Variation);

			// If we found the definition of the return value, there is no need to add more definitions as they won't contribute to the outcome
			if (ReturnValueSymbolName != nullptr && CodeChunk.SymbolName == ReturnValueSymbolName)
			{
				break;
			}
		}
	}
	return Definitions;
}

// GetFixedParameterCode
void FHLSLMaterialTranslator::GetFixedParameterCode(int32 StartChunk, int32 EndChunk, int32 ResultIndex, TArray<FShaderCodeChunk>& CodeChunks, FString& OutDefinitions, FString& OutValue, ECompiledPartialDerivativeVariation OriginalVariation, bool bReduceAfterReturnValue)
{
	CHECK_DDC_QUERY_FINISHED_ELSE_RETURN();;

	// Only allow the analytic variation for pixel shaders.
	ECompiledPartialDerivativeVariation Variation = OriginalVariation;

	// This function is hardcoded to finite differences for now.
	if (ResultIndex != INDEX_NONE)
	{
		checkf(ResultIndex >= 0 && ResultIndex < CodeChunks.Num(), TEXT("Index out of range %d/%d [%s]"), ResultIndex, CodeChunks.Num(), *Material->GetFriendlyName());
		check(!CodeChunks[ResultIndex].UniformExpression || CodeChunks[ResultIndex].UniformExpression->IsConstant());
		if (CodeChunks[ResultIndex].UniformExpression && CodeChunks[ResultIndex].UniformExpression->IsConstant())
		{
			// Handle a constant uniform expression being the only code chunk hooked up to a material input
			const FShaderCodeChunk& ResultChunk = CodeChunks[ResultIndex];
			OutValue = ResultChunk.AtDefinition(Variation);
		}
		else
		{
			const FShaderCodeChunk& ResultChunk = CodeChunks[ResultIndex];
			// Combine the definition lines and the return statement.
			// Also specify the return symbol name to terminate the iteration over all definitions earlier,
			// if there are unnecessary statements that don't contribute to the outcome.
			check(ResultChunk.bInline || ResultChunk.SymbolName.Len() > 0);
			const TCHAR* ReturnValueSymbolName = (bReduceAfterReturnValue && !ResultChunk.bInline ? *ResultChunk.SymbolName : nullptr);
			OutDefinitions = GetDefinitions(CodeChunks, StartChunk, EndChunk, Variation, ReturnValueSymbolName);
			OutValue = ResultChunk.bInline ? ResultChunk.AtDefinition(Variation) : ResultChunk.SymbolName;
			if (Variation == CompiledPDV_Analytic && ResultChunk.DerivativeStatus == EDerivativeStatus::Valid)
			{
				OutValue = FString("(") + OutValue + TEXT(").Value");
			}
		}
	}
	else
	{
		OutValue = TEXT("0");
	}
}

void FHLSLMaterialTranslator::GetFixedParameterCode(int32 ResultIndex, TArray<FShaderCodeChunk>& CodeChunks, FString& OutDefinitions, FString& OutValue, ECompiledPartialDerivativeVariation Variation)
{
	GetFixedParameterCode(0, CodeChunks.Num(), ResultIndex, CodeChunks, OutDefinitions, OutValue, Variation);
}

static void AppendIndent(int32 Level, FString& OutValue)
{
	for (int32 i = 0; i < Level; ++i)
	{
		OutValue += '\t';
	}
}

void FHLSLMaterialTranslator::LinkParentScopes(TArray<FShaderCodeChunk>& CodeChunks)
{
	// Add all chunks to parent's child array
	for (int32 ChunkIndex = 0; ChunkIndex < CodeChunks.Num(); ++ChunkIndex)
	{
		FShaderCodeChunk& ChildChunk = CodeChunks[ChunkIndex];
		if (ChildChunk.UsedScopeIndex != INDEX_NONE)
		{
			FShaderCodeChunk& ParentChunk = CodeChunks[ChildChunk.UsedScopeIndex];
			ParentChunk.ScopedChunks.Add(ChunkIndex);
		}
	}
}

void FHLSLMaterialTranslator::GetScopeCode(int32 IndentLevel, int32 ScopeChunkIndex, const TArray<FShaderCodeChunk>& CodeChunks, TSet<int32>& EmittedChunks, FString& OutValue)
{
	bool bAlreadyAddedChunk = false;
	EmittedChunks.Add(ScopeChunkIndex, &bAlreadyAddedChunk);
	if (bAlreadyAddedChunk)
	{
		// Only emit code for the chunk the first time we visit it
		return;
	}

	const FShaderCodeChunk& CodeChunk = CodeChunks[ScopeChunkIndex];

	// Add code for any dependencies of the current chunk
	for (int32 ReferencedChunkIndex : CodeChunk.ReferencedCodeChunks)
{
		GetScopeCode(IndentLevel, ReferencedChunkIndex, CodeChunks, EmittedChunks, OutValue);
	}

	if ((!CodeChunk.UniformExpression && !CodeChunk.bInline) || CodeChunk.Type == MCT_VoidStatement)
	{
		AppendIndent(IndentLevel, OutValue);
		OutValue += CodeChunk.AtDefinition(CompiledPDV_FiniteDifferences); // TODO - deriv
	}

	if (CodeChunk.ScopedChunks.Num() > 0)
	{
		AppendIndent(IndentLevel, OutValue);
		OutValue += TEXT("{\n");
		for (int32 ChildChunkIndex : CodeChunk.ScopedChunks)
		{
			const FShaderCodeChunk& ChildChunk = CodeChunks[ChildChunkIndex];
			check(ChildChunk.UsedScopeIndex == ScopeChunkIndex);
			if (ChildChunk.DeclaredScopeIndex != ScopeChunkIndex)
			{
				GetScopeCode(IndentLevel + 1, ChildChunkIndex, CodeChunks, EmittedChunks, OutValue);
			}
		}
		for (int32 ChildChunkIndex : CodeChunk.ScopedChunks)
		{
			const FShaderCodeChunk& ChildChunk = CodeChunks[ChildChunkIndex];
			if (ChildChunk.DeclaredScopeIndex == ScopeChunkIndex)
			{
				GetScopeCode(IndentLevel + 1, ChildChunkIndex, CodeChunks, EmittedChunks, OutValue);
			}
		}
		AppendIndent(IndentLevel, OutValue);
		OutValue += TEXT("}\n");
	}
}

/** Used to get a user friendly type from EMaterialValueType */
const TCHAR* FHLSLMaterialTranslator::DescribeType(EMaterialValueType Type) const
{
	switch(Type)
	{
	case MCT_Float1:				return TEXT("float");
	case MCT_Float2:				return TEXT("float2");
	case MCT_Float3:				return TEXT("float3");
	case MCT_Float4:				return TEXT("float4");
	case MCT_Float:					return TEXT("float");
	case MCT_Texture2D:				return TEXT("texture2D");
	case MCT_TextureCube:			return TEXT("textureCube");
	case MCT_Texture2DArray:		return TEXT("texture2DArray");
	case MCT_TextureCubeArray:		return TEXT("textureCubeArray");
	case MCT_VolumeTexture:			return TEXT("volumeTexture");
	case MCT_StaticBool:			return TEXT("static bool");
	case MCT_Bool:					return TEXT("bool");
	case MCT_MaterialAttributes:	return TEXT("MaterialAttributes");
	case MCT_TextureExternal:		return TEXT("TextureExternal");
	case MCT_TextureVirtual:		return TEXT("TextureVirtual");
	case MCT_SparseVolumeTexture:	return TEXT("SparseVolumeTexture");
	case MCT_VTPageTableResult:		return TEXT("VTPageTableResult");
	case MCT_ShadingModel:			return TEXT("ShadingModel");
	case MCT_UInt:					return TEXT("uint");
	case MCT_UInt1:					return TEXT("uint");
	case MCT_UInt2:					return TEXT("uint2");
	case MCT_UInt3:					return TEXT("uint3");
	case MCT_UInt4:					return TEXT("uint4");
	case MCT_Substrate:				return TEXT("Substrate");
	case MCT_LWCScalar:				return TEXT("LWCScalar");
	case MCT_LWCVector2:			return TEXT("LWCVector2");
	case MCT_LWCVector3:			return TEXT("LWCVector3");
	case MCT_LWCVector4:			return TEXT("LWCVector4");
	case MCT_TextureCollection:		return TEXT("TextureCollection");
	case MCT_TextureMeshPaint:		return TEXT("TextureMeshPaint");
	case MCT_TextureMaterialCache:  return TEXT("TextureMaterialCache");
	case MCT_MaterialCacheABuffer:	return TEXT("MaterialCacheABufferUntyped");
	default:						return TEXT("unknown");
	};
}

/** Used to get an HLSL type from EMaterialValueType */
const TCHAR* FHLSLMaterialTranslator::HLSLTypeString(EMaterialValueType Type) const
{
	switch(Type)
	{
	case MCT_Float1:				return TEXT("MaterialFloat");
	case MCT_Float2:				return TEXT("MaterialFloat2");
	case MCT_Float3:				return TEXT("MaterialFloat3");
	case MCT_Float4:				return TEXT("MaterialFloat4");
	case MCT_Float:					return TEXT("MaterialFloat");
	case MCT_Texture2D:				return TEXT("Texture2D");
	case MCT_TextureCube:			return TEXT("TextureCube");
	case MCT_Texture2DArray:		return TEXT("Texture2DArray");
	case MCT_TextureCubeArray:		return TEXT("TextureCubeArray");
	case MCT_VolumeTexture:			return TEXT("Texture3D");
	case MCT_StaticBool:			return TEXT("static bool");
	case MCT_Bool:					return TEXT("bool");
	case MCT_MaterialAttributes:	return TEXT("FMaterialAttributes");
	case MCT_TextureExternal:		return TEXT("TextureExternal");
	case MCT_TextureVirtual:		return TEXT("TextureVirtual");
	case MCT_SparseVolumeTexture:	return TEXT("FSparseVolumeTexture");
	case MCT_VTPageTableResult:		return TEXT("VTPageTableResult");
	case MCT_ShadingModel:			return TEXT("uint");
	case MCT_UInt:					return TEXT("uint");
	case MCT_UInt1:					return TEXT("uint");
	case MCT_UInt2:					return TEXT("uint2");
	case MCT_UInt3:					return TEXT("uint3");
	case MCT_UInt4:					return TEXT("uint4");
	case MCT_Substrate:				return TEXT("FSubstrateData");
	case MCT_LWCScalar:				return TEXT("FWSScalar");
	case MCT_LWCVector2:			return TEXT("FWSVector2");
	case MCT_LWCVector3:			return TEXT("FWSVector3");
	case MCT_LWCVector4:			return TEXT("FWSVector4");
	case MCT_TextureCollection:		return TEXT("FResourceCollection");
	case MCT_TextureMeshPaint:		return TEXT("TextureMeshPaint");
	case MCT_TextureMaterialCache:	return TEXT("TextureMaterialCache");
	case MCT_MaterialCacheABuffer:	return TEXT("MaterialCacheABufferUntyped");
	default:						return TEXT("unknown");
	};
}

/** Used to get an HLSL type from EMaterialValueType */
const TCHAR* FHLSLMaterialTranslator::HLSLTypeStringDeriv(EMaterialValueType Type, EDerivativeStatus DerivativeStatus) const
{
	switch(Type)
	{
	case MCT_Float1:				return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FloatDeriv") : TEXT("MaterialFloat");
	case MCT_Float2:				return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FloatDeriv2") : TEXT("MaterialFloat2");
	case MCT_Float3:				return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FloatDeriv3") : TEXT("MaterialFloat3");
	case MCT_Float4:				return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FloatDeriv4") : TEXT("MaterialFloat4");
	case MCT_Float:					return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FloatDeriv") : TEXT("MaterialFloat");
	case MCT_Texture2D:				return TEXT("texture2D");
	case MCT_TextureCube:			return TEXT("textureCube");
	case MCT_Texture2DArray:		return TEXT("texture2DArray");
	case MCT_TextureCubeArray:		return TEXT("textureCubeArray");
	case MCT_VolumeTexture:			return TEXT("volumeTexture");
	case MCT_StaticBool:			return TEXT("static bool");
	case MCT_Bool:					return TEXT("bool");
	case MCT_MaterialAttributes:	return TEXT("FMaterialAttributes");
	case MCT_TextureExternal:		return TEXT("TextureExternal");
	case MCT_TextureVirtual:		return TEXT("TextureVirtual");
	case MCT_SparseVolumeTexture:	return TEXT("FSparseVolumeTextureUniforms");
	case MCT_VTPageTableResult:		return TEXT("VTPageTableResult");
	case MCT_ShadingModel:			return TEXT("uint");
	case MCT_UInt:					return TEXT("uint");
	case MCT_UInt1:					return TEXT("uint");
	case MCT_UInt2:					return TEXT("uint2");
	case MCT_UInt3:					return TEXT("uint3");
	case MCT_UInt4:					return TEXT("uint4");
	case MCT_Substrate:				return TEXT("FSubstrateData");
	case MCT_LWCScalar:				return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FWSScalarDeriv") : TEXT("FWSScalar");
	case MCT_LWCVector2:			return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FWSVector2Deriv") : TEXT("FWSVector2");
	case MCT_LWCVector3:			return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FWSVector3Deriv") : TEXT("FWSVector3");
	case MCT_LWCVector4:			return (DerivativeStatus == EDerivativeStatus::Valid) ? TEXT("FWSVector4Deriv") : TEXT("FWSVector4");
	case MCT_TextureCollection:		return TEXT("FResourceCollection");
	case MCT_TextureMeshPaint:		return TEXT("TextureMeshPaint");
	case MCT_TextureMaterialCache:	return TEXT("TextureMaterialCache");
	case MCT_MaterialCacheABuffer:	return TEXT("MaterialCacheABufferUntyped");
	default:						return TEXT("unknown");
	};
}

int32 FHLSLMaterialTranslator::NonPixelShaderExpressionError()
{
	return Errorf(TEXT("Invalid node used in vertex/hull/domain shader input!"));
}

int32 FHLSLMaterialTranslator::ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::Type RequiredFeatureLevel)
{
	if (FeatureLevel < RequiredFeatureLevel)
	{
		FString FeatureLevelName, RequiredLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);
		GetFeatureLevelName(RequiredFeatureLevel, RequiredLevelName);
		return Errorf(TEXT("Node not supported in feature level %s. %s required."), *FeatureLevelName, *RequiredLevelName);
	}

	return 0;
}

int32 FHLSLMaterialTranslator::ErrorUnlessPlatformSupports(const bool (*SupportFunction)(const FStaticShaderPlatform Platform), const TCHAR* ConditionString)
{
	if (!SupportFunction(Platform))
	{
		FString ShaderPlatformName = FDataDrivenShaderPlatformInfo::GetName(Platform).ToString();
		return Errorf(TEXT("Node not supported in shader platform %s. The node requires %s support."), *ShaderPlatformName, ConditionString);
	}

	return 0;
}

int32 FHLSLMaterialTranslator::NonVertexShaderExpressionError()
{
	return Errorf(TEXT("Invalid node used in pixel/hull/domain shader input!"));
}

int32 FHLSLMaterialTranslator::NonVertexOrPixelShaderExpressionError()
{
	return Errorf(TEXT("Invalid node used in hull/domain shader input!"));
}

void FHLSLMaterialTranslator::AddEstimatedTextureSample(const uint32 Count)
{
	if (IsCurrentlyCompilingForPreviousFrame())
	{
		// Ignore non-actionable cases
		return;
	}

	if (ShaderFrequency == SF_Pixel || ShaderFrequency == SF_Compute)
	{
		MaterialCompilationOutput.EstimatedNumTextureSamplesPS += Count;
	}
	else
	{
		MaterialCompilationOutput.EstimatedNumTextureSamplesVS += Count;
	}
}

void FHLSLMaterialTranslator::AddLWCFuncUsage(ELWCFunctionKind Kind, const uint32 Count)
{
	if (IsCurrentlyCompilingForPreviousFrame())
	{
		// Ignore non-actionable cases
		return;
	}
	if (ShaderFrequency == SF_Pixel)
	{
		MaterialCompilationOutput.EstimatedLWCFuncUsagesPS[(int)Kind] += Count;
	}
	else if (ShaderFrequency == SF_Compute)
	{
		MaterialCompilationOutput.EstimatedLWCFuncUsagesCS[(int)Kind] += Count;
	}
	else
	{
		MaterialCompilationOutput.EstimatedLWCFuncUsagesVS[(int)Kind] += Count;
	}
}

FString FHLSLMaterialTranslator::GetWorldPositionOrDefault(int32 WorldPosition, EPositionOrigin PositionOrigin)
{
	FString WorldPosCode;
	if (PositionOrigin == EPositionOrigin::Absolute)
	{
		WorldPosCode = WorldPosition == INDEX_NONE ? FString("GetWorldPosition(Parameters)") : CoerceParameter(WorldPosition, MCT_LWCVector3);
	}
	else if (PositionOrigin == EPositionOrigin::CameraRelative)
	{
		WorldPosCode = WorldPosition == INDEX_NONE ? FString("GetTranslatedWorldPosition(Parameters)") : CoerceParameter(WorldPosition, MCT_Float3);
	}
	else { check(0); }
	return WorldPosCode;
}

/** Creates a unique symbol name and adds it to the symbol list. */
FString FHLSLMaterialTranslator::CreateSymbolName(const TCHAR* SymbolNameHint)
{
	NextSymbolIndex++;
	return FString(SymbolNameHint) + FString::FromInt(NextSymbolIndex);
}

/** Adds an already formatted inline or referenced code chunk */
int32 FHLSLMaterialTranslator::AddCodeChunkInner(uint64 Hash, const TCHAR* FormattedType, const TCHAR* FormattedCode, EMaterialValueType Type, EDerivativeStatus DerivativeStatus, bool bInlined)
{
	check(bAllowCodeChunkGeneration);

	if (Type == MCT_Unknown)
	{
		return INDEX_NONE;
	}

	const bool bFromTextureCollection = (Type & MCT_TextureCollection) != 0;

	int32 CodeIndex = INDEX_NONE;
	if (Type == MCT_VoidStatement)
	{
		CodeIndex = CurrentScopeChunks->Num();
		const FString Statement = FString() + FormattedCode + HLSL_LINE_TERMINATOR;
		new(*CurrentScopeChunks) FShaderCodeChunk(Hash, *Statement, *Statement, TEXT(""), Type, DerivativeStatus, true);
	}
	else if (bInlined)
	{
		CodeIndex = CurrentScopeChunks->Num();
		// Adding an inline code chunk, the definition will be the code to inline
		new(*CurrentScopeChunks) FShaderCodeChunk(Hash, FormattedCode, FormattedCode, TEXT(""), Type, DerivativeStatus, true);
	}
	// Can only create temporaries for certain types
	else if ((Type & (MCT_Float | MCT_LWCType | MCT_VTPageTableResult | MCT_Unexposed | MCT_UInt)) || Type == MCT_ShadingModel || Type == MCT_MaterialAttributes || Type == MCT_Substrate || Type == MCT_UInt || bFromTextureCollection || Type == MCT_MaterialCacheABuffer)
	{
		// Check for existing
		for (int32 i = 0; i < CurrentScopeChunks->Num(); ++i)
		{
			if ((*CurrentScopeChunks)[i].Hash == Hash)
			{
				CodeIndex = i;
				break;
			}
		}

		if (CodeIndex == INDEX_NONE)
		{
			CodeIndex = CurrentScopeChunks->Num();
			// Allocate a local variable name
			const FString SymbolName = CreateSymbolName(TEXT("Local"));
			// Construct the definition string which stores the result in a temporary and adds a newline for readability
			const FString LocalVariableDefinitionFinite = FString("	") + FormattedType + TEXT(" ") + SymbolName + TEXT(" = ") + FormattedCode + TEXT(";") + HLSL_LINE_TERMINATOR;
			// Construct the definition string which stores the result in a temporary and adds a newline for readability
			const FString LocalVariableDefinitionAnalytic = FString("	") + FormattedType+ TEXT(" ") + SymbolName + TEXT(" = ") + FormattedCode + TEXT(";") + HLSL_LINE_TERMINATOR;
			// Adding a code chunk that creates a local variable
			new(*CurrentScopeChunks) FShaderCodeChunk(Hash, *LocalVariableDefinitionFinite, *LocalVariableDefinitionAnalytic, SymbolName, Type, DerivativeStatus, false);
		}
	}
	else
	{
		if (Type & MCT_Texture)
		{
			return Errorf(TEXT("Operation not supported on a Texture"));
		}
		else if (Type == MCT_StaticBool)
		{
			return Errorf(TEXT("Operation not supported on a Static Bool"));
		}
		else
		{
			return Errorf(TEXT("Operation not supported for type %s"), DescribeType(Type));
		}
	}
	
	AddCodeChunkToCurrentScope(CodeIndex);
	return CodeIndex;
}

int32 FHLSLMaterialTranslator::AddCodeChunkInner(uint64 Hash, const TCHAR* FormattedCode, EMaterialValueType Type, EDerivativeStatus DerivativeStatus, bool bInlined)
{
	EMaterialValueType EffectiveType = EMaterialValueType(Type & ~MCT_TextureCollection);
	return AddCodeChunkInner(Hash, HLSLTypeString(EffectiveType), FormattedCode, Type, DerivativeStatus, bInlined);
}
			
static inline uint32 GetTCharStringBytes(const TCHAR* String)
{
	uint32 Length = 0u;
	while (String[Length])
	{
		++Length;
	}
	return Length * sizeof(TCHAR);
}

int32 FHLSLMaterialTranslator::AddInternalCodeChunk(EMaterialValueType Type, const TCHAR* InternalType, const TCHAR* FormattedCode, const TRefCountPtr<FMaterialUniformExpression>& UniformExpression)
{
	const uint64 Hash = CityHash64(reinterpret_cast<const char*>(FormattedCode), GetTCharStringBytes(FormattedCode));
	
	int32 CodeIndex = INDEX_NONE;
	for (int32 i = 0; i < CurrentScopeChunks->Num(); ++i)
	{
		if ((*CurrentScopeChunks)[i].Hash == Hash)
		{
			CodeIndex = i;
			break;
		}
	}
	
	if (CodeIndex == INDEX_NONE)
	{
		CodeIndex = CurrentScopeChunks->Num();
		const FString SymbolName = CreateSymbolName(TEXT("Local"));
		const FString LocalVariableDefinitionFinite = FString("	") + InternalType + TEXT(" ") + SymbolName + TEXT(" = ") + FormattedCode + TEXT(";") + HLSL_LINE_TERMINATOR;
		const FString LocalVariableDefinitionAnalytic = FString("	") + InternalType + TEXT(" ") + SymbolName + TEXT(" = ") + FormattedCode + TEXT(";") + HLSL_LINE_TERMINATOR;

		FShaderCodeChunk* Chunk = new(*CurrentScopeChunks) FShaderCodeChunk(
			Hash, *LocalVariableDefinitionFinite, *LocalVariableDefinitionAnalytic,
			SymbolName, Type, EDerivativeStatus::NotAware, false
		);

		Chunk->bIntermediate     = true;
		Chunk->UniformExpression = UniformExpression;
	}
	
	AddCodeChunkToCurrentScope(CodeIndex);
	return CodeIndex;
}

/** Adds an already formatted inline or referenced code chunk, and notes the derivative status. */
int32 FHLSLMaterialTranslator::AddCodeChunkInnerDeriv(const TCHAR* FormattedCodeFinite, const TCHAR* FormattedCodeAnalytic, EMaterialValueType Type, bool bInlined, EDerivativeStatus DerivativeStatus)
{
	const bool bEmitInvalidDerivToken = GDebugEmitInvalidDerivTokensEnabled && !IsDerivativeValid(DerivativeStatus);

	const uint64 Hash = CityHash64WithSeed((const char*)FormattedCodeFinite, GetTCharStringBytes(FormattedCodeFinite),
		CityHash64((const char*)FormattedCodeAnalytic, GetTCharStringBytes(FormattedCodeAnalytic)));

	check(bAllowCodeChunkGeneration);

	if (Type == MCT_Unknown)
	{
		return INDEX_NONE;
	}

	check(Type != MCT_VoidStatement);

	int32 CodeIndex = INDEX_NONE;
	if (bInlined)
	{
		CodeIndex = CurrentScopeChunks->Num();
		// Adding an inline code chunk, the definition will be the code to inline
		new(*CurrentScopeChunks) FShaderCodeChunk(Hash, FormattedCodeFinite, FormattedCodeAnalytic, TEXT(""), Type, DerivativeStatus, true);
	}
	// Can only create temporaries for certain types
	else if ((Type & (MCT_Float | MCT_LWCType | MCT_VTPageTableResult | MCT_UInt)) || Type == MCT_ShadingModel || Type == MCT_MaterialAttributes || Type == MCT_Substrate)
	{
		// Check for existing
		for (int32 i = 0; i < CurrentScopeChunks->Num(); ++i)
		{
			if ((*CurrentScopeChunks)[i].Hash == Hash)
			{
				CodeIndex = i;
				break;
			}
		}

		if (CodeIndex == INDEX_NONE)
		{
			CodeIndex = CurrentScopeChunks->Num();
			// Allocate a local variable name
			const FString SymbolName = CreateSymbolName(TEXT("Local"));
			// Construct the definition string which stores the result in a temporary and adds a newline for readability
			const FString LocalVariableDefinitionFinite = FString("	") + HLSLTypeString(Type) + TEXT(" ") + SymbolName + TEXT(" = ") + FormattedCodeFinite + TEXT(";") + HLSL_LINE_TERMINATOR;
			// Analytic version too
			const FString LocalVariableDefinitionAnalytic = FString("	") + (bEmitInvalidDerivToken ? "$" : "") + HLSLTypeStringDeriv(Type, DerivativeStatus) + TEXT(" ") + SymbolName + TEXT(" = ") + FormattedCodeAnalytic + TEXT(";") + HLSL_LINE_TERMINATOR;
			// Adding a code chunk that creates a local variable
			new(*CurrentScopeChunks) FShaderCodeChunk(Hash, *LocalVariableDefinitionFinite, *LocalVariableDefinitionAnalytic, SymbolName, Type, DerivativeStatus, false);
		}
	}
	else
	{
		if (Type & MCT_Texture)
		{
			return Errorf(TEXT("Operation not supported on a Texture"));
		}

		if (Type == MCT_StaticBool)
		{
			return Errorf(TEXT("Operation not supported on a Static Bool"));
		}

	}

	AddCodeChunkToCurrentScope(CodeIndex);
	return CodeIndex;
}

int32 FHLSLMaterialTranslator::AddCodeChunkInnerDeriv(const TCHAR* FormattedCode, EMaterialValueType Type, bool bInlined, EDerivativeStatus DerivativeStatus)
{
	return AddCodeChunkInnerDeriv(FormattedCode, FormattedCode, Type, bInlined, DerivativeStatus);
}

/** 
	* Constructs the formatted code chunk and creates a new local variable definition from it. 
	* This should be used over AddInlinedCodeChunk when the code chunk adds actual instructions, and especially when calling a function.
	* Creating local variables instead of inlining simplifies the generated code and reduces redundant expression chains,
	* Making compiles faster and enabling the shader optimizer to do a better job.
	*/
int32 FHLSLMaterialTranslator::AddCodeChunkInner(EMaterialValueType Type, const TCHAR* FormattedType, EDerivativeStatus DerivativeStatus, bool bInlined, const TCHAR* Format, ...)
{
	TStringBuilder<256> Builder;

	va_list Args;
	va_start(Args, Format);
	const TCHAR* FormattedCode = Builder.AppendV(Format, Args).GetData();
	va_end(Args);

	const uint64 Hash = CityHash64((const char*)FormattedCode, Builder.Len() * sizeof(TCHAR));
	return AddCodeChunkInner(Hash, FormattedType, FormattedCode, Type, DerivativeStatus, bInlined);
}

int32 FHLSLMaterialTranslator::AddCodeChunkInner(EMaterialValueType Type, EDerivativeStatus DerivativeStatus, bool bInlined, const TCHAR* Format, ...)
{
	TStringBuilder<256> Builder;

	va_list Args;
	va_start(Args, Format);
	const TCHAR* FormattedCode = Builder.AppendV(Format, Args).GetData();
	va_end(Args);
	
	EMaterialValueType EffectiveType = EMaterialValueType(Type & ~MCT_TextureCollection);

	const uint64 Hash = CityHash64((const char*)FormattedCode, Builder.Len() * sizeof(TCHAR));
	return AddCodeChunkInner(Hash, HLSLTypeString(EffectiveType), FormattedCode, Type, DerivativeStatus, bInlined);
}

int32 FHLSLMaterialTranslator::AddCodeChunkWithHash(uint64 BaseHash, EMaterialValueType Type, const TCHAR* Format, ...)
{
	TStringBuilder<256> Builder;

	va_list Args;
	va_start(Args, Format);
	const TCHAR* FormattedCode = Builder.AppendV(Format, Args).GetData();
	va_end(Args);

	uint64 Hash = CityHash64((const char*)FormattedCode, Builder.Len() * sizeof(TCHAR));
	Hash = CityHash128to64({ BaseHash, Hash });
	return AddCodeChunkInner(Hash, FormattedCode, Type, EDerivativeStatus::NotAware, false);
}

int32 FHLSLMaterialTranslator::AddExternalCodeChunk(const FName& InExternalCodeIdentifier)
{
	if (const FMaterialExternalCodeDeclaration* ExternalCodeDeclaration = MaterialExternalCodeRegistry::Get().FindExternalCode(InExternalCodeIdentifier))
	{
		return AddExternalCodeChunk(*ExternalCodeDeclaration);
	}
	return Errorf(TEXT("External code identifier '%s' not found"), *InExternalCodeIdentifier.ToString());
}

int32 FHLSLMaterialTranslator::AddExternalCodeChunk(const FMaterialExternalCodeDeclaration& InExternalCode)
{
	// Validate this external code can be used in the current shader stage
	if (((int32)InExternalCode.ShaderFrequency & (1 << (int32)ShaderFrequency)) == 0)
	{
		FName ShaderFrequencyName = StaticEnum<EMaterialShaderFrequency>()->GetNameByValue(1ull << (int32)ShaderFrequency);
		return Errorf(TEXT("External code identifier '%s' is not available in the %s shader."), *InExternalCode.Name.ToString(), *ShaderFrequencyName.ToString());
	}

	// Validate this external code can be used for the current material domain. Empty list implies no restriction on material domains.
	if (!InExternalCode.Domains.IsEmpty())
	{
		checkf(Material != nullptr, TEXT("Missing material while emitting external HLSL code '%s'"), *InExternalCode.Name.ToString());
		if (InExternalCode.Domains.Find(Material->GetMaterialDomain()) == INDEX_NONE)
		{
			return Error(*FormatUnsupportedMaterialDomainError(InExternalCode, Material->GetAssetPath()));
		}
	}

	// Emit external code definition as new code chunk
	return AddCodeChunkInner(InExternalCode.GetReturnTypeValue(), InExternalCode.Derivative, InExternalCode.bIsInlined, TEXT("%s"), *InExternalCode.Definition);
}

int32 FHLSLMaterialTranslator::AddUniformExpressionInner(uint64 Hash, FMaterialUniformExpression* UniformExpression, EMaterialValueType Type, const TCHAR* FormattedCode)
{
	check(bAllowCodeChunkGeneration);

	if (Type == MCT_Unknown)
	{
		return INDEX_NONE;
	}

	check(UniformExpression);

	// Only a texture uniform expression can have MCT_Texture type
	if ((Type & MCT_Texture) && !UniformExpression->GetTextureUniformExpression()
		&& !UniformExpression->GetExternalTextureUniformExpression())
	{
		return Errorf(TEXT("Operation not supported on a Texture"));
	}

	// External textures must have an external texture uniform expression
	if ((Type & MCT_TextureExternal) && !UniformExpression->GetExternalTextureUniformExpression())
	{
		return Errorf(TEXT("Operation not supported on an external texture"));
	}

	if (Type == MCT_StaticBool)
	{
		return Errorf(TEXT("Operation not supported on a Static Bool"));
	}

	if (Type == MCT_MaterialAttributes)
	{
		return Errorf(TEXT("Operation not supported on a MaterialAttributes"));
	}

	bool bFoundExistingExpression = false;
	// Search for an existing code chunk with the same uniform expression in the array of all uniform expressions used by this material.
	for (int32 ExpressionIndex = 0; ExpressionIndex < UniformExpressions.Num() && !bFoundExistingExpression; ExpressionIndex++)
	{
		FMaterialUniformExpression* TestExpression = UniformExpressions[ExpressionIndex].UniformExpression;
		check(TestExpression);
		if (TestExpression->IsIdentical(UniformExpression))
		{
			bFoundExistingExpression = true;
			// This code chunk has an identical uniform expression to the new expression, reuse it.
			// This allows multiple material properties to share uniform expressions because AccessUniformExpression uses AddUniqueItem when adding uniform expressions.
			check(Type == UniformExpressions[ExpressionIndex].Type);
			// Search for an existing code chunk with the same uniform expression in the array of code chunks for this material property.
			for (int32 ChunkIndex = 0; ChunkIndex < CurrentScopeChunks->Num(); ChunkIndex++)
			{
				FMaterialUniformExpression* OtherExpression = (*CurrentScopeChunks)[ChunkIndex].UniformExpression;
				if (OtherExpression && OtherExpression->IsIdentical(UniformExpression))
				{
					delete UniformExpression;
					// Reuse the entry in CurrentScopeChunks
					AddCodeChunkToCurrentScope(ChunkIndex);
					return ChunkIndex;
				}
			}
			delete UniformExpression;
			// Use the existing uniform expression from a different material property,
			// And continue so that a code chunk using the uniform expression will be generated for this material property.
			UniformExpression = TestExpression;
			break;
		}

#if 0
		// Test for the case where we have non-identical expressions of the same type and name.
		// This means they exist with separate values and the one retrieved for shading will
		// effectively be random, as we evaluate the first found during expression traversal
		if (TestExpression->GetType() == UniformExpression->GetType())
		{
			if (TestExpression->GetType() == &FMaterialUniformExpressionScalarParameter::StaticType)
			{
				FMaterialUniformExpressionScalarParameter* ScalarParameterA = (FMaterialUniformExpressionScalarParameter*)TestExpression;
				FMaterialUniformExpressionScalarParameter* ScalarParameterB = (FMaterialUniformExpressionScalarParameter*)UniformExpression;

				if (!ScalarParameterA->GetParameterInfo().Name.IsNone() && ScalarParameterA->GetParameterInfo() == ScalarParameterB->GetParameterInfo())
				{
					delete UniformExpression;
					return Errorf(TEXT("Invalid scalar parameter '%s' found. Identical parameters must have the same value."), *(ScalarParameterA->GetParameterInfo().Name.ToString()));
				}
			}
			else if (TestExpression->GetType() == &FMaterialUniformExpressionVectorParameter::StaticType)
			{
				FMaterialUniformExpressionVectorParameter* VectorParameterA = (FMaterialUniformExpressionVectorParameter*)TestExpression;
				FMaterialUniformExpressionVectorParameter* VectorParameterB = (FMaterialUniformExpressionVectorParameter*)UniformExpression;

				// Note: Skipping NAME_SelectionColor here as this behavior is relied on for editor materials
				if (!VectorParameterA->GetParameterInfo().Name.IsNone() && VectorParameterA->GetParameterInfo() == VectorParameterB->GetParameterInfo()
					&& VectorParameterA->GetParameterInfo().Name != NAME_SelectionColor)
				{
					delete UniformExpression;
					return Errorf(TEXT("Invalid vector parameter '%s' found. Identical parameters must have the same value."), *(VectorParameterA->GetParameterInfo().Name.ToString()));
				}
			}
		}
#endif
	}

	const int32 ReturnIndex = CurrentScopeChunks->Num();
	// Create a new code chunk for the uniform expression
	// Note that uniforms have a known-zero derivative
	new(*CurrentScopeChunks) FShaderCodeChunk(Hash, UniformExpression, FormattedCode, FormattedCode, Type, EDerivativeStatus::Zero);

	if (!bFoundExistingExpression)
	{
		// Add an entry to the material-wide list of uniform expressions
		UniformExpression->UniformIndex = UniformExpressions.Num();
		new(UniformExpressions) FShaderCodeChunk(Hash, UniformExpression, FormattedCode, FormattedCode, Type, EDerivativeStatus::Zero);
	}

	AddCodeChunkToCurrentScope(ReturnIndex);
	return ReturnIndex;
}

// AddUniformExpression - Adds an input to the Code array and returns its index.
int32 FHLSLMaterialTranslator::AddUniformExpression(FAddUniformExpressionScope& Scope, FMaterialUniformExpression* UniformExpression, EMaterialValueType Type, const TCHAR* Format, ...)
{
	TStringBuilder<256> Builder;

	va_list Args;
	va_start(Args, Format);
	const TCHAR* FormattedCode = Builder.AppendV(Format, Args).GetData();
	va_end(Args);

	const uint64 Hash = CityHash64((const char*)FormattedCode, Builder.Len() * sizeof(TCHAR));
	return AddUniformExpressionInner(Hash, UniformExpression, Type, FormattedCode);
}

// From MaterialUniformExpressions.cpp
extern void WriteMaterialUniformAccess(UE::Shader::EValueComponentType ComponentType, uint32 NumComponents, uint32 UniformOffset, const TCHAR* MaterialNamePrefix, FStringBuilderBase& OutResult);

void SetupNumericUniformExpressionOffset(FUniformExpressionSet& UniformExpressionSet, 
	FMaterialUniformExpression* CodeChunkUniformExpression, 
	uint32& UniformPreshaderOffset,
	const uint32 NumComponents,
	bool bIsLWC,
	int32 PreshaderGapInterval)
{
	if (ensureAlways(CodeChunkUniformExpression->UniformOffset == INDEX_NONE))
	{
		const uint32 RegisterOffset = UniformPreshaderOffset % 4;
		if (!bIsLWC && RegisterOffset + NumComponents > 4u)
		{
			// If this uniform would span multiple registers, align offset to the next register to avoid this
			UniformPreshaderOffset = Align(UniformPreshaderOffset, 4u);
		}

		// Optionally insert an empty vector element (gap) at the given interval, to work around a platform specific shader compiler bug
		if (!PreshaderGapInterval)
		{
			PreshaderGapInterval = GPreshaderGapInterval;
		}
		if (PreshaderGapInterval > 0)
		{
			// Set a minimum of 4 (one out of every four elements is padding)
			PreshaderGapInterval = FMath::Max(PreshaderGapInterval, 4);

			// Check if we are on the vector element that is the end of the gap interval, or we will pass the end of the gap interval if the
			// preshader is larger than a single vector.  The divides will change value when a modulus gap interval boundary is crossed.
			uint32 PreshaderGapOffset = FMath::Max(4u, bIsLWC ? NumComponents * 2u : NumComponents);
			if (UniformPreshaderOffset / (PreshaderGapInterval *4) != (UniformPreshaderOffset + PreshaderGapOffset) / (PreshaderGapInterval *4))
			{
				UniformPreshaderOffset += 4u;
			}
		}

		const UE::Shader::EValueComponentType ComponentType = bIsLWC ? UE::Shader::EValueComponentType::Double : UE::Shader::EValueComponentType::Float;

		UniformExpressionSet.WriteUniformPreshaderEntry(
			UniformPreshaderOffset, UE::Shader::MakeValueType(ComponentType, NumComponents),
			[&CodeChunkUniformExpression](UE::Shader::FPreshaderData& UniformPreshaderData) -> void
			{
				CodeChunkUniformExpression->WriteNumberOpcodes(UniformPreshaderData);
			}
		);

		CodeChunkUniformExpression->UniformOffset = UniformPreshaderOffset;
		UniformPreshaderOffset += bIsLWC ? NumComponents * 2u : NumComponents;
	}
}

void SetupBooleanUniformExpressionOffset(FUniformExpressionSet& UniformExpressionSet, 
	FMaterialUniformExpression* CodeChunkUniformExpression, 
	uint32& UniformPreshaderOffset,
	uint32& CurrentNumBoolComponents, 
	uint32& CurrentBoolUniformOffset, 
	const uint32 NumComponents)
{
	if (ensureAlways(CodeChunkUniformExpression->UniformOffset == INDEX_NONE))
	{
		// 'Bool' uniforms are packed into bits
		if (CurrentNumBoolComponents + NumComponents > 32u)
		{
			CurrentBoolUniformOffset = UniformPreshaderOffset++;
			CurrentNumBoolComponents = 0u;
		}

		int32 UniformOffset = CurrentBoolUniformOffset * 32u + CurrentNumBoolComponents;

		UniformExpressionSet.WriteUniformPreshaderEntry(
			UniformOffset, UE::Shader::MakeValueType(UE::Shader::EValueComponentType::Bool, NumComponents),
			[&CodeChunkUniformExpression](UE::Shader::FPreshaderData& UniformPreshaderData) -> void
			{
				CodeChunkUniformExpression->WriteNumberOpcodes(UniformPreshaderData);
			}
		);

		CodeChunkUniformExpression->UniformOffset = UniformOffset;
		CurrentNumBoolComponents += NumComponents;
	}

}

static void EmitBoolPreshaderBufferString(FStringBuilderBase& FormattedCode, const FMaterialUniformExpression* CodeChunkUniformExpression, const TCHAR* CurrentMaterialUBName)
{
	const uint32 UniformOffset = CodeChunkUniformExpression->UniformOffset / 32u;
	const uint32 NumBoolComponents = CodeChunkUniformExpression->UniformOffset % 32u;

	const uint32 RegisterIndex = UniformOffset / 4;
	const uint32 RegisterOffset = UniformOffset % 4;
	FormattedCode.Appendf(TEXT("UnpackUniform_%s(asuint(%s.PreshaderBuffer[%u][%u]), %u)"),
		TEXT("bool"),
		CurrentMaterialUBName,
		RegisterIndex,
		RegisterOffset,
		NumBoolComponents);

}

static void EmitNumericPreshaderBufferString(FStringBuilderBase& FormattedCode, const EMaterialValueType MaterialValueType, const TCHAR* MaterialUBName ,const uint32 UniformOffset)
{
	const bool bIsLWC = IsLWCType(MaterialValueType);
	const uint32 NumComponents = GetNumComponents(MaterialValueType);

	if (bIsLWC)
	{
		if (NumComponents == 1)
		{
			FormattedCode.Append(TEXT("DFToWS(MakeDFScalar("));
		}
		else
		{
			FormattedCode.Appendf(TEXT("DFToWS(MakeDFVector%d("), NumComponents);
		}

		WriteMaterialUniformAccess(UE::Shader::EValueComponentType::Float, NumComponents, UniformOffset, MaterialUBName, FormattedCode); // High
		FormattedCode.Append(TEXT(","));
		WriteMaterialUniformAccess(UE::Shader::EValueComponentType::Float, NumComponents, UniformOffset + NumComponents, MaterialUBName, FormattedCode); // Low
		FormattedCode.Append(TEXT("))"));
	}
	else
	{
		if (IsUIntNumericType(MaterialValueType))
		{
			FormattedCode.Append(TEXT("asuint("));
		}

		WriteMaterialUniformAccess(UE::Shader::EValueComponentType::Float, NumComponents, UniformOffset, MaterialUBName, FormattedCode);

		if (IsUIntNumericType(MaterialValueType))
		{
			FormattedCode.Append(TEXT(")"));
		}
	}
}

static const TCHAR* GetMaterialPreshaderBufferPrefix()
{
	return TEXT("__Material_PreshaderBuffer_");
}

void FHLSLMaterialTranslator::GenerateUniformExpressionMacro(FStringBuilderBase& FormattedCode, const FShaderCodeChunk&	CodeChunk)
{
	// We first use UniformOffset as a unique id to emit the __Material_PreshaderBuffer_%u macro below.
	// Once we're done traversing the graph, FHLSLMaterialTranslator::GenerateUniformExpressionMacroDefinitions 
	// will set UniformOffset to its final value for runtime use, ie the preshader offset in the UB
	if (CodeChunk.UniformExpression->UniformOffset == INDEX_NONE)
	{
		CodeChunk.UniformExpression->UniformOffset = UniformExpressionShaderFrequencies.Num();
		UniformExpressionShaderFrequencies.Add(	FUniformExpressionInPreshader{ .UniformExpression = CodeChunk.UniformExpression, .CodeChunkMaterialValueType = CodeChunk.Type });
	}

	int32 PreshaderBufferWrapperIndex = CodeChunk.UniformExpression->UniformOffset;
	FUniformExpressionInPreshader& UniformExpressionInPreshader = UniformExpressionShaderFrequencies[PreshaderBufferWrapperIndex];
	FormattedCode.Appendf(TEXT("%s%u"), GetMaterialPreshaderBufferPrefix(), PreshaderBufferWrapperIndex);
}

FString FHLSLMaterialTranslator::GenerateUniformExpressionMacroDefinitions()
{
	TArray<int32> UniformWrapperIndices;
	UniformWrapperIndices.Reserve(UniformExpressionShaderFrequencies.Num());
	for (const FUniformExpressionInPreshader& UniformMetaData : UniformExpressionShaderFrequencies)
	{
		check(UniformMetaData.UniformExpression->UniformOffset != INDEX_NONE);
		UniformWrapperIndices.Add(UniformMetaData.UniformExpression->UniformOffset);
	}

	for (const FUniformExpressionInPreshader& UniformMetaData : UniformExpressionShaderFrequencies)
	{
		UniformMetaData.UniformExpression->UniformOffset = INDEX_NONE;
	}

	FUniformExpressionSet& MaterialCompilationOutputUniformExpressionSet = MaterialCompilationOutput.UniformExpressionSet;

	auto SetupUniformExpressionInPreshader = [this]
		(const FUniformExpressionInPreshader& UniformMetaData)
	{
		const uint32 NumComponents = GetNumComponents(UniformMetaData.CodeChunkMaterialValueType);
		const bool bIsLWC = IsLWCType(UniformMetaData.CodeChunkMaterialValueType);

		if (UniformMetaData.CodeChunkMaterialValueType == MCT_Bool)
		{
			SetupBooleanUniformExpressionOffset(MaterialCompilationOutput.UniformExpressionSet,
				UniformMetaData.UniformExpression,
				UniformPreshaderOffset,
				CurrentNumBoolComponents,
				CurrentBoolUniformOffset,
				NumComponents);

		}
		else if (IsFloatNumericType(UniformMetaData.CodeChunkMaterialValueType) || IsUIntNumericType(UniformMetaData.CodeChunkMaterialValueType))
		{
			SetupNumericUniformExpressionOffset(MaterialCompilationOutput.UniformExpressionSet, 
				UniformMetaData.UniformExpression, 
				UniformPreshaderOffset,
				NumComponents,
				bIsLWC,
				Material->GetPreshaderGap());
		}
		else
		{
			checkNoEntry();
		}
	};

	// First process parameters that will be used by the compact version so that we can get a single memcpy when compacting
	if (UseMaterialVSUniformBuffer())
	{
		for (const FUniformExpressionInPreshader& UniformMetaData : UniformExpressionShaderFrequencies)
		{
			if (HasShaderFrequency(UniformMetaData.UniformExpression->ShaderFrequencyMask, SF_Vertex))
			{
				SetupUniformExpressionInPreshader(UniformMetaData);
			}

		}

		UniformPreshaderOffsetVS = UniformPreshaderOffset;

		for (const FUniformExpressionInPreshader& UniformMetaData : UniformExpressionShaderFrequencies)
		{
			if (!HasShaderFrequency(UniformMetaData.UniformExpression->ShaderFrequencyMask, SF_Vertex))
			{
				SetupUniformExpressionInPreshader(UniformMetaData);
			}
		}
	}
	else
	{
		for (const FUniformExpressionInPreshader& UniformMetaData : UniformExpressionShaderFrequencies)
		{
			SetupUniformExpressionInPreshader(UniformMetaData);
		}
	}

	TStringBuilder<1024> OutResult;
	check(UniformWrapperIndices.Num() == UniformExpressionShaderFrequencies.Num());
	for (int32 UniformExpressionIndex = 0; UniformExpressionIndex < UniformExpressionShaderFrequencies.Num(); ++UniformExpressionIndex)
	{
		const FUniformExpressionInPreshader& UniformMetaData = UniformExpressionShaderFrequencies[UniformExpressionIndex];
		EShaderFrequency UniformExpressionVSOnlyFrequency = HasShaderFrequency(UniformMetaData.UniformExpression->ShaderFrequencyMask, SF_Vertex) ? SF_Vertex : SF_NumFrequencies;
		const TCHAR* MaterialVSName = GetCurrentMaterialUBNameInternal(UniformExpressionVSOnlyFrequency);

		OutResult.Appendf(TEXT("#define %s%u "), GetMaterialPreshaderBufferPrefix(), UniformWrapperIndices[UniformExpressionIndex]);
		if (UniformMetaData.CodeChunkMaterialValueType == MCT_Bool)
		{
			EmitBoolPreshaderBufferString(OutResult, UniformMetaData.UniformExpression, MaterialVSName);
		}
		else if (IsFloatNumericType(UniformMetaData.CodeChunkMaterialValueType) || IsUIntNumericType(UniformMetaData.CodeChunkMaterialValueType))
		{
			EmitNumericPreshaderBufferString(OutResult, UniformMetaData.CodeChunkMaterialValueType, MaterialVSName, UniformMetaData.UniformExpression->UniformOffset);
		}
		else
		{
			checkNoEntry();
		}

		OutResult.Append(TEXT("\n"));
	}

	return OutResult.ToString();
}

// AccessUniformExpression - Adds code to access the value of a uniform expression to the Code array and returns its index.
int32 FHLSLMaterialTranslator::AccessUniformExpression(int32 Index)
{
	check(Index >= 0 && Index < CurrentScopeChunks->Num());
	const FShaderCodeChunk&	CodeChunk = (*CurrentScopeChunks)[Index];
	check(CodeChunk.UniformExpression && !CodeChunk.UniformExpression->IsConstant());
	const bool bIsLWC = IsLWCType(CodeChunk.Type);

	// If this is a uniform expression referenced from another uniform expression, return a unique stub expression, to
	// avoid instantiating a preshader that's not actually needed.
	if (bCullIntermediateUniformExpressions && AddingUniformExpression)
	{
		check(CodeChunk.UniformExpression->UniformIndex != INDEX_NONE);
		return AddInlinedCodeChunkZeroDeriv(CodeChunk.Type, TEXT("UniformStub[$%d]"), CodeChunk.UniformExpression->UniformIndex);
	}

	FMaterialUniformExpressionTexture* TextureUniformExpression = CodeChunk.UniformExpression->GetTextureUniformExpression();
	FMaterialUniformExpressionTextureCollection* TextureCollectionUniformExpression = CodeChunk.UniformExpression->GetTextureCollectionUniformExpression();
	FMaterialUniformExpressionExternalTexture* ExternalTextureUniformExpression = CodeChunk.UniformExpression->GetExternalTextureUniformExpression();

	// Any code chunk can have a texture uniform expression (eg FMaterialUniformExpressionFlipBookTextureParameter),
	// But a texture code chunk must have a texture uniform expression
	check(!(CodeChunk.Type & MCT_Texture) || TextureUniformExpression || ExternalTextureUniformExpression);
	// Texture collection samples must have a corresponding uniform expression
	check(!(CodeChunk.Type & MCT_TextureCollection) || TextureCollectionUniformExpression);
	// External texture samples must have a corresponding uniform expression
	check(!(CodeChunk.Type & MCT_TextureExternal) || ExternalTextureUniformExpression);
	// Virtual texture samples must have a corresponding uniform expression
	check(!(CodeChunk.Type & MCT_TextureVirtual) || TextureUniformExpression);

	TStringBuilder<1024> FormattedCode;

	if (CodeChunk.Type == MCT_Bool)
	{
		check(CodeChunk.UniformExpression->GetType() == &FMaterialUniformExpressionStaticBoolParameter::StaticType);
		FMaterialUniformExpressionStaticBoolParameter* StaticBoolParameter = static_cast<FMaterialUniformExpressionStaticBoolParameter*>(CodeChunk.UniformExpression.GetReference());
		check(!bIsLWC);
		GenerateUniformExpressionMacro(FormattedCode, CodeChunk);
	}
	else if (IsFloatNumericType(CodeChunk.Type) || IsUIntNumericType(CodeChunk.Type))
	{
		check(CodeChunk.DerivativeStatus != EDerivativeStatus::Valid);

		GenerateUniformExpressionMacro(FormattedCode, CodeChunk);
	}
	else if(CodeChunk.Type & MCT_Texture)
	{
		int32 TextureInputIndex = INDEX_NONE;
		const TCHAR* BaseName = TEXT("");
		bool GenerateCode = true;
		switch(CodeChunk.Type)
		{
		case MCT_Texture2D:
			TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Standard2D].AddUnique(TextureUniformExpression);
			BaseName = TEXT("Texture2D");
			break;
		case MCT_TextureCube:
			TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Cube].AddUnique(TextureUniformExpression);
			BaseName = TEXT("TextureCube");
			break;
		case MCT_Texture2DArray:
			TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Array2D].AddUnique(TextureUniformExpression);
			BaseName = TEXT("Texture2DArray");
			break;
		case MCT_TextureCubeArray:
			TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::ArrayCube].AddUnique(TextureUniformExpression);
			BaseName = TEXT("TextureCubeArray");
			break;
		case MCT_VolumeTexture:
			TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Volume].AddUnique(TextureUniformExpression);
			BaseName = TEXT("VolumeTexture");
			break;
		case MCT_TextureExternal:
			TextureInputIndex = UniformExternalTextureExpressions.AddUnique(ExternalTextureUniformExpression);
			BaseName = TEXT("ExternalTexture");
			break;
		case MCT_TextureVirtual:
			TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Virtual].AddUnique(TextureUniformExpression);
			GenerateCode = false;
			break;
		default: UE_LOGF(LogMaterial, Fatal,"Unrecognized texture material value type: %u",(int32)CodeChunk.Type);
		};
		if(GenerateCode)
		{
			FormattedCode.Appendf(TEXT("%s.%s_%u"), GetCurrentMaterialUniformBufferName(), BaseName, TextureInputIndex);
		}
	}
	else if (CodeChunk.Type == MCT_SparseVolumeTexture)
	{
		int32 TextureInputIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::SparseVolume].AddUnique(TextureUniformExpression);
		FormattedCode.Appendf(TEXT("SparseVolumeTextureUnpackUniforms(%s.SVTPackedUniform[%d*2], %s.SVTPackedUniform[%d*2+1])"), GetCurrentMaterialUniformBufferName(), TextureInputIndex, GetCurrentMaterialUniformBufferName(), TextureInputIndex);
	}
	else if (CodeChunk.Type == MCT_TextureCollection)
	{
		// If unallocated, assign a type prefix
		if (TextureCollectionUniformExpression->GetTextureCollectionTypePrefixIndex() == UINT32_MAX)
		{
			uint32 TypePrefix = 0;
			for (int32 i = 0; i < UniformTextureCollectionExpressions.Num(); i++)
			{
				TypePrefix += UniformTextureCollectionExpressions[i]->IsVirtualCollection() == TextureCollectionUniformExpression->IsVirtualCollection();
			}

			TextureCollectionUniformExpression->SetTextureCollectionTypePrefixIndex(TypePrefix);
		}
		
		int32 TextureCollectionInputIndex = UniformTextureCollectionExpressions.AddUnique(TextureCollectionUniformExpression);
		FormattedCode.Appendf(TEXT("%s.TextureCollection_%u"), GetCurrentMaterialUniformBufferName(), TextureCollectionInputIndex);
	}
	else
	{
		UE_LOGF(LogMaterial, Fatal,"User input of unknown type: %ls",DescribeType(CodeChunk.Type));
	}


	return AddInlinedCodeChunkZeroDeriv(CodeChunk.Type,FormattedCode.ToString());
}

FString FHLSLMaterialTranslator::CoerceValue(const FString& Code, EMaterialValueType SourceType, EMaterialValueType DestType, EMaterialCastFlags AdditionalCastFlags)
{
	EMaterialCastFlags CastFlags = EMaterialCastFlags::ReplicateScalar | AdditionalCastFlags;
	if (DestType == MCT_Float || DestType == MCT_Float1 || DestType == MCT_LWCScalar)
	{
		// CoerceValue allows truncating to scalar types only
		CastFlags |= EMaterialCastFlags::AllowTruncate;
	}
	return CastValue(Code, SourceType, DestType, CastFlags);
}

// CoerceParameter
FString FHLSLMaterialTranslator::CoerceParameter(int32 Index,EMaterialValueType DestType, EMaterialCastFlags AdditionalCastFlags)
{
	return CoerceValue(GetParameterCode(Index), GetParameterType(Index), DestType, AdditionalCastFlags);
}

// GetParameterType
EMaterialValueType FHLSLMaterialTranslator::GetParameterType(int32 Index) const
{
	check(Index >= 0 && Index < CurrentScopeChunks->Num());
	return (*CurrentScopeChunks)[Index].Type;
}

// GetParameterUniformExpression
FMaterialUniformExpression* FHLSLMaterialTranslator::GetParameterUniformExpression(int32 Index) const
{
	check(Index >= 0 && Index < CurrentScopeChunks->Num());

	const FShaderCodeChunk& Chunk = (*CurrentScopeChunks)[Index];

	return Chunk.UniformExpression;
}

bool FHLSLMaterialTranslator::GetTextureForExpression(int32 Index, int32& OutTextureIndex, EMaterialSamplerType& OutSamplerType, TOptional<FName>& OutParameterName) const
{
	check(Index >= 0 && Index < CurrentScopeChunks->Num());
	const FShaderCodeChunk& Chunk = (*CurrentScopeChunks)[Index];
	const EMaterialValueType TexInputType = Chunk.Type;
	if ((TexInputType & MCT_Texture) == 0)
	{
		return false;
	}

	// If 'InputExpression' is connected, we use need to find the texture object that was passed in
	// In this case, the texture/sampler assigned on this expression node are not used
	FMaterialUniformExpression* TextureUniformBase = Chunk.UniformExpression;
	checkf(TextureUniformBase, TEXT("TexInputType is %" UINT64_FMT ", but missing FMaterialUniformExpression"), TexInputType);

	if (FMaterialUniformExpressionTexture* TextureUniform = TextureUniformBase->GetTextureUniformExpression())
	{
		OutSamplerType = TextureUniform->GetSamplerType();
		OutTextureIndex = TextureUniform->GetTextureIndex();
		if (FMaterialUniformExpressionTextureParameter* TextureParameterUniform = TextureUniform->GetTextureParameterUniformExpression())
		{
			OutParameterName = TextureParameterUniform->GetParameterName();
		}
	}
	else if (FMaterialUniformExpressionExternalTexture* ExternalTextureUniform = TextureUniformBase->GetExternalTextureUniformExpression())
	{
		OutTextureIndex = ExternalTextureUniform->GetSourceTextureIndex();
		OutSamplerType = SAMPLERTYPE_External;
		if (FMaterialUniformExpressionExternalTextureParameter* ExternalTextureParameterUniform = ExternalTextureUniform->GetExternalTextureParameterUniformExpression())
		{
			OutParameterName = ExternalTextureParameterUniform->GetParameterName();
		}
	}

	return true;
}

bool FHLSLMaterialTranslator::GetTextureCollectionForExpression(int32 Index, int32& OutTextureCollectionIndex, TOptional<FName>& OutParameterName) const
{
	check(Index >= 0 && Index < CurrentScopeChunks->Num());
	const FShaderCodeChunk& Chunk = (*CurrentScopeChunks)[Index];
	const EMaterialValueType TexInputType = Chunk.Type;
	if ((TexInputType & MCT_TextureCollection) == 0)
	{
		return false;
	}

	// If 'InputExpression' is connected, we use need to find the texture collection object that was passed in
	// In this case, the texture collection assigned on this expression node is not used
	FMaterialUniformExpression* UniformExpression = Chunk.UniformExpression;
	checkf(UniformExpression, TEXT("TexInputType is %" UINT64_FMT ", but missing FMaterialUniformExpression"), TexInputType);

	if (FMaterialUniformExpressionTextureCollection* TextureCollectionUniform = UniformExpression->GetTextureCollectionUniformExpression())
	{
		OutTextureCollectionIndex = TextureCollectionUniform->GetTextureCollectionIndex();
		if (FMaterialUniformExpressionTextureCollectionParameter* TextureCollectionParameterUniform = TextureCollectionUniform->GetTextureCollectionParameterUniformExpression())
		{
			OutParameterName = TextureCollectionParameterUniform->GetParameterName();
		}
	}

	return true;
}

// GetArithmeticResultType
EMaterialValueType FHLSLMaterialTranslator::GetArithmeticResultType(EMaterialValueType TypeA, EMaterialValueType TypeB)
{
	if (!IsPrimitiveType(TypeA) || !IsPrimitiveType(TypeB))
	{
		Errorf(TEXT("Attempting to perform arithmetic on non-primitive types: %s %s"), DescribeType(TypeA),DescribeType(TypeB));
		return MCT_Unknown;
	}

	if(TypeA == TypeB)
	{
		return TypeA;
	}
	else if (IsLWCType(TypeA) || IsLWCType(TypeB))
	{
		const EMaterialValueType LWCTypeA = MakeLWCType(TypeA);
		const EMaterialValueType LWCTypeB = MakeLWCType(TypeB);
		if (LWCTypeA == LWCTypeB)
		{
			return LWCTypeA;
		}
		else if (LWCTypeA == MCT_LWCScalar && IsFloatNumericType(LWCTypeB))
		{
			return LWCTypeB;
		}
		else if (LWCTypeB == MCT_LWCScalar && IsFloatNumericType(LWCTypeA))
		{
			return LWCTypeA;
		}
	}
	else if (TypeA == MCT_Float || TypeA == MCT_Float1)
	{
		return TypeB;
	}
	else if(TypeB == MCT_Float || TypeB == MCT_Float1)
	{
		return TypeA;
	}

	Errorf(TEXT("Arithmetic between types %s and %s are undefined"), DescribeType(TypeA), DescribeType(TypeB));
	return MCT_Unknown;
}

EMaterialValueType FHLSLMaterialTranslator::GetArithmeticResultType(int32 A,int32 B)
{
	check(A >= 0 && A < CurrentScopeChunks->Num());
	check(B >= 0 && B < CurrentScopeChunks->Num());

	EMaterialValueType	TypeA = (*CurrentScopeChunks)[A].Type,
		TypeB = (*CurrentScopeChunks)[B].Type;

	return GetArithmeticResultType(TypeA,TypeB);
}

int32 FHLSLMaterialTranslator::FindOrAddUserSceneTexture(FName UserSceneTextureName)
{
	return MaterialCompilationOutput.FindOrAddUserSceneTexture(UserSceneTextureName);
}

// FMaterialCompiler interface.

/** 
	* Sets the current material property being compiled.  
	* This affects the internal state of the compiler and the results of all functions except GetFixedParameterCode.
	* @param OverrideShaderFrequency SF_NumFrequencies to not override
	*/
void FHLSLMaterialTranslator::SetMaterialProperty(EMaterialProperty InProperty, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime)
{
	MaterialProperty = InProperty;
	SetBaseMaterialAttribute(FMaterialAttributeDefinitionMap::GetID(InProperty));

	if(OverrideShaderFrequency != SF_NumFrequencies)
	{
		ShaderFrequency = OverrideShaderFrequency;
	}
	else
	{
		ShaderFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(InProperty);
	}

	bCompilingPreviousFrame = bUsePreviousFrameTime;
	AssignShaderFrequencyScope(ShaderFrequency);
}

void FHLSLMaterialTranslator::PushMaterialAttribute(const FGuid& InAttributeID)
{
	MaterialAttributesStack.Push(InAttributeID);
}

FGuid FHLSLMaterialTranslator::PopMaterialAttribute()
{
	return MaterialAttributesStack.Pop(EAllowShrinking::No);
}

const FGuid FHLSLMaterialTranslator::GetMaterialAttribute()
{
	checkf(MaterialAttributesStack.Num() > 0, TEXT("Tried to query empty material attributes stack."));
	return MaterialAttributesStack.Top();
}

void FHLSLMaterialTranslator::SetBaseMaterialAttribute(const FGuid& InAttributeID)
{
	// This is atypical behavior but is done to allow cleaner code and preserve existing paths.
	// A base property is kept on the stack and updated by SetMaterialProperty(), the stack is only utilized during translation
	checkf(MaterialAttributesStack.Num() == 1, TEXT("Tried to set non-base attribute on stack."));
	MaterialAttributesStack.Top() = InAttributeID;
}

UMaterialExpressionCustomOutput* FHLSLMaterialTranslator::GetTopCustomOutput()
{
	return TopCustomOutput;
}

void FHLSLMaterialTranslator::PushParameterOwner(const FMaterialParameterInfo& InOwnerInfo)
{
	ParameterOwnerStack.Push(InOwnerInfo);
}

FMaterialParameterInfo FHLSLMaterialTranslator::PopParameterOwner()
{
	return ParameterOwnerStack.Pop(EAllowShrinking::No);
}

EShaderFrequency FHLSLMaterialTranslator::GetCurrentShaderFrequency() const
{
	return ShaderFrequency;
}

bool FHLSLMaterialTranslator::IsTangentSpaceNormal() const
{
	check(Material);
	return Material->GetMaterialDomain() == MD_DeferredDecal || Material->IsTangentSpaceNormal();
}

FMaterialShadingModelField FHLSLMaterialTranslator::GetMaterialShadingModels() const
{
	check(Material);
	return Material->GetShadingModels();
}

FMaterialShadingModelField FHLSLMaterialTranslator::GetCompiledShadingModels() const
{
	check(Material);

	// If the material gets its shading model from material expressions and we have compiled one or more shading model expressions already, 
	// then use that shading model field instead. It's the most optimal set of shading models
	if (Material->IsShadingModelFromMaterialExpression() && ShadingModelsFromCompilation.IsValid())
	{
		return ShadingModelsFromCompilation;
	}
	else
	{
		FMaterialShadingModelField MaterialShadingModels = Material->GetShadingModels();
		UMaterialInterface::FilterOutPlatformShadingModels(Platform, MaterialShadingModels);
		return MaterialShadingModels;
	}
}

int32 FHLSLMaterialTranslator::Error(const TCHAR* Text)
{
	// Optionally append errors into proxy arrays which allow pre-translation stages to selectively include errors later
	bool bUsingErrorProxy = (CompileErrorsSink && CompileErrorExpressionsSink);	
	TArray<FString>& CompileErrors = bUsingErrorProxy ? *CompileErrorsSink : Material->CompileErrors;
	TArray<TObjectPtr<UMaterialExpression>>& ErrorExpressions = bUsingErrorProxy ? *CompileErrorExpressionsSink : Material->ErrorExpressions;

	FString ErrorString;
	UMaterialExpression* ExpressionToError = nullptr;

	check(ShaderFrequency < SF_NumFrequencies);
	auto& CurrentFunctionStack = FunctionStacks[ShaderFrequency];
	if (CurrentFunctionStack.Num() > 1)
	{
		// Only add the function call node to ErrorExpressions, since we can't add a reference to the expressions inside the function as they are private objects.
		// Add the first function node on the stack because that's the one visible in the material being compiled, the rest are all nested functions.
		ExpressionToError = CurrentFunctionStack[1]->FunctionCall;
		check(ExpressionToError);

		// Build full callstack for error message.
		ErrorString = FString("(Function ");
		for (int32 StackIndex = 1; StackIndex < CurrentFunctionStack.Num(); ++StackIndex)
		{
			FMaterialFunctionCompileState const* Function = CurrentFunctionStack[StackIndex];
			if (Function == nullptr || Function->FunctionCall == nullptr || Function->FunctionCall->MaterialFunction == nullptr)
			{
				continue;
			}

			ErrorString += Function->FunctionCall->MaterialFunction->GetName();

			if (StackIndex < CurrentFunctionStack.Num() - 1)
			{
				ErrorString += TEXT("|");
			}
		}
		ErrorString += FString(") ");
	}

	if (CurrentFunctionStack.Last()->ExpressionStack.Num() > 0)
	{
		UMaterialExpression* ErrorExpression = CurrentFunctionStack.Last()->ExpressionStack.Last().Expression;
		check(ErrorExpression);

		if (ErrorExpression->GetClass() != UMaterialExpressionMaterialFunctionCall::StaticClass()
			&& ErrorExpression->GetClass() != UMaterialExpressionFunctionInput::StaticClass()
			&& ErrorExpression->GetClass() != UMaterialExpressionFunctionOutput::StaticClass())
		{
			// Add the expression currently being compiled to ErrorExpressions so we can draw it differently
			ExpressionToError = ErrorExpression;

			const int32 ChopCount = FCString::Strlen(TEXT("MaterialExpression"));
			const FString ErrorClassName = ErrorExpression->GetClass()->GetName();

			// Add the node type to the error message
			ErrorString += FString("(Node ") + ErrorClassName.Right(ErrorClassName.Len() - ChopCount) + TEXT(") ");
		}
	}
			
	ErrorString += Text;

	if (!bUsingErrorProxy)
	{
		// Standard error handling, immediately append one-off errors and signal failure
		TranslationResult = EHLSLMaterialTranslatorResult::Failure;
		if (ExpressionToError)
		{
			ExpressionToError->LastErrorText = Text;
			ensure(ErrorExpressions.Num() == CompileErrors.Num());
			const int32 NumErrors = FMath::Min(ErrorExpressions.Num(), CompileErrors.Num());
			for (int32 i = 0; i < NumErrors; i++)
			{
				if(ErrorExpressions[i] == ExpressionToError && CompileErrors[i] == ErrorString)
				{
					return INDEX_NONE;
				}
			}
		}
	}

	// When a proxy is intercepting errors, ignore the failure and match arrays to allow later error type selection
	CompileErrors.Add(ErrorString);
	ErrorExpressions.Add(ExpressionToError);

	return INDEX_NONE;
}

void FHLSLMaterialTranslator::AppendExpressionError(UMaterialExpression* Expression, const TCHAR* Text)
{
	if (Expression && Text)
	{
		FString ErrorText(Text);

		Material->ErrorExpressions.Add(Expression);
		Expression->LastErrorText = ErrorText;
		Material->CompileErrors.Add(ErrorText);
	}
}

int32 FHLSLMaterialTranslator::CallExpression(FMaterialExpressionKey ExpressionKey,FMaterialCompiler* Compiler)
{
	// For any translated result not relying on material attributes, we can discard the attribute ID from the key
	// to allow result sharing. In cases where we detect an expression loop we must err on the side of caution
	if (ExpressionKey.Expression && !ExpressionKey.Expression->IsResultMaterialAttributes(ExpressionKey.OutputIndex))
	{
		ExpressionKey.MaterialAttributeID = FGuid(0, 0, 0, 0);
	}

	// Some expressions can discard output indices and share compiles with a swizzle/mask
	if (ExpressionKey.Expression && ExpressionKey.Expression->CanIgnoreOutputIndex())
	{
		ExpressionKey.OutputIndex = INDEX_NONE;
	}

	// Substrate BSDF expression should not be de-duplicated using expression output hash. 
	// This is automatically handled via the compiler SubstrateTreeStack.
	// It means that a node can be blended at multiple point of the graph (allowing acyclic graph instead of tree, e.g. a Slab can be used into multiple input).
	// We do this for all SubstrateData which can be output from BSDF nodes (slabs) or other nodes such as material functions.
	const bool bExpressionIsSubstrate = ExpressionKey.Expression && ExpressionKey.Expression->IsResultSubstrateMaterial(ExpressionKey.OutputIndex);

	// Check if this expression has already been translated.
	check(ShaderFrequency < SF_NumFrequencies);
	auto& CurrentFunctionStack = FunctionStacks[ShaderFrequency];
	FMaterialFunctionCompileState* CurrentFunctionState = CurrentFunctionStack.Last();

	static bool sDebugCacheDuplicateCode = true;
	int32* ExistingCodeIndex = sDebugCacheDuplicateCode && !bExpressionIsSubstrate ? CurrentFunctionState->ExpressionCodeMap.Find(ExpressionKey) : nullptr;
	int32 Result = INDEX_NONE;
	if (ExistingCodeIndex)
	{
		Result = *ExistingCodeIndex;
		AddCodeChunkToCurrentScope(Result);
	}
	else
	{
		// Disallow reentrance.
		if (CurrentFunctionState->ExpressionStack.Find(ExpressionKey) != INDEX_NONE)
		{
			return Error(TEXT("Reentrant expression"));
		}

		// The first time this expression is called, translate it.
		CurrentFunctionState->ExpressionStack.Add(ExpressionKey);
		const int32 FunctionDepth = CurrentFunctionStack.Num();
			
		// Attempt to share function states between function calls
		UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(ExpressionKey.Expression);
		if (FunctionCall)
		{
			FMaterialExpressionKey ReuseCompileStateExpressionKey = ExpressionKey;
			ReuseCompileStateExpressionKey.OutputIndex = INDEX_NONE; // Discard the output so we can share the stack internals
			ReuseCompileStateExpressionKey.MaterialAttributeID = FGuid(0, 0, 0, 0); //Discard the Material Attribute ID so we can share the stack internals

			FMaterialFunctionCompileState* SharedFunctionState = CurrentFunctionState->FindOrAddSharedFunctionState(ReuseCompileStateExpressionKey, FunctionCall);
			FunctionCall->SetSharedCompileState(SharedFunctionState);
		}

		ReferencedCodeChunks.Reset();

		Result = ExpressionKey.Expression->Compile(Compiler, ExpressionKey.OutputIndex);

		// Restore state
		if (FunctionCall)
		{
			FunctionCall->SetSharedCompileState(nullptr);
		}

		FMaterialExpressionKey PoppedExpressionKey = CurrentFunctionState->ExpressionStack.Pop();

		// Verify state integrity
		check(PoppedExpressionKey == ExpressionKey);
		check(FunctionDepth == CurrentFunctionStack.Num());

		// Cache the translation.
		CurrentFunctionStack.Last()->ExpressionCodeMap.Add(ExpressionKey,Result);

		if (Result != INDEX_NONE)
		{
			FShaderCodeChunk& ResultChunk = (*CurrentScopeChunks)[Result];					//crash here
			ResultChunk.ReferencedCodeChunks = MoveTemp(ReferencedCodeChunks);
		}
		ReferencedCodeChunks.Reset();
	}

		return Result;
	}

int32 FHLSLMaterialTranslator::CallExpressionExec(UMaterialExpression* Expression)
{
	check(ShaderFrequency < SF_NumFrequencies);
	auto& CurrentFunctionStack = FunctionStacks[ShaderFrequency];
	FMaterialFunctionCompileState* CurrentFunctionState = CurrentFunctionStack.Last();

	int32 Result = INDEX_NONE;
	/*int32* ExistingCodeIndex = CurrentFunctionState->ExecExpressionCodeMap.Find(Expression);
	if (ExistingCodeIndex)
	{
		Result = *ExistingCodeIndex;
		FShaderCodeChunk& Chunk = (*CurrentScopeChunks)[Result];
		const int32 CurrentScopeIndex = ScopeStack.Last();
		const FShaderCodeChunk& CurrentScope = (*CurrentScopeChunks)[CurrentScopeIndex];
		const FShaderCodeChunk& PrevScope = (*CurrentScopeChunks)[Chunk.UsedScopeIndex];
		if (CurrentScope.UsedScopeIndex == PrevScope.UsedScopeIndex)
		{
			AddCodeChunkToScope(Result, CurrentScope.UsedScopeIndex);
		}
		else
		{
			return Errorf(TEXT("Invalid scopes"));
		}
	}
	else*/
	{
		ReferencedCodeChunks.Reset();
		Result = Expression->Compile(this, UMaterialExpression::CompileExecutionOutputIndex);
		CurrentFunctionState->ExecExpressionCodeMap.Add(Expression, Result);

		if (Result != INDEX_NONE)
		{
			FShaderCodeChunk& ResultChunk = (*CurrentScopeChunks)[Result];
			ResultChunk.ReferencedCodeChunks = MoveTemp(ReferencedCodeChunks);
		}
		ReferencedCodeChunks.Reset();
	}
	return Result;
}

void FHLSLMaterialTranslator::AddCodeChunkToCurrentScope(int32 ChunkIndex)
{
	if (ChunkIndex != INDEX_NONE && ScopeStack.Num() > 0)
	{
		const int32 CurrentScopeIndex = ScopeStack.Last();
		AddCodeChunkToScope(ChunkIndex, CurrentScopeIndex);
	}
}

void FHLSLMaterialTranslator::AddCodeChunkToScope(int32 ChunkIndex, int32 ScopeIndex)
{
	if (ChunkIndex != INDEX_NONE)
	{
		FShaderCodeChunk& CurrentScope = (*CurrentScopeChunks)[ScopeIndex];

		FShaderCodeChunk& Chunk = (*CurrentScopeChunks)[ChunkIndex];
		if (Chunk.DeclaredScopeIndex == INDEX_NONE)
		{
			check(Chunk.UsedScopeIndex == INDEX_NONE);
			Chunk.DeclaredScopeIndex = ScopeIndex;
			Chunk.UsedScopeIndex = ScopeIndex;
			Chunk.ScopeLevel = CurrentScope.ScopeLevel + 1;
		}
		else if (Chunk.UsedScopeIndex != ScopeIndex)
		{
			// Find the most derived scope that's shared by the current scope, and the scope this code was previously referenced from
			int32 ScopeIndex0 = ScopeIndex;
			int32 ScopeIndex1 = Chunk.UsedScopeIndex;
			while (ScopeIndex0 != ScopeIndex1)
			{
				const FShaderCodeChunk& Scope0 = (*CurrentScopeChunks)[ScopeIndex0];
				const FShaderCodeChunk& Scope1 = (*CurrentScopeChunks)[ScopeIndex1];
				if (Scope0.ScopeLevel > Scope1.ScopeLevel)
				{
					check(Scope0.UsedScopeIndex != INDEX_NONE);
					ScopeIndex0 = Scope0.UsedScopeIndex;
				}
				else
				{
					check(Scope1.UsedScopeIndex != INDEX_NONE);
					ScopeIndex1 = Scope1.UsedScopeIndex;
				}
			}

			const FShaderCodeChunk& Scope = (*CurrentScopeChunks)[ScopeIndex0];
			Chunk.UsedScopeIndex = ScopeIndex0;
			Chunk.ScopeLevel = Scope.ScopeLevel + 1;
		}
	}
}

EMaterialValueType FHLSLMaterialTranslator::GetType(int32 Code)
{
	if(Code != INDEX_NONE)
	{
		return GetParameterType(Code);
	}
	else
	{
		return MCT_Unknown;
	}
}

EMaterialQualityLevel::Type FHLSLMaterialTranslator::GetQualityLevel()
{
	return QualityLevel;
}

ERHIFeatureLevel::Type FHLSLMaterialTranslator::GetFeatureLevel()
{
	return FeatureLevel;
}

EShaderPlatform FHLSLMaterialTranslator::GetShaderPlatform()
{
	return Platform;
}

const ITargetPlatform* FHLSLMaterialTranslator::GetTargetPlatform() const
{
	return TargetPlatform;
}

bool FHLSLMaterialTranslator::IsMaterialPropertyUsed(EMaterialProperty Property, int32 PropertyChunkIndex) const
{
	if (PropertyChunkIndex == -1)
	{
		return false;
	}
	else
	{
		FVector4f DefaultValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);
		EMaterialValueType ValueType = FMaterialAttributeDefinitionMap::GetValueType(Property);
		int32 ComponentCount = GetNumComponents(ValueType);

		return IsMaterialPropertyUsed(Property, PropertyChunkIndex, FLinearColor(DefaultValue), ComponentCount);
	}
}

/** 
	* Casts the passed in code to DestType, or generates a compile error if the cast is not valid. 
	* This will truncate a type (float4 -> float3) but not add components (float2 -> float3), however a float1 can be cast to any float type by replication. 
	*/
int32 FHLSLMaterialTranslator::ValidCast(int32 Code, EMaterialValueType DestType)
{
	if(Code == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialCastFlags Flags = EMaterialCastFlags::ValidCast;
	const EMaterialValueType SourceType = GetParameterType(Code);

	if (SourceType & DestType)
	{
		return Code;
	}
	else if ((SourceType & (MCT_TextureVirtual | MCT_TextureMeshPaint | MCT_TextureMaterialCache)) && (DestType & MCT_Texture2D))
	{
		return Code;
	}
	else if (DestType == MCT_MaterialAttributes)
	{
		// We can feed any type into a material attributes socket as we're really just passing them through.
		return Code;
	}
	else if (GetParameterUniformExpression(Code) && !GetParameterUniformExpression(Code)->IsConstant())
	{
		return ValidCast(AccessUniformExpression(Code), DestType);
	}
	else if (IsFloatNumericType(SourceType) && IsFloatNumericType(DestType))
	{
		const FDerivInfo CodeDerivInfo = GetDerivInfo(Code);

		FString FiniteCode = CastValue(GetParameterCode(Code), SourceType, DestType, Flags);
		if (IsAnalyticDerivEnabled() && IsDerivativeValid(CodeDerivInfo.DerivativeStatus))
		{
			if (CodeDerivInfo.DerivativeStatus == EDerivativeStatus::Valid)
			{
				FString DerivString = *GetParameterCodeDeriv(Code, CompiledPDV_Analytic);
				FString DDXCode = CastValue(DerivString + TEXT(".Ddx"), MakeNonLWCType(SourceType), MakeNonLWCType(DestType), Flags);
				FString DDYCode = CastValue(DerivString + TEXT(".Ddy"), MakeNonLWCType(SourceType), MakeNonLWCType(DestType), Flags);
				FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, DDXCode, DDYCode, GetDerivType(DestType));
				return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, DestType, false, EDerivativeStatus::Valid);
			}
			else
			{
				return AddInlinedCodeChunkZeroDeriv(DestType, *FiniteCode);
			}
		}
		else
		{
			return AddInlinedCodeChunk(DestType, *FiniteCode);
		}
	}
	
	return Errorf(TEXT("Cannot cast from %s to %s."), DescribeType(SourceType), DescribeType(DestType));
}

EMaterialCastFlags GetForceCastFlags(uint32 ForceCastFlags)
{
	EMaterialCastFlags CastFlags = EMaterialCastFlags::AllowTruncate | EMaterialCastFlags::AllowAppendZeroes;
	if ((ForceCastFlags & MFCF_ReplicateValue) || !(ForceCastFlags & MFCF_ExactMatch))
	{
		// Replicate scalar if requested, or if we don't require an exact match (this can happen when force-casting to/from LWC
		// The only way we *don't* replicate scalar is requesting an exact match without the ReplicateValue flag
		// TODO - My guess is that case probably isn't relevant, and we should just always replicate scalar on cast, but trying to preserve behavior for now
		CastFlags |= EMaterialCastFlags::ReplicateScalar;
	}
	return CastFlags;
}

int32 FHLSLMaterialTranslator::ForceCast(int32 Code, EMaterialValueType DestType, uint32 ForceCastFlags)
{
	if(Code == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(Code) && !GetParameterUniformExpression(Code)->IsConstant())
	{
		return ForceCast(AccessUniformExpression(Code),DestType,ForceCastFlags);
	}

	const EMaterialValueType SourceType = GetParameterType(Code);

	if ((ForceCastFlags & MFCF_ExactMatch) ? (SourceType == DestType) : (SourceType & DestType))
	{
		return Code;
	}
	else if (IsFloatNumericType(SourceType) && IsFloatNumericType(DestType))
	{
		const FDerivInfo CodeDerivInfo = GetDerivInfo(Code);

		EMaterialCastFlags CastFlags = GetForceCastFlags(ForceCastFlags);

		FString FiniteCode = CastValue(GetParameterCode(Code), SourceType, DestType, CastFlags);
		if (IsAnalyticDerivEnabled() && IsDerivativeValid(CodeDerivInfo.DerivativeStatus))
		{
			if (CodeDerivInfo.DerivativeStatus == EDerivativeStatus::Valid)
			{
				FString DerivString = *GetParameterCodeDeriv(Code, CompiledPDV_Analytic);
				FString DDXCode = CastValue(DerivString + TEXT(".Ddx"), MakeNonLWCType(SourceType), MakeNonLWCType(DestType), CastFlags);
				FString DDYCode = CastValue(DerivString + TEXT(".Ddy"), MakeNonLWCType(SourceType), MakeNonLWCType(DestType), CastFlags);
				FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, DDXCode, DDYCode, GetDerivType(DestType));
				return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, DestType, false, EDerivativeStatus::Valid);
			}
			else
			{
				return AddInlinedCodeChunkZeroDeriv(DestType, *FiniteCode);
			}
		}
		else
		{
			return AddInlinedCodeChunk(DestType, *FiniteCode);
		}
	}
	else if (IsNumericType(SourceType) && IsNumericType(DestType))
	{
		EMaterialCastFlags CastFlags = GetForceCastFlags(ForceCastFlags) | EMaterialCastFlags::AllowInteger;
		FString FiniteCode = CastValue(GetParameterCode(Code), SourceType, DestType, CastFlags);
		return AddInlinedCodeChunk(DestType, *FiniteCode);
	}
	else if ((SourceType & MCT_StaticBool) && IsFloatNumericType(DestType))
	{
		FString StaticBoolToFloat = GetParameterCode(Code).Equals("true") ? "1.0f" : "0.0f";
		StaticBoolToFloat = CastValue(StaticBoolToFloat, MCT_Float, DestType, GetForceCastFlags(ForceCastFlags));
		return AddInlinedCodeChunk(DestType, *StaticBoolToFloat);
	}
	else if ((SourceType & (MCT_TextureVirtual|MCT_TextureMeshPaint|MCT_TextureMaterialCache)) && (DestType & MCT_Texture2D))
	{
		return Code;
	}
	else
	{
		return Errorf(TEXT("Cannot force a cast between non-numeric types."));
	}
}

int32 FHLSLMaterialTranslator::CastShadingModelToFloat(int32 Code)
{
	const EMaterialValueType SourceType = GetParameterType(Code);
	if (SourceType != MCT_ShadingModel)
	{
		return Errorf(TEXT("Operation only supported for shading model"));
	}

	return AddCodeChunk(MCT_Float1, TEXT("((%s)(%s))"), HLSLTypeString(MCT_Float1), *GetParameterCode(Code));
}

int32 FHLSLMaterialTranslator::TruncateLWC(int32 Code)
{
	const int32 LWCTruncateMode = UE::MaterialTranslatorUtils::GetLWCTruncateMode();
	const bool bAllowLWCTruncate = LWCTruncateMode == 1 || LWCTruncateMode == 2;

	if (Code == INDEX_NONE || !bAllowLWCTruncate)
	{
		return Code;
	}

	const EMaterialValueType TruncateType = GetParameterType(Code);
	if (IsLWCType(TruncateType))
	{
		EMaterialValueType TruncatedType = MakeNonLWCType(TruncateType);
		
		if (TruncateType == MCT_LWCScalar)
		{
			TruncatedType = MCT_Float; // Remap from MCT_Float1 to MCT_Float to support replication
		}

		return ValidCast(Code, TruncatedType);
	}

	return Code;
}

int32 FHLSLMaterialTranslator::CastToNonLWCIfDisabled(int32 Code)
{
	int32 Result = Code;
	if (!GLWCEnabled)
	{
		const EMaterialValueType Type = GetParameterType(Code);
		if (IsLWCType(Type))
		{
			Result = ValidCast(Code, MakeNonLWCType(Type));
		}
	}
	return Result;
}

FString FHLSLMaterialTranslator::CastValue(const FString& Code, EMaterialValueType SourceType, EMaterialValueType DestType, EMaterialCastFlags Flags)
{
	if (SourceType == DestType)
	{
		return Code;
	}
	
	const bool bAllowTruncate = EnumHasAnyFlags(Flags, EMaterialCastFlags::AllowTruncate);
	const bool bAllowAppendZeroes = EnumHasAnyFlags(Flags, EMaterialCastFlags::AllowAppendZeroes);
	const bool bAllowReplicateScalar = EnumHasAnyFlags(Flags, EMaterialCastFlags::ReplicateScalar);
	const bool bAllowInteger = EnumHasAnyFlags(Flags, EMaterialCastFlags::AllowInteger);

	const EMaterialValueType AllowedTypes = EMaterialValueType(MCT_Float | MCT_LWCType | (bAllowInteger ? MCT_UInt : 0));

	if (IsMaterialValueType(SourceType, AllowedTypes) && IsMaterialValueType(DestType, AllowedTypes))
	{
		const uint32 NumSourceComponents = GetNumComponents(SourceType);
		const uint32 NumDestComponents = GetNumComponents(DestType);
		const bool bReplicateScalar = bAllowReplicateScalar && (NumSourceComponents == 1);
		if (!bReplicateScalar && !bAllowAppendZeroes && NumDestComponents > NumSourceComponents)
		{
			Errorf(TEXT("Cannot cast from smaller type %s to larger type %s."), DescribeType(SourceType), DescribeType(DestType));
			return FString();
		}
		if (!bReplicateScalar && !bAllowTruncate && NumDestComponents < NumSourceComponents)
		{
			Errorf(TEXT("Cannot cast from larger type %s to smaller type %s."), DescribeType(SourceType), DescribeType(DestType));
			return FString();
		}

		const bool bIsLWC = IsLWCType(DestType);
		if (bIsLWC != IsLWCType(SourceType))
		{
			if (bIsLWC)
			{
				// float->LWC
				AddLWCFuncUsage(ELWCFunctionKind::Promote);
				return FString::Printf(TEXT("WSPromote(%s)"), *CastValue(Code, SourceType, MakeNonLWCType(DestType), Flags));
			}
			else
			{
				//LWC->float
				AddLWCFuncUsage(ELWCFunctionKind::Demote);
				return CastValue(FString::Printf(TEXT("WSDemote(%s)"), *Code), MakeNonLWCType(SourceType), DestType, Flags);
			}
		}

		FString Result;
		uint32 NumComponents = 0u;
		bool bNeedClosingParen = false;
		if (bIsLWC)
		{
			AddLWCFuncUsage(ELWCFunctionKind::Constructor);
			Result = TEXT("MakeWSVector(");
			bNeedClosingParen = true;
		}
		else
		{
			if (NumSourceComponents == NumDestComponents)
			{
				NumComponents = NumDestComponents;
				Result += Code;
			}
			else if (bReplicateScalar)
			{
				NumComponents = NumDestComponents;
				// Cast the scalar to the correct type, HLSL language will replicate the scalar when performing this cast
				Result += FString::Printf(TEXT("((%s)%s)"), HLSLTypeString(DestType), *Code);
			}
			else
			{
				NumComponents = FMath::Min(NumSourceComponents, NumDestComponents);
				if (NumComponents < NumDestComponents)
				{
					Result = FString(HLSLTypeString(DestType)) + TEXT("(");
					bNeedClosingParen = true;
				}
				if (NumComponents == NumSourceComponents)
				{
					// If we're taking all the components from the source, can avoid adding a swizzle
					Result += Code;
				}
				else
				{
					static const TCHAR* Mask[] = { TEXT("<ERROR>"), TEXT("x"), TEXT("xy"), TEXT("xyz"), TEXT("xyzw") };
					check(NumComponents <= 4);
					Result += FString::Printf(TEXT("%s.%s"), *Code, Mask[NumComponents]);
				}
			}
		}

		if (bNeedClosingParen)
		{
			for (uint32 ComponentIndex = NumComponents; ComponentIndex < NumDestComponents; ++ComponentIndex)
			{
				if (ComponentIndex > 0u)
				{
					Result += TEXT(",");
				}
				if (bIsLWC)
				{
					if (!bReplicateScalar && ComponentIndex >= NumSourceComponents)
					{
						check(bAllowAppendZeroes);
						AddLWCFuncUsage(ELWCFunctionKind::Promote);
						Result += TEXT("WSPromote(0.0f)");
					}
					else
					{
						Result += FString::Printf(TEXT("WSGetComponent(%s, %d)"), *Code, bReplicateScalar ? 0 : ComponentIndex);
					}
				}
				else
				{
					// Non-LWC case should only be zero-filling here, other cases should have already been handled
					check(bAllowAppendZeroes);
					check(!bReplicateScalar);
					check(ComponentIndex >= NumSourceComponents);
					Result += TEXT("0.0f");
				}
			}
			NumComponents = NumDestComponents;
			Result += TEXT(")");
		}
		check(NumComponents == NumDestComponents);
		return Result;
	}

	// If the type came from a texture collection, we'll need to remove that flag to resolve the cast.
	if (SourceType & MCT_TextureCollection && SourceType != MCT_TextureCollection)
	{
		return CastValue(Code, EMaterialValueType(SourceType & ~MCT_TextureCollection), DestType, Flags);
	}

	Errorf(TEXT("Cannot cast between non-numeric types %s to %s."), DescribeType(SourceType), DescribeType(DestType));
	return FString();
}

/** Pushes a function onto the compiler's function stack, which indicates that compilation is entering a function. */
void FHLSLMaterialTranslator::PushFunction(FMaterialFunctionCompileState* FunctionState)
{
	check(ShaderFrequency < SF_NumFrequencies);
	auto& CurrentFunctionStack = FunctionStacks[ShaderFrequency];
	CurrentFunctionStack.Push(FunctionState);
}	

/** Pops a function from the compiler's function stack, which indicates that compilation is leaving a function. */
FMaterialFunctionCompileState* FHLSLMaterialTranslator::PopFunction()
{
	check(ShaderFrequency < SF_NumFrequencies);
	auto& CurrentFunctionStack = FunctionStacks[ShaderFrequency];
	return CurrentFunctionStack.Pop();
}

int32 FHLSLMaterialTranslator::GetCurrentFunctionStackDepth()
{
	check(ShaderFrequency < SF_NumFrequencies);
	auto& CurrentFunctionStack = FunctionStacks[ShaderFrequency];
	return CurrentFunctionStack.Num();
}

int32 FHLSLMaterialTranslator::AccessCollectionParameter(UMaterialParameterCollection* ParameterCollection, int32 ParameterIndex, int32 ComponentIndex)
{
	if (!ParameterCollection || ParameterIndex == -1)
	{
		return INDEX_NONE;
	}

	int32 CollectionIndex = ParameterCollections.Find(ParameterCollection);

	if (CollectionIndex == INDEX_NONE)
	{
		if (ParameterCollections.Num() >= MaxNumParameterCollectionsPerMaterial)
		{
			return Error(TEXT("Material references too many MaterialParameterCollections!  A material may only reference 2 different collections."));
		}

		ParameterCollections.Add(ParameterCollection);
		CollectionIndex = ParameterCollections.Num() - 1;
	}

	int32 VectorChunk = AddCodeChunkZeroDeriv(MCT_Float4,TEXT("MaterialCollection%u.Vectors[%u]"),CollectionIndex,ParameterIndex);

	return ComponentMask(VectorChunk, 
		ComponentIndex == -1 ? true : ComponentIndex % 4 == 0,
		ComponentIndex == -1 ? true : ComponentIndex % 4 == 1,
		ComponentIndex == -1 ? true : ComponentIndex % 4 == 2,
		ComponentIndex == -1 ? true : ComponentIndex % 4 == 3);
}

int32 FHLSLMaterialTranslator::CollectionTransform(int32 InputIndex, const TStaticArray<int32, 5>& CollectionParameters, EParameterCollectionTransformType TransformType)
{
	FString InputCode = GetParameterCode(InputIndex);
	EMaterialValueType InputType = GetParameterType(InputIndex);

	switch (TransformType)
	{
	case EParameterCollectionTransformType::Position:
	default:
		if (InputType == MCT_Float3 || InputType == MCT_LWCVector3)
		{
			// Treat input as a translation vector (w = 1)
			return AddCodeChunkZeroDeriv(MakeNonLWCType(InputType), TEXT("(mul(float4(WSDemote(%s), 1.0), float4x4(%s, %s, %s, %s))).xyz"),
				*InputCode,
				*GetParameterCode(CollectionParameters[0]),
				*GetParameterCode(CollectionParameters[1]),
				*GetParameterCode(CollectionParameters[2]),
				*GetParameterCode(CollectionParameters[3]));
		}
		else
		{
			// Caller should only pass in 3 or 4 element vector inputs
			check(InputType == MCT_Float4 || InputType == MCT_LWCVector4);

			// Treat input as a homogenous vector (w = user specified)
			return AddCodeChunkZeroDeriv(MCT_Float4, TEXT("(mul(WSDemote(%s), float4x4(%s, %s, %s, %s)))"),
				*InputCode,
				*GetParameterCode(CollectionParameters[0]),
				*GetParameterCode(CollectionParameters[1]),
				*GetParameterCode(CollectionParameters[2]),
				*GetParameterCode(CollectionParameters[3]));
		}
	case EParameterCollectionTransformType::Vector:
		// Treat input as a direction vector (w = 0)
		return AddCodeChunkZeroDeriv(MCT_Float3, TEXT("(mul(float4(WSDemote(%s).xyz, 0.0), float4x4(%s, %s, %s, float4(0.0, 0.0, 0.0, 0.0)))).xyz"),
			*InputCode,
			*GetParameterCode(CollectionParameters[0]),
			*GetParameterCode(CollectionParameters[1]),
			*GetParameterCode(CollectionParameters[2]));
	case EParameterCollectionTransformType::Projection:
		// Optimized to save many ALU for a standard perspective or orthographic projection matrix, where most of the elements of the matrix are zero.
		return AddCodeChunkZeroDeriv(MCT_Float4, TEXT("(WSDemote(%s).xyzz*float4(%s.x, %s.y, %s.zw) + float4(0.0, 0.0, %s.zw))"),
			*InputCode,
			*GetParameterCode(CollectionParameters[0]),
			*GetParameterCode(CollectionParameters[1]),
			*GetParameterCode(CollectionParameters[2]),
			*GetParameterCode(CollectionParameters[3]));
	case EParameterCollectionTransformType::LocalToWorld:
		AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix);
		return AddCodeChunkZeroDeriv(MakeLWCType(InputType), TEXT("WSMultiply(WSDemote(%s), MakeWSMatrix(%s.xyz, float4x4(%s, %s, %s, %s)))"),
			*InputCode,
			*GetParameterCode(CollectionParameters[4]),		// Tile offset
			*GetParameterCode(CollectionParameters[0]),
			*GetParameterCode(CollectionParameters[1]),
			*GetParameterCode(CollectionParameters[2]),
			*GetParameterCode(CollectionParameters[3]));
	case EParameterCollectionTransformType::WorldToLocal:
		AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix);
		return AddCodeChunkZeroDeriv(MakeNonLWCType(InputType), TEXT("WSMultiply(WSPromote(%s), MakeWSInverseMatrix(%s.xyz, float4x4(%s, %s, %s, %s)))"),
			*InputCode,
			*GetParameterCode(CollectionParameters[4]),		// Tile offset
			*GetParameterCode(CollectionParameters[0]),
			*GetParameterCode(CollectionParameters[1]),
			*GetParameterCode(CollectionParameters[2]),
			*GetParameterCode(CollectionParameters[3]));
	}
}

bool FHLSLMaterialTranslator::GetParameterOverrideValueForCurrentFunction(EMaterialParameterType ParameterType, FName ParameterName, FMaterialParameterMetadata& OutResult) const
{
	const TArray<FMaterialFunctionCompileState*>& FunctionStack = FunctionStacks[ShaderFrequency];

	// Give every function in the callstack on opportunity to override the parameter value
	// Parameters in outer functions take priority
	// For example, if a layer instance calls a function instance that includes an overriden parameter, we want to use the value from the layer instance rather than the function instance
	bool bResult = false;
	for (const FMaterialFunctionCompileState* FunctionState : FunctionStack)
	{
		const UMaterialFunctionInterface* CurrentFunction = (FunctionState && FunctionState->FunctionCall) ? FunctionState->FunctionCall->MaterialFunction.Get() : nullptr;
		if (CurrentFunction)
		{
			if (CurrentFunction->GetParameterOverrideValue(ParameterType, ParameterName, OutResult))
			{
				bResult = true;
				break;
			}
		}
	}

	return bResult;
}

int32 FHLSLMaterialTranslator::NumericParameter(EMaterialParameterType ParameterType, FName ParameterName, const UE::Shader::FValue& InDefaultValue)
{
	const UE::Shader::EValueType ValueType = GetShaderValueType(ParameterType);
	check(InDefaultValue.GetType() == ValueType);
	UE::Shader::FValue DefaultValue(InDefaultValue);

	// If we're compiling a function, give the function a chance to override the default parameter value
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValueForCurrentFunction(ParameterType, ParameterName, Meta))
	{
		DefaultValue = Meta.Value.AsShaderValue();
		check(DefaultValue.GetType() == ValueType);
	}

	const uint32* PrevDefaultOffset = DefaultUniformValues.Find(DefaultValue);
	uint32 DefaultOffset;
	if (PrevDefaultOffset)
	{
		DefaultOffset = *PrevDefaultOffset;
	}
	else
	{
		DefaultOffset = MaterialCompilationOutput.UniformExpressionSet.AddDefaultParameterValue(DefaultValue);
		DefaultUniformValues.Add(DefaultValue, DefaultOffset);
	}

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	FAddUniformExpressionScope Scope(this);
	const int32 ParameterIndex = MaterialCompilationOutput.UniformExpressionSet.FindOrAddNumericParameter(ParameterType, ParameterInfo, DefaultOffset);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionNumericParameter(ParameterInfo, ParameterIndex), GetMaterialValueType(ParameterType), TEXT(""));
}

int32 FHLSLMaterialTranslator::Constant(float X)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionConstant(FLinearColor(X,X,X,X),MCT_Float),MCT_Float,TEXT("%0.8ff"),X);
}

int32 FHLSLMaterialTranslator::Constant2(float X,float Y)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionConstant(FLinearColor(X,Y,0,0),MCT_Float2),MCT_Float2,TEXT("MaterialFloat2(%0.8ff,%0.8ff)"),X,Y);
}

int32 FHLSLMaterialTranslator::Constant3(float X,float Y,float Z)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionConstant(FLinearColor(X,Y,Z,0),MCT_Float3),MCT_Float3,TEXT("MaterialFloat3(%0.8ff,%0.8ff,%0.8ff)"),X,Y,Z);
}

int32 FHLSLMaterialTranslator::Constant4(float X,float Y,float Z,float W)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionConstant(FLinearColor(X,Y,Z,W),MCT_Float4),MCT_Float4,TEXT("MaterialFloat4(%0.8ff,%0.8ff,%0.8ff,%0.8ff)"),X,Y,Z,W);
}

int32 FHLSLMaterialTranslator::GenericConstant(const UE::Shader::FValue& Value)
{
	FAddUniformExpressionScope Scope(this);
	TStringBuilder<1024> String;
	Value.ToString(UE::Shader::EValueStringFormat::HLSL, String);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionGenericConstant(Value), GetMaterialValueType(Value.GetType()), String.ToString());
}
	
int32 FHLSLMaterialTranslator::ViewProperty(EMaterialExposedViewProperty Property, bool InvProperty)
{
	const FMaterialExposedViewPropertyMeta& PropertyMeta = MaterialExternalCodeRegistry::Get().GetExternalViewPropertyCode(Property);

	FString Code = FString{ PropertyMeta.PropertyCode };

	if (InvProperty && !PropertyMeta.InvPropertyCode.IsEmpty())
	{
		Code = FString{ PropertyMeta.InvPropertyCode };
	}

	// Resolved templated code
	Code.ReplaceInline(TEXT("<PREV>"), bCompilingPreviousFrame ? TEXT("Prev") : TEXT(""));
		
	if (InvProperty && PropertyMeta.InvPropertyCode.IsEmpty())
	{
		// fall back to compute the property's inverse from PropertyCode
		return Div(Constant(1.f), AddCodeChunkZeroDeriv(PropertyMeta.Type, *Code));
	}

	const int32 Result = AddCodeChunkZeroDeriv(PropertyMeta.Type, *Code);
	return CastToNonLWCIfDisabled(Result);
}


int32 FHLSLMaterialTranslator::IsOrthographic()
{
	return AddExternalCodeChunk(TEXT("IsOrthographic"));
}


int32 FHLSLMaterialTranslator::GameTime(bool bPeriodic, float Period)
{
	if (!bPeriodic)
	{
		return AddInlinedCodeChunkZeroDeriv(MCT_Float, bCompilingPreviousFrame ? TEXT("View.PrevFrameGameTime") : TEXT("View.GameTime"));
	}
	else if (Period == 0.0f)
	{
		return Constant(0.0f);
	}

	int32 PeriodChunk = Constant(Period);

	if (bCompilingPreviousFrame)
	{
		return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("fmod(View.PrevFrameGameTime,%s)"), *GetParameterCode(PeriodChunk));
	}

	// Note: not using FHLSLMaterialTranslator::Fmod(), which will emit MaterialFloat types which will be converted to fp16 on mobile.
	// We want full 32 bit float precision until the fmod when using a period.
	return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("fmod(View.GameTime,%s)"), *GetParameterCode(PeriodChunk));
}

int32 FHLSLMaterialTranslator::RealTime(bool bPeriodic, float Period)
{
	if (!bPeriodic)
	{
		if (bCompilingPreviousFrame)
		{
			return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("View.PrevFrameRealTime"));
		}

		return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("View.RealTime"));
	}
	else if (Period == 0.0f)
	{
		return Constant(0.0f);
	}

	int32 PeriodChunk = Constant(Period);

	if (bCompilingPreviousFrame)
	{
		return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("fmod(View.PrevFrameRealTime,%s)"), *GetParameterCode(PeriodChunk));
	}

	return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("fmod(View.RealTime,%s)"), *GetParameterCode(PeriodChunk));
}

int32 FHLSLMaterialTranslator::DeltaTime()
{
	// explicitly avoid trying to return previous frame's delta time for bCompilingPreviousFrame here
	// DeltaTime expression is designed to be used when generating custom motion vectors, by using world position offset along with previous frame switch
	// in this context, we will technically be evaluating the previous frame, but we want to use the current frame's delta tick in order to offset the vector used to create previous position
	return AddExternalCodeChunk(TEXT("DeltaTime"));
}

int32 FHLSLMaterialTranslator::PeriodicHint(int32 PeriodicCode)
{
	if(PeriodicCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(PeriodicCode))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionPeriodic(GetParameterUniformExpression(PeriodicCode)),GetParameterType(PeriodicCode),TEXT("%s"),*GetParameterCode(PeriodicCode));
	}
	else
	{
		return PeriodicCode;
	}
}

int32 FHLSLMaterialTranslator::Sine(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Sin),MCT_Float,TEXT("sin(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this,FMaterialDerivativeAutogen::EFunc1::Sin,X);
		}
		else
		{
		return AddCodeChunk(GetParameterType(X),TEXT("sin(%s)"),*GetParameterCode(X));
	}
}
}

int32 FHLSLMaterialTranslator::Cosine(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Cos),MCT_Float,TEXT("cos(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this,FMaterialDerivativeAutogen::EFunc1::Cos,X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X),TEXT("cos(%s)"),*GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Tangent(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Tan),MCT_Float,TEXT("tan(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Tan, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X),TEXT("tan(%s)"),*GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Arcsine(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Asin),MCT_Float,TEXT("asin(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Asin, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X),TEXT("asin(%s)"),*GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::ArcsineFast(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Asin),MCT_Float,TEXT("asinFast(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::AsinFast, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X), TEXT("asinFast(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Arccosine(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Acos),MCT_Float,TEXT("acos(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Acos, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X), TEXT("acos(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::ArccosineFast(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Acos),MCT_Float,TEXT("acosFast(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::AcosFast, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X), TEXT("acosFast(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Arctangent(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Atan),MCT_Float,TEXT("atan(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Atan, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X), TEXT("atan(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::ArctangentFast(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(X),TMO_Atan),MCT_Float,TEXT("atanFast(%s)"),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::AtanFast, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X), TEXT("atanFast(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Arctangent2(int32 Y, int32 X)
{
	if(Y == INDEX_NONE || X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(Y) && GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(Y),GetParameterUniformExpression(X),TMO_Atan2),MCT_Float,TEXT("atan2(%s, %s)"),*CoerceParameter(Y,MCT_Float),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Atan2, Y, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(Y), TEXT("atan2(%s, %s)"), *GetParameterCode(Y), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Arctangent2Fast(int32 Y, int32 X)
{
	if(Y == INDEX_NONE || X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(Y) && GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTrigMath(GetParameterUniformExpression(Y),GetParameterUniformExpression(X),TMO_Atan2),MCT_Float,TEXT("atan2Fast(%s, %s)"),*CoerceParameter(Y,MCT_Float),*CoerceParameter(X,MCT_Float));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Atan2Fast, Y, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(Y), TEXT("atan2Fast(%s, %s)"), *GetParameterCode(Y), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Floor(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFloor(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("floor(%s)"),*GetParameterCode(X));
	}
	else
	{
		const EMaterialValueType ValueType = GetParameterType(X);
		if (IsLWCType(ValueType))
		{
			AddLWCFuncUsage(ELWCFunctionKind::Other);
			return AddCodeChunkZeroDeriv(ValueType, TEXT("WSFloor(%s)"), *GetParameterCode(X));
		}
		else
		{
			return AddCodeChunkZeroDeriv(ValueType, TEXT("floor(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Ceil(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionCeil(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("ceil(%s)"),*GetParameterCode(X));
	}
	else
	{
		const EMaterialValueType ValueType = GetParameterType(X);
		if (IsLWCType(ValueType))
		{
			AddLWCFuncUsage(ELWCFunctionKind::Other);
			return AddCodeChunkZeroDeriv(ValueType, TEXT("WSCeil(%s)"), *GetParameterCode(X));
		}
		else
		{
			return AddCodeChunkZeroDeriv(ValueType, TEXT("ceil(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Round(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionRound(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("round(%s)"),*GetParameterCode(X));
	}
	else
	{
		const EMaterialValueType ValueType = GetParameterType(X);
		if (IsLWCType(ValueType))
		{
			AddLWCFuncUsage(ELWCFunctionKind::Other);
			return AddCodeChunkZeroDeriv(ValueType, TEXT("WSRound(%s)"), *GetParameterCode(X));
		}
		else
		{
			return AddCodeChunkZeroDeriv(ValueType, TEXT("round(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Truncate(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionTruncate(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("trunc(%s)"),*GetParameterCode(X));
	}
	else
	{
		const EMaterialValueType ValueType = GetParameterType(X);
		if (IsLWCType(ValueType))
		{
			AddLWCFuncUsage(ELWCFunctionKind::Other);
			return AddCodeChunkZeroDeriv(ValueType, TEXT("WSTrunc(%s)"), *GetParameterCode(X));
		}
		else
		{
			return AddCodeChunkZeroDeriv(ValueType, TEXT("trunc(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Sign(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionSign(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("sign(%s)"),*GetParameterCode(X));
	}
	else
	{
		const EMaterialValueType ValueType = GetParameterType(X);
		if (IsLWCType(ValueType))
		{
			AddLWCFuncUsage(ELWCFunctionKind::Other);
			return AddCodeChunkZeroDeriv(MakeNonLWCType(ValueType), TEXT("WSSign(%s)"), *GetParameterCode(X));
		}
		else
		{
			return AddCodeChunkZeroDeriv(ValueType, TEXT("sign(%s)"), *GetParameterCode(X));
		}
	}
}	

int32 FHLSLMaterialTranslator::Frac(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFrac(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("frac(%s)"),*GetParameterCode(X));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Frac, X);
		}
		else
		{
			const EMaterialValueType Type = GetParameterType(X);
			if (IsLWCType(Type))
			{
				AddLWCFuncUsage(ELWCFunctionKind::Other);
				return AddCodeChunk(MakeNonLWCType(Type), TEXT("WSFracDemote(%s)"), *GetParameterCode(X));
			}
			else
			{
				return AddCodeChunk(Type, TEXT("frac(%s)"), *GetParameterCode(X));
			}
		}
	}
}

int32 FHLSLMaterialTranslator::Fmod(int32 A, int32 B)
{
	if ((A == INDEX_NONE) || (B == INDEX_NONE))
	{
		return INDEX_NONE;
	}

	if (GetParameterUniformExpression(A) && GetParameterUniformExpression(B))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFmod(GetParameterUniformExpression(A),GetParameterUniformExpression(B)),
			GetParameterType(A),TEXT("fmod(%s,%s)"),*GetParameterCode(A),*CoerceParameter(B,GetParameterType(A)));
	}
	else
	{
		const FDerivInfo& BDerivInfo = GetDerivInfo(B);
		if (IsAnalyticDerivEnabled() && BDerivInfo.DerivativeStatus == EDerivativeStatus::Zero)
		{
			// Analytic derivatives only make sense when RHS derivatives are zero.
			return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Fmod, A, B);
		}
		else
		{
			return AddCodeChunk(GetParameterType(A), TEXT("fmod(%s,%s)"), *GetParameterCode(A), *CoerceParameter(B, GetParameterType(A)));
		}
		
	}
}

int32 FHLSLMaterialTranslator::Modulo(int32 A, int32 B)
{
	if ((A == INDEX_NONE) || (B == INDEX_NONE))
	{
		return INDEX_NONE;
	}

	if (GetParameterUniformExpression(A) && GetParameterUniformExpression(B))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionModulo(GetParameterUniformExpression(A),GetParameterUniformExpression(B)),
			GetParameterType(A),TEXT("(%s %% %s)"),*GetParameterCode(A),*CoerceParameter(B,GetParameterType(A), EMaterialCastFlags::AllowInteger));
	}

	const FDerivInfo& BDerivInfo = GetDerivInfo(B, true);
	if (IsAnalyticDerivEnabled() && BDerivInfo.DerivativeStatus == EDerivativeStatus::Zero)
	{
		// Analytic derivatives only make sense when RHS derivatives are zero.
		return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Modulo, A, B);
	}

	return AddCodeChunk(GetParameterType(A), TEXT("(%s %% %s)"), *GetParameterCode(A), *CoerceParameter(B, GetParameterType(A), EMaterialCastFlags::AllowInteger));
}

/**
* Creates the new shader code chunk needed for the Abs expression
*
* @param	X - Index to the FMaterialCompiler::CodeChunk entry for the input expression
* @return	Index to the new FMaterialCompiler::CodeChunk entry for this expression
*/	
int32 FHLSLMaterialTranslator::Abs(int32 X)
{
	if (X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionAbs(GetParameterUniformExpression(X)), GetParameterType(X), TEXT("abs(%s)"), *GetParameterCode(X));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Abs, X);
		}
		else
		{
			return AddCodeChunk(GetParameterType(X), TEXT("abs(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::ReflectionVector()
{
	if (ShaderFrequency != SF_Vertex)
	{
		bUsesTransformVector = true;
	}
	return AddExternalCodeChunk(TEXT("ReflectionVector"));
}

int32 FHLSLMaterialTranslator::ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal)
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonPixelShaderExpressionError();
	}

	if (CustomWorldNormal == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (ShaderFrequency != SF_Vertex)
	{
		bUsesTransformVector = true;
	}

	const TCHAR* ShouldNormalize = (!!bNormalizeCustomWorldNormal) ? TEXT("true") : TEXT("false");

	return AddCodeChunk(MCT_Float3,TEXT("ReflectionAboutCustomWorldNormal(Parameters, %s, %s)"), *CoerceParameter(CustomWorldNormal, MCT_Float3), ShouldNormalize);
}

int32 FHLSLMaterialTranslator::CameraVector()
{
	if (ShaderFrequency != SF_Vertex)
	{
		bUsesTransformVector = true;
	}
	return AddExternalCodeChunk(TEXT("CameraVector"));
}

int32 FHLSLMaterialTranslator::LightVector()
{
	return AddExternalCodeChunk(TEXT("LightVector"));
}

int32 FHLSLMaterialTranslator::GetViewportUV()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute && ShaderFrequency != SF_Vertex)
	{
		return Errorf(TEXT("GetViewportUV() node is only available in vertex or pixel shader input."));
	}

	FString FiniteCode = "GetViewportUV(Parameters)";
	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, TEXT("float2(View.ViewSizeAndInvSize.z, 0.0f)"), TEXT("float2(0.0f, View.ViewSizeAndInvSize.w)"), EDerivativeType::Float2);
		return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, MCT_Float2, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddCodeChunk(MCT_Float2, *FiniteCode);
	}
}

int32 FHLSLMaterialTranslator::GetPixelPosition()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute && ShaderFrequency != SF_Vertex)
	{
		return Errorf(TEXT("GetPixelPosition() node is only available in vertex or pixel shader input."));
	}

	FString FiniteCode = "GetPixelPosition(Parameters)";
	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, TEXT("float2(1.0f, 0.0f)"), TEXT("float2(0.0f, 1.0f)"), EDerivativeType::Float2);
		return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, MCT_Float2, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddCodeChunk(MCT_Float2, *FiniteCode);
	}
}

int32 FHLSLMaterialTranslator::ParticleMacroUV()
{
	return AddExternalCodeChunk(TEXT("ParticleMacroUV"));
}

int32 FHLSLMaterialTranslator::ParticleSubUV(int32 TextureIndex, EMaterialSamplerType SamplerType, int32 MipValue0Index, int32 MipValue1Index, ETextureMipValueMode MipValueMode, bool bBlend)
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonPixelShaderExpressionError();
	}

	if (TextureIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	int32 ParticleSubUV;
	const FString TexCoordCode = "Parameters.Particle.SubUVCoords[%u].xy";
	const int32 TexCoord1 = AddCodeChunk(MCT_Float2,*TexCoordCode,0);

	if(bBlend)
	{
		// Out	 = linear interpolate... using 2 sub-images of the texture
		// A	 = RGB sample texture with Parameters.Particle.SubUVCoords[0]
		// B	 = RGB sample texture with Parameters.Particle.SubUVCoords[1]
		// Alpha = Parameters.Particle.SubUVLerp

		const int32 TexCoord2 = AddCodeChunk( MCT_Float2,*TexCoordCode,1);
		const int32 SubImageLerp = AddCodeChunk(MCT_Float, TEXT("Parameters.Particle.SubUVLerp"));

		const int32 TexSampleA = TextureSample(TextureIndex, TexCoord1, SamplerType, MipValue0Index, MipValue1Index, MipValueMode);
		const int32 TexSampleB = TextureSample(TextureIndex, TexCoord2, SamplerType, MipValue0Index, MipValue1Index, MipValueMode);
		ParticleSubUV = Lerp( TexSampleA,TexSampleB, SubImageLerp);
	} 
	else
	{
		ParticleSubUV = TextureSample(TextureIndex, TexCoord1, SamplerType, MipValue0Index, MipValue1Index, MipValueMode);
	}
	
	bUsesParticleSubUVs = true;
	return ParticleSubUV;
}

int32 FHLSLMaterialTranslator::ParticleSubUVProperty(int32 PropertyIndex)
{
	int32 Result = INDEX_NONE;
	switch (PropertyIndex)
	{
	case 0:
		Result = AddExternalCodeChunk(TEXT("ParticleSubUVCoords0"));
		break;
	case 1:
		Result = AddExternalCodeChunk(TEXT("ParticleSubUVCoords1"));
		break;
	case 2:
		Result = AddExternalCodeChunk(TEXT("ParticleSubUVLerp"));
		break;
	default:
		checkNoEntry();
		break;
	}

	bUsesParticleSubUVs = true;
	return Result;
}

int32 FHLSLMaterialTranslator::ParticleColor()
{
	bUsesParticleColor |= (ShaderFrequency != SF_Vertex);
	return AddExternalCodeChunk(TEXT("ParticleColor"));
}

int32 FHLSLMaterialTranslator::ParticlePosition(EPositionOrigin OriginType)
{
	if (ShaderFrequency != SF_Vertex && ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonVertexOrPixelShaderExpressionError();
	}
	bNeedsParticlePosition = true;

	const TCHAR* TranslatedWP = TEXT("Parameters.Particle.TranslatedWorldPositionAndSize.xyz");
	const TCHAR* AbsoluteWP = TEXT("GetParticleWorldPosition(Parameters)");
	if ( bCompilingPreviousFrame && ShaderFrequency == SF_Vertex )
	{
		TranslatedWP = TEXT("Parameters.Particle.PrevTranslatedWorldPositionAndSize.xyz");
		AbsoluteWP = TEXT("GetPrevParticleWorldPosition(Parameters)");
	}

	if (OriginType == EPositionOrigin::CameraRelative)
	{
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TranslatedWP);
	}
	else
	{
		AddLWCFuncUsage(ELWCFunctionKind::Subtract);
		const int32 Result = AddInlinedCodeChunkZeroDeriv(MCT_LWCVector3, AbsoluteWP);
		return CastToNonLWCIfDisabled(Result);
	}
}

int32 FHLSLMaterialTranslator::ParticleRadius()
{
	bNeedsParticlePosition = true;
	return AddExternalCodeChunk(TEXT("ParticleRadius"));
}

int32 FHLSLMaterialTranslator::SphericalParticleOpacity(int32 Density)
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonPixelShaderExpressionError();
	}

	if (Density == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	bNeedsParticlePosition = true;
	bUsesSphericalParticleOpacity = true;
	bNeedsWorldPositionExcludingShaderOffsets = true;
	bUsesSceneDepth = true;
	return AddCodeChunk(MCT_Float, TEXT("GetSphericalParticleOpacity(Parameters,%s)"), *GetParameterCode(Density));
}

int32 FHLSLMaterialTranslator::ParticleRelativeTime()
{
	bNeedsParticleTime = true;
	return AddExternalCodeChunk(TEXT("ParticleRelativeTime"));
}

int32 FHLSLMaterialTranslator::ParticleMotionBlurFade()
{
	bUsesParticleMotionBlur = true;
	return AddExternalCodeChunk(TEXT("ParticleMotionBlurFade"));
}

int32 FHLSLMaterialTranslator::ParticleRandom()
{
	bNeedsParticleRandom = true;
	return AddExternalCodeChunk(TEXT("ParticleRandom"));
}


int32 FHLSLMaterialTranslator::ParticleDirection()
{
	bNeedsParticleVelocity = true;
	return AddExternalCodeChunk(TEXT("ParticleDirection"));
}

int32 FHLSLMaterialTranslator::ParticleSpeed()
{
	bNeedsParticleVelocity = true;
	return AddExternalCodeChunk(TEXT("ParticleSpeed"));
}

int32 FHLSLMaterialTranslator::ParticleSize()
{
	bNeedsParticleSize = true;
	return AddExternalCodeChunk(TEXT("ParticleSize"));
}

int32 FHLSLMaterialTranslator::ParticleSpriteRotation()
{
	bNeedsParticleSpriteRotation = true;
	return AddExternalCodeChunk(TEXT("ParticleSpriteRotation"));
}

int32 FHLSLMaterialTranslator::LocalPosition(EPositionIncludedOffsets IncludedOffsets, ELocalPositionOrigin OriginType)
{
	if (OriginType == ELocalPositionOrigin::InstancePreSkinning)
	{
		return AddExternalCodeChunk(TEXT("PreSkinnedPosition"));
	}

	// If compiling for the previous frame in the vertex shader
	bool bPreviousIsAvailable = ShaderFrequency == SF_Vertex;
	const TCHAR* PrevPart = (bCompilingPreviousFrame && bPreviousIsAvailable) ? TEXT("Prev") : TEXT("");

	const TCHAR* OriginPart = nullptr;
	switch (OriginType)
	{
		case ELocalPositionOrigin::Instance:
		{
			OriginPart = TEXT("Instance");
			break;
		}
		case ELocalPositionOrigin::Primitive:
		{
			OriginPart = TEXT("Primitive");
			break;
		}
		default: checkNoEntry();
	}

	const TCHAR* NoMaterialOffsetsPart = nullptr;
	switch (IncludedOffsets)
	{
		case EPositionIncludedOffsets::IncludeOffsets:
		{
			NoMaterialOffsetsPart = TEXT("");
			break;
		}
		case EPositionIncludedOffsets::ExcludeOffsets:
		{
			// No material offset only available in the pixel shader.
			if (ShaderFrequency == SF_Pixel)
			{
				NoMaterialOffsetsPart = TEXT("_NoMaterialOffsets");
			}
			else
			{
				NoMaterialOffsetsPart = TEXT("");
			}
			break;
		}
		default: checkNoEntry();
	}

	bUsesInstanceWorldToLocalPS |= (ShaderFrequency == SF_Pixel && OriginType == ELocalPositionOrigin::Instance);

	TArray<FStringFormatArg> FormatArgs =
	{
		PrevPart,
		OriginPart,
		NoMaterialOffsetsPart,
	};
	FString FiniteCode = FString::Format(TEXT("Get{0}Position{1}Space{2}(Parameters)"), FormatArgs);

	int32 Result = INDEX_NONE;
	if (IsAnalyticDerivEnabled())
	{
		// Generate derivative based on WorldPosition_DDX
		int32 WorldPositionCode = WorldPosition(IncludedOffsets == EPositionIncludedOffsets::IncludeOffsets ? WPT_CameraRelative : WPT_CameraRelativeNoOffsets);
		FTransformParameters Parameters{};
		int32 TransformedPositionCode = TransformPosition(MCB_TranslatedWorld, OriginType == ELocalPositionOrigin::Instance ? MCB_Instance : MCB_Local, Parameters, WorldPositionCode);
		if (TransformedPositionCode < 0) // TransformPosition failed
		{
			return TransformedPositionCode;
		}
		FString TransformedPositionAnalyticCode = GetParameterCodeDeriv(TransformedPositionCode, CompiledPDV_Analytic);

		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(
			*FiniteCode, 
			*(TransformedPositionAnalyticCode + TEXT(".Ddx")), 
			*(TransformedPositionAnalyticCode + TEXT(".Ddy")), 
			GetDerivType(MCT_Float3));
		Result = AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, MCT_Float3, false, EDerivativeStatus::Valid);
	}
	else
	{
		Result = AddInlinedCodeChunk(MCT_Float3, *FiniteCode);
	}

	return Result;
}

int32 FHLSLMaterialTranslator::WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets)
{
	// If compiling for the previous frame in the vertex shader
	bool bPreviousIsAvailable = ShaderFrequency == SF_Vertex;
	const TCHAR* PrevPart = (bCompilingPreviousFrame && bPreviousIsAvailable) ? TEXT("Prev") : TEXT("");

	bool bNoMaterialOffsetsIsAvailable = ShaderFrequency == SF_Pixel;

	// If this material has no expressions for world position offset or world displacement, the non-offset world position will
	// be exactly the same as the offset one, so there is no point bringing in the extra code.
	// Also, we can't access the full offset world position in anything other than the pixel shader, because it won't have
	// been calculated yet
	const TCHAR* OriginPart = nullptr;
	const TCHAR* NoMaterialOffsetsPart = nullptr;
	EMaterialValueType Type = (EMaterialValueType)0;
	switch (WorldPositionIncludedOffsets)
	{
	case WPT_Default:
		{
			OriginPart = TEXT("World");
			NoMaterialOffsetsPart = TEXT("");
			Type = MCT_LWCVector3;
			break;
		}

	case WPT_ExcludeAllShaderOffsets:
		{
			bNeedsWorldPositionExcludingShaderOffsets |= (ShaderFrequency == SF_Pixel);
			OriginPart = TEXT("World");
			NoMaterialOffsetsPart = bNoMaterialOffsetsIsAvailable ? TEXT("_NoMaterialOffsets") : TEXT("");
			Type = MCT_LWCVector3;
			break;
		}

	case WPT_CameraRelative:
		{
			OriginPart = TEXT("TranslatedWorld");
			NoMaterialOffsetsPart = TEXT("");
			Type = MCT_Float3;
			break;
		}

	case WPT_CameraRelativeNoOffsets:
		{
			bNeedsWorldPositionExcludingShaderOffsets |= (ShaderFrequency == SF_Pixel);
			OriginPart = TEXT("TranslatedWorld");
			NoMaterialOffsetsPart = bNoMaterialOffsetsIsAvailable ? TEXT("_NoMaterialOffsets") : TEXT("");
			Type = MCT_Float3;
			break;
		}

	default:
		{
			Errorf(TEXT("Encountered unknown world position type '%d'"), WorldPositionIncludedOffsets);
			return INDEX_NONE;
		}
	}
	
	bUsesVertexPosition = true;
	// In PP material WorldPosition is reconstructed from SceneDepth
	bUsesSceneDepth |= (ShaderFrequency == SF_Pixel && Material->GetMaterialDomain() == MD_PostProcess);

	TArray<FStringFormatArg> FormatArgs =
	{
		PrevPart,
		OriginPart,
		NoMaterialOffsetsPart,
	};
	FString FiniteCode = FString::Format(TEXT("Get{0}{1}Position{2}(Parameters)"), FormatArgs);

	int32 Result = INDEX_NONE;
	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, TEXT("Parameters.WorldPosition_DDX"), TEXT("Parameters.WorldPosition_DDY"), GetDerivType(Type));
		Result = AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, Type, false, EDerivativeStatus::Valid);
	}
	else
	{
		Result = AddInlinedCodeChunk(Type, *FiniteCode);
	}

	return CastToNonLWCIfDisabled(Result);
}

int32 FHLSLMaterialTranslator::ObjectWorldPosition(EPositionOrigin OriginType)
{
	if (OriginType == EPositionOrigin::CameraRelative)
	{
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3,TEXT("GetObjectTranslatedWorldPosition(Parameters)"));
	}
	else
	{
		const int32 Result = AddInlinedCodeChunkZeroDeriv(MCT_LWCVector3,TEXT("GetObjectWorldPosition(Parameters)"));
		return CastToNonLWCIfDisabled(Result);
	}
}

int32 FHLSLMaterialTranslator::ObjectRadius()
{
	return AddExternalCodeChunk(TEXT("ObjectRadius"));
}

int32 FHLSLMaterialTranslator::ObjectBounds()
{
	return AddExternalCodeChunk(TEXT("ObjectBounds"));
}

int32 FHLSLMaterialTranslator::ObjectLocalBounds(int32 OutputIndex)
{
	if (!CheckPrimitivePropertyCompatibity(ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return INDEX_NONE;
	}

	switch (OutputIndex)
	{
	case 0: // Half extents
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("((GetPrimitiveData(Parameters).LocalObjectBoundsMax - GetPrimitiveData(Parameters).LocalObjectBoundsMin) / 2.0f)"));
	case 1: // Full extents
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("(GetPrimitiveData(Parameters).LocalObjectBoundsMax - GetPrimitiveData(Parameters).LocalObjectBoundsMin)"));
	case 2: // Min point
		return GetPrimitiveProperty(MCT_Float3, TEXT("ObjectLocalBounds"), TEXT("LocalObjectBoundsMin"));
	case 3: // Max point
		return GetPrimitiveProperty(MCT_Float3, TEXT("ObjectLocalBounds"), TEXT("LocalObjectBoundsMax"));
	default:
		check(false);
	}

	return INDEX_NONE; 
}

int32 FHLSLMaterialTranslator::InstanceLocalBounds(int32 OutputIndex)
{
	if (!CheckPrimitivePropertyCompatibity(ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return INDEX_NONE;
	}

	switch (OutputIndex)
	{
	case 0: // Half extents
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("(GetPrimitiveData(Parameters).InstanceLocalBoundsExtent)"));
	case 1: // Full extents
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("(GetPrimitiveData(Parameters).InstanceLocalBoundsExtent * 2.0f)"));
	case 2: // Min point
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("(GetPrimitiveData(Parameters).InstanceLocalBoundsCenter - GetPrimitiveData(Parameters).InstanceLocalBoundsExtent)"));
	case 3: // Max point
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("(GetPrimitiveData(Parameters).InstanceLocalBoundsCenter + GetPrimitiveData(Parameters).InstanceLocalBoundsExtent)"));
	default:
		check(false);
	}

	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::PreSkinnedLocalBounds(int32 OutputIndex)
{
	if (!CheckPrimitivePropertyCompatibity(ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return INDEX_NONE;
	}

	switch (OutputIndex)
	{
	case 0: // Half extents
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("((GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMax - GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMin) / 2.0f)"));
	case 1: // Full extents
		return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("(GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMax - GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMin)"));
	case 2: // Min point
		return GetPrimitiveProperty(MCT_Float3, TEXT("PreSkinnedLocalBounds"), TEXT("PreSkinnedLocalBoundsMin"));
	case 3: // Max point
		return GetPrimitiveProperty(MCT_Float3, TEXT("PreSkinnedLocalBounds"), TEXT("PreSkinnedLocalBoundsMax"));
	default:
		check(false);
	}

	return INDEX_NONE; 
}

int32 FHLSLMaterialTranslator::DistanceCullFade()
{
	bUsesDistanceCullFade = true;

	return AddInlinedCodeChunk(MCT_Float,TEXT("GetDistanceCullFade()"));		
}

int32 FHLSLMaterialTranslator::ActorWorldPosition(EPositionOrigin OriginType)
{
	if (OriginType == EPositionOrigin::CameraRelative)
	{
		if (bCompilingPreviousFrame && ShaderFrequency == SF_Vertex)
		{
			// Decal VS doesn't have material code so FMaterialVertexParameters
			// and primitve uniform buffer are guaranteed to exist if ActorPosition
			// material node is used in VS
			AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix, 2);
			return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("GetPreviousActorTranslatedWorldPosition(Parameters)"));
		}
		else
		{
			return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("GetActorTranslatedWorldPosition(Parameters)"));
		}
	}
	else
	{
		int32 Result = INDEX_NONE;
		if (bCompilingPreviousFrame && ShaderFrequency == SF_Vertex)
		{
			// Decal VS doesn't have material code so FMaterialVertexParameters
			// and primitive uniform buffer are guaranteed to exist if ActorPosition
			// material node is used in VS
			AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix, 2);
			Result = AddInlinedCodeChunkZeroDeriv(MCT_LWCVector3, TEXT("GetPreviousActorWorldPosition(Parameters)"));
		}
		else
		{
			Result = AddInlinedCodeChunkZeroDeriv(MCT_LWCVector3, TEXT("GetActorWorldPosition(Parameters)"));
		}

		return CastToNonLWCIfDisabled(Result);
	}
}

int32 FHLSLMaterialTranslator::DynamicBranch(int32 Condition, int32 A, int32 B)
{
	if (Condition == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (B == INDEX_NONE)
	{
		return A;
	}

	if (A == INDEX_NONE)
	{
		return B;
	}

	if (A == B)
	{
		return A;
	}

	EMaterialValueType TypeA = GetParameterType(A);
	EMaterialValueType TypeB = GetParameterType(B);
	if (IsLWCType(TypeA) || IsLWCType(TypeB))
	{
		TypeA = MakeLWCType(TypeA);
		TypeB = MakeLWCType(TypeB);
	}

	EMaterialValueType ResultType = MCT_Unknown;
	if (TypeA == TypeB)
	{
		ResultType = TypeA;
	}
	else if (!IsFloatNumericType(TypeA) || !IsFloatNumericType(TypeB))
	{
		Errorf(TEXT("Cannot branch on non float numeric Types if they are not equal: %s %s"), DescribeType(TypeA), DescribeType(TypeB));
		return INDEX_NONE;
	}
	else
	{
		ResultType = GetNumComponents(TypeA) > GetNumComponents(TypeB) ? TypeA : TypeB;
	}

	A = ForceCast(A, ResultType, MFCF_ReplicateValue);
	B = ForceCast(B, ResultType, MFCF_ReplicateValue);

	checkf(Condition >= 0 && Condition < CurrentScopeChunks->Num(), TEXT("Index %d/%d, Platform=%d"), Condition, CurrentScopeChunks->Num(), (int)Platform);
	const FShaderCodeChunk& CodeChunk = (*CurrentScopeChunks)[Condition];

	if (CodeChunk.UniformExpression && CodeChunk.UniformExpression->GetType() == &FMaterialUniformExpressionStaticBoolParameter::StaticType)
	{
		FMaterialUniformExpressionStaticBoolParameter* StaticBoolParameter = static_cast<FMaterialUniformExpressionStaticBoolParameter*>(CodeChunk.UniformExpression.GetReference());
		AddCodeChunk(MCT_VoidStatement, TEXT("//%s"), *StaticBoolParameter->GetParameterName().ToString());
	}

	const FString ConditionCode = GetParameterCode(Condition);
	if (ConditionCode == TEXT("true"))
	{
		const FString ThenBranchCode = GetParameterCode(A);
		return AddCodeChunk(ResultType, TEXT("%s"), *ThenBranchCode);
	}
	else if (ConditionCode == TEXT("false"))
	{
		// Minor constant folding optimization: Only generate 
		const FString ElseBranchCode = GetParameterCode(B);
		return AddCodeChunk(ResultType, TEXT("%s"), *ElseBranchCode);
	}
	else if ((ResultType & MCT_Float) != 0)
	{
		// Use lerp() intrinsic for floating-point values to avoid dynamic branching; Here the then/else branches are reversed!
		const FString ThenBranchCode = GetParameterCode(A);
		const FString ElseBranchCode = GetParameterCode(B);
		return AddCodeChunk(ResultType, TEXT("lerp(%s, %s, %s)"), *ElseBranchCode, *ThenBranchCode, *ConditionCode);
	}
	else if ((ResultType & (MCT_UInt | MCT_Bool)) != 0)
	{
		// Use ternary-operator to simplify output for dynamic branch of two numerical values
		const FString ThenBranchCode = GetParameterCode(A);
		const FString ElseBranchCode = GetParameterCode(B);
		return AddCodeChunk(ResultType, TEXT("%s ? %s : %s"), *ConditionCode, *ThenBranchCode, *ElseBranchCode);
	}
	else
	{
		// Fallback to switch-case statement for then/else branches for all other types
		const FString SymbolName = CreateSymbolName(TEXT("Static"));
		const FString ThenBranchCode = GetParameterCode(A);
		const FString ElseBranchCode = GetParameterCode(B);
		AddCodeChunk(MCT_VoidStatement, TEXT("%s %s;"), HLSLTypeString(ResultType), *SymbolName);
		AddCodeChunk(MCT_VoidStatement, TEXT("[branch] switch (int(%s)){ default: %s = %s; break; case 0: %s = %s; break;}"), *ConditionCode, *SymbolName, *ThenBranchCode, *SymbolName, *ElseBranchCode);
		return AddCodeChunk(ResultType, *SymbolName);
	}
}

// Compare two inputs and return true if they are either the same, or if they evaluate to equal constant expressions.
static bool AreEqualExpressions(int32 A, int32 B, TArray<FShaderCodeChunk> const& InCurrentScopeChunks, FMaterial const& InMaterial)
{
	if (A == B)
	{
		return true;
	}
	if (A == -1 || B == -1)
	{
		return false;
	}
	
	FShaderCodeChunk const& ChunkA = InCurrentScopeChunks[A];
	FShaderCodeChunk const& ChunkB = InCurrentScopeChunks[B];

	if (ChunkA.UniformExpression != nullptr && ChunkB.UniformExpression != nullptr)
	{
		if (ChunkA.UniformExpression == ChunkB.UniformExpression)
		{
			return true;
		}
		if (ChunkA.UniformExpression->IsConstant() && ChunkB.UniformExpression->IsConstant() && ChunkA.Type == ChunkB.Type)
		{
			FMaterialRenderContext DummyContext(nullptr, InMaterial, nullptr);
			FLinearColor ValueA, ValueB;
			ChunkA.UniformExpression->GetNumberValue(DummyContext, ValueA);
			ChunkB.UniformExpression->GetNumberValue(DummyContext, ValueB);
			if (ValueA == ValueB)
			{
				return true;
			}
		}
	}

	return false;
}

int32 FHLSLMaterialTranslator::If(int32 A,int32 B,int32 AGreaterThanB,int32 AEqualsB,int32 ALessThanB,int32 ThresholdArg)
{
	if (A == INDEX_NONE || B == INDEX_NONE || AGreaterThanB == INDEX_NONE || ALessThanB == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (AreEqualExpressions(AGreaterThanB, ALessThanB, *CurrentScopeChunks, *Material))
	{
		if (AEqualsB == INDEX_NONE || AreEqualExpressions(AGreaterThanB, AEqualsB, *CurrentScopeChunks, *Material))
		{
			// All inputs are considered equal. So simply return one of them.
			return AGreaterThanB;
		}
	}

	if (IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateIfFunc(*this, A, B, AGreaterThanB, AEqualsB, ALessThanB, ThresholdArg);
	}
	
	if (AEqualsB != INDEX_NONE)
	{
		if (ThresholdArg == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		EMaterialValueType ResultType = GetArithmeticResultType(GetParameterType(AGreaterThanB),GetArithmeticResultType(AEqualsB,ALessThanB));

		int32 CoercedAGreaterThanB = ForceCast(AGreaterThanB,ResultType);
		int32 CoercedAEqualsB = ForceCast(AEqualsB,ResultType);
		int32 CoercedALessThanB = ForceCast(ALessThanB,ResultType);

		if(CoercedAGreaterThanB == INDEX_NONE || CoercedAEqualsB == INDEX_NONE || CoercedALessThanB == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		return AddCodeChunk(
			ResultType,
			TEXT("select(abs(%s - %s) > %s, select(%s >= %s, %s, %s), %s)"),
			*GetParameterCode(A),
			*GetParameterCode(B),
			*GetParameterCode(ThresholdArg),
			*GetParameterCode(A),
			*GetParameterCode(B),
			*GetParameterCode(CoercedAGreaterThanB),
			*GetParameterCode(CoercedALessThanB),
			*GetParameterCode(CoercedAEqualsB)
			);
	}
	else
	{
		EMaterialValueType ResultType = GetArithmeticResultType(AGreaterThanB,ALessThanB);

		int32 CoercedAGreaterThanB = ForceCast(AGreaterThanB,ResultType);
		int32 CoercedALessThanB = ForceCast(ALessThanB,ResultType);

		if(CoercedAGreaterThanB == INDEX_NONE || CoercedALessThanB == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		return AddCodeChunk(
			ResultType,
			TEXT("select(%s >= %s, %s, %s)"),
			*GetParameterCode(A),
			*GetParameterCode(B),
			*GetParameterCode(CoercedAGreaterThanB),
			*GetParameterCode(CoercedALessThanB)
			);
	}
}

void FHLSLMaterialTranslator::AllocateSlot(TBitArray<>& InBitArray, int32 InSlotIndex, int32 InSlotCount) const
{
	// Grow as needed
	int32 NumSlotsNeeded = InSlotIndex + InSlotCount;
	int32 CurrentNumSlots = InBitArray.Num();
	if(NumSlotsNeeded > CurrentNumSlots)
	{
		InBitArray.Add(false, NumSlotsNeeded - CurrentNumSlots);
	}

	// Allocate the requested slot(s)
	for (int32 i = InSlotIndex; i < NumSlotsNeeded; ++i)
	{
		InBitArray[i] = true;
	}
}

#if WITH_EDITOR
int32 FHLSLMaterialTranslator::MaterialBakingWorldPosition()
{
	if (ShaderFrequency == SF_Vertex)
	{
		AllocateSlot(AllocatedUserVertexTexCoords, 6, 2);
	}
	else
	{
		AllocateSlot(AllocatedUserTexCoords, 6, 2);
	}

	// Note: inlining is important so that on GLES devices, where half precision is used in the pixel shader, 
	// The UV does not get assigned to a half temporary in cases where the texture sample is done directly from interpolated UVs
	return AddInlinedCodeChunk(MCT_Float3, TEXT("float3(Parameters.TexCoords[6].x, Parameters.TexCoords[6].y, Parameters.TexCoords[7].x)"));
}
#endif
		

int32 FHLSLMaterialTranslator::TextureCoordinate(uint32 CoordinateIndex, bool UnMirrorU, bool UnMirrorV)
{
	const uint32 MaxNumCoordinates = 8;

	if (CoordinateIndex >= MaxNumCoordinates)
	{
		return Errorf(TEXT("Only %u texture coordinate sets can be used by this feature level, currently using %u"), MaxNumCoordinates, CoordinateIndex + 1);
	}

	if (ShaderFrequency == SF_Vertex)
	{
		AllocateSlot(AllocatedUserVertexTexCoords, CoordinateIndex);
	}
	else
	{
		AllocateSlot(AllocatedUserTexCoords, CoordinateIndex);
	}

	FString TexCoordCode = FString::Printf(TEXT("Parameters.TexCoords[%u].xy"), CoordinateIndex);
	FString	SampleCodeFinite = TexCoordCode;
	
	if (UnMirrorU || UnMirrorV)
	{
		SampleCodeFinite = FString::Printf(TEXT("%s(%s, Parameters)"), (UnMirrorU && UnMirrorV) ? TEXT("UnMirrorUV") : (UnMirrorU ? TEXT("UnMirrorU") : TEXT("UnMirrorV")), *SampleCodeFinite);
	}

	if (IsAnalyticDerivEnabled())
	{
		FString TexCoordCodeDDX = FString::Printf(TEXT("Parameters.TexCoords_DDX[%u].xy"), CoordinateIndex);
		FString TexCoordCodeDDY = FString::Printf(TEXT("Parameters.TexCoords_DDY[%u].xy"), CoordinateIndex);
		FString TexCoordCodeAnalytic = DerivativeAutogen.ConstructDeriv(TexCoordCode, TexCoordCodeDDX, TexCoordCodeDDY, EDerivativeType::Float2);
		FString SampleCodeAnalytic = DerivativeAutogen.ApplyUnMirror(TexCoordCodeAnalytic, UnMirrorU, UnMirrorV);

		return AddCodeChunkInnerDeriv(*SampleCodeFinite, *SampleCodeAnalytic, MCT_Float2, false, EDerivativeStatus::Valid);
	}
	else
	{
	// Note: inlining is important so that on GLES devices, where half precision is used in the pixel shader, 
	// The UV does not get assigned to a half temporary in cases where the texture sample is done directly from interpolated UVs
		return AddInlinedCodeChunk(MCT_Float2, *SampleCodeFinite, CoordinateIndex);
	}
}

void FHLSLMaterialTranslator::SetPotentiallyManipulateTexCoords()
{
	// This cannot be set from FHLSLMaterialTranslator::TextureCoordinate because it is called by texture sampling node for default UVs as coordinate index 0 for instance.
	bPotentiallyManipulateTexCoords = true;
}

uint32 FHLSLMaterialTranslator::AcquireVTStackIndex(
	const FMaterialVTSampleInfo& SampleInfo,
	const FString& UV_Value,
	const FString& UV_Ddx,
	const FString& UV_Ddy)
{
	using namespace UE::MaterialTranslatorUtils;

	const uint64 CoordinateHash = GetParameterHash(SampleInfo.CoordinateIndex);
	const uint64 MipValue0Hash = GetParameterHash(SampleInfo.MipValue0Index);
	const uint64 MipValue1Hash = GetParameterHash(SampleInfo.MipValue1Index);
	const uint64 PageTableHash = GetParameterHash(SampleInfo.PageTableIndex);
	const uint64 PageTableUniformHash = GetParameterHash(SampleInfo.PageTableUniformIndex);
	
	// First check to see if we have an existing VTStack that matches this key, that can still fit another layer
	for (int32 Index = 0; Index < VTStacks.Num(); ++Index)
	{
		const FMaterialVirtualTextureStack& Stack = MaterialCompilationOutput.UniformExpressionSet.VTStacks[Index];
		const FMaterialVTStackEntry& Entry = VTStacks[Index];
		
		const bool bStackIsFull = Stack.AreLayersFull() && Stack.FindLayer(SampleInfo.VirtualTextureIndex) == INDEX_NONE;
		
		if (!bStackIsFull &&
			Entry.ScopeID == CurrentScopeID &&
			Entry.ShaderFrequency == ShaderFrequency &&
			Entry.CoordinateHash == CoordinateHash &&
			Entry.MipValue0Hash == MipValue0Hash &&
			Entry.MipValue1Hash == MipValue1Hash &&
			Entry.PageTableHash == PageTableHash &&
			Entry.PageTableUniformHash == PageTableUniformHash &&
			Entry.SampleInfo.MipValueMode == SampleInfo.MipValueMode &&
			Entry.SampleInfo.AddressU == SampleInfo.AddressU &&
			Entry.SampleInfo.AddressV == SampleInfo.AddressV &&
			Entry.SampleInfo.AspectRatio == SampleInfo.AspectRatio &&
			Entry.SampleInfo.PreallocatedStackTextureIndex == SampleInfo.PreallocatedStackTextureIndex &&
			Entry.SampleInfo.bAdaptive == SampleInfo.bAdaptive &&
			Entry.SampleInfo.bGenerateFeedback == SampleInfo.bGenerateFeedback &&
			Entry.SampleInfo.bMeshPaint == SampleInfo.bMeshPaint &&
			Entry.SampleInfo.bMaterialCache == SampleInfo.bMaterialCache &&
			Entry.SampleInfo.bCollection == SampleInfo.bCollection)
		{
			return Index;
		}
	}

	// Need to allocate a new VTStack
	const int32 StackIndex = VTStacks.AddDefaulted();
	FMaterialVTStackEntry& Entry = VTStacks[StackIndex];
	Entry.SampleInfo = SampleInfo;
	Entry.ScopeID = CurrentScopeID;
	Entry.CoordinateHash = CoordinateHash;
	Entry.MipValue0Hash = MipValue0Hash;
	Entry.MipValue1Hash = MipValue1Hash;
	Entry.PageTableHash = PageTableHash;
	Entry.PageTableUniformHash = PageTableUniformHash;
	Entry.ShaderFrequency = ShaderFrequency;

	MaterialCompilationOutput.UniformExpressionSet.VTStacks.Add(FMaterialVirtualTextureStack(SampleInfo.PreallocatedStackTextureIndex));

	// These two arrays need to stay in sync
	check(VTStacks.Num() == MaterialCompilationOutput.UniformExpressionSet.VTStacks.Num());

	// Select LoadVirtualPageTable function name for this context
	FString BaseFunctionName = SampleInfo.bAdaptive ? TEXT("TextureLoadVirtualPageTableAdaptive") : TEXT("TextureLoadVirtualPageTable");

	// Optionally sample without virtual texture feedback but only for miplevel mode
	check(SampleInfo.bGenerateFeedback || SampleInfo.MipValueMode == TMVM_MipLevel)
	FString FeedbackParameter = SampleInfo.bGenerateFeedback ? FString::Printf(TEXT(", Parameters.VirtualTextureFeedback")) : TEXT("");

	EDerivativeStatus UVDerivativeStatus = GetDerivativeStatus(SampleInfo.CoordinateIndex);
	const bool bHasValidDerivative = IsAnalyticDerivEnabled() && IsDerivativeValid(UVDerivativeStatus);

	FString PageTable        = SampleInfo.PageTableIndex        != INDEX_NONE ? GetParameterCode(SampleInfo.PageTableIndex)        : FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_%d"), StackIndex);
	FString PageTableUniform = SampleInfo.PageTableUniformIndex != INDEX_NONE ? GetParameterCode(SampleInfo.PageTableUniformIndex) : FString::Printf(TEXT("VTPageTableUniform_Unpack(VIRTUALTEXTURE_PAGETABLE_UNIFORM_%d)"), StackIndex);

	// Code to load the VT page table...this will execute the first time a given VT stack is accessed
	// Additional stack layers will simply reuse these results
	switch (SampleInfo.MipValueMode)
	{
	case TMVM_None:
	{
		FString SampleCodeFinite = FString::Printf(TEXT(
			"%s("
			"%s, %s, "
			"%s, %s, %s, "
			"0, Parameters.SvPosition.xy, "
			"Parameters.VirtualTextureFeedback)"),
			*BaseFunctionName,
			*PageTable, *PageTableUniform,
			*CoerceParameter(SampleInfo.CoordinateIndex, MCT_Float2), GetVTAddressMode(SampleInfo.AddressU), GetVTAddressMode(SampleInfo.AddressV));

		if (bHasValidDerivative)
		{
			FString SampleCodeAnalytic = FString::Printf(TEXT(
				"%sGrad("
				"%s, %s, "
				"%s, %s, %s, "
				"%s, %s, Parameters.SvPosition.xy, "
				"Parameters.VirtualTextureFeedback)"),
				*BaseFunctionName,
				*PageTable, *PageTableUniform,
				*UV_Value, GetVTAddressMode(SampleInfo.AddressU), GetVTAddressMode(SampleInfo.AddressV), 
				*UV_Ddx, *UV_Ddy);
			Entry.CodeIndex = AddCodeChunkInnerDeriv(*SampleCodeFinite, *SampleCodeAnalytic, MCT_VTPageTableResult, false, EDerivativeStatus::NotValid);
		}
		else
		{
			Entry.CodeIndex = AddCodeChunk(MCT_VTPageTableResult, *SampleCodeFinite);
		}
		break;
	}
		
	case TMVM_MipBias:
	{
		FString SampleCodeFinite = FString::Printf(TEXT(
			"%s("
			"%s, %s, "
			"%s, %s, %s, "
			"%s, Parameters.SvPosition.xy, "
			"Parameters.VirtualTextureFeedback)"),
			*BaseFunctionName,
			*PageTable, *PageTableUniform,
			*CoerceParameter(SampleInfo.CoordinateIndex, MCT_Float2), GetVTAddressMode(SampleInfo.AddressU), GetVTAddressMode(SampleInfo.AddressV), 
			*CoerceParameter(SampleInfo.MipValue0Index, MCT_Float1));
		if (bHasValidDerivative)
		{
			FString SampleCodeAnalytic = FString::Printf(TEXT(
				"%sGrad("
				"%s, %s, "
				"%s, %s, %s, "
				"%s, %s, Parameters.SvPosition.xy, "
				"Parameters.VirtualTextureFeedback)"),
				*BaseFunctionName,
				*PageTable, *PageTableUniform,
				*UV_Value, GetVTAddressMode(SampleInfo.AddressU), GetVTAddressMode(SampleInfo.AddressV),
				*UV_Ddx, *UV_Ddy);
			Entry.CodeIndex = AddCodeChunkInnerDeriv(*SampleCodeFinite, *SampleCodeAnalytic, MCT_VTPageTableResult, false, EDerivativeStatus::NotValid);
		}
		else
		{
			Entry.CodeIndex = AddCodeChunk(MCT_VTPageTableResult, *SampleCodeFinite);
		}
		break;
	}
	case TMVM_MipLevel:
		Entry.CodeIndex = AddCodeChunk(MCT_VTPageTableResult, TEXT(
			"%sLevel("
			"%s, %s, "
			"%s, %s, %s, "
			"%s"
			"%s)"),
			*BaseFunctionName,
			*PageTable, *PageTableUniform,
			*CoerceParameter(SampleInfo.CoordinateIndex, MCT_Float2), GetVTAddressMode(SampleInfo.AddressU), GetVTAddressMode(SampleInfo.AddressV), 
			*CoerceParameter(SampleInfo.MipValue0Index, MCT_Float1),
			*FeedbackParameter);
		break;
	case TMVM_Derivative:
		Entry.CodeIndex = AddCodeChunk(MCT_VTPageTableResult, TEXT(
			"%sGrad("
			"%s, %s, "
			"%s, %s, %s, "
			"%s, %s, Parameters.SvPosition.xy, "
			"Parameters.VirtualTextureFeedback)"),
			*BaseFunctionName,
			*PageTable, *PageTableUniform,
			*CoerceParameter(SampleInfo.CoordinateIndex, MCT_Float2), GetVTAddressMode(SampleInfo.AddressU), GetVTAddressMode(SampleInfo.AddressV),
			*CoerceParameter(SampleInfo.MipValue0Index, MCT_Float2), *CoerceParameter(SampleInfo.MipValue1Index, MCT_Float2));
		break;
	default:
		checkNoEntry();
		break;
	}

	return StackIndex;
}


static FString ApplySamplerType(const FString& InSampleCode, EMaterialSamplerType SamplerType)
{
	FString DstSampleCode;

	switch( SamplerType )
	{
	case SAMPLERTYPE_External:
		DstSampleCode = FString::Printf(TEXT("ProcessMaterialExternalTextureLookup(%s)"), *InSampleCode);
		break;

	case SAMPLERTYPE_Color:
		DstSampleCode = FString::Printf( TEXT("ProcessMaterialColorTextureLookup(%s)"), *InSampleCode );
		break;
	case SAMPLERTYPE_VirtualColor:
		// has a mobile specific workaround
		DstSampleCode = FString::Printf( TEXT("ProcessMaterialVirtualColorTextureLookup(%s)"), *InSampleCode );
		break;

	case SAMPLERTYPE_LinearColor:
	case SAMPLERTYPE_VirtualLinearColor:
		DstSampleCode = FString::Printf(TEXT("ProcessMaterialLinearColorTextureLookup(%s)"), *InSampleCode);
		break;

	case SAMPLERTYPE_Alpha:
	case SAMPLERTYPE_VirtualAlpha:
	case SAMPLERTYPE_DistanceFieldFont:
		DstSampleCode = FString::Printf( TEXT("ProcessMaterialAlphaTextureLookup(%s)"), *InSampleCode );
		break;

	case SAMPLERTYPE_Grayscale:
	case SAMPLERTYPE_VirtualGrayscale:
		DstSampleCode = FString::Printf( TEXT("ProcessMaterialGreyscaleTextureLookup(%s)"), *InSampleCode );
		break;

	case SAMPLERTYPE_LinearGrayscale:
	case SAMPLERTYPE_VirtualLinearGrayscale:
		DstSampleCode = FString::Printf(TEXT("ProcessMaterialLinearGreyscaleTextureLookup(%s)"), *InSampleCode);
		break;
		
		// Normal maps need to be unpacked in the pixel shader.
	case SAMPLERTYPE_Normal:
		DstSampleCode = FString::Printf(TEXT("UnpackNormalMap(%s)"), *InSampleCode);
		break;
	case SAMPLERTYPE_VirtualNormal:
		DstSampleCode = FString::Printf( TEXT("UnpackNormalMapVT(%s)"), *InSampleCode );
		break;

	case SAMPLERTYPE_Masks:
	case SAMPLERTYPE_VirtualMasks:
		DstSampleCode = InSampleCode;
		break;

	case SAMPLERTYPE_Data:
		DstSampleCode = InSampleCode;
		break;

	default:
		check(0);
		break;
	}

	return DstSampleCode;
}

static bool SamplerDebugSupported(EMaterialValueType TextureType, bool bVirtualTexture, bool bDecal)
{
	return IsDebugTextureSampleEnabled() && (TextureType == MCT_Texture2D || bVirtualTexture) && !bDecal;
}

FString GetIndirectVirtualTextureUniformParameters(const TCHAR* MaterialUBName, int32 VirtualTextureIndex)
{
	if (!ensure(VirtualTextureIndex != INDEX_NONE))
	{
		VirtualTextureIndex = 0;
	}

	FString Offset = FString::Printf(TEXT("(IndirectVirtualTextureUniformDWord4Count * %i)"), VirtualTextureIndex);
	return FString::Printf(
		TEXT("%s.TextureCollectionPackedUniforms[%s + 0], %s.TextureCollectionPackedUniforms[%s + 1]"
		", %s.TextureCollectionPackedUniforms[%s + 2], %s.TextureCollectionPackedUniforms[%s + 3]"), 
		MaterialUBName, *Offset, MaterialUBName, *Offset, MaterialUBName, *Offset, MaterialUBName, *Offset);
}

int32 FHLSLMaterialTranslator::TextureSample(
	int32 TextureIndex,
	int32 CoordinateIndex,
	EMaterialSamplerType SamplerType,
	int32 MipValue0Index,
	int32 MipValue1Index,
	ETextureMipValueMode MipValueMode,
	ESamplerSourceMode SamplerSource,
	ETextureGatherMode GatherMode,
	int32 TextureReferenceIndex,
	bool AutomaticViewMipBias,
	bool AdaptiveVirtualTexture,
	bool EnableFeedback
)
{
	using namespace UE::MaterialTranslatorUtils;

	if (TextureIndex == INDEX_NONE || CoordinateIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	EMaterialValueType TextureType = GetParameterType(TextureIndex);

	if (!(TextureType & MCT_Texture))
	{
		Errorf(TEXT("Sampling unknown texture type: %s"), DescribeType(TextureType));
		return INDEX_NONE;
	}

	const bool bFromCollection = (TextureType & MCT_TextureCollection) != 0;
	TextureType = EMaterialValueType(TextureType & ~MCT_TextureCollection);

	if (ShaderFrequency != SF_Pixel && MipValueMode == TMVM_MipBias)
	{
		Errorf(TEXT("MipBias is only supported in the pixel shader"));
		return INDEX_NONE;
	}

	FMaterialVTSampleInfo SampleInfo;
	
	const bool bMeshPaint = TextureType == MCT_TextureMeshPaint;
	const bool bMaterialCache = TextureType == MCT_TextureMaterialCache;
	const bool bVirtualTexture = bMeshPaint || bMaterialCache || (TextureType & MCT_TextureVirtual) != 0;

	if (MipValueMode == TMVM_Derivative)
	{
		if (MipValue0Index == INDEX_NONE)
		{
			return Errorf(TEXT("Missing DDX(UVs) parameter"));
		}
		else if (MipValue1Index == INDEX_NONE)
		{
			return Errorf(TEXT("Missing DDY(UVs) parameter"));
		}
		else if (!IsFloatNumericType(GetParameterType(MipValue0Index)))
		{
			return Errorf(TEXT("Invalid DDX(UVs) parameter"));
		}
		else if (!IsFloatNumericType(GetParameterType(MipValue1Index)))
		{
			return Errorf(TEXT("Invalid DDY(UVs) parameter"));
		}
	}
	else if (MipValueMode != TMVM_None && MipValue0Index != INDEX_NONE && !IsFloatNumericType(GetParameterType(MipValue0Index)))
	{
		return Errorf(TEXT("Invalid mip map parameter"));
	}

	if (GatherMode != TGM_None)
	{
		if (bVirtualTexture)
		{
			return Errorf(TEXT("Gather does not support virtual textures."));
		}

		if (MipValueMode != TMVM_None)
		{
			return Errorf(TEXT("Gather does not support mip map overrides (implicitly accesses a specific mip)."));
		}

		if (!(TextureType & (MCT_Texture2D | MCT_TextureCube | MCT_Texture2DArray | MCT_TextureCubeArray)))
		{
			return Errorf(TEXT("Gather only supports 2D and Cube texture formats."));
		}

		AutomaticViewMipBias = false;
	}

	// if we are not in the PS we need a mip level
	if (ShaderFrequency != SF_Pixel)
	{
		MipValueMode = TMVM_MipLevel;
		AutomaticViewMipBias = false;

		if (MipValue0Index == INDEX_NONE)
		{
			MipValue0Index = Constant(0.f);
		}
	}

	// Automatic view mip bias is only for surface and decal domains.
	if (Material->GetMaterialDomain() != MD_Surface && Material->GetMaterialDomain() != MD_DeferredDecal)
	{
		AutomaticViewMipBias = false;
	}

	// If mobile, then disabling AutomaticViewMipBias.
	if (FeatureLevel < ERHIFeatureLevel::SM5)
	{
		AutomaticViewMipBias = false;
	}

	// If not 2D texture, disable AutomaticViewMipBias.
	if (!(TextureType & (MCT_Texture2D | MCT_TextureVirtual | MCT_TextureMeshPaint)))
	{
		AutomaticViewMipBias = false;
	}

	FString TextureName;
	int32 VirtualTextureIndex = INDEX_NONE;
	if (TextureType == MCT_TextureCube)
	{
		TextureName = CoerceParameter(TextureIndex, MCT_TextureCube);
	}
	else if (TextureType == MCT_Texture2DArray)
	{
		TextureName = CoerceParameter(TextureIndex, MCT_Texture2DArray);
	}
	else if (TextureType == MCT_TextureCubeArray)
	{
		TextureName = CoerceParameter(TextureIndex, MCT_TextureCubeArray);
	}
	else if (TextureType == MCT_VolumeTexture)
	{
		TextureName = CoerceParameter(TextureIndex, MCT_VolumeTexture);
	}
	else if (TextureType == MCT_TextureExternal)
	{
		TextureName = CoerceParameter(TextureIndex, MCT_TextureExternal);
	}
	else if (bMeshPaint)
	{
		check(bVirtualTexture);

		TextureName = TEXT("Scene.MeshPaint.PhysicalTexture, View.SharedBilinearClampedSampler");

		NumVtSamples++;
	}
	else if (bMaterialCache)
	{
		check(bVirtualTexture);

		FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(TextureIndex); 
		if (!UniformExpression || !UniformExpression->GetTextureUniformExpression())
		{
			return Errorf(TEXT("Unable to find material cache uniform expression."));
		}

		FMaterialUniformExpressionTexture* TextureUniform = UniformExpression->GetTextureUniformExpression();

		// Binding index is tied to discovery order
		uint32 MaterialCacheTagIndex = TextureUniform->GetMaterialCacheTagIndex();

		SampleInfo.PageTableIndex = AddInternalCodeChunk(
			MCT_Unexposed, TEXT("Texture2D<uint4>"),
			*FString::Printf(TEXT("%s.MaterialCacheTagPageTable0_%d"), GetCurrentMaterialUniformBufferName(), MaterialCacheTagIndex)
		);
		
		SampleInfo.PageTableUniformIndex = AddCodeChunkInner(
			MCT_Unexposed, TEXT("VTPageTableUniform"),
			EDerivativeStatus::NotAware, false,
			TEXT("MaterialCacheUnpackPageTableUniform(%s.PackedUniform)"), *GetParameterCode(TextureUniform->GetTextureIndex())
		);

		// Physical texture indexing assumes TagIndex * Stride + LayerIndex
		TextureName = FString::Printf(
			TEXT("%s.MaterialCacheTagPhysical_%d, View.SharedBilinearClampedSampler"),
			GetCurrentMaterialUniformBufferName(), MaterialCacheTagIndex * MaterialCacheMaxRuntimeLayers + TextureUniform->GetTextureLayerIndex()
		);

		NumVtSamples++;
	}
	else if (bFromCollection && bVirtualTexture)
	{
		CoerceParameter(TextureIndex, TextureType);
		
		FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(TextureIndex); 
		if (!UniformExpression || !UniformExpression->GetTextureCollectionUniformExpression())
		{
			return Errorf(TEXT("The provided uniform expression is not from a texture collection"));
		}
		
		FMaterialUniformExpressionTextureCollection* CollectionExpression = UniformExpression->GetTextureCollectionUniformExpression();
		VirtualTextureIndex = CollectionExpression->GetTextureCollectionTypePrefixIndex();

		if (SamplerSource != SSM_FromTextureAsset)
		{
			const bool   bUseAnisoSampler  = VirtualTextureScalability::IsAnisotropicFilteringEnabled() && MipValueMode != TMVM_MipLevel;
			const TCHAR* SharedSamplerName = bUseAnisoSampler ? TEXT("View.SharedBilinearAnisoClampedSampler") : TEXT("View.SharedBilinearClampedSampler");
			TextureName += FString::Printf(TEXT("%s.TextureCollectionPhysical_%d, GetMaterialSharedSampler(%s.TextureCollectionSampler_%d, %s)"), GetCurrentMaterialUniformBufferName(), VirtualTextureIndex, GetCurrentMaterialUniformBufferName(), VirtualTextureIndex, SharedSamplerName);
		}
		else
		{
			TextureName += FString::Printf(TEXT("%s.TextureCollectionPhysical_%d, %s.TextureCollectionSampler_%d"), GetCurrentMaterialUniformBufferName(), VirtualTextureIndex, GetCurrentMaterialUniformBufferName(), VirtualTextureIndex);
		}
		
		NumVtSamples++;
	}
	else if (bVirtualTexture)
	{
		// Note, this does not really do anything (by design) other than adding it to the UniformExpressionSet
		/*TextureName =*/ CoerceParameter(TextureIndex, TextureType);

		FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(TextureIndex); 
		if (UniformExpression == nullptr)
		{
			return Errorf(TEXT("Unable to find VT uniform expression."));
		}
		FMaterialUniformExpressionTexture* TextureUniformExpression = UniformExpression->GetTextureUniformExpression();
		if (TextureUniformExpression == nullptr)
		{
			return Errorf(TEXT("The provided uniform expression is not a texture"));
		}

		VirtualTextureIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Virtual].Find(TextureUniformExpression);
		check(UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Virtual].IsValidIndex(VirtualTextureIndex));

		if (SamplerSource != SSM_FromTextureAsset)
		{
			// VT doesn't care if the shared sampler is wrap or clamp. It only cares if it is aniso or not.
			// The wrap/clamp/mirror operation is handled in the shader explicitly.
			const bool bUseAnisoSampler = VirtualTextureScalability::IsAnisotropicFilteringEnabled() && MipValueMode != TMVM_MipLevel;
			const TCHAR* SharedSamplerName = bUseAnisoSampler ? TEXT("View.SharedBilinearAnisoClampedSampler") : TEXT("View.SharedBilinearClampedSampler");
			TextureName += FString::Printf(TEXT("%s.VirtualTexturePhysical_%d, GetMaterialSharedSampler(%s.VirtualTexturePhysical_%dSampler, %s)"),
				GetCurrentMaterialUniformBufferName(), VirtualTextureIndex, GetCurrentMaterialUniformBufferName(), VirtualTextureIndex, SharedSamplerName);
		}
		else
		{
			TextureName += FString::Printf(TEXT("%s.VirtualTexturePhysical_%d, %s.VirtualTexturePhysical_%dSampler"),
				GetCurrentMaterialUniformBufferName(), VirtualTextureIndex, GetCurrentMaterialUniformBufferName(), VirtualTextureIndex);
		}

		const bool bIsStrictValidation = EnumHasAnyFlags(Material->GetMaterialTranslateValidationFlags(), EMaterialTranslateValidationFlags::Strict_RuntimeVirtualTexture);
		if (bIsStrictValidation && bIsInRuntimeVirtualTextureOutput)
		{
			return Errorf(TEXT("Virtual Texture samples are not allowed during Runtime Virtual Texture Output."));
		}
		if (MaterialProperty == MP_Normal)
		{
			bUsesVirtualTextureSampleForNormalProperty = true;
		}

		NumVtSamples++;
	}
	else // MCT_Texture2D
	{
		TextureName = CoerceParameter(TextureIndex, MCT_Texture2D);
	}

	// Won't be able to get the texture, if this is an external texture sample
	const UTexture* Texture = nullptr;
	if (TextureType != MCT_TextureExternal && !bFromCollection && !bMeshPaint && !bMaterialCache)
	{
		FMaterialUniformExpression* Expression = (*CurrentScopeChunks)[TextureIndex].UniformExpression;
		const FMaterialUniformExpressionTexture* TextureExpression = Expression ? Expression->GetTextureUniformExpression() : nullptr;
		if (ensure(TextureExpression))
		{
			Texture = Cast<UTexture>(Material->GetMatDefaultTextureAtIdx(TextureExpression->GetTextureIndex()));
		}
	}

	TextureAddress StaticAddressX = TA_Wrap;
	TextureAddress StaticAddressY = TA_Wrap;
	TextureAddress StaticAddressZ = TA_Wrap;
	GetTextureAddressForSamplerSource(Texture, SamplerSource, StaticAddressX, StaticAddressY, StaticAddressZ);

	EMaterialValueType UVsType;
	switch (TextureType)
	{
	case MCT_TextureCubeArray:
		UVsType = MCT_Float4;
		break;
	case MCT_TextureCube:
	case MCT_Texture2DArray:
	case MCT_VolumeTexture:
		UVsType = MCT_Float3;
		break;
	default:
		UVsType = MCT_Float2;
		break;
	}
	
	int32 NonLWCCoordinateIndex = CoordinateIndex;
	const bool bLWCCoordinates = IsLWCType(GetParameterType(CoordinateIndex));
	if (bLWCCoordinates)
	{
		// Apply texture address math manually, using LWC-scale operations, then convert the result to float
		// This could potentially cause problems if content is relying on SSM_FromTextureAsset, and having texture parameters change address mode in MI
		// Trade-off would be skip manual address mode in this case, and just accept precision loss
		auto AddressModeToString = [](TextureAddress InAddress)
		{
			switch (InAddress)
			{
			case TA_Clamp: return TEXT("LWCADDRESSMODE_CLAMP");
			case TA_Wrap: return TEXT("LWCADDRESSMODE_WRAP");
			case TA_Mirror: return TEXT("LWCADDRESSMODE_MIRROR");
			default: checkNoEntry(); return TEXT("");
			}
		};
		AddLWCFuncUsage(ELWCFunctionKind::Other, 1);
		const uint32 NumComponents = GetNumComponents(UVsType);
		switch (NumComponents)
		{
		case 1u:
			NonLWCCoordinateIndex = AddCodeChunkZeroDeriv(UVsType, TEXT("WSApplyAddressMode(%s, %s)"),
				*CoerceParameter(CoordinateIndex, MCT_LWCScalar),
				AddressModeToString(StaticAddressX));
			break;
		case 2u:
			NonLWCCoordinateIndex = AddCodeChunkZeroDeriv(UVsType, TEXT("WSApplyAddressMode(%s, %s, %s)"),
				*CoerceParameter(CoordinateIndex, MCT_LWCVector2),
				AddressModeToString(StaticAddressX),
				AddressModeToString(StaticAddressY));
			break;
		case 3u:
			NonLWCCoordinateIndex = AddCodeChunkZeroDeriv(UVsType, TEXT("WSApplyAddressMode(%s, %s, %s, %s)"),
				*CoerceParameter(CoordinateIndex, MCT_LWCVector3),
				AddressModeToString(StaticAddressX),
				AddressModeToString(StaticAddressY),
				AddressModeToString(StaticAddressZ));
			break;
		default:
			checkf(false, TEXT("Invalid number of components %d"), NumComponents);
			break;
		}

		// Explicitly compute the derivatives for LWC UVs
		// This is needed for 100% correct functionality, otherwise filtering seams are possible where there is discontinuity in LWC->float UV conversion
		// This is expensive though, and discontinuities can be minimized by carefully choosing conversion operation
		// Disabled for now, may enable as an option in the future
		const bool bExplicitLWCDerivatives = false;
		if (bExplicitLWCDerivatives && (MipValueMode == TMVM_None || MipValueMode == TMVM_MipBias))
		{
			int32 MipScaleIndex = INDEX_NONE;
			if (MipValueMode == TMVM_MipBias)
			{
				MipScaleIndex = AddCodeChunkZeroDeriv(UVsType, TEXT("exp2(%s)"), *CoerceParameter(MipValue0Index, MCT_Float1));
			}
			
			AddLWCFuncUsage(ELWCFunctionKind::Other, 2);
			MipValue0Index = AddCodeChunkZeroDeriv(UVsType, TEXT("WSDdxDemote(%s)"), *GetParameterCode(CoordinateIndex));
			MipValue1Index = AddCodeChunkZeroDeriv(UVsType, TEXT("WSDdyDemote(%s)"), *GetParameterCode(CoordinateIndex));
			if (MipScaleIndex != INDEX_NONE)
			{
				MipValue0Index = Mul(MipValue0Index, MipScaleIndex);
				MipValue1Index = Mul(MipValue1Index, MipScaleIndex);
			}

			MipValueMode = TMVM_Derivative;
		}
	}

	FString SamplerStateCode;
	bool RequiresManualViewMipBias = AutomaticViewMipBias;

	if (!bVirtualTexture) //VT does not have explict samplers (and always requires manual view mip bias)
	{
		if (SamplerSource == SSM_FromTextureAsset)
		{
			SamplerStateCode = FString::Printf(TEXT("%sSampler"), *TextureName);
		}
		else
		{
			FString SharedSamplerName;

			if (SamplerSource == SSM_Wrap_WorldGroupSettings)
			{
				// Use the shared sampler to save sampler slots
				SharedSamplerName = AutomaticViewMipBias ? FString(TEXT("View.MaterialTextureBilinearWrapedSampler")) : FString::Printf(TEXT("%s.Wrap_WorldGroupSettings"), GetCurrentMaterialUniformBufferName());
				RequiresManualViewMipBias = false;
			}
			else if (SamplerSource == SSM_Clamp_WorldGroupSettings)
			{
				// Use the shared sampler to save sampler slots
				SharedSamplerName = AutomaticViewMipBias ? FString(TEXT("View.MaterialTextureBilinearClampedSampler")) : FString::Printf(TEXT("%s.Clamp_WorldGroupSettings"), GetCurrentMaterialUniformBufferName());
				RequiresManualViewMipBias = false;
			}
			else if (SamplerSource == SSM_TerrainWeightmapGroupSettings)
			{
				SharedSamplerName = FString(TEXT("View.LandscapeWeightmapSampler"));
				RequiresManualViewMipBias = false;
			}

			if (!SharedSamplerName.IsEmpty())
			{
				if (bFromCollection)
				{
					SamplerStateCode = SharedSamplerName;
				}
				else
				{
					SamplerStateCode = FString::Printf(TEXT("GetMaterialSharedSampler(%sSampler,%s)"), *TextureName, *SharedSamplerName);
				}
			}
		}
	}

	const EDerivativeStatus UvDerivativeStatus = GetDerivativeStatus(CoordinateIndex);

	if (RequiresManualViewMipBias)
	{
		if (MipValueMode == TMVM_Derivative)
		{
			// When doing derivative based sampling, multiply.
			int32 Multiplier = AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("View.MaterialTextureDerivativeMultiply"));
			MipValue0Index = Mul(MipValue0Index, Multiplier);
			MipValue1Index = Mul(MipValue1Index, Multiplier);
		}
		else if (MipValue0Index != INDEX_NONE && (MipValueMode == TMVM_MipLevel || MipValueMode == TMVM_MipBias))
		{
			// Adds bias to existing input level bias.
			MipValue0Index = Add(MipValue0Index, AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("View.MaterialTextureMipBias")));
		}
		else
		{
			// Sets bias.
			MipValue0Index = AddInlinedCodeChunkZeroDeriv(MCT_Float1, TEXT("View.MaterialTextureMipBias"));
		}

		// If no Mip mode, then use MipBias.
		MipValueMode = MipValueMode == TMVM_None ? TMVM_MipBias : MipValueMode;
	}

	FString MipValue0Code = "0.0f";
	FString MipValue1Code = "0.0f";
	if (MipValue0Index != INDEX_NONE && (MipValueMode == TMVM_MipBias || MipValueMode == TMVM_MipLevel))
	{
		MipValue0Code = CoerceParameter(MipValue0Index, MCT_Float1);
	}
	else if (MipValueMode == TMVM_Derivative)
	{
		MipValue0Code = CoerceParameter(MipValue0Index, UVsType);
		MipValue1Code = CoerceParameter(MipValue1Index, UVsType);
	}

	FString TextureTypeName;
	if (GatherMode != TGM_None)
	{
		// Should have errored out above
		check(TextureType& (MCT_Texture2D | MCT_TextureCube | MCT_Texture2DArray | MCT_TextureCubeArray));

		if (TextureType == MCT_TextureCube)
		{
			TextureTypeName = TEXT("TextureCubeGather");
		}
		else if (TextureType == MCT_Texture2DArray)
		{
			TextureTypeName = TEXT("Texture2DArrayGather");
		}
		else if (TextureType == MCT_TextureCubeArray)
		{
			TextureTypeName = TEXT("TextureCubeArrayGather");
		}
		else // MCT_Texture2D
		{
			TextureTypeName = TEXT("Texture2DGather");
		}

		switch (GatherMode)
		{
		case TGM_Red:	TextureTypeName += TEXT("Red");  break;
		case TGM_Green:	TextureTypeName += TEXT("Green");  break;
		case TGM_Blue:	TextureTypeName += TEXT("Blue");  break;
		case TGM_Alpha:	TextureTypeName += TEXT("Alpha");  break;
		}
	}
	else
	{
		if (TextureType == MCT_TextureCube)
		{
			TextureTypeName = TEXT("TextureCubeSample");
		}
		else if (TextureType == MCT_Texture2DArray)
		{
			TextureTypeName = TEXT("Texture2DArraySample");
		}
		else if (TextureType == MCT_TextureCubeArray)
		{
			TextureTypeName = TEXT("TextureCubeArraySample");
		}
		else if (TextureType == MCT_VolumeTexture)
		{
			TextureTypeName = TEXT("Texture3DSample");
		}
		else if (TextureType == MCT_TextureExternal)
		{
			TextureTypeName = TEXT("TextureExternalSample");
		}
		else if (bVirtualTexture)
		{
			TextureTypeName = TEXT("TextureVirtualSample");
		}
		else // MCT_Texture2D
		{
			TextureTypeName = TEXT("Texture2DSample");
		}
	}

	const bool bStoreTexCoordScales = ShaderFrequency == SF_Pixel && TextureReferenceIndex != INDEX_NONE;
	const bool bDecal = ShaderFrequency == SF_Pixel && Material->GetMaterialDomain() == MD_DeferredDecal && MipValueMode == TMVM_None;

	if (IsDebugTextureSampleEnabled() && !IsDerivativeValid(UvDerivativeStatus))
	{
		UE_LOGF(LogMaterial, Warning, "Unknown derivatives: '%ls'[%ls]: %ls", *Material->GetDebugName(), *Material->GetAssetPath().ToString(), *AtParameterCodeChunk(CoordinateIndex).DefinitionAnalytic);
	}

	if (bStoreTexCoordScales)
	{
		AddCodeChunk(MCT_Float, TEXT("MaterialStoreTexCoordScale(Parameters, %s, %d)"), *CoerceParameter(NonLWCCoordinateIndex, UVsType), (int)TextureReferenceIndex);
	}

	FString UV_Value = CoerceParameter(NonLWCCoordinateIndex, UVsType);
	FString UV_Ddx = "0.0f";
	FString UV_Ddy = "0.0f";
	FString UV_Scale = "1.0f";
	if (IsAnalyticDerivEnabled() && UvDerivativeStatus == EDerivativeStatus::Valid)
	{
		const EMaterialValueType SourceUVsType = GetParameterType(CoordinateIndex);
		const FString UVAnalytic = GetParameterCodeDeriv(CoordinateIndex, CompiledPDV_Analytic);
		UV_Ddx = CoerceValue(UVAnalytic + TEXT(".Ddx"), MakeNonLWCType(SourceUVsType), UVsType); // Ddx/y are never LWC scale
		UV_Ddy = CoerceValue(UVAnalytic + TEXT(".Ddy"), MakeNonLWCType(SourceUVsType), UVsType);
		if (MipValueMode == TMVM_MipBias)
		{
			UV_Scale = FString::Printf(TEXT("exp2(%s)"), *MipValue0Code);
			UV_Ddx = FString::Printf(TEXT("(%s)*exp2(%s)"), *UV_Ddx, *MipValue0Code);
			UV_Ddy = FString::Printf(TEXT("(%s)*exp2(%s)"), *UV_Ddy, *MipValue0Code);
		}
	}

	int32 SamplingCodeIndex = INDEX_NONE;
	if (bVirtualTexture)
	{
		// Index of the page table to sample.
		int32 VTStackIndex = INDEX_NONE; 
		// Index of the channel in the page table to sample.
		int32 VTPageTableLayerIndex = INDEX_NONE; 
		// Name for binding of the page table uniform.
		FString VTPackedUniformName;

		// Only support GPU feedback from pixel shader
		//todo[vt]: Support feedback from other shader types
		const bool bGenerateFeedback = EnableFeedback && ShaderFrequency == SF_Pixel;

		SampleInfo.MipValueMode = MipValueMode;
		SampleInfo.AddressU = StaticAddressX;
		SampleInfo.AddressV = StaticAddressY;
		SampleInfo.AspectRatio = 1.0f;
		SampleInfo.CoordinateIndex = CoordinateIndex;
		SampleInfo.MipValue0Index = MipValue0Index;
		SampleInfo.MipValue1Index = MipValue1Index;
		SampleInfo.bAdaptive = AdaptiveVirtualTexture;
		SampleInfo.bGenerateFeedback = bGenerateFeedback;
		SampleInfo.bMeshPaint = bMeshPaint;
		SampleInfo.bMaterialCache = bMaterialCache;
		SampleInfo.bCollection = bFromCollection;
        SampleInfo.VirtualTextureIndex = VirtualTextureIndex;

		if (bMeshPaint)
		{
			VTStackIndex = AcquireVTStackIndex(SampleInfo, UV_Value, UV_Ddx, UV_Ddy);
			VTPageTableLayerIndex = 0;
			VTPackedUniformName = TEXT("VTUniform_Unpack(Scene.MeshPaint.PackedUniform)");
		}
		else if (bMaterialCache)
		{
			VTStackIndex = AcquireVTStackIndex(SampleInfo, UV_Value, UV_Ddx, UV_Ddy);
			VTPageTableLayerIndex = 0;

			FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(TextureIndex); 
			if (!UniformExpression || !UniformExpression->GetTextureUniformExpression())
			{
				return Errorf(TEXT("Unable to find material cache uniform expression."));
			}

			FMaterialUniformExpressionTexture* TextureUniform = UniformExpression->GetTextureUniformExpression();
			VTPackedUniformName = FString::Printf(TEXT("VTUniform_Unpack(%s.MaterialCacheTagPageTableUniform[%d])"), GetCurrentMaterialUniformBufferName(), TextureUniform->GetMaterialCacheTagIndex());
		}
		else if (bFromCollection)
		{
			FString TextureCode = GetParameterCode(TextureIndex);
			
			SampleInfo.PageTableIndex = AddInternalCodeChunk(
				MCT_Unexposed, TEXT("Texture2D<uint4>"),
				*FString::Printf(TEXT("%s.TextureCollectionPageTable_%i"), GetCurrentMaterialUniformBufferName(), VirtualTextureIndex)
			);

			FString Uniforms = GetParameterCode(AddInternalCodeChunk(
				MCT_Unexposed, TEXT("FIndirectVirtualTextureUniform"),
				*FString::Printf(TEXT("GetIndirectVirtualTextureUniform(%s)"), *GetIndirectVirtualTextureUniformParameters(GetCurrentMaterialUniformBufferName(), VirtualTextureIndex))
			));

			// Manually unpack the page table to redirect the page boundaries
			SampleInfo.PageTableUniformIndex = AddInternalCodeChunk(
				MCT_Unexposed, TEXT("VTPageTableUniform"),
				*FString::Printf(
					TEXT("UnpackIndirectVirtualTexturePageTableUniform(%s, %s.PackedPageTableUniform[0], %s.PackedPageTableUniform[1])"),
					*TextureCode, *Uniforms, *Uniforms
				)
			);

			VTPackedUniformName = FString::Printf(TEXT("VTUniform_Unpack(%s.PackedUniform)"), *Uniforms);
			
			VTStackIndex = AcquireVTStackIndex(SampleInfo, UV_Value, UV_Ddx, UV_Ddy);
			VTPageTableLayerIndex = 0;
		}
		else 
		{
			check(VirtualTextureIndex >= 0);
			VTPackedUniformName = FString::Printf(TEXT("VTUniform_Unpack(%s.VTPackedUniform[%d])"), GetCurrentMaterialUniformBufferName(), VirtualTextureIndex);

			SampleInfo.CoordinateIndex = NonLWCCoordinateIndex;

			const int32 VTTextureLayerIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Virtual][VirtualTextureIndex]->GetTextureLayerIndex();
			if (VTTextureLayerIndex != INDEX_NONE)
			{
				SampleInfo.PreallocatedStackTextureIndex = TextureReferenceIndex;
				
				// The layer index in the virtual texture stack is already known
				// Create a page table sample for each new combination of virtual texture and sample parameters
				VTStackIndex = AcquireVTStackIndex(SampleInfo, UV_Value, UV_Ddx, UV_Ddy);
				VTPageTableLayerIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::Virtual][VirtualTextureIndex]->GetPageTableLayerIndex();

				MaterialCompilationOutput.UniformExpressionSet.VTStacks[VTStackIndex].SetLayer(VTTextureLayerIndex, VirtualTextureIndex);
			}
			else
			{
				// Textures can only be combined in a VT stack if they have the same aspect ratio
				// This also means that any texture parameters set in material instances for VTs must match the aspect ratio of the texture in the parent material
				// (Otherwise could potentially break stacks)
				check(Texture);

				// Using Source size because we care about the aspect ratio of each block (each block of multi-block texture must have same aspect ratio)
				// We can still combine multi-block textures of different block aspect ratios, as long as each block has the same ratio
				// This is because we only need to overlay VT pages from within a given block
				const float TextureAspectRatio = (float)Texture->Source.GetSizeX() / (float)Texture->Source.GetSizeY();

				SampleInfo.AspectRatio = TextureAspectRatio;
				
				// Create a page table sample for each new set of sample parameters
				VTStackIndex = AcquireVTStackIndex(SampleInfo, UV_Value, UV_Ddx, UV_Ddy);

				// Find or allocate the layer in the virtual texture stack for this physical sample
				VTPageTableLayerIndex = MaterialCompilationOutput.UniformExpressionSet.VTStacks[VTStackIndex].FindLayer(VirtualTextureIndex);
				if (VTPageTableLayerIndex == INDEX_NONE)
				{
					VTPageTableLayerIndex = MaterialCompilationOutput.UniformExpressionSet.VTStacks[VTStackIndex].AddLayer();
					MaterialCompilationOutput.UniformExpressionSet.VTStacks[VTStackIndex].SetLayer(VTPageTableLayerIndex, VirtualTextureIndex);
				}
			}
		}

		// VT MipValueMode logic (most of work for VT case is in page table lookup)
		if (MipValueMode == TMVM_MipLevel)
		{
			TextureTypeName += TEXT("Level");
		}

		const FMaterialVTStackEntry& VTStackEntry = VTStacks[VTStackIndex];
		const FString VTPageTableResult_Finite = GetParameterCode(VTStackEntry.CodeIndex);

		// 'Texture name/sampler', 'PageTableResult', 'LayerIndex', 'PackedUniform'
		FString SampleCodeFinite = FString::Printf(TEXT("%s(%s, %s, %d, %s)"), *TextureTypeName, *TextureName, *VTPageTableResult_Finite, VTPageTableLayerIndex, *VTPackedUniformName);
		SampleCodeFinite = ApplySamplerType(SampleCodeFinite, SamplerType);

		if (IsAnalyticDerivEnabled() && IsDerivativeValid(UvDerivativeStatus))
		{
			const FString VTPageTableResult_Analytic = GetParameterCodeDeriv(VTStackEntry.CodeIndex, CompiledPDV_Analytic);
			// 'Texture name/sampler', 'PageTableResult', 'LayerIndex', 'PackedUniform'
			FString SampleCodeAnalytic;

			if (SamplerDebugSupported(TextureType, bVirtualTexture, bDecal))
			{
				SampleCodeAnalytic = FString::Printf(TEXT("Debug%s(%s, %s, %d, %s, %s)"), *TextureTypeName, *TextureName, *VTPageTableResult_Analytic, VTPageTableLayerIndex, *VTPackedUniformName, *UV_Scale);
			}
			else
			{
				SampleCodeAnalytic = FString::Printf(TEXT("%s(%s, %s, %d, %s)"), *TextureTypeName, *TextureName, *VTPageTableResult_Analytic, VTPageTableLayerIndex, *VTPackedUniformName);
			}

			SampleCodeAnalytic = ApplySamplerType(SampleCodeAnalytic, SamplerType);

			EDerivativeStatus TextureSampleDerivativeStatus = (UvDerivativeStatus == EDerivativeStatus::Zero) ? EDerivativeStatus::Zero : EDerivativeStatus::NotValid;

			SamplingCodeIndex = AddCodeChunkInnerDeriv(*SampleCodeFinite, *SampleCodeAnalytic, MCT_Float4, false, TextureSampleDerivativeStatus);
		}
		else
		{
			SamplingCodeIndex = AddCodeChunk(MCT_Float4, *SampleCodeFinite, *TextureName, *VTPageTableResult_Finite, VTPageTableLayerIndex, VirtualTextureIndex);
		}
	}
	else
	{
		// Non-VT MipValueMode logic
		// 
		// Re-route decal texture sampling so platforms may add specific workarounds there (assume no workarounds for Gather)
		if (bDecal && GatherMode == TGM_None)
		{
			TextureTypeName += TEXT("_Decal");
		}

		// Note that gather implicitly samples from the first mip, and doesn't support explicit mip levels or derivatives
		FString SampleCodeFinite;
		if (MipValueMode == TMVM_None || GatherMode != TGM_None)
		{
			SampleCodeFinite = FString::Printf(TEXT("%s(%s,%s,%s)"), *TextureTypeName, *TextureName, *SamplerStateCode, *UV_Value);
		}
		else if (MipValueMode == TMVM_MipLevel)
		{
			SampleCodeFinite = FString::Printf(TEXT("%sLevel(%s,%s,%s,%s)"), *TextureTypeName, *TextureName, *SamplerStateCode, *UV_Value, *MipValue0Code);
		}
		else if (MipValueMode == TMVM_MipBias)
		{
			SampleCodeFinite = FString::Printf(TEXT("%sBias(%s,%s,%s,%s)"), *TextureTypeName, *TextureName, *SamplerStateCode, *UV_Value, *MipValue0Code);
		}
		else if (MipValueMode == TMVM_Derivative)
		{
			SampleCodeFinite = FString::Printf(TEXT("%sGrad(%s,%s,%s,%s,%s)"), *TextureTypeName, *TextureName, *SamplerStateCode, *UV_Value, *MipValue0Code, *MipValue1Code);
		}
		else
		{
			check(0);
		}

		SampleCodeFinite = ApplySamplerType(SampleCodeFinite, SamplerType);

		if (IsAnalyticDerivEnabled() && IsDerivativeValid(UvDerivativeStatus) && !bDecal && GatherMode == TGM_None)
		{
			FString SampleCodeAnalytic = SampleCodeFinite;
			if (MipValueMode == TMVM_None || MipValueMode == TMVM_MipBias)
			{
				
				if (SamplerDebugSupported(TextureType, bVirtualTexture, bDecal))
					SampleCodeAnalytic = FString::Printf(TEXT("Debug%sGrad(%s,%s,%s,%s,%s,%s)"), *TextureTypeName, *TextureName, *SamplerStateCode, *UV_Value, *UV_Ddx, *UV_Ddy, *UV_Scale);
				else
					SampleCodeAnalytic = FString::Printf(TEXT("%sGrad(%s,%s,%s,%s,%s)"), *TextureTypeName, *TextureName, *SamplerStateCode, *UV_Value, *UV_Ddx, *UV_Ddy);

				SampleCodeAnalytic = ApplySamplerType(SampleCodeAnalytic, SamplerType);
			}

			EDerivativeStatus TextureSampleDerivativeStatus = (UvDerivativeStatus == EDerivativeStatus::Zero) ? EDerivativeStatus::Zero : EDerivativeStatus::NotValid;

			SamplingCodeIndex = AddCodeChunkInnerDeriv(*SampleCodeFinite, *SampleCodeAnalytic, MCT_Float4, false /*?*/, TextureSampleDerivativeStatus);
		}
		else
		{
			SamplingCodeIndex = AddCodeChunk(MCT_Float4, *SampleCodeFinite);
		}
	}

	AddEstimatedTextureSample();
	if (bStoreTexCoordScales)
	{
		FString SamplingCode = CoerceParameter(SamplingCodeIndex, MCT_Float4);
		AddCodeChunk(MCT_Float, TEXT("MaterialStoreTexSample(Parameters, %s, %d)"), *SamplingCode, (int)TextureReferenceIndex);
	}

	return SamplingCodeIndex;
}

int32 FHLSLMaterialTranslator::TextureProperty(int32 TextureIndex, EMaterialExposedTextureProperty Property)
{
	const EMaterialValueType TextureType = GetParameterType(TextureIndex);
	if (TextureType != MCT_Texture2D &&
		TextureType != MCT_TextureVirtual &&
		TextureType != MCT_VolumeTexture &&
		TextureType != MCT_Texture2DArray &&
		TextureType != MCT_SparseVolumeTexture)
	{
		return Errorf(TEXT("Texture size only available for Texture2D, TextureVirtual, Texture2DArray, VolumeTexture and SparseVolumeTexture, not %s"), DescribeType(TextureType));
	}

	FMaterialUniformExpressionTexture* TextureExpression = (*CurrentScopeChunks)[TextureIndex].UniformExpression->GetTextureUniformExpression();
	if (!TextureExpression)
	{
		return Errorf(TEXT("Expected a texture expression"));
	}

	const EMaterialValueType ValueType = UE::MaterialTranslatorUtils::GetTexturePropertyValueType(TextureType);
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTextureProperty(TextureExpression, Property), ValueType, TEXT(""));
}

static const TCHAR* GetTextureObjectType(EMaterialValueType Type)
{
	switch (Type)
	{
	case MCT_Texture2D:				return TEXT("Texture2D");
	case MCT_TextureCube:			return TEXT("TextureCube");
	case MCT_Texture2DArray:		return TEXT("Texture2DArray");
	case MCT_TextureCubeArray:		return TEXT("TextureCubeArray");
	case MCT_VolumeTexture:			return TEXT("Texture3D");
	default:						return TEXT("unknown");
	};
}

int32 FHLSLMaterialTranslator::TextureFromCollection(int32 TextureCollectionCodeIndex, int32 IndexIntoCollectionCodeIndex, EMaterialValueType ResultTextureType)
{
	if (ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::SM6) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (TextureCollectionCodeIndex == INDEX_NONE)
	{
		return Errorf(TEXT("Missing TextureCollection parameter"));
	}

	if (IndexIntoCollectionCodeIndex == INDEX_NONE)
	{
		return Errorf(TEXT("Missing CollectionIndex parameter"));
	}

	const EMaterialValueType TextureType = GetParameterType(TextureCollectionCodeIndex);
	if (TextureType != MCT_TextureCollection)
	{
		return Errorf(TEXT("TextureFromCollection only available for Texture Collections, not %s"), DescribeType(TextureType));
	}

	FMaterialUniformExpressionTextureCollection* TextureCollectionExpression = GetParameterUniformExpression(TextureCollectionCodeIndex)->GetTextureCollectionUniformExpression();
	if (!TextureCollectionExpression)
	{
		return Errorf(TEXT("Expected a texture collection expression"));
	}

	if ((ResultTextureType & MCT_Texture) == 0)
	{
		return Errorf(TEXT("Expected a texture return type"));
	}

	if (TextureCollectionExpression->IsVirtualCollection())
	{
		if (ResultTextureType != MCT_Texture2D)
		{
			return Errorf(TEXT("Texture collection virtual sampling only supports 2d textures"));
		}

		AccessUniformExpression(TextureCollectionCodeIndex);
		
		FString Uniforms = GetParameterCode(AddInternalCodeChunk(
			MCT_Unexposed, TEXT("FIndirectVirtualTextureUniform"),
			*FString::Printf(TEXT("GetIndirectVirtualTextureUniform(%s)"), *GetIndirectVirtualTextureUniformParameters(GetCurrentMaterialUniformBufferName(), TextureCollectionExpression->GetTextureCollectionTypePrefixIndex()))
		));
		
		return AddInternalCodeChunk(
			static_cast<EMaterialValueType>(ResultTextureType | MCT_TextureVirtual | MCT_TextureCollection | MCT_Unexposed), TEXT("FIndirectVirtualTextureEntry"),
			*FString::Printf(
				TEXT("LoadIndirectVirtualTexture(%s, %s, %s)"),
				*GetParameterCode(TextureCollectionCodeIndex),
				*Uniforms,
				*GetParameterCode(IndexIntoCollectionCodeIndex)
			),
			TextureCollectionExpression
		);
	}
	
	return AddCodeChunk(
		EMaterialValueType(ResultTextureType | MCT_TextureCollection),
		TEXT("TextureFromCollection_%s(%s, %s)"),
		GetTextureObjectType(ResultTextureType),
		*GetParameterCode(TextureCollectionCodeIndex),
		*GetParameterCode(IndexIntoCollectionCodeIndex)
	);
}

int32 FHLSLMaterialTranslator::TextureStreamingInfo(int32 TextureReferenceIndex, int32 TextureIndex, int32 CoordinateIndex)
{
	EMaterialValueType TextureType = GetParameterType(TextureIndex);

	if (!(TextureType & MCT_Texture))
	{
		Errorf(TEXT("Sampling unknown texture type: %s"), DescribeType(TextureType));
		return INDEX_NONE;
	}

	if (CoordinateIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return AddCodeChunk(MCT_Float, TEXT("MaterialStoreTexCoordScale(Parameters, %s, %d)"), *CoerceParameter(CoordinateIndex, MCT_Float2), TextureReferenceIndex);
}

int32 FHLSLMaterialTranslator::DefaultMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout)
{
	FMaterialCacheTag& Tag = FindOrAddMaterialTagStack(Layout);
	FString TypeName = FString::Printf(TEXT("FMaterialCacheABuffer_%d"), Tag.StackIndex);
	return AddCodeChunkInner(MCT_MaterialCacheABuffer, *TypeName, EDerivativeStatus::NotAware, false, TEXT("DefaultMaterialCacheABuffer_%d()"), Tag.StackIndex);
}

int32 FHLSLMaterialTranslator::SetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex, int32 Value)
{
	// Naming counter
	uint32 LocalCounter = 0;

	for (int32 i = 0; i < Layout.Layers.Num(); i++)
	{
		const FMaterialCacheLayer& Layer = Layout.Layers[i];
		
		for (EMaterialCacheAttribute Attribute : Layer.Attributes)
		{
			if (LocalCounter++ == AttributeIndex)
			{
				return AddCodeChunk(
					MCT_VoidStatement,
					TEXT("\t%s.%s%i = %s;"),
					*GetParameterCode(AttributeSet),
					*GetMaterialCacheAttributeDecoration(Attribute),
					AttributeIndex,
					*GetParameterCode(Value)
				);
			}
		}
	}

	checkf(false, TEXT("Invalid index"));
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::GetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex)
{
	// Naming counter
	uint32 LocalCounter = 0;

	for (int32 i = 0; i < Layout.Layers.Num(); i++)
	{
		const FMaterialCacheLayer& Layer = Layout.Layers[i];
		
		for (EMaterialCacheAttribute Attribute : Layer.Attributes)
		{
			if (LocalCounter++ == AttributeIndex)
			{
				return AddCodeChunk(
					GetMaterialCacheAttributeValueType(Attribute),
					TEXT("%s.%s%i"),
					*GetParameterCode(AttributeSet),
					*GetMaterialCacheAttributeDecoration(Attribute),
					AttributeIndex
				);
			}
		}
	}

	checkf(false, TEXT("Invalid index"));
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::MaterialCacheOutput(UMaterialExpressionCustomOutput* Output, const FMaterialCacheTagLayout& Layout, int32 AttributeSet)
{
	FMaterialCacheTag& Tag = FindOrAddMaterialTagStack(Layout);
	FString TypeName = FString::Printf(TEXT("FMaterialCacheABuffer_%d"), Tag.StackIndex);
	return CustomOutput(Output->GetDescription(), TypeName, Output->GetFunctionName(), Tag.StackIndex, AttributeSet, EMaterialCustomOutputFlags::AllowAttributeConnection);
}

FHLSLMaterialTranslator::FMaterialCacheTag& FHLSLMaterialTranslator::FindOrAddMaterialTagStack(const FMaterialCacheTagLayout& Layout)
{
	if (FMaterialCacheTag* Tag = MaterialCacheTags.Find(Layout.Guid))
	{
		return *Tag;
	}
	
	// New tag
	FMaterialCacheTag& Tag = MaterialCacheTags.Add(Layout.Guid);
	Tag.Layers = Layout.Layers;
	Tag.StackIndex = MaterialCompilationOutput.UniformExpressionSet.MaterialCacheTagStacks.Num();

	// Add tag stack to expression set
	FMaterialCacheTagStack& TagStack = MaterialCompilationOutput.UniformExpressionSet.MaterialCacheTagStacks.Emplace_GetRef();
	TagStack.TagGuid = Layout.Guid;
	TagStack.ShaderFrequencyMask = 0;

	return Tag;
}

int32 FHLSLMaterialTranslator::MaterialCacheTextureDescriptor(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, uint32 LayerIndex)
{
	FAddUniformExpressionScope Scope(this);
	FMaterialCacheTag& Tag = FindOrAddMaterialTagStack(Layout);

	// Default to own primitive id if none supplied
	FString OverloadArgument = PrimitiveIDIndex == INDEX_NONE ? TEXT("Parameters.PrimitiveId") : GetParameterCode(PrimitiveIDIndex);

	// TODO[MP]: If we're reading our own tag offset, no need to index back into the primitive data again
	// Though, it'll probably combine the reads anyway.
	int32 CodeIndex = AddInlinedCodeChunkZeroDeriv(
		MCT_TextureMaterialCache,
		TEXT("%s.MaterialCacheTagBuffer_%u[GetMaterialCacheTagOffset(GetPrimitiveData(%s))]"),
		GetCurrentMaterialUniformBufferName(), Tag.StackIndex, *OverloadArgument
	);

	// Assign allocated stack index through texture uniform
	FMaterialUniformExpressionTexture* Uniform = new FMaterialUniformExpressionTexture(CodeIndex, LayerIndex, 0, SAMPLERTYPE_Color);
	Uniform->SetMaterialCacheTagIndex(Tag.StackIndex);
	return AddUniformExpression(Scope, Uniform, MCT_TextureMaterialCache, TEXT(""));
}

int32 FHLSLMaterialTranslator::SampleMaterialCache(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, int32 TexCoordIndex)
{
	FString ParameterStack;

	// Sample each layer and append to parameter stack
	for (int32 i = 0; i < Layout.Layers.Num(); i++)
	{
		int32 Layer = TextureSample(
			MaterialCacheTextureDescriptor(Layout, PrimitiveIDIndex, i),
			TexCoordIndex,
			SAMPLERTYPE_LinearColor,
			INDEX_NONE,
			INDEX_NONE,
			TMVM_None,
			SSM_FromTextureAsset,
			TGM_None,
			INDEX_NONE,
			false,
			false,
			true
		);

		if (i != 0)
		{
			ParameterStack += TEXT(", ");
		}

		ParameterStack += GetParameterCode(Layer);
	}

	FMaterialCacheTag& Tag = FindOrAddMaterialTagStack(Layout);
	
	// Supported on all shader stages
	FString TypeName = FString::Printf(TEXT("FMaterialCacheABuffer_%d"), Tag.StackIndex);
	return AddCodeChunkInner(MCT_MaterialCacheABuffer,  *TypeName, EDerivativeStatus::NotAware, false, TEXT("UnpackMaterialCacheABuffer_%d(%s)"), Tag.StackIndex, *ParameterStack);
}

int32 FHLSLMaterialTranslator::TextureDecalMipmapLevel(int32 TextureSizeInput)
{
	if (Material->GetMaterialDomain() != MD_DeferredDecal)
	{
		return Errorf(TEXT("Decal mipmap level only available in the decal material domain."));
	}

	EMaterialValueType TextureSizeType = GetParameterType(TextureSizeInput);

	if (TextureSizeType != MCT_Float2)
	{
		Errorf(TEXT("Unmatching conversion %s -> float2"), DescribeType(TextureSizeType));
		return INDEX_NONE;
	}

	FString TextureSize = CoerceParameter(TextureSizeInput, MCT_Float2);

	return AddCodeChunk(
		MCT_Float1,
		TEXT("ComputeDecalMipmapLevel(Parameters,%s)"),
		*TextureSize
		);
}

int32 FHLSLMaterialTranslator::TextureDecalDerivative(bool bDDY)
{
	if (Material->GetMaterialDomain() != MD_DeferredDecal)
	{
		return Errorf(TEXT("Decal derivatives only available in the decal material domain."));
	}

	return AddCodeChunk(
		MCT_Float2,
		bDDY ? TEXT("ComputeDecalDDY(Parameters)") : TEXT("ComputeDecalDDX(Parameters)")
		);
}

int32 FHLSLMaterialTranslator::DecalColor()
{
	return AddExternalCodeChunk(TEXT("DecalColor"));
}

int32 FHLSLMaterialTranslator::DecalLifetimeOpacity()
{
	return AddExternalCodeChunk(TEXT("DecalLifetimeOpacity"));
}

int32 FHLSLMaterialTranslator::PixelDepth()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute && ShaderFrequency != SF_Vertex)
	{
		return Errorf(TEXT("Invalid node used in hull/domain shader input!"));
	}

	FString FiniteCode = "GetPixelDepth(Parameters)";
	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(TEXT("Parameters.ScreenPosition.w"), TEXT("Parameters.ScreenPosition_DDX.w"), TEXT("Parameters.ScreenPosition_DDY.w"), EDerivativeType::Float1);
		return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, MCT_Float, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddInlinedCodeChunk(MCT_Float, *FiniteCode);
	}
}

/** Calculate screen aligned UV coordinates from an offset fraction or texture coordinate */
int32 FHLSLMaterialTranslator::GetScreenAlignedUV(int32 Offset, int32 ViewportUV, bool bUseOffset)
{
	if(bUseOffset)
	{
		return AddCodeChunk(MCT_Float2, TEXT("CalcScreenUVFromOffsetFraction(GetScreenPosition(Parameters), %s)"), *GetParameterCode(Offset));
	}
	else if (ViewportUV != INDEX_NONE)
	{
		int32 BufferUV = AddCodeChunk(MCT_Float2, TEXT("MaterialFloat2(ViewportUVToBufferUV(%s))"), *CoerceParameter(ViewportUV, MCT_Float2));

		EMaterialDomain MaterialDomain = Material->GetMaterialDomain();
		int32 Min = AddInlinedCodeChunkZeroDeriv(MCT_Float2, MaterialDomain == MD_Surface ? TEXT("ResolvedView.BufferBilinearUVMinMax.xy") : TEXT("View.BufferBilinearUVMinMax.xy"));
		int32 Max = AddInlinedCodeChunkZeroDeriv(MCT_Float2, MaterialDomain == MD_Surface ? TEXT("ResolvedView.BufferBilinearUVMinMax.zw") : TEXT("View.BufferBilinearUVMinMax.zw"));
		return Clamp(BufferUV, Min, Max);
	}
	else
	{
		return AddInlinedCodeChunk(MCT_Float2, TEXT("ScreenAlignedPosition(GetScreenPosition(Parameters))"));
	}
}

int32 FHLSLMaterialTranslator::SceneDepth(int32 Offset, int32 ViewportUV, bool bUseOffset)
{
	if (ShaderFrequency == SF_Vertex && IsMobilePlatform(Platform) 
		&& (MobileAllowFramebufferFetch(Platform) && !MobileUsesFullDepthPrepass(Platform)))
	{
		// mobile currently does not support this, we need to read a separate copy of the depth, we must disable framebuffer fetch and force scene texture reads.
		return Errorf(TEXT("Cannot read scene depth from the vertex shader with the Mobile feature level"));
	}

	if (Material->IsTranslucencyWritingVelocity())
	{
		return Errorf(TEXT("Translucenct material with 'Output Velocity' enabled will write to depth buffer, therefore cannot read from depth buffer at the same time."));
	}

	if (Offset == INDEX_NONE && bUseOffset)
	{
		return INDEX_NONE;
	}

	bUsesSceneDepth = true;
	AddEstimatedTextureSample();

	FString	UserDepthCode("CalcSceneDepth(%s)");
	int32 TexCoordCode = GetScreenAlignedUV(Offset, ViewportUV, bUseOffset);
	// add the code string
	return AddCodeChunk(
		MCT_Float,
		*UserDepthCode,
		*GetParameterCode(TexCoordCode)
		);
}

// @param SceneTextureId of type ESceneTextureId e.g. PPI_SubsurfaceColor
int32 FHLSLMaterialTranslator::SceneTextureLookup(int32 ViewportUV, uint32 InSceneTextureId, bool bFiltered, bool bClamped, bool bUnused)
{
	ESceneTextureId SceneTextureId = (ESceneTextureId)InSceneTextureId;

	// Guard against using unsupported textures with SLW
	const bool bHasSingleLayerWaterSM = GetMaterialShadingModels().HasShadingModel(MSM_SingleLayerWater);
	if (bHasSingleLayerWaterSM && SceneTextureId != PPI_CustomDepth && SceneTextureId != PPI_CustomStencil)
	{
		return Error(TEXT("Only custom depth and custom stencil can be sampled with SceneTexture when used with the Single Layer Water shading model."));
	}
	
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Vertex)
	{
		// we can relax this later if needed
		return NonPixelShaderExpressionError();
	}

	if (SceneTextureId == PPI_DecalMask)
	{
		return Error(TEXT("Decal Mask bit was moved from GBuffer to the Stencil Buffer for performance optimisation so therefore no longer available."));
	}

	UseSceneTextureId(SceneTextureId, true);

	if (bUnused)
	{
		// If the output is unused, we just need to reach the call to UseSceneTextureId above so the shader compiler includes the scene texture, then
		// emit an arbitrary unused constant expression.  This code path is for Custom HLSL references to scene textures, where custom code is calling
		// SceneTextureFetch or SceneTextureLookup, and the result of the input pin itself isn't used.
		return Constant4(0.0f, 0.0f, 0.0f, 0.0f);
	}

	FString SceneTextureIdString = UE::MaterialTranslatorUtils::SceneTextureIdToHLSLString(SceneTextureId);

	int32 BufferUV;
	if (ViewportUV != INDEX_NONE)
	{
		BufferUV = AddCodeChunk(MCT_Float2,
			TEXT("ClampSceneTextureUV(ViewportUVToSceneTextureUV(%s, %s), %s)"),
			*CoerceParameter(ViewportUV, MCT_Float2), *SceneTextureIdString, *SceneTextureIdString);
	}
	else
	{
		if (bClamped)
		{
			BufferUV = AddInlinedCodeChunk(MCT_Float2, TEXT("ClampSceneTextureUV(GetDefaultSceneTextureUV(Parameters, %s), %s)"), *SceneTextureIdString, *SceneTextureIdString);
		}
		else
		{
			BufferUV = AddInlinedCodeChunk(MCT_Float2, TEXT("GetDefaultSceneTextureUV(Parameters, %s)"), *SceneTextureIdString);
		}
	}

	AddEstimatedTextureSample();

	int32 LookUp = INDEX_NONE;

	if (FeatureLevel >= ERHIFeatureLevel::SM5)
	{
		LookUp = AddCodeChunk(
			MCT_Float4,
			TEXT("SceneTextureLookup(%s, %s, %s)"),
			*CoerceParameter(BufferUV, MCT_Float2), *SceneTextureIdString, bFiltered ? TEXT("true") : TEXT("false")
		);
	}
	else // mobile
	{
		LookUp = AddCodeChunk(MCT_Float4, TEXT("MobileSceneTextureLookup(Parameters, %s, %s)"), *SceneTextureIdString, *CoerceParameter(BufferUV, MCT_Float2));
	}

	// Substrate only
	// When SceneTexture lookup node is used, single/simple paths are disabled to ensure texture decoding is properly handled.
	// Reading SceneTexture, when Substrate is enabled, implies unpacking material buffer data. The unpacking function exists in different 'flavor' 
	// for optimization purpose (simple/single/complex). To avoid compiling out single or complex unpacking paths (due to defines set by analyzing 
	// the current shader, vs. scene texture pixels), we force Simple/Single versions to be disabled
	FSubstrateCompilationContext& SubstrateCtx = SubstrateTranslatorData.SubstrateCompilationContext[SubstrateTranslatorData.CurrentSubstrateCompilationContext];;
	SubstrateCtx.SubstrateMaterialComplexity.bIsSimple = false;
	SubstrateCtx.SubstrateMaterialComplexity.bIsSingle = false;
	
	if (((SceneTextureId >= PPI_PostProcessInput0 && SceneTextureId <= PPI_PostProcessInput6) || (SceneTextureId >= PPI_UserSceneTexture0 && SceneTextureId <= PPI_UserSceneTexture6)) &&
		Material->GetMaterialDomain() == MD_PostProcess && Material->GetBlendableLocation() != BL_SceneColorAfterTonemapping && !Material->GetDisablePreExposureScale())
	{
		return AddInlinedCodeChunk(MCT_Float4, TEXT("(float4(View.OneOverPreExposure.xxx, 1) * %s)"), *CoerceParameter(LookUp, MCT_Float4));
	}
	else
	{
		return LookUp;
	}
}

int32 FHLSLMaterialTranslator::GetSceneTextureViewSize(int32 SceneTextureId, bool InvProperty)
{
	FString SceneTextureIdString = UE::MaterialTranslatorUtils::SceneTextureIdToHLSLString((ESceneTextureId)SceneTextureId);

	if (InvProperty)
	{
		return AddCodeChunkZeroDeriv(MCT_Float2, TEXT("GetSceneTextureViewSize(%s).zw"), *SceneTextureIdString);
	}
	return AddCodeChunkZeroDeriv(MCT_Float2, TEXT("GetSceneTextureViewSize(%s).xy"), *SceneTextureIdString);
}

// @param bTextureLookup true: texture, false:no texture lookup, usually to get the size
void FHLSLMaterialTranslator::UseSceneTextureId(ESceneTextureId SceneTextureId, bool bTextureLookup)
{
	MaterialCompilationOutput.bNeedsSceneTextures = true;
	MaterialCompilationOutput.SetIsSceneTextureUsed(SceneTextureId);

	if (Material->GetMaterialDomain() == MD_DeferredDecal)
	{
		const bool bSceneTextureSupportsDecal = SceneTextureId == PPI_SceneDepth || SceneTextureId == PPI_WorldNormal || SceneTextureId == PPI_CustomDepth || SceneTextureId == PPI_CustomStencil;
		if (!bSceneTextureSupportsDecal)
		{
			// Note: For DBuffer decals CustomDepth and CustomStencil are not available if r.CustomDepth.Order = 1
			Errorf(TEXT("Decals can only access SceneDepth, CustomDepth, CustomStencil, and WorldNormal."));
		}

		if (SceneTextureId == PPI_WorldNormal && IsMobilePlatform(Platform) && !IsMobileDeferredShadingEnabled(Platform))
		{
			Errorf(TEXT("Decals can only access WorldNormal when deferred shading is used."));
		}

		if (SceneTextureId == PPI_WorldNormal && Material->HasNormalConnected() && !IsUsingDBuffers(Platform))
		{
			// GBuffer decals can't bind Normal for read and write.
			// Note: DBuffer decals can support this but only if the sampled WorldNormal isn't connected to the output normal.
			Errorf(TEXT("Decals that read WorldNormal cannot output to normal at the same time. Enable DBuffer to support this."));
		}
	}

	if(SceneTextureId == PPI_SceneColor && Material->GetMaterialDomain() != MD_Surface)
	{
		if(Material->GetMaterialDomain() == MD_PostProcess)
		{
			Errorf(TEXT("SceneColor lookups are only available when MaterialDomain = Surface. PostProcessMaterials should use the SceneTexture PostProcessInput0."));
		}
		else
		{
			Errorf(TEXT("SceneColor lookups are only available when MaterialDomain = Surface."));
		}
	}

	if(bTextureLookup)
	{
		bNeedsSceneTexturePostProcessInputs = bNeedsSceneTexturePostProcessInputs
			|| ((SceneTextureId >= PPI_PostProcessInput0 && SceneTextureId <= PPI_PostProcessInput6)
			|| (SceneTextureId >= PPI_UserSceneTexture0 && SceneTextureId <= PPI_UserSceneTexture6)
			|| SceneTextureId == PPI_Velocity
			|| SceneTextureId == PPI_SceneColor);

	}

	if (SceneTextureId == PPI_SceneDepth && bTextureLookup)
	{
		bUsesSceneDepth = true;
	}

	const bool bNeedsGBuffer = MaterialCompilationOutput.NeedsGBuffer();

	if (bNeedsGBuffer)
	{
		const bool bMobilePlatform = IsMobilePlatform(Platform);
		
		if ((bMobilePlatform && !IsMobileDeferredShadingEnabled(Platform)) || 
			(!bMobilePlatform && IsForwardShadingEnabled(Platform)))
		{
			Errorf(TEXT("GBuffer scene textures not available with forward shading (platform id %d)."), Platform);
		}
		
		// Post-process can't access memoryless GBuffer on mobile
		if (bMobilePlatform && (!IsMobileDeferredShadingEnabled(Platform) || MobileAllowFramebufferFetch(Platform)))
		{
			if (Material->GetMaterialDomain() == MD_PostProcess)
			{
				Errorf(TEXT("GBuffer scene textures not available in post-processing with mobile shading (platform id %d)."), Platform);
			}

			if (Material->IsMobileSeparateTranslucencyEnabled())
			{
				Errorf(TEXT("GBuffer scene textures not available for separate translucency with mobile shading (platform id %d)."), Platform);
			}
		}
	}

	if (SceneTextureId == PPI_Velocity)
	{
		if (Material->GetMaterialDomain() != MD_PostProcess)
		{
			Errorf(TEXT("Velocity scene textures are only available in post process materials."));
		}
	}

	// not yet tracked:
	//   PPI_SeparateTranslucency, PPI_CustomDepth, PPI_AmbientOcclusion
}

int32 FHLSLMaterialTranslator::SceneColor(int32 Offset, int32 ViewportUV, bool bUseOffset)
{
	if (Offset == INDEX_NONE && bUseOffset)
	{
		return INDEX_NONE;
	}

	if (ShaderFrequency != SF_Pixel)
	{
		return NonPixelShaderExpressionError();
	}

	if(Material->GetMaterialDomain() != MD_Surface)
	{
		Errorf(TEXT("SceneColor lookups are only available when MaterialDomain = Surface."));
	}

	if (IsMobilePlatform(Platform) && !IsMobileDeferredShadingEnabled(Platform))
	{
		Errorf(TEXT("SceneColor lookups are only available when deferred shading is used"));
	}

	MaterialCompilationOutput.SetIsSceneTextureUsed(PPI_SceneColor);
	AddEstimatedTextureSample();

	int32 ScreenUVCode = GetScreenAlignedUV(Offset, ViewportUV, bUseOffset);
	return AddCodeChunk(
		MCT_Float4,
		TEXT("DecodeSceneColorAndAlpharForMaterialNode(%s)"),
		*GetParameterCode(ScreenUVCode)
		);
}

int32 FHLSLMaterialTranslator::DBufferTextureLookup(int32 ViewportUV, uint32 DBufferTextureIndex)
{
	if (Material->GetMaterialDomain() != MD_Surface || IsTranslucentBlendMode(Material->GetBlendMode()))
	{
		Errorf(TEXT("DBuffer scene textures are only available on opaque or masked surfaces."));
	}

	int32 BufferUV = INDEX_NONE;
	if (ViewportUV != INDEX_NONE)
	{
		BufferUV = AddCodeChunk(MCT_Float2,	TEXT("ClampSceneTextureUV(ViewportUVToBufferUV(%s), 0)"), *CoerceParameter(ViewportUV, MCT_Float2));
	}
	else
	{
		BufferUV = AddInlinedCodeChunk(MCT_Float2, TEXT("GetDefaultSceneTextureUV(Parameters, 0)"));
	}

	MaterialCompilationOutput.SetIsDBufferTextureUsed(DBufferTextureIndex);
	// set separate flag to indicate that material uses DBuffer lookup specifically
	// can't rely on UsedDBufferTextures because those bits are also set depending on the default decal response behavior.
	MaterialCompilationOutput.SetIsDBufferTextureLookupUsed(true);
	AddEstimatedTextureSample();

	return AddCodeChunk(MCT_Float4, TEXT("MaterialExpressionDBufferTextureLookup(Parameters, %s, %d)"), *CoerceParameter(BufferUV, MCT_Float2), (int)DBufferTextureIndex);
}

int32 FHLSLMaterialTranslator::PathTracingBufferTextureLookup(int32 ViewportUV, uint32 PathTracingBufferTextureIndex)
{
	if (Material->GetMaterialDomain() != MD_PostProcess)
	{
		Errorf(TEXT("Path tracing buffer textures are only available on post process material."));
	}

	int32 BufferUV = INDEX_NONE;
	if (ViewportUV != INDEX_NONE)
	{
		BufferUV = ViewportUV;
	}
	else
	{
		BufferUV = AddInlinedCodeChunk(MCT_Float2, TEXT("GetDefaultPathTracingBufferTextureUV(Parameters, 0)"));
	}

	MaterialCompilationOutput.SetIsPathTracingBufferTextureUsed(PathTracingBufferTextureIndex);
	AddEstimatedTextureSample();

	return AddCodeChunk(MCT_Float4, TEXT("MaterialExpressionPathTracingBufferTextureLookup(Parameters, %s, %d)"), *CoerceParameter(BufferUV, MCT_Float2), (int)PathTracingBufferTextureIndex);
}

int32 FHLSLMaterialTranslator::Switch(int32 SwitchValueInput, int32 DefaultInput, TArray<int32>& CompiledInputs)
{
	if (SwitchValueInput == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (DefaultInput == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (CompiledInputs.Num() == 0)
	{
		return INDEX_NONE;
	}

	for (int32 Input : CompiledInputs)
	{
		if (Input == INDEX_NONE)
		{
			return INDEX_NONE;
		}
	}

	// If our selector is constant, we can skip the generation of code below and simply
    // grab/return the appropriate input directly. This should greatly simplify the
    // expression in constant parameter cases.
	FMaterialUniformExpression* ExpressionSwitchValue = GetParameterUniformExpression(SwitchValueInput);
	EMaterialValueType SwitchValueType = GetParameterType(SwitchValueInput);
	if (SwitchValueType == MCT_Float && ExpressionSwitchValue && ExpressionSwitchValue->IsConstant())
	{		
		FLinearColor Value;
		FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
		ExpressionSwitchValue->GetNumberValue(DummyContext, Value);

		for (int32 InputIndex = 0; InputIndex < CompiledInputs.Num(); ++InputIndex)
		{
			if (Value.R >= InputIndex && Value.R < InputIndex + 1)
			{
				return CompiledInputs[InputIndex];
			}
		}
		return DefaultInput;
	}
	
	EMaterialValueType ResultType = GetParameterType(CompiledInputs[0]);
	for (int32 Input : CompiledInputs)
	{
		ResultType = GetArithmeticResultType(ResultType, GetParameterType(Input));
	}

	FString SwitchCode;
	FString SwitchValueParameter = CoerceParameter(SwitchValueInput, MCT_Float);
	FString DefaultParameter = CoerceParameter(DefaultInput, ResultType);


	// TODO: The following represents two potential implementations for the dynamic switch logic in HLSL.
	// The first option is an expression that mathamatically cancels out any "unmatched" case,
	// leaving only the desired value passing through. This approach works, but is less optimal 
	// due to needing to evalulate every potential case value as part of the process.
	// The second option is potentially more optimal, as it uses a native switch statement in HLSL, something
	// that the compiler should be able to optimize more easily with less unneeded evaluations. However,
	// this approach currently doesn't work well with the existing material translation code in the engine.
	// The existing design supports expressions returning floating point type values, which are used by
	// the analyical derivative components elsewhere. Attempting to return a void type or insert type-less
	// statements into the generated code has been problematic.
	//
	// Ideally, the second option can be made to function correctly and replace the first, which is why
	// it's being left in as a preprocessor switch for the time being. However, for expedieceny, we want
	//to use the first option for the time being, to provide a working, if less efficicent, switch implementation.
#if 1
	// We form the "switch" statement from an equivalent math expression,
    // using step functions and additions to select between inputs.
	for (int32 InputIndex = 0; InputIndex < CompiledInputs.Num(); ++InputIndex)
	{
		FString InputParameter = CoerceParameter(CompiledInputs[InputIndex], ResultType);
		SwitchCode += FString::Printf(TEXT("step(%f, floor(%s) + 0.5f) * step(floor(%s) + 0.5f, %f) * %s +"),
			float(InputIndex), *SwitchValueParameter,        // If greater than the lower bound
			*SwitchValueParameter, float(InputIndex) + 1.0f, // and less  than the upper bound parameters
			*InputParameter);                             // return the selected selected input
	}
	SwitchCode += FString::Printf(TEXT("(step(floor(%s) + 0.5f, %f) + step(%f, floor(%s) + 0.5f)) * %s"),
	    *SwitchValueParameter, float(0),                    // If less than Zero,
		float(CompiledInputs.Num()), *SwitchValueParameter, // and greater than number of inputs,
		*DefaultParameter);                              // return the default
	return AddCodeChunk(ResultType, *SwitchCode);
#else
	int32 ReturnValue = AddCodeChunk(ResultType, TEXT("0"));
	FString ReturnParameter = CoerceParameter(ReturnValue, ResultType);

	SwitchCode += FString::Printf(TEXT("switch(%s){"), *SwitchValueParameter);
	for (int32 InputIndex = 0; InputIndex < CompiledInputs.Num(); ++InputIndex)
	{
		FString InputParameter = CoerceParameter(CompiledInputs[InputIndex], ResultType);
		SwitchCode += FString::Printf(TEXT("case %d: %s = %s; break;"), InputIndex, *ReturnParameter, *InputParameter);
	}
	SwitchCode += FString::Printf(TEXT("default: %s = %s; }"), *ReturnParameter, *DefaultParameter);
	
	return AddCodeChunkInner(MCT_VoidStatement, EDerivativeStatus::NotAware, true, *SwitchCode);
#endif
}

int32 FHLSLMaterialTranslator::Texture(UTexture* InTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource, ETextureMipValueMode MipValueMode)
{
	EMaterialValueType ShaderType = InTexture->GetMaterialType();
	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(InTexture);

#if DO_CHECK
	// UE-3518: Additional pre-assert logging to help determine the cause of this failure.
	if (TextureReferenceIndex == INDEX_NONE)
	{
		const TArray<TObjectPtr<UObject>>& ReferencedTextures = Material->GetMatDefaultTextures();
		UE_LOGF(LogMaterial, Error, "Compiler->Texture() failed to find texture '%ls' in referenced list of size '%i':", *InTexture->GetName(), ReferencedTextures.Num());
		for (int32 i = 0; i < ReferencedTextures.Num(); ++i)
		{
			UE_LOGF(LogMaterial, Error, "%i: '%ls'", i, ReferencedTextures[i] ? *ReferencedTextures[i]->GetName() : TEXT("nullptr"));
		}
	}
#endif
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->Texture() without implementing UMaterialExpression::GetReferencedTexture properly"));

	const bool bVirtualTexturesEnabeled = UseVirtualTexturing(Platform);
	bool bVirtual = ShaderType == MCT_TextureVirtual;
	if (bVirtualTexturesEnabeled == false && ShaderType == MCT_TextureVirtual)
	{
		bVirtual = false;
		ShaderType = MCT_Texture2D;
	}
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTexture(TextureReferenceIndex, SamplerType, SamplerSource, bVirtual),ShaderType,TEXT(""));
}

int32 FHLSLMaterialTranslator::TextureParameter(FName ParameterName, UTexture* InDefaultValue, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource)
{
	UTexture* DefaultValue = InDefaultValue;

	// If we're compiling a function, give the function a chance to override the default parameter value
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValueForCurrentFunction(EMaterialParameterType::Texture, ParameterName, Meta))
	{
		DefaultValue = Meta.Value.Texture;
	}

	EMaterialValueType ShaderType = DefaultValue->GetMaterialType();
	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(DefaultValue);
	if (TextureReferenceIndex == INDEX_NONE)
	{
		FString TextureName;
		DefaultValue->GetName(TextureName);
		Errorf(TEXT("Could not resolve referenced texture '%s'."), *TextureName);
		return INDEX_NONE;
	}

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	const bool bVirtualTexturesEnabled = UseVirtualTexturing(Platform);
	bool bVirtual = ShaderType == MCT_TextureVirtual;
	if (bVirtualTexturesEnabled == false && ShaderType == MCT_TextureVirtual)
	{
		bVirtual = false;
		ShaderType = MCT_Texture2D;
	}
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTextureParameter(ParameterInfo, TextureReferenceIndex, SamplerType, SamplerSource, bVirtual),ShaderType,TEXT(""));
}

int32 FHLSLMaterialTranslator::TextureCollection(UTextureCollection* InTextureCollection, int32& TextureCollectionReferenceIndex)
{
	TextureCollectionReferenceIndex = Material->GetReferencedTextureCollections().Find(InTextureCollection);
	if (TextureCollectionReferenceIndex < 0)
	{
		return INDEX_NONE;
	}

	const bool bIsVirtualCollection = InTextureCollection->IsVirtualCollection();

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTextureCollection(TextureCollectionReferenceIndex, UINT32_MAX, bIsVirtualCollection), MCT_TextureCollection, TEXT(""));
}

int32 FHLSLMaterialTranslator::TextureCollectionParameter(FName ParameterName, UTextureCollection* InDefaultValue, int32& TextureCollectionReferenceIndex)
{
	UTextureCollection* DefaultValue = InDefaultValue;

	// If we're compiling a function, give the function a chance to override the default parameter value
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValueForCurrentFunction(EMaterialParameterType::TextureCollection, ParameterName, Meta))
	{
		DefaultValue = Meta.Value.TextureCollection;
	}

	TextureCollectionReferenceIndex = Material->GetReferencedTextureCollections().Find(DefaultValue);
	if (TextureCollectionReferenceIndex == INDEX_NONE)
	{
		FString TextureCollectionName;
		DefaultValue->GetName(TextureCollectionName);
		Errorf(TEXT("Could not resolve referenced texture collection '%s'."), *TextureCollectionName);
		return INDEX_NONE;
	}

	const bool bIsVirtualCollection = InDefaultValue->IsVirtualCollection();

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTextureCollectionParameter(ParameterInfo, TextureCollectionReferenceIndex, UINT32_MAX, bIsVirtualCollection), MCT_TextureCollection, TEXT(""));
}

int32 FHLSLMaterialTranslator::TextureCollectionCount(int32 InTextureCollectionCodeIndex)
{
	if (ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::SM6) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialValueType TextureType = GetParameterType(InTextureCollectionCodeIndex);
	if (TextureType != MCT_TextureCollection)
	{
		return Errorf(TEXT("TextureCollectionCount only available for Texture Collections, not %s"), DescribeType(TextureType));
	}

	FMaterialUniformExpressionTextureCollection* TextureCollectionExpression = GetParameterUniformExpression(InTextureCollectionCodeIndex)->GetTextureCollectionUniformExpression();
	if (!TextureCollectionExpression)
	{
		return Errorf(TEXT("Expected a texture collection expression"));
	}
	
	if (TextureCollectionExpression->IsVirtualCollection())
	{
		// Make sure the uniform is present
		AccessUniformExpression(InTextureCollectionCodeIndex);

		return AddCodeChunk(
			EMaterialValueType(MCT_UInt1),
			TEXT("GetIndirectVirtualTextureCount(GetIndirectVirtualTextureUniform(%s))"),
			*GetIndirectVirtualTextureUniformParameters(GetCurrentMaterialUniformBufferName(), TextureCollectionExpression->GetTextureCollectionTypePrefixIndex())
		);
	}
	
	return AddCodeChunk(
		EMaterialValueType(MCT_UInt1),
		TEXT("GetCountFromResourceCollection(%s)"),
		*GetParameterCode(InTextureCollectionCodeIndex)
	);
}

int32 FHLSLMaterialTranslator::VirtualTexture(URuntimeVirtualTexture* InTexture, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) 
{
	if (!UseVirtualTexturing(Platform))
	{
		return INDEX_NONE;
	}

	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(InTexture);
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->VirtualTexture() without implementing UMaterialExpression::GetReferencedTexture properly"));

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTexture(TextureReferenceIndex, TextureLayerIndex, PageTableLayerIndex, SamplerType), MCT_TextureVirtual, TEXT(""));
}

int32 FHLSLMaterialTranslator::VirtualTextureParameter(FName ParameterName, URuntimeVirtualTexture* InDefaultValue, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType)
{
	if (!UseVirtualTexturing(Platform))
	{
		return INDEX_NONE;
	}

	URuntimeVirtualTexture* DefaultValue = InDefaultValue;

	// If we're compiling a function, give the function a chance to override the default parameter value
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValueForCurrentFunction(EMaterialParameterType::RuntimeVirtualTexture, ParameterName, Meta))
	{
		DefaultValue = Meta.Value.RuntimeVirtualTexture;
	}

	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(DefaultValue);
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->VirtualTexture() without implementing UMaterialExpression::GetReferencedTexture properly"));

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTextureParameter(ParameterInfo, TextureReferenceIndex, TextureLayerIndex, PageTableLayerIndex, SamplerType), MCT_TextureVirtual, TEXT(""));
}

int32 FHLSLMaterialTranslator::VirtualTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionRuntimeVirtualTextureUniform(TextureIndex, VectorIndex), GetMaterialValueType(Type), TEXT(""));
}

int32 FHLSLMaterialTranslator::VirtualTextureUniform(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type)
{
	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionRuntimeVirtualTextureUniform(ParameterInfo, TextureIndex, VectorIndex), GetMaterialValueType(Type), TEXT(""));
}

int32 FHLSLMaterialTranslator::VirtualTextureWorldToUV(int32 WorldPositionIndex, int32 P0, int32 P1, int32 P2, EPositionOrigin PositionOrigin)
{
	if (!UseVirtualTexturing(Platform) || WorldPositionIndex == INDEX_NONE || P0 == INDEX_NONE || P1 == INDEX_NONE || P2 == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EDerivativeStatus WorldPositionDerivStatus = GetDerivativeStatus(WorldPositionIndex);

	EMaterialValueType WorldPositionType = (EMaterialValueType)0;
	switch (PositionOrigin)
	{
	case EPositionOrigin::Absolute: { WorldPositionType = MCT_LWCVector3; break; }
	case EPositionOrigin::CameraRelative: { WorldPositionType = MCT_Float3; break; } //P0 is camera-relative
	default: { checkNoEntry(); }
	}
	FString CodeFinite = FString::Printf(TEXT("VirtualTextureWorldToUV(%s, %s, %s, %s)"), *CoerceParameter(WorldPositionIndex, WorldPositionType), *GetParameterCode(P0), *GetParameterCode(P1), *GetParameterCode(P2));
	if (IsAnalyticDerivEnabled() && IsDerivativeValid(WorldPositionDerivStatus))
	{
		if (WorldPositionDerivStatus == EDerivativeStatus::Valid)
		{
			WorldPositionIndex = ValidCast(WorldPositionIndex, WorldPositionType);
			FString WorldPositionDeriv = GetParameterCodeDeriv(WorldPositionIndex, CompiledPDV_Analytic);
			FString CodeAnalytic = FString::Printf(TEXT("VirtualTextureWorldToUVDeriv(%s, %s, %s, %s)"), *WorldPositionDeriv, *GetParameterCode(P0), *GetParameterCode(P1), *GetParameterCode(P2));
			return AddCodeChunkInnerDeriv(*CodeFinite, *CodeAnalytic, MCT_Float2, false, EDerivativeStatus::Valid);
		}
		else
		{
			return AddInlinedCodeChunkZeroDeriv(MCT_Float2, *CodeFinite);
		}
	}
	else
	{
		return AddInlinedCodeChunk(MCT_Float2, *CodeFinite);
	}
}

int32 FHLSLMaterialTranslator::VirtualTextureUnpack(int32 CodeIndex0, int32 CodeIndex1, int32 CodeIndex2, int32 P0, EVirtualTextureUnpackType UnpackType)
{
	if (UnpackType == EVirtualTextureUnpackType::BaseColorYCoCg)
	{
		FString	SampleCode("VirtualTextureUnpackBaseColorYCoCg(%s)");
		return CodeIndex0 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex0));
	}
	else if (UnpackType == EVirtualTextureUnpackType::NormalBC3)
	{
		FString	SampleCode("VirtualTextureUnpackNormalBC3(%s)");
		return CodeIndex1 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex1));
	}
	else if (UnpackType == EVirtualTextureUnpackType::NormalBC5)
	{
		FString	SampleCode("VirtualTextureUnpackNormalBC5(%s)");
		return CodeIndex1 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex1));
	}
	else if (UnpackType == EVirtualTextureUnpackType::NormalBC3BC3)
	{
		FString	SampleCode("VirtualTextureUnpackNormalBC3BC3(%s, %s)");
		return CodeIndex0 == INDEX_NONE || CodeIndex1 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex0), *GetParameterCode(CodeIndex1));
	}
	else if (UnpackType == EVirtualTextureUnpackType::NormalBC5BC1)
	{
		FString	SampleCode("VirtualTextureUnpackNormalBC5BC1(%s, %s)");
		return CodeIndex1 == INDEX_NONE || CodeIndex2 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex1), *GetParameterCode(CodeIndex2));
	}
	else if (UnpackType == EVirtualTextureUnpackType::HeightR16)
	{
		FString	SampleCode("VirtualTextureUnpackHeight(%s, %s)");
		return CodeIndex0 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float, *SampleCode, *GetParameterCode(CodeIndex0), *GetParameterCode(P0));
	}
	else if (UnpackType == EVirtualTextureUnpackType::DisplacementR16)
	{
		FString	SampleCode("%s.r");
		return CodeIndex0 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float, *SampleCode, *GetParameterCode(CodeIndex0));
	}
	else if (UnpackType == EVirtualTextureUnpackType::NormalBGR565)
	{
		FString	SampleCode("VirtualTextureUnpackNormalBGR565(%s)");
		return CodeIndex1 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex1));
	}
	if (UnpackType == EVirtualTextureUnpackType::BaseColorSRGB)
	{
		FString	SampleCode("VirtualTextureUnpackBaseColorSRGB(%s)");
		return CodeIndex0 == INDEX_NONE ? INDEX_NONE : AddCodeChunk(MCT_Float3, *SampleCode, *GetParameterCode(CodeIndex0));
	}
	return CodeIndex0;
}

int32 FHLSLMaterialTranslator::VirtualTextureCustomData()
{
	if (bIsInRuntimeVirtualTextureOutput)
	{
		// Call hlsl helper function.
		return AddInlinedCodeChunkZeroDeriv(MCT_Float4, TEXT("GetRuntimeVirtualTextureCustomData()"));
	}
	// Set zero if we are not writing RVT here.
	return Constant4(0, 0, 0, 0);
}

int32 FHLSLMaterialTranslator::ExternalTexture(const FGuid& ExternalTextureGuid)
{
	bool bOnlyInPixelShader = GetFeatureLevel() < ERHIFeatureLevel::SM5;

	if (bOnlyInPixelShader && ShaderFrequency != SF_Pixel)
	{
		return NonPixelShaderExpressionError();
	}

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTexture(ExternalTextureGuid), MCT_TextureExternal, TEXT(""));
}

int32 FHLSLMaterialTranslator::ExternalTexture(UTexture* InTexture, int32& TextureReferenceIndex)
{
	bool bOnlyInPixelShader = GetFeatureLevel() < ERHIFeatureLevel::SM5;

	if (bOnlyInPixelShader && ShaderFrequency != SF_Pixel)
	{
		return NonPixelShaderExpressionError();
	}

	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(InTexture);
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->ExternalTexture() without implementing UMaterialExpression::GetReferencedTexture properly"));

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTexture(TextureReferenceIndex), MCT_TextureExternal, TEXT(""));
}

int32 FHLSLMaterialTranslator::ExternalTextureParameter(FName ParameterName, UTexture* DefaultValue, int32& TextureReferenceIndex)
{
	bool bOnlyInPixelShader = GetFeatureLevel() < ERHIFeatureLevel::SM5;

	if (bOnlyInPixelShader && ShaderFrequency != SF_Pixel)
	{
		return NonPixelShaderExpressionError();
	}

	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(DefaultValue);
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->ExternalTextureParameter() without implementing UMaterialExpression::GetReferencedTexture properly"));
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTextureParameter(ParameterName, TextureReferenceIndex), MCT_TextureExternal, TEXT(""));
}

int32 FHLSLMaterialTranslator::ExternalTextureCoordinateScaleRotation(int32 TextureReferenceIndex, TOptional<FName> ParameterName)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTextureCoordinateScaleRotation(TextureReferenceIndex, ParameterName), MCT_Float4, TEXT(""));
}
int32 FHLSLMaterialTranslator::ExternalTextureCoordinateScaleRotation(const FGuid& ExternalTextureGuid)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTextureCoordinateScaleRotation(ExternalTextureGuid), MCT_Float4, TEXT(""));
}
int32 FHLSLMaterialTranslator::ExternalTextureCoordinateOffset(int32 TextureReferenceIndex, TOptional<FName> ParameterName)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTextureCoordinateOffset(TextureReferenceIndex, ParameterName), MCT_Float2, TEXT(""));
}
int32 FHLSLMaterialTranslator::ExternalTextureCoordinateOffset(const FGuid& ExternalTextureGuid)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionExternalTextureCoordinateOffset(ExternalTextureGuid), MCT_Float2, TEXT(""));
}

UObject* FHLSLMaterialTranslator::GetReferencedTexture(int32 Index)
{
	return Material->GetMatDefaultTextureAtIdx(Index);
}

UTextureCollection* FHLSLMaterialTranslator::GetReferencedTextureCollection(int32 Index)
{
	return Material->GetReferencedTextureCollections()[Index];
}

int32 FHLSLMaterialTranslator::StaticBool(bool bValue)
{
	return AddInlinedCodeChunk(MCT_StaticBool,(bValue ? TEXT("true") : TEXT("false")));
}

int32 FHLSLMaterialTranslator::DynamicBoolParameter(FName ParameterName, bool bDefaultValue)
{
	// Look up the value we are compiling with for this static parameter.
	bool bValue = bDefaultValue;

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	const UE::Shader::EValueType ValueType = GetShaderValueType(EMaterialParameterType::StaticSwitch);
	UE::Shader::FValue DefaultValue(bValue);

	const uint32* PrevDefaultOffset = DefaultUniformValues.Find(DefaultValue);
	uint32 DefaultOffset;
	if (PrevDefaultOffset)
	{
		DefaultOffset = *PrevDefaultOffset;
	}
	else
	{
		DefaultOffset = MaterialCompilationOutput.UniformExpressionSet.AddDefaultParameterValue(DefaultValue);
		DefaultUniformValues.Add(DefaultValue, DefaultOffset);
	}

	FAddUniformExpressionScope Scope(this);
	const int32 ParameterIndex = MaterialCompilationOutput.UniformExpressionSet.FindOrAddNumericParameter(EMaterialParameterType::StaticSwitch, ParameterInfo, DefaultOffset);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionStaticBoolParameter(ParameterInfo, ParameterIndex), MCT_Bool, TEXT(""));
}

int32 FHLSLMaterialTranslator::StaticBoolParameter(FName ParameterName,bool bDefaultValue)
{
	// Look up the value we are compiling with for this static parameter.
	bool bValue = bDefaultValue;

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	for (const FStaticSwitchParameter& Parameter : StaticParameters.StaticSwitchParameters)
	{
		if (Parameter.ParameterInfo == ParameterInfo)
		{
			bValue = Parameter.Value;
			break;
		}
	}

	return StaticBool(bValue);
}
	
int32 FHLSLMaterialTranslator::StaticComponentMask(int32 Vector,FName ParameterName,bool bDefaultR,bool bDefaultG,bool bDefaultB,bool bDefaultA)
{
	// Look up the value we are compiling with for this static parameter.
	bool bValueR = bDefaultR;
	bool bValueG = bDefaultG;
	bool bValueB = bDefaultB;
	bool bValueA = bDefaultA;

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	for (const FStaticComponentMaskParameter& Parameter : StaticParameters.EditorOnly.StaticComponentMaskParameters)
	{
		if (Parameter.ParameterInfo == ParameterInfo)
		{
			bValueR = Parameter.R;
			bValueG = Parameter.G;
			bValueB = Parameter.B;
			bValueA = Parameter.A;
			break;
		}
	}

	return ComponentMask(Vector,bValueR,bValueG,bValueB,bValueA);
}

const FMaterialLayersFunctions* FHLSLMaterialTranslator::GetMaterialLayers()
{
	return StaticParameters.bHasMaterialLayers ? &CachedMaterialLayers : nullptr;
}

void FHLSLMaterialTranslator::FeedbackMaterialLayersInstancedGraphFromCompilation(const FMaterialLayersFunctions* InLayers)
{
	if (InLayers && StaticParameters.bHasMaterialLayers)
	{
		if (Material)
		{
			Material->FeedbackMaterialLayersInstancedGraphFromCompilation(InLayers);
		}
	}
}

bool FHLSLMaterialTranslator::GetStaticBoolValue(int32 BoolIndex, bool& bSucceeded)
{
	bSucceeded = true;
	if (BoolIndex == INDEX_NONE)
	{
		bSucceeded = false;
		return false;
	}

	if (GetParameterType(BoolIndex) != MCT_StaticBool)
	{
		Errorf(TEXT("Failed to cast %s input to static bool type"), DescribeType(GetParameterType(BoolIndex)));
		bSucceeded = false;
		return false;
	}

	if (GetParameterCode(BoolIndex).Contains(TEXT("true")))
	{
		return true;
	}
	return false;
}

int32 FHLSLMaterialTranslator::StaticTerrainLayerWeight(FName LayerName,int32 Default, bool bTextureArray)
{
	// Look up the weight-map index for this static parameter.
	int32 WeightmapCode = INDEX_NONE;
	int32 NumWeightmapParameters = 0;
	for(int32 ParameterIndex = 0;ParameterIndex < StaticParameters.EditorOnly.TerrainLayerWeightParameters.Num(); ++ParameterIndex)
	{
		const FStaticTerrainLayerWeightParameter& Parameter = StaticParameters.EditorOnly.TerrainLayerWeightParameters[ParameterIndex];

		if(Parameter.LayerName != LayerName)
		{
			continue;
		}

		int32 WeightmapIndex =  Parameter.WeightmapIndex;

		if(WeightmapIndex == INDEX_NONE)
		{
			continue;
		}

		FMaterialParameterInfo GlobalParameterInfo;
		PushParameterOwner(GlobalParameterInfo);

		int32 UVIndex = TextureCoordinate(3, false, false);
		constexpr EMaterialSamplerType SamplerType = SAMPLERTYPE_Masks;
		int32 TextureReferenceIndex = INDEX_NONE;
		int32 SampleCodeIndex = 0;
		 
		if (bTextureArray)
		{
			int32 TextureArrayCodeIndex = TextureParameter(TEXT("WeightmapArray"), GEngine->WeightMapArrayPlaceholderTexture, TextureReferenceIndex, SamplerType);
			int32 ConstantSliceIndex = Constant(WeightmapIndex);
			int32 UVWIndex = AppendVector(UVIndex, ConstantSliceIndex);
			SampleCodeIndex = TextureSample(TextureArrayCodeIndex, UVWIndex, SamplerType, /*MipValue0Index = */INDEX_NONE, /*MipValue1Index = */INDEX_NONE, /*MipValueMode = */TMVM_None, /*SamplerSource = */SSM_TerrainWeightmapGroupSettings);
		}
		else
		{
			FString WeightmapName = FString::Printf(TEXT("Weightmap%d"), WeightmapIndex);
			int32 TextureCodeIndex = TextureParameter(FName(*WeightmapName), GEngine->WeightMapPlaceholderTexture, TextureReferenceIndex, SamplerType);
			SampleCodeIndex = TextureSample(TextureCodeIndex, UVIndex, SamplerType, /*MipValue0Index = */INDEX_NONE, /*MipValue1Index = */INDEX_NONE, /*MipValueMode = */TMVM_None, /*SamplerSource = */SSM_TerrainWeightmapGroupSettings);
		}
	
		FString LayerMaskName = FString::Printf(TEXT("LayerMask_%s"), *Parameter.LayerName.ToString());
		int32 CurrentWeightmapCode = Dot(SampleCodeIndex, VectorParameter(FName(*LayerMaskName), FLinearColor(1.f, 0.f, 0.f, 0.f)));	
		 
		if(WeightmapCode == INDEX_NONE)
		{
			WeightmapCode = CurrentWeightmapCode;
		}
		else
		{
			WeightmapCode = Add(WeightmapCode, CurrentWeightmapCode);
		}

		PopParameterOwner();
		++NumWeightmapParameters;
	}

	if((NumWeightmapParameters == 0) && Material->IsPreview())
	{
		return Default;
	}
	else 
	{
		if (NumWeightmapParameters > 1)
		{
			WeightmapCode = Clamp(WeightmapCode, Constant(0.0), Constant(1.0));
		}

		return WeightmapCode;
	}
}

int32 FHLSLMaterialTranslator::ExternalCode(const FMaterialExternalCodeDeclaration& InExternalCode)
{
	return AddExternalCodeChunk(InExternalCode);
}

int32 FHLSLMaterialTranslator::FontSignedDistanceData()
{
	return AddExternalCodeChunk(TEXT("FontSignedDistanceData"));
}

int32 FHLSLMaterialTranslator::VertexColor()
{
	bUsesVertexColor |= (ShaderFrequency != SF_Vertex);

	FString FiniteCode = "Parameters.VertexColor";
	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, TEXT("Parameters.VertexColor_DDX"), TEXT("Parameters.VertexColor_DDY"), EDerivativeType::Float4);
		return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, MCT_Float4, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddInlinedCodeChunk(MCT_Float4, *FiniteCode);
	}
}

int32 FHLSLMaterialTranslator::MeshPaintTextureCoordinateIndex()
{
	return AddExternalCodeChunk(TEXT("MeshPaintTextureCoordinateIndex"));
}

int32 FHLSLMaterialTranslator::MeshPaintTextureDescriptor()
{
	return AddExternalCodeChunk(TEXT("MeshPaintTextureDescriptor"));
}

int32 FHLSLMaterialTranslator::MeshPaintTextureReplace(int32 Invalid, int32 Valid)
{
	return GenericSwitch(TEXT("GetMeshPaintTextureDescriptorIsValid(GetPrimitiveData(Parameters))"), Valid, Invalid);
}

int32 FHLSLMaterialTranslator::PreSkinnedPosition()
{
	return AddExternalCodeChunk(TEXT("PreSkinnedPosition"));
}

int32 FHLSLMaterialTranslator::PreSkinnedNormal()
{
	return AddExternalCodeChunk(TEXT("PreSkinnedNormal"));
}

int32 FHLSLMaterialTranslator::VertexInterpolator(uint32 InterpolatorIndex)
{
	if (ShaderFrequency != SF_Pixel)
	{
		return Errorf(TEXT("Custom interpolator outputs only available in pixel shaders."));
	}

	UMaterialExpressionVertexInterpolator** InterpolatorPtr = CustomVertexInterpolators.FindByPredicate([InterpolatorIndex](const UMaterialExpressionVertexInterpolator* Item) { return Item && Item->InterpolatorIndex == InterpolatorIndex; });
	if (InterpolatorPtr == nullptr)
	{
		return Errorf(TEXT("Invalid custom interpolator index."));
	}

	MaterialCompilationOutput.bUsesVertexInterpolator = true;

	UMaterialExpressionVertexInterpolator* Interpolator = *InterpolatorPtr;
	check(Interpolator->InterpolatorIndex == InterpolatorIndex);
	check(Interpolator->InterpolatedType & MCT_Float);

	// Assign interpolator offset and accumulate size
	int32 InterpolatorSize = 0;
	switch (Interpolator->InterpolatedType)
	{
	case MCT_Float4:	InterpolatorSize = 4; break;
	case MCT_Float3:	InterpolatorSize = 3; break;
	case MCT_Float2:	InterpolatorSize = 2; break;
	default:			InterpolatorSize = 1;
	};

	if (Interpolator->InterpolatorOffset == INDEX_NONE)
	{
		Interpolator->InterpolatorOffset = CurrentCustomVertexInterpolatorOffset;
		CurrentCustomVertexInterpolatorOffset += InterpolatorSize;
	}
	check(CurrentCustomVertexInterpolatorOffset != INDEX_NONE && Interpolator->InterpolatorOffset < CurrentCustomVertexInterpolatorOffset);

	// Copy interpolated data from pixel parameters to local
	const TCHAR* TypeName = HLSLTypeString(Interpolator->InterpolatedType);
	const TCHAR* Swizzle[2] = { TEXT("x"), TEXT("y") };
	const int32 Offset = Interpolator->InterpolatorOffset;
	
	// Note: We reference the UV define directly to avoid having to pre-accumulate UV counts before property translation
	FString ValueCode	= FString::Printf(TEXT("%s(Parameters.TexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_X].%s"),		TypeName, InterpolatorIndex, Swizzle[Offset%2]);
	FString DDXCode		= FString::Printf(TEXT("%s(Parameters.TexCoords_DDX[VERTEX_INTERPOLATOR_%i_TEXCOORDS_X].%s"),	TypeName, InterpolatorIndex, Swizzle[Offset%2]);
	FString DDYCode		= FString::Printf(TEXT("%s(Parameters.TexCoords_DDY[VERTEX_INTERPOLATOR_%i_TEXCOORDS_X].%s"),	TypeName, InterpolatorIndex, Swizzle[Offset%2]);
	if (InterpolatorSize >= 2)
	{
		ValueCode	+= FString::Printf(TEXT(", Parameters.TexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Y].%s"),		InterpolatorIndex, Swizzle[(Offset+1)%2]);
		DDXCode		+= FString::Printf(TEXT(", Parameters.TexCoords_DDX[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Y].%s"),	InterpolatorIndex, Swizzle[(Offset+1)%2]);
		DDYCode		+= FString::Printf(TEXT(", Parameters.TexCoords_DDY[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Y].%s"),	InterpolatorIndex, Swizzle[(Offset+1)%2]);

		if (InterpolatorSize >= 3)
		{
			ValueCode	+= FString::Printf(TEXT(", Parameters.TexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Z].%s"),		InterpolatorIndex, Swizzle[(Offset+2)%2]);
			DDXCode		+= FString::Printf(TEXT(", Parameters.TexCoords_DDX[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Z].%s"),	InterpolatorIndex, Swizzle[(Offset+2)%2]);
			DDYCode		+= FString::Printf(TEXT(", Parameters.TexCoords_DDY[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Z].%s"),	InterpolatorIndex, Swizzle[(Offset+2)%2]);

			if (InterpolatorSize >= 4)
			{
				check(InterpolatorSize == 4);
				ValueCode	+= FString::Printf(TEXT(", Parameters.TexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_W].%s"),		InterpolatorIndex, Swizzle[(Offset+3)%2]);
				DDXCode		+= FString::Printf(TEXT(", Parameters.TexCoords_DDX[VERTEX_INTERPOLATOR_%i_TEXCOORDS_W].%s"),	InterpolatorIndex, Swizzle[(Offset+3)%2]);
				DDYCode		+= FString::Printf(TEXT(", Parameters.TexCoords_DDY[VERTEX_INTERPOLATOR_%i_TEXCOORDS_W].%s"),	InterpolatorIndex, Swizzle[(Offset+3)%2]);
			}
		}
	}

	ValueCode.Append(TEXT(")"));
	DDXCode.Append(TEXT(")"));
	DDYCode.Append(TEXT(")"));

	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(*ValueCode, *DDXCode, *DDYCode, GetDerivType(Interpolator->InterpolatedType));
		return AddCodeChunkInnerDeriv(*ValueCode, *AnalyticCode, Interpolator->InterpolatedType, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddCodeChunk(Interpolator->InterpolatedType, *ValueCode);
	}
}

bool FHLSLMaterialTranslator::CoerceConstantType(FLinearColor SourceValue, EMaterialValueType SourceType, EMaterialValueType DestType, FLinearColor& OutResult)
{
	EMaterialCastFlags CastFlags = EMaterialCastFlags::ReplicateScalar;
	if (DestType == MCT_Float || DestType == MCT_Float1 || DestType == MCT_LWCScalar)
	{
		// CoerceValue allows truncating to scalar types only
		CastFlags |= EMaterialCastFlags::AllowTruncate;
	}
	return CastConstantType(SourceValue, SourceType, DestType, CastFlags, OutResult);
}

bool FHLSLMaterialTranslator::CastConstantType(FLinearColor SourceValue, EMaterialValueType SourceType, EMaterialValueType DestType, EMaterialCastFlags Flags, FLinearColor& OutResult)
{
	const bool bAllowTruncate = EnumHasAnyFlags(Flags, EMaterialCastFlags::AllowTruncate);
	const bool bAllowAppendZeroes = EnumHasAnyFlags(Flags, EMaterialCastFlags::AllowAppendZeroes);
	bool bReplicateScalar = EnumHasAnyFlags(Flags, EMaterialCastFlags::ReplicateScalar);

	// Constants only function as floats
	SourceType = MakeNonLWCType(SourceType);
	DestType = MakeNonLWCType(DestType);

	// Assumed default value
	OutResult = SourceValue;

	if (SourceType == DestType)
	{
		return true;
	}
	else if (IsFloatNumericType(SourceType) && IsFloatNumericType(DestType))
	{
		const uint32 NumSourceComponents = GetNumComponents(SourceType);
		const uint32 NumDestComponents = GetNumComponents(DestType);
		if (NumSourceComponents != 1)
		{
			bReplicateScalar = false;
		}
		if (!bReplicateScalar && !bAllowAppendZeroes && NumDestComponents > NumSourceComponents)
		{
			Errorf(TEXT("Cannot cast from smaller type %s to larger type %s."), DescribeType(SourceType), DescribeType(DestType));
			return false;
		}
		if (!bReplicateScalar && !bAllowTruncate && NumDestComponents < NumSourceComponents)
		{
			Errorf(TEXT("Cannot cast from larger type %s to smaller type %s."), DescribeType(SourceType), DescribeType(DestType));
			return false;
		}

		FString Result;
		uint32 NumComponents = 0u;
		if (NumSourceComponents == NumDestComponents)
		{
			return true;
		}
		else if (bReplicateScalar)
		{
			OutResult = GetTypeMaskedValue(DestType, FLinearColor(SourceValue.R, SourceValue.R, SourceValue.R, SourceValue.R), nullptr);
			return true;
		}
		else
		{
			OutResult = GetTypeMaskedValue(DestType, SourceValue, nullptr);
			return true;
		}

	}

	Errorf(TEXT("Cannot cast between non-numeric types %s to %s."), DescribeType(SourceType), DescribeType(DestType));
	return false;
}

bool FHLSLMaterialTranslator::GetConstParameterValue(FMaterialUniformExpression* Expression, FLinearColor& OutConstValue)
{
	if (Expression == nullptr || !Expression->IsConstant())
	{
		return false;
	}

	FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
	Expression->GetNumberValue(DummyContext, OutConstValue);
	return true;
}

FLinearColor FHLSLMaterialTranslator::GetTypeMaskedValue(EMaterialValueType Type, FLinearColor ConstValue, bool* OutSuccess)
{
	bool bValid = true;
	FLinearColor OutValue = ConstValue;
	switch (Type)
	{
	case MCT_Float:
	case MCT_Float1:
	case MCT_LWCScalar:
		OutValue = FLinearColor(ConstValue.R, 0.0f, 0.0f, 0.0f);
		break;
	case MCT_Float2:
	case MCT_LWCVector2:
		OutValue = FLinearColor(ConstValue.R, ConstValue.G, 0.0f, 0.0f);
		break;
	case MCT_Float3:
	case MCT_LWCVector3:
		OutValue = FLinearColor(ConstValue.R, ConstValue.G, ConstValue.B, 0.0f);
		break;
	case MCT_Float4:
	case MCT_LWCVector4:
		OutValue = ConstValue;
		break;
	default:
		bValid = false;
	}

	if (OutSuccess)
	{
		*OutSuccess = bValid;
	}

	return OutValue;
}

bool FHLSLMaterialTranslator::GetConstMaskedParameterValue(EMaterialValueType Type, FMaterialUniformExpression* Expression, FLinearColor& OutConstValue)
{
	if (Expression == nullptr || !Expression->IsConstant())
	{
		return false;
	}

	FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);

	FLinearColor ConstantValue;
	Expression->GetNumberValue(DummyContext, ConstantValue);

	bool bValid;
	OutConstValue = GetTypeMaskedValue(Type, ConstantValue, &bValid);
	return bValid;
}

int32 FHLSLMaterialTranslator::LWCCastIfNeccessary(EMaterialValueType ResultType, int32 ResultCode)
{
	if (!IsNumericType(ResultType))
	{
		return Error(TEXT("LWCCastIfNeccessary Cannot cast non numeric type"));
	}

	EMaterialValueType CurrentType = GetType(ResultCode);
	if(CurrentType == ResultType)
	{
		return ResultCode;
	}

	EMaterialValueType ResultNonLWCType = MakeNonLWCType(ResultType);
	EMaterialValueType CurrentNonLWCType = MakeNonLWCType(CurrentType);

	if (ResultNonLWCType == CurrentNonLWCType || ResultNonLWCType == MCT_Float || CurrentNonLWCType == MCT_Float)
	{
		return ForceCast(ResultCode, ResultType);
	}
	return Error(TEXT("LWCCastIfNeccessary can only cast between LWC and Non LWC types.  Current result has a different number of components than requested type"));
}

int32 FHLSLMaterialTranslator::ConstResultValue(EMaterialValueType Type, FLinearColor ConstantValue)
{
	EMaterialValueType NonLWCType = MakeNonLWCType(Type);
	int32 Result;
	switch (NonLWCType)
	{
	case MCT_Float:
	case MCT_Float1:
		Result = Constant(ConstantValue.R);
		break;
	case MCT_Float2:
		Result = Constant2(ConstantValue.R, ConstantValue.G);
		break;
	case MCT_Float3:
		Result = Constant3(ConstantValue.R, ConstantValue.G, ConstantValue.B);
		break;
	case MCT_Float4:
		Result = Constant4(ConstantValue.R, ConstantValue.G, ConstantValue.B, ConstantValue.A);
		break;
	default:
		// Error would already be output by GetArithmeticResultType in this case, just return an invalid index
		return INDEX_NONE;
	}

	return LWCCastIfNeccessary(Type, Result);
}

int32 FHLSLMaterialTranslator::ConstResultValue(EMaterialValueType Type, float ConstantValue)
{
	return ConstResultValue(Type, FLinearColor(ConstantValue, ConstantValue, ConstantValue, ConstantValue));
}

int32 FHLSLMaterialTranslator::ConstArithmeticResultValue(int LeftEpression, int RightExpression, FLinearColor ConstantValue)
{
	check(LeftEpression != INDEX_NONE);
	check(RightExpression != INDEX_NONE);

	EMaterialValueType ResultType = GetArithmeticResultType(LeftEpression, RightExpression);
	return ConstResultValue(ResultType, ConstantValue);
}

int32 FHLSLMaterialTranslator::ConstArithmeticResultValue(int LeftEpression, int RightExpression, float ConstantValue)
{
	check(LeftEpression != INDEX_NONE);
	check(RightExpression != INDEX_NONE);

	EMaterialValueType ResultType = GetArithmeticResultType(LeftEpression, RightExpression);
	return ConstResultValue(ResultType, ConstantValue);
}

bool FHLSLMaterialTranslator::IsExpressionConstantValue(int Code, float ConstantValue)
{
	check(Code != INDEX_NONE);

	FLinearColor Value;
	FMaterialUniformExpression* Expression = GetParameterUniformExpression(Code);
	if (GetConstParameterValue(Expression, Value))
	{
		EMaterialValueType CodeType = (*CurrentScopeChunks)[Code].Type;
		int64_t ConstantValueInt = (int64_t)ConstantValue;

		switch (CodeType)
		{
		case MCT_Float:
		case MCT_Float1:
			return Value.R == ConstantValue;
		case MCT_Float2:
			return Value.R == ConstantValue && Value.G == ConstantValue;
		case MCT_Float3:
			return Value.R == ConstantValue && Value.G == ConstantValue && Value.B == ConstantValue;
		case MCT_Float4:
			return Value.R == ConstantValue && Value.G == ConstantValue && Value.B == ConstantValue && Value.A == ConstantValue;
		case MCT_Bool:
		case MCT_UInt:
		case MCT_UInt1:
			return (int64_t)Value.R == ConstantValueInt;
		case MCT_UInt2:
			return (int64_t)Value.R == ConstantValueInt && (int64_t)Value.G == ConstantValueInt;
		case MCT_UInt3:
			return (int64_t)Value.R == ConstantValueInt && (int64_t)Value.G == ConstantValueInt && (int64_t)Value.B == ConstantValueInt;
		case MCT_UInt4:
			return (int64_t)Value.R == ConstantValueInt && (int64_t)Value.G == ConstantValueInt && (int64_t)Value.B == ConstantValueInt && (int64_t)Value.A == ConstantValueInt;
		default:
			return false;
		}
	}

	return false;
}

int32 FHLSLMaterialTranslator::Add(int32 A, int32 B)
{
	if (A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);
	const EMaterialValueType ResultType = GetArithmeticResultType(A, B);
	if (ExpressionA && ExpressionB)
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(GetParameterUniformExpression(A), GetParameterUniformExpression(B), FMO_Add), ResultType, TEXT("(%s + %s)"), *GetParameterCode(A), *GetParameterCode(B));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Add, A, B);
		}
		else
		{
			if (ResultType & MCT_LWCType)
			{
				AddLWCFuncUsage(ELWCFunctionKind::Add);
				return AddCodeChunk(ResultType, TEXT("WSAdd(%s, %s)"), *GetParameterCode(A), *GetParameterCode(B));
			}
			else
			{
				return AddCodeChunk(ResultType, TEXT("(%s + %s)"), *GetParameterCode(A), *GetParameterCode(B));
			}
		}
	}
}

int32 FHLSLMaterialTranslator::Sub(int32 A, int32 B)
{
	if (A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);
	const EMaterialValueType ResultType = GetArithmeticResultType(A, B);
	if (ExpressionA && ExpressionB)
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(GetParameterUniformExpression(A), GetParameterUniformExpression(B), FMO_Sub), ResultType, TEXT("(%s - %s)"), *GetParameterCode(A), *GetParameterCode(B));
	}
	else
	{
		int32 ResultCode = INDEX_NONE;

		if (IsAnalyticDerivEnabled())
		{
			ResultCode = DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Sub, A, B);
		}
		else
		{
			if (ResultType & MCT_LWCType)
			{
				AddLWCFuncUsage(ELWCFunctionKind::Subtract);
				ResultCode = AddCodeChunk(ResultType, TEXT("WSSubtract(%s, %s)"), *GetParameterCode(A), *GetParameterCode(B));
			}
			else
			{
				ResultCode = AddCodeChunk(ResultType, TEXT("(%s - %s)"), *GetParameterCode(A), *GetParameterCode(B));
			}
		}

		// If both sides are LWC and we are subtracting, assume relative space going forward and truncate to single precision float.
		const EMaterialValueType AType = GetParameterType(A);
		const EMaterialValueType BType = GetParameterType(B);

		const bool bTruncateToFloat = (AType == BType) && IsLWCType(AType) && IsLWCType(BType);
		if (bTruncateToFloat)
		{
			const bool bAllowLWCTruncate = UE::MaterialTranslatorUtils::GetLWCTruncateMode() == 2; // Allow automatic truncate?
			if (bAllowLWCTruncate)
			{
				return TruncateLWC(ResultCode);
			}
		}

		return ResultCode;
	}
}

int32 FHLSLMaterialTranslator::Mul(int32 A,int32 B)
{
	if(A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialValueType ResultType = GetArithmeticResultType(A, B);

	if(ResultType == MCT_Unknown)
	{
		return INDEX_NONE;
	}

	int32 ConstOneReturnValue = INDEX_NONE;

	// If either the left or the right is const zero, assume constant zero
	if (IsExpressionConstantValue(A, 0.0f) || IsExpressionConstantValue(B, 0.0f))
	{
		return ConstArithmeticResultValue(A, B, 0.0);
	}
	else if(IsExpressionConstantValue(A, 1.0f))
	{
		ConstOneReturnValue = B;
	}
	else if (IsExpressionConstantValue(B, 1.0f))
	{
		ConstOneReturnValue = A;
	}

	if(ConstOneReturnValue != INDEX_NONE)
	{
		int32 Return = ConstOneReturnValue;
		// This should only be possible with scalar x vector arithmatic, so only attempt to append scalar onto the vector result
		while (GetNumComponents(GetParameterType(Return)) < GetNumComponents(ResultType))
		{
			Return = AppendVector(Return, ConstOneReturnValue);
		}
		return LWCCastIfNeccessary(ResultType, Return);
	}


	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);
	if (ExpressionA && ExpressionB)
	{
		if (ExpressionA->IsConstant() && ExpressionB->IsConstant())
		{
			FLinearColor ValueA, ValueB;
			GetConstParameterValue(ExpressionA, ValueA);
			GetConstParameterValue(ExpressionB, ValueB);

			FLinearColor ConstantValue = ValueA * ValueB;
			return ConstResultValue(ResultType, ConstantValue);
		}

		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(ExpressionA, ExpressionB, FMO_Mul),GetArithmeticResultType(A,B),TEXT("(%s * %s)"),*GetParameterCode(A),*GetParameterCode(B));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this,FMaterialDerivativeAutogen::EFunc2::Mul,A,B);
		}
		else
		{
			return AddCodeChunk(GetArithmeticResultType(A,B),TEXT("(%s * %s)"),*GetParameterCode(A),*GetParameterCode(B));
		}
	}
}

int32 FHLSLMaterialTranslator::Div(int32 A, int32 B)
{
	if (A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialValueType ResultType = GetArithmeticResultType(A, B);

	if (IsExpressionConstantValue(A, 0.0f))
	{
		return ConstArithmeticResultValue(A, B, 0.0);
	}
	else if(IsExpressionConstantValue(B, 1.0f))
	{
		int32 Return = A;
		// This should only be possible with scalar x vector arithmatic, so only attempt to append scalar onto the vector result
		while (GetNumComponents(GetParameterType(Return)) < GetNumComponents(ResultType))
		{
			Return = AppendVector(Return, A);
		}
		return LWCCastIfNeccessary(ResultType, Return);
	}

	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);
	if (ExpressionA && ExpressionB)
	{
		// Const division, unless it would result in division by zero, the hlsl compiler does not like inf outputs
		if (ExpressionA->IsConstant() && ExpressionB->IsConstant() && !IsExpressionConstantValue(B, 0.0f))
		{
			FLinearColor ValueA, ValueB;
			GetConstParameterValue(ExpressionA, ValueA);
			GetConstParameterValue(ExpressionB, ValueB);

			FLinearColor ConstantValue = ValueA / ValueB;
			return ConstResultValue(ResultType, ConstantValue);
		}

		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(ExpressionA, ExpressionB, FMO_Div), GetArithmeticResultType(A, B), TEXT("(%s / %s)"), *GetParameterCode(A), *GetParameterCode(B));
	}
	else if (ExpressionB && !ExpressionB->IsConstant())
	{
		// Division is often optimized as multiplication by reciprocal
		// If the divisor is a uniform expression, we can fold the reciprocal into the preshader
		int32 RcpB;
		{
			// The reciprocal is a uniform expression -- the Mul below is not, so we need the scope limited to this call
			FAddUniformExpressionScope Scope(this);
			RcpB = AddUniformExpression(Scope, new FMaterialUniformExpressionRcp(ExpressionB), GetParameterType(B), TEXT("rcp(%s)"), *GetParameterCode(B));
		}
		return Mul(A, RcpB);
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Div, A, B);
		}
		else
		{
			return AddCodeChunk(GetArithmeticResultType(A, B), TEXT("(%s / %s)"), *GetParameterCode(A), *GetParameterCode(B));
		}
	}
}

int32 FHLSLMaterialTranslator::Dot(int32 A,int32 B)
{
	if(A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	EMaterialValueType TypeA = GetParameterType(A);
	EMaterialValueType TypeB = GetParameterType(B);

	if (IsExpressionConstantValue(A, 0.0f) || IsExpressionConstantValue(B, 0.0f))
	{
		EMaterialValueType ResultType = (IsLWCType(TypeA) || IsLWCType(TypeB)) ? MCT_LWCScalar : MCT_Float;
		return LWCCastIfNeccessary(ResultType, Constant(0.0f));	
	}

	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);
	if(ExpressionA && ExpressionB)
	{
		FAddUniformExpressionScope Scope(this);
		if (TypeA == MCT_Float && TypeB == MCT_Float)
		{
			return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(ExpressionA,ExpressionB,FMO_Mul),MCT_Float,TEXT("(%s * %s)"),*GetParameterCode(A),*GetParameterCode(B));
		}
		else
		{
			if (TypeA == TypeB)
			{
				return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(ExpressionA,ExpressionB,FMO_Dot,TypeA),MCT_Float,TEXT("dot(%s,%s)"),*GetParameterCode(A),*GetParameterCode(B));
			}
			else
			{
				// Promote scalar (or truncate the bigger type)
				if (TypeA == MCT_Float || (TypeB != MCT_Float && GetNumComponents(TypeA) > GetNumComponents(TypeB)))
				{
					return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(ExpressionA,ExpressionB,FMO_Dot,TypeB),MCT_Float,TEXT("dot(%s,%s)"),*CoerceParameter(A, TypeB),*GetParameterCode(B));
				}
				else
				{
					return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(ExpressionA,ExpressionB,FMO_Dot,TypeA),MCT_Float,TEXT("dot(%s,%s)"),*GetParameterCode(A),*CoerceParameter(B, TypeA));
				}
			}
		}
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Dot, A, B);
		}
		else
		{
			// Promote scalar (or truncate the bigger type)
			if (TypeA == MCT_Float || (TypeB != MCT_Float && GetNumComponents(TypeA) > GetNumComponents(TypeB)))
			{
				return AddCodeChunk(MCT_Float,TEXT("dot(%s, %s)"), *CoerceParameter(A, TypeB), *GetParameterCode(B));
			}
			else
			{
				return AddCodeChunk(MCT_Float,TEXT("dot(%s, %s)"), *GetParameterCode(A), *CoerceParameter(B, TypeA));
			}
		}
	}
}

int32 FHLSLMaterialTranslator::Cross(int32 A,int32 B)
{
	if(A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	EMaterialValueType ResultType = GetArithmeticResultType(A, B);

	int ComponentCount = GetNumComponents(ResultType);
	bool bIsValid = ComponentCount != 2 && ComponentCount != 1;

	if(bIsValid && (IsExpressionConstantValue(A, 0.0f) || IsExpressionConstantValue(B, 0.0f)))
	{
		return ConstArithmeticResultValue(A, B, 0.0f);
	}
	
	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);
	if (ExpressionA && ExpressionB)
	{
		if (!bIsValid)
		{
			return Errorf(TEXT("Cross product requires 3-component vector input."));
		}

		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionFoldedMath(GetParameterUniformExpression(A),GetParameterUniformExpression(B),FMO_Cross,ResultType),MCT_Float3,TEXT("cross(%s,%s)"),*GetParameterCode(A),*GetParameterCode(B));
	}
	else if(IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Cross, A, B);
	}
	else
	{
		return AddCodeChunk(MCT_Float3, TEXT("cross(%s,%s)"), *CoerceParameter(A, MCT_Float3), *CoerceParameter(B, MCT_Float3));
	}
}

int32 FHLSLMaterialTranslator::Power(int32 Base,int32 Exponent)
{
	if(Base == INDEX_NONE || Exponent == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExponentExpression = GetParameterUniformExpression(Exponent);
	FMaterialUniformExpression* BaseExpression = GetParameterUniformExpression(Base);

	// Pow can only have known constant results if the exponent is const
	// Having a constant base of 0 will result in zero, unless the exponent is 0, so it cannot
	// be assumed
	if (ExponentExpression && ExponentExpression->IsConstant())
	{
		// This will validate that the parameter types are valid, even though we already
		// know the result type must be the base type
		if (GetArithmeticResultType(Base, Exponent) == MCT_Unknown)
		{
			return INDEX_NONE;
		}

		int32 ConstantResult = INDEX_NONE;
		if (IsExpressionConstantValue(Exponent, 1.0f))
		{
			ConstantResult = Base;
		}
		else if (IsExpressionConstantValue(Exponent, 0.0f))
		{
			ConstantResult = Constant(1.0f);
		}
		else if (IsExpressionConstantValue(Base, 0.0f))
		{
			ConstantResult = Constant(0.0f);
		}

		if(ConstantResult != INDEX_NONE)
		{
			EMaterialValueType ConstResultType = GetParameterType(Base);
			int32 Return = ConstantResult;
			// This should only be possible with scalar x vector arithmatic, so only attempt to append scalar onto the vector result
			while (GetNumComponents(GetParameterType(Return)) < GetNumComponents(ConstResultType))
			{
				Return = AppendVector(Return, ConstantResult);
			}

			if(GetNumComponents(GetParameterType(Return)) < GetNumComponents(ConstResultType))
			{
				int32 NumChannels = GetNumComponents(ConstResultType);
				Return = ComponentMask(Return, true, NumChannels > 1, NumChannels > 2, NumChannels > 3);
			}

			return LWCCastIfNeccessary(ConstResultType, Return);
		}
	}


	// The code in these two does not seem to match.
	// The first path does not coerce the exponent value, which will result in all component getting pow for their corresponding exponent
	// The second path forces the exponent to a float, which makes all components getting pow with only that component
	if (IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc2(*this,FMaterialDerivativeAutogen::EFunc2::PowPositiveClamped,Base,Exponent);
	}
	else
	{
		// Clamp Pow input to >= 0 to help avoid common NaN cases
		return AddCodeChunk(GetParameterType(Base),TEXT("PositiveClampedPow(%s,%s)"),*GetParameterCode(Base),*CoerceParameter(Exponent,MCT_Float));
	}
}

int32 FHLSLMaterialTranslator::Exponential(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionExponential(GetParameterUniformExpression(X)), GetParameterType(X), TEXT("exp(%s)"), *GetParameterCode(X));
	}
	else if(IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Exp, X);
	}
	else
	{
		return AddCodeChunk(GetParameterType(X), TEXT("exp(%s)"), *GetParameterCode(X));
	}
}

int32 FHLSLMaterialTranslator::Exponential2(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionExponential2(GetParameterUniformExpression(X)), GetParameterType(X), TEXT("exp2(%s)"), *GetParameterCode(X));
	}
	else if(IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Exp2, X);
	}
	else
	{
		return AddCodeChunk(GetParameterType(X), TEXT("exp2(%s)"), *GetParameterCode(X));
	}
}

int32 FHLSLMaterialTranslator::Logarithm(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionLogarithm(GetParameterUniformExpression(X)), GetParameterType(X), TEXT("log(%s)"), *GetParameterCode(X));
	}
	else if(IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Log, X);
	}
	else
	{
		return AddCodeChunk(GetParameterType(X), TEXT("log(%s)"), *GetParameterCode(X));
	}
}
	
int32 FHLSLMaterialTranslator::Logarithm2(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionLogarithm2(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("log2(%s)"),*GetParameterCode(X));
	}
	else if (IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Log2, X);
	}
	else
	{
		return AddCodeChunk(GetParameterType(X), TEXT("log2(%s)"), *GetParameterCode(X));
	}
}

int32 FHLSLMaterialTranslator::Logarithm10(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionLogarithm10(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("log10(%s)"),*GetParameterCode(X));
	}
	else if (IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Log10, X);
	}
	else
	{
		return AddCodeChunk(GetParameterType(X), TEXT("log10(%s)"), *GetParameterCode(X));
	}
}

int32 FHLSLMaterialTranslator::SquareRoot(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if(GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionSquareRoot(GetParameterUniformExpression(X)),GetParameterType(X),TEXT("sqrt(%s)"),*GetParameterCode(X));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this,FMaterialDerivativeAutogen::EFunc1::Sqrt,X);
		}
		else
		{
		return AddCodeChunk(GetParameterType(X),TEXT("sqrt(%s)"),*GetParameterCode(X));
	}
}
}

int32 FHLSLMaterialTranslator::Length(int32 X)
{
	if (X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionLength(GetParameterUniformExpression(X), GetParameterType(X)), MCT_Float, TEXT("length(%s)"), *GetParameterCode(X));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Length, X);
		}
		else
		{
			return AddCodeChunk(MCT_Float, TEXT("length(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Normalize(int32 X)
{
	if (X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialValueType ResultType = MakeNonLWCType(GetParameterType(X));

	if (GetParameterUniformExpression(X))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionNormalize(GetParameterUniformExpression(X)), ResultType, TEXT("normalize(%s)"), *GetParameterCode(X));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this, FMaterialDerivativeAutogen::EFunc1::Normalize, X);
		}
		else
		{
			return AddCodeChunk(ResultType, TEXT("normalize(%s)"), *GetParameterCode(X));
		}
	}
}

int32 FHLSLMaterialTranslator::Step(int32 Y, int32 X)
{
	if (X == INDEX_NONE || Y == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionX = GetParameterUniformExpression(X);
	FMaterialUniformExpression* ExpressionY = GetParameterUniformExpression(Y);

	EMaterialValueType ResultType = GetArithmeticResultType(X, Y);

	//Constant folding.
	if (ExpressionX && ExpressionY)
	{
		// when x == y return 1.0f
		if (ExpressionX == ExpressionY)
		{
			const int32 EqualResult = 1.0f;
			return ConstResultValue(ResultType, EqualResult);
		}

		if (ExpressionX->IsConstant() && ExpressionY->IsConstant())
		{
			FLinearColor ValueX, ValueY;
			FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
			ExpressionX->GetNumberValue(DummyContext, ValueX);
			ExpressionY->GetNumberValue(DummyContext, ValueY);

			float Red = ValueX.R >= ValueY.R ? 1 : 0;
			if (ResultType == MCT_Float || ResultType == MCT_Float1)
			{
				return Constant(Red);
			}

			float Green = ValueX.G >= ValueY.G ? 1 : 0;
			if (ResultType == MCT_Float2)
			{
				return Constant2(Red, Green);
			}

			float Blue = ValueX.B >= ValueY.B ? 1 : 0;
			if (ResultType == MCT_Float3)
			{
				return Constant3(Red, Green, Blue);
			}

			float Alpha = ValueX.A >= ValueY.A ? 1 : 0;
			if (ResultType == MCT_Float4)
			{
				return Constant4(Red, Green, Blue, Alpha);
			}
		}
	}

	if (IsLWCType(ResultType))
	{
		AddLWCFuncUsage(ELWCFunctionKind::Other);
		return AddCodeChunkZeroDeriv(MakeNonLWCType(ResultType), TEXT("WSStep(%s,%s)"), *CoerceParameter(Y, ResultType), *CoerceParameter(X, ResultType));
	}
	else
	{
		return AddCodeChunkZeroDeriv(ResultType, TEXT("step(%s,%s)"), *CoerceParameter(Y, ResultType), *CoerceParameter(X, ResultType));
	}
}

int32 FHLSLMaterialTranslator::SmoothStep(int32 X, int32 Y, int32 A)
{
	if (X == INDEX_NONE || Y == INDEX_NONE || A == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionX = GetParameterUniformExpression(X);
	FMaterialUniformExpression* ExpressionY = GetParameterUniformExpression(Y);
	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	bool bExpressionsAreEqual = false;

	// According to https://docs.microsoft.com/en-us/windows/win32/direct3dhlsl/dx-graphics-hlsl-smoothstep
	// Smoothstep's min and max and return result in the same size as the alpha.
	// Therefore the result type (and each input) should be GetParameterType(A);

	// However, for usability reasons, we will use the ArithmiticType of the three.
	// This is important to do, because it allows a user to input a vector into the min or max
	// and get a vector result, without putting inputs into the other two constants.
	// This is not exactly the behavior of raw HLSL, but it is a more intuitive experience
	// and mimics more closely the LinearInterpolate node.
	// Incompatible inputs will be caught by the CoerceParameters below.

	EMaterialValueType ResultType = GetArithmeticResultType(X, Y);
	ResultType = GetArithmeticResultType(ResultType, GetParameterType(A));


	// Skip over interpolations where inputs are equal

	float EqualResult = 0.0f;
	// smoothstep( x, y, y ) == 1.0
	if (Y == A)
	{
		bExpressionsAreEqual = true;
		EqualResult = 1.0f;
	}

	// smoothstep( x, y, x ) == 0.0
	if (X == A)
	{
		bExpressionsAreEqual = true;
		EqualResult = 0.0f;
	}

	if (bExpressionsAreEqual)
	{
		return ConstResultValue(ResultType, EqualResult);
	}

	// smoothstep( x, x, a ) could create a div by zero depending on implementation.
	// The common implementation is to treat smoothstep as a step in these situations.
	if (X == Y)
	{
		bExpressionsAreEqual = true;
	}
	else if (ExpressionX && ExpressionY)
	{
		if (ExpressionX->IsConstant() && ExpressionY->IsConstant() && (*CurrentScopeChunks)[X].Type == (*CurrentScopeChunks)[Y].Type)
		{
			FLinearColor ValueX, ValueY;
			FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
			ExpressionX->GetNumberValue(DummyContext, ValueX);
			ExpressionY->GetNumberValue(DummyContext, ValueY);

			if (ValueX == ValueY)
			{
				bExpressionsAreEqual = true;
			}
		}
	}

	if (bExpressionsAreEqual)
	{
		return Step(X, A);
	}

	//When all inputs are constant, we can precompile the operation.
	if (ExpressionX && ExpressionY && ExpressionA && ExpressionX->IsConstant() && ExpressionY->IsConstant() && ExpressionA->IsConstant())
	{
		FLinearColor ValueX, ValueY, ValueA;
		FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
		ExpressionX->GetNumberValue(DummyContext, ValueX);
		ExpressionY->GetNumberValue(DummyContext, ValueY);
		ExpressionA->GetNumberValue(DummyContext, ValueA);

		float Red = FMath::SmoothStep(ValueX.R, ValueY.R, ValueA.R);
		if (ResultType == MCT_Float || ResultType == MCT_Float1)
		{
			return Constant(Red);
		}

		float Green = FMath::SmoothStep(ValueX.G, ValueY.G, ValueA.G);
		if (ResultType == MCT_Float2)
		{
			return Constant2(Red, Green);
		}

		float Blue = FMath::SmoothStep(ValueX.B, ValueY.B, ValueA.B);
		if (ResultType == MCT_Float3)
		{
			return Constant3(Red, Green, Blue);
		}

		float Alpha = FMath::SmoothStep(ValueX.A, ValueY.A, ValueA.A);
		if (ResultType == MCT_Float4)
		{
			return Constant4(Red, Green, Blue, Alpha);
		}
	}

	if (IsLWCType(ResultType))
	{
		AddLWCFuncUsage(ELWCFunctionKind::Other);
		return AddCodeChunk(MakeNonLWCType(ResultType), TEXT("WSSmoothStepDemote(%s,%s,%s)"), *CoerceParameter(X, ResultType), *CoerceParameter(Y, ResultType), *CoerceParameter(A, ResultType));
	}
	else
	{
		return AddCodeChunk(ResultType, TEXT("smoothstep(%s,%s,%s)"), *CoerceParameter(X, ResultType), *CoerceParameter(Y, ResultType), *CoerceParameter(A, ResultType));
	}
}

int32 FHLSLMaterialTranslator::InvLerp(int32 X, int32 Y, int32 A)
{
	if (X == INDEX_NONE || Y == INDEX_NONE || A == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionX = GetParameterUniformExpression(X);
	FMaterialUniformExpression* ExpressionY = GetParameterUniformExpression(Y);
	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	bool bExpressionsAreEqual = false;

	EMaterialValueType ResultType = GetParameterType(A);


	// Skip over interpolations where inputs are equal.

	float EqualResult = 0.0f;
	// (y-x)/(y-x) == 1.0
	if (Y == A)
	{
		bExpressionsAreEqual = true;
		EqualResult = 1.0;
	}

	// (x-x)/(y-x) == 0.0
	if (X == A)
	{
		bExpressionsAreEqual = true;
		EqualResult = 0.0f;
	}

	if (bExpressionsAreEqual)
	{
		return ConstResultValue(ResultType, EqualResult);
	}

	// (a-x)/(x-x) will create a div by zero.
	if (X == Y)
	{
		bExpressionsAreEqual = true;
	}
	else if (ExpressionX && ExpressionY)
	{
		if (ExpressionX->IsConstant() && ExpressionY->IsConstant() && (*CurrentScopeChunks)[X].Type == (*CurrentScopeChunks)[Y].Type)
		{
			FLinearColor ValueX, ValueY;
			FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
			ExpressionX->GetNumberValue(DummyContext, ValueX);
			ExpressionY->GetNumberValue(DummyContext, ValueY);

			if (ValueX == ValueY)
			{
				bExpressionsAreEqual = true;
			}
		}
	}

	if (bExpressionsAreEqual)
	{
		Error(TEXT("Div by Zero: InvLerp A == B."));
	}

	//When all inputs are constant, we can precompile the operation.
	if (ExpressionX && ExpressionY && ExpressionA && ExpressionX->IsConstant() && ExpressionY->IsConstant() && ExpressionA->IsConstant())
	{
		FLinearColor ValueX, ValueY, ValueA;
		FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
		ExpressionX->GetNumberValue(DummyContext, ValueX);
		ExpressionY->GetNumberValue(DummyContext, ValueY);
		ExpressionA->GetNumberValue(DummyContext, ValueA);

		float Red = FMath::GetRangePct(ValueX.R, ValueY.R, ValueA.R);
		if (ResultType == MCT_Float || ResultType == MCT_Float1)
		{
			return Constant(Red);
		}

		float Green = FMath::GetRangePct(ValueX.G, ValueY.G, ValueA.G);
		if (ResultType == MCT_Float2)
		{
			return Constant2(Red, Green);
		}

		float Blue = FMath::GetRangePct(ValueX.B, ValueY.B, ValueA.B);
		if (ResultType == MCT_Float3)
		{
			return Constant3(Red, Green, Blue);
		}

		float Alpha = FMath::GetRangePct(ValueX.A, ValueY.A, ValueA.A);
		if (ResultType == MCT_Float4)
		{
			return Constant4(Red, Green, Blue, Alpha);
		}
	}

	int32 Numerator = Sub(A, X);
	int32 Denominator = Sub(Y, X);

	return Div(Numerator, Denominator);
}

int32 FHLSLMaterialTranslator::Lerp(int32 X,int32 Y,int32 A)
{
	if (X == INDEX_NONE || Y == INDEX_NONE || A == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionX = GetParameterUniformExpression(X);
	FMaterialUniformExpression* ExpressionY = GetParameterUniformExpression(Y);
	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	bool bExpressionsAreEqual = false;

	// Skip over interpolations where inputs are equal
	if (X == Y)
	{
		bExpressionsAreEqual = true;
	}
	else if (ExpressionX && ExpressionY)
	{
		if (ExpressionX->IsConstant() && ExpressionY->IsConstant() && (*CurrentScopeChunks)[X].Type == (*CurrentScopeChunks)[Y].Type)
		{
			FLinearColor ValueX, ValueY;
			FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
			ExpressionX->GetNumberValue(DummyContext, ValueX);
			ExpressionY->GetNumberValue(DummyContext, ValueY);

			if (ValueX == ValueY)
			{
				bExpressionsAreEqual = true;
			}
		}
	}

	if (bExpressionsAreEqual)
	{
		return X;
	}

	EMaterialValueType ResultType = GetArithmeticResultType(X,Y);
	EMaterialValueType AlphaType = (*CurrentScopeChunks)[A].Type;
	
	if (!IsPrimitiveType(AlphaType))
	{
		Errorf(TEXT("Lerp alpha argument is not primitive (it is '%s' instead)"), DescribeType(AlphaType));
		return INDEX_NONE;
	}

	if ((AlphaType & MCT_Float) != 0 && ExpressionA && ExpressionA->IsConstant())
	{
		// Skip over interpolations that explicitly select an input
		FLinearColor AlphaValue;
		FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
		ExpressionA->GetNumberValue(DummyContext, AlphaValue);

		const EMaterialValueType ResultFloatType = MakeNonLWCType(ResultType);
		const EMaterialValueType AlphaCastType = (ResultFloatType == AlphaType) ? ResultFloatType : MCT_Float1;

		if (!CastConstantType(AlphaValue, AlphaCastType, ResultFloatType, EMaterialCastFlags::ReplicateScalar | EMaterialCastFlags::AllowTruncate, AlphaValue))
		{
			return INDEX_NONE;
		}

		// Cast will not change the value if the types are the same, but we want the true masked value for comparisons below
		AlphaValue = GetTypeMaskedValue(ResultFloatType, AlphaValue, nullptr);
		if (AlphaCastType == MCT_Float1 && ResultFloatType != MCT_Float1)
		{
			AlphaValue = GetTypeMaskedValue(ResultFloatType, FLinearColor(AlphaValue.R, AlphaValue.R, AlphaValue.R, AlphaValue.R), nullptr);
		}

		const FLinearColor ZeroValue = GetTypeMaskedValue(ResultFloatType, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), nullptr);
		const FLinearColor OneValue = GetTypeMaskedValue(ResultFloatType, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), nullptr);
		
		if (AlphaValue == ZeroValue)
		{
			return X;
		}
		else if (AlphaValue == OneValue)
		{
			return Y;
		}

		// If all inputs are constant and we have a float result type, produce a constant result
		if ((ResultType & MCT_Float) != 0 && ExpressionX && ExpressionY && ExpressionX->IsConstant() && ExpressionY->IsConstant())
		{
			FLinearColor ValueX, ValueY;

			ExpressionX->GetNumberValue(DummyContext, ValueX);
			ExpressionY->GetNumberValue(DummyContext, ValueY);
			if (!CastConstantType(ValueX, (*CurrentScopeChunks)[X].Type, ResultType, EMaterialCastFlags::ReplicateScalar, ValueX))
			{
				return INDEX_NONE;
			}
			if (!CastConstantType(ValueY, (*CurrentScopeChunks)[Y].Type, ResultType, EMaterialCastFlags::ReplicateScalar, ValueY))
			{
				return INDEX_NONE;
			}
			FLinearColor ConstLerp = FMath::Lerp(ValueX, ValueY, AlphaValue);
			return ConstResultValue(ResultType, ConstLerp);
		}
	}

	if (IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateLerpFunc(*this, X, Y, A);
	}
	else
	{
		return AddCodeChunk(ResultType,TEXT("lerp(%s, %s, %s)"),*CoerceParameter(X,ResultType),*CoerceParameter(Y,ResultType),*CoerceParameter(A,AlphaType));
	}
}

int32 FHLSLMaterialTranslator::Min(int32 A,int32 B)
{
	if(A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);

	if(ExpressionA && ExpressionB)
	{
		FLinearColor ValueA, ValueB;
		if (GetConstParameterValue(ExpressionA, ValueA) && GetConstParameterValue(ExpressionB, ValueB))
		{
			EMaterialValueType TypeA = GetParameterType(A);
			EMaterialValueType TypeB = GetParameterType(B);
			FLinearColor Result;
			if (!CoerceConstantType(ValueB, TypeB, TypeA, ValueB))
			{
				return INDEX_NONE;
			}

			Result.R = FMath::Min(ValueA.R, ValueB.R);
			Result.G = FMath::Min(ValueA.G, ValueB.G);
			Result.B = FMath::Min(ValueA.B, ValueB.B);
			Result.A = FMath::Min(ValueA.A, ValueB.A);

			return ConstResultValue(TypeA, Result);
		}
		
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionMin(ExpressionA, ExpressionB),GetParameterType(A),TEXT("min(%s,%s)"),*GetParameterCode(A),*CoerceParameter(B,GetParameterType(A)));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this,FMaterialDerivativeAutogen::EFunc2::Min,A,B);
		}
		else
		{
			return AddCodeChunk(GetParameterType(A),TEXT("min(%s,%s)"),*GetParameterCode(A),*CoerceParameter(B,GetParameterType(A)));
		}
	}
}

int32 FHLSLMaterialTranslator::Max(int32 A,int32 B)
{
	if(A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* ExpressionA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(B);

	if (ExpressionA && ExpressionB)
	{
		FLinearColor ValueA, ValueB;
		if (GetConstParameterValue(ExpressionA, ValueA) && GetConstParameterValue(ExpressionB, ValueB))
		{
			EMaterialValueType TypeA = GetParameterType(A);
			EMaterialValueType TypeB = GetParameterType(B);
			FLinearColor Result;
			if (!CoerceConstantType(ValueB, TypeB, TypeA, ValueB))
			{
				return INDEX_NONE;
			}

			Result.R = FMath::Max(ValueA.R, ValueB.R);
			Result.G = FMath::Max(ValueA.G, ValueB.G);
			Result.B = FMath::Max(ValueA.B, ValueB.B);
			Result.A = FMath::Max(ValueA.A, ValueB.A);

			return ConstResultValue(TypeA, Result);
		}

		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionMax(ExpressionA, ExpressionB),GetParameterType(A),TEXT("max(%s,%s)"),*GetParameterCode(A),*CoerceParameter(B,GetParameterType(A)));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc2(*this,FMaterialDerivativeAutogen::EFunc2::Max,A,B);
		}
		else
		{
			return AddCodeChunk(GetParameterType(A),TEXT("max(%s,%s)"),*GetParameterCode(A),*CoerceParameter(B,GetParameterType(A)));
		}
	}
}

int32 FHLSLMaterialTranslator::Clamp(int32 X, int32 A, int32 B)
{
	if (X == INDEX_NONE || A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* UniformA = GetParameterUniformExpression(A);
	FMaterialUniformExpression* UniformB = GetParameterUniformExpression(B);
	FMaterialUniformExpression* UniformX = GetParameterUniformExpression(X);

	FLinearColor ValueA, ValueB, ValueX;
	bool AIsConst = GetConstParameterValue(UniformA, ValueA);
	bool BIsConst = GetConstParameterValue(UniformB, ValueB);
	bool XIsConst = GetConstParameterValue(UniformX, ValueX);

	if ((AIsConst || BIsConst) && XIsConst)
	{
		// Just use Min and Max, this will const collapse one or both max/min operations
		// All parameters are const, just use max and mins const collapse
		int32 AResult = Max(X, A);
		return Min(AResult, B);
	}
	else if (AIsConst && BIsConst)
	{
		// Check to see if we're clamping between 0-1, in that case we can use Saturate, which has some additional optimizations
		if (ValueA == FLinearColor(0.0f, 0.0f, 0.0f, 0.0f) && ValueB == FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
		{
			return Saturate(X);
		}
	}

	if (UniformX && UniformA && UniformB)
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionClamp(UniformX, UniformA, UniformB), GetParameterType(X), TEXT("min(max(%s,%s),%s)"), *GetParameterCode(X), *CoerceParameter(A, GetParameterType(X)), *CoerceParameter(B, GetParameterType(X)));
	}

	if (IsAnalyticDerivEnabled())
	{
		// Make sure 'X' is given as first parameter, to ensure proper type coercion
		// LWC_TODO - should handle the case where an LWC value is clamped between 2 scalars, could use optimized clamping function in that case
		int32 MaxXA = DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Max, X, A);
		int32 Result = DerivativeAutogen.GenerateExpressionFunc2(*this, FMaterialDerivativeAutogen::EFunc2::Min, MaxXA, B);
		return Result;
	}
	else
	{
		return AddCodeChunk(GetParameterType(X), TEXT("min(max(%s,%s),%s)"), *GetParameterCode(X), *CoerceParameter(A, GetParameterType(X)), *CoerceParameter(B, GetParameterType(X)));
	}
}

int32 FHLSLMaterialTranslator::Saturate(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialUniformExpression* UniformX = GetParameterUniformExpression(X);
	FLinearColor ValueX;

	if (GetConstParameterValue(UniformX, ValueX))
	{
		// Just use Min and Max, this will const collapse one or both max/min operations
		// All parameters are const, just use max and mins const collapse
		int32 AResult = Max(X, Constant(0.0f));
		return Min(AResult, Constant(1.0f));
	}

	if(UniformX)
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionSaturate(UniformX),GetParameterType(X),TEXT("saturate(%s)"),*GetParameterCode(X));
	}
	else
	{
		if (IsAnalyticDerivEnabled())
		{
			return DerivativeAutogen.GenerateExpressionFunc1(*this,FMaterialDerivativeAutogen::EFunc1::Saturate,X);
		}
		else
		{
		return AddCodeChunk(GetParameterType(X),TEXT("saturate(%s)"),*GetParameterCode(X));
	}
}
}

static FString ComponentMaskLWC(const FString& SourceString, EMaterialValueType SourceType, EMaterialValueType ResultType, bool R, bool G, bool B, bool A)
{
	const TCHAR* ComponentAccess[] =
	{
		TEXT("WSGetX"),
		TEXT("WSGetY"),
		TEXT("WSGetZ"),
		TEXT("WSGetW"),
	};

	bool bNeedClosingParen = false;
	bool bNeedComma = false;
	FString Result;
	if (ResultType != MCT_LWCScalar)
	{
		Result = TEXT("MakeWSVector(");
		bNeedClosingParen = true;
	}
	if (R)
	{
		Result += FString::Printf(TEXT("%s(%s)"), ComponentAccess[0], *SourceString);
		bNeedComma = true;
	}
	if (G)
	{
		if (bNeedComma) Result += TEXT(", ");
		Result += FString::Printf(TEXT("%s(%s)"), ComponentAccess[SourceType == MCT_LWCScalar ? 0 : 1], *SourceString);
		bNeedComma = true;
	}
	if (B)
	{
		if (bNeedComma) Result += TEXT(", ");
		Result += FString::Printf(TEXT("%s(%s)"), ComponentAccess[SourceType == MCT_LWCScalar ? 0 : 2], *SourceString);
		bNeedComma = true;
	}
	if (A)
	{
		if (bNeedComma) Result += TEXT(", ");
		Result += FString::Printf(TEXT("%s(%s)"), ComponentAccess[SourceType == MCT_LWCScalar ? 0 : 3], *SourceString);
		bNeedComma = true;
	}
	if (bNeedClosingParen)
	{
		Result += TEXT(")");
	}
	return Result;
}

int32 FHLSLMaterialTranslator::VectorComponent(int32 Vector, int32 Component)
{
	if ((Vector == INDEX_NONE) || (Component == INDEX_NONE))
	{
		return INDEX_NONE;
	}
	EMaterialValueType VectorType = GetParameterType(Vector);
	EMaterialValueType ComponentType = GetParameterType(Component);
	if (!(VectorType & (MCT_Float2 | MCT_Float3 | MCT_Float4 | MCT_UInt2 | MCT_UInt3 | MCT_UInt4)))
	{
		return Errorf(TEXT("Vector type is invalid (%s: %s) for VectorComponent instruction"), *GetParameterCode(Vector), DescribeType(VectorType));
	}
	if (!(ComponentType & (MCT_Float1 | MCT_UInt1)))
	{
		return Errorf(TEXT("Component type is invalid (%s: %s) for VectorComponent instruction"), *GetParameterCode(Component), DescribeType(ComponentType));
	}

	EMaterialValueType ResultType = (VectorType & MCT_Float ? MCT_Float1 : MCT_UInt1);

	return AddCodeChunk(ResultType, TEXT("%s[int(%s)]"), *GetParameterCode(Vector), *GetParameterCode(Component));
}

int32 FHLSLMaterialTranslator::BitfieldExtract(int32 Value, int32 BitOffset, int32 BitCount)
{
	if ((Value == INDEX_NONE) || (BitOffset == INDEX_NONE) || (BitCount == INDEX_NONE))
	{
		return INDEX_NONE;
	}
	EMaterialValueType ValueType = GetParameterType(Value);
	EMaterialValueType BitOffsetType = GetParameterType(BitOffset);
	EMaterialValueType BitCountType = GetParameterType(BitCount);
	if (!(ValueType & (MCT_Float1 | MCT_UInt1)))
	{
		return Errorf(TEXT("Value type is invalid (%s: %s) for BitfieldExtract instruction"), *GetParameterCode(Value), DescribeType(ValueType));
	}
	if (!(BitOffsetType & (MCT_Float1 | MCT_UInt1)))
	{
		return Errorf(TEXT("BitOffset type is invalid (%s: %s) for BitfieldExtract instruction"), *GetParameterCode(BitOffset), DescribeType(BitOffsetType));
	}
	if (!(BitCountType & (MCT_Float1 | MCT_UInt1)))
	{
		return Errorf(TEXT("BitCount type is invalid (%s: %s) for BitfieldExtract instruction"), *GetParameterCode(BitCount), DescribeType(BitCountType));
	}

	int32 Mask_IDX = AddCodeChunk(MCT_UInt1, TEXT("~(0xffffffff << uint(%s))"), *GetParameterCode(BitCount));

	int32 IntValue_IDX = Value;
	if (ValueType & MCT_Float1)
	{
		IntValue_IDX = AddCodeChunk(MCT_UInt1, TEXT("asuint(%s)"), *GetParameterCode(Value));
	}

	int32 IntBitOffset_IDX = BitOffset;
	if (BitOffsetType & MCT_Float1)
	{
		IntBitOffset_IDX = AddCodeChunk(MCT_UInt1, TEXT("uint(%s)"), *GetParameterCode(BitOffset));
	}

	return AddCodeChunk(MCT_UInt1, TEXT("(%s >> %s) & %s"), *GetParameterCode(IntValue_IDX), *GetParameterCode(IntBitOffset_IDX), *GetParameterCode(Mask_IDX));
}

int32 FHLSLMaterialTranslator::ComponentMask(int32 Vector,bool R,bool G,bool B,bool A)
{
	if(Vector == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	EMaterialValueType VectorType = GetParameterType(Vector);
	uint32 NumValidComponents = 0u;
	if (VectorType & MCT_Float4) NumValidComponents = 4u;
	else if (VectorType & MCT_Float3) NumValidComponents = 3u;
	else if (VectorType & MCT_Float2) NumValidComponents = 2u;
	else if (VectorType & MCT_Float1) NumValidComponents = 1u;
	else if (VectorType == MCT_LWCScalar) NumValidComponents = 4u; // Allow .gba mask on scalar values, will use .rrrr as needed
	else if (VectorType == MCT_LWCVector2) NumValidComponents = 2u;
	else if (VectorType == MCT_LWCVector3) NumValidComponents = 3u;
	else if (VectorType == MCT_LWCVector4) NumValidComponents = 4u;

	if(	(A && NumValidComponents < 4u) ||
		(B && NumValidComponents < 3u) ||
		(G && NumValidComponents < 2u) ||
		(R && NumValidComponents < 1u))
	{
		return Errorf(TEXT("Not enough components in (%s: %s) for component mask %u%u%u%u"),*GetParameterCode(Vector),DescribeType(VectorType),R,G,B,A);
	}

	const bool bIsLWC = (VectorType & MCT_LWCType);
	EMaterialValueType ResultType;
	switch((R ? 1 : 0) + (G ? 1 : 0) + (B ? 1 : 0) + (A ? 1 : 0))
	{
	case 1: ResultType = bIsLWC ? MCT_LWCScalar : MCT_Float; break;
	case 2: ResultType = bIsLWC ? MCT_LWCVector2 : MCT_Float2; break;
	case 3: ResultType = bIsLWC ? MCT_LWCVector3 : MCT_Float3; break;
	case 4: ResultType = bIsLWC ? MCT_LWCVector4 : MCT_Float4; break;
	default: 
		return Errorf(TEXT("Couldn't determine result type of component mask %u%u%u%u"),R,G,B,A);
	};

	FString MaskString = FString::Printf(TEXT("%s%s%s%s"),
		R ? TEXT("r") : TEXT(""),
		// If VectorType is set to MCT_Float/MCT_LWCScalar which means it could be any of the float types, assume it is a float1
		G ? ((VectorType == MCT_Float || VectorType == MCT_LWCScalar) ? TEXT("r") : TEXT("g")) : TEXT(""),
		B ? ((VectorType == MCT_Float || VectorType == MCT_LWCScalar) ? TEXT("r") : TEXT("b")) : TEXT(""),
		A ? ((VectorType == MCT_Float || VectorType == MCT_LWCScalar) ? TEXT("r") : TEXT("a")) : TEXT("")
		);

	auto* Expression = GetParameterUniformExpression(Vector);
	if (Expression)
	{
		int8 Mask[4] = {-1, -1, -1, -1};
		for (int32 Index = 0; Index < MaskString.Len(); ++Index)
		{
			Mask[Index] = SwizzleComponentToIndex(MaskString[Index]);
		}
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(
			Scope,
			new FMaterialUniformExpressionComponentSwizzle(Expression, Mask[0], Mask[1], Mask[2], Mask[3]),
			ResultType,
			TEXT("%s.%s"),
			*GetParameterCode(Vector),
			*MaskString
			);
	}

	FString SourceString = GetParameterCode(Vector);

	const EDerivativeStatus VectorDerivStatus = GetDerivativeStatus(Vector);
	FString CodeFinite;
	if (bIsLWC)
	{
		CodeFinite = ComponentMaskLWC(SourceString, VectorType, ResultType, R, G, B, A);
	}
	else
	{
		CodeFinite = FString::Printf(TEXT("%s.%s"), *SourceString, *MaskString);
	}

	if (IsAnalyticDerivEnabled() && IsDerivativeValid(VectorDerivStatus))
	{
		const FString VectorDeriv = GetParameterCodeDeriv(Vector, CompiledPDV_Analytic);

		FString CodeAnalytic;
		if (VectorDerivStatus == EDerivativeStatus::Valid)
		{
			FString ValueDeriv;
			if (bIsLWC)
			{
				ValueDeriv = ComponentMaskLWC(VectorDeriv + TEXT(".Value"), VectorType, ResultType, R, G, B, A);
			}
			else
			{
				ValueDeriv = FString::Printf(TEXT("%s.Value.%s"), *VectorDeriv, *MaskString);
			}

			CodeAnalytic = DerivativeAutogen.ConstructDeriv(
				ValueDeriv,
				FString::Printf(TEXT("%s.Ddx.%s"), *VectorDeriv, *MaskString),
				FString::Printf(TEXT("%s.Ddy.%s"), *VectorDeriv, *MaskString),
				GetDerivType(ResultType));

			return AddCodeChunkInnerDeriv(*CodeFinite, *CodeAnalytic, ResultType, false, EDerivativeStatus::Valid);
		}
		else
		{
			return AddInlinedCodeChunkZeroDeriv(ResultType, *CodeFinite);
		}
	}
	else
	{
		return AddInlinedCodeChunk(ResultType, *CodeFinite);
	}
}

int32 FHLSLMaterialTranslator::AppendVector(int32 A,int32 B)
{
	if(A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialValueType TypeA = GetParameterType(A);
	const EMaterialValueType TypeB = GetParameterType(B);
	const bool bIsLWC = IsLWCType(TypeA) || IsLWCType(TypeB);
	const int32 NumComponentsA = GetNumComponents(TypeA);
	const int32 NumComponentsB = GetNumComponents(TypeB);
	const int32 NumResultComponents = NumComponentsA + NumComponentsB;
	if (NumResultComponents > 4)
	{
		return Errorf(TEXT("Can't append %s to %s"), DescribeType(TypeA), DescribeType(TypeB));
	}

	const EMaterialValueType ResultType = bIsLWC ? GetLWCVectorType(NumResultComponents) : GetVectorType(NumResultComponents);

	if(GetParameterUniformExpression(A) && GetParameterUniformExpression(B))
	{
		FAddUniformExpressionScope Scope(this);
		return AddUniformExpression(Scope, new FMaterialUniformExpressionAppendVector(GetParameterUniformExpression(A),GetParameterUniformExpression(B),GetNumComponents(GetParameterType(A))),ResultType,TEXT("MaterialFloat%u(%s,%s)"),NumResultComponents,*GetParameterCode(A),*GetParameterCode(B));
	}
	else
	{
		FString FiniteCode;
		if (bIsLWC)
		{
			AddLWCFuncUsage(ELWCFunctionKind::Promote, 2);
			AddLWCFuncUsage(ELWCFunctionKind::Constructor);
			FiniteCode = FString::Printf(TEXT("MakeWSVector(WSPromote(%s),WSPromote(%s))"), *GetParameterCode(A), *GetParameterCode(B));
		}
		else
		{
			FiniteCode = FString::Printf(TEXT("MaterialFloat%u(%s,%s)"), NumResultComponents, *GetParameterCode(A), *GetParameterCode(B));
		}
		
		const EDerivativeStatus ADerivativeStatus = GetDerivativeStatus(A);
		const EDerivativeStatus BDerivativeStatus = GetDerivativeStatus(B);
		if (IsAnalyticDerivEnabled() && IsDerivativeValid(ADerivativeStatus) && IsDerivativeValid(BDerivativeStatus))
		{
			if (ADerivativeStatus == EDerivativeStatus::Zero && BDerivativeStatus == EDerivativeStatus::Zero)
			{
				return AddInlinedCodeChunkZeroDeriv(ResultType, *FiniteCode);
			}
			else
			{
				FString A_DDX = GetFloatZeroVector(NumComponentsA);
				FString A_DDY = GetFloatZeroVector(NumComponentsA);
				FString B_DDX = GetFloatZeroVector(NumComponentsB);
				FString B_DDY = GetFloatZeroVector(NumComponentsB);

				if (ADerivativeStatus == EDerivativeStatus::Valid)
				{
					FString Deriv = *GetParameterCodeDeriv(A, CompiledPDV_Analytic);
					A_DDX = Deriv + TEXT(".Ddx");
					A_DDY = Deriv + TEXT(".Ddy");
				}

				if (BDerivativeStatus == EDerivativeStatus::Valid)
				{
					FString Deriv = *GetParameterCodeDeriv(B, CompiledPDV_Analytic);
					B_DDX = Deriv + TEXT(".Ddx");
					B_DDY = Deriv + TEXT(".Ddy");
				}

				FString DDXCode = FString::Printf(TEXT("MaterialFloat%u(%s, %s)"), NumResultComponents, *A_DDX, *B_DDX);
				FString DDYCode = FString::Printf(TEXT("MaterialFloat%u(%s, %s)"), NumResultComponents, *A_DDY, *B_DDY);
				FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, *DDXCode, *DDYCode, GetDerivType(ResultType));
				return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, ResultType, false, EDerivativeStatus::Valid);
			}
		}
		else
		{
			return AddInlinedCodeChunk(ResultType, *FiniteCode);
		}
	}
}

int32 FHLSLMaterialTranslator::HsvToRgb(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FString HSV = CreateSymbolName(TEXT("HSV"));
	FString H = CreateSymbolName(TEXT("H"));
	FString R = CreateSymbolName(TEXT("R"));
	FString G = CreateSymbolName(TEXT("G"));
	FString B = CreateSymbolName(TEXT("B"));
	FString RGB = CreateSymbolName(TEXT("RGB"));

	EMaterialValueType ValueType = GetType(X);
	FString Type = HLSLTypeString(ValueType);
	FString CodeStr{ Type + TEXT(" ") + HSV + TEXT(" = %s; ") +
	TEXT("float ") + H + TEXT(" = ") + HSV + TEXT(".r; ") +
	TEXT("float ") + R + TEXT(" = abs(") + H + TEXT(" * 6 - 3) - 1; ") +
	TEXT("float ") + G + TEXT(" = 2 - abs(") + H + TEXT(" * 6 - 2); ") +
	TEXT("float ") + B + TEXT(" = 2 - abs(") + H + TEXT(" * 6 - 4); ") +
	TEXT("float3 ") + RGB + TEXT(" = saturate(float3(") + R + TEXT(",") + G + TEXT(",") + B + TEXT(")); ") };

	FString Code{ TEXT("((") + RGB + TEXT(" - 1) * ") + HSV + TEXT(".y + 1) * ") + HSV + TEXT(".z") };


	if(ValueType == EMaterialValueType::MCT_Float4)
	{
		Code = Type + TEXT("(") + Code + TEXT(", ") + HSV + TEXT(".w);");
	}

	AddInlinedCodeChunk(EMaterialValueType::MCT_VoidStatement, *CodeStr, *GetParameterCode(X));
	return AddCodeChunk(GetParameterType(X), *Code);
}

int32 FHLSLMaterialTranslator::RgbToHsv(int32 X)
{
	if(X == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FString RGB = CreateSymbolName(TEXT("RGB"));
	FString P = CreateSymbolName(TEXT("P"));
	FString Q = CreateSymbolName(TEXT("Q"));
	FString Chroma = CreateSymbolName(TEXT("Chroma"));
	FString Hue = CreateSymbolName(TEXT("Hue"));
	FString HCV = CreateSymbolName(TEXT("HCV"));
	FString S = CreateSymbolName(TEXT("s"));

	EMaterialValueType ValueType = GetType(X);

	FString Type = HLSLTypeString(ValueType);
	FString CodeStr{ Type + TEXT(" ") + RGB + TEXT("= %s;") +
	TEXT("float4 ") + P + TEXT(" = (") + RGB + TEXT(".g < ") + RGB + TEXT(".b) ? float4(") + RGB + TEXT(".bg, -1.0f, 2.0f / 3.0f) : float4(") + RGB + TEXT(".gb, 0.0f, -1.0f / 3.0f); ") +
	TEXT("float4 ") + Q + TEXT(" = (") + RGB + TEXT(".r < ") + P + TEXT(".x) ? float4(") + P + TEXT(".xyw, ") + RGB + TEXT(".r)	: float4(") + RGB + TEXT(".r, ") + P + TEXT(".yzx);") +
	TEXT("float ") + Chroma + TEXT("= ") + Q + TEXT(".x - min(") + Q + TEXT(".w, ") + Q + TEXT(".y);") +
	TEXT("float ") + Hue + TEXT(" = abs((") + Q + TEXT(".w - ") + Q + TEXT(".y) / (6.0f * ") + Chroma + TEXT(" + 1e-10f) + ") + Q + TEXT(".z);") +
	TEXT("float3 ") + HCV + TEXT(" = float3(") + Hue + TEXT(", ") + Chroma + TEXT(", ") + Q + TEXT(".x);") +
	TEXT("float ") + S + TEXT(" = ") + HCV + TEXT(".y / (") + HCV + TEXT(".z + 1e-10f);") };

	FString Code{ Type + TEXT("(") + HCV + TEXT(".x, ") + S + TEXT(", ") + HCV + TEXT(".z") };

	if(ValueType == EMaterialValueType::MCT_Float4)
	{
		Code += TEXT(",") + RGB + TEXT(".w");
	}

	Code += TEXT(");");

	AddInlinedCodeChunk(EMaterialValueType::MCT_VoidStatement, *CodeStr, *GetParameterCode(X));
	return AddCodeChunk(GetParameterType(X), *Code);
}

static FString MultiplyMatrix(const TCHAR* Vector, const TCHAR* Matrix, int AWComponent)
{
	if (AWComponent)
	{
		return FString::Printf(TEXT("mul(MaterialFloat4(%s, 1.0f), %s).xyz"), Vector, Matrix);
	}
	else
	{
		return FString::Printf(TEXT("mul(%s, (MaterialFloat3x3)(%s))"), Vector, Matrix);
	}
}

static FString MultiplyTranslatedMatrix(const TCHAR* Vector, const TCHAR* MatrixPreTranslation, int AWComponent, bool bCompilingPreviousFrame)
{
	if (AWComponent)
	{
		if (bCompilingPreviousFrame)
		{
			return FString::Printf(TEXT("mul(MaterialFloat4(%s, 1.0f), DFFastToTranslatedWorld(%s, ResolvedView.PrevPreViewTranslation)).xyz"), Vector, MatrixPreTranslation);
		}
		else
		{
			return FString::Printf(TEXT("mul(MaterialFloat4(%s, 1.0f), DFFastToTranslatedWorld(%s, ResolvedView.PreViewTranslation)).xyz"), Vector, MatrixPreTranslation);
		}
	}
	else
	{
		return FString::Printf(TEXT("mul(%s, DFToFloat3x3(%s))"), Vector, MatrixPreTranslation);
	}
}

static FString MultiplyTransposeMatrix(const TCHAR* Matrix, const TCHAR* Vector, int AWComponent)
{
	if (AWComponent)
	{
		return FString::Printf(TEXT("mul(%s, MaterialFloat4(%s, 1.0f)).xyz"), Matrix, Vector);
	}
	else
	{
		return FString::Printf(TEXT("mul((MaterialFloat3x3)(%s), %s)"), Matrix, Vector);
	}
}

static FString LWCMultiplyMatrix(const TCHAR* Vector, const TCHAR* Matrix, int AWComponent, bool Demote)
{
	if (AWComponent)
	{
		return FString::Printf(TEXT("WSMultiply%s(%s, %s)"), Demote ? TEXT("Demote") : TEXT(""), Vector, Matrix);
	}
	else
	{
		return FString::Printf(TEXT("WSMultiplyVector(%s, %s)"), Vector, Matrix);
	}
}

int32 FHLSLMaterialTranslator::TransformBase(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A, int AWComponent)
{
	if (A == INDEX_NONE)
	{
		// unable to compile
		return INDEX_NONE;
	}

	const EMaterialValueType SourceType = GetParameterType(A);
	const bool bIsPositionTransform = AWComponent != 0;
		
	{ // validation
		if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute && ShaderFrequency != SF_Vertex)
		{
			return NonPixelShaderExpressionError();
		}

		if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute && ShaderFrequency != SF_Vertex)
		{
			if ((SourceCoordBasis == MCB_Local || DestCoordBasis == MCB_Local))
			{
				return Errorf(TEXT("Local space is only supported for vertex, compute or pixel shader"));
			}
		}

		if (bIsPositionTransform && (SourceCoordBasis == MCB_Tangent || DestCoordBasis == MCB_Tangent))
		{
			return Errorf(TEXT("Tangent basis not available for position transformations"));
		}

		if (SourceCoordBasis == MCB_FirstPerson && DestCoordBasis != MCB_FirstPerson)
		{
			return Errorf(TEXT("Transforming from First Person Space is not supported"));
		}
		
		// Construct float3(0,0,x) out of the input if it is a scalar
		// This way artists can plug in a scalar and it will be treated as height, or a vector displacement
		const uint32 NumInputComponents = GetNumComponents(SourceType);
		if (NumInputComponents == 1u && SourceCoordBasis == MCB_Tangent)
		{
			A = AppendVector(Constant2(0, 0), A);
		}
		else if (NumInputComponents < 3u)
		{
			return Errorf(TEXT("input must be a 3-component vector (current: %s: %s) or a scalar (if source is Tangent)"), *GetParameterCode(A), DescribeType(SourceType));
		}
	}
		
	if (SourceCoordBasis == DestCoordBasis)
	{
		// no transformation needed
		return A;
	}
		
	FString CodeStr;
	FString CodeDerivStr;
	EMaterialCommonBasis IntermediaryBasis = MCB_World;

	switch (SourceCoordBasis)
	{
		case MCB_Tangent:
		{
			check(!bIsPositionTransform);
			if (DestCoordBasis == MCB_World)
			{
				CodeStr = TEXT("mul(<A>, Parameters.TangentToWorld)");
				CodeDerivStr = TEXT("mul(<A>, Parameters.TangentToWorld)");
			}
			// else use MCB_World as intermediary basis
			break;
		}
		case MCB_Local:
		{
			if (DestCoordBasis == MCB_World)
			{
				if (bIsPositionTransform) { AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix); }
				CodeStr = TEXT("TransformLocal<TO><PREV>World(Parameters, <A>)");
				CodeDerivStr = TEXT("TransformLocal<TO><PREV>World(Parameters, <A>)");
			}
			else if (DestCoordBasis == MCB_TranslatedWorld)
			{
				if (bIsPositionTransform)
				{
					CodeStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Get<PREV>LocalToWorldDF(Parameters)"), AWComponent, bCompilingPreviousFrame);
					CodeDerivStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Get<PREV>LocalToWorldDF(Parameters)"), 0, bCompilingPreviousFrame);
				}
			}
			else if (DestCoordBasis == MCB_PeriodicWorld || DestCoordBasis == MCB_FirstPerson)
			{
				IntermediaryBasis = MCB_TranslatedWorld;
			}
			// else use MCB_World as intermediary basis
			break;
		}
		case MCB_TranslatedWorld:
		{
			if (DestCoordBasis == MCB_World)
			{
				if (bIsPositionTransform)
				{
					AddLWCFuncUsage(ELWCFunctionKind::Subtract);
					CodeStr = TEXT("WSSubtract(<A>, Get<PREV>PreViewTranslation(Parameters))");
				}
				else
				{
					CodeStr = TEXT("<A>");
				}
				CodeDerivStr = TEXT("<A>");
			}
			else if (DestCoordBasis == MCB_Camera)
			{
				CodeStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>TranslatedWorldToCameraView"), AWComponent);
				CodeDerivStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>TranslatedWorldToCameraView"), 0);
			}
			else if (DestCoordBasis == MCB_View)
			{
				CodeStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>TranslatedWorldToView"), AWComponent);
				CodeDerivStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>TranslatedWorldToView"), 0);
			}
			else if (DestCoordBasis == MCB_Tangent)
			{
				CodeStr = MultiplyTransposeMatrix(TEXT("Parameters.TangentToWorld"), TEXT("<A>"), AWComponent);
				CodeDerivStr = MultiplyTransposeMatrix(TEXT("Parameters.TangentToWorld"), TEXT("<A>"), 0);
			}
			else if (DestCoordBasis == MCB_Local)
			{
				const EMaterialDomain Domain = (const EMaterialDomain)Material->GetMaterialDomain();

				if (Domain != MD_Surface && Domain != MD_Volume)
				{
					Errorf(TEXT("This transformation is only supported in the 'Surface' material domain."));
					return INDEX_NONE;
				}

				// TODO: inconsistent with TransformLocal<TO>World with instancing
				// We have explicit options for "local" and "instance" spaces, but then GetLocalToWorld returns instance space, while GetWorldToLocal always returns primitive space. 
				// It's inconsistent, but replacing either will break existing materials.
				CodeStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Get<PREV>WorldToLocalDF(Parameters)"), AWComponent, bCompilingPreviousFrame);
				CodeDerivStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Get<PREV>WorldToLocalDF(Parameters)"), 0, bCompilingPreviousFrame);
			}
			else if (DestCoordBasis == MCB_MeshParticle)
			{
				CodeStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Parameters.Particle.WorldToParticle"), AWComponent, bCompilingPreviousFrame);
				CodeDerivStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Parameters.Particle.WorldToParticle"), 0, bCompilingPreviousFrame);
				bUsesParticleWorldToLocal = true;
			}
			else if (DestCoordBasis == MCB_Instance)
			{
				CodeStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("GetWorldToInstanceDF(Parameters)"), AWComponent, bCompilingPreviousFrame);
				CodeDerivStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("GetWorldToInstanceDF(Parameters)"), 0, bCompilingPreviousFrame);
				bUsesInstanceWorldToLocalPS |= ShaderFrequency == SF_Pixel;
			}
			else if (DestCoordBasis == MCB_PeriodicWorld)
			{
				if (Parameters.PeriodicWorldTileSizeIndex == INDEX_NONE)
				{
					return Errorf(TEXT("Missing periodic world tile size"));
				}
				int32 PeriodicWorldOrigin = CalculatePeriodicWorldPositionOrigin(Parameters.PeriodicWorldTileSizeIndex);
				CodeStr = FString::Printf(TEXT("(<A> - (%s))"), *GetParameterCode(PeriodicWorldOrigin));
				CodeDerivStr = TEXT("<A>");
			}
			else if (DestCoordBasis == MCB_FirstPerson)
			{
				if (Parameters.FirstPersonInterpolationAlphaIndex == INDEX_NONE)
				{
					return Errorf(TEXT("Missing first person interpolation alpha"));
				}
				int32 LerpAlphaFloat1Index = ForceCast(Parameters.FirstPersonInterpolationAlphaIndex, MCT_Float1);
				int32 LerpAlphaClampedIndex = Saturate(LerpAlphaFloat1Index);
				CodeStr = FString::Printf(TEXT("TransformTo<PREVIOUS>FirstPerson(<A>, %s)"), *GetParameterCode(LerpAlphaClampedIndex));
				CodeDerivStr = CodeStr; // The first person transform is actually a 3x3 matrix and can therefore be used for derivatives as well.
			}
			// else use MCB_World as intermediary basis
			break;
		}
		case MCB_World:
		{
			if (DestCoordBasis == MCB_Tangent)
			{
				CodeStr = MultiplyTransposeMatrix(TEXT("Parameters.TangentToWorld"), TEXT("<A>"), AWComponent);
				CodeDerivStr = MultiplyTransposeMatrix(TEXT("Parameters.TangentToWorld"), TEXT("<A>"), 0);
			}
			else if (DestCoordBasis == MCB_Local)
			{
				const EMaterialDomain Domain = (const EMaterialDomain)Material->GetMaterialDomain();

				if(Domain != MD_Surface && Domain != MD_Volume)
				{
					// TODO: for decals we could support it
					Errorf(TEXT("This transformation is only supported in the 'Surface' material domain."));
					return INDEX_NONE;
				}

				// TODO: inconsistent with TransformLocal<TO>World with instancing
				if (bIsPositionTransform) { AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix); }
				CodeStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("Get<PREV>WorldToLocal(Parameters)"), AWComponent, true);
				CodeDerivStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("Get<PREV>WorldToLocal(Parameters)"), 0, true);
			}
			else if (DestCoordBasis == MCB_TranslatedWorld)
			{
				if (bIsPositionTransform)
				{
					AddLWCFuncUsage(ELWCFunctionKind::Add);
					CodeStr = TEXT("WSAddDemote(<A>, Get<PREV>PreViewTranslation(Parameters))");
				}
				else
				{
					CodeStr = TEXT("<A>");
				}
				CodeDerivStr = TEXT("<A>");
			}
			else if (DestCoordBasis == MCB_MeshParticle)
			{
				if (bIsPositionTransform) { AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix); }
				CodeStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("GetWorldToParticle(Parameters)"), AWComponent, true);
				CodeDerivStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("GetWorldToParticle(Parameters)"), 0, true);
				bUsesParticleWorldToLocal = true;
			}
			else if (DestCoordBasis == MCB_Instance)
			{
				if (bIsPositionTransform) { AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix); }
				CodeStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("GetWorldToInstance(Parameters)"), AWComponent, true);
				CodeDerivStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("GetWorldToInstance(Parameters)"), 0, true);
				bUsesInstanceWorldToLocalPS |= ShaderFrequency == SF_Pixel;
			}

			// else use MCB_TranslatedWorld as intermediary basis
			IntermediaryBasis = MCB_TranslatedWorld;
			break;
		}
		case MCB_Camera:
		{
			if (DestCoordBasis == MCB_TranslatedWorld)
			{
				CodeStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>CameraViewToTranslatedWorld"), AWComponent);
				CodeDerivStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>CameraViewToTranslatedWorld"), 0);
			}
			// else use MCB_TranslatedWorld as intermediary basis
			IntermediaryBasis = MCB_TranslatedWorld;
			break;
		}
		case MCB_View:
		{
			if (DestCoordBasis == MCB_TranslatedWorld)
			{
				CodeStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>ViewToTranslatedWorld"), AWComponent);
				CodeDerivStr = MultiplyMatrix(TEXT("<A>"), TEXT("ResolvedView.<PREV>ViewToTranslatedWorld"), 0);
			}
			// else use MCB_TranslatedWorld as intermediary basis
			IntermediaryBasis = MCB_TranslatedWorld;
			break;
		}
		case MCB_MeshParticle:
		{
			if (DestCoordBasis == MCB_World)
			{
				if (bIsPositionTransform) { AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix); }
				CodeStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("GetParticleToWorld(Parameters)"), AWComponent, false);
				CodeDerivStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("GetParticleToWorld(Parameters)"), 0, false);
				bUsesParticleLocalToWorld = true;
			}
			else if (DestCoordBasis == MCB_TranslatedWorld)
			{
				CodeStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Parameters.Particle.ParticleToWorld"), AWComponent, bCompilingPreviousFrame);
				CodeDerivStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Parameters.Particle.ParticleToWorld"), 0, bCompilingPreviousFrame);
				bUsesParticleLocalToWorld = true;
			}
			else if (DestCoordBasis == MCB_PeriodicWorld || DestCoordBasis == MCB_FirstPerson)
			{
				IntermediaryBasis = MCB_TranslatedWorld;
			}
			// use World as an intermediary base
			break;
		}
		case MCB_Instance:
		{
			if (DestCoordBasis == MCB_World)
			{
				if (bIsPositionTransform) { AddLWCFuncUsage(ELWCFunctionKind::MultiplyVectorMatrix); }
				CodeStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("Get<PREV>InstanceToWorld(Parameters)"), AWComponent, false);
				CodeDerivStr = LWCMultiplyMatrix(TEXT("<A>"), TEXT("Get<PREV>InstanceToWorld(Parameters)"), 0, false);
				bUsesInstanceLocalToWorldPS |= ShaderFrequency == SF_Pixel;
			}
			else if (DestCoordBasis == MCB_TranslatedWorld)
			{
				CodeStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Get<PREV>InstanceToWorldDF(Parameters)"), AWComponent, bCompilingPreviousFrame);
				CodeDerivStr = MultiplyTranslatedMatrix(TEXT("<A>"), TEXT("Get<PREV>InstanceToWorldDF(Parameters)"), 0, bCompilingPreviousFrame);
				bUsesInstanceLocalToWorldPS |= ShaderFrequency == SF_Pixel;
			}
			else if (DestCoordBasis == MCB_PeriodicWorld || DestCoordBasis == MCB_FirstPerson)
			{
				IntermediaryBasis = MCB_TranslatedWorld;
			}
			// use World as an intermediary base
			break;
		}
		case MCB_PeriodicWorld:
		{
			if (DestCoordBasis == MCB_TranslatedWorld)
			{
				if (Parameters.PeriodicWorldTileSizeIndex == INDEX_NONE)
				{
					return Errorf(TEXT("Missing periodic world tile size"));
				}
				int32 PeriodicWorldOrigin = CalculatePeriodicWorldPositionOrigin(Parameters.PeriodicWorldTileSizeIndex);
				CodeStr = FString::Printf(TEXT("(<A> + (%s))"), *GetParameterCode(PeriodicWorldOrigin));
				CodeDerivStr = TEXT("<A>");
			}

			// else use MCB_TranslatedWorld as intermediary basis
			IntermediaryBasis = MCB_TranslatedWorld;
			break;
		}
		case MCB_FirstPerson:
		{
			checkNoEntry(); // MCB_FirstPerson is not supported as a source basis. This should've been caught earlier in validation.
			break;
		}

		default:
			check(0);
			break;
	}

	if (CodeStr.IsEmpty())
	{
		// check intermediary basis so we don't have infinite recursion
		check(IntermediaryBasis != SourceCoordBasis);
		check(IntermediaryBasis != DestCoordBasis);

		// use intermediary basis
		const int32 IntermediaryA = TransformBase(SourceCoordBasis, IntermediaryBasis, Parameters, A, AWComponent);

		return TransformBase(IntermediaryBasis, DestCoordBasis, Parameters, IntermediaryA, AWComponent);
	}
		
	if (bIsPositionTransform)
	{
		CodeStr.ReplaceInline(TEXT("<TO>"),TEXT("PositionTo"));
	}
	else
	{
		CodeStr.ReplaceInline(TEXT("<TO>"),TEXT("VectorTo"));
	}
	CodeDerivStr.ReplaceInline(TEXT("<TO>"), TEXT("VectorTo"));
		
	if (bCompilingPreviousFrame)
	{
		CodeStr.ReplaceInline(TEXT("<PREV>"),TEXT("Prev"));
		CodeStr.ReplaceInline(TEXT("<PREVIOUS>"), TEXT("Previous"));
		CodeDerivStr.ReplaceInline(TEXT("<PREV>"), TEXT("Prev"));
		CodeDerivStr.ReplaceInline(TEXT("<PREVIOUS>"), TEXT("Previous"));
	}
	else
	{
		CodeStr.ReplaceInline(TEXT("<PREV>"),TEXT(""));
		CodeStr.ReplaceInline(TEXT("<PREVIOUS>"), TEXT(""));
		CodeDerivStr.ReplaceInline(TEXT("<PREV>"), TEXT(""));
		CodeDerivStr.ReplaceInline(TEXT("<PREVIOUS>"), TEXT(""));
	}

	int32 CastA = A;
	if (SourceCoordBasis == MCB_World && AWComponent)
	{
		CastA = ValidCast(CastA, MCT_LWCVector3);
	}
	else
	{
		CastA = ValidCast(CastA, MCT_Float3);
	}

	CodeStr.ReplaceInline(TEXT("<A>"), *GetParameterCode(CastA));

	if (ShaderFrequency != SF_Vertex && (DestCoordBasis == MCB_Tangent || SourceCoordBasis == MCB_Tangent))
	{
		bUsesTransformVector = true;
	}

	const EMaterialValueType ResultType = (DestCoordBasis == MCB_World && AWComponent) ? MCT_LWCVector3 : MCT_Float3;
	const EDerivativeStatus ADerivStatus = GetDerivativeStatus(CastA);

	int32 Result;
	if (IsAnalyticDerivEnabled() && IsDerivativeValid(ADerivStatus))
	{
		if (ADerivStatus == EDerivativeStatus::Valid)
		{
			FString CastADeriv = GetParameterCodeDeriv(CastA, CompiledPDV_Analytic);
			FString CodeAnalytic = DerivativeAutogen.ConstructDeriv(
				CodeStr,
				CodeDerivStr.Replace(TEXT("<A>"), *(CastADeriv + TEXT(".Ddx"))),
				CodeDerivStr.Replace(TEXT("<A>"), *(CastADeriv + TEXT(".Ddy"))), GetDerivType(ResultType));
			Result = AddCodeChunkInnerDeriv(*CodeStr, *CodeAnalytic, ResultType, false, EDerivativeStatus::Valid);
		}
		else
		{
			Result = AddCodeChunkZeroDeriv(ResultType, *CodeStr);
		}
	}
	else
	{
		Result = AddCodeChunk(ResultType, *CodeStr);
	}
	
	return CastToNonLWCIfDisabled(Result);
}

int32 FHLSLMaterialTranslator::TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A)
{
	return TransformBase(SourceCoordBasis, DestCoordBasis, Parameters, A, 0);
}

int32 FHLSLMaterialTranslator::TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A)
{
	return TransformBase(SourceCoordBasis, DestCoordBasis, Parameters, A, 1);
}

int32 FHLSLMaterialTranslator::CalculatePeriodicWorldPositionOrigin(int TileScaleIndex)
{
	// float3 PeriodicWorldOrigin = GetPeriodicWorldOrigin(Scale);
	FString PeriodicWorldOrigin = CreateSymbolName(TEXT("PeriodicWorldOrigin"));
	FString FuncName = IsConstFloatOfPow2Expression(TileScaleIndex) ? TEXT("GetPeriodicWorldOrigin_Pow2") : TEXT("GetPeriodicWorldOrigin");
	FString Code = FString::Format(TEXT("	float3 {0} = {1}({2});{3}"), { PeriodicWorldOrigin, FuncName, *GetParameterCode(TileScaleIndex), HLSL_LINE_TERMINATOR });

	AddInlinedCodeChunk(EMaterialValueType::MCT_VoidStatement, *Code);
	return AddCodeChunk(MCT_Float3, *PeriodicWorldOrigin);
}

int32 FHLSLMaterialTranslator::TransformNormalFromRequestedBasisToWorld(int32 NormalCodeChunk)
{
	// When feeding tangent space or world space, we want to have the final normal/tangent normalized when stored in SharedLocalBases.
	// So that we do not have to normalise it later in any forward processes and avoid overhead in instructions.
	if (IsTangentSpaceNormal())
	{
		// See TransformTangentNormalToWorld definitions in MaterialTemplate.ush
		return AddCodeChunk(MCT_Float3, TEXT("TransformTangentNormalToWorld(Parameters.TangentToWorld, %s)"), *GetParameterCode(NormalCodeChunk));
	}
	return Normalize(NormalCodeChunk);
}

int32 FHLSLMaterialTranslator::DynamicParameter(FLinearColor& DefaultValue, uint32 ParameterIndex)
{
	if (ShaderFrequency != SF_Vertex && ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonVertexOrPixelShaderExpressionError();
	}

	DynamicParticleParameterMask |= (1 << ParameterIndex);

	int32 Default = Constant4(DefaultValue.R, DefaultValue.G, DefaultValue.B, DefaultValue.A);
	return AddInlinedCodeChunkZeroDeriv(
		MCT_Float4,
		TEXT("GetDynamicParameter(Parameters.Particle, %s, %u)"),
		*GetParameterCode(Default),
		ParameterIndex
		);
}

int32 FHLSLMaterialTranslator::LightmapUVs()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonPixelShaderExpressionError();
	}

	if (ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::SM5) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	bUsesLightmapUVs = true;

	const TCHAR* FiniteCode = TEXT("GetLightmapUVs(Parameters)");
	if (IsAnalyticDerivEnabled())
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(
			FiniteCode,
			TEXT("GetLightmapUVs_DDX(Parameters)"),
			TEXT("GetLightmapUVs_DDY(Parameters)"), EDerivativeType::Float2);
		return AddCodeChunkInnerDeriv(FiniteCode, *AnalyticCode, MCT_Float2, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddCodeChunk(MCT_Float2, FiniteCode);
	}
}

int32 FHLSLMaterialTranslator::PrecomputedAOMask()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonPixelShaderExpressionError();
	}

	if (ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::SM5) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	bUsesAOMaterialMask = true;

	int32 ResultIdx = INDEX_NONE;
	FString CodeChunk = FString::Printf(TEXT("Parameters.AOMaterialMask"));
	ResultIdx = AddCodeChunk(
		MCT_Float,
		*CodeChunk
		);
	return ResultIdx;
}

int32 FHLSLMaterialTranslator::GenericSwitch(const TCHAR* SwitchExpressionText, int32 IfTrue, int32 IfFalse)
{
	if (IfTrue == INDEX_NONE || IfFalse == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	// exactly the same inputs on both sides - no need to generate anything extra
	if (IfTrue == IfFalse)
	{
		return IfTrue;
	}

	FMaterialUniformExpression* IfTrueExpression = GetParameterUniformExpression(IfTrue);
	FMaterialUniformExpression* IfFalseExpression = GetParameterUniformExpression(IfFalse);
	if (IfTrueExpression &&
		IfFalseExpression &&
		IfTrueExpression->IsConstant() &&
		IfFalseExpression->IsConstant())
	{
		FMaterialRenderContext DummyContext(nullptr, *Material, nullptr);
		FLinearColor IfTrueValue;
		FLinearColor IfFalseValue;
		IfTrueExpression->GetNumberValue(DummyContext, IfTrueValue);
		IfFalseExpression->GetNumberValue(DummyContext, IfFalseValue);
		if (IfTrueValue == IfFalseValue)
		{
			// If both inputs are wired to == constant values, avoid adding the runtime switch
			// This will avoid breaking various offline checks for constant values
			return IfTrue;
		}
	}

	// Both branches of '?:' need to be the same type
	const EMaterialValueType ResultType = GetArithmeticResultType(IfTrue, IfFalse);
	const FString IfTrueCode = CoerceParameter(IfTrue, ResultType);
	const FString IfFalseCode = CoerceParameter(IfFalse, ResultType);
	
	if (IsLWCType(ResultType))
	{
		AddLWCFuncUsage(ELWCFunctionKind::Other);
		return AddCodeChunk(ResultType, TEXT("WSSelect(%s, %s, %s)"), SwitchExpressionText, *IfTrueCode, *IfFalseCode);
	}
	else
	{
		return AddCodeChunk(ResultType, TEXT("(%s ? (%s) : (%s))"), SwitchExpressionText, *IfTrueCode, *IfFalseCode);
	}
}

bool FHLSLMaterialTranslator::IsConstFloatOfPow2Expression(int32 ExpCode)
{
	FMaterialUniformExpression* ExpressionB = GetParameterUniformExpression(ExpCode);
	FLinearColor ValueB;
	bool bIsPow2 = false;
	if (ExpressionB && GetConstParameterValue(ExpressionB, ValueB))
	{
		auto IsFloatPowerOfTwo = [](float Value) { return ((*reinterpret_cast<int*>(&Value)) & 0x007FFFFF) == 0; }; // zero mantisse
		bIsPow2 = IsFloatPowerOfTwo(ValueB.R) && IsFloatPowerOfTwo(ValueB.G) && IsFloatPowerOfTwo(ValueB.B) && IsFloatPowerOfTwo(ValueB.A);
	}
	return bIsPow2;
}

int32 FHLSLMaterialTranslator::GIReplace(int32 Direct, int32 StaticIndirect, int32 DynamicIndirect)
{ 
	return GenericSwitch(TEXT("GetGIReplaceState()"), DynamicIndirect, Direct);
}

int32 FHLSLMaterialTranslator::ShadowReplace(int32 Default, int32 Shadow)
{
	return GenericSwitch(TEXT("GetShadowReplaceState()"), Shadow, Default);
}

int32 FHLSLMaterialTranslator::NaniteReplace(int32 Default, int32 Nanite)
{
	return GenericSwitch(TEXT("GetNaniteReplaceState()"), Nanite, Default);
}

int32 FHLSLMaterialTranslator::MaterialCache(int32 Default, int32 MaterialCache)
{
	return GenericSwitch(TEXT("GetMaterialCacheState()"), MaterialCache, Default);
}

int32 FHLSLMaterialTranslator::ReflectionCapturePassSwitch(int32 Default, int32 Reflection)
{
	return GenericSwitch(TEXT("GetReflectionCapturePassSwitchState()"), Reflection, Default);
}

int32 FHLSLMaterialTranslator::RayTracingQualitySwitchReplace(int32 Normal, int32 RayTraced)
{
	return GenericSwitch(TEXT("GetRayTracingQualitySwitch()"), RayTraced, Normal);
}

int32 FHLSLMaterialTranslator::PathTracingQualitySwitchReplace(int32 Normal, int32 PathTraced)
{
	return GenericSwitch(TEXT("GetPathTracingQualitySwitch()"), PathTraced, Normal);
}

int32 FHLSLMaterialTranslator::PathTracingRayTypeSwitch(int32 Main, int32 Shadow, int32 IndirectDiffuse, int32 IndirectSpecular, int32 IndirectVolume)
{
	// Generate a sequential series of switches so we can easily account for type promotion across the ports
	// Any input port that does not have anything connected to it (INDEX_NONE) defaults back to Main
	int32 TmpA = GenericSwitch(TEXT("GetPathTracingIsShadow()")          , Shadow           == INDEX_NONE ? Main : Shadow          , Main);
	int32 TmpB = GenericSwitch(TEXT("GetPathTracingIsIndirectDiffuse()") , IndirectDiffuse  == INDEX_NONE ? Main : IndirectDiffuse , TmpA);
	int32 TmpC = GenericSwitch(TEXT("GetPathTracingIsIndirectSpecular()"), IndirectSpecular == INDEX_NONE ? Main : IndirectSpecular, TmpB);
	int32 TmpD = GenericSwitch(TEXT("GetPathTracingIsIndirectVolume()")  , IndirectVolume   == INDEX_NONE ? Main : IndirectVolume  , TmpC);
	return TmpD;
}

int32 FHLSLMaterialTranslator::LightmassReplace(int32 Realtime, int32 Lightmass)
{
	return GenericSwitch(TEXT("GetLightmassReplaceState()"), Lightmass, Realtime);
}

int32 FHLSLMaterialTranslator::ObjectOrientation()
{ 
	return AddInlinedCodeChunkZeroDeriv(MCT_Float3,TEXT("GetObjectOrientation(Parameters)"));
}

int32 FHLSLMaterialTranslator::RotateAboutAxis(int32 NormalizedRotationAxisAndAngleIndex, int32 PositionOnAxisIndex, int32 PositionIndex)
{
	if (NormalizedRotationAxisAndAngleIndex == INDEX_NONE
		|| PositionOnAxisIndex == INDEX_NONE
		|| PositionIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	else
	{
		const EMaterialValueType PositionOnAxisType = GetParameterType(PositionOnAxisIndex);
		const EMaterialValueType PositionType = GetParameterType(PositionIndex);
		const EMaterialValueType InputType = IsLWCType(PositionOnAxisType) || IsLWCType(PositionType) ? MCT_LWCVector3 : MCT_Float3;
		return AddCodeChunk(
			MCT_Float3,
			TEXT("RotateAboutAxis(%s,%s,%s)"),
			*CoerceParameter(NormalizedRotationAxisAndAngleIndex,MCT_Float4),
			*CoerceParameter(PositionOnAxisIndex, InputType),
			*CoerceParameter(PositionIndex, InputType)
			);	
	}
}

int32 FHLSLMaterialTranslator::TwoSidedSign()
{
	return AddExternalCodeChunk(TEXT("TwoSidedSign"));
}

int32 FHLSLMaterialTranslator::VertexNormal()
{
	if (ShaderFrequency != SF_Vertex)
	{
		bUsesTransformVector = true;
	}

	FString FiniteCode = FString("Parameters.TangentToWorld[2]");

	if (IsAnalyticDerivEnabled() && ShaderFrequency != SF_Vertex)
	{
		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(FiniteCode, TEXT("Parameters.WorldGeoNormal_DDX"), TEXT("Parameters.WorldGeoNormal_DDY"), EDerivativeType::Float3);
		return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, MCT_Float3, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddInlinedCodeChunk(MCT_Float3, *FiniteCode);
	}
}

int32 FHLSLMaterialTranslator::VertexTangent()
{
	if (ShaderFrequency != SF_Vertex)
	{
		bUsesTransformVector = true;
	}
	return AddExternalCodeChunk(TEXT("VertexTangent"));
}

int32 FHLSLMaterialTranslator::PixelNormalWS()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Compute)
	{
		return NonPixelShaderExpressionError();
	}
	if(MaterialProperty == MP_Normal)
	{
		return Errorf(TEXT("Invalid node PixelNormalWS used for Normal input."));
	}
	const bool bIsStrictValidation = EnumHasAnyFlags(Material->GetMaterialTranslateValidationFlags(), EMaterialTranslateValidationFlags::Strict_RuntimeVirtualTexture);
	if (bIsStrictValidation && bIsInRuntimeVirtualTextureOutput && bUsesVirtualTextureSampleForNormalProperty)
	{
		return Errorf(TEXT("PixelNormalWS samples Virtual Textures and so is not allowed during Runtime Virtual Texture Output."));
	}
	if (ShaderFrequency != SF_Vertex)
	{
		bUsesTransformVector = true;
	}
	return AddInlinedCodeChunk(MCT_Float3,TEXT("Parameters.WorldNormal"));	
}

int32 FHLSLMaterialTranslator::IsFirstPerson()
{
	return AddExternalCodeChunk(TEXT("IsFirstPerson"));
}

int32 FHLSLMaterialTranslator::Derivative(int32 A, EDervativeComponent Component)
{
	if (A == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (ShaderFrequency == SF_Compute)
	{
		// running a material in a compute shader pass (e.g. when using SVOGI)
		return AddInlinedCodeChunk(MCT_Float, TEXT("0"));
	}

	if (ShaderFrequency != SF_Pixel)
	{
		return NonPixelShaderExpressionError();
	}

	bUsesExplicitDerivatives = true;

	const FDerivInfo ADerivInfo = GetDerivInfo(A);
	const EMaterialValueType ResultType = MakeNonLWCType(ADerivInfo.Type);
	const bool bIsLWCType = IsLWCType(ADerivInfo.Type);
	const TCHAR* FunctionName = bIsLWCType ? TEXT("WSDd") : TEXT("DD");
	const TCHAR* FunctionNamePostfix = bIsLWCType ? TEXT("Demote") : TEXT("");
	
	// For non-LWC types, we use the DDX() and DDY() functions defined in Common.ush, which correctly deal with shader types that do not support hardware derivatives.
	const TCHAR* ComponentLowerCase = TEXT("");
	const TCHAR* ComponentUpperCase = TEXT("");
	switch (Component)
	{
	case EDervativeComponent::X:
		ComponentLowerCase = TEXT("x");
		ComponentUpperCase = TEXT("X");
		break;
	case EDervativeComponent::Y:
		ComponentLowerCase = TEXT("y");
		ComponentUpperCase = TEXT("Y");
		break;
	default:
		checkNoEntry();
		break;
	}
    
	if (bIsLWCType)
	{
		AddLWCFuncUsage(ELWCFunctionKind::Other);
	}
	
    // Spirv expects DDX/Y to have 32 bit width, so need to ensure we use float and not half
    const EMaterialValueType ParameterType = GetParameterType(A);
    const TCHAR* CastType = TEXT("");
    
    switch (ParameterType)
    {
    	case MCT_Float1: case MCT_Float: CastType = TEXT("(float)"); break;
	    case MCT_Float2: CastType = TEXT("(float2)"); break;
	    case MCT_Float3: CastType = TEXT("(float3)"); break;
    	case MCT_Float4: CastType = TEXT("(float4)"); break;
    }
            
	const FString FiniteCode = FString::Printf(TEXT("%s%s%s(%s%s)"), FunctionName, bIsLWCType ? ComponentLowerCase : ComponentUpperCase, FunctionNamePostfix, CastType, *GetParameterCode(A));

	if (IsAnalyticDerivEnabled() && ADerivInfo.DerivativeStatus == EDerivativeStatus::Valid)
	{
		FString ADeriv = GetParameterCodeDeriv(A, CompiledPDV_Analytic);

		FString AnalyticCode = DerivativeAutogen.ConstructDeriv(
			FString::Printf(TEXT("%s.Dd%s"), *ADeriv, ComponentLowerCase),
			FString::Printf(TEXT("DD%s(%s.Ddx)"), ComponentUpperCase, *ADeriv),
			FString::Printf(TEXT("DD%s(%s.Ddy)"), ComponentUpperCase, *ADeriv),
			MakeNonLWCType(ADerivInfo.DerivType)
		);

		return AddCodeChunkInnerDeriv(*FiniteCode, *AnalyticCode, ResultType, false, EDerivativeStatus::Valid);
	}
	else
	{
		return AddCodeChunkInnerDeriv(*FiniteCode, ResultType, false, (ADerivInfo.DerivativeStatus == EDerivativeStatus::Zero) ? EDerivativeStatus::Zero : EDerivativeStatus::NotValid);
	}
}

int32 FHLSLMaterialTranslator::DDX( int32 A )
{
	return Derivative(A, EDervativeComponent::X);
}

int32 FHLSLMaterialTranslator::DDY( int32 A )
{
	return Derivative(A, EDervativeComponent::Y);
}

int32 FHLSLMaterialTranslator::AntialiasedTextureMask(int32 Tex, int32 UV, float Threshold, uint8 Channel)
{
	if (ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::SM5) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (Tex == INDEX_NONE || UV == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	int32 ThresholdConst = Constant(Threshold);
	int32 ChannelConst = Constant(Channel);
	FString TextureName = CoerceParameter(Tex, GetParameterType(Tex));

	return AddCodeChunk(MCT_Float, 
		TEXT("AntialiasedTextureMask(%s,%sSampler,%s,%s,%s)"), 
		*GetParameterCode(Tex),
		*TextureName,
		*GetParameterCode(UV),
		*GetParameterCode(ThresholdConst),
		*GetParameterCode(ChannelConst));
}

int32 FHLSLMaterialTranslator::DepthOfFieldFunction(int32 Depth, int32 FunctionValueIndex)
{
	if (Depth == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return AddCodeChunk(MCT_Float, 
		TEXT("MaterialExpressionDepthOfFieldFunction(%s, %d)"), 
		*GetParameterCode(Depth), FunctionValueIndex);
}

int32 FHLSLMaterialTranslator::PostVolumeUserFlagTestFunction(int32 Input)
{
	if (Input == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return AddCodeChunk(MCT_Float,
		TEXT("PostVolumeUserFlagTest(%s)"),
		*GetParameterCode(Input));
}

int32 FHLSLMaterialTranslator::Sobol(int32 Cell, int32 Index, int32 Seed)
{
	AddEstimatedTextureSample(2);

	return AddCodeChunk(MCT_Float2,
		TEXT("floor(%s) + float2(SobolIndex(SobolPixel(uint2(%s)), uint(%s)) ^ uint2(%s * 0x10000) & 0xffff) / 0x10000"),
		*GetParameterCode(Cell),
		*GetParameterCode(Cell),
		*GetParameterCode(Index),
		*GetParameterCode(Seed));
}

int32 FHLSLMaterialTranslator::TemporalSobol(int32 Index, int32 Seed)
{
	AddEstimatedTextureSample(2);

	return AddCodeChunk(MCT_Float2,
		TEXT("float2(SobolIndex(SobolPixel(uint2(Parameters.SvPosition.xy)), uint(View.StateFrameIndexMod8 + 8 * %s)) ^ uint2(%s * 0x10000) & 0xffff) / 0x10000"),
		*GetParameterCode(Index),
		*GetParameterCode(Seed));
}

int32 FHLSLMaterialTranslator::Noise(int32 Position, EPositionOrigin PositionOrigin, float Scale, int32 Quality, uint8 NoiseFunction, bool bTurbulence, int32 Levels, float OutputMin, float OutputMax, float LevelScale, int32 FilterWidth, bool bTiling, uint32 RepeatSize)
{
	if(Position == INDEX_NONE || FilterWidth == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (NoiseFunction == NOISEFUNCTION_SimplexTex ||
		NoiseFunction == NOISEFUNCTION_GradientTex ||
		NoiseFunction == NOISEFUNCTION_GradientTex3D)
	{
		AddEstimatedTextureSample();
	}

	// to limit performance problems due to values outside reasonable range
	Levels = FMath::Clamp(Levels, 1, 10);

	int32 ScaleConst = Constant(Scale);
	int32 QualityConst = Constant(Quality);
	int32 NoiseFunctionConst = Constant(NoiseFunction);
	int32 TurbulenceConst = Constant(bTurbulence);
	int32 LevelsConst = Constant(Levels);
	int32 OutputMinConst = Constant(OutputMin);
	int32 OutputMaxConst = Constant(OutputMax);
	int32 LevelScaleConst = Constant(LevelScale);
	int32 TilingConst = Constant(bTiling);
	int32 RepeatSizeConst = Constant(RepeatSize);

	if (PositionOrigin == EPositionOrigin::CameraRelative)
	{
		//LWC_TODO: add support for translated world positions in the corresponding HLSL function
		Position = TransformPosition(MCB_TranslatedWorld, MCB_World, Position);
	}

	return AddCodeChunk(MCT_Float, 
		TEXT("MaterialExpressionNoise(%s,%s,%s,%s,%s,%s,%s,%s,%s,%s,%s,%s)"), 
		*GetParameterCode(Position),
		*GetParameterCode(ScaleConst),
		*GetParameterCode(QualityConst),
		*GetParameterCode(NoiseFunctionConst),
		*GetParameterCode(TurbulenceConst),
		*GetParameterCode(LevelsConst),
		*GetParameterCode(OutputMinConst),
		*GetParameterCode(OutputMaxConst),
		*GetParameterCode(LevelScaleConst),
		*GetParameterCode(FilterWidth),
		*GetParameterCode(TilingConst),
		*GetParameterCode(RepeatSizeConst));
}

int32 FHLSLMaterialTranslator::VectorNoise(int32 Position, EPositionOrigin PositionOrigin, int32 Quality, uint8 NoiseFunction, bool bTiling, uint32 TileSize)
{
	if (Position == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PositionOrigin == EPositionOrigin::CameraRelative)
	{
		//LWC_TODO: add support for translated world positions in the corresponding HLSL function
		Position = TransformPosition(MCB_TranslatedWorld, MCB_World, Position);
	}

	int32 QualityConst = Constant(Quality);
	int32 NoiseFunctionConst = Constant(NoiseFunction);
	int32 TilingConst = Constant(bTiling);
	int32 TileSizeConst = Constant(TileSize);

	if (NoiseFunction == VNF_GradientALU || NoiseFunction == VNF_VoronoiALU)
	{
		return AddCodeChunk(MCT_Float4,
			TEXT("MaterialExpressionVectorNoise(%s,%s,%s,%s,%s)"),
			*CoerceParameter(Position, MCT_Float3), // LWC_TODO - maybe possible/useful to add LWC-aware noise functions
			*GetParameterCode(QualityConst),
			*GetParameterCode(NoiseFunctionConst),
			*GetParameterCode(TilingConst),
			*GetParameterCode(TileSizeConst));
	}
	else
	{
		return AddCodeChunk(MCT_Float3,
			TEXT("MaterialExpressionVectorNoise(%s,%s,%s,%s,%s).xyz"),
			*CoerceParameter(Position, MCT_Float3),
			*GetParameterCode(QualityConst),
			*GetParameterCode(NoiseFunctionConst),
			*GetParameterCode(TilingConst),
			*GetParameterCode(TileSizeConst));
	}
}

int32 FHLSLMaterialTranslator::ScalarBlueNoise()
{
	if (Material->GetMaterialDomain() != EMaterialDomain::MD_Surface)
	{
		Errorf(TEXT("ScalarBlueNoise node can only be used within Surface domain materials %s (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
	}

	AddEstimatedTextureSample(1);

	return AddCodeChunk(MCT_Float1, TEXT("ViewScalarBlueNoise(GetPixelPosition(Parameters), View.StateFrameIndex)"));
}

int32 FHLSLMaterialTranslator::BlackBody( int32 Temp )
{
	if( Temp == INDEX_NONE )
	{
		return INDEX_NONE;
	}

	return AddCodeChunk( MCT_Float3, TEXT("MaterialExpressionBlackBody(%s)"), *GetParameterCode(Temp) );
}

int32 FHLSLMaterialTranslator::GetHairUV()
{
	return AddCodeChunk(MCT_Float2, TEXT("MaterialExpressionGetHairUV(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairDimensions()
{
	return AddCodeChunk(MCT_Float2, TEXT("MaterialExpressionGetHairDimensions(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairSeed()
{
	return AddCodeChunk(MCT_Float1, TEXT("MaterialExpressionGetHairSeed(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairClumpID()
{
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionGetHairClumpID(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairTangent(bool bUseTangentSpace)
{
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionGetHairTangent(Parameters, %s)"), bUseTangentSpace ? TEXT("true") : TEXT("false"));
}

int32 FHLSLMaterialTranslator::GetHairRootUV()
{
	return AddCodeChunk(MCT_Float2, TEXT("MaterialExpressionGetHairRootUV(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairBaseColor()
{
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionGetHairBaseColor(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairRoughness()
{
	return AddCodeChunk(MCT_Float1, TEXT("MaterialExpressionGetHairRoughness(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairAO()
{
	return AddCodeChunk(MCT_Float1, TEXT("MaterialExpressionGetHairAO(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairDepth()
{
	return AddCodeChunk(MCT_Float1, TEXT("MaterialExpressionGetHairDepth(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairCoverage()
{
	return AddCodeChunk(MCT_Float1, TEXT("MaterialExpressionGetHairCoverage(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairAuxilaryData()
{
	return AddCodeChunk(MCT_Float4, TEXT("MaterialExpressionGetHairAuxilaryData(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairAtlasUVs()
{
	return AddCodeChunk(MCT_Float2, TEXT("MaterialExpressionGetAtlasUVs(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairGroupIndex()
{
	return AddCodeChunk(MCT_Float1, TEXT("MaterialExpressionGetHairGroupIndex(Parameters)"));
}

int32 FHLSLMaterialTranslator::GetHairColorFromMelanin(int32 Melanin, int32 Redness, int32 DyeColor)
{
	if (Melanin == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (Redness == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (DyeColor == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionGetHairColorFromMelanin(%s, %s, %s)"), *GetParameterCode(Melanin), *GetParameterCode(Redness), *GetParameterCode(DyeColor));
}

int32 FHLSLMaterialTranslator::DistanceToNearestSurface(int32 PositionArg, EPositionOrigin PositionOrigin)
{
	if (ErrorUnlessPlatformSupports(FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields, TEXT("DistanceField")) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PositionArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	MaterialCompilationOutput.bUsesGlobalDistanceField = true;

	return AddCodeChunk(MCT_Float, TEXT("GetDistanceToNearestSurfaceGlobal(%s)"), *GetWorldPositionOrDefault(PositionArg, PositionOrigin));
}

int32 FHLSLMaterialTranslator::DistanceFieldGradient(int32 PositionArg, EPositionOrigin PositionOrigin)
{
	if (ErrorUnlessPlatformSupports(FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields, TEXT("DistanceField")) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PositionArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	MaterialCompilationOutput.bUsesGlobalDistanceField = true;

	return AddCodeChunk(MCT_Float3, TEXT("GetDistanceFieldGradientGlobal(%s)"), *GetWorldPositionOrDefault(PositionArg, PositionOrigin));
}

int32 FHLSLMaterialTranslator::DistanceFieldApproxAO(int32 PositionArg, EPositionOrigin PositionOrigin, int32 NormalArg, int32 BaseDistanceArg, int32 RadiusArg, uint32 NumSteps, float StepScale)
{
	if (ErrorUnlessPlatformSupports(FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields, TEXT("DistanceField")) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PositionArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (NormalArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (BaseDistanceArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (RadiusArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	MaterialCompilationOutput.bUsesGlobalDistanceField = true;

	NumSteps = FMath::Clamp(NumSteps, 1, 4);
	StepScale = FMath::Max(StepScale, 1.0f);

	const int32 NumStepsConst = Constant(NumSteps);
	const int32 NumStepsMinus1Const = Constant(NumSteps - 1);
	const int32 StepScaleConst = Constant(StepScale);

	int32 StepDistance;
	int32 DistanceBias;
	int32 MaxDistance;

	if (NumSteps == 1)
	{
		StepDistance = Constant(0);
		DistanceBias = BaseDistanceArg;
		MaxDistance = BaseDistanceArg;
	}
	else
	{
		StepDistance = Div(Sub(RadiusArg, BaseDistanceArg), Sub(Power(StepScaleConst, NumStepsMinus1Const), Constant(1)));
		DistanceBias = Sub(BaseDistanceArg, StepDistance);
		MaxDistance = RadiusArg;
	}

	return AddCodeChunk(MCT_Float,
		TEXT("CalculateDistanceFieldApproxAO(%s, %s, %s, %s, %s, %s, %s)"),
		*GetWorldPositionOrDefault(PositionArg, PositionOrigin),
		*CoerceParameter(NormalArg, MCT_Float3),
		*GetParameterCode(NumStepsConst),
		*CoerceParameter(StepDistance, MCT_Float),
		*GetParameterCode(StepScaleConst),
		*CoerceParameter(DistanceBias, MCT_Float),
		*CoerceParameter(MaxDistance, MCT_Float));
}

int32 FHLSLMaterialTranslator::ProcessPhysicsField(int32 PositionArg, EPositionOrigin PositionOrigin, const int32 OutputType, const int32 TargetIndex, const bool bSampleField)
{
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// TODO: add physics field samlping to mobile
		if (OutputType == EFieldOutputType::Field_Output_Vector)
		{
			return Constant3(0.0f, 0.0f, 0.0f);
		}
		else if (OutputType == EFieldOutputType::Field_Output_Scalar || OutputType == EFieldOutputType::Field_Output_Integer)
		{
			return Constant(0.0f);
		}
	}

	if (ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::SM5) == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PositionArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (TargetIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PositionOrigin == EPositionOrigin::CameraRelative)
	{
		// LWC_TODO: support translated-world coordinates
		PositionArg = TransformPosition(MCB_TranslatedWorld, MCB_World, PositionArg);
	}

	// LWC_TODO: LWC aware physics field
	if (OutputType == EFieldOutputType::Field_Output_Vector)
	{
		return bSampleField ? AddCodeChunk(MCT_Float3, TEXT("MatPhysicsField_SamplePhysicsVectorField(%s,%d)"), *CoerceParameter(PositionArg, MCT_Float3), static_cast<uint8>(TargetIndex)) : 
							  AddCodeChunk(MCT_Float3, TEXT("MatPhysicsField_EvalPhysicsVectorField(%s,%d)"), *CoerceParameter(PositionArg, MCT_Float3), static_cast<uint8>(TargetIndex));
	}
	else if (OutputType == EFieldOutputType::Field_Output_Scalar)
	{
		return bSampleField ? AddCodeChunk(MCT_Float, TEXT("MatPhysicsField_SamplePhysicsScalarField(%s,%d)"), *CoerceParameter(PositionArg, MCT_Float3), static_cast<uint8>(TargetIndex)) : 
		                      AddCodeChunk(MCT_Float, TEXT("MatPhysicsField_EvalPhysicsScalarField(%s,%d)"), *CoerceParameter(PositionArg, MCT_Float3), static_cast<uint8>(TargetIndex));
	}
	else if (OutputType == EFieldOutputType::Field_Output_Integer)
	{
		return bSampleField ? AddCodeChunk(MCT_Float, TEXT("MatPhysicsField_SamplePhysicsIntegerField(%s,%d)"), *CoerceParameter(PositionArg, MCT_Float3), static_cast<uint8>(TargetIndex)) :
							  AddCodeChunk(MCT_Float, TEXT("MatPhysicsField_EvalPhysicsIntegerField(%s,%d)"), *CoerceParameter(PositionArg, MCT_Float3), static_cast<uint8>(TargetIndex));
	}
	else
	{
		return INDEX_NONE;
	}
}

int32 FHLSLMaterialTranslator::AtmosphericFogColor( int32 WorldPosition, EPositionOrigin PositionOrigin )
{
	return SkyAtmosphereAerialPerspective(WorldPosition, PositionOrigin);
}

int32 FHLSLMaterialTranslator::AtmosphericLightVector()
{
	return AddExternalCodeChunk(TEXT("AtmosphericLightVector"));
}

int32 FHLSLMaterialTranslator::AtmosphericLightColor()
{
	return AddExternalCodeChunk(TEXT("AtmosphericLightColor"));
}

int32 FHLSLMaterialTranslator::SkyAtmosphereLightIlluminance(int32 WorldPosition, EPositionOrigin PositionOrigin, int32 LightIndex)
{
	bUsesSkyAtmosphere = true;
	FString WorldPosCode = GetWorldPositionOrDefault(WorldPosition, PositionOrigin);
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyAtmosphereLightIlluminance(Parameters, %s, %d)"), *WorldPosCode, LightIndex);
}

int32 FHLSLMaterialTranslator::SkyAtmosphereLightIlluminanceOnGround(int32 LightIndex)
{
	bUsesSkyAtmosphere = true;
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyAtmosphereLightIlluminanceOnGround(Parameters, %d)"), LightIndex);
}

int32 FHLSLMaterialTranslator::SkyAtmosphereLightDirection(int32 LightIndex)
{
	bUsesSkyAtmosphere = true;
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyAtmosphereLightDirection(Parameters, %d)"), LightIndex);
}

int32 FHLSLMaterialTranslator::SkyAtmosphereLightDiskLuminance(int32 LightIndex, int32 OverrideAtmosphereLightDiscCosHalfApexAngle)
{
	bUsesSkyAtmosphere = true;
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyAtmosphereLightDiskLuminance(Parameters, %d, %s)"), LightIndex, 
		OverrideAtmosphereLightDiscCosHalfApexAngle == INDEX_NONE ? *GetParameterCode(Constant(-1.0f)) : *GetParameterCode(OverrideAtmosphereLightDiscCosHalfApexAngle));
}

int32 FHLSLMaterialTranslator::SkyAtmosphereViewLuminance(int32 WorldDirectionOverrideCodeChunk)
{
	if (WorldDirectionOverrideCodeChunk == INDEX_NONE)
	{
		const int32 CameraVectorCode = CameraVector();
		if (CameraVectorCode == INDEX_NONE)
		{
			return INDEX_NONE;
		}
		WorldDirectionOverrideCodeChunk = Mul(Constant3(-1.0f, -1.0f, -1.0f), CameraVectorCode);
	}

	bUsesSkyAtmosphere = true;
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyAtmosphereViewLuminance(Parameters, %s)"), *GetParameterCode(WorldDirectionOverrideCodeChunk));
}

int32 FHLSLMaterialTranslator::SkyAtmosphereAerialPerspective(int32 WorldPosition, EPositionOrigin PositionOrigin)
{
	bUsesSkyAtmosphere = true;
	FString WorldPosCode = GetWorldPositionOrDefault(WorldPosition, PositionOrigin);
	return AddCodeChunk(MCT_Float4, TEXT("MaterialExpressionSkyAtmosphereAerialPerspective(Parameters, %s)"), *WorldPosCode);
}

int32 FHLSLMaterialTranslator::SkyAtmosphereDistantLightScatteredLuminance()
{
	bUsesSkyAtmosphere = true;
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyAtmosphereDistantLightScatteredLuminance(Parameters)"));
}

int32 FHLSLMaterialTranslator::SkyLightEnvMapSample(int32 DirectionCodeChunk, int32 RoughnessCodeChunk)
{
	if (DirectionCodeChunk == INDEX_NONE || RoughnessCodeChunk == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (Material->GetMaterialDomain() != MD_Surface)
	{
		return Errorf(TEXT("The SkyLightEnvMapSample node can only be used when material Domain is set to Surface."));
	}
	if (Material->IsSky())
	{
		UE_LOGF(LogMaterial, Warning, "Using SkyLightEnvMapSample from a IsSky material can result in visual artifact. For instance, if the previous frame capture was super bright, it might leak onto a new frame, e.g. transtion from menu to game.");
	}

	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionSkyLightEnvMapSample(%s, %s)"), *GetParameterCode(DirectionCodeChunk), *GetParameterCode(RoughnessCodeChunk));
}

int32 FHLSLMaterialTranslator::MainDirectionalLightIlluminance()
{
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionMainDirectionalLightIlluminance()"));
}

int32 FHLSLMaterialTranslator::MainDirectionalLightDirection()
{
	return AddCodeChunk(MCT_Float3, TEXT("MaterialExpressionMainDirectionalLightDirection()"));
}

int32 FHLSLMaterialTranslator::SceneDepthWithoutWater(int32 Offset, int32 ViewportUV, bool bUseOffset, float FallbackDepth)
{
	if (ShaderFrequency == SF_Vertex)
	{
		// Mobile currently does not support this, we need to read a separate copy of the depth, we must disable framebuffer fetch and force scene texture reads.
		// (Texture bindings are not setup properly for any platform so we're disallowing usage in vertex shader altogether now)
		return Errorf(TEXT("Cannot read scene depth without water from the vertex shader."));
	}

	const EMaterialDomain MaterialDomain = Material->GetMaterialDomain();

	if (MaterialDomain != MD_PostProcess)
	{
		if (!Material->GetShadingModels().HasShadingModel(MSM_SingleLayerWater))
		{
			return Errorf(TEXT("Can only read scene depth below water when material Shading Model is Single Layer Water or when material Domain is PostProcess."));
		}

		if (MaterialDomain != MD_Surface)
		{
			return Errorf(TEXT("Can only read scene depth below water when material Domain is set to Surface or PostProcess."));
		}

		if (IsTranslucentBlendMode(Material->GetBlendMode()))
		{
			return Errorf(TEXT("Can only read scene depth below water when material Blend Mode isn't translucent."));
		}
	}

	if (Offset == INDEX_NONE && bUseOffset)
	{
		return INDEX_NONE;
	}

	AddEstimatedTextureSample();

	const FString UserDepthCode("MaterialExpressionSceneDepthWithoutWater(%s, %s)");
	const FString FallbackString(FString::SanitizeFloat(FallbackDepth));
	const int32 TexCoordCode = GetScreenAlignedUV(Offset, ViewportUV, bUseOffset);

	// add the code string
	return AddCodeChunk(
		MCT_Float,
		*UserDepthCode,
		*GetParameterCode(TexCoordCode),
		*FallbackString
	);
}

int32 FHLSLMaterialTranslator::GetCloudSampleAltitude()
{
	return AddExternalCodeChunk(TEXT("GetCloudSampleAltitude"));
}

int32 FHLSLMaterialTranslator::GetCloudSampleAltitudeInLayer()
{
	return AddExternalCodeChunk(TEXT("GetCloudSampleAltitudeInLayer"));
}

int32 FHLSLMaterialTranslator::GetCloudSampleNormAltitudeInLayer()
{
	return AddExternalCodeChunk(TEXT("GetCloudSampleNormAltitudeInLayer"));
}

int32 FHLSLMaterialTranslator::GetCloudSampleShadowSampleDistance()
{
	return AddExternalCodeChunk(TEXT("GetCloudSampleShadowSampleDistance"));
}

int32 FHLSLMaterialTranslator::GetVolumeSampleConservativeDensity()
{
	return AddExternalCodeChunk(TEXT("GetVolumeSampleConservativeDensity"));
}

int32 FHLSLMaterialTranslator::GetCloudEmptySpaceSkippingSphereCenterWorldPosition()
{
	return AddExternalCodeChunk(TEXT("GetCloudEmptySpaceSkippingSphereCenterWorldPosition"));
}

int32 FHLSLMaterialTranslator::GetCloudEmptySpaceSkippingSphereRadius()
{
	return AddExternalCodeChunk(TEXT("GetCloudEmptySpaceSkippingSphereRadius"));
}

int32 FHLSLMaterialTranslator::CustomPrimitiveData(int32 OutputIndex, EMaterialValueType Type)
{
	check(OutputIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);

	const int32 NumComponents = GetNumComponents(Type);

	FString HlslCode;
			
	// Only float2, float3 and float4 need this
	if (NumComponents > 1)
	{
		HlslCode.Append(FString::Printf(TEXT("float%d("), NumComponents));
	}

	for (int i = 0; i < NumComponents; i++)
	{
		const int32 CurrentOutputIndex = OutputIndex + i;

		// Check if we are accessing inside the array, otherwise default to 0
		if (CurrentOutputIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats)
		{
			const int32 CustomDataIndex = CurrentOutputIndex / 4;
			const int32 ElementIndex = CurrentOutputIndex % 4; // Index x, y, z or w

			HlslCode.Append(FString::Printf(TEXT("GetPrimitiveData(Parameters).CustomPrimitiveData[%d][%d]"), CustomDataIndex, ElementIndex));
		}
		else
		{
			HlslCode.Append(TEXT("0.0f"));
		}

		if (i+1 < NumComponents)
		{
			HlslCode.Append(", ");
		}
	}

	// This is the matching parenthesis to the first append
	if (NumComponents > 1)
	{
		HlslCode.AppendChar(')');
	}

	return AddCodeChunkZeroDeriv(Type, TEXT("%s"), *HlslCode);
}

int32 FHLSLMaterialTranslator::ShadingModel(EMaterialShadingModel InSelectedShadingModel)
{
	// If the shading model is masked out, fallback to default shading model
	uint32 PlatformShadingModelsMask = GetPlatformShadingModelsMask(Platform);
	if ((PlatformShadingModelsMask & (1u << (uint32)InSelectedShadingModel)) == 0)
	{
		InSelectedShadingModel = MSM_DefaultLit;
	}
	
	if (InSelectedShadingModel < MSM_NUM)
	{
		ShadingModelsFromCompilation.AddShadingModel(InSelectedShadingModel);
	}
	return AddInlinedCodeChunk(MCT_ShadingModel, TEXT("%d"), InSelectedShadingModel);
}

int32 FHLSLMaterialTranslator::DefaultMaterialAttributes()
{
	// Zero-initialized attribute set
	int32   Attributes    = AddCodeChunk(MCT_MaterialAttributes, TEXT("(FMaterialAttributes)0"));
	FString AttributesStr = CoerceParameter(Attributes, MCT_MaterialAttributes);

	// Default initialize attributes
	for (FGuid AttributeGUID : FMaterialAttributeDefinitionMap::GetOrderedVisibleAttributeList())
	{
		if (FMaterialAttributeDefinitionMap::GetValueType(AttributeGUID) == MCT_Substrate)
		{
			continue;
		}
		
		if (FMaterialAttributeDefinitionMap::GetShaderFrequency(AttributeGUID) != ShaderFrequency)
		{
			continue;
		}

		const FString PropertyName = FMaterialAttributeDefinitionMap::GetAttributeName(AttributeGUID);
		const int32 Default = FMaterialAttributeDefinitionMap::CompileDefaultExpression(this, AttributeGUID);
		Attributes = AddCodeChunk(MCT_MaterialAttributes, TEXT("FMaterialAttributes_Set%s(%s, %s)"), *PropertyName, *GetParameterCode(Attributes), *GetParameterCode(Default));
	}

	return Attributes;
}

int32 FHLSLMaterialTranslator::SetMaterialAttribute(int32 MaterialAttributes, int32 Value, const FGuid& AttributeID)
{
	const EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(AttributeID);
	const FString PropertyName = FMaterialAttributeDefinitionMap::GetAttributeName(AttributeID);
	const EMaterialValueType PropertyType = FMaterialAttributeDefinitionMap::GetValueType(AttributeID);
	const EShaderFrequency Frequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(AttributeID);
	
	if (MaterialAttributes == INDEX_NONE ||
		GetParameterType(MaterialAttributes) != MCT_MaterialAttributes)
	{
		return Error(TEXT("Expected MaterialAttributes"));
	}

	if (Frequency != ShaderFrequency)
	{
		return Errorf(TEXT("Can't set material attribute %s from shader stage %d"), *PropertyName, ShaderFrequency);
	}

	const int32 CastValue = ValidCast(Value, PropertyType);
	FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(CastValue);
	bool bSetDefaultValue = false;
	if (UniformExpression && UniformExpression->IsConstant())
	{
		FMaterialRenderContext Context(nullptr, *Material, nullptr);
		FLinearColor ConstantValue(ForceInitToZero);
		UniformExpression->GetNumberValue(Context, ConstantValue);
		const FVector4f DefaultValue = FMaterialAttributeDefinitionMap::GetDefaultValue(AttributeID);
		bSetDefaultValue = (ConstantValue == FLinearColor(DefaultValue));
	}

	uint64 AttributeMask = GetParameterMaterialAttributeMask(MaterialAttributes);
	if (bSetDefaultValue)
	{
		if (!(AttributeMask & (1ull << Property)))
		{
			// Setting default value to an input that already has the default value set, this is a NOP
			return MaterialAttributes;
		}
		// Otherwise, explicitly set the value back to default
		AttributeMask &= ~(1ull << Property);
	}
	else
	{
		// Setting a non-default value
		AttributeMask |= (1ull << Property);
	}

	const int32 Result = AddCodeChunk(MCT_MaterialAttributes, TEXT("FMaterialAttributes_Set%s(%s, %s)"),
		*PropertyName,
		*GetParameterCode(MaterialAttributes),
		*GetParameterCode(CastValue));

	SetParameterMaterialAttributes(Result, AttributeMask);

	return Result;
}

int32 FHLSLMaterialTranslator::BeginScope()
{
	const int32 Result = AddCodeChunk(MCT_VoidStatement, TEXT(""));
	ScopeStack.Add(Result);
	return Result;
}

int32 FHLSLMaterialTranslator::BeginScope_If(int32 Condition)
{
	const int32 ConditionAsFloat = ForceCast(Condition, MCT_Float1);
	if (ConditionAsFloat == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 Result = AddCodeChunk(MCT_VoidStatement, TEXT("if (%s != 0.0f)"), *GetParameterCode(ConditionAsFloat));
	ScopeStack.Add(Result);

	return Result;
}

int32 FHLSLMaterialTranslator::BeginScope_Else()
{
	const int32 Result = AddCodeChunk(MCT_VoidStatement, TEXT("else"));
	ScopeStack.Add(Result);

	return Result;
}

int32 FHLSLMaterialTranslator::BeginScope_For(const UMaterialExpression* Expression, int32 StartIndex, int32 EndIndex, int32 IndexStep)
{
	const int32 StartIndexAsFloat = ForceCast(StartIndex, MCT_Float1);
	const int32 EndIndexAsFloat = ForceCast(EndIndex, MCT_Float1);
	const int32 IndexStepAsFloat = ForceCast(IndexStep, MCT_Float1);
	if (StartIndexAsFloat == INDEX_NONE || EndIndexAsFloat == INDEX_NONE || IndexStepAsFloat == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 LoopIndex = NumForLoops[ShaderFrequency]++;
	check(!ForLoopMap[ShaderFrequency].Contains(Expression));
	ForLoopMap[ShaderFrequency].Add(Expression, LoopIndex);

	const int32 Result = AddCodeChunk(MCT_VoidStatement, TEXT("for (float ForLoopCounter%d = %s; ForLoopCounter%d < %s; ForLoopCounter%d += %s)"),
		LoopIndex, *GetParameterCode(StartIndexAsFloat),
		LoopIndex, *GetParameterCode(EndIndexAsFloat),
		LoopIndex, *GetParameterCode(IndexStepAsFloat));
	ScopeStack.Add(Result);
	return Result;
}

int32 FHLSLMaterialTranslator::EndScope()
{
	return ScopeStack.Pop(EAllowShrinking::No);
}

int32 FHLSLMaterialTranslator::ForLoopIndex(const UMaterialExpression* Expression)
{
	const int32* Result = ForLoopMap[ShaderFrequency].Find(Expression);
	if (Result)
	{
		return AddInlinedCodeChunk(MCT_Float1, TEXT("ForLoopCounter%d"), *Result);
	}
	return Error(TEXT("Expression is not a for-loop"));
}

int32 FHLSLMaterialTranslator::ReturnMaterialAttributes(int32 MaterialAttributes)
{
	if (MaterialAttributes == INDEX_NONE ||
		GetParameterType(MaterialAttributes) != MCT_MaterialAttributes)
	{
		return Error(TEXT("Expected MaterialAttributes"));
	}

	const uint64 AttributeMask = GetParameterMaterialAttributeMask(MaterialAttributes);
	MaterialAttributesReturned[ShaderFrequency] |= AttributeMask;

	return AddCodeChunk(MCT_VoidStatement, TEXT("return %s;"), *GetParameterCode(MaterialAttributes));
}

int32 FHLSLMaterialTranslator::SetLocal(const FName& LocalName, int32 Value)
{
	const int32 ValueAsFloat = ForceCast(Value, MCT_Float1);
	if (ValueAsFloat == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	FMaterialLocalVariableEntry& Entry = LocalVariables[ShaderFrequency].FindOrAdd(LocalName);
	if (Entry.DeclarationCodeIndex == INDEX_NONE)
	{
		Entry.Name = LocalName.ToString();
		Entry.DeclarationCodeIndex = AddCodeChunk(MCT_VoidStatement, TEXT("float %s = %s;"), *Entry.Name, *GetParameterCode(ValueAsFloat));
		return Entry.DeclarationCodeIndex;
	}
	else
	{
		// ensure the declaration is visible in the current scope
		AddCodeChunkToCurrentScope(Entry.DeclarationCodeIndex);
		return AddCodeChunk(MCT_VoidStatement, TEXT("%s = %s;"), *Entry.Name, *GetParameterCode(ValueAsFloat));
	}
}

int32 FHLSLMaterialTranslator::GetLocal(const FName& LocalName)
{
	const FMaterialLocalVariableEntry* Entry = LocalVariables[ShaderFrequency].Find(LocalName);
	if (!Entry)
	{
		return Errorf(TEXT("Local %s used before being set"), *LocalName.ToString());
	}

	// ensure the declaration is visible in the current scope
	AddCodeChunkToCurrentScope(Entry->DeclarationCodeIndex);
	return AddInlinedCodeChunk(MCT_Float1, TEXT("%s"), *Entry->Name);
}

int32 FHLSLMaterialTranslator::NeuralOutput(int32 ViewportUV, uint32 NeuralIndexType)
{
	if (Material->GetMaterialDomain() != MD_PostProcess)
	{
		Errorf(TEXT("NNE Output Node are only available on post process material."));
	}

	AddEstimatedTextureSample();
	MaterialCompilationOutput.bUsedWithNeuralNetworks = true;
	
	if (NeuralIndexType == 0)
	{
		if (ViewportUV == INDEX_NONE)
		{
			ViewportUV = AddInlinedCodeChunk(MCT_Float2, TEXT("GetViewportUV(Parameters)"));
		}

		return AddCodeChunk(MCT_Float4, TEXT("NeuralTextureOutput(Parameters,%s)"),
			*CoerceParameter(ViewportUV, MCT_Float2));
	}
	else if (NeuralIndexType == 1)
	{
		int32 BufferIndex = INDEX_NONE;

		if (ViewportUV == INDEX_NONE)
		{
			ViewportUV = AddInlinedCodeChunk(MCT_Float2, TEXT("GetViewportUV(Parameters)"));
			BufferIndex = AppendVector(Constant2(0.0f, 0.0f), ViewportUV);
		}
		else
		{
			BufferIndex = ViewportUV;
		}

		return AddCodeChunk(MCT_Float4, TEXT("NeuralBufferOutput(Parameters,%s)"),
			*CoerceParameter(BufferIndex, MCT_Float4));
	}

	return INDEX_NONE;
}

FSubstrateOperator& FHLSLMaterialTranslator::SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending)
{
	return SubstrateTranslatorData.SubstrateCompilationRegisterOperator(OperatorType, SubstrateExpressionGuid, ChildMaterialExpressionGuid, Parent, SubstrateParentExpressionGuid, bUseParameterBlending);
}

FSubstrateOperator& FHLSLMaterialTranslator::SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid)
{
	return SubstrateTranslatorData.SubstrateCompilationGetOperator(SubstrateExpressionGuid);
}

FSubstrateOperator* FHLSLMaterialTranslator::SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex)
{
	return SubstrateTranslatorData.SubstrateCompilationGetOperatorFromIndex(OperatorIndex);
}

FSubstrateRegisteredSharedLocalBasis FHLSLMaterialTranslator::SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk)
{
	// Specific implementation for FHLSLMaterialTranslator on the common FSubstrateTranslatorData

	FSubstrateCompilationContext& SubstrateCtx = SubstrateTranslatorData.SubstrateCompilationContext[SubstrateTranslatorData.CurrentSubstrateCompilationContext];
	check(NormalCodeChunk != INDEX_NONE);
	check(SubstrateCtx.NextFreeSubstrateShaderNormalIndex < 255);	// Out of shared local basis slots

	FSubstrateRegisteredSharedLocalBasis SubstrateRegisteredSharedLocalBasis;
	SubstrateRegisteredSharedLocalBasis.NormalCodeChunk		= NormalCodeChunk;
	SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash	= GetParameterHash(NormalCodeChunk);
	SubstrateRegisteredSharedLocalBasis.TangentCodeChunk		= INDEX_NONE;
	SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash	= GetParameterHash(INDEX_NONE);
	SubstrateRegisteredSharedLocalBasis.GraphSharedLocalBasisIndex = SubstrateCtx.NextFreeSubstrateShaderNormalIndex++;

	// Find any basis which match the Normal code chunk
	// A normal can be duplicated when it is paired with different tangent, so find the first one which matches
	TArray<FSubstrateSharedLocalBasesInfo*> NormalInfos;
	SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.MultiFindPointer(SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash, NormalInfos);
	if (NormalInfos.Num() == 0)
	{
		SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.Add(SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash, { SubstrateRegisteredSharedLocalBasis, *GetParameterCode(SubstrateRegisteredSharedLocalBasis.NormalCodeChunk), FString() });
		return SubstrateRegisteredSharedLocalBasis;
	}
	// Return the first existing code chunk which match the normal chunk code
	return NormalInfos[0]->SharedData;
}

FSubstrateRegisteredSharedLocalBasis FHLSLMaterialTranslator::SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk, int32 TangentCodeChunk)
{
	// Specific implementation for FHLSLMaterialTranslator on the common FSubstrateTranslatorData

	FSubstrateCompilationContext& SubstrateCtx = SubstrateTranslatorData.SubstrateCompilationContext[SubstrateTranslatorData.CurrentSubstrateCompilationContext];
	check(NormalCodeChunk != INDEX_NONE);
	check(SubstrateCtx.NextFreeSubstrateShaderNormalIndex < 255);	// Out of shared local basis slots
	

	FSubstrateRegisteredSharedLocalBasis SubstrateRegisteredSharedLocalBasis;
	SubstrateRegisteredSharedLocalBasis.NormalCodeChunk		= NormalCodeChunk;
	SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash	= GetParameterHash(NormalCodeChunk);
	SubstrateRegisteredSharedLocalBasis.TangentCodeChunk		= TangentCodeChunk;
	SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash	= GetParameterHash(TangentCodeChunk);
	SubstrateRegisteredSharedLocalBasis.GraphSharedLocalBasisIndex = SubstrateCtx.NextFreeSubstrateShaderNormalIndex++;

	// Find a basis which matches both the Normal & the Tangent code chunks
	TArray<FSubstrateSharedLocalBasesInfo*> NormalInfos;
	SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.MultiFindPointer(SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash, NormalInfos);
	for (FSubstrateSharedLocalBasesInfo* NormalInfo : NormalInfos)
	{
		// * Either we find a perfect match (normal & tangent matches)
		// * Or we find a normal which doesn't have a tangent associated with, and we set the tangent for code
		if (SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash == NormalInfo->SharedData.TangentCodeChunk)
		{
			return NormalInfo->SharedData;
		}
		else if (NormalInfo->SharedData.TangentCodeChunk == INDEX_NONE)
		{
			NormalInfo->SharedData.TangentCodeChunk		= TangentCodeChunk;
			NormalInfo->SharedData.TangentCodeChunkHash = SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash;
			NormalInfo->TangentCode				= *GetParameterCode(TangentCodeChunk);
			return NormalInfo->SharedData;
		}
	}

	// Allocate a new slot for a new shared local basis
	SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.Add(SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash, { SubstrateRegisteredSharedLocalBasis, *GetParameterCode(SubstrateRegisteredSharedLocalBasis.NormalCodeChunk), *GetParameterCode(SubstrateRegisteredSharedLocalBasis.TangentCodeChunk) });
	return SubstrateRegisteredSharedLocalBasis;
}

FString FHLSLMaterialTranslator::GetSubstrateSharedLocalBasisIndexMacro(const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis)
{
	return Substrate::GetSubstrateSharedLocalBasisIndexMacroInner(SharedLocalBasis, SubstrateTranslatorData.CurrentSubstrateCompilationContext);
}

int32 FHLSLMaterialTranslator::SubstrateAddParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 ACodeChunk, int32 BCodeChunk)
{
	if (ACodeChunk == INDEX_NONE || BCodeChunk == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(MCT_Float, TEXT("AddParameterBlendingBSDFCoverageToNormalMix(%s, %s)"), *GetParameterCode(ACodeChunk), *GetParameterCode(BCodeChunk));
}

int32 FHLSLMaterialTranslator::SubstrateVerticalLayeringParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 TopCodeChunk)
{
	if (TopCodeChunk == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(MCT_Float, TEXT("VerticalLayeringParameterBlendingBSDFCoverageToNormalMix(%s)"), *GetParameterCode(TopCodeChunk));
}

int32 FHLSLMaterialTranslator::SubstrateHorizontalMixingParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 BackgroundCodeChunk, int32 ForegroundCodeChunk, int32 HorizontalMixCodeChunk)
{
	if (BackgroundCodeChunk == INDEX_NONE || ForegroundCodeChunk == INDEX_NONE || HorizontalMixCodeChunk == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(MCT_Float, TEXT("HorizontalMixingParameterBlendingBSDFCoverageToNormalMix(%s, %s, %s)"), *GetParameterCode(BackgroundCodeChunk), *GetParameterCode(ForegroundCodeChunk), *GetParameterCode(HorizontalMixCodeChunk));
}

int32 FHLSLMaterialTranslator::SubstrateCreateAndRegisterNullMaterial()
{
	int32 OutputCodeChunk = AddInlinedCodeChunk(MCT_Substrate, TEXT("GetInitialisedSubstrateData()"));
	return OutputCodeChunk;
}

FString FHLSLMaterialTranslator::SubstrateGetCastParameterCode(int32 Index, EMaterialValueType DestType)
{
	const int32 CastParameter = ForceCast(Index, DestType);

	// When casting a Texture Object, it will fail to assign a texture uniform to float3
	if (CastParameter == INDEX_NONE)
	{
		return FString();
	}

	return GetParameterCode(CastParameter);
}

FString FHLSLMaterialTranslator::SubstrateGetCastParameterCodeWithDeriv(int32 Index, EMaterialValueType DestType)
{
	int32 CastParameter = ForceCast(Index, DestType);
	if (IsAnalyticDerivEnabled())
	{
		return GetParameterCodeDeriv(CastParameter, CompiledPDV_Analytic);
	}
	else
	{
		return GetParameterCode(CastParameter);
	}
}

int32 FHLSLMaterialTranslator::SubstrateSlabBSDF(
	int32 DiffuseAlbedo, int32 F0, int32 F90,
	int32 Roughness, int32 Anisotropy,
	int32 SSSProfileId, int32 SSSMFP, int32 SSSMFPScale, int32 SSSPhaseAniso, int32 SSSType,
	int32 EmissiveColor,
	int32 SecondRoughness, int32 SecondRoughnessWeight, int32 SecondRoughnessAsSimpleClearCoat, int32 ClearCoatBottomNormal,
	int32 FuzzAmount, int32 FuzzColor, int32 FuzzRoughness,
	int32 Thickness,
	int32 GlintValue, int32 GlintUV,
	int32 SpecularProfileId,
	bool bIsAtTheBottomOfTopology,
	int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
	FSubstrateOperator* PromoteToOperator)
{
	const FString NormalCode = GetParameterCode(Normal);
	const FString TangentCode = Tangent != INDEX_NONE ? *GetParameterCode(Tangent) : TEXT("NONE");
	const FString ThicknessCode = GetParameterCode(Thickness);
	const bool bIsThinSurface = Material->IsThinSurface();

	int32 ClearCoatUseSecondNormal = Constant(ClearCoatBottomNormal != Normal ? 1.0f : 0.0f);

	if (PromoteToOperator)
	{
		if (PromoteToOperator->Index == INDEX_NONE || PromoteToOperator->BSDFIndex == INDEX_NONE)
		{
			Errorf(TEXT("Invalid SubstrateSlabBSDF operator and BSDF indices during promotion in Material %s (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
			return INDEX_NONE;
		}

		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(GetSubstrateSlabBSDF(Parameters.SubstratePixelFootprint,\n\
				/*Normal*/                           %s,\n\
				/*DiffuseAlbedo*/                    %s,\n\
				/*F0*/                               %s,\n\
				/*F90*/                              %s,\n\
				/*Roughness*/                        %s,\n\
				/*Anisotropy*/                       %s,\n\
				/*SSSProfileId*/                     ExtractSubsurfaceProfileInt(%s),\n\
				/*bSupportDefaultSSSProfile*/		 false,\n\
				/*SSSMFP*/                           %s,\n\
				/*SSSMFPScale*/                      %s,\n\
				/*SSSPhaseAniso*/                    %s,\n\
				/*SSSType*/                          %s,\n\
				/*EmissiveColor*/                    %s,\n\
				/*SecondRoughness*/                  %s,\n\
				/*SecondRoughnessWeight*/            %s,\n\
				/*SecondRoughnessAsSimpleClearCoat*/ %s,\n\
				/*ClearCoatUseSecondNormal*/         %s,\n\
				/*ClearCoatBottomNormal*/            %s,\n\
				/*FuzzAmount*/                       %s,\n\
				/*FuzzColor*/                        %s,\n\
				/*FuzzRoughness*/                    %s,\n\
				/*GlintValue*/                       %s,\n\
				/*GlintUV*/                          %s,\n\
				/*SpecularProfileId*/                %s,\n\
				/*Thickness*/                        %s,\n\
				/*IsThin*/                           %s,\n\
				/*IsAtBottom*/                       %s,\n\
				/*LocalBasisIndex*/                  %s,\n\
				Parameters.%s.Types)\n\
				 /* Normal = %s ; Tangent = %s ; Thickness = %s */, %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*SubstrateGetCastParameterCode(Normal,					MCT_Float3),
			*SubstrateGetCastParameterCode(DiffuseAlbedo,			MCT_Float3),
			*SubstrateGetCastParameterCode(F0,						MCT_Float3),
			*SubstrateGetCastParameterCode(F90,						MCT_Float3),
			*SubstrateGetCastParameterCode(Roughness,				MCT_Float),
			*SubstrateGetCastParameterCode(Anisotropy,				MCT_Float),
			*SubstrateGetCastParameterCode(SSSProfileId,			MCT_Float),
			*SubstrateGetCastParameterCode(SSSMFP,					MCT_Float3),
			*SubstrateGetCastParameterCode(SSSMFPScale,				MCT_Float),
			*SubstrateGetCastParameterCode(SSSPhaseAniso,			MCT_Float),
			*SubstrateGetCastParameterCode(SSSType,					MCT_Float),
			*SubstrateGetCastParameterCode(EmissiveColor,			MCT_Float3),
			*SubstrateGetCastParameterCode(SecondRoughness,			MCT_Float),
			*SubstrateGetCastParameterCode(SecondRoughnessWeight,	MCT_Float),
			*SubstrateGetCastParameterCode(SecondRoughnessAsSimpleClearCoat, MCT_Float),
			*SubstrateGetCastParameterCode(ClearCoatUseSecondNormal,MCT_Float),
			*SubstrateGetCastParameterCode(ClearCoatBottomNormal,	MCT_Float3),
			*SubstrateGetCastParameterCode(FuzzAmount,				MCT_Float),
			*SubstrateGetCastParameterCode(FuzzColor,				MCT_Float3),
			*SubstrateGetCastParameterCode(FuzzRoughness,			MCT_Float1),
			*SubstrateGetCastParameterCode(GlintValue,				MCT_Float),
			*SubstrateGetCastParameterCodeWithDeriv(GlintUV,		MCT_Float2),
			*SubstrateGetCastParameterCode(SpecularProfileId,		MCT_Float),
			*SubstrateGetCastParameterCode(Thickness,				MCT_Float),
			bIsThinSurface ? TEXT("true") : TEXT("false"),
			bIsAtTheBottomOfTopology ? TEXT("true") : TEXT("false"),
			*SharedLocalBasisIndexMacro,
			*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*NormalCode,
			*TangentCode,
			*ThicknessCode,
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0
		);
	}
	
	return AddCodeChunk(
		MCT_Substrate, TEXT("GetSubstrateSlabBSDF(Parameters.SubstratePixelFootprint,\n\
			/*Normal*/                           %s,\n\
			/*DiffuseAlbedo*/                    %s,\n\
			/*F0*/                               %s,\n\
			/*F90*/                              %s,\n\
			/*Roughness*/                        %s,\n\
			/*Anisotropy*/                       %s,\n\
			/*SSSProfileId*/                     ExtractSubsurfaceProfileInt(%s),\n\
			/*bSupportDefaultSSSProfile*/		 false,\n\
			/*SSSMFP*/                           %s,\n\
			/*SSSMFPScale*/                      %s,\n\
			/*SSSPhaseAniso*/                    %s,\n\
			/*SSSType*/                          %s,\n\
			/*EmissiveColor*/                    %s,\n\
			/*SecondRoughness*/                  %s,\n\
			/*SecondRoughnessWeight*/            %s,\n\
			/*SecondRoughnessAsSimpleClearCoat*/ %s,\n\
			/*ClearCoatUseSecondNormal*/         %s,\n\
			/*ClearCoatBottomNormal*/            %s,\n\
			/*FuzzAmount*/                       %s,\n\
			/*FuzzColor*/                        %s,\n\
			/*FuzzRoughness*/                    %s,\n\
			/*GlintValue*/                       %s,\n\
			/*GlintUV*/                          %s,\n\
			/*SpecularProfileId*/                %s,\n\
			/*Thickness*/                        %s,\n\
			/*IsThin*/                           %s,\n\
			/*IsAtBottom*/                       %s,\n\
			/*LocalBasisIndex*/                  %s,\n\
			Parameters.%s.Types) /* Normal = %s ; Tangent = %s ; Thickness = %s */"),
		*SubstrateGetCastParameterCode(Normal,					MCT_Float3),
		*SubstrateGetCastParameterCode(DiffuseAlbedo,			MCT_Float3),
		*SubstrateGetCastParameterCode(F0,						MCT_Float3),
		*SubstrateGetCastParameterCode(F90,						MCT_Float3),
		*SubstrateGetCastParameterCode(Roughness,				MCT_Float),
		*SubstrateGetCastParameterCode(Anisotropy,				MCT_Float),
		*SubstrateGetCastParameterCode(SSSProfileId,			MCT_Float),
		*SubstrateGetCastParameterCode(SSSMFP,					MCT_Float3),
		*SubstrateGetCastParameterCode(SSSMFPScale,				MCT_Float),
		*SubstrateGetCastParameterCode(SSSPhaseAniso,			MCT_Float),
		*SubstrateGetCastParameterCode(SSSType,					MCT_Float),
		*SubstrateGetCastParameterCode(EmissiveColor,			MCT_Float3),
		*SubstrateGetCastParameterCode(SecondRoughness,			MCT_Float),
		*SubstrateGetCastParameterCode(SecondRoughnessWeight,	MCT_Float),
		*SubstrateGetCastParameterCode(SecondRoughnessAsSimpleClearCoat, MCT_Float),
		*SubstrateGetCastParameterCode(Constant(0.0f),			MCT_Float),
		*SubstrateGetCastParameterCode(Constant3(0.0f, 0.0f, 0.0f), MCT_Float3),
		*SubstrateGetCastParameterCode(FuzzAmount,				MCT_Float),
		*SubstrateGetCastParameterCode(FuzzColor,				MCT_Float3),
		*SubstrateGetCastParameterCode(FuzzRoughness,			MCT_Float1),
		*SubstrateGetCastParameterCode(GlintValue,				MCT_Float),
		*SubstrateGetCastParameterCodeWithDeriv(GlintUV, 		MCT_Float2),
		*SubstrateGetCastParameterCode(SpecularProfileId,		MCT_Float),
		*SubstrateGetCastParameterCode(Thickness, 				MCT_Float),
		bIsThinSurface ? TEXT("true") : TEXT("false"),
		bIsAtTheBottomOfTopology ? TEXT("true") : TEXT("false"),
		*SharedLocalBasisIndexMacro,
		*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*NormalCode,
		*TangentCode,
		*ThicknessCode
	);
}

int32 FHLSLMaterialTranslator::SubstrateConversionFromLegacy(
	bool bHasDynamicShadingModels,
	int32 BaseColor, int32 Specular, int32 Metallic,
	int32 Roughness, int32 Anisotropy,
	int32 SubSurfaceColor, int32 SubSurfaceProfileId,
	int32 ClearCoat, int32 ClearCoatRoughness,
	int32 EmissiveColor,
	int32 Opacity,
	int32 ThinTranslucentTransmittanceColor,
	int32 ThinTranslucentSurfaceCoverage,
	int32 WaterScatteringCoefficients, int32 WaterAbsorptionCoefficients, int32 WaterPhaseG, int32 ColorScaleBehindWater,
	int32 InShadingModel,
	int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
	int32 ClearCoat_Normal, int32 ClearCoat_Tangent, const FString& ClearCoat_SharedLocalBasisIndexMacro,
	int32 CustomTangent_Tangent,
	FSubstrateOperator* PromoteToOperator)
{
	if (Normal == INDEX_NONE)
	{
		Errorf(TEXT("SubstrateConversionFromLegacy requires a valid normal input in Material %s (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
		return INDEX_NONE;
	}
	if (ClearCoat_Normal == INDEX_NONE)
	{
		Errorf(TEXT("SubstrateConversionFromLegacy requires a valid clear coat normal input in Material %s (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
		return INDEX_NONE;
	}
	const FString NormalCode = GetParameterCode(Normal);
	const FString TangentCode = Tangent != INDEX_NONE ? *GetParameterCode(Tangent) : TEXT("NONE");
	const int32 RawTangent = Tangent != INDEX_NONE ? Tangent : Normal;

	const FString ClearCoat_NormalCode = GetParameterCode(ClearCoat_Normal);
	const FString ClearCoat_TangentCode = Tangent != INDEX_NONE ? *GetParameterCode(ClearCoat_Tangent) : TEXT("NONE");

	UMaterial* BaseMaterial = Material->GetMaterialInterface()->GetBaseMaterial();
	// Material is probably an instanced material, so we check that this is the case before potentially overriding the ShadingModel from the material instance.
	UMaterialInterface* BaseMaterialInterface = static_cast<UMaterialInterface*>(BaseMaterial);
	UMaterialInterface*     MaterialInterface = Material->GetMaterialInterface();
	if (BaseMaterial && (BaseMaterialInterface != MaterialInterface))
	{
		FMaterialShadingModelField BaseMaterialShadingModels	= BaseMaterial->GetShadingModels();
		FMaterialShadingModelField MaterialShadingModels		= Material->GetShadingModels();

		// If the potentially instanced material does not have the same shading model as the instance material, this means it would have overridden the shading model.
		// From the UI, only a single shading model is selectable, so we simply apply the one coming form the material instance.
		if (MaterialShadingModels.IsValid() && MaterialShadingModels.CountShadingModels()==1 && MaterialShadingModels != BaseMaterialShadingModels)
		{
			bHasDynamicShadingModels = false;	// No need to go dynamic when there is only a single shading model selected.
			InShadingModel = ShadingModel(MaterialShadingModels.GetFirstShadingModel());
		}
	}

	if (InShadingModel == INDEX_NONE)
	{
		Errorf(TEXT("SubstrateConversionFromLegacy requires a valid shading model input in Material %s (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
		return INDEX_NONE;
	}

	if (PromoteToOperator)
	{
		if (PromoteToOperator->Index == INDEX_NONE || PromoteToOperator->BSDFIndex == INDEX_NONE)
		{
			Errorf(TEXT("Invalid SubstrateSlabBSDF operator and BSDF indices during promotion in Material %s (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
			return INDEX_NONE;
		}

		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateConvertLegacyMaterial%s(Parameters.SubstratePixelFootprint,\n\
				/*BaseColor*/                         %s,\n\
				/*Specular*/                          %s,\n\
				/*Metallic*/                          %s,\n\
				/*Roughness*/                         %s,\n\
				/*Anisotropy*/                        %s,\n\
				/*SubSurfaceColor*/                   %s,\n\
				/*SubSurfaceProfileId*/               ExtractSubsurfaceProfileInt(%s),\n\
				/*ClearCoat*/                         %s,\n\
				/*ClearCoatRoughness*/                %s,\n\
				/*EmissiveColor*/                     %s,\n\
				/*Opacity*/                           %s,\n\
				/*ThinTranslucentTransmittanceColor*/ %s,\n\
				/*ThinTranslucentSurfaceCoverage*/    %s,\n\
				/*WaterScatteringCoefficients*/       %s,\n\
				/*WaterAbsorptionCoefficients*/       %s,\n\
				/*WaterPhaseG*/                       %s,\n\
				/*ColorScaleBehindWater*/             %s,\n\
				/*InShadingModel*/                    %s,\n\
				/*Normal*/                            %s,\n\
				/*Tangent*/                           %s,\n\
				/*ClearCoat_Normal*/                  %s,\n\
				/*CustomTangent_Tangent*/             %s,\n\
				/*LocalBasisIndex*/                   %s,\n\
				/*ClearCoat_SharedLocalBasisIndex*/   %s,\n\
				Parameters.%s.Types)\n\
				/* Normal = %s ; Tangent = %s ; ClearCoat_Normal = %s ; ClearCoat_Tangent = %s */, %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			bHasDynamicShadingModels ? TEXT("Dynamic") : TEXT("Static"),
			*SubstrateGetCastParameterCode(BaseColor,							MCT_Float3),
			*SubstrateGetCastParameterCode(Specular,							MCT_Float),
			*SubstrateGetCastParameterCode(Metallic,							MCT_Float),
			*SubstrateGetCastParameterCode(Roughness,							MCT_Float),
			*SubstrateGetCastParameterCode(Anisotropy,							MCT_Float),
			*SubstrateGetCastParameterCode(SubSurfaceColor,						MCT_Float3),
			*SubstrateGetCastParameterCode(SubSurfaceProfileId,					MCT_Float),
			*SubstrateGetCastParameterCode(ClearCoat,							MCT_Float),
			*SubstrateGetCastParameterCode(ClearCoatRoughness,					MCT_Float),
			*SubstrateGetCastParameterCode(EmissiveColor,						MCT_Float3),
			*SubstrateGetCastParameterCode(Opacity,								MCT_Float),
			*SubstrateGetCastParameterCode(ThinTranslucentTransmittanceColor,	MCT_Float3),
			*SubstrateGetCastParameterCode(ThinTranslucentSurfaceCoverage,		MCT_Float1),
			*SubstrateGetCastParameterCode(WaterScatteringCoefficients,			MCT_Float3),
			*SubstrateGetCastParameterCode(WaterAbsorptionCoefficients,			MCT_Float3),
			*SubstrateGetCastParameterCode(WaterPhaseG,							MCT_Float),
			*SubstrateGetCastParameterCode(ColorScaleBehindWater,				MCT_Float3),
			*GetParameterCode(InShadingModel),
			// Raw access to Normal/Tangent/ClearCoatNormal/CustomTangent for conversion purpose
			*SubstrateGetCastParameterCode(Normal,								MCT_Float3),
			*SubstrateGetCastParameterCode(RawTangent,							MCT_Float3),
			*SubstrateGetCastParameterCode(ClearCoat_Normal,					MCT_Float3),
			*SubstrateGetCastParameterCode(CustomTangent_Tangent,				MCT_Float3),
			*SharedLocalBasisIndexMacro,
			*ClearCoat_SharedLocalBasisIndexMacro,
			*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			// Regular normal basis
			*NormalCode,
			*TangentCode,
			// Clear coat bottom layer normal basis
			*ClearCoat_NormalCode,
			*ClearCoat_TangentCode,
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateConvertLegacyMaterial%s(Parameters.SubstratePixelFootprint,\n\
			/*BaseColor*/                         %s,\n\
			/*Specular*/                          %s,\n\
			/*Metallic*/                          %s,\n\
			/*Roughness*/                         %s,\n\
			/*Anisotropy*/                        %s,\n\
			/*SubSurfaceColor*/                   %s,\n\
			/*SubSurfaceProfileId*/               ExtractSubsurfaceProfileInt(%s),\n\
			/*ClearCoat*/                         %s,\n\
			/*ClearCoatRoughness*/                %s,\n\
			/*EmissiveColor*/                     %s,\n\
			/*Opacity*/                           %s,\n\
			/*ThinTranslucentTransmittanceColor*/ %s,\n\
			/*ThinTranslucentSurfaceCoverage*/    %s,\n\
			/*WaterScatteringCoefficients*/       %s,\n\
			/*WaterAbsorptionCoefficients*/       %s,\n\
			/*WaterPhaseG*/                       %s,\n\
			/*ColorScaleBehindWater*/             %s,\n\
			/*InShadingModel*/                    %s,\n\
			/*Normal*/                            %s,\n\
			/*Tangent*/                           %s,\n\
			/*ClearCoat_Normal*/                  %s,\n\
			/*CustomTangent_Tangent*/             %s,\n\
			/*LocalBasisIndex*/                   %s,\n\
			/*ClearCoat_SharedLocalBasisIndex*/   %s,\n\
			Parameters.%s.Types)\n\
			/* Normal = %s ; Tangent = %s ; ClearCoat_Normal = %s ; ClearCoat_Tangent = %s */"),
		bHasDynamicShadingModels ? TEXT("Dynamic") : TEXT("Static"),
		*SubstrateGetCastParameterCode(BaseColor,							MCT_Float3),
		*SubstrateGetCastParameterCode(Specular,							MCT_Float),
		*SubstrateGetCastParameterCode(Metallic,							MCT_Float),
		*SubstrateGetCastParameterCode(Roughness,							MCT_Float),
		*SubstrateGetCastParameterCode(Anisotropy,							MCT_Float),
		*SubstrateGetCastParameterCode(SubSurfaceColor,						MCT_Float3),
		*SubstrateGetCastParameterCode(SubSurfaceProfileId,					MCT_Float),
		*SubstrateGetCastParameterCode(ClearCoat,							MCT_Float),
		*SubstrateGetCastParameterCode(ClearCoatRoughness,					MCT_Float),
		*SubstrateGetCastParameterCode(EmissiveColor,						MCT_Float3),
		*SubstrateGetCastParameterCode(Opacity,								MCT_Float),
		*SubstrateGetCastParameterCode(ThinTranslucentTransmittanceColor,	MCT_Float3),
		*SubstrateGetCastParameterCode(ThinTranslucentSurfaceCoverage,		MCT_Float1),
		*SubstrateGetCastParameterCode(WaterScatteringCoefficients,			MCT_Float3),
		*SubstrateGetCastParameterCode(WaterAbsorptionCoefficients,			MCT_Float3),
		*SubstrateGetCastParameterCode(WaterPhaseG,							MCT_Float),
		*SubstrateGetCastParameterCode(ColorScaleBehindWater,				MCT_Float3),
		*GetParameterCode(InShadingModel),
		// Raw access to Normal/Tangent/ClearCoatNormal/CustomTangent for conversion purpose
		*SubstrateGetCastParameterCode(Normal,								MCT_Float3),
		*SubstrateGetCastParameterCode(RawTangent,							MCT_Float3),
		*SubstrateGetCastParameterCode(ClearCoat_Normal,					MCT_Float3),
		*SubstrateGetCastParameterCode(CustomTangent_Tangent,				MCT_Float3),
		*SharedLocalBasisIndexMacro,
		*ClearCoat_SharedLocalBasisIndexMacro,
		*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		// Regular normal basis
		*NormalCode,
		*TangentCode,
		// Clear coat bottom layer normal basis
		*ClearCoat_NormalCode,
		*ClearCoat_TangentCode
	);
}

int32 FHLSLMaterialTranslator::SubstrateVolumetricFogCloudBSDF(int32 Albedo, int32 Extinction, int32 EmissiveColor, int32 AmbientOcclusion, bool bEmissiveOnly)
{
	// EmissiveOnly isrequired due to the legacy material shading model set by the user, which could be Unlit or Lit for volumetric domain.
	// It was set on the material detail panel or overriden from a mateiral instance. Note: material attributes could not be used with the volumetric domain.

	UMaterial* BaseMaterial = Material->GetMaterialInterface()->GetBaseMaterial();
	// Material is probably an instanced material, so we check that this is the case before potentially overriding the ShadingModel from the material instance.
	UMaterialInterface* BaseMaterialInterface = static_cast<UMaterialInterface*>(BaseMaterial);
	UMaterialInterface* MaterialInterface = Material->GetMaterialInterface();
	if (BaseMaterial && (BaseMaterialInterface != MaterialInterface))
	{
		FMaterialShadingModelField BaseMaterialShadingModels = BaseMaterial->GetShadingModels();
		FMaterialShadingModelField MaterialShadingModels = Material->GetShadingModels();

		// If the potentially instanced material does not have the same shading model as the instance material, this means it would have overridden the shading model.
		// From the UI, only a single shading model is selectable, so we simply apply the one coming form the material instance.
		if (MaterialShadingModels.IsValid() && MaterialShadingModels.CountShadingModels() == 1 && MaterialShadingModels != BaseMaterialShadingModels)
		{
			// Use the mode required by the overriden shading model.
			bEmissiveOnly = MaterialShadingModels.GetFirstShadingModel() == MSM_Unlit ? true : false;
		}
	}

	// Override to EmissiveOnly if that is the final shading model.
	if (bEmissiveOnly)
	{
		Albedo = Extinction = Constant3(0.0f, 0.0f, 0.0f);
		AmbientOcclusion = Constant(1.0f);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("GetSubstrateVolumeFogCloudBSDF(%s, %s, %s, %s)"),
		*SubstrateGetCastParameterCode(Albedo,				MCT_Float3),
		*SubstrateGetCastParameterCode(Extinction,			MCT_Float3),
		*SubstrateGetCastParameterCode(EmissiveColor,		MCT_Float3),
		*SubstrateGetCastParameterCode(AmbientOcclusion,	MCT_Float)
	);
}

int32 FHLSLMaterialTranslator::SubstrateUnlitBSDF(int32 EmissiveColor, int32 TransmittanceColor, int32 Normal, FSubstrateOperator* PromoteToOperator)
{
	const FString EmissiveColorCode = SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3);
	const FString TransmittanceColorCode = SubstrateGetCastParameterCode(TransmittanceColor, MCT_Float3);
	const FString NormalCode = SubstrateGetCastParameterCode(Normal, MCT_Float3);

	if (PromoteToOperator)
	{
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(GetSubstrateUnlitBSDF(%s, %s, %s), %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*EmissiveColorCode,
			*TransmittanceColorCode,
			*NormalCode,
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0);
	}
	
	return AddCodeChunk(
		MCT_Substrate, TEXT("GetSubstrateUnlitBSDF(%s, %s, %s)"),
		*SubstrateGetCastParameterCode(EmissiveColor,		MCT_Float3),
		*SubstrateGetCastParameterCode(TransmittanceColor,	MCT_Float3),
		*SubstrateGetCastParameterCode(Normal,				MCT_Float3)
	);
}

int32 FHLSLMaterialTranslator::SubstrateUIBSDF(int32 EmissiveColor, int32 Opacity, FSubstrateOperator* PromoteToOperator)
{
	if (PromoteToOperator)
	{
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateCreateUIMaterial(%s, %s), %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
			*SubstrateGetCastParameterCode(Opacity, MCT_Float1),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateCreateUIMaterial(%s, %s)"),
		*SubstrateGetCastParameterCode(EmissiveColor,		MCT_Float3),
		*SubstrateGetCastParameterCode(Opacity,				MCT_Float1)
	);
}

int32 FHLSLMaterialTranslator::SubstrateHairBSDF(int32 BaseColor, int32 Scatter, int32 Specular, int32 Roughness, int32 Backlit, int32 EmissiveColor, int32 Tangent, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator)
{
	if (PromoteToOperator)
	{
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(GetSubstrateHairBSDF(%s, %s, %s, %s, %s, %s, false/*bComplexTransmittance*/, %s), %u, %u, %u, %u) /* Tangent:%s */"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*SubstrateGetCastParameterCode(BaseColor, MCT_Float3),
			*SubstrateGetCastParameterCode(Scatter, MCT_Float),
			*SubstrateGetCastParameterCode(Specular, MCT_Float),
			*SubstrateGetCastParameterCode(Roughness, MCT_Float),
			*SubstrateGetCastParameterCode(Backlit, MCT_Float),
			*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
			*SharedLocalBasisIndexMacro,
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0,
			*GetParameterCode(Tangent)
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("GetSubstrateHairBSDF(%s, %s, %s, %s, %s, %s, false/*bComplexTransmittance*/, %s)"),
		*SubstrateGetCastParameterCode(BaseColor, MCT_Float3),
		*SubstrateGetCastParameterCode(Scatter, MCT_Float),
		*SubstrateGetCastParameterCode(Specular, MCT_Float),
		*SubstrateGetCastParameterCode(Roughness, MCT_Float),
		*SubstrateGetCastParameterCode(Backlit, MCT_Float),
		*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
		*SharedLocalBasisIndexMacro
	);
}

int32 FHLSLMaterialTranslator::SubstrateEyeBSDF(int32 DiffuseAlbedo, int32 Roughness, int32 IrisMask, int32 IrisDistance, int32 IrisNormal, int32 IrisPlaneNormal, int32 SSSProfileId, int32 EmissiveColor, int32 CorneaNormal, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator)
{
	if (PromoteToOperator)
	{
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(GetSubstrateEyeBSDF(%s, %s, %s, %s, %s, %s, ExtractSubsurfaceProfileInt(%s), %s, %s), %u, %u, %u, %u) /* Cornea:%s Iris:%s */"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*SubstrateGetCastParameterCode(DiffuseAlbedo, MCT_Float3),
			*SubstrateGetCastParameterCode(Roughness, MCT_Float),
			*SubstrateGetCastParameterCode(IrisMask, MCT_Float),
			*SubstrateGetCastParameterCode(IrisDistance, MCT_Float),
			*SubstrateGetCastParameterCode(IrisNormal, MCT_Float3),
			*SubstrateGetCastParameterCode(IrisPlaneNormal, MCT_Float3),
			*SubstrateGetCastParameterCode(SSSProfileId, MCT_Float),
			*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
			*SharedLocalBasisIndexMacro,
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0,
			*GetParameterCode(CorneaNormal),
			*GetParameterCode(IrisNormal)
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("GetSubstrateEyeBSDF(%s, %s, %s, %s, %s, %s, ExtractSubsurfaceProfileInt(%s), %s, %s)"),
		*SubstrateGetCastParameterCode(DiffuseAlbedo, MCT_Float3),
		*SubstrateGetCastParameterCode(Roughness, MCT_Float),
		*SubstrateGetCastParameterCode(IrisMask, MCT_Float),
		*SubstrateGetCastParameterCode(IrisDistance, MCT_Float),
		*SubstrateGetCastParameterCode(IrisNormal, MCT_Float3),
		*SubstrateGetCastParameterCode(IrisPlaneNormal, MCT_Float3),
		*SubstrateGetCastParameterCode(SSSProfileId, MCT_Float),
		*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
		*SharedLocalBasisIndexMacro
	);
}

int32 FHLSLMaterialTranslator::SubstrateSingleLayerWaterBSDF(
	int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness, int32 EmissiveColor, int32 TopMaterialOpacity,
	int32 WaterAlbedo, int32 WaterExtinction, int32 WaterPhaseG, int32 ColorScaleBehindWater, int32 Normal, const FString& SharedLocalBasisIndexMacro,
	FSubstrateOperator* PromoteToOperator)
{
	return AddCodeChunk(
		MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(GetSubstrateSingleLayerWaterBSDF(%s, %s, %s, %s, %s, %s, %s, %s, %s, %s, %s), %u, %u, %u, %u) /* Normal:%s */"),
		*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*SubstrateGetCastParameterCode(BaseColor,				MCT_Float3),
		*SubstrateGetCastParameterCode(Metallic,				MCT_Float),
		*SubstrateGetCastParameterCode(Specular,				MCT_Float),
		*SubstrateGetCastParameterCode(Roughness,				MCT_Float),
		*SubstrateGetCastParameterCode(EmissiveColor,			MCT_Float3),
		*SubstrateGetCastParameterCode(TopMaterialOpacity,		MCT_Float),
		*SubstrateGetCastParameterCode(WaterAlbedo,			MCT_Float3),
		*SubstrateGetCastParameterCode(WaterExtinction,		MCT_Float3),
		*SubstrateGetCastParameterCode(WaterPhaseG,			MCT_Float),
		*SubstrateGetCastParameterCode(ColorScaleBehindWater,	MCT_Float3),
		*SharedLocalBasisIndexMacro,
		PromoteToOperator->Index,
		PromoteToOperator->BSDFIndex,
		PromoteToOperator->LayerDepth,
		PromoteToOperator->bIsBottom ? 1 : 0,
		*GetParameterCode(Normal)
	);
}

int32 FHLSLMaterialTranslator::SubstrateToonBSDF(
	int32 ToonProfileId,
	int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness,
	int32 EmissiveColor, int32 Normal,
	int32 PatternUVs,
	int32 Anisotropy, int32 Tangent,
	const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator)
{
	if (ToonProfileId <= INDEX_NONE)
	{
		Errorf(TEXT("Substrate error: invalid Toon profile specified (asset: %s [%s]: ).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
		return INDEX_NONE;
	}

	if (PromoteToOperator)
	{
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(GetSubstrateToonBSDF(%s, %s, %s, %s, %s, %s, %s, %s, %s, %s, Parameters.%s.Types), %u, %u, %u, %u) /* Normal:%s */"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*SubstrateGetCastParameterCode(ToonProfileId, MCT_Float),
			*SubstrateGetCastParameterCode(BaseColor, MCT_Float3),
			*SubstrateGetCastParameterCode(Metallic, MCT_Float),
			*SubstrateGetCastParameterCode(Specular, MCT_Float),
			*SubstrateGetCastParameterCode(Roughness, MCT_Float),
			*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
			*SubstrateGetCastParameterCode(PatternUVs, MCT_Float2),
			*SubstrateGetCastParameterCode(Anisotropy, MCT_Float),
			*SubstrateGetCastParameterCode(Tangent, MCT_Float3),
			*SharedLocalBasisIndexMacro,
			*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0,
			*GetParameterCode(Normal)
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("GetSubstrateToonBSDF(%s, %s, %s, %s, %s, %s, %s, %s, %s, %s, Parameters.%s.Types)"),
		*SubstrateGetCastParameterCode(ToonProfileId, MCT_Float),
		*SubstrateGetCastParameterCode(BaseColor, MCT_Float3),
		*SubstrateGetCastParameterCode(Metallic, MCT_Float),
		*SubstrateGetCastParameterCode(Specular, MCT_Float),
		*SubstrateGetCastParameterCode(Roughness, MCT_Float),
		*SubstrateGetCastParameterCode(EmissiveColor, MCT_Float3),
		*SubstrateGetCastParameterCode(PatternUVs, MCT_Float2),
		*SubstrateGetCastParameterCode(Anisotropy, MCT_Float),
		*SubstrateGetCastParameterCode(Tangent, MCT_Float3),
		*SharedLocalBasisIndexMacro,
		*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext)
	);
}

int32 FHLSLMaterialTranslator::SubstrateHorizontalMixing(int32 Background, int32 Foreground, int32 Mix, int OperatorIndex, uint32 MaxDistanceFromLeaves)
{
	if (Foreground == INDEX_NONE || Background == INDEX_NONE || Mix == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(
		MCT_Substrate, TEXT("Parameters.%s.SubstrateHorizontalMixing(%s, %s, %s, %u, %u)"),
		*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*GetParameterCode(Background),
		*GetParameterCode(Foreground),
		*GetParameterCode(Mix),
		OperatorIndex,
		MaxDistanceFromLeaves
	);
}

int32 FHLSLMaterialTranslator::SubstrateHorizontalMixingParameterBlending(
	int32 Background, int32 Foreground, int32 HorizontalMixCodeChunk, int32 NormalMixCodeChunk, const FString& SharedLocalBasisIndexMacro, int32 BackgroundBSDFNormalCodeChunk, int32 ForegroundBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator)
{
	if (Foreground == INDEX_NONE || Background == INDEX_NONE || HorizontalMixCodeChunk == INDEX_NONE || NormalMixCodeChunk == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 CameraVectorCode = CameraVector();
	if (CameraVectorCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 BackgroundNoV = Saturate(Dot(BackgroundBSDFNormalCodeChunk, CameraVectorCode));
	const int32 ForegroundNoV = Saturate(Dot(ForegroundBSDFNormalCodeChunk, CameraVectorCode));

	if (PromoteToOperator)
	{
		check(PromoteToOperator->Index != INDEX_NONE);
		check(PromoteToOperator->BSDFIndex != INDEX_NONE);
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateHorizontalMixingParameterBlending(%s, %s, %s, %s, %s, Parameters.%s.Types, %s, %s), %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*GetParameterCode(Background),
			*GetParameterCode(Foreground),
			*GetParameterCode(HorizontalMixCodeChunk),
			*GetParameterCode(NormalMixCodeChunk),
			*SharedLocalBasisIndexMacro,
			*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*GetParameterCode(BackgroundNoV),
			*GetParameterCode(ForegroundNoV),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateHorizontalMixingParameterBlending(%s, %s, %s, %s, %s, Parameters.%s.Types, %s, %s)"),
		*GetParameterCode(Background),
		*GetParameterCode(Foreground),
		*GetParameterCode(HorizontalMixCodeChunk),
		*GetParameterCode(NormalMixCodeChunk),
		*SharedLocalBasisIndexMacro,
		*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*GetParameterCode(BackgroundNoV),
		*GetParameterCode(ForegroundNoV)
	);
}

int32 FHLSLMaterialTranslator::SubstrateVerticalLayering(int32 Top, int32 Base, int32 Thickness, int OperatorIndex, uint32 MaxDistanceFromLeaves)
{
	if (Top == INDEX_NONE || Base == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	const FString ThicknessCode = Thickness != INDEX_NONE ? GetParameterCode(Thickness) : TEXT("NONE");;
	return AddCodeChunk(
		MCT_Substrate, TEXT("Parameters.%s.SubstrateVerticalLayering(%s, %s, %u, %u) /* Thickness = %s */"),
		*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*GetParameterCode(Top),
		*GetParameterCode(Base),
		OperatorIndex,
		MaxDistanceFromLeaves,
		*ThicknessCode
	);
}

int32 FHLSLMaterialTranslator::SubstrateVerticalLayeringParameterBlending(int32 Top, int32 Base, int32 Thickness, const FString& SharedLocalBasisIndexMacro, int32 TopBSDFNormalCodeChunk, int32 BaseBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator)
{
	if (Top == INDEX_NONE || Base == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const FString ThicknessCode = Thickness != INDEX_NONE ? GetParameterCode(Thickness) : TEXT("NONE");

	const int32 CameraVectorCode = CameraVector();
	if (CameraVectorCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 TopNoV  = Saturate(Dot(TopBSDFNormalCodeChunk, CameraVectorCode));
	const int32 BaseNoV = Saturate(Dot(BaseBSDFNormalCodeChunk, CameraVectorCode));

	if (PromoteToOperator)
	{
		check(PromoteToOperator->Index != INDEX_NONE);
		check(PromoteToOperator->BSDFIndex != INDEX_NONE);
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateVerticalLayeringParameterBlending(%s, %s, %s, %s, %s), %u, %u, %u, %u) /* Thickness = %s */"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*GetParameterCode(Top),
			*GetParameterCode(Base),
			*SharedLocalBasisIndexMacro,
			*GetParameterCode(TopNoV),
			*GetParameterCode(BaseNoV),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0,
			*ThicknessCode
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateVerticalLayeringParameterBlending(%s, %s, %s, %s, %s) /* Thickness = %s */"),
		*GetParameterCode(Top),
		*GetParameterCode(Base),
		*SharedLocalBasisIndexMacro,
		*GetParameterCode(TopNoV),
		*GetParameterCode(BaseNoV),
		*ThicknessCode
	);
}

int32 FHLSLMaterialTranslator::SubstrateAdd(int32 A, int32 B, int OperatorIndex, uint32 MaxDistanceFromLeaves)
{
	if (A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(
		MCT_Substrate, TEXT("Parameters.%s.SubstrateAdd(%s, %s, %u, %u)"),
		*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*GetParameterCode(A),
		*GetParameterCode(B),
		OperatorIndex,
		MaxDistanceFromLeaves
	);
}

int32 FHLSLMaterialTranslator::SubstrateAddParameterBlending(int32 A, int32 B, int32 AMixWeight, const FString& SharedLocalBasisIndexMacro, int32 ABSDFNormalCodeChunk, int32 BBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator)
{
	if (A == INDEX_NONE || B == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 CameraVectorCode = CameraVector();
	if (CameraVectorCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 ANoV = Saturate(Dot(ABSDFNormalCodeChunk, CameraVectorCode));
	const int32 BNoV = Saturate(Dot(BBSDFNormalCodeChunk, CameraVectorCode));

	if (PromoteToOperator)
	{
		check(PromoteToOperator->Index != INDEX_NONE);
		check(PromoteToOperator->BSDFIndex != INDEX_NONE);
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateAddParameterBlending(%s, %s, %s, %s, %s, %s), %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*GetParameterCode(A),
			*GetParameterCode(B),
			*GetParameterCode(AMixWeight),
			*SharedLocalBasisIndexMacro,
			*GetParameterCode(ANoV),
			*GetParameterCode(BNoV),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateAddParameterBlending(%s, %s, %s, %s, %s, %s)"),
		*GetParameterCode(A),
		*GetParameterCode(B),
		*GetParameterCode(AMixWeight),
		*SharedLocalBasisIndexMacro,
		*GetParameterCode(ANoV),
		*GetParameterCode(BNoV)
	);
}

int32 FHLSLMaterialTranslator::SubstrateWeight(int32 A, int32 Weight, int OperatorIndex, uint32 MaxDistanceFromLeaves)
{
	if (A == INDEX_NONE || Weight == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(
		MCT_Substrate, TEXT("Parameters.%s.SubstrateWeight(%s, %s, %u, %u)"),
		*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
		*GetParameterCode(A),
		*GetParameterCode(Weight),
		OperatorIndex,
		MaxDistanceFromLeaves
	);
}

int32 FHLSLMaterialTranslator::SubstrateWeightParameterBlending(int32 A, int32 Weight, FSubstrateOperator* PromoteToOperator)
{
	if (A == INDEX_NONE || Weight == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PromoteToOperator)
	{
		check(PromoteToOperator->Index != INDEX_NONE);
		check(PromoteToOperator->BSDFIndex != INDEX_NONE);
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateWeightParameterBlending(%s, %s), %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*GetParameterCode(A),
			*GetParameterCode(Weight),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateWeightParameterBlending(%s, %s)"),
		*GetParameterCode(A),
		*GetParameterCode(Weight)
	);
}

int32 FHLSLMaterialTranslator::SubstrateSelectParameterBlending(int32 A, int32 B, int32 SelectValue, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator)
{
	if (A == INDEX_NONE || B == INDEX_NONE || SelectValue == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (PromoteToOperator)
	{
		check(PromoteToOperator->Index != INDEX_NONE);
		check(PromoteToOperator->BSDFIndex != INDEX_NONE);
		return AddCodeChunk(
			MCT_Substrate, TEXT("Parameters.%s.PromoteParameterBlendedBSDFToOperator(SubstrateSelectParameterBlending(%s, %s, %s, %s, Parameters.%s.Types), %u, %u, %u, %u)"),
			*Substrate::GetParametersSubstrateTreeName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			*GetParameterCode(A),
			*GetParameterCode(B),
			*GetParameterCode(SelectValue),
			*SharedLocalBasisIndexMacro,
			*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext),
			PromoteToOperator->Index,
			PromoteToOperator->BSDFIndex,
			PromoteToOperator->LayerDepth,
			PromoteToOperator->bIsBottom ? 1 : 0
		);
	}

	return AddCodeChunk(
		MCT_Substrate, TEXT("SubstrateSelectParameterBlending(%s, %s, %s, %s, Parameters.%s.Types)"),
		*GetParameterCode(A),
		*GetParameterCode(B),
		*GetParameterCode(SelectValue),
		*SharedLocalBasisIndexMacro,
		*Substrate::GetParametersSharedLocalBasesName(SubstrateTranslatorData.CurrentSubstrateCompilationContext)
	);
}

int32 FHLSLMaterialTranslator::SubstrateTransmittanceToMFP(int32 TransmittanceColor, int32 DesiredThickness, int32 OutputIndex)
{
	if (OutputIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 DefaultThicknessCodechunk = AddInlinedCodeChunk(MCT_Float1, TEXT("%f"), SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
	switch (OutputIndex)
	{
	case 0:
		return AddCodeChunk(MCT_Float3, 
			// For the math to be valid, input to TransmittanceToMeanFreePath must be in meter.
			// Then the output needs to be is converted to centimeters.
			TEXT("(TransmittanceToMeanFreePath(%s, %s * CENTIMETER_TO_METER) * METER_TO_CENTIMETER)"), 
			*GetParameterCode(TransmittanceColor),
			*GetParameterCode(DesiredThickness == INDEX_NONE ? DefaultThicknessCodechunk : DesiredThickness));
	case 1:
		// Thickness to be plugged into other nodes thickness input.
		// This matches the Slab node default using SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM
		return DesiredThickness == INDEX_NONE ? DefaultThicknessCodechunk : DesiredThickness;
	}
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::SubstrateMetalnessToDiffuseAlbedoF0(int32 BaseColor, int32 Specular, int32 Metallic, int32 OutputIndex)
{
	if (OutputIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	switch (OutputIndex)
	{
	case 0:
		return AddCodeChunk(MCT_Float3,
			TEXT("ComputeDiffuseAlbedo(%s, saturate(%s))"),
			*GetParameterCode(BaseColor),
			*GetParameterCode(Metallic));
	case 1:
		return AddCodeChunk(MCT_Float3,
			TEXT("ComputeF0(%s, %s, saturate(%s))"),
			*GetParameterCode(Specular),
			*GetParameterCode(BaseColor),
			*GetParameterCode(Metallic));
	}
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::SubstrateHazinessToSecondaryRoughness(int32 BaseRoughness, int32 Haziness, int32 OutputIndex)
{
	if (OutputIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	switch (OutputIndex)
	{
	case 0:
		return AddCodeChunk(MCT_Float1,
			TEXT("SubstrateComputeHazeRoughness(saturate(%s))"),
			*GetParameterCode(BaseRoughness));
	case 1:
		return AddCodeChunk(MCT_Float1,
			TEXT("SubstrateComputeHazeWeight(saturate(%s), saturate(%s))"),
			*GetParameterCode(BaseRoughness),
			*GetParameterCode(Haziness));
	}
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::SubstrateThinFilm(int32 NormalCodeChunk, int32 SpecularColorCodeChunk, int32 EdgeSpecularColorCodeChunk, int32 ThicknessCodeChunk, int32 IORCodeChunk, int32 OutputIndex)
{
	if (NormalCodeChunk == INDEX_NONE || SpecularColorCodeChunk == INDEX_NONE || EdgeSpecularColorCodeChunk == INDEX_NONE
		|| ThicknessCodeChunk == INDEX_NONE || IORCodeChunk == INDEX_NONE)
	{
		Errorf(TEXT("Substrate error: one of SubstrateThinFilm node input is invalid (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
		return INDEX_NONE;
	}
	if (OutputIndex < 0 || OutputIndex > 1)
	{
		Errorf(TEXT("Substrate error: SubstrateThinFilm output index is invalid (asset: %s).\r\n"), *Material->GetDebugName(), *Material->GetAssetPath().ToString());
		return INDEX_NONE;
	}
	const int32 CameraVectorCode = CameraVector();
	if (CameraVectorCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	return AddCodeChunk(
		MCT_Float3, TEXT("SubstrateGetThinFilmF0F90(%s, %s, %s, %s, %s)%s"),
		*GetParameterCode(Dot(NormalCodeChunk, CameraVectorCode)),
		*GetParameterCode(SpecularColorCodeChunk),
		*GetParameterCode(EdgeSpecularColorCodeChunk),
		*GetParameterCode(ThicknessCodeChunk),
		*GetParameterCode(IORCodeChunk),
		OutputIndex == 0 ? TEXT(".F0") : TEXT(".F90")
	);
}

int32 FHLSLMaterialTranslator::SubstrateCompilePreview(int32 SubstrateDataCodeChunk)
{
	if (SubstrateDataCodeChunk == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	// Instead of using a preview color, we might go with a sphere+lighting preview. 
	// The only problem we would have to solve is to process the Substrate tree before for the sub tree. 
	// Or have dedicated lighting preview functions for SubstrateData that have not be filled in by the Substrate tree processing.
	// PreviewColor is filled up using SME_MaterialPreview, driving SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW.
	int32 PreviewCodeChunk = AddCodeChunk(MCT_Float3, TEXT("%s.PreviewColor"), *GetParameterCode(SubstrateDataCodeChunk));
	return PreviewCodeChunk;
}

bool FHLSLMaterialTranslator::SubstrateSkipsOpacityEvaluation()
{
	return !IsTranslucentBlendMode(*Material)
		&& GetSubstrateMaterialExportType() == ESubstrateMaterialExport::SME_None	// We never skip when exporting information
		&& Material->GetShadingModels().CountShadingModels() == 1
		&& !Material->GetShadingModels().HasShadingModel(MSM_SingleLayerWater)
		&& !Material->GetShadingModels().HasShadingModel(MSM_Subsurface)
		&& !Material->GetShadingModels().HasShadingModel(MSM_SubsurfaceProfile)
		&& !Material->GetShadingModels().HasShadingModel(MSM_TwoSidedFoliage)
		&& !Material->GetShadingModels().HasShadingModel(MSM_PreintegratedSkin);
}

FGuid FHLSLMaterialTranslator::SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex)
{
	return SubstrateTranslatorData.SubstrateTreeStackPush(Expression, InputIndex);
}

FGuid FHLSLMaterialTranslator::SubstrateTreeStackGetPathUniqueId()
{
	return SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId();
}

FGuid FHLSLMaterialTranslator::SubstrateTreeStackGetParentPathUniqueId()
{
	return SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId();
}

void FHLSLMaterialTranslator::SubstrateTreeStackPop()
{
	SubstrateTranslatorData.SubstrateTreeStackPop();
}

bool FHLSLMaterialTranslator::GetSubstrateTreeOutOfStackDepthOccurred()
{
	return SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred();
}

int32 FHLSLMaterialTranslator::SubstrateThicknessStackGetThicknessIndex()
{
	return SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
}

int32 FHLSLMaterialTranslator::SubstrateThicknessStackGetThicknessCode(int32 Index)
{
	FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression* ThicknessExpression = SubstrateTranslatorData.SubstrateThicknessStackGetExpression(Index);

	int32 OutCode = INDEX_NONE;
	if (ThicknessExpression == nullptr)
	{
		UE_LOGF(LogMaterial, Error, " SubstrateThichkness: %i could not be found)", Index);
	}
	else if (ThicknessExpression->ExpressionInput)
	{
		FExpressionInput* ExpressionInput = ThicknessExpression->ExpressionInput;
		OutCode = ExpressionInput->GetTracedInput().Expression ? ExpressionInput->Compile(this) : Constant(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
		EMaterialValueType Type = GetType(OutCode);
		if (IsLWCType(Type))
		{
			Type = MakeNonLWCType(Type);
			OutCode = ValidCast(OutCode, Type);
		}
	}
	else if (ThicknessExpression->MaterialInput)
	{
		FScalarMaterialInput* MaterialInput = ThicknessExpression->MaterialInput;
		OutCode = MaterialInput->UseConstant ? Constant(MaterialInput->Constant) : Constant(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
		EMaterialValueType Type = GetType(OutCode);
		if (IsLWCType(Type))
		{
			Type = MakeNonLWCType(Type);
			OutCode = ValidCast(OutCode, Type);
		}	
	}
	if (OutCode == INDEX_NONE)
	{
		OutCode = Constant(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
	}
	return OutCode;
}

int32 FHLSLMaterialTranslator::SubstrateThicknessStackPush(UMaterialExpression* Expression, FScalarMaterialInput* Input)
{
	return SubstrateTranslatorData.SubstrateThicknessStackPush(Expression, Input);
}

int32 FHLSLMaterialTranslator::SubstrateThicknessStackPush(UMaterialExpression* Expression, FExpressionInput* Input)
{
	return SubstrateTranslatorData.SubstrateThicknessStackPush(Expression, Input);
}

void FHLSLMaterialTranslator::SubstrateThicknessStackPop()
{
	SubstrateTranslatorData.SubstrateThicknessStackPop();
}

struct FSubstrateTranslatorDataInterface& FHLSLMaterialTranslator::GetSubstrateTranslatorData()
{
	return SubstrateTranslatorData;
}

int32 FHLSLMaterialTranslator::MapARPassthroughCameraUV(int32 UV)
{
	if (UV == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	int32 UVPair0 = AddInlinedCodeChunkZeroDeriv(MCT_Float4, TEXT("ResolvedView.XRPassthroughCameraUVs[0]"));
	int32 UVPair1 = AddInlinedCodeChunkZeroDeriv(MCT_Float4, TEXT("ResolvedView.XRPassthroughCameraUVs[1]"));

	int32 ULerp = Lerp(UVPair0, UVPair1, ComponentMask(UV, 1, 0, 0, 0));
	return Lerp(ComponentMask(ULerp, 1, 1, 0, 0), ComponentMask(ULerp, 0, 0, 1, 1), ComponentMask(UV, 0, 1, 0, 0));
}

int32 FHLSLMaterialTranslator::AccessMaterialAttribute(int32 CodeIndex, const FGuid& AttributeID)
{
	check(GetParameterType(CodeIndex) == MCT_MaterialAttributes);

	const FString AttributeName = FMaterialAttributeDefinitionMap::GetAttributeName(AttributeID);
	const EMaterialValueType AttributeType = FMaterialAttributeDefinitionMap::GetValueType(AttributeID);
	return AddInlinedCodeChunk(
		AttributeType,
		TEXT("%s.%s"),
		*GetParameterCode(CodeIndex),
		*AttributeName);
}

int32 FHLSLMaterialTranslator::CustomExpression( class UMaterialExpressionCustom* Custom, int32 OutputIndex, TArray<int32>& CompiledInputs )
{
	const FMaterialCustomExpressionEntry* CustomEntry = nullptr;
	for (const FMaterialCustomExpressionEntry& Entry : CustomExpressions)
	{
		if (Entry.Expression == Custom &&
			Entry.ScopeID == CurrentScopeID)
		{
			bool bInputsMatch = true;
			for(int32 InputIndex = 0; InputIndex < CompiledInputs.Num(); ++InputIndex)
			{
				const uint64 InputHash = GetParameterHash(CompiledInputs[InputIndex]);
				if (Entry.InputHash[InputIndex] != InputHash)
				{
					bInputsMatch = false;
					break;
				}
			}

			if (bInputsMatch)
			{
				CustomEntry = &Entry;
				break;
			}
		}
	}

	if (!CustomEntry)
	{
		FString OutputTypeString;
		EMaterialValueType OutputType;
		switch (Custom->OutputType)
		{
		case CMOT_Float2:
			OutputType = MCT_Float2;
			OutputTypeString = TEXT("MaterialFloat2");
			break;
		case CMOT_Float3:
			OutputType = MCT_Float3;
			OutputTypeString = TEXT("MaterialFloat3");
			break;
		case CMOT_Float4:
			OutputType = MCT_Float4;
			OutputTypeString = TEXT("MaterialFloat4");
			break;
		case CMOT_MaterialAttributes:
			OutputType = MCT_MaterialAttributes;
			OutputTypeString = TEXT("FMaterialAttributes");
			break;
		default:
			OutputType = MCT_Float;
			OutputTypeString = TEXT("MaterialFloat");
			break;
		}

		// Declare implementation function
		FString InputParamDecl;
		check(Custom->Inputs.Num() == CompiledInputs.Num());
		for (int32 i = 0; i < Custom->Inputs.Num(); i++)
		{
			// skip over unnamed inputs
			if (Custom->Inputs[i].InputName.IsNone())
			{
				continue;
			}
			InputParamDecl += TEXT(",");
			const FString InputNameStr = Custom->Inputs[i].InputName.ToString();
			switch (GetParameterType(CompiledInputs[i]))
			{
			case MCT_Float:
			case MCT_Float1:
				InputParamDecl += TEXT("MaterialFloat ");
				InputParamDecl += InputNameStr;
				break;
			case MCT_Float2:
				InputParamDecl += TEXT("MaterialFloat2 ");
				InputParamDecl += InputNameStr;
				break;
			case MCT_Float3:
				InputParamDecl += TEXT("MaterialFloat3 ");
				InputParamDecl += InputNameStr;
				break;
			case MCT_Float4:
				InputParamDecl += TEXT("MaterialFloat4 ");
				InputParamDecl += InputNameStr;
				break;
			case MCT_LWCScalar:
				InputParamDecl += TEXT("float ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", FWSScalar LWC");
				InputParamDecl += InputNameStr;
				break;
			case MCT_LWCVector2:
				InputParamDecl += TEXT("float2 ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", FWSVector2 LWC");
				InputParamDecl += InputNameStr;
				break;
			case MCT_LWCVector3:
				InputParamDecl += TEXT("float3 ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", FWSVector3 LWC");
				InputParamDecl += InputNameStr;
				break;
			case MCT_LWCVector4:
				InputParamDecl += TEXT("float4 ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", FWSVector4 LWC");
				InputParamDecl += InputNameStr;
				break;
			case MCT_Texture2D:
				InputParamDecl += TEXT("Texture2D ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", SamplerState ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT("Sampler ");
				break;
			case MCT_TextureCube:
				InputParamDecl += TEXT("TextureCube ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", SamplerState ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT("Sampler ");
				break;
			case MCT_Texture2DArray:
				InputParamDecl += TEXT("Texture2DArray ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", SamplerState ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT("Sampler ");
				break;
			case MCT_TextureCubeArray:
				InputParamDecl += TEXT("TextureCubeArray ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", SamplerState ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT("Sampler ");
				break;
			case MCT_TextureExternal:
				InputParamDecl += TEXT("TextureExternal ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", SamplerState ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT("Sampler ");
				break;
			case MCT_VolumeTexture:
				InputParamDecl += TEXT("Texture3D ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT(", SamplerState ");
				InputParamDecl += InputNameStr;
				InputParamDecl += TEXT("Sampler ");
				break;
			case MCT_TextureCollection:
				InputParamDecl += TEXT("FResourceCollection ");
				InputParamDecl += InputNameStr;
				break;
			default:
				return Errorf(TEXT("Bad type %s for %s input %s"), DescribeType(GetParameterType(CompiledInputs[i])), *Custom->Description, *InputNameStr);
			}
		}

		for (const FCustomOutput& CustomOutput : Custom->AdditionalOutputs)
		{
			if (CustomOutput.OutputName.IsNone())
			{
				continue;
			}

			InputParamDecl += TEXT(", inout "); // use 'inout', so custom code may optionally avoid setting certain outputs (will default to 0)
			const FString OutputNameStr = CustomOutput.OutputName.ToString();
			switch (CustomOutput.OutputType)
			{
			case CMOT_Float1:
				InputParamDecl += TEXT("MaterialFloat ");
				InputParamDecl += OutputNameStr;
				break;
			case CMOT_Float2:
				InputParamDecl += TEXT("MaterialFloat2 ");
				InputParamDecl += OutputNameStr;
				break;
			case CMOT_Float3:
				InputParamDecl += TEXT("MaterialFloat3 ");
				InputParamDecl += OutputNameStr;
				break;
			case CMOT_Float4:
				InputParamDecl += TEXT("MaterialFloat4 ");
				InputParamDecl += OutputNameStr;
				break;
			case CMOT_MaterialAttributes:
				InputParamDecl += TEXT("FMaterialAttributes ");
				InputParamDecl += OutputNameStr;
				break;
			default:
				return Errorf(TEXT("Bad type %d for %s output %s"), static_cast<int32>(CustomOutput.OutputType.GetValue()), *Custom->Description, *OutputNameStr);
			}
		}

		int32 CustomExpressionIndex = CustomExpressions.Num();
		FString Code = Custom->Code;

		// Detect clip() or discard. These conditionally kill pixels and may require additional shader permutations. 
		if (Custom->HasPixelDiscard())
		{
			MaterialCompilationOutput.bUsesCustomHLSLPixelDiscard = true;
		}

		if (!Code.Contains(TEXT("return")))
		{
			Code = FString("return ") + Code + TEXT(";");
		}
		Code.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);

		TArray<int8> SceneTextureInfoIgnored;
		FString ModifiedCode =  UE::MaterialTranslatorUtils::CustomExpressionSceneTextureInputFixup(Custom, *Code, SceneTextureInfoIgnored);
		if (!ModifiedCode.IsEmpty())
		{
			Code = ModifiedCode;
		}

		FString ParametersType = ShaderFrequency == SF_Vertex ? TEXT("Vertex") : TEXT("Pixel");

		FMaterialCustomExpressionEntry& Entry = CustomExpressions.AddDefaulted_GetRef();
		CustomEntry = &Entry;
		Entry.Expression = Custom;
		Entry.ScopeID = CurrentScopeID;
		Entry.InputHash.Empty(CompiledInputs.Num());
		for (int32 InputIndex = 0; InputIndex < CompiledInputs.Num(); ++InputIndex)
		{
			const uint64 InputHash = GetParameterHash(CompiledInputs[InputIndex]);
			Entry.InputHash.Add(InputHash);
		}

		for (FCustomDefine DefineEntry : Custom->AdditionalDefines)
		{
			FString DefineStatement = TEXT("#ifndef ") + DefineEntry.DefineName + HLSL_LINE_TERMINATOR;
			DefineStatement += TEXT("#define ") + DefineEntry.DefineName + TEXT(" ") + DefineEntry.DefineValue + HLSL_LINE_TERMINATOR;
			DefineStatement += TEXT("#endif//") + DefineEntry.DefineName + HLSL_LINE_TERMINATOR;

			Entry.Implementation += DefineStatement;
		}

		for (FString IncludeFile : Custom->IncludeFilePaths)
		{
			FString IncludeStatement = "#include ";
			IncludeStatement += TEXT("\"");
			IncludeStatement += IncludeFile;
			IncludeStatement += TEXT("\"");
			IncludeStatement += HLSL_LINE_TERMINATOR;

			Entry.Implementation += IncludeStatement;
		}

		Entry.Implementation += FString::Printf(TEXT("%s CustomExpression%d(FMaterial%sParameters Parameters%s)\n{\n%s\n}\n"), *OutputTypeString, CustomExpressionIndex, *ParametersType, *InputParamDecl, *Code);
		const uint64 ImplementationHash = CityHash64((char*)*Entry.Implementation, Entry.Implementation.Len() * sizeof(TCHAR));

		Entry.OutputCodeIndex.Empty(Custom->AdditionalOutputs.Num() + 1);
		Entry.OutputCodeIndex.Add(INDEX_NONE); // Output0 will hold the return value for the custom expression function, patch it in later

		// Create local temp variables to hold results of additional outputs
		for (const FCustomOutput& CustomOutput : Custom->AdditionalOutputs)
		{
			if (CustomOutput.OutputName.IsNone())
			{
				continue;
			}

			// We're creating 0-initialized values to be filled in by the custom expression, so generate hashes based on code/name of the output
			const FString OutputName = CustomOutput.OutputName.ToString();
			const uint64 BaseHash = CityHash64WithSeed((char*)*OutputName, OutputName.Len() * sizeof(TCHAR), ImplementationHash);

			int32 OutputCode = INDEX_NONE;
			switch (CustomOutput.OutputType)
			{
			case CMOT_Float1: OutputCode = AddCodeChunkWithHash(BaseHash, MCT_Float, TEXT("0.0f")); break;
			case CMOT_Float2: OutputCode = AddCodeChunkWithHash(BaseHash, MCT_Float2, TEXT("MaterialFloat2(0.0f, 0.0f)")); break;
			case CMOT_Float3: OutputCode = AddCodeChunkWithHash(BaseHash, MCT_Float3, TEXT("MaterialFloat3(0.0f, 0.0f, 0.0f)")); break;
			case CMOT_Float4: OutputCode = AddCodeChunkWithHash(BaseHash, MCT_Float4, TEXT("MaterialFloat4(0.0f, 0.0f, 0.0f, 0.0f)")); break;
			case CMOT_MaterialAttributes: OutputCode = AddCodeChunkWithHash(BaseHash, MCT_MaterialAttributes, TEXT("(FMaterialAttributes)0.0f")); break;
			default: checkNoEntry(); break;
			}
			Entry.OutputCodeIndex.Add(OutputCode);
		}

		// Add call to implementation function
		FString CodeChunk = FString::Printf(TEXT("CustomExpression%d(Parameters"), CustomExpressionIndex);
		for (int32 i = 0; i < CompiledInputs.Num(); i++)
		{
			// skip over unnamed inputs
			if (Custom->Inputs[i].InputName.IsNone())
			{
				continue;
			}

			FString ParamCode = GetParameterCode(CompiledInputs[i]);
			EMaterialValueType ParamType = GetParameterType(CompiledInputs[i]);

			CodeChunk += TEXT(",");
			if (ParamType & MCT_LWCType)
			{
				// LWC types get two values, first the value converted to float, then the raw LWC value
				// This way legacy custom expressions can continue to operate on regular float values
				AddLWCFuncUsage(ELWCFunctionKind::Demote);
				CodeChunk += TEXT("WSDemote(");
				CodeChunk += *ParamCode;
				CodeChunk += TEXT("),");
			}
			CodeChunk += *ParamCode;
			if (ParamType == MCT_Texture2D || ParamType == MCT_TextureCube || ParamType == MCT_TextureCubeArray || ParamType == MCT_Texture2DArray || ParamType == MCT_TextureExternal || ParamType == MCT_VolumeTexture)
			{
				CodeChunk += TEXT(",");
				CodeChunk += *ParamCode;
				CodeChunk += TEXT("Sampler");
			}
		}
		// Pass 'out' parameters
		for (int32 i = 1; i < Entry.OutputCodeIndex.Num(); ++i)
		{
			FString ParamCode = GetParameterCode(Entry.OutputCodeIndex[i]);
			CodeChunk += TEXT(",");
			CodeChunk += *ParamCode;
		}

		CodeChunk += TEXT(")");

		// Save result of function as first output
		Entry.OutputCodeIndex[0] = AddCodeChunk(
			OutputType,
			*CodeChunk
		);
	}

	check(CustomEntry);
	if (!CustomEntry->OutputCodeIndex.IsValidIndex(OutputIndex))
	{
		return Errorf(TEXT("Invalid custom expression OutputIndex %d"), OutputIndex);
	}

	int32 Result = CustomEntry->OutputCodeIndex[OutputIndex];
	if (Custom->IsResultMaterialAttributes(OutputIndex))
	{
		Result = AccessMaterialAttribute(Result, GetMaterialAttribute());
	}
	return Result;
}

int32 FHLSLMaterialTranslator::CustomOutput(const FString& CustomOutputName, const FString& OutputTypeString, const FString& FunctionName, int32 OutputIndex, int32 OutputCode, EMaterialCustomOutputFlags Flags)
{
	if (!EnumHasAnyFlags(Flags, EMaterialCustomOutputFlags::AllowAttributeConnection) && MaterialProperty != MP_MAX)
	{
		return Errorf(TEXT("A Custom Output node should not be attached to the %s material property"), *FMaterialAttributeDefinitionMap::GetAttributeName(MaterialProperty));
	}

	if (OutputCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	EMaterialValueType OutputType = GetParameterType(OutputCode);
	if (!IsFloatNumericType(OutputType) && OutputType != MCT_MaterialAttributes && OutputType != MCT_MaterialCacheABuffer)
	{
		return Errorf(TEXT("Bad type %s for %s"), DescribeType(OutputType), *CustomOutputName);
	}

	FString Definitions;
	FString Body;

	// For now, only emit the analytic definition for the material cache
	const bool bShouldEmitAnalyticDefinition = IsAnalyticDerivEnabled() && Material->HasMaterialCacheOutput();

	// Analytic statements
	FString DefinitionsAnalytic;
	FString BodyAnalytic;
	
	// For now just grab the finite differences version, and use both for finite and analytic. Should fix later.
	if ((*CurrentScopeChunks)[OutputCode].UniformExpression && !(*CurrentScopeChunks)[OutputCode].UniformExpression->IsConstant())
	{
		Body = GetParameterCode(OutputCode);
	}
	else
	{
		GetFixedParameterCode(OutputCode, *CurrentScopeChunks, Definitions, Body, CompiledPDV_FiniteDifferences);
		
		if (bShouldEmitAnalyticDefinition)
		{
			GetFixedParameterCode(OutputCode, *CurrentScopeChunks, DefinitionsAnalytic, BodyAnalytic, CompiledPDV_Analytic);
		}
	}

	const FString FunctionNameBase = FString::Printf(TEXT("%s%d%s"), *FunctionName, OutputIndex, bCompilingPreviousFrame ? TEXT("Prev") : TEXT(""));

	const TCHAR* NonLWSTypeStr = HLSLTypeString(MakeNonLWCType(OutputType));
	const TCHAR* LWSTypeStr    = HLSLTypeString(MakeLWCType(OutputType));

	// Don't use 'inout' for Vertex parameters, to work around potential compiler bug with FXC
	const TCHAR* ParameterQualifierStr = ShaderFrequency == SF_Vertex ? TEXT("const") : TEXT("inout");
	const TCHAR* ShaderFrequencyStr    = ShaderFrequency == SF_Vertex ? TEXT("Vertex") : TEXT("Pixel");

	// Primary function will have _LWC suffix if it returns an LWC type
	// We also define a pre-processor symbol to indicate the custom output function is available so that
	// shaders can implement some fallback behavior for cases where the custom output has not been added
	FString ImplementationCodeFinite = FString::Printf(TEXT("#define HAVE_%s 1\n%s %s%s(%s FMaterial%sParameters Parameters)\n{\n%s return %s;\n}\n"),
		*FunctionNameBase,
		*OutputTypeString,
		*FunctionNameBase,
		IsLWCType(OutputType) ? TEXT("_LWC") : TEXT(""),
		ParameterQualifierStr,
		ShaderFrequencyStr,
		*Definitions,
		*Body);
	if (IsLWCType(OutputType))
	{
		// Add a wrapper with no suffix to return a non-LWC type
		AddLWCFuncUsage(ELWCFunctionKind::Demote);
		ImplementationCodeFinite += FString::Printf(TEXT("%s %s(%s FMaterial%sParameters Parameters) { return WSDemote(%s_LWC(Parameters)); }\n"),
			NonLWSTypeStr,
			*FunctionNameBase,
			ParameterQualifierStr,
			ShaderFrequencyStr,
			*FunctionNameBase);
	}
	else
	{
		// Add a wrapper with LWC suffix to return a LWC type
		AddLWCFuncUsage(ELWCFunctionKind::Promote);
		ImplementationCodeFinite += FString::Printf(TEXT("%s %s_LWC(%s FMaterial%sParameters Parameters) { return WSPromote(%s(Parameters)); }\n"),
			LWSTypeStr,
			*FunctionNameBase,
			ParameterQualifierStr,
			ShaderFrequencyStr,
			*FunctionNameBase);
	}

	DerivativeVariations[CompiledPDV_FiniteDifferences].CustomOutputImplementations.Add(ImplementationCodeFinite);

	if (bShouldEmitAnalyticDefinition)
	{
		// Partial-derivative/dual helpers
		static const TCHAR* Header = TEXT("#define DERIV_BASE_VALUE(_X) _X.Value\n");
		static const TCHAR* Footer = TEXT("#undef  DERIV_BASE_VALUE(_X)\n#define DERIV_BASE_VALUE(_X) _X\n");
		
		FString ImplementationCodeAnalytic = FString::Printf(
			TEXT("%s%s %s%d_Analytic(%s FMaterial%sParameters Parameters)\n{\n%s return %s;\n}\n%s"),
			Header,
			*OutputTypeString, *FunctionName,
			OutputIndex,
			ParameterQualifierStr,
			ShaderFrequencyStr,
			*DefinitionsAnalytic, *BodyAnalytic,
			Footer
		);

		// LWC handling
		if (IsLWCType(OutputType))
		{
			// Add a wrapper with no suffix to return a non-LWC type
			AddLWCFuncUsage(ELWCFunctionKind::Demote);
			ImplementationCodeAnalytic += FString::Printf(TEXT("%s %s_Analytic(%s FMaterial%sParameters Parameters) { return WSDemote(%s_LWC_Analytic(Parameters)); }\n"),
				NonLWSTypeStr, *FunctionNameBase,
				ParameterQualifierStr,
				ShaderFrequencyStr,
				*FunctionNameBase
			);
		}
		else
		{
			// Add a wrapper with LWC suffix to return a LWC type
			AddLWCFuncUsage(ELWCFunctionKind::Promote);
			ImplementationCodeAnalytic += FString::Printf(TEXT("%s %s_LWC_Analytic(%s FMaterial%sParameters Parameters) { return WSPromote(%s_Analytic(Parameters)); }\n"),
				LWSTypeStr, *FunctionNameBase,
				ParameterQualifierStr,
				ShaderFrequencyStr,
				*FunctionNameBase
			);
		}
		
		DerivativeVariations[CompiledPDV_Analytic].CustomOutputImplementations.Add(ImplementationCodeAnalytic);
	}

	// return value is not used
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::CustomOutput(class UMaterialExpressionCustomOutput* Custom, int32 OutputIndex, int32 OutputCode, EMaterialCustomOutputFlags Flags)
{
	if (OutputCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	
	return CustomOutput(Custom->GetDescription(), HLSLTypeString(GetParameterType(OutputCode)), Custom->GetFunctionName(), OutputIndex, OutputCode, Flags);
}

int32 FHLSLMaterialTranslator::PushRuntimeVirtualTextureOutput()
{
	bIsInRuntimeVirtualTextureOutput = true;
	// return value is not used
	return INDEX_NONE;
}

int32 FHLSLMaterialTranslator::PopRuntimeVirtualTextureOutput(uint8 AttributeMask)
{
	bIsInRuntimeVirtualTextureOutput = false;
	MaterialCompilationOutput.bHasRuntimeVirtualTextureOutputNode |= AttributeMask != 0;
	MaterialCompilationOutput.RuntimeVirtualTextureOutputAttributeMask |= AttributeMask;
	// return value is not used
	return INDEX_NONE;
}

bool FHLSLMaterialTranslator::IsInRuntimeVirtualTextureOutput() const
{
	return bIsInRuntimeVirtualTextureOutput;
}

/**
	* Adds code to return a random value shared by all geometry for any given instanced static mesh
	*
	* @return	Code index
	*/
int32 FHLSLMaterialTranslator::PerInstanceRandom()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Vertex && ShaderFrequency != SF_RayHitGroup)
	{
		return NonVertexOrPixelShaderExpressionError();
	}
	else
	{
		bUsesPerInstanceRandomPS |= (ShaderFrequency == SF_Pixel);
		return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("GetPerInstanceRandom(Parameters)"));
	}
}

/**
	* Returns a mask that either enables or disables selection on a per-instance basis when instancing
	*
	* @return	Code index
	*/
int32 FHLSLMaterialTranslator::PerInstanceFadeAmount()
{
	if (ShaderFrequency != SF_Pixel && ShaderFrequency != SF_Vertex)
	{
		return NonVertexOrPixelShaderExpressionError();
	}
	else
	{
		bUsesPerInstanceFadeAmount = true;
		return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("GetPerInstanceFadeAmount(Parameters)"));
	}
}

/**
 *	Returns a custom data on a per-instance basis when instancing
 *	@DataIndex - index in array that represents custom data
 *
 *	@return	Code index
 */
int32 FHLSLMaterialTranslator::PerInstanceCustomData(int32 DataIndex, int32 DefaultValueIndex)
{
	if (DefaultValueIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	// The case where ShaderFrequency is not SF_Vertex only works with Nanite - TODO: Edge case the error
	MaterialCompilationOutput.bUsesPerInstanceCustomData = true;
	return AddInlinedCodeChunkZeroDeriv(MCT_Float, TEXT("GetPerInstanceCustomData(Parameters, %d, %s)"), DataIndex, *GetParameterCode(DefaultValueIndex));
}

/**
 *	Returns a custom data on a per-instance basis when instancing
 *	@DataIndex - index in array that represents custom data
 *
 *	@return	Code index
 */
int32 FHLSLMaterialTranslator::PerInstanceCustomData3Vector(int32 DataIndex, int32 DefaultValueIndex)
{
	if (DefaultValueIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	// The case when ShaderFrequency is not SF_Vertex only works with Nanite - TODO: Edge case the error
	MaterialCompilationOutput.bUsesPerInstanceCustomData = true;
	return AddInlinedCodeChunkZeroDeriv(MCT_Float3, TEXT("GetPerInstanceCustomData3Vector(Parameters, %d, %s)"), DataIndex, *GetParameterCode(DefaultValueIndex));
}

/**
	* Returns a float2 texture coordinate after 2x2 transform and offset applied
	*
	* @return	Code index
	*/
int32 FHLSLMaterialTranslator::RotateScaleOffsetTexCoords(int32 TexCoordCodeIndex, int32 RotationScale, int32 Offset)
{
	if (IsAnalyticDerivEnabled())
	{
		return DerivativeAutogen.GenerateRotateScaleOffsetTexCoordsFunc(*this, TexCoordCodeIndex, RotationScale, Offset);
	}
	else
	{
	return AddCodeChunk(MCT_Float2,
		TEXT("RotateScaleOffsetTexCoords(%s, %s, %s.xy)"),
		*GetParameterCode(TexCoordCodeIndex),
		*GetParameterCode(RotationScale),
		*GetParameterCode(Offset));
}
}

/**
* Handles SpeedTree vertex animation (wind, smooth LOD)
*
* @return	Code index
*/
int32 FHLSLMaterialTranslator::SpeedTree(int32 GeometryArg, int32 WindArg, int32 LODArg, float BillboardThreshold, bool bAccurateWindVelocities, bool bExtraBend, int32 ExtraBendArg)
{ 
	if (Material && Material->IsUsedWithSkeletalMesh())
	{
		return Error(TEXT("SpeedTree node not currently supported for Skeletal Meshes, please disable usage flag."));
	}

	if (ShaderFrequency != SF_Vertex)
	{
		return NonVertexShaderExpressionError();
	}
	else
	{
		bUsesSpeedTree = true;

		AllocateSlot(AllocatedUserVertexTexCoords, 2, 6);

		// Only generate previous frame's computations if required and opted-in
		const bool bEnablePreviousFrameInformation = bCompilingPreviousFrame && bAccurateWindVelocities;
		return AddCodeChunk(MCT_Float3, TEXT("GetSpeedTreeVertexOffset(Parameters, %s, %s, %s, %g, %s, %s, %s)"), *GetParameterCode(GeometryArg), *GetParameterCode(WindArg), *GetParameterCode(LODArg), BillboardThreshold, bEnablePreviousFrameInformation ? TEXT("true") : TEXT("false"), bExtraBend ? TEXT("true") : TEXT("false"), *GetParameterCode(ExtraBendArg, TEXT("float3(0,0,0)")));
	}
}

/**Experimental access to the EyeAdaptation RT for Post Process materials. Can be one frame behind depending on the value of BlendableLocation. */
int32 FHLSLMaterialTranslator::EyeAdaptation()
{
	MaterialCompilationOutput.bUsesEyeAdaptation = true;

	return AddExternalCodeChunk(TEXT("EyeAdaptation"));
}

/**Experimental access to the EyeAdaptation RT for applying an inverse. */
int32 FHLSLMaterialTranslator::EyeAdaptationInverse(int32 LightValueArg, int32 AlphaArg)
{
	if (ShaderFrequency != SF_Pixel)
	{
		return NonPixelShaderExpressionError();
	}

	if (LightValueArg == INDEX_NONE || AlphaArg == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const EMaterialValueType LightValueType = GetParameterType(LightValueArg);
	const EMaterialValueType ResultType = LightValueType;

	if (!IsFloatNumericType(LightValueType))
	{
		Errorf(TEXT("EyeAdaptationInverse expects a float numeric type for LightValue"));
		return INDEX_NONE;
	}

	if (GetParameterType(AlphaArg) != MCT_Float)
	{
		Errorf(TEXT("EyeAdaptationInverse expects a float type for Alpha"));
		return INDEX_NONE;
	}

	MaterialCompilationOutput.bUsesEyeAdaptation = true;

	int32 Multiplier = AddInlinedCodeChunk(MCT_Float, TEXT("EyeAdaptationInverseLookup(%s)"), *GetParameterCode(AlphaArg));

	// return LightValue scaled by inverse eye adaptation
	return Mul(LightValueArg, Multiplier);
}

const bool FHLSLMaterialTranslator::CheckPrimitivePropertyCompatibity(const TCHAR* ExpressionName)
{
	const EMaterialDomain Domain = Material->GetMaterialDomain();
	if (Domain == MD_Surface || Domain == MD_Volume)
	{
		return true;
	}

	Errorf(TEXT("Material expression '%s' is only compatible with Surface or Volume materials."), ExpressionName);
	return false;
}

// to only have one piece of code dealing with error handling if the Primitive constant buffer is not used.
// @param Name e.g. TEXT("ObjectWorldPositionAndRadius.w")
int32 FHLSLMaterialTranslator::GetPrimitiveProperty(EMaterialValueType Type, const TCHAR* ExpressionName, const TCHAR* HLSLName)
{
	if(!CheckPrimitivePropertyCompatibity(ExpressionName))
	{
		return INDEX_NONE;
	}

	return AddInlinedCodeChunkZeroDeriv(Type, TEXT("GetPrimitiveData(Parameters).%s"), HLSLName);
}

// The compiler can run in a different state and this affects caching of sub expression, Expressions are different (e.g. ResolvedView.PrevWorldViewOrigin) when using previous frame's values
bool FHLSLMaterialTranslator::IsCurrentlyCompilingForPreviousFrame() const
{
	return bCompilingPreviousFrame;
}

bool FHLSLMaterialTranslator::IsDevelopmentFeatureEnabled(const FName& FeatureName) const
{
	return UE::MaterialTranslatorUtils::IsDevelopmentFeatureEnabled(FeatureName, Platform, Material->GetMaterialInterface()->GetMaterial());
}

void FHLSLMaterialTranslator::PrepareEnvironmentDefines()
{
	EShaderPlatform InPlatform = GetShaderPlatform();
	*EnvironmentDefines = {};

	bool bMaterialRequestsDualSourceBlending = false;

	EnvironmentDefines->bNeedsParticlePosition = bNeedsParticlePosition || Material->ShouldGenerateSphericalParticleNormals() || bUsesSphericalParticleOpacity;
	EnvironmentDefines->bNeedsParticleVelocity = bNeedsParticleVelocity || Material->IsUsedWithNiagaraMeshParticles();
	EnvironmentDefines->bUseDynamicParameters = bool(DynamicParticleParameterMask);
	EnvironmentDefines->DynamicParametersMask = DynamicParticleParameterMask;
	EnvironmentDefines->bNeedsParticleTime = bNeedsParticleTime;
	EnvironmentDefines->bUsesParticleMotionBlur = bUsesParticleMotionBlur;
	EnvironmentDefines->bNeedsParticleRandom = bNeedsParticleRandom;
	EnvironmentDefines->bSphericalParticleOpacity = bUsesSphericalParticleOpacity;
	EnvironmentDefines->bUseParticleSubUVs = bUsesParticleSubUVs;
	EnvironmentDefines->bLightmapUVAccess = bUsesLightmapUVs;
	EnvironmentDefines->bUsesAOMaterialMask = bUsesAOMaterialMask;
	EnvironmentDefines->bUsesSpeedtree = bUsesSpeedTree;
	EnvironmentDefines->bNeedsWorldPositionExcludingShaderOffsets = bNeedsWorldPositionExcludingShaderOffsets;
	EnvironmentDefines->bNeedsParticleSize = bNeedsParticleSize;
	EnvironmentDefines->bNeedsParticleSpriteRotation = bNeedsParticleSpriteRotation;
	EnvironmentDefines->bNeedsSceneTextures = MaterialCompilationOutput.bNeedsSceneTextures;
	EnvironmentDefines->bAlphaPropagatePostProcessInput0 = Material->GetMaterialDomain() == MD_PostProcess && !Material->GetBlendableOutputAlpha() && MaterialCompilationOutput.IsSceneTextureUsed(PPI_PostProcessInput0);
	EnvironmentDefines->bAlphaPropagateUserSceneTexture = Material->GetMaterialDomain() == MD_PostProcess && !Material->GetBlendableOutputAlpha() && !MaterialCompilationOutput.UserSceneTextureInputs.IsEmpty();
	EnvironmentDefines->UsedSceneTextures = MaterialCompilationOutput.UsedSceneTextures;
	EnvironmentDefines->bUsesEyeAdaptation = MaterialCompilationOutput.bUsesEyeAdaptation;
	EnvironmentDefines->bVirtualTextureOutput = MaterialCompilationOutput.bHasRuntimeVirtualTextureOutputNode;
	EnvironmentDefines->bUsesPerInstanceCustomData = MaterialCompilationOutput.bUsesPerInstanceCustomData && (Material->IsUsedWithInstancedStaticMeshes() || Material->IsUsedWithInstancedSkinnedMesh());
	EnvironmentDefines->bUsesPerInstanceRandomPS = bUsesPerInstanceRandomPS && (Material->IsUsedWithInstancedStaticMeshes() || Material->IsUsedWithInstancedSkinnedMesh());
	EnvironmentDefines->bUsesPerInstanceFadeAmount = bUsesPerInstanceFadeAmount && Material->IsUsedWithInstancedStaticMeshes();
	EnvironmentDefines->bUsesVertexInterpolator = MaterialCompilationOutput.bUsesVertexInterpolator;
	EnvironmentDefines->bUsesSkyAtmosphere = bUsesSkyAtmosphere;
	EnvironmentDefines->bUsesVertexColor = bUsesVertexColor;
	EnvironmentDefines->bUsesParticleColor = bUsesParticleColor;
	EnvironmentDefines->bUsesParticleLocalToWorld = bUsesParticleLocalToWorld;
	EnvironmentDefines->bUsesParticleWorldToLocal = bUsesParticleWorldToLocal;
	EnvironmentDefines->bUsesInstanceLocalToWorldPS = bUsesInstanceLocalToWorldPS;
	EnvironmentDefines->bUsesInstanceWorldToLocalPS = bUsesInstanceWorldToLocalPS;
	EnvironmentDefines->bUsesTransformVector = bUsesTransformVector;
	EnvironmentDefines->bUsesPixelDepthOffset = bUsesPixelDepthOffset;
	// we want USES_WORLD_POSITION_OFFSET to be readable as a bool compile argument, hence the != 0 comparison 
	// (bUsesWorldPositionOffset is actually a 1-bit uint32 bitfield member)
	EnvironmentDefines->bUsesWorldPositionOffset = (bUsesWorldPositionOffset != 0);
	EnvironmentDefines->bUsesDisplacement = (bUsesDisplacement != 0);
	EnvironmentDefines->bUsesEmissiveColor = bUsesEmissiveColor;
	// Distortion uses tangent space transform 
	EnvironmentDefines->bUsesDistortion = Material->IsDistorted();
	EnvironmentDefines->bUsesExplicitDerivatives = bUsesExplicitDerivatives;
	EnvironmentDefines->bUsesFirstPersonInterpolation = bUsesFirstPersonInterpolation;
	EnvironmentDefines->bMaterialEnableTranslucencyFogging = Material->ShouldApplyFogging();
	EnvironmentDefines->bMaterialEnableTranslucencyCloudFogging = Material->ShouldApplyCloudFogging();
	EnvironmentDefines->bMaterialIsSky = Material->IsSky();
	EnvironmentDefines->bMaterialComputeFogPerPixel = Material->ComputeFogPerPixel();
	EnvironmentDefines->bMaterialFullyRough = bIsFullyRough || Material->IsFullyRough();
	EnvironmentDefines->bMaterialUsesAnisotropy = (bUsesAnisotropy || EnumHasAnyFlags(MaterialCompilationOutput.SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::Anisotropy)) && FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(InPlatform);
	EnvironmentDefines->bMaterialUsesSpecularProfile = EnumHasAnyFlags(MaterialCompilationOutput.SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SpecularProfile) && Substrate::IsSpecularProfileEnabled(InPlatform);
	EnvironmentDefines->MaterialDecalReadMask = MaterialCompilationOutput.UsedDBufferTextures;
	EnvironmentDefines->bMaterialUsesDecalLookup = MaterialCompilationOutput.bUsesDBufferTextureLookup;
	EnvironmentDefines->MaterialPathTracingBufferRead = MaterialCompilationOutput.UsedPathTracingBufferTextures;
	EnvironmentDefines->MaterialNeuralPostProcess = (MaterialCompilationOutput.bUsedWithNeuralNetworks || Material->IsUsedWithNeuralNetworks()) && Material->IsPostProcessMaterial();
	EnvironmentDefines->PixelDepthOffsetMode = Material->GetPixelDepthOffsetMode();
	EnvironmentDefines->NumCustomizedUVs = Material->GetNumCustomizedUVs();
	EnvironmentDefines->bMaterialEnableTranslucentLocalLightShadow = Material->AllowTranslucentLocalLightShadow();
	EnvironmentDefines->bMaterialEnableTranslucentHighQualityLocalLightShadow = Material->GetTranslucentLocalLightShadowQuality() > 0;
	EnvironmentDefines->bMaterialEnableTranslucentHighQualityDirectionalLightShadow = Material->GetTranslucentDirectionalLightShadowQuality() > 0;

	// Count the number of VTStacks (each stack will allocate a feedback slot)
	EnvironmentDefines->NumVirtualTextureSamples = VTStacks.Num();

	// Check if any feedback slots are in use. We can simplify shader and remove EARLYZ optimizations if none are.
	uint32 NumVirtualTextureFeedbackRequests = 0;
	for (int i = 0; i < VTStacks.Num(); ++i)
	{
		if (VTStacks[i].SampleInfo.bGenerateFeedback)
		{
			++NumVirtualTextureFeedbackRequests;
		}
	}
	EnvironmentDefines->NumVirtualTextureFeedbackRequests = NumVirtualTextureFeedbackRequests;
	EnvironmentDefines->bMaterialVirtualTextureFeedback = NumVirtualTextureFeedbackRequests > 0;

	// Setup defines to map each VT stack to either 1 or 2 page table textures, depending on how many layers it uses
	EnvironmentDefines->VirtualPageTypes.SetNumUninitialized(VTStacks.Num());
	for (int i = 0; i < VTStacks.Num(); ++i)
	{
		const FMaterialVirtualTextureStack& Stack = MaterialCompilationOutput.UniformExpressionSet.VTStacks[i];
		EnvironmentDefines->VirtualPageTypes[i] = FEnvironmentDefines::LOCAL_TABLE0;
		if (Stack.GetNumLayers() > 4u)
		{
			EnvironmentDefines->VirtualPageTypes[i] |= FEnvironmentDefines::LOCAL_TABLE1;
		}
		if (VTStacks[i].SampleInfo.bAdaptive)
		{
			EnvironmentDefines->VirtualPageTypes[i] |= FEnvironmentDefines::LOCAL_ADAPTIVE_INDIRECTION;
		}
		if (VTStacks[i].SampleInfo.bMeshPaint)
		{
			EnvironmentDefines->VirtualPageTypes[i] |= FEnvironmentDefines::TABLE_MESH_PAINT;
		}
		if (VTStacks[i].SampleInfo.bMaterialCache)
		{
			EnvironmentDefines->VirtualPageTypes[i] |= FEnvironmentDefines::TABLE_MATERIAL_CACHE;
		}
		if (VTStacks[i].SampleInfo.bCollection)
		{
			EnvironmentDefines->VirtualPageTypes[i] |= FEnvironmentDefines::TABLE_COLLECTION;
		}
	}

	// Set all the shading models for this material here 
	// If the material gets its shading model from the material expressions, then we use the result from the compilation (assuming it's valid).
	// This result will potentially be tighter than what GetShadingModels() returns, because it only picks up the shading models from the expressions that get compiled for a specific feature level and quality level
	// For example, the material might have shading models behind static switches. GetShadingModels() will return both the true and the false paths from that switch, whereas the shading model field from the compilation will only contain the actual shading model selected 
	FMaterialShadingModelField ShadingModels = GetCompiledShadingModels();

	ensure(ShadingModels.IsValid());

	// This is to have platforms use the simple single layer water shading similar to mobile: no dynamic lights, only sun and sky, no distortion, no colored transmittance on background, no custom depth read.
	const bool bSingleLayerWaterUsesSimpleShading = FDataDrivenShaderPlatformInfo::GetWaterUsesSimpleForwardShading(InPlatform) && IsForwardShadingEnabled(InPlatform);
	// Value must match SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE in SingleLayerWaterCommon.ush!
	EnvironmentDefines->bSingleLayerWaterShadingQuality = (ShadingModels.HasShadingModel(MSM_SingleLayerWater) && bSingleLayerWaterUsesSimpleShading);

	if (ShadingModels.IsLit())
	{
		EnvironmentDefines->bShadingModelsIsLit = true;

		int NumSetMaterials = 0;

		for (int i = 0; i < MSM_NUM; ++i)
		{
			if (ShadingModels.HasShadingModel((EMaterialShadingModel)i))
			{
				EnvironmentDefines->MaterialShadingModelEnabled |= 1 << i;
				++NumSetMaterials;
			}
		}

		EnvironmentDefines->bMaterialSubsurfaceProfileUseCurvature = ShadingModels.HasShadingModel(MSM_SubsurfaceProfile) && bUsesCurvature;
		EnvironmentDefines->bMaterialShadingModelEyeUseCurvature = ShadingModels.HasShadingModel(MSM_Eye) && bUsesCurvature;
		EnvironmentDefines->bDisableForwardLocalLights = ShadingModels.HasShadingModel(MSM_SingleLayerWater) && FDataDrivenShaderPlatformInfo::GetRequiresDisableForwardLocalLights(Platform);

		const bool bIsWaterDistanceFieldShadowEnabled = IsWaterDistanceFieldShadowEnabled(InPlatform);
		const bool bIsWaterVSMFilteringEnabled = IsWaterVirtualShadowMapFilteringEnabled(InPlatform);
		EnvironmentDefines->bSingleLayerWaterSeparatedMainLight = (ShadingModels.HasShadingModel(MSM_SingleLayerWater) && (bIsWaterDistanceFieldShadowEnabled || bIsWaterVSMFilteringEnabled));
		EnvironmentDefines->bMaterialSingleShadingModel = (NumSetMaterials == 1);

		ensure(NumSetMaterials != 0);
		if (NumSetMaterials == 0)
		{
			// Should not really end up here
			UE_LOGF(LogMaterial, Warning, "Unknown material shading model(s). Setting to MSM_DefaultLit");
			EnvironmentDefines->MaterialShadingModelEnabled |= 1 << MSM_DefaultLit;
		}
	}
	else
	{
		// Unlit shading model can only exist by itself
		EnvironmentDefines->bMaterialSingleShadingModel = true;
		EnvironmentDefines->MaterialShadingModelEnabled |= 1 << MSM_Unlit;
	}

	if (Material->GetMaterialDomain() == MD_Volume)
	{
		TArray<const UMaterialExpressionVolumetricAdvancedMaterialOutput*> VolumetricAdvancedExpressions;
		Material->GetMaterialInterface()->GetMaterial()->GetAllExpressionsOfType(VolumetricAdvancedExpressions);
		if (VolumetricAdvancedExpressions.Num() > 0)
		{
			if (VolumetricAdvancedExpressions.Num() > 1)
			{
				UE_LOGF(LogMaterial, Fatal, "Only a single UMaterialExpressionVolumetricAdvancedMaterialOutput node is supported.");
			}

			EnvironmentDefines->bMaterialVolumetricAdvanced = true;

			const UMaterialExpressionVolumetricAdvancedMaterialOutput* VolumetricAdvancedNode = VolumetricAdvancedExpressions[0];
			EnvironmentDefines->bMaterialVolumetricAdvancedPhasePerSample = VolumetricAdvancedNode->GetEvaluatePhaseOncePerSample();

			EnvironmentDefines->bMaterialVolumetricAdvancedGreyscaleMaterial = VolumetricAdvancedNode->bGrayScaleMaterial;
			EnvironmentDefines->bMaterialVolumetricAdvancedRaymarchVolumeShadow = VolumetricAdvancedNode->bRayMarchVolumeShadow;
			EnvironmentDefines->bMaterialVolumetricAdvancedClampMultiscatteringContribution = VolumetricAdvancedNode->bClampMultiScatteringContribution;
			EnvironmentDefines->MaterialVolumetricAdvancedMultiscatteringOctaveCount = VolumetricAdvancedNode->GetMultiScatteringApproximationOctaveCount();
			EnvironmentDefines->bMaterialVolumetricAdvancedConservativeDensity = VolumetricAdvancedNode->ConservativeDensity.IsConnected();
			EnvironmentDefines->bMaterialVolumetricAdvancedOverrideAmbientOcclusion = Material->HasAmbientOcclusionConnected() || SubstrateTranslatorData.bSubstrateWritesAmbientOcclusion;
			EnvironmentDefines->bMaterialVolumetricAdvancedAdvancedGroundContribution = VolumetricAdvancedNode->bGroundContribution;
		}

		TArray<const UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput*> EmptySpaceSkippingOutputExpressions;
		Material->GetMaterialInterface()->GetMaterial()->GetAllExpressionsOfType(EmptySpaceSkippingOutputExpressions);
		if (EmptySpaceSkippingOutputExpressions.Num() > 0)
		{
			if (VolumetricAdvancedExpressions.Num() > 1)
			{
				UE_LOGF(LogMaterial, Fatal, "Only a single UMaterialExpressionEmptySpaceSkippingOutput node is supported.");
			}
			EnvironmentDefines->bMaterialVolumetricCloudEmptySpaceSkippingOutput = true;
		}
	}

	// Substrate requests dual source blending only for BLEND_TranslucentColoredTransmittance
	bMaterialRequestsDualSourceBlending |= SubstrateTranslatorData.bMaterialIsSubstrate && Material->GetBlendMode() == EBlendMode::BLEND_TranslucentColoredTransmittance;

	// if duals source blending (colored transmittance) is not supported on a platform, it will fall back to standard alpha blending (grey scale transmittance)
	EnvironmentDefines->bDualSourceColorBlendingEnabled = Material->IsDualBlendingEnabled();

	// Finalise compilation and defines. Always need to be called to setup all the defines.
	FString DescriptionStringCommentForDebug;
	SubstrateTranslatorData.GenerateMaterialCompilationOutputAndDefines(
		bOpacityPropertyIsUsed,
		(int32)GetSubstrateMaterialExportType(), (int32)GetSubstrateMaterialExportContext(), (int32)GetSubstrateMaterialExportLegacyBlendMode(),
		DescriptionStringCommentForDebug,
		EnvironmentDefines.Get() ? &EnvironmentDefines.Get()->SubstrateDefines : nullptr);
	ResourcesString += DescriptionStringCommentForDebug;

	EnvironmentDefines->bTextureSampleDebug = IsDebugTextureSampleEnabled();
	EnvironmentDefines->ParameterCollections = ParameterCollections;
}

void FHLSLMaterialTranslator::AsyncQueryDDC(
	UE::DerivedData::FRequestOwner& DDCRequestOwner,
	FSharedBuffer& EnvironmentDefinesBuffer)
{
	UE::DerivedData::FCacheKey CacheKey{ MaterialTranslationDDCBucket, DDCKeyHash };
	UE::DerivedData::FCacheGetRequest Request;
	Request.Name = Material->GetMaterialInterface()->GetName();
	Request.Key = CacheKey;
	Request.Policy = UE::DerivedData::ECachePolicy::Default;

	UE::DerivedData::GetCache().Get(
		{ Request },
		DDCRequestOwner,
		[&](UE::DerivedData::FCacheGetResponse&& Response)
		{
			// If the request was't canceled it means it came back with an answer. Mark the query complete.
			if (Response.Status != UE::DerivedData::EStatus::Canceled)
			{
				DDCQueryCompleted = true;
			}

			// If the status isn't Ok, the request either missed or was canceled.
			if (Response.Status != UE::DerivedData::EStatus::Ok)
			{
				return;
			}

			// Try fetching the translation results from the DDC using the generated key hash
			FSharedBuffer MaterialCompilationOutputBuffer = Response.Record.GetValue(MaterialCompilationOutputId).GetData().Decompress();
			FSharedBuffer TranslationResultsBuffer = Response.Record.GetValue(MaterialResultsOutputId).GetData().Decompress();
			EnvironmentDefinesBuffer = Response.Record.GetValue(EnvironmentDefinesId).GetData().Decompress();

			// Load the results if we hit the cache.
			if (MaterialCompilationOutputBuffer.IsNull() || TranslationResultsBuffer.IsNull() || EnvironmentDefinesBuffer.IsNull())
			{
				return;
			}

			STAT(double SerializeTime = FPlatformTime::Seconds());

			// Read the material compilation output
			FShaderMapPointerTable PointerTable;
			FPlatformTypeLayoutParameters LayoutParams;
			LayoutParams.InitializeForPlatform(GetTargetPlatform());
			FMemoryReaderView MaterialCompilationOutputReader{ TArrayView<uint8>{ (uint8*)MaterialCompilationOutputBuffer.GetData(), (int)MaterialCompilationOutputBuffer.GetSize() } };
			FMemoryImageObject LoadedContent = FMemoryImageResult::LoadFromArchive(MaterialCompilationOutputReader, StaticGetTypeLayoutDesc<FMaterialCompilationOutput>(), &PointerTable, LayoutParams);
			DDCMaterialCompilationOutput = *(FMaterialCompilationOutput*)LoadedContent.Object;

			// Read the array of material string parameters
			FMemoryReaderView ResultsMemoryReader{ TArrayView<uint8>{ (uint8*)TranslationResultsBuffer.GetData(), (int)TranslationResultsBuffer.GetSize() } };
			ResultsMemoryReader << MaterialSourceTemplateParams;

			STAT(DDCRequestSerializeTime = FPlatformTime::Seconds() - SerializeTime);
			
			// The DDC request hit the cache and usable data was retrieved from it.
			DDCQueryHit = true;
		}
	);	
}

void FHLSLMaterialTranslator::PushResultsToDDC()
{
	UE::DerivedData::FCacheKey CacheKey{ MaterialTranslationDDCBucket, DDCKeyHash };
	UE::DerivedData::FCacheRecordBuilder RecordBuilder{ CacheKey };

	// Push the material compilation output
	FBufferArchive MaterialCompilationOutputBuffer;
	FShaderMapPointerTable PointerTable;

	FMemoryImage MemoryImage;
	MemoryImage.PrevPointerTable = &PointerTable;
	MemoryImage.PointerTable = &PointerTable;
	MemoryImage.TargetLayoutParameters.InitializeForArchive(MaterialCompilationOutputBuffer);

	FMemoryImageWriter Writer(MemoryImage);
	Writer.WriteRootObject(&MaterialCompilationOutput, StaticGetTypeLayoutDesc<FMaterialCompilationOutput>());

	FMemoryImageResult MemoryImageResult;
	MemoryImage.Flatten(MemoryImageResult, true);
	MemoryImageResult.SaveToArchive(MaterialCompilationOutputBuffer);
	RecordBuilder.AddValue(MaterialCompilationOutputId, FSharedBuffer::MakeView(MaterialCompilationOutputBuffer.GetData(), MaterialCompilationOutputBuffer.Num()));

	// Push the material source string parameters
	FBufferArchive ResultsBuffer;
	ResultsBuffer << MaterialSourceTemplateParams;
	RecordBuilder.AddValue(MaterialResultsOutputId, FSharedBuffer::MakeView(ResultsBuffer.GetData(), ResultsBuffer.Num()));

	// Push the material shader defines
	FBufferArchive EnvironmentDefinesBuffer;
	FObjectAndNameAsStringProxyArchive EnvironmentDefinesBufferProxy{ EnvironmentDefinesBuffer, true };
	EnvironmentDefines->Serialize(EnvironmentDefinesBufferProxy);
	RecordBuilder.AddValue(EnvironmentDefinesId, FSharedBuffer::MakeView(EnvironmentDefinesBuffer.GetData(), EnvironmentDefinesBuffer.Num()));

	// Push the all the data to the DDC
	UE::DerivedData::FRequestOwner RequestOwner{ UE::DerivedData::EPriority::Normal };
	UE::DerivedData::ECachePolicy Policy = UE::DerivedData::ECachePolicy::Default;
	UE::DerivedData::FCachePutRequest Request{ { Material->GetMaterialInterface()->GetName() }, RecordBuilder.Build(), Policy };
	UE::DerivedData::GetCache().Put({ Request }, RequestOwner);
	RequestOwner.KeepAlive();
}

static FString GetSwizzleFromOffset(const FMaterialCacheLayer& Layer, EMaterialCacheAttribute Attribute, bool bIsStore)
{
	uint8 Offset         = GetMaterialCacheLayerAttributeSwizzleOffset(Layer, Attribute, bIsStore);
	uint8 ComponentCount = GetMaterialCacheAttributeComponentCount(Attribute, bIsStore);
	
	FString Swizzle;
	for (uint8 i = 0; i < ComponentCount; i++)
	{
		switch (Offset++)
		{
		default:
			checkf(false, TEXT("Invalid offset"));
			return Swizzle;
		case 0:
			Swizzle += TEXT("x");
			break;
		case 1:
			Swizzle += TEXT("y");
			break;
		case 2:
			Swizzle += TEXT("z");
			break;
		case 3:
			Swizzle += TEXT("w");
			break;
		}
	}

	return Swizzle;
}

void FHLSLMaterialTranslator::GenerateMaterialCacheSource()
{
	// All templated sections
	FString GeneratedTypes;
	FString GeneratedCommon;
	FString GeneratedPass;

	// If there's no material cache, early out
	if (!Material->SamplesMaterialCache() && !Material->IsDefaultMaterial())
	{
		MaterialSourceTemplateParams.Add({ TEXT("material_cache_types"),  GeneratedTypes });
		MaterialSourceTemplateParams.Add({ TEXT("material_cache_common"), GeneratedCommon });
		MaterialSourceTemplateParams.Add({ TEXT("material_cache_pass"),   GeneratedPass   });
		return;
	}

	// Allow materials to output to the material cache without a node, such as with default materials
	// So pack the default tag
	if (MaterialCacheTags.IsEmpty())
	{
		FMaterialCacheTag Tag;
		PackMaterialCacheAttributeLayers(DefaultMaterialCacheAttributes, Tag.Layers);
		MaterialCacheTags.Add(FGuid(), Tag);
	}

	// Generate structures and definitions per tag (unique)
	for (auto&& [Guid, Tag] : MaterialCacheTags)
	{
		// Tag_0 Tag_1...
		FString Postfix = FString::Printf(TEXT("%d"), Tag.StackIndex);
		
		// struct FMaterialCacheABuffer { ... }
		// Stores the unpacked attributes 
		{
			GeneratedTypes += FString::Printf(TEXT("struct FMaterialCacheABuffer_%s\n"), *Postfix);
			GeneratedTypes += TEXT("{\n");
			
			GeneratedTypes += TEXT("\tfloat Weight;\n");
			GeneratedTypes += TEXT("\tbool Clipped;\n");

			// Naming counter
			uint32 LocalCounter = 0;

			// Store the individual attributes, not the layers
			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				const FMaterialCacheLayer& Layer = Tag.Layers[i];
				
				for (EMaterialCacheAttribute Attribute : Layer.Attributes)
				{
					GeneratedTypes += FString::Printf(
						TEXT("\tfloat%u %s;\n"),
						GetMaterialCacheAttributeComponentCount(Attribute, false),
						*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(Attribute), LocalCounter++)
					);
				}
			}
			
			GeneratedTypes += TEXT("};\n\n");
		}

		// FMaterialCacheABuffer_% DefaultMaterialCacheABuffer()
		// Get the default ABuffer
		{
			GeneratedCommon += FString::Printf(TEXT("FMaterialCacheABuffer_%s DefaultMaterialCacheABuffer_%s()\n"), *Postfix, *Postfix);
			GeneratedCommon += TEXT("{\n");
			GeneratedCommon += FString::Printf(TEXT("\tFMaterialCacheABuffer_%s ABuffer = (FMaterialCacheABuffer_%s)0;\n"), *Postfix, *Postfix);
			GeneratedCommon += TEXT("\tABuffer.Weight = 1.0f;\n");

			// Naming counter
			uint32 LocalCounter = 0;

			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				const FMaterialCacheLayer& Layer = Tag.Layers[i];
				
				for (EMaterialCacheAttribute Attribute : Layer.Attributes)
				{
					FString DefaultValue;
					
					if (Attribute == EMaterialCacheAttribute::Normal)
					{
						DefaultValue = TEXT("float3(0, 0, 1)");
					}

					// Just use the defaulted (0)
					if (DefaultValue.IsEmpty())
					{
						LocalCounter++;
						continue;
					}
					
					GeneratedCommon += FString::Printf(
						TEXT("\tABuffer.%s = %s;\n"),
						*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(Attribute), LocalCounter++),
						*DefaultValue
					);
				}
			}
			
			GeneratedCommon += TEXT("\treturn ABuffer;\n");
			GeneratedCommon += TEXT("}\n\n");
		}

		// FMaterialCacheABuffer_% UnpackMaterialCacheABuffer(float4 Layer0, ...)
		// Unpack an ABuffer from its layers
		{
			FString ParameterStack;

			// Setup parameter stack
			// Note: Initially used arrays float4 Layers[N], however, this doesn't work cleanly with the translator when getting the parameter codes
			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				if (i != 0)
				{
					ParameterStack += TEXT(", ");
				}

				ParameterStack += FString::Printf(TEXT("float4 Layer%i"), i);
			}
			
			GeneratedCommon += FString::Printf(TEXT("FMaterialCacheABuffer_%s UnpackMaterialCacheABuffer_%s(%s)\n"), *Postfix, *Postfix, *ParameterStack);
			GeneratedCommon += TEXT("{\n");
			GeneratedCommon += FString::Printf(TEXT("\tFMaterialCacheABuffer_%s ABuffer = (FMaterialCacheABuffer_%s)0;\n"), *Postfix, *Postfix);

			// Naming counter
			uint32 LocalCounter = 0;

			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				const FMaterialCacheLayer& Layer = Tag.Layers[i];
				
				// Special case
				if (Layer.bYCoCg)
				{
					GeneratedCommon += FString::Printf(
						TEXT("\tABuffer.%s = VirtualTextureUnpackBaseColorYCoCg(Layer%i.xyzw);\n"),
						*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(Layer.Attributes[0]), LocalCounter++),
						i
					);
					
					check(Layer.Attributes.Num() == 1);
					continue;
				}

				if (Layer.Identity == EMaterialCacheAttributeIdentity::None)
				{
					for (EMaterialCacheAttribute Attribute : Layer.Attributes)
					{
						FString Value = FString::Printf(TEXT("Layer%i.%s"), i, *GetSwizzleFromOffset(Layer, Attribute, true));
						
						switch (Attribute)
						{
							default:
								break;
							case EMaterialCacheAttribute::Normal:
								Value = FString::Printf(TEXT("OctahedronToUnitVector(%s * 2.0f - 1.0f)"), *Value);
								break;
						}
						
						GeneratedCommon += FString::Printf(
							TEXT("\tABuffer.%s = %s;\n"),
							*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(Attribute), LocalCounter++),
							*Value
						);
					}
				}
				else
				{
					switch (Layer.Identity)
					{
						default:
						{
							checkNoEntry();
							break;
						}
						case EMaterialCacheAttributeIdentity::BaseColorRoughness:
						{
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.xyz;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::BaseColor), LocalCounter++),
								i
							);
								
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.w;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Roughness), LocalCounter++),
								i
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::NormalSpecularOpacity:
						{
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = OctahedronToUnitVector(Layer%i.xy * 2.0f - 1.0f);\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Normal), LocalCounter++),
								i
							);
								
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.z;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Specular), LocalCounter++),
								i
							);
								
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.w;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Opacity), LocalCounter++),
								i
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::SpecularRoughnessMetallicOpacity:
						{
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.x;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Specular), LocalCounter++),
								i
							);
							
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.y;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Roughness), LocalCounter++),
								i
							);
							
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.z;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Metallic), LocalCounter++),
								i
							);
							
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.w;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Opacity), LocalCounter++),
								i
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::MetallicWorldPositionOffset:
						{
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.x;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Metallic), LocalCounter++),
								i
							);
								
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.yzw;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::WorldPosition), LocalCounter++),
								i
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::RoughnessMetallic:
						{
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.x;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Roughness), LocalCounter++),
								i
							);
							
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.y;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Metallic), LocalCounter++),
								i
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::Displacement:
						{
							GeneratedCommon += FString::Printf(
								TEXT("\tABuffer.%s = Layer%i.x;\n"),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Displacement), LocalCounter++),
								i
							);
							break;
						}
					}
				}
			}
			
			GeneratedCommon += TEXT("\treturn ABuffer;\n");
			GeneratedCommon += TEXT("}\n\n");
		}

		// FMaterialCacheABuffer_% GetABufferFromMaterialInputs(...)
		// Constructs an ABuffer from the material inputs, a helper function
		{
			GeneratedPass += FString::Printf(TEXT("FMaterialCacheABuffer_%s GetABufferFromMaterialInputs_%s(in FPixelMaterialInputs PixelMaterialInputs)\n"), *Postfix, *Postfix);
			GeneratedPass += TEXT("{\n");
			
			GeneratedPass += FString::Printf(TEXT("\tFMaterialCacheABuffer_%s ABuffer = (FMaterialCacheABuffer_%s)0;\n"), *Postfix, *Postfix);
			GeneratedPass += TEXT("\tABuffer.Weight = 1.0f;\n");

			// Naming counter
			uint32 LocalCounter = 0;

			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				const FMaterialCacheLayer& Layer = Tag.Layers[i];

				for (EMaterialCacheAttribute Attribute : Layer.Attributes)
				{
					FString Value;
					switch (Attribute)
					{
					case EMaterialCacheAttribute::BaseColor:
						Value = TEXT("saturate(PixelMaterialInputs.BaseColor)");
						break;
					case EMaterialCacheAttribute::Normal:
						Value = TEXT("normalize(PixelMaterialInputs.Normal)");
						break;
					case EMaterialCacheAttribute::Roughness:
						Value = TEXT("saturate(PixelMaterialInputs.Roughness)");
						break;
					case EMaterialCacheAttribute::Specular:
						Value = TEXT("saturate(PixelMaterialInputs.Specular)");
						break;
					case EMaterialCacheAttribute::Metallic:
						Value = TEXT("saturate(PixelMaterialInputs.Metallic)");
						break;
					case EMaterialCacheAttribute::Displacement:
						Value = TEXT("saturate(PixelMaterialInputs.Displacement)");
						break;
					case EMaterialCacheAttribute::Opacity:
					case EMaterialCacheAttribute::WorldPosition:
					case EMaterialCacheAttribute::WorldHeight:
					case EMaterialCacheAttribute::Mask:
					case EMaterialCacheAttribute::Float:
						LocalCounter++;
						continue;
					}
					
					GeneratedPass += FString::Printf(
						TEXT("\tABuffer.%s = %s;\n"),
						*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(Attribute), LocalCounter++),
						*Value
					);
				}
			}
			
			GeneratedPass += TEXT("\treturn ABuffer;\n");
			GeneratedPass += TEXT("}\n\n");
		}

		// FMaterialCacheABuffer_% LoadMaterialCacheABufferPixel(...)
		// Load the material cache ABuffer from a given location
		// ! This is only used for the actual material cache rendering, hence why the destinatioon ABuffer targets arent differentiated
		{
			GeneratedPass += FString::Printf(TEXT("FMaterialCacheABuffer_%s LoadMaterialCacheABufferPixel_%s(uint3 PhysicalLocation, uint2 PagePixel)\n"), *Postfix, *Postfix);
			GeneratedPass += TEXT("{\n");
			GeneratedPass += TEXT("\tPhysicalLocation.xy += PagePixel;\n");

			FString ArgumentStack;

			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				if (i != 0)
				{
					ArgumentStack += TEXT(", ");
				}

				ArgumentStack += FString::Printf(TEXT("MaterialCachePass.RWABuffer_%i[PhysicalLocation]"), i);
			}
			
			GeneratedPass += FString::Printf(TEXT("\treturn UnpackMaterialCacheABuffer_%s(%s);\n"), *Postfix, *ArgumentStack);
			GeneratedPass += TEXT("}\n\n");
		}

		// FMaterialCacheABuffer_% StoreMaterialCacheABufferPixel(...)
		// Store the material cache ABuffer to a given location
		// ! This is only used for the actual material cache rendering, hence why the destinatioon ABuffer targets arent differentiated
		{
			GeneratedPass += FString::Printf(TEXT("void StoreMaterialCacheABufferPixel_%s(in FMaterialCacheABuffer_%s ABuffer, uint3 PhysicalLocation, uint2 PagePixel)\n"), *Postfix, *Postfix);
			GeneratedPass += TEXT("{\n");
			
			GeneratedPass += TEXT("\tPhysicalLocation.xy += PagePixel;\n");

			// Naming counter
			uint32 LocalCounter = 0;

			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				const FMaterialCacheLayer& Layer = Tag.Layers[i];

				GeneratedPass += FString::Printf(TEXT("\tfloat4 Layer%i = 0;\n"), i);

				if (Layer.Identity == EMaterialCacheAttributeIdentity::None)
				{
					for (EMaterialCacheAttribute Attribute : Layer.Attributes)
					{
						FString Value = FString::Printf(TEXT("ABuffer.%s%u"), *GetMaterialCacheAttributeDecoration(Attribute), LocalCounter++);
						
						switch (Attribute)
						{
						default:
							break;
						case EMaterialCacheAttribute::Normal:
							Value = FString::Printf(TEXT("UnitVectorToOctahedron(%s.xyz) * 0.5f + 0.5f"), *Value);
							break;
						}
						
						GeneratedPass += FString::Printf(
							TEXT("\tLayer%u.%s = %s;\n"),
							i,
							*GetSwizzleFromOffset(Layer, Attribute, true),
							*Value
						);
					}
				}
				else
				{
					switch (Layer.Identity)
					{
						default:
						{
							checkNoEntry();
							break;
						}
						case EMaterialCacheAttributeIdentity::BaseColorRoughness:
						{
							int32 BaseColorIndex = LocalCounter++;
							int32 RoughnessIndex = LocalCounter++;
							
							GeneratedPass += FString::Printf(
								TEXT("\tLayer%u = float4(ABuffer.%s, ABuffer.%s);\n"),
								i,
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::BaseColor), BaseColorIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Roughness), RoughnessIndex)
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::NormalSpecularOpacity:
						{
							int32 NormalIndex = LocalCounter++;
							int32 SpecularIndex = LocalCounter++;
							int32 OpacityIndex = LocalCounter++;
						
							GeneratedPass += FString::Printf(
								TEXT("\tLayer%u = float4(UnitVectorToOctahedron(ABuffer.%s.xyz) * 0.5f + 0.5f, ABuffer.%s, ABuffer.%s);\n"),
								i,
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Normal), NormalIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Specular), SpecularIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Opacity), OpacityIndex)
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::SpecularRoughnessMetallicOpacity:
						{
							int32 SpecularIndex = LocalCounter++;
							int32 RoughnessIndex = LocalCounter++;
							int32 MetallicIndex = LocalCounter++;
							int32 OpacityIndex = LocalCounter++;
							
							GeneratedPass += FString::Printf(
								TEXT("\tLayer%u = float4(ABuffer.%s, ABuffer.%s, ABuffer.%s, ABuffer.%s);\n"),
								i,
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Specular), SpecularIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Roughness), RoughnessIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Metallic), MetallicIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Opacity), OpacityIndex)
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::MetallicWorldPositionOffset:
						{
							int32 MetallicIndex = LocalCounter++;
							int32 PositionIndex = LocalCounter++;
							
							GeneratedPass += FString::Printf(
								TEXT("\tLayer%u = float4(ABuffer.%s, ABuffer.%s);\n"),
								i,
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Metallic), MetallicIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::WorldPosition), PositionIndex)
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::RoughnessMetallic:
						{
							int32 RoughnessIndex = LocalCounter++;
							int32 MetallicIndex = LocalCounter++;
							
							GeneratedPass += FString::Printf(
								TEXT("\tLayer%u = float4(ABuffer.%s, ABuffer.%s, 0, 0);\n"),
								i,
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Roughness), RoughnessIndex),
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Metallic), MetallicIndex)
							);
							break;
						}
						case EMaterialCacheAttributeIdentity::Displacement:
						{
							int32 DisplacementIndex = LocalCounter++;
							
							GeneratedPass += FString::Printf(
								TEXT("\tLayer%u = float4(ABuffer.%s, 0, 0, 0);\n"),
								i,
								*FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(EMaterialCacheAttribute::Displacement), DisplacementIndex)
							);
							break;
						}
					}
				}

				GeneratedPass += FString::Printf(
					TEXT("\tMaterialCachePass.RWABuffer_%i[PhysicalLocation] = Layer%u;\n"),
					i, i
				);
			}
			
			GeneratedPass += TEXT("}\n\n");
		}

		// FMaterialCacheABuffer_% BlendMaterialCacheFixedFunctionLerp(...)
		// Interpolates all attributes, fixed function style, of an ABuffer
		{
			GeneratedPass += FString::Printf(TEXT("FMaterialCacheABuffer_%s BlendMaterialCacheFixedFunctionLerp_%s(in FMaterialCacheABuffer_%s Bottom, in FMaterialCacheABuffer_%s Top, float Weight)\n"), *Postfix, *Postfix, *Postfix, *Postfix);
			GeneratedPass += TEXT("{\n");
			GeneratedPass += FString::Printf(TEXT("\tFMaterialCacheABuffer_%s Out = (FMaterialCacheABuffer_%s)0;\n"), *Postfix, *Postfix);

			// Naming counter
			uint32 LocalCounter = 0;

			for (int32 i = 0; i < Tag.Layers.Num(); i++)
			{
				const FMaterialCacheLayer& Layer = Tag.Layers[i];

				for (EMaterialCacheAttribute Attribute : Layer.Attributes)
				{
					FString Name = FString::Printf(TEXT("%s%u"), *GetMaterialCacheAttributeDecoration(Attribute), LocalCounter++);
					
					GeneratedPass += FString::Printf(
						TEXT("\tOut.%s = lerp(Bottom.%s, Top.%s, Weight);\n"),
						*Name, *Name, *Name
					);
				}
			}
			
			GeneratedPass += TEXT("\treturn Out;\n");
			GeneratedPass += TEXT("}\n\n");
		}	
	}

	// Replace templated sections
	MaterialSourceTemplateParams.Add({ TEXT("material_cache_types"),  GeneratedTypes });
	MaterialSourceTemplateParams.Add({ TEXT("material_cache_common"), GeneratedCommon });
	MaterialSourceTemplateParams.Add({ TEXT("material_cache_pass"),   GeneratedPass   });
}

void FHLSLMaterialTranslator::PrepareMaterialSourceStringParameters()
{
	MaterialSourceTemplateParams.Empty();
	MaterialSourceTemplateParams.Reserve(FMaterialSourceTemplate::Get().GetTemplate(GetShaderPlatform()).GetNumNamedParameters());

	// Assign slots to vertex interpolators
	FString VertexInterpolatorsOffsetsDefinition;
	TArray<uint32> UsedTexCoordIndices;
	TBitArray<> FinalAllocatedCoords = GetVertexInterpolatorsOffsets(VertexInterpolatorsOffsetsDefinition, UsedTexCoordIndices);
	const uint32 NumUserVertexTexCoords = GetNumUserVertexTexCoords();
	const uint32 NumUserTexCoords = GetNumUserTexCoords();
	const uint32 NumCustomVectors = FMath::DivideAndRoundUp((uint32)CurrentCustomVertexInterpolatorOffset, 2u);
	const uint32 NumTexCoordVectors = FinalAllocatedCoords.FindLast(true) + 1;

	MaterialSourceTemplateParams.Add({ TEXT("num_material_texcoords_vertex"), FString::Printf(TEXT("%u"), NumUserVertexTexCoords) });
	MaterialSourceTemplateParams.Add({ TEXT("num_material_texcoords"), FString::Printf(TEXT("%u"), NumUserTexCoords) });
	MaterialSourceTemplateParams.Add({ TEXT("num_custom_vertex_interpolators"), FString::Printf(TEXT("%u"), NumCustomVectors) });
	MaterialSourceTemplateParams.Add({ TEXT("num_tex_coord_interpolators"), FString::Printf(TEXT("%u"), NumTexCoordVectors) });
	MaterialSourceTemplateParams.Add({ TEXT("vertex_interpolators_offsets_definition"), VertexInterpolatorsOffsetsDefinition });

	FString MaterialAttributesDeclaration;
	FString MaterialAttributesUtilities;

	const TArray<FGuid>& OrderedVisibleAttributes = FMaterialAttributeDefinitionMap::GetOrderedVisibleAttributeList();

	// Reserve enough space vased on an estimate of the length of a single attribute string
	MaterialAttributesDeclaration.Reserve(30 + MaterialAttributesDeclaration.Len() * 128);
	MaterialAttributesUtilities.Reserve(MaterialAttributesDeclaration.Len() * 256);

	MaterialAttributesDeclaration.Append(TEXT("struct FMaterialAttributes\n{\n"));

	const EMaterialShadingModel DefaultShadingModel = Material->GetShadingModels().GetFirstShadingModel();
	for (const FGuid& AttributeID : OrderedVisibleAttributes)
	{
		const FString PropertyName = FMaterialAttributeDefinitionMap::GetAttributeName(AttributeID);
		const EMaterialValueType PropertyType = FMaterialAttributeDefinitionMap::GetValueType(AttributeID);
		const TCHAR* HLSLType = nullptr;

		switch (PropertyType)
		{
			case MCT_Float1: case MCT_Float: HLSLType = TEXT("float"); break;
			case MCT_Float2: HLSLType = TEXT("float2"); break;
			case MCT_Float3: HLSLType = TEXT("float3"); break;
			case MCT_Float4: HLSLType = TEXT("float4"); break;
			case MCT_UInt: case MCT_UInt1: case MCT_ShadingModel: HLSLType = TEXT("uint"); break;
			case MCT_UInt2: HLSLType = TEXT("uint2"); break;
			case MCT_UInt3: HLSLType = TEXT("uint3"); break;
			case MCT_UInt4: HLSLType = TEXT("uint4"); break;
			case MCT_Substrate: HLSLType = TEXT("FSubstrateData"); break;
			default: break;
		}

		if (HLSLType)
		{
			const FVector4f DefaultValue = FMaterialAttributeDefinitionMap::GetDefaultValue(AttributeID);

			MaterialAttributesDeclaration.Appendf(TEXT("\t%s %s;") HLSL_LINE_TERMINATOR, HLSLType, *PropertyName);

			// Chainable method to set the attribute
			MaterialAttributesUtilities.Appendf(TEXT("FMaterialAttributes FMaterialAttributes_Set%s(FMaterialAttributes InAttributes, %s InValue) { InAttributes.%s = InValue; return InAttributes; }") HLSL_LINE_TERMINATOR,
				*PropertyName, HLSLType, *PropertyName);
		}
	}

	MaterialAttributesDeclaration.Append(TEXT("};\n"));

	MaterialSourceTemplateParams.Add({ TEXT("material_declarations"), MaterialAttributesDeclaration });
	MaterialSourceTemplateParams.Add({ TEXT("material_attributes_utilities"), MaterialAttributesUtilities });

	// Stores the shared shader results member declarations
	FString PixelMembersDeclaration[CompiledPDV_MAX];

	FString NormalAssignment[CompiledPDV_MAX];

	// Stores the code to initialize all inputs after MP_Normal
	FString PixelMembersSetupAndAssignments[CompiledPDV_MAX];

	for (int32 Index = 0; Index < CompiledPDV_MAX; Index++)
	{
		GetSharedInputsMaterialCode(PixelMembersDeclaration[Index], NormalAssignment[Index], PixelMembersSetupAndAssignments[Index], (ECompiledPartialDerivativeVariation)Index);
	}

	// PixelMembersDeclaration should be the same for all variations, but might change in the future. There are cases where work is shared
	// between the pixel and vertex shader, but with Nanite all work has to be moved into the pixel shader, which means we will want
	// different inputs. But for now, we are keeping them the same.
	MaterialSourceTemplateParams.Add({ TEXT("pixel_material_inputs"), PixelMembersDeclaration[CompiledPDV_FiniteDifferences] });

	{
		FString DerivativeHelpers = DerivativeAutogen.GenerateUsedFunctions(*this);
		FString DerivativeHelpersAndResources = DerivativeHelpers + ResourcesString;
		MaterialSourceTemplateParams.Add({ TEXT("uniform_material_expressions"), DerivativeHelpersAndResources });
	}

	// Anything used bye the GenerationFunctionCode() like WorldPositionOffset shouldn't be using texures, right?
	// Let those use the standard finite differences textures, since they should be the same. If we actually want
	// those to handle texture reads properly, we'll have to make extra versions.
	ECompiledPartialDerivativeVariation BaseDerivativeVariation = CompiledPDV_FiniteDifferences;

	if (bCompileForComputeShader)
	{
		MaterialSourceTemplateParams.Add({ TEXT("get_material_emissive_for_cs"), GenerateFunctionCode(CompiledMP_EmissiveColorCS, BaseDerivativeVariation) });
	}
	else
	{
		MaterialSourceTemplateParams.Add({ TEXT("get_material_emissive_for_cs"), TEXT("return 0") });
	}

	MaterialSourceTemplateParams.Add({ TEXT("get_material_translucency_directional_lighting_intensity"), FString::Printf(TEXT("return %.5f"), Material->GetTranslucencyDirectionalLightingIntensity()) });
	MaterialSourceTemplateParams.Add({ TEXT("get_material_translucent_shadow_density_scale"), FString::Printf(TEXT("return %.5f"), Material->GetTranslucentShadowDensityScale()) });
	MaterialSourceTemplateParams.Add({ TEXT("get_material_translucent_self_shadow_density_scale"), FString::Printf(TEXT("return %.5f"), Material->GetTranslucentSelfShadowDensityScale()) });
	MaterialSourceTemplateParams.Add({ TEXT("get_material_translucent_self_shadow_second_density_scale"), FString::Printf(TEXT("return %.5f"), Material->GetTranslucentSelfShadowSecondDensityScale()) });
	MaterialSourceTemplateParams.Add({ TEXT("get_material_translucent_self_shadow_second_opacity"), FString::Printf(TEXT("return %.5f"), Material->GetTranslucentSelfShadowSecondOpacity()) });
	MaterialSourceTemplateParams.Add({ TEXT("get_material_translucent_backscattering_exponent"), FString::Printf(TEXT("return %.5f"), Material->GetTranslucentBackscatteringExponent()) });

	{
		FLinearColor Extinction = Material->GetTranslucentMultipleScatteringExtinction();

		MaterialSourceTemplateParams.Add({ TEXT("get_material_translucent_multiple_scattering_extinction"), FString::Printf(TEXT("return MaterialFloat3(%.5f, %.5f, %.5f)"), Extinction.R, Extinction.G, Extinction.B) });
	}

	MaterialSourceTemplateParams.Add({ TEXT("get_material_opacity_mask_clip_value"), FString::Printf(TEXT("return %.5f"), Material->GetOpacityMaskClipValue()) });

	MaterialSourceTemplateParams.Add({ TEXT("get_material_world_position_offset_raw"), *GenerateFunctionCode(MP_WorldPositionOffset, BaseDerivativeVariation) });
	MaterialSourceTemplateParams.Add({ TEXT("get_material_previous_world_position_offset_raw"), *GenerateFunctionCode(CompiledMP_PrevWorldPositionOffset, BaseDerivativeVariation) });
	
	// Print custom texture coordinate assignments, should be fine with regular derivatives
	FString CustomUVAssignments;

	int32 LastProperty = -1;
	for (uint32 CustomUVIndex = 0; CustomUVIndex < NumUserTexCoords; CustomUVIndex++)
	{
		if (CustomUVIndex == 0)
		{
			CustomUVAssignments.Append(DerivativeVariations[BaseDerivativeVariation].TranslatedCodeChunkDefinitions[MP_CustomizedUVs0 + CustomUVIndex]);
		}

		if (DerivativeVariations[BaseDerivativeVariation].TranslatedCodeChunkDefinitions[MP_CustomizedUVs0 + CustomUVIndex].Len() > 0)
		{
			LastProperty = MP_CustomizedUVs0 + CustomUVIndex;
		}
		CustomUVAssignments.Appendf(TEXT("\tOutTexCoords[%u] = %s;") HLSL_LINE_TERMINATOR, CustomUVIndex, *DerivativeVariations[BaseDerivativeVariation].TranslatedCodeChunks[MP_CustomizedUVs0 + CustomUVIndex]);
	}

	MaterialSourceTemplateParams.Add({ TEXT("get_material_customized_u_vs"), CustomUVAssignments });

	// Print custom vertex shader interpolator assignments
	FString CustomInterpolatorAssignments;

	for (uint32 InterpolatorIndex : UsedTexCoordIndices)
	{
		CustomInterpolatorAssignments.Appendf(TEXT("\tOutTexCoords[%i] = 0.0f.xx;") HLSL_LINE_TERMINATOR, InterpolatorIndex);
	}

	for (UMaterialExpressionVertexInterpolator* Interpolator : CustomVertexInterpolators)
	{
		if (Interpolator->InterpolatorOffset != INDEX_NONE)
		{
			check(Interpolator->InterpolatorIndex != INDEX_NONE);
			check(Interpolator->InterpolatedType & MCT_Float);

			const EMaterialValueType Type = Interpolator->InterpolatedType == MCT_Float ? MCT_Float1 : Interpolator->InterpolatedType;
			const TCHAR* Swizzle[2] = { TEXT("x"), TEXT("y") };
			const int32 Offset = Interpolator->InterpolatorOffset;
			const int32 Index = Interpolator->InterpolatorIndex;

			// Note: We reference the UV define directly to avoid having to pre-accumulate UV counts before property translation
			CustomInterpolatorAssignments.Appendf(TEXT("\tOutTexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_X].%s = VertexInterpolator%i(Parameters).x;") HLSL_LINE_TERMINATOR, Index, Swizzle[Offset % 2], Index);

			if (Type >= MCT_Float2)
			{
				CustomInterpolatorAssignments.Appendf(TEXT("\tOutTexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Y].%s = VertexInterpolator%i(Parameters).y;") HLSL_LINE_TERMINATOR, Index, Swizzle[(Offset + 1) % 2], Index);

				if (Type >= MCT_Float3)
				{
					CustomInterpolatorAssignments.Appendf(TEXT("\tOutTexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_Z].%s = VertexInterpolator%i(Parameters).z;") HLSL_LINE_TERMINATOR, Index, Swizzle[(Offset + 2) % 2], Index);

					if (Type == MCT_Float4)
					{
						CustomInterpolatorAssignments.Appendf(TEXT("\tOutTexCoords[VERTEX_INTERPOLATOR_%i_TEXCOORDS_W].%s = VertexInterpolator%i(Parameters).w;") HLSL_LINE_TERMINATOR, Index, Swizzle[(Offset + 3) % 2], Index);
					}
				}
			}
		}
	}

	MaterialSourceTemplateParams.Add({ TEXT("get_custom_interpolators"), CustomInterpolatorAssignments });

	{
		// Initializers required for Normal
		MaterialSourceTemplateParams.Add({ TEXT("calc_pixel_material_inputs_initial_calculations"), DerivativeVariations[CompiledPDV_FiniteDifferences].TranslatedCodeChunkDefinitions[MP_Normal] });
		MaterialSourceTemplateParams.Add({ TEXT("calc_pixel_material_inputs_normal"), NormalAssignment[CompiledPDV_FiniteDifferences] });
		// Finally the rest of common code followed by assignment into each input
		MaterialSourceTemplateParams.Add({ TEXT("calc_pixel_material_inputs_other_inputs"), PixelMembersSetupAndAssignments[CompiledPDV_FiniteDifferences] });
	}
	{
		// Initializers required for Normal
		MaterialSourceTemplateParams.Add({ TEXT("calc_pixel_material_inputs_analytic_derivatives_initial"), DerivativeVariations[CompiledPDV_Analytic].TranslatedCodeChunkDefinitions[MP_Normal] });
		MaterialSourceTemplateParams.Add({ TEXT("calc_pixel_material_inputs_analytic_derivatives_normal"), NormalAssignment[CompiledPDV_Analytic] });
		// Finally the rest of common code followed by assignment into each input
		MaterialSourceTemplateParams.Add({ TEXT("calc_pixel_material_inputs_analytic_derivatives_other_inputs"), PixelMembersSetupAndAssignments[CompiledPDV_Analytic] });
	}

	MaterialSourceTemplateParams.Add({ TEXT("user_scene_texture_remap"), UE::MaterialTranslatorUtils::GenerateUserSceneTextureRemapHLSLDefines(MaterialCompilationOutput) });
}

int32 FHLSLMaterialTranslator::SparseVolumeTexture(USparseVolumeTexture* Texture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType)
{
	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(Texture);
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->SparseVolumeTexture() without implementing UMaterialExpression::GetReferencedTexture properly"));

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTexture(TextureReferenceIndex, SamplerType), MCT_SparseVolumeTexture, TEXT(""));
}

int32 FHLSLMaterialTranslator::SparseVolumeTextureParameter(FName ParameterName, USparseVolumeTexture* InDefaultTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType)
{
	USparseVolumeTexture* DefaultTexture = InDefaultTexture;

	// If we're compiling a function, give the function a chance to override the default parameter value
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValueForCurrentFunction(EMaterialParameterType::SparseVolumeTexture, ParameterName, Meta))
	{
		DefaultTexture = Meta.Value.SparseVolumeTexture;
	}

	TextureReferenceIndex = Material->GetMatDefaultTextureIdx(DefaultTexture);
	checkf(TextureReferenceIndex != INDEX_NONE, TEXT("Material expression called Compiler->SparseVolumeTextureParameter() without implementing UMaterialExpression::GetReferencedTexture properly"));

	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;

	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionTextureParameter(ParameterInfo, TextureReferenceIndex, SamplerType), MCT_SparseVolumeTexture, TEXT(""));
}

int32 FHLSLMaterialTranslator::SparseVolumeTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type)
{
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionSparseVolumeTextureUniform(TextureIndex, VectorIndex), GetMaterialValueType(Type), TEXT(""));
}

int32 FHLSLMaterialTranslator::SparseVolumeTextureUniformParameter(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type)
{
	FMaterialParameterInfo ParameterInfo = GetParameterAssociationInfo();
	ParameterInfo.Name = ParameterName;
	FAddUniformExpressionScope Scope(this);
	return AddUniformExpression(Scope, new FMaterialUniformExpressionSparseVolumeTextureUniform(ParameterInfo, TextureIndex, VectorIndex), GetMaterialValueType(Type), TEXT(""));
}

static const TCHAR* GetSparseVolumeTextureAddressMode(TextureAddress Address)
{
	switch (Address)
	{
	case TA_Wrap: return TEXT("SVTADDRESSMODE_WRAP");
	case TA_Clamp: return TEXT("SVTADDRESSMODE_CLAMP");
	case TA_Mirror: return TEXT("SVTADDRESSMODE_MIRROR");
	default: checkNoEntry(); return nullptr;
	}
}

int32 FHLSLMaterialTranslator::SparseVolumeTextureSamplePageTable(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipLevelIndex, ESamplerSourceMode SamplerSource, bool bIsManualLinearMipMapSecondSample)
{
	if (SparseVolumeTextureIndex == INDEX_NONE || UVWIndex == INDEX_NONE || MipLevelIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	EMaterialValueType TextureType = GetParameterType(SparseVolumeTextureIndex);
	if ((TextureType & MCT_SparseVolumeTexture) == 0)
	{
		return Errorf(TEXT("FHLSLMaterialTranslator::SparseVolumeTextureSamplePageTable expects MCT_SparseVolumeTexture but was passed %s ERROR."), DescribeType(TextureType));
	}

	FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(SparseVolumeTextureIndex);
	if (UniformExpression == nullptr)
	{
		return Errorf(TEXT("Unable to find SVT uniform expression."));
	}
	FMaterialUniformExpressionTexture* TextureUniformExpression = UniformExpression->GetTextureUniformExpression();
	if (TextureUniformExpression == nullptr)
	{
		return Errorf(TEXT("The provided uniform expression is not a texture"));
	}

	int32 UVWAsFloat3Index = ForceCast(UVWIndex, MCT_Float3);
	int32 MipLevelAsFloatIndex = ForceCast(MipLevelIndex, MCT_Float1);

	// Make sure the SVT has been added to UniformTextureExpressions
	AccessUniformExpression(SparseVolumeTextureIndex);

	int32 SVTReferenceIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::SparseVolume].Find(TextureUniformExpression);
	check(UniformTextureExpressions[(uint32)EMaterialTextureParameterType::SparseVolume].IsValidIndex(SVTReferenceIndex));

	const USparseVolumeTexture* SVTexture = Cast<USparseVolumeTexture>(Material->GetMatDefaultTextureAtIdx(TextureUniformExpression->GetTextureIndex()));

	// StaticAddress mode at time of compile
	// Similar to VirtualTextures, this may not be 100% correct, if SamplerSource is set to 'SSM_FromTextureAsset', as texture parameter may change the address mode in a derived instance
	TextureAddress StaticAddressX = TA_Wrap;
	TextureAddress StaticAddressY = TA_Wrap;
	TextureAddress StaticAddressZ = TA_Wrap;
	switch (SamplerSource)
	{
	case SSM_FromTextureAsset:
		if (SVTexture)
		{
			StaticAddressX = SVTexture->GetTextureAddressX();
			StaticAddressY = SVTexture->GetTextureAddressY();
			StaticAddressZ = SVTexture->GetTextureAddressZ();
		}
		break;
	case SSM_Wrap_WorldGroupSettings:
		StaticAddressX = TA_Wrap;
		StaticAddressY = TA_Wrap;
		StaticAddressZ = TA_Wrap;
		break;
	case SSM_Clamp_WorldGroupSettings: // fallthrough
	case SSM_TerrainWeightmapGroupSettings:
		StaticAddressX = TA_Clamp;
		StaticAddressY = TA_Clamp;
		StaticAddressZ = TA_Clamp;
		break;
	default:
		checkNoEntry();
		break;
	}

	AddEstimatedTextureSample();
	const TCHAR* FunctionPostfix = bIsManualLinearMipMapSecondSample ? TEXT("SecondMipWrapper") : TEXT("");
	FString SampleCode = FString::Printf(TEXT("SparseVolumeTextureSamplePageTable%s(%s.SparseVolumeTexturePageTable_%d, %s, %s, %s, %s, %s, %s)"),
		FunctionPostfix,
		GetCurrentMaterialUniformBufferName(),
		SVTReferenceIndex, 
		*GetParameterCode(SparseVolumeTextureIndex), 
		*GetParameterCode(UVWAsFloat3Index), 
		GetSparseVolumeTextureAddressMode(StaticAddressX), 
		GetSparseVolumeTextureAddressMode(StaticAddressY), 
		GetSparseVolumeTextureAddressMode(StaticAddressZ), 
		*GetParameterCode(MipLevelAsFloatIndex));
	return AddCodeChunk(MCT_Float3, *SampleCode);
}

int32 FHLSLMaterialTranslator::SparseVolumeTextureSamplePhysicalTileData(int32 SparseVolumeTextureIndex, int32 VoxelCoordIndex, int32 PhysicalTileDataIdxIndex, bool bIsManualLinearMipMapSecondSample)
{
	if (SparseVolumeTextureIndex == INDEX_NONE || VoxelCoordIndex == INDEX_NONE || PhysicalTileDataIdxIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	EMaterialValueType TextureType = GetParameterType(SparseVolumeTextureIndex);
	if ((TextureType & MCT_SparseVolumeTexture) == 0)
	{
		return Errorf(TEXT("FHLSLMaterialTranslator::SparseVolumeTextureSamplePhysicalTileData expects MCT_SparseVolumeTexture but was passed %s ERROR."), DescribeType(TextureType));
	}

	FMaterialUniformExpression* UniformExpression = GetParameterUniformExpression(SparseVolumeTextureIndex);
	if (UniformExpression == nullptr)
	{
		return Errorf(TEXT("Unable to find SVT uniform expression."));
	}
	FMaterialUniformExpressionTexture* TextureUniformExpression = UniformExpression->GetTextureUniformExpression();
	if (TextureUniformExpression == nullptr)
	{
		return Errorf(TEXT("The provided uniform expression is not a texture"));
	}

	int32 VoxelCoordAsFloat3Index = ForceCast(VoxelCoordIndex, MCT_Float3);
	int32 IndexAsFloatIndex = ForceCast(PhysicalTileDataIdxIndex, MCT_Float1);

	// Make sure the SVT has been added to UniformTextureExpressions
	AccessUniformExpression(SparseVolumeTextureIndex);

	int32 SVTReferenceIndex = UniformTextureExpressions[(uint32)EMaterialTextureParameterType::SparseVolume].Find(TextureUniformExpression);
	check(UniformTextureExpressions[(uint32)EMaterialTextureParameterType::SparseVolume].IsValidIndex(SVTReferenceIndex));

	AddEstimatedTextureSample();
	const TCHAR* FunctionPostfix = bIsManualLinearMipMapSecondSample ? TEXT("SecondMipWrapper") : TEXT("");
	return AddCodeChunk(MCT_Float4, TEXT("SparseVolumeTextureSamplePhysicalTileData%s(%s.SparseVolumeTexturePhysicalA_%d, %s.SparseVolumeTexturePhysicalB_%d, %s.SparseVolumeTexturePhysical_%dSampler, %s, %s)"),
		FunctionPostfix,
		GetCurrentMaterialUniformBufferName(),
		SVTReferenceIndex,
		GetCurrentMaterialUniformBufferName(),
		SVTReferenceIndex,
		GetCurrentMaterialUniformBufferName(),
		SVTReferenceIndex,
		*GetParameterCode(VoxelCoordAsFloat3Index),
		*GetParameterCode(IndexAsFloatIndex));
}

int32 FHLSLMaterialTranslator::SparseVolumeTextureSample(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipValue0Index, int32 MipValue1Index, int32 PhysicalTileDataIdxIndex, ETextureMipValueMode MipValueMode, ESamplerSourceMode SamplerSource)
{
	if (MipValueMode == TMVM_Derivative)
	{
		if (MipValue0Index == INDEX_NONE)
		{
			return Errorf(TEXT("Missing DDX(UVs) parameter"));
		}
		else if (MipValue1Index == INDEX_NONE)
		{
			return Errorf(TEXT("Missing DDY(UVs) parameter"));
		}
		else if (!IsFloatNumericType(GetParameterType(MipValue0Index)))
		{
			return Errorf(TEXT("Invalid DDX(UVs) parameter"));
		}
		else if (!IsFloatNumericType(GetParameterType(MipValue1Index)))
		{
			return Errorf(TEXT("Invalid DDY(UVs) parameter"));
		}
	}
	else if (MipValueMode != TMVM_None && MipValue0Index != INDEX_NONE && !IsFloatNumericType(GetParameterType(MipValue0Index)))
	{
		return Errorf(TEXT("Invalid mip map parameter"));
	}

	int32 MipLevelIndex = INDEX_NONE;
	int32 MipLevelBiasIndex = Constant(0.0f);
	int32 DDXIndex = INDEX_NONE;
	int32 DDYIndex = INDEX_NONE;
	const bool bDeriveMipLevel = (MipValueMode == TMVM_None || MipValueMode == TMVM_MipBias || MipValueMode == TMVM_Derivative);

	if (MipValueMode == TMVM_None || MipValueMode == TMVM_MipBias)
	{
		if (MipValueMode == TMVM_MipBias)
		{
			MipLevelBiasIndex = MipValue0Index;
		}

		DDXIndex = DDX(UVWIndex);
		DDYIndex = DDY(UVWIndex);
	}
	else if (MipValueMode == TMVM_MipLevel)
	{
		MipLevelIndex = MipValue0Index;
	}
	else if (MipValueMode == TMVM_Derivative)
	{
		DDXIndex = MipValue0Index;
		DDYIndex = MipValue1Index;
	}
	else
	{
		checkNoEntry();
	}

	if (bDeriveMipLevel)
	{
		MipLevelIndex = AddCodeChunk(MCT_Float1, TEXT("SparseVolumeTextureCalculateMipLevel(%s, %s, %s, %s, %s)"), 
			*GetParameterCode(SparseVolumeTextureIndex), 
			*GetParameterCode(DDXIndex), 
			*GetParameterCode(DDYIndex), 
			*GetParameterCode(MipLevelBiasIndex),
			TEXT("Parameters.SvPosition.xy"));
	}

	// Sample the first mip
	int32 MipLevel0Index = Floor(MipLevelIndex);
	int32 VoxelCoordMip0Index = SparseVolumeTextureSamplePageTable(SparseVolumeTextureIndex, UVWIndex, MipLevel0Index, SamplerSource, false /*bIsManualLinearMipMapSecondSample*/);
	int32 Mip0SampleIndex = SparseVolumeTextureSamplePhysicalTileData(SparseVolumeTextureIndex, VoxelCoordMip0Index, PhysicalTileDataIdxIndex, false /*bIsManualLinearMipMapSecondSample*/);

	// Sample the second mip
	int32 MipLevel1Index = Ceil(MipLevelIndex);
	int32 VoxelCoordMip1Index = SparseVolumeTextureSamplePageTable(SparseVolumeTextureIndex, UVWIndex, MipLevel1Index, SamplerSource, true /*bIsManualLinearMipMapSecondSample*/);
	int32 Mip1SampleIndex = SparseVolumeTextureSamplePhysicalTileData(SparseVolumeTextureIndex, VoxelCoordMip1Index, PhysicalTileDataIdxIndex, true /*bIsManualLinearMipMapSecondSample*/);

	// Lerp
	int32 LerpAlphaIndex = Frac(MipLevelIndex);
	return AddCodeChunk(MCT_Float4, TEXT("SparseVolumeTextureCombineMipSamples(%s, %s, %s)"), *GetParameterCode(Mip0SampleIndex), *GetParameterCode(Mip1SampleIndex), *GetParameterCode(LerpAlphaIndex));
}

#endif // WITH_EDITORONLY_DATA
