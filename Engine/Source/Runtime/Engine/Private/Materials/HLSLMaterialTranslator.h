// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================
	HLSLMaterialTranslator.h: Translates material expressions into HLSL code.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Algo/Transform.h"
#include "Misc/Guid.h"
#include "HAL/IConsoleManager.h"
#include "ShaderParameters.h"
#include "StaticParameterSet.h"
#include "MaterialShared.h"
#include "Stats/StatsMisc.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionAbsorptionMediumMaterialOutput.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "MaterialCompiler.h"
#include "RenderUtils.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Hash/CityHash.h"
#include "VT/RuntimeVirtualTexture.h"
#include "Field/FieldSystemTypes.h"
#include "Containers/Map.h"
#include "Shader/ShaderTypes.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Runtime/RenderCore/Internal/ShaderCompilerDefinitions.h"
#include "Runtime/Engine/Internal/MaterialCache/MaterialCacheAttribute.h"

#if WITH_EDITORONLY_DATA
#include "Materials/MaterialExpressionSceneTexture.h"
#include "Materials/MaterialExpressionUserSceneTexture.h"
#include "Materials/MaterialExpressionNoise.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionVertexInterpolator.h"
#include "Materials/MaterialUniformExpressions.h"
#include "ParameterCollection.h"
#include "Materials/MaterialParameterCollection.h"
#include "Containers/LazyPrintf.h"
#include "Containers/HashTable.h"
#include "Engine/Texture2D.h"
#include "HLSLMaterialDerivativeAutogen.h"
#include "SubstrateTranslatorCommon.h"
#endif

class Error;
class FAddUniformExpressionScope;
class FShaderKeyGenerator;

namespace UE::DerivedData
{
	class FRequestOwner;
}

#if WITH_EDITORONLY_DATA

enum EMaterialExpressionVisitResult
{
	MVR_CONTINUE,
	MVR_STOP,
};

class IMaterialExpressionVisitor
{
public:
	virtual ~IMaterialExpressionVisitor() = default;
	virtual EMaterialExpressionVisitResult Visit(UMaterialExpression* InExpression) = 0;
};

struct FShaderCodeChunk
{
	/**
	 * Hash of the code chunk, used to determine equivalent chunks created from different expressions
	 * By default this is simply the hash of the code string
	 */
	uint64 Hash;


	uint64 MaterialAttributeMask;

	/** 
	 * Definition string of the code chunk. 
	 * If !bInline && !UniformExpression || UniformExpression->IsConstant(), this is the definition of a local variable named by SymbolName.
	 * Otherwise if bInline || (UniformExpression && UniformExpression->IsConstant()), this is a code expression that needs to be inlined.
	 * This string uses hardware finite differences.
	 */
	FString DefinitionFinite;
	/** 
	* Definition string of the code chunk, but with analytic partial derivatives.
	*/
	FString DefinitionAnalytic;
	/** 
	 * Name of the local variable used to reference this code chunk. 
	 * If bInline || UniformExpression, there will be no symbol name and Definition should be used directly instead.
	 */
	FString SymbolName;
	/** Reference to a uniform expression, if this code chunk has one. */
	TRefCountPtr<FMaterialUniformExpression> UniformExpression;

	/** All the chunks that are scoped under this chunk.  This is populated once translation is complete, rather than on-the-fly, as a chunk's parent scope may change during translation */
	TArray<int32> ScopedChunks;

	TArray<int32> ReferencedCodeChunks;

	EMaterialValueType Type;

	int32 DeclaredScopeIndex;
	int32 UsedScopeIndex;
	int32 ScopeLevel;

	/** Whether the code chunk should be inlined or not.  If true, SymbolName is empty and Definition contains the code to inline. */
	bool bInline;
	bool bIntermediate;
	/** The status of partial derivatives at this expression. **/
	EDerivativeStatus DerivativeStatus;

	FString& AtDefinition(ECompiledPartialDerivativeVariation Variation)
	{
		checkf(Variation == CompiledPDV_FiniteDifferences || Variation == CompiledPDV_Analytic, TEXT("Invalid partial derivative variation: %d"), Variation);
		return Variation == CompiledPDV_FiniteDifferences ? DefinitionFinite : DefinitionAnalytic;
	}

	const FString& AtDefinition(ECompiledPartialDerivativeVariation Variation) const
	{
		checkf(Variation == CompiledPDV_FiniteDifferences || Variation == CompiledPDV_Analytic, TEXT("Invalid partial derivative variation: %d"), Variation);
		return Variation == CompiledPDV_FiniteDifferences ? DefinitionFinite : DefinitionAnalytic;
	}

	/** Ctor for creating a new code chunk with no associated uniform expression. */
	FShaderCodeChunk(uint64 InHash, const TCHAR* InDefinitionFinite, const TCHAR* InDefinitionAnalytic, const FString& InSymbolName, EMaterialValueType InType, EDerivativeStatus InDerivativeStatus, bool bInInline):
		Hash(InHash),
		MaterialAttributeMask(0u),
		DefinitionFinite(InDefinitionFinite),
		DefinitionAnalytic(InDefinitionAnalytic),
		SymbolName(InSymbolName),
		UniformExpression(NULL),
		Type(InType),
		DeclaredScopeIndex(INDEX_NONE),
		UsedScopeIndex(INDEX_NONE),
		ScopeLevel(0),
		bInline(bInInline),
		bIntermediate(false),
		DerivativeStatus(InDerivativeStatus)
	{}

	/** Ctor for creating a new code chunk with a uniform expression. */
	FShaderCodeChunk(uint64 InHash, FMaterialUniformExpression* InUniformExpression, const TCHAR* InDefinitionFinite, const TCHAR* InDefinitionAnalytic, EMaterialValueType InType, EDerivativeStatus InDerivativeStatus):
		Hash(InHash),
		MaterialAttributeMask(0u),
		DefinitionFinite(InDefinitionFinite),
		DefinitionAnalytic(InDefinitionAnalytic),
		UniformExpression(InUniformExpression),
		Type(InType),
		DeclaredScopeIndex(INDEX_NONE),
		UsedScopeIndex(INDEX_NONE),
		ScopeLevel(0),
		bInline(false),
		bIntermediate(false),
		DerivativeStatus(InDerivativeStatus)
	{}
};

struct FMaterialCustomExpressionEntry
{
	uint64 ScopeID;
	const UMaterialExpressionCustom* Expression;
	FString Implementation;
	TArray<uint64> InputHash;
	TArray<int32> OutputCodeIndex;
};


struct FMaterialDerivativeVariation
{
	/** Code chunk definitions corresponding to each of the material inputs, only initialized after Translate has been called. */
	FString TranslatedCodeChunkDefinitions[CompiledMP_MAX];

	/** Code chunks corresponding to each of the material inputs, only initialized after Translate has been called. */
	FString TranslatedCodeChunks[CompiledMP_MAX];

	/** Any custom output function implementations */
	TArray<FString> CustomOutputImplementations;
};

struct FMaterialLocalVariableEntry
{
	FString Name;
	int32 DeclarationCodeIndex = INDEX_NONE;
};

enum class EMaterialCastFlags : uint32
{
	None = 0u,
	ReplicateScalar = (1u << 0),
	AllowTruncate = (1u << 1),
	AllowAppendZeroes = (1u << 2),
	AllowInteger = (1u << 3),

	ValidCast = ReplicateScalar | AllowTruncate,
};
ENUM_CLASS_FLAGS(EMaterialCastFlags);

/** 
 * Enumerates Translate() results.
 * If RetryWithoutDDC is returned, the caller should call Translate() again
 * on a new instance of FHLSLMaterialTranslator forcing DDC query off (i.e. call
 * Translate(false)). This is because an instance of FHLSLMaterialTranslator
 * is single use.
 */
enum class EHLSLMaterialTranslatorResult
{
	Success,
	Failure,
	RetryWithoutDDC,
};

// Specify which shader frequency an entry appears in
using FShaderFrequencyMask = uint64;

class FHLSLMaterialTranslator : public FMaterialCompiler
{
	friend class FMaterialDerivativeAutogen;
protected:
	struct FMaterialVTSampleInfo
	{
		bool operator==(const FMaterialVTSampleInfo&) const = default;
		
		int32 VirtualTextureIndex = INDEX_NONE;
		ETextureMipValueMode MipValueMode = TMVM_None;
		TextureAddress AddressU = TA_Clamp;
		TextureAddress AddressV = TA_Clamp;
		int32 CoordinateIndex = INDEX_NONE;
		int32 MipValue0Index = INDEX_NONE;
		int32 MipValue1Index = INDEX_NONE;
		int32 PreallocatedStackTextureIndex = INDEX_NONE;
		int32 PageTableIndex = INDEX_NONE;
		int32 PageTableUniformIndex = INDEX_NONE;
		bool bAdaptive = false;
		bool bGenerateFeedback = false;
		bool bMeshPaint = false;
		bool bMaterialCache = false;
		bool bCollection = false;
		float AspectRatio = 0.0f;
		int32 CodeIndex = INDEX_NONE;
	};

	/** Translator specific structure during generation of VT stacks. This is later converted to FMaterialVirtualTextureStack entries in FUniformExpressionSet. */
	struct FMaterialVTStackEntry
	{
		FMaterialVTSampleInfo SampleInfo;
		uint64 ScopeID;
		uint64 CoordinateHash;
		uint64 MipValue0Hash;
		uint64 MipValue1Hash;
		uint64 PageTableHash;
		uint64 PageTableUniformHash;
		int32 CodeIndex;
		EShaderFrequency ShaderFrequency;
	};

	/** Discovered tags during translation */
	struct FMaterialCacheTag
	{
		/** All runtime layers of this tag */
		TArray<FMaterialCacheLayer> Layers;

		/** Allocated stack index of this tag */
		int32 StackIndex = 0;
	};

	/** Data that is different for the finite and anlytical partial derivative options. **/
	FMaterialDerivativeVariation DerivativeVariations[CompiledPDV_MAX];

	/** The shader frequency of the current material property being compiled. */
	EShaderFrequency ShaderFrequency;
	/** The current material property being compiled.  This affects the behavior of all compiler functions except GetFixedParameterCode. */
	EMaterialProperty MaterialProperty;
	/** Stack of currently compiling material attributes*/
	TArray<FGuid> MaterialAttributesStack;
	/** Stack of currently compiling material parameter owners*/
	TArray<FMaterialParameterInfo> ParameterOwnerStack;
	/** The code chunks corresponding to the currently compiled property or custom output. */
	TArray<FShaderCodeChunk>* CurrentScopeChunks;
	uint64 CurrentScopeID;
	uint64 NextTempScopeID;

	UMaterialExpressionCustomOutput* TopCustomOutput = nullptr;

	// List of Shared pixel properties. Used to share generated code
	bool SharedPixelProperties[CompiledMP_MAX];

	/** Stores the resource declarations */
	FString ResourcesString;

	/* Stack that tracks compiler state specific to the function currently being compiled. */
	TArray<FMaterialFunctionCompileState*> FunctionStacks[SF_NumFrequencies];

	/** Material being compiled.  Only transient compilation output like error information can be stored on the FMaterial. */
	FMaterial* Material;

	FStaticParameterSet StaticParameters;
	FMaterialLayersFunctions CachedMaterialLayers;
	EShaderPlatform Platform;
	/** Quality level being compiled for. */
	EMaterialQualityLevel::Type QualityLevel;

	/** Feature level being compiled for. */
	ERHIFeatureLevel::Type FeatureLevel;

	FString TranslatedAttributesCodeChunks[SF_NumFrequencies];

	uint64 MaterialAttributesReturned[SF_NumFrequencies];

	TArray<int32> ScopeStack;

	TArray<int32> ReferencedCodeChunks;

	// Array of code chunks per material property
	TArray<FShaderCodeChunk> SharedPropertyCodeChunks[SF_NumFrequencies];

	TMap<const UMaterialExpression*, int32> ForLoopMap[SF_NumFrequencies];
	int32 NumForLoops[SF_NumFrequencies];

	TMap<FName, FMaterialLocalVariableEntry> LocalVariables[SF_NumFrequencies];

	struct FUniformExpressionInPreshader
	{
		TRefCountPtr<FMaterialUniformExpression> UniformExpression;
		EMaterialValueType CodeChunkMaterialValueType = (EMaterialValueType)0;
	};

	// Uniform expressions used across all material properties
	TArray<FShaderCodeChunk> UniformExpressions;
	TArray<TRefCountPtr<FMaterialUniformExpressionTexture> > UniformTextureExpressions[NumMaterialTextureParameterTypes];
	TArray<TRefCountPtr<FMaterialUniformExpressionTextureCollection>> UniformTextureCollectionExpressions;
	TArray<TRefCountPtr<FMaterialUniformExpressionExternalTexture>> UniformExternalTextureExpressions;
	TArray<FUniformExpressionInPreshader> UniformExpressionShaderFrequencies;

	TMap<UE::Shader::FValue, uint32> DefaultUniformValues;
	uint32 UniformPreshaderOffset = 0u;
	uint32 UniformPreshaderOffsetVS = 0u;
	uint32 CurrentBoolUniformOffset = ~0u;
	uint32 CurrentNumBoolComponents = 32u;

	/** Parameter collections referenced by this material.  The position in this array is used as an index on the shader parameter. */
	TArray<UMaterialParameterCollection*> ParameterCollections;

	// Index of the next symbol to create
	int32 NextSymbolIndex;

	/** Any custom expression function implementations */
	//TArray<FString> CustomExpressionImplementations;
	//TMap<UMaterialExpressionCustom*, int32> CachedCustomExpressions;
	TArray<FMaterialCustomExpressionEntry> CustomExpressions;

	/** Custom vertex interpolators */
	TArray<UMaterialExpressionVertexInterpolator*> CustomVertexInterpolators;
	/** Index to assign to next vertex interpolator. */
	int32 NextVertexInterpolatorIndex;
	/** Current float-width offset for custom vertex interpolators */
	int32 CurrentCustomVertexInterpolatorOffset;

	/** VT Stacks */
	TArray<FMaterialVTStackEntry> VTStacks;
	
	/** All material tags discovered */
	TMap<FGuid, FMaterialCacheTag> MaterialCacheTags;

	/** Used by interpolator pre-translation to hold potential errors until actually confirmed. */
	TArray<FString>* CompileErrorsSink;
	TArray<TObjectPtr<UMaterialExpression>>* CompileErrorExpressionsSink;

	/** Keeps track of which variations of analytic derivative functions are used, and generates the code during translation. **/
	FMaterialDerivativeAutogen DerivativeAutogen;

	/** The translation result. */
	EHLSLMaterialTranslatorResult TranslationResult;
	/** Whether the compute shader material inputs were compiled. */
	uint32 bCompileForComputeShader : 1;
	/** Whether the compiled material uses scene depth. */
	uint32 bUsesSceneDepth : 1;
	/** true if the material needs particle position. */
	uint32 bNeedsParticlePosition : 1;
	/** true if the material needs particle velocity. */
	uint32 bNeedsParticleVelocity : 1;
	/** true if the material needs particle relative time. */
	uint32 bNeedsParticleTime : 1;
	/** true if the material uses particle motion blur. */
	uint32 bUsesParticleMotionBlur : 1;
	/** true if the material needs particle random value. */
	uint32 bNeedsParticleRandom : 1;
	/** true if the material uses spherical particle opacity. */
	uint32 bUsesSphericalParticleOpacity : 1;
	/** true if the material uses particle sub uvs. */
	uint32 bUsesParticleSubUVs : 1;
	/** Boolean indicating using LightmapUvs */
	uint32 bUsesLightmapUVs : 1;
	/** Whether the material uses AO Material Mask */
	uint32 bUsesAOMaterialMask : 1;
	/** true if needs SpeedTree code */
	uint32 bUsesSpeedTree : 1;
	/** Boolean indicating the material uses worldspace position without shader offsets applied */
	uint32 bNeedsWorldPositionExcludingShaderOffsets : 1;
	/** true if the material needs particle size. */
	uint32 bNeedsParticleSize : 1;
	/** true if the material needs particle sprite rotation. */
	uint32 bNeedsParticleSpriteRotation : 1;
	/** true if any scene texture expressions are reading from post process inputs */
	uint32 bNeedsSceneTexturePostProcessInputs : 1;
	/** true if any atmospheric fog expressions are used */
	uint32 bUsesAtmosphericFog : 1;
	/** true if any SkyAtmosphere expressions are used */
	uint32 bUsesSkyAtmosphere : 1;
	/** true if the material reads vertex color in the pixel shader. */
	uint32 bUsesVertexColor : 1;
	/** true if the material reads particle color in the pixel shader. */
	uint32 bUsesParticleColor : 1;
	/** true if the material reads mesh particle local to world in the pixel shader. */
	uint32 bUsesParticleLocalToWorld : 1;
	/** true if the material reads mesh particle world to local in the pixel shader. */
	uint32 bUsesParticleWorldToLocal : 1;

	/** true if the material reads per instance local to world in the pixel shader. */
	uint32 bUsesInstanceLocalToWorldPS : 1;
	/** true if the material reads per instance world to local in the pixel shader. */
	uint32 bUsesInstanceWorldToLocalPS : 1;
	/** true if the material uses per instance random in the pixel shader. */
	uint32 bUsesPerInstanceRandomPS : 1;

	/** true if the material uses any type of vertex position */
	uint32 bUsesVertexPosition : 1;
	/** true if the material potentially manipulates any type of TexCoord for texture sampling */
	uint32 bPotentiallyManipulateTexCoords : 1;

	uint32 bUsesTransformVector : 1;
	// True if the current property requires last frame's information
	uint32 bCompilingPreviousFrame : 1;
	/** True if material will output accurate velocities during base pass rendering. */
	uint32 bOutputsBasePassVelocities : 1;
	uint32 bUsesPixelDepthOffset : 1;
	uint32 bUsesWorldPositionOffset : 1;
	uint32 bUsesDisplacement : 1;
	uint32 bUsesEmissiveColor : 1;
	uint32 bUsesDistanceCullFade : 1;
	/** true if the Roughness input evaluates to a constant 1.0 */
	uint32 bIsFullyRough : 1;
	/** true if allowed to generate code chunks. Translator operates in two phases; generate all code chunks & query meta data based on generated code chunks. */
	uint32 bAllowCodeChunkGeneration : 1;

	/** True if this material write anisotropy material property */
	uint32 bUsesAnisotropy : 1;

	/** True if the opacity input is plugged in */
	uint32 bOpacityPropertyIsUsed : 1;
	
	uint32 bUsesCurvature : 1;
	/** true if PerInstanceFadeAmount expression is used */
	uint32 bUsesPerInstanceFadeAmount : 1;

	uint32 bCullIntermediateUniformExpressions : 1;
	
	/** The generated shader source uses explicit ddx()/ddy() calls */
	uint32 bUsesExplicitDerivatives : 1;

	/** The material uses the first person custom output to interpolate between world space and first person. */
	uint32 bUsesFirstPersonInterpolation : 1;

	/** 
	  * Tracks if any virtual texture samples are used in the subgraph for the normal property.
	  * This is read later to know if the PixelNormalWS node is "connected" to virtual texture samples.
	  */
	uint32 bUsesVirtualTextureSampleForNormalProperty : 1;
	/** Flag that tracks if we are compiling a runtime virtual texture output subgraph. */
	uint32 bIsInRuntimeVirtualTextureOutput : 1;

	/** Incremented and decremented by FAddUniformExpressionScope.  See comments on that class below */
	int32 AddingUniformExpression;

	/** Tracks the texture coordinates used by this material. */
	TBitArray<> AllocatedUserTexCoords;
	/** Tracks the texture coordinates used by the vertex shader in this material. */
	TBitArray<> AllocatedUserVertexTexCoords;

	uint32 DynamicParticleParameterMask;

	/** Will contain all the shading models picked up from the material expression graph */
	FMaterialShadingModelField ShadingModelsFromCompilation;

	struct FEnvironmentDefines;

	/** Substrate data for a Translator*/
	Substrate::FSubstrateTranslatorData SubstrateTranslatorData;

	/** Tracks the total number of vt samples in the shader. */
	uint32 NumVtSamples;

	const ITargetPlatform* TargetPlatform;
	
	/** The DDC key for the material translation results */
	FIoHash DDCKeyHash;

public: 
	/**
	 * Returns a string representation that identify the translator version. Used to keep shader DDC keys valid when material translation internals change.
	 */
	static void AppendVersion(FShaderKeyGenerator& KeyGen, EShaderPlatform Platform);

	FHLSLMaterialTranslator(FMaterial* InMaterial,
		FMaterialCompilationOutput& InMaterialCompilationOutput,
		const FStaticParameterSet& InStaticParameters,
		EShaderPlatform InPlatform,
		EMaterialQualityLevel::Type InQualityLevel,
		ERHIFeatureLevel::Type InFeatureLevel,
		const ITargetPlatform* InTargetPlatform = nullptr,
		const FSubstrateCompilationConfig* InSubstrateCompilationConfig = nullptr,
		const FString& MaterialTranslationDDCKeyString = {});

	~FHLSLMaterialTranslator();

	bool ShouldStopTranslating() const override;

	int32 GetNumUserTexCoords() const;
	int32 GetNumUserVertexTexCoords() const;

	void ClearAllFunctionStacks();
	void ClearFunctionStack(uint32 Frequency);

	void AssignTempScope(TArray<FShaderCodeChunk>& InScope);
	void AssignShaderFrequencyScope(EShaderFrequency InShaderFrequency);

	template<typename ExpressionsArrayType>
	void GatherCustomVertexInterpolators(const ExpressionsArrayType& Expressions);

	void CompileCustomOutputs(TArray<UMaterialExpressionCustomOutput*>& CustomOutputExpressions, TSet<UClass*>& SeenCustomOutputExpressionsClasses, bool bIsBeforeAttributes);
	uint64 CompileMaterialAttributesCustomOutputs(uint64 CustomOutputNodeBitmask);

	template<typename ExpressionsArrayType>
	EMaterialExpressionVisitResult VisitExpressionsRecursive(const ExpressionsArrayType& Expressions, IMaterialExpressionVisitor& InVisitor);

	EMaterialExpressionVisitResult VisitExpressionsForProperty(EMaterialProperty InProperty, IMaterialExpressionVisitor& InVisitor);

	void ValidateVtPropertyLimits();
	void ValidateShadingModelsForFeatureLevel(const FMaterialShadingModelField& ShadingModels);

	EHLSLMaterialTranslatorResult Translate(bool bForceDisableDDCQuery);

	void GetMaterialEnvironment(EShaderPlatform InPlatform, FShaderCompilerEnvironment& OutEnvironment);
	
	// Assign custom interpolators to slots, packing them as much as possible in unused slots.
	TBitArray<> GetVertexInterpolatorsOffsets(FString& VertexInterpolatorsOffsetsDefinitionCode, TArray<uint32>& UsedTexCoordIndices) const;

	void GetSharedInputsMaterialCode(FString& PixelMembersDeclaration, FString& NormalAssignment, FString& PixelMembersInitializationEpilog, ECompiledPartialDerivativeVariation DerivativeVariation);

	FString GetMaterialShaderCode();

	const FShaderCodeChunk& AtParameterCodeChunk(int32 Index) const;
	EDerivativeStatus GetDerivativeStatus(int32 Index) const;
	FDerivInfo GetDerivInfo(int32 Index, bool bAllowNonFloat = false);

protected:
	bool GetParameterOverrideValueForCurrentFunction(EMaterialParameterType ParameterType, FName ParameterName, FMaterialParameterMetadata& OutResult) const;

	bool IsMaterialPropertyUsed(EMaterialProperty Property, int32 PropertyChunkIndex, const FLinearColor& ReferenceValue, int32 NumComponents) const;

	// only used by GetMaterialShaderCode()
	// @param Index ECompiledMaterialProperty or EMaterialProperty
	FString GenerateFunctionCode(uint32 Index, ECompiledPartialDerivativeVariation Variation) const;

	// GetParameterCode, with DERIV_BASE_VALUE if necessary
	virtual FString GetParameterCode(int32 Index, const TCHAR* Default = 0);

	// GetParameterCode, no DERIV_BASE_VALUE
	virtual FString GetParameterCodeRaw(int32 Index, const TCHAR* Default = 0);

public:
	// Must always be valid
	virtual FString GetParameterCodeDeriv(int32 Index, ECompiledPartialDerivativeVariation Variation);

protected:
	uint64 GetParameterHash(int32 Index);

	uint64 GetParameterMaterialAttributeMask(int32 Index);
	void SetParameterMaterialAttributes(int32 Index, uint64 Mask);

	/** Creates a string of all definitions needed for the given material input. */
	FString GetDefinitions(const TArray<FShaderCodeChunk>& CodeChunks, int32 StartChunk, int32 EndChunk, ECompiledPartialDerivativeVariation Variation, const TCHAR* ReturnValueSymbolName = nullptr) const;

	// GetFixedParameterCode
	void GetFixedParameterCode(int32 StartChunk, int32 EndChunk, int32 ResultIndex, TArray<FShaderCodeChunk>& CodeChunks, FString& OutDefinitions, FString& OutValue, ECompiledPartialDerivativeVariation Variation, bool bReduceAfterReturnValue = false);

	void LinkParentScopes(TArray<FShaderCodeChunk>& CodeChunks);

	void GetScopeCode(int32 IndentLevel, int32 ScopeChunkIndex, const TArray<FShaderCodeChunk>& CodeChunks, TSet<int32>& EmittedChunks, FString& OutValue);

	void GetFixedParameterCode(int32 ResultIndex, TArray<FShaderCodeChunk>& CodeChunks, FString& OutDefinitions, FString& OutValue, ECompiledPartialDerivativeVariation Variation);

	/** Used to get a user friendly type from EMaterialValueType */
	const TCHAR* DescribeType(EMaterialValueType Type) const;

	/** Used to get an HLSL type from EMaterialValueType */
	const TCHAR* HLSLTypeString(EMaterialValueType Type) const;

	/** Used to get an HLSL type from EMaterialValueType */
	const TCHAR* HLSLTypeStringDeriv(EMaterialValueType Type, EDerivativeStatus DerivativeStatus) const;

	int32 NonPixelShaderExpressionError();

	int32 ErrorUnlessFeatureLevelSupported(ERHIFeatureLevel::Type RequiredFeatureLevel);
	int32 ErrorUnlessPlatformSupports(const bool (*SupportFunction)(const FStaticShaderPlatform Platform), const TCHAR* ConditionString);

	int32 NonVertexShaderExpressionError();
	int32 NonVertexOrPixelShaderExpressionError();

	void AddEstimatedTextureSample(const uint32 Count = 1);
	void AddLWCFuncUsage(ELWCFunctionKind Kind, const uint32 Count = 1);

	/** If InWorldPosition is set, coerce it to the correct type, else get the position from the material params */
	FString GetWorldPositionOrDefault(int32 InWorldPosition, EPositionOrigin PositionOrigin);

	/** Creates a unique symbol name and adds it to the symbol list. */
	FString CreateSymbolName(const TCHAR* SymbolNameHint);

	void AddCodeChunkToCurrentScope(int32 ChunkIndex);
	void AddCodeChunkToScope(int32 ChunkIndex, int32 ScopeIndex);

	/** Adds an already formatted inline or referenced code chunk */
	int32 AddCodeChunkInner(uint64 Hash, const TCHAR* FormattedType, const TCHAR* FormattedCode, EMaterialValueType Type, EDerivativeStatus DerivativeStatus, bool bInlined);
	int32 AddCodeChunkInner(uint64 Hash, const TCHAR* FormattedCode, EMaterialValueType Type, EDerivativeStatus DerivativeStatus, bool bInlined);

	/** Adds an already formatted inline or referenced code chunk */
	int32 AddInternalCodeChunk(EMaterialValueType Type, const TCHAR* InternalType, const TCHAR* FormattedCode, const TRefCountPtr<FMaterialUniformExpression>& UniformExpression = nullptr);

public:
	/** Adds an already formatted inline or referenced code chunk, and notes the derivative status. */
	int32 AddCodeChunkInnerDeriv(const TCHAR* FormattedCodeFinite, const TCHAR* FormattedCodeAnalytic, EMaterialValueType Type, bool bInlined, EDerivativeStatus DerivativeStatus);
	int32 AddCodeChunkInnerDeriv(const TCHAR* FormattedCode, EMaterialValueType Type, bool bInlined, EDerivativeStatus DerivativeStatus);

	FString CastValue(const FString& Code, EMaterialValueType SourceType, EMaterialValueType DestType, EMaterialCastFlags Flags);

	// CoerceParameter
	FString CoerceParameter(int32 Index, EMaterialValueType DestType, EMaterialCastFlags AdditionalCastFlags = EMaterialCastFlags::None);
	FString CoerceValue(const FString& Code, EMaterialValueType SourceType, EMaterialValueType DestType, EMaterialCastFlags AdditionalCastFlags = EMaterialCastFlags::None);

	int32 CastToNonLWCIfDisabled(int32 Code);

	EMaterialValueType GetArithmeticResultType(int32 A, int32 B);

protected:

	/** 
	 * Constructs the formatted code chunk and creates a new local variable definition from it. 
	 * This should be used over AddInlinedCodeChunk when the code chunk adds actual instructions, and especially when calling a function.
	 * Creating local variables instead of inlining simplifies the generated code and reduces redundant expression chains,
	 * Making compiles faster and enabling the shader optimizer to do a better job.
	 */
	
	int32 AddCodeChunkInner(EMaterialValueType Type, const TCHAR* FormattedType, EDerivativeStatus DerivativeStatus, bool bInlined, const TCHAR* Format, ...);
	int32 AddCodeChunkInner(EMaterialValueType Type, EDerivativeStatus DerivativeStatus, bool bInlined, const TCHAR* Format, ...);
	int32 AddCodeChunkWithHash(uint64 BaseHash, EMaterialValueType Type, const TCHAR* Format, ...);

	template <typename... Types>
	int32 AddCodeChunk(EMaterialValueType Type, const TCHAR* Format, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to AddCodeChunk");
		return AddCodeChunkInner(Type, EDerivativeStatus::NotAware, false, Format, Args...);
	}

	template <typename... Types>
	int32 AddCodeChunkZeroDeriv(EMaterialValueType Type, const TCHAR* Format, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to AddCodeChunkZeroDeriv");
		return AddCodeChunkInner(Type, EDerivativeStatus::Zero, false, Format, Args...);
	}

	template <typename... Types>
	int32 AddCodeChunkFiniteDeriv(EMaterialValueType Type, const TCHAR* Format, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to AddCodeChunkFiniteDeriv");
		return AddCodeChunkInner(Type, EDerivativeStatus::NotValid, false, Format, Args...);
	}
	

	/** 
	 * Constructs the formatted code chunk and creates an inlined code chunk from it. 
	 * This should be used instead of AddCodeChunk when the code chunk does not add any actual shader instructions, for example a component mask.
	 */
	template <typename... Types>
	int32 AddInlinedCodeChunk(EMaterialValueType Type, const TCHAR* Format, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to AddInlinedCodeChunk");
		return AddCodeChunkInner(Type, EDerivativeStatus::NotAware, true, Format, Args...);
	}

	template <typename... Types>
	int32 AddInlinedCodeChunkZeroDeriv(EMaterialValueType Type, const TCHAR* Format, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to AddInlinedCodeChunkZeroDeriv");
		return AddCodeChunkInner(Type, EDerivativeStatus::Zero, true, Format, Args...);
	}

	template <typename... Types>
	int32 AddInlinedCodeChunkFiniteDeriv(EMaterialValueType Type, const TCHAR* Format, Types... Args)
	{
		static_assert((TIsValidVariadicFunctionArg<Types>::Value && ...), "Invalid argument(s) passed to AddInlinedCodeChunkFiniteDeriv");
		return AddCodeChunkInner(Type, EDerivativeStatus::NotValid, true, Format, Args...);
	}

	// Constructs a code chunk that comes from an external HLSL function. Its return type and function implementation is data-driven and defined in BaseEditor.ini.
	int32 AddExternalCodeChunk(const FName& InExternalCodeIdentifier);
	int32 AddExternalCodeChunk(const FMaterialExternalCodeDeclaration& InExternalCode);

	int32 AddUniformExpressionInner(uint64 Hash, FMaterialUniformExpression* UniformExpression, EMaterialValueType Type, const TCHAR* FormattedCode);

	// AddUniformExpression - Adds an input to the Code array and returns its index.
	int32 AddUniformExpression(FAddUniformExpressionScope& Scope, FMaterialUniformExpression* UniformExpression, EMaterialValueType Type, const TCHAR* Format, ...);

	// AccessUniformExpression - Adds code to access the value of a uniform expression to the Code array and returns its index.
	int32 AccessUniformExpression(int32 Index);

	int32 AccessMaterialAttribute(int32 CodeIndex, const FGuid& AttributeID);

	const TCHAR* GetCurrentMaterialUniformBufferName() const;

	// GetParameterType
	virtual EMaterialValueType GetParameterType(int32 Index) const override;

	// GetParameterUniformExpression
	virtual FMaterialUniformExpression* GetParameterUniformExpression(int32 Index) const override;

	virtual bool GetTextureForExpression(int32 Index, int32& OutTextureIndex, EMaterialSamplerType& OutSamplerType, TOptional<FName>& OutParameterName) const override;
	virtual bool GetTextureCollectionForExpression(int32 Index, int32& OutTextureCollectionIndex, TOptional<FName>& OutParameterName) const override;

	// GetArithmeticResultType
	EMaterialValueType GetArithmeticResultType(EMaterialValueType TypeA, EMaterialValueType TypeB);

	// Same as GetConstParameterValue, but ensures that the value that comes out has zero in components that are not included in EMaterialValueType
	FLinearColor GetTypeMaskedValue(EMaterialValueType Type, FLinearColor ConstValue, bool* OutSuccess);

	// Same as GetConstParameterValue, but ensures that the value that comes out has zero in components that are not included in EMaterialValueType
	bool GetConstMaskedParameterValue(EMaterialValueType Type, FMaterialUniformExpression* Expression, FLinearColor& OutConstValue);

	// Attempts to fetch the constant value for a parameter code.
	// returns true if the value was a valid const expression
	// OutConstValue will only get filled in if it is valid
	bool GetConstParameterValue(FMaterialUniformExpression* Expression, FLinearColor& OutConstValue);

	// Properly casts a const type to another const type
	// Returns false if the case fails
	bool CoerceConstantType(FLinearColor SourceValue, EMaterialValueType SourceType, EMaterialValueType DestinationType, FLinearColor& OutResult);
	bool CastConstantType(FLinearColor SourceValue, EMaterialValueType SourceType, EMaterialValueType DestinationType, EMaterialCastFlags Flags, FLinearColor& OutResult);
	int32 LWCCastIfNeccessary(EMaterialValueType ResultType, int32 ResultCode);

	// Returns a constant of the specified typ
	int32 ConstResultValue(EMaterialValueType Type, FLinearColor ConstantValue);
	int32 ConstResultValue(EMaterialValueType Type, float ConstantValue);
	

	// Returns a valid constant result for an arithmetic expression that has a known const value
	int32 ConstArithmeticResultValue(int LeftEpression, int RightExpression, FLinearColor ConstantValue);
	int32 ConstArithmeticResultValue(int LeftEpression, int RightExpression, float ConstantValue);

	// Returns true of the expression results in a specific constant value
	bool IsExpressionConstantValue(int Code, float ConstantValue);

	int32 GenericSwitch(const TCHAR* Function, int32 IfTrue, int32 IfFalse);

	bool IsConstFloatOfPow2Expression(int32 ExpressionCode);

	FString SubstrateGetCastParameterCode(int32 Index, EMaterialValueType DestType);
	FString SubstrateGetCastParameterCodeWithDeriv(int32 Index, EMaterialValueType DestType);

	// FMaterialCompiler interface.

	/** 
	 * Sets the current material property being compiled.  
	 * This affects the internal state of the compiler and the results of all functions except GetFixedParameterCode.
	 * @param OverrideShaderFrequency SF_NumFrequencies to not override
	 */
	virtual void SetMaterialProperty(EMaterialProperty InProperty, EShaderFrequency OverrideShaderFrequency = SF_NumFrequencies, bool bUsePreviousFrameTime = false) override;
	virtual void PushMaterialAttribute(const FGuid& InAttributeID) override;
	virtual FGuid PopMaterialAttribute() override;
	virtual const FGuid GetMaterialAttribute() override;
	virtual void SetBaseMaterialAttribute(const FGuid& InAttributeID) override;

	virtual UMaterialExpressionCustomOutput* GetTopCustomOutput() override;

	virtual void PushParameterOwner(const FMaterialParameterInfo& InOwnerInfo) override;
	virtual FMaterialParameterInfo PopParameterOwner() override;

	FORCEINLINE FMaterialParameterInfo GetParameterAssociationInfo()
	{
		check(ParameterOwnerStack.Num());
		return ParameterOwnerStack.Last();
	}

	virtual EShaderFrequency GetCurrentShaderFrequency() const override;

	bool IsTangentSpaceNormal() const;

	virtual FMaterialShadingModelField GetMaterialShadingModels() const override;
	virtual FMaterialShadingModelField GetCompiledShadingModels() const override;

	virtual int32 Error(const TCHAR* Text) override;

	virtual void AppendExpressionError(UMaterialExpression* Expression, const TCHAR* Text) override;

	virtual int32 CallExpression(FMaterialExpressionKey ExpressionKey, FMaterialCompiler* Compiler) override;

	virtual int32 CallExpressionExec(UMaterialExpression* Expression) override;

	virtual EMaterialValueType GetType(int32 Code) override;
	virtual EMaterialQualityLevel::Type GetQualityLevel() override;
	virtual ERHIFeatureLevel::Type GetFeatureLevel() override;
	virtual EShaderPlatform GetShaderPlatform() override;
	virtual const ITargetPlatform* GetTargetPlatform() const override;
	virtual bool IsMaterialPropertyUsed(EMaterialProperty Property, int32 PropertyChunkIndex) const override;

	/** 
	 * Casts the passed in code to DestType, or generates a compile error if the cast is not valid. 
	 * This will truncate a type (float4 -> float3) but not add components (float2 -> float3), however a float1 can be cast to any float type by replication. 
	 */
	virtual int32 ValidCast(int32 Code, EMaterialValueType DestType) override;
	virtual int32 ForceCast(int32 Code, EMaterialValueType DestType, uint32 ForceCastFlags = 0) override;

	virtual int32 CastShadingModelToFloat(int32 Code) override;

	virtual int32 TruncateLWC(int32 Code) override;

	/** Pushes a function onto the compiler's function stack, which indicates that compilation is entering a function. */
	virtual void PushFunction(FMaterialFunctionCompileState* FunctionState) override;

	/** Pops a function from the compiler's function stack, which indicates that compilation is leaving a function. */
	virtual FMaterialFunctionCompileState* PopFunction() override;

	virtual int32 GetCurrentFunctionStackDepth() override;

	virtual int32 AccessCollectionParameter(UMaterialParameterCollection* ParameterCollection, int32 ParameterIndex, int32 ComponentIndex) override;
	virtual int32 CollectionTransform(int32 InputIndex, const TStaticArray<int32, 5>& CollectionParameters, EParameterCollectionTransformType TransformType) override;

	virtual int32 NumericParameter(EMaterialParameterType ParameterType, FName ParameterName, const UE::Shader::FValue& DefaultValue) override;
	virtual int32 Constant(float X) override;
	virtual int32 Constant2(float X, float Y) override;
	virtual int32 Constant3(float X, float Y, float Z) override;
	virtual int32 Constant4(float X, float Y, float Z, float W) override;
	virtual int32 GenericConstant(const UE::Shader::FValue& Value) override;
	
	virtual int32 ViewProperty(EMaterialExposedViewProperty Property, bool bInvProperty) override;
	virtual int32 IsOrthographic() override;

	virtual int32 GameTime(bool bPeriodic, float Period) override;
	virtual int32 RealTime(bool bPeriodic, float Period) override;
	virtual int32 DeltaTime() override;

	virtual int32 PeriodicHint(int32 PeriodicCode) override;
	
	virtual int32 Sine(int32 X) override;
	virtual int32 Cosine(int32 X) override;
	virtual int32 Tangent(int32 X) override;
	virtual int32 Arcsine(int32 X) override;
	virtual int32 ArcsineFast(int32 X) override;
	virtual int32 Arccosine(int32 X) override;
	virtual int32 ArccosineFast(int32 X) override;
	virtual int32 Arctangent(int32 X) override;
	virtual int32 ArctangentFast(int32 X) override;
	virtual int32 Arctangent2(int32 Y, int32 X) override;
	virtual int32 Arctangent2Fast(int32 Y, int32 X) override;
	virtual int32 Floor(int32 X) override;
	virtual int32 Ceil(int32 X) override;
	virtual int32 Round(int32 X) override;
	virtual int32 Truncate(int32 X) override;
	virtual int32 Sign(int32 X) override;
	virtual int32 Frac(int32 X) override;
	virtual int32 Fmod(int32 A, int32 B) override;
	virtual int32 Modulo(int32 A, int32 B) override;

	/**
	* Creates the new shader code chunk needed for the Abs expression
	*
	* @param	X - Index to the FMaterialCompiler::CodeChunk entry for the input expression
	* @return	Index to the new FMaterialCompiler::CodeChunk entry for this expression
	*/	
	virtual int32 Abs(int32 X) override;

	virtual int32 ReflectionVector() override;

	virtual int32 ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal) override;

	virtual int32 CameraVector() override;
	virtual int32 LightVector() override;

	virtual int32 GetViewportUV() override;

	virtual int32 GetPixelPosition() override;

	virtual int32 ParticleMacroUV() override;
	virtual int32 ParticleSubUV(int32 TextureIndex, EMaterialSamplerType SamplerType, int32 MipValue0Index, int32 MipValue1Index, ETextureMipValueMode MipValueMode, bool bBlend) override;
	virtual int32 ParticleSubUVProperty(int32 PropertyIndex) override;
	virtual int32 ParticleColor() override;
	virtual int32 ParticlePosition(EPositionOrigin OriginType) override;
	virtual int32 ParticleRadius() override;

	virtual int32 SphericalParticleOpacity(int32 Density) override;

	virtual int32 ParticleRelativeTime() override;
	virtual int32 ParticleMotionBlurFade() override;
	virtual int32 ParticleRandom() override;
	virtual int32 ParticleDirection() override;
	virtual int32 ParticleSpeed() override;
	virtual int32 ParticleSize() override;
	virtual int32 ParticleSpriteRotation() override;

	virtual int32 LocalPosition(EPositionIncludedOffsets IncludedOffsets, ELocalPositionOrigin OriginType) override;
	virtual int32 WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets) override;

	virtual int32 ObjectWorldPosition(EPositionOrigin OriginType) override;
	virtual int32 ObjectRadius() override;
	virtual int32 ObjectBounds() override;

	virtual int32 ObjectLocalBounds(int32 OutputIndex) override;
	virtual int32 InstanceLocalBounds(int32 OutputIndex) override;
	virtual int32 PreSkinnedLocalBounds(int32 OutputIndex) override;

	virtual int32 DistanceCullFade() override;

	virtual int32 ActorWorldPosition(EPositionOrigin OriginType) override;

	virtual int32 DynamicBranch(int32 Condition, int32 A, int32 B) override;
	virtual int32 If(int32 A, int32 B, int32 AGreaterThanB, int32 AEqualsB, int32 ALessThanB, int32 ThresholdArg) override;

	void AllocateSlot(TBitArray<>& InBitArray, int32 InSlotIndex, int32 InSlotCount = 1) const;

#if WITH_EDITOR
	virtual int32 MaterialBakingWorldPosition() override;
#endif

	virtual int32 TextureCoordinate(uint32 CoordinateIndex, bool UnMirrorU, bool UnMirrorV) override;
	virtual void SetPotentiallyManipulateTexCoords() override;

	uint32 AcquireVTStackIndex(
		const FMaterialVTSampleInfo& SampleInfo,
		const FString& UV_Value,
		const FString& UV_Ddx,
		const FString& UV_Ddy
	);

	virtual int32 TextureSample(
		int32 TextureIndex,
		int32 CoordinateIndex,
		EMaterialSamplerType SamplerType,
		int32 MipValue0Index = INDEX_NONE,
		int32 MipValue1Index = INDEX_NONE,
		ETextureMipValueMode MipValueMode = TMVM_None,
		ESamplerSourceMode SamplerSource = SSM_FromTextureAsset,
		ETextureGatherMode GatherMode = TGM_None,
		int32 TextureReferenceIndex = INDEX_NONE,
		bool AutomaticViewMipBias = false,
		bool AdaptiveVirtualTexture = false,
		bool EnableFeedback = true
	) override;

	virtual int32 TextureProperty(int32 TextureIndex, EMaterialExposedTextureProperty Property) override;
	virtual int32 TextureFromCollection(int32 TextureCollectionCodeIndex, int32 IndexIntoCollection, EMaterialValueType ResultTextureType) override;

	virtual int32 TextureStreamingInfo(int32 TextureReferenceIndex, int32 TextureIndex, int32 CoordinateIndex) override;

	FMaterialCacheTag& FindOrAddMaterialTagStack(const FMaterialCacheTagLayout& Layout);
	
	virtual int32 MaterialCacheTextureDescriptor(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, uint32 LayerIndex) override;
	virtual int32 SampleMaterialCache(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, int32 TexCoordIndex) override;
	virtual int32 DefaultMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout) override;
	virtual int32 SetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex, int32 Value) override;
	virtual int32 GetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex) override;
	virtual int32 MaterialCacheOutput(UMaterialExpressionCustomOutput* CustomOutput, const FMaterialCacheTagLayout& Layout, int32 AttributeSet) override;
	
	virtual int32 TextureDecalMipmapLevel(int32 TextureSizeInput) override;
	virtual int32 TextureDecalDerivative(bool bDDY) override;

	virtual int32 DecalColor() override;
	virtual int32 DecalLifetimeOpacity() override;

	virtual int32 PixelDepth() override;

	/** Calculate screen aligned UV coordinates from an offset fraction or texture coordinate */
	int32 GetScreenAlignedUV(int32 Offset, int32 ViewportUV, bool bUseOffset);

	virtual int32 SceneDepth(int32 Offset, int32 ViewportUV, bool bUseOffset) override;
	
	// @param SceneTextureId of type ESceneTextureId e.g. PPI_SubsurfaceColor
	virtual int32 SceneTextureLookup(int32 ViewportUV, uint32 InSceneTextureId, bool bFiltered, bool bClamped, bool bUnused) override;

	virtual int32 GetSceneTextureViewSize(int32 SceneTextureId, bool InvProperty) override;

	virtual int32 FindOrAddUserSceneTexture(FName UserSceneTextureName) override;

	virtual int32 DBufferTextureLookup(int32 ViewportUV, uint32 DBufferTextureIndex) override;

	virtual int32 PathTracingBufferTextureLookup(int32 ViewportUV, uint32 PathTracingBufferTextureIndex) override;

	// @param bTextureLookup true: texture, false:no texture lookup, usually to get the size
	void UseSceneTextureId(ESceneTextureId SceneTextureId, bool bTextureLookup);

	virtual int32 SceneColor(int32 Offset, int32 ViewportUV, bool bUseOffset) override;

	virtual int32 Switch(int32 SwitchValueInput, int32 DefaultInput, TArray<int32>& CompiledInputs) override;
	virtual int32 Texture(UTexture* InTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource = SSM_FromTextureAsset, ETextureMipValueMode MipValueMode = TMVM_None) override;
	virtual int32 TextureParameter(FName ParameterName, UTexture* DefaultValue, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource = SSM_FromTextureAsset) override;
	
	virtual int32 TextureCollection(UTextureCollection* TextureCollection, int32& TextureCollectionReferenceIndex) override;
	virtual int32 TextureCollectionParameter(FName ParameterName, UTextureCollection* DefaultValue, int32& TextureCollectionReferenceIndex) override;
	virtual int32 TextureCollectionCount(int32 InTextureCollectionCodeIndex) override;

	virtual int32 VirtualTexture(URuntimeVirtualTexture* InTexture, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override;
	virtual int32 VirtualTextureParameter(FName ParameterName, URuntimeVirtualTexture* DefaultValue, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override;
	virtual int32 VirtualTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override;
	virtual int32 VirtualTextureUniform(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override;
	virtual int32 VirtualTextureWorldToUV(int32 WorldPositionIndex, int32 P0, int32 P1, int32 P2, EPositionOrigin PositionOrigin) override;
	virtual int32 VirtualTextureUnpack(int32 CodeIndex0, int32 CodeIndex1, int32 CodeIndex2, int32 P0, EVirtualTextureUnpackType UnpackType) override;
	virtual int32 VirtualTextureCustomData() override;

	virtual int32 ExternalTexture(const FGuid& ExternalTextureGuid) override;
	virtual int32 ExternalTexture(UTexture* InTexture, int32& TextureReferenceIndex) override;
	virtual int32 ExternalTextureParameter(FName ParameterName, UTexture* DefaultValue, int32& TextureReferenceIndex) override;
	virtual int32 ExternalTextureCoordinateScaleRotation(int32 TextureReferenceIndex, TOptional<FName> ParameterName) override;
	virtual int32 ExternalTextureCoordinateScaleRotation(const FGuid& ExternalTextureGuid) override;
	virtual int32 ExternalTextureCoordinateOffset(int32 TextureReferenceIndex, TOptional<FName> ParameterName) override;
	virtual int32 ExternalTextureCoordinateOffset(const FGuid& ExternalTextureGuid) override;

	virtual int32 SparseVolumeTexture(USparseVolumeTexture* Texture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override;
	virtual int32 SparseVolumeTextureParameter(FName ParameterName, USparseVolumeTexture* InDefaultTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override;
	virtual int32 SparseVolumeTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override;
	virtual int32 SparseVolumeTextureUniformParameter(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override;
	virtual int32 SparseVolumeTextureSamplePageTable(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipLevelIndex, ESamplerSourceMode SamplerSource, bool bIsManualLinearMipMapSecondSample) override;
	virtual int32 SparseVolumeTextureSamplePhysicalTileData(int32 SparseVolumeTextureIndex, int32 VoxelCoordIndex, int32 PhysicalTileDataIdxIndex, bool bIsManualLinearMipMapSecondSample) override;
	virtual int32 SparseVolumeTextureSample(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipValue0Index, int32 MipValue1Index, int32 PhysicalTileDataIdxIndex, ETextureMipValueMode MipValueMode, ESamplerSourceMode SamplerSource) override;

	virtual UObject* GetReferencedTexture(int32 Index) override;
	virtual UTextureCollection* GetReferencedTextureCollection(int32 Index) override;

	virtual int32 StaticBool(bool bValue) override;
	virtual int32 StaticBoolParameter(FName ParameterName, bool bDefaultValue) override;
	virtual int32 DynamicBoolParameter(FName ParameterName, bool bDefaultValue) override;
	virtual int32 StaticComponentMask(int32 Vector, FName ParameterName, bool bDefaultR, bool bDefaultG, bool bDefaultB, bool bDefaultA) override;
	virtual const FMaterialLayersFunctions* GetMaterialLayers() override;

	virtual void FeedbackMaterialLayersInstancedGraphFromCompilation(const FMaterialLayersFunctions* InLayers) override;

	virtual bool GetStaticBoolValue(int32 BoolIndex, bool& bSucceeded) override;

	virtual int32 StaticTerrainLayerWeight(FName ParameterName, int32 Default, bool bTextureArray = false) override;

	virtual int32 ExternalCode(const FMaterialExternalCodeDeclaration& InExternalCode) override;

	virtual int32 FontSignedDistanceData() override;

	virtual int32 VertexColor() override;
	virtual int32 MeshPaintTextureCoordinateIndex() override;
	virtual int32 MeshPaintTextureDescriptor() override;
	virtual int32 MeshPaintTextureReplace(int32 Invalid, int32 Valid) override;

	virtual int32 PreSkinnedPosition() override;
	virtual int32 PreSkinnedNormal() override;

	virtual int32 VertexInterpolator(uint32 InterpolatorIndex) override;

	virtual int32 Add(int32 A, int32 B) override;
	virtual int32 Sub(int32 A, int32 B) override;
	virtual int32 Mul(int32 A, int32 B) override;
	virtual int32 Div(int32 A, int32 B) override;
	virtual int32 Dot(int32 A, int32 B) override;
	virtual int32 Cross(int32 A, int32 B) override;
	virtual int32 Power(int32 Base, int32 Exponent) override;
	virtual int32 Exponential(int32 X) override;
	virtual int32 Exponential2(int32 X) override;
	virtual int32 Logarithm(int32 X) override;
	virtual int32 Logarithm2(int32 X) override;
	virtual int32 Logarithm10(int32 X) override;
	virtual int32 SquareRoot(int32 X) override;
	virtual int32 Length(int32 X) override;
	virtual int32 Normalize(int32 X) override;
	virtual int32 Step(int32 Y, int32 X) override;
	virtual int32 SmoothStep(int32 X, int32 Y, int32 A) override;
	virtual int32 InvLerp(int32 X, int32 Y, int32 A) override;
	virtual int32 Lerp(int32 A, int32 B, int32 S) override;
	virtual int32 Min(int32 A, int32 B) override;
	virtual int32 Max(int32 A, int32 B) override;
	virtual int32 Clamp(int32 X, int32 A, int32 B) override;
	virtual int32 Saturate(int32 X) override;
	virtual int32 ComponentMask(int32 Vector, bool R, bool G, bool B, bool A) override;
	virtual int32 VectorComponent(int32 Vector, int32 Component) override;
	virtual int32 BitfieldExtract(int32 Value, int32 BitOffset, int32 BitCount) override;
	virtual int32 AppendVector(int32 A, int32 B) override;
	virtual int32 HsvToRgb(int32 X) override;
	virtual int32 RgbToHsv(int32 X) override;

	int32 TransformBase(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A, int AWComponent);
	
	virtual int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override;
	virtual int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override;
	
	int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, int32 A)
	{
		FTransformParameters Parameters{};
		return TransformVector(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}
	
	int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, int32 A)
	{
		FTransformParameters Parameters{};
		return TransformPosition(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}
	
	int32 CalculatePeriodicWorldPositionOrigin(int TileScaleIndex);
	virtual int32 TransformNormalFromRequestedBasisToWorld(int32 NormalCodeChunk) override;
	virtual int32 DynamicParameter(FLinearColor& DefaultValue, uint32 ParameterIndex = 0) override;
	virtual int32 LightmapUVs() override;
	virtual int32 PrecomputedAOMask() override;
	virtual int32 GIReplace(int32 Direct, int32 StaticIndirect, int32 DynamicIndirect) override;
	virtual int32 ShadowReplace(int32 Default, int32 Shadow) override;
	virtual int32 NaniteReplace(int32 Default, int32 Nanite) override;
	virtual int32 MaterialCache(int32 Default, int32 MaterialCache) override;
	virtual int32 ReflectionCapturePassSwitch(int32 Default, int32 Reflection) override;

	virtual int32 RayTracingQualitySwitchReplace(int32 Normal, int32 RayTraced) override;
	virtual int32 PathTracingQualitySwitchReplace(int32 Normal, int32 PathTraced) override;
	virtual int32 PathTracingRayTypeSwitch(int32 Main, int32 Shadow, int32 IndirectDiffuse, int32 IndirectSpecular, int32 IndirectVolume) override;
	virtual int32 LightmassReplace(int32 Realtime, int32 Lightmass) override;

	virtual int32 ObjectOrientation() override;

	virtual int32 RotateAboutAxis(int32 NormalizedRotationAxisAndAngleIndex, int32 PositionOnAxisIndex, int32 PositionIndex) override;

	virtual int32 TwoSidedSign() override;
	virtual int32 VertexNormal() override;
	virtual int32 VertexTangent() override;
	virtual int32 PixelNormalWS() override;
	virtual int32 IsFirstPerson() override;

	virtual int32 DDX(int32 A) override;
	virtual int32 DDY(int32 A) override;

	enum class EDervativeComponent
	{
		X, 
		Y,
	};
	int32 Derivative(int32 A, EDervativeComponent Component);

	virtual int32 AntialiasedTextureMask(int32 Tex, int32 UV, float Threshold, uint8 Channel) override;
	virtual int32 DepthOfFieldFunction(int32 Depth, int32 FunctionValueIndex) override;
	virtual int32 PostVolumeUserFlagTestFunction(int32 Input) override;
	virtual int32 Sobol(int32 Cell, int32 Index, int32 Seed) override;
	virtual int32 TemporalSobol(int32 Index, int32 Seed) override;
	virtual int32 Noise(int32 Position, EPositionOrigin PositionOrigin, float Scale, int32 Quality, uint8 NoiseFunction, bool bTurbulence, int32 Levels, float OutputMin, float OutputMax, float LevelScale, int32 FilterWidth, bool bTiling, uint32 RepeatSize) override;
	virtual int32 VectorNoise(int32 Position, EPositionOrigin PositionOrigin, int32 Quality, uint8 NoiseFunction, bool bTiling, uint32 TileSize) override;
	virtual int32 ScalarBlueNoise() override;

	virtual int32 BlackBody(int32 Temp) override;

	virtual int32 GetHairUV() override;
	virtual int32 GetHairDimensions() override;
	virtual int32 GetHairSeed() override;
	virtual int32 GetHairClumpID() override;
	virtual int32 GetHairTangent(bool bUseTangentSpace) override;
	virtual int32 GetHairRootUV() override;
	virtual int32 GetHairBaseColor() override;
	virtual int32 GetHairAO() override;
	virtual int32 GetHairRoughness() override;
	virtual int32 GetHairDepth() override;
	virtual int32 GetHairCoverage() override;
	virtual int32 GetHairAuxilaryData() override;
	virtual int32 GetHairAtlasUVs() override;
	virtual int32 GetHairGroupIndex() override;
	virtual int32 GetHairColorFromMelanin(int32 Melanin, int32 Redness, int32 DyeColor) override;
	virtual int32 DistanceToNearestSurface(int32 PositionArg, EPositionOrigin PositionOrigin) override;
	virtual int32 DistanceFieldGradient(int32 PositionArg, EPositionOrigin PositionOrigin) override;
	virtual int32 DistanceFieldApproxAO(int32 PositionArg, EPositionOrigin PositionOrigin, int32 NormalArg, int32 BaseDistanceArg, int32 RadiusArg, uint32 NumSteps, float StepScale) override;
	virtual int32 ProcessPhysicsField(int32 PositionArg, EPositionOrigin PositionOrigin, const int32 OutputType, const int32 TargetIndex, const bool bSampleField) override;
	virtual int32 AtmosphericFogColor(int32 WorldPosition, EPositionOrigin PositionOrigin) override;
	virtual int32 AtmosphericLightVector() override;
	virtual int32 AtmosphericLightColor() override;

	virtual int32 SkyAtmosphereLightIlluminance(int32 WorldPosition, EPositionOrigin PositionOrigin, int32 LightIndex) override;
	virtual int32 SkyAtmosphereLightIlluminanceOnGround(int32 LightIndex) override;
	virtual int32 SkyAtmosphereLightDirection(int32 LightIndex) override;
	virtual int32 SkyAtmosphereLightDiskLuminance(int32 LightIndex, int32 OverrideAtmosphereLightDiscCosHalfApexAngle) override;
	virtual int32 SkyAtmosphereViewLuminance(int32 WorldDirectionOverrideCodeChunk) override;
	virtual int32 SkyAtmosphereAerialPerspective(int32 WorldPosition, EPositionOrigin PositionOrigin) override;
	virtual int32 SkyAtmosphereDistantLightScatteredLuminance() override;
	virtual int32 SkyLightEnvMapSample(int32 DirectionCodeChunk, int32 RoughnessCodeChunk) override;
	virtual int32 MainDirectionalLightIlluminance() override;
	virtual int32 MainDirectionalLightDirection() override;

	// Water
	virtual int32 SceneDepthWithoutWater(int32 Offset, int32 ViewportUV, bool bUseOffset, float FallbackDepth) override;

	virtual int32 GetCloudSampleAltitude() override;
	virtual int32 GetCloudSampleAltitudeInLayer() override;
	virtual int32 GetCloudSampleNormAltitudeInLayer() override;
	virtual int32 GetCloudSampleShadowSampleDistance() override;
	virtual int32 GetVolumeSampleConservativeDensity() override;
	virtual int32 GetCloudEmptySpaceSkippingSphereCenterWorldPosition() override;
	virtual int32 GetCloudEmptySpaceSkippingSphereRadius() override;

	virtual int32 CustomPrimitiveData(int32 OutputIndex, EMaterialValueType Type) override;

	virtual int32 ShadingModel(EMaterialShadingModel InSelectedShadingModel) override;

	virtual int32 MapARPassthroughCameraUV(int32 UV) override;

	virtual int32 CustomExpression(class UMaterialExpressionCustom* Custom, int32 OutputIndex, TArray<int32>& CompiledInputs) override;
	virtual int32 CustomOutput(class UMaterialExpressionCustomOutput* Custom, int32 OutputIndex, int32 OutputCode, EMaterialCustomOutputFlags Flags) override;
	int32 CustomOutput(const FString& CustomOutputName, const FString& OutputTypeString, const FString& FunctionName, int32 OutputIndex, int32 OutputCode, EMaterialCustomOutputFlags Flags); // Only used internally and intentionally not exposed on FMaterialCompiler.

	virtual int32 PushRuntimeVirtualTextureOutput() override;
	virtual int32 PopRuntimeVirtualTextureOutput(uint8 MaterialAttributeMask) override;
	virtual bool IsInRuntimeVirtualTextureOutput() const override;

	// Material attributes
	virtual int32 DefaultMaterialAttributes() override;
	virtual int32 SetMaterialAttribute(int32 MaterialAttributes, int32 Value, const FGuid& AttributeID) override;

	// Exec
	virtual int32 BeginScope() override;
	virtual int32 BeginScope_If(int32 Condition) override;
	virtual int32 BeginScope_Else() override;
	virtual int32 BeginScope_For(const UMaterialExpression* Expression, int32 StartIndex, int32 EndIndex, int32 IndexStep) override;
	virtual int32 EndScope() override;
	virtual int32 ForLoopIndex(const UMaterialExpression* Expression) override;
	virtual int32 ReturnMaterialAttributes(int32 MaterialAttributes) override;
	virtual int32 SetLocal(const FName& LocalName, int32 Value) override;
	virtual int32 GetLocal(const FName& LocalName) override;

	// Neural network nodes
	virtual int32 NeuralOutput(int32 ViewportUV, uint32 NeuralIndexType) override;

	// Substrate
	virtual int32 SubstrateCreateAndRegisterNullMaterial() override;
	virtual int32 SubstrateSlabBSDF(
		int32 DiffuseAlbedo, int32 F0, int32 F90,
		int32 Roughness, int32 Anisotropy,
		int32 SSSProfileId, int32 SSSMFP, int32 SSSMFPScale, int32 SSSPhaseAniso, int32 SSSType,
		int32 EmissiveColor, 
		int32 SecondRoughness, int32 SecondRoughnessWeight, int32 SecondRoughnessAsSimpleClearCoat, int32 ClearCoatBottomNormal,
		int32 FuzzAmount, int32 FuzzColor, int32 FuzzRoughness,
		int32 Thickness,
		int32 GlintValue, int32 GlintUV,
		int32 SpecularProfileId,
		bool bIsAtTheBottomOfTopology,
		int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
		FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateConversionFromLegacy(
		bool bHasDynamicShadingModel,
		int32 BaseColor, int32 Specular, int32 Metallic,
		int32 Roughness, int32 Anisotropy,
		int32 SubSurfaceColor, int32 SubSurfaceProfileId,
		int32 ClearCoat, int32 ClearCoatRoughness,
		int32 EmissiveColor,
		int32 Opacity,
		int32 ThinTranslucentTransmittanceColor,
		int32 ThinTranslucentSurfaceCoverage,
		int32 WaterScatteringCoefficients, int32 WaterAbsorptionCoefficients, int32 WaterPhaseG, int32 ColorScaleBehindWater,
		int32 ShadingModel,
		int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
		int32 ClearCoat_Normal, int32 ClearCoat_Tangent, const FString& ClearCoat_SharedLocalBasisIndexMacro,
		int32 CustomTangent_Tangent,
		FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateVolumetricFogCloudBSDF(int32 Albedo, int32 Extinction, int32 EmissiveColor, int32 AmbientOcclusion, bool bEmissiveOnly) override;
	virtual int32 SubstrateUnlitBSDF(int32 EmissiveColor, int32 TransmittanceColor, int32 Normal, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateUIBSDF(int32 EmissiveColor, int32 Opacity, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateHairBSDF(int32 BaseColor, int32 Scatter, int32 Specular, int32 Roughness, int32 Backlit, int32 EmissiveColor, int32 Tangent, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateEyeBSDF(int32 DiffuseColor, int32 Roughness, int32 IrisMask, int32 IrisDistance, int32 EmissiveColor, int32 CorneaNormal, int32 IrisNormal, int32 IrisPlaneNormal, int32 SSSProfileId, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateSingleLayerWaterBSDF(
		int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness, 
		int32 EmissiveColor, int32 TopMaterialOpacity, int32 WaterAlbedo, int32 WaterExtinction, int32 WaterPhaseG, 
		int32 ColorScaleBehindWater, int32 Normal, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateToonBSDF(
		int32 ToonProfileId,
		int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness,
		int32 EmissiveColor, int32 Normal,
		int32 PatternUVs,
		int32 Anisotropy, int32 Tangent,
		const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateHorizontalMixing(int32 Background, int32 Foreground, int32 Mix, int OperatorIndex, uint32 MaxDistanceFromLeaves) override;
	virtual int32 SubstrateHorizontalMixingParameterBlending(int32 Background, int32 Foreground, int32 HorizontalMixCodeChunk, int32 NormalMixCodeChunk, const FString& SharedLocalBasisIndexMacro, int32 BackgroundBSDFNormalCodeChunk, int32 ForegroundBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateVerticalLayering(int32 Top, int32 Base, int32 Thickness, int OperatorIndex, uint32 MaxDistanceFromLeaves) override;
	virtual int32 SubstrateVerticalLayeringParameterBlending(int32 Top, int32 Base, int32 Thickness, const FString& SharedLocalBasisIndexMacro, int32 TopBSDFNormalCodeChunk, int32 BaseBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateAdd(int32 A, int32 B, int OperatorIndex, uint32 MaxDistanceFromLeaves) override;
	virtual int32 SubstrateAddParameterBlending(int32 A, int32 B, int32 AMixWeight, const FString& SharedLocalBasisIndexMacro, int32 ABSDFNormalCodeChunk, int32 BBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateWeight(int32 A, int32 Weight, int OperatorIndex, uint32 MaxDistanceFromLeaves) override;
	virtual int32 SubstrateWeightParameterBlending(int32 A, int32 Weight, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateSelectParameterBlending(int32 A, int32 B, int32 SelectValue, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override;
	virtual int32 SubstrateTransmittanceToMFP(int32 TransmittanceColor, int32 DesiredThickness, int32 OutputIndex) override;
	virtual int32 SubstrateMetalnessToDiffuseAlbedoF0(int32 BaseColor, int32 Specular, int32 Metallic, int32 OutputIndex) override;
	virtual int32 SubstrateHazinessToSecondaryRoughness(int32 BaseRoughness, int32 Haziness, int32 OutputIndex) override;
	virtual int32 SubstrateThinFilm(int32 NormalCodeChunk, int32 SpecularColorCodeChunk, int32 EdgeSpecularColorCodeChunk, int32 ThicknessCodeChunk, int32 IORCodeChunk, int32 OutputIndex) override;
	virtual int32 SubstrateCompilePreview(int32 SubstrateDataCodeChunk) override;
	virtual bool SubstrateSkipsOpacityEvaluation() override;

	virtual FGuid SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex) override;
	virtual FGuid SubstrateTreeStackGetPathUniqueId() override;
	virtual FGuid SubstrateTreeStackGetParentPathUniqueId() override;
	virtual void SubstrateTreeStackPop() override;
	virtual bool GetSubstrateTreeOutOfStackDepthOccurred() override;
	
	virtual int32 SubstrateThicknessStackGetThicknessIndex() override;
	virtual int32 SubstrateThicknessStackGetThicknessCode(int32 Index) override;
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FScalarMaterialInput* Input) override;
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FExpressionInput* Input) override;
	virtual void SubstrateThicknessStackPop() override;

	virtual struct FSubstrateTranslatorDataInterface& GetSubstrateTranslatorData() override;

	virtual FSubstrateOperator& SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending = false) override;
	virtual FSubstrateOperator& SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid) override;
	virtual FSubstrateOperator* SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex) override;

	virtual FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk) override;
	virtual FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk, int32 TangentCodeChunk) override;
	virtual FString GetSubstrateSharedLocalBasisIndexMacro(const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis) override;
	virtual int32 SubstrateAddParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 ACodeChunk, int32 BCodeChunk) override;
	virtual int32 SubstrateVerticalLayeringParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 TopCodeChunk) override;
	virtual int32 SubstrateHorizontalMixingParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 BackgroundCodeChunk, int32 ForegroundCodeChunk, int32 HorizontalMixCodeChunk) override;

	/**
	 * Adds code to return a random value shared by all geometry for any given instanced static mesh
	 *
	 * @return	Code index
	 */
	virtual int32 PerInstanceRandom() override;

	/**
	 * Returns a mask that either enables or disables selection on a per-instance basis when instancing
	 *
	 * @return	Code index
	 */
	virtual int32 PerInstanceFadeAmount() override;

	/**
	 * Returns a custom data on a per-instance basis when instancing
	 * @DataIndex - index in array that represents custom data
	 *
	 * @return	Code index
	 */
	virtual int32 PerInstanceCustomData(int32 DataIndex, int32 DefaultValueIndex) override;

	/**
	 * Returns a custom data on a per-instance basis when instancing
	 * @DataIndex - index in array that represents custom data
	 *
	 * @return	Code index
	 */
	virtual int32 PerInstanceCustomData3Vector(int32 DataIndex, int32 DefaultValueIndex) override;

	/**
	 * Returns a float2 texture coordinate after 2x2 transform and offset applied
	 *
	 * @return	Code index
	 */
	virtual int32 RotateScaleOffsetTexCoords(int32 TexCoordCodeIndex, int32 RotationScale, int32 Offset) override;

	/**
	* Handles SpeedTree vertex animation (wind, smooth LOD)
	*
	* @return	Code index
	*/
	virtual int32 SpeedTree(int32 GeometryArg, int32 WindArg, int32 LODArg, float BillboardThreshold, bool bAccurateWindVelocities, bool bExtraBend, int32 ExtraBendArg) override;

	/**Experimental access to the EyeAdaptation RT for Post Process materials. Can be one frame behind depending on the value of BlendableLocation. */
	virtual int32 EyeAdaptation() override;

	/**Experimental access to the EyeAdaptation RT for applying an inverse. */
	virtual int32 EyeAdaptationInverse(int32 LightValueArg, int32 AlphaArg) override;

	const bool CheckPrimitivePropertyCompatibity(const TCHAR* ExpressionName);

	/**
	 * To only have one piece of code dealing with error handling if the Primitive constant buffer is not used.
	 * @param Name e.g. TEXT("ObjectWorldPositionAndRadius.w")
	 */
	int32 GetPrimitiveProperty(EMaterialValueType Type, const TCHAR* ExpressionName, const TCHAR* HLSLName);

	/**
	 * The compiler can run in a different state and this affects caching of sub expression, Expressions are different(e.g. ResolvedView.PrevWorldViewOrigin) when using previous frame's values.
	 */
	virtual bool IsCurrentlyCompilingForPreviousFrame() const;

	virtual bool IsDevelopmentFeatureEnabled(const FName& FeatureName) const override;

	/**
	 * Effectively performs the translation without querying the DDC first.
	 */
	void TranslateMaterial();

	/**
	 * Queries the DDC cache for a cached translation.
	 * @return Whether the speecified key is in the DDC.
	 */
	void AsyncQueryDDC(UE::DerivedData::FRequestOwner& DDCRequestOwner, FSharedBuffer& EnvironmentDefinesBuffer);

	/**
	 * Pushes the final results to the DDC cache.
	 */
	void PushResultsToDDC();

	/**
	 * Prepares the material source generation parameters.
	 */
	void PrepareMaterialSourceStringParameters();

	/**
	 * Generates all material cache structures and definitions
	 */
	void GenerateMaterialCacheSource();

	/**
	 * Prepares the Environment Defines array based on compilation results.
	 */
	void PrepareEnvironmentDefines();


	void GenerateUniformExpressionMacro(FStringBuilderBase& FormattedCode, const FShaderCodeChunk&	CodeChunk);
	FString GenerateUniformExpressionMacroDefinitions();

	/** The final compilation output. */
	FMaterialCompilationOutput& MaterialCompilationOutput;
	FMaterialCompilationOutput DDCMaterialCompilationOutput;

	/** The material shader source template parameters */
	TMap<FString, FString> MaterialSourceTemplateParams;

	/** The output material shader defines */
	TUniquePtr<FEnvironmentDefines> EnvironmentDefines;

	/** Whether the async DDC query was completed (instead of terminating because canceled) */
	TAtomic<bool> DDCQueryCompleted;

	/** Signals when the async DDC query task has completed AND there was a hit */
	TAtomic<bool> DDCQueryHit;

#ifdef STATS
	/** When a translation DDC request hits, this is set to the time spent serializing. */
	double DDCRequestSerializeTime = 0;
#endif

	friend class FAddUniformExpressionScope;
};

/**
 * The purpose of this class is to avoid generating unused preshaders, meaning preshaders not actually fetched in the HLSL code, saving performance.
 * The translator normally generates preshaders when the function "AccessUniformExpression" is called the first time on a uniform expression.  We want
 * to detect in that function whether the uniform expression access is from HLSL code or a parent uniform expression.  In the latter case, there is no
 * need to generate a preshader.  The FAddUniformExpressionScope class provides that context.
 *
 * This class sets a flag in the translator when we enter a code path where we are adding a uniform expression (technically uses a count, so scopes can
 * be nested).  To enforce usage of this scope class everywhere, the translator's AddUniformExpression function requires it to be passed in.  The reason
 * we need a separate scope class, rather than setting the state in the AddUniformExpression function, is that the code that needs the context is actually
 * the arguments to the AddUniformExpression function, not the function itself.  Calls to GetParameterCode and other utility functions that call
 * GetParameterCode (CoerceParameter, ValidCast, ForceCast) are what call "AccessUniformExpression", and need to know whether they are being passed to
 * "AddUniformExpression", versus functions like AddCodeChunk that emit the HLSL.
 *
 * If you make a mistake and place the scope somewhere it doesn't belong, it will generate stub code that will result in shader compile errors on
 * symbols that look like "UniformStub[$0]".  Most likely, any new opcodes will be cut-and-paste from existing code, and thus already have the scope in
 * the right place, so bugs should be rare.
 */
class FAddUniformExpressionScope
{
public:
	FAddUniformExpressionScope(FHLSLMaterialTranslator* InTranslator)
		: Translator(InTranslator)
	{
		Translator->AddingUniformExpression++;
	}
	~FAddUniformExpressionScope()
	{
		Translator->AddingUniformExpression--;
		check(Translator->AddingUniformExpression >= 0);
	}
private:
	FHLSLMaterialTranslator* Translator;
};

#endif // WITH_EDITORONLY_DATA
