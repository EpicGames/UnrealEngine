// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnMaterial.cpp: Shader implementation.
=============================================================================*/

#include "Materials/Material.h"

#include "Stats/StatsMisc.h"
#include "Misc/FeedbackContext.h"
#include "Stats/StatsTrace.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/LinkerLoad.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UnrealEngine.h"

#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Materials/MaterialExpressionQualitySwitch.h"
#include "Materials/MaterialExpressionFeatureLevelSwitch.h"
#include "Materials/MaterialExpressionShadingPathSwitch.h"
#include "Materials/MaterialExpressionShaderStageSwitch.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionGetMaterialAttributes.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureOutput.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionVertexInterpolator.h"
#include "Materials/MaterialExpressionShadingModel.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialSharedPrivate.h"
#include "PSOPrecacheMaterial.h"

#include "SceneManagement.h"
#include "SceneView.h"
#include "Materials/MaterialUniformExpressions.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "Engine/ToonProfile.h"
#include "Engine/TextureCollection.h"
#include "EditorSupportDelegates.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ShaderCompiler.h"
#include "Materials/MaterialParameterCollection.h"
#include "ShaderPlatformQualitySettings.h"
#include "MaterialShaderQualitySettings.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/CookStats.h"
#include "MaterialCompiler.h"
#include "MaterialDomain.h"
#include "MaterialShaderType.h"
#include "Materials/MaterialInstanceSupport.h"
#include "Interfaces/ITargetPlatform.h"
#include "Materials/MaterialExpressionComment.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialInput.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "Materials/MaterialExpressionCloudLayer.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionTangentOutput.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialIRModuleBuilder.h"
#include "MaterialCachedData.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "HAL/FileManager.h"
#include "BuildSettings.h"
#include "LocalVertexFactory.h"

#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#endif

#if WITH_EDITOR
#include "Cooker/CookEvents.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "ObjectCacheEventSink.h"
#include "MaterialGraph/MaterialGraph.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Engine/RendererSettings.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/SavePackage.h"
#include "Misc/MapErrors.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "MaterialEditorUtilities.h"
#endif
#include "ShaderCodeLibrary.h"
#include "String/ParseTokens.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Misc/ScopedSlowTask.h"
#include "PSOPrecacheSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Material)

#define LOCTEXT_NAMESPACE "Material"

static TAutoConsoleVariable<bool> CVarMaterialLogErrorOnFailure(
	TEXT("r.MaterialLogErrorOnFailure"),
	false,
	TEXT("When enabled, when a material fails to compile it will issue an Error instead of a Warning.\n")
	TEXT("Default: false"),
	ECVF_RenderThreadSafe);
	
static TAutoConsoleVariable<bool> CVarMaterialsDuplicateVerbatim(
	TEXT("r.MaterialsDuplicateVerbatim"),
	false,
	TEXT("When enabled, when a material or material function is duplicated, it will not change StateId (which influences DDC keys) pre-emptively.\n")
	TEXT("Default: false"),
	ECVF_Default);

namespace MaterialImpl
{
	// Filtered list of properties that we follow on mobile platforms.
	struct FMobileProperty
	{
		EMaterialProperty Property;
		EShaderFrequency Frequency;
	};
	static constexpr FMobileProperty GMobileRelevantMaterialProperties[] =
	{
		{ MP_WorldPositionOffset, SF_Vertex },
		{ MP_EmissiveColor, SF_Pixel },
		{ MP_BaseColor, SF_Pixel },
		{ MP_Normal, SF_Pixel },
		{ MP_OpacityMask, SF_Pixel },
	};
}

#if WITH_EDITOR
const FMaterialsWithDirtyUsageFlags FMaterialsWithDirtyUsageFlags::DefaultAnnotation;

void FMaterialsWithDirtyUsageFlags::MarkUsageFlagDirty(EMaterialUsage UsageFlag)
{
	MaterialFlagsThatHaveChanged |= (1u << UsageFlag);
}

bool FMaterialsWithDirtyUsageFlags::IsUsageFlagDirty(EMaterialUsage UsageFlag)
{
	return (MaterialFlagsThatHaveChanged & (1u << UsageFlag)) != 0;
}

FUObjectAnnotationSparseBool GMaterialsThatNeedSamplerFixup;
FUObjectAnnotationSparse<FMaterialsWithDirtyUsageFlags,true> GMaterialsWithDirtyUsageFlags;
FUObjectAnnotationSparseBool GMaterialsThatNeedExpressionsFlipped;
FUObjectAnnotationSparseBool GMaterialsThatNeedCoordinateCheck;
FUObjectAnnotationSparseBool GMaterialsThatNeedCommentFix;
FUObjectAnnotationSparseBool GMaterialsThatNeedDecalFix;
FUObjectAnnotationSparseBool GMaterialsThatNeedFeatureLevelSM6Fix;

#endif // #if WITH_EDITOR

FMaterialResource::FMaterialResource()
	: FMaterial()
	, Material(nullptr)
	, MaterialInstance(nullptr)
{
}

FMaterialResource::~FMaterialResource()
{
}

int32 FMaterialResource::CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const
{
#if WITH_EDITOR
	if (Compiler->ShouldStopTranslating())
	{
		return INDEX_NONE;
	}

	// needs to be called in this function!!
	// sets CurrentShaderFrequency
	Compiler->SetMaterialProperty(Property, OverrideShaderFrequency, bUsePreviousFrameTime);

	EShaderFrequency ShaderFrequency = Compiler->GetCurrentShaderFrequency();
	
	int32 SelectionColorIndex = INDEX_NONE;
	int32 SelectionColorToggle = INDEX_NONE;

	if (ShaderFrequency == SF_Pixel &&
		GetMaterialDomain() != MD_Volume &&
		Compiler->IsDevelopmentFeatureEnabled(NAME_SelectionColor))
	{
		// RGB stores SelectionColor value, A is toggle on/off switch for SelectionColor
		int32 SelectionColorVector = Compiler->VectorParameter(NAME_SelectionColor, FLinearColor::Transparent);
		SelectionColorIndex = Compiler->ComponentMask(SelectionColorVector, 1, 1, 1, 0);
		SelectionColorToggle = Compiler->ComponentMask(SelectionColorVector, 0, 0, 0, 1);
	}

	//Compile the material instance if we have one.
	UMaterialInterface* MaterialInterface = MaterialInstance ? static_cast<UMaterialInterface*>(MaterialInstance) : Material.Get();

	int32 Ret = INDEX_NONE;

	switch(Property)
	{
		case MP_EmissiveColor:
			if (SelectionColorIndex != INDEX_NONE)
			{
				// Alpha channel is used to as toggle between EmissiveColor and SelectionColor
				Ret = Compiler->Lerp(MaterialInterface->CompileProperty(Compiler, MP_EmissiveColor, MFCF_ForceCast), SelectionColorIndex, SelectionColorToggle);
			}
			else
			{
				Ret = MaterialInterface->CompileProperty(Compiler, MP_EmissiveColor);
			}
			break;

		case MP_DiffuseColor: 
			Ret = MaterialInterface->CompileProperty(Compiler, MP_DiffuseColor, MFCF_ForceCast);
			break;

		case MP_BaseColor: 
			Ret = MaterialInterface->CompileProperty(Compiler, MP_BaseColor, MFCF_ForceCast);
			break;

		case MP_Opacity:
		case MP_OpacityMask:
			// Force basic opaque surfaces to skip masked/translucent-only attributes.
			// Some features can force the material to create a masked variant which unintentionally runs this dormant code
			if (GetMaterialDomain() != MD_Surface || !IsOpaqueBlendMode(GetBlendMode()) || (GetShadingModels().IsLit() && !GetShadingModels().HasOnlyShadingModel(MSM_DefaultLit)))
			{
				Ret = MaterialInterface->CompileProperty(Compiler, Property);
			}
			else
			{
				Ret = FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property);
			}
			break;
		case MP_ShadingModel:
			if (AllowPerPixelShadingModels(Compiler->GetShaderPlatform()) 
				|| !MaterialInterface->IsShadingModelFromMaterialExpression())
			{
				Ret = MaterialInterface->CompileProperty(Compiler, Property);
			}
			else
			{
				// if platform does not support per-pixel shading models, always fallback to DefaultLit
				Ret = Compiler->ShadingModel(MSM_DefaultLit);
			}
			break;
		case MP_MaterialAttributes:
			Ret = MaterialInterface->CompileProperty(Compiler, Property);
			break;

		default:
			Ret = MaterialInterface->CompileProperty(Compiler, Property);
	};
	
	EMaterialValueType AttributeType = FMaterialAttributeDefinitionMap::GetValueType(Property);

	if (Ret != INDEX_NONE)
	{
		FMaterialUniformExpression* Expression = Compiler->GetParameterUniformExpression(Ret);

		if (Expression && Expression->IsConstant())
		{
			// Where possible we want to preserve constant expressions allowing default value checks
			EMaterialValueType ResultType = Compiler->GetParameterType(Ret);
			EMaterialValueType ExactAttributeType = (AttributeType == MCT_Float) ? MCT_Float1 : AttributeType;
			EMaterialValueType ExactResultType = (ResultType == MCT_Float) ? MCT_Float1 : ResultType;

			if (ExactAttributeType == ExactResultType)
			{
				return Ret;
			}
			else if (ResultType == MCT_Float || (ExactAttributeType == MCT_Float1 && ResultType & MCT_Float))
			{
				return Compiler->ComponentMask(Ret, true, ExactAttributeType >= MCT_Float2, ExactAttributeType >= MCT_Float3, ExactAttributeType >= MCT_Float4);
			}
		}
	}

	// MaterialAttributes are expected to give a void statement, don't need to cast that
	if (Property != MP_MaterialAttributes)
	{
		// Output should always be the right type for this property
		Ret = Compiler->ForceCast(Ret, AttributeType);
	}
	return Ret;

#else // WITH_EDITOR
	check(0); // This is editor-only function
	return INDEX_NONE;
#endif // WITH_EDITOR
}

int32 FMaterialResource::CompileCustomAttribute(const FGuid& AttributeID, FMaterialCompiler* Compiler) const
{
#if WITH_EDITOR
	return Material->CompilePropertyEx(Compiler, AttributeID);
#else // WITH_EDITOR
	check(0); // This is editor-only function
	return INDEX_NONE;
#endif // WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void FMaterialResource::GatherCustomOutputExpressions(TArray<UMaterialExpressionCustomOutput*>& OutCustomOutputs) const
{
	Material->GetAllCustomOutputExpressions(OutCustomOutputs);
}

void FMaterialResource::GatherExpressionsForCustomInterpolators(TArray<UMaterialExpression*>& OutExpressions) const
{
	Material->GetAllExpressionsForCustomInterpolators(OutExpressions);
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void FMaterialResource::BeginAllowCachingStaticParameterValues()
{
	if (MaterialInstance)
	{
		MaterialInstance->BeginAllowCachingStaticParameterValues();
	}
}

void FMaterialResource::EndAllowCachingStaticParameterValues()
{
	if (MaterialInstance)
	{
		MaterialInstance->EndAllowCachingStaticParameterValues();
	}
}
#endif // WITH_EDITOR

void FMaterialResource::BuildShaderMapIdOverride(const FBuildShaderMapIdArgs& Args) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialResource::BuildShaderMapIdOverride);

	FMaterial::BuildShaderMapIdOverride(Args);
#if WITH_EDITOR
	FMaterialShaderMapId& OutId = *Args.OutId;
	const FMaterialCachedExpressionData& CachedData = GetCachedExpressionData();
	CachedData.AppendReferencedFunctionIdsTo(OutId.ReferencedFunctions);
	CachedData.AppendReferencedParameterCollectionIdsTo(OutId.ReferencedParameterCollections);

	OutId.ExternalCodeReferencesHash = CachedData.GetExternalCodeReferencesHash();

	OutId.TextureReferencesHash = Material->GetForceRecompileTextureIdsHash();

	if (MaterialInstance)
	{
		OutId.BasePropertyOverridesHash = MaterialInstance->GetBasePropertyOverridesHash();

		FStaticParameterSet CompositedStaticParameters;
		MaterialInstance->GetStaticParameterValues(CompositedStaticParameters);
		OutId.UpdateFromParameterSet(CompositedStaticParameters);
	}
	else
	{
		FStaticParameterSet CompositedStaticParameters;
		Material->GetStaticParameterValues(CompositedStaticParameters);
		OutId.UpdateFromParameterSet(CompositedStaticParameters);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void FMaterialResource::GetStaticParameterSet(FStaticParameterSet& OutSet) const
{
	FMaterial::GetStaticParameterSet(OutSet);

	// Get the set from instance
	if (MaterialInstance)
	{
		MaterialInstance->GetStaticParameterValues(OutSet);
	}
	else
	{
		Material->GetStaticParameterValues(OutSet);
	}
}
#endif // WITH_EDITORONLY_DATA

/**
 * A resource which represents the default instance of a UMaterial to the renderer.
 * Note that default parameter values are stored in the FMaterialUniformExpressionXxxParameter objects now.
 * This resource is only responsible for the selection color.
 */
class FDefaultMaterialInstance : public FMaterialRenderProxy
{
public:

	/**
	 * Called from the game thread to destroy the material instance on the rendering thread.
	 */
	void GameThread_Destroy()
	{
		FDefaultMaterialInstance* Resource = this;
		ENQUEUE_RENDER_COMMAND(FDestroyDefaultMaterialInstanceCommand)(
			[Resource](FRHICommandList& RHICmdList)
		{
			delete Resource;
		});
	}

	virtual void StartCacheUniformExpressions() const override
	{
		Material->StartCacheUniformExpressions();
	}

	virtual void FinishCacheUniformExpressions() const override
	{
		Material->FinishCacheUniformExpressions();
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		const FMaterialRenderProxy* Fallback = &GetFallbackRenderProxy();
		if (Fallback == this)
		{ 
			// If we are the default material, must not try to fall back to the default material in an error state as that will be infinite recursion
			return nullptr;
		}
		return Fallback;
	}

	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		checkSlow(IsInParallelRenderingThread());
		const FMaterial* MaterialResource = Material->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
		if (MaterialResource && MaterialResource->GetRenderingThreadShaderMap())
		{
			return MaterialResource;
		}
		return nullptr;
	}

	virtual UMaterialInterface* GetMaterialInterface() const override
	{
		return Material;
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override
	{
		const FMaterialResource* MaterialResource = Material->GetMaterialResource(Context.Material.GetShaderPlatform());
		if (MaterialResource && MaterialResource->GetRenderingThreadShaderMap())
		{
			if (Type == EMaterialParameterType::Scalar && ParameterInfo.Name == SubsurfaceProfile::GetSubsurfaceProfileParameterName())
			{
				// When using Substrate with hidden conversion, this path can still be used. Since GetSubsurfaceProfileRT() won't be correct 
				// (it will retrieve the SSS profile override instead of the actual SSS Profile), we retrieve the SSS profile through the 
				// array version at index 0
				if (Substrate::IsSubstrateEnabled())
				{
					const USubsurfaceProfile* SSProfileOverrideRT = GetSubsurfaceProfileRT();
					const USubsurfaceProfile* SSProfileRT = NumSubsurfaceProfileRT() > 0 ? GetSubsurfaceProfileRT(0) : nullptr;
					// Set the root material Profile, or the profile overriden by any instances.
					OutValue = SubsurfaceProfile::GetSubsurfaceProfileId(SSProfileOverrideRT ? SSProfileOverrideRT : SSProfileRT);
					return true;
				}
				else
				{
					// Legacy single SubsurfaceProfile (Substrate do not add this one in the material shader)
					OutValue = SubsurfaceProfile::GetSubsurfaceProfileId(GetSubsurfaceProfileRT());
					return true;
				}
			}
			else if (Type == EMaterialParameterType::Scalar && NumSubsurfaceProfileRT() > 0)
			{
				const USubsurfaceProfile* SSProfileOverrideRT = GetSubsurfaceProfileRT();
				// Substrate general SubsurfaceProfile
				for (uint32 It = 0, Count = NumSubsurfaceProfileRT(); It < Count; ++It)
				{
					const USubsurfaceProfile* SSProfileRT = GetSubsurfaceProfileRT(It);
					if (ParameterInfo.Name == SubsurfaceProfile::CreateSubsurfaceProfileParameterName(SSProfileRT))
					{
						// Set the root material Profile, or the profile overriden by any instances.
						OutValue = SubsurfaceProfile::GetSubsurfaceProfileId(SSProfileOverrideRT ? SSProfileOverrideRT : SSProfileRT);
						return true;
					}
				}
			}
			else if (Type == EMaterialParameterType::Scalar && NumSpecularProfileRT() > 0)
			{
				for (uint32 It=0,Count=NumSpecularProfileRT();It<Count;++It)
				{
					if (ParameterInfo.Name == SpecularProfile::GetSpecularProfileParameterName(GetSpecularProfileRT(It)))
					{
						const USpecularProfile* SpecularProfile = GetSpecularProfileRT(It);
						const USpecularProfile* SpecularProfileOverride = GetSpecularProfileOverrideRT();
						OutValue = SpecularProfile::GetSpecularProfileId(SpecularProfileOverride ? SpecularProfileOverride : SpecularProfile);
						return true;
					}
				}
			}
			else if (Type == EMaterialParameterType::Scalar && NumToonProfileRT() > 0)
			{
				for (uint32 It=0,Count=NumToonProfileRT();It<Count;++It)
				{
					if (ParameterInfo.Name == ToonProfile::GetToonProfileParameterName(GetToonProfileRT(It)))
					{
						const UToonProfile* ToonProfile = GetToonProfileRT(It);
						const UToonProfile* ToonProfileOverride = GetToonProfileOverrideRT();
						OutValue = ToonProfile::GetToonProfileId(ToonProfileOverride ? ToonProfileOverride : ToonProfile);
						return true;
					}
				}
			}

			return false;
		}
		else
		{
			return GetFallbackRenderProxy().GetParameterValue(Type, ParameterInfo, OutValue, Context);
		}
	}

	// FRenderResource interface.
	virtual FString GetFriendlyName() const { return Material->GetName(); }

	// Constructor.
	FDefaultMaterialInstance(UMaterial* InMaterial)
		: FMaterialRenderProxy(GetPathNameSafe(InMaterial))
		, Material(InMaterial)
	{}

private:

	/** Get the fallback material. */
	FMaterialRenderProxy& GetFallbackRenderProxy() const
	{
		return *(UMaterial::GetDefaultMaterial(Material->MaterialDomain)->GetRenderProxy());
	}

	UMaterial* Material;
};

#if WITH_EDITOR
static bool GAllowCompilationInPostLoad=true;
#else
#define GAllowCompilationInPostLoad true
#endif

void UMaterial::ForceNoCompilationInPostLoad(bool bForceNoCompilation)
{
#if WITH_EDITOR
	GAllowCompilationInPostLoad = !bForceNoCompilation;
#endif
}

static UMaterialFunction* GConvertFromDiffSpecMaterialFunction = NULL;

static UMaterial* GDefaultMaterials[MD_MAX] = {0};
static bool       GDefaultMaterialsInitialized[MD_MAX] = {false};

static const TCHAR* GDefaultMaterialNames[MD_MAX] =
{
	// Surface
	TEXT("engine-ini:/Script/Engine.Engine.DefaultMaterialName"),
	// Deferred Decal
	TEXT("engine-ini:/Script/Engine.Engine.DefaultDeferredDecalMaterialName"),
	// Light Function
	TEXT("engine-ini:/Script/Engine.Engine.DefaultLightFunctionMaterialName"),
	// Volume
	//@todo - get a real MD_Volume default material
	TEXT("engine-ini:/Script/Engine.Engine.DefaultMaterialName"),
	// Post Process
	TEXT("engine-ini:/Script/Engine.Engine.DefaultPostProcessMaterialName"),
	// User Interface 
	TEXT("engine-ini:/Script/Engine.Engine.DefaultMaterialName"),
	// Virtual Texture
	TEXT("engine-ini:/Script/Engine.Engine.DefaultMaterialName")
};

// Need to know if the default materials have been initialized in case dynamic shader preloading is enabled,
// we don't want to kick preload shader jobs since they can call GetDefaultMaterial and causing a crash in InitDefaultMaterials.
static bool bDefaultMaterialInitialized = false;
static bool bDefaultMaterialInitializationStarted = false;

bool UMaterialInterface::IsDefaultMaterialInitialized()
{
	return bDefaultMaterialInitialized;
}

UMaterial* TryResolveDefaultMaterial(int32 Domain)
{
	FString ResolvedPath = ResolveIniObjectsReference(GDefaultMaterialNames[Domain]);
	return FindObject<UMaterial>(nullptr, *ResolvedPath);
}

// We can save time if instead of blocking after compilation of each synchronous material we block after scheduling all of them
bool GPoolSpecialMaterialsCompileJobs = true;
bool PoolSpecialMaterialsCompileJobs()
{
	return GPoolSpecialMaterialsCompileJobs;
}

namespace MaterialImpl
{
	void ConditionalFinishInitialization(int32 Domain)
	{
		if (!bDefaultMaterialInitialized)
		{
			SCOPED_BOOT_TIMING("UMaterialInterface::InitDefaultMaterials");

			if (!GDefaultMaterialsInitialized[Domain])
			{
				FString ResolvedPath = ResolveIniObjectsReference(GDefaultMaterialNames[Domain]);
				GDefaultMaterials[Domain] = FindObject<UMaterial>(nullptr, *ResolvedPath);
				checkf(GDefaultMaterials[Domain] != nullptr, TEXT("Cannot load default material '%s' [Domain=%s] from path '%s'"), GDefaultMaterialNames[Domain], *MaterialDomainString((EMaterialDomain)Domain), *ResolvedPath);
				if (GDefaultMaterials[Domain])
				{
					GDefaultMaterials[Domain]->AddToRoot();
				}
				GDefaultMaterialsInitialized[Domain] = true;
			}

			bool bAllDefaultMaterialInitialized = true;
			for (int32 Index = 0; Index < MD_MAX; ++Index)
			{
				bAllDefaultMaterialInitialized &= GDefaultMaterialsInitialized[Index];
			}
	
			if (bAllDefaultMaterialInitialized)
			{
				bDefaultMaterialInitialized = true;

				// Block after scheduling for compilation all (hopefully) default materials.
				// Even if not all of them ended up being post-loaded, block here just out of extra caution
				if (GPoolSpecialMaterialsCompileJobs == true)
				{
					GPoolSpecialMaterialsCompileJobs = false;
					if (GShaderCompilingManager)
					{
						GShaderCompilingManager->FinishAllCompilation();
					}
				}

				// Now precache PSOs for MD_Surface default material after the default materials are marked initialize
				// PSO precaching can request default materials so they have to marked as initialized to avoid endless recursion
				if (PipelineStateCache::IsPSOPrecachingEnabled() || IsPSOShaderPreloadingEnabled())
				{
					UMaterialInterface::PrecacheFallbackMaterialPSOs(GDefaultMaterials[MD_Surface]);
				}
			}
		}
	}
}

void UMaterialInterface::InitDefaultMaterials()
{
	// The check for initialization is purely an optimization as initializing
	// the default materials is only done very early in the boot process.
	if (!bDefaultMaterialInitializationStarted)
	{
		SCOPED_BOOT_TIMING("UMaterialInterface::InitDefaultMaterials");
		check(IsInGameThread());
		if (!IsInGameThread())
		{
			return;
		}

		// We only do this step once, launch async loading tasks with top priority to fullfill these requests as soon as possible.
		// When done loading, these will call ConditionalFinishInitialization which handles the remaining of the initialization.
		bDefaultMaterialInitializationStarted = true;

		// Explicitly initialize UMaterialInterface and UMaterial CDOs in this order to avoid issues with recursion and
		// async loading depending on initialization order of compiled-in UObjects.
		// i.e. when UE_WITH_CONSTINIT_UOBJECT is set.
		UMaterialInterface::StaticClass()->GetDefaultObject();
		UMaterial::StaticClass()->GetDefaultObject();
		
		for (int32 Domain = 0; Domain < MD_MAX; ++Domain)
		{
			FString ResolvedPath = ResolveIniObjectsReference(GDefaultMaterialNames[Domain]);

			FLoadPackageAsyncOptionalParams Params;
			Params.PackagePriority = MAX_int32;
			Params.CompletionDelegate = MakeUnique<FLoadPackageAsyncDelegate>(FLoadPackageAsyncDelegate::CreateLambda(
				[Domain](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
				{
					MaterialImpl::ConditionalFinishInitialization(Domain);
				}));

			LoadPackageAsync(FPackagePath::FromPackageNameChecked(FTopLevelAssetPath(ResolvedPath).GetPackageName()), MoveTemp(Params));
		}
	}
}

void UMaterialInterface::PrecacheDefaultMaterialPSOs()
{
	PrecacheFallbackMaterialPSOs(GDefaultMaterials[MD_Surface]);
}

void UMaterialInterface::PrecacheFallbackMaterialPSOs(UMaterialInterface* MaterialInterface)
{
	if (!GIsRHIInitialized || (MaterialInterface == nullptr) || !UPSOPrecacheSettingsManager::GetSettings().bAllDefaultMaterialPSOs)
	{
		return;
	}
	
	if (!PipelineStateCache::IsPSOPrecachingEnabled() && !IsPSOShaderPreloadingEnabled())
	{
		return;
	}

	TArray<FMaterialPSOPrecacheRequestID> MaterialPrecacheRequestIDs;

	FPSOPrecacheParams PrecachePSOParams;
	PrecachePSOParams.bDefaultMaterial = true;
	PrecachePSOParams.bRequireRTDynamicGeometryUpdate = true;
	FPSOPrecacheVertexFactoryDataList AllVertexFactoryTypes;
	for (TLinkedList<FVertexFactoryType*>::TIterator It(FVertexFactoryType::GetTypeList()); It; It.Next())
	{
		FPSOPrecacheVertexFactoryData VFData;
		VFData.VertexFactoryType = *It;
		AllVertexFactoryTypes.Add(VFData);
	}

	auto PrecachePSOs = [ & ](EComponentMobility::Type Mobility, bool bReverseCulling)
	{
		PrecachePSOParams.Mobility = Mobility;
		PrecachePSOParams.bReverseCulling = bReverseCulling;
		MaterialInterface->PrecachePSOs(AllVertexFactoryTypes, PrecachePSOParams, EPSOPrecachePriority::High, MaterialPrecacheRequestIDs);
	};

	PrecachePSOs(EComponentMobility::Static, false /*bReverseCulling*/);
	PrecachePSOs(EComponentMobility::Movable, false /*bReverseCulling*/);
	PrecachePSOs(EComponentMobility::Static, true /*bReverseCulling*/);
	PrecachePSOs(EComponentMobility::Movable, true /*bReverseCulling*/);
}

void UMaterialInterface::PostCDOContruct()
{
	if (FPlatformProperties::RequiresCookedData())
	{
		UMaterial::StaticClass()->GetDefaultObject();
		UMaterialInterface::InitDefaultMaterials();
	}
}

void UMaterialInterface::PostLoadDefaultMaterials()
{
	LLM_SCOPE(ELLMTag::Materials);

	// Here we prevent this function from being called recursively. Mostly this
	// is an optimization and guarantees that default materials are post loaded
	// in the order material domains are defined. Surface -> deferred decal -> etc.
	static bool bPostLoaded = false;
	static TSet<UMaterialInterface*> OnStack;
	if (!bPostLoaded)
	{
		check(IsInGameThread());
		bPostLoaded = true;

		if (!bDefaultMaterialInitialized)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialInterface::PostLoadDefaultMaterials);

			check(IsInGameThread());
			OnStack.Add(this);
			ON_SCOPE_EXIT { OnStack.Remove(this); };

			for (int32 Domain = 0; Domain < MD_MAX; ++Domain)
			{
				UMaterial* Material = TryResolveDefaultMaterial(Domain);
				if (OnStack.Contains(Material))
				{
					// We obviously can't sync load ourselves while we're in our own postload function
					continue;
				}

				// This will forceload the object if it hasn't already been loaded.
				// Sync loading from inside PostLoad is supported by the loader.
				Material = UMaterial::GetDefaultMaterial((EMaterialDomain)Domain);
				check(Material);
				if (Material)
				{
					// The only reason these would fail is if there is a recursive loads.. (i.e. a material trying to load and postload itself indirectly since we already handle the direct case)
					check(!Material->HasAnyFlags(RF_NeedLoad));
					check(!Material->HasAnyFlags(RF_NeedPostLoad));
				}
			}
		}
	}
}

void UMaterialInterface::AssertDefaultMaterialsExist()
{
	for (int32 Domain = 0; Domain < MD_MAX; ++Domain)
	{
		check(GDefaultMaterials[Domain] != NULL);
	}
}

void UMaterialInterface::AssertDefaultMaterialsPostLoaded()
{
	for (int32 Domain = 0; Domain < MD_MAX; ++Domain)
	{
		check(GDefaultMaterials[Domain] != NULL);
		check(!GDefaultMaterials[Domain]->HasAnyFlags(RF_NeedPostLoad));
	}
}

FString MaterialDomainString(EMaterialDomain MaterialDomain)
{
	static const UEnum* Enum = StaticEnum<EMaterialDomain>();
	check(Enum);
	return Enum->GetNameStringByValue(int64(MaterialDomain));
}

static TAutoConsoleVariable<int32> CVarDiscardUnusedQualityLevels(
	TEXT("r.DiscardUnusedQuality"),
	0,
	TEXT("Whether to keep or discard unused quality level shadermaps in memory.\n")
	TEXT("0: keep all quality levels in memory. (default)\n")
	TEXT("1: Discard unused quality levels on load."),
	ECVF_ReadOnly);

namespace UE::MaterialInterface::Private
{

void SerializeInlineShaderMaps(
	FArchive& Ar,
	TArray<FMaterialResource>& OutLoadedResources,
	const FName& SerializingAsset
#if WITH_EDITOR
	, const TMap<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>* PlatformMaterialResourcesToSavePtr
#endif
)
{
	LLM_SCOPE(ELLMTag::Shaders);
	SCOPED_LOADTIMER(SerializeInlineShaderMaps);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Ar.IsSaving())
	{
#if WITH_EDITOR
		int32 NumResourcesToSave = 0;
		const TArray<FMaterialResourceForCooking> *MaterialResourcesToSavePtr = nullptr;
		if (Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData())
		{
			check( PlatformMaterialResourcesToSavePtr );
			auto& PlatformMaterialResourcesToSave = *PlatformMaterialResourcesToSavePtr;

			MaterialResourcesToSavePtr = PlatformMaterialResourcesToSave.Find( Ar.CookingTarget() );
			check(MaterialResourcesToSavePtr != nullptr || Ar.GetLinker() == nullptr);
			if (MaterialResourcesToSavePtr!= nullptr )
			{
				NumResourcesToSave = MaterialResourcesToSavePtr->Num();
			}
		}

		Ar << NumResourcesToSave;

		if (MaterialResourcesToSavePtr
			&& NumResourcesToSave > 0)
		{
			FMaterialResourceMemoryWriter ResourceAr(Ar);
			for (const FMaterialResourceForCooking& ResourceForCooking : *MaterialResourcesToSavePtr)
			{
				FMaterialResource& Resource = *ResourceForCooking.Resource;
				FMaterialResourceWriteScope Scope(&ResourceAr, Resource);
				Resource.SerializeInlineShaderMap(ResourceAr);
			}
		}
#else
		int32 NumResourcesToSave = 0;
		Ar << NumResourcesToSave;
#endif
	}
	else if (Ar.IsLoading())
	{
		int32 NumLoadedResources = 0;
		Ar << NumLoadedResources;

		if (NumLoadedResources > 0)
		{
			ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
			EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num;
			OutLoadedResources.Empty(NumLoadedResources);
			FMaterialResourceProxyReader ResourceAr(Ar, FeatureLevel, QualityLevel);
			for (int32 ResourceIndex = 0; ResourceIndex < NumLoadedResources; ++ResourceIndex)
			{
				FMaterialResource& LoadedResource = OutLoadedResources[OutLoadedResources.AddDefaulted()];
				LoadedResource.SerializeInlineShaderMap(ResourceAr, SerializingAsset);
			}
		}
	}
}

} // namespace UE::MaterialInterface::Private

void ProcessSerializedInlineShaderMaps(UMaterialInterface* Owner, TArray<FMaterialResource>& LoadedResources, TArray<FMaterialResource*>& OutMaterialResourcesLoaded)
{
	LLM_SCOPE(ELLMTag::Shaders);
	check(IsInGameThread());

	if (LoadedResources.Num() == 0)
	{
		// Nothing to process
		return;
	}
	UMaterialInstance* OwnerMaterialInstance = Cast<UMaterialInstance>(Owner);
	UMaterial* OwnerMaterial = nullptr;
	if (OwnerMaterialInstance)
	{
		OwnerMaterial = OwnerMaterialInstance->GetBaseMaterial();
	}
	else
	{
		OwnerMaterial = CastChecked<UMaterial>(Owner);
	}

#if WITH_EDITORONLY_DATA
	const bool bLoadingCooked = FPlatformProperties::RequiresCookedData() || Owner->GetOutermost()->bIsCookedForEditor;
#else
	const bool bLoadingCooked = FPlatformProperties::RequiresCookedData();
#endif
	for (FMaterialResource& Resource : LoadedResources)
	{
		Resource.RegisterInlineShaderMap(bLoadingCooked);
	}

	const bool bDiscardUnusedQualityLevels = CVarDiscardUnusedQualityLevels.GetValueOnAnyThread() != 0;
	const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;

	for (int32 ResourceIndex = 0; ResourceIndex < LoadedResources.Num(); ResourceIndex++)
	{
		FMaterialResource& LoadedResource = LoadedResources[ResourceIndex];
		FMaterialShaderMap* LoadedShaderMap = LoadedResource.GetGameThreadShaderMap();

		if (LoadedShaderMap && LoadedShaderMap->GetShaderPlatform() == GMaxRHIShaderPlatform)
		{
			const EMaterialQualityLevel::Type LoadedQualityLevel = LoadedShaderMap->GetShaderMapId().QualityLevel;
			const ERHIFeatureLevel::Type LoadedFeatureLevel = LoadedShaderMap->GetShaderMapId().FeatureLevel;
			check(GShaderPlatformForFeatureLevel[LoadedFeatureLevel] == GMaxRHIShaderPlatform);

			bool bIncludeShaderMap = true;
			if (bDiscardUnusedQualityLevels)
			{
				// Only include shader map if QL matches, or doesn't depend on QL
				bIncludeShaderMap = (LoadedQualityLevel == ActiveQualityLevel) || (LoadedQualityLevel == EMaterialQualityLevel::Num);
			}

			if (bIncludeShaderMap)
			{
#if WITH_ODSC
				// Check if already have something from ODSC so that we avoid using this shadermap for a few frames and making the request to replace it by something we already have
				// If some permutations are missing, we will still perform additional requests
				if (FODSCManager::IsODSCActive())
				{
					FMaterialShaderMap* ExistingShaderMap = FODSCManager::FindMaterialShaderMap(Owner->GetPathName(), LoadedShaderMap->GetShaderMapId());
					if (ExistingShaderMap)
					{
						LoadedShaderMap = ExistingShaderMap;
					}
				}
#endif

				FMaterialResource* CurrentResource = FindOrCreateMaterialResource(OutMaterialResourcesLoaded, OwnerMaterial, OwnerMaterialInstance, LoadedShaderMap->GetShaderPlatform(), LoadedQualityLevel);
				CurrentResource->SetInlineShaderMap(LoadedShaderMap);
			}
		}
	}
}

// Deprecated in 5.7
extern FMaterialResource* FindMaterialResource(const TArray<FMaterialResource*>& MaterialResources, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality)
{
	return FindMaterialResource(const_cast<TArray<FMaterialResource*>&>(MaterialResources), GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), QualityLevel, bAllowDefaultQuality);
}

// Deprecated in 5.7
FMaterialResource* FindMaterialResource(TArray<FMaterialResource*>& MaterialResources, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality)
{
	return FindMaterialResource(const_cast<TArray<FMaterialResource*>&>(MaterialResources), GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), QualityLevel, bAllowDefaultQuality);
}

FMaterialResource* FindMaterialResource(const TArray<FMaterialResource*>& MaterialResources, EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality)
{
	return FindMaterialResource(const_cast<TArray<FMaterialResource*>&>(MaterialResources), InShaderPlatform, QualityLevel, bAllowDefaultQuality);
}

FMaterialResource* FindMaterialResource(TArray<FMaterialResource*>& MaterialResources, EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality)
{
	FMaterialResource* DefaultResource = nullptr;
	for (int32 ResourceIndex = 0, NumMaterialResources = MaterialResources.Num(); ResourceIndex < NumMaterialResources; ++ResourceIndex)
	{
		FMaterialResource* CurrentResource = MaterialResources[ResourceIndex];
		if (CurrentResource->GetShaderPlatform() == InShaderPlatform)
		{
			const EMaterialQualityLevel::Type CurrentQualityLevel = CurrentResource->GetQualityLevel();
			if (CurrentQualityLevel == QualityLevel)
			{
				// exact match
				return CurrentResource;
			}
			else if (bAllowDefaultQuality && CurrentQualityLevel == EMaterialQualityLevel::Num)
			{
				// return the default resource, if we don't find a resource for the requested quality level
				DefaultResource = CurrentResource;
			}
		}
	}
	return DefaultResource;
}

// Deprecated in 5.7
FMaterialResource* FindOrCreateMaterialResource(TArray<FMaterialResource*>& MaterialResources,
	UMaterial* OwnerMaterial,
	UMaterialInstance* OwnerMaterialInstance,
	ERHIFeatureLevel::Type InFeatureLevel,
	EMaterialQualityLevel::Type InQualityLevel)
{
	return FindOrCreateMaterialResource(MaterialResources, OwnerMaterial, OwnerMaterialInstance, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), InQualityLevel);
}

FMaterialResource* FindOrCreateMaterialResource(TArray<FMaterialResource*>& MaterialResources,
	UMaterial* OwnerMaterial,
	UMaterialInstance* OwnerMaterialInstance,
	EShaderPlatform InShaderPlatform,
	EMaterialQualityLevel::Type InQualityLevel)
{
	check(OwnerMaterial);

	EMaterialQualityLevel::Type QualityLevelForResource = InQualityLevel;
	if (InQualityLevel != EMaterialQualityLevel::Num)
	{
		// See if we have an explicit resource for the requested quality
		TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num>> QualityLevelsUsed;
		if (OwnerMaterialInstance)
		{
			OwnerMaterialInstance->GetQualityLevelUsage(QualityLevelsUsed, InShaderPlatform);
		}
		else
		{
			OwnerMaterial->GetQualityLevelUsage(QualityLevelsUsed, InShaderPlatform);
		}
		if (!QualityLevelsUsed[InQualityLevel])
		{
			// No explicit resource, just use the default
			QualityLevelForResource = EMaterialQualityLevel::Num;
		}
	}

	FMaterialResource* CurrentResource = FindMaterialResource(MaterialResources, InShaderPlatform, QualityLevelForResource, false);
	if (!CurrentResource)
	{
		CurrentResource = OwnerMaterialInstance ? OwnerMaterialInstance->AllocatePermutationResource() : OwnerMaterial->AllocateResource();
		CurrentResource->SetMaterial(OwnerMaterial, OwnerMaterialInstance, InShaderPlatform, QualityLevelForResource);
		MaterialResources.Add(CurrentResource);
	}
	else
	{
		// Make sure the resource we found still has the correct owner
		// This needs to be updated for various complicated reasons...
		// * Since these pointers are passed to reference collector, the GC may null them out
		// * Landscape does lots of complicated material reparenting under the hood, which can cause these pointers to get stale
		CurrentResource->SetMaterial(OwnerMaterial);
		CurrentResource->SetMaterialInstance(OwnerMaterialInstance);
	}

	return CurrentResource;
}

UMaterial* UMaterial::GetDefaultMaterial(EMaterialDomain Domain)
{
	InitDefaultMaterials();
	check(Domain >= MD_Surface && Domain < MD_MAX);

	if (!GDefaultMaterialsInitialized[Domain])
	{
		FString ResolvedPath = ResolveIniObjectsReference(GDefaultMaterialNames[Domain]);
		LoadObject<UMaterial>(nullptr, *ResolvedPath, nullptr, LOAD_DisableDependencyPreloading, nullptr);
		MaterialImpl::ConditionalFinishInitialization(Domain);
	}

	check(GDefaultMaterials[Domain] != nullptr);
	UMaterial* Default = GDefaultMaterials[Domain];
	return Default;
}

bool UMaterial::IsDefaultMaterial() const
{
	bool bDefault = false;
	for (int32 Domain = MD_Surface; !bDefault && Domain < MD_MAX; ++Domain)
	{
		bDefault = (this == GDefaultMaterials[Domain]);
	}

	if (!bDefaultMaterialInitialized)
	{
		for (int32 Domain = 0; !bDefault && Domain < MD_MAX; ++Domain)
		{
			bDefault = (this == TryResolveDefaultMaterial(Domain));
		}
	}

	return bDefault;
}

UMaterial::UMaterial()
	: Super()
	, ReleasedByRT(true)
{
	SetInitialValues();
}

UMaterial::UMaterial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ReleasedByRT(true)
{
	SetInitialValues();
}

UMaterial::UMaterial(FVTableHelper& Helper)
	: Super(Helper)
	, ReleasedByRT(true)
{
	SetInitialValues();
}

void UMaterial::SetInitialValues()
{
	BlendMode = BLEND_Opaque;
	ShadingModel = MSM_DefaultLit;
	ShadingModels = FMaterialShadingModelField(ShadingModel); 
	TranslucencyLightingMode = TLM_VolumetricNonDirectional;
	TranslucencyDirectionalLightingIntensity = 1.0f;
	TranslucentShadowDensityScale = 0.5f;
	TranslucentSelfShadowDensityScale = 2.0f;
	TranslucentSelfShadowSecondDensityScale = 10.0f;
	TranslucentSelfShadowSecondOpacity = 0.0f;
	TranslucentBackscatteringExponent = 30.0f;
	TranslucentMultipleScatteringExtinction = FLinearColor(1.0f, 0.833f, 0.588f, 1.0f);
	TranslucentShadowStartOffset = 100.0f;

#if WITH_EDITORONLY_DATA
	DiffuseColor_DEPRECATED.Constant = FColor(128,128,128);
	SpecularColor_DEPRECATED.Constant = FColor(128,128,128);
#endif
	OpacityMaskClipValue = 0.3333f;
	bCastDynamicShadowAsMasked = false;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bUsedWithStaticLighting = false;
	bUsedWithWater = false;
	bUsedWithVolumetricCloud = false;
	bUsedWithHeterogeneousVolumes = false;
	bUsedWithStaticMesh = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bEnableSeparateTranslucency_DEPRECATED = true;
	bEnableMobileSeparateTranslucency = false;
	TranslucencyPass = MTP_AfterDOF;
	bEnableResponsiveAA = false;
	bScreenSpaceReflections = false;
	bContactShadows = false;
	bTangentSpaceNormal = true;
	bUseLightmapDirectionality = true;
	bAutomaticallySetUsageInEditor = true;

	bUseMaterialAttributes = false;
	bCastRayTracedShadows = true;
	bUseTranslucencyVertexFog = true;
	bAllowFrontLayerTranslucency = true;
	bCompatibleWithLumenCardSharing = false;
	bHasPixelAnimation = false;
	bApplyCloudFogging = false;
	bIsSky = false;
	BlendableLocation = BL_SceneColorAfterTonemapping;
	BlendablePriority = 0;
	BlendableOutputAlpha = false;
	bIsBlendable = true;
	PreshaderGap = 0;
	bDisablePreExposureScale = false;
	bEnableStencilTest = false;

	// Shadow on translucent material
	bAllowTranslucentLocalLightShadow = false;
	TranslucentLocalLightShadowQuality = 0;
	TranslucentDirectionalLightShadowQuality = 0;

	// We default to false=incompatible, and rely onto the automatic detection executed from the HLSLTranslator.
	bForceCompatibleWithLightFunctionAtlas = false;

	bUseEmissiveForDynamicAreaLighting = false;
	RefractionDepthBias = 0.0f;
	MaterialDecalResponse = MDR_ColorNormalRoughness;

	SubstrateCompilationConfig = FSubstrateCompilationConfig();
	bSubstrateRoughnessTracking = true;

	bAllowDevelopmentShaderCompile = true;
	bIsMaterialEditorStatsMaterial = false;

	bRelaxRuntimeVirtualTextureRestrictions = false;
	bRelaxRuntimeVirtualTextureRestrictionsForOldContent = false;

	RefractionMethod = RM_None;
	RefractionCoverageMode = RCM_CoverageAccountedFor;
	PixelDepthOffsetMode = PDOM_AlongCameraVector;

	bAllowVariableRateShading = true;

#if WITH_EDITORONLY_DATA
	MaterialGraph = NULL;
#endif //WITH_EDITORONLY_DATA

	bIsPreviewMaterial = false;
	bIsFunctionPreviewMaterial = false;

	PhysMaterial = nullptr;
	PhysMaterialMask = nullptr;

	FloatPrecisionMode = EMaterialFloatPrecisionMode::MFPM_Default;

	bRootNodeOverridesDefaultDistortion_DEPRECATED = false;
}

void UMaterial::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
#if WITH_EDITOR
	GMaterialsWithDirtyUsageFlags.RemoveAnnotation(this);
#endif
}

#if WITH_EDITOR
void UMaterial::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);
	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		const ITargetPlatform* TargetPlatform = CookContext.GetTargetPlatform();
		check(TargetPlatform);

		TArray<FMaterialResourceForCooking>* Resources = CachedMaterialResourcesForCooking.Find(TargetPlatform);
		UE::MaterialInterface::Private::RecordMaterialDependenciesForCook(CookContext,
			Resources ? *Resources : TArray<FMaterialResourceForCooking>());
	}
}
#endif

void UMaterial::PostInitProperties()
{
	LLM_SCOPE(ELLMTag::Materials);

	Super::PostInitProperties();
	
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// CDO needs a deterministic guid for incremental cook of blueprints.
		StateId = FGuid();
	}
	else
	{
		DefaultMaterialInstance = new FDefaultMaterialInstance(this);

		// Initialize StateId to something unique, in case this is a new material
		FPlatformMisc::CreateGuid(StateId);
	}
}

FMaterialResource* UMaterial::AllocateResource()
{
	LLM_SCOPE(ELLMTag::Materials);

	return new FMaterialResource();
}

void UMaterial::GetUsedTextures(TArray<UTexture*>& OutTextures, TOptional<EMaterialQualityLevel::Type> InSpecificQualityLevel, TOptional<EShaderPlatform> InSpecificShaderPlatform) const
{
	OutTextures.Empty();

	if (!FPlatformProperties::IsServerOnly())
	{
		const FInt32Range QualityLevelRange = BuildQualityLevelRange(InSpecificQualityLevel);
		const FInt32Range ShaderPlatformRange = BuildShaderPlatformRange(InSpecificShaderPlatform);

		TArray<const FMaterialResource*, TInlineAllocator<4>> MatchedResources = GetMaterialResources(this, QualityLevelRange, ShaderPlatformRange);
		for (const FMaterialResource* CurrentResource : MatchedResources)
		{
			GetTextureExpressionValues(CurrentResource, OutTextures, nullptr, true, false);
		}
	}
}

void UMaterial::GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform) const
{
	OutTextures.Empty();
	OutIndices.Empty();

	check(QualityLevel != EMaterialQualityLevel::Num && InShaderPlatform != EShaderPlatform::SP_NumPlatforms);

	if (!FPlatformProperties::IsServerOnly())
	{
		const FMaterialResource* CurrentResource = FindMaterialResource(MaterialResources, InShaderPlatform, QualityLevel, true);
		if (CurrentResource)
		{
			GetTextureExpressionValues(CurrentResource, OutTextures, &OutIndices, false, false);
		}
	}
}

void UMaterial::StartCacheUniformExpressions() const
{
	bCachingUniformExpressions = true;
}

void UMaterial::FinishCacheUniformExpressions() const
{
	bCachingUniformExpressions = false;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UMaterial::LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const
{
	auto World = GetWorld();
	const EMaterialQualityLevel::Type QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	const ERHIFeatureLevel::Type FeatureLevel = World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel;

	Ar.Logf(TEXT("%sMaterial: %s"), FCString::Tab(Indent), *GetName());

	if (FPlatformProperties::IsServerOnly())
	{
		Ar.Logf(TEXT("%sNo Textures: IsServerOnly"), FCString::Tab(Indent + 1));
	}
	else
	{
		const FMaterialResource* MaterialResource = FindMaterialResource(MaterialResources, GetFeatureLevelShaderPlatform_Checked(FeatureLevel), QualityLevel, false);
		if (MaterialResource)
		{
			if (MaterialResource->HasValidGameThreadShaderMap())
			{
				TArray<UTexture*> Textures;
				// GetTextureExpressionValues(MaterialResource, Textures);
				{
					for (int32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; TypeIndex++)
					{
						for (const FMaterialTextureParameterInfo& Parameter : MaterialResource->GetUniformTextureExpressions((EMaterialTextureParameterType)TypeIndex))
						{
							UTexture* Texture = NULL;
							Parameter.GetGameThreadTextureValue(this, *MaterialResource, Texture);
							if (Texture)
							{
								Textures.AddUnique(Texture);
							}
						}
					}
				}

				for (UTexture* Texture : Textures)
				{
					if (Texture)
					{
						Ar.Logf(TEXT("%s%s"), FCString::Tab(Indent + 1), *Texture->GetName());
					}
				}
			}
			else
			{
				Ar.Logf(TEXT("%sNo Textures : Invalid GameThread ShaderMap"), FCString::Tab(Indent + 1));
			}
		}
		else
		{
			Ar.Logf(TEXT("%sNo Textures : Invalid MaterialResource"), FCString::Tab(Indent + 1));
		}
	}
}
#endif

void UMaterial::OverrideTexture(const UTexture* InTextureToOverride, UTexture* OverrideTexture, ERHIFeatureLevel::Type InFeatureLevel)
{
#if WITH_EDITOR
	bool bShouldRecacheMaterialExpressions = false;
	ERHIFeatureLevel::Type FeatureLevelsToUpdate[1] = { InFeatureLevel };
	int32 NumFeatureLevelsToUpdate = 1;
	
	for (int32 FeatureLevelIndex = 0; FeatureLevelIndex < NumFeatureLevelsToUpdate; ++FeatureLevelIndex)
	{
		FMaterialResource* Resource = GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(FeatureLevelsToUpdate[FeatureLevelIndex]));
		if (Resource)
		{
			// Iterate over both the 2D textures and cube texture expressions.
			for (int32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; TypeIndex++)
			{
				const TArrayView<const FMaterialTextureParameterInfo> Parameters = Resource->GetUniformTextureExpressions((EMaterialTextureParameterType)TypeIndex);
				// Iterate over each of the material's texture expressions.
				for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ++ParameterIndex)
				{
					const FMaterialTextureParameterInfo& Parameter = Parameters[ParameterIndex];

					// Evaluate the expression in terms of this material instance.
					UTexture* Texture = NULL;
					Parameter.GetGameThreadTextureValue(this, *Resource, Texture);
					if (Texture != NULL && Texture == InTextureToOverride)
					{
						// Override this texture!
						Resource->TransientOverrides.SetTextureOverride((EMaterialTextureParameterType)TypeIndex, Parameter, OverrideTexture);
						bShouldRecacheMaterialExpressions = true;
					}
				}
			}
		}
	}

	if (bShouldRecacheMaterialExpressions)
	{
		RecacheUniformExpressions(false);
	}
#endif // WITH_EDITOR
}

bool ShouldRecacheMaterialExpressions(const FMaterialResource* Resource, const FHashedMaterialParameterInfo& ParameterInfo)
{
	if (!Resource)
	{
		return false;
	}

	const TArrayView<const FMaterialNumericParameterInfo> Parameters = Resource->GetUniformNumericParameterExpressions();
	// Iterate over each of the material's vector expressions.
	for (int32 i = 0; i < Parameters.Num(); ++i)
	{
		const FMaterialNumericParameterInfo& Parameter = Parameters[i];
		if (Parameter.ParameterInfo == ParameterInfo)
		{
			return true;
		}
	}

	return false;
}

void UMaterial::SetNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value, EShaderPlatform InShaderPlatform)
{
#if WITH_EDITOR
	FMaterialResource* Resource = GetMaterialResource(InShaderPlatform);
	if (Resource)
	{
		Resource->TransientOverrides.SetNumericOverride(Type, ParameterInfo, Value);
		if (ShouldRecacheMaterialExpressions(Resource, ParameterInfo))
		{
			RecacheUniformExpressions(false);
		}
	}
#endif // #if WITH_EDITOR
}

void UMaterial::ClearNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, EShaderPlatform InShaderPlatform)
{
#if WITH_EDITOR
	FMaterialResource* Resource = GetMaterialResource(InShaderPlatform);
	if (Resource)
	{
		Resource->TransientOverrides.ClearNumericOverride(Type, ParameterInfo);
		if (ShouldRecacheMaterialExpressions(Resource, ParameterInfo))
		{
			RecacheUniformExpressions(false);
		}
	}
#endif // #if WITH_EDITOR
}

void UMaterial::RecacheUniformExpressions(bool bRecreateUniformBuffer) const
{
	// Ensure that default material is available before caching expressions.
	UMaterial::GetDefaultMaterial(MD_Surface);

	if (DefaultMaterialInstance)
	{
		DefaultMaterialInstance->CacheUniformExpressions_GameThread(bRecreateUniformBuffer);
	}

#if WITH_EDITOR
	// Need to invalidate all child material instances as well.
	RecacheMaterialInstanceUniformExpressions(this, bRecreateUniformBuffer);
#endif
}

uint32 UMaterial::GetUsageFlags() const
{
	uint32 UsageFlags = 0;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UsageFlags |= bUsedWithSkeletalMesh ? (1u << MATUSAGE_SkeletalMesh) : 0;
	UsageFlags |= bUsedWithParticleSprites ? (1u << MATUSAGE_ParticleSprites) : 0;
	UsageFlags |= bUsedWithBeamTrails ? (1u << MATUSAGE_BeamTrails) : 0;
	UsageFlags |= bUsedWithMeshParticles ? (1u << MATUSAGE_MeshParticles) : 0;
	UsageFlags |= bUsedWithNiagaraSprites ? (1u << MATUSAGE_NiagaraSprites) : 0;
	UsageFlags |= bUsedWithNiagaraRibbons ? (1u << MATUSAGE_NiagaraRibbons) : 0;
	UsageFlags |= bUsedWithNiagaraMeshParticles ? (1u << MATUSAGE_NiagaraMeshParticles) : 0;
	UsageFlags |= bUsedWithStaticLighting ? (1u << MATUSAGE_StaticLighting) : 0;
	UsageFlags |= bUsedWithMorphTargets ? (1u << MATUSAGE_MorphTargets) : 0;
	UsageFlags |= bUsedWithSplineMeshes ? (1u << MATUSAGE_SplineMesh) : 0;
	UsageFlags |= bUsedWithInstancedStaticMeshes ? (1u << MATUSAGE_InstancedStaticMeshes) : 0;
	UsageFlags |= bUsedWithGeometryCollections ? (1u << MATUSAGE_GeometryCollections) : 0;
	UsageFlags |= bUsedWithClothing ? (1u << MATUSAGE_Clothing) : 0;
	UsageFlags |= bUsedWithGeometryCache ? (1u << MATUSAGE_GeometryCache) : 0;
	UsageFlags |= bUsedWithWater ? (1u << MATUSAGE_Water) : 0;
	UsageFlags |= bUsedWithHairStrands ? (1u << MATUSAGE_HairStrands) : 0;
	UsageFlags |= bUsedWithLidarPointCloud ? (1u << MATUSAGE_LidarPointCloud) : 0;
	UsageFlags |= bUsedWithVirtualHeightfieldMesh ? (1u << MATUSAGE_VirtualHeightfieldMesh) : 0;
	UsageFlags |= bUsedWithNanite ? (1u << MATUSAGE_Nanite) : 0;
	UsageFlags |= bUsedWithVoxels ? (1u << MATUSAGE_Voxels) : 0;
	UsageFlags |= bUsedWithCurves ? (1u << MATUSAGE_Curves) : 0;
	UsageFlags |= bUsedWithVolumetricCloud ? (1u << MATUSAGE_VolumetricCloud) : 0;
	UsageFlags |= bUsedWithHeterogeneousVolumes ? (1u << MATUSAGE_HeterogeneousVolumes) : 0;
	UsageFlags |= bUsedWithStaticMesh ? (1u << MATUSAGE_StaticMesh) : 0;
	UsageFlags |= bUsedWithEditorCompositing ? (1u << MATUSAGE_EditorCompositing) : 0;
	UsageFlags |= bUsedWithNeuralNetworks ? (1u << MATUSAGE_NeuralNetworks) : 0;
	UsageFlags |= bUsedWithMeshDeformer ? (1u << MATUSAGE_MeshDeformer) : 0;
	UsageFlags |= bUsedWithInstancedSkinnedMesh ? (1u << MATUSAGE_InstancedSkinnedMesh) : 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return UsageFlags;
}

bool UMaterial::GetUsageByFlag(EMaterialUsage Usage) const
{
	bool UsageValue = false;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	switch(Usage)
	{
		case MATUSAGE_SkeletalMesh: UsageValue = bUsedWithSkeletalMesh; break;
		case MATUSAGE_ParticleSprites: UsageValue = bUsedWithParticleSprites; break;
		case MATUSAGE_BeamTrails: UsageValue = bUsedWithBeamTrails; break;
		case MATUSAGE_MeshParticles: UsageValue = bUsedWithMeshParticles; break;
		case MATUSAGE_NiagaraSprites: UsageValue = bUsedWithNiagaraSprites; break;
		case MATUSAGE_NiagaraRibbons: UsageValue = bUsedWithNiagaraRibbons; break;
		case MATUSAGE_NiagaraMeshParticles: UsageValue = bUsedWithNiagaraMeshParticles; break;
		case MATUSAGE_StaticLighting: UsageValue = bUsedWithStaticLighting; break;
		case MATUSAGE_MorphTargets: UsageValue = bUsedWithMorphTargets; break;
		case MATUSAGE_SplineMesh: UsageValue = bUsedWithSplineMeshes; break;
		case MATUSAGE_InstancedStaticMeshes: UsageValue = bUsedWithInstancedStaticMeshes; break;
		case MATUSAGE_GeometryCollections: UsageValue = bUsedWithGeometryCollections; break;
		case MATUSAGE_Clothing: UsageValue = bUsedWithClothing; break;
		case MATUSAGE_GeometryCache: UsageValue = bUsedWithGeometryCache; break;
		case MATUSAGE_Water: UsageValue = bUsedWithWater; break;
		case MATUSAGE_HairStrands: UsageValue = bUsedWithHairStrands; break;
		case MATUSAGE_LidarPointCloud: UsageValue = bUsedWithLidarPointCloud; break;
		case MATUSAGE_VirtualHeightfieldMesh: UsageValue = bUsedWithVirtualHeightfieldMesh; break;
		case MATUSAGE_Nanite: UsageValue = bUsedWithNanite; break;
		case MATUSAGE_Voxels: UsageValue = bUsedWithVoxels; break;
		case MATUSAGE_Curves: UsageValue = bUsedWithCurves; break;
		case MATUSAGE_VolumetricCloud: UsageValue = bUsedWithVolumetricCloud; break;
		case MATUSAGE_HeterogeneousVolumes: UsageValue = bUsedWithHeterogeneousVolumes; break;
		case MATUSAGE_StaticMesh: UsageValue = bUsedWithStaticMesh; break;
		case MATUSAGE_EditorCompositing: UsageValue = bUsedWithEditorCompositing; break;
		case MATUSAGE_NeuralNetworks: UsageValue = bUsedWithNeuralNetworks; break;
		case MATUSAGE_MeshDeformer: UsageValue = bUsedWithMeshDeformer; break;
		case MATUSAGE_InstancedSkinnedMesh: UsageValue = bUsedWithInstancedSkinnedMesh; break;
		default: UE_LOGF(LogMaterial, Fatal,"Unknown material usage: %u", (int32)Usage);
	};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return UsageValue;
}

bool UMaterial::IsUsageFlagDirty(EMaterialUsage Usage)
{
#if WITH_EDITOR
	return GMaterialsWithDirtyUsageFlags.GetAnnotation(this).IsUsageFlagDirty(Usage);
#else
	return false;
#endif
}

// Deprecated in 5.7
bool UMaterial::IsCompilingOrHadCompileError(ERHIFeatureLevel::Type InFeatureLevel)
{
	return IsCompilingOrHadCompileError(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

bool UMaterial::IsCompilingOrHadCompileError(EShaderPlatform InShaderPlatform)
{
	const FMaterialResource* Res = GetMaterialResource(InShaderPlatform);
	return Res == nullptr || Res->GetGameThreadShaderMap() == nullptr;
}

#if WITH_EDITORONLY_DATA
TConstArrayView<TObjectPtr<UMaterialExpression>> UMaterial::GetExpressions() const
{
	return GetEditorOnlyData()->ExpressionCollection.Expressions;
}
#endif

#if WITH_EDITOR
TConstArrayView<TObjectPtr<UMaterialExpressionComment>> UMaterial::GetEditorComments() const
{
	return GetEditorOnlyData()->ExpressionCollection.EditorComments;
}

const FMaterialExpressionCollection& UMaterial::GetExpressionCollection() const
{
	return GetEditorOnlyData()->ExpressionCollection;
}

FMaterialExpressionCollection& UMaterial::GetExpressionCollection()
{
	return GetEditorOnlyData()->ExpressionCollection;
}

void UMaterial::AssignExpressionCollection(const FMaterialExpressionCollection& InCollection)
{
	GetEditorOnlyData()->ExpressionCollection = InCollection;
}

bool UMaterial::SetParameterValueEditorOnly(const FName& ParameterName, const FMaterialParameterMetadata& Meta)
{
	bool bResult = false;
	for (UMaterialExpression* Expression : GetExpressions())
	{
		if (Expression && Expression->SetParameterValue(ParameterName, Meta, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
		{
			bResult = true;
		}
		else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (FunctionCall->MaterialFunction)
			{
				TArray<UMaterialFunctionInterface*> Functions;
				Functions.Add(FunctionCall->MaterialFunction);
				FunctionCall->MaterialFunction->GetDependentFunctions(Functions);

				for (UMaterialFunctionInterface* Function : Functions)
				{
					for (const TObjectPtr<UMaterialExpression>& FunctionExpression : Function->GetExpressions())
					{
						if (FunctionExpression && FunctionExpression->SetParameterValue(ParameterName, Meta, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
						{
							bResult = true;
						}
					}
				}
			}
		}
	}

	return bResult;
}

bool UMaterial::SetVectorParameterValueEditorOnly(FName ParameterName, FLinearColor InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetScalarParameterValueEditorOnly(FName ParameterName, float InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetTextureParameterValueEditorOnly(FName ParameterName, class UTexture* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetTextureCollectionParameterValueEditorOnly(FName ParameterName, class UTextureCollection* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetRuntimeVirtualTextureParameterValueEditorOnly(FName ParameterName, class URuntimeVirtualTexture* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetSparseVolumeTextureParameterValueEditorOnly(FName ParameterName, class USparseVolumeTexture* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetFontParameterValueEditorOnly(FName ParameterName, class UFont* InFontValue, int32 InFontPage)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = FMaterialParameterValue(InFontValue, InFontPage);
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetStaticSwitchParameterValueEditorOnly(FName ParameterName, bool InValue, FGuid InExpressionGuid)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	Meta.ExpressionGuid = InExpressionGuid;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterial::SetStaticComponentMaskParameterValueEditorOnly(FName ParameterName, bool R, bool G, bool B, bool A, FGuid InExpressionGuid)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = FMaterialParameterValue(R, G, B, A);
	Meta.ExpressionGuid = InExpressionGuid;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}
#endif

void UMaterial::MarkUsageFlagDirty(EMaterialUsage Usage, bool CurrentValue, bool NewValue)
{
#if WITH_EDITOR
	if(CurrentValue != NewValue)
	{
		FMaterialsWithDirtyUsageFlags Annotation = GMaterialsWithDirtyUsageFlags.GetAnnotation(this);
		Annotation.MarkUsageFlagDirty(Usage);
		GMaterialsWithDirtyUsageFlags.AddAnnotation(this, MoveTemp(Annotation));
	}
#endif
}

void UMaterial::SetUsageByFlag(EMaterialUsage Usage, bool NewValue)
{
	bool bOldValue = GetUsageByFlag(Usage);
	MarkUsageFlagDirty(Usage, bOldValue, NewValue);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	switch(Usage)
	{
		case MATUSAGE_SkeletalMesh:
		{
			bUsedWithSkeletalMesh = NewValue; break;
		}
		case MATUSAGE_ParticleSprites:
		{
			bUsedWithParticleSprites = NewValue; break;
		}
		case MATUSAGE_BeamTrails:
		{
			bUsedWithBeamTrails = NewValue; break;
		}
		case MATUSAGE_MeshParticles:
		{
			bUsedWithMeshParticles = NewValue; break;
		}
		case MATUSAGE_NiagaraSprites:
		{
			bUsedWithNiagaraSprites = NewValue; break;
		}
		case MATUSAGE_NiagaraRibbons:
		{
			bUsedWithNiagaraRibbons = NewValue; break;
		}
		case MATUSAGE_NiagaraMeshParticles:
		{
			bUsedWithNiagaraMeshParticles = NewValue; break;
		}
		case MATUSAGE_StaticLighting:
		{
			bUsedWithStaticLighting = NewValue; break;
		}
		case MATUSAGE_MorphTargets:
		{
			bUsedWithMorphTargets = NewValue; break;
		}
		case MATUSAGE_SplineMesh:
		{
			bUsedWithSplineMeshes = NewValue; break;
		}
		case MATUSAGE_InstancedStaticMeshes:
		{
			bUsedWithInstancedStaticMeshes = NewValue; break;
		}
		case MATUSAGE_GeometryCollections:
		{
			bUsedWithGeometryCollections = NewValue; break;
		}
		case MATUSAGE_Clothing:
		{
			bUsedWithClothing = NewValue; break;
		}
		case MATUSAGE_GeometryCache:
		{
			bUsedWithGeometryCache = NewValue; break;
		}
		case MATUSAGE_Water:
		{
			bUsedWithWater = NewValue; break;
		}
		case MATUSAGE_HairStrands:
		{
			bUsedWithHairStrands = NewValue; break;
		}
		case MATUSAGE_LidarPointCloud:
		{
			bUsedWithLidarPointCloud = NewValue; break;
		}
		case MATUSAGE_VirtualHeightfieldMesh:
		{
			bUsedWithVirtualHeightfieldMesh = NewValue; break;
		}
		case MATUSAGE_Nanite:
		{
			bUsedWithNanite = NewValue; break;
		}
		case MATUSAGE_Voxels:
		{
			bUsedWithVoxels = NewValue; break;
		}
		case MATUSAGE_Curves:
		{
			bUsedWithCurves = NewValue; break;
		}
		case MATUSAGE_VolumetricCloud:
		{
			bUsedWithVolumetricCloud = NewValue; break;
		}
		case MATUSAGE_HeterogeneousVolumes:
		{
			bUsedWithHeterogeneousVolumes = NewValue; break;
		}
		case MATUSAGE_StaticMesh:
		{
			bUsedWithStaticMesh = NewValue; break;
		}
		case MATUSAGE_EditorCompositing:
		{
			bUsedWithEditorCompositing = NewValue; break;
		}
		case MATUSAGE_NeuralNetworks:
		{
			bUsedWithNeuralNetworks = NewValue; break;
		}
		case MATUSAGE_MeshDeformer:
		{
			bUsedWithMeshDeformer = NewValue; break;
		}
		case MATUSAGE_InstancedSkinnedMesh:
		{
			bUsedWithInstancedSkinnedMesh = NewValue; break;
		}
		default: UE_LOGF(LogMaterial, Fatal, "Unknown material usage: %u", (int32)Usage);
	};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UMaterial::SetMaterialUsage(EMaterialUsage Usage)
{
	// Material usage is only relevant for materials that can be applied onto a mesh / use with different vertex factories.
	if (MaterialDomain != MD_Surface && MaterialDomain != MD_DeferredDecal && MaterialDomain != MD_Volume)
	{
		return false;
	}

	// Check that the material has been flagged for use with the given usage flag.
	if (!GetUsageByFlag(Usage) && !bUsedAsSpecialEngineMaterial)
	{
		// For materials which do not have their bUsedWith____ correctly set the DefaultMaterial<type> should be used in game
		// Leaving this GIsEditor ensures that in game on PC will not look different than on the Consoles as we will not be compiling shaders on the fly
		// 
		// Using FApp::IsGame is not enough, during UWorld::SendAllEndOfFrameUpdates() it returns false even it PIE is running.
		// Ideally, FApp::IsGame should be revisited, but to avoid causing regressions, we also check IsPlayingSessionInEditor.
		bool bIsGame = FApp::IsGame();
#if WITH_EDITOR
		bIsGame = bIsGame || (GEditor && GEditor->IsPlayingSessionInEditor());
#endif
		if (GIsEditor && !bIsGame && bAutomaticallySetUsageInEditor)
		{
			check(IsInGameThread());
			//Do not warn the user during automation testing
			if (!GIsAutomationTesting)
			{
				UE_LOGF(LogMaterial, Display, "Material %ls needed to set usage flag %ls", *GetPathName(), *GetUsageName(Usage));
			}

			// Open a material update context so this material can be modified safely.
			{
				FMaterialUpdateContext UpdateContext(
					// We need to sync with the rendering thread but don't reregister components
					// because SetMaterialUsage may be called during registration!
					FMaterialUpdateContext::EOptions::SyncWithRenderingThread
				);
				UpdateContext.AddMaterial(this);

				// If the flag is missing in the editor, set it, and recompile shaders.
				SetUsageByFlag(Usage, true);

				// Compile and force the Id to be regenerated, since we changed the material in a way that changes compilation
				CacheResourceShadersForRendering(true);
			}

			// Mark the package dirty so that hopefully it will be saved with the new usage flag.
			// This is important because the only way an artist can fix an infinite 'compile on load' scenario is by saving with the new usage flag
			if (!MarkPackageDirty())
			{
#if WITH_EDITOR
				// The package could not be marked as dirty as we're loading content in the editor. Add a Map Check error to notify the user.
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Material"), FText::FromString(GetPathName()));
				Arguments.Add(TEXT("Usage"), FText::FromString(*GetUsageName(Usage)));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_SetMaterialUsage", "Material {Material} was missing the usage flag {Usage}. If the material asset is not re-saved, it may not render correctly when run outside the editor."), Arguments)))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_FixMaterialUsage", "Fix"), LOCTEXT("MapCheck_FixMaterialUsage_Desc", "Click to set the usage flag correctly and mark the asset file as needing to be saved."), FOnActionTokenExecuted::CreateUObject(this, &UMaterial::FixupMaterialUsageAfterLoad), true));
				FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
#endif
			}

#if WITH_EDITOR
			// Notify any open material editors.
			FEditorSupportDelegates::MaterialUsageFlagsChanged.Broadcast(this, Usage);
#endif
		}
		else
		{
			const uint32 UsageFlagBit = (1u << Usage);
			if ((UsageFlagWarnings & UsageFlagBit) == 0)
			{
				UE_LOGF(LogMaterial, Warning, "Material %ls missing usage flag %ls! Default Material will be used in game.", *GetPathName(), *GetUsageName(Usage));

				if (bAutomaticallySetUsageInEditor)
				{
					UE_LOGF(LogMaterial, Warning, "     The material will recompile every editor launch until resaved.");
				}
#if WITH_EDITOR
				else if (GEditor && !GEditor->IsPlaySessionInProgress())
				{
					// Temporarily Turn off the toast to avoid spam which can result in GPU crashes from too many swap chains from being created.

					//FFormatNamedArguments Args;
					//Args.Add(TEXT("UsageName"), FText::FromString(GetUsageName(Usage)));
					//FNotificationInfo Info(FText::Format(LOCTEXT("CouldntSetMaterialUsage","Material didn't allow automatic setting of usage flag {UsageName} needed to render on this component, using Default Material instead."), Args));
					//Info.ExpireDuration = 5.0f;
					//Info.bUseSuccessFailIcons = true;

					// Give the user feedback as to why they are seeing the default material
					//FSlateNotificationManager::Get().AddNotification(Info);
				}
#endif

				UsageFlagWarnings |= UsageFlagBit;
			}

			// Return failure if the flag is missing in game, since compiling shaders in game is not supported on some platforms.
			return false;
		}
	}
	return true;
}

#if WITH_EDITOR
void UMaterial::FixupMaterialUsageAfterLoad()
{
	// All we need to do here is mark the package dirty as the usage itself was set on load.
	MarkPackageDirty();
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UMaterial::IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
{
	for (UMaterialExpression* Expression : GetExpressions())
	{
		if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (!FunctionCall->IterateDependentFunctions(Predicate))
			{
				return false;
			}
		}
		else if (UMaterialExpressionMaterialAttributeLayers* Layers = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
		{
			if (!Layers->IterateDependentFunctions(Predicate))
			{
				return false;
			}
		}
	}
	return true;
}

void UMaterial::GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
{
	IterateDependentFunctions([&DependentFunctions](UMaterialFunctionInterface* MaterialFunction) -> bool
	{
		DependentFunctions.AddUnique(MaterialFunction);
		return true;
	});
}

void BuildExpressionParametersGuidsMapRecursive(TConstArrayView<TObjectPtr<UMaterialExpression>> InMaterialExpression, TMap<FGuid, TObjectPtr<UMaterialExpression>>& OutExpressionMap)
{
	const int32 NumExpressions = InMaterialExpression.Num();
	for (int32 ExpressionIndex = 0; ExpressionIndex < NumExpressions; ++ExpressionIndex)
	{
		UMaterialExpression* ExpressionPtr = InMaterialExpression[ExpressionIndex];
		if (ExpressionPtr)
		{
			const FGuid ExpressionGuid = ExpressionPtr->GetParameterExpressionId();
			if (ExpressionPtr->bIsParameterExpression && !OutExpressionMap.Contains(ExpressionGuid))
			{
				OutExpressionMap.Add(ExpressionGuid, TObjectPtr<UMaterialExpression>(ExpressionPtr));
			}
		}
		else if (UMaterialFunctionInterface* MaterialFunction = UMaterial::GetExpressionFunctionPointer(ExpressionPtr))
		{
			BuildExpressionParametersGuidsMapRecursive(MaterialFunction->GetExpressions(), OutExpressionMap);
		}
		else if (TOptional<UMaterial::FLayersInterfaces> LayersInterfaces = UMaterial::GetExpressionLayers(ExpressionPtr))
		{
			for (const UMaterialFunctionInterface* Layer : LayersInterfaces->Layers)
			{
				if (Layer)
				{
					BuildExpressionParametersGuidsMapRecursive(Layer->GetExpressions(), OutExpressionMap);
				}
			}

			for (const UMaterialFunctionInterface* Blend : LayersInterfaces->Blends)
			{
				if (Blend)
				{
					BuildExpressionParametersGuidsMapRecursive(Blend->GetExpressions(), OutExpressionMap);
				}
			}
		}
	}
}

void UMaterial::BuildExpressionParametersGuidsMap(TMap<FGuid, TObjectPtr<UMaterialExpression>>& OutExpressionMap)
{
	BuildExpressionParametersGuidsMapRecursive(GetExpressions(), OutExpressionMap);
}

#endif // WITH_EDITORONLY_DATA

extern FPostProcessMaterialNode* FindExistingBlendablePostProcessNode(const FFinalPostProcessSettings& Dest, const UMaterialInterface* Material, const UMaterial* Base);

void UMaterialInterface::OverrideBlendableSettings(class FSceneView& View, float Weight) const
{
	check(Weight > 0.0f && Weight <= 1.0f);

	FFinalPostProcessSettings& Dest = View.FinalPostProcessSettings;

	const UMaterial* Base = GetMaterial();

	//	should we use UMaterial::GetDefaultMaterial(Domain) instead of skipping the material

	if (!Base || Base->MaterialDomain != MD_PostProcess)
	{
		return;
	}

	if (View.IsRuntimeReflectionCapture())
	{
		// Runtime reflection captures support applying a post process material, even when no View.State is set.
		// These can't use a MID because they have no View.State to store it on, but since it's a singleton material,
		// it can't blend, and so it doesn't need one.  The "const_cast" is unavoidable here, as FPostProcessMaterialNode
		// supports modification of the material interface when it's a MID, forcing the member to be non-const, but it
		// won't actually be modified when not a MID.
		FPostProcessMaterialNode MaterialNode(const_cast<UMaterialInterface*>(this), GetBlendableLocation(Base), 0, false);
		Dest.BlendableManager.PushBlendableData(1.0f, MaterialNode);
		return;
	}
	else if(!View.State)
	{
		return;
	}

	// Materials that write to UserSceneTexture outputs are automatically non-blendable
	bool bIsBlendable = Base->bIsBlendable && GetUserSceneTextureOutput(Base) == NAME_None;

	FPostProcessMaterialNode* DestNode = bIsBlendable ? FindExistingBlendablePostProcessNode(Dest, this, Base) : nullptr;

	// is this the first one of this material?
	if(!DestNode)
	{
		UMaterialInstanceDynamic* InitialMID = View.State->GetReusableMID((UMaterialInterface*)this);

		if(InitialMID)
		{
			// If the initial node is faded in partly we add the base material (it's assumed to be the neutral state, see docs)
			// and then blend in the material instance (it it's the base there is no need for that)
			const UMaterialInterface* SourceData = (Weight < 1.0f) ? Base : this;

			InitialMID->CopyScalarAndVectorParameters(*SourceData, View.GetShaderPlatform());

			FPostProcessMaterialNode InitialNode(InitialMID, GetBlendableLocation(Base), GetBlendablePriority(Base), bIsBlendable);

			// no blending needed on this one
			FPostProcessMaterialNode* InitialDestNode = Dest.BlendableManager.PushBlendableData(1.0f, InitialNode);

			if(Weight < 1.0f && this != Base)
			{
				// We are not done, we still need to fade with SrcMID
				DestNode = InitialDestNode;
			}
		}
	}

	if(DestNode)
	{
		// we apply this material on top of an existing one
		UMaterialInstanceDynamic* DestMID = DestNode->GetMID();
		check(DestMID);

		// The blending functions below only work on instances, so skip blending if we are just a material interface.
		// This can happen in two scenarios:
		// 1. A wrong material type is assigned into the post process material array.
		// 2. The same material is placed in twice.  The MID lookup code above won't create two separate MIDs for the same
		//    material resulting in this code trying to blend a MID w/ a MaterialInterface.
		if (this->IsA<UMaterialInstance>())
		{
			// Attempt to cast the UMaterialInterface to a UMaterialInstance pointer.
			UMaterialInstance* SrcMI = (UMaterialInstance*)this;
			check(SrcMI);

			// Here we could check for Weight=1.0 and use copy instead of interpolate but that case quite likely not intended anyway.

			// a material already exists, blend (Scalar and Vector parameters) with existing ones
			DestMID->K2_InterpolateMaterialInstanceParams(DestMID, SrcMI, Weight);
		}
	}
}

UMaterial* UMaterial::GetMaterial()
{
	return this;
}

const UMaterial* UMaterial::GetMaterial() const
{
	return this;
}

const UMaterial* UMaterial::GetMaterial_Concurrent(TMicRecursionGuard) const
{
	return this;
}

void UMaterial::GetMaterialInheritanceChain(FMaterialInheritanceChain& OutChain) const
{
	check(!OutChain.BaseMaterial);
	OutChain.BaseMaterial = this;
	if (!OutChain.CachedExpressionData)
	{
		const FMaterialCachedExpressionData* LocalData = CachedExpressionData.Get();
		OutChain.CachedExpressionData = LocalData ? LocalData : &FMaterialCachedExpressionData::EmptyData;
	}
}

#if WITH_EDITORONLY_DATA

UMaterialFunctionInterface* UMaterial::GetExpressionFunctionPointer(const UMaterialExpression* Expression)
{
	const UMaterialExpressionMaterialFunctionCall* ExpressionFunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression);
	if (ExpressionFunctionCall && ExpressionFunctionCall->MaterialFunction)
	{
		return ExpressionFunctionCall->MaterialFunction;
	}
	return nullptr;
}

TOptional<UMaterial::FLayersInterfaces> UMaterial::GetExpressionLayers(const UMaterialExpression* Expression)
{
	const UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression);
	if (LayersExpression)
	{
		FLayersInterfaces Interfaces;
		Interfaces.Layers = LayersExpression->GetLayers();
		Interfaces.Blends = LayersExpression->GetBlends();
		return Interfaces;
	}
	return {};
}

void UMaterial::UpdateTransientExpressionData()
{
	for (UMaterialExpression* Expression : GetExpressions())
	{
		if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			// Update the function call node, so it can relink inputs and outputs as needed
			// Update even if MaterialFunctionNode->MaterialFunction is NULL, because we need to remove the invalid inputs in that case
			FunctionCall->UpdateFromFunctionResource();
		}
	}
}

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR

static FSubstrateMaterialInfo GetSubstrateMaterialInfo(UMaterialEditorOnlyData* EditorOnly)
{
	FSubstrateMaterialInfo Out;
	if (Substrate::IsSubstrateEnabled() && EditorOnly->FrontMaterial.IsConnected())
	{
		check(EditorOnly->FrontMaterial.Expression);
		if (EditorOnly->FrontMaterial.Expression->IsResultSubstrateMaterial(EditorOnly->FrontMaterial.OutputIndex))
		{
			EditorOnly->FrontMaterial.Expression->GatherSubstrateMaterialInfo(Out, EditorOnly->FrontMaterial.OutputIndex);
		}
	}
	return Out;
}

static void UpdatePropertyConnectedMask(const FSubstrateMaterialInfo& InSubstrateMaterialInfo, FMaterialCachedExpressionData* Out)
{
	if (Substrate::IsSubstrateEnabled() && InSubstrateMaterialInfo.IsValid() && Out)
	{
		// Mask of all input collected by Substrate BSDF nodes
		static const uint64 ConnectionMask =
			  (1ull << MP_BaseColor)
			| (1ull << MP_Metallic)
			| (1ull << MP_Specular)
			| (1ull << MP_Roughness)
			| (1ull << MP_Anisotropy)
			| (1ull << MP_EmissiveColor)
			| (1ull << MP_Normal)
			| (1ull << MP_Tangent)
			| (1ull << MP_SubsurfaceColor)
			| (1ull << MP_CustomData0)
			| (1ull << MP_CustomData1)
			| (1ull << MP_Opacity)
			| (1ull << MP_ShadingModel)
			| (1ull << MP_DiffuseColor)
			| (1ull << MP_SpecularColor);

		// Override the cached expression data with collected connection from InSubstrateMaterialInfo, but preserve all other input (e.g., refraction)
		Out->PropertyConnectedMask &= ~ConnectionMask;
		Out->PropertyConnectedMask |= InSubstrateMaterialInfo.GetPropertyConnected();
	}
}

void UMaterial::UpdateCachedExpressionData()
{
	//@note FH: temporary preemptive PostLoad until zenloader load ordering improvements
	ConditionalPostLoad();

	if (bLoadedCachedExpressionData)
	{
		const FMaterialCachedExpressionData& UsedCachedData = GetCachedExpressionData();
		// Try to always use Editor Only Data textures in Editor, if it exists
		if (UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures.Num() != 0)
		{
			ReferencedDefaultTextures = UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures;
		}

		// Don't need to rebuild cached data if it was serialized
		return;
	}

	FMaterialCachedExpressionData* LocalCachedExpressionData = new FMaterialCachedExpressionData();

	// Run graph analysis to harvest material information used to select which shader permutations are needed to be generated (e.g. detect what material properties are written to
	// taking under consideration the presence of static switches to select among two input subgraphs).
	LocalCachedExpressionData->AnalyzeMaterial(*this);
	
	if (LocalCachedExpressionData->bHasMaterialLayers)
	{
		// Set all layers as linked to parent (there is no parent for base UMaterials)
		LocalCachedExpressionData->EditorOnlyData->MaterialLayers.LinkAllLayersToParent();
	}

	LocalCachedExpressionData->Validate(*this);

	if (Substrate::IsSubstrateEnabled())
	{
		UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
		FSubstrateMaterialInfo SubstrateMaterialInfo = GetSubstrateMaterialInfo(EditorOnly);
		UpdatePropertyConnectedMask(SubstrateMaterialInfo, LocalCachedExpressionData);
	}

	CachedExpressionData.Reset(LocalCachedExpressionData);
	EditorOnlyData->CachedExpressionData = LocalCachedExpressionData->EditorOnlyData;
	ReferencedDefaultTextures = CachedExpressionData->EditorOnlyData->EditorReferencedDefaultTextures;

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyReferencedTextureChanged_Concurrent(this);
#endif
}
#endif // WITH_EDITOR

bool UMaterial::GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutResult, EMaterialGetParameterValueFlags Flags) const
{
	if (EnumHasAnyFlags(Flags, EMaterialGetParameterValueFlags::CheckNonOverrides) && CachedExpressionData)
	{
		return CachedExpressionData->GetParameterValue(Type, ParameterInfo, OutResult);
	}

	return false;
}

bool UMaterial::GetMaterialLayers(FMaterialLayersFunctions& OutLayers, TMicRecursionGuard) const
{
	if (CachedExpressionData && CachedExpressionData->bHasMaterialLayers)
	{
		OutLayers.GetRuntime() = CachedExpressionData->MaterialLayers;
#if WITH_EDITORONLY_DATA
		if (CachedExpressionData->EditorOnlyData)
		{
			// cooked materials can strip out material layer information
			if (CachedExpressionData->EditorOnlyData->MaterialLayers.LayerStates.Num() != 0)
			{
				OutLayers.EditorOnly = CachedExpressionData->EditorOnlyData->MaterialLayers;
				OutLayers.Validate();
			}
			else
			{
				return false;
			}
		}
#endif // WITH_EDITORONLY_DATA
		return true;
	}
	return false;
}

bool UMaterial::GetRefractionSettings(float& OutBiasValue) const
{
	OutBiasValue = RefractionDepthBias;
	return true;
}

EBlendableLocation UMaterial::GetBlendableLocation(const UMaterial* Base) const
{
	check(Base == this);
	return BlendableLocation;
}

int32 UMaterial::GetBlendablePriority(const UMaterial* Base) const
{
	check(Base == this);
	return BlendablePriority;
}

void UMaterial::GetDependencies(TSet<UMaterialInterface*>& Dependencies) 
{
	Dependencies.Add(this);
}

FMaterialRenderProxy* UMaterial::GetRenderProxy() const
{
	return DefaultMaterialInstance;
}

UPhysicalMaterial* UMaterial::GetPhysicalMaterial() const
{
	if (GEngine)
	{
		return (PhysMaterial != nullptr) ? PhysMaterial : GEngine->DefaultPhysMaterial;
	}
	return nullptr;
}

UPhysicalMaterialMask* UMaterial::GetPhysicalMaterialMask() const
{
	return PhysMaterialMask;
}

UPhysicalMaterial* UMaterial::GetPhysicalMaterialFromMap(int32 Index) const
{
	if (Index >= 0 && Index < EPhysicalMaterialMaskColor::MAX)
	{
		return PhysicalMaterialMap[Index];
	}
	return nullptr;
}

TArrayView<const TObjectPtr<UPhysicalMaterial>> UMaterial::GetRenderTracePhysicalMaterialOutputs() const
{
	return MakeArrayView(RenderTracePhysicalMaterialOutputs);
}

void UMaterial::SetRenderTracePhysicalMaterialOutputs(TArrayView<TObjectPtr<class UPhysicalMaterial>> PhysicalMaterials)
{
	RenderTracePhysicalMaterialOutputs = PhysicalMaterials;
}

#if WITH_EDITOR
void UMaterial::GetShaderTags(TArray<FName>& OutShaderTags) const
{
	if (!CachedExpressionData || !CachedExpressionData->EditorOnlyData)
	{
		return;
	}

	OutShaderTags.Append(CachedExpressionData->EditorOnlyData->ShaderTags);
}

EMaterialTranslateValidationFlags UMaterial::GetMaterialTranslateValidationFlags() const
{
	EMaterialTranslateValidationFlags Flags = EMaterialTranslateValidationFlags::None;
	if (!bRelaxRuntimeVirtualTextureRestrictions && !bRelaxRuntimeVirtualTextureRestrictionsForOldContent)
	{
		Flags |= EMaterialTranslateValidationFlags::Strict_RuntimeVirtualTexture;
	}
	return Flags;
}
#endif

UMaterialInterface* UMaterial::GetNaniteOverride(TMicRecursionGuard RecursionGuard) const
{
	return NaniteOverrideMaterial.GetOverrideMaterial();
}

/** Helper functions for text output of properties... */
#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(txt) case txt: return TEXT(#txt);
#endif

#ifndef TEXT_TO_ENUM
#define TEXT_TO_ENUM(eVal, txt)		if (FCString::Stricmp(TEXT(#eVal), txt) == 0)	return eVal;
#endif

const TCHAR* UMaterial::GetMaterialShadingModelString(EMaterialShadingModel InMaterialShadingModel)
{
	switch (InMaterialShadingModel)
	{
		FOREACH_ENUM_EMATERIALSHADINGMODEL(CASE_ENUM_TO_TEXT)
	}
	return TEXT("MSM_DefaultLit");
}

EMaterialShadingModel UMaterial::GetMaterialShadingModelFromString(const TCHAR* InMaterialShadingModelStr)
{
	#define TEXT_TO_SHADINGMODEL(m) TEXT_TO_ENUM(m, InMaterialShadingModelStr);
	FOREACH_ENUM_EMATERIALSHADINGMODEL(TEXT_TO_SHADINGMODEL)
	#undef TEXT_TO_SHADINGMODEL
	return MSM_DefaultLit;
}

const TCHAR* UMaterial::GetBlendModeString(EBlendMode InBlendMode)
{
	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	switch (InBlendMode)
	{
	case BLEND_Opaque:							return TEXT("BLEND_Opaque");
	case BLEND_Masked:							return TEXT("BLEND_Masked");
	// BLEND_Translucent & BLEND_TranslucentGreyTransmittance are mapped onto the same enum index
	case BLEND_Translucent:						return bSubstrateEnabled ? TEXT("BLEND_TranslucentGreyTransmittance") : TEXT("BLEND_Translucent");
	case BLEND_Additive:						return TEXT("BLEND_Additive");
	// BLEND_Modulate & BLEND_ColoredTransmittanceOnly are mapped onto the same enum index
	case BLEND_Modulate:						return bSubstrateEnabled ? TEXT("BLEND_ColoredTransmittanceOnly") : TEXT("BLEND_Modulate");
	case BLEND_AlphaComposite:					return TEXT("BLEND_AlphaComposite");
	case BLEND_AlphaHoldout:					return TEXT("BLEND_AlphaHoldout");
	case BLEND_TranslucentColoredTransmittance: return bSubstrateEnabled ? TEXT("BLEND_TranslucentColoredTransmittance") : TEXT("BLEND_TranslucentColoredTransmittance_SUBSTRATEONLY");
	}
	return TEXT("BLEND_Opaque");
}

EBlendMode UMaterial::GetBlendModeFromString(const TCHAR* InBlendModeStr)
{
	#define TEXT_TO_BLENDMODE(b) TEXT_TO_ENUM(b, InBlendModeStr);
	FOREACH_ENUM_EBLENDMODE(TEXT_TO_BLENDMODE)
	#undef TEXT_TO_BLENDMODE
	return BLEND_Opaque;
}

static FAutoConsoleVariable GCompileMaterialsForShaderFormatCVar(
	TEXT("r.CompileMaterialsForShaderFormat"),
	TEXT(""),
	TEXT("When enabled, compile materials for this shader format in addition to those for the running platform.\n")
	TEXT("Note that these shaders are compiled and immediately tossed. This is only useful when directly inspecting output via r.DebugDumpShaderInfo.")
	);

#if WITH_EDITOR
uint64 UMaterial::GetForceRecompileTextureIdsHash() const
{
	TArray<UTexture*> ForceRecompileTextures;
	for (const UMaterialExpression *MaterialExpression : GetExpressions())
	{
		if (MaterialExpression == nullptr)
		{
			continue;
		}
		TArray<UTexture*> ExpressionForceRecompileTextures;
		MaterialExpression->GetTexturesForceMaterialRecompile(ExpressionForceRecompileTextures);
		for (UTexture *ForceRecompileTexture : ExpressionForceRecompileTextures)
		{
			ForceRecompileTextures.AddUnique(ForceRecompileTexture);
		}
	}
	if (ForceRecompileTextures.Num() <= 0)
	{
		//There is no Texture that trig a recompile of the material, nothing to add to the hash
		return 0u;
	}

	FXxHash64Builder TextureCompileDependencies;

	for (UTexture *ForceRecompileTexture : ForceRecompileTextures)
	{
		TextureCompileDependencies.Update(&ForceRecompileTexture->GetLightingGuid(), sizeof(FGuid));
	}

	return TextureCompileDependencies.Finalize().Hash;
}

bool UMaterial::IsTextureForceRecompileCacheRessource(UTexture *Texture)
{
	for (const UMaterialExpression *MaterialExpression : GetExpressions())
	{
		if (MaterialExpression == nullptr)
		{
			continue;
		}
		TArray<UTexture*> ExpressionForceRecompileTextures;
		MaterialExpression->GetTexturesForceMaterialRecompile(ExpressionForceRecompileTextures);
		for (UTexture *ForceRecompileTexture : ExpressionForceRecompileTextures)
		{
			if (Texture == ForceRecompileTexture)
			{
				return true;
			}
		}
	}
	return false;
}

void UMaterial::UpdateMaterialShaderCacheAndTextureReferences()
{
	//Cancel any current compilation jobs that are in flight for this material.
	CancelOutstandingCompilation();

	//Force a recompute of the DDC key
	CacheResourceShadersForRendering(true);
	
	// Ensure that the ReferencedTextureGuids array is up to date.
	if (GIsEditor)
	{
		UpdateLightmassTextureTracking();
	}

	// Ensure that any components with static elements using this material have their render state recreated
	// so changes are propagated to them. The preview material is only applied to the preview mesh component,
	// and that reregister is handled by the material editor.
	if (!bIsPreviewMaterial && !bIsFunctionPreviewMaterial && !bIsMaterialEditorStatsMaterial)
	{
		FGlobalComponentRecreateRenderStateContext RecreateComponentsRenderState;
	}
	// needed for UMaterial as it doesn't have the InitResources() override where this is called
	PropagateDataToMaterialProxy();
}

#endif //WITH_EDITOR

UE_TRACE_EVENT_BEGIN(Cpu, CacheResourceShadersForRendering, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, MaterialName)
UE_TRACE_EVENT_END()

void UMaterial::CacheResourceShadersForRendering(bool bRegenerateId, EMaterialShaderPrecompileMode PrecompileMode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterial::CacheResourceShadersForRendering);

#if CPUPROFILERTRACE_ENABLED
	FString TraceMaterialName;
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		TraceMaterialName = GetFullName();
	}
	UE_TRACE_LOG_SCOPED_T(Cpu, CacheResourceShadersForRendering, CpuChannel)
		<< CacheResourceShadersForRendering.MaterialName(*TraceMaterialName);
#endif

#if WITH_EDITOR
	// Always rebuild the shading model field on recompile
	RebuildShadingModelField();
#endif //WITH_EDITOR

	if (bRegenerateId)
	{
		// Regenerate this material's Id if requested
		// Since we can't provide an explanation for why we've been asked to change the guid.
		// We can't give this function a unique transformation id, let it generate a new one.
		ReleaseResourcesAndMutateDDCKey();
	}

	if (FApp::CanEverRender())
	{
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();

		TArray<FMaterialResource*> ResourcesToCache;
		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			// Only cache shaders for the quality level that will actually be used to render
			// In cooked build, there is no shader compilation but this is still needed
			// to register the loaded shadermap
			FMaterialResource* CurrentResource = FindOrCreateMaterialResource(MaterialResources, this, nullptr, ShaderPlatform, ActiveQualityLevel);
			check(CurrentResource);

			ResourcesToCache.Reset();
			ResourcesToCache.Add(CurrentResource);
			CacheShadersForResources(ShaderPlatform, ResourcesToCache, PrecompileMode);
		}

#if WITH_EDITOR
		FString AdditionalFormatToCache = GCompileMaterialsForShaderFormatCVar->GetString();
		if (!AdditionalFormatToCache.IsEmpty())
		{
			EShaderPlatform AdditionalPlatform = ShaderFormatToLegacyShaderPlatform(FName(*AdditionalFormatToCache));
			if (AdditionalPlatform != SP_NumPlatforms)
			{
				TArray<FMaterialResourceForCooking> CookResourcesToCache;
				CacheResourceShadersForCooking(AdditionalPlatform, CookResourcesToCache);
			}
		}
#endif

		RecacheUniformExpressions(true);
	}
}

#if WITH_EDITOR
void UMaterial::CacheResourceShadersForCooking(EShaderPlatform ShaderPlatform,
	TArray<FMaterialResourceForCooking>& OutCachedMaterialResources, const ITargetPlatform* TargetPlatform,
	bool bBlocking)
{
	// Only new resources need to have CacheShaders() called on them, whereas OutCachedMaterialResources
	// may already contain resources for another shader platform
	TArray<FMaterialResource*> NewResourcesToCache;
	GetNewResources(ShaderPlatform, NewResourcesToCache);

#if WITH_EDITOR
	// The editor needs to block if the caching call comes from cook on the fly, where the polling mechanisms are not active.
	// This is important so that the jobs finish and the CacheShadersCompletion() callback is triggered via FinishCacheShaders()!
	if (bBlocking)
#endif
	{
		CacheShadersForResources(ShaderPlatform, NewResourcesToCache, EMaterialShaderPrecompileMode::Background, TargetPlatform);
	}
#if WITH_EDITOR
	else
	{
		// For cooking, we can call the begin function and it will be completed as part of the polling mechanism.
		BeginCacheShadersForResources(ShaderPlatform, NewResourcesToCache, EMaterialShaderPrecompileMode::Background, TargetPlatform);
	}
#endif

	OutCachedMaterialResources.Reserve(OutCachedMaterialResources.Num() + NewResourcesToCache.Num());
	for (FMaterialResource* Resource : NewResourcesToCache)
	{
		OutCachedMaterialResources.Add({ Resource, ShaderPlatform });
	}
}
#endif // WITH_EDITOR

void UMaterial::GetNewResources(EShaderPlatform ShaderPlatform, TArray<FMaterialResource*>& NewResourcesToCache)
{
	ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);

	TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num> > QualityLevelsUsed;
	GetQualityLevelUsageForCooking(QualityLevelsUsed, ShaderPlatform);

	const UShaderPlatformQualitySettings* MaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(ShaderPlatform);
	bool bNeedDefaultQuality = false;

	for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; QualityLevelIndex++)
	{
		// Add all quality levels actually used
		if (QualityLevelsUsed[QualityLevelIndex])
		{
			FMaterialResource* NewResource = AllocateResource();
			NewResource->SetMaterial(this, nullptr, ShaderPlatform, (EMaterialQualityLevel::Type)QualityLevelIndex);
			NewResourcesToCache.Add(NewResource);
		}
		else
		{
			const FMaterialQualityOverrides& QualityOverrides = MaterialQualitySettings->GetQualityOverrides((EMaterialQualityLevel::Type)QualityLevelIndex);
			if (!QualityOverrides.bDiscardQualityDuringCook)
			{
				// don't have an explicit resource for this quality level, but still need to support it, so make sure we include a default quality resource
				bNeedDefaultQuality = true;
			}
		}
	}

	if (bNeedDefaultQuality)
	{
		FMaterialResource* NewResource = AllocateResource();
		NewResource->SetMaterial(this, nullptr, ShaderPlatform);
		NewResourcesToCache.Add(NewResource);
	}
}

namespace MaterialImpl
{
	void HandleCacheShadersForResourcesErrors(bool bSuccess, EShaderPlatform ShaderPlatform, UMaterial* This, FMaterialResource* CurrentResource)
	{
		if (!bSuccess)
		{
			const bool bIsDefaultMaterial = This->IsDefaultMaterial();
			FString ErrorString;
			if (bIsDefaultMaterial)
			{
				ErrorString += FString::Printf(TEXT("Failed to compile Default Material for platform %s!\n"),
					*LegacyShaderPlatformToShaderFormat(ShaderPlatform).ToString());
			}
			else
			{
				ErrorString += FString::Printf(TEXT("Failed to compile Material for platform %s, Default Material will be used in game.\n"),
					*LegacyShaderPlatformToShaderFormat(ShaderPlatform).ToString());
			}

#if WITH_EDITOR
			const TArray<FString>& CompileErrors = CurrentResource->GetCompileErrors();
			for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
			{
				ErrorString += FString::Printf(TEXT("	%s\n"), *CompileErrors[ErrorIndex]);
			}
#endif
			
			if (bIsDefaultMaterial)
			{
				if (AreShaderErrorsFatal())
				{
					UE_ASSET_LOG(LogMaterial, Fatal, This, TEXT("%s"), *ErrorString);
				}
				else
				{
					UE_ASSET_LOG(LogMaterial, Error, This, TEXT("%s"), *ErrorString);
				}
			}
			else if (CVarMaterialLogErrorOnFailure.GetValueOnAnyThread())
			{
				UE_ASSET_LOG(LogMaterial, Error, This, TEXT("%s"), *ErrorString);
			}
			else
			{
				UE_ASSET_LOG(LogMaterial, Warning, This, TEXT("%s"), *ErrorString);
			}
		}
	}
}

void UMaterial::CacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
#if WITH_EDITOR
	check(!HasAnyFlags(RF_NeedPostLoad));
#endif
	for (int32 ResourceIndex = 0; ResourceIndex < ResourcesToCache.Num(); ResourceIndex++)
	{
		FMaterialResource* CurrentResource = ResourcesToCache[ResourceIndex];
		const bool bSuccess = CurrentResource->CacheShaders(PrecompileMode, TargetPlatform);

		MaterialImpl::HandleCacheShadersForResourcesErrors(bSuccess, ShaderPlatform, this, CurrentResource);
	}
}

#if WITH_EDITOR

void UMaterial::BeginCacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	check(!HasAnyFlags(RF_NeedPostLoad));

	for (int32 ResourceIndex = 0; ResourceIndex < ResourcesToCache.Num(); ResourceIndex++)
	{
		FMaterialResource* CurrentResource = ResourcesToCache[ResourceIndex];
		CurrentResource->BeginCacheShaders(PrecompileMode, TargetPlatform,
			[WeakThis = MakeWeakObjectPtr(this), CurrentResource, ShaderPlatform](bool bSuccess)
			{
				if (UMaterial* This = WeakThis.Get())
				{
					MaterialImpl::HandleCacheShadersForResourcesErrors(bSuccess, ShaderPlatform, This, CurrentResource);
				}
			}
		);
	}
}

#endif

void UMaterial::CacheShaders(EMaterialShaderPrecompileMode CompileMode)
{
	CacheResourceShadersForRendering(false, CompileMode);
}

#if WITH_EDITOR
void UMaterial::CacheGivenTypesForCooking(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, EMaterialQualityLevel::Type QualityLevel, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*> PipelineTypes, const TArray<const FShaderType*>& ShaderTypes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterial::CacheGivenTypes);

	if (QualityLevel == EMaterialQualityLevel::Num)
	{
		QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	}

	FMaterialResource* CurrentResource = FindOrCreateMaterialResource(MaterialResources, this, nullptr, ShaderPlatform, QualityLevel);
	check(CurrentResource);

	// Prepare the resource for compilation, but don't compile the completed shader map.
	const bool bSuccess = CurrentResource->CacheShaders(EMaterialShaderPrecompileMode::None);
	if (bSuccess)
	{
		CurrentResource->CacheGivenTypes(VFTypes, PipelineTypes, ShaderTypes);
	}
}
#endif

bool UMaterial::IsComplete() const
{
	bool bComplete = true;
	if (FApp::CanEverRender())
	{
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();

		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			FMaterialResource* CurrentResource = FindMaterialResource(MaterialResources, ShaderPlatform, ActiveQualityLevel, true);
			if (CurrentResource && !CurrentResource->IsGameThreadShaderMapComplete())
			{
				bComplete = false;
				break;
			}
		}
	}
	return bComplete;
}

#if WITH_EDITOR
bool UMaterial::IsCompiling() const
{
	bool bIsCompiling = false;
	if (FApp::CanEverRender())
	{
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();

		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			FMaterialResource* CurrentResource = FindMaterialResource(MaterialResources, ShaderPlatform, ActiveQualityLevel, true);
			if (CurrentResource && !CurrentResource->IsCompilationFinished())
			{
				bIsCompiling = true;
				break;
			}
		}
	}
	return bIsCompiling;
}
#endif

FGraphEventArray UMaterial::PrecachePSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& InPreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs)
{
	FGraphEventArray GraphEvents;
	if (FApp::CanEverRender() && MaterialResources.Num() > 0 && (PipelineStateCache::IsPSOPrecachingEnabled() || IsPSOShaderPreloadingEnabled()) && UMaterialInterface::IsDefaultMaterialInitialized())
	{
		EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			FMaterialResource* MaterialResource = FindMaterialResource(MaterialResources, GetFeatureLevelShaderPlatform_Checked(FeatureLevel), ActiveQualityLevel, true/*bAllowDefaultMaterial*/);
			if (MaterialResource)
			{
				GraphEvents.Append(MaterialResource->CollectPSOs(VertexFactoryDataList, InPreCacheParams, Priority, OutMaterialPSORequestIDs));
			}
		}
	}
	return GraphEvents;
}

void UMaterial::ReleaseResourcesAndMutateDDCKey(const FGuid& TransformationId)
{
	if (TransformationId.IsValid())
	{
		// Combine current guid with the transformation applied.
		StateId.A ^= TransformationId.A;
		StateId.B ^= TransformationId.B;
		StateId.C ^= TransformationId.C;
		StateId.D ^= TransformationId.D;
		
	}
	else
	{
		FPlatformMisc::CreateGuid(StateId);
	}

	if(FApp::CanEverRender())
	{
		for (FMaterialResource* CurrentResource : MaterialResources)
		{
			CurrentResource->ReleaseShaderMap();
		}

		// Release all resources because we could have changed the quality levels (e.g. in material editor).
		FMaterialResourceDeferredDeletionArray ResourcesToFree = MoveTemp(MaterialResources);
		MaterialResources.Reset();
		FMaterial::DeferredDeleteArray(ResourcesToFree);
	}
}

const FParameterGroupData* UMaterial::GetSortingGroup(const FString& InGroupName)
{
#if WITH_EDITOR
	UMaterialEditorOnlyData* LocalData = GetEditorOnlyData();
	if (LocalData)
	{
		return LocalData->ParameterGroupData.FindByKey(InGroupName);
	}
#endif
	return nullptr;
}

bool UMaterial::AttemptInsertNewGroupName(const FString& InNewName)
{
#if WITH_EDITOR
	UMaterialEditorOnlyData* LocalData = GetEditorOnlyData();
	if (LocalData)
	{
		FParameterGroupData* ParameterGroupDataElement = LocalData->ParameterGroupData.FindByKey(InNewName);
		if (ParameterGroupDataElement == nullptr)
		{
			FParameterGroupData NewGroupData;
			NewGroupData.GroupName = InNewName;
			NewGroupData.GroupSortPriority = 0;
			LocalData->ParameterGroupData.Add(NewGroupData);
			return true;
		}
	}
#endif
	return false;
}

FMaterialResource* UMaterial::GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel)
{
	if (QualityLevel == EMaterialQualityLevel::Num)
	{
		QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	}
	return FindMaterialResource(MaterialResources, InShaderPlatform, QualityLevel, true);
}

const FMaterialResource* UMaterial::GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel) const
{
	if (QualityLevel == EMaterialQualityLevel::Num)
	{
		QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	}
	return FindMaterialResource(MaterialResources, InShaderPlatform, QualityLevel, true);
}

void UMaterial::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Materials);
	SCOPED_LOADTIMER(MaterialSerializeTime);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);


#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsCooking() && ShouldStripUnusedDefaultTextures())
	{
		constexpr bool bKeepProjectPrivateTextures = true;
		StripUnusedDefaultTextures(bKeepProjectPrivateTextures);
	}
#endif

	Super::Serialize(Ar);

	if (Ar.UEVer() >= VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS)
	{
		UE::MaterialInterface::Private::SerializeInlineShaderMaps(
			Ar, LoadedMaterialResources
#if WITH_EDITOR
			, NAME_None
			, &CachedMaterialResourcesForCooking
#else
			, GetFName()
#endif
			);
	}
	else
	{
#if WITH_EDITOR
		FMaterialResource* LegacyResource = AllocateResource();
		LegacyResource->LegacySerialize(Ar);
		StateId = LegacyResource->GetLegacyId();
		delete LegacyResource;
#endif
	}

#if WITH_EDITOR
	// CachedExpressionData is moved to UMaterialInterface
	// Actual data will be regenerated on load in editor, so here we just need to handle skipping over any legacy data that might be in the archive
	{
		bool bLocalSavedCachedExpressionData_DEPRECATED = false;
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::MaterialSavedCachedData &&
			Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::MaterialInterfaceSavedCachedData)
		{
			Ar << bLocalSavedCachedExpressionData_DEPRECATED;
		}

		if (Ar.IsLoading() && bSavedCachedExpressionData_DEPRECATED)
		{
			bSavedCachedExpressionData_DEPRECATED = false;
			bLocalSavedCachedExpressionData_DEPRECATED = true;
		}

		if (bLocalSavedCachedExpressionData_DEPRECATED)
		{
			FMaterialCachedExpressionData LocalCachedExpressionData;
			UScriptStruct* Struct = FMaterialCachedExpressionData::StaticStruct();
			Struct->SerializeTaggedProperties(Ar, (uint8*)&LocalCachedExpressionData, Struct, nullptr);
		}
	}
#endif // WITH_EDITOR

#if WITH_EDITOR
	if (Ar.UEVer() < VER_UE4_FLIP_MATERIAL_COORDS)
	{
		GMaterialsThatNeedExpressionsFlipped.Set(this);
	}
	else if (Ar.UEVer() < VER_UE4_FIX_MATERIAL_COORDS)
	{
		GMaterialsThatNeedCoordinateCheck.Set(this);
	}
	else if (Ar.UEVer() < VER_UE4_FIX_MATERIAL_COMMENTS)
	{
		GMaterialsThatNeedCommentFix.Set(this);
	}

	if (Ar.UEVer() < VER_UE4_ADD_LINEAR_COLOR_SAMPLER)
	{
		GMaterialsThatNeedSamplerFixup.Set(this);
	}
#endif // #if WITH_EDITOR

	static_assert(MP_MAX == 35, "New material properties must have DoMaterialAttributeReorder called on them to ensure that any future reordering of property pins is correctly applied.");

	if (Ar.UEVer() < VER_UE4_MATERIAL_MASKED_BLENDMODE_TIDY)
	{
		//Set based on old value. Real check may not be possible here in cooked builds?
		//Cached using acutal check in PostEditChangProperty().
		if (BlendMode == BLEND_Masked && !bIsMasked_DEPRECATED)
		{
			bCanMaskedBeAssumedOpaque = true;
		}
		else
		{
			bCanMaskedBeAssumedOpaque = false;
		}
	}

	if(Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::IntroducedMeshDecals)
	{
		if(MaterialDomain == MD_DeferredDecal)
		{
			BlendMode = BLEND_Translucent;
		}
	}

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::NaniteForceMaterialUsage)
	{
		bool bForceNaniteUsage = false;
		if (Ar.IsSaving() && Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector())
		{
			static auto NaniteForceEnableMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.ForceEnableMeshes"));
			static bool bForceNaniteUsageValue = (NaniteForceEnableMeshesCVar && NaniteForceEnableMeshesCVar->GetValueOnAnyThread() != 0);
			bForceNaniteUsage = bForceNaniteUsageValue;
		}
		Ar << bForceNaniteUsage;
		if (Ar.IsLoading() && bForceNaniteUsage)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			bUsedWithNanite = true;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	
#if WITH_EDITOR
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RemoveDecalBlendMode)
	{
		if (MaterialDomain == MD_DeferredDecal)
		{
			GMaterialsThatNeedDecalFix.Set(this);
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MaterialFeatureLevelNodeFixForSM6)
	{
		GMaterialsThatNeedFeatureLevelSM6Fix.Set(this);
	}
#endif

#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && FShaderLibraryCooker::NeedsShaderStableKeys(EShaderPlatform::SP_NumPlatforms))
	{
		SaveShaderStableKeys(Ar.CookingTarget());
	}
#endif

#if WITH_EDITORONLY_DATA
	if (MaterialDomain == MD_Volume && Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::VolumeExtinctionBecomesRGB)
	{
		// Note: we work on _DEPRECATED data because that will be used later to create the EditorOnly data in PostLoad().
		if (Opacity_DEPRECATED.IsConnected()) // Base material input cannot have default values so we only deal with connected expression
		{
			// Change expression output from the Opacity to SubSurfaceColor that is now representing RGB extinction. Leave opacity connected as it is unused now anyway
			SubsurfaceColor_DEPRECATED.Connect(Opacity_DEPRECATED.OutputIndex, Opacity_DEPRECATED.Expression);
			// Now disconnect Opacity
			Opacity_DEPRECATED.Expression = nullptr;

			// Now force the material to recompile and we use a hash of the original StateId.
			// This is to avoid having different StateId each time we load the material and to not forever recompile it,i.e. use a cached version.
			static FGuid VolumeExtinctionBecomesRGBConversionGuid(TEXT("2768E88D-9B58-4C53-9CB9-75696D1DF0CD"));
			ReleaseResourcesAndMutateDDCKey(VolumeExtinctionBecomesRGBConversionGuid);
		}
	}
#endif // WITH_EDITORONLY_DATA

	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MaterialTranslucencyPass)
	{
		if (bEnableSeparateTranslucency_DEPRECATED == false)
		{
			TranslucencyPass = MTP_BeforeDOF;
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MaterialPixelDepthOffsetMode)
	{
		// All previous material must use the legacy pixel depth offset mode. New material will use the new mode.
		PixelDepthOffsetMode = PDOM_Legacy;
	}

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SubsurfaceProfileGuid)
	{
		// Add the unique legacy profile to the Substrate profile list the first time we load an older asset.
		if (SubsurfaceProfiles.IsEmpty() && SubsurfaceProfile)
		{
			SubsurfaceProfiles.Add(SubsurfaceProfile);
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddMeshDeformerUsageFlag)
	{
		// Default mesh deformer usage to match the skeletal mesh usage.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bUsedWithMeshDeformer = bUsedWithSkeletalMesh;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsCooking() && ShouldStripUnusedDefaultTextures())
	{
		// Undo textures removal for Multi Platform Cook
		ReferencedDefaultTextures = CachedExpressionData->EditorOnlyData->EditorReferencedDefaultTextures;
	}
#endif
}

void UMaterial::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!CVarMaterialsDuplicateVerbatim.GetValueOnAnyThread())
	{
		// Reset the StateId on duplication since it needs to be unique for each material.
		FPlatformMisc::CreateGuid(StateId);
	}
}

#if WITH_EDITOR

void UMaterial::BackwardsCompatibilityInputConversion()
{
	if( ShadingModel != MSM_Unlit )
	{
		UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
		bool bIsDS = DiffuseColor_DEPRECATED.IsConnected() || SpecularColor_DEPRECATED.IsConnected();
		bool bIsBMS = EditorOnly->BaseColor.IsConnected() || EditorOnly->Metallic.IsConnected() || EditorOnly->Specular.IsConnected();

		if( bIsDS && !bIsBMS )
		{
			// ConvertFromDiffSpec
			if (!GConvertFromDiffSpecMaterialFunction)
			{
				GConvertFromDiffSpecMaterialFunction = LoadObject< UMaterialFunction >(nullptr, TEXT("/Engine/Functions/Engine_MaterialFunctions01/Shading/ConvertFromDiffSpec.ConvertFromDiffSpec"), nullptr, LOAD_None, nullptr);
				checkf(GConvertFromDiffSpecMaterialFunction, TEXT("Cannot load ConvertFromDiffSpec"));
				GConvertFromDiffSpecMaterialFunction->AddToRoot();
				check(!GConvertFromDiffSpecMaterialFunction->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
			}

			check( GConvertFromDiffSpecMaterialFunction );

			UMaterialExpressionMaterialFunctionCall* FunctionExpression = NewObject<UMaterialExpressionMaterialFunctionCall>(this);
			EditorOnly->ExpressionCollection.Expressions.Add( FunctionExpression );

			FunctionExpression->MaterialExpressionEditorX += 200;

			FunctionExpression->MaterialFunction = GConvertFromDiffSpecMaterialFunction;
			FunctionExpression->UpdateFromFunctionResource();

			if( DiffuseColor_DEPRECATED.IsConnected() )
			{
				FunctionExpression->GetInput(0)->Connect( DiffuseColor_DEPRECATED.OutputIndex, DiffuseColor_DEPRECATED.Expression );
			}

			if( SpecularColor_DEPRECATED.IsConnected() )
			{
				FunctionExpression->GetInput(1)->Connect( SpecularColor_DEPRECATED.OutputIndex, SpecularColor_DEPRECATED.Expression );
			}

			EditorOnly->BaseColor.Connect( 0, FunctionExpression );
			EditorOnly->Metallic.Connect( 1, FunctionExpression );
			EditorOnly->Specular.Connect( 2, FunctionExpression );
		}
	}
}

void UMaterial::BackwardsCompatibilityVirtualTextureOutputConversion()
{
	// Remove MD_RuntimeVirtualTexture support and replace with an explicit UMaterialExpressionRuntimeVirtualTextureOutput.
	if (MaterialDomain == MD_RuntimeVirtualTexture)
	{
		// Change this guid if you change the conversion code below
		static FGuid BackwardsCompatibilityVirtualTextureOutputConversionGuid(TEXT("BABD7074-001F-4FC2-BDE5-3A0C436F4414"));

		UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();

		MaterialDomain = MD_Surface;

		if (!bUseMaterialAttributes)
		{
			// Create a new UMaterialExpressionRuntimeVirtualTextureOutput node and route the old material attribute output to it.
			UMaterialExpressionRuntimeVirtualTextureOutput* OutputExpression = NewObject<UMaterialExpressionRuntimeVirtualTextureOutput>(this);
			EditorOnly->ExpressionCollection.Expressions.Add(OutputExpression);

			OutputExpression->MaterialExpressionEditorX = EditorX;
			OutputExpression->MaterialExpressionEditorY = EditorY - 300;

			if (EditorOnly->BaseColor.IsConnected())
			{
				OutputExpression->GetInput(0)->Connect(EditorOnly->BaseColor.OutputIndex, EditorOnly->BaseColor.Expression);
			}
			if (EditorOnly->Specular.IsConnected())
			{
				OutputExpression->GetInput(1)->Connect(EditorOnly->Specular.OutputIndex, EditorOnly->Specular.Expression);
			}
			if (EditorOnly->Roughness.IsConnected())
			{
				OutputExpression->GetInput(2)->Connect(EditorOnly->Roughness.OutputIndex, EditorOnly->Roughness.Expression);
			}
			if (EditorOnly->Normal.IsConnected())
			{
				if (bTangentSpaceNormal)
				{
					OutputExpression->GetInput(3)->Connect(EditorOnly->Normal.OutputIndex, EditorOnly->Normal.Expression);
				}
				else
				{
					// Apply the tangent space to world transform that would be applied in the material output.
					UMaterialExpressionTransform* TransformExpression = NewObject<UMaterialExpressionTransform>(this);
					EditorOnly->ExpressionCollection.Expressions.Add(TransformExpression);

					TransformExpression->MaterialExpressionEditorX = EditorX - 300;
					TransformExpression->MaterialExpressionEditorY = EditorY - 300;
					TransformExpression->TransformSourceType = TRANSFORMSOURCE_Tangent;
					TransformExpression->TransformType = TRANSFORM_World;
					TransformExpression->Input.Connect(EditorOnly->Normal.OutputIndex, EditorOnly->Normal.Expression);

					OutputExpression->GetInput(3)->Connect(0, TransformExpression);
				}
			}
			if (EditorOnly->Opacity.IsConnected())
			{
				OutputExpression->GetInput(5)->Connect(EditorOnly->Opacity.OutputIndex, EditorOnly->Opacity.Expression);
			}

			if (!IsOpaqueBlendMode(BlendMode))
			{
				// Full alpha blend modes were mostly/always used with MD_RuntimeVirtualTexture to allow pin connections.
				// But we will assume the intention for any associated MD_Surface output is opaque or alpha mask and force convert here.
				if (EditorOnly->Opacity.IsConnected())
				{
					EditorOnly->OpacityMask.Connect(EditorOnly->Opacity.OutputIndex, EditorOnly->Opacity.Expression);
					EditorOnly->Opacity.Expression = nullptr;
				}
				BlendMode = EditorOnly->OpacityMask.IsConnected() ? BLEND_Masked : BLEND_Opaque;
				bCanMaskedBeAssumedOpaque = !EditorOnly->OpacityMask.Expression && !(EditorOnly->OpacityMask.UseConstant && EditorOnly->OpacityMask.Constant < 0.999f);
			}
		}

		// Recompile after changes with a guid representing the conversion applied here.
		ReleaseResourcesAndMutateDDCKey(BackwardsCompatibilityVirtualTextureOutputConversionGuid);
	}
}

void UMaterial::BackwardsCompatibilityDecalConversion()
{
	if (GMaterialsThatNeedDecalFix.Get(this))
	{
		// Change this guid if you change the conversion code below
		static FGuid BackwardsCompatibilityDecalConversionGuid(TEXT("352069F8-1B8C-406A-9B88-6946BCDF2C10"));
		
		UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();

		GMaterialsThatNeedDecalFix.Clear(this);

		// Move stain and alpha composite setting into material blend mode.
		if (DecalBlendMode == DBM_AlphaComposite)
		{
			BlendMode = BLEND_AlphaComposite;
		}
		else if (DecalBlendMode == DBM_Stain)
		{
			BlendMode = BLEND_Modulate;
		}
		else
		{
			BlendMode = BLEND_Translucent;
		}

		// Disconnect outputs according to old DBuffer blend mode.
		if (DecalBlendMode == DBM_DBuffer_Normal || DecalBlendMode == DBM_DBuffer_Roughness || DecalBlendMode == DBM_DBuffer_NormalRoughness)
		{
			EditorOnly->BaseColor.Expression = nullptr;
		}
		if (DecalBlendMode == DBM_DBuffer_Color || DecalBlendMode == DBM_DBuffer_Roughness || DecalBlendMode == DBM_DBuffer_ColorRoughness || DecalBlendMode == DBM_AlphaComposite)
		{
			EditorOnly->Normal.Expression = nullptr;
		}
		if (DecalBlendMode == DBM_DBuffer_Color || DecalBlendMode == DBM_DBuffer_Normal || DecalBlendMode == DBM_DBuffer_ColorNormal)
		{
			EditorOnly->Roughness.Expression = EditorOnly->Specular.Expression = EditorOnly->Metallic.Expression = nullptr;
		}

		// Previously translucent decals used default values in all unconnected attributes (except for normal).
		// For backwards compatibility we connect those attributes with defaults.
		if (DecalBlendMode == DBM_Translucent || DecalBlendMode == DBM_AlphaComposite || DecalBlendMode == DBM_Stain)
		{
			if (!EditorOnly->BaseColor.IsConnected() || !EditorOnly->Metallic.IsConnected())
			{
				UMaterialExpressionConstant* Expression = NewObject<UMaterialExpressionConstant>(this);
				EditorOnly->ExpressionCollection.Expressions.Add(Expression);

				Expression->MaterialExpressionEditorX = EditorX - 100;
				Expression->MaterialExpressionEditorY = EditorY - 120;
				Expression->R = 0.f;

				if (!EditorOnly->BaseColor.IsConnected())
				{
					EditorOnly->BaseColor.Connect(0, Expression);
				}
				if (!EditorOnly->Metallic.IsConnected())
				{
					EditorOnly->Metallic.Connect(0, Expression);
				}
			}

			if (!EditorOnly->Roughness.IsConnected() || !EditorOnly->Specular.IsConnected())
			{
				UMaterialExpressionConstant* Expression = NewObject<UMaterialExpressionConstant>(this);
				EditorOnly->ExpressionCollection.Expressions.Add(Expression);

				Expression->MaterialExpressionEditorX = EditorX - 100;
				Expression->MaterialExpressionEditorY = EditorY - 60;
				Expression->R = .5f;

				if (!EditorOnly->Roughness.IsConnected())
				{
					EditorOnly->Roughness.Connect(0, Expression);
				}
				if (!EditorOnly->Specular.IsConnected())
				{
					EditorOnly->Specular.Connect(0, Expression);
				}
			}
		}

		// Recompile after changes with a guid representing the conversion applied here.
		ReleaseResourcesAndMutateDDCKey(BackwardsCompatibilityDecalConversionGuid);
	}
}
#endif // WITH_EDITOR

static void AddSurfaceSubstrateShadingModelFromMaterialShadingModels(FSubstrateMaterialInfo& OutInfo, const FMaterialShadingModelField& InShadingModels)
{
	if (InShadingModels.HasShadingModel(MSM_Unlit))				{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_Unlit); }
	if (InShadingModels.HasShadingModel(MSM_DefaultLit))		{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_DefaultLit); }
	if (InShadingModels.HasShadingModel(MSM_Subsurface))		{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceWrap); }
	if (InShadingModels.HasShadingModel(MSM_PreintegratedSkin))	{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceMFP); }
	if (InShadingModels.HasShadingModel(MSM_ClearCoat))			{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_ClearCoat); }
	if (InShadingModels.HasShadingModel(MSM_SubsurfaceProfile))	{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceProfile); }
	if (InShadingModels.HasShadingModel(MSM_TwoSidedFoliage))	{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceThinTwoSided); }
	if (InShadingModels.HasShadingModel(MSM_Hair))				{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_Hair); }
	if (InShadingModels.HasShadingModel(MSM_Cloth))				{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_Cloth); }
	if (InShadingModels.HasShadingModel(MSM_Eye))				{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_Eye); }
	if (InShadingModels.HasShadingModel(MSM_SingleLayerWater))	{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_SingleLayerWater); }
	if (InShadingModels.HasShadingModel(MSM_ThinTranslucent))	{ OutInfo.AddShadingModel(ESubstrateShadingModel::SSM_ThinTranslucent); }
}

static void AddSurfaceSubstrateShadingModelFromMaterialShadingModel(FSubstrateMaterialInfo& OutInfo, const EMaterialShadingModel& InShadingModel)
{
	if (InShadingModel < MSM_NUM)
	{
		FMaterialShadingModelField ShadingModel;
		ShadingModel.AddShadingModel(InShadingModel);
		AddSurfaceSubstrateShadingModelFromMaterialShadingModels(OutInfo, ShadingModel);
	}
}

EBlendMode ConvertLegacyBlendMode(EBlendMode InBlendMode, FMaterialShadingModelField InShadingModels)
{
	if (InShadingModels.CountShadingModels() == 1 && InShadingModels.GetFirstShadingModel() == EMaterialShadingModel::MSM_ThinTranslucent)
	{
		return BLEND_TranslucentColoredTransmittance;
	}
	else if (InBlendMode == BLEND_Translucent)
	{
		return BLEND_TranslucentGreyTransmittance;
	}
	return InBlendMode;
}

#define SUBSTRATE_MOVE_CONNECTION 0
#define SUBSTRATE_COPY_CONNECTION 1

// Current conversion version - This means that the material has run ConvertMaterialToSubstrateMaterial() once, 
// and its data has been converted.
static int32 GetSubstrateConversionVersion()	{ return  1; }
// Default version - This means that:
//  * The material has run ConvertMaterialToSubstrateMaterial() once, but no conversion was needed 
//  * Or that the material has been updated/touched since its conversion (e.g., manual edit)
// In both case, there is no longer needs to rerun ConvertMaterialToSubstrateMaterial()
static int32 GetSubstrateNoConversionVersion()	{ return  0; }
// Invalid version - This means that the material has never run ConvertMaterialToSubstrateMaterial()
static int32 GetSubstrateInvalidVersion()		{ return -1; }
// Substrate version
// 0: Glint conversion
static int32 GetSubstrateVersion()				{ return 0; }

#if WITH_EDITOR


bool UMaterial::UserRequestsConvertMaterialToSubstrateMaterial()
{
	bool bUpdated = false;
	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
	if (Substrate::IsSubstrateEnabled() && EditorOnly && !EditorOnly->FrontMaterial.IsConnected())
	{
		const FScopedTransaction Transaction(LOCTEXT("SubstrateConversion", "Substrate Conversion"));

		GetTransientPackage()->Modify();
		EditorOnly->Modify();
		MaterialGraph->Modify();
		Modify(true);

		bUpdated = ConvertMaterialToSubstrateMaterial(true /*bAllowEmptyMaterialUpdate*/, true /*bRequestedByUser*/);
		if (bUpdated)
		{
			if (MaterialGraph)
			{
				MaterialGraph->RebuildGraph();
			}

			GetTransientPackage()->MarkPackageDirty();
			EditorOnly->MarkPackageDirty();
			MaterialGraph->MarkPackageDirty();
			MarkPackageDirty();

			UpdateCachedExpressionData();

			PropagateDataToMaterialProxy();

			PostEditChange();

			ForceRecompileForRendering();
		}
	}
	return bUpdated;
}

bool UMaterial::ConvertMaterialToSubstrateMaterial(bool bAllowEmptyMaterialUpdate, bool bRequestedByUser)
{
	/*
	* The data flow for legacy material conversion node that can be used in isolation is as such:
	*
	* --- Conversion time - UMaterial::ConvertMaterialToSubstrateMaterial()
	*     Shading model => legacy node setup (shading model from expression? relink input : otherwise set the shading model on the conversion node itself)
	*
	* --- GatherSubstrateMaterialInfo
	*     Send back what is the shading model used, or if it is "from expression"
	*
	* --- UMaterial::RebuildShadingModelField()
	*     Set material Domain, ShadingModel and ShadingModels according to the returned value from GatherSubstrateMaterialInfo gathered from the graph.
	*     Those values are not authorable but do enable shading models via defines from the HLSLTranslator. Seeing them being correct also helps to understand and debug.
	*
	* --- Material instance shading model override
	*     Overridden from the HLSLTranslator when detected by comparing base and instanced materials.
	*/
	if (!Substrate::IsSubstrateEnabled())
	{
		return false;
	}

	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();

	// * If the current material has already been converted & versioned, skip the conversion. 
	// * Store the current version of Substrate's auto-conversion
	//   This allows to version the conversion, and safely update auto-converted materials if they have been saved.
	// * If no conversion was needed, the conversion version is set to 0, to avoid rerunning this function
	if ((EditorOnly->SubstrateConversionVersion == GetSubstrateConversionVersion() || EditorOnly->SubstrateConversionVersion == GetSubstrateNoConversionVersion()) && EditorOnly->SubstrateVersion == GetSubstrateVersion() && !bRequestedByUser)
	{
		return false;
	}
	else if (bRequestedByUser && EditorOnly->FrontMaterial.IsConnected())
	{
		RebuildShadingModelField();
		return false; // We do not convert if the front material pin has is already connected.
	}
	else if(Substrate::IsHiddenMaterialAssetConversionEnabled() && !bRequestedByUser)
	{
		RebuildShadingModelField();
		return false; // This is to work with ShouldAllowSubstrateReset. So that a material authored with IsHiddenMaterialAssetConversionEnabled using the root node inputs can then be converted to substrate.
	}

	// Store current node post from the root node.
	int32 CurrentNodePosX = EditorX;
	int32 ConstantNodePosX = EditorX;
	int32 ConstantNodePosY = EditorY;
	const int32 TranslationOffsetX = 350.0f;
	auto SetPosXAndMoveReferenceToTheRight = [&](UMaterialExpression* Node)
	{
		Node->MaterialExpressionEditorX = CurrentNodePosX;
		CurrentNodePosX += TranslationOffsetX;
	};
	auto ReplaceNodeAndMoveToTheRight = [&](UMaterialExpression* NodeToReplace, UMaterialExpression* NewNode)
	{
		NewNode->MaterialExpressionEditorX = NodeToReplace->MaterialExpressionEditorX;
		NewNode->MaterialExpressionEditorY = NodeToReplace->MaterialExpressionEditorY;
		NodeToReplace->MaterialExpressionEditorX = NewNode->MaterialExpressionEditorX + TranslationOffsetX;
		CurrentNodePosX = FMath::Max(NodeToReplace->MaterialExpressionEditorX + TranslationOffsetX, CurrentNodePosX);
	};
	auto BeginConstantNodeLayout = [&]()
	{
		// Leave some space for constants
		CurrentNodePosX += TranslationOffsetX;
	};
	auto LaydownConstant = [&](UMaterialExpression* Node)
	{
		// Position the constant on the constant column, each one on top of each other
		Node->MaterialExpressionEditorX = ConstantNodePosX;
		Node->MaterialExpressionEditorY = ConstantNodePosY;
		ConstantNodePosY += Node->IsA<UMaterialExpressionConstant>() ? 80 : 200;
	};

	// Recommendation: use the non indexed version whenever possible. However, indexed version are mandatory for material function which do not have explicit input.
	auto ConnectionToIndexed = [](auto& OldNodeInput, UMaterialExpression* NewNode, uint32 NewInputIndex, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		if (OldNodeInput.IsConnected())
		{
			NewNode->GetInput(NewInputIndex)->Connect(OldNodeInput.OutputIndex, OldNodeInput.Expression);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
		}
	};
	auto ConnectionTo = [](auto& OldNodeInput, FExpressionInput& NewNodeInput, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		if (OldNodeInput.IsConnected())
		{
			NewNodeInput.Connect(OldNodeInput.OutputIndex, OldNodeInput.Expression);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
		}
	};
	auto ScalarMatInputConnectionToIndexed = [&](FScalarMaterialInput& OldNodeInput, UMaterialExpression* NewNode, uint32 NewInputIndex, EMaterialProperty Property, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		FVector4f DefaultPinValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);

		if (OldNodeInput.IsConnected())
		{
			ConnectionToIndexed(OldNodeInput, NewNode, NewInputIndex, OperationType);
		}
		else if (OldNodeInput.UseConstant && DefaultPinValue.X != OldNodeInput.Constant)
		{
			UMaterialExpressionConstant* ExpressionConstant = NewObject<UMaterialExpressionConstant>(this);
			ExpressionConstant->Material = this;
			EditorOnly->ExpressionCollection.AddExpression(ExpressionConstant);
			ExpressionConstant->R = OldNodeInput.Constant;
			NewNode->GetInput(NewInputIndex)->Connect(0, ExpressionConstant);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
			LaydownConstant(ExpressionConstant);
		}
	};
	auto ScalarMatInputConnectionTo = [&](FScalarMaterialInput& OldNodeInput, FExpressionInput& NewNodeInput, EMaterialProperty Property, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		FVector4f DefaultPinValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);

		if (OldNodeInput.IsConnected())
		{
			ConnectionTo(OldNodeInput, NewNodeInput, OperationType);
		}
		else if (OldNodeInput.UseConstant && DefaultPinValue.X != OldNodeInput.Constant)
		{
			UMaterialExpressionConstant* ExpressionConstant = NewObject<UMaterialExpressionConstant>(this);
			ExpressionConstant->Material = this;
			EditorOnly->ExpressionCollection.AddExpression(ExpressionConstant);
			ExpressionConstant->R = OldNodeInput.Constant;
			NewNodeInput.Connect(0, ExpressionConstant);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
			LaydownConstant(ExpressionConstant);
		}
	};
	auto ColorMatInputConnectionToIndexed = [&](FColorMaterialInput& OldNodeInput, UMaterialExpression* NewNode, uint32 NewInputIndex, EMaterialProperty Property, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		FVector4f DefaultPinValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);
		FLinearColor DefaultPinLinearColor = FLinearColor(DefaultPinValue.X, DefaultPinValue.Y, DefaultPinValue.Z);
		FLinearColor CurrentPinLinearColor = OldNodeInput.Constant;

		if (OldNodeInput.IsConnected())
		{
			ConnectionToIndexed(OldNodeInput, NewNode, NewInputIndex, OperationType);
		}
		else if (OldNodeInput.UseConstant && DefaultPinLinearColor != CurrentPinLinearColor)
		{
			UMaterialExpressionConstant3Vector* ExpressionConstant = NewObject<UMaterialExpressionConstant3Vector>(this);
			ExpressionConstant->Material = this;
			EditorOnly->ExpressionCollection.AddExpression(ExpressionConstant);
			ExpressionConstant->Constant = CurrentPinLinearColor;
			NewNode->GetInput(NewInputIndex)->Connect(0, ExpressionConstant);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
			LaydownConstant(ExpressionConstant);
		}
	};
	auto ColorMatInputConnectionTo = [&](FColorMaterialInput& OldNodeInput, FExpressionInput& NewNodeInput, EMaterialProperty Property, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		FVector4f DefaultPinValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);
		FLinearColor DefaultPinLinearColor = FLinearColor(DefaultPinValue.X, DefaultPinValue.Y, DefaultPinValue.Z);
		FLinearColor CurrentPinLinearColor = OldNodeInput.Constant;

		if (OldNodeInput.IsConnected())
		{
			ConnectionTo(OldNodeInput, NewNodeInput, OperationType);
		}
		else if (OldNodeInput.UseConstant && DefaultPinLinearColor != CurrentPinLinearColor)
		{
			UMaterialExpressionConstant3Vector* ExpressionConstant = NewObject<UMaterialExpressionConstant3Vector>(this);
			ExpressionConstant->Material = this;
			EditorOnly->ExpressionCollection.AddExpression(ExpressionConstant);
			ExpressionConstant->Constant = CurrentPinLinearColor;
			NewNodeInput.Connect(0, ExpressionConstant);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
			LaydownConstant(ExpressionConstant);
		}
	};
	auto Vector3MatInputConnectionToIndexed = [&](FVectorMaterialInput& OldNodeInput, UMaterialExpression* NewNode, uint32 NewInputIndex, EMaterialProperty Property, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		FVector4f DefaultPinValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);
		FVector3f DefaultPinVec3f = FVector3f(DefaultPinValue.X, DefaultPinValue.Y, DefaultPinValue.Z);
		FVector3f CurrentPinVec3f = FVector3f(OldNodeInput.Constant.X, OldNodeInput.Constant.Y, OldNodeInput.Constant.Z);

		if (OldNodeInput.IsConnected())
		{
			ConnectionToIndexed(OldNodeInput, NewNode, NewInputIndex, OperationType);
		}
		else if (OldNodeInput.UseConstant && DefaultPinVec3f != CurrentPinVec3f)
		{
			UMaterialExpressionConstant3Vector* ExpressionConstant = NewObject<UMaterialExpressionConstant3Vector>(this);
			ExpressionConstant->Material = this;
			EditorOnly->ExpressionCollection.AddExpression(ExpressionConstant);
			ExpressionConstant->Constant = CurrentPinVec3f;
			NewNode->GetInput(NewInputIndex)->Connect(0, ExpressionConstant);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
			LaydownConstant(ExpressionConstant);
		}
	};
	auto Vector3MatInputConnectionTo = [&](FVectorMaterialInput& OldNodeInput, FExpressionInput& NewNodeInput, EMaterialProperty Property, uint32 OperationType = SUBSTRATE_MOVE_CONNECTION)
	{
		FVector4f DefaultPinValue = FMaterialAttributeDefinitionMap::GetDefaultValue(Property);
		FVector3f DefaultPinVec3f = FVector3f(DefaultPinValue.X, DefaultPinValue.Y, DefaultPinValue.Z);
		FVector3f CurrentPinVec3f = FVector3f(OldNodeInput.Constant.X, OldNodeInput.Constant.Y, OldNodeInput.Constant.Z);

		if (OldNodeInput.IsConnected())
		{
			ConnectionTo(OldNodeInput, NewNodeInput, OperationType);
		}
		else if (OldNodeInput.UseConstant && DefaultPinVec3f != CurrentPinVec3f)
		{
			UMaterialExpressionConstant3Vector* ExpressionConstant = NewObject<UMaterialExpressionConstant3Vector>(this);
			ExpressionConstant->Material = this;
			EditorOnly->ExpressionCollection.AddExpression(ExpressionConstant);
			ExpressionConstant->Constant = CurrentPinVec3f;
			NewNodeInput.Connect(0, ExpressionConstant);
			if (OperationType == SUBSTRATE_MOVE_CONNECTION)
			{
				OldNodeInput.Expression = nullptr;
			}
			LaydownConstant(ExpressionConstant);
		}
	};

	bool bCustomNodesGathered = false;
	UMaterialExpressionThinTranslucentMaterialOutput* ThinTranslucentOutput = nullptr;
	UMaterialExpressionSingleLayerWaterMaterialOutput* SingleLayerWaterOutput = nullptr;
	UMaterialExpressionClearCoatNormalCustomOutput* ClearCoatBottomNormalOutput = nullptr;
	UMaterialExpressionTangentOutput* TangentOutput = nullptr;
	auto GatherCustomNodes = [&]()
	{
		if (!bCustomNodesGathered)
		{
			bCustomNodesGathered = true;
			TArray<class UMaterialExpressionCustomOutput*> CustomOutputExpressions;
			GetAllCustomOutputExpressions(CustomOutputExpressions);
			for (UMaterialExpressionCustomOutput* Expression : CustomOutputExpressions)
			{
				// Gather custom output for thin translucency
				if (ThinTranslucentOutput == nullptr && Cast<UMaterialExpressionThinTranslucentMaterialOutput>(Expression))
				{
					ThinTranslucentOutput = Cast<UMaterialExpressionThinTranslucentMaterialOutput>(Expression);
				}

				// Gather custom output for single layer water
				if (SingleLayerWaterOutput == nullptr && Cast<UMaterialExpressionSingleLayerWaterMaterialOutput>(Expression))
				{
					SingleLayerWaterOutput = Cast<UMaterialExpressionSingleLayerWaterMaterialOutput>(Expression);
				}

				// Gather custom output for clear coat
				if (ClearCoatBottomNormalOutput == nullptr && Cast<UMaterialExpressionClearCoatNormalCustomOutput>(Expression))
				{
					ClearCoatBottomNormalOutput = Cast<UMaterialExpressionClearCoatNormalCustomOutput>(Expression);
				}

				// Gather custom output for tangent (unused atm)
				if (TangentOutput == nullptr && Cast<UMaterialExpressionTangentOutput>(Expression))
				{
					TangentOutput = Cast<UMaterialExpressionTangentOutput>(Expression);
				}

				if (ThinTranslucentOutput && SingleLayerWaterOutput && ClearCoatBottomNormalOutput && TangentOutput)
				{
					break;
				}
			}
		}
	};

	bool bInvalidateShader = false;
	bool bEmptyShader = false;
	// We only convert materials to substrate when IsHiddenMaterialAssetConversionEnabled is false.
	// However, we do run the Substrate conversion/update code to make sure material with FrontMaterial plugged in are still converted.
	if (Substrate::IsSubstrateEnabled() && (!Substrate::IsHiddenMaterialAssetConversionEnabled() || bRequestedByUser))
	{
		BeginConstantNodeLayout();

		// SSS Profile
		const bool bHasShadingModelMixture		= ShadingModels.CountShadingModels() > 1;
		const bool bRequireSubsurfacePasses		= ShadingModels.HasShadingModel(MSM_SubsurfaceProfile) || ShadingModels.HasShadingModel(MSM_Subsurface) || ShadingModels.HasShadingModel(MSM_PreintegratedSkin) || ShadingModels.HasShadingModel(MSM_Eye);
		const bool bRequireNoSubsurfaceProfile	= !bHasShadingModelMixture && (ShadingModel == MSM_Subsurface || ShadingModel == MSM_PreintegratedSkin); // Insure there is no profile, as this would take priority otherwise

		// Empty material (i.e., material without any expression) can be 'patched' with UE4Legacy's MF in order to have correct visuals by default
		const bool bUpdateEmptyMaterial = bAllowEmptyMaterialUpdate && !bUseMaterialAttributes && !EditorOnly->FrontMaterial.IsConnected() && GetExpressions().IsEmpty();

		bool bRelinkCustomOutputNodes = false;
		UMaterialExpressionSubstrateShadingModels* ConvertNode = nullptr;
		// Connect all the legacy pin into the conversion node
		{
			const bool bHasAnySubstrateNodes = HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionSubstrateBSDF>();
			if (bUseMaterialAttributes && EditorOnly->MaterialAttributes.Expression && !EditorOnly->FrontMaterial.IsConnected() && !EditorOnly->MaterialAttributes.Expression->IsResultSubstrateMaterial(EditorOnly->MaterialAttributes.OutputIndex) && !bHasAnySubstrateNodes) // M_Rifle cause issues there
			{
				UMaterialExpressionSubstrateConvertMaterialAttributes* ConvertAttributeNode = NewObject<UMaterialExpressionSubstrateConvertMaterialAttributes>(this);
				EditorOnly->ExpressionCollection.AddExpression(ConvertAttributeNode);
				ConvertAttributeNode->Material = this;
				SetPosXAndMoveReferenceToTheRight(ConvertAttributeNode);
				ConvertAttributeNode->SubsurfaceProfile = bRequireNoSubsurfaceProfile ? nullptr : SubsurfaceProfile;

				// * Copy the material attribute connection to the conversion node.
				// * Leave the material attribute existing connection plugged to the root node, 
				//   so that other input (PixelDepthOffset, WorldPositionOffset, ...) get pull 
				//   from the material attributes node
				ConnectionTo(EditorOnly->MaterialAttributes, ConvertAttributeNode->MaterialAttributes, SUBSTRATE_COPY_CONNECTION);

				// Reconnect custom output to material attribute conversion node
				{
					check(ConvertAttributeNode);
					GatherCustomNodes();

					if (SingleLayerWaterOutput)
					{
						ConnectionTo(SingleLayerWaterOutput->ScatteringCoefficients,	ConvertAttributeNode->WaterScatteringCoefficients);
						ConnectionTo(SingleLayerWaterOutput->AbsorptionCoefficients,	ConvertAttributeNode->WaterAbsorptionCoefficients);
						ConnectionTo(SingleLayerWaterOutput->PhaseG,					ConvertAttributeNode->WaterPhaseG);
						ConnectionTo(SingleLayerWaterOutput->ColorScaleBehindWater,		ConvertAttributeNode->ColorScaleBehindWater);
					}
				}

				// If the graph contains a 'tangent output' node and uses 'eye' shading model, we add the custom tangent to the material attribute list
				const bool bRequireCustomTangent = ShadingModels.HasShadingModel(MSM_Eye) && TangentOutput;
				if (bRequireCustomTangent)
				{
					const FGuid MaterialAttributeGuid = FMaterialAttributeDefinitionMap::GetID(MP_MaterialAttributes);
					const FGuid CustomEyeTangentGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("CustomEyeTangent"));

					UMaterialExpressionSetMaterialAttributes* SetAttributesNode = NewObject<UMaterialExpressionSetMaterialAttributes>(this);
					EditorOnly->ExpressionCollection.AddExpression(SetAttributesNode); // Add the SetAttributesNode to ensure the MaterialAttribute input are correctly tracked & cached
					SetAttributesNode->Material = this;
					SetAttributesNode->Inputs[0].Connect(0, EditorOnly->MaterialAttributes.Expression);
					SetAttributesNode->AttributeSetTypes.Add(CustomEyeTangentGuid);
					SetAttributesNode->Inputs.Add(TangentOutput->Input);

					// Connect to conversion node
					ConvertAttributeNode->MaterialAttributes.Connect(0, SetAttributesNode);

					// Connect to root node
					EditorOnly->MaterialAttributes.Connect(0, SetAttributesNode);
				}

				// Connect converted Substrate data to root node
				EditorOnly->FrontMaterial.Connect(0, ConvertAttributeNode);

				// Shading Model
				// * either use the shader graph expression 
				// * or add a constant shading model
				if (ShadingModel == MSM_FromMaterialExpression)
				{
					ConvertAttributeNode->ShadingModelOverride = MSM_FromMaterialExpression;
				}
				else
				{
					// Store Substrate shading model of the converted material. 
					check(ShadingModels.CountShadingModels() == 1);
					ConvertAttributeNode->ShadingModelOverride = ShadingModel;

					// Apply is thin surface if needed
					if (ShadingModel == MSM_TwoSidedFoliage)
					{
						bIsThinSurface = true;
					}
				}

				if (MaterialDomain == MD_DeferredDecal)
				{
					// For now we don't enforce shading model since it could be driven by expression and we don't have much 
					// control on this, but only DefaultLit should be supported.

					// Now pass through the convert to decal node, which flag the material as SSM_Decal, which will set the domain to Decal.
					UMaterialExpressionSubstrateConvertToDecal* ConvertToDecalNode = NewObject<UMaterialExpressionSubstrateConvertToDecal>(this);
					EditorOnly->ExpressionCollection.AddExpression(ConvertToDecalNode);
					ConvertAttributeNode->Material = this;
					ReplaceNodeAndMoveToTheRight(ConvertAttributeNode, ConvertToDecalNode);
					ConvertToDecalNode->DecalMaterial.Connect(0, ConvertAttributeNode);

					EditorOnly->FrontMaterial.Connect(0, ConvertToDecalNode);
				}

				BlendMode = ConvertLegacyBlendMode(BlendMode, ShadingModels);
				RefractionCoverageMode = RCM_CoverageIgnored;
				bInvalidateShader = true;
			}
			else if (bUpdateEmptyMaterial)
			{
				// Empty material: Create by default a slab node
				UMaterialFunction* DefaultMF = LoadObject<UMaterialFunction>(nullptr, TEXT("/Engine/Functions/Substrate/SMF_UE4Legacy.SMF_UE4Legacy")); 
				if (DefaultMF)
				{
					DefaultMF->UpdateFromFunctionResource();
					DefaultMF->PostEditChange();
					DefaultMF->ConditionalPostLoad();

					UMaterialExpressionMaterialFunctionCall* MFCallNode = NewObject<UMaterialExpressionMaterialFunctionCall>(this);
					if (MFCallNode->SetMaterialFunction(DefaultMF))
					{
						// This is needed for input/output expressions to be set correctly, otherwise compilation will fail.
						EditorOnly->ExpressionCollection.AddExpression(MFCallNode);

						SetPosXAndMoveReferenceToTheRight(MFCallNode);
						EditorOnly->FrontMaterial.Connect(0, MFCallNode);

						MFCallNode->UpdateFromFunctionResource();
						MFCallNode->PostEditChange();
						MFCallNode->ConditionalPostLoad();

						ColorMatInputConnectionToIndexed(EditorOnly->BaseColor,		MFCallNode, 0, MP_BaseColor);
						ScalarMatInputConnectionToIndexed(EditorOnly->Metallic,		MFCallNode, 1, MP_Metallic);
						ScalarMatInputConnectionToIndexed(EditorOnly->Specular,		MFCallNode, 2, MP_Specular);
						ScalarMatInputConnectionToIndexed(EditorOnly->Roughness,	MFCallNode, 3, MP_Roughness);
						Vector3MatInputConnectionToIndexed(EditorOnly->Normal,		MFCallNode, 4, MP_Normal);
						ColorMatInputConnectionToIndexed(EditorOnly->EmissiveColor,	MFCallNode, 5, MP_EmissiveColor);
						ScalarMatInputConnectionToIndexed(EditorOnly->Opacity,		MFCallNode, 6, MP_Opacity);
					}
				}
				else
				{
					// Or if it cannot be found, a slab node
					UMaterialExpressionSubstrateSlabBSDF* SlabNode = NewObject<UMaterialExpressionSubstrateSlabBSDF>(this);
					EditorOnly->ExpressionCollection.AddExpression(SlabNode);
					SlabNode->Material = this;
					SetPosXAndMoveReferenceToTheRight(SlabNode);
					EditorOnly->FrontMaterial.Connect(0, SlabNode);
				}
				bRelinkCustomOutputNodes = false;
				bInvalidateShader = true;
				bEmptyShader = true;
			}
			else if (!bUseMaterialAttributes && !EditorOnly->FrontMaterial.IsConnected() && !bHasAnySubstrateNodes)
			{
				if (MaterialDomain == MD_Surface)
				{
					ConvertNode = NewObject<UMaterialExpressionSubstrateShadingModels>(this);
					EditorOnly->ExpressionCollection.AddExpression(ConvertNode); // Add the SetAttributesNode to ensure the MaterialAttribute input are correctly tracked & cached
					ConvertNode->Material = this;
					SetPosXAndMoveReferenceToTheRight(ConvertNode);
					ConvertNode->SubsurfaceProfile = bRequireNoSubsurfaceProfile ? nullptr : SubsurfaceProfile;
					ColorMatInputConnectionTo(EditorOnly->BaseColor,			ConvertNode->BaseColor,			MP_BaseColor);
					ScalarMatInputConnectionTo(EditorOnly->Metallic,			ConvertNode->Metallic,			MP_Metallic);
					ScalarMatInputConnectionTo(EditorOnly->Specular,			ConvertNode->Specular,			MP_Specular);
					ScalarMatInputConnectionTo(EditorOnly->Roughness,			ConvertNode->Roughness,			MP_Roughness);
					ScalarMatInputConnectionTo(EditorOnly->Anisotropy,			ConvertNode->Anisotropy,		MP_Anisotropy);
					ColorMatInputConnectionTo(EditorOnly->EmissiveColor,		ConvertNode->EmissiveColor,		MP_EmissiveColor);
					Vector3MatInputConnectionTo(EditorOnly->Normal,				ConvertNode->Normal,			MP_Normal, SUBSTRATE_COPY_CONNECTION);
					Vector3MatInputConnectionTo(EditorOnly->Tangent,			ConvertNode->Tangent,			MP_Tangent);
					ColorMatInputConnectionTo(EditorOnly->SubsurfaceColor,		ConvertNode->SubSurfaceColor,	MP_SubsurfaceColor);
					ScalarMatInputConnectionTo(EditorOnly->ClearCoat,			ConvertNode->ClearCoat,			MP_CustomData0);
					ScalarMatInputConnectionTo(EditorOnly->ClearCoatRoughness,	ConvertNode->ClearCoatRoughness,MP_CustomData1);
					ScalarMatInputConnectionTo(EditorOnly->Opacity,				ConvertNode->Opacity,			MP_Opacity, SUBSTRATE_COPY_CONNECTION);	// We only copy, to keep Opacity on the root node in case BLEND_AlphaComposite is selected.
					bRelinkCustomOutputNodes = true;
			
					// Shading Model
					// * either use the shader graph expression 
					// * or add a constant shading model
					if (ShadingModel == MSM_FromMaterialExpression)
					{
						if (!EditorOnly->ShadingModelFromMaterialExpression.IsConnected())
						{
							ConvertNode->ShadingModelOverride = MSM_DefaultLit;
						}
						else
						{
							// Reconnect the shading model expression. 
							// Note: assign the expression directly, as using ConvertNode->GetInput(19)->Connect(..) causes the expression to not be assigned
							ConvertNode->ShadingModel.Connect(EditorOnly->ShadingModelFromMaterialExpression.OutputIndex, EditorOnly->ShadingModelFromMaterialExpression.Expression);
						}

						// Store Substrate shading model of the converted material. 
						GatherCustomNodes();
						if (SingleLayerWaterOutput)
						{
							ShadingModels.AddShadingModel(MSM_SingleLayerWater);
						}

						check(ShadingModels.CountShadingModels() >= 1);
					}
					else
					{
						ConvertNode->ShadingModelOverride = ShadingModel;
						check(ShadingModels.CountShadingModels() == 1);

						if (ShadingModel == MSM_TwoSidedFoliage)
						{
							bIsThinSurface = true;
						}
					}

					EditorOnly->FrontMaterial.Connect(0, ConvertNode);

					bInvalidateShader = true;
				}
				else if (MaterialDomain == MD_Volume)
				{
					UMaterialExpressionSubstrateVolumetricFogCloudBSDF* VolBSDF = NewObject<UMaterialExpressionSubstrateVolumetricFogCloudBSDF>(this);
					EditorOnly->ExpressionCollection.AddExpression(VolBSDF);
					VolBSDF->Material = this;
					SetPosXAndMoveReferenceToTheRight(VolBSDF);
					ColorMatInputConnectionTo(EditorOnly->BaseColor,		VolBSDF->Albedo,			MP_BaseColor);	
					ColorMatInputConnectionTo(EditorOnly->SubsurfaceColor,	VolBSDF->Extinction,		MP_SubsurfaceColor);
					ColorMatInputConnectionTo(EditorOnly->EmissiveColor,	VolBSDF->EmissiveColor,		MP_EmissiveColor);	
					ScalarMatInputConnectionTo(EditorOnly->AmbientOcclusion,VolBSDF->AmbientOcclusion,	MP_AmbientOcclusion);

					VolBSDF->bEmissiveOnly = ShadingModel == MSM_Unlit;

					// SUBSTRATE_TODO remove the VolumetricAdvancedOutput node and add the input onto FogCloudBSDF even if only used by the cloud renderer?
					EditorOnly->FrontMaterial.Connect(0, VolBSDF);
					bInvalidateShader = true;
				}
				else if (MaterialDomain == MD_LightFunction)
				{
					// Some materials don't have their shading mode set correctly to Unlit. Since only Unlit is supported, forcing it here.
					ShadingModel = MSM_Unlit;
					ShadingModels.ClearShadingModels();
					ShadingModels.AddShadingModel(MSM_Unlit);

					// Only Emissive & Opacity are valid input for PostProcess material
					UMaterialExpressionSubstrateLightFunction* LightFunctionNode = NewObject<UMaterialExpressionSubstrateLightFunction>(this);
					EditorOnly->ExpressionCollection.AddExpression(LightFunctionNode);
					LightFunctionNode->Material = this;
					SetPosXAndMoveReferenceToTheRight(LightFunctionNode);
					ColorMatInputConnectionTo(EditorOnly->EmissiveColor, LightFunctionNode->Color, MP_EmissiveColor);

					EditorOnly->FrontMaterial.Connect(0, LightFunctionNode);
					bInvalidateShader = true;
				}
				else if (MaterialDomain == MD_PostProcess)
				{
					// Some materials don't have their shading mode set correctly to Unlit. Since only Unlit is supported, forcing it here.
					ShadingModel = MSM_Unlit;
					ShadingModels.ClearShadingModels();
					ShadingModels.AddShadingModel(MSM_Unlit);

					if (MaterialDomain == MD_PostProcess && !IsPostProcessMaterialOutputingAlpha())
					{
						BlendMode = BLEND_Opaque;
					}

					UMaterialExpressionSubstratePostProcess* PostProcNode = NewObject<UMaterialExpressionSubstratePostProcess>(this);
					EditorOnly->ExpressionCollection.AddExpression(PostProcNode);
					PostProcNode->Material = this;
					SetPosXAndMoveReferenceToTheRight(PostProcNode);

					ColorMatInputConnectionTo(EditorOnly->EmissiveColor, PostProcNode->Color, MP_EmissiveColor);
					ScalarMatInputConnectionTo(EditorOnly->Opacity, PostProcNode->Opacity, MP_Opacity, SUBSTRATE_COPY_CONNECTION);	// We only copy, to keep Opacity on the root node in case BLEND_AlphaComposite is selected.

					EditorOnly->FrontMaterial.Connect(0, PostProcNode);
					bInvalidateShader = true;
				}
				else if (MaterialDomain == MD_DeferredDecal)
				{
					// Some decal materials don't have their shading mode set correctly to DefaultLit. Since only DefaultLit is supported, forcing it here.
					ShadingModel = MSM_DefaultLit;
					ShadingModels.ClearShadingModels();
					ShadingModels.AddShadingModel(MSM_DefaultLit);

					ConvertNode = NewObject<UMaterialExpressionSubstrateShadingModels>(this);
					EditorOnly->ExpressionCollection.AddExpression(ConvertNode);
					ConvertNode->Material = this;
					SetPosXAndMoveReferenceToTheRight(ConvertNode);
					ColorMatInputConnectionTo(EditorOnly->BaseColor,			ConvertNode->BaseColor,			MP_BaseColor);
					ScalarMatInputConnectionTo(EditorOnly->Metallic,			ConvertNode->Metallic,			MP_Metallic);
					ScalarMatInputConnectionTo(EditorOnly->Specular,			ConvertNode->Specular,			MP_Specular);
					ScalarMatInputConnectionTo(EditorOnly->Roughness,			ConvertNode->Roughness,			MP_Roughness);
					ScalarMatInputConnectionTo(EditorOnly->Anisotropy,			ConvertNode->Anisotropy,		MP_Anisotropy);
					ColorMatInputConnectionTo(EditorOnly->EmissiveColor,		ConvertNode->EmissiveColor,		MP_EmissiveColor);
					Vector3MatInputConnectionTo(EditorOnly->Normal,				ConvertNode->Normal,			MP_Normal, SUBSTRATE_COPY_CONNECTION);
					Vector3MatInputConnectionTo(EditorOnly->Tangent,			ConvertNode->Tangent,			MP_Tangent);
					ColorMatInputConnectionTo(EditorOnly->SubsurfaceColor,		ConvertNode->SubSurfaceColor,	MP_SubsurfaceColor);
					ScalarMatInputConnectionTo(EditorOnly->ClearCoat,			ConvertNode->ClearCoat,			MP_CustomData0);
					ScalarMatInputConnectionTo(EditorOnly->ClearCoatRoughness,	ConvertNode->ClearCoatRoughness,MP_CustomData1);
					ScalarMatInputConnectionTo(EditorOnly->Opacity,				ConvertNode->Opacity,			MP_Opacity, SUBSTRATE_COPY_CONNECTION);	// We only copy, to keep Opacity on the root node in case BLEND_AlphaComposite is selected.

					// Add constant for the Unlit shading model
					ConvertNode->ShadingModelOverride = ShadingModel;
					check(ShadingModels.CountShadingModels() == 1);

					// Now pass through the convert to decal node, which flag the material as SSM_Decal, which will set the domain to Decal.
					UMaterialExpressionSubstrateConvertToDecal* ConvertToDecalNode= NewObject<UMaterialExpressionSubstrateConvertToDecal>(this);
					EditorOnly->ExpressionCollection.AddExpression(ConvertToDecalNode);
					ConvertToDecalNode->Material = this;
					ReplaceNodeAndMoveToTheRight(ConvertToDecalNode, ConvertNode);
					ConvertToDecalNode->DecalMaterial.Connect(0, ConvertNode);

					EditorOnly->FrontMaterial.Connect(0, ConvertToDecalNode);
					bInvalidateShader = true;
				}
				else if (MaterialDomain == MD_UI)
				{
					// Some materials don't have their shading mode set correctly to Unlit. Since only Unlit is supported, forcing it here.
					ShadingModel = MSM_Unlit;
					ShadingModels.ClearShadingModels();
					ShadingModels.AddShadingModel(MSM_Unlit);

					UMaterialExpressionSubstrateUI* UINode = NewObject<UMaterialExpressionSubstrateUI>(this);
					EditorOnly->ExpressionCollection.AddExpression(UINode);
					UINode->Material = this;
					SetPosXAndMoveReferenceToTheRight(UINode);
					ColorMatInputConnectionTo(EditorOnly->EmissiveColor,UINode->Color,	MP_EmissiveColor);
					ScalarMatInputConnectionTo(EditorOnly->Opacity,		UINode->Opacity,MP_Opacity, SUBSTRATE_COPY_CONNECTION);	// We only copy, to keep Opacity on the root node in case BLEND_AlphaComposite is selected.

					EditorOnly->FrontMaterial.Connect(0, UINode);
					bInvalidateShader = true;
				}

				BlendMode = ConvertLegacyBlendMode(BlendMode, ShadingModels);
				RefractionCoverageMode = RCM_CoverageIgnored;
			}
		}

		if (bRelinkCustomOutputNodes)
		{
			check(ConvertNode);
			GatherCustomNodes();

			if (ThinTranslucentOutput)
			{
				FExpressionInput* TransmittanceColorInput = &ThinTranslucentOutput->TransmittanceColor;
				FExpressionInput* ThinTranslucentSurfaceCoverageInput = &ThinTranslucentOutput->SurfaceCoverage;
				if (TransmittanceColorInput && TransmittanceColorInput->IsConnected())
				{
					ConvertNode->TransmittanceColor.Connect(TransmittanceColorInput->OutputIndex, TransmittanceColorInput->Expression);
				}
				if (ThinTranslucentSurfaceCoverageInput && ThinTranslucentSurfaceCoverageInput->IsConnected())
				{
					ConvertNode->ThinTranslucentSurfaceCoverage.Connect(ThinTranslucentSurfaceCoverageInput->OutputIndex, ThinTranslucentSurfaceCoverageInput->Expression);
				}
			}
			if (SingleLayerWaterOutput)
			{
				ConnectionTo(SingleLayerWaterOutput->ScatteringCoefficients,	ConvertNode->WaterScatteringCoefficients);
				ConnectionTo(SingleLayerWaterOutput->AbsorptionCoefficients,	ConvertNode->WaterAbsorptionCoefficients);
				ConnectionTo(SingleLayerWaterOutput->PhaseG,					ConvertNode->WaterPhaseG);
				ConnectionTo(SingleLayerWaterOutput->ColorScaleBehindWater,		ConvertNode->ColorScaleBehindWater);
			}
			if (ClearCoatBottomNormalOutput)
			{
				ConnectionTo(ClearCoatBottomNormalOutput->Input, ConvertNode->ClearCoatNormal, SUBSTRATE_COPY_CONNECTION);
			}
			if (TangentOutput)
			{
				ConnectionTo(TangentOutput->Input, ConvertNode->Tangent, SUBSTRATE_COPY_CONNECTION);
			}
		}
	}

	// Glint conversion parametrization
	if (EditorOnly->SubstrateVersion < 0)
	{
		// This is now done in ConvertSlabExpressionGlintDensity() during the material's PostLoad() function
		// This conversion change SubstrateVersion from -1 -> 0
	}
	//else if (EditorOnly->SubstrateVersion < 1)
	//{ [...] }

	if (bInvalidateShader)
	{
		// Now force the material to recompile and we use a hash of the original StateId.
		// This is to avoid having different StateId each time we load the material and to not forever recompile it, i.e. use a cached version.
		static FGuid LegacyToSubstrateConversionGuid(TEXT("0DAD35FE-21AE-4274-8B41-6C9D47285D8A"));
		ReleaseResourcesAndMutateDDCKey(LegacyToSubstrateConversionGuid);
	}

	// For rebuild the shading mode since we have change it
	RebuildShadingModelField();

	if (bInvalidateShader)
	{
		// Set the root node position.
		EditorX = CurrentNodePosX;

		// We might have moved connections above so update the CachedExpressionData from the EditorOnly connection data (ground truth).
		UpdateCachedExpressionData();
	}

	// Store the versions, 
	if (bInvalidateShader)
	{
		// 1. Store current Substrate version
		EditorOnly->SubstrateVersion = GetSubstrateVersion();

		// 2. Store current Substrate conversion version
		const bool bIsUpToDate = EditorOnly->SubstrateConversionVersion == GetSubstrateNoConversionVersion() || EditorOnly->SubstrateConversionVersion == GetSubstrateConversionVersion();
		if (!bIsUpToDate)
		{
			if (bEmptyShader) // == new shader
			{
				EditorOnly->SubstrateConversionVersion = GetSubstrateNoConversionVersion();
			}
			else // == Converted shader
			{
				EditorOnly->SubstrateConversionVersion = GetSubstrateConversionVersion();
				EditorOnly->bIsSubsutrateAutoConvertedAndNotSaved = true;
			}
		}
	}

	return bInvalidateShader;
}

static void ConvertSlabExpressionMaterialSubSurfaceType(TConstArrayView<TObjectPtr<UMaterialExpression>> InExpressions, EBlendMode InBlendMode, bool bInIsThinSurface)
{
	for (UMaterialExpression* Expression : InExpressions)
	{
		if (UMaterialExpressionSubstrateSlabBSDF* SlabExpression = Cast<UMaterialExpressionSubstrateSlabBSDF>(Expression))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (!IsOpaqueBlendMode(InBlendMode))
			{
				SlabExpression->SubSurfaceType = EMaterialSubSurfaceType::MSS_SimpleVolume;
			}
			else if (SlabExpression->bUseSSSDiffusion)
			{
				SlabExpression->SubSurfaceType = EMaterialSubSurfaceType::MSS_Diffusion;
			}
			else if (bInIsThinSurface)
			{
				SlabExpression->SubSurfaceType = EMaterialSubSurfaceType::MSS_TwoSidedWrap;
			}
			else
			{
				SlabExpression->SubSurfaceType = EMaterialSubSurfaceType::MSS_Wrap;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (FunctionCall->MaterialFunction)
			{
				ConvertSlabExpressionMaterialSubSurfaceType(FunctionCall->MaterialFunction->GetExpressions(), InBlendMode, bInIsThinSurface);
			}
		}
	}
}

static void ConvertSlabExpressionGlintDensity(UMaterial* InMaterial)
{
	// Convert existing Substrate material with glint. 
	// The FrontMaterial is expected to be connected. We rely on the Substrate version to make different 
	// between material already converted and not (in addition to the global object version which now insure all new material will have the 
	// correct glint density parameterization)
	UMaterialEditorOnlyData* EditorOnly = InMaterial->GetEditorOnlyData();
	if (EditorOnly && EditorOnly->SubstrateVersion < 0 && EditorOnly->FrontMaterial.IsConnected())
	{
		TArray<UMaterialExpressionSubstrateSlabBSDF*> SlabExpressions;
		InMaterial->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionSubstrateSlabBSDF>(SlabExpressions);
		for (UMaterialExpressionSubstrateSlabBSDF* SlabExpression : SlabExpressions)
		{
			if (SlabExpression->GlintValue.IsConnected())
			{
				// Gather custom output for thin translucency
				// Empty material: Create by default a slab node
				UMaterialFunction* ConversionMF = LoadObject<UMaterialFunction>(nullptr, TEXT("/Engine/Functions/Substrate/SMF_GlintConversion.SMF_GlintConversion"));
				if (ConversionMF)
				{
					ConversionMF->UpdateFromFunctionResource();
					ConversionMF->PostEditChange();
					ConversionMF->ConditionalPostLoad();

					UMaterialExpressionMaterialFunctionCall* MFCallNode = NewObject<UMaterialExpressionMaterialFunctionCall>(InMaterial);
					if (MFCallNode->SetMaterialFunction(ConversionMF))
					{
						// This is needed for input/output expressions to be set correctly, otherwise compilation will fail.
						EditorOnly->ExpressionCollection.AddExpression(MFCallNode);
						MFCallNode->MaterialExpressionEditorX = SlabExpression->MaterialExpressionEditorX - 200;
						MFCallNode->MaterialExpressionEditorY = SlabExpression->MaterialExpressionEditorY + 500;

						// Connect GlintValue graph to the MF input
						if (SlabExpression->GlintValue.IsConnected())
						{
							MFCallNode->GetInput(0/*Density*/)->Connect(SlabExpression->GlintValue.OutputIndex, SlabExpression->GlintValue.Expression);
							SlabExpression->GlintValue.Expression = nullptr;
						}

						// Connect the MF output to the GlintValue input
						SlabExpression->GlintValue.Connect(0, MFCallNode);

						MFCallNode->UpdateFromFunctionResource();
						MFCallNode->PostEditChange();
						MFCallNode->ConditionalPostLoad();
					}
				}
			}
		}

		// Update substrate version
		// Even if no glint conversion was done, we save the new Substrate's version (from -1 to 0)
		EditorOnly->SubstrateVersion = GetSubstrateVersion();
	}
}
#endif // WITH_EDITOR

TMap<FGuid, UMaterialInterface*> LightingGuidFixupMap;

template<typename InputType>
static void MoveExpressionInput(InputType& Src, InputType& OutDst)
{
	if (Src.IsConnected() || Src.IsConstant())
	{
		OutDst = MoveTemp(Src);
	}
}

void UMaterial::PostLoad()
{
	LLM_SCOPE(ELLMTag::Materials);

	SCOPED_LOADTIMER(MaterialPostLoad);

	Super::PostLoad();

	if (FApp::CanEverRender())
	{
		// Resources can be processed / registered now that we're back on the main thread
		ProcessSerializedInlineShaderMaps(this, LoadedMaterialResources, MaterialResources);
	}
	else
	{
		// Discard all loaded material resources
		for (FMaterialResource& Resource : LoadedMaterialResources)
		{
			Resource.DiscardShaderMap();
		}		
	}
	// Empty the list of loaded resources, we don't need it anymore
	LoadedMaterialResources.Empty();

	NaniteOverrideMaterial.FixupLegacySoftReference(this);

#if WITH_EDITORONLY_DATA
	const FPackageFileVersion UEVer = GetLinkerUEVersion();
	const int32 RenderObjVer = GetLinkerCustomVersion(FRenderingObjectVersion::GUID);
	const int32 UE5MainVer = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);
	const int32 FortMainVer = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
	check(EditorOnly != nullptr);

	MoveExpressionInput(BaseColor_DEPRECATED, EditorOnly->BaseColor);
	MoveExpressionInput(Metallic_DEPRECATED, EditorOnly->Metallic);
	MoveExpressionInput(Specular_DEPRECATED, EditorOnly->Specular);
	MoveExpressionInput(Roughness_DEPRECATED, EditorOnly->Roughness);
	MoveExpressionInput(Anisotropy_DEPRECATED, EditorOnly->Anisotropy);
	MoveExpressionInput(Normal_DEPRECATED, EditorOnly->Normal);
	MoveExpressionInput(Tangent_DEPRECATED, EditorOnly->Tangent);
	MoveExpressionInput(EmissiveColor_DEPRECATED, EditorOnly->EmissiveColor);
	MoveExpressionInput(Tangent_DEPRECATED, EditorOnly->Tangent);
	MoveExpressionInput(Opacity_DEPRECATED, EditorOnly->Opacity);
	MoveExpressionInput(OpacityMask_DEPRECATED, EditorOnly->OpacityMask);
	MoveExpressionInput(WorldPositionOffset_DEPRECATED, EditorOnly->WorldPositionOffset);
	MoveExpressionInput(SubsurfaceColor_DEPRECATED, EditorOnly->SubsurfaceColor);
	MoveExpressionInput(ClearCoat_DEPRECATED, EditorOnly->ClearCoat);
	MoveExpressionInput(ClearCoatRoughness_DEPRECATED, EditorOnly->ClearCoatRoughness);
	MoveExpressionInput(AmbientOcclusion_DEPRECATED, EditorOnly->AmbientOcclusion);
	MoveExpressionInput(Refraction_DEPRECATED, EditorOnly->Refraction);
	MoveExpressionInput(MaterialAttributes_DEPRECATED, EditorOnly->MaterialAttributes);
	MoveExpressionInput(PixelDepthOffset_DEPRECATED, EditorOnly->PixelDepthOffset);
	MoveExpressionInput(ShadingModelFromMaterialExpression_DEPRECATED, EditorOnly->ShadingModelFromMaterialExpression);
	MoveExpressionInput(FrontMaterial_DEPRECATED, EditorOnly->FrontMaterial);
	for (int32 i = 0; i < 8; ++i)
	{
		MoveExpressionInput(CustomizedUVs_DEPRECATED[i], EditorOnly->CustomizedUVs[i]);
	}

	DoMaterialAttributeReorder(&DiffuseColor_DEPRECATED, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&SpecularColor_DEPRECATED, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->BaseColor, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Metallic, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Specular, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Roughness, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Anisotropy, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Normal, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Tangent, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->EmissiveColor, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Opacity, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->OpacityMask, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->WorldPositionOffset, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->SubsurfaceColor, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->ClearCoat, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->ClearCoatRoughness, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->AmbientOcclusion, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Refraction, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[0], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[1], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[2], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[3], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[4], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[5], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[6], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->CustomizedUVs[7], UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->PixelDepthOffset, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->ShadingModelFromMaterialExpression, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->FrontMaterial, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->SurfaceThickness, UEVer, RenderObjVer, UE5MainVer);
	DoMaterialAttributeReorder(&EditorOnly->Displacement, UEVer, RenderObjVer, UE5MainVer);

	if (ParameterGroupData_DEPRECATED.Num() > 0)
	{
		ensure(EditorOnly->ParameterGroupData.Num() == 0);
		EditorOnly->ParameterGroupData = MoveTemp(ParameterGroupData_DEPRECATED);
	}

	if (Expressions_DEPRECATED.Num() > 0)
	{
		ensure(EditorOnly->ExpressionCollection.Expressions.Num() == 0);
		EditorOnly->ExpressionCollection.Expressions = MoveTemp(Expressions_DEPRECATED);
	}

	if (EditorComments_DEPRECATED.Num() > 0)
	{
		ensure(EditorOnly->ExpressionCollection.EditorComments.Num() == 0);
		EditorOnly->ExpressionCollection.EditorComments = MoveTemp(EditorComments_DEPRECATED);
	}

	// Can't use new material compilation validations on existing materials.
	bRelaxRuntimeVirtualTextureRestrictionsForOldContent = (UE5MainVer < FUE5MainStreamObjectVersion::RuntimeVirtualTextureMaterialValidation);

#endif // WITH_EDITORONLY_DATA

	if (!IsDefaultMaterial())
	{
		AssertDefaultMaterialsPostLoaded();
	}	

#if WITH_EDITOR
	if ( GIsEditor && GetOuter() == GetTransientPackage() && FCString::Strstr(*GetName(), TEXT("MEStatsMaterial_")))
	{
		bIsMaterialEditorStatsMaterial = true;
	}
#endif // WITH_EDITOR

	if( GetLinkerUEVersion() < VER_UE4_REMOVED_MATERIAL_USED_WITH_UI_FLAG && bUsedWithUI_DEPRECATED == true )
	{
		MaterialDomain = MD_UI;
	}

#if WITH_EDITORONLY_DATA
	// Ensure expressions have been postloaded before we use them for compiling
	// Any UObjects used by material compilation must be postloaded here
	for (UMaterialExpression* Expression : GetExpressions())
	{
		if (Expression)
		{
			Expression->ConditionalPostLoad();
		}
	}
#endif // WITH_EDITORONLY_DATA

	// Fixup for legacy materials which didn't recreate the lighting guid properly on duplication
	if (GetLinker() && GetLinker()->UEVer() < VER_UE4_BUMPED_MATERIAL_EXPORT_GUIDS)
	{
		UMaterialInterface** ExistingMaterial = LightingGuidFixupMap.Find(GetLightingGuid());

		if (ExistingMaterial)
		{
			SetLightingGuid();
		}

		LightingGuidFixupMap.Add(GetLightingGuid(), this);
	}

	// Fix the shading model to be valid.  Loading a material saved with a shading model that has been removed will yield a MSM_MAX.
	if(ShadingModel == MSM_MAX)
	{
		ShadingModel = MSM_DefaultLit;
	}

	// Take care of loading materials that were not compiled when the shading model field existed
	if (ShadingModel != MSM_FromMaterialExpression)
	{
		ShadingModels = FMaterialShadingModelField(ShadingModel);
	}

	if(DecalBlendMode == DBM_MAX)
	{
		DecalBlendMode = DBM_Translucent;
	}

	if(bUseFullPrecision_DEPRECATED && (FloatPrecisionMode == EMaterialFloatPrecisionMode::MFPM_Half || FloatPrecisionMode == EMaterialFloatPrecisionMode::MFPM_Half_WithShaderParameters))
	{
		FloatPrecisionMode = EMaterialFloatPrecisionMode::MFPM_Full;
		bUseFullPrecision_DEPRECATED = false;
	}

#if !WITH_EDITOR
	// Filter out ShadingModels field to a current platform settings
	FilterOutPlatformShadingModels(GMaxRHIShaderPlatform, ShadingModels);
	// Override material shader model if it was filtered out
	if (!ShadingModels.HasShadingModel(ShadingModel) &&
		(ShadingModel != MSM_FromMaterialExpression || !AllowPerPixelShadingModels(GMaxRHIShaderPlatform)))
	{
		ShadingModel = ShadingModels.GetFirstShadingModel();
	}
#endif

#if WITH_EDITOR
	// Clean up any removed material expression classes. If running in editor, also release resources and mutate DDC key.
	if (EditorOnly->ExpressionCollection.Expressions.Remove(nullptr) != 0 && GIsEditor)
	{
		// Force this material to recompile because its expressions have changed
		// We're not providing a deterministic transformation guid because there could be many different ways expression
		// could change. Each conversion code removing such expression would need its own guid.
		ReleaseResourcesAndMutateDDCKey();
	}
#endif // WITH_EDITOR

	if (!StateId.IsValid())
	{
		// Fixup for some legacy content
		// This path means recompiling every time the material is loaded until it is saved
		FPlatformMisc::CreateGuid(StateId);
	}

#if WITH_EDITOR

	BackwardsCompatibilityInputConversion();
	BackwardsCompatibilityVirtualTextureOutputConversion();
	BackwardsCompatibilityDecalConversion();

	if ( GMaterialsThatNeedSamplerFixup.Get( this ) )
	{
		GMaterialsThatNeedSamplerFixup.Clear( this );
		for (UMaterialExpression* Expression : GetExpressions())
		{
			UMaterialExpressionTextureBase* TextureExpression = Cast<UMaterialExpressionTextureBase>(Expression);
			if ( TextureExpression && TextureExpression->Texture )
			{
				switch( TextureExpression->Texture->CompressionSettings )
				{
				case TC_Normalmap:
					TextureExpression->SamplerType = SAMPLERTYPE_Normal;
					break;
					
				case TC_Grayscale:
					TextureExpression->SamplerType = TextureExpression->Texture->SRGB ? SAMPLERTYPE_Grayscale : SAMPLERTYPE_LinearGrayscale;
					break;

				case TC_Masks:
					TextureExpression->SamplerType = SAMPLERTYPE_Masks;
					break;

				case TC_Alpha:
					TextureExpression->SamplerType = SAMPLERTYPE_Alpha;
					break;
				default:
					TextureExpression->SamplerType = TextureExpression->Texture->SRGB ? SAMPLERTYPE_Color : SAMPLERTYPE_LinearColor;
					break;
				}
			}
		}
	}
#endif // #if WITH_EDITOR

	// needed for UMaterial as it doesn't have the InitResources() override where this is called
	PropagateDataToMaterialProxy();

#if WITH_EDITOR
	// cooked materials will not have any expressions in them, so this will obliterate the saved cached expression data
	if (!GetOutermost()->bIsCookedForEditor)
	{
		// Fix up if we are missing expressions
		if (GetExpressions().Num() == 0)
		{
			FMaterialEditorUtilities::InitExpressions(this);
		}
		UpdateCachedExpressionData();
	}
	else
	{
#if WITH_EDITORONLY_DATA
		// cooked materials will need to update their expressions, but not their cached expression data.
		UpdateTransientExpressionData();
#endif
	}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	if (MaterialDomain == MD_Volume && UE5MainVer < FUE5MainStreamObjectVersion::MaterialHasIsUsedWithVolumetricCloudFlag)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bUsedWithVolumetricCloud = HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionCloudSampleAttribute>()
								|| HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionVolumetricAdvancedMaterialOutput>()
								|| HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionVolumetricAdvancedMaterialInput>();
		if (bUsedWithVolumetricCloud)
		{
			static FGuid BackwardsCompatibilityUsedWithVolumetricCloudConversionGuid(TEXT("6F6336BF-D377-4FA5-9887-A7457BE6DE92"));
			ReleaseResourcesAndMutateDDCKey(BackwardsCompatibilityUsedWithVolumetricCloudConversionGuid);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

#if WITH_EDITOR
	// Convert UMaterialExpressionSubstrateSlabBSDF::bUseSSSDiffution into UMaterialExpressionSubstrateSlabBSDF::SubSurfaceType
	if (Substrate::IsSubstrateEnabled())
	{
		if (FortMainVer < FFortniteMainBranchObjectVersion::AddMaterialSubstrateSubsurfaceType)
		{
			ConvertSlabExpressionMaterialSubSurfaceType(GetExpressions(), BlendMode, bIsThinSurface);
		}

		if (FortMainVer < FFortniteMainBranchObjectVersion::ConvertGlintDensity)
		{
			ConvertSlabExpressionGlintDensity(this);
		}

		// Substrate materials conversion needs to be done after expressions are cached, otherwise material function won't have 
		// valid inputs in certain cases
		if (ConvertMaterialToSubstrateMaterial(true /*bAllowEmptyMaterialUpdate*/))
		{
			// Call PropagateDataToMaterialProxy in order to propagate Subsurface profiles and specular profiles data to the material proxy
			PropagateDataToMaterialProxy();
		}
	}
	
	// Avoid running the following checks when running cooks
	if (!IsRunningCookCommandlet())
	{
		if(FloatPrecisionMode == EMaterialFloatPrecisionMode::MFPM_Half && GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::MaterialHalfPrecisionFixup)
		{
			static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.FloatPrecisionMode"));
			if (CVar)
			{
				int CVarMobilePrecisionMode = FMath::Clamp(CVar->GetInt(), (int32)EMobileFloatPrecisionMode::Half, (int32)EMobileFloatPrecisionMode::Full);
				if (CVarMobilePrecisionMode == EMobileFloatPrecisionMode::Full)
				{
					// Choosing 'Use Half Precision' on a per material property in the Material Editor was not working as intended when the cvar r.Mobile.FloatPrecisionMode=2 (it was a no-op)
					// With this option now fixed, we revert this material's Float Precision option from Half --> Default to maintain the generated code the same and avoid performance/visual regressions
					FloatPrecisionMode = EMaterialFloatPrecisionMode::MFPM_Default;

					const FText FloatPrecisionModeSelectedOnMaterial =
						FText::Format(LOCTEXT("MapCheck_Message_MaterialUseHalfPrecisionCheck",
							"Material {0} had the option \"Use Half Precision\" from Float Precision Mode checked. This option was not working as intended before so it's being reverted to Default to maintain previous behaviour."),
							FText::FromName(this->GetFName()));

					FMessageLog("MapCheck").Warning()
						->AddToken(FTextToken::Create(FloatPrecisionModeSelectedOnMaterial))
						->AddToken(FActionToken::Create(LOCTEXT("MapCheck_SaveMaterialWithDefaultFloatingPointPrecision", "Please save the material to keep this change"),
							LOCTEXT("MapCheck_SaveMaterialWithDefaultFloatingPointPrecision_Desc", "Reverts this material precision from Half -> Default to preserve previous behaviour and saves it"),
							FOnActionTokenExecuted::CreateWeakLambda(this, [this]()
								{
									this->Modify();
									this->MarkPackageDirty();
									UPackage* Package = this->GetOutermost();

									FSavePackageArgs SaveArgs;
									SaveArgs.TopLevelFlags = RF_Public | RF_Standalone;

									const FString PackageFilename = FPackageName::LongPackageNameToFilename(Package->GetName(), FPackageName::GetAssetPackageExtension());
									UPackage::SavePackage(Package, this, *PackageFilename, SaveArgs);

								})));
				}
			}
		}
	}

#endif // WITH_EDITOR

	checkf(CachedExpressionData, TEXT("Missing cached expression data for material, should have been either serialized or created during PostLoad"));

	for (const FMaterialParameterCollectionInfo& CollectionInfo : CachedExpressionData->ParameterCollectionInfos)
	{
		if (CollectionInfo.ParameterCollection)
		{
			CollectionInfo.ParameterCollection->ConditionalPostLoad();
		}
	}

#if WITH_EDITOR
	if (GMaterialsThatNeedExpressionsFlipped.Get(this))
	{
		GMaterialsThatNeedExpressionsFlipped.Clear(this);
		FlipExpressionPositions(GetExpressions(), EditorOnly->ExpressionCollection.EditorComments, true, this);
	}
	else if (GMaterialsThatNeedCoordinateCheck.Get(this))
	{
		GMaterialsThatNeedCoordinateCheck.Clear(this);
		if (HasFlippedCoordinates())
		{
			FlipExpressionPositions(GetExpressions(), EditorOnly->ExpressionCollection.EditorComments, false, this);
		}
		FixCommentPositions(EditorOnly->ExpressionCollection.EditorComments);
	}
	else if (GMaterialsThatNeedCommentFix.Get(this))
	{
		GMaterialsThatNeedCommentFix.Clear(this);
		FixCommentPositions(EditorOnly->ExpressionCollection.EditorComments);
	}

	if (GMaterialsThatNeedFeatureLevelSM6Fix.Get(this))
	{
		GMaterialsThatNeedFeatureLevelSM6Fix.Clear(this);
		if (FixFeatureLevelNodesForSM6(EditorOnly->ExpressionCollection.Expressions))
		{
			// Change this guid if you change the conversion logic.
			static FGuid BackwardsCompatibilityFeatureLevelSM6ConversionGuid(TEXT("FC75DED7-2FB3-463B-B56C-8295871A340C"));
			ReleaseResourcesAndMutateDDCKey(BackwardsCompatibilityFeatureLevelSM6ConversionGuid);
		}
	}

	// Before, refraction was only enabled when the refraction pin was plugged in.
	// Now it is enabled only when not OFF. Otherwise:
	//    - if plugged the pin override the physically based material refraction
	//    - if unplugged the refraction is converted from the material F0.
	if (UE5MainVer < FUE5MainStreamObjectVersion::MaterialRefractionModeNone)
	{
		const bool bRefractionPinPluggedIn = IsRefractionPinPluggedIn(EditorOnly);

		// Update to the new variable, accounting for the old default value.
		RefractionMethod = RefractionMode_DEPRECATED;

		if (!bRefractionPinPluggedIn)
		{
			// The root node refraction pin was not plugged in so set refraction mode as none.
			// We do this for all domains and blending modes (translucent, opaque, etc).
			RefractionMethod = RM_None;

			// Need to mutate since UPROPERTY like bUsesDistortion is changed and shaders need to recompile.
			static FGuid MaterialRefractionModeOFFConversionGuid(TEXT("094C0316-EA95-4B39-A3FA-CA126A92989B"));
			ReleaseResourcesAndMutateDDCKey(MaterialRefractionModeOFFConversionGuid);
		}
		// else keep the current refraction mode.
	}
#endif // WITH_EDITOR

	STAT(double MaterialLoadTime = 0);
	{
		SCOPE_SECONDS_COUNTER(MaterialLoadTime);

		//Don't compile shaders in post load for dev overhead materials.
		if (FApp::CanEverRender() && !bIsMaterialEditorStatsMaterial && GAllowCompilationInPostLoad)
		{
			// Before caching shader resources we have to make sure all referenced textures have been post loaded
			// as we depend on their resources being valid.
			for (UObject* Texture : ReferencedDefaultTextures)
			{
				if (Texture)
				{
					Texture->ConditionalPostLoad();
				}
			}

			const bool bSkipCompilationOnPostLoad = IsMaterialMapDDCEnabled() == false;
			if (bSkipCompilationOnPostLoad)
			{
				CacheResourceShadersForRendering(false, EMaterialShaderPrecompileMode::None);
			}
			else
			{
				CacheResourceShadersForRendering(false);
			}
		}
	}
	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_MaterialLoading,(float)MaterialLoadTime);

#if WITH_EDITOR
	if( GIsEditor && !IsTemplate() )
	{
		// Ensure that the ReferencedTextureGuids array is up to date.
		UpdateLightmassTextureTracking();
	}
#endif // WITH_EDITOR

	

	if(IsPSOShaderPreloadingEnabled())
	{ 
		// When dynamic preload shaders is enabled, we need to prelaod some material domains since there is no
		// code logic within the PSO precaching system.
		if (IsUIMaterial() || IsDeferredDecal() || (MaterialDomain == MD_LightFunction) || IsPostProcessMaterial())
		{
			FGraphEventArray Unused;
			PreloadMaterialShaderMap(GetMaterialResource(GMaxRHIShaderPlatform), Unused);
		}
	}
	else if (IsDeferredDecal() || IsUIMaterial() || (MaterialDomain == MD_LightFunction) || IsPostProcessMaterial())
	{
		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
		VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));
		FPSOPrecacheParams PSOPrecacheParams;
		PrecachePSOs(VertexFactoryDataList, PSOPrecacheParams, EPSOPrecachePriority::High, MaterialPSOPrecacheRequestIDs);
	}

	CacheTexturesSamplingInfo();
}

#if WITH_EDITORONLY_DATA
void UMaterial::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMaterialEditorOnlyData::StaticClass()));
}
#endif

void UMaterial::DumpDebugInfo(FOutputDevice& OutputDevice) const
{
	for (FMaterialResource* Resource : MaterialResources)
	{
		Resource->DumpDebugInfo(OutputDevice);
	}

#if WITH_EDITOR
	for (const TPair<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>& It : CachedMaterialResourcesForCooking)
	{
		for (const FMaterialResourceForCooking& ResourceForCooking : It.Value)
		{
			ResourceForCooking.Resource->DumpDebugInfo(OutputDevice);
		}
	}
#endif
}

TObjectPtr<UObject> UMaterial::GetDefaultTextureAtIdx(const int32 InIdx, TMicRecursionGuard RecursionGuard)
{
	const TArrayView<const TObjectPtr<UObject>> UsedDefaultTextures = GetReferencedTextures();

	if (!UsedDefaultTextures.IsValidIndex(InIdx))
	{
		static bool bWarnedOnce = false;
		if (!bWarnedOnce)
		{
			UE_LOGF(LogMaterial, Warning, "%ls: Requesting an invalid TextureIndex! (%u / %u)", *GetName(), InIdx, UsedDefaultTextures.Num());
			bWarnedOnce = true;
		}
		return nullptr;
	}

	return UsedDefaultTextures[InIdx];
}

void UMaterial::SaveShaderStableKeys(const class ITargetPlatform* TP)
{
#if WITH_EDITOR
	FStableShaderKeyAndValue SaveKeyVal;
	SaveKeyVal.ClassNameAndObjectPath.SetCompactFullNameFromObject(this);
	SaveShaderStableKeysInner(TP, SaveKeyVal);
#endif
}

void UMaterial::SaveShaderStableKeysInner(const class ITargetPlatform* TP, const FStableShaderKeyAndValue& InSaveKeyVal)
{
#if WITH_EDITOR
	FStableShaderKeyAndValue SaveKeyVal(InSaveKeyVal);
	SaveKeyVal.MaterialDomain = FName(*MaterialDomainString(MaterialDomain));
	TArray<FMaterialResourceForCooking>* MatRes = CachedMaterialResourcesForCooking.Find(TP);
	if (MatRes)
	{
		for (const FMaterialResourceForCooking& Mat : *MatRes)
		{
			if (Mat.Resource)
			{
				Mat.Resource->SaveShaderStableKeys(SaveKeyVal);
			}
		}
	}
#endif
}

#if WITH_EDITOR
void UMaterial::GetShaderTypes(EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform, TArray<FDebugShaderTypeInfo>& OutShaderInfo)
{
	TArray<FMaterialResource*> NewResourcesToCache;
	GetNewResources(ShaderPlatform, NewResourcesToCache);

	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(TargetPlatform);

	for (FMaterialResource* Resource : NewResourcesToCache)
	{
		Resource->GetShaderTypes(LayoutParams, OutShaderInfo, FConservativeShaderParameters(Resource));
		delete Resource;
	}

	NewResourcesToCache.Empty();
}

void UMaterial::AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const
{
	if (!CachedMaterialResourcesForCooking.IsEmpty())
	{
		for (auto Iter = CachedMaterialResourcesForCooking.CreateConstIterator(); Iter; ++Iter)
		{
			for (const FMaterialResourceForCooking& Resource : Iter.Value())
			{
				OutDebugInfo << "Resource for platform " << LexToString(Resource.Platform) << ", IsCachingShaders=" << (Resource.Resource->IsCachingShaders() ? "true" : "false") << "\n";
				Resource.Resource->AppendCompileStateDebugInfo(OutDebugInfo);
			}
		}
	}
	else
	{
		OutDebugInfo << "No resources created\n";
	}
}
#endif // WITH_EDITOR

bool UMaterial::IsPropertyConnected(EMaterialProperty Property) const
{
	return GetCachedExpressionData().IsPropertyConnected(Property);
}

bool UMaterial::HasBaseColorConnected() const
{
	return IsPropertyConnected(MP_BaseColor);
}

bool UMaterial::HasRoughnessConnected() const
{
	return IsPropertyConnected(MP_Roughness);
}

bool UMaterial::HasAmbientOcclusionConnected() const
{
	return IsPropertyConnected(MP_AmbientOcclusion);
}

bool UMaterial::HasNormalConnected() const
{
	return IsPropertyConnected(MP_Normal);
}

bool UMaterial::HasSpecularConnected() const
{
	return IsPropertyConnected(MP_Specular);
}

bool UMaterial::HasMetallicConnected() const
{
	return IsPropertyConnected(MP_Metallic);
}

bool UMaterial::HasEmissiveColorConnected() const
{
	return IsPropertyConnected(MP_EmissiveColor);
}

bool UMaterial::HasAnisotropyConnected() const
{
	return IsPropertyConnected(MP_Anisotropy);
}

bool UMaterial::HasSurfaceThicknessConnected() const
{
	return IsPropertyConnected(MP_SurfaceThickness);
}

bool UMaterial::HasSubstrateFrontMaterialConnected() const
{
	return IsPropertyConnected(MP_FrontMaterial);
}

bool UMaterial::HasVertexPositionOffsetConnected() const
{
	return IsPropertyConnected(MP_WorldPositionOffset);
}

bool UMaterial::HasDisplacementConnected() const
{
	return IsPropertyConnected(MP_Displacement);
}

bool UMaterial::HasPixelDepthOffsetConnected() const
{
	return IsPropertyConnected(MP_PixelDepthOffset);
}

void UMaterial::PropagateDataToMaterialProxy()
{
	UpdateMaterialRenderProxy(*DefaultMaterialInstance);
}

#if WITH_EDITOR
void UMaterial::BeginCacheForCookedPlatformData( const ITargetPlatform *TargetPlatform )
{
	LLM_SCOPE(ELLMTag::Materials);
	TArray<FMaterialResourceForCooking>* CachedMaterialResourcesForPlatform = CachedMaterialResourcesForCooking.Find(TargetPlatform);

	if (CachedMaterialResourcesForPlatform == nullptr)
	{
		TArray<FName> DesiredShaderFormats;
		TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);
		GetCmdLineFilterShaderFormats(DesiredShaderFormats);

		CachedMaterialResourcesForPlatform = &CachedMaterialResourcesForCooking.FindOrAdd(TargetPlatform);
		if (DesiredShaderFormats.Num())
		{
			// Cache for all the shader formats that the cooking target requires
			for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
			{
				const EShaderPlatform LegacyShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);

				// Begin caching shaders for the target platform and store the material resource being compiled into CachedMaterialResourcesForCooking
				CacheResourceShadersForCooking(LegacyShaderPlatform, *CachedMaterialResourcesForPlatform, TargetPlatform);
			}
		}
	}
}

bool UMaterial::IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform ) 
{
	LLM_SCOPE(ELLMTag::Materials);
	const TArray<FMaterialResourceForCooking>* CachedMaterialResourcesForPlatform = CachedMaterialResourcesForCooking.Find(TargetPlatform);

	if (UE_LOG_ACTIVE(LogMaterial, VeryVerbose))
	{
		TStringBuilder<2048> CookStateInfo;
		AppendCompileStateDebugInfo(CookStateInfo);
		UE_LOGF(LogMaterial, VeryVerbose, "Material [%ls] Cook State:", *GetName());

		UE::String::ParseTokens(CookStateInfo.ToView(), TEXT('\n'),
			[](FStringView Line)
			{
				UE_LOGF(LogMaterial, VeryVerbose, "%.*ls", Line.Len(), Line.GetData());
			}, UE::String::EParseTokensOptions::SkipEmpty);
	}
	

	// this should always succeed if BeginCacheForCookedPlatformData is called first
	if (CachedMaterialResourcesForPlatform != nullptr)
	{
		for (const FMaterialResourceForCooking& MaterialResource : *CachedMaterialResourcesForPlatform)
		{
			if (MaterialResource.Resource->IsCompilationFinished() == false)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void UMaterial::ClearCachedCookedPlatformData( const ITargetPlatform *TargetPlatform )
{
	if (GIsBuildMachine)
	{
		// Dump debug info for the DefaultMaterial.
		TRACE_CPUPROFILER_EVENT_SCOPE(DumpDebugShaderInfoDefaultMaterial);

		UMaterial* DefaultMaterial = GDefaultMaterials[EMaterialDomain::MD_Surface];
		if (this == DefaultMaterial)
		{
			// Make the file in the automation directory so it will be uploaded as a build artifact.
			const FString Filename = FString::Printf(TEXT("%s%s-%s-%s.csv"), FApp::GetProjectName(), BuildSettings::GetBuildVersion(), *GetName(), *FDateTime::Now().ToString());
			const FString FilenameFull = FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs"), Filename);

			if (FArchive* FileArchive = IFileManager::Get().CreateDebugFileWriter(*FilenameFull))
			{
				FOutputDeviceArchiveWrapper Wrapper(FileArchive);
				DumpDebugInfo(Wrapper);
				FileArchive->Close();
				delete FileArchive;
			}
		}
	}

	TArray<TRefCountPtr<FMaterialResource>> MaterialsToDelete;
	{
		TArray<FMaterialResourceForCooking> CachedMaterialResourcesForPlatform;
		CachedMaterialResourcesForCooking.RemoveAndCopyValue(TargetPlatform, CachedMaterialResourcesForPlatform);
		MaterialsToDelete.Reserve(CachedMaterialResourcesForPlatform.Num());
		for (FMaterialResourceForCooking& MaterialToDelete : CachedMaterialResourcesForPlatform)
		{
			MaterialsToDelete.Add(MoveTemp(MaterialToDelete.Resource));
		}
	}
	FMaterial::DeferredDeleteArray(MaterialsToDelete);
}

void UMaterial::ClearAllCachedCookedPlatformData()
{
	TArray<TRefCountPtr<FMaterialResource>> MaterialsToDelete;
	for (TPair<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>& It : CachedMaterialResourcesForCooking)
	{
		MaterialsToDelete.Reserve(MaterialsToDelete.Num() + It.Value.Num());
		for (FMaterialResourceForCooking& MaterialToDelete : It.Value)
		{
			MaterialsToDelete.Add(MoveTemp(MaterialToDelete.Resource));
		}
	}
	CachedMaterialResourcesForCooking.Empty();
	FMaterial::DeferredDeleteArray(MaterialsToDelete);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
bool UMaterial::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
		FString PropertyName = InProperty->GetName();
		const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
		const bool bFrontMaterialConnected = EditorOnly->FrontMaterial.IsConnected();
		const bool bSubstrateHiddenMaterialAssetConversion = bSubstrateEnabled && Substrate::IsHiddenMaterialAssetConversionEnabled() && !bFrontMaterialConnected;

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, MaterialDomain))
		{
			// Material domain is no longer tweakable with Substrate. It is instead derived from the graph.
			return !bSubstrateEnabled || bSubstrateHiddenMaterialAssetConversion;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, PhysMaterial) || PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, PhysMaterialMask))
		{
			return MaterialDomain == MD_Surface;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, OpacityMaskClipValue) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, DitherOpacityMask)
			)
		{
			return IsMaskedBlendMode(BlendMode) ||
			bCastDynamicShadowAsMasked ||
			IsTranslucencyWritingCustomDepth() ||
			IsTranslucencyWritingVelocity();
		}

		if ( PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bCastDynamicShadowAsMasked) )
		{
			return IsTranslucentOnlyBlendMode(BlendMode);
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, MaterialDecalResponse))
		{
			static auto* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DBuffer"));
			static auto* CVarMobile = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.DBuffer"));

			return MaterialDomain == MD_Surface && (CVar->GetValueOnGameThread() > 0 || CVarMobile->GetValueOnGameThread() > 0);
		}		

		if(MaterialDomain == MD_PostProcess)
		{
			// some settings don't make sense for postprocess materials

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bTangentSpaceNormal) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bDisableDepthTest) ||
				PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUseMaterialAttributes)
				)
			{
				return false;
			}
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bFullyRough) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bNormalCurvatureToRoughness) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TwoSided) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUseLightmapDirectionality) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUseHQForwardReflections) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bForwardBlendsSkyLightCubemaps) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUsePlanarForwardReflections)
			)
		{
			return MaterialDomain == MD_Surface;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, BlendableLocation) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, BlendablePriority) || 
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, BlendableOutputAlpha) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bDisablePreExposureScale) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bIsBlendable) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bEnableStencilTest) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, StencilCompare) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, StencilRefValue) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, NeuralProfileId) ||
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUsedWithNeuralNetworks)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			)
		{
			return MaterialDomain == MD_PostProcess;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, UserSceneTexture) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, UserTextureDivisor) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, ResolutionRelativeToInput))
		{
			// "Replacing Tonemapper" blendable location doesn't support a UserSceneTexture output
			return MaterialDomain == MD_PostProcess && BlendableLocation != BL_ReplacingTonemapper;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, BlendMode))
		{
			if (bSubstrateEnabled && bFrontMaterialConnected)
			{
				return ((MaterialDomain != MD_PostProcess && MaterialDomain != MD_LightFunction && MaterialDomain != MD_Volume) || IsPostProcessMaterialOutputingAlpha());
			}
			else
			{
				return (MaterialDomain == MD_DeferredDecal || MaterialDomain == MD_Surface || MaterialDomain == MD_Volume || MaterialDomain == MD_UI || IsPostProcessMaterialOutputingAlpha());
			}
		}
	
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, ShadingModel))
		{
			// Volume can also use the unlit mode for emissive only volumetric effect (also overridable on material instance)
			return (!bSubstrateEnabled || bSubstrateHiddenMaterialAssetConversion) && (MaterialDomain == MD_Surface || MaterialDomain == MD_Volume);
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bIsThinSurface))
		{
			return bSubstrateEnabled && MaterialDomain == MD_Surface;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUsedWithStaticMesh))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// Don't bother to allow changes on DeferredDecal or Volume, they have shaders mixed into LocalVertexFactory which are needed
			return MaterialDomain == MD_Surface || MaterialDomain == MD_UI || MaterialDomain == MD_PostProcess || MaterialDomain == MD_LightFunction;
		}
		else if (FCString::Strncmp(*PropertyName, TEXT("bUsedWith"), 9) == 0)
		{
			return MaterialDomain == MD_DeferredDecal || MaterialDomain == MD_Surface || MaterialDomain == MD_Volume;
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUsesDistortion))
		{
			return MaterialDomain == MD_DeferredDecal || MaterialDomain == MD_Surface;
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, RefractionDepthBias))
		{
			return EditorOnly->Refraction.IsConnected();
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, RefractionCoverageMode))
		{
			return bSubstrateEnabled && !bSubstrateHiddenMaterialAssetConversion;
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bHasPixelAnimation))
		{
			return MaterialDomain == MD_Surface && IsOpaqueOrMaskedBlendMode(BlendMode);
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucencyPass)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bEnableResponsiveAA)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bScreenSpaceReflections)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bContactShadows)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bDisableDepthTest)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bUseTranslucencyVertexFog)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bComputeFogPerPixel)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bOutputTranslucentVelocity)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bIsTranslucencyVelocityFromDepth))
		{
			return MaterialDomain != MD_DeferredDecal && IsTranslucentBlendMode(BlendMode);
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bApplyCloudFogging))
		{
			const bool bApplyFogging = bUseTranslucencyVertexFog;
			return bApplyFogging && MaterialDomain != MD_DeferredDecal && IsTranslucentBlendMode(BlendMode);
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, bIsSky))
		{
			return (!bSubstrateEnabled && (MaterialDomain != MD_DeferredDecal && GetShadingModels().IsUnlit() && !IsTranslucentBlendMode(BlendMode)))
				|| (bSubstrateEnabled && (MaterialDomain != MD_DeferredDecal && !IsTranslucentBlendMode(BlendMode)));
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucencyLightingMode)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucencyDirectionalLightingIntensity)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentShadowDensityScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentSelfShadowDensityScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentSelfShadowSecondDensityScale)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentSelfShadowSecondOpacity)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentBackscatteringExponent)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentMultipleScatteringExtinction)
			|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, TranslucentShadowStartOffset))
		{
			return (!bSubstrateEnabled && (MaterialDomain != MD_DeferredDecal && IsTranslucentBlendMode(BlendMode) && GetShadingModels().IsLit()))
				|| (bSubstrateEnabled && (MaterialDomain != MD_DeferredDecal && IsTranslucentBlendMode(BlendMode)));
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterial, SubsurfaceProfile))
		{
			return MaterialDomain == MD_Surface && UseSubsurfaceProfile(ShadingModels) && IsOpaqueOrMaskedBlendMode(BlendMode);
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassMaterialInterfaceSettings, bCastShadowAsMasked))
		{
			return !IsOpaqueBlendMode(BlendMode) && BlendMode != BLEND_Modulate;
		}
	}

	return true;
}

bool UMaterial::Modify(bool bAlwaysMarkDirty)
{
	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
	if (EditorOnly)
	{
		EditorOnly->Modify(bAlwaysMarkDirty);
	}
	return Super::Modify(bAlwaysMarkDirty);
}

void UMaterial::PreEditChange(FProperty* PropertyThatChanged)
{
	Super::PreEditChange(PropertyThatChanged);
}

void UMaterial::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	return PostEditChangePropertyInternal(PropertyChangedEvent, EPostEditChangeEffectOnShaders::Default);
}

static bool IsMaterialAlreadyConvertedToSubstrate(const UMaterial* InMaterial, const UMaterialEditorOnlyData* InEditorOnly)
{
	return 
		InEditorOnly->FrontMaterial.IsConnected() || 
		InMaterial->HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionSubstrateBSDF>();
}

static bool ShouldAllowSubstrateReset(const UMaterialEditorOnlyData* EditorOnly)
{
	// We do reset the conversion to as "has been run" when Substrate is authored using the root node instead of FrontMaterial.
	return !Substrate::IsHiddenMaterialAssetConversionEnabled() || (EditorOnly && EditorOnly->FrontMaterial.IsConnected());
}

void UMaterial::PostEditChangePropertyInternal(FPropertyChangedEvent& PropertyChangedEvent, const EPostEditChangeEffectOnShaders EffectOnShaders)
{
	// PreEditChange is not enforced to be called before PostEditChange.
	// CacheResourceShadersForRendering if called will cause a rendering thread race condition with a debug mechanism (bDeletedThroughDeferredCleanup) if there is no flush or
	// FMaterialUpdateContext present.
	FlushRenderingCommands();

	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	//Cancel any current compilation jobs that are in flight for this material.
	CancelOutstandingCompilation();

	const UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();

	if (Substrate::IsSubstrateEnabled() && ShouldAllowSubstrateReset(EditorOnly))
	{
		// If a modification is made to the material, revert the auto-conversion version, to mark the material as 'edited'
		const bool bResetConversionVersion = PropertyThatChanged != nullptr || PropertyChangedEvent.ChangeType != EPropertyChangeType::Unspecified;
		if (bResetConversionVersion && EditorOnly)
		{
			GetEditorOnlyData()->ResetSubstrateConversionVersion();
		}

		// Apply Substrate material conversion if needed.
		// For imported materials, the conversion won't happen during the PostLoad() phase. 
		// Usually material importers create material by code, and then calls PostEditChange(), which calls PostEditChangeProperty()
		// This late conversion ensures that imported materials are converted properly with Substrate.
		if (!IsMaterialAlreadyConvertedToSubstrate(this, EditorOnly))
		{
			ConvertMaterialToSubstrateMaterial(false /*bAllowEmptyMaterialUpdate*/);
		}
	}

	// Check for distortion in material 
	bUsesDistortion = RefractionMethod != RM_None;

	// If BLEND_TranslucentColoredTransmittance is selected while Substrate is not enabled, force BLEND_Translucent blend mode
	if (!Substrate::IsSubstrateEnabled() && BlendMode == BLEND_TranslucentColoredTransmittance)
	{
		BlendMode = BLEND_Translucent;
	}

	bool bRequiresCompilation = true;
	if( PropertyThatChanged ) 
	{
		// Don't recompile the material if we only changed the PhysMaterial property.
		if (PropertyThatChanged->GetName() == TEXT("PhysMaterial") || PropertyThatChanged->GetName() == TEXT("PhysMaterialMask") || PropertyThatChanged->GetName() == TEXT("PhysicalMaterialMap"))
		{
			bRequiresCompilation = false;
		}
	}

	TranslucencyDirectionalLightingIntensity = FMath::Clamp(TranslucencyDirectionalLightingIntensity, .1f, 10.0f);

	// Don't want to recompile after a duplicate because it's just been done by PostLoad, nor during interactive changes to prevent constant recompilation while spinning properties.
	if( PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate || PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive )
	{
		bRequiresCompilation = false;
	}
	
	if (bRequiresCompilation)
	{
		UpdateCachedExpressionData();

		// When redirecting an object pointer, we trust that the DDC hash will detect the change and that we don't need to force a recompile.
		const bool bRegenerateId = PropertyChangedEvent.ChangeType != EPropertyChangeType::Redirected && EffectOnShaders != EPostEditChangeEffectOnShaders::DoesNotInvalidate;
		CacheResourceShadersForRendering(bRegenerateId, EMaterialShaderPrecompileMode::None);

		// Ensure that the ReferencedTextureGuids array is up to date.
		if (GIsEditor)
		{
			UpdateLightmassTextureTracking();
		}

		// Ensure that any components with static elements using this material have their render state recreated
		// so changes are propagated to them. The preview material is only applied to the preview mesh component,
		// and that reregister is handled by the material editor.
		if (!bIsPreviewMaterial && !bIsFunctionPreviewMaterial && !bIsMaterialEditorStatsMaterial)
		{
			FGlobalComponentRecreateRenderStateContext RecreateComponentsRenderState;
		}
	}

	// needed for UMaterial as it doesn't have the InitResources() override where this is called
	PropagateDataToMaterialProxy();

	// many property changes can require rebuild of graph so always mark as changed
	// not interested in PostEditChange calls though as the graph may have instigated it
	if (PropertyThatChanged && MaterialGraph)
	{
		MaterialGraph->RebuildGraph();
	}
} 

bool UMaterial::AddExpressionParameter(UMaterialExpression* Expression, TMap<FName, TArray<UMaterialExpression*> >& ParameterTypeMap)
{
	if(Expression && Expression->HasAParameterName())
	{
		const FName ParameterName = Expression->GetParameterName();
		TArray<UMaterialExpression*>* ExpressionList = ParameterTypeMap.Find(ParameterName);
		if (!ExpressionList)
		{
			ExpressionList = &ParameterTypeMap.Add(ParameterName, TArray<UMaterialExpression*>());
		}
		ExpressionList->Add(Expression);
		return true;
	}
	return false;
}

bool UMaterial::RemoveExpressionParameter(UMaterialExpression* Expression)
{
	if (Expression && Expression->HasAParameterName())
	{
		const FName ParameterName = Expression->GetParameterName();
		TArray<UMaterialExpression*>* ExpressionList = EditorParameters.Find(ParameterName);
		if (ExpressionList)
		{
			return ExpressionList->Remove(Expression) > 0;
		}
	}
	return false;
}

bool UMaterial::IsParameter(const UMaterialExpression* Expression)
{
	return Expression->HasAParameterName();
}

bool UMaterial::IsDynamicParameter(const UMaterialExpression* Expression)
{
	if (Expression->IsA(UMaterialExpressionDynamicParameter::StaticClass()))
	{
		return true;
	}

	return false;
}

void UMaterial::BuildEditorParameterList()
{
	EditorParameters.Empty();
	for(UMaterialExpression* Expression : GetExpressions())
	{
		AddExpressionParameter(Expression, EditorParameters);
	}
}

bool UMaterial::HasDuplicateParameters(const UMaterialExpression* Expression)
{
	FName ExpressionName;
	if(GetExpressionParameterName(Expression, ExpressionName))
	{
		const TArray<UMaterialExpression*>* ExpressionList = EditorParameters.Find(ExpressionName);
		if(ExpressionList)
		{
			const EMaterialParameterType ParameterType = Expression->GetParameterType();
			for (UMaterialExpression* CurNode : *ExpressionList)
			{
				if(CurNode != Expression && CurNode->GetParameterType() == ParameterType)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool UMaterial::HasDuplicateDynamicParameters(const UMaterialExpression* InExpression)
{
	const UMaterialExpressionDynamicParameter* DynParam = Cast<UMaterialExpressionDynamicParameter>(InExpression);
	if (DynParam)
	{
		for (UMaterialExpression* Expression : GetExpressions())
		{
			UMaterialExpressionDynamicParameter* CheckDynParam = Cast<UMaterialExpressionDynamicParameter>(Expression);
			if (CheckDynParam != InExpression)
			{
				return true;
			}
		}
	}
	return false;
}

void UMaterial::UpdateExpressionDynamicParameters(const UMaterialExpression* InExpression)
{
	const UMaterialExpressionDynamicParameter* DynParam = Cast<UMaterialExpressionDynamicParameter>(InExpression);
	if (DynParam)
	{
		for (UMaterialExpression* Expression : GetExpressions())
		{
			UMaterialExpressionDynamicParameter* CheckParam = Cast<UMaterialExpressionDynamicParameter>(Expression);
			if (CheckParam && CheckParam->CopyDynamicParameterProperties(DynParam))
			{
				CheckParam->GraphNode->ReconstructNode();
			}
		}
	}
}

void UMaterial::PropagateExpressionParameterChanges(const UMaterialExpression* Parameter)
{
	FMaterialParameterMetadata Meta;
	if (Parameter->GetParameterValue(Meta))
	{
		PropagateExpressionParameterChanges(Parameter->GetParameterName(), Meta);
	}
}

void UMaterial::PropagateExpressionParameterChanges(const FName& ParameterName, const FMaterialParameterMetadata& Meta)
{
	TArray<UMaterialExpression*>* ExpressionList = EditorParameters.Find(ParameterName);
	if (ExpressionList && ExpressionList->Num() > 1)
	{
		for (UMaterialExpression* Expression : *ExpressionList)
		{
			const EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::NoUpdateExpressionGuid | EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority;
			if (Expression->SetParameterValue(ParameterName, Meta, Flags))
			{
				Expression->Modify();
				Expression->Desc = Meta.Description;
				Expression->GraphNode->OnUpdateCommentText(Meta.Description);
			}
		}
	}
}

void UMaterial::UpdateExpressionParameterName(UMaterialExpression* Expression)
{
	for(TMap<FName, TArray<UMaterialExpression*> >::TIterator Iter(EditorParameters); Iter; ++Iter)
	{
		if(Iter.Value().Remove(Expression) > 0)
		{
			if(Iter.Value().Num() == 0)
			{
				EditorParameters.Remove(Iter.Key());
			}

			AddExpressionParameter(Expression, EditorParameters);
			break;
		}
	}
}

void UMaterial::RebuildShadingModelField()
{
	ShadingModels.ClearShadingModels();
	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();

	if (Substrate::IsSubstrateEnabled() && EditorOnly->FrontMaterial.IsConnected())
	{
		FSubstrateMaterialInfo SubstrateMaterialInfo = GetSubstrateMaterialInfo(EditorOnly);
		UpdatePropertyConnectedMask(SubstrateMaterialInfo, this->CachedExpressionData.Get());

#if WITH_EDITOR
		extern bool IsPreviewUnlitMaterial(const UMaterial * InMaterial);
		const bool bIsMaterialFunctionPreview = IsPreviewUnlitMaterial(this);
#else
		const bool bIsMaterialFunctionPreview = false;
#endif

		if (SubstrateMaterialInfo.IsValid() && SubstrateMaterialInfo.GetSubstrateTreeOutOfStackDepthOccurred())
		{
			SubstrateMaterialInfo.AddShadingModel(SSM_Unlit);
			MaterialDomain = EMaterialDomain::MD_Surface;
			ShadingModel = MSM_Unlit;
			ShadingModels.AddShadingModel(MSM_Unlit);
			CancelOutstandingCompilation();
			UE_LOGF(LogMaterial, Error, "%ls: Substrate - Cyclic graph detected when we only support acyclic graph.", *GetName());
			return;
		}

		bool bSanitizeMaterial = false;
		if (!SubstrateMaterialInfo.IsValid() && !SubstrateMaterialInfo.HasShadingModelFromExpression() )
		{
			bSanitizeMaterial = true;
			UE_LOGF(LogMaterial, Error, "%ls: Material information is invalid.", *GetName());
		}

		if (SubstrateMaterialInfo.HasShadingModel(SSM_Decal))
		{
			// Keep the decals information and remove the shading from expression part 
			// since we are going to bake down the material to a single slab using parameter blending.
			SubstrateMaterialInfo.SetShadingModelFromExpression(false);
		}

		if (SubstrateMaterialInfo.CountShadingModels() > 1)
		{
			if (SubstrateMaterialInfo.HasShadingModelFromExpression())
			{
				if (BlendMode == EBlendMode::BLEND_Opaque || BlendMode == EBlendMode::BLEND_Masked)
				{
					// Nothing
				}
				else
				{
					// For transparent, we will fall back to use DefaultLit with simple volumetric
					bSanitizeMaterial = true;
				}
			}
			else if (SubstrateMaterialInfo.CountShadingModels() == 2 && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_DefaultLit) && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_SubsurfaceMFP))
			{
				if (BlendMode == EBlendMode::BLEND_Opaque || BlendMode == EBlendMode::BLEND_Masked)
				{
					SubstrateMaterialInfo.SetSingleShadingModel(SSM_SubsurfaceMFP);	// We only consider SSS subsurface post process for opaque materials.
				}
				else
				{
					bSanitizeMaterial = true;										// For transparent, we will fall back to use DefaultLit with simple volumetric
				}
			}
			else if (SubstrateMaterialInfo.CountShadingModels() == 2 && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_DefaultLit) && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_SubsurfaceProfile))
			{
				if (BlendMode == EBlendMode::BLEND_Opaque || BlendMode == EBlendMode::BLEND_Masked)
				{
					SubstrateMaterialInfo.SetSingleShadingModel(SSM_SubsurfaceProfile);// We only consider SSS subsurface post process for opaque materials.
				}
				else
				{
					bSanitizeMaterial = true;										// For transparent, we will fall back to use DefaultLit with simple volumetric
				}
			}
			else if (SubstrateMaterialInfo.CountShadingModels() > 1 && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_SubsurfaceMFP))
			{
				// When we gather the worst case slab material, we can hit multiple shading models when gathering from multiple BSDF possibilities of switch nodes.
				// We do not want to sanitize the material in this case, but only keep the worst case: subsurface with all potential features on a slab.
				if (BlendMode == EBlendMode::BLEND_Opaque || BlendMode == EBlendMode::BLEND_Masked)
				{
					// We only consider SSS subsurface post process for opaque materials. This can also be triggered by switch node accumulating multiple slab node.
					// We still keep all the other material info to consider the worst case when encountering multiple shading models. And we also keep around the encountered subsurface profiles.
					SubstrateMaterialInfo.SetSingleShadingModel(SSM_SubsurfaceMFP);
				}
				else
				{
					// For transparent, we will fall back to use DefaultLit worst case with simple volumetric.
					bSanitizeMaterial = true;
				}
			}
			else if (SubstrateMaterialInfo.CountShadingModels() > 1 && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_SubsurfaceProfile))
			{
				// When we gather the worst case slab material, we can hit multiple shading models when gathering from multiple BSDF possibilities of switch nodes.
				// We do not want to sanitize the material in this case, but only keep the worst case: subsurface with all potential features on a slab.
				if (BlendMode == EBlendMode::BLEND_Opaque || BlendMode == EBlendMode::BLEND_Masked)
				{
					// We only consider SSS subsurface post process for opaque materials. This can also be triggered by switch node accumulating multiple slab node.
					// We still keep all the other material info to consider the worst case when encountering multiple shading models. And we also keep around the encountered subsurface profiles.
					SubstrateMaterialInfo.SetSingleShadingModel(SSM_SubsurfaceProfile);
				}
				else
				{
					// For transparent, we will fall back to use DefaultLit worst case with simple volumetric.
					bSanitizeMaterial = true;
				}
			}
			else if (SubstrateMaterialInfo.CountShadingModels() == 2 && SubstrateMaterialInfo.HasShadingModel(ESubstrateShadingModel::SSM_Decal))
			{
				// If material has SSM_Decal it has to have 'decal' domain and DefaultLit shading model
				if (MaterialDomain != MD_DeferredDecal || !SubstrateMaterialInfo.HasShadingModel(SSM_DefaultLit))
				{
					bSanitizeMaterial = true;
				}

				// If blend mode is not one of the translucent blend modes, force sanitization
				if (!(IsTranslucentOnlyBlendMode(BlendMode) || BlendMode == BLEND_AlphaComposite || IsModulateBlendMode(BlendMode)))
				{
					bSanitizeMaterial = true;
				}
			}
			else if (SubstrateMaterialInfo.CountShadingModels() > 1 && MaterialDomain == MD_Surface)
			{
				// Case with SSS Profile or SSS MFP are already been handled by above cases. Simply fallback onto DefaultLit
				if (BlendMode == EBlendMode::BLEND_Opaque || BlendMode == EBlendMode::BLEND_Masked)
				{
					SubstrateMaterialInfo.SetSingleShadingModel(SSM_DefaultLit);
				}
				else
				{
					// For transparent, we will fall back to use DefaultLit worst case with simple volumetric.
					bSanitizeMaterial = true;
				}
			}
			else
			{
				// Clear the material to default Lit
				bSanitizeMaterial = true;
				UE_LOGF(LogMaterial, Error, "%ls: Material has more than a single material represented.", *GetName());
			}
		}

		if ((SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SubsurfaceMFP) || SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SubsurfaceProfile)) 
			&& (BlendMode != EBlendMode::BLEND_Opaque && BlendMode != EBlendMode::BLEND_Masked))
		{
			// For transparent, we will fall back to use DefaultLit with simple volumetric
			bSanitizeMaterial = true;
		}

		if (bSanitizeMaterial)
		{
			const FSubstrateMaterialInfo PreSanitizeSubstrateMaterialInfo = SubstrateMaterialInfo;
			SubstrateMaterialInfo = FSubstrateMaterialInfo();
			SubstrateMaterialInfo.AddShadingModel(SSM_DefaultLit);

			if (MaterialDomain == MD_DeferredDecal || PreSanitizeSubstrateMaterialInfo.HasShadingModel(SSM_Decal))
			{
				SubstrateMaterialInfo.AddShadingModel(SSM_Decal);
			}
			else if (MaterialDomain == MD_LightFunction)
			{
				SubstrateMaterialInfo.AddShadingModel(SSM_LightFunction);
			}
			else if (MaterialDomain == MD_Volume)
			{
				SubstrateMaterialInfo.AddShadingModel(SSM_VolumetricFogCloud);
			}
			else if (MaterialDomain == MD_PostProcess)
			{
				SubstrateMaterialInfo.AddShadingModel(SSM_PostProcess);
			}
			else if (MaterialDomain == MD_UI)
			{
				SubstrateMaterialInfo.AddShadingModel(SSM_UI);
			}
			else if (MaterialDomain == MD_RuntimeVirtualTexture)
			{
				// TODO
			}
			else if (MaterialDomain == MD_Surface)
			{
				// Nothing to do, the node should have added its own type. And if not type but from expression, we are going to generate that below.
				//AddSurfaceSubstrateShadingModelFromMaterialShadingModel(SubstrateMaterialInfo, ShadingModel);
			}
		}
		
		if (SubstrateMaterialInfo.HasShadingModelFromExpression())
		{
			MaterialDomain = EMaterialDomain::MD_Surface;
			ShadingModel = MSM_FromMaterialExpression;

			{
				TArray<UMaterialExpressionShadingModel*> ShadingModelExpressions;
				GetAllExpressionsInMaterialAndFunctionsOfType(ShadingModelExpressions);

				for (UMaterialExpressionShadingModel* MatExpr : ShadingModelExpressions)
				{
					// Ensure the Shading model is valid
					if (MatExpr->ShadingModel < MSM_NUM)
					{
						ShadingModels.AddShadingModel(MatExpr->ShadingModel);
						AddSurfaceSubstrateShadingModelFromMaterialShadingModel(SubstrateMaterialInfo, MatExpr->ShadingModel);
					}
				}

				// If no expressions have been found, set a default
				if (!ShadingModels.IsValid())
				{
					ShadingModels.AddShadingModel(MSM_DefaultLit);
					AddSurfaceSubstrateShadingModelFromMaterialShadingModel(SubstrateMaterialInfo, MSM_DefaultLit);
				}
			}
		}
		else
		{
			if (bIsMaterialFunctionPreview)
			{
				// Keep material domain selected from the preview window and set unlit for preview to work.
				ShadingModel = MSM_Unlit;
			}
			// Now derive some properties from the material into the legacy fields
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_Unlit))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_Unlit;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_DefaultLit))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_DefaultLit;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SubsurfaceProfile) || SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SubsurfaceMFP))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_SubsurfaceProfile;
				if (BlendMode != EBlendMode::BLEND_Opaque && BlendMode != EBlendMode::BLEND_Masked)
				{
					UE_LOGF(LogMaterial, Error, "%ls: Material has subsurface data, and its blending mode is not set to Opaque or Masked. Forcing blend mode to Opaque.", *GetName());
					BlendMode = EBlendMode::BLEND_Opaque;
				}
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SubsurfaceWrap))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_Subsurface;
				// This is a valid shading mode for both opaque/masked/translucent blend modes
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SubsurfaceThinTwoSided))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_TwoSidedFoliage;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_ThinTranslucent))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_ThinTranslucent;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_VolumetricFogCloud))
			{
				MaterialDomain = EMaterialDomain::MD_Volume;
				// Volumetric shading model do support Unlit as an EmissiveOnly mode, on top of the lit mode.
				TArray<UMaterialExpressionSubstrateVolumetricFogCloudBSDF*> ShadingModelExpressions;
				GetAllExpressionsInMaterialAndFunctionsOfType(ShadingModelExpressions);
				if (ShadingModelExpressions.Num() > 0)
				{
					// We can only keep a single shading model due to an assert in UMaterialInstance::UpdateOverridableBaseProperties which want a single shading model for non material attribute workflow.
					UMaterialExpressionSubstrateVolumetricFogCloudBSDF* MatExpr = ShadingModelExpressions[0];
					ShadingModel = MatExpr->bEmissiveOnly ? MSM_Unlit : MSM_DefaultLit;
				}
				BlendMode = EBlendMode::BLEND_Additive;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_Hair))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_Hair;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_Eye))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_Eye;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_Cloth))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_Cloth;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_ClearCoat))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_ClearCoat;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_SingleLayerWater))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_SingleLayerWater;
				if (BlendMode != EBlendMode::BLEND_Opaque && BlendMode != EBlendMode::BLEND_Masked)
				{
					BlendMode = EBlendMode::BLEND_Opaque;
				}
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_Toon))
			{
				MaterialDomain = EMaterialDomain::MD_Surface;
				ShadingModel = MSM_DefaultLit;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_LightFunction))
			{
				MaterialDomain = EMaterialDomain::MD_LightFunction;
				ShadingModel = MSM_Unlit;
				BlendMode = EBlendMode::BLEND_Opaque;
			}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_PostProcess))
			{
				MaterialDomain = EMaterialDomain::MD_PostProcess;
				ShadingModel = MSM_Unlit;
				// We keep the blend mode resulting from ConvertLegacyToSubstrateBlendMode because post processes can be translucent.
				// However, we do force opaque mode if blending has been disabled via the post-process specific BlendableOutputAlpha option.
				if (!IsPostProcessMaterialOutputingAlpha())
				{
					BlendMode = BLEND_Opaque;
				}
				}
			else if (SubstrateMaterialInfo.HasOnlyShadingModel(SSM_UI))
			{
				MaterialDomain = EMaterialDomain::MD_UI;
				ShadingModel = MSM_Unlit;
			}
			else if (SubstrateMaterialInfo.HasShadingModel(SSM_Decal))
			{
				// Decal can have multiple shading model
				MaterialDomain = EMaterialDomain::MD_DeferredDecal;
				ShadingModel = MSM_DefaultLit;

				if (!(IsTranslucentOnlyBlendMode(BlendMode) || BlendMode == BLEND_AlphaComposite || IsModulateBlendMode(BlendMode)))
				{
					BlendMode = BLEND_TranslucentGreyTransmittance;
				}
			}

			// Also update the ShadingModels for remaining pipeline operation
			ShadingModels.AddShadingModel(ShadingModel);
		}

		// Now, reset the subsurface profile (in case it has been removed from any slab before) and set it only if needed.
		SubsurfaceProfile = nullptr;
		if ((SubstrateMaterialInfo.HasShadingModel(SSM_Eye) || SubstrateMaterialInfo.HasShadingModel(SSM_SubsurfaceProfile)) && SubstrateMaterialInfo.CountSubsurfaceProfiles() > 0)
		{
			SubsurfaceProfile = SubstrateMaterialInfo.GetSubsurfaceProfile(0);
		}

		// Set subsurface profiles if any
		SubsurfaceProfiles.SetNum(SubstrateMaterialInfo.CountSubsurfaceProfiles());
		for (int32 It = 0, Count = SubstrateMaterialInfo.CountSubsurfaceProfiles(); It < Count; ++It)
		{
			SubsurfaceProfiles[It] = SubstrateMaterialInfo.GetSubsurfaceProfile(It);
		}

		// Set specular profile if any
		SpecularProfiles.SetNum(SubstrateMaterialInfo.CountSpecularProfiles());
		for (int32 It = 0, Count = SubstrateMaterialInfo.CountSpecularProfiles(); It<Count; ++It)
		{
			SpecularProfiles[It] = SubstrateMaterialInfo.GetSpecularProfile(It);
		}

		// Set Toon profile if any
		ToonProfiles.SetNum(SubstrateMaterialInfo.CountToonProfiles());
		for (int32 It = 0, Count = SubstrateMaterialInfo.CountToonProfiles(); It<Count; ++It)
		{
			ToonProfiles[It] = SubstrateMaterialInfo.GetToonProfile(It);
		}
	}
	// If using shading model from material expression, go through the expressions and look for the ShadingModel expression to figure out what shading models need to be supported in this material.
	// This might not be the same as what is actually compiled in to the shader, since there might be feature switches, static switches etc. that skip certain shading models.
	else if (ShadingModel == MSM_FromMaterialExpression)
	{
		TArray<UMaterialExpressionShadingModel*> ShadingModelExpressions;
		GetAllExpressionsInMaterialAndFunctionsOfType(ShadingModelExpressions);

		ShadingModel = MSM_FromMaterialExpression;

		for (UMaterialExpressionShadingModel* MatExpr : ShadingModelExpressions)
		{
			// Ensure the Shading model is valid
			if (MatExpr->ShadingModel < MSM_NUM)
			{
				ShadingModels.AddShadingModel(MatExpr->ShadingModel);
			}
		}

		// If no expressions have been found, set a default
		if (!ShadingModels.IsValid())
		{
			ShadingModels.AddShadingModel(MSM_DefaultLit);
		}
	}
	else 
	{
		// If a shading model has been selected directly for the material, set it here
		ShadingModels.AddShadingModel(ShadingModel);
	}

#if WITH_EDITORONLY_DATA
	// Build a string with all the shading models on this material. Used to display the used shading models in this material
	auto ShadingModelToStringLambda = 
	[](EMaterialShadingModel InShadingModel) -> FString
	{ 
		return StaticEnum<EMaterialShadingModel>()->GetDisplayNameTextByValue(InShadingModel).ToString();
	};
	UsedShadingModels = GetShadingModelFieldString(ShadingModels, FShadingModelToStringDelegate::CreateLambda(ShadingModelToStringLambda), " | ");
#endif
}

bool UMaterial::GetExpressionParameterName(const UMaterialExpression* Expression, FName& OutName)
{
	bool bRet = false;
	if (Expression->HasAParameterName())
	{
		OutName = Expression->GetParameterName();
		bRet = true;
	}
	return bRet;
}
#endif // WITH_EDITOR

void UMaterial::BeginDestroy()
{
	TArray<TRefCountPtr<FMaterialResource>> ResourcesToDestroy;
	for (FMaterialResource* Resource : MaterialResources)
	{
		Resource->SetOwnerBeginDestroyed();
		if (Resource->PrepareDestroy_GameThread())
		{
			ResourcesToDestroy.Add(Resource);
		}
	}

	Super::BeginDestroy();

	if (DefaultMaterialInstance || ResourcesToDestroy.Num() > 0)
	{
		ReleasedByRT = false;

		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)(
			[ResourcesToDestroy = MoveTemp(ResourcesToDestroy), this](FRHICommandListImmediate& RHICmdList) mutable
		{
			FMaterialRenderProxy* LocalResource = DefaultMaterialInstance;
			if (LocalResource)
			{
				LocalResource->MarkForGarbageCollection();
				LocalResource->ReleaseResource();
			}

			for (FMaterialResource* Resource : ResourcesToDestroy)
			{
				Resource->PrepareDestroy_RenderThread();
			}

			// Release the references before assigning the bool below.
			ResourcesToDestroy.Empty();

			// And remove from deferred uniform expression cache queue if it's in that
			if (bCachingUniformExpressions)
			{
				LocalResource->CancelCacheUniformExpressions();
				bCachingUniformExpressions = false;
			}

			ReleasedByRT = true;
		});
	}
}

bool UMaterial::IsReadyForFinishDestroy()
{
	bool bReady = Super::IsReadyForFinishDestroy();

	return bReady && ReleasedByRT;
}

void UMaterial::ReleaseResources()
{
	for (FMaterialResource* Resource : MaterialResources)
	{
		delete Resource;
	}
	MaterialResources.Empty();
	
#if WITH_EDITOR
	if (!GExitPurge)
	{
		ClearAllCachedCookedPlatformData();
	}
#endif
	if (DefaultMaterialInstance)
	{
		DefaultMaterialInstance->GameThread_Destroy();
		DefaultMaterialInstance = nullptr;
	}
}

void UMaterial::FinishDestroy()
{
	ReleaseResources();

	Super::FinishDestroy();
}

void UMaterial::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (DefaultMaterialInstance)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(FDefaultMaterialInstance));
	}

	for (FMaterialResource* CurrentResource : MaterialResources)
	{
		CurrentResource->GetResourceSizeEx(CumulativeResourceSize);
	}
}

void UMaterial::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UMaterial* This = CastChecked<UMaterial>(InThis);

	for (FMaterialResource* CurrentResource : This->MaterialResources)
	{
		CurrentResource->AddReferencedObjects(Collector);
	}

#if WITH_EDITOR
	for (TPair<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>& It : This->CachedMaterialResourcesForCooking)
	{
		for (FMaterialResourceForCooking& CurrentResource : It.Value)
		{
			if (CurrentResource.Resource)
			{
				CurrentResource.Resource->AddReferencedObjects(Collector);
			}
		}
	}
#endif

#if WITH_EDITORONLY_DATA
	Collector.AddReferencedObject(This->MaterialGraph, This);
#endif

	Super::AddReferencedObjects(This, Collector);
}

bool UMaterial::CanBeClusterRoot() const 
{
	return true;
}

void UMaterial::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
void UMaterial::CancelOutstandingCompilation()
{
	for (int32 ShaderPlatformIndex = 0; ShaderPlatformIndex < EShaderPlatform::SP_NumPlatforms; ++ShaderPlatformIndex)
	{
		if (FMaterialResource* Res = GetMaterialResource((EShaderPlatform)ShaderPlatformIndex))
		{
			Res->CancelCompilation();
		}
	}
}
#endif

void UMaterial::UpdateMaterialShaders(TArray<const FShaderType*>& ShaderTypesToFlush, TArray<const FShaderPipelineType*>& ShaderPipelineTypesToFlush, TArray<const FVertexFactoryType*>& VFTypesToFlush, EShaderPlatform ShaderPlatform)
{
	bool bAnyMaterialShaderTypes = VFTypesToFlush.Num() > 0 || ShaderPipelineTypesToFlush.Num() > 0;

	if (!bAnyMaterialShaderTypes)
	{
		for (int32 TypeIndex = 0; TypeIndex < ShaderTypesToFlush.Num(); TypeIndex++)
		{
			if (ShaderTypesToFlush[TypeIndex]->GetMaterialShaderType() || ShaderTypesToFlush[TypeIndex]->GetMeshMaterialShaderType())
			{
				bAnyMaterialShaderTypes = true;
				break;
			}
		}
	}

	if (bAnyMaterialShaderTypes)
	{
		// Create a material update context so we can safely update materials.
		{
			FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::Default, ShaderPlatform);

			int32 NumMaterials = 0;

			for (TObjectIterator<UMaterial> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
			{
				NumMaterials++;
			}

			GWarn->StatusUpdate(0, NumMaterials, NSLOCTEXT("Material", "BeginAsyncMaterialShaderCompilesTask", "Kicking off async material shader compiles..."));

			int32 UpdateStatusDivisor = FMath::Max<int32>(NumMaterials / 20, 1);
			int32 MaterialIndex = 0;

			// Reinitialize the material shader maps
			for (TObjectIterator<UMaterial> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
			{
				UMaterial* BaseMaterial = *It;
				UpdateContext.AddMaterial(BaseMaterial);
				BaseMaterial->CacheResourceShadersForRendering(false);

				// Limit the frequency of progress updates
				if (MaterialIndex % UpdateStatusDivisor == 0)
				{
					GWarn->UpdateProgress(MaterialIndex, NumMaterials);
				}
				MaterialIndex++;
			}

			// The material update context will safely update all dependent material instances when
			// it leaves scope.
		}

#if WITH_EDITOR
		// Update any FMaterials not belonging to a UMaterialInterface, for example FExpressionPreviews
		// If we did not do this, the editor would crash the next time it tried to render one of those previews
		// And didn't find a shader that had been flushed for the preview's shader map.
		FMaterial::UpdateEditorLoadedMaterialResources(ShaderPlatform);
#endif
	}
}

#if WITH_EDITOR
void UMaterial::CompileMaterialsForRemoteRecompile(
	const TArray<UMaterialInterface*>& MaterialsToCompile,
	EShaderPlatform ShaderPlatform,
	ITargetPlatform* TargetPlatform,
	TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap> > >& OutShaderMaps)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterial::CompileMaterialsForRemoteRecompile);

	// Build a map from UMaterial / UMaterialInstance to the resources which are being compiled
	TMap<FString, TArray<TRefCountPtr<FMaterialResource>> > CompilingResources;

	// compile the requested materials
	for (int32 Index = 0; Index < MaterialsToCompile.Num(); Index++)
	{
		// get the material resource from the UMaterialInterface
		UMaterialInterface* Material = MaterialsToCompile[Index];
		UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material);
		UMaterial* BaseMaterial = Cast<UMaterial>(Material);

		TArray<FMaterialResourceForCooking> ResourcesForCooking;
		if (MaterialInstance && MaterialInstance->bHasStaticPermutationResource)
		{
			MaterialInstance->CacheResourceShadersForCooking(ShaderPlatform, ResourcesForCooking,
				EMaterialShaderPrecompileMode::Default, TargetPlatform, true /* Blocking */);
		}
		else if (BaseMaterial)
		{
			BaseMaterial->CacheResourceShadersForCooking(ShaderPlatform, ResourcesForCooking,
				TargetPlatform, true /* Blocking */);
		}

		if (!ResourcesForCooking.IsEmpty())
		{
			TArray<TRefCountPtr<FMaterialResource>>& ResourceArray = CompilingResources.Add(Material->GetPathName(), 
				TArray<TRefCountPtr<FMaterialResource>>());
			ResourceArray.Reserve(ResourcesForCooking.Num());
			for (FMaterialResourceForCooking& ResourceForCooking : ResourcesForCooking)
			{
				ResourceArray.Add(MoveTemp(ResourceForCooking.Resource));
			}
		}
	}

	// Wait until all compilation is finished and all of the gathered FMaterialResources have their GameThreadShaderMap up to date
	GShaderCompilingManager->FinishAllCompilation();

	// This is heavy handed, but wait until we've set the render thread shader map before proceeding to delete the FMaterialResource below.
	// This is code that should be run on the cooker so shouldn't be a big deal.
	FlushRenderingCommands();

	for(TMap<FString, TArray<TRefCountPtr<FMaterialResource>> >::TIterator It(CompilingResources); It; ++It)
	{
		TArray<TRefCountPtr<FMaterialResource>>& ResourceArray = It.Value();
		TArray<TRefCountPtr<FMaterialShaderMap>>& OutShaderMapArray = OutShaderMaps.Add(It.Key(), TArray<TRefCountPtr<FMaterialShaderMap>>());

		for (int32 Index = 0; Index < ResourceArray.Num(); Index++)
		{
			TRefCountPtr<FMaterialResource> CurrentResource = MoveTemp(ResourceArray[Index]);
			OutShaderMapArray.Add(CurrentResource->GetGameThreadShaderMap());
		}
	}
}

void UMaterial::CompileODSCMaterialsForRemoteRecompile(TArray<FODSCRequestPayload> ShadersToRecompile, TMap<FString, TArray<TRefCountPtr<class FMaterialShaderMap>>>& OutShaderMaps, 
													   TFunction<UMaterialInterface*(const FString&)> ODSCCustomLoadMaterial)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterial::CompileODSCMaterialsForRemoteRecompile);

	// Build a map from UMaterial / UMaterialInstance to the resources which are being compiled
	TMap<FString, TArray<FMaterialResource*> > CompilingResources;

	if (ShadersToRecompile.Num())
	{
		UE_LOGF(LogShaders, Display, "Received %d shaders to compile.", ShadersToRecompile.Num());
	}

	// Group all shader types we need to compile by material so we can issue them all in a single call.
	struct FShadersToCompile
	{
		EShaderPlatform ShaderPlatform;
		ERHIFeatureLevel::Type FeatureLevel;
		EMaterialQualityLevel::Type QualityLevel;
		TArray<const FVertexFactoryType*> VFTypes;
		TArray<const FShaderPipelineType*> PipelineTypes;
		TArray<const FShaderType*> ShaderTypes;
		FString OriginalMaterialName;
	};
	TMap<UMaterialInterface*, FShadersToCompile> CoalescedShadersToCompile;
	
	TSet<FString> MaterialsFailingToLoad;
	for (const FODSCRequestPayload& payload : ShadersToRecompile)
	{
		UE_LOGF(LogShaders, Verbose, "");
		UE_LOGF(LogShaders, Verbose, "Material:    %ls ", *payload.MaterialName);

		UMaterialInterface* MaterialInterface = nullptr;
		if (ODSCCustomLoadMaterial)
		{
			MaterialInterface = ODSCCustomLoadMaterial(payload.MaterialName);
		}
		else
		{
			MaterialInterface = LoadObject<UMaterialInterface>(nullptr, *payload.MaterialName);
		}

		if (!MaterialInterface)
		{
			MaterialsFailingToLoad.Add(payload.MaterialName);
		}

		if (MaterialInterface)
		{
			FShadersToCompile& Shaders = CoalescedShadersToCompile.FindOrAdd(MaterialInterface);
			const FVertexFactoryType* VFType = FVertexFactoryType::GetVFByName(payload.VertexFactoryName);
			const FShaderPipelineType* PipelineType = FShaderPipelineType::GetShaderPipelineTypeByName(payload.PipelineName);

			Shaders.ShaderPlatform = payload.ShaderPlatform;
			Shaders.FeatureLevel = payload.FeatureLevel;
			Shaders.QualityLevel = payload.QualityLevel;
			Shaders.OriginalMaterialName = payload.MaterialName;

			if (VFType)
			{
				UE_LOGF(LogShaders, Verbose, "VF Type:     %ls ", *payload.VertexFactoryName);
			}

			if (PipelineType)
			{
				UE_LOGF(LogShaders, Verbose, "Pipeline Type: %ls", *payload.PipelineName);

				Shaders.VFTypes.Add(VFType);
				Shaders.PipelineTypes.Add(PipelineType);
				Shaders.ShaderTypes.Add(nullptr);
			}
			else
			{
				for (const FString& ShaderTypeName : payload.ShaderTypeNames)
				{
					const FShaderType* ShaderType = FShaderType::GetShaderTypeByName(*ShaderTypeName);
					if (ShaderType)
					{
						UE_LOGF(LogShaders, Verbose, "\tShader Type: %ls", *ShaderTypeName);

						Shaders.VFTypes.Add(VFType);
						Shaders.PipelineTypes.Add(nullptr);
						Shaders.ShaderTypes.Add(ShaderType);
					}
				}
			}
		}
	}

	// pass one, cache the coalesced shaders in case there are ordering dependencies in the list
	for (const auto& Entry : CoalescedShadersToCompile)
	{
		UMaterialInterface* MaterialInterface = Entry.Key;
		const FShadersToCompile& Shaders = Entry.Value;
		MaterialInterface->CacheGivenTypesForCooking(Shaders.ShaderPlatform, Shaders.FeatureLevel, Shaders.QualityLevel, Shaders.VFTypes, Shaders.PipelineTypes, Shaders.ShaderTypes);
	}

	// pass two can now run through the shaders in the same order successfully
	for (const auto& Entry : CoalescedShadersToCompile)
	{
		UMaterialInterface* MaterialInterface = Entry.Key;
		const FShadersToCompile& Shaders = Entry.Value;

		TArray<FMaterialResource*>& ResourceArray = CompilingResources.Add(Shaders.OriginalMaterialName, TArray<FMaterialResource*>());
		FMaterialResource* MaterialResource = MaterialInterface->GetMaterialResource(Shaders.ShaderPlatform, Shaders.QualityLevel);
		check(MaterialResource);
		check(MaterialResource->GetFeatureLevel() == Shaders.FeatureLevel);
		check(MaterialResource->GetShaderPlatform() == Shaders.ShaderPlatform);
		ResourceArray.Add(MaterialResource);
	}

	// Wait until all compilation is finished and all of the gathered FMaterialResources have their GameThreadShaderMap up to date
	GShaderCompilingManager->FinishAllCompilation();

	// This is heavy handed, but wait until we've set the render thread shader map before proceeding to delete the FMaterialResource below.
	// This is code that should be run on the cooker so shouldn't be a big deal.
	FlushRenderingCommands();

	int32 NumShaders = 0;
	int32 NumShaderPipelines = 0;
	int32 NumShaderMaps = 0;
	for (TMap<FString, TArray<FMaterialResource*> >::TIterator It(CompilingResources); It; ++It)
	{
		TArray<FMaterialResource*>& ResourceArray = It.Value();
		TArray<TRefCountPtr<FMaterialShaderMap> >& OutShaderMapArray = OutShaderMaps.Add(It.Key(), TArray<TRefCountPtr<FMaterialShaderMap> >());

		for (int32 Index = 0; Index < ResourceArray.Num(); Index++)
		{
			FMaterialResource* CurrentResource = ResourceArray[Index];
			FMaterialShaderMap* MaterialShaderMap = CurrentResource->GetGameThreadShaderMap();
			OutShaderMapArray.Add(MaterialShaderMap);

			int32 ThisMaterialNumShaders = 0;
			int32 ThisMaterialNumShaderPipelines = 0;
			MaterialShaderMap->CountNumShaders(ThisMaterialNumShaders, ThisMaterialNumShaderPipelines);

			NumShaders += ThisMaterialNumShaders;
			NumShaderPipelines += ThisMaterialNumShaderPipelines;
			++NumShaderMaps;
		}
	}

	UE_LOGF(LogShaders, Display, "Sending %d shaders, %d pipelines for %d shader maps", NumShaders, NumShaderPipelines, NumShaderMaps);

	// Report errors last to ensure visibility: some messages may have been output during shader compilation
	for (FString& MaterialFailingToLoad : MaterialsFailingToLoad)
	{
		UE_LOGF(LogShaders, Warning, "Failed to load %ls, skipping shader reloading", *MaterialFailingToLoad);
	}
}
#endif // WITH_EDITOR

bool UMaterial::UpdateLightmassTextureTracking()
{
	bool bTexturesHaveChanged = false;
#if WITH_EDITORONLY_DATA
	TArray<UTexture*> UsedTextures;
	
	GetUsedTextures(UsedTextures);
	if (UsedTextures.Num() != ReferencedTextureGuids.Num())
	{
		bTexturesHaveChanged = true;
		// Just clear out all the guids and the code below will
		// fill them back in...
		ReferencedTextureGuids.Empty(UsedTextures.Num());
		ReferencedTextureGuids.AddZeroed(UsedTextures.Num());
	}
	
	for (int32 CheckIdx = 0; CheckIdx < UsedTextures.Num(); CheckIdx++)
	{
		UTexture* Texture = UsedTextures[CheckIdx];
		if (Texture)
		{
			if (ReferencedTextureGuids[CheckIdx] != Texture->GetLightingGuid())
			{
				ReferencedTextureGuids[CheckIdx] = Texture->GetLightingGuid();
				bTexturesHaveChanged = true;
			}
		}
		else
		{
			if (ReferencedTextureGuids[CheckIdx] != FGuid(0,0,0,0))
			{
				ReferencedTextureGuids[CheckIdx] = FGuid(0,0,0,0);
				bTexturesHaveChanged = true;
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	return bTexturesHaveChanged;
}

#if WITH_EDITOR

FExpressionInput* UMaterial::GetExpressionInputForProperty(EMaterialProperty InProperty)
{
	FMaterialInputDescription Description;
	if (GetExpressionInputDescription(InProperty, Description))
	{
		if (!Description.bHidden)
		{
			return Description.Input;
		}
	}
	return nullptr;
}

static void SetMaterialInputDescription(FColorMaterialInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Float3;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = Input.UseConstant;
	OutDescription.bHidden = bHidden;
	const FLinearColor ConstantColor(Input.Constant);
	OutDescription.ConstantValue = UE::Shader::FValue(ConstantColor.R, ConstantColor.G, ConstantColor.B);
}

static void SetMaterialInputDescription(FVectorMaterialInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Float3;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = Input.UseConstant;
	OutDescription.bHidden = bHidden;
	OutDescription.ConstantValue = Input.Constant;
}

static void SetMaterialInputDescription(FVector2MaterialInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Float2;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = Input.UseConstant;
	OutDescription.bHidden = bHidden;
	OutDescription.ConstantValue = Input.Constant;
}

static void SetMaterialInputDescription(FScalarMaterialInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Float1;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = Input.UseConstant;
	OutDescription.bHidden = bHidden;
	OutDescription.ConstantValue = Input.Constant;
}

static void SetMaterialInputDescription(FShadingModelMaterialInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Int1;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = false;
	OutDescription.bHidden = bHidden;
}

static void SetMaterialInputDescription(FMaterialAttributesInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Struct;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = false;
	OutDescription.bHidden = bHidden;
}

static void SetMaterialInputDescription(FSubstrateMaterialInput& Input, bool bHidden, FMaterialInputDescription& OutDescription)
{
	OutDescription.Type = UE::Shader::EValueType::Struct;
	OutDescription.Input = &Input;
	OutDescription.bUseConstant = false;
	OutDescription.bHidden = bHidden;
}

bool UMaterial::GetExpressionInputDescription(EMaterialProperty InProperty, FMaterialInputDescription& OutDescription)
{
	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();
	switch (InProperty)
	{
	case MP_EmissiveColor: SetMaterialInputDescription(EditorOnly->EmissiveColor, false, OutDescription); return true;
	case MP_Opacity: SetMaterialInputDescription(EditorOnly->Opacity, false, OutDescription); return true;
	case MP_OpacityMask: SetMaterialInputDescription(EditorOnly->OpacityMask, false, OutDescription); return true;
	case MP_BaseColor: SetMaterialInputDescription(EditorOnly->BaseColor, false, OutDescription); return true;
	case MP_Metallic: SetMaterialInputDescription(EditorOnly->Metallic, false, OutDescription); return true;
	case MP_Specular: SetMaterialInputDescription(EditorOnly->Specular, false, OutDescription); return true;
	case MP_Roughness: SetMaterialInputDescription(EditorOnly->Roughness, false, OutDescription); return true;
	case MP_Anisotropy: SetMaterialInputDescription(EditorOnly->Anisotropy, false, OutDescription); return true;
	case MP_Normal: SetMaterialInputDescription(EditorOnly->Normal, false, OutDescription); return true;
	case MP_Tangent: SetMaterialInputDescription(EditorOnly->Tangent, false, OutDescription); return true;
	case MP_WorldPositionOffset: SetMaterialInputDescription(EditorOnly->WorldPositionOffset, false, OutDescription); return true;
	case MP_Displacement: SetMaterialInputDescription(EditorOnly->Displacement, false, OutDescription); return true;
	case MP_SubsurfaceColor: SetMaterialInputDescription(EditorOnly->SubsurfaceColor, false, OutDescription); return true;
	case MP_CustomData0: SetMaterialInputDescription(EditorOnly->ClearCoat, false, OutDescription); return true;
	case MP_CustomData1: SetMaterialInputDescription(EditorOnly->ClearCoatRoughness, false, OutDescription); return true;
	case MP_AmbientOcclusion: SetMaterialInputDescription(EditorOnly->AmbientOcclusion, false, OutDescription); return true;
	case MP_Refraction: SetMaterialInputDescription(EditorOnly->Refraction, false, OutDescription); return true;
	case MP_MaterialAttributes: SetMaterialInputDescription(EditorOnly->MaterialAttributes, false, OutDescription); return true;
	case MP_PixelDepthOffset: SetMaterialInputDescription(EditorOnly->PixelDepthOffset, false, OutDescription); return true;
	case MP_ShadingModel: SetMaterialInputDescription(EditorOnly->ShadingModelFromMaterialExpression, false, OutDescription); return true;
	case MP_SurfaceThickness: SetMaterialInputDescription(EditorOnly->SurfaceThickness, false, OutDescription); return true;
	case MP_FrontMaterial: SetMaterialInputDescription(EditorOnly->FrontMaterial, false, OutDescription); return true;
	default:
		if (InProperty >= MP_CustomizedUVs0 && InProperty <= MP_CustomizedUVs7)
		{
			SetMaterialInputDescription(EditorOnly->CustomizedUVs[InProperty - MP_CustomizedUVs0], false, OutDescription);
			return true;
		}
		return false;
	}
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UMaterial::GetAllFunctionOutputExpressions(TArray<class UMaterialExpressionFunctionOutput*>& OutFunctionOutputs) const
{
	for (UMaterialExpression* Expression : GetExpressions())
	{
		UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);
		if (FunctionOutput)
		{
			OutFunctionOutputs.Add(FunctionOutput);
		}
	}
}

void UMaterial::GetAllCustomOutputExpressions(TArray<class UMaterialExpressionCustomOutput*>& OutCustomOutputs) const
{
	for (UMaterialExpression* Expression : GetExpressions())
	{
		UMaterialExpressionCustomOutput* CustomOutput = Cast<UMaterialExpressionCustomOutput>(Expression);
		if (CustomOutput)
		{
			OutCustomOutputs.Add(CustomOutput);
		}
	}
}

void UMaterial::GetAllExpressionsForCustomInterpolators(TArray<class UMaterialExpression*>& OutExpressions) const
{
	for (UMaterialExpression* Expression : GetExpressions())
	{
		if (Expression &&
			(Expression->IsA(UMaterialExpressionVertexInterpolator::StaticClass()) ||
			Expression->IsA(UMaterialExpressionMaterialFunctionCall::StaticClass()) ||
			Expression->IsA(UMaterialExpressionMaterialAttributeLayers::StaticClass())) )
		{
				OutExpressions.Add(Expression);
		}
	}
}

bool UMaterial::SupportsShadingModelOverride() const
{
	// If the material contains a Substrate's SubstrateShadingModels node, then we can support shading model override.
	// That is when SubstrateShadingModels nodes are used. And only if hidden conversion is not enabled, otherwise we always display it.
	bool Out = true;
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsHiddenMaterialAssetConversionEnabled())
	{
		for (UMaterialExpression* Expression : GetExpressions())
		{		
			if (UMaterialExpressionSubstrateShadingModels* ShadingModelNode = Cast<UMaterialExpressionSubstrateShadingModels>(Expression))
			{
				return true;
			}
		}
		Out = false;
	}
	return Out;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
bool UMaterial::GetAllReferencedExpressions(TArray<UMaterialExpression*>& OutExpressions, struct FStaticParameterSet* InStaticParameterSet,
	ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality, ERHIShadingPath::Type InShadingPath, const bool bInRecurseIntoMaterialFunctions, TSet<UClass*>* InMobileCustomOutputExpressionTypesToQuery)
{
	using namespace MaterialImpl;

	TSet<UMaterialExpression*> TmpExpressions;
	OutExpressions.Empty();

	// For mobile only consider nodes connected to material properties that affect mobile, and exclude any custom outputs
	if (InFeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		if (bUseMaterialAttributes)
		{
			TArray<UMaterialExpression*> MPRefdExpressions;
			if (GetExpressionsInPropertyChain(MP_MaterialAttributes, MPRefdExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, bInRecurseIntoMaterialFunctions) == true)
			{
				for (int32 AddIdx = 0; AddIdx < MPRefdExpressions.Num(); AddIdx++)
				{
					TmpExpressions.Add(MPRefdExpressions[AddIdx]);
				}
			}
		}
		else
		{
			for (FMobileProperty MobileProperty : GMobileRelevantMaterialProperties)
			{
				EMaterialProperty MaterialProp = MobileProperty.Property;
				TArray<UMaterialExpression*> MPRefdExpressions;
				if (GetExpressionsInPropertyChain(MaterialProp, MPRefdExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, bInRecurseIntoMaterialFunctions) == true)
				{
					for (int32 AddIdx = 0; AddIdx < MPRefdExpressions.Num(); AddIdx++)
					{
						TmpExpressions.Add(MPRefdExpressions[AddIdx]);
					}
				}
			}
		}

		// TODO: Need an actual ShaderPlatform for a more precise result. This just grabs the handiest one on the current machine.
		EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(ERHIFeatureLevel::ES3_1);
		
		bool bMobileUseVirtualTexturing = UseVirtualTexturing(ShaderPlatform);
		if (bMobileUseVirtualTexturing || (InMobileCustomOutputExpressionTypesToQuery && InMobileCustomOutputExpressionTypesToQuery->Num() > 0))
		{
			TArray<class UMaterialExpressionCustomOutput*> CustomOutputExpressions;
			GetAllCustomOutputExpressions(CustomOutputExpressions);
			for (UMaterialExpressionCustomOutput* Expression : CustomOutputExpressions)
			{
				if ((bMobileUseVirtualTexturing && Expression->IsA<UMaterialExpressionRuntimeVirtualTextureOutput>()) ||
					(InMobileCustomOutputExpressionTypesToQuery && InMobileCustomOutputExpressionTypesToQuery->Contains(Expression->GetClass())))
				{
					TSet<FExpressionInput*> ProcessedInputs;
					RecursiveGetExpressionChain(Expression, ProcessedInputs, TmpExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, SF_NumFrequencies, MP_MAX, bInRecurseIntoMaterialFunctions);
				}
			}
		}
	}
	else
	{
	    for (int32 MPIdx = 0; MPIdx < MP_MAX; MPIdx++)
	    {
		    EMaterialProperty MaterialProp = EMaterialProperty(MPIdx);
		    TArray<UMaterialExpression*> MPRefdExpressions;
			if (GetExpressionsInPropertyChain(MaterialProp, MPRefdExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, bInRecurseIntoMaterialFunctions) == true)
			{
			    for (int32 AddIdx = 0; AddIdx < MPRefdExpressions.Num(); AddIdx++)
			    {
				    TmpExpressions.Add(MPRefdExpressions[AddIdx]);
			    }
		    }
	    }
    
	    TArray<class UMaterialExpressionCustomOutput*> CustomOutputExpressions;
	    GetAllCustomOutputExpressions(CustomOutputExpressions);
	    for (UMaterialExpressionCustomOutput* Expression : CustomOutputExpressions)
	    {
			TSet<FExpressionInput*> ProcessedInputs;
			RecursiveGetExpressionChain(Expression, ProcessedInputs, TmpExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, SF_NumFrequencies, MP_MAX, bInRecurseIntoMaterialFunctions);
		}

		// If this is a material function, we want to also trace function outputs
		TArray<class UMaterialExpressionFunctionOutput*> FunctionOutputExpressions;
		GetAllFunctionOutputExpressions(FunctionOutputExpressions);
		for (UMaterialExpressionFunctionOutput* Expression : FunctionOutputExpressions)
		{
			TSet<FExpressionInput*> ProcessedInputs;
			RecursiveGetExpressionChain(Expression, ProcessedInputs, TmpExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, SF_NumFrequencies, MP_MAX, bInRecurseIntoMaterialFunctions);
		}
	}


	OutExpressions = TmpExpressions.Array();

	return true;
}


bool UMaterial::GetExpressionsInPropertyChain(EMaterialProperty InProperty, 
	TArray<UMaterialExpression*>& OutExpressions, struct FStaticParameterSet* InStaticParameterSet,
	ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality, ERHIShadingPath::Type InShadingPath, const bool bInRecurseIntoMaterialFunctions)
{
	TSet<UMaterialExpression*> TmpExpressions;
	OutExpressions.Empty();
	FExpressionInput* StartingExpression = GetExpressionInputForProperty(InProperty);

	if (StartingExpression == NULL)
	{
		// Failed to find the starting expression
		return false;
	}

	TSet<FExpressionInput*> ProcessedInputs;
	if (StartingExpression->Expression)
	{
		ProcessedInputs.Add(StartingExpression);
		
		EShaderFrequency ShaderFrequency = SF_NumFrequencies;
		// These properties are "special", attempting to pass them to FMaterialAttributeDefinitionMap::GetShaderFrequency() will generate log spam
		if (!(InProperty == MP_MaterialAttributes || InProperty == MP_CustomOutput))
		{
			ShaderFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(InProperty);
		}

		RecursiveGetExpressionChain(StartingExpression->Expression, ProcessedInputs, TmpExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, ShaderFrequency, InProperty, bInRecurseIntoMaterialFunctions);
	}
	OutExpressions = TmpExpressions.Array();
	return true;
}

bool UMaterial::GetGroupSortPriority(const FString& InGroupName, int32& OutSortPriority) const
{
	const UMaterialEditorOnlyData* LocalData = GetEditorOnlyData();
	if (LocalData)
	{
		const FParameterGroupData* ParameterGroupDataElement = LocalData->ParameterGroupData.FindByPredicate([&InGroupName](const FParameterGroupData& DataElement)
		{
			return InGroupName == DataElement.GroupName;
		});
		if (ParameterGroupDataElement != nullptr)
		{
			OutSortPriority = ParameterGroupDataElement->GroupSortPriority;
			return true;
		}
	}
	return false;
}

bool UMaterial::GetTexturesInPropertyChain(EMaterialProperty InProperty, TArray<UTexture*>& OutTextures,
	TArray<FName>* OutTextureParamNames, struct FStaticParameterSet* InStaticParameterSet,
	ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality)
{
	TArray<UMaterialExpression*> ChainExpressions;
	if (GetExpressionsInPropertyChain(InProperty, ChainExpressions, InStaticParameterSet, InFeatureLevel, InQuality) == true)
	{
		// Extract the texture and texture parameter expressions...
		for (int32 ExpressionIdx = 0; ExpressionIdx < ChainExpressions.Num(); ExpressionIdx++)
		{
			UMaterialExpression* MatExp = ChainExpressions[ExpressionIdx];
			if (MatExp != NULL)
			{
				// Is it a texture sample or texture parameter sample?
				UMaterialExpressionTextureBase* TextureSampleExp = Cast<UMaterialExpressionTextureBase>(MatExp);
				if (TextureSampleExp != NULL)
				{
					// Check the default texture...
					if (TextureSampleExp->Texture != NULL)
					{
						OutTextures.Add(TextureSampleExp->Texture);
					}

					if (OutTextureParamNames != NULL)
					{
						// If the expression is a parameter, add it's name to the texture names array
						UMaterialExpressionTextureSampleParameter* TextureSampleParamExp = Cast<UMaterialExpressionTextureSampleParameter>(MatExp);
						if (TextureSampleParamExp != NULL)
						{
							OutTextureParamNames->AddUnique(TextureSampleParamExp->ParameterName);
						}
					}
				}
			}
		}
	
		return true;
	}

	return false;
}

bool UMaterial::RecursiveGetExpressionChain(
	UMaterialExpression* InExpression, TSet<FExpressionInput*>& InOutProcessedInputs, 
	TSet<UMaterialExpression*>& OutExpressions, struct FStaticParameterSet* InStaticParameterSet,
	ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality, ERHIShadingPath::Type InShadingPath, EShaderFrequency InShaderFrequency, EMaterialProperty InProperty, const bool bInRecurseIntoMaterialFunctions)
{
	using namespace MaterialImpl;

	OutExpressions.Add(InExpression);
	TArray<FExpressionInput*, TInlineAllocator<8>> Inputs;
	TArray<EShaderFrequency, TInlineAllocator<8>> InputsFrequency;
	
	UMaterialExpressionFeatureLevelSwitch* FeatureLevelSwitchExp;
	UMaterialExpressionQualitySwitch* QualitySwitchExp;
	UMaterialExpressionShadingPathSwitch* ShadingPathSwitchExp;
	UMaterialExpressionMakeMaterialAttributes* MakeMaterialAttributesExp;
	UMaterialExpressionSetMaterialAttributes* SetMaterialAttributesExp;
	UMaterialExpressionShaderStageSwitch* ShaderStageSwitchExp;
	UMaterialExpressionNamedRerouteUsage* RerouteUsageExp;
	UMaterialExpressionMaterialFunctionCall* MaterialFunctionCallExp;

	const bool bMobileOnly = InFeatureLevel <= ERHIFeatureLevel::ES3_1;

	// special case the various switch nodes to only follow the currently active path
	if (InFeatureLevel != ERHIFeatureLevel::Num && (FeatureLevelSwitchExp = Cast<UMaterialExpressionFeatureLevelSwitch>(InExpression)) != nullptr)
	{
		if (FeatureLevelSwitchExp->Inputs[InFeatureLevel].IsConnected())
		{
			Inputs.Add(&FeatureLevelSwitchExp->Inputs[InFeatureLevel]);
			InputsFrequency.Add(InShaderFrequency);
		}
		else
		{
			Inputs.Add(&FeatureLevelSwitchExp->Default);
			InputsFrequency.Add(InShaderFrequency);
		}
	}
	else if (InQuality != EMaterialQualityLevel::Num && (QualitySwitchExp = Cast<UMaterialExpressionQualitySwitch>(InExpression)) != nullptr)
	{
		if (QualitySwitchExp->Inputs[InQuality].IsConnected())
		{
			Inputs.Add(&QualitySwitchExp->Inputs[InQuality]);
			InputsFrequency.Add(InShaderFrequency);
		}
		else
		{
			Inputs.Add(&QualitySwitchExp->Default);
			InputsFrequency.Add(InShaderFrequency);
		}
	}
	else if (InShadingPath != ERHIShadingPath::Num && (ShadingPathSwitchExp = Cast<UMaterialExpressionShadingPathSwitch>(InExpression)) != nullptr)
	{
		if (ShadingPathSwitchExp->Inputs[InShadingPath].IsConnected())
		{
			Inputs.Add(&ShadingPathSwitchExp->Inputs[InShadingPath]);
			InputsFrequency.Add(InShaderFrequency);
		}
		else
		{
			Inputs.Add(&ShadingPathSwitchExp->Default);
			InputsFrequency.Add(InShaderFrequency);
		}
	}
	else if (InShaderFrequency != SF_NumFrequencies && (ShaderStageSwitchExp = Cast<UMaterialExpressionShaderStageSwitch>(InExpression)) != nullptr)
	{
		if (UMaterialExpressionShaderStageSwitch::ShouldUsePixelShaderInput(InShaderFrequency))
		{
			Inputs.Add(&ShaderStageSwitchExp->PixelShader);
			InputsFrequency.Add(InShaderFrequency);
		}
		else
		{
			Inputs.Add(&ShaderStageSwitchExp->VertexShader);
			InputsFrequency.Add(InShaderFrequency);
		}
	}
	else if ((MakeMaterialAttributesExp = Cast<UMaterialExpressionMakeMaterialAttributes>(InExpression)) != nullptr)
	{
		// MP_MAX means we want to follow all properties
		// MP_MaterialAttributes means we want to follow properties that are part of the material attributes, which, in the case of UMaterialExpressionMakeMaterialAttributes, means: every one of them
		if ((InProperty == MP_MAX) || (InProperty == MP_MaterialAttributes))
		{
			// If we're requested to return only the properties relevant for mobile, only follow up those: 
			if (bMobileOnly)
			{
				for (FMobileProperty MobileProperty : GMobileRelevantMaterialProperties)
				{
					FExpressionInput* Input = MakeMaterialAttributesExp->GetExpressionInput(MobileProperty.Property);
					if (Input != nullptr)
					{
						Inputs.Add(Input);
						InputsFrequency.Add(MobileProperty.Frequency);
					}
				}
			}
			else
			{
				// Follow all properties.
				Inputs.Empty(InExpression->CountInputs());
				for (FExpressionInputIterator It{ InExpression }; It; ++It)
				{
					Inputs.Push(It.Input);
				}

				InputsFrequency.Init(InShaderFrequency, Inputs.Num());
			}
		}
		else
		{
			// Only follow the specified InProperty.
			FExpressionInput* Input = MakeMaterialAttributesExp->GetExpressionInput(InProperty);
			if ((Input != nullptr) 
				&& (!bMobileOnly || IsPropertyRelevantForMobile(InProperty)))
			{
				Inputs.Add(Input);
				InputsFrequency.Add(InShaderFrequency);
			}
		}
	}
	else if ((SetMaterialAttributesExp = Cast<UMaterialExpressionSetMaterialAttributes>(InExpression)) != nullptr)
	{
		checkf(SetMaterialAttributesExp->GetInput(0) && (SetMaterialAttributesExp->GetInputValueType(0) == MCT_MaterialAttributes), TEXT("There must always be one input at least : the material attribute pin"));
		// Always add the material attribute input, so that we keep on traversing up the property chain : 
		Inputs.Add(SetMaterialAttributesExp->GetInput(0));
		InputsFrequency.Add(InShaderFrequency);

		TArray<EMaterialProperty> AttributeProperties;
		AttributeProperties.Reserve(SetMaterialAttributesExp->AttributeSetTypes.Num());
		Algo::Transform(SetMaterialAttributesExp->AttributeSetTypes, AttributeProperties, [](const FGuid& InAttributeID) { return FMaterialAttributeDefinitionMap::GetProperty(InAttributeID); });
		
		// MP_MAX means we want to follow all properties
		// MP_MaterialAttributes means we want to follow properties that are part of the material attributes, which, in the case of UMaterialExpressionSetMaterialAttributes, means: every one of them
		if ((InProperty == MP_MAX) || (InProperty == MP_MaterialAttributes))
		{
			// If we're requested to return only the properties relevant for mobile, only follow up those: 
			if (bMobileOnly)
			{
				for (FMobileProperty MobileProperty : GMobileRelevantMaterialProperties)
				{
					int32 FoundAttributeIndex = INDEX_NONE;
					if (AttributeProperties.Find(MobileProperty.Property, FoundAttributeIndex))
					{
						FExpressionInput* AttributeInput = SetMaterialAttributesExp->GetInput(FoundAttributeIndex + 1); // Need +1 for MaterialAttributes input pin.
						Inputs.Add(AttributeInput);
						InputsFrequency.Add(MobileProperty.Frequency);
					}
				}
			}
			else
			{
				// Follow all properties.
				Inputs.Empty(InExpression->CountInputs());
				for (FExpressionInputIterator It{ InExpression }; It; ++It)
				{
					Inputs.Push(It.Input);
				}
				InputsFrequency.Init(InShaderFrequency, Inputs.Num());
			}
		}
		else
		{
			// Only follow the specified InProperty.
			int32 FoundAttributeIndex = INDEX_NONE;
			if (AttributeProperties.Find(InProperty, FoundAttributeIndex))
			{
				FExpressionInput* AttributeInput = SetMaterialAttributesExp->GetInput(FoundAttributeIndex + 1); // Need +1 for MaterialAttributes input pin.
				if ((AttributeInput != nullptr)
					&& (!bMobileOnly || IsPropertyRelevantForMobile(InProperty)))
				{ 
					Inputs.Add(AttributeInput);
					InputsFrequency.Add(InShaderFrequency);
				}
			}
		}
	}
	else if ((RerouteUsageExp = Cast<UMaterialExpressionNamedRerouteUsage>(InExpression)) != nullptr)
	{
		// continue searching from the reroute declaration
		if (RerouteUsageExp->Declaration != nullptr)
		{
			RecursiveGetExpressionChain(RerouteUsageExp->Declaration, InOutProcessedInputs, OutExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, InShaderFrequency, InProperty, bInRecurseIntoMaterialFunctions);
		}
	}
	else if (bInRecurseIntoMaterialFunctions && (MaterialFunctionCallExp = Cast<UMaterialExpressionMaterialFunctionCall>(InExpression)) != nullptr)
	{
		// NOTE: we do the simple thing here, and assume that ALL of the MaterialFunction outputs are connected,
		// and that all of the inputs feed into each output.  This simplifies the problem enough that we don't need
		// to build another structure to track processed inputs within each MaterialFunctionCall separately.
		// However, it does mean that we may add some expression nodes (from inside MaterialFunction) to OutExpressions,
		// that would not be considered active by an analysis that traces actual connections into the MaterialFunction
		UMaterialFunctionInterface* MFI = MaterialFunctionCallExp->MaterialFunction.Get();
		if (MFI != nullptr)
		{
			TArray<FFunctionExpressionInput> FuncInputs;
			TArray<FFunctionExpressionOutput> FuncOutputs;
			MFI->GetInputsAndOutputs(FuncInputs, FuncOutputs);

			// here we assume ALL outputs of the MaterialFunctionCall are active
			for (FFunctionExpressionOutput& Out : FuncOutputs)
			{
				if (Out.ExpressionOutput)
				{
					Inputs.Add(&Out.ExpressionOutput->A);
					InputsFrequency.Add(InShaderFrequency);
				}
			}
		}

		// here we assume ALL inputs to the MaterialFunctionCall are active
		Inputs.Reserve(Inputs.Num() + InExpression->CountInputs());
		for (FExpressionInputIterator It{ InExpression }; It; ++It)
		{
			Inputs.Push(It.Input);
			InputsFrequency.Push(InShaderFrequency);
		}
	}
	else
	{
		Inputs.Empty(InExpression->CountInputs());
		for (FExpressionInputIterator It{ InExpression }; It; ++It)
		{
			Inputs.Push(It.Input);
		}

		InputsFrequency.Init(InShaderFrequency, Inputs.Num());
	}

	check(Inputs.Num() == InputsFrequency.Num());

	for (int32 InputIdx = 0; InputIdx < Inputs.Num(); InputIdx++)
	{
		FExpressionInput* InnerInput = Inputs[InputIdx];
		if (InnerInput != NULL)
		{
			if (!InOutProcessedInputs.Contains(InnerInput))
			{
				if (InnerInput->Expression)
				{
					bool bProcessInput = true;
					if (InStaticParameterSet != NULL)
					{
						// By default, static switches use B...
						// Is this a static switch parameter?
						//@todo. Handle Terrain weight map layer expression here as well!
						UMaterialExpressionStaticSwitchParameter* StaticSwitchExp = Cast<UMaterialExpressionStaticSwitchParameter>(InExpression);
						if (StaticSwitchExp != NULL)
						{
							bool bUseInputA = StaticSwitchExp->DefaultValue;
							FName StaticSwitchExpName = StaticSwitchExp->ParameterName;
							for (int32 CheckIdx = 0; CheckIdx < InStaticParameterSet->StaticSwitchParameters.Num(); CheckIdx++)
							{
								FStaticSwitchParameter& SwitchParam = InStaticParameterSet->StaticSwitchParameters[CheckIdx];
								if (SwitchParam.ParameterInfo.Name == StaticSwitchExpName)
								{
									// Found it...
									if (SwitchParam.bOverride == true)
									{
										bUseInputA = SwitchParam.Value;
										break;
									}
								}
							}

							if (bUseInputA == true)
							{
								if (InnerInput->Expression != StaticSwitchExp->A.Expression)
								{
									bProcessInput = false;
								}
							}
							else
							{
								if (InnerInput->Expression != StaticSwitchExp->B.Expression)
								{
									bProcessInput = false;
								}
							}
						}
					}

					if (bProcessInput == true)
					{
						InOutProcessedInputs.Add(InnerInput);
						RecursiveGetExpressionChain(InnerInput->Expression, InOutProcessedInputs, OutExpressions, InStaticParameterSet, InFeatureLevel, InQuality, InShadingPath, InputsFrequency[InputIdx], InProperty, bInRecurseIntoMaterialFunctions);
					}
				}
			}
		}
	}

	return true;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterial::RecursiveUpdateRealtimePreview( UMaterialExpression* InExpression, TArray<UMaterialExpression*>& InOutExpressionsToProcess )
{
	// remove ourselves from the list to process
	InOutExpressionsToProcess.Remove(InExpression);

	bool bOldRealtimePreview = InExpression->bRealtimePreview;

	// See if we know ourselves if we need realtime preview or not.
	InExpression->bRealtimePreview = InExpression->NeedsRealtimePreview();

	if( InExpression->bRealtimePreview )
	{
		if( InExpression->bRealtimePreview != bOldRealtimePreview )
		{
			InExpression->bNeedToUpdatePreview = true;
		}

		return;		
	}

	// We need to examine our inputs. If any of them need realtime preview, so do we.
	for (FExpressionInputIterator It{ InExpression }; It; ++It)
	{
		if (It.Input != NULL && It.Input->Expression != NULL)
		{
			// See if we still need to process this expression, and if so do that first.
			if (InOutExpressionsToProcess.Find(It.Input->Expression) != INDEX_NONE)
			{
				RecursiveUpdateRealtimePreview(It.Input->Expression, InOutExpressionsToProcess);
			}

			// If our input expression needed realtime preview, we do too.
			if( It.Input->Expression->bRealtimePreview )
			{

				InExpression->bRealtimePreview = true;
				if( InExpression->bRealtimePreview != bOldRealtimePreview )
				{
					InExpression->bNeedToUpdatePreview = true;
				}
				return;		
			}
		}
	}

	if( InExpression->bRealtimePreview != bOldRealtimePreview )
	{
		InExpression->bNeedToUpdatePreview = true;
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterial::CompilePropertyEx( FMaterialCompiler* Compiler, const FGuid& AttributeID )
{
	const EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(AttributeID);
	UMaterialEditorOnlyData* EditorOnly = GetEditorOnlyData();

	if( bUseMaterialAttributes && MP_DiffuseColor != Property && MP_SpecularColor != Property && MP_FrontMaterial != Property)
	{
		return EditorOnly->MaterialAttributes.CompileWithDefault(Compiler, AttributeID);
	}

	switch (Property)
	{
		case MP_Opacity:				return EditorOnly->Opacity.CompileWithDefault(Compiler, Property);
		case MP_OpacityMask:			return EditorOnly->OpacityMask.CompileWithDefault(Compiler, Property);
		case MP_Metallic:				return EditorOnly->Metallic.CompileWithDefault(Compiler, Property);
		case MP_Specular:				return EditorOnly->Specular.CompileWithDefault(Compiler, Property);
		case MP_Roughness:				return EditorOnly->Roughness.CompileWithDefault(Compiler, Property);
		case MP_Anisotropy:				return EditorOnly->Anisotropy.CompileWithDefault(Compiler, Property);
		case MP_CustomData0:			return EditorOnly->ClearCoat.CompileWithDefault(Compiler, Property);
		case MP_CustomData1:			return EditorOnly->ClearCoatRoughness.CompileWithDefault(Compiler, Property);
		case MP_AmbientOcclusion:		return EditorOnly->AmbientOcclusion.CompileWithDefault(Compiler, Property);
		case MP_Refraction:				return EditorOnly->Refraction.CompileWithDefault(Compiler, Property);
		case MP_EmissiveColor:			return EditorOnly->EmissiveColor.CompileWithDefault(Compiler, Property);
		case MP_BaseColor:				return EditorOnly->BaseColor.CompileWithDefault(Compiler, Property);
		case MP_SubsurfaceColor:		return EditorOnly->SubsurfaceColor.CompileWithDefault(Compiler, Property);
		case MP_Tangent:				return EditorOnly->Tangent.CompileWithDefault(Compiler, Property);
		case MP_WorldPositionOffset:	return EditorOnly->WorldPositionOffset.CompileWithDefault(Compiler, Property);
		case MP_Displacement:			return EditorOnly->Displacement.CompileWithDefault(Compiler, Property);
		case MP_PixelDepthOffset:		return EditorOnly->PixelDepthOffset.CompileWithDefault(Compiler, Property);
		case MP_ShadingModel:			return EditorOnly->ShadingModelFromMaterialExpression.CompileWithDefault(Compiler, Property);
		case MP_SurfaceThickness:		return EditorOnly->SurfaceThickness.CompileWithDefault(Compiler, Property);
		case MP_FrontMaterial:			return EditorOnly->FrontMaterial.CompileWithDefault(Compiler, Property);

		case MP_Normal:
			// If material is not using tangent space normals, and nothing is connected to the normal pin,
			// instead of setting the normal to the constant use the vertex normal.
			if (!this->bTangentSpaceNormal && EditorOnly->Normal.UseConstant)
			{
				return Compiler->VertexNormal();
			}
			return EditorOnly->Normal.CompileWithDefault(Compiler, Property);

		default:
			if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
			{
				const int32 TextureCoordinateIndex = Property - MP_CustomizedUVs0;

				if (TextureCoordinateIndex < NumCustomizedUVs && EditorOnly->CustomizedUVs[TextureCoordinateIndex].Expression)
				{
					return EditorOnly->CustomizedUVs[TextureCoordinateIndex].CompileWithDefault(Compiler, Property);
				}
				else
				{
					// The user did not customize this UV, pass through the vertex texture coordinates
					return Compiler->TextureCoordinate(TextureCoordinateIndex, false, false);
				}
			}
		
	}

	check(0);
	return INDEX_NONE;
}

bool UMaterial::ShouldForcePlanePreview()
{
	const USceneThumbnailInfoWithPrimitive* MaterialThumbnailInfo = Cast<USceneThumbnailInfoWithPrimitive>(ThumbnailInfo);
	if (!MaterialThumbnailInfo)
	{
		MaterialThumbnailInfo = USceneThumbnailInfoWithPrimitive::StaticClass()->GetDefaultObject<USceneThumbnailInfoWithPrimitive>();
	}
	// UI and particle sprite material thumbnails always get a 2D plane centered at the camera which is a better representation of the what the material will look like
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bUsedWithNiagara = bUsedWithNiagaraSprites || bUsedWithNiagaraRibbons || bUsedWithNiagaraMeshParticles;
	return Super::ShouldForcePlanePreview() || IsUIMaterial() || (bUsedWithParticleSprites && !MaterialThumbnailInfo->bUserModifiedShape) || (bUsedWithNiagara && !MaterialThumbnailInfo->bUserModifiedShape);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UMaterial::NotifyCompilationFinished(UMaterialInterface* Material)
{
	UMaterial::OnMaterialCompilationFinished().Broadcast(Material);
}

void UMaterial::ForceRecompileForRendering(EMaterialShaderPrecompileMode CompileMode)
{
	UpdateCachedExpressionData();
	CacheResourceShadersForRendering(false,  CompileMode);
}

#if WITH_EDITOR
bool UMaterial::TryGetUsedTexturesInCookCache(TArray<TObjectPtr<UObject>>& OutTextures)
{
	OutTextures.Empty();
	const FMaterialCachedExpressionData& UsedCachedData = GetCachedExpressionData();
	TArray<const FMaterialResource*, TInlineAllocator<4>> MatchedResources = GetCookedCacheMaterialResources();

	for (const FMaterialResource* CurrentResource : MatchedResources)
	{
		const FUniformExpressionSet& CurrentSet = CurrentResource->GetUniformExpressions();

		for (int32 TypeIdx = 0; TypeIdx < NumMaterialTextureParameterTypes; ++TypeIdx)
		{
			for (int32 TextureIdx = 0; TextureIdx < CurrentSet.GetNumTextures((EMaterialTextureParameterType)TypeIdx); ++TextureIdx)
			{
				const FMaterialTextureParameterInfo& Parameter = CurrentSet.GetTextureParameter((EMaterialTextureParameterType)TypeIdx, TextureIdx);
				OutTextures.AddUnique(UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures[Parameter.TextureIndex]);
			}
		}
	}

	const bool bValidCompilationOutput = MatchedResources.Num() > 0;
	return bValidCompilationOutput;
}
#endif

bool UMaterial::CheckInValidStateForCompilation(class FMaterialCompiler* Compiler) const
{
	bool bSuccess = true;
	
	// Check and report errors due to duplicate parameters set to distinct values.
	for (const auto& Param : CachedExpressionData->ParameterExpressionsAndValues)
	{
		const FString ParamName = Param.Key.ToString();
		const FMaterialCachedExpressionData::FParamExpressionsValues& ParameterExpressionsValues = Param.Value;
		if (ParameterExpressionsValues.bHasDuplicatesWithDifferentValues)
		{
			bSuccess = false;
			for (const auto& ExpressionsValues : ParameterExpressionsValues.ExpressionsValues)
			{
				UMaterialExpression* Expression = ExpressionsValues.Get<0>();
				const FString& ParamValue = ExpressionsValues.Get<1>();

				FString ErrorMessage = FString::Printf(TEXT("Parameter %s with default value %s from asset %s has duplicates with different default values."),
					*ParamName, *ParamValue, *Expression->GetAssetPathName());

				Compiler->AppendExpressionError(Expression,  *ErrorMessage);
			}
		}
	}
	return bSuccess;
}

UMaterial::FMaterialCompilationFinished UMaterial::MaterialCompilationFinishedEvent;
UMaterial::FMaterialCompilationFinished& UMaterial::OnMaterialCompilationFinished()
{
	return MaterialCompilationFinishedEvent;
}
#endif // WITH_EDITOR

void UMaterial::AllMaterialsCacheResourceShadersForRendering(bool bUpdateProgressDialog, bool bCacheAllRemainingShaders)
{
#if WITH_EDITOR
	FScopedSlowTask SlowTask(100.f, NSLOCTEXT("Engine", "CacheMaterialShadersMessage", "Caching material shaders"), true);
	if (bUpdateProgressDialog)
	{
		SlowTask.Visibility = ESlowTaskVisibility::ForceVisible;
		SlowTask.MakeDialog();
	}
#endif // WITH_EDITOR

	TArray<UObject*> MaterialArray;
	GetObjectsOfClass(UMaterial::StaticClass(), MaterialArray, true, RF_ClassDefaultObject, EInternalObjectFlags::None);
	float TaskIncrement = (float)100.0f / MaterialArray.Num();

	// ensure default materials are cached first. Default materials must be available to fallback to during async compile.
 	MaterialArray.Sort([](const UObject& L, const UObject& R)
 	{
 		return ((const UMaterial&)L).IsDefaultMaterial() > ((const UMaterial&)R).IsDefaultMaterial();
	});

	for (UObject* MaterialObj : MaterialArray)
	{
		UMaterial* Material = (UMaterial*)MaterialObj;

		Material->CacheResourceShadersForRendering(false, bCacheAllRemainingShaders ? EMaterialShaderPrecompileMode::Default : EMaterialShaderPrecompileMode::None);

#if WITH_EDITOR
		if (bUpdateProgressDialog)
		{
			SlowTask.EnterProgressFrame(TaskIncrement);
		}
#endif // WITH_EDITOR
	}
}

/**
 * Lists all materials that read from scene color.
 */
static void ListSceneColorMaterials()
{
	int32 NumSceneColorMaterials = 0;

	UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type FeatureLevel) 
	{
		FString FeatureLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);

		for (TObjectIterator<UMaterialInterface> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterialInterface* Mat = *It;
			const FMaterial* MatRes = Mat->GetRenderProxy()->GetMaterialNoFallback(FeatureLevel);
			if (MatRes && MatRes->RequiresSceneColorCopy_GameThread())
			{
				UMaterial* BaseMat = Mat->GetMaterial();
				UE_LOGF(LogConsoleResponse, Display, "[TransPass=%d][FeatureLevel=%ls] %ls",
					BaseMat ? (int32)BaseMat->TranslucencyPass : (int32)MTP_MAX,
					*FeatureLevelName,
					*Mat->GetPathName()
					);
				NumSceneColorMaterials++;
			}
		}
	});
	UE_LOGF(LogConsoleResponse,Display,"%d loaded materials read from scene color.",NumSceneColorMaterials);
}

static FAutoConsoleCommand CmdListSceneColorMaterials(
	TEXT("r.ListSceneColorMaterials"),
	TEXT("Lists all materials that read from scene color."),
	FConsoleCommandDelegate::CreateStatic(ListSceneColorMaterials)
	);

float UMaterial::GetOpacityMaskClipValue() const
{
	return OpacityMaskClipValue;
}

bool UMaterial::GetCastDynamicShadowAsMasked() const
{
	return bCastDynamicShadowAsMasked;
}

EBlendMode UMaterial::GetBlendMode() const
{
	if (EBlendMode(BlendMode) == BLEND_Masked)
	{
		if (bCanMaskedBeAssumedOpaque)
		{
			return BLEND_Opaque;
		}
		else
		{
			return BLEND_Masked;
		}
	}
	else
	{
		if (GetShadingModels() == MSM_ThinTranslucent && !HasSubstrateFrontMaterialConnected() && Substrate::IsHiddenMaterialAssetConversionEnabled())
		{
			// Set the correct blend mode when using Substrate hidden conversion for thin translucent materials. Only when the FrontMaterial is not pluggedin.
			return BLEND_TranslucentColoredTransmittance;
		}
		return BlendMode;
	}
}

FMaterialShadingModelField UMaterial::GetShadingModels() const
{
	switch (MaterialDomain)
	{
		case MD_Surface:
		case MD_Volume:
			return ShadingModels;
		case MD_DeferredDecal:
		case MD_RuntimeVirtualTexture:
			return MSM_DefaultLit;

		// Post process and light function materials must be rendered with the unlit model.
		case MD_PostProcess:
		case MD_LightFunction:
		case MD_UI:
			return MSM_Unlit;

		default:
			checkNoEntry();
			return MSM_Unlit;
	}
}

bool UMaterial::IsShadingModelFromMaterialExpression() const
{
	return ShadingModel == MSM_FromMaterialExpression;
}

bool UMaterial::IsTwoSided() const
{
	return TwoSided != 0;
}

bool UMaterial::IsThinSurface() const
{
	return bIsThinSurface != 0;
}

bool UMaterial::IsDitheredLODTransition() const
{
	return DitheredLODTransition != 0;
}

bool UMaterial::IsTranslucencyWritingCustomDepth() const
{
	return AllowTranslucentCustomDepthWrites != 0 && IsTranslucentBlendMode(GetBlendMode());
}

bool UMaterial::IsTranslucencyWritingVelocity() const
{
	return bOutputTranslucentVelocity && IsTranslucentBlendMode(GetBlendMode());
}

bool UMaterial::IsTranslucencyVelocityFromDepth() const
{
	return bIsTranslucencyVelocityFromDepth;
}

bool UMaterial::IsTranslucencyWritingFrontLayerTransparency() const
{
	return IsTranslucentBlendMode(GetBlendMode())
		&& (TranslucencyLightingMode == TLM_Surface || TranslucencyLightingMode == TLM_SurfacePerPixelLighting) 
		&& bAllowFrontLayerTranslucency;
}

bool UMaterial::IsMasked() const
{
	return IsMaskedBlendMode(GetBlendMode()) || (IsTranslucentOnlyBlendMode(GetBlendMode()) && GetCastDynamicShadowAsMasked());
}

bool UMaterial::IsDeferredDecal() const
{
	return MaterialDomain == MD_DeferredDecal;
}

bool UMaterial::IsUIMaterial() const
{
	return MaterialDomain == MD_UI;
}

bool UMaterial::IsPostProcessMaterial() const
{
	return MaterialDomain == MD_PostProcess;
}

bool UMaterial::IsLightFunctionMaterial() const
{
	return MaterialDomain == MD_LightFunction;
}

bool UMaterial::IsPostProcessMaterialOutputingAlpha() const
{
	return UMaterial::IsPostProcessMaterial() && (BlendableOutputAlpha || BlendableLocation == BL_TranslucencyAfterDOF);
}

bool UMaterial::WritesToRuntimeVirtualTexture() const
{
	return GetCachedExpressionData().bHasRuntimeVirtualTextureOutput;
}

bool UMaterial::HasMeshPaintTexture() const
{
	return GetCachedExpressionData().bHasMeshPaintTexture;
}

bool UMaterial::HasVertexInterpolator() const
{
	return GetCachedExpressionData().bHasVertexInterpolator;
}

bool UMaterial::HasCustomizedUVs() const
{
	return GetCachedExpressionData().bHasCustomizedUVs;
}

bool UMaterial::HasCustomPrimitiveData() const
{
	const FMaterialCachedExpressionData& CachedData = GetCachedExpressionData();
	for (int32 Index : CachedData.ScalarPrimitiveDataIndexValues)
	{
		if (Index != INDEX_NONE)
		{
			return true;
		}
	}
	for (int32 Index : CachedData.VectorPrimitiveDataIndexValues)
	{
		if (Index != INDEX_NONE)
		{
			return true;
		}
	}
	return false;
}

USubsurfaceProfile* UMaterial::GetSubsurfaceProfile_Internal() const
{
	checkSlow(IsInGameThread());
	return SubsurfaceProfile; 
}

USubsurfaceProfile* UMaterial::GetSubsurfaceProfileRoot_Internal() const
{
	checkSlow(IsInGameThread());
	return SubsurfaceProfile; 
}

uint32 UMaterial::NumSubsurfaceProfileRoot_Internal() const
{
	return SubsurfaceProfiles.Num();
}

USubsurfaceProfile* UMaterial::GetSubsurfaceProfileRoot_Internal(uint32 Index) const
{
	checkSlow(IsInGameThread());
	check(Index<uint32(SubsurfaceProfiles.Num()));
	return SubsurfaceProfiles[Index];
}

USubsurfaceProfile* UMaterial::GetSubsurfaceProfileOverride_Internal() const
{
	checkSlow(IsInGameThread());
	return nullptr; // No override for root material
}

uint32 UMaterial::NumSpecularProfile_Internal() const
{
	return SpecularProfiles.Num();
}

USpecularProfile* UMaterial::GetSpecularProfile_Internal(uint32 Index) const
{
	checkSlow(IsInGameThread());
	check(Index<uint32(SpecularProfiles.Num()));
	return SpecularProfiles[Index];
}

uint32 UMaterial::NumToonProfile_Internal() const
{
	return ToonProfiles.Num();
}

UToonProfile* UMaterial::GetToonProfile_Internal(uint32 Index) const
{
	checkSlow(IsInGameThread());
	check(Index<uint32(ToonProfiles.Num()));
	return ToonProfiles[Index];
}

UNeuralProfile* UMaterial::GetNeuralProfile_Internal() const
{
	checkSlow(IsInGameThread());
	return NeuralProfile;
}

bool UMaterial::CastsRayTracedShadows() const
{
	return bCastRayTracedShadows;
}

bool UMaterial::IsTessellationEnabled() const
{
	return bEnableTessellation;
}

FDisplacementScaling UMaterial::GetDisplacementScaling() const
{
	return DisplacementScaling;
}

bool UMaterial::IsDisplacementFadeEnabled() const
{
	return bEnableDisplacementFade;
}

FDisplacementFadeRange UMaterial::GetDisplacementFadeRange() const
{
	return DisplacementFadeRange;
}

float UMaterial::GetMaxWorldPositionOffsetDisplacement() const
{
	return MaxWorldPositionOffsetDisplacement;
}

bool UMaterial::ShouldAlwaysEvaluateWorldPositionOffset() const
{
	return bAlwaysEvaluateWorldPositionOffset;
}

bool UMaterial::HasPixelAnimation() const
{
	return bHasPixelAnimation;
}

bool UMaterial::HasSubstrateRoughnessTracking() const
{
	return bSubstrateRoughnessTracking;
}

bool UMaterial::IsCompatibleWithLumenCardSharing() const
{
	return bCompatibleWithLumenCardSharing;
}

bool UMaterial::IsMobileSeparateTranslucencyEnabled() const
{
	return bEnableMobileSeparateTranslucency && !IsUIMaterial() && !IsDeferredDecal();
}

void UMaterial::SetShadingModel(EMaterialShadingModel NewModel)
{
	ensure(ShadingModel < MSM_NUM);
	ShadingModel = NewModel;
	ShadingModels = FMaterialShadingModelField(ShadingModel);
}

// This is used to list the supported properties (i.e. when Substrate is enabled/disabled)
bool UMaterial::IsPropertySupported(EMaterialProperty InProperty) const
{
	bool bSupported = true;

	if (Substrate::IsSubstrateEnabled() && !Substrate::IsHiddenMaterialAssetConversionEnabled())
	{
		bSupported = false;
		switch (InProperty)
		{
		case MP_Refraction:
		case MP_Opacity:	// For Substrate, this is only use for premultiply alpha blending where throughput applied to the background can be overriden.
		case MP_OpacityMask:
		case MP_AmbientOcclusion:
		case MP_WorldPositionOffset:
		case MP_Displacement:
		case MP_PixelDepthOffset:
		case MP_SurfaceThickness:
		case MP_FrontMaterial:
			bSupported = true;
			break;
		case MP_MaterialAttributes:
			bSupported = bUseMaterialAttributes;
			break;
		}

		if (InProperty >= MP_CustomizedUVs0 && InProperty <= MP_CustomizedUVs7)
		{
			// When bUseMaterialAttributes is enabled all MP_CustomizedUVs are valid to match legacy behavior
			bSupported = bUseMaterialAttributes || (InProperty - MP_CustomizedUVs0) < NumCustomizedUVs ;
		}
	}
	return bSupported;
}

bool UMaterial::IsPropertyRelevantForMobile(EMaterialProperty InProperty)
{
	using namespace MaterialImpl;

	for (FMobileProperty MobileProperty : GMobileRelevantMaterialProperties)
	{
		if (MobileProperty.Property == InProperty)
		{
			return true;
		}
	}
	return false;
}

static bool IsPropertyActive_Internal(EMaterialProperty InProperty,
	EMaterialDomain Domain,
	EBlendMode BlendMode,
	FMaterialShadingModelField ShadingModels,
	ETranslucencyLightingMode TranslucencyLightingMode,
	ERefractionMode RefractionMethod,
	bool bIsTessellationEnabled,
	bool bBlendableOutputAlpha,
	bool bUsesDistortion,
	bool bUsesShadingModelFromMaterialExpression,
	bool bIsTranslucencyWritingVelocity,
	bool bIsThinSurface,
	bool bIsSupported,
	bool bFrontMaterialIsConnected)
{
	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	const bool bSubstrateHiddenConversion = Substrate::IsHiddenMaterialAssetConversionEnabled();
	const bool bShowAsSubstrate = bSubstrateEnabled && (!bSubstrateHiddenConversion || bFrontMaterialIsConnected);

	if (bSubstrateEnabled && bSubstrateHiddenConversion && InProperty == MP_FrontMaterial)
	{
		return true; // If hidden conversion, always enable FrontMaterial in case we want to plug a Substrate material in there.
	}

	const bool bSubstrateOpacityOverrideAllowed = BlendMode == BLEND_AlphaComposite; // Should we always have it enabled to be able to be plugged in an fed when blend mode is toggled later on a material instance?

	if (Domain == MD_PostProcess)
	{
		if (bShowAsSubstrate)
		{
			return InProperty == MP_FrontMaterial || (InProperty == MP_Opacity && bSubstrateOpacityOverrideAllowed);
		}
		else
		{
			return InProperty == MP_EmissiveColor || (bBlendableOutputAlpha && InProperty == MP_Opacity);
		}
	}
	else if (Domain == MD_LightFunction)
	{
		// light functions should already use MSM_Unlit but we also we don't want WorldPosOffset
		if (bShowAsSubstrate)
		{
			return InProperty == MP_FrontMaterial;
		}
		else
		{
			return InProperty == MP_EmissiveColor;
		}
	}
	else if (Domain == MD_DeferredDecal)
	{
		if (bShowAsSubstrate)
		{
			return InProperty == MP_FrontMaterial
				|| InProperty == MP_WorldPositionOffset
				|| InProperty == MP_AmbientOcclusion
				|| (InProperty == MP_Opacity && bSubstrateOpacityOverrideAllowed);
		}
		else if (InProperty >= MP_CustomizedUVs0 && InProperty <= MP_CustomizedUVs7)
		{
			return true;
		}
		else if (InProperty == MP_MaterialAttributes)
		{
			// todo: MaterialAttruibutes would not return true, should it? Why we don't check for the checkbox in the material
			return true;
		}
		else if (InProperty == MP_WorldPositionOffset)
		{
			// Note: DeferredDecals don't support this but MeshDecals do
			return true;
		}

		if (BlendMode == BLEND_Translucent)
		{
			return InProperty == MP_EmissiveColor
				|| InProperty == MP_Normal
				|| InProperty == MP_Metallic
				|| InProperty == MP_Specular
				|| InProperty == MP_BaseColor
				|| InProperty == MP_Roughness
				|| InProperty == MP_Opacity
				|| InProperty == MP_AmbientOcclusion;
		}
		else if (BlendMode == BLEND_AlphaComposite)
		{
			// AlphaComposite decals never write normal.
			return InProperty == MP_EmissiveColor
				|| InProperty == MP_Metallic
				|| InProperty == MP_Specular
				|| InProperty == MP_BaseColor
				|| InProperty == MP_Roughness
				|| InProperty == MP_Opacity;
		}
		else if (BlendMode == BLEND_Modulate)
		{
			return InProperty == MP_EmissiveColor
				|| InProperty == MP_Normal
				|| InProperty == MP_Metallic
				|| InProperty == MP_Specular
				|| InProperty == MP_BaseColor
				|| InProperty == MP_Roughness
				|| InProperty == MP_Opacity;
		}
		else
		{
			return false;
		}
	}
	else if (Domain == MD_Volume)
	{
		if (bShowAsSubstrate)
		{
			return InProperty == MP_FrontMaterial;
		}
		return InProperty == MP_EmissiveColor
			|| InProperty == MP_SubsurfaceColor
			|| InProperty == MP_BaseColor
			|| InProperty == MP_AmbientOcclusion;
	}
	else if (Domain == MD_UI)
	{
		if (bShowAsSubstrate)
		{
			return InProperty == MP_FrontMaterial
				|| (InProperty == MP_WorldPositionOffset)
				|| (InProperty == MP_OpacityMask && IsMaskedBlendMode(BlendMode))
				|| (InProperty == MP_Opacity && bSubstrateOpacityOverrideAllowed)
				|| (InProperty >= MP_CustomizedUVs0 && InProperty <= MP_CustomizedUVs7);
		}
		else
		{
			return InProperty == MP_EmissiveColor
				|| (InProperty == MP_WorldPositionOffset)
				|| (InProperty == MP_OpacityMask && IsMaskedBlendMode(BlendMode))
				|| (InProperty == MP_Opacity && IsTranslucentBlendMode(BlendMode) && BlendMode != BLEND_Modulate)
				|| (InProperty >= MP_CustomizedUVs0 && InProperty <= MP_CustomizedUVs7);
		}
	}

	// Now processing MD_Surface

	const bool bIsTranslucentBlendMode = IsTranslucentBlendMode(BlendMode);
	const bool bIsNonDirectionalTranslucencyLightingMode = TranslucencyLightingMode == TLM_VolumetricNonDirectional || TranslucencyLightingMode == TLM_VolumetricPerVertexNonDirectional;
	const bool bIsVolumetricTranslucencyLightingMode = TranslucencyLightingMode == TLM_VolumetricNonDirectional
		|| TranslucencyLightingMode == TLM_VolumetricDirectional
		|| TranslucencyLightingMode == TLM_VolumetricPerVertexNonDirectional
		|| TranslucencyLightingMode == TLM_VolumetricPerVertexDirectional;
	
	bool Active = true;

	if (bShowAsSubstrate)
	{
		Active = false;
		if (bIsSupported)
		{
			switch (InProperty)
			{
			case MP_Refraction:
				Active = (bIsTranslucentBlendMode && !IsAlphaHoldoutBlendMode(BlendMode) && !IsModulateBlendMode(BlendMode) && bUsesDistortion && RefractionMethod != RM_IndexOfRefractionFromF0) || ShadingModels.HasShadingModel(MSM_SingleLayerWater);
				break;
			case MP_Opacity:
				// Opacity is used as alpha override for alpha composite blending. 
				Active = bSubstrateOpacityOverrideAllowed;
				break;
			case MP_OpacityMask:
				Active = IsMaskedBlendMode(BlendMode);
				break;
			case MP_AmbientOcclusion:
				Active = ShadingModels.IsLit();
				break;
			case MP_WorldPositionOffset:
				Active = true;
				break;
			case MP_Displacement:
				Active = bIsTessellationEnabled;
				break;
			case MP_PixelDepthOffset:
				Active = (!bIsTranslucentBlendMode) || (bIsTranslucencyWritingVelocity);
				break;
			case MP_SurfaceThickness:
				Active = bIsThinSurface;
				break;
			case MP_FrontMaterial:
				Active = true;
				break;
			case MP_MaterialAttributes:
				Active = true;
				break;
			}

			if (InProperty >= MP_CustomizedUVs0 && InProperty <= MP_CustomizedUVs7)
			{
				Active = true;
			}
		}
	}
	else
	{
		switch (InProperty)
		{
		case MP_DiffuseColor:
		case MP_SpecularColor:
			Active = false;
			break;
		case MP_Refraction:
			Active = (bIsTranslucentBlendMode && !IsAlphaHoldoutBlendMode(BlendMode) && !IsModulateBlendMode(BlendMode) && bUsesDistortion && RefractionMethod != RM_IndexOfRefractionFromF0) || ShadingModels.HasShadingModel(MSM_SingleLayerWater);
			break;
		case MP_Opacity:
			Active = (bIsTranslucentBlendMode && !IsModulateBlendMode(BlendMode)) || ShadingModels.HasShadingModel(MSM_SingleLayerWater);
			if (IsSubsurfaceShadingModel(ShadingModels))
			{
				Active = true;
			}
			break;
		case MP_OpacityMask:
			Active = IsMaskedBlendMode(BlendMode);
			break;
		case MP_BaseColor:
		case MP_AmbientOcclusion:
			Active = ShadingModels.IsLit();
			break;
		case MP_Specular:
		case MP_Roughness:
			Active = ShadingModels.IsLit() && (!bIsTranslucentBlendMode || !bIsVolumetricTranslucencyLightingMode);
			break;
		case MP_Anisotropy:
			Active = ShadingModels.HasAnyShadingModel({ MSM_DefaultLit, MSM_ClearCoat }) && (!bIsTranslucentBlendMode || !bIsVolumetricTranslucencyLightingMode);
			break;
		case MP_Metallic:
			// Subsurface models store opacity in place of Metallic in the GBuffer
			Active = ShadingModels.IsLit() && (!bIsTranslucentBlendMode || !bIsVolumetricTranslucencyLightingMode);
			break;
		case MP_Normal:
			Active = (ShadingModels.IsLit() && (!bIsTranslucentBlendMode || !bIsNonDirectionalTranslucencyLightingMode)) || bUsesDistortion;
			break;
		case MP_Tangent:
			Active = ShadingModels.HasAnyShadingModel({ MSM_DefaultLit, MSM_ClearCoat }) && (!bIsTranslucentBlendMode || !bIsVolumetricTranslucencyLightingMode);
			break;
		case MP_SubsurfaceColor:
			Active = ShadingModels.HasAnyShadingModel({ MSM_Subsurface, MSM_PreintegratedSkin, MSM_TwoSidedFoliage, MSM_Cloth });
			break;
		case MP_CustomData0:
			Active = ShadingModels.HasAnyShadingModel({ MSM_ClearCoat, MSM_Hair, MSM_Cloth, MSM_Eye, MSM_SubsurfaceProfile });
			break;
		case MP_CustomData1:
			Active = ShadingModels.HasAnyShadingModel({ MSM_ClearCoat, MSM_Eye });
			break;
		case MP_EmissiveColor:
			// Emissive is always active, even for light functions and post process materials, 
			// but not for AlphaHoldout
			Active = !IsAlphaHoldoutBlendMode(BlendMode);
			break;
		case MP_WorldPositionOffset:
			Active = true;
			break;
		case MP_Displacement:
			Active = bIsTessellationEnabled;
			break;
		case MP_PixelDepthOffset:
			Active = (!bIsTranslucentBlendMode) || (bIsTranslucencyWritingVelocity);
			break;
		case MP_ShadingModel:
			Active = bUsesShadingModelFromMaterialExpression;
			break;
		case MP_SurfaceThickness:
			Active = false; // Only shown when material is Shown as Substrate (e.g., with FrontMaterial connected)
			break;
		case MP_FrontMaterial:
			{
				Active = bSubstrateHiddenConversion;
				break;
			}
		case MP_MaterialAttributes:
		default:
			Active = true;
			break;
		}
	}
	return Active;
}

bool UMaterial::IsPropertyActive(EMaterialProperty InProperty) const
{
	return IsPropertyActiveInDerived(InProperty, this);
}

#if WITH_EDITOR
bool UMaterial::IsPropertyActiveInEditor(EMaterialProperty InProperty) const
{
	// explicitly DON'T use getters for BlendMode/ShadingModel...these getters may return an optimized value
	// we want the actual value that's been set by the user in the material editor
	return IsPropertyActive_Internal(InProperty,
		MaterialDomain,
		BlendMode,
		ShadingModels,
		TranslucencyLightingMode,
		RefractionMethod,
		IsTessellationEnabled(),
		IsPostProcessMaterialOutputingAlpha(),
		bUsesDistortion,
		IsShadingModelFromMaterialExpression(),
		IsTranslucencyWritingVelocity(),
		IsThinSurface(),
		IsPropertySupported(InProperty),
		GetEditorOnlyData()->FrontMaterial.IsConnected());
}
#endif // WITH_EDITOR

bool UMaterial::IsPropertyActiveInDerived(EMaterialProperty InProperty, const UMaterialInterface* DerivedMaterial) const
{
	bool bFrontMaterialConnected = false;
#if WITH_EDITOR
	if (!bUseMaterialAttributes)
	{
		// We can only check connected pins if we are not using material attributes because the connections are kept.
		// This with the goal to bring back the connections when bUseMaterialAttributes is disabled again.
		ensureMsgf(!(GetEditorOnlyData()->Refraction.IsConnected() && !GetCachedExpressionData().IsPropertyConnected(MP_Refraction)),
			TEXT("GetCachedExpressionData() says refraction isn't connected, but GetEditorOnlyData() says it is"));
	}
	bFrontMaterialConnected = GetEditorOnlyData()->FrontMaterial.IsConnected();
#endif
	return IsPropertyActive_Internal(InProperty,
		MaterialDomain,
		DerivedMaterial->GetBlendMode(),
		DerivedMaterial->GetShadingModels(),
		TranslucencyLightingMode,
		RefractionMethod,
		DerivedMaterial->IsTessellationEnabled(),
		IsPostProcessMaterialOutputingAlpha(),
		bUsesDistortion,
		DerivedMaterial->IsShadingModelFromMaterialExpression(),
		IsTranslucencyWritingVelocity(),
		IsThinSurface(),
		IsPropertySupported(InProperty),
		bFrontMaterialConnected);
}

#if WITH_EDITORONLY_DATA
void UMaterial::FlipExpressionPositions(TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions, TConstArrayView<TObjectPtr<UMaterialExpressionComment>> Comments, bool bScaleCoords, UMaterial* InMaterial)
{
	// Rough estimate of average increase in node size for the new editor
	const float PosScaling = bScaleCoords ? 1.25f : 1.0f;

	if (InMaterial)
	{
		InMaterial->EditorX = -InMaterial->EditorX;
	}
	for (int32 ExpressionIndex = 0; ExpressionIndex < Expressions.Num(); ExpressionIndex++)
	{
		UMaterialExpression* Expression = Expressions[ExpressionIndex];
		if (Expression)
		{
			Expression->MaterialExpressionEditorX = -Expression->MaterialExpressionEditorX * PosScaling;
			Expression->MaterialExpressionEditorY *= PosScaling;
		}
	}
	for (int32 ExpressionIndex = 0; ExpressionIndex < Comments.Num(); ExpressionIndex++)
	{
		UMaterialExpressionComment* Comment = Comments[ExpressionIndex];
		if (Comment)
		{
			Comment->MaterialExpressionEditorX = (-Comment->MaterialExpressionEditorX - Comment->SizeX) * PosScaling;
			Comment->MaterialExpressionEditorY *= PosScaling;
			Comment->SizeX *= PosScaling;
			Comment->SizeY *= PosScaling;
		}
	}
}

void UMaterial::FixCommentPositions(TConstArrayView<TObjectPtr<UMaterialExpressionComment>> Comments)
{
	// equivalent to 1/1.25 * 0.25 to get the amount that should have been used when first flipping
	const float SizeScaling = 0.2f;

	for (int32 Index = 0; Index < Comments.Num(); Index++)
	{
		UMaterialExpressionComment* Comment = Comments[Index];
		Comment->MaterialExpressionEditorX -= Comment->SizeX * SizeScaling;
	}
}

bool UMaterial::HasFlippedCoordinates()
{
	uint32 ReversedInputCount = 0;
	uint32 StandardInputCount = 0;

	// Check inputs to see if they are right of the root node
	for (int32 InputIndex = 0; InputIndex < MP_MAX; InputIndex++)
	{
		FExpressionInput* Input = GetExpressionInputForProperty((EMaterialProperty)InputIndex);
		if (Input && Input->Expression)
		{
			if (Input->Expression->MaterialExpressionEditorX > EditorX)
			{
				++ReversedInputCount;
			}
			else
			{
				++StandardInputCount;
			}
		}
	}

	// Can't be sure coords are flipped if most are set out correctly
	return ReversedInputCount > StandardInputCount;
}

bool UMaterial::FixFeatureLevelNodesForSM6(TArray<UMaterialExpression*> const& InExpressions)
{
	bool bRequiredChange = false;
	for (int32 ExpressionIndex = 0; ExpressionIndex < InExpressions.Num(); ExpressionIndex++)
	{
		UMaterialExpressionFeatureLevelSwitch* Expression = Cast<UMaterialExpressionFeatureLevelSwitch>(InExpressions[ExpressionIndex]);
		if (Expression != nullptr)
		{
			FExpressionInput const& ExpressionSM5 = Expression->Inputs[ERHIFeatureLevel::SM5];
			FExpressionInput& ExpressionSM6 = Expression->Inputs[ERHIFeatureLevel::SM6];

			if (ExpressionSM5.IsConnected() && !ExpressionSM6.IsConnected())
			{
				ExpressionSM6.Connect(ExpressionSM5.OutputIndex, ExpressionSM5.Expression);
				bRequiredChange = true;
			}
		}
	}

	return bRequiredChange;
}

#endif //WITH_EDITORONLY_DATA

void UMaterial::GetLightingGuidChain(bool bIncludeTextures, TArray<FGuid>& OutGuids) const
{
#if WITH_EDITORONLY_DATA
	if (bIncludeTextures)
	{
		OutGuids.Append(ReferencedTextureGuids);
	}

	OutGuids.Add(StateId);
	Super::GetLightingGuidChain(bIncludeTextures, OutGuids);
#endif
}

#if WITH_EDITOR
uint32 UMaterial::ComputeAllStateCRC() const
{
	uint32 CRC = Super::ComputeAllStateCRC();
	// TODO:  Find a way to avoid invalidating StateID on changes that can't affect computation, like repositioning nodes in the material graph.
	CRC = FCrc::TypeCrc32(StateId, CRC);
	return CRC;
}

bool UMaterial::IsRefractionPinPluggedIn(const UMaterialEditorOnlyData* EditorOnly)
{
	// check for a distortion value
	if (EditorOnly->Refraction.Expression
		|| (EditorOnly->Refraction.UseConstant && FMath::Abs(EditorOnly->Refraction.Constant - 1.0f) >= UE_KINDA_SMALL_NUMBER))
	{
		return true;
	}

	// check the material attributes for refraction expressions as well
	if (EditorOnly->MaterialAttributes.Expression)
	{
		// The node that possibly sets the refraction attribute can be anywhere in the sub-graph that eventually connects
		// to the material output expression. We need to look for any node of type [Make/Set]MaterialAttributes in this subgraph
		// and check whether it writes to the Refraction attribute.

		// For that we use GetAllExpressionsInMaterialAndFunctionsOfType which truly parse all functions.  
		// It however do not check if the node is really connected to the graph so it can be over conservative.

		TArray<UMaterialExpressionMakeMaterialAttributes*> MakeAttributeExpressions;
		GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionMakeMaterialAttributes>(MakeAttributeExpressions);
		for (UMaterialExpressionMakeMaterialAttributes* MakeAttributeExpression : MakeAttributeExpressions)
		{
			if (MakeAttributeExpression && MakeAttributeExpression->Refraction.Expression)
			{
				return true;
			}
		}

		TArray<UMaterialExpressionSetMaterialAttributes*> SetMaterialAttributesExpressions;
		GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionSetMaterialAttributes>(SetMaterialAttributesExpressions);
		for (UMaterialExpressionSetMaterialAttributes* SetMaterialAttributesExpression : SetMaterialAttributesExpressions)
		{
			for (int32 InputIndex = 0; InputIndex < SetMaterialAttributesExpression->Inputs.Num(); InputIndex++)
			{
				FName InputName = SetMaterialAttributesExpression->GetInputName(InputIndex);
				if (InputName == TEXT("Refraction"))
				{
					return true;
				}
			}
		}
	}

	return false;
}
#endif // WITH_EDITOR

UMaterialEditorOnlyData::UMaterialEditorOnlyData()
{
	BaseColor.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_BaseColor);
	Metallic.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Metallic).X;
	Specular.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Specular).X;
	Roughness.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Roughness).X;
	Anisotropy.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Anisotropy).X;
	Normal.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Normal);
	Tangent.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Tangent);
	EmissiveColor.Constant = FLinearColor(FMaterialAttributeDefinitionMap::GetDefaultValue(MP_EmissiveColor));
	Opacity.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Opacity).X;
	OpacityMask.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_OpacityMask).X;
	WorldPositionOffset.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_WorldPositionOffset);
	Displacement.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Displacement).X;
	SubsurfaceColor.Constant = FLinearColor(FMaterialAttributeDefinitionMap::GetDefaultValue(MP_SubsurfaceColor));
	ClearCoat.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_CustomData0).X;
	ClearCoatRoughness.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_CustomData1).X;
	AmbientOcclusion.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_AmbientOcclusion).X;
	Refraction.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_Refraction).X;
	SurfaceThickness.Constant = FMaterialAttributeDefinitionMap::GetDefaultValue(MP_SurfaceThickness).X;
	SubstrateConversionVersion = GetSubstrateInvalidVersion();
	SubstrateVersion = GetSubstrateInvalidVersion();
}

void UMaterialEditorOnlyData::ResetSubstrateConversionVersion()
{
#if WITH_EDITOR
	if (ShouldAllowSubstrateReset(this))
#endif
	{
		SubstrateConversionVersion = GetSubstrateNoConversionVersion();
	}
}

bool UMaterialEditorOnlyData::IsSubstrateAutoConverted() const
{
	return SubstrateConversionVersion > GetSubstrateNoConversionVersion();
}

#undef LOCTEXT_NAMESPACE
