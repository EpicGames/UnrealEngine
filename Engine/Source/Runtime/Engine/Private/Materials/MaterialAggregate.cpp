// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialAggregate.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Materials/MaterialIREmitter.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "RenderUtils.h"
#include "SceneTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialAggregate)

#if WITH_EDITOR

EMaterialValueType FMaterialAggregateAttribute::ToMaterialValueType() const
{
	switch (Type)
	{
		case EMaterialAggregateAttributeType::Bool1: return MCT_Bool;
		case EMaterialAggregateAttributeType::Bool2: return MCT_Unknown; // MCT does't have this yet
		case EMaterialAggregateAttributeType::Bool3: return MCT_Unknown; // MCT does't have this yet
		case EMaterialAggregateAttributeType::Bool4: return MCT_Unknown; // MCT does't have this yet
		
		case EMaterialAggregateAttributeType::UInt1: return MCT_UInt1;
		case EMaterialAggregateAttributeType::UInt2: return MCT_UInt2;
		case EMaterialAggregateAttributeType::UInt3: return MCT_UInt3;
		case EMaterialAggregateAttributeType::UInt4: return MCT_UInt4;
		
		case EMaterialAggregateAttributeType::Float1: return MCT_Float1;
		case EMaterialAggregateAttributeType::Float2: return MCT_Float2;
		case EMaterialAggregateAttributeType::Float3: return MCT_Float3;
		case EMaterialAggregateAttributeType::Float4: return MCT_Float4;
		
		case EMaterialAggregateAttributeType::Normal: return MCT_Float3;
		case EMaterialAggregateAttributeType::ShadingModel: return MCT_ShadingModel;
		case EMaterialAggregateAttributeType::MaterialAttributes: return MCT_MaterialAttributes;
		case EMaterialAggregateAttributeType::SubstrateData: return MCT_Substrate;

		case EMaterialAggregateAttributeType::Aggregate:
			return Aggregate.Get() == MaterialAttributesAggregate::Get() ? MCT_MaterialAttributes : MCT_Unknown;
		
		default:
			checkNoEntry();
			return {};
	}
}

static bool ContainsCyclicReference(const UMaterialAggregate* Aggregate, TArray<const UMaterialAggregate*>& Stack)
{
	if (!Aggregate)
	{
		return false;
	}

	// Check if Current is already in the stack (i.e., a cycle)
	if (Stack.Contains(Aggregate))
	{
		return true;
	}

	Stack.Push(Aggregate);

	// Look for cycles in any attribute of this aggregate
	for (const FMaterialAggregateAttribute& Attr : Aggregate->Attributes)
	{
		if (Attr.Type == EMaterialAggregateAttributeType::Aggregate && Attr.Aggregate)
		{
			if (ContainsCyclicReference(Attr.Aggregate, Stack))
			{
				return true;
			}
		}
	}

	Stack.Pop();
	return false;
}

// Makes sure that all attributes with an aggregate type don't reference back to this creating a cyclic dependency.
static void ClearCyclicDependencies(UMaterialAggregate* Aggregate, bool bDisplayNotifications)
{
	TArray<const UMaterialAggregate*> Stack;
	for (FMaterialAggregateAttribute& Attr : Aggregate->Attributes)
	{
		Stack.Empty(Stack.Num());
		if (Attr.Type == EMaterialAggregateAttributeType::Aggregate && ContainsCyclicReference(Aggregate, Stack))
		{
			if (bDisplayNotifications)
			{
				// Display a notification message.
				FString Message = FString::Printf(TEXT("Setting '%s' to reference material aggregate '%s' would introduce a cyclic dependency."), *Attr.Name.ToString(), *Aggregate->GetName());

				FNotificationInfo Info(FText::FromString(Message));
				Info.ExpireDuration = 2.5f;
				Info.bFireAndForget = true;
				FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
			}

			// Reset the attribute to null.
			Attr.Aggregate = nullptr;
		}
	}
}

int32 UMaterialAggregate::FindAttributeIndexByName(FName Name) const
{
	// @massimo.tristano We might want to make this constant time
	return Attributes.IndexOfByPredicate([Name](const FMaterialAggregateAttribute& Attribute)
	{
		return Attribute.Name.IsEqual(Name, ENameCase::CaseSensitive);
	});
}

void UMaterialAggregate::PostLoad()
{
	ClearCyclicDependencies(this, false);

	Super::PostLoad();
}

void UMaterialAggregate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (!(PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet))
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FMaterialAggregateAttribute, Type))
	{
		for (FMaterialAggregateAttribute& Attr : Attributes)
		{
			if (Attr.Type != EMaterialAggregateAttributeType::Aggregate)
			{
				Attr.Aggregate = nullptr;
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FMaterialAggregateAttribute, Aggregate))
	{
		ClearCyclicDependencies(this, true);
	}
}

#endif // #if WITH_EDITOR

#if WITH_EDITOR
namespace MaterialAttributesAggregate
{

static EMaterialAggregateAttributeType MaterialValueTypeToMaterialAggregateAttributeType(EMaterialValueType Type)
{
	switch (Type)
	{
		case MCT_Float:
			// FMaterialAttributeDefinitionMap::InitializeAttributeMap treats Float as Float1. This is a known bug,
			// but fixing it would require resaving all materials because component masks are cached. This would cause the
			// old trnaslator to generate an error about "not enough components".
			// Fallthrough
		case MCT_Float1: return EMaterialAggregateAttributeType::Float1;
		case MCT_Float2: return EMaterialAggregateAttributeType::Float2;
		case MCT_Float3: return EMaterialAggregateAttributeType::Float3;
		case MCT_Float4: return EMaterialAggregateAttributeType::Float4;
	
		case MCT_UInt:
		case MCT_UInt1: return EMaterialAggregateAttributeType::UInt1;
		case MCT_UInt2: return EMaterialAggregateAttributeType::UInt2;
		case MCT_UInt3: return EMaterialAggregateAttributeType::UInt3;
		case MCT_UInt4: return EMaterialAggregateAttributeType::UInt4;

		case MCT_Bool: return EMaterialAggregateAttributeType::Bool1;

		case MCT_ShadingModel: return EMaterialAggregateAttributeType::ShadingModel;
		case MCT_MaterialAttributes: return EMaterialAggregateAttributeType::MaterialAttributes;
		case MCT_Substrate: return EMaterialAggregateAttributeType::SubstrateData;

		default:
			checkf(false, TEXT("This material value type '%d' cannot be expressed to a material aggregate attribute type."), (int32)Type);
			return {};
	}
}

static EMaterialAggregateAttributeType GetPropertyType(EMaterialProperty Property)
{
	switch (Property)
	{
		case MP_Normal: return EMaterialAggregateAttributeType::Normal;
		case MP_SubsurfaceColor: return EMaterialAggregateAttributeType::Float4;
		default: return MaterialValueTypeToMaterialAggregateAttributeType(FMaterialAttributeDefinitionMap::GetValueType(Property));
	}
}

static FMaterialAggregateAttribute MakeAttributeFromProperty(EMaterialProperty Property)
{ 
	FMaterialAggregateAttribute Attribute;
	Attribute.Name = FName(FMaterialAttributeDefinitionMap::GetAttributeName(Property));
	Attribute.Type = GetPropertyType(Property);
	
	// ShadingModel defaults to 0, i.e. unlit. This is overriden for the old material system via editor only data, but we can't access that here so we force the value to Lit(1.0)
	Attribute.DefaultValue = (Property == MP_ShadingModel) ? FVector4f(1.0f, 0.0f, 0.0f, 0.0f) : FMaterialAttributeDefinitionMap::GetDefaultValue(Property); // Should that be changed for substrate?
	
	return Attribute;
}

struct FAttributePropertyIndexMap
{
	static const uint32 NumMaxCustomizedUVs = 8;

	// Constant time lookups for Property/Index mapping
	EMaterialProperty IndexToProperty[MP_MAX];

	//
	int PropertyToIndex[MP_MAX];

	//
	int NumAttributes = 0;

	static const FAttributePropertyIndexMap& Get()
	{
		static const FAttributePropertyIndexMap Instance;
		return Instance;
	}

	// This map is the ground truth for which material properties we compile in the new compiler.
	// Dynamically extendable whilst retaining constant time referencing.
	FAttributePropertyIndexMap()
	{
		//Initialise all empty values to defaults so we aren't referencing incorrect data after creating the maps
		for (int32 ArrayIndex = 0; ArrayIndex < MP_MAX; ArrayIndex++)
		{
			IndexToProperty[ArrayIndex] = MP_MAX;
			PropertyToIndex[ArrayIndex] = INDEX_NONE;
		}

		// The normal input is read back from the value set in the material attribute.
		// For this reason, the normal attribute is evaluated and set first, ensuring that
		// other inputs can read its value.
		PushAttribute(MP_Normal);

		PushAttribute(MP_BaseColor);
		PushAttribute(MP_Metallic);
		PushAttribute(MP_Specular);
		PushAttribute(MP_Roughness);
		PushAttribute(MP_Anisotropy);
		PushAttribute(MP_EmissiveColor);
		PushAttribute(MP_Opacity);
		PushAttribute(MP_OpacityMask);
		PushAttribute(MP_Tangent);
		PushAttribute(MP_WorldPositionOffset);
		PushAttribute(MP_SubsurfaceColor);
		PushAttribute(MP_CustomData0);
		PushAttribute(MP_CustomData1);
		PushAttribute(MP_AmbientOcclusion);
		PushAttribute(MP_Refraction);
		PushAttribute(MP_CustomizedUVs0);
		PushAttribute(MP_CustomizedUVs1);
		PushAttribute(MP_CustomizedUVs2);
		PushAttribute(MP_CustomizedUVs3);
		PushAttribute(MP_CustomizedUVs4);
		PushAttribute(MP_CustomizedUVs5);
		PushAttribute(MP_CustomizedUVs6);
		PushAttribute(MP_CustomizedUVs7);
		PushAttribute(MP_PixelDepthOffset);
		PushAttribute(MP_ShadingModel);
		PushAttribute(MP_Displacement);

		// Bridge legacy material attributes to Substrate's inputs last (when required)
		if (Substrate::IsSubstrateEnabled())
		{			
			PushAttribute(MP_FrontMaterial);
			PushAttribute(MP_SurfaceThickness);
		}
	}

	void PushAttribute(EMaterialProperty Property)
	{
		IndexToProperty[NumAttributes] = Property;
		PropertyToIndex[Property] = NumAttributes++;
	}

	bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < NumAttributes;
	}
};

const TConstArrayView<EMaterialProperty> GetMaterialProperties()
{
	const auto& Properties = FAttributePropertyIndexMap::Get();
	return { Properties.IndexToProperty, Properties.NumAttributes };
}

const UMaterialAggregate* Get()
{
	static const TObjectPtr<UMaterialAggregate> Instance = [] () {
		TObjectPtr<UMaterialAggregate> MA = NewObject<UMaterialAggregate>(GetTransientPackage());
		MA->Rename(TEXT("MaterialAttributes"));
		MA->AddToRoot();
		for (EMaterialProperty Property : GetMaterialProperties())
		{
			MA->Attributes.Add(MakeAttributeFromProperty(Property));
		}
		return MA;
	}();
	return Instance.Get();
}

const FMaterialAggregateAttribute* GetAttribute(EMaterialProperty Property)
{
	int32 PropertyIndex = MaterialPropertyToAttributeIndex( Property);
	if (FAttributePropertyIndexMap::Get().IsValidIndex(PropertyIndex))
	{
		return &Get()->Attributes[PropertyIndex];
	}
	return nullptr;
}

int32 MaterialPropertyToAttributeIndex(EMaterialProperty Property)
{
	check(Property < MP_MAX);
	return (Property < MP_MAX) ? FAttributePropertyIndexMap::Get().PropertyToIndex[Property] : INDEX_NONE;
}

EMaterialProperty AttributeIndexToMaterialProperty(int32 MaterialAttributeIndex)
{
	ensure(FAttributePropertyIndexMap::Get().IsValidIndex(MaterialAttributeIndex));
	return FAttributePropertyIndexMap::Get().IndexToProperty[MaterialAttributeIndex];
}

} // MaterialAttributesAggregate
#endif // #if WITH_EDITOR

