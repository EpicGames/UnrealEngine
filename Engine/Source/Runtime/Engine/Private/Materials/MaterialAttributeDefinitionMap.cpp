// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialAttributeDefinitionMap.h"

#include "HAL/CriticalSection.h"
#include "MaterialCompiler.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionShadingModel.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionBentNormalCustomOutput.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionTangentOutput.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionFirstPersonOutput.h"
#include "Materials/MaterialExpressionMaterialCache.h"
#include "Materials/MaterialExpressionTemporalResponsivenessOutput.h"
#include "Materials/MaterialExpressionMotionVectorWorldOffsetOutput.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "SubstrateDefinitions.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "MaterialShared"

FMaterialAttributeDefintion::FMaterialAttributeDefintion(
	const FGuid& InAttributeID, const FString& InAttributeName, EMaterialProperty InProperty,
	EMaterialValueType InValueType, const FVector4& InDefaultValue, EShaderFrequency InShaderFrequency,
	int32 InTexCoordIndex /*= INDEX_NONE*/, bool bInIsHidden /*= false*/, MaterialAttributeBlendFunction InBlendFunction /*= nullptr*/)
	: AttributeID(InAttributeID)
	, DefaultValue(InDefaultValue)
	, AttributeName(InAttributeName)
	, Property(InProperty)
	, ValueType(InValueType)
	, ShaderFrequency(InShaderFrequency)
	, TexCoordIndex(InTexCoordIndex)
	, BlendFunction(InBlendFunction)
	, bIsHidden(bInIsHidden)
{
	checkf(ValueType & MCT_Float || ValueType == MCT_ShadingModel || ValueType == MCT_Substrate || ValueType == MCT_MaterialAttributes, TEXT("Unsupported material attribute type %" UINT64_FMT), ValueType);
}

int32 FMaterialAttributeDefintion::CompileDefaultValue(FMaterialCompiler* Compiler) const
{
	int32 Ret;

	// TODO: Temporarily preserving hack from 4.13 to change default value for two-sided foliage model 
	if (Property == MP_SubsurfaceColor && Compiler->GetCompiledShadingModels().HasShadingModel(MSM_TwoSidedFoliage))
	{
		check(ValueType == MCT_Float3);
		return Compiler->Constant3(0, 0, 0);
	}

	if (Property == MP_ShadingModel)
	{
		check(ValueType == MCT_ShadingModel);
		// Default to the first shading model of the material. If the material is using a single shading model selected through the dropdown, this is how it gets written to the shader as a constant (optimizing out all the dynamic branches)
		return Compiler->ShadingModel(Compiler->GetMaterialShadingModels().GetFirstShadingModel());
	}

	if (Property == MP_Normal)
	{
		if (Compiler->IsTangentSpaceNormal())
		{
			return Compiler->Constant3(0.0f, 0.0f, 1.0f);	// Tangent space normal
		}
		return Compiler->VertexNormal();					// World space normal
	}

	if (Property == MP_Tangent)
	{
		if (Compiler->IsTangentSpaceNormal())
		{
			return Compiler->Constant3(1.0f, 0.0f, 0.0f);	// Tangent space tangent
		}
		return Compiler->VertexTangent();					// World space tangent
	}

	if (Property == MP_FrontMaterial)
	{
		check(ValueType == MCT_Substrate);
		return Compiler->SubstrateCreateAndRegisterNullMaterial();
	}

	if (Property == MP_Displacement)
	{
		// Initialize displacement with an out of bounds value so we can detect that it's not been hooked up
		return Compiler->Constant(-1.0f);
	}

	if (TexCoordIndex == INDEX_NONE)
	{
		// Standard value type
		switch (ValueType)
		{
		case MCT_Float:
		case MCT_Float1: Ret = Compiler->Constant(DefaultValue.X); break;
		case MCT_Float2: Ret = Compiler->Constant2(DefaultValue.X, DefaultValue.Y); break;
		case MCT_Float3: Ret = Compiler->Constant3(DefaultValue.X, DefaultValue.Y, DefaultValue.Z); break;
		default: Ret = Compiler->Constant4(DefaultValue.X, DefaultValue.Y, DefaultValue.Z, DefaultValue.W);
		}
	}
	else
	{
		// Texture coordinates allow pass through for default	
		Ret = Compiler->TextureCoordinate(TexCoordIndex, false, false);
	}

	return Ret;
}

//////////////////////////////////////////////////////////////////////////

FMaterialCustomOutputAttributeDefintion::FMaterialCustomOutputAttributeDefintion(
	const FGuid& InAttributeID, UClass* InExpressionClass, const FString& InAttributeName, const FString& InFunctionName, int32 InOutputIndex, bool bInNeedsPreviousFrameEvaluation, EMaterialProperty InProperty,
	EMaterialValueType InValueType, const FVector4& InDefaultValue, EShaderFrequency InShaderFrequency, MaterialAttributeBlendFunction InBlendFunction /*= nullptr*/)
	: FMaterialAttributeDefintion(InAttributeID, InAttributeName, InProperty, InValueType, InDefaultValue, InShaderFrequency, INDEX_NONE, false, InBlendFunction)
	, CustomOutputExpressionStaticClass(InExpressionClass)
	, FunctionName(InFunctionName)
	, OutputIndex(InOutputIndex)
	, bNeedsPreviousFrameEvaluation(bInNeedsPreviousFrameEvaluation)
{
}

//////////////////////////////////////////////////////////////////////////

FMaterialAttributeDefinitionMap FMaterialAttributeDefinitionMap::GMaterialPropertyAttributesMap;
namespace UE::MaterialAttributeDefinitionMap
{

FRWLock GlobalDDCStringLock;

} // namespace UE::MaterialAttributeDefinitionMap

FMaterialAttributeDefinitionMap::FMaterialAttributeDefinitionMap()
	: AttributeDDCString(TEXT(""))
	, bIsInitialized(false)
{
	AttributeMap.Empty(MP_MAX);
	InitializeAttributeMap();
}

/** Compiles the default expression for a material attribute */
int32 FMaterialAttributeDefinitionMap::CompileDefaultExpression(FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return Attribute->CompileDefaultValue(Compiler);
}

/** Compiles the default expression for a material attribute */
int32 FMaterialAttributeDefinitionMap::CompileDefaultExpression(FMaterialCompiler* Compiler, const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return Attribute->CompileDefaultValue(Compiler);
}

/** Returns the display name of a material attribute */
const FString& FMaterialAttributeDefinitionMap::GetAttributeName(EMaterialProperty Property)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return Attribute->AttributeName;
}

/** Returns the display name of a material attribute */
const FString& FMaterialAttributeDefinitionMap::GetAttributeName(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return Attribute->AttributeName;
}

/** Returns the display name of a material attribute, accounting for overrides based on properties of a given material */
FText FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(EMaterialProperty Property, UMaterial* Material)
{
	if (!Material)
	{
		return FText::FromString(GetAttributeName(Property));
	}

	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);
	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return GetAttributeOverrideForMaterialInLock(Attribute->AttributeID, Material);
}

/** Returns the display name of a material attribute, accounting for overrides based on properties of a given material */
FText FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(const FGuid& AttributeID, UMaterial* Material)
{
	if (!Material)
	{
		return FText::FromString(GetAttributeName(AttributeID));
	}

	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);
	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return GetAttributeOverrideForMaterialInLock(AttributeID, Material);
}

/** Returns the value type of a material attribute */
EMaterialValueType FMaterialAttributeDefinitionMap::GetValueType(EMaterialProperty Property)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return Attribute->ValueType;
}

/** Returns the value type of a material attribute */
EMaterialValueType FMaterialAttributeDefinitionMap::GetValueType(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return Attribute->ValueType;
}

/** Returns the default value of a material property */
FVector4f FMaterialAttributeDefinitionMap::GetDefaultValue(EMaterialProperty Property)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return (FVector4f)Attribute->DefaultValue;
}

/** Returns the default value of a material attribute */
FVector4f FMaterialAttributeDefinitionMap::GetDefaultValue(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return (FVector4f)Attribute->DefaultValue;
}

/** Returns the shader frequency of a material attribute */
EShaderFrequency FMaterialAttributeDefinitionMap::GetShaderFrequency(EMaterialProperty Property)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return Attribute->ShaderFrequency;
}

/** Returns the shader frequency of a material attribute */
EShaderFrequency FMaterialAttributeDefinitionMap::GetShaderFrequency(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return Attribute->ShaderFrequency;
}

int32 FMaterialAttributeDefinitionMap::GetIndex(EMaterialProperty Property)
{
	// GetIndexInLock that takes a Property actually doesn't need the lock, so
	// we don't enter it here; this function is just an alias for GetIndexInLock.
	return GetIndexInLock(Property);
}

int32 FMaterialAttributeDefinitionMap::GetIndexInLock(EMaterialProperty Property)
{
	if (Property == MP_CustomOutput || Property == MP_MAX)
	{
		return INDEX_NONE;
	}
	return static_cast<int32>(Property);
}

int32 FMaterialAttributeDefinitionMap::GetIndex(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);
	return GetIndexInLock(AttributeID);
}

int32 FMaterialAttributeDefinitionMap::GetIndexInLock(const FGuid& AttributeID)
{
	if (FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID))
	{
		if (Attribute->Property == MP_CustomOutput)
		{
			const int32 CustomIndex = GMaterialPropertyAttributesMap.CustomAttributes.IndexOfByPredicate([Attribute](const FMaterialCustomOutputAttributeDefintion& A) { return A.AttributeID == Attribute->AttributeID; });
			const int32 ResultIndex = MP_MAX + CustomIndex;
			return ResultIndex;
		}
		else if (Attribute->Property != MP_MAX)
		{
			return static_cast<int32>(Attribute->Property);
		}
	}
	return INDEX_NONE;
}

uint64 FMaterialAttributeDefinitionMap::GetBitmask(EMaterialProperty Property)
{
	// GetBitmaskInLock that takes a Property actually doesn't need the lock, so
	// we don't enter it here; this function is just an alias for GetBitmaskInLock.
	return GetBitmaskInLock(Property);
}

uint64 FMaterialAttributeDefinitionMap::GetBitmaskInLock(EMaterialProperty Property)
{
	const int32 Index = FMaterialAttributeDefinitionMap::GetIndexInLock(Property);
	if (Index != INDEX_NONE)
	{
		const uint64 Result = 1ull << static_cast<uint32>(Index);
		return Result;
	}
	return 0;
}

uint64 FMaterialAttributeDefinitionMap::GetBitmask(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	return GetBitmaskInLock(AttributeID);
}

uint64 FMaterialAttributeDefinitionMap::GetBitmaskInLock(const FGuid& AttributeID)
{
	const int32 Index = FMaterialAttributeDefinitionMap::GetIndexInLock(AttributeID);
	if (Index != INDEX_NONE)
	{
		const uint64 Result = 1ull << static_cast<uint32>(Index);
		return Result;
	}
	return 0;
}

bool FMaterialAttributeDefinitionMap::IsAttributeInBitmask(uint64 Bitmask, EMaterialProperty Property)
{
	const uint64 AttributeMask = GetBitmask(Property);
	return (AttributeMask & Bitmask) != 0;
}

bool FMaterialAttributeDefinitionMap::IsAttributeInBitmask(uint64 Bitmask, const FGuid& AttributeID)
{
	const uint64 AttributeMask = GetBitmask(AttributeID);
	return (AttributeMask & Bitmask) != 0;
}

FGuid FMaterialAttributeDefinitionMap::GetID(EMaterialProperty Property)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(Property);
	return Attribute->AttributeID;
}

EMaterialProperty FMaterialAttributeDefinitionMap::GetProperty(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	if (FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID))
	{
		return Attribute->Property;
	}
	return MP_MAX;
}

MaterialAttributeBlendFunction FMaterialAttributeDefinitionMap::GetBlendFunction(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
	return Attribute->BlendFunction;
}

FGuid FMaterialAttributeDefinitionMap::GetDefaultID()
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	return GMaterialPropertyAttributesMap.Find(MP_MAX)->AttributeID;
}

void FMaterialAttributeDefinitionMap::InitializeAttributeMap()
{
	check(!bIsInitialized);
	bIsInitialized = true;
	const bool bHideAttribute = true;
	LLM_SCOPE(ELLMTag::Materials);

	// All types plus default/missing attribute
	AttributeMap.Empty(MP_MAX + 1);

	// Basic attributes
	Add(FGuid(0x69B8D336, 0x16ED4D49, 0x9AA49729, 0x2F050F7A), TEXT("BaseColor"),		MP_BaseColor,		MCT_Float3,	FVector4(0,0,0,1),	SF_Pixel);
	Add(FGuid(0x57C3A161, 0x7F064296, 0xB00B24A5, 0xA496F34C), TEXT("Metallic"),		MP_Metallic,		MCT_Float,	FVector4(0,0,0,0),	SF_Pixel);
	Add(FGuid(0x9FDAB399, 0x25564CC9, 0x8CD2D572, 0xC12C8FED), TEXT("Specular"),		MP_Specular,		MCT_Float,	FVector4(.5,0,0,0), SF_Pixel);
	Add(FGuid(0xD1DD967C, 0x4CAD47D3, 0x9E6346FB, 0x08ECF210), TEXT("Roughness"),		MP_Roughness,		MCT_Float,	FVector4(.5,0,0,0), SF_Pixel);
	Add(FGuid(0x55E2B4FB, 0xC1C54DB2, 0x9F11875F, 0x7231EB1E), TEXT("Anisotropy"),		MP_Anisotropy,		MCT_Float,	FVector4(0,0,0,0),  SF_Pixel);
	Add(FGuid(0xB769B54D, 0xD08D4440, 0xABC21BA6, 0xCD27D0E2), TEXT("EmissiveColor"),	MP_EmissiveColor,	MCT_Float3,	FVector4(0,0,0,0),	SF_Pixel);
	Add(FGuid(0xB8F50FBA, 0x2A754EC1, 0x9EF672CF, 0xEB27BF51), TEXT("Opacity"),			MP_Opacity,			MCT_Float,	FVector4(1,0,0,0),	SF_Pixel);
	Add(FGuid(0x679FFB17, 0x2BB5422C, 0xAD520483, 0x166E0C75), TEXT("OpacityMask"),		MP_OpacityMask,		MCT_Float,	FVector4(1,0,0,0),	SF_Pixel);
	Add(FGuid(0x0FA2821A, 0x200F4A4A, 0xB719B789, 0xC1259C64), TEXT("Normal"),			MP_Normal,			MCT_Float3,	FVector4(0,0,1,0),	SF_Pixel);
	Add(FGuid(0xD5F8E9CF, 0xCDC3468D, 0xB10E4465, 0x596A7BBA), TEXT("Tangent"),			MP_Tangent,			MCT_Float3,	FVector4(1,0,0,0),	SF_Pixel);

	// Advanced attributes
	Add(FGuid(0xF905F895, 0xD5814314, 0x916D2434, 0x8C40CE9E), TEXT("WorldPositionOffset"),		MP_WorldPositionOffset,		MCT_Float3,	FVector4(0,0,0,0),	SF_Vertex);
	Add(FGuid(0x199A7166, 0xC67041DC, 0xA68EAD0D, 0x7017D0AD), TEXT("Displacement"),			MP_Displacement,			MCT_Float,	FVector4(0.5,0,0,0), SF_Pixel);
	Add(FGuid(0x5B8FC679, 0x51CE4082, 0x9D777BEE, 0xF4F72C44), TEXT("SubsurfaceColor"),			MP_SubsurfaceColor,			MCT_Float3,	FVector4(1,1,1,0),	SF_Pixel);
	Add(FGuid(0x9E502E69, 0x3C8F48FA, 0x94645CFD, 0x28E5428D), TEXT("CustomData0"),				MP_CustomData0,				MCT_Float,	FVector4(1,0,0,0),	SF_Pixel);
	Add(FGuid(0xBE4F2FFD, 0x12FC4296, 0xB0124EEA, 0x12C28D92), TEXT("CustomData1"),				MP_CustomData1,				MCT_Float,	FVector4(.1,0,0,0),	SF_Pixel);
	Add(FGuid(0xE8EBD0AD, 0xB1654CBE, 0xB079C3A8, 0xB39B9F15), TEXT("AmbientOcclusion"),		MP_AmbientOcclusion,		MCT_Float,	FVector4(1,0,0,0),	SF_Pixel);
	Add(FGuid(0xD0B0FA03, 0x14D74455, 0xA851BAC5, 0x81A0788B), TEXT("Refraction"),				MP_Refraction,				MCT_Float3,	FVector4(1,0,0,0),	SF_Pixel);
	Add(FGuid(0x0AC97EC3, 0xE3D047BA, 0xB610167D, 0xC4D919FF), TEXT("PixelDepthOffset"),		MP_PixelDepthOffset,		MCT_Float,	FVector4(0,0,0,0),	SF_Pixel);
	Add(FGuid(0xD9423FFF, 0xD77E4D82, 0x8FF9CF5E, 0x055D1255), TEXT("ShadingModel"),			MP_ShadingModel,			MCT_ShadingModel, FVector4(0, 0, 0, 0), SF_Pixel, INDEX_NONE, false, &CompileShadingModelBlendFunction);
	Add(FGuid(0x42BDD2E0, 0xBE714189, 0xA0984BC3, 0xDD0BE872), TEXT("SurfaceThickness"),		MP_SurfaceThickness,		MCT_Float,  FVector4(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM, 0, 0, 0), SF_Pixel);
	Add(FGuid(0x5973A03E, 0x13A74E08, 0x92D0CEDD, 0xF2936CF8), TEXT("FrontMaterial"),			MP_FrontMaterial,			MCT_Substrate, FVector4(0,0,0,0),	SF_Pixel);

	// Used when compiling material with execution pins, which are compiling all attributes together
	Add(FGuid(0xE0ED040B, 0x82794D93, 0xBD2D59B2, 0xA5BBF41C), TEXT("MaterialAttributes"),		MP_MaterialAttributes,		MCT_MaterialAttributes, FVector4(0,0,0,0), SF_Pixel, INDEX_NONE, bHideAttribute);

	// Texture coordinates
	Add(FGuid(0xD30EC284, 0xE13A4160, 0x87BB5230, 0x2ED115DC), TEXT("CustomizedUV0"), MP_CustomizedUVs0, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 0);
	Add(FGuid(0xC67B093C, 0x2A5249AA, 0xABC97ADE, 0x4A1F49C5), TEXT("CustomizedUV1"), MP_CustomizedUVs1, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 1);
	Add(FGuid(0x85C15B24, 0xF3E047CA, 0x85856872, 0x01AE0F4F), TEXT("CustomizedUV2"), MP_CustomizedUVs2, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 2);
	Add(FGuid(0x777819DC, 0x31AE4676, 0xB864EF77, 0xB807E873), TEXT("CustomizedUV3"), MP_CustomizedUVs3, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 3);
	Add(FGuid(0xDA63B233, 0xDDF44CAD, 0xB93D867B, 0x8DAFDBCC), TEXT("CustomizedUV4"), MP_CustomizedUVs4, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 4);
	Add(FGuid(0xC2F52B76, 0x4A034388, 0x89119528, 0x2071B190), TEXT("CustomizedUV5"), MP_CustomizedUVs5, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 5);
	Add(FGuid(0x8214A8CA, 0x0CB944CF, 0x9DFD78DB, 0xE48BB55F), TEXT("CustomizedUV6"), MP_CustomizedUVs6, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 6);
	Add(FGuid(0xD8F8D01F, 0xC6F74715, 0xA3CFB4FF, 0x9EF51FAC), TEXT("CustomizedUV7"), MP_CustomizedUVs7, MCT_Float2, FVector4(0,0,0,0), SF_Vertex, 7);

	// Lightmass attributes	
	Add(FGuid(0x68934E1B, 0x70EB411B, 0x86DF5AA5, 0xDF2F626C), TEXT("DiffuseColor"),	MP_DiffuseColor,	MCT_Float3, FVector4(0,0,0,0), SF_Pixel, INDEX_NONE, bHideAttribute);
	Add(FGuid(0xE89CBD84, 0x62EA48BE, 0x80F88521, 0x2B0C403C), TEXT("SpecularColor"),	MP_SpecularColor,	MCT_Float3, FVector4(0,0,0,0), SF_Pixel, INDEX_NONE, bHideAttribute);

	// Debug attributes
	Add(FGuid(0x5BF6BA94, 0xA3264629, 0xA253A05B, 0x0EABBB86), TEXT("Missing"), MP_MAX, MCT_Float, FVector4(0,0,0,0), SF_Pixel, INDEX_NONE, bHideAttribute);

	// Removed attributes
	Add(FGuid(0x2091ECA2, 0xB59248EE, 0x8E2CD578, 0xD371926D), TEXT("WorldDisplacement"), MP_WorldDisplacement_DEPRECATED, MCT_Float3, FVector4(0, 0, 0, 0), SF_Vertex, INDEX_NONE, bHideAttribute);
	Add(FGuid(0xA0119D44, 0xC456450D, 0x9C39C933, 0x1F72D8D1), TEXT("TessellationMultiplier"), MP_TessellationMultiplier_DEPRECATED, MCT_Float, FVector4(1, 0, 0, 0), SF_Vertex, INDEX_NONE, bHideAttribute);

	// UMaterialExpression custom outputs
	AddCustomAttributeInner(FGuid(0xfbd7b46e, 0xb1234824, 0xbde76b23, 0x609f984c), UMaterialExpressionBentNormalCustomOutput::StaticClass(), "BentNormal", "GetBentNormal", 0, MCT_Float3, FVector4(0, 0, 1, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0xAA3D5C04, 0x16294716, 0xBBDEC869, 0x6A27DD72), UMaterialExpressionClearCoatNormalCustomOutput::StaticClass(), "ClearCoatBottomNormal", "ClearCoatBottomNormal", 0, MCT_Float3, FVector4(0, 0, 1, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0x8EAB2CB2, 0x73634A24, 0x8CD14F47, 0x3F9C8E55), UMaterialExpressionTangentOutput::StaticClass(), "CustomEyeTangent", "GetTangentOutput", 0, MCT_Float3, FVector4(0, 0, 0, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0xF2D8C70E, 0x42ECA0D1, 0x4652D0AD, 0xB785A065), UMaterialExpressionThinTranslucentMaterialOutput::StaticClass(), "TransmittanceColor", "GetThinTranslucentMaterialOutput", 0, MCT_Float3, FVector4(0.5, 0.5, 0.5, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0xAAA1D0E0, 0xE312458E, 0xF3556BAD, 0xA01277CD), UMaterialExpressionThinTranslucentMaterialOutput::StaticClass(), "ThinTranslucentSurfaceCoverage", "GetThinTranslucentMaterialOutput", 1, MCT_Float, FVector4(1, 0, 0, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0xDB380E51, 0xBDED47B6, 0xB185A8BE, 0x8EACBFA4), UMaterialExpressionFirstPersonOutput::StaticClass(), "FirstPersonInterpolationAlpha", "GetFirstPersonOutput", 0, MCT_Float, FVector4(1, 0, 0, 0), SF_Vertex, true /*bNeedsPreviousFrameEvaluation*/);
	AddCustomAttributeInner(FGuid(0x6B1237CA, 0x1B8C43C2, 0xA9C8A020, 0x430F1258), UMaterialExpressionMaterialCache::StaticClass(), "MaterialCache", "MaterialCache", 0, MCT_Float4, FVector4(0, 0, 0, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0xB58CD3FB, 0x8CB4411D, 0x83C4C11E, 0xF89953AD), UMaterialExpressionTemporalResponsivenessOutput::StaticClass(), "TemporalResponsiveness", "GetTemporalResponsivenessOutput", 0, MCT_Float, FVector4(0, 0, 0, 0), SF_Pixel);
	AddCustomAttributeInner(FGuid(0x48A2CEAB, 0x8E63473F, 0x821345BC, 0x6FA907F7), UMaterialExpressionMotionVectorWorldOffsetOutput::StaticClass(), "MotionVectorWorldOffset", "GetMotionVectorWorldOffsetOutput", 0, MCT_Float3, FVector4(0, 0, 0, 0), SF_Pixel);
}

void FMaterialAttributeDefinitionMap::Add(const FGuid& AttributeID, const FString& AttributeName, EMaterialProperty Property,
	EMaterialValueType ValueType, const FVector4& DefaultValue, EShaderFrequency ShaderFrequency,
	int32 TexCoordIndex /*= INDEX_NONE*/, bool bIsHidden /*= false*/, MaterialAttributeBlendFunction BlendFunction /*= nullptr*/)
{
	checkf(!AttributeMap.Contains(Property), TEXT("Tried to add duplicate material property."));
	AttributeMap.Add(Property, FMaterialAttributeDefintion(AttributeID, AttributeName, Property, ValueType, DefaultValue, ShaderFrequency, TexCoordIndex, bIsHidden, BlendFunction));
	if (!bIsHidden)
	{
		OrderedVisibleAttributeList.Add(AttributeID);
	}
}

FMaterialAttributeDefintion* FMaterialAttributeDefinitionMap::Find(const FGuid& AttributeID)
{
	for (auto& Attribute : CustomAttributes)
	{
		if (Attribute.AttributeID == AttributeID)
		{
			return &Attribute;
		}
	}

	for (auto& Attribute : AttributeMap)
	{
		if (Attribute.Value.AttributeID == AttributeID)
		{
			return &Attribute.Value;
		}
	}

	UE_LOGF(LogMaterial, Warning, "Failed to find material attribute, AttributeID: %ls.", *AttributeID.ToString(EGuidFormats::Digits));
	return Find(MP_MAX);
}

FMaterialAttributeDefintion* FMaterialAttributeDefinitionMap::Find(EMaterialProperty Property)
{
	if (FMaterialAttributeDefintion* Attribute = AttributeMap.Find(Property))
	{
		return Attribute;
	}

	UE_LOGF(LogMaterial, Warning, "Failed to find material attribute, PropertyType: %i.", (uint32)Property);
	return Find(MP_MAX);
}

FText FMaterialAttributeDefinitionMap::GetAttributeOverrideForMaterialInLock(const FGuid& AttributeID, UMaterial* Material)
{
	TArray<TKeyValuePair<EMaterialShadingModel, FString>> CustomPinNames;
	EMaterialProperty Property = GMaterialPropertyAttributesMap.Find(AttributeID)->Property;

	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	const bool bHasSubstrateFrontMaterialConnected = Material->HasSubstrateFrontMaterialConnected();

	switch (Property)
	{
	case MP_EmissiveColor:
		return Material->IsUIMaterial() ? LOCTEXT("UIOutputColor", "Final Color") : LOCTEXT("EmissiveColor", "Emissive Color");
	case MP_Opacity:
		return bSubstrateEnabled && bHasSubstrateFrontMaterialConnected ? LOCTEXT("OpacityOverride", "Opacity Override") : LOCTEXT("Opacity", "Opacity");
	case MP_OpacityMask:
		return LOCTEXT("OpacityMask", "Opacity Mask");
	case MP_DiffuseColor:
		return LOCTEXT("DiffuseColor", "Diffuse Color");
	case MP_SpecularColor:
		return LOCTEXT("SpecularColor", "Specular Color");
	case MP_BaseColor:
		return Material->MaterialDomain == MD_Volume ? LOCTEXT("Albedo", "Albedo") : LOCTEXT("BaseColor", "Base Color");
	case MP_Metallic:
		CustomPinNames.Add({ MSM_Hair, LOCTEXT("Scatter", "Scatter").ToString()});
		CustomPinNames.Add({ MSM_Eye, LOCTEXT("Curvature", "Curvature").ToString()});
		return FText::FromString(GetPinNameFromShadingModelFieldInLock(Material->GetShadingModels(), CustomPinNames,
			LOCTEXT("Metallic", "Metallic").ToString()));
	case MP_Specular:
		return LOCTEXT("Specular", "Specular");
	case MP_Roughness:
		return LOCTEXT("Roughness", "Roughness");
	case MP_Anisotropy:
		return LOCTEXT("Anisotropy", "Anisotropy");
	case MP_Normal:
		CustomPinNames.Add({ MSM_Hair, LOCTEXT("Tangent", "Tangent").ToString() });
		return FText::FromString(GetPinNameFromShadingModelFieldInLock(Material->GetShadingModels(), CustomPinNames,
			LOCTEXT("Normal", "Normal").ToString()));
	case MP_Tangent:
		return LOCTEXT("Tangent", "Tangent");
	case MP_WorldPositionOffset:
		return Material->IsUIMaterial() ? LOCTEXT("ScreenPosition", "Screen Position") : LOCTEXT("WorldPositionOffset", "World Position Offset");
	case MP_Displacement:
		return LOCTEXT("Displacement", "Displacement");
	case MP_WorldDisplacement_DEPRECATED:
		return LOCTEXT("WorldDisplacement", "World Displacement");
	case MP_TessellationMultiplier_DEPRECATED:
		return LOCTEXT("TessellationMultiplier", "Tessellation Multiplier");
	case MP_SubsurfaceColor:
		if (Material->MaterialDomain == MD_Volume)
		{
			return LOCTEXT("Extinction", "Extinction");
		}
		CustomPinNames.Add({ MSM_Cloth, LOCTEXT("FuzzColor", "Fuzz Color").ToString() });
		return FText::FromString(GetPinNameFromShadingModelFieldInLock(Material->GetShadingModels(), CustomPinNames,
			LOCTEXT("SubsurfaceColor", "Subsurface Color").ToString()));
	case MP_CustomData0:
		CustomPinNames.Add({ MSM_ClearCoat, LOCTEXT("ClearCoat", "Clear Coat").ToString() });
		CustomPinNames.Add({ MSM_Hair, LOCTEXT("Backlit", "Backlit").ToString() });
		CustomPinNames.Add({ MSM_Cloth, LOCTEXT("Cloth", "Cloth").ToString() });
		CustomPinNames.Add({ MSM_Eye, LOCTEXT("IrisMask", "Iris Mask").ToString() });
		CustomPinNames.Add({ MSM_SubsurfaceProfile, LOCTEXT("Curvature", "Curvature").ToString() });
		return FText::FromString(GetPinNameFromShadingModelFieldInLock(Material->GetShadingModels(), CustomPinNames,
			LOCTEXT("CustomData0", "Custom Data 0").ToString()));
	case MP_CustomData1:
		CustomPinNames.Add({ MSM_ClearCoat, LOCTEXT("ClearCoatRoughness", "Clear Coat Roughness").ToString() });
		CustomPinNames.Add({ MSM_Eye, LOCTEXT("IrisDistance", "Iris Distance").ToString() });
		return FText::FromString(GetPinNameFromShadingModelFieldInLock(Material->GetShadingModels(), CustomPinNames,
			LOCTEXT("CustomData1", "Custom Data 1").ToString()));
	case MP_AmbientOcclusion:
		return LOCTEXT("AmbientOcclusion", "Ambient Occlusion");
	case MP_Refraction:
		return LOCTEXT("Refraction", "Refraction");
	case MP_CustomizedUVs0:
		return LOCTEXT("CustomizedUV0", "Customized UV 0");
	case MP_CustomizedUVs1:
		return LOCTEXT("CustomizedUV1", "Customized UV 1");
	case MP_CustomizedUVs2:
		return LOCTEXT("CustomizedUV2", "Customized UV 2");
	case MP_CustomizedUVs3:
		return LOCTEXT("CustomizedUV3", "Customized UV 3");
	case MP_CustomizedUVs4:
		return LOCTEXT("CustomizedUV4", "Customized UV 4");
	case MP_CustomizedUVs5:
		return LOCTEXT("CustomizedUV5", "Customized UV 5");
	case MP_CustomizedUVs6:
		return LOCTEXT("CustomizedUV6", "Customized UV 6");
	case MP_CustomizedUVs7:
		return LOCTEXT("CustomizedUV7", "Customized UV 7");
	case MP_PixelDepthOffset:
		return LOCTEXT("PixelDepthOffset", "Pixel Depth Offset");
	case MP_ShadingModel:
		return LOCTEXT("ShadingModel", "Shading Model");
	case MP_SurfaceThickness:
		return LOCTEXT("SurfaceThickness", "Surface Thickness");
	case MP_FrontMaterial:
		return LOCTEXT("FrontMaterial", "Front Material");
	case MP_CustomOutput:
		return FText::FromString(GetAttributeName(AttributeID));

	}
	return  LOCTEXT("Missing", "Missing");
}

FString FMaterialAttributeDefinitionMap::GetPinNameFromShadingModelFieldInLock(FMaterialShadingModelField InShadingModels,
	const TArray<TKeyValuePair<EMaterialShadingModel, FString>>& InCustomShadingModelPinNames,
	const FString& InDefaultPinName)
{
	FString OutPinName;
	for (const TKeyValuePair<EMaterialShadingModel, FString>& CustomShadingModelPinName : InCustomShadingModelPinNames)
	{
		if (InShadingModels.HasShadingModel(CustomShadingModelPinName.Key))
		{
			// Add delimiter
			if (!OutPinName.IsEmpty())
			{
				OutPinName.Append(" or ");
			}

			// Append the name and remove the shading model from the temp field
			OutPinName.Append(CustomShadingModelPinName.Value);
			InShadingModels.RemoveShadingModel(CustomShadingModelPinName.Key);
		}
	}

	// There are other shading models present, these don't have their own specific name for this pin, so use a default one
	if (InShadingModels.CountShadingModels() != 0)
	{
		// Add delimiter
		if (!OutPinName.IsEmpty())
		{
			OutPinName.Append(" or ");
		}

		OutPinName.Append(InDefaultPinName);
	}

	ensure(!OutPinName.IsEmpty());
	return OutPinName;
}

void FMaterialAttributeDefinitionMap::AppendDDCKeyString(FString& String)
{
	FShaderKeyGenerator KeyGen(String);
	AppendDDCKey(KeyGen);
}

void FMaterialAttributeDefinitionMap::AppendDDCKey(FShaderKeyGenerator& KeyGen)
{
	{
		FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);
		const FString& DDCString = GMaterialPropertyAttributesMap.AttributeDDCString;
		if (DDCString.Len() != 0)
		{
			// TODO: In debug force re-generate DDC string and compare to catch invalid runtime changes
			KeyGen.Append(DDCString);
			return;
		}
	}

	FWriteScopeLock WriteScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);
	FString& DDCString = GMaterialPropertyAttributesMap.AttributeDDCString;
	if (DDCString.Len() == 0)
	{
		FString AttributeIDs;

		for (const auto& Attribute : GMaterialPropertyAttributesMap.AttributeMap)
		{
			AttributeIDs += Attribute.Value.AttributeID.ToString(EGuidFormats::Digits);
		}

		for (const auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
		{
			AttributeIDs += Attribute.AttributeID.ToString(EGuidFormats::Digits);
		}

		FSHA1 HashState;
		HashState.UpdateWithString(*AttributeIDs, AttributeIDs.Len());
		HashState.Final();

		FSHAHash Hash;
		HashState.GetHash(&Hash.Hash[0]);
		DDCString = Hash.ToString();
	}
	else
	{
		// TODO: In debug force re-generate DDC string and compare to catch invalid runtime changes
	}

	KeyGen.Append(DDCString);
}

void FMaterialAttributeDefinitionMap::AddCustomAttribute(
	const FGuid& AttributeID,
	UClass* ExpressionClass,
	const FString& AttributeName,
	const FString& FunctionName,
	int32 OutputIndex,
	EMaterialValueType ValueType,
	const FVector4& DefaultValue,
	EShaderFrequency ShaderFrequency,
	bool bNeedsPreviousFrameEvaluation,
	MaterialAttributeBlendFunction BlendFunction /*= nullptr*/)
{
	FWriteScopeLock WriteScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	GMaterialPropertyAttributesMap.AddCustomAttributeInner(AttributeID, ExpressionClass, AttributeName, FunctionName,
		OutputIndex, ValueType, DefaultValue, ShaderFrequency, bNeedsPreviousFrameEvaluation, BlendFunction);
}

void FMaterialAttributeDefinitionMap::AddCustomAttributeInner(
	const FGuid& AttributeID,
	UClass* ExpressionClass,
	const FString& AttributeName,
	const FString& FunctionName,
	int32 OutputIndex,
	EMaterialValueType ValueType,
	const FVector4& DefaultValue,
	EShaderFrequency ShaderFrequency,
	bool bNeedsPreviousFrameEvaluation,
	MaterialAttributeBlendFunction BlendFunction /*= nullptr*/)
{
	// Make sure that we init CustomAttributes before DDCString is initialized (before first shader load)
	check(AttributeDDCString.Len() == 0);

	FMaterialCustomOutputAttributeDefintion UserAttribute(AttributeID, ExpressionClass, AttributeName, FunctionName, OutputIndex, bNeedsPreviousFrameEvaluation, MP_CustomOutput, ValueType, DefaultValue, ShaderFrequency, BlendFunction);
#if DO_CHECK
	for (auto& Attribute : AttributeMap)
	{
		checkf(Attribute.Value.AttributeID != AttributeID, TEXT("Tried to add duplicate custom output attribute (%s) already in base attributes (%s)."), *AttributeName, *(Attribute.Value.AttributeName));
	}
	checkf(!CustomAttributes.Contains(UserAttribute), TEXT("Tried to add duplicate custom output attribute (%s)."), *AttributeName);
#endif
	CustomAttributes.Add(UserAttribute);

	if (!UserAttribute.bIsHidden)
	{
		OrderedVisibleAttributeList.Add(AttributeID);
	}
}

FGuid FMaterialAttributeDefinitionMap::GetCustomAttributeID(const FString& AttributeName)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
	{
		if (Attribute.AttributeName == AttributeName)
		{
			return Attribute.AttributeID;
		}
	}

	return GMaterialPropertyAttributesMap.Find(MP_MAX)->AttributeID;
}

const FMaterialCustomOutputAttributeDefintion* FMaterialAttributeDefinitionMap::GetCustomAttribute(const FString& AttributeName)
{
	// This lock is mostly useless since it goes out of scope before we finish reading the out result
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
	{
		if (Attribute.AttributeName == AttributeName)
		{
			return &Attribute;
		}
	}

	return nullptr;
}

TOptional<FMaterialCustomOutputAttributeDefintion> FMaterialAttributeDefinitionMap::GetCustomAttributeCopy(const FString& AttributeName)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
	{
		if (Attribute.AttributeName == AttributeName)
		{
			return Attribute;
		}
	}

	return TOptional<FMaterialCustomOutputAttributeDefintion>();
}

const FMaterialCustomOutputAttributeDefintion* FMaterialAttributeDefinitionMap::GetCustomAttribute(const FGuid& AttributeID)
{
	// This lock is mostly useless since it goes out of scope before we finish reading the out result
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
	{
		if (Attribute.AttributeID == AttributeID)
		{
			return &Attribute;
		}
	}

	return nullptr;
}

TOptional<FMaterialCustomOutputAttributeDefintion> FMaterialAttributeDefinitionMap::GetCustomAttributeCopy(const FGuid& AttributeID)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
	{
		if (Attribute.AttributeID == AttributeID)
		{
			return Attribute;
		}
	}

	return TOptional<FMaterialCustomOutputAttributeDefintion>();
}

void FMaterialAttributeDefinitionMap::GetCustomAttributeList(TArray<FMaterialCustomOutputAttributeDefintion>& CustomAttributeList)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	CustomAttributeList.Empty(GMaterialPropertyAttributesMap.CustomAttributes.Num());
	for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
	{
		CustomAttributeList.Add(Attribute);
	}
}

const TArray<FGuid>& FMaterialAttributeDefinitionMap::GetOrderedVisibleAttributeList()
{
	// This lock is mostly useless since it goes out of scope before we finish reading the out result
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);
	return GMaterialPropertyAttributesMap.OrderedVisibleAttributeList;
}

FMaterialAttributeDefinitionMap::FReadLockedOrderVisibleAttributeList::FReadLockedOrderVisibleAttributeList()
	: List(GMaterialPropertyAttributesMap.OrderedVisibleAttributeList)
{
	UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock.ReadLock();
}

FMaterialAttributeDefinitionMap::FReadLockedOrderVisibleAttributeList::~FReadLockedOrderVisibleAttributeList()
{
	UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock.ReadUnlock();
}

FMaterialAttributeDefinitionMap::FReadLockedOrderVisibleAttributeList FMaterialAttributeDefinitionMap::GetReadLockedOrderedVisibleAttributeList()
{
	return FReadLockedOrderVisibleAttributeList();
}

#if WITH_EDITOR
uint64 FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions)
{
	return GetConnectedMaterialAttributesBitmask_Internal(Expressions);
}

uint64 FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(TConstArrayView<UMaterialExpression*> Expressions)
{
	return GetConnectedMaterialAttributesBitmask_Internal(Expressions);
}

template<typename ExpressionsArrayType>
inline uint64 FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask_Internal(const ExpressionsArrayType& Expressions)
{
	uint64 Out = 0ull;
	for (const UMaterialExpression* Expression : Expressions)
	{
		if (Expression)
		{
			if (Expression->IsA(UMaterialExpressionSetMaterialAttributes::StaticClass()))
			{
				const UMaterialExpressionSetMaterialAttributes* Attr = Cast<UMaterialExpressionSetMaterialAttributes>(Expression);
				Out |= Attr->GetConnectedInputs();
			}
			else if (Expression->IsA(UMaterialExpressionMakeMaterialAttributes::StaticClass()))
			{
				const UMaterialExpressionMakeMaterialAttributes* Attr = Cast<UMaterialExpressionMakeMaterialAttributes>(Expression);
				Out |= Attr->GetConnectedInputs();
			}
			else if (Expression->IsA(UMaterialExpressionMaterialFunctionCall::StaticClass()))
			{
				const UMaterialExpressionMaterialFunctionCall* Attr = Cast<UMaterialExpressionMaterialFunctionCall>(Expression);
				if (Attr->MaterialFunction)
				{
					Out |= GetConnectedMaterialAttributesBitmask(Attr->MaterialFunction->GetExpressions());
				}
			}
		}
	}
	return Out;
}

uint64 FMaterialAttributeDefinitionMap::GetCustomOutputNodesBitmask(TConstArrayView<UMaterialExpressionCustomOutput*> Expressions)
{
	uint64 Out = 0ull;
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	for (const UMaterialExpressionCustomOutput* Expression : Expressions)
	{
		if (Expression)
		{
			for (auto& Attribute : GMaterialPropertyAttributesMap.CustomAttributes)
			{
				if (Expression->IsA(Attribute.CustomOutputExpressionStaticClass))
				{
					Out |= GetBitmaskInLock(Attribute.AttributeID);
				}
			}
		}
	}
	return Out;
}
#endif // WITH_EDITOR

void FMaterialAttributeDefinitionMap::GetAttributeNameToIDList(TArray<TPair<FString, FGuid>>& NameToIDList)
{
	FReadScopeLock ReadScopeLock(UE::MaterialAttributeDefinitionMap::GlobalDDCStringLock);

	NameToIDList.Empty(GMaterialPropertyAttributesMap.OrderedVisibleAttributeList.Num());
	for (const FGuid& AttributeID : GMaterialPropertyAttributesMap.OrderedVisibleAttributeList)
	{
		FMaterialAttributeDefintion* Attribute = GMaterialPropertyAttributesMap.Find(AttributeID);
		NameToIDList.Emplace(Attribute->AttributeName, AttributeID);
	}
}

#undef LOCTEXT_NAMESPACE
