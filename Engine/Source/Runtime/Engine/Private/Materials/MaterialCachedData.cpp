// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCachedData.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionCollectionTransform.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Materials/MaterialExpressionQualitySwitch.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionPerInstanceCustomData.h"
#include "Materials/MaterialExpressionPerInstanceRandom.h"
#include "Materials/MaterialExpressionVertexInterpolator.h"
#include "Materials/MaterialExpressionSceneColor.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureOutput.h"
#include "Materials/MaterialExpressionFirstPersonOutput.h"
#include "Materials/MaterialExpressionTemporalResponsivenessOutput.h"
#include "Materials/MaterialExpressionMotionVectorWorldOffsetOutput.h"
#include "Materials/MaterialExpressionLandscapeGrassOutput.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionLayerStack.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionLandscapeGrassOutput.h"
#include "Materials/MaterialExpressionUserSceneTexture.h"
#include "Materials/MaterialExpressionMeshPaintTextureObject.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionActorPositionWS.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialParameterCollection.h"
#include "VT/RuntimeVirtualTexture.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Engine/Font.h"
#include "LandscapeGrassType.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Engine/TextureCollection.h"
#include "ShaderCompilerCore.h"
#include "MaterialShared.h"
#include "MaterialCache/MaterialCacheMaterial.h"
#include "Materials/MaterialExpressionMaterialCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialCachedData)

#define LOCTEXT_NAMESPACE "Material"

const FMaterialCachedParameterEntry FMaterialCachedParameterEntry::EmptyData{};
const FMaterialCachedExpressionData FMaterialCachedExpressionData::EmptyData{};
const FMaterialCachedExpressionEditorOnlyData FMaterialCachedExpressionEditorOnlyData::EmptyData{};

static_assert((uint64)(EMaterialProperty::MP_MaterialAttributes)-1 < (8 * sizeof(FMaterialCachedExpressionData::PropertyConnectedMask)), "PropertyConnectedMask cannot contain entire EMaterialProperty enumeration.");

FString FParameterAssetPathsEntry::ToString() const
{
	FString Result;
	for (const auto& Element : AssetPaths)
	{
		const FString& AssetPath = Element.Key;
		const int32 PathCounter = Element.Value;

		if (PathCounter > 1)
		{
			Result += FString::Printf(TEXT("\n %dx %s"), PathCounter, *AssetPath);
		}
		else
		{
			Result += FString::Printf(TEXT("\n %s"), *AssetPath);
		}
	}

	return Result;
}

FMaterialCachedExpressionData::FMaterialCachedExpressionData()
	: FunctionInfosStateCRC(0xffffffff)
	, bHasMaterialLayers(false)
	, bHasRuntimeVirtualTextureOutput(false)
	, bHasFirstPersonOutput(false)
	, bUsesTemporalResponsiveness(false)
	, bUsesMotionVectorWorldOffset(false)
	, bSamplesMaterialCache(false)
	, bHasMaterialCacheOutput(false)
	, bMaterialCacheHasNonUVDerivedExpression(false)
	, bHasSceneColor(false)
	, bHasPerInstanceCustomData(false)
	, bHasPerInstanceRandom(false)
	, bHasVertexInterpolator(false)
	, bHasCustomizedUVs(false)
	, bHasMeshPaintTexture(false)
	, bHasWorldPosition(false)
{
	QualityLevelsUsed.AddDefaulted(EMaterialQualityLevel::Num);
#if WITH_EDITORONLY_DATA
	EditorOnlyData = MakeShared<FMaterialCachedExpressionEditorOnlyData>();
#endif // WITH_EDITORONLY_DATA
}

void FMaterialCachedExpressionData::AddReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITORONLY_DATA
 	Collector.AddStableReferenceArray(&EditorOnlyData->EditorReferencedDefaultTextures);
#endif
	Collector.AddStableReferenceArray(&ReferencedTextureCollections);
	for (TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : NamedGrassTypes)
	{
		Collector.AddStableReference(&ItPair.Value);
	}
	Collector.AddStableReferenceArray(&MaterialCacheTags);
	Collector.AddStableReferenceArray(&MaterialLayers.Layers);
	Collector.AddStableReferenceArray(&MaterialLayers.Blends);
	for (FMaterialFunctionInfo& FunctionInfo : FunctionInfos)
	{
		Collector.AddStableReference(&FunctionInfo.Function);
	}
	for (FMaterialParameterCollectionInfo& ParameterCollectionInfo : ParameterCollectionInfos)
	{
		Collector.AddStableReference(&ParameterCollectionInfo.ParameterCollection);
	}
}

void FMaterialCachedExpressionData::AppendReferencedFunctionIdsTo(TArray<FGuid>& Ids) const
{
	Ids.Reserve(Ids.Num() + FunctionInfos.Num());
	for (const FMaterialFunctionInfo& FunctionInfo : FunctionInfos)
	{
		Ids.AddUnique(FunctionInfo.StateId);
	}
}

void FMaterialCachedExpressionData::AppendReferencedParameterCollectionIdsTo(TArray<FGuid>& Ids) const
{
	Ids.Reserve(Ids.Num() + ParameterCollectionInfos.Num());
	for (const FMaterialParameterCollectionInfo& CollectionInfo : ParameterCollectionInfos)
	{
		Ids.AddUnique(CollectionInfo.StateId);
	}
}

uint64 FMaterialCachedExpressionData::GetExternalCodeReferencesHash() const
{
	FXxHash64Builder Hasher;
	for (const TObjectPtr<UClass>& ExternalCodeExpressionClass : ReferencedExternalCodeExpressionClasses)
	{
		if (const UObject* DefaultExternalCodeExpression = ExternalCodeExpressionClass->GetDefaultObject())
		{
			const UMaterialExpressionExternalCodeBase* ExternalCodeExpressionBase = CastChecked<UMaterialExpressionExternalCodeBase>(DefaultExternalCodeExpression);
			for (const FName& ExternalCodeIdentifier : ExternalCodeExpressionBase->ExternalCodeIdentifiers)
			{
				if (const FMaterialExternalCodeDeclaration* ExternalCodeDeclaration = MaterialExternalCodeRegistry::Get().FindExternalCode(ExternalCodeIdentifier))
				{
					ExternalCodeDeclaration->UpdateHash(Hasher);
				}
			}
		}
	}
	return Hasher.Finalize().Hash;
}

#if WITH_EDITOR
static bool TryAddParameter(FMaterialCachedExpressionData& CachedData,
	EMaterialParameterType Type,
	const FMaterialParameterInfo& ParameterInfo,
	const FMaterialCachedParameterEditorInfo& InEditorInfo,
	int32& OutIndex, 
	TOptional<FMaterialCachedParameterEditorInfo>& OutPreviousEditorInfo)
{
	check(CachedData.EditorOnlyData);
	FMaterialCachedParameterEntry& Entry = CachedData.GetParameterTypeEntry(Type);
	FMaterialCachedParameterEditorEntry& EditorEntry = CachedData.EditorOnlyData->EditorEntries[(int32)Type];

	FSetElementId ElementId = Entry.ParameterInfoSet.FindId(ParameterInfo);
	OutIndex = INDEX_NONE;
	if (!ElementId.IsValidId())
	{
		ElementId = Entry.ParameterInfoSet.Add(ParameterInfo);
		OutIndex = ElementId.AsInteger();
		EditorEntry.EditorInfo.Insert(InEditorInfo, OutIndex);
		// should be valid as long as we don't ever remove elements from ParameterInfoSet
		check(Entry.ParameterInfoSet.Num() == EditorEntry.EditorInfo.Num());
		return true;
	}

	// Update any editor values that haven't been set yet
	// TODO still need to do this??
	OutIndex = ElementId.AsInteger();

	FMaterialCachedParameterEditorInfo& EditorInfo = EditorEntry.EditorInfo[OutIndex];
	// Copy the previous parameter's original info before eventually replacing it, for error reporting purposes :
	OutPreviousEditorInfo.Emplace(EditorInfo);

	if (!EditorInfo.ExpressionGuid.IsValid())
	{
		EditorInfo.ExpressionGuid = InEditorInfo.ExpressionGuid;
	}
	if (EditorInfo.Description.IsEmpty())
	{
		EditorInfo.Description = InEditorInfo.Description;
	}
	if (EditorInfo.Group.IsNone())
	{
		EditorInfo.Group = InEditorInfo.Group;
		EditorInfo.SortPriority = InEditorInfo.SortPriority;
	}
	
	// Still return false, to signify this parameter was already added (don't want to add it again)
	return false;
}

FString FMaterialCachedExpressionData::GetParameterValueAsText(const FMaterialParameterMetadata& ParameterMeta)
{
	switch (ParameterMeta.Value.Type)
	{
	case EMaterialParameterType::Scalar:
		return FString::Printf(TEXT("%f"), ParameterMeta.Value.AsScalar());

	case EMaterialParameterType::Vector:
		return ParameterMeta.Value.AsLinearColor().ToString();

	case EMaterialParameterType::DoubleVector:
		return ParameterMeta.Value.AsVector4d().ToString();

	case EMaterialParameterType::Texture:
		return TSoftObjectPtr<UTexture>(ParameterMeta.Value.Texture).ToString();

	case EMaterialParameterType::TextureCollection:
		return TSoftObjectPtr<UTextureCollection>(ParameterMeta.Value.TextureCollection).ToString();

	case EMaterialParameterType::ParameterCollection:
		return TSoftObjectPtr<UMaterialParameterCollection>(ParameterMeta.Value.ParameterCollection).ToString();

	case EMaterialParameterType::Font:
		return FString::Printf(TEXT("%s(%i)"), *TSoftObjectPtr<UFont>(ParameterMeta.Value.Font.Value).ToString(), ParameterMeta.Value.Font.Page);

	case EMaterialParameterType::RuntimeVirtualTexture:
		return TSoftObjectPtr<URuntimeVirtualTexture>(ParameterMeta.Value.RuntimeVirtualTexture).ToString();

	case EMaterialParameterType::SparseVolumeTexture:
		return TSoftObjectPtr<USparseVolumeTexture>(ParameterMeta.Value.SparseVolumeTexture).ToString();

	case EMaterialParameterType::StaticSwitch:
		return FString::Printf(TEXT("%s%s"), ParameterMeta.Value.AsStaticSwitch() ? TEXT("true") : TEXT("false"), 
			ParameterMeta.bDynamicSwitchParameter ? TEXT("(dynamic)") : TEXT(""));

	case EMaterialParameterType::StaticComponentMask:
	{
		const FStaticComponentMaskValue MaskValue = ParameterMeta.Value.AsStaticComponentMask();
		return FString::Printf(TEXT("R=%s,G=%s,B=%s,A=%s")
			, MaskValue.R ? TEXT("true") : TEXT("false")
			, MaskValue.G ? TEXT("true") : TEXT("false")
			, MaskValue.B ? TEXT("true") : TEXT("false")
			, MaskValue.A ? TEXT("true") : TEXT("false"));
	}
	default:
		return {};
	}

}

// Returns whether a parameter that is a duplicate has the same value as the existing one
bool FMaterialCachedExpressionData::CheckDuplicateParameterValuesEqual(const int32 InExistingParamIdx, const FMaterialParameterMetadata& DuplicateParameterMeta)
{
	switch (DuplicateParameterMeta.Value.Type)
	{
	case EMaterialParameterType::Scalar:
		return ScalarValues[InExistingParamIdx] == DuplicateParameterMeta.Value.AsScalar();

	case EMaterialParameterType::Vector:
		return  VectorValues[InExistingParamIdx] == DuplicateParameterMeta.Value.AsLinearColor();

	case EMaterialParameterType::DoubleVector:
		return DoubleVectorValues[InExistingParamIdx] == DuplicateParameterMeta.Value.AsVector4d();

	case EMaterialParameterType::Texture:
		return TextureValues[InExistingParamIdx] == DuplicateParameterMeta.Value.Texture;

	case EMaterialParameterType::TextureCollection:
		return TextureCollectionValues[InExistingParamIdx] == DuplicateParameterMeta.Value.TextureCollection;

	case EMaterialParameterType::ParameterCollection:
		return ParameterCollectionValues[InExistingParamIdx] == DuplicateParameterMeta.Value.ParameterCollection;

	case EMaterialParameterType::Font:
		return FontValues[InExistingParamIdx] == DuplicateParameterMeta.Value.Font.Value && FontPageValues[InExistingParamIdx] == DuplicateParameterMeta.Value.Font.Page;

	case EMaterialParameterType::RuntimeVirtualTexture:
		return RuntimeVirtualTextureValues[InExistingParamIdx] == DuplicateParameterMeta.Value.RuntimeVirtualTexture;

	case EMaterialParameterType::SparseVolumeTexture:
		return SparseVolumeTextureValues[InExistingParamIdx] == DuplicateParameterMeta.Value.SparseVolumeTexture;

	case EMaterialParameterType::StaticSwitch:
		return (StaticSwitchValues[InExistingParamIdx] == DuplicateParameterMeta.Value.AsStaticSwitch()) && (DynamicSwitchValues[InExistingParamIdx] == DuplicateParameterMeta.bDynamicSwitchParameter);

	case EMaterialParameterType::StaticComponentMask:
		return EditorOnlyData->StaticComponentMaskValues[InExistingParamIdx] == DuplicateParameterMeta.Value.AsStaticComponentMask();

	default:
		return true;
	}
}

void FMaterialCachedExpressionData::InsertParameterValue(const FMaterialParameterMetadata& ParameterMeta, const int32 Index, UObject*& OutReferencedTexture, UTextureCollection*& OutReferencedTextureCollection)
{
	switch (ParameterMeta.Value.Type)
	{
	case EMaterialParameterType::Scalar:
		ScalarValues.Insert(ParameterMeta.Value.AsScalar(), Index);
		EditorOnlyData->ScalarMinMaxValues.Insert(FVector2D(ParameterMeta.ScalarMin, ParameterMeta.ScalarMax), Index);
		EditorOnlyData->ScalarEnumerationValues.Insert(ParameterMeta.ScalarEnumeration, Index);
		EditorOnlyData->ScalarEnumerationIndexValues.Insert(ParameterMeta.ScalarEnumerationIndex, Index);
		ScalarPrimitiveDataIndexValues.Insert(ParameterMeta.PrimitiveDataIndex, Index);
		if (ParameterMeta.bUsedAsAtlasPosition)
		{
			EditorOnlyData->ScalarCurveValues.Insert(ParameterMeta.ScalarCurve.Get(), Index);
			EditorOnlyData->ScalarCurveAtlasValues.Insert(ParameterMeta.ScalarAtlas.Get(), Index);
			OutReferencedTexture = ParameterMeta.ScalarAtlas.Get();
		}
		else
		{
			EditorOnlyData->ScalarCurveValues.Insert(nullptr, Index);
			EditorOnlyData->ScalarCurveAtlasValues.Insert(nullptr, Index);
		}
		break;

	case EMaterialParameterType::Vector:
		VectorValues.Insert(ParameterMeta.Value.AsLinearColor(), Index);
		EditorOnlyData->VectorChannelNameValues.Insert(ParameterMeta.ChannelNames, Index);
		EditorOnlyData->VectorUsedAsChannelMaskValues.Insert(ParameterMeta.bUsedAsChannelMask, Index);
		VectorPrimitiveDataIndexValues.Insert(ParameterMeta.PrimitiveDataIndex, Index);
		break;

	case EMaterialParameterType::DoubleVector:
		DoubleVectorValues.Insert(ParameterMeta.Value.AsVector4d(), Index);
		break;

	case EMaterialParameterType::Texture:
		TextureValues.Insert(ParameterMeta.Value.Texture, Index);
		EditorOnlyData->TextureChannelNameValues.Insert(ParameterMeta.ChannelNames, Index);
		OutReferencedTexture = ParameterMeta.Value.Texture;
		break;

	case EMaterialParameterType::TextureCollection:
		TextureCollectionValues.Insert(ParameterMeta.Value.TextureCollection, Index);
		OutReferencedTextureCollection = ParameterMeta.Value.TextureCollection;
		break;

	case EMaterialParameterType::ParameterCollection:
		ParameterCollectionValues.Insert(ParameterMeta.Value.ParameterCollection, Index);
		break;

	case EMaterialParameterType::Font:
		FontValues.Insert(ParameterMeta.Value.Font.Value, Index);
		FontPageValues.Insert(ParameterMeta.Value.Font.Page, Index);
		if (ParameterMeta.Value.Font.Value && ParameterMeta.Value.Font.Value->Textures.IsValidIndex(ParameterMeta.Value.Font.Page))
		{
			OutReferencedTexture = ParameterMeta.Value.Font.Value->Textures[ParameterMeta.Value.Font.Page];
		}
		break;

	case EMaterialParameterType::RuntimeVirtualTexture:
		RuntimeVirtualTextureValues.Insert(ParameterMeta.Value.RuntimeVirtualTexture, Index);
		OutReferencedTexture = ParameterMeta.Value.RuntimeVirtualTexture;
		break;

	case EMaterialParameterType::SparseVolumeTexture:
		SparseVolumeTextureValues.Insert(ParameterMeta.Value.SparseVolumeTexture, Index);
		OutReferencedTexture = ParameterMeta.Value.SparseVolumeTexture;
		break;

	case EMaterialParameterType::StaticSwitch:
		StaticSwitchValues.Insert(ParameterMeta.Value.AsStaticSwitch(), Index);
		DynamicSwitchValues.Insert(ParameterMeta.bDynamicSwitchParameter, Index);
		break;

	case EMaterialParameterType::StaticComponentMask:
		EditorOnlyData->StaticComponentMaskValues.Insert(ParameterMeta.Value.AsStaticComponentMask(), Index);
		break;

	default:
		checkNoEntry();
		break;
	}
}

void FMaterialCachedExpressionData::AddParameter(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& ParameterMeta, const FString& InAssetPath, UObject*& OutReferencedTexture, UTextureCollection*& OutReferencedTextureCollection, UMaterialExpression* InExpression)
{
	check(EditorOnlyData);

	if (!InAssetPath.IsEmpty())
	{
		FParameterAssetPathsEntry& Entry = EditorOnlyData->AssetPathsMap.FindOrAdd(ParameterInfo.Name);
		TMap<FString, int32>& AssetPaths = Entry.AssetPaths;

		int32& PathsCounter = AssetPaths.FindOrAdd(InAssetPath);
		++PathsCounter;
	}

	const FMaterialCachedParameterEditorInfo EditorInfo(ParameterMeta.ExpressionGuid, ParameterMeta.Description, ParameterMeta.Group, ParameterMeta.SortPriority);
	int32 Index = INDEX_NONE;
	TOptional<FMaterialCachedParameterEditorInfo> PreviousEditorInfo;

	const bool bNonDuplicate = TryAddParameter(*this, ParameterMeta.Value.Type, ParameterInfo, EditorInfo, Index, PreviousEditorInfo);
	if (bNonDuplicate)
	{
		InsertParameterValue(ParameterMeta, Index, OutReferencedTexture, OutReferencedTextureCollection);
	}

	FParsedExpressionData ExpressionData;
	ExpressionData.ParameterIdx = Index;
	ExpressionData.ParameterMeta = ParameterMeta;
	ExpressionData.Expression = InExpression;
	ExpressionData.ParameterName = ParameterInfo.Name;

	ParsedExpressions.Add(ExpressionData);
}

void FMaterialCachedExpressionData::UpdateForFunction(const FMaterialCachedExpressionContext& Context, UMaterialFunctionInterface* Function, EMaterialParameterAssociation Association, int32 ParameterIndex)
{
	if (!Function)
	{
		return;
	}

	// Update expressions for all dependent functions first, before processing the remaining expressions in this function
	// This is important so we add parameters in the proper order (parameter values are latched the first time a given parameter name is encountered)
	FMaterialCachedExpressionContext LocalContext(Context);
	LocalContext.CurrentFunction = Function;
	LocalContext.bUpdateFunctionExpressions = false; // we update functions explicitly
	
	FMaterialCachedExpressionData* Self = this;
	auto ProcessFunction = [Self, &LocalContext, Association, ParameterIndex](UMaterialFunctionInterface* InFunction) -> bool
	{
		Self->UpdateForExpressions(LocalContext, InFunction->GetExpressions(), Association, ParameterIndex);

		FMaterialFunctionInfo NewFunctionInfo;
		NewFunctionInfo.Function = InFunction;
		NewFunctionInfo.StateId = InFunction->StateId;
		Self->FunctionInfos.Add(NewFunctionInfo);
		Self->FunctionInfosStateCRC = FCrc::TypeCrc32(InFunction->StateId, Self->FunctionInfosStateCRC);

		return true;
	};
	Function->IterateDependentFunctions(ProcessFunction);

	ProcessFunction(Function);
}

void FMaterialCachedExpressionData::UpdateForLayerFunctions(const FMaterialCachedExpressionContext& Context, const FMaterialLayersFunctions& LayerFunctions)
{
	for (int32 LayerIndex = 0; LayerIndex < LayerFunctions.Layers.Num(); ++LayerIndex)
	{
		UpdateForFunction(Context, LayerFunctions.Layers[LayerIndex], LayerParameter, LayerIndex);
	}

	for (int32 BlendIndex = 0; BlendIndex < LayerFunctions.Blends.Num(); ++BlendIndex)
	{
		UpdateForFunction(Context, LayerFunctions.Blends[BlendIndex], BlendParameter, BlendIndex);
	}
}

void FMaterialCachedExpressionData::UpdateForExpressions(const FMaterialCachedExpressionContext& Context, TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions, EMaterialParameterAssociation Association, int32 ParameterIndex)
{
	check(EditorOnlyData);
	static const FGuid FirstPersonInterpolationAlphaGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("FirstPersonInterpolationAlpha"));
	static const FGuid TemporalResponsivenessGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("TemporalResponsiveness"));
	static const FGuid MotionVectorWorldOffsetGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("MotionVectorWorldOffset"));

	// Enabled specific quality levels based on material quality level closure count difference as compared to ignoring per material quality level constraints.
	// See r.Substrate.MaterialQualityLevelsClosuresPerPixel.
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform))
	{
		uint32 ReferenceClosurePerPixelCount = Substrate::GetClosurePerPixel(GMaxRHIShaderPlatform, Substrate::IgnoreMaterialQualityLevelsForClosurePerPixel);
		for (int32 MaterialQualityLevel = 0; MaterialQualityLevel < EMaterialQualityLevel::Num; MaterialQualityLevel++)
		{
			if (ReferenceClosurePerPixelCount != Substrate::GetClosurePerPixel(GMaxRHIShaderPlatform, MaterialQualityLevel))
			{
				QualityLevelsUsed[MaterialQualityLevel] = true;
			}
		}
	}

	for (UMaterialExpression* Expression : Expressions)
	{
		if (!Expression)
		{
			continue;
		}

		UObject* ReferencedTexture = nullptr;
		UTextureCollection* ReferencedTextureCollection = nullptr;

		// Add any expression specific custom shader tags
		TArray<FName> ShaderTags;
		Expression->GetShaderTags(ShaderTags);
		EditorOnlyData->ShaderTags.Append(ShaderTags);

		FMaterialParameterMetadata ParameterMeta;
		FText ErrorContext;
		if (Expression->GetParameterValue(ParameterMeta))
		{
			const FName ParameterName = Expression->GetParameterName();

			// If we're processing a function, give that a chance to override the parameter value
			if (Context.CurrentFunction)
			{
				FMaterialParameterMetadata OverrideParameterMeta;
				if (Context.CurrentFunction->GetParameterOverrideValue(ParameterMeta.Value.Type, ParameterName, OverrideParameterMeta))
				{
					ParameterMeta.Value = OverrideParameterMeta.Value;
					ParameterMeta.ExpressionGuid = OverrideParameterMeta.ExpressionGuid;
					ParameterMeta.bUsedAsAtlasPosition = OverrideParameterMeta.bUsedAsAtlasPosition;
					ParameterMeta.ScalarAtlas = OverrideParameterMeta.ScalarAtlas;
					ParameterMeta.ScalarCurve = OverrideParameterMeta.ScalarCurve;
				}
			}

			const FMaterialParameterInfo ParameterInfo(ParameterName, Association, ParameterIndex);

			const FString AssetPath = Expression->GetAssetPathName();

			// Try add the parameter. If this fails, the parameter is being added twice with different values. Report it as error.
			AddParameter(ParameterInfo, ParameterMeta, AssetPath, ReferencedTexture, ReferencedTextureCollection, Expression);
		}
		else
		{
			FParsedExpressionData ExpressionData;
			ExpressionData.Expression = Expression;
			ParsedExpressions.Add(ExpressionData);
		}


		if (ReferencedTexture)
		{
			EditorOnlyData->EditorReferencedDefaultTextures.AddUnique(ReferencedTexture);
		}
		else if (ReferencedTextureCollection)
		{
			ReferencedTextureCollections.AddUnique(ReferencedTextureCollection);
		}
		else if (UTextureCollection* TextureCollection = Expression->GetReferencedTextureCollection())
		{
			ReferencedTextureCollections.AddUnique(TextureCollection);
		}
		else if (Expression->CanReferenceTexture())
		{
			// We first try to extract the referenced texture from the parameter value, that way we'll also get the proper texture in case value is overriden by a function instance
			const UMaterialExpression::ReferencedTextureArray ExpressionReferencedTextures = Expression->GetReferencedTextures();
			for (UObject* ExpressionReferencedTexture : ExpressionReferencedTextures)
			{
				EditorOnlyData->EditorReferencedDefaultTextures.AddUnique(ExpressionReferencedTexture);
			}
		}

		Expression->GetLandscapeLayerNames(EditorOnlyData->LandscapeLayerNames);

		Expression->GetIncludeFilePaths(EditorOnlyData->ExpressionIncludeFilePaths);

		if (UMaterialExpressionUserSceneTexture* ExpressionUserSceneTexture = Cast<UMaterialExpressionUserSceneTexture>(Expression))
		{
			if (!ExpressionUserSceneTexture->UserSceneTexture.IsNone())
			{
				EditorOnlyData->UserSceneTextureInputs.Add(ExpressionUserSceneTexture->UserSceneTexture);
			}
		}

		if (UMaterialExpressionExternalCodeBase* ExternalCodeExpression = Cast<UMaterialExpressionExternalCodeBase>(Expression))
		{
			ReferencedExternalCodeExpressionClasses.AddUnique(Expression->GetClass());
		}

		if (UMaterialExpressionCollectionParameter* ExpressionCollectionParameter = Cast<UMaterialExpressionCollectionParameter>(Expression))
		{
			UMaterialParameterCollection* Collection = ExpressionCollectionParameter->Collection;
			if (Collection)
			{
				FMaterialParameterCollectionInfo NewInfo;
				NewInfo.ParameterCollection = Collection;
				NewInfo.StateId = Collection->StateId;
				ParameterCollectionInfos.AddUnique(NewInfo);
			}
		}
		else if (UMaterialExpressionCollectionTransform* ExpressionCollectionTransform = Cast<UMaterialExpressionCollectionTransform>(Expression))
		{
			UMaterialParameterCollection* Collection = ExpressionCollectionTransform->Collection;
			if (Collection)
			{
				FMaterialParameterCollectionInfo NewInfo;
				NewInfo.ParameterCollection = Collection;
				NewInfo.StateId = Collection->StateId;
				ParameterCollectionInfos.AddUnique(NewInfo);
			}
		}
		else if (UMaterialExpressionDynamicParameter* ExpressionDynamicParameter = Cast< UMaterialExpressionDynamicParameter>(Expression))
		{
			DynamicParameterNames.Empty(ExpressionDynamicParameter->ParamNames.Num());
			for (const FString& Name : ExpressionDynamicParameter->ParamNames)
			{
				DynamicParameterNames.Add(*Name);
			}
		}
		else if (UMaterialExpressionLandscapeGrassOutput* ExpressionGrassOutput = Cast<UMaterialExpressionLandscapeGrassOutput>(Expression))
		{
			for (const FGrassInput& Input : ExpressionGrassOutput->GrassTypes)
			{
				NamedGrassTypes.Add(Input.Name, Input.GrassType);
			}
		}
		else if (UMaterialExpressionQualitySwitch* QualitySwitchNode = Cast<UMaterialExpressionQualitySwitch>(Expression))
		{
			const FExpressionInput DefaultInput = QualitySwitchNode->Default.GetTracedInput();

			for (int32 InputIndex = 0; InputIndex < EMaterialQualityLevel::Num; InputIndex++)
			{
				if (QualitySwitchNode->Inputs[InputIndex].IsConnected())
				{
					// We can ignore quality levels that are defined the same way as 'Default'
					// This avoids compiling a separate explicit quality level resource, that will end up exactly the same as the default resource
					const FExpressionInput Input = QualitySwitchNode->Inputs[InputIndex].GetTracedInput();
					if (Input.Expression != DefaultInput.Expression ||
						Input.OutputIndex != DefaultInput.OutputIndex)
					{
						QualityLevelsUsed[InputIndex] = true;
					}
				}
			}
		}
		else if (Expression->IsA(UMaterialExpressionRuntimeVirtualTextureOutput::StaticClass()))
		{
			bHasRuntimeVirtualTextureOutput = true;
		}
		else if (Expression->IsA(UMaterialExpressionFirstPersonOutput::StaticClass()))
		{
			bHasFirstPersonOutput = true;
		}
		else if (UMaterialExpressionTemporalResponsivenessOutput* TemporalResponsivenessOutputNode = Cast<UMaterialExpressionTemporalResponsivenessOutput>(Expression))
		{
			if (TemporalResponsivenessOutputNode->Input.IsConnected())
			{
				bUsesTemporalResponsiveness = true;
			}

		}else if (UMaterialExpressionMotionVectorWorldOffsetOutput* MotionVectorWorldOffsetOutputNode = Cast<UMaterialExpressionMotionVectorWorldOffsetOutput>(Expression))
		{
			if (MotionVectorWorldOffsetOutputNode->Input.IsConnected())
			{
				bUsesMotionVectorWorldOffset = true;
			}
		}
		else if (UMaterialExpressionMaterialCache* ExpressionCache = Cast<UMaterialExpressionMaterialCache>(Expression))
		{
			MaterialCacheTags.Add(ExpressionCache->Tag);

			// Any cache usage allows for sampling
			bSamplesMaterialCache = true;

			// Sample expressions do not output cache data
			bHasMaterialCacheOutput = !ExpressionCache->bIsSample;
		}
		else if (Expression->IsA(UMaterialExpressionSceneColor::StaticClass()))
		{
			bHasSceneColor = true;
		}
		else if (Expression->IsA(UMaterialExpressionPerInstanceRandom::StaticClass()))
		{
			bHasPerInstanceRandom = true;
		}
		else if (Expression->IsA(UMaterialExpressionPerInstanceCustomData::StaticClass()))
		{
			bHasPerInstanceCustomData = true;
		}
		else if (Expression->IsA(UMaterialExpressionPerInstanceCustomData3Vector::StaticClass()))
		{
			bHasPerInstanceCustomData = true;
		}
		else if (Expression->IsA(UMaterialExpressionVertexInterpolator::StaticClass()))
		{
			bHasVertexInterpolator = true;
		}
		else if (Expression->IsA(UMaterialExpressionMeshPaintTextureObject::StaticClass()))
		{
			bHasMeshPaintTexture = true;
		}
		else if (Expression->IsA(UMaterialExpressionWorldPosition::StaticClass()))
		{
			bHasWorldPosition = true;
		}
		else if (Expression->IsA(UMaterialExpressionActorPositionWS::StaticClass()))
		{
			bHasWorldPosition = true;
		}
		else if (UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
		{
			checkf(Association == GlobalParameter, TEXT("UMaterialExpressionMaterialAttributeLayers can't be nested"));
			// Only a single layers expression is allowed/expected...creating additional layer expression will cause a compile error
			if (!bHasMaterialLayers)
			{
				const FMaterialLayersFunctions& Layers = Context.LayerOverrides ? *Context.LayerOverrides : LayersExpression->DefaultLayers;
				UpdateForLayerFunctions(Context, Layers);
				if (UMaterialExpressionLayerStack* LayerStackExpression = Cast<UMaterialExpressionLayerStack>(LayersExpression))
				{
					LayerStackExpression->ResolveLayerInputs();
					Layers.LayerStackCache = LayerStackExpression->GetSharedAvailableFunctionsCache();
				}

				// TODO(?) - Layers for MIs are currently duplicated here and in FStaticParameterSet
				bHasMaterialLayers = true;
				MaterialLayers = Layers.GetRuntime();
				EditorOnlyData->MaterialLayers = Layers.EditorOnly;
				FMaterialLayersFunctions::Validate(MaterialLayers, EditorOnlyData->MaterialLayers);
				LayersExpression->RebuildLayerGraph(false);
			}
		}
		else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (Context.bUpdateFunctionExpressions)
			{
				UpdateForFunction(Context, FunctionCall->MaterialFunction, GlobalParameter, -1);

				// Update the function call node, so it can relink inputs and outputs as needed
				// Update even if MaterialFunctionNode->MaterialFunction is NULL, because we need to remove the invalid inputs in that case
				FunctionCall->UpdateFromFunctionResource();
			}
		}
		else if (UMaterialExpressionSetMaterialAttributes* SetMatAttributes = Cast<UMaterialExpressionSetMaterialAttributes>(Expression))
		{
			for (int32 PinIndex = 0; PinIndex < SetMatAttributes->AttributeSetTypes.Num(); ++PinIndex)
			{
				// For this material attribute pin do we have something connected?
				const FGuid& Guid = SetMatAttributes->AttributeSetTypes[PinIndex];
				const FExpressionInput& AttributeInput = SetMatAttributes->Inputs[PinIndex + 1];
				const EMaterialProperty MaterialProperty = FMaterialAttributeDefinitionMap::GetProperty(Guid);
				if (AttributeInput.Expression)
				{
					SetPropertyConnected(MaterialProperty);
					if (Guid == FirstPersonInterpolationAlphaGuid)
					{
						bHasFirstPersonOutput = true;
					}
					else if (Guid == TemporalResponsivenessGuid)
					{
						bUsesTemporalResponsiveness = true;
					}
					else if (Guid == MotionVectorWorldOffsetGuid)
					{
						bUsesMotionVectorWorldOffset = true;
					}
				}
			}
		}
		else if (UMaterialExpressionMakeMaterialAttributes* MakeMatAttributes = Cast<UMaterialExpressionMakeMaterialAttributes>(Expression))
		{
			auto SetMatAttributeConditionally = [&](EMaterialProperty InMaterialProperty, bool InIsConnected)
			{
				if (InIsConnected)
				{
					SetPropertyConnected(InMaterialProperty);
				}
			};

			SetMatAttributeConditionally(EMaterialProperty::MP_BaseColor, MakeMatAttributes->BaseColor.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Metallic, MakeMatAttributes->Metallic.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Specular, MakeMatAttributes->Specular.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Roughness, MakeMatAttributes->Roughness.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Anisotropy, MakeMatAttributes->Anisotropy.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_EmissiveColor, MakeMatAttributes->EmissiveColor.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Opacity, MakeMatAttributes->Opacity.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_OpacityMask, MakeMatAttributes->OpacityMask.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Normal, MakeMatAttributes->Normal.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Tangent, MakeMatAttributes->Tangent.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_WorldPositionOffset, MakeMatAttributes->WorldPositionOffset.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_SubsurfaceColor, MakeMatAttributes->SubsurfaceColor.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomData0, MakeMatAttributes->ClearCoat.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomData1, MakeMatAttributes->ClearCoatRoughness.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_AmbientOcclusion, MakeMatAttributes->AmbientOcclusion.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Refraction, MakeMatAttributes->Refraction.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs0, MakeMatAttributes->CustomizedUVs[0].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs1, MakeMatAttributes->CustomizedUVs[1].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs2, MakeMatAttributes->CustomizedUVs[2].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs3, MakeMatAttributes->CustomizedUVs[3].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs4, MakeMatAttributes->CustomizedUVs[4].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs5, MakeMatAttributes->CustomizedUVs[5].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs6, MakeMatAttributes->CustomizedUVs[6].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_CustomizedUVs7, MakeMatAttributes->CustomizedUVs[7].IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_PixelDepthOffset, MakeMatAttributes->PixelDepthOffset.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_ShadingModel, MakeMatAttributes->ShadingModel.IsConnected());
			SetMatAttributeConditionally(EMaterialProperty::MP_Displacement, MakeMatAttributes->Displacement.IsConnected());
		}
	}

	if (bHasMaterialCacheOutput)
	{
		for (UMaterialExpression* MaterialExpression : Expressions)
		{
			if (MaterialExpression && MaterialCacheIsExpressionNonUVDerived(MaterialExpression, MaterialCacheUVCoordinatesUsedMask))
			{
				bMaterialCacheHasNonUVDerivedExpression = true;
				break;
			}
		}
	}
}

void FMaterialCachedExpressionData::UpdatePropertyConnections(UMaterial& Material)
{
	if (!Material.bUseMaterialAttributes)
	{
		for (int32 PropertyIndex = 0; PropertyIndex < MP_MAX; ++PropertyIndex)
		{
			const EMaterialProperty Property = (EMaterialProperty)PropertyIndex;
			const FExpressionInput* Input = Material.GetExpressionInputForProperty(Property);
			if (Input && Input->IsConnected())
			{
				SetPropertyConnected(Property);
			}
		}
	}
}

#if WITH_EDITOR
void FMaterialCachedExpressionData::DoPostAnalyzeChecks(const UMaterial& BaseMat)
{
	static const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Material.PedanticErrorChecksEnabled"));
	const bool bPedanticErrorChecksEnabled = CVar && CVar->GetBool();

	for (const FParsedExpressionData& ParameterData : ParsedExpressions)
	{
		// Legacy materials may have a static switch with MaterialAttributes on one branch
		// and a non-MaterialAttributes value (e.g. scalar constant) on the other.
		// During translation, the non-MaterialAttributes branch gets broadcast to ALL material properties, causing
		// IsMaterialPropertyUsed() to report them as used, which leads to wrong shader parameters.
		// To fix this, detect the mismatch here and conservatively mark all properties as connected for those legacy materials.
		{
			auto MarkAllPropertiesConnectedForMismatchedMaterialAttributesSwitch = [&](const FExpressionInput& InputA, const FExpressionInput& InputB)
			{
				const FExpressionInput TracedA = InputA.GetTracedInput();
				const FExpressionInput TracedB = InputB.GetTracedInput();

				if (!TracedA.IsConnected() || !TracedB.IsConnected())
				{
					return;
				}

				const bool bAIsMaterialAttributes = TracedA.Expression->IsResultMaterialAttributes(TracedA.OutputIndex);
				const bool bBIsMaterialAttributes = TracedB.Expression->IsResultMaterialAttributes(TracedB.OutputIndex);
				const bool bAIsSubstrateAttributes = TracedA.Expression->IsResultSubstrateMaterial(TracedA.OutputIndex);
				const bool bBIsSubstrateAttributes = TracedB.Expression->IsResultSubstrateMaterial(TracedB.OutputIndex);

				if (bAIsMaterialAttributes != bBIsMaterialAttributes)
				{
					for (int32 PropertyIndex = 0; PropertyIndex < MP_MaterialAttributes; ++PropertyIndex)
					{
						// Substrate specific property handled separately.
						if (PropertyIndex != MP_FrontMaterial || bAIsSubstrateAttributes != bBIsSubstrateAttributes)
						{
							SetPropertyConnected((EMaterialProperty)PropertyIndex);
						}
					}
				}
			};

			if (UMaterialExpressionStaticSwitch* StaticSwitch = Cast<UMaterialExpressionStaticSwitch>(ParameterData.Expression))
			{
				MarkAllPropertiesConnectedForMismatchedMaterialAttributesSwitch(StaticSwitch->A, StaticSwitch->B);
			}
			else if (UMaterialExpressionStaticSwitchParameter* StaticSwitchParam = Cast<UMaterialExpressionStaticSwitchParameter>(ParameterData.Expression))
			{
				MarkAllPropertiesConnectedForMismatchedMaterialAttributesSwitch(StaticSwitchParam->A, StaticSwitchParam->B);
			}
		}

		if (bPedanticErrorChecksEnabled)
		{
			if (ParameterData.ParameterIdx == INDEX_NONE)
			{
				continue;
			}

			const bool bEqualValues = CheckDuplicateParameterValuesEqual(ParameterData.ParameterIdx, ParameterData.ParameterMeta);
			const FString CurrentParamValueText = GetParameterValueAsText(ParameterData.ParameterMeta);

			FParamExpressionsValues& ParamExpressionsValues = ParameterExpressionsAndValues.FindOrAdd(ParameterData.ParameterName);
			ParamExpressionsValues.ExpressionsValues.AddUnique({ ParameterData.Expression,  CurrentParamValueText });
			ParamExpressionsValues.bHasDuplicatesWithDifferentValues |= !bEqualValues;
		}
	}
}
#endif

void FMaterialCachedExpressionData::AnalyzeMaterial(UMaterial& Material, const FMaterialLayersFunctions* LayersOverrides)
{

	UpdatePropertyConnections(Material);
		
	FMaterialCachedExpressionContext Context;
	Context.LayerOverrides = LayersOverrides;

	UpdateForExpressions(Context, Material.GetExpressions(), EMaterialParameterAssociation::GlobalParameter, INDEX_NONE);

#if WITH_EDITOR
	DoPostAnalyzeChecks(Material);
	ParsedExpressions.Empty();
#endif
}

void FMaterialCachedExpressionData::Validate(const UMaterialInterface& Material)
{
	if (EditorOnlyData)
	{
		for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
		{
			const FMaterialCachedParameterEditorEntry& EditorEntry = EditorOnlyData->EditorEntries[TypeIndex];
			const FMaterialCachedParameterEntry& Entry = GetParameterTypeEntry((EMaterialParameterType)TypeIndex);
			check(EditorEntry.EditorInfo.Num() == Entry.ParameterInfoSet.Num());
		}
		FMaterialLayersFunctions::Validate(MaterialLayers, EditorOnlyData->MaterialLayers);

		TArray<FShaderCompilerError> Errors;
		if (!FPlatformProperties::RequiresCookedData() && AllowShaderCompiling())
		{
			for (auto PathIt = EditorOnlyData->ExpressionIncludeFilePaths.CreateIterator(); PathIt; ++PathIt)
			{
				const FString& IncludeFilePath = *PathIt;
				bool bValidExpressionIncludePath = false;

				if (!IncludeFilePath.IsEmpty())
				{
					// Mute log errors by using the variant that gives use the errors instead of logging them.
					FString ValidatedPath = GetShaderSourceFilePath(IncludeFilePath, &Errors);

					// Keep the array small during the loop since we don't care about errors.
					Errors.Reset();

					if (!ValidatedPath.IsEmpty())
					{
						ValidatedPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ValidatedPath);
						if (FPaths::FileExists(ValidatedPath))
						{
							bValidExpressionIncludePath = true;
						}
					}
				}

				if (!bValidExpressionIncludePath)
				{
					UE_LOGF(LogMaterial, Warning, "Expression include file path '%ls' is invalid, removing from cached data for material '%ls'.", *IncludeFilePath, *Material.GetPathName());
					PathIt.RemoveCurrent();
				}
			}
		}

		// Sort to make hashing less dependent on the order of expression visiting
		EditorOnlyData->ExpressionIncludeFilePaths.Sort(TLess<>());
	}
}

#endif // WITH_EDITOR

int32 FMaterialCachedExpressionData::FindParameterIndex(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo) const
{
	const FMaterialCachedParameterEntry& Entry = GetParameterTypeEntry(Type);
	const FSetElementId ElementId = Entry.ParameterInfoSet.FindId(FMaterialParameterInfo(ParameterInfo));
	return ElementId.AsInteger();
}

void FMaterialCachedExpressionData::GetParameterValueByIndex(EMaterialParameterType Type, int32 ParameterIndex, FMaterialParameterMetadata& OutResult) const
{
#if WITH_EDITORONLY_DATA
	bool bIsEditorOnlyDataStripped = true;
	if (EditorOnlyData)
	{
		const FMaterialCachedParameterEditorEntry& EditorEntry = EditorOnlyData->EditorEntries[(int32)Type];
		bIsEditorOnlyDataStripped = EditorEntry.EditorInfo.Num() == 0;
		if (!bIsEditorOnlyDataStripped)
		{
			const FMaterialCachedParameterEditorInfo& EditorInfo = EditorEntry.EditorInfo[ParameterIndex];
			OutResult.ExpressionGuid = EditorInfo.ExpressionGuid;
			OutResult.Description = EditorInfo.Description;
			OutResult.Group = EditorInfo.Group;
			OutResult.SortPriority = EditorInfo.SortPriority;
		}
	}
#endif // WITH_EDITORONLY_DATA

	switch (Type)
	{
	case EMaterialParameterType::Scalar:
		OutResult.Value = ScalarValues[ParameterIndex];
		OutResult.PrimitiveDataIndex = ScalarPrimitiveDataIndexValues[ParameterIndex];
#if WITH_EDITORONLY_DATA
		if (EditorOnlyData && !bIsEditorOnlyDataStripped)
		{
			OutResult.ScalarMin = EditorOnlyData->ScalarMinMaxValues[ParameterIndex].X;
			OutResult.ScalarMax = EditorOnlyData->ScalarMinMaxValues[ParameterIndex].Y;
			OutResult.ScalarEnumeration = EditorOnlyData->ScalarEnumerationValues[ParameterIndex];
			OutResult.ScalarEnumerationIndex = EditorOnlyData->ScalarEnumerationIndexValues[ParameterIndex];
			{
				const TSoftObjectPtr<UCurveLinearColor>& Curve = EditorOnlyData->ScalarCurveValues[ParameterIndex];
				const TSoftObjectPtr<UCurveLinearColorAtlas>& Atlas = EditorOnlyData->ScalarCurveAtlasValues[ParameterIndex];
				if (!Curve.IsNull() && !Atlas.IsNull())
				{
					OutResult.ScalarCurve = Curve;
					OutResult.ScalarAtlas = Atlas;
					OutResult.bUsedAsAtlasPosition = true;
				}
			}
		}
#endif // WITH_EDITORONLY_DATA
		break;
	case EMaterialParameterType::Vector:
		OutResult.Value = VectorValues[ParameterIndex];
		OutResult.PrimitiveDataIndex = VectorPrimitiveDataIndexValues[ParameterIndex];
#if  WITH_EDITORONLY_DATA
		if (EditorOnlyData && !bIsEditorOnlyDataStripped)
		{
			OutResult.ChannelNames = EditorOnlyData->VectorChannelNameValues[ParameterIndex];
			OutResult.bUsedAsChannelMask = EditorOnlyData->VectorUsedAsChannelMaskValues[ParameterIndex];
		}
#endif // WITH_EDITORONLY_DATA
		break;
	case EMaterialParameterType::DoubleVector:
		OutResult.Value = DoubleVectorValues[ParameterIndex];
		break;
	case EMaterialParameterType::Texture:
		OutResult.Value = TextureValues[ParameterIndex].LoadSynchronous();
#if WITH_EDITORONLY_DATA
		if (EditorOnlyData && !bIsEditorOnlyDataStripped)
		{
			OutResult.ChannelNames = EditorOnlyData->TextureChannelNameValues[ParameterIndex];
		}
#endif // WITH_EDITORONLY_DATA
		break;
	case EMaterialParameterType::TextureCollection:
		OutResult.Value = TextureCollectionValues[ParameterIndex].LoadSynchronous();
		break;
	case EMaterialParameterType::ParameterCollection:
		OutResult.Value = ParameterCollectionValues[ParameterIndex].LoadSynchronous();
		break;
	case EMaterialParameterType::RuntimeVirtualTexture:
		OutResult.Value = RuntimeVirtualTextureValues[ParameterIndex].LoadSynchronous();
		break;
	case EMaterialParameterType::SparseVolumeTexture:
		OutResult.Value = SparseVolumeTextureValues[ParameterIndex].LoadSynchronous();
		break;
	case EMaterialParameterType::Font:
		OutResult.Value = FMaterialParameterValue(FontValues[ParameterIndex].LoadSynchronous(), FontPageValues[ParameterIndex]);
		break;
	case EMaterialParameterType::StaticSwitch:
		OutResult.Value = StaticSwitchValues[ParameterIndex];
		OutResult.bDynamicSwitchParameter = DynamicSwitchValues[ParameterIndex];
		break;
#if WITH_EDITORONLY_DATA
	case EMaterialParameterType::StaticComponentMask:
		if (EditorOnlyData && !bIsEditorOnlyDataStripped)
		{
			OutResult.Value = EditorOnlyData->StaticComponentMaskValues[ParameterIndex];
		}
		break;
#endif // WITH_EDITORONLY_DATA
	default:
		checkNoEntry();
		break;
	}
}

bool FMaterialCachedExpressionData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	return false;
}

void FMaterialCachedExpressionData::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::IncreaseMaterialAttributesInputMask)
		{
			PropertyConnectedMask = uint64(PropertyConnectedBitmask_DEPRECATED);
		}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Handle loading cached expression data where grass types were still serialized as an array. This could happen when loading a material with bIsCookedForEditor,
		//  in which case we wouldn't have access to the grass name associated with the ULandscapeGrassType asset, since it's part of UMaterialExpressionLandscapeGrassOutput and is only 
		//  updated when UpdateCachedExpressionData is called (which is not the case for bIsCookedForEditor stuff)
		if (!GrassTypes_DEPRECATED.IsEmpty())
		{
			check(NamedGrassTypes.IsEmpty()); // This situation should only happen when loading cooked data that doesn't have the new, named, grass types yet
			for (TObjectPtr<ULandscapeGrassType> GrassType : GrassTypes_DEPRECATED)
			{
				if (GrassType != nullptr)
				{
					// Just use the grass type's fallback name as a fallback :
					NamedGrassTypes.Add(GrassType->GetGrassNameFallback(), GrassType);
				}
			}
			GrassTypes_DEPRECATED.Empty();
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	}

#if WITH_EDITORONLY_DATA
	if(Ar.IsLoading())
	{
		bool bIsEditorOnlyDataStripped = true;
		if (EditorOnlyData)
		{
			const FMaterialCachedParameterEditorEntry& EditorEntry = EditorOnlyData->EditorEntries[(int32)EMaterialParameterType::StaticSwitch];
			bIsEditorOnlyDataStripped = EditorEntry.EditorInfo.Num() == 0;
		}

		if (EditorOnlyData && !bIsEditorOnlyDataStripped)
		{
			StaticSwitchValues = EditorOnlyData->StaticSwitchValues_DEPRECATED;
			check(DynamicSwitchValues.Num() == 0);
			DynamicSwitchValues.AddDefaulted(StaticSwitchValues.Num());
		}
	}
#endif
}

bool FMaterialCachedExpressionData::GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutResult) const
{
	const int32 Index = FindParameterIndex(Type, ParameterInfo);
	if (Index != INDEX_NONE)
	{
		GetParameterValueByIndex(Type, Index, OutResult);
		return true;
	}

	return false;
}

const FGuid& FMaterialCachedExpressionData::GetExpressionGuid(EMaterialParameterType Type, int32 Index) const
{
#if WITH_EDITORONLY_DATA
	if (EditorOnlyData)
	{
		// cooked materials can strip out expression guids
		if (EditorOnlyData->EditorEntries[(int32)Type].EditorInfo.Num() != 0)
		{
			return EditorOnlyData->EditorEntries[(int32)Type].EditorInfo[Index].ExpressionGuid;
		}
	}
#endif // WITH_EDITORONLY_DATA
	static const FGuid EmptyGuid;
	return EmptyGuid;
}

void FMaterialCachedExpressionData::GetAllParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const
{
	const FMaterialCachedParameterEntry& Entry = GetParameterTypeEntry(Type);
	const int32 NumParameters = Entry.ParameterInfoSet.Num();
	OutParameters.Reserve(OutParameters.Num() + NumParameters);

	for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
	{
		const FMaterialParameterInfo& ParameterInfo = Entry.ParameterInfoSet[FSetElementId::FromInteger(ParameterIndex)];
		FMaterialParameterMetadata& Result = OutParameters.Emplace(ParameterInfo);
		GetParameterValueByIndex(Type, ParameterIndex, Result);
	}
}

void FMaterialCachedExpressionData::GetAllParameterInfoOfType(EMaterialParameterType Type, TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	const FMaterialCachedParameterEntry& Entry = GetParameterTypeEntry(Type);
	const int32 NumParameters = Entry.ParameterInfoSet.Num();
	OutParameterInfo.Reserve(OutParameterInfo.Num() + NumParameters);
	OutParameterIds.Reserve(OutParameterIds.Num() + NumParameters);

	for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
	{
		const int32 ParameterIndex = It.GetId().AsInteger();
		OutParameterInfo.Add(*It);
		OutParameterIds.Add(GetExpressionGuid(Type, ParameterIndex));
	}
}

void FMaterialCachedExpressionData::GetAllGlobalParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const
{
	const FMaterialCachedParameterEntry& Entry = GetParameterTypeEntry(Type);
	const int32 NumParameters = Entry.ParameterInfoSet.Num();
	OutParameters.Reserve(OutParameters.Num() + NumParameters);

	for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
	{
		const FMaterialParameterInfo& ParameterInfo = Entry.ParameterInfoSet[FSetElementId::FromInteger(ParameterIndex)];
		if (ParameterInfo.Association == GlobalParameter)
		{
			FMaterialParameterMetadata& Meta = OutParameters.FindOrAdd(ParameterInfo);
			if (Meta.Value.Type == EMaterialParameterType::None)
			{
				GetParameterValueByIndex(Type, ParameterIndex, Meta);
			}
		}
	}
}

void FMaterialCachedExpressionData::GetAllGlobalParameterInfoOfType(EMaterialParameterType Type, TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	const FMaterialCachedParameterEntry& Entry = GetParameterTypeEntry(Type);
	const int32 NumParameters = Entry.ParameterInfoSet.Num();
	OutParameterInfo.Reserve(OutParameterInfo.Num() + NumParameters);
	OutParameterIds.Reserve(OutParameterIds.Num() + NumParameters);

	for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
	{
		const FMaterialParameterInfo& ParameterInfo = *It;
		if (ParameterInfo.Association == GlobalParameter)
		{
			const int32 ParameterIndex = It.GetId().AsInteger();
			OutParameterInfo.Add(*It);
			OutParameterIds.Add(GetExpressionGuid(Type, ParameterIndex));
		}
	}
}

#if WITH_EDITORONLY_DATA
int32 FMaterialCachedExpressionData::GetDefaultTextureIdx(UObject* InTexture) const
{
	return EditorOnlyData->EditorReferencedDefaultTextures.Find(TObjectPtr<UObject>(InTexture));
}
#endif

#undef LOCTEXT_NAMESPACE
