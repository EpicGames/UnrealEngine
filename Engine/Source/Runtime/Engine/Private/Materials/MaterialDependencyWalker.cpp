// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialExpressionDependencyWalker.cpp - Material expressions implementation.
=============================================================================*/

#if WITH_EDITOR

#include "Materials/MaterialDependencyWalker.h"
#include "Materials/MaterialExpressionTextureSample.h"


static bool WalkMaterialDependencyGraphInternal(const UMaterialExpression* Expression, int32 CurrentDepth, int32 MaxDepth, int32& OutDepth, uint32 SearchFlags, FMaterialDependencySearchMetadata& OutMetaData)
{
	// Check if end of search has been reached
	const int32 NextDepth = CurrentDepth + 1;
	if (Expression == nullptr || NextDepth > MaxDepth)
	{
		OutDepth = INDEX_NONE;
		return false;
	}
	OutDepth = FMath::Max(OutDepth, NextDepth);

	// Check for meta data we are searching for and continue walking the graph
	if (const auto* TextureExpression = Cast<UMaterialExpressionTextureBase>(Expression))
	{
		OutMetaData.bHasTextureInput = true;
		if ((SearchFlags & MDSF_TextureDependencyOnly) != 0)
		{
			return false;
		}
	}

	// Continue to walk the dependency graph along the input expressions
	for (int32 InputIndex = 0; const FExpressionInput* Input = Expression->GetInput(InputIndex); ++InputIndex)
	{
		if (!WalkMaterialDependencyGraphInternal(Input->Expression, NextDepth, MaxDepth, OutDepth, SearchFlags, OutMetaData))
		{
			return false;
		}
	}

	return true;
}

ENGINE_API int32 WalkMaterialDependencyGraph(const UMaterialExpression* Expression, int32 MaxDepth, uint32 SearchFlags, FMaterialDependencySearchMetadata& OutMetaData)
{
	int32 MaxTraversalDepth = INDEX_NONE;
	(void)WalkMaterialDependencyGraphInternal(Expression, 0, MaxDepth, MaxTraversalDepth, SearchFlags, OutMetaData);
	return MaxTraversalDepth;
}


#endif
