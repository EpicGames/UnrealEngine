// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialEnumeration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialEnumeration)

int32 IMaterialEnumerationProvider::GetValueOrDefault(FName EntryName, int32 DefaultValue) const
{
	int32 Value;
	ResolveValue(EntryName, Value, DefaultValue);
	return Value;
}
