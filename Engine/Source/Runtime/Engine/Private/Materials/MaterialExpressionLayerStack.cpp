// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialExpressionLayerStack.cpp - Material expression Layer Stacks implementation.
=============================================================================*/
#include "Materials/MaterialExpressionLayerStack.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "MaterialGraph/MaterialGraphSchema.h"
#endif

//
// FMaterialLayerInput
//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLayerStack)
FMaterialLayerInput::FMaterialLayerInput(FName NewInputName, EFunctionInputType NewInputType)
{
	InputName = NewInputName;
	InputType = NewInputType;
}

FString FMaterialLayerInput::GetInputName() const
{
	FString OutName;
	//Statics are disallowed allowed in the layer stack system.
	if (InputType == FunctionInput_StaticBool
		|| InputType == FunctionInput_MAX)
	{
		return OutName;
	}

	const FString TypeString = UMaterialExpressionFunctionInput::GetInputTypeDisplayName(InputType);
	if (InputName.IsValid() && !TypeString.IsEmpty())
	{
		FString NameString = InputName.ToString();
		OutName = NameString.Appendf(TEXT(" (%s)"), *TypeString);
	}
	return OutName;
}

//
// UMaterialExpressionLayerStack
//
UMaterialExpressionLayerStack::UMaterialExpressionLayerStack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	CachedInputs.Empty();
#endif
}

void UMaterialExpressionLayerStack::PostLoad()
{
#if WITH_EDITOR
	bAreAvailableLayersValid = false;
	ResolveLayerInputs();
	GetSharedAvailableFunctionsCache();
#endif
	Super::PostLoad();
}

void UMaterialExpressionLayerStack::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
#if WITH_EDITOR
	GetSharedAvailableFunctionsCache();
#endif
}

TArray<FSoftObjectPath> UMaterialExpressionLayerStack::GetPathsFromAvailableFunctions(TSet<TObjectPtr<UMaterialFunctionInterface>>& InFunctions)
{
	//We use paths to manage the available layers/blends so we can evaluate against asset (meta)data instead of requiring loaded objects.
	TArray<FSoftObjectPath> OutPaths;
	for (TObjectPtr<UMaterialFunctionInterface> Function : InFunctions)
	{
		if (!Function)
		{
			continue;
		}

		/**
		* We always map the base function to ensure all functions and instances in the vertical can be referenced regardless of heirarchy
		* This should be handled by the existing code which manages the available layers/blends list but no harm in ensuring this
		* is the only info we pass to the UI for asset management.
		*/
		if (UMaterialFunctionInterface* BaseFunction = Function->GetBaseFunction())
		{
			OutPaths.AddUnique(FSoftObjectPath(BaseFunction).ToString());
		}
	}
	return MoveTemp(OutPaths);
}

#if WITH_EDITOR
TSharedPtr<FMaterialLayerStackFunctionsCache> UMaterialExpressionLayerStack::GetSharedAvailableFunctionsCache()
{
	/**
	* Managing available layers across expressions and the UI requires an interface via the FMaterialLayersFunctions class
	* However modifying the runtime data directly is difficult without causing issues with managing lifetime of the referenced items.
	* Utilising a dedicated shared cache between parents and instances ensures the available layers/blends are always available for reference.
	*/
	if (!SharedCache)
	{
		SharedCache = MakeShared<FMaterialLayerStackFunctionsCache>();
	}

	SharedCache->AvailableLayerPaths = GetAvailableLayerPaths();
	SharedCache->AvailableBlendPaths = GetAvailableBlendPaths();
	return SharedCache;
}

void UMaterialExpressionLayerStack::ResolveLayerInputs()
{
	bool bHasChanged = false;
	bool bClearAllInputs = false;
	if (AvailableLayers.IsEmpty() && AvailableBlends.IsEmpty())
	{
		bClearAllInputs = !LayerInputs.IsEmpty();
	}
	else
	{
		//Helper function for retrieving per layer inputs.
		auto GetLayerInputs = [](UMaterialFunctionInterface* CurrentFunction, TMap<FString, FMaterialLayerInput>& CollectedInputs) -> void
			{
				//If null or if the expression has already been recursed, skip.
				if (!CurrentFunction)
				{
					return;
				}

				TArray<FFunctionExpressionInput> FunctionInputs;
				TArray<FFunctionExpressionOutput> FunctionOutputs;
				CurrentFunction->GetInputsAndOutputs(FunctionInputs, FunctionOutputs);

				for (FFunctionExpressionInput& CurrentInput : FunctionInputs)
				{
					//For now we have to only process blend input types that are not marked top/bottom.
					if (!CurrentInput.ExpressionInput || CurrentInput.ExpressionInput->BlendInputRelevance != EBlendInputRelevance::General)
					{
						continue;
					}

					/**
					* We create a map of the existing name which contains the Name and the Type to ensure we allow the maximum number of unique entries
					* even if 2 inputs have the same name but different types, this should be allowed.
					*/
					FMaterialLayerInput NewInput(CurrentInput.Input.InputName, CurrentInput.ExpressionInput->InputType);
					FString CurrentName = NewInput.GetInputName();
					if (!CurrentName.IsEmpty() && !CollectedInputs.Contains(CurrentName))
					{
						CollectedInputs.Add(CurrentName, NewInput);
					}
				}
			};

		TMap<FString, FMaterialLayerInput> AllInputs;
		for (UMaterialFunctionInterface* Layer : AvailableLayers)
		{
			GetLayerInputs(Layer, AllInputs);
		}

		for (UMaterialFunctionInterface* Blend : AvailableBlends)
		{
			GetLayerInputs(Blend, AllInputs);
		}

		if (AllInputs.IsEmpty())
		{
			bClearAllInputs = !LayerInputs.IsEmpty();
		}
		else
		{
			/**
			* This loop ensures we retain existing inputs that already have node connections,
			* whilst also tracking any inputs that need to be removed from the existing list.
			*/
			TArray<FString> RemoveList;
			for (FMaterialLayerInput& ExistingInput : LayerInputs)
			{
				FString ExistingInputName = ExistingInput.GetInputName();
				FMaterialLayerInput* NewInput = AllInputs.Find(ExistingInputName);
				if (NewInput)
				{
					AllInputs.Remove(ExistingInputName);
				}
				else
				{
					if (GraphNode)
					{
						UEdGraphPin* ThisPin = GraphNode->FindPin(*ExistingInputName, EGPD_Input);
						GraphNode->RemovePin(ThisPin);
					}
					RemoveList.Add(ExistingInputName);
				}
			}

			bool bRemovingInputs = !RemoveList.IsEmpty();
			bHasChanged = !AllInputs.IsEmpty() || bRemovingInputs;
			if (bHasChanged)
			{
				Modify();
				if (bRemovingInputs)
				{
					//Remove any inputs that no longer exist in the available layers/blends
					LayerInputs.RemoveAll([&RemoveList](FMaterialLayerInput& LayerInput)
						{
							return RemoveList.Contains(LayerInput.GetInputName());
						}
					);
				}

				//Add any new inputs
				for (TPair<FString, FMaterialLayerInput>& InputPair : AllInputs)
				{
					LayerInputs.Add(InputPair.Value);
				}
			}
		}
	}

	if (bClearAllInputs)
	{
		Modify();
		if (GraphNode)
		{
			TArrayView<UEdGraphPin*> ExistingPins = GraphNode->Pins;
			for (UEdGraphPin* InputPin : ExistingPins)
			{
				if (InputPin->Direction == EGPD_Input)
				{
					GraphNode->RemovePin(InputPin);
				}
			}
		}
		LayerInputs.Empty();
		bHasChanged = true;
	}

	if (bHasChanged)
	{
		if (LayerInputs.Num() > 1)
		{
			//Sort the layer inputs, first by type, then in alphabetical order
			LayerInputs.Sort([](const FMaterialLayerInput& A, const FMaterialLayerInput& B) {
				if (A.InputType < B.InputType)
				{
					return true;
				}
				else if (A.InputType == B.InputType)
				{
					return A.InputName.ToString() < B.InputName.ToString();
				}
				return false;
				});
		}

		CacheLayerInputs();

		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
		bAreAvailableLayersValid = false;
	}
}

UMaterialFunctionInterface* UMaterialExpressionLayerStack::ExtractParentFunctionFromInstance(FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction)
{
	/**
	* Instances need to be mapped to their parent, we should allow all instances in a vertical of an allowed, validated base.
	*/
	if (UMaterialFunctionInstance* Instance = Cast<UMaterialFunctionInstance>(CurrentFunction))
	{
		if (UMaterialFunctionInterface* Parent = Instance->GetBaseFunction())
		{
			CurrentFunction = Parent;
		}
		else
		{
			LogError(Compiler, TEXT("Function %s: instance has no parent set."), *CurrentFunction->GetName());
			CurrentFunction = nullptr;
		}
	}
	return CurrentFunction;
}

bool UMaterialExpressionLayerStack::PollFunctionExpressionsForLayerUsage(FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction, FValidLayerUsageTracker& Tracker, bool bCheckStatics)
{
	CurrentFunction = ExtractParentFunctionFromInstance(Compiler, CurrentFunction);
	if (!CurrentFunction || !CurrentFunction->IsA<UMaterialFunction>())
	{
		LogError(Compiler, TEXT("Function %s: Invalid base material function being validated. Instances cannot be validated, only their base functions."), *CurrentFunction->GetName());
		return false;
	}

	TConstArrayView<TObjectPtr<UMaterialExpression>> AllExpressions = CurrentFunction->GetExpressions();
	for (TObjectPtr<UMaterialExpression> Expression : AllExpressions)
	{
		if (UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(Expression))
		{
			if (InputExpression->GetInputValueType(0) == MCT_MaterialAttributes)
			{
				Tracker.MAInputCount++;
			}
		}

		if (bCheckStatics && !Tracker.bContainsStatics && Expression->IsStaticExpression())
		{
			Tracker.bContainsStatics = true;
		}

		if (UMaterialExpressionFunctionOutput* OutputExpression = Cast<UMaterialExpressionFunctionOutput>(Expression))
		{
			if (OutputExpression->GetOutputValueType(0) == MCT_MaterialAttributes)
			{
				Tracker.MAOutputCount++;
			}
		}
	}

	bool bResult = true;
	if (bCheckStatics && Tracker.bContainsStatics)
	{
		LogError(Compiler, TEXT("Function %s: cannot contain any static nodes as these are incompatible with the layering."), *CurrentFunction->GetName());
		bResult = false;
	}

	if (Tracker.MAOutputCount != AcceptableNumLayerBlendMAOutputs)
	{
		LogError(Compiler, TEXT("Function %s, must have exactly %d Material Attributes type output node. It has %d"), *CurrentFunction->GetName(), AcceptableNumLayerBlendMAOutputs, Tracker.MAOutputCount);
		bResult = false;
	}

	return bResult;
}

bool UMaterialExpressionLayerStack::ValidateFunctionForLayerUsage(FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction, bool bCheckStatics)
{
	EMaterialFunctionUsage Usage = CurrentFunction->GetMaterialFunctionUsage();
	if (!(Usage == EMaterialFunctionUsage::Default || Usage == EMaterialFunctionUsage::MaterialLayer))
	{
		LogError(Compiler, TEXT("Function %s: function usage is not set for use as a layer."), *CurrentFunction->GetName());
		return false;
	}

	FValidLayerUsageTracker Tracker;
	bool bResult = PollFunctionExpressionsForLayerUsage(Compiler, CurrentFunction, Tracker, bCheckStatics);

	if (Tracker.MAInputCount > AcceptableNumLayerMAInputs)
	{
		LogError(Compiler, TEXT("Layer %s: can only have up to %d Material Attributes type output node. It has %d"), *CurrentFunction->GetName(), AcceptableNumLayerMAInputs, Tracker.MAInputCount);
		bResult = false;
	}

	return bResult;
}

bool UMaterialExpressionLayerStack::ValidateFunctionForBlendUsage(FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction, bool bCheckStatics)
{
	EMaterialFunctionUsage Usage = CurrentFunction->GetMaterialFunctionUsage();
	if (Usage != EMaterialFunctionUsage::MaterialLayerBlend)
	{
		LogError(Compiler, TEXT("Function %s: function usage is not set for use as a blend."), *CurrentFunction->GetName());
		return false;
	}

	FValidLayerUsageTracker Tracker;
	bool bResult = PollFunctionExpressionsForLayerUsage(Compiler, CurrentFunction, Tracker, bCheckStatics);
	if (Tracker.MAInputCount != AcceptableNumBlendMAInputs)
	{
		LogError(Compiler, TEXT("Blend %s: must have exactly %d Material Attributes type input node. It has %d"), *CurrentFunction->GetName(), AcceptableNumBlendMAInputs, Tracker.MAInputCount);
		bResult = false;
	}

	return true;
}

bool UMaterialExpressionLayerStack::ValidateLayerConfiguration(FMaterialCompiler* Compiler, bool bReportErrors)
{
	bool bIsValid = true;

	if (!bAreAvailableLayersValid)
	{
		//Helper function for retrieving per layer inputs.
		auto ValidateAvailableFunctions = [Compiler](TSet<TObjectPtr<UMaterialFunctionInterface>>& AvailableFunctions, bool& bIsValid, const bool bIsBlendArray) -> void
			{
				for (TObjectPtr<UMaterialFunctionInterface>& Function : AvailableFunctions)
				{
					if (Function)
					{
						UMaterialFunctionInterface* BaseFunction = ExtractParentFunctionFromInstance(Compiler, Function);
						bool bRequiresValidation = false;
						if (BaseFunction == Function)
						{
							bRequiresValidation = true;
						}
						else
						{
							/**
							* The validation process involves resolving the parent to ensure any function instance can be used
							* so to save having to check each entry in the available list constantly, we only want parents in the available layers.
							* This way once we have the parent of an actually used layer, we can speed up the comparison.
							*
							* Additionally, we need to ensure we aren't duplicating existing entries.
							*/
							const FString FunctionType = bIsBlendArray ? "blends" : "layers";
							if (AvailableFunctions.Contains(BaseFunction))
							{
								UE_LOGF(LogMaterial, Warning, "%ls not appended because available %ls already contain the base (%ls).", *Function->GetName(), *FunctionType, *BaseFunction->GetName());
								Function = nullptr;
							}
							else
							{
								UE_LOGF(LogMaterial, Warning, "Resolving base (%ls) of %ls for the available %ls list.", *BaseFunction->GetName(), *Function->GetName(), *FunctionType);
								Function = BaseFunction;
								bRequiresValidation = true;
							}
						}

						if (bRequiresValidation)
						{
							bIsValid = bIsBlendArray ? ValidateFunctionForBlendUsage(Compiler, BaseFunction, true) : ValidateFunctionForLayerUsage(Compiler, BaseFunction, true);
						}
					}
				}
			};
		ValidateAvailableFunctions(AvailableLayers, bIsValid, false);
		ValidateAvailableFunctions(AvailableBlends, bIsValid, true);
		bAreAvailableLayersValid = bIsValid;

		if (!bIsValid)
		{
			LogError(Compiler, TEXT("LayerStack Available Layers/Blends contain invalid functions."));
			return false;
		}
	}

	for (FMaterialLayerInput& LayerInput : LayerInputs)
	{
		if (!LayerInput.IsConnected())
		{
			LogError(Compiler, TEXT("LayerStack \"%s\" input pin must be connected."), *LayerInput.GetInputName());
			bIsValid = false;
		}
	}

	// If layer inputs are not connected, early out. We want to force users to apply default values at least at the base material level for now.
	if (!bIsValid)
	{
		return false;
	}

	/**
	* Despite available layers / blends having their validation cached, we still have to then validate the utilised blends / layers.
	* Fortunately the cached available functions allows us to just ensure they are present in the existing list to reduce check logic required.
	*/
	const TArray<UMaterialFunctionInterface*>& Layers = GetLayers();
	int32 NumActiveLayers = 0;
	for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
	{
		UMaterialFunctionInterface* Layer = Layers[LayerIndex];
		if (Layer)
		{
			Layer = ExtractParentFunctionFromInstance(Compiler, Layer);
			if (AvailableLayers.Contains(Layer))
			{
				NumActiveLayers++;
			}
			else
			{
				LogError(Compiler, TEXT("Layer %i, %s, is not set as an allowed layer in the base layer stack node."), LayerIndex, *Layer->GetName());
				bIsValid = false;
			}
		}
	}

	const TArray<UMaterialFunctionInterface*>& Blends = GetBlends();
	int32 NumActiveBlends = 0;
	for (int32 BlendIndex = 0; BlendIndex < Blends.Num(); ++BlendIndex)
	{
		UMaterialFunctionInterface* Blend = Blends[BlendIndex];
		if (Blend)
		{
			Blend = ExtractParentFunctionFromInstance(Compiler, Blend);
			if (AvailableBlends.Contains(Blend))
			{
				NumActiveBlends++;
			}
			else
			{
				LogError(Compiler, TEXT("Blend %i, %s, is not set as an allowed blend in the base layer stack node."), BlendIndex, *Blend->GetName());
				bIsValid = false;
			}
		}
	}

	if (!bIsValid)
	{
		return false;
	}

	// Currently we want only allow these configurations but in future we will unlock blends and layer only lists being usable together, however a new UI is likely required for this.
	bool bValidConfiguration = (NumActiveLayers >= 0 && NumActiveBlends == 0)						// Default behaviour if both 0, otherwise layers only configuration
							|| (NumActiveLayers >= 2 && NumActiveBlends == NumActiveLayers - 1);	// Blend graph

	if (!bValidConfiguration)
	{
		LogError(Compiler, TEXT("Invalid number of layers (%i) or blends (%i) assigned. Number of blends must be equal to 0, or 1 less than the number of active layers."), NumActiveLayers, NumActiveBlends);
	}
	return bValidConfiguration;
}

void UMaterialExpressionLayerStack::CacheLayerInputs()
{
	CachedInputs.Empty();
	if (!LayerInputs.IsEmpty())
	{
		CachedInputs.Reserve(LayerInputs.Num());
		for (FExpressionInput& CurrentInput : LayerInputs)
		{
			CachedInputs.Push(&CurrentInput);
		}
	}
}

void UMaterialExpressionLayerStack::RebuildLayerGraph(bool bReportErrors)
{
	if (ValidateLayerConfiguration(nullptr, bReportErrors))
	{
		// Reset graph connectivity
		bIsLayerGraphBuilt = false;

		//Similar to the parent implementation, this helper function creates a new Function call for each referenced function to create the new graph
		auto ProcessFunctionCallers = [this](const TArray<UMaterialFunctionInterface*>& Functions, TArray<TObjectPtr<UMaterialExpressionMaterialFunctionCall>>& CallerArray, int32& CallerCount, EMaterialParameterAssociation InAssociation) -> void
			{
				CallerCount = 0;
				if (Functions.IsEmpty())
				{
					CallerArray.Empty();
					return;
				}

				int32 NumFunctions = Functions.Num();
				int32 NumCallers = CallerArray.Num();

				if (NumFunctions > NumCallers)
				{
					while (CallerArray.Num() < NumFunctions)
					{
						CallerArray.Push(NewObject<UMaterialExpressionMaterialFunctionCall>(GetTransientPackage()));
					}
				}
				else if (NumFunctions < NumCallers)
				{
					CallerArray.SetNum(NumFunctions, EAllowShrinking::Yes);
				}

				for (int32 LayerIndex = 0; LayerIndex < Functions.Num(); LayerIndex++)
				{
					//Create the new function call
					TObjectPtr<UMaterialExpressionMaterialFunctionCall>& FunctionCaller = CallerArray[LayerIndex];
					UMaterialFunctionInterface* CurrentFunction = Functions[LayerIndex];
					FunctionCaller->FunctionParameterInfo.Index = LayerIndex;
					if (FunctionCaller->MaterialFunction != CurrentFunction)
					{
						FunctionCaller->SetMaterialFunction(CurrentFunction);
						FunctionCaller->FunctionParameterInfo.Association = InAssociation;						
						FunctionCaller->UpdateFromFunctionResource();
					}

					if (!this->LayerInputs.IsEmpty() && !FunctionCaller->FunctionInputs.IsEmpty())
					{
						for (FMaterialLayerInput& LayerInput : this->LayerInputs)
						{
							if (!LayerInput.Expression)
							{
								continue;
							}

							for (int32 CallInputIndex = 0; CallInputIndex < FunctionCaller->FunctionInputs.Num(); CallInputIndex++)
							{
								FFunctionExpressionInput& ThisInput = FunctionCaller->FunctionInputs[CallInputIndex];
								if (!ThisInput.ExpressionInput)
								{
									continue;
								}

								//If an exposed layer input at the parent material is exposed and connected, map the overrides
								if (ThisInput.Input.InputName == LayerInput.InputName
									&& ThisInput.ExpressionInput->InputType == LayerInput.InputType)
								{
									//Sets the read only inputs, but will be overridden through the stack if any sublayers have matching outputs
									ThisInput.Input.Connect(LayerInput.OutputIndex, LayerInput.Expression);
								}
							}
						}
					}

					if (FunctionCaller->MaterialFunction)
					{
						CallerCount++;
					}
				}
			};

		//ProcessLayers
		ProcessFunctionCallers(GetLayers(), LayerCallers, NumActiveLayerCallers, EMaterialParameterAssociation::LayerParameter);

		//ProcessBlends
		ProcessFunctionCallers(GetBlends(), BlendCallers, NumActiveBlendCallers, EMaterialParameterAssociation::BlendParameter);

		// Assemble function chain so each layer blends with the previous
		if (NumActiveLayerCallers >= 2 && NumActiveBlendCallers == NumActiveLayerCallers - 1)
		{
			int32 CurrentLayerIndex = 0;
			TObjectPtr<UMaterialExpressionMaterialFunctionCall> Bottom = LayerCallers[CurrentLayerIndex];
			TObjectPtr<UMaterialExpressionMaterialFunctionCall> Top = LayerCallers[++CurrentLayerIndex];
			for (int32 BlendIndex = 0; BlendIndex < NumActiveBlendCallers; BlendIndex++)
			{
				TObjectPtr<UMaterialExpressionMaterialFunctionCall> Blend = BlendCallers[BlendIndex];
				bool bBottomSet = false;
				bool bTopSet = false;
				for (FFunctionExpressionInput& ThisInput : Blend->FunctionInputs)
				{
					/**
					* For now we only want to connect MA types, and we restrict the number we accept in the entries
					* as we can't account for name matching with the required blend inputs (hence the EBlendInputRelevance setting).
					* However this will allow us to assign other input/output matching types at a later stage.
					*/
					if (ThisInput.ExpressionInput && ThisInput.ExpressionInput->InputType == FunctionInput_MaterialAttributes)
					{
						if (!bBottomSet && ThisInput.ExpressionInput->BlendInputRelevance == EBlendInputRelevance::Bottom)
						{
							ThisInput.Input.Connect(0, Bottom);
							bBottomSet = true;
						}

						if (!bTopSet && ThisInput.ExpressionInput->BlendInputRelevance == EBlendInputRelevance::Top)
						{
							ThisInput.Input.Connect(0, Top);
							bTopSet = true;
						}
					}

					if (bBottomSet && bTopSet)
					{
						break;
					}
				}

				if (++CurrentLayerIndex < NumActiveLayerCallers)
				{
					Bottom = Blend;
					Top = LayerCallers[CurrentLayerIndex];
				}
				else
				{
					break;
				}
			}
			bIsLayerGraphBuilt = true;
		}
		else if (NumActiveBlendCallers == 0)
		{
			//If no blends are present, create a layer chain
			if (NumActiveLayerCallers > 1)
			{
				TObjectPtr<UMaterialExpressionMaterialFunctionCall> CurrentLayer = LayerCallers[NumActiveLayerCallers - 1];
				for (int32 LayerIndex = NumActiveLayerCallers - 2; LayerIndex >= 0; LayerIndex--)
				{
					TObjectPtr<UMaterialExpressionMaterialFunctionCall> PreviousLayer = LayerCallers[LayerIndex];
					for (int32 CallInputIndex = 0; CallInputIndex < CurrentLayer->FunctionInputs.Num(); CallInputIndex++)
					{
						FFunctionExpressionInput& CurrentInput = CurrentLayer->FunctionInputs[CallInputIndex];
						EMaterialValueType CurrentInputType = CurrentInput.ExpressionInput->GetInputValueType(0);
						if (!CurrentInput.ExpressionInput || CurrentInputType != MCT_MaterialAttributes)
						{
							//If we hit an input that can't be connected, break the chain connection.
							if (CallInputIndex < CurrentLayer->FunctionInputs.Num() - 1)
							{
								UE_LOGF(LogMaterial, Warning, "LayerStack's layer-only graph chain in \"%ls\" is cut short due to missing MaterialAttributes input in %ls.",
									Material ? *(Material->GetName()) : TEXT("Unknown"), *CurrentLayer->GetName());
							}
							break;
						}

						//For each input, iterate through the outputs and connect if there's a match
						//This may seem cumbersome, but this is laying the groundwork for mapping matching inputs and outputs outside of the MA type.
						for (int32 CallOutputIndex = 0; CallOutputIndex < PreviousLayer->FunctionOutputs.Num(); CallOutputIndex++)
						{
							FFunctionExpressionOutput& PreviousOutput = PreviousLayer->FunctionOutputs[CallOutputIndex];
							if (PreviousOutput.ExpressionOutput
								&& CurrentInput.Input.InputName == PreviousOutput.ExpressionOutput->OutputName
								&& CanConnectMaterialValueTypes(CurrentInputType, PreviousOutput.ExpressionOutput->GetOutputValueType(0)))
							{
								CurrentInput.Input.Connect(0, PreviousOutput.ExpressionOutput);
								break;
							}
						}
					}
					CurrentLayer = PreviousLayer;
				}
			}
			//We always say true at this point because regardless of having a layer chain, a single layer, or no entries we want to compile.
			bIsLayerGraphBuilt = true;
		}
	}

	if (!bIsLayerGraphBuilt && bReportErrors)
	{
		UE_LOGF(LogMaterial, Warning, "Failed to build LayerStack graph for %ls.", Material ? *(Material->GetName()) : TEXT("Unknown"));
	}
}

void UMaterialExpressionLayerStack::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		FName PropertyName = PropertyChangedEvent.Property->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionLayerStack, AvailableLayers)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionLayerStack, AvailableBlends))
		{
			//If we change the available layers/blends sets, the validation of their usage needs to be re-evaluated.
			bAreAvailableLayersValid = false;
			ResolveLayerInputs();
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionLayerStack::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Layer Stack"));
}

void UMaterialExpressionLayerStack::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Evaluates the material layer stack against it's available layer functions, and outputs the merged attributes via the specified blends."), 40, OutToolTip);
}

FName UMaterialExpressionLayerStack::GetInputName(int32 InputIndex) const
{
	return LayerInputs.IsValidIndex(InputIndex) ? FName(LayerInputs[InputIndex].GetInputName()) : NAME_None;
}

FExpressionInput* UMaterialExpressionLayerStack::GetInput(int32 InputIndex)
{
	return LayerInputs.IsValidIndex(InputIndex) ? &LayerInputs[InputIndex] : nullptr;
}

TArrayView<FExpressionInput*> UMaterialExpressionLayerStack::GetInputsView()
{
	return CachedInputs;
}

EMaterialValueType UMaterialExpressionLayerStack::GetInputValueType(int32 InputIndex)
{
	return LayerInputs.IsValidIndex(InputIndex) ? UMaterialExpressionFunctionInput::GetMaterialTypeFromInputType(LayerInputs[InputIndex].InputType) : MCT_Unknown;
}
#endif //WITH_EDITOR
