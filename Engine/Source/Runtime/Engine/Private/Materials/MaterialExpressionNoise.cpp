// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialExpressionNoise.cpp - Material expressions implementation.
=============================================================================*/

#include "Materials/MaterialExpressionNoise.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/HLSLMaterialTranslator.h"
#include "Materials/MaterialSharedPrivate.h"
#if WITH_EDITOR
#include "MaterialGraph/MaterialGraphSchema.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionNoise)

#define LOCTEXT_NAMESPACE "MaterialExpressionNoise"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionNoise
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionNoise::UMaterialExpressionNoise(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Utility;
		FConstructorStatics()
			: NAME_Utility(LOCTEXT("Utility", "Utility"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Scale = 1.0f;
	Levels = 6;
	Quality = 1;
	OutputMin = -1.0f;
	OutputMax = 1.0f;
	LevelScale = 2.0f;
	NoiseFunction = NOISEFUNCTION_SimplexTex;
	bTurbulence = true;
	bTiling = false;
	RepeatSize = 512;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Utility);
#endif

}

#if WITH_EDITOR
bool UMaterialExpressionNoise::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		FName PropertyFName = InProperty->GetFName();

		bool bTilableNoiseType = NoiseFunction == NOISEFUNCTION_GradientALU || NoiseFunction == NOISEFUNCTION_ValueALU
			|| NoiseFunction == NOISEFUNCTION_GradientTex || NoiseFunction == NOISEFUNCTION_VoronoiALU;

		bool bSupportsQuality = (NoiseFunction == NOISEFUNCTION_VoronoiALU);

		if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, bTiling))
		{
			bIsEditable = bTilableNoiseType;
		}
		else if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, RepeatSize))
		{
			bIsEditable = bTilableNoiseType && bTiling;
		}

		if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionNoise, Quality))
		{
			bIsEditable = bSupportsQuality;
		}
	}

	return bIsEditable;
}

FName UMaterialExpressionNoise::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &Position)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionNoise::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionNoise::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionInput;

	if (Position.GetTracedInput().Expression)
	{
		PositionInput = Position.Compile(Compiler);
	}
	else
	{
		PositionInput = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType, true));
	}

	int32 FilterWidthInput;

	if (FilterWidth.GetTracedInput().Expression)
	{
		FilterWidthInput = FilterWidth.Compile(Compiler);
	}
	else
	{
		FilterWidthInput = Compiler->Constant(0);
	}

	return Compiler->Noise(PositionInput, WorldPositionOriginType, Scale, Quality, NoiseFunction, bTurbulence, Levels, OutputMin, OutputMax, LevelScale, FilterWidthInput, bTiling, RepeatSize);
}

void UMaterialExpressionNoise::GetCaption(TArray<FString>& OutCaptions) const
{
	const UEnum* NFEnum = StaticEnum<ENoiseFunction>();
	check(NFEnum);
	OutCaptions.Add(NFEnum->GetDisplayNameTextByValue(NoiseFunction).ToString());
	OutCaptions.Add(TEXT("Noise"));
}
#endif // WITH_EDITOR



///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionVectorNoise
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionVectorNoise::UMaterialExpressionVectorNoise(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Utility;
		FConstructorStatics()
			: NAME_Utility(LOCTEXT("Utility", "Utility"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Quality = 1;
	NoiseFunction = VNF_CellnoiseALU;
	bTiling = false;
	TileSize = 300;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Utility);
#endif
}

#if WITH_EDITOR
bool UMaterialExpressionVectorNoise::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		FName PropertyFName = InProperty->GetFName();

		bool bSupportsQuality = (NoiseFunction == VNF_VoronoiALU);

		if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, TileSize))
		{
			bIsEditable = bTiling;
		}

		else if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionVectorNoise, Quality))
		{
			bIsEditable = bSupportsQuality;
		}
	}

	return bIsEditable;
}

FName UMaterialExpressionVectorNoise::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &Position)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionVectorNoise::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionVectorNoise::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionInput;

	if (Position.GetTracedInput().Expression)
	{
		PositionInput = Position.Compile(Compiler);
	}
	else
	{
		PositionInput = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType, true));
	}

	return Compiler->VectorNoise(PositionInput, WorldPositionOriginType, Quality, NoiseFunction, bTiling, TileSize);
}

void UMaterialExpressionVectorNoise::GetCaption(TArray<FString>& OutCaptions) const
{
	const UEnum* VNFEnum = StaticEnum<EVectorNoiseFunction>();
	check(VNFEnum);
	OutCaptions.Add(VNFEnum->GetDisplayNameTextByValue(NoiseFunction).ToString());
	OutCaptions.Add(TEXT("Vector Noise"));
}
#endif // WITH_EDITOR



///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionScalarBlueNoise
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionScalarBlueNoise::UMaterialExpressionScalarBlueNoise(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Utility;
		FConstructorStatics()
			: NAME_Utility(LOCTEXT("Noise", "Noise"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Utility);
#endif
}

#if WITH_EDITOR

EMaterialValueType UMaterialExpressionScalarBlueNoise::GetOutputValueType(int32 OutputIndex) 
{
	return MCT_Float1;
}

int32 UMaterialExpressionScalarBlueNoise::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ScalarBlueNoise();
}

void UMaterialExpressionScalarBlueNoise::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Scalar Blue Noise"));
}

void UMaterialExpressionScalarBlueNoise::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	OutToolTip.Add(TEXT("Return a blue noise value in [0.1] for each pixel on screen."));
	OutToolTip.Add(TEXT("Be aware that this node might not play well when seen in ray traced reflections or path tracing,"));
	OutToolTip.Add(TEXT("when secondary rays cannot use pixel position."));
}

void UMaterialExpressionScalarBlueNoise::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	GetExpressionToolTip(OutToolTip); // Single ouptput so reuse the expression tooltip.
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
