// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionSettings.h"
#include "UObject/Class.h"

#if WITH_EDITOR

FMaterialExpressionSettings* FMaterialExpressionSettings::Get()
{
	static FMaterialExpressionSettings Inst;
	return &Inst;
}

void FMaterialExpressionSettings::RegisterIsClassPathAllowedDelegate(const FName OwnerName, FOnIsClassPathAllowed Delegate)
{
	IsClassPathAllowedDelegates.Add(OwnerName, Delegate);
}

void FMaterialExpressionSettings::UnregisterIsClassPathAllowedDelegate(const FName OwnerName)
{
	IsClassPathAllowedDelegates.Remove(OwnerName);
}

bool FMaterialExpressionSettings::IsClassPathAllowed(const FTopLevelAssetPath& InClassPath) const
{
	for (const TPair<FName, FOnIsClassPathAllowed>& DelegatePair : IsClassPathAllowedDelegates)
	{
		if (!DelegatePair.Value.Execute(InClassPath))
		{
			return false;
		}
	}
	return true;
}

bool FMaterialExpressionSettings::HasClassPathFiltering() const
{
	return IsClassPathAllowedDelegates.Num() > 0;
}

#endif // WITH_EDITOR

bool IsExpressionClassPermitted(const UClass* const Class)
{
	if (Class)
	{
		// Always allow cooked content
		if (Class->bCooked)
		{
			return true;
		}

#if WITH_EDITOR
		// Uncooked content must be explicitly allowed if in restrictive mode
		const bool bIsClassPathAllowed = FMaterialExpressionSettings::Get()->IsClassPathAllowed(FTopLevelAssetPath(Class->GetPathName()));
		if (!bIsClassPathAllowed)
		{
			return false;
		}
#endif
	}
	return true;
}
