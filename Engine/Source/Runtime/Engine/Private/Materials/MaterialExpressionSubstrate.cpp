// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialExpressionSubstrate.cpp - Substrate material expressions implementation.
=============================================================================*/

#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/HLSLMaterialTranslator.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "Engine/ToonProfile.h"
#include "MaterialExpressionIO.h"

#if WITH_EDITOR
#include "MaterialGraph/MaterialGraphSchema.h"
#include "SubstrateMaterial.h"
#include "SubstrateTranslatorCommon.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionSubstrate)

#define LOCTEXT_NAMESPACE "MaterialExpressionSubstrate"

///////////////////////////////////////////////////////////////////////////////
// Substrate

EMaterialSubSurfaceType SubstrateMergeSubSurfaceType(EMaterialSubSurfaceType A, EMaterialSubSurfaceType B)
{
	// Merge two type of SSS type, by using the most complex behavior, i.e., No < Wrap < Diffusion < Diffusion Profile
	// This code needs to be in sync with SubstrateMergeSSSType() in Substrate.ush
	return EMaterialSubSurfaceType(FMath::Max(uint32(A), uint32(B)));
}

#if WITH_EDITOR
static int32 SubstrateBlendNormal(class FMaterialCompiler* Compiler, int32 NormalCodeChunk0, int32 NormalCodeChunk1, int32 MixCodeChunk)
{
	int32 SafeMixCodeChunk = Compiler->Saturate(MixCodeChunk);
	int32 LerpedNormal = Compiler->Lerp(NormalCodeChunk0, NormalCodeChunk1, SafeMixCodeChunk);
	int32 BlendedNormalCodeChunk = Compiler->Div(LerpedNormal, Compiler->SquareRoot(Compiler->Dot(LerpedNormal, LerpedNormal)));
	return BlendedNormalCodeChunk;
}

void AssignOperatorIndexIfNotNull(int32& NextOperatorPin, FSubstrateOperator* Operator)
{
	NextOperatorPin = Operator ? Operator->Index : INDEX_NONE;
}

void CombineFlagForParameterBlending(FSubstrateOperator& DstOp, FSubstrateOperator* OpA, FSubstrateOperator* OpB)
{
	if (OpA && OpB)
	{
		DstOp.CombineFlagsForParameterBlending(*OpA, *OpB);
	}
	else if (OpA)
	{
		DstOp.CopyFlagsForParameterBlending(*OpA);
	}
	else if (OpB)
	{
		DstOp.CopyFlagsForParameterBlending(*OpB);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR

// Optionnaly cast CodeChunk type to non-LWC type. 
// Input can be built of WorldPosition data, which would force the derived data to have LWC type 
// creating issues, as Substrate functions' inputs don't support LWC
static int32 CastToNonLWCType(class FMaterialCompiler* Compiler, int32 CodeChunk)
{
	EMaterialValueType Type = Compiler->GetType(CodeChunk);
	if (IsLWCType(Type))
	{
		Type = MakeNonLWCType(Type);
		CodeChunk = Compiler->ValidCast(CodeChunk, Type);
	}
	return CodeChunk;
}

// The compilation of an expression can sometimes lead to a INDEX_NONE code chunk when editing material graphs 
// or when the node is inside a material function, linked to an input pin of the material function and that input is not plugged in to anything.
// But for normals or tangents, Substrate absolutely need a valid code chunk to de-duplicate when stored in memory. 
// Also, we want all our nodes to have default, as that is needed when creating BSDF, when registering code chunk representing material topology.
static int32 CompileWithDefaultFloat1(class FMaterialCompiler* Compiler, FExpressionInput& Input, float X, const FScalarMaterialInput* RootNodeInput = nullptr)
{
	int32 DefaultCodeChunk = Compiler->Constant(X);
	if (RootNodeInput && RootNodeInput->UseConstant)
	{
		DefaultCodeChunk = Compiler->Constant(RootNodeInput->Constant);
	}
	int32 CodeChunk = Input.GetTracedInput().Expression ? Input.Compile(Compiler) : DefaultCodeChunk;
	CodeChunk = CastToNonLWCType(Compiler, CodeChunk);
	return CodeChunk == INDEX_NONE ? DefaultCodeChunk : CodeChunk;
}
static int32 CompileWithDefaultFloat2(class FMaterialCompiler* Compiler, FExpressionInput& Input, float X, float Y, const FVector2MaterialInput* RootNodeInput = nullptr)
{
	int32 DefaultCodeChunk = Compiler->Constant2(X, Y);
	if (RootNodeInput && RootNodeInput->UseConstant)
	{
		DefaultCodeChunk = Compiler->Constant2(RootNodeInput->Constant.X, RootNodeInput->Constant.Y);
	}
	int32 CodeChunk = Input.GetTracedInput().Expression ? Input.Compile(Compiler) : DefaultCodeChunk;
	CodeChunk = CastToNonLWCType(Compiler, CodeChunk);
	return CodeChunk == INDEX_NONE ? DefaultCodeChunk : CodeChunk;
}
static int32 CompileWithDefaultFloat3(class FMaterialCompiler* Compiler, FExpressionInput& Input, float X, float Y, float Z, const FColorMaterialInput* RootNodeInput = nullptr)
{
	int32 DefaultCodeChunk = Compiler->Constant3(X, Y, Z);
	if (RootNodeInput && RootNodeInput->UseConstant)
	{
		DefaultCodeChunk = Compiler->Constant3(RootNodeInput->Constant.R, RootNodeInput->Constant.G, RootNodeInput->Constant.B);
	}
	int32 CodeChunk = Input.GetTracedInput().Expression ? Input.Compile(Compiler) : DefaultCodeChunk;
	CodeChunk = CastToNonLWCType(Compiler, CodeChunk);
	return CodeChunk == INDEX_NONE ? DefaultCodeChunk : CodeChunk;
}
static int32 CompileWithDefaultNormalWS(class FMaterialCompiler* Compiler, FExpressionInput& Input, bool bConvertToRequestedSpace = true)
{
	if (Input.GetTracedInput().Expression != nullptr)
	{
		int32 NormalCodeChunk = Input.Compile(Compiler);

		if (NormalCodeChunk == INDEX_NONE)
		{
			// Nothing is plug in from the linked input, so specify world space normal the BSDF node expects.
			return Compiler->VertexNormal();
		}

		// Ensure the normal has always a valid float3 type
		NormalCodeChunk = Compiler->ForceCast(NormalCodeChunk, MCT_Float3, MFCF_ExactMatch | MFCF_ReplicateValue);
		NormalCodeChunk = CastToNonLWCType(Compiler, NormalCodeChunk);
		NormalCodeChunk = NormalCodeChunk == INDEX_NONE ? Compiler->VertexNormal() : NormalCodeChunk;

		// Transform into world space normal if needed. BSDF nodes always expects world space normal as input.
		return bConvertToRequestedSpace ? Compiler->TransformNormalFromRequestedBasisToWorld(NormalCodeChunk) : NormalCodeChunk;
	}
	// Nothing is plug in on the BSDF node, so specify world space normal the node expects.
	return Compiler->VertexNormal();
}
static int32 CompileWithDefaultTangentWS(class FMaterialCompiler* Compiler, FExpressionInput& Input, bool bConvertToRequestedSpace = true)
{
	if (Input.GetTracedInput().Expression != nullptr)
	{
		int32 TangentCodeChunk = Input.Compile(Compiler);

		if (TangentCodeChunk == INDEX_NONE)
		{
			// Nothing is plug in from the linked input, so specify world space tangent the BSDF node expects.
			return Compiler->VertexTangent();
		}

		// Ensure the tangent has always a valid float3 type
		TangentCodeChunk = Compiler->ForceCast(TangentCodeChunk, MCT_Float3, MFCF_ExactMatch | MFCF_ReplicateValue);

		// Transform into world space tangent if needed. BSDF nodes always expects world space tangent as input.
		return bConvertToRequestedSpace ? Compiler->TransformNormalFromRequestedBasisToWorld(TangentCodeChunk) : TangentCodeChunk;
	}
	// Nothing is plug in on the BSDF node, so specify world space tangent the node expects.
	return Compiler->VertexTangent();
}

static int32 CreateSubsurfaceProfileParameter(class FMaterialCompiler* Compiler, USubsurfaceProfile* InProfile)
{
	check(InProfile);
	const FName SubsurfaceProfileParameterName = SubsurfaceProfile::CreateSubsurfaceProfileParameterName(InProfile);
	const int32 SSSProfileCodeChunk = Compiler->ForceCast(Compiler->ScalarParameter(SubsurfaceProfileParameterName, 1.0f), MCT_Float1);
	return SSSProfileCodeChunk;
}

static int32 CreateDefaultSubsurfaceProfileParameter(class FMaterialCompiler* Compiler)
{
	const int32 SSSProfileCodeChunk = Compiler->ForceCast(Compiler->ScalarParameter(SubsurfaceProfile::GetSubsurfaceProfileParameterName(), 1.0f), MCT_Float1);
	return SSSProfileCodeChunk;
}


#endif // WITH_EDITOR

UMaterialExpressionSubstrateShadingModels::UMaterialExpressionSubstrateShadingModels(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Conversion", "Substrate Conversion")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
FExpressionInput* UMaterialExpressionSubstrateShadingModels::GetInput(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0: return &BaseColor;
	case 1: return &Metallic;
	case 2: return &Specular;
	case 3: return &Roughness;
	case 4: return &Anisotropy;
	case 5: return &EmissiveColor;
	case 6: return &Normal;
	case 7: return &Tangent;
	case 8: return &SubSurfaceColor;
	case 9: return &ClearCoat;
	case 10: return &ClearCoatRoughness;
	case 11: return &Opacity;
	case 12: return &TransmittanceColor;
	case 13: return &WaterScatteringCoefficients;
	case 14: return &WaterAbsorptionCoefficients;
	case 15: return &WaterPhaseG;
	case 16: return &ColorScaleBehindWater;
	case 17: return &ClearCoatNormal;
	case 18: return &CustomTangent;
	case 19: return &ThinTranslucentSurfaceCoverage;
	case 20: return &ShadingModel;
	default: return nullptr;
	}
}

#define LEGACY_DIRECT_ATTRIBUTE_MAPPING(CodeChunkResult, MaterialProperty, Code)	Compiler->PushMaterialAttribute(FMaterialAttributeDefinitionMap::GetID(MaterialProperty));\
																					int32 CodeChunkResult = Code;\
																					Compiler->PopMaterialAttribute();


int32 UMaterialExpressionSubstrateShadingModels::CompileCommon(class FMaterialCompiler* Compiler,
	FExpressionInput& BaseColor, FExpressionInput& Specular, FExpressionInput& Metallic, FExpressionInput& Roughness, FExpressionInput& EmissiveColor,
	FExpressionInput& Opacity, FExpressionInput& SubSurfaceColor, FExpressionInput& ClearCoat, FExpressionInput& ClearCoatRoughness,
	FExpressionInput& ShadingModel, TEnumAsByte<enum EMaterialShadingModel> ShadingModelOverride,
	FExpressionInput& TransmittanceColor, FExpressionInput& ThinTranslucentSurfaceCoverage,
	FExpressionInput& WaterScatteringCoefficients, FExpressionInput& WaterAbsorptionCoefficients, FExpressionInput& WaterPhaseG, FExpressionInput& ColorScaleBehindWater,
	const bool bHasAnisotropy, FExpressionInput& Anisotropy,
	FExpressionInput& Normal, FExpressionInput& Tangent,
	FExpressionInput& ClearCoatNormal, FExpressionInput& CustomTangent,
	const bool bHasSSS, USubsurfaceProfile* SSSProfile,
	const UMaterialEditorOnlyData* EditorOnlyData)
{
	LEGACY_DIRECT_ATTRIBUTE_MAPPING(RoughnessCodeChunk, MP_Roughness, CompileWithDefaultFloat1(Compiler, Roughness, 0.5f, EditorOnlyData ? &EditorOnlyData->Roughness : nullptr));
	LEGACY_DIRECT_ATTRIBUTE_MAPPING(AnisotropyCodeChunk, MP_Anisotropy, CompileWithDefaultFloat1(Compiler, Anisotropy, 0.0f, EditorOnlyData ? &EditorOnlyData->Anisotropy : nullptr));

	// Regular normal basis
	LEGACY_DIRECT_ATTRIBUTE_MAPPING(NormalCodeChunk, MP_Normal, CompileWithDefaultNormalWS(Compiler, Normal));

	// When computing NormalCodeChunk, we invoke TransformNormalFromRequestedBasisToWorld which requires input to be float or float3.
	// Certain material do not respect this requirement. We handle here a simple recovery when source material doesn't have a valid 
	// normal (e.g., vec2 normal), and avoid crashing the material compilation. The error will still be reported by the compiler up 
	// to the user, but the compilation will succeed.
	if (NormalCodeChunk == INDEX_NONE) { NormalCodeChunk = Compiler->VertexNormal(); }

	int32 TangentCodeChunk = INDEX_NONE;
	if (bHasAnisotropy)
	{
		LEGACY_DIRECT_ATTRIBUTE_MAPPING(TangentCodeChunkTmp, MP_Tangent, CompileWithDefaultTangentWS(Compiler, Tangent));
		TangentCodeChunk = TangentCodeChunkTmp;
	}
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk, TangentCodeChunk);
	const FString BasisIndexMacro = Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis);

	const bool bHasCoatNormal = ClearCoatNormal.IsConnected();
	// Clear coat normal basis
	int32 ClearCoat_NormalCodeChunk = INDEX_NONE;
	int32 ClearCoat_TangentCodeChunk = INDEX_NONE;
	FString ClearCoat_BasisIndexMacro;
	FSubstrateRegisteredSharedLocalBasis ClearCoat_NewRegisteredSharedLocalBasis;
	if (bHasCoatNormal)
	{
		ClearCoat_NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, ClearCoatNormal);
		ClearCoat_TangentCodeChunk = TangentCodeChunk;
		ClearCoat_NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, ClearCoat_NormalCodeChunk, ClearCoat_TangentCodeChunk);
		ClearCoat_BasisIndexMacro = Compiler->GetSubstrateSharedLocalBasisIndexMacro(ClearCoat_NewRegisteredSharedLocalBasis);
	}
	else
	{
		ClearCoat_NormalCodeChunk = NormalCodeChunk;
		ClearCoat_TangentCodeChunk = TangentCodeChunk;
		ClearCoat_NewRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
		ClearCoat_BasisIndexMacro = BasisIndexMacro;
	}

	// Custom tangent. No need to register it as a local basis, as it is only used for eye shading internal conversion
	int32 CustomTangent_TangentCodeChunk = INDEX_NONE;
	const bool bHasCustomTangent = CustomTangent.IsConnected();
	if (bHasCustomTangent)
	{
		// Legacy code doesn't do tangent <-> world basis conversion on tangent output, when provided.
		CustomTangent_TangentCodeChunk = CompileWithDefaultNormalWS(Compiler, CustomTangent, false /*bConvertToRequestedSpace*/);
	}
	else
	{
		CustomTangent_TangentCodeChunk = NormalCodeChunk;
	}

	int32 SSSProfileCodeChunk = INDEX_NONE;
	if (bHasSSS && SSSProfile)
	{
		SSSProfileCodeChunk = CreateSubsurfaceProfileParameter(Compiler, SSSProfile);
	}
	else
	{
		SSSProfileCodeChunk = CreateDefaultSubsurfaceProfileParameter(Compiler);
	}

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	int32 OpacityCodeChunk = INDEX_NONE;
	const float DefaultOpacity = 1.0f;
	if (!Compiler->SubstrateSkipsOpacityEvaluation())
	{
		// We evaluate opacity only for shading models and blending mode requiring it.
		// For instance, a translucent shader reading depth for soft fading should no evaluate opacity when an instance forces an opaque mode.
		LEGACY_DIRECT_ATTRIBUTE_MAPPING(OpacityCodeChunkTmp, MP_Opacity, CompileWithDefaultFloat1(Compiler, Opacity, DefaultOpacity, EditorOnlyData ? &EditorOnlyData->Opacity : nullptr));
		OpacityCodeChunk = OpacityCodeChunkTmp;
	}
	if(OpacityCodeChunk == INDEX_NONE)
	{
		OpacityCodeChunk = Compiler->Constant(DefaultOpacity);
	}

	LEGACY_DIRECT_ATTRIBUTE_MAPPING(EmissiveCodeChunk, MP_EmissiveColor, CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f, EditorOnlyData ? &EditorOnlyData->EmissiveColor : nullptr));

	LEGACY_DIRECT_ATTRIBUTE_MAPPING(BaseColorCodeChunk, MP_BaseColor, CompileWithDefaultFloat3(Compiler, BaseColor, 0.0f, 0.0f, 0.0f, EditorOnlyData ? &EditorOnlyData->BaseColor : nullptr));
	LEGACY_DIRECT_ATTRIBUTE_MAPPING(SpecularChunk, MP_Specular, CompileWithDefaultFloat1(Compiler, Specular, 0.5f, EditorOnlyData ? &EditorOnlyData->Specular: nullptr));
	LEGACY_DIRECT_ATTRIBUTE_MAPPING(MetallicCodeChunk, MP_Metallic, CompileWithDefaultFloat1(Compiler, Metallic, 0.0f, EditorOnlyData ? &EditorOnlyData->Metallic : nullptr));

	LEGACY_DIRECT_ATTRIBUTE_MAPPING(SubSurfaceColorCodeChunk, MP_SubsurfaceColor, CompileWithDefaultFloat3(Compiler, SubSurfaceColor, 1.0f, 1.0f, 1.0f, EditorOnlyData ? &EditorOnlyData->SubsurfaceColor : nullptr));

	LEGACY_DIRECT_ATTRIBUTE_MAPPING(ClearCoatCodeChunk, MP_CustomData0, CompileWithDefaultFloat1(Compiler, ClearCoat, 1.0f, EditorOnlyData ? &EditorOnlyData->ClearCoat : nullptr));
	LEGACY_DIRECT_ATTRIBUTE_MAPPING(ClearCoatRoughnessCodeChunk, MP_CustomData1, CompileWithDefaultFloat1(Compiler, ClearCoatRoughness, 0.1f, EditorOnlyData ? &EditorOnlyData->ClearCoatRoughness: nullptr));

	int32 ShadingModelCodeChunk = ShadingModel.IsConnected() ? CompileWithDefaultFloat1(Compiler, ShadingModel, float(MSM_DefaultLit)) : Compiler->Constant(float(ShadingModelOverride));
	int32 ShadingModelCount = Compiler->GetMaterialShadingModels().CountShadingModels();
	const bool bHasDynamicShadingModels = ShadingModelCount > 1;
	int32 OutputCodeChunk = Compiler->SubstrateConversionFromLegacy(
		bHasDynamicShadingModels,
		// Metalness workflow
		BaseColorCodeChunk,
		SpecularChunk,
		MetallicCodeChunk,
		// Roughness
		RoughnessCodeChunk,
		AnisotropyCodeChunk,
		// SSS
		SubSurfaceColorCodeChunk,
		SSSProfileCodeChunk != INDEX_NONE ? SSSProfileCodeChunk : Compiler->Constant(0.0f),
		// Clear Coat / Custom
		ClearCoatCodeChunk,
		ClearCoatRoughnessCodeChunk,
		// Misc
		EmissiveCodeChunk,
		OpacityCodeChunk,
		CompileWithDefaultFloat3(Compiler, TransmittanceColor, 0.5f, 0.5f, 0.5f),
		CompileWithDefaultFloat1(Compiler, ThinTranslucentSurfaceCoverage, 1.0f),
		// Water
		CompileWithDefaultFloat3(Compiler, WaterScatteringCoefficients, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat3(Compiler, WaterAbsorptionCoefficients, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, WaterPhaseG, 0.0f),
		CompileWithDefaultFloat3(Compiler, ColorScaleBehindWater, 1.0f, 1.0f, 1.0f),
		// Shading model
		ShadingModelCodeChunk,
		NormalCodeChunk,
		TangentCodeChunk,
		BasisIndexMacro,
		ClearCoat_NormalCodeChunk,
		ClearCoat_TangentCodeChunk,
		ClearCoat_BasisIndexMacro,
		CustomTangent_TangentCodeChunk,
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

int32 UMaterialExpressionSubstrateShadingModels::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return UMaterialExpressionSubstrateShadingModels::CompileCommon(Compiler,
		BaseColor, Specular, Metallic, Roughness, EmissiveColor,
		Opacity, SubSurfaceColor, ClearCoat, ClearCoatRoughness,
		ShadingModel, ShadingModelOverride,
		TransmittanceColor, ThinTranslucentSurfaceCoverage,
		WaterScatteringCoefficients, WaterAbsorptionCoefficients, WaterPhaseG, ColorScaleBehindWater,
		HasAnisotropy(), Anisotropy,
		Normal, Tangent,
		ClearCoatNormal, CustomTangent,
		HasSSS(), SubsurfaceProfile);
}


void UMaterialExpressionSubstrateShadingModels::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (GraphNode && PropertyChangedEvent.Property != nullptr)
	{
		GraphNode->ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionSubstrateShadingModels::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Shading Models"));
}

EMaterialValueType UMaterialExpressionSubstrateShadingModels::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateShadingModels::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)	   return MCT_Float3; // BaseColor
	else if (InputIndex == 1)  return MCT_Float1; // Metallic
	else if (InputIndex == 2)  return MCT_Float1; // Specular
	else if (InputIndex == 3)  return MCT_Float1; // Roughness
	else if (InputIndex == 4)  return MCT_Float1; // Anisotropy
	else if (InputIndex == 5)  return MCT_Float3; // EmissiveColor
	else if (InputIndex == 6)  return MCT_Float3; // Normal
	else if (InputIndex == 7)  return MCT_Float3; // Tangent
	else if (InputIndex == 8)  return MCT_Float3; // SubSurfaceColor
	else if (InputIndex == 9)  return MCT_Float1; // ClearCoat/Custom0
	else if (InputIndex == 10) return MCT_Float1; // ClearCoatRoughness/Custom1
	else if (InputIndex == 11) return MCT_Float1; // Opacity
	else if (InputIndex == 12) return MCT_Float3; // TransmittanceColor
	else if (InputIndex == 13) return MCT_Float3; // WaterScatteringCoefficients
	else if (InputIndex == 14) return MCT_Float3; // WaterAbsorptionCoefficients
	else if (InputIndex == 15) return MCT_Float1; // WaterPhaseG
	else if (InputIndex == 16) return MCT_Float3; // ColorScaleBehindWater
	else if (InputIndex == 17) return MCT_Float3; // ClearCoatNormal / IrisNormal
	else if (InputIndex == 18) return MCT_Float3; // CustomTangent
	else if (InputIndex == 19) return MCT_Float1; // ThinTranslucentSurfaceCoverage
	else if (InputIndex == 20) return MCT_ShadingModel; // ShadingModel
	else if (InputIndex == 21) return MCT_ShadingModel; // EMaterialShadingModel with ShowAsInputPin seems to always show at the bottom

	check(false);
	return MCT_Float1;
}

FName UMaterialExpressionSubstrateShadingModels::GetInputName(int32 InputIndex) const
{
	const bool bShadingModelFromExpression = ShadingModel.IsConnected();

	if (InputIndex == 0)		return TEXT("BaseColor");
	else if (InputIndex == 1)	return TEXT("Metallic");
	else if (InputIndex == 2)	return TEXT("Specular");
	else if (InputIndex == 3)	return TEXT("Roughness");
	else if (InputIndex == 4)	return TEXT("Anisotropy");
	else if (InputIndex == 5)	return TEXT("Emissive Color");
	else if (InputIndex == 6)	return TEXT("Normal");
	else if (InputIndex == 7)	return TEXT("Tangent");
	else if (InputIndex == 8)
	{
		if (!bShadingModelFromExpression && ShadingModelOverride == MSM_Cloth)
		{
			return TEXT("Fuzz Color");
		}
		return TEXT("Subsurface Color");
	}
	else if (InputIndex == 9)
	{
		if (!bShadingModelFromExpression)
		{
			if (ShadingModelOverride == MSM_Cloth)
			{
				return TEXT("Fuzz Amount");
			}
			else if (ShadingModelOverride == MSM_Eye)
			{
				return TEXT("Iris Mask");
			}
			else if (ShadingModelOverride == MSM_Hair)
			{
				return TEXT("Backlit");
			}
			else if (ShadingModelOverride == MSM_ClearCoat)
			{
				return TEXT("Clear Coat");
			}
			return TEXT("Unused");
		}
		return TEXT("Custom0");
	}
	else if (InputIndex == 10)
	{
		if (!bShadingModelFromExpression)
		{
			if (ShadingModelOverride == MSM_Eye)
			{
				return TEXT("Iris Distance");
			}
			else if (ShadingModelOverride == MSM_ClearCoat)
			{
				return TEXT("Clear Coat Roughness");
			}
			return TEXT("Unused");
		}
		return TEXT("Custom1");
	}
	else if (InputIndex == 11)	return TEXT("Opacity");
	else if (InputIndex == 12)	return TEXT("Thin Translucent Transmittance Color");
	else if (InputIndex == 13)	return TEXT("Water Scattering Coefficients");
	else if (InputIndex == 14)	return TEXT("Water Absorption Coefficients");
	else if (InputIndex == 15)	return TEXT("Water Phase G");
	else if (InputIndex == 16)	return TEXT("Color Scale BehindWater");
	else if (InputIndex == 17)
	{
		if (!bShadingModelFromExpression && ShadingModelOverride == MSM_ClearCoat)
		{
			return TEXT("Clear Coat Bottom Normal");
		}
		else if (!bShadingModelFromExpression && ShadingModelOverride == MSM_Eye)
		{
			return TEXT("Iris Normal");
		}
		return TEXT("Unused");
	}
	else if (InputIndex == 18)
	{
		if (!bShadingModelFromExpression && ShadingModelOverride == MSM_Eye)
		{
			return TEXT("Iris Tangent");
		}
		return TEXT("Custom Tangent");
	}
	else if (InputIndex == 19)	return TEXT("Thin Translucent Surface Coverage");
	else if (InputIndex == 20)	return TEXT("Single Shading Model");
	else if (InputIndex == 21)	return TEXT("Shading Model From Expression");
	return TEXT("Unknown");
}

void UMaterialExpressionSubstrateShadingModels::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (OutputIndex == 0)
	{
		OutToolTip.Add(TEXT("TT Ouput"));
		return;
	}
	Super::GetConnectorToolTip(InputIndex, INDEX_NONE, OutToolTip);
}

bool UMaterialExpressionSubstrateShadingModels::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateShadingModels::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// Track connected input
	if (BaseColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_BaseColor); }
	if (Metallic.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Metallic); }
	if (Specular.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Specular); }
	if (Roughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (Anisotropy.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Anisotropy); }
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	if (Normal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Normal); }
	if (Tangent.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Tangent); }
	if (SubSurfaceColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_SubsurfaceColor); }
	if (ClearCoat.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_CustomData0); }
	if (ClearCoatRoughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_CustomData1); }
	if (Opacity.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Opacity); }

	if (ShadingModel.IsConnected())
	{
		SubstrateMaterialInfo.AddPropertyConnected(MP_ShadingModel);

		// If the ShadingModel pin is plugged in, we must use a shading model from expression path.
		SubstrateMaterialInfo.SetShadingModelFromExpression(true);
	}
	else
	{
		// If the ShadingModel pin is NOT plugged in, we simply use the shading model selected on the root node drop box.
		if (ShadingModelOverride == MSM_Unlit) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Unlit); }
		if (ShadingModelOverride == MSM_DefaultLit) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_DefaultLit); }
		if (ShadingModelOverride == MSM_Subsurface) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceWrap); }
		if (ShadingModelOverride == MSM_PreintegratedSkin) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceWrap); }
		if (ShadingModelOverride == MSM_ClearCoat) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_ClearCoat); }
		if (ShadingModelOverride == MSM_SubsurfaceProfile) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceProfile); }
		if (ShadingModelOverride == MSM_TwoSidedFoliage) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceThinTwoSided); }
		if (ShadingModelOverride == MSM_Hair) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Hair); }
		if (ShadingModelOverride == MSM_Cloth) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Cloth); }
		if (ShadingModelOverride == MSM_Eye) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Eye); }
		if (ShadingModelOverride == MSM_SingleLayerWater) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SingleLayerWater); }
		if (ShadingModelOverride == MSM_ThinTranslucent) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_ThinTranslucent); }
	}

	if (SubsurfaceProfile)
	{
		SubstrateMaterialInfo.AddSubsurfaceProfile(SubsurfaceProfile);
	}
}

static void AppendLegacyShadingModelToBSDFFeature(const FMaterialShadingModelField& ShadingModels, const bool bAnisotropyEnabled, ESubstrateBsdfFeature& BsdfFeatures)
{
	if (ShadingModels.HasShadingModel(MSM_Eye))			{ BsdfFeatures |= ESubstrateBsdfFeature::Eye; }
	if (ShadingModels.HasShadingModel(MSM_Hair))		{ BsdfFeatures |= ESubstrateBsdfFeature::Hair; }
	if (ShadingModels.HasShadingModel(MSM_Cloth))		{ BsdfFeatures |= ESubstrateBsdfFeature::Fuzz; }
	if (ShadingModels.HasShadingModel(MSM_ClearCoat))	{ BsdfFeatures |= ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat; }

	if (ShadingModels.HasShadingModel(MSM_Subsurface) || ShadingModels.HasShadingModel(MSM_SubsurfaceProfile) ||
		ShadingModels.HasShadingModel(MSM_TwoSidedFoliage) || ShadingModels.HasShadingModel(MSM_Eye) ||
		ShadingModels.HasShadingModel(MSM_PreintegratedSkin))
	{
		BsdfFeatures |= ESubstrateBsdfFeature::SSS;
	}

	if (bAnisotropyEnabled && (ShadingModels.HasShadingModel(MSM_DefaultLit) || ShadingModels.HasShadingModel(MSM_ClearCoat)))
	{
		BsdfFeatures |= ESubstrateBsdfFeature::Anisotropy;
	}
}

FSubstrateOperator* UMaterialExpressionSubstrateShadingModels::SubstrateGenerateMaterialTopologyTreeCommon(
	class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
	const FExpressionInput& EmissiveColor,
	const FExpressionInput& Anisotropy,
	const FExpressionInput& ClearCoatNormal,
	const FExpressionInput& CustomTangent,
	const FExpressionInput& ShadingModel)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(
		Compiler->GetSubstrateTranslatorData(), ThisExpressionGuid, Parent, OutputIndex,
		EmissiveColor,
		Anisotropy,
		ClearCoatNormal,
		CustomTangent,
		ShadingModel);
}

FSubstrateOperator* UMaterialExpressionSubstrateShadingModels::SubstrateGenerateMaterialTopologyTreeCommon(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
	const FExpressionInput& EmissiveColor,
	const FExpressionInput& Anisotropy,
	const FExpressionInput& ClearCoatNormal,
	const FExpressionInput& CustomTangent,
	const FExpressionInput& ShadingModel)
{
	// Note Thickness has no meaning/usage in the context of SubstrateLegacyConversionNode
	int32 ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();

	auto AddDefaultWorstCase = [&](ESubstrateBsdfFeature In)
	{
		FSubstrateOperator& SlabOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
		SlabOperator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;
		SlabOperator.BSDFFeatures = In | (Anisotropy.IsConnected() ? ESubstrateBsdfFeature::Anisotropy : ESubstrateBsdfFeature::None);
		SlabOperator.SubSurfaceType = uint8(EMaterialSubSurfaceType::MSS_None);
		SlabOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
		SlabOperator.ThicknessIndex = ThicknessIndex;

		return &SlabOperator;
	};

	// Get the shading models resulting from the UMaterial::RebuildShadingModelField().
	FMaterialShadingModelField ShadingModels = SubstrateTranslatorData.GetMaterialShadingModels();

	bool bEyeIrisNormalPluggedIn = ClearCoatNormal.IsConnected();
	bool bEyeIrisTangentPluggedIn = CustomTangent.IsConnected();
	auto ApplyEyeIrisUsedFeatures = [&](FSubstrateOperator* Operator)
	{
		if (bEyeIrisNormalPluggedIn)	Operator->BSDFFeatures |= ESubstrateBsdfFeature::EyeIrisNormalPluggedIn;
		if (bEyeIrisTangentPluggedIn)	Operator->BSDFFeatures |= ESubstrateBsdfFeature::EyeIrisTangentPluggedIn;
	};

	// Logic about shading models and complexity should match UMaterialExpressionSubstrateShadingModels::Compile.
	const bool bHasShadingModelFromExpression = ShadingModel.IsConnected(); // We keep HasShadingModelFromExpression in case all shading models cannot be safely recovered from material functions.
	if ((ShadingModels.CountShadingModels() > 1) || bHasShadingModelFromExpression)
	{
		// Special case for unlit only material to get fast path
		if (ShadingModels.HasOnlyShadingModel(MSM_Unlit))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
			Operator.ThicknessIndex = ThicknessIndex;
			return &Operator;
		}

		// Be sure to track eye/hair feature, even though they are not part of Slab BSDF. This is important later for issuing 
		// the correct material complexity, as hair/eye requires 'complex' complexity (not 'simple') for correct packing
		ESubstrateBsdfFeature BsdfFeatures = ESubstrateBsdfFeature::None;
		AppendLegacyShadingModelToBSDFFeature(ShadingModels, Anisotropy.IsConnected(), BsdfFeatures);

		FSubstrateOperator* Operator = AddDefaultWorstCase(BsdfFeatures);
		if (ShadingModels.HasShadingModel(MSM_Eye) || bHasShadingModelFromExpression)
		{
			ApplyEyeIrisUsedFeatures(Operator);
		}
		return Operator;
	}
	// else
	{
		check(ShadingModels.CountShadingModels() == 1);

		if (ShadingModels.HasShadingModel(MSM_Unlit))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
			Operator.ThicknessIndex = ThicknessIndex;
			Operator.bBSDFWritesEmissive = true;
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_DefaultLit))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::None);
		}
		else if (ShadingModels.HasShadingModel(MSM_ThinTranslucent))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::MFPPluggedIn);
		}
		else if (ShadingModels.HasShadingModel(MSM_SubsurfaceProfile))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_Subsurface))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_TwoSidedFoliage))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_PreintegratedSkin))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_Cloth))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::Fuzz);
		}
		else if (ShadingModels.HasShadingModel(MSM_ClearCoat))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;
			Operator.ThicknessIndex = ThicknessIndex;
			Operator.BSDFFeatures = ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat | (Anisotropy.IsConnected() ? ESubstrateBsdfFeature::Anisotropy : ESubstrateBsdfFeature::None);
			Operator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_Hair))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_HAIR;
			Operator.BSDFFeatures = ESubstrateBsdfFeature::Hair;
			Operator.ThicknessIndex = ThicknessIndex;
			Operator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_Eye))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_EYE;
			Operator.BSDFFeatures = ESubstrateBsdfFeature::Eye;
			ApplyEyeIrisUsedFeatures(&Operator);

			Operator.ThicknessIndex = ThicknessIndex;
			Operator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_SingleLayerWater))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER;
			Operator.ThicknessIndex = ThicknessIndex;
			Operator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
			return &Operator;
		}

		check(false);
		static FSubstrateOperator DefaultOperatorOnError;
		return &DefaultOperatorOnError;
	}
}

FSubstrateOperator* UMaterialExpressionSubstrateShadingModels::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(
		SubstrateTranslatorData, this->MaterialExpressionGuid, Parent, OutputIndex,
		EmissiveColor,
		Anisotropy,
		ClearCoatNormal,
		CustomTangent,
		ShadingModel);
}

bool UMaterialExpressionSubstrateShadingModels::HasSSS() const
{
	return SubsurfaceProfile != nullptr;
}

bool UMaterialExpressionSubstrateShadingModels::HasAnisotropy() const
{
	return Anisotropy.IsConnected();
}

#endif // WITH_EDITOR


UMaterialExpressionSubstrateBSDF::UMaterialExpressionSubstrateBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateBSDF::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Compile the SubstrateData output.
	int32 SubstrateDataCodeChunk = Compile(Compiler, OutputIndex);
	// Convert the SubstrateData to a preview color.
	int32 PreviewCodeChunk = Compiler->SubstrateCompilePreview(SubstrateDataCodeChunk);
	return PreviewCodeChunk;
}
#endif

UMaterialExpressionSubstrateSlabBSDF::UMaterialExpressionSubstrateSlabBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseSSSDiffusion(true)
	, SubSurfaceType(EMaterialSubSurfaceType::MSS_Diffusion)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif

	// Sanity check
	static_assert(MSS_MAX == SSS_TYPE_COUNT);
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateSlabBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FGuid PathUniqueId = Compiler->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(PathUniqueId);

	// We also cannot ignore the tangent when using the default Tangent because GetTangentBasis
	// used in SubstrateGetBSDFSharedBasis cannot be relied on for smooth tangent used for lighting on any mesh.
	const bool bHasAnisotropy = SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy);

	int32 SSSProfileCodeChunk = INDEX_NONE;
	if (SubstrateOperator.Has(ESubstrateBsdfFeature::SSS) && SubsurfaceProfile)
	{
		SSSProfileCodeChunk = CreateSubsurfaceProfileParameter(Compiler, SubsurfaceProfile);
	}
	else
	{
		SSSProfileCodeChunk = CreateDefaultSubsurfaceProfileParameter(Compiler);
	}

	int32 SpecularProfileCodeChunk = INDEX_NONE;
	if (SubstrateOperator.Has(ESubstrateBsdfFeature::SpecularProfile))
	{
		const FName SpecularProfileParameterName = SpecularProfile::CreateSpecularProfileParameterName(SpecularProfile);
		SpecularProfileCodeChunk = Compiler->ForceCast(Compiler->ScalarParameter(SpecularProfileParameterName, 1.0f), MCT_Float1);
	}

	const float DefaultSpecular = 0.5f;
	const float DefaultF0 = DielectricSpecularToF0(DefaultSpecular);

	int32 NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, Normal);
	int32 TangentCodeChunk = bHasAnisotropy ? CompileWithDefaultTangentWS(Compiler, Tangent) : INDEX_NONE;
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk, TangentCodeChunk);

	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	int32 ThicknesCodeChunk = INDEX_NONE;
	if (SubstrateOperator.ThicknessIndex != INDEX_NONE)
	{
		ThicknesCodeChunk = Compiler->SubstrateThicknessStackGetThicknessCode(SubstrateOperator.ThicknessIndex);
	}
	else
	{
		// Thickness is not tracked properly, this can happen when opening a material function in editor
		ThicknesCodeChunk = Compiler->Constant(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
	}
	check(ThicknesCodeChunk != INDEX_NONE);

	int32 DiffuseAlbedoCodeChunk = CompileWithDefaultFloat3(Compiler, DiffuseAlbedo, 0.18f, 0.18f, 0.18f);
	int32 F0CodeChunk = CompileWithDefaultFloat3(Compiler, F0, DefaultF0, DefaultF0, DefaultF0);
	int32 RoughnessCodeChunk = CompileWithDefaultFloat1(Compiler, Roughness, 0.5f);
	int32 AnisotropyCodeChunk = CompileWithDefaultFloat1(Compiler, Anisotropy, 0.0f);
	int32 F90CodeChunk = CompileWithDefaultFloat3(Compiler, F90, 1.0f, 1.0f, 1.0f);
	int32 SSSMFPCodeChunk = CompileWithDefaultFloat3(Compiler, SSSMFP, 0.0f, 0.0f, 0.0f);
	int32 SSSMFPScaleCodeChunk = CompileWithDefaultFloat1(Compiler, SSSMFPScale, 1.0f);
	int32 SSSPhaseAnisotropyCodeChunk = CompileWithDefaultFloat1(Compiler, SSSPhaseAnisotropy, 0.0f);
	int32 SecondRoughnessCodeChunk = CompileWithDefaultFloat1(Compiler, SecondRoughness, 0.0f);
	int32 SecondRoughnessWeightCodeChunk = CompileWithDefaultFloat1(Compiler, SecondRoughnessWeight, 0.0f);
	int32 FuzzAmountCodeChunk = CompileWithDefaultFloat1(Compiler, FuzzAmount, 0.0f);
	int32 FuzzColorCodeChunk = CompileWithDefaultFloat3(Compiler, FuzzColor, 0.0f, 0.0f, 0.0f);
	int32 FuzzRoughnessCodeChunk = HasFuzzRoughness() ? CompileWithDefaultFloat1(Compiler, FuzzRoughness, 0.5f) : RoughnessCodeChunk;
	int32 GlintValueCodeChunk = CompileWithDefaultFloat1(Compiler, GlintValue, 1.0f);
	int32 GlintUVCodeChunk = CompileWithDefaultFloat2(Compiler, GlintUV, 0.0f, 0.0f);

	// Disable some features if requested by the simplification process
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::MFPPluggedIn) || SubSurfaceType == EMaterialSubSurfaceType::MSS_None)
	{
		SSSMFPCodeChunk = Compiler->Constant3(0.0f, 0.0f, 0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::EdgeColor))
	{
		F90CodeChunk = Compiler->Constant3(1.0f, 1.0f, 1.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::Fuzz))
	{
		FuzzAmountCodeChunk = Compiler->Constant(0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat))
	{
		SecondRoughnessWeightCodeChunk = Compiler->Constant(0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy))
	{
		AnisotropyCodeChunk = Compiler->Constant(0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::Glint))
	{
		GlintValueCodeChunk = Compiler->Constant(1.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::SpecularProfile))
	{
		SpecularProfileCodeChunk = INDEX_NONE;
	}

	int32 OutputCodeChunk = Compiler->SubstrateSlabBSDF(
		DiffuseAlbedoCodeChunk,
		F0CodeChunk,
		F90CodeChunk,
		RoughnessCodeChunk,
		AnisotropyCodeChunk,
		SSSProfileCodeChunk != INDEX_NONE ? SSSProfileCodeChunk : Compiler->Constant(0.0f),
		SSSMFPCodeChunk,
		SSSMFPScaleCodeChunk,
		SSSPhaseAnisotropyCodeChunk,
		Compiler->Constant(SubSurfaceType),
		CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f),
		SecondRoughnessCodeChunk,
		SecondRoughnessWeightCodeChunk,
		Compiler->Constant(0.0f),										// SecondRoughnessAsSimpleClearCoat
		NormalCodeChunk,
		FuzzAmountCodeChunk,
		FuzzColorCodeChunk,
		FuzzRoughnessCodeChunk,
		ThicknesCodeChunk,
		GlintValueCodeChunk,
		GlintUVCodeChunk,
		SpecularProfileCodeChunk != INDEX_NONE ? SpecularProfileCodeChunk : Compiler->Constant(0.0f),
		SubstrateOperator.bIsBottom > 0 ? true : false, 				// bIsAtTheBottomOfTopology
		NormalCodeChunk,
		TangentCodeChunk,
		Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

FSubstrateMaterialComplexity UMaterialExpressionSubstrateSlabBSDF::GetHighestComplexity() const
{
	// This function returns the highest complexity of a materials.
	// It will be lowered depending on features enabled per platform.

	ESubstrateBsdfFeature FeatureMask = ESubstrateBsdfFeature::None;
	if (HasGlint()) 			{ FeatureMask |= ESubstrateBsdfFeature::Glint; } 
	if (HasAnisotropy()) 		{ FeatureMask |= ESubstrateBsdfFeature::Anisotropy; } 
	if (HasSpecularProfile()) 	{ FeatureMask |= ESubstrateBsdfFeature::SpecularProfile; } 
	if (HasEdgeColor()) 		{ FeatureMask |= ESubstrateBsdfFeature::EdgeColor; } 
	if (HasFuzz()) 				{ FeatureMask |= ESubstrateBsdfFeature::Fuzz; } 
	if (HasSecondRoughness()) 	{ FeatureMask |= ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat; } 
	if (HasMFPPluggedIn()) 		{ FeatureMask |= ESubstrateBsdfFeature::MFPPluggedIn; } 
	if (HasSSS()) 				{ FeatureMask |= ESubstrateBsdfFeature::SSS; } 

	FSubstrateMaterialComplexity Out;
	Out.Reset();
	if (EnumHasAnyFlags(FeatureMask,ESubstrateBsdfFeature::ComplexSpecialMask))
	{
		Out.bIsComplexSpecial = true;
	}
	else if (EnumHasAnyFlags(FeatureMask,ESubstrateBsdfFeature::ComplexMask))
	{
		// Nothing
	}
	else if (EnumHasAnyFlags(FeatureMask,ESubstrateBsdfFeature::SingleMask))
	{
		Out.bIsSingle = true;
	}
	else
	{
		Out.bIsSimple = true;
	}

	return Out;
}

void UMaterialExpressionSubstrateSlabBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	// The node complexity is manually maintained to match FSubstrateCompilationContext::SubstrateGenerateDerivedMaterialOperatorData and shaders.
	OutCaptions.Add(TEXT("Substrate Slab BSDF - ") + FSubstrateMaterialComplexity::ToString(GetHighestComplexity().SubstrateMaterialType()));
}

void UMaterialExpressionSubstrateSlabBSDF::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT0", "Substrate Slab BSDF")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT1", "Complexity = ")).ToString() + FSubstrateMaterialComplexity::ToString(GetHighestComplexity().SubstrateMaterialType()));
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT2", "The complexity represents the cost of the shading path (Lighting, Lumen, SSS) the material will follow:")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT3", " - Simple means the Slab only relies on Diffuse, F0 and Roughness. It will follow a fast shading path.")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT4", " - Single means the Slab uses more features such as F90, Fuzz, Second Roughness, MFP or SSS. It will follow a more expenssive shading path.")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT5", " - Complex means a Slab uses anisotropic lighting, with any of the previous features.")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SlabNodeTT6", " - Complex Special means the Slab is using more advanced features such as glints or specular LUT. This is the most expenssive shading path.")).ToString());
}

EMaterialValueType UMaterialExpressionSubstrateSlabBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateSlabBSDF::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_Float3; // DiffuseAlbedo
	}
	else if (InputIndex == 1)
	{
		return MCT_Float3; // F0
	}
	else if (InputIndex == 2)
	{
		return MCT_Float3; // F90
	}
	else if (InputIndex == 3)
	{
		return MCT_Float1; // Roughness
	}
	else if (InputIndex == 4)
	{
		return MCT_Float1; // Anisotropy
	}
	else if (InputIndex == 5)
	{
		return MCT_Float3; // Normal
	}
	else if (InputIndex == 6)
	{
		return MCT_Float3; // Tangent
	}
	else if (InputIndex == 7)
	{
		return MCT_Float3; // SSSMFP
	}
	else if (InputIndex == 8)
	{
		return MCT_Float1; // SSSMFPScale
	}
	else if (InputIndex == 9)
	{
		return MCT_Float1; // SSSPhaseAniso
	}
	else if (InputIndex == 10)
	{
		return MCT_Float3; // Emissive Color
	}
	else if (InputIndex == 11)
	{
		return MCT_Float1; // SecondRoughness
	}
	else if (InputIndex == 12)
	{
		return MCT_Float1; // SecondRoughnessWeight
	}
	else if (InputIndex == 13)
	{
		return MCT_Float1; // FuzzRoughness
	}
	else if (InputIndex == 14)
	{
		return MCT_Float1; // FuzzAmount
	}
	else if (InputIndex == 15)
	{
		return MCT_Float3; // FuzzColor
	}
	else if (InputIndex == 16)
	{
		return MCT_Float; // GlintValue
	}
	else if (InputIndex == 17)
	{
		return MCT_Float2; // GlintUV
	}

	check(false);
	return MCT_Float1;
}

FName UMaterialExpressionSubstrateSlabBSDF::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Diffuse Albedo");
	}
	else if (InputIndex == 1)
	{
		return TEXT("F0");
	}
	else if (InputIndex == 2)
	{
		return  TEXT("F90");
	}
	else if (InputIndex == 3)
	{
		return TEXT("Roughness");
	}
	else if (InputIndex == 4)
	{
		return TEXT("Anisotropy");
	}
	else if (InputIndex == 5)
	{
		return TEXT("Normal");
	}
	else if (InputIndex == 6)
	{
		return TEXT("Tangent");
	}
	else if (InputIndex == 7)
	{
		return TEXT("SSS MFP");
	}
	else if (InputIndex == 8)
	{
		return TEXT("SSS MFP Scale");
	}
	else if (InputIndex == 9)
	{
		return TEXT("SSS Phase Anisotropy");
	}
	else if (InputIndex == 10)
	{
		return TEXT("Emissive Color");
	}
	else if (InputIndex == 11)
	{
		return TEXT("Second Roughness");
	}
	else if (InputIndex == 12)
	{
		return TEXT("Second Roughness Weight");
	}
	else if (InputIndex == 13)
	{
		return TEXT("Fuzz Roughness");
	}
	else if (InputIndex == 14)
	{
		return TEXT("Fuzz Amount");
	}
	else if (InputIndex == 15)
	{
		return TEXT("Fuzz Color");
	}
	else if (InputIndex == 16)
	{
		return Substrate::IsGlintEnabled(GMaxRHIShaderPlatform) ? TEXT("Glint Density") : TEXT("Glint Density (Disabled)");
	}
	else if (InputIndex == 17)
	{
		return Substrate::IsGlintEnabled(GMaxRHIShaderPlatform) ? TEXT("Glint UVs") : TEXT("Glint UVs (Disabled)");
	}

	return TEXT("Unknown");
}

void UMaterialExpressionSubstrateSlabBSDF::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (OutputIndex == 0)
	{
		OutToolTip.Add(TEXT("TT Ouput"));
		return;
	}

	Super::GetConnectorToolTip(InputIndex, INDEX_NONE, OutToolTip);
}

bool UMaterialExpressionSubstrateSlabBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateSlabBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// Track connected inputs
	if (DiffuseAlbedo.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_DiffuseColor); }
	if (F0.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_SpecularColor); }
	if (Roughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (Anisotropy.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Anisotropy); }
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	if (Normal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Normal); }
	if (Tangent.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Tangent); }
	if (SSSMFP.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_SubsurfaceColor); }

	if (HasSSS())
	{
		// We still do not know if this is going to be a real SSS node because it is only possible for BSDF at the bottom of the stack. Nevertheless, we take the worst case into account.
		if (SubsurfaceProfile)
		{
			SubstrateMaterialInfo.AddShadingModel(SSM_SubsurfaceProfile);
			SubstrateMaterialInfo.AddSubsurfaceProfile(SubsurfaceProfile);
		}
		else
		{
			SubstrateMaterialInfo.AddShadingModel(SSM_SubsurfaceMFP);
		}
	}
	else
	{
		SubstrateMaterialInfo.AddShadingModel(SSM_DefaultLit);
	}

	if (HasSpecularProfile())
	{
		SubstrateMaterialInfo.AddSpecularProfile(SpecularProfile);
	}

	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateSlabBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;

	if (HasEdgeColor())			{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::EdgeColor; }
	if (HasFuzz())				{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::Fuzz; }
	if (HasSecondRoughness())	{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat; }
	if (HasSSS())				{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::SSS; }
	if (HasMFPPluggedIn())		{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::MFPPluggedIn; }
	if (HasAnisotropy())		{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::Anisotropy; }
	if (HasGlint())				{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::Glint; }
	if (HasSpecularProfile())	{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::SpecularProfile; }
	if (HasSSSProfile())		{ SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat; } // If a Slab has a subsurface profile, it will have haziness in order to to support the dual-specular lobe from the profile.

	SubstrateOperator.SubSurfaceType = SubSurfaceType;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	return &SubstrateOperator;
}

bool UMaterialExpressionSubstrateSlabBSDF::HasSSS() const
{
	return SubsurfaceProfile != nullptr || SSSMFP.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasSSSProfile() const
{
	return SubsurfaceProfile != nullptr;
}

bool UMaterialExpressionSubstrateSlabBSDF::HasMFPPluggedIn() const
{
	return SSSMFP.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasEdgeColor() const
{
	return F90.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasFuzz() const
{
	return FuzzAmount.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasFuzzRoughness() const
{
	return FuzzRoughness.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasSecondRoughness() const
{
	return SecondRoughnessWeight.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasAnisotropy() const
{
	return Anisotropy.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasGlint() const
{
	// We do not check Substrate::IsGlintEnabled() here. Because we want the glint coverage to affect lower platforms, so the data must flow.
	// The Translator will disable glint and reduce memory footprint if required for a plaform.
	return GlintValue.IsConnected();
}

bool UMaterialExpressionSubstrateSlabBSDF::HasSpecularProfile() const
{
	return SpecularProfile != nullptr;
}

void UMaterialExpressionSubstrateSlabBSDF::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (GraphNode && PropertyChangedEvent.Property != nullptr)
	{
		GraphNode->ReconstructNode();
		GraphNode->Modify();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR



UMaterialExpressionSubstrateSimpleClearCoatBSDF::UMaterialExpressionSubstrateSimpleClearCoatBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateSimpleClearCoatBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const float DefaultSpecular = 0.5f;
	const float DefaultF0 = DielectricSpecularToF0(DefaultSpecular);

	int32 NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, Normal);
	const int32 NullTangentCodeChunk = INDEX_NONE;
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk, NullTangentCodeChunk);

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	int32 ThicknessCodeChunk = Compiler->SubstrateThicknessStackGetThicknessCode(SubstrateOperator.ThicknessIndex);
	check(ThicknessCodeChunk != INDEX_NONE);

	int32 RoughnessCodeChunk = CompileWithDefaultFloat1(Compiler, Roughness, 0.5f);

	int32 BottomNormalCodeChunk = NormalCodeChunk;
	const bool bHasCoatBottomNormal = BottomNormal.IsConnected();
	if (bHasCoatBottomNormal)
	{
		BottomNormalCodeChunk = Compiler->ForceCast(CompileWithDefaultNormalWS(Compiler, BottomNormal), MCT_Float3, MFCF_ExactMatch | MFCF_ReplicateValue);
	}

	int32 OutputCodeChunk = Compiler->SubstrateSlabBSDF(
		CompileWithDefaultFloat3(Compiler, DiffuseAlbedo, 0.18f, 0.18f, 0.18f),		// DiffuseAlbedo
		CompileWithDefaultFloat3(Compiler, F0, DefaultF0, DefaultF0, DefaultF0),	// F0
		Compiler->Constant3(1.0f, 1.0f, 1.0f),					// F90		
		RoughnessCodeChunk,										// Roughness
		Compiler->Constant(0.0f),								// Anisotropy
		Compiler->Constant(0.0f),								// SSSProfile
		Compiler->Constant3(0.0f, 0.0f, 0.0f),					// SSSMFP
		Compiler->Constant(0.0f),								// SSSMFPScale
		Compiler->Constant(0.0f),								// SSSPhaseAnisotropy
		Compiler->Constant(EMaterialSubSurfaceType::MSS_None),	// SSSType
		CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, ClearCoatRoughness, 0.1f),
		CompileWithDefaultFloat1(Compiler, ClearCoatCoverage, 1.0f),
		Compiler->Constant(1.0f),								// SecondRoughnessAsSimpleClearCoat == true for UMaterialExpressionSubstrateSimpleClearCoatBSDF
		BottomNormalCodeChunk,									// ClearCoatBottomNormal
		Compiler->Constant(0.0f),								// FuzzAmount
		Compiler->Constant3(0.0f, 0.0f, 0.0f),					// FuzzColor
		RoughnessCodeChunk,										// FuzzRoughness
		ThicknessCodeChunk,										// Thickness
		Compiler->Constant(1.0f),								// GlintValue
		Compiler->Constant2(0.0f, 0.0f),						// GlintUV
		Compiler->Constant(0.0f),								// SpecularProfile
		true,													// bIsAtTheBottomOfTopology
		NormalCodeChunk,
		NullTangentCodeChunk,
		Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateSimpleClearCoatBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Simple Clear Coat"));
}

void UMaterialExpressionSubstrateSimpleClearCoatBSDF::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	OutToolTip.Add(FText(LOCTEXT("SCCNodeTT0", "Substrate Simple Clear Coat BSDF")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SCCNodeTT1", "It corresponds to the legacy \"Clear Coat\" shading model (also available using the \"Substrate Shading Models\" node).")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SCCNodeTT2", "It has the complexity \"Single\".")).ToString());
	OutToolTip.Add(FText(LOCTEXT("SCCNodeTT3", "Clear coat top layer controls are limited to coverage and roughness. It always has a fixed grey scale transmittance and F0=0.04.")).ToString());
}

EMaterialValueType UMaterialExpressionSubstrateSimpleClearCoatBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateSimpleClearCoatBSDF::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_Float3; // DiffuseAlbedo
	}
	else if (InputIndex == 1)
	{
		return MCT_Float3; // F0
	}
	else if (InputIndex == 2)
	{
		return MCT_Float1; // Roughness
	}
	else if (InputIndex == 3)
	{
		return MCT_Float1; // ClearCoatCoverage 
	}
	else if (InputIndex == 4)
	{
		return MCT_Float1; // ClearCoatRoughness
	}
	else if (InputIndex == 5)
	{
		return MCT_Float3; // Normal
	}
	else if (InputIndex == 6)
	{
		return MCT_Float3; // Emissive Color
	}
	else if (InputIndex == 7)
	{
		return MCT_Float3; // Bottom Normal
	}

	check(false);
	return MCT_Float1;
}

FName UMaterialExpressionSubstrateSimpleClearCoatBSDF::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Diffuse Albedo");
	}
	else if (InputIndex == 1)
	{
		return TEXT("F0");
	}
	else if (InputIndex == 2)
	{
		return TEXT("Roughness");
	}
	else if (InputIndex == 3)
	{
		return TEXT("Clear Coat Coverage");
	}
	else if (InputIndex == 4)
	{
		return TEXT("Clear Coat Roughness");
	}
	else if (InputIndex == 5)
	{
		return TEXT("Normal");
	}
	else if (InputIndex == 6)
	{
		return TEXT("Emissive Color");
	}
	else if (InputIndex == 7)
	{
		return TEXT("Bottom Normal");
	}

	return TEXT("Unknown");
}

bool UMaterialExpressionSubstrateSimpleClearCoatBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateSimpleClearCoatBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// Track connected inputs
	if (DiffuseAlbedo.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_BaseColor); }
	SubstrateMaterialInfo.AddPropertyConnected(MP_Metallic); // Metallic is always connected with Diffuse/F0 parameterisation
	if (F0.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Specular); }
	if (Roughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (Normal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Normal); }
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }

	SubstrateMaterialInfo.AddShadingModel(SSM_ClearCoat);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateSimpleClearCoatBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;
	SubstrateOperator.BSDFFeatures = ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat; // This node explicitly requires simple clear coat
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateVolumetricFogCloudBSDF::UMaterialExpressionSubstrateVolumetricFogCloudBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionSubstrateVolumetricFogCloudBSDF::CompileCommon(class FMaterialCompiler* Compiler,
	FExpressionInput& Albedo, FExpressionInput& Extinction, FExpressionInput& EmissiveColor, FExpressionInput& AmbientOcclusion,
	bool bEmissiveOnly,	
	const UMaterialEditorOnlyData* EditorOnlyData)
{
	int32 OutputCodeChunk = Compiler->SubstrateVolumetricFogCloudBSDF(
		CompileWithDefaultFloat3(Compiler, Albedo,				0.0f, 0.0f, 0.0f,	EditorOnlyData ? &EditorOnlyData->BaseColor : nullptr),
		CompileWithDefaultFloat3(Compiler, Extinction,			0.0f, 0.0f, 0.0f,	EditorOnlyData ? &EditorOnlyData->SubsurfaceColor : nullptr),
		CompileWithDefaultFloat3(Compiler, EmissiveColor,		0.0f, 0.0f, 0.0f,	EditorOnlyData ? &EditorOnlyData->EmissiveColor : nullptr),
		CompileWithDefaultFloat1(Compiler, AmbientOcclusion,	1.0f,				EditorOnlyData ? &EditorOnlyData->AmbientOcclusion : nullptr),
		bEmissiveOnly);

	return OutputCodeChunk;
}

int32 UMaterialExpressionSubstrateVolumetricFogCloudBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return CompileCommon(Compiler,
		Albedo, Extinction, EmissiveColor, AmbientOcclusion,
		bEmissiveOnly);
}

void UMaterialExpressionSubstrateVolumetricFogCloudBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	FString Caption = "Substrate Volumetric-Fog-Cloud BSDF";
	if (bEmissiveOnly)
	{
		Caption += TEXT("(Emissive Only)");
	}
	OutCaptions.Add(Caption);
}

EMaterialValueType UMaterialExpressionSubstrateVolumetricFogCloudBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateVolumetricFogCloudBSDF::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float3;
	case 2:
		return MCT_Float3;
	case 3:
		return MCT_Float1;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateVolumetricFogCloudBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateVolumetricFogCloudBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	SubstrateMaterialInfo.AddShadingModel(SSM_VolumetricFogCloud);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateVolumetricFogCloudBSDF::SubstrateGenerateMaterialTopologyTreeCommon(
	class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
	const FExpressionInput& EmissiveColor,
	const FExpressionInput& AmbientOcclusion)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(
		Compiler->GetSubstrateTranslatorData(), ThisExpressionGuid, Parent, OutputIndex,
		EmissiveColor,
		AmbientOcclusion);
}

FSubstrateOperator* UMaterialExpressionSubstrateVolumetricFogCloudBSDF::SubstrateGenerateMaterialTopologyTreeCommon(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
	const FExpressionInput& EmissiveColor,
	const FExpressionInput& AmbientOcclusion)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	SubstrateOperator.bBSDFWritesAmbientOcclusion = AmbientOcclusion.IsConnected();
	return &SubstrateOperator;
}

FSubstrateOperator* UMaterialExpressionSubstrateVolumetricFogCloudBSDF::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(
		SubstrateTranslatorData, this->MaterialExpressionGuid, Parent, OutputIndex,
		EmissiveColor,
		AmbientOcclusion);
}

#endif // WITH_EDITOR



UMaterialExpressionSubstrateLightFunction::UMaterialExpressionSubstrateLightFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Extras", "Substrate Extras")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateLightFunction::CompileCommon(class FMaterialCompiler* Compiler,
	FExpressionInput& Color, 
	const UMaterialEditorOnlyData* EditorOnlyData)
{
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	int32 OutputCodeChunk = Compiler->SubstrateUnlitBSDF(
		CompileWithDefaultFloat3(Compiler, Color, 0.0f, 0.0f, 0.0f, EditorOnlyData ? &EditorOnlyData->EmissiveColor : nullptr),
		Compiler->Constant3(1.0f, 1.0f, 1.0f),	// Opacity / Transmittance is ignored by light functions.
		Compiler->Constant3(0.0f, 0.0f, 1.0f),	// place holder normal
		&SubstrateOperator);
	return OutputCodeChunk;
}

int32 UMaterialExpressionSubstrateLightFunction::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return CompileCommon(Compiler, Color);
}

void UMaterialExpressionSubstrateLightFunction::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Light Function"));
}

EMaterialValueType UMaterialExpressionSubstrateLightFunction::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateLightFunction::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateLightFunction::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateLightFunction::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	SubstrateMaterialInfo.AddShadingModel(SSM_LightFunction);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateLightFunction::SubstrateGenerateMaterialTopologyTreeCommon(
	class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(Compiler->GetSubstrateTranslatorData(), ThisExpressionGuid, Parent, OutputIndex);
}

FSubstrateOperator* UMaterialExpressionSubstrateLightFunction::SubstrateGenerateMaterialTopologyTreeCommon(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
	SubstrateOperator.SubUsage = SUBSTRATE_OPERATOR_SUBUSAGE_LIGHTFUNCTION;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	return &SubstrateOperator;
}

FSubstrateOperator* UMaterialExpressionSubstrateLightFunction::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(SubstrateTranslatorData, this->MaterialExpressionGuid, Parent, OutputIndex);
}

#endif // WITH_EDITOR



UMaterialExpressionSubstratePostProcess::UMaterialExpressionSubstratePostProcess(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Extras", "Substrate Extras")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstratePostProcess::CompileCommon(class FMaterialCompiler* Compiler,
	FExpressionInput& Color, FExpressionInput& Opacity,
	const UMaterialEditorOnlyData* EditorOnlyData)
{
	int OpacityCodeChunk = CompileWithDefaultFloat1(Compiler, Opacity, 1.0f, EditorOnlyData ? &EditorOnlyData->Opacity : nullptr);
	int TransmittanceCodeChunk = Compiler->Saturate(Compiler->Sub(Compiler->Constant(1.0f), OpacityCodeChunk));

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	int32 OutputCodeChunk = Compiler->SubstrateUnlitBSDF(
		CompileWithDefaultFloat3(Compiler, Color, 0.0f, 0.0f, 0.0f, EditorOnlyData ? &EditorOnlyData->EmissiveColor : nullptr),
		TransmittanceCodeChunk,
		Compiler->Constant3(0.0f, 0.0f, 1.0f),	// place holder normal
		&SubstrateOperator);
	return OutputCodeChunk;
}

int32 UMaterialExpressionSubstratePostProcess::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return CompileCommon(Compiler, Color, Opacity);
}

void UMaterialExpressionSubstratePostProcess::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Post Process"));
}

EMaterialValueType UMaterialExpressionSubstratePostProcess::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstratePostProcess::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float1;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstratePostProcess::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstratePostProcess::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	SubstrateMaterialInfo.AddShadingModel(SSM_PostProcess);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstratePostProcess::SubstrateGenerateMaterialTopologyTreeCommon(
	class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(Compiler->GetSubstrateTranslatorData(), ThisExpressionGuid, Parent, OutputIndex);
}

FSubstrateOperator* UMaterialExpressionSubstratePostProcess::SubstrateGenerateMaterialTopologyTreeCommon(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
	SubstrateOperator.SubUsage = SUBSTRATE_OPERATOR_SUBUSAGE_POSTPROCESS;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	return &SubstrateOperator;
}

FSubstrateOperator* UMaterialExpressionSubstratePostProcess::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(SubstrateTranslatorData, this->MaterialExpressionGuid, Parent, OutputIndex);
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateUI::UMaterialExpressionSubstrateUI(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Extras", "Substrate Extras")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateUI::CompileCommon(class FMaterialCompiler* Compiler,
	FExpressionInput& Color, FExpressionInput& Opacity,
	const UMaterialEditorOnlyData* EditorOnlyData)
{
	int OpacityCodeChunk = CompileWithDefaultFloat1(Compiler, Opacity, 1.0f, EditorOnlyData ? &EditorOnlyData->Opacity : nullptr);

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	int32 OutputCodeChunk = Compiler->SubstrateUIBSDF(
		CompileWithDefaultFloat3(Compiler, Color, 0.0f, 0.0f, 0.0f, EditorOnlyData ? &EditorOnlyData->EmissiveColor : nullptr),
		OpacityCodeChunk,
		&SubstrateOperator);
	return OutputCodeChunk;
}

int32 UMaterialExpressionSubstrateUI::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return CompileCommon(Compiler, Color, Opacity);
}

void UMaterialExpressionSubstrateUI::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate UI"));
}

EMaterialValueType UMaterialExpressionSubstrateUI::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateUI::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateUI::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateUI::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	SubstrateMaterialInfo.AddShadingModel(SSM_UI);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateUI::SubstrateGenerateMaterialTopologyTreeCommon(
	class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(Compiler->GetSubstrateTranslatorData(), ThisExpressionGuid, Parent, OutputIndex);
}

FSubstrateOperator* UMaterialExpressionSubstrateUI::SubstrateGenerateMaterialTopologyTreeCommon(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
	SubstrateOperator.SubUsage = SUBSTRATE_OPERATOR_SUBUSAGE_UI;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	return &SubstrateOperator;
}

FSubstrateOperator* UMaterialExpressionSubstrateUI::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(SubstrateTranslatorData, this->MaterialExpressionGuid, Parent, OutputIndex);
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateConvertToDecal::UMaterialExpressionSubstrateConvertToDecal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Extras", "Substrate Extras")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateConvertToDecal::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!DecalMaterial.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing DecalMaterial input"));
	}

	int32 CoverageCodeChunk = Coverage.GetTracedInput().Expression ? Coverage.Compile(Compiler) : Compiler->Constant(1.0f);
	Compiler->SubstrateTreeStackPush(this, 0);
	int32 DecalMaterialCodeChunk = DecalMaterial.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();

	int32 OutputCodeChunk = INDEX_NONE;
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	if (!SubstrateOperator.bUseParameterBlending)
	{
		return Compiler->Errorf(TEXT("Substrate Convert To Decal node must receive SubstrateData a parameter blended Substrate material sub tree."));
	}
	if (!SubstrateOperator.bRootOfParameterBlendingSubTree)
	{
		return Compiler->Errorf(TEXT("Substrate Convert To Decal node must be the root of a parameter blending sub tree: no more Substrate operations can be applied a over its output."));
	}

	// Propagate the parameter blended normal
	FSubstrateOperator* Operator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = Operator->BSDFRegisteredSharedLocalBasis;

	OutputCodeChunk = Compiler->SubstrateWeightParameterBlending(
		DecalMaterialCodeChunk, CoverageCodeChunk,
		SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateConvertToDecal::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Convert To Decal"));
}

EMaterialValueType UMaterialExpressionSubstrateConvertToDecal::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateConvertToDecal::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Substrate;
	case 1:
		return MCT_Float1;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateConvertToDecal::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateConvertToDecal::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (!SubstrateMaterialInfo.PushSubstrateTreeStack())
	{
		return;
	}

	FExpressionInput TracedInput = DecalMaterial.GetTracedInput();
	if (TracedInput.Expression)
	{
		TracedInput.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInput.OutputIndex);
	}
	SubstrateMaterialInfo.AddShadingModel(SSM_Decal);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	SubstrateMaterialInfo.PopSubstrateTreeStack();
}

FSubstrateOperator* UMaterialExpressionSubstrateConvertToDecal::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	const bool bUseParameterBlending = true;
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_WEIGHT, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId(), bUseParameterBlending);
	SubstrateOperator.SubUsage = SUBSTRATE_OPERATOR_SUBUSAGE_DECAL;
	if (SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred())
	{
		return &SubstrateOperator; // Out ot stack space, return now to fail the compilation
	}

	FExpressionInput TracedInput = DecalMaterial.GetTracedInput();
	UMaterialExpression* ChildDecalMaterialExpression = TracedInput.Expression;
	FSubstrateOperator* OpA = nullptr;
	if (ChildDecalMaterialExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 0);
		OpA = ChildDecalMaterialExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInput.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.LeftIndex, OpA);
	}
	CombineFlagForParameterBlending(SubstrateOperator, OpA);

	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateUnlitBSDF::UMaterialExpressionSubstrateUnlitBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateUnlitBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	const int32 EmissiveColorIdx = CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f);
	const int32 TransmittanceColorIdx = CompileWithDefaultFloat3(Compiler, TransmittanceColor, 1.0f, 1.0f, 1.0f);
	const int32 NormalIdx = CompileWithDefaultNormalWS(Compiler, Normal);

	int32 OutputCodeChunk = Compiler->SubstrateUnlitBSDF(
		EmissiveColorIdx,
		TransmittanceColorIdx,
		NormalIdx,
		&SubstrateOperator);
	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateUnlitBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Unlit BSDF"));
}

EMaterialValueType UMaterialExpressionSubstrateUnlitBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateUnlitBSDF::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float3;
	case 2:
		return MCT_Float3;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateUnlitBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateUnlitBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	SubstrateMaterialInfo.AddShadingModel(SSM_Unlit);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateUnlitBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateHairBSDF::UMaterialExpressionSubstrateHairBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateHairBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// For hair, the shared local basis normal in fact represent the tangent
	int32 TangentCodeChunk = CompileWithDefaultTangentWS(Compiler, Tangent);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, TangentCodeChunk);

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	int32 OutputCodeChunk = Compiler->SubstrateHairBSDF(
		CompileWithDefaultFloat3(Compiler, BaseColor, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, Scatter, 0.0f),
		CompileWithDefaultFloat1(Compiler, Specular, 0.5f),
		CompileWithDefaultFloat1(Compiler, Roughness, 0.5f),
		CompileWithDefaultFloat1(Compiler, Backlit, 0.0f),
		CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f),
		TangentCodeChunk,
		Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateHairBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Hair BSDF"));
}

EMaterialValueType UMaterialExpressionSubstrateHairBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateHairBSDF::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float1;
	case 2:
		return MCT_Float1;
	case 3:
		return MCT_Float1;
	case 4:
		return MCT_Float1;
	case 5:
		return MCT_Float3;
	case 6:
		return MCT_Float3;
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateHairBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateHairBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// Track connected inputs
	if (BaseColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_BaseColor); }
	if (Specular.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Specular); }
	if (Roughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	if (Tangent.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Tangent); }

	SubstrateMaterialInfo.AddShadingModel(SSM_Hair);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateHairBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_HAIR;
	SubstrateOperator.BSDFFeatures = ESubstrateBsdfFeature::Hair;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	return &SubstrateOperator;
}
#endif // WITH_EDITOR

UMaterialExpressionSubstrateEyeBSDF::UMaterialExpressionSubstrateEyeBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateEyeBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CorneaNormalCodeChunk = CompileWithDefaultTangentWS(Compiler, CorneaNormal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, CorneaNormalCodeChunk);

	int32 SSSProfileCodeChunk = INDEX_NONE;
	if (SubsurfaceProfile != nullptr)
	{
		SSSProfileCodeChunk = CreateSubsurfaceProfileParameter(Compiler, SubsurfaceProfile);
	}
	else
	{
		SSSProfileCodeChunk = CreateDefaultSubsurfaceProfileParameter(Compiler);
	}

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	 
	int32 OutputCodeChunk = Compiler->SubstrateEyeBSDF(
		CompileWithDefaultFloat3(Compiler, DiffuseColor, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, Roughness, 0.5f),
		CompileWithDefaultFloat1(Compiler, IrisMask, 0.0f),
		CompileWithDefaultFloat1(Compiler, IrisDistance, 0.0f),
		CompileWithDefaultNormalWS(Compiler, IrisNormal),
		CompileWithDefaultNormalWS(Compiler, IrisPlaneNormal),
		SSSProfileCodeChunk != INDEX_NONE ? SSSProfileCodeChunk : Compiler->Constant(0.0f),
		CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f),
		CorneaNormalCodeChunk,
		Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateEyeBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Eye BSDF"));
}

EMaterialValueType UMaterialExpressionSubstrateEyeBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateEyeBSDF::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0: return MCT_Float3; // DiffuseColor
	case 1: return MCT_Float1; // Roughness
	case 2: return MCT_Float3; // Cornea normal
	case 3: return MCT_Float3; // IrisNormal
	case 4: return MCT_Float3; // IrisPlaneNormal
	case 5: return MCT_Float1; // IrisMask
	case 6: return MCT_Float1; // IrisDistance
	case 7: return MCT_Float3; // EmissiveColor
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateEyeBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateEyeBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// Track connected inputs
	if (DiffuseColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_BaseColor); }
	if (Roughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (CorneaNormal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Normal); }
	if (IrisNormal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Tangent); }
	if (IrisPlaneNormal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Tangent); }
	if (IrisMask.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_CustomData0); }
	if (IrisDistance.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_CustomData1); }
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	if (SubsurfaceProfile)
	{
		SubstrateMaterialInfo.AddSubsurfaceProfile(SubsurfaceProfile);
	}
	SubstrateMaterialInfo.AddShadingModel(SSM_Eye);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateEyeBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_EYE;
	SubstrateOperator.BSDFFeatures = ESubstrateBsdfFeature::Eye;
	if (IrisNormal.IsConnected())
	{
		SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::EyeIrisNormalPluggedIn;
	}
	if (IrisPlaneNormal.IsConnected())
	{
		SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::EyeIrisTangentPluggedIn;
	}

	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	return &SubstrateOperator;
}
#endif // WITH_EDITOR


UMaterialExpressionSubstrateSingleLayerWaterBSDF::UMaterialExpressionSubstrateSingleLayerWaterBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateSingleLayerWaterBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, Normal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk);

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	if (SubstrateOperator.bUseParameterBlending)
	{
		return Compiler->Errorf(TEXT("Substrate SingleLayerWater BSDF node cannot be used with parameter blending."));
	}
	else if (SubstrateOperator.bRootOfParameterBlendingSubTree)
	{
		return Compiler->Errorf(TEXT("Substrate SingleLayerWater BSDF node cannot be the root of a parameter blending sub tree."));
	}

	int32 OutputCodeChunk = Compiler->SubstrateSingleLayerWaterBSDF(
		CompileWithDefaultFloat3(Compiler, BaseColor, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, Metallic, 0.0f),
		CompileWithDefaultFloat1(Compiler, Specular, 0.5f),
		CompileWithDefaultFloat1(Compiler, Roughness, 0.5f),
		CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, TopMaterialOpacity, 0.0f),
		CompileWithDefaultFloat3(Compiler, WaterAlbedo, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat3(Compiler, WaterExtinction, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, WaterPhaseG, 0.0f),
		CompileWithDefaultFloat3(Compiler, ColorScaleBehindWater, 1.0f, 1.0f, 1.0f),
		NormalCodeChunk,
		Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
		&SubstrateOperator);

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateSingleLayerWaterBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Single Layer Water BSDF"));
}

EMaterialValueType UMaterialExpressionSubstrateSingleLayerWaterBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateSingleLayerWaterBSDF::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3; // BaseColor
	case 1:
		return MCT_Float1; // Metallic
	case 2:
		return MCT_Float1; // Specular
	case 3:
		return MCT_Float1; // Roughness
	case 4:
		return MCT_Float3; // Normal
	case 5:
		return MCT_Float3; // Emissive Color
	case 6:
		return MCT_Float1; // TopMaterialOpacity
	case 7:
		return MCT_Float3; // WaterAlbedo
	case 8:
		return MCT_Float3; // WaterExtinction
	case 9:
		return MCT_Float1; // WaterPhaseG
	case 10:
		return MCT_Float3; // ColorScaleBehindWater
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateSingleLayerWaterBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateSingleLayerWaterBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// Track connected inputs
	if (BaseColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_BaseColor); }
	if (Metallic.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Metallic); }
	if (Specular.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Specular); }
	if (Roughness.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (EmissiveColor.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	if (Normal.IsConnected()) { SubstrateMaterialInfo.AddPropertyConnected(MP_Normal); }

	SubstrateMaterialInfo.AddShadingModel(SSM_SingleLayerWater);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateSingleLayerWaterBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	return &SubstrateOperator;
}
#endif // WITH_EDITOR








UMaterialExpressionSubstrateToonBSDF::UMaterialExpressionSubstrateToonBSDF(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate BSDFs", "Substrate BSDFs")) {}
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateToonBSDF::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, Normal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk);

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	int32 ToonProfileIdCodeChunk = INDEX_NONE;
	if (ToonProfile != nullptr)
	{
		const FName ToonProfileParameterName = ToonProfile::CreateToonProfileParameterName(ToonProfile);
		ToonProfileIdCodeChunk = Compiler->ForceCast(Compiler->ScalarParameter(ToonProfileParameterName, 1.0f), MCT_Float1);
	}
	if (ToonProfileIdCodeChunk == INDEX_NONE)
	{
		ToonProfileIdCodeChunk = Compiler->Constant(0.0f);
	}

	// Unplugged PatternUVs result in UV0 being used, scaled by 10 to make offset/hatching textures visible by default.
	int32 PatternUVsCodeChunk = PatternUVs.GetTracedInput().Expression ? PatternUVs.Compile(Compiler) : Compiler->Mul(Compiler->TextureCoordinate(0, false, false), Compiler->Constant(10.0f));

	int32 OutputCodeChunk = Compiler->SubstrateToonBSDF(
		ToonProfileIdCodeChunk,
		CompileWithDefaultFloat3(Compiler, BaseColor, 0.18f, 0.18f, 0.18f),
		CompileWithDefaultFloat1(Compiler, Metallic, 0.0f),
		CompileWithDefaultFloat1(Compiler, Specular, 0.5f),
		CompileWithDefaultFloat1(Compiler, Roughness, 0.5f),
		CompileWithDefaultFloat3(Compiler, EmissiveColor, 0.0f, 0.0f, 0.0f),
		NormalCodeChunk,
		PatternUVsCodeChunk,
		CompileWithDefaultFloat1(Compiler, Anisotropy, 0.0f),
		CompileWithDefaultTangentWS(Compiler, Tangent),
		Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateToonBSDF::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Toon BSDF - EXPERIMENTAL"));
}

EMaterialValueType UMaterialExpressionSubstrateToonBSDF::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateToonBSDF::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3; // BaseColor
	case 1:
		return MCT_Float1; // Metallic
	case 2:
		return MCT_Float1; // Specular
	case 3:
		return MCT_Float1; // Roughness
	case 4:
		return MCT_Float3; // Normal
	case 5:
		return MCT_Float3; // Emissive Color
	case 6:
		return MCT_Float2; // Pattern UVs
	case 7:
		return MCT_Float1; // Anisotropy
	case 8:
		return MCT_Float3; // Tangent
	}

	check(false);
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateToonBSDF::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateToonBSDF::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	SubstrateMaterialInfo.AddShadingModel(SSM_Toon);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	if (ToonProfile != nullptr)
	{
		SubstrateMaterialInfo.AddToonProfile(ToonProfile);
	}

	if (Anisotropy.IsConnected())
	{
		SubstrateMaterialInfo.AddPropertyConnected(MP_Anisotropy);
	}
}

FSubstrateOperator* UMaterialExpressionSubstrateToonBSDF::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	SubstrateOperator.BSDFType = SUBSTRATE_BSDF_TYPE_TOON;
	SubstrateOperator.ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();
	SubstrateOperator.bBSDFWritesEmissive = EmissiveColor.IsConnected();
	SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::Toon;

	if (Anisotropy.IsConnected())
	{
		SubstrateOperator.BSDFFeatures |= ESubstrateBsdfFeature::Anisotropy;
	}

	return &SubstrateOperator;
}
#endif // WITH_EDITOR





UMaterialExpressionSubstrateHorizontalMixing::UMaterialExpressionSubstrateHorizontalMixing(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseParameterBlending(false)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Ops", "Substrate Operators")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateHorizontalMixing::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Foreground.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Foreground input"));
	}
	if (!Background.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Background input"));
	}

	Compiler->SubstrateTreeStackPush(this, 0);
	int32 BackgroundCodeChunk = Background.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();
	Compiler->SubstrateTreeStackPush(this, 1);
	int32 ForegroundCodeChunk = Foreground.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();

	const int32 HorizontalMixCodeChunk = CompileWithDefaultFloat1(Compiler, Mix, 0.5f);

	int32 OutputCodeChunk = INDEX_NONE;
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		if (ForegroundCodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("Foreground input graphs could not be evaluated for parameter blending."));
		}
		if (BackgroundCodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("Background input graphs could not be evaluated for parameter blending."));
		}
		const int32 NormalMixCodeChunk = Compiler->SubstrateHorizontalMixingParameterBlendingBSDFCoverageToNormalMixCodeChunk(BackgroundCodeChunk, ForegroundCodeChunk, HorizontalMixCodeChunk);

		FSubstrateOperator* BackgroundBSDFOperator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* ForegroundBSDFOperator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!BackgroundBSDFOperator || !ForegroundBSDFOperator)
		{
			return Compiler->Errorf(TEXT("Missing input on horizontal blending node."));
		}

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		const int32 NewNormalCodeChunk = SubstrateBlendNormal(Compiler, BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, NormalMixCodeChunk);
		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		int32 NewTangentCodeChunk = INDEX_NONE;
		if (ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE && BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = SubstrateBlendNormal(Compiler, BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, NormalMixCodeChunk);
		}
		else if (ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
		}
		else if (BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NewNormalCodeChunk, NewTangentCodeChunk);

		OutputCodeChunk = Compiler->SubstrateHorizontalMixingParameterBlending(
			BackgroundCodeChunk, ForegroundCodeChunk, HorizontalMixCodeChunk, NormalMixCodeChunk, Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
			BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk,
			SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr);

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		OutputCodeChunk = Compiler->SubstrateHorizontalMixing(
			BackgroundCodeChunk,
			ForegroundCodeChunk,
			HorizontalMixCodeChunk,
			SubstrateOperator.Index,
			SubstrateOperator.MaxDistanceFromLeaves);
	}

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateHorizontalMixing::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bUseParameterBlending)
	{
		OutCaptions.Add(TEXT("Substrate Horizontal Blend (Parameter Blend)"));
	}
	else
	{
		OutCaptions.Add(TEXT("Substrate Horizontal Blend"));
	}
}

EMaterialValueType UMaterialExpressionSubstrateHorizontalMixing::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateHorizontalMixing::GetInputValueType(int32 InputIndex)
{
	return InputIndex == 2 ? MCT_Float1 : MCT_Substrate;
}

bool UMaterialExpressionSubstrateHorizontalMixing::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateHorizontalMixing::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (!SubstrateMaterialInfo.PushSubstrateTreeStack())
	{
		return;
	}

	FExpressionInput TracedInputA = Foreground.GetTracedInput();
	FExpressionInput TracedInputB = Background.GetTracedInput();
	if (TracedInputA.Expression)
	{
		TracedInputA.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputA.OutputIndex);
	}
	if (TracedInputB.Expression)
	{
		TracedInputB.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputB.OutputIndex);
	}
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	SubstrateMaterialInfo.PopSubstrateTreeStack();
}

FSubstrateOperator* UMaterialExpressionSubstrateHorizontalMixing::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_HORIZONTAL, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId(), bUseParameterBlending);
	if (SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred())
	{
		return &SubstrateOperator; // Out ot stack space, return now to fail the compilation
	}

	FExpressionInput TracedInputA = Background.GetTracedInput();
	FExpressionInput TracedInputB = Foreground.GetTracedInput();
	UMaterialExpression* ChildAExpression = TracedInputA.Expression;
	UMaterialExpression* ChildBExpression = TracedInputB.Expression;
	FSubstrateOperator* OpA = nullptr;
	FSubstrateOperator* OpB = nullptr;
	if (ChildAExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 0);
		OpA = ChildAExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputA.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.LeftIndex, OpA);
	}
	if (ChildBExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 1);
		OpB = ChildBExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputB.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.RightIndex, OpB);
	}
	CombineFlagForParameterBlending(SubstrateOperator, OpA, OpB);

	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateVerticalLayering::UMaterialExpressionSubstrateVerticalLayering(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseParameterBlending(false)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Ops", "Substrate Operators")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateVerticalLayering::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Top.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Top input"));
	}
	if (!Base.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Base input"));
	}

	Compiler->SubstrateTreeStackPush(this, 0);
	int32 TopCodeChunk = Top.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();
	Compiler->SubstrateTreeStackPush(this, 1);
	int32 BaseCodeChunk = Base.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();
	Compiler->SubstrateTreeStackPush(this, 2);
	int32 ThicknessCodeChunk = Thickness.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();

	int32 OutputCodeChunk = INDEX_NONE;
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		FSubstrateOperator* TopBSDFOperator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* BaseBSDFOperator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!TopBSDFOperator || !BaseBSDFOperator)
		{
			return Compiler->Errorf(TEXT("Missing input on vertical layering node."));
		}
		if (TopCodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("Top input graph could not be evaluated for parameter blending."));
		}
		if (BaseCodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("Base input graph could not be evaluated for parameter blending."));
		}

		const int32 TopNormalMixCodeChunk = Compiler->SubstrateVerticalLayeringParameterBlendingBSDFCoverageToNormalMixCodeChunk(TopCodeChunk);

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		const int32 NewNormalCodeChunk = SubstrateBlendNormal(Compiler, BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, TopBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, TopNormalMixCodeChunk);
		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		int32 NewTangentCodeChunk = INDEX_NONE;
		if (TopBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE && BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = SubstrateBlendNormal(Compiler, BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, TopBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, TopNormalMixCodeChunk);
		}
		else if (TopBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = TopBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
		}
		else if (BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NewNormalCodeChunk, NewTangentCodeChunk);

		OutputCodeChunk = Compiler->SubstrateVerticalLayeringParameterBlending(
			TopCodeChunk, BaseCodeChunk, ThicknessCodeChunk, Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis), 
			TopBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk,
			SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr);

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		OutputCodeChunk = Compiler->SubstrateVerticalLayering(TopCodeChunk, BaseCodeChunk, ThicknessCodeChunk, SubstrateOperator.Index, SubstrateOperator.MaxDistanceFromLeaves);
	}


	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateVerticalLayering::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bUseParameterBlending)
	{
		OutCaptions.Add(TEXT("Substrate Vertical Layer (Parameter Blend)"));
	}
	else
	{
		OutCaptions.Add(TEXT("Substrate Vertical Layer"));
	}
}

FName UMaterialExpressionSubstrateVerticalLayering::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Top");
	}
	else if (InputIndex == 1)
	{
		return TEXT("Bottom");
	}
	else if (InputIndex == 2)
	{
		return  TEXT("Top Thickness");
	}

	return TEXT("Unknown");
}

EMaterialValueType UMaterialExpressionSubstrateVerticalLayering::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateVerticalLayering::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 2)
	{
		return MCT_Float;
	}
	return MCT_Substrate;
}

bool UMaterialExpressionSubstrateVerticalLayering::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateVerticalLayering::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (!SubstrateMaterialInfo.PushSubstrateTreeStack())
	{
		return;
	}

	FExpressionInput TracedInputTop = Top.GetTracedInput();
	FExpressionInput TracedInputBase = Base.GetTracedInput();
	if (TracedInputTop.Expression)
	{
		TracedInputTop.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputTop.OutputIndex);
	}
	if (TracedInputBase.Expression)
	{
		TracedInputBase.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputBase.OutputIndex);
	}
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	SubstrateMaterialInfo.PopSubstrateTreeStack();
}

FSubstrateOperator* UMaterialExpressionSubstrateVerticalLayering::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_VERTICAL, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId(), bUseParameterBlending);
	if (SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred())
	{
		return &SubstrateOperator; // Out ot stack space, return now to fail the compilation
	}

	FExpressionInput TracedInputTop = Top.GetTracedInput();
	FExpressionInput TracedInputBase = Base.GetTracedInput();
	UMaterialExpression* ChildAExpression = TracedInputTop.Expression;
	UMaterialExpression* ChildBExpression = TracedInputBase.Expression;
	FSubstrateOperator* OpA = nullptr;
	FSubstrateOperator* OpB = nullptr;

	// Top - Use the vertical operator thickness
	if (ChildAExpression)
	{
		SubstrateTranslatorData.SubstrateThicknessStackPush(this, &Thickness);
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 0);
		OpA = ChildAExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputTop.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		SubstrateTranslatorData.SubstrateThicknessStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.LeftIndex, OpA);
	}
	// Bottom - Use the propagated thickness from parent
	if (ChildBExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 1);
		OpB = ChildBExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputBase.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.RightIndex, OpB);
	}
	CombineFlagForParameterBlending(SubstrateOperator, OpA, OpB);

	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateAdd::UMaterialExpressionSubstrateAdd(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseParameterBlending(false)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Ops", "Substrate Operators")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateAdd::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing A input"));
	}
	if (!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing B input"));
	}

	Compiler->SubstrateTreeStackPush(this, 0);
	int32 ACodeChunk = A.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();
	Compiler->SubstrateTreeStackPush(this, 1);
	int32 BCodeChunk = B.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();

	int32 OutputCodeChunk = INDEX_NONE;
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		FSubstrateOperator* ABSDFOperator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* BBSDFOperator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!ABSDFOperator || !BBSDFOperator)
		{
			return Compiler->Errorf(TEXT("Missing input on add node."));
		}
		if (ACodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("A input graph could not be evaluated for parameter blending."));
		}
		if (BCodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("B input graph could not be evaluated for parameter blending."));
		}

		const int32 ANormalMixCodeChunk = Compiler->SubstrateAddParameterBlendingBSDFCoverageToNormalMixCodeChunk(ACodeChunk, BCodeChunk);

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		const int32 NewNormalCodeChunk = SubstrateBlendNormal(Compiler, BBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, ABSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, ANormalMixCodeChunk);
		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		int32 NewTangentCodeChunk = INDEX_NONE;
		if (ABSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE && BBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = SubstrateBlendNormal(Compiler, BBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, ABSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, ANormalMixCodeChunk);
		}
		else if (ABSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = ABSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
		}
		else if (BBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
		{
			NewTangentCodeChunk = BBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NewNormalCodeChunk, NewTangentCodeChunk);

		OutputCodeChunk = Compiler->SubstrateAddParameterBlending(
			ACodeChunk, BCodeChunk, ANormalMixCodeChunk, Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis),
			ABSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, BBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunk,
			SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr);

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		OutputCodeChunk = Compiler->SubstrateAdd(ACodeChunk, BCodeChunk, SubstrateOperator.Index, SubstrateOperator.MaxDistanceFromLeaves);
	}

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateAdd::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bUseParameterBlending)
	{
		OutCaptions.Add(TEXT("Substrate Add (Parameter Blend)"));
	}
	else
	{
		OutCaptions.Add(TEXT("Substrate Add"));
	}
}

EMaterialValueType UMaterialExpressionSubstrateAdd::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateAdd::GetInputValueType(int32 InputIndex)
{
	return MCT_Substrate;
}

bool UMaterialExpressionSubstrateAdd::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateAdd::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (!SubstrateMaterialInfo.PushSubstrateTreeStack())
	{
		return;
	}

	FExpressionInput TracedInputA = A.GetTracedInput();
	FExpressionInput TracedInputB = B.GetTracedInput();
	if (TracedInputA.Expression)
	{
		TracedInputA.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputA.OutputIndex);
	}
	if (TracedInputB.Expression)
	{
		TracedInputB.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputB.OutputIndex);
	}
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	SubstrateMaterialInfo.PopSubstrateTreeStack();
}

FSubstrateOperator* UMaterialExpressionSubstrateAdd::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_ADD, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId(), bUseParameterBlending);
	if (SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred())
	{
		return &SubstrateOperator; // Out ot stack space, return now to fail the compilation
	}

	FExpressionInput TracedInputA = A.GetTracedInput();
	FExpressionInput TracedInputB = B.GetTracedInput();
	UMaterialExpression* ChildAExpression = TracedInputA.Expression;
	UMaterialExpression* ChildBExpression = TracedInputB.Expression;
	FSubstrateOperator* OpA = nullptr;
	FSubstrateOperator* OpB = nullptr;
	if (ChildAExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 0);
		OpA = ChildAExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputA.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.LeftIndex, OpA);
	}
	if (ChildBExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 1);
		OpB = ChildBExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputB.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.RightIndex, OpB);
	}
	CombineFlagForParameterBlending(SubstrateOperator, OpA, OpB);

	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateWeight::UMaterialExpressionSubstrateWeight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Ops", "Substrate Operators")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateWeight::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing A input"));
	}

	Compiler->SubstrateTreeStackPush(this, 0);
	int32 ACodeChunk = A.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();
	int32 WeightCodeChunk = Weight.GetTracedInput().Expression ? Weight.Compile(Compiler) : Compiler->Constant(1.0f);

	int32 OutputCodeChunk = INDEX_NONE;
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		// Propagate the parameter blended normal
		FSubstrateOperator* Operator = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		if (!Operator)
		{
			return Compiler->Errorf(TEXT("Missing input on weight node."));
		}
		if (ACodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("A input graph could not be evaluated for parameter blending."));
		}
		if (WeightCodeChunk == INDEX_NONE)
		{
			return Compiler->Errorf(TEXT("Weight input graph could not be evaluated for parameter blending."));
		}

		OutputCodeChunk = Compiler->SubstrateWeightParameterBlending(
			ACodeChunk, WeightCodeChunk,
			SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr);

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = Operator->BSDFRegisteredSharedLocalBasis;
	}
	else
	{
		OutputCodeChunk = Compiler->SubstrateWeight(ACodeChunk, WeightCodeChunk, SubstrateOperator.Index, SubstrateOperator.MaxDistanceFromLeaves);
	}

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateWeight::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Coverage Weight"));
}

EMaterialValueType UMaterialExpressionSubstrateWeight::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateWeight::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_Substrate;
	}
	return MCT_Float1;
}

bool UMaterialExpressionSubstrateWeight::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateWeight::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (!SubstrateMaterialInfo.PushSubstrateTreeStack())
	{
		return;
	}

	FExpressionInput TracedInputA = A.GetTracedInput();
	if (TracedInputA.Expression)
	{
		TracedInputA.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputA.OutputIndex);
	}
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	SubstrateMaterialInfo.PopSubstrateTreeStack();
}

FSubstrateOperator* UMaterialExpressionSubstrateWeight::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_WEIGHT, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
	if (SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred())
	{
		return &SubstrateOperator; // Out ot stack space, return now to fail the compilation
	}

	FExpressionInput TracedInputA = A.GetTracedInput();
	UMaterialExpression* ChildAExpression = TracedInputA.Expression;
	FSubstrateOperator* OpA = nullptr;
	if (ChildAExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 0);
		OpA = ChildAExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputA.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.LeftIndex, OpA);
	}
	CombineFlagForParameterBlending(SubstrateOperator, OpA);

	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateSelect::UMaterialExpressionSubstrateSelect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Threshold(0.5f)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Ops", "Substrate Operators")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateSelect::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing A input"));
	}
	if (!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing B input"));
	}

	Compiler->SubstrateTreeStackPush(this, 0);
	int32 ACodeChunk = A.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();
	Compiler->SubstrateTreeStackPush(this, 1);
	int32 BCodeChunk = B.Compile(Compiler);
	Compiler->SubstrateTreeStackPop();

	// if SelectValue is not pluggedin, need to be 0. Otherwise it must be a float value.
	int32 SelectValueCodeChunk = CompileWithDefaultFloat1(Compiler, SelectValue, 0.0f);
	SelectValueCodeChunk = Compiler->ValidCast(SelectValueCodeChunk, MCT_Float1);

	int32 ZeroCodeChunk = Compiler->Constant(0.0f);
	int32 OneCodeChunk = Compiler->Constant(1.0f);
	int32 ThresholdCodeChunk = Compiler->Constant(Threshold);
	SelectValueCodeChunk = Compiler->If(SelectValueCodeChunk, ThresholdCodeChunk, OneCodeChunk, ZeroCodeChunk, ZeroCodeChunk, ZeroCodeChunk);
	// Now, SelectValueCodeChunk is 0 or 1 for any threshold.

	auto SubstrateSelectNormal = [&](int32 NormalA, int32 NormalB)
	{
		return Compiler->If(SelectValueCodeChunk, ZeroCodeChunk, NormalB, NormalA, NormalA, ZeroCodeChunk);
	};

	// Compute the new Normal and Tangent resulting from the selection using code chunk
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	FSubstrateOperator* OperatorA = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
	FSubstrateOperator* OperatorB = Compiler->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);

	if (!OperatorA || !OperatorB)
	{
		const FString ErroneousInput = (!OperatorA && !OperatorB) ? TEXT("A and B") : (!OperatorA ? TEXT("A") : TEXT("B"));
		return Compiler->Errorf(TEXT("Substrate Select node has invalid input %s."), *ErroneousInput);
	}
	if (OperatorA->BSDFRegisteredSharedLocalBasis.NormalCodeChunk == INDEX_NONE || OperatorB->BSDFRegisteredSharedLocalBasis.NormalCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Substrate Select node requires both inputs to have a valid local basis. Unlit BSDF does not register a local basis, so it cannot be used with the Select node."));
	}

	const int32 NewNormalCodeChunk = SubstrateSelectNormal( OperatorA->BSDFRegisteredSharedLocalBasis.NormalCodeChunk, OperatorB->BSDFRegisteredSharedLocalBasis.NormalCodeChunk);
	// The tangent is optional so we treat it differently if INDEX_NONE is specified
	int32 NewTangentCodeChunk = INDEX_NONE;
	if (OperatorA->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE && OperatorB->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
	{
		NewTangentCodeChunk = SubstrateSelectNormal(OperatorB->BSDFRegisteredSharedLocalBasis.TangentCodeChunk, OperatorA->BSDFRegisteredSharedLocalBasis.TangentCodeChunk);
	}
	else if (OperatorA->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
	{
		NewTangentCodeChunk = OperatorA->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
	}
	else if (OperatorB->BSDFRegisteredSharedLocalBasis.TangentCodeChunk != INDEX_NONE)
	{
		NewTangentCodeChunk = OperatorB->BSDFRegisteredSharedLocalBasis.TangentCodeChunk;
	}
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NewNormalCodeChunk, NewTangentCodeChunk);

	int32 OutputCodeChunk = INDEX_NONE;
	if (SubstrateOperator.bUseParameterBlending)
	{
		OutputCodeChunk = Compiler->SubstrateSelectParameterBlending(ACodeChunk, BCodeChunk, SelectValueCodeChunk,
			Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis), 
			SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr);
	}
	else
	{
		Compiler->Errorf(TEXT("The Select node can only use parameter blending to only select between one of two BSDF."));
	}

	// Propagate the parameter blended normal
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	return OutputCodeChunk;
}

void UMaterialExpressionSubstrateSelect::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bUseParameterBlending)
	{
		OutCaptions.Add(TEXT("Substrate Select (Parameter Blend)"));
	}
	else
	{
		OutCaptions.Add(TEXT("Substrate Select"));
	}
}

EMaterialValueType UMaterialExpressionSubstrateSelect::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Substrate;
}

EMaterialValueType UMaterialExpressionSubstrateSelect::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 2)
	{
		return MCT_Float1;
	}
	return MCT_Substrate;
}

bool UMaterialExpressionSubstrateSelect::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return true;
}

void UMaterialExpressionSubstrateSelect::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (!SubstrateMaterialInfo.PushSubstrateTreeStack())
	{
		return;
	}

	FExpressionInput TracedInputA = A.GetTracedInput();
	FExpressionInput TracedInputB = B.GetTracedInput();
	if (TracedInputA.Expression)
	{
		TracedInputA.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputA.OutputIndex);
	}
	if (TracedInputB.Expression)
	{
		TracedInputB.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, TracedInputB.OutputIndex);
	}
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);

	SubstrateMaterialInfo.PopSubstrateTreeStack();
}

FSubstrateOperator* UMaterialExpressionSubstrateSelect::SubstrateGenerateMaterialTopologyTree(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_SELECT, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), this->MaterialExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId(), bUseParameterBlending);
	if (SubstrateTranslatorData.GetSubstrateTreeOutOfStackDepthOccurred())
	{
		return &SubstrateOperator; // Out ot stack space, return now to fail the compilation
	}

	FExpressionInput TracedInputA = A.GetTracedInput();
	FExpressionInput TracedInputB = B.GetTracedInput();
	UMaterialExpression* ChildAExpression = TracedInputA.Expression;
	UMaterialExpression* ChildBExpression = TracedInputB.Expression;
	FSubstrateOperator* OpA = nullptr;
	FSubstrateOperator* OpB = nullptr;
	if (ChildAExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 0);
		OpA = ChildAExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputA.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.LeftIndex, OpA);
	}
	if (ChildBExpression)
	{
		SubstrateTranslatorData.SubstrateTreeStackPush(this, 1);
		OpB = ChildBExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, this, TracedInputB.OutputIndex);
		SubstrateTranslatorData.SubstrateTreeStackPop();
		AssignOperatorIndexIfNotNull(SubstrateOperator.RightIndex, OpB);
	}

	// Since A or B can be used, we need to combine all their flag to support the most expenssive use case selected.
	CombineFlagForParameterBlending(SubstrateOperator, OpA, OpB);

	return &SubstrateOperator;
}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateUtilityBase::UMaterialExpressionSubstrateUtilityBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UMaterialExpressionSubstrateTransmittanceToMFP::UMaterialExpressionSubstrateTransmittanceToMFP(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Helpers", "Substrate Helpers")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);

	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("MFP")));
	Outputs.Add(FExpressionOutput(TEXT("Thickness")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateTransmittanceToMFP::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 TransmittanceColorCodeChunk = TransmittanceColor.GetTracedInput().Expression ? TransmittanceColor.Compile(Compiler) : Compiler->Constant(0.5f);
	int32 ThicknessCodeChunk = Thickness.GetTracedInput().Expression ? Thickness.Compile(Compiler) : Compiler->Constant(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
	if (TransmittanceColorCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("TransmittanceColor input graph could not be evaluated for TransmittanceToMFP."));
	}
	if (ThicknessCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("ThicknessCodeChunk input graph could not be evaluated for TransmittanceToMFP."));
	}
	return Compiler->SubstrateTransmittanceToMFP(
		TransmittanceColorCodeChunk,
		ThicknessCodeChunk,
		OutputIndex);
}

void UMaterialExpressionSubstrateTransmittanceToMFP::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Transmittance-To-MeanFreePath"));
}

EMaterialValueType UMaterialExpressionSubstrateTransmittanceToMFP::GetOutputValueType(int32 OutputIndex)
{
	switch (OutputIndex)
	{
	case 0:
		return MCT_Float3; // MFP
	case 1:
		return MCT_Float1; // Thickness
	}

	check(false);
	return MCT_Float1;
}

EMaterialValueType UMaterialExpressionSubstrateTransmittanceToMFP::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3; // Transmittance
	case 1:
		return MCT_Float1; // Thickness
	}

	check(false);
	return MCT_Float1;
}
void UMaterialExpressionSubstrateTransmittanceToMFP::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (InputIndex != INDEX_NONE)
	{
		switch (InputIndex)
		{
		case 0:
			ConvertToMultilineToolTip(FText(LOCTEXT("TransMFPNodeI0", "The colored transmittance for a view perpendicular to the surface. The transmittance for other view orientations will automatically be deduced according to surface thickness.")).ToString(), 80, OutToolTip);
			break;
		case 1:
			ConvertToMultilineToolTip(FText(LOCTEXT("TransMFPNodeI1", "The thickness (in centimeter) at which the desired colored transmittance is reached. Default thickness: 0.01cm. Another use case example: this node output called thickness can be modulated before it is plugged in a slab node.this can be used to achieve simple scattering/transmittance variation of the same material.")).ToString(), 80, OutToolTip);
			break;
		}
	}
	else if (OutputIndex != INDEX_NONE)
	{
		switch (OutputIndex)
		{
		case 0:
			ConvertToMultilineToolTip(FText(LOCTEXT("TransMFPNodeO0", "The Mean Free Path defining the participating media constituting the slab of material (unit = centimeters).")).ToString(), 80, OutToolTip);
			break;
		case 1:
			ConvertToMultilineToolTip(FText(LOCTEXT("TransMFPNodeO1", "The thickness of the slab of material (unit = centimeters).")).ToString(), 80, OutToolTip);
			break;
		}
	}
}

void UMaterialExpressionSubstrateTransmittanceToMFP::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	OutToolTip.Add(FText(LOCTEXT("TransMFPNodeTT0", "Convert a transmittance color corresponding to a slab of participating media viewed perpendicularly to its surface.")).ToString());
	OutToolTip.Add(FText(LOCTEXT("TransMFPNodeTT1", "This node directly maps to the Slab MFP input. It is recommended to use it when specifying the colored transmittance of a top layer slab.")).ToString());
	OutToolTip.Add(FText(LOCTEXT("TransMFPNodeTT2", "For Subsurface scattering, you might prefer to specify the MFP(light mean free path) as world space centimeter directly.")).ToString());
}
#endif // WITH_EDITOR

UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Helpers", "Substrate Helpers")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);

	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("DiffuseAlbedo")));
	Outputs.Add(FExpressionOutput(TEXT("F0")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 BaseColorCodeChunk = BaseColor.GetTracedInput().Expression ? BaseColor.Compile(Compiler) : Compiler->Constant(0.18f);
	int32 SpecularCodeChunk = Specular.GetTracedInput().Expression ? Specular.Compile(Compiler) : Compiler->Constant(0.5f);
	int32 MetallicCodeChunk = Metallic.GetTracedInput().Expression ? Metallic.Compile(Compiler) : Compiler->Constant(0.0f);
	if (BaseColorCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("BaseColor input graph could not be evaluated for MetalnessToDiffuseAlbedoF0."));
	}
	if (SpecularCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Specular input graph could not be evaluated for MetalnessToDiffuseAlbedoF0."));
	}
	if (MetallicCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Metallic input graph could not be evaluated for MetalnessToDiffuseAlbedoF0."));
	}
	return Compiler->SubstrateMetalnessToDiffuseAlbedoF0(
		BaseColorCodeChunk,
		SpecularCodeChunk,
		MetallicCodeChunk,
		OutputIndex);
}

void UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Metalness-To-DiffuseAlbedo-F0"));
}

EMaterialValueType UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::GetOutputValueType(int32 OutputIndex)
{
	switch (OutputIndex)
	{
	case 0:
		return MCT_Float3; // Diffuse Albedo
	case 1:
		return MCT_Float3; // F0
	}

	check(false);
	return MCT_Float1;
}

EMaterialValueType UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0) { return MCT_Float3; }
	if (InputIndex == 1) { return MCT_Float1; }
	if (InputIndex == 2) { return MCT_Float1; }
	return MCT_Float1;
}

void UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	switch (OutputIndex)
	{
	case 1: ConvertToMultilineToolTip(FText(LOCTEXT("MetalnessNodeO0", "Defines the overall color of the Material. (type = float3, unit = unitless, defaults to 0.18)")).ToString(), 80, OutToolTip); break;
	case 2: ConvertToMultilineToolTip(FText(LOCTEXT("MetalnessNodeO1", "Controls how \"metal-like\" your surface looks like. 0 means dielectric, 1 means conductor (type = float, unit = unitless, defaults to 0)")).ToString(), 80, OutToolTip); break;
	case 3: ConvertToMultilineToolTip(FText(LOCTEXT("MetalnessNodeO2", "Used to scale the current amount of specularity on non-metallic surfaces and is a value between 0 and 1 (type = float, unit = unitless, defaults to plastic 0.5)")).ToString(), 80, OutToolTip); break;
	}
}

void UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(FText(LOCTEXT("MetalnessNodeTT0", "Convert a metalness parameterization (BaseColor/Specular/Metallic) into DiffuseAlbedo/F0 parameterization.")).ToString(), 80, OutToolTip);

}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateHazinessToSecondaryRoughness::UMaterialExpressionSubstrateHazinessToSecondaryRoughness(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Helpers", "Substrate Helpers")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);

	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Second Roughness")));
	Outputs.Add(FExpressionOutput(TEXT("Second Roughness Weight")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateHazinessToSecondaryRoughness::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 BaseRoughnessCodeChunk = BaseRoughness.GetTracedInput().Expression ? BaseRoughness.Compile(Compiler) : Compiler->Constant(0.1f);
	int32 HazinessCodeChunk = Haziness.GetTracedInput().Expression ? Haziness.Compile(Compiler) : Compiler->Constant(0.5f);
	if (BaseRoughnessCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("BaseRoughness input graph could not be evaluated for HazinessToSecondaryRoughness."));
	}
	if (HazinessCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Haziness input graph could not be evaluated for HazinessToSecondaryRoughness."));
	}
	return Compiler->SubstrateHazinessToSecondaryRoughness(
		BaseRoughnessCodeChunk,
		HazinessCodeChunk,
		OutputIndex);
}

void UMaterialExpressionSubstrateHazinessToSecondaryRoughness::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Haziness-To-Secondary-Roughness"));
}

EMaterialValueType UMaterialExpressionSubstrateHazinessToSecondaryRoughness::GetOutputValueType(int32 OutputIndex)
{
	switch (OutputIndex)
	{
	case 0:
		return MCT_Float1; // Second Roughness
	case 1:
		return MCT_Float1; // Second Roughness Weight
	}

	check(false);
	return MCT_Float1;
}

EMaterialValueType UMaterialExpressionSubstrateHazinessToSecondaryRoughness::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float1; // BaseRoughness
	case 1:
		return MCT_Float1; // Haziness
	}

	check(false);
	return MCT_Float1;
}
void UMaterialExpressionSubstrateHazinessToSecondaryRoughness::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (InputIndex != INDEX_NONE)
	{
		switch (InputIndex)
		{
		case 0:
			ConvertToMultilineToolTip(FText(LOCTEXT("HazinessNodeI0", "The base roughness of the surface. It represented the smoothest part of the reflection.")).ToString(), 80, OutToolTip);
			break;
		case 1:
			ConvertToMultilineToolTip(FText(LOCTEXT("HazinessNodeI1", "Haziness represent the amount of irregularity of the surface. A high value will lead to a second rough specular lobe causing the surface too look `milky`.")).ToString(), 80, OutToolTip);
			break;
		}
	}
	else if (OutputIndex != INDEX_NONE)
	{
		switch (OutputIndex)
		{
		case 0:
			ConvertToMultilineToolTip(FText(LOCTEXT("HazinessNodeO0", "The roughness of the second lobe.")).ToString(), 80, OutToolTip);
			break;
		case 1:
			ConvertToMultilineToolTip(FText(LOCTEXT("HazinessNodeO1", "The weight of the secondary specular lobe, while the primary specular lobe will have a weight of (1 - SecondRoughnessWeight).")).ToString(), 80, OutToolTip);
			break;
		}
	}
}

void UMaterialExpressionSubstrateHazinessToSecondaryRoughness::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(FText(LOCTEXT("HazinessNodeTT0", "Compute a second specular lobe roughness from a base surface roughness and haziness. This parameterisation ensure that the haziness makes physically and is perceptually easy to author.")).ToString(), 80, OutToolTip);

}
#endif // WITH_EDITOR



UMaterialExpressionSubstrateThinFilm::UMaterialExpressionSubstrateThinFilm(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Substrate;
		FConstructorStatics() : NAME_Substrate(LOCTEXT("Substrate Helpers", "Substrate Helpers")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Substrate);
#endif

	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Specular Color")));
	Outputs.Add(FExpressionOutput(TEXT("Edge Specular Color")));
}

#if WITH_EDITOR
int32 UMaterialExpressionSubstrateThinFilm::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, Normal);

	int32 F0CodeChunk = F0.GetTracedInput().Expression ? F0.Compile(Compiler) : Compiler->Constant3(0.04f, 0.04f, 0.04f);
	int32 F90CodeChunk = F90.GetTracedInput().Expression ? F90.Compile(Compiler) : Compiler->Constant3(1.0f, 1.0f, 1.0f);

	int32 ThicknessCodeChunk = Thickness.GetTracedInput().Expression ? Thickness.Compile(Compiler) : Compiler->Constant(1.0f);
	int32 IORCodeChunk = IOR.GetTracedInput().Expression ? IOR.Compile(Compiler) : Compiler->Constant(1.44f);

	if (NormalCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("NormalCode input graph could not be evaluated for ThinFilm."));
	}
	if (F0CodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("F0 input graph could not be evaluated for ThinFilm."));
	}
	if (F90CodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("F90 input graph could not be evaluated for ThinFilm."));
	}
	if (ThicknessCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Thickness input graph could not be evaluated for ThinFilm."));
	}
	if (IORCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("IOR input graph could not be evaluated for ThinFilm."));
	}

	return Compiler->SubstrateThinFilm(NormalCodeChunk, F0CodeChunk, F90CodeChunk, ThicknessCodeChunk, IORCodeChunk, OutputIndex);
}

void UMaterialExpressionSubstrateThinFilm::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Thin-Film"));
}

EMaterialValueType UMaterialExpressionSubstrateThinFilm::GetOutputValueType(int32 OutputIndex)
{
	switch (OutputIndex)
	{
	case 0:
		return MCT_Float3; // F0
	case 1:
		return MCT_Float3; // F90
	}

	check(false);
	return MCT_Float1;
}

EMaterialValueType UMaterialExpressionSubstrateThinFilm::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0) { return MCT_Float3; } // Normal
	if (InputIndex == 1) { return MCT_Float3; } // F0
	if (InputIndex == 2) { return MCT_Float3; } // F90
	if (InputIndex == 3) { return MCT_Float1; } // Thickness
	if (InputIndex == 4) { return MCT_Float1; } // IOR

	check(false);
	return MCT_Float1;
}
void UMaterialExpressionSubstrateThinFilm::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (OutputIndex != INDEX_NONE)
	{
		switch (OutputIndex)
		{
		case 0:
			ConvertToMultilineToolTip(FText(LOCTEXT("ThinFilmNodeO0", "F0 accounting for thin film interferences. This is percentage of light reflected as specular from a surface when the view is perpendicular to the surface. (type = float3, unit = unitless, defaults to plastic 0.04)")).ToString(), 80, OutToolTip);
			break;
		case 1:
			ConvertToMultilineToolTip(FText(LOCTEXT("ThinFilmNodeO1", "F90 accounting for thin film interferences. the percentage of light reflected as specular from a surface when the view is tangent to the surface. (type = float3, unit = unitless, defaults to 1.0f).")).ToString(), 80, OutToolTip);
			break;
		}
		return;
	}

	// Else use the default input tooltip
	Super::GetConnectorToolTip(InputIndex, OutputIndex, OutToolTip);
}

void UMaterialExpressionSubstrateThinFilm::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(FText(LOCTEXT("ThinFilmNodeTT0", "Compute the resulting material specular parameter F0 and F90 according to input surface properties as well as the thin film parameters.")).ToString(), 80, OutToolTip);
}
#endif // WITH_EDITOR

// Return a conservative list of connected material attribute inputs
#if WITH_EDITOR
static uint64 GetConnectedMaterialAttributesInputs(const UMaterial* InMaterial)
{
	if (!InMaterial) return 0;
	return FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(InMaterial->GetExpressions());
}
#endif

UMaterialExpressionSubstrateConvertMaterialAttributes::UMaterialExpressionSubstrateConvertMaterialAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_Strata;
		FConstructorStatics() : NAME_Strata(LOCTEXT("Substrate Conversion", "Substrate Conversion")) { }
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Strata);
#endif

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""))); // Substrate
	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
FExpressionInput* UMaterialExpressionSubstrateConvertMaterialAttributes::GetInput(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0: return &MaterialAttributes;
	case 1: return &WaterScatteringCoefficients;
	case 2: return &WaterAbsorptionCoefficients;
	case 3: return &WaterPhaseG;
	case 4: return &ColorScaleBehindWater;
	default: return nullptr;
	}
}

int32 UMaterialExpressionSubstrateConvertMaterialAttributes::CompileCommon(class FMaterialCompiler* Compiler, int32 OutputIndex,
	const uint64 CachedConnectedMaterialAttributesInputs, FMaterialAttributesInput& MaterialAttributes, enum EMaterialShadingModel ShadingModelOverride,
	FExpressionInput& WaterScatteringCoefficients, FExpressionInput& WaterAbsorptionCoefficients, FExpressionInput& WaterPhaseG, FExpressionInput& ColorScaleBehindWater,
	const bool bHasSSS, USubsurfaceProfile* SSSProfile,
	FExpressionInput& ClearCoatNormal, FExpressionInput& CustomTangent)
{
	static const FGuid ClearCoatBottomNormalGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("ClearCoatBottomNormal"));
	static const FGuid CustomEyeTangentGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("CustomEyeTangent"));
	static const FGuid TransmittanceColorGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("TransmittanceColor"));
	static const FGuid ThinTranslucentSurfaceCoverageGuid = FMaterialAttributeDefinitionMap::GetCustomAttributeID(TEXT("ThinTranslucentSurfaceCoverage"));

	if (OutputIndex != 0)
	{
		return Compiler->Error(TEXT("Output pin index error"));
	}

	// We also cannot ignore the tangent when using the default Tangent because GetTangentBasis
	// used in SubstrateGetBSDFSharedBasis cannot be relied on for smooth tangent used for lighting on any mesh.

	const bool bHasAnisotropy = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, MP_Anisotropy);

	// Regular normal basis
	int32 NormalCodeChunk = MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Normal));
	NormalCodeChunk = Compiler->ForceCast(NormalCodeChunk, MCT_Float3, MFCF_ExactMatch | MFCF_ReplicateValue);

	// When computing NormalCodeChunk, we invoke TransformNormalFromRequestedBasisToWorld which requires input to be float or float3.
	// Certain material do not respect this requirement. We handle here a simple recovery when source material doesn't have a valid 
	// normal (e.g., vec2 normal), and avoid crashing the material compilation. The error will still be reported by the compiler up 
	// to the user, but the compilation will succeed.
	// This is done before TransformNormalFromRequestedBasisToWorld because TransformNormalFromRequestedBasisToWorld(INDEX_NONE) would crash inside GetParameterCode.
	if (NormalCodeChunk == INDEX_NONE) 
	{
		NormalCodeChunk = Compiler->VertexNormal();
	}
	NormalCodeChunk = Compiler->TransformNormalFromRequestedBasisToWorld(NormalCodeChunk);

	int32 TangentCodeChunk = INDEX_NONE;
	if (bHasAnisotropy)
	{
		TangentCodeChunk = MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Tangent));
		TangentCodeChunk = Compiler->ForceCast(TangentCodeChunk, MCT_Float3, MFCF_ExactMatch | MFCF_ReplicateValue);
		if (TangentCodeChunk == INDEX_NONE)
		{
			TangentCodeChunk = Compiler->VertexTangent(); // Similar fix as for NormalCodeChunk above.
		}
		TangentCodeChunk = Compiler->TransformNormalFromRequestedBasisToWorld(TangentCodeChunk);
	}
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk, TangentCodeChunk);
	const FString BasisIndexMacro = Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis);

	const bool bHasCoatNormal = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, ClearCoatBottomNormalGuid);
	// Clear coat normal basis
	int32 ClearCoat_NormalCodeChunk = INDEX_NONE;
	int32 ClearCoat_TangentCodeChunk = INDEX_NONE;
	FString ClearCoat_BasisIndexMacro;
	FSubstrateRegisteredSharedLocalBasis ClearCoat_NewRegisteredSharedLocalBasis;
	if (bHasCoatNormal)
	{
		ClearCoat_NormalCodeChunk = MaterialAttributes.CompileWithDefault(Compiler, ClearCoatBottomNormalGuid);
		ClearCoat_TangentCodeChunk = TangentCodeChunk;
		ClearCoat_NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, ClearCoat_NormalCodeChunk, ClearCoat_TangentCodeChunk);
		ClearCoat_BasisIndexMacro = Compiler->GetSubstrateSharedLocalBasisIndexMacro(ClearCoat_NewRegisteredSharedLocalBasis);
	}
	else if (ClearCoatNormal.IsConnected())
	{
		ClearCoat_NormalCodeChunk = CompileWithDefaultNormalWS(Compiler, ClearCoatNormal);
		ClearCoat_TangentCodeChunk = TangentCodeChunk;
		ClearCoat_NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, ClearCoat_NormalCodeChunk, ClearCoat_TangentCodeChunk);
		ClearCoat_BasisIndexMacro = Compiler->GetSubstrateSharedLocalBasisIndexMacro(ClearCoat_NewRegisteredSharedLocalBasis);
	}
	else
	{
		ClearCoat_NormalCodeChunk = NormalCodeChunk;
		ClearCoat_TangentCodeChunk = TangentCodeChunk;
		ClearCoat_NewRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
		ClearCoat_BasisIndexMacro = BasisIndexMacro;
	}

	// Custom tangent. No need to register it as a local basis, as it is only used for eye shading internal conversion
	int32 CustomTangent_TangentCodeChunk = INDEX_NONE;
	const bool bHasCustomTangent = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, CustomEyeTangentGuid);
	if (bHasCustomTangent)
	{
		// Legacy code doesn't do tangent <-> world basis conversion on tangent output, when provided.
		CustomTangent_TangentCodeChunk = MaterialAttributes.CompileWithDefault(Compiler, CustomEyeTangentGuid); // CompileWithDefaultNormalWS(Compiler, CustomTangent, false /*bConvertToRequestedSpace*/);
		if (CustomTangent_TangentCodeChunk == INDEX_NONE)
		{
			// Nothing is plug in from the linked input, so specify world space normal the BSDF node expects.
			CustomTangent_TangentCodeChunk = Compiler->VertexNormal();
		}
	}
	else if (CustomTangent.IsConnected())
	{
		// Legacy code doesn't do tangent <-> world basis conversion on tangent output, when provided.
		CustomTangent_TangentCodeChunk = CompileWithDefaultNormalWS(Compiler, CustomTangent, false /*bConvertToRequestedSpace*/);
	}
	else
	{
		CustomTangent_TangentCodeChunk = NormalCodeChunk;
	}

	// SSS profile
	// Need to handle this by looking at the material instead of the node?
	int32 SSSProfileCodeChunk = INDEX_NONE;
	if (bHasSSS && SSSProfile)
	{
		SSSProfileCodeChunk = CreateSubsurfaceProfileParameter(Compiler, SSSProfile);
	}
	else
	{
		SSSProfileCodeChunk = CreateDefaultSubsurfaceProfileParameter(Compiler);
	}
	SSSProfileCodeChunk = SSSProfileCodeChunk != INDEX_NONE ? SSSProfileCodeChunk : Compiler->Constant(0.0f);

	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(Compiler->SubstrateTreeStackGetPathUniqueId());
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	// Opacity
	int32 OpacityCodeChunk = INDEX_NONE;
	if (!Compiler->SubstrateSkipsOpacityEvaluation())
	{
		// We evaluate opacity only for shading models and blending mode requiring it.
		// For instance, a translucent shader reading depth for soft fading should no evaluate opacity when an instance forces an opaque mode.
		OpacityCodeChunk = MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Opacity));
	}
	else
	{
		OpacityCodeChunk = Compiler->Constant(1.0f);
	}

	// Transmittance Color
	const bool bHasTransmittanceColor = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, TransmittanceColorGuid);
	int32 TransmittanceColorChunk = INDEX_NONE;
	if (bHasTransmittanceColor)
	{
		TransmittanceColorChunk = MaterialAttributes.CompileWithDefault(Compiler, TransmittanceColorGuid);
	}
	else
	{
		TransmittanceColorChunk = Compiler->Constant3(0.5f, 0.5f, 0.5f);
	}
	// Thin Translucent Surface Coverage
	const bool bHasThinTranslucentSurfaceCoverage = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, ThinTranslucentSurfaceCoverageGuid);
	int32 ThinTranslucentSurfaceCoverageChunk = INDEX_NONE;
	if (bHasThinTranslucentSurfaceCoverage)
	{
		ThinTranslucentSurfaceCoverageChunk = MaterialAttributes.CompileWithDefault(Compiler, ThinTranslucentSurfaceCoverageGuid);
	}
	else
	{
		ThinTranslucentSurfaceCoverageChunk = Compiler->Constant(1.0f);
	}

	int32 ShadingModelCodeChunk = MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_ShadingModel));
	const bool bHasShadingModelExpression = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, MP_ShadingModel)
		|| ShadingModelOverride == MSM_FromMaterialExpression; // In this case, rely on the default compilation to return DefaultLit.
	if (!bHasShadingModelExpression)
	{
		ShadingModelCodeChunk = Compiler->Constant(float(ShadingModelOverride));
	}
	int32 ShadingModelCount = Compiler->GetMaterialShadingModels().CountShadingModels();
	const bool bHasDynamicShadingModels = ShadingModelCount > 1;
	int32 OutputCodeChunk = Compiler->SubstrateConversionFromLegacy(
		bHasDynamicShadingModels,
		// Metalness workflow
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_BaseColor)),
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Specular)),
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Metallic)),
		// Roughness
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Roughness)),
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_Anisotropy)),
		// SSS
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_SubsurfaceColor)),
		SSSProfileCodeChunk,
		// Clear Coat / Custom
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_CustomData0)),// Clear coat
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_CustomData1)),// Clear coat roughness
		// Misc
		MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(MP_EmissiveColor)),
		OpacityCodeChunk,
		TransmittanceColorChunk,
		ThinTranslucentSurfaceCoverageChunk,
		// Water
		CompileWithDefaultFloat3(Compiler, WaterScatteringCoefficients, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat3(Compiler, WaterAbsorptionCoefficients, 0.0f, 0.0f, 0.0f),
		CompileWithDefaultFloat1(Compiler, WaterPhaseG, 0.0f),
		CompileWithDefaultFloat3(Compiler, ColorScaleBehindWater, 1.0f, 1.0f, 1.0f),
		// Shading model
		ShadingModelCodeChunk,
		NormalCodeChunk,
		TangentCodeChunk,
		BasisIndexMacro,
		ClearCoat_NormalCodeChunk,
		ClearCoat_TangentCodeChunk,
		ClearCoat_BasisIndexMacro,
		CustomTangent_TangentCodeChunk,
		!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr);

	return OutputCodeChunk;
}

int32 UMaterialExpressionSubstrateConvertMaterialAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FExpressionInput DummyInput;
	return CompileCommon(Compiler, OutputIndex, GetConnectedMaterialAttributesInputs(Material), MaterialAttributes, ShadingModelOverride,
		WaterScatteringCoefficients, WaterAbsorptionCoefficients, WaterPhaseG, ColorScaleBehindWater,
		HasSSS(), SubsurfaceProfile,
		DummyInput, DummyInput);
}

int32 UMaterialExpressionSubstrateConvertMaterialAttributes::CompileDefaultSlab(FMaterialCompiler* Compiler, int32 EmissiveOverride)
{
	if (!Substrate::IsSubstrateEnabled())
	{
		return Compiler->SubstrateCreateAndRegisterNullMaterial();
	}

	FGuid PathUniqueId = Compiler->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = Compiler->SubstrateCompilationGetOperator(PathUniqueId);

	int32 NormalCodeChunk = FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_Normal);

	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, NormalCodeChunk, INDEX_NONE);
	const FString BasisIndexMacro = Compiler->GetSubstrateSharedLocalBasisIndexMacro(NewRegisteredSharedLocalBasis);

	return Compiler->SubstrateConversionFromLegacy(
		false,
		// Metalness workflow
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_BaseColor),
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_Specular),
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_Metallic),
		// Roughness
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_Roughness),
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_Anisotropy),
		// SSS
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_SubsurfaceColor),
		CreateDefaultSubsurfaceProfileParameter(Compiler),
		// Clear Coat / Custom
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_CustomData0),// Clear coat
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_CustomData1),// Clear coat roughness
		// Misc
		EmissiveOverride != INDEX_NONE ? EmissiveOverride : FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_EmissiveColor),
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_Opacity),
		Compiler->Constant3(0.5f, 0.5f, 0.5f),
		Compiler->Constant(1.0f),
		// Water
		Compiler->Constant3(0.0f, 0.0f, 0.0f),
		Compiler->Constant3(0.0f, 0.0f, 0.0f),
		Compiler->Constant(0.0f),
		Compiler->Constant3(1.0f, 1.0f, 1.0f),
		// Shading model
		FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, MP_ShadingModel),
		NormalCodeChunk,
		INDEX_NONE,
		BasisIndexMacro,
		NormalCodeChunk,
		INDEX_NONE,
		BasisIndexMacro,
		NormalCodeChunk,
		(SubstrateOperator.Index != INDEX_NONE && SubstrateOperator.BSDFIndex != INDEX_NONE) && (!SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree)) ? &SubstrateOperator : nullptr);
}

int32 UMaterialExpressionSubstrateConvertMaterialAttributes::CompileDefaultSlab(FMaterialCompiler* Compiler, FVector3f EmissiveOverride)
{
	return CompileDefaultSlab(Compiler, Compiler->Constant3(EmissiveOverride.X, EmissiveOverride.Y, EmissiveOverride.Z));
}

void UMaterialExpressionSubstrateConvertMaterialAttributes::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (GraphNode && PropertyChangedEvent.Property != nullptr)
	{
		GraphNode->ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionSubstrateConvertMaterialAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Substrate Convert Material Attributes"));
}

EMaterialValueType UMaterialExpressionSubstrateConvertMaterialAttributes::GetOutputValueType(int32 OutputIndex)
{
	switch (OutputIndex)
	{
	case 0: return MCT_Substrate;
	}
	check(false);
	return MCT_Float1;
}

EMaterialValueType UMaterialExpressionSubstrateConvertMaterialAttributes::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)	  return MCT_MaterialAttributes; // MaterialAttributes
	else if (InputIndex == 1) return MCT_Float3; // WaterScatteringCoefficients
	else if (InputIndex == 2) return MCT_Float3; // WaterAbsorptionCoefficients
	else if (InputIndex == 3) return MCT_Float1; // WaterPhaseG
	else if (InputIndex == 4) return MCT_Float3; // ColorScaleBehindWater
	else if (InputIndex == 5) return MCT_ShadingModel; // ShadingModelOverride (as it uses 'ShowAsInputPin' metadata)

	check(false);
	return MCT_Float1;
}

FName UMaterialExpressionSubstrateConvertMaterialAttributes::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)		return TEXT("Attributes");
	else if (InputIndex == 1)	return TEXT("Water Scattering Coefficients (Water)");
	else if (InputIndex == 2)	return TEXT("Water Absorption Coefficients (Water)");
	else if (InputIndex == 3)	return TEXT("Water Phase G (Water)");
	else if (InputIndex == 4)	return TEXT("Color Scale BehindWater (Water)");
	else if (InputIndex == 5)	return TEXT("Shading Model From Expression");
	return NAME_None;
}

void UMaterialExpressionSubstrateConvertMaterialAttributes::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	switch (OutputIndex)
	{
	case 0: OutToolTip.Add(TEXT("TT Out Substrate Data")); break;
	}
	Super::GetConnectorToolTip(InputIndex, INDEX_NONE, OutToolTip);
}

bool UMaterialExpressionSubstrateConvertMaterialAttributes::IsResultSubstrateMaterial(int32 OutputIndex)
{
	return OutputIndex == 0;
}

bool UMaterialExpressionSubstrateConvertMaterialAttributes::IsResultMaterialAttributes(int32 OutputIndex)
{
	return false;
}

void UMaterialExpressionSubstrateConvertMaterialAttributes::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	const uint64 Cached = GetConnectedMaterialAttributesInputs(Material);

	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_BaseColor)) { SubstrateMaterialInfo.AddPropertyConnected(MP_BaseColor); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Metallic)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Metallic); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Specular)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Specular); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Roughness)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Roughness); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Anisotropy)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Anisotropy); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_EmissiveColor)) { SubstrateMaterialInfo.AddPropertyConnected(MP_EmissiveColor); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Normal)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Normal); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Tangent)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Tangent); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_SubsurfaceColor)) { SubstrateMaterialInfo.AddPropertyConnected(MP_SubsurfaceColor); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_CustomData0)) { SubstrateMaterialInfo.AddPropertyConnected(MP_CustomData0); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_CustomData1)) { SubstrateMaterialInfo.AddPropertyConnected(MP_CustomData1); }
	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_Opacity)) { SubstrateMaterialInfo.AddPropertyConnected(MP_Opacity); }

	if (FMaterialAttributeDefinitionMap::IsAttributeInBitmask(Cached, MP_ShadingModel) || ShadingModelOverride == MSM_FromMaterialExpression)
	{
		SubstrateMaterialInfo.AddPropertyConnected(MP_ShadingModel);

		// If the ShadingModel pin is plugged in, we must use a shading model from expression path.
		SubstrateMaterialInfo.SetShadingModelFromExpression(true);
	}
	else
	{
		// If the ShadingModel pin is NOT plugged in, we simply use the shading model selected on the root node drop box.
		if (ShadingModelOverride == MSM_Unlit) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Unlit); }
		if (ShadingModelOverride == MSM_DefaultLit) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_DefaultLit); }
		if (ShadingModelOverride == MSM_Subsurface) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceWrap); }
		if (ShadingModelOverride == MSM_PreintegratedSkin) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceWrap); }
		if (ShadingModelOverride == MSM_ClearCoat) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_ClearCoat); }
		if (ShadingModelOverride == MSM_SubsurfaceProfile) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceProfile); }
		if (ShadingModelOverride == MSM_TwoSidedFoliage) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SubsurfaceThinTwoSided); }
		if (ShadingModelOverride == MSM_Hair) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Hair); }
		if (ShadingModelOverride == MSM_Cloth) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Cloth); }
		if (ShadingModelOverride == MSM_Eye) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_Eye); }
		if (ShadingModelOverride == MSM_SingleLayerWater) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_SingleLayerWater); }
		if (ShadingModelOverride == MSM_ThinTranslucent) { SubstrateMaterialInfo.AddShadingModel(ESubstrateShadingModel::SSM_ThinTranslucent); }
	}

	if (SubsurfaceProfile)
	{
		SubstrateMaterialInfo.AddSubsurfaceProfile(SubsurfaceProfile);
	}
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionSubstrateConvertMaterialAttributes::SubstrateGenerateMaterialTopologyTreeCommon(
	class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
	const uint64 CachedConnectedMaterialAttributesInputs, const bool bShadingModelFromMaterialExpression, const bool bIsEmissiveConnected)
{
	return SubstrateGenerateMaterialTopologyTreeCommon(Compiler->GetSubstrateTranslatorData(), ThisExpressionGuid, Parent, OutputIndex,
		CachedConnectedMaterialAttributesInputs, bShadingModelFromMaterialExpression, bIsEmissiveConnected);
}

FSubstrateOperator* UMaterialExpressionSubstrateConvertMaterialAttributes::SubstrateGenerateMaterialTopologyTreeCommon(
	struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
	const uint64 CachedConnectedMaterialAttributesInputs, const bool bShadingModelFromMaterialExpression, const bool bIsEmissiveConnected)
{
	// Note Thickness has no meaning/usage in the context of SubstrateLegacyConversionNode
	int32 ThicknessIndex = SubstrateTranslatorData.SubstrateThicknessStackGetThicknessIndex();

	const bool bHasAnisotropy = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, MP_Anisotropy);

	auto AddDefaultWorstCase = [&](ESubstrateBsdfFeature In)
	{
		FSubstrateOperator& SlabOperator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
		SlabOperator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;
		SlabOperator.BSDFFeatures = In | (bHasAnisotropy ? ESubstrateBsdfFeature::Anisotropy : ESubstrateBsdfFeature::None);
		SlabOperator.ThicknessIndex = ThicknessIndex;

		return &SlabOperator;
	};

	// Get the shading models resulting from the UMaterial::RebuildShadingModelField().
	FMaterialShadingModelField ShadingModels = SubstrateTranslatorData.GetMaterialShadingModels();

	// Logic about shading models and complexity should match UMaterialExpressionSubstrateConvertMaterialAttributes::Compile.
	const bool bHasShadingModelFromExpression = FMaterialAttributeDefinitionMap::IsAttributeInBitmask(CachedConnectedMaterialAttributesInputs, MP_ShadingModel) || bShadingModelFromMaterialExpression; // We keep HasShadingModelFromExpression in case all shading models cannot be safely recovered from material functions.
	if ((ShadingModels.CountShadingModels() > 1) || bHasShadingModelFromExpression)
	{
		ESubstrateBsdfFeature Features = ESubstrateBsdfFeature::None;
		AppendLegacyShadingModelToBSDFFeature(ShadingModels, bHasAnisotropy, Features);
		return AddDefaultWorstCase(Features);
	}
	else
	{
		check(ShadingModels.CountShadingModels() == 1);

		if (ShadingModels.HasShadingModel(MSM_Unlit))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_UNLIT;
			Operator.ThicknessIndex = ThicknessIndex;
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_DefaultLit))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::None);
		}
		else if (ShadingModels.HasShadingModel(MSM_ThinTranslucent))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::None);
		}
		else if (ShadingModels.HasShadingModel(MSM_SubsurfaceProfile))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_Subsurface))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_TwoSidedFoliage))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_PreintegratedSkin))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::SSS);
		}
		else if (ShadingModels.HasShadingModel(MSM_Cloth))
		{
			return AddDefaultWorstCase(ESubstrateBsdfFeature::Fuzz);
		}
		else if (ShadingModels.HasShadingModel(MSM_ClearCoat))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;
			Operator.ThicknessIndex = ThicknessIndex;
			Operator.BSDFFeatures = ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat | (bHasAnisotropy ? ESubstrateBsdfFeature::Anisotropy : ESubstrateBsdfFeature::None);
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_Hair))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_HAIR;
			Operator.BSDFFeatures = ESubstrateBsdfFeature::Hair;
			Operator.ThicknessIndex = ThicknessIndex;
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_Eye))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_EYE;
			Operator.BSDFFeatures = ESubstrateBsdfFeature::Eye;
			Operator.ThicknessIndex = ThicknessIndex;
			return &Operator;
		}
		else if (ShadingModels.HasShadingModel(MSM_SingleLayerWater))
		{
			FSubstrateOperator& Operator = SubstrateTranslatorData.SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF_LEGACY, SubstrateTranslatorData.SubstrateTreeStackGetPathUniqueId(), ThisExpressionGuid, Parent, SubstrateTranslatorData.SubstrateTreeStackGetParentPathUniqueId());
			Operator.BSDFType = SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER;
			Operator.ThicknessIndex = ThicknessIndex;
			Operator.bBSDFWritesEmissive = bIsEmissiveConnected;
			return &Operator;
		}

		check(false);
		static FSubstrateOperator DefaultOperatorOnError;
		return &DefaultOperatorOnError;
	}
}

FSubstrateOperator* UMaterialExpressionSubstrateConvertMaterialAttributes::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	const uint64 Cached = GetConnectedMaterialAttributesInputs(Material);

	return SubstrateGenerateMaterialTopologyTreeCommon(
		SubstrateTranslatorData, MaterialExpressionGuid, Parent, OutputIndex,
		Cached, ShadingModelOverride == MSM_FromMaterialExpression, MaterialAttributes.IsConnected(MP_EmissiveColor));
}

bool UMaterialExpressionSubstrateConvertMaterialAttributes::HasSSS() const
{
	return SubsurfaceProfile != nullptr;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
