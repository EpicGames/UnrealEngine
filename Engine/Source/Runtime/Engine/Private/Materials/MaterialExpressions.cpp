// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialExpressions.cpp - Material expressions implementation.
=============================================================================*/

#include "Field/FieldSystemTypes.h"
#include "Misc/MessageDialog.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Internationalization/LocKeyFuncs.h"
#include "UObject/NaniteResearchStreamObjectVersion.h"
#include "Materials/MaterialExpressionChannelMaskParameterColor.h"
#include "UObject/UObjectAnnotation.h"
#include "RenderUtils.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "SubstrateDefinitions.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "MaterialSharedPrivate.h"
#include "Materials/HLSLMaterialTranslator.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionLayerStack.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionMaterialLayer.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialInstanceSupport.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialExpressionUtils.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/VolumeTexture.h"
#include "Engine/SubsurfaceProfile.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "VT/RuntimeVirtualTexture.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ColorManagement/ColorSpace.h"
#include "Containers/Array.h"
#include "Logging/StructuredLog.h"
#include "ShaderPlatformConfig.h"

#include "Materials/MaterialExpressionAbs.h"
#include "Materials/MaterialExpressionAbsorptionMediumMaterialOutput.h"
#include "Materials/MaterialExpressionActorPositionWS.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionArccosine.h"
#include "Materials/MaterialExpressionArccosineFast.h"
#include "Materials/MaterialExpressionArcsine.h"
#include "Materials/MaterialExpressionArcsineFast.h"
#include "Materials/MaterialExpressionArctangent.h"
#include "Materials/MaterialExpressionArctangentFast.h"
#include "Materials/MaterialExpressionArctangent2.h"
#include "Materials/MaterialExpressionArctangent2Fast.h"
#include "Materials/MaterialExpressionAtmosphericFogColor.h"
#include "Materials/MaterialExpressionBentNormalCustomOutput.h"
#include "Materials/MaterialExpressionBindlessSwitch.h"
#include "Materials/MaterialExpressionBlackBody.h"
#include "Materials/MaterialExpressionBlend.h"
#include "Materials/MaterialExpressionBlendMaterialAttributes.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionBumpOffset.h"
#include "Materials/MaterialExpressionCameraPositionWS.h"
#include "Materials/MaterialExpressionCameraVectorWS.h"
#include "Materials/MaterialExpressionCeil.h"
#include "Materials/MaterialExpressionChannelMaskParameter.h"
#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionCollectionTransform.h"
#include "Materials/MaterialExpressionColorRamp.h"
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionConstantBiasScale.h"
#include "Materials/MaterialExpressionConvert.h"
#include "Materials/MaterialExpressionCosine.h"
#include "Materials/MaterialExpressionCrossProduct.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionDBufferTexture.h"
#include "Materials/MaterialExpressionDDX.h"
#include "Materials/MaterialExpressionDDY.h"
#include "Materials/MaterialExpressionDecalColor.h"
#include "Materials/MaterialExpressionDecalDerivative.h"
#include "Materials/MaterialExpressionDecalLifetimeOpacity.h"
#include "Materials/MaterialExpressionDecalMipmapLevel.h"
#include "Materials/MaterialExpressionDepthFade.h"
#include "Materials/MaterialExpressionDepthOfFieldFunction.h"
#include "Materials/MaterialExpressionDeriveNormalZ.h"
#include "Materials/MaterialExpressionDesaturation.h"
#include "Materials/MaterialExpressionDistance.h"
#include "Materials/MaterialExpressionDistanceCullFade.h"
#include "Materials/MaterialExpressionDistanceFieldsRenderingSwitch.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionDotProduct.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "Materials/MaterialExpressionCloudLayer.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionEyeAdaptation.h"
#include "Materials/MaterialExpressionEyeAdaptationInverse.h"
#include "Materials/MaterialExpressionFeatureLevelSwitch.h"
#include "Materials/MaterialExpressionDataDrivenShaderPlatformInfoSwitch.h"
#include "Materials/MaterialExpressionRequiredSamplersSwitch.h"
#include "Materials/MaterialExpressionFloor.h"	
#include "Materials/MaterialExpressionFloatToUInt.h"	
#include "Materials/MaterialExpressionFmod.h"
#include "Materials/MaterialExpressionFontSignedDistance.h"
#include "Materials/MaterialExpressionFontSample.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionFrac.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionGIReplace.h"
#include "Materials/MaterialExpressionRayTracingQualitySwitch.h"
#include "Materials/MaterialExpressionPathTracingQualitySwitch.h"
#include "Materials/MaterialExpressionPathTracingRayTypeSwitch.h"
#include "Materials/MaterialExpressionPathTracingBufferTexture.h"
#include "Materials/MaterialExpressionGetMaterialAttributes.h"
#include "Materials/MaterialExpressionHairAttributes.h"
#include "Materials/MaterialExpressionHairColor.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionInverseLinearInterpolate.h"
#include "Materials/MaterialExpressionLightmapUVs.h"
#include "Materials/MaterialExpressionMeshPaintTextureCoordinateIndex.h"
#include "Materials/MaterialExpressionMeshPaintTextureObject.h"
#include "Materials/MaterialExpressionMeshPaintTextureReplace.h"
#include "Materials/MaterialExpressionPrecomputedAOMask.h"
#include "Materials/MaterialExpressionLightmassReplace.h"
#include "Materials/MaterialExpressionLightVector.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionRgbToHsv.h"
#include "Materials/MaterialExpressionHsvToRgb.h"
#include "Materials/MaterialExpressionExponential.h"
#include "Materials/MaterialExpressionExponential2.h"
#include "Materials/MaterialExpressionLength.h"
#include "Materials/MaterialExpressionLocalPosition.h"
#include "Materials/MaterialExpressionLogarithm.h"
#include "Materials/MaterialExpressionLogarithm2.h"
#include "Materials/MaterialExpressionLogarithm10.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMax.h"
#include "Materials/MaterialExpressionMaterialProxyReplace.h"
#include "Materials/MaterialExpressionMin.h"
#include "Materials/MaterialExpressionModulo.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNaniteReplace.h"
#include "Materials/MaterialExpressionMaterialCache.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionNeuralPostProcessNode.h"
#include "Materials/MaterialExpressionObjectBounds.h"
#include "Materials/MaterialExpressionObjectLocalBounds.h"
#include "Materials/MaterialExpressionObjectOrientation.h"
#include "Materials/MaterialExpressionObjectPositionWS.h"
#include "Materials/MaterialExpressionObjectRadius.h"
#include "Materials/MaterialExpressionBounds.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionPanner.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialExpressionPreSkinnedLocalBounds.h"
#include "Materials/MaterialExpressionPreviousFrameSwitch.h"
#include "Materials/MaterialExpressionRerouteBase.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionShadowReplace.h"
#include "Materials/MaterialExpressionSign.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialExpressionStaticComponentMaskParameter.h"
#include "Materials/MaterialExpressionSubsurfaceMediumMaterialOutput.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionDoubleVectorParameter.h"
#include "Materials/MaterialExpressionParticleColor.h"
#include "Materials/MaterialExpressionParticleDirection.h"
#include "Materials/MaterialExpressionParticleMacroUV.h"
#include "Materials/MaterialExpressionParticleMotionBlurFade.h"
#include "Materials/MaterialExpressionParticleRandom.h"
#include "Materials/MaterialExpressionParticlePositionWS.h"
#include "Materials/MaterialExpressionParticleRadius.h"
#include "Materials/MaterialExpressionParticleRelativeTime.h"
#include "Materials/MaterialExpressionParticleSize.h"
#include "Materials/MaterialExpressionParticleSpriteRotation.h"
#include "Materials/MaterialExpressionParticleSpeed.h"
#include "Materials/MaterialExpressionPerInstanceFadeAmount.h"
#include "Materials/MaterialExpressionPerInstanceRandom.h"
#include "Materials/MaterialExpressionPerInstanceCustomData.h"
#include "Materials/MaterialExpressionPixelDepth.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"
#include "Materials/MaterialExpressionPostVolumeUserFlagTest.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionPreSkinnedNormal.h"
#include "Materials/MaterialExpressionPreSkinnedPosition.h"
#include "Materials/MaterialExpressionQualitySwitch.h"
#include "Materials/MaterialExpressionShadingPathSwitch.h"
#include "Materials/MaterialExpressionReflectionVectorWS.h"
#include "Materials/MaterialExpressionRotateAboutAxis.h"
#include "Materials/MaterialExpressionRotator.h"
#include "Materials/MaterialExpressionRound.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureCustomData.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureOutput.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureReplace.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionVirtualTextureFeatureSwitch.h"
#include "Materials/MaterialExpressionSaturate.h"
#include "Materials/MaterialExpressionSceneColor.h"
#include "Materials/MaterialExpressionSceneDepth.h"
#include "Materials/MaterialExpressionSceneDepthWithoutWater.h"
#include "Materials/MaterialExpressionSceneTexelSize.h"
#include "Materials/MaterialExpressionSceneTexture.h"
#include "Materials/MaterialExpressionScreenPosition.h"
#include "Materials/MaterialExpressionShadingModel.h"
#include "Materials/MaterialExpressionSine.h"
#include "Materials/MaterialExpressionSmoothStep.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionFirstPersonOutput.h"
#include "Materials/MaterialExpressionSobol.h"
#include "Materials/MaterialExpressionSpeedTree.h"
#include "Materials/MaterialExpressionSphereMask.h"
#include "Materials/MaterialExpressionSphericalParticleOpacity.h"
#include "Materials/MaterialExpressionSquareRoot.h"
#include "Materials/MaterialExpressionSRGBColorToWorkingColorSpace.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionStep.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionTangent.h"
#include "Materials/MaterialExpressionTangentOutput.h"
#include "Materials/MaterialExpressionTemporalSobol.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionSwitch.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionParticleSubUV.h"
#include "Materials/MaterialExpressionParticleSubUVProperties.h"
#include "Materials/MaterialExpressionTextureCollection.h"
#include "Materials/MaterialExpressionTextureCollectionParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionTextureObjectFromCollection.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "Materials/MaterialExpressionAntialiasedTextureMask.h"
#include "Materials/MaterialExpressionTextureSampleParameterSubUV.h"
#include "Materials/MaterialExpressionTextureSampleParameterCube.h"
#include "Materials/MaterialExpressionTextureSampleParameter2DArray.h"
#include "Materials/MaterialExpressionTextureSampleParameterCubeArray.h"
#include "Materials/MaterialExpressionTextureSampleParameterVolume.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTime.h"
#include "Materials/MaterialExpressionDeltaTime.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionTruncate.h"
#include "Materials/MaterialExpressionTruncateLWC.h"
#include "Materials/MaterialExpressionTwoSidedSign.h"
#include "Materials/MaterialExpressionUserSceneTexture.h"
#include "Materials/MaterialExpressionVertexColor.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionVertexTangentWS.h"
#include "Materials/MaterialExpressionViewProperty.h"
#include "Materials/MaterialExpressionIsOrthographic.h"
#include "Materials/MaterialExpressionViewSize.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialInput.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionDistanceToNearestSurface.h"
#include "Materials/MaterialExpressionDistanceFieldGradient.h"
#include "Materials/MaterialExpressionDistanceFieldApproxAO.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionAtmosphericLightVector.h"
#include "Materials/MaterialExpressionAtmosphericLightColor.h"
#include "Materials/MaterialExpressionSkyAtmosphereLightIlluminance.h"
#include "Materials/MaterialExpressionSkyAtmosphereLightDirection.h"
#include "Materials/MaterialExpressionSkyAtmosphereViewLuminance.h"
#include "Materials/MaterialExpressionSkyLightEnvMapSample.h"
#include "Materials/MaterialExpressionMaterialLayerOutput.h"
#include "Materials/MaterialExpressionCurveAtlasRowParameter.h"
#include "Materials/MaterialExpressionMapARPassthroughCameraUV.h"
#include "Materials/MaterialExpressionShaderStageSwitch.h"
#include "Materials/MaterialExpressionReflectionCapturePassSwitch.h"
#include "Materials/MaterialExpressionReflectionCapturePassSwitch.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionSamplePhysicsField.h"
#include "Materials/MaterialExpressionIfThenElse.h"
#include "Materials/MaterialExpressionGenericConstant.h"
#include "Materials/MaterialExpressionSparseVolumeTextureBase.h"
#include "Materials/MaterialExpressionSparseVolumeTextureObject.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionIsFirstPerson.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialExpressionRecordTextureStreamingInfo.h"
#include "Materials/MaterialExpressionOperator.h"
#include "Materials/MaterialExpressionAggregate.h"
#include "Materials/MaterialExpressionTemporalResponsivenessOutput.h"
#include "Materials/MaterialExpressionMotionVectorWorldOffsetOutput.h"
#include "Materials/MaterialExpressionMainDirectionalLight.h"
#include "EditorSupportDelegates.h"
#if WITH_EDITOR
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "MaterialGraph/MaterialGraphNode_Composite.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialBase.h"
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialConvert.h"
#include "Editor/MaterialEditor/Public/FindInMaterial.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "SubstrateMaterial.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "MaterialEditor/PreviewMaterial.h"
#else
#include "Materials/MaterialExpressionVertexInterpolator.h"
#include "Materials/MaterialParameterCollection.h"
#endif //WITH_EDITOR
#include "Materials/MaterialInstanceConstant.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Curves/CurveLinearColor.h"
#include "MaterialExpressionSettings.h"
#include "UObject/ObjectEditorOptionalSupport.h"

#define LOCTEXT_NAMESPACE "MaterialExpression"

#define SWAP_REFERENCE_TO( ExpressionInput, ToBeRemovedExpression, ToReplaceWithExpression )	\
if( ExpressionInput.Expression == ToBeRemovedExpression )										\
{																								\
	ExpressionInput.Expression = ToReplaceWithExpression;										\
}

#if WITH_EDITOR
FUObjectAnnotationSparseBool GMaterialFunctionsThatNeedExpressionsFlipped;
FUObjectAnnotationSparseBool GMaterialFunctionsThatNeedCoordinateCheck;
FUObjectAnnotationSparseBool GMaterialFunctionsThatNeedCommentFix;
FUObjectAnnotationSparseBool GMaterialFunctionsThatNeedSamplerFixup;
FUObjectAnnotationSparseBool GMaterialFunctionsThatNeedFeatureLevelSM6Fix;

static const TCHAR* CPD_UI_ErrorMessage = TEXT("Custom Primitive Data can't be used with the UI material domain.");

bool GAllowSkipAlphaComputationForMaterialBlend = true;
static FAutoConsoleVariableRef CVarAllowSkipAlphaComputationForMaterialBlend(
	TEXT("r.MaterialEditor.AllowSkipAlphaComputationForMaterialBlend"),
	GAllowSkipAlphaComputationForMaterialBlend,
	TEXT("If true, Allow to skip alpha computation when evaluating material blend nodes if the 2 values are equal"),
	ECVF_ReadOnly
);

static void ThrowSubstrateUnsupportedSwitchNodeError(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, const FString& NodeClassName)
{
	SubstrateTranslatorData.Errorf(FString::Printf(TEXT("%s nodes do not support flowing Substrate data.\nThis is because the Substrate tree representing material topology must be known at translation time.\nBut this node uses branching with usage known at compilation time using #define or at runtime using dynamic branching."), *NodeClassName));
}

#endif // #if WITH_EDITOR

// Helper functions to handle determinism during cooking
namespace CookDeterminism
{
	// Used to ensure that same object can get different values for different fields
	enum class ESeed : uint32
	{
		Expression = 0,
		Parameter,
		InterfaceInit,
		InterfaceDuplicate,
		InterfacePostLoad,
		Compile,
		FunctionInput,
		FunctionOutput,
		RerouteVariable,
		RerouteColor,
	};

	FLinearColor MakeRandomColor(FStringView Str, ESeed Seed)
	{
		if (IsRunningCookCommandlet())
		{
			uint32 Hue = 157 * (GetTypeHash(Str) + static_cast<uint32>(Seed));
			return FLinearColor::MakeFromHSV8((uint8)Hue, 255, 255);
		}

		return FLinearColor::MakeRandomColor();
	}

	FGuid NewGuid(FStringView Str, ESeed Seed)
	{
		if (IsRunningCookCommandlet())
		{
			return FGuid::NewDeterministicGuid(Str, static_cast<uint64>(Seed));
		}

		return FGuid::NewGuid();
	}

	void LogWarningIfCooking(FStringView Message)
	{
		if (IsRunningCookCommandlet())
		{
			UE_LOGFMT(LogMaterial, Warning, "Non-determinism in material cooking: {Str}", Message);
		}
	}
};

/** Returns whether the given expression class is allowed. */
bool IsAllowedExpressionType(const UClass* Class, const bool bMaterialFunction)
{
	UMaterialExpression* DefaultExpression = CastChecked<UMaterialExpression>(Class->GetDefaultObject());
	UObject* MaterialOrFunction = bMaterialFunction ? UMaterialFunction::StaticClass()->GetDefaultObject() : UMaterial::StaticClass()->GetDefaultObject() ;
	return DefaultExpression->IsAllowedIn(MaterialOrFunction);
}

/** Parses a string into multiple lines, for use with tooltips. */
void ConvertToMultilineToolTip(const FString& InToolTip, const int32 TargetLineLength, TArray<FString>& OutToolTip)
{
	int32 CurrentPosition = 0;
	int32 LastPosition = 0;
	OutToolTip.Empty(1);

	while (CurrentPosition < InToolTip.Len())
	{
		// Move to the target position
		CurrentPosition += TargetLineLength;

		if (CurrentPosition < InToolTip.Len())
		{
			// Keep moving until we get to a space, or the end of the string
			while (CurrentPosition < InToolTip.Len() && InToolTip[CurrentPosition] != TCHAR(' '))
			{
				CurrentPosition++;
			}

			// Move past the space
			if (CurrentPosition < InToolTip.Len() && InToolTip[CurrentPosition] == TCHAR(' '))
			{
				CurrentPosition++;
			}

			// Add a new line, ending just after the space we just found
			OutToolTip.Add(InToolTip.Mid(LastPosition, CurrentPosition - LastPosition));
			LastPosition = CurrentPosition;
		}
		else
		{
			// Add a new line, right up to the end of the input string
			OutToolTip.Add(InToolTip.Mid(LastPosition, InToolTip.Len() - LastPosition));
		}
	}
}

extern ENGINE_API void GetMaterialValueTypeDescriptions(const uint32 MaterialValueType, TArray<FText>& OutDescriptions)
{
	return GetMaterialValueTypeDescriptions(static_cast<EMaterialValueType>(MaterialValueType), OutDescriptions);
}

extern ENGINE_API void GetMaterialValueTypeDescriptions(const EMaterialValueType MaterialValueType, TArray<FText>& OutDescriptions)
{
	// Get exact float type if possible
	EMaterialValueType MaskedFloatType = static_cast<EMaterialValueType>(MaterialValueType & MCT_Float);
	if (MaskedFloatType)
	{
		switch (MaskedFloatType)
		{
			case MCT_Float:
			case MCT_Float1: OutDescriptions.Add(LOCTEXT("Float", "Float")); break;
			case MCT_Float2: OutDescriptions.Add(LOCTEXT("Float2", "Float 2")); break;
			case MCT_Float3: OutDescriptions.Add(LOCTEXT("Float3", "Float 3")); break;
			case MCT_Float4: OutDescriptions.Add(LOCTEXT("Float4", "Float 4")); break;
			default: break;
		}
	}

	// Get exact int type if possible
	EMaterialValueType MaskedUIntType = static_cast<EMaterialValueType>(MaterialValueType & MCT_UInt);
	if (MaskedUIntType)
	{
		switch (MaskedUIntType)
		{
			case MCT_UInt:
			case MCT_UInt1: OutDescriptions.Add(LOCTEXT("UInt", "UInt")); break;
			case MCT_UInt2: OutDescriptions.Add(LOCTEXT("UInt2", "UInt 2")); break;
			case MCT_UInt3: OutDescriptions.Add(LOCTEXT("UInt3", "UInt 3")); break;
			case MCT_UInt4: OutDescriptions.Add(LOCTEXT("UInt4", "UInt 4")); break;
			default: break;
		}
	}
	// Get exact texture type if possible
	EMaterialValueType MaskedTextureType = static_cast<EMaterialValueType>(MaterialValueType & MCT_Texture);
	if (MaskedTextureType)
	{
		switch (MaskedTextureType)
		{
			case MCT_Texture2D:
				OutDescriptions.Add(LOCTEXT("Texture2D", "Texture 2D"));
				break;
			case MCT_TextureCube:
				OutDescriptions.Add(LOCTEXT("TextureCube", "Texture Cube"));
				break;
			case MCT_Texture2DArray:
				OutDescriptions.Add(LOCTEXT("Texture2DArray", "Texture 2D Array"));
				break;
			case MCT_TextureCubeArray:
				OutDescriptions.Add(LOCTEXT("TextureCubeArray", "Texture Cube Array"));
				break;
			case MCT_VolumeTexture:
				OutDescriptions.Add(LOCTEXT("VolumeTexture", "Volume Texture"));
				break;
			case MCT_Texture:
				OutDescriptions.Add(LOCTEXT("Texture", "Texture"));
				break;
			default:
				break;
		}
	}

	if (MaterialValueType & MCT_StaticBool)
		OutDescriptions.Add(LOCTEXT("StaticBool", "Bool"));
	if (MaterialValueType & MCT_Bool)
		OutDescriptions.Add(LOCTEXT("Bool", "Bool"));
	if (MaterialValueType & MCT_MaterialAttributes)
		OutDescriptions.Add(LOCTEXT("MaterialAttributes", "Material Attributes"));
	if (MaterialValueType & MCT_ShadingModel)
		OutDescriptions.Add(LOCTEXT("ShadingModel", "Shading Model"));
	if (MaterialValueType & MCT_Substrate)
		OutDescriptions.Add(LOCTEXT("Substrate", "Substrate Material"));
	if (MaterialValueType & MCT_TextureCollection)
		OutDescriptions.Add(LOCTEXT("TextureCollection", "Texture Collection"));
	if (MaterialValueType & MCT_Unknown)
		OutDescriptions.Add(LOCTEXT("Unknown", "Unknown"));
}

bool CanConnectMaterialValueTypes(const uint32 InputType, const uint32 OutputType)
{
	return CanConnectMaterialValueTypes(static_cast<EMaterialValueType>(InputType), static_cast<EMaterialValueType>(OutputType));
}

bool CanConnectMaterialValueTypes(const EMaterialValueType InputType, const EMaterialValueType OutputType)
{
	if ((InputType & MCT_Execution) || (OutputType & MCT_Execution))
	{
		// exec pins can only connect to other exec pins
		return InputType == OutputType;
	}

	if (InputType & MCT_Unknown)
	{
		// can plug anything into unknown inputs
		return true;
	}
	if (OutputType & MCT_Unknown)
	{
		// TODO: Decide whether these should connect to everything
		// Usually means that inputs haven't been connected yet so makes workflow easier
		return true;
	}
	if (InputType & OutputType)
	{
		return true;
	}
	// Need to do more checks here to see whether types can be cast
	// just check if both are float for now
	if ((InputType & MCT_Numeric) && (OutputType & MCT_Numeric))
	{
		return true;
	}
	if (InputType == MCT_Bool && OutputType == MCT_StaticBool)
	{
		// StaticBool is allowed to connect to Bool (but not the other way around)
		return true;
	}
	return false;
}

#if WITH_EDITOR


void ValidateParameterNameInternal(class UMaterialExpression* ExpressionToValidate, class UMaterial* OwningMaterial, const bool bAllowDuplicateName)
{
	if (OwningMaterial != nullptr)
	{
		int32 NameIndex = 1;
		bool bFoundValidName = false;
		FName PotentialName;

		// Find an available unique name
		while (!bFoundValidName)
		{
			PotentialName = ExpressionToValidate->GetParameterName();

			// Parameters cannot be named Name_None, use the default name instead
			if (PotentialName == NAME_None)
			{
				PotentialName = UMaterialExpressionParameter::ParameterDefaultName;
			}

			if (!bAllowDuplicateName)
			{
				if (NameIndex != 1)
				{
					PotentialName.SetNumber(NameIndex);
				}

				bFoundValidName = true;

				for (const UMaterialExpression* Expression : OwningMaterial->GetExpressions())
				{
					if (Expression != nullptr && Expression->HasAParameterName())
					{
						// Validate that the new name doesn't violate the expression's rules (by default, same name as another of the same class)
						if (Expression != ExpressionToValidate && Expression->GetParameterName() == PotentialName && Expression->HasClassAndNameCollision(ExpressionToValidate))
						{
							bFoundValidName = false;
							break;
						}
					}
				}

				++NameIndex;
			}
			else
			{
				bFoundValidName = true;
			}
		}

		if (bAllowDuplicateName)
		{
			// Check for any matching values
			for (UMaterialExpression* Expression : OwningMaterial->GetExpressions())
			{
				if (Expression != nullptr && Expression->HasAParameterName())
				{
					// Name are unique per class type
					if (Expression != ExpressionToValidate && Expression->GetParameterName() == PotentialName && Expression->GetClass() == ExpressionToValidate->GetClass())
					{
						FMaterialParameterMetadata Meta;
						if (OwningMaterial->GetParameterValue(Expression->GetParameterType(), PotentialName, Meta))
						{
							const EMaterialExpressionSetParameterValueFlags Flags =
								EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty |
								EMaterialExpressionSetParameterValueFlags::NoUpdateExpressionGuid;
							verify(ExpressionToValidate->SetParameterValue(PotentialName, Meta, Flags));
						}
						break;
					}
				}
			}
		}

		ExpressionToValidate->SetParameterName(PotentialName);
	}
}

/**
 * Helper function that wraps the supplied texture coordinates in the necessary math to transform them for external textures
 *
 * @param Compiler                The compiler to add code to
 * @param TexCoordCodeIndex       Index to the code chunk that supplies the vanilla texture coordinates
 * @param TextureReferenceIndex   Index of the texture within the material (used to access the external texture transform at runtime)
 * @param ParameterName           (Optional) Parameter name of the texture parameter that's assigned to the sample (used to access the external texture transform at runtime)
 * @return Index to a new code chunk that supplies the transformed UV coordinates
 */
int32 CompileExternalTextureCoordinates(FMaterialCompiler* Compiler, const int32 TexCoordCodeIndex, const int32 TextureReferenceIndex, const TOptional<FName> ParameterName = TOptional<FName>())
{
	if (TexCoordCodeIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 ScaleRotationCode = Compiler->ExternalTextureCoordinateScaleRotation(TextureReferenceIndex, ParameterName);
	const int32 OffsetCode = Compiler->ExternalTextureCoordinateOffset(TextureReferenceIndex, ParameterName);

	return Compiler->RotateScaleOffsetTexCoords(TexCoordCodeIndex, ScaleRotationCode, OffsetCode);
}

/**
 * Compile a texture sample taking into consideration external textures (which may use different sampling code in the shader on some platforms)
 *
 * @param Compiler                The compiler to add code to
 * @param Texture                 UTexture pointer used for the compiler
 * @param TexCoordCodeIndex       Index to the code chunk that supplies the vanilla texture coordinates
 * @param SamplerType             The type of sampler that is to be used
 * @param ParameterName           (Optional) Parameter name of the texture parameter that's assigned to the sample
 * @param MipValue0Index          (Optional) Mip value (0) code index when mips are being used
 * @param MipValue1Index          (Optional) Mip value (1) code index when mips are being used
 * @param MipValueMode            (Optional) Texture MIP value mode
 * @param SamplerSource           (Optional) Sampler source override
 * @return Index to a new code chunk that samples the texture
 */
ENGINE_API int32 CompileTextureSample(
	FMaterialCompiler* Compiler,
	UTexture* Texture,
	int32 TexCoordCodeIndex,
	const EMaterialSamplerType SamplerType,
	const TOptional<FName> ParameterName = TOptional<FName>(),
	const int32 MipValue0Index=INDEX_NONE,
	const int32 MipValue1Index=INDEX_NONE,
	const ETextureMipValueMode MipValueMode=TMVM_None,
	const ESamplerSourceMode SamplerSource=SSM_FromTextureAsset,
	const bool AutomaticViewMipBias=false,
	const ETextureGatherMode GatherMode=TGM_None
	)
{
	int32 TextureReferenceIndex = INDEX_NONE;
	int32 TextureCodeIndex = INDEX_NONE;
	if (SamplerType == SAMPLERTYPE_External)
	{
		// External sampler, so generate the necessary external uniform expression based on whether we're using a parameter name or not
		TextureCodeIndex = ParameterName.IsSet() ? Compiler->ExternalTextureParameter(ParameterName.GetValue(), Texture, TextureReferenceIndex) : Compiler->ExternalTexture(Texture, TextureReferenceIndex);

		// External textures need an extra transform applied to the UV coordinates
		TexCoordCodeIndex = CompileExternalTextureCoordinates(Compiler, TexCoordCodeIndex, TextureReferenceIndex, ParameterName);
	}
	else
	{
		TextureCodeIndex = ParameterName.IsSet() ? Compiler->TextureParameter(ParameterName.GetValue(), Texture, TextureReferenceIndex, SamplerType, SamplerSource) : Compiler->Texture(Texture, TextureReferenceIndex, SamplerType, SamplerSource, MipValueMode);
	}

	return Compiler->TextureSample(
					TextureCodeIndex,
					TexCoordCodeIndex,
					SamplerType,
					MipValue0Index,
					MipValue1Index,
					MipValueMode,
					SamplerSource,
					GatherMode,
					TextureReferenceIndex,
					AutomaticViewMipBias);
}
#endif // WITH_EDITOR

/**
 * Compile a select "blend" between ShadingModels
 *
 * @param Compiler				The compiler to add code to
 * @param A						Select A if Alpha is less than 0.5f
 * @param B						Select B if Alpha is greater or equal to 0.5f
 * @param Alpha					Bland factor [0..1]
 * @return						Index to a new code chunk
 */
int32 CompileShadingModelBlendFunction(FMaterialCompiler* Compiler, const int32 A, const int32 B, const int32 Alpha)
{
	if (A == INDEX_NONE || B == INDEX_NONE || Alpha == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	const int32 MidPoint = Compiler->Constant(0.5f);

	return Compiler->If(Alpha, MidPoint, B, INDEX_NONE, A, INDEX_NONE);
}

int32 CompileSubstrateBlendFunction(FMaterialCompiler* Compiler, const int32 A, const int32 B, const int32 Alpha)
{
	return INDEX_NONE;
}

void FMaterialExpressionCollection::AddExpression(UMaterialExpression* InExpression)
{
	Expressions.AddUnique(InExpression);
}

void FMaterialExpressionCollection::RemoveExpression(UMaterialExpression* InExpression)
{
	Expressions.Remove(InExpression);
}

void FMaterialExpressionCollection::AddComment(UMaterialExpressionComment* InExpression)
{
	EditorComments.AddUnique(InExpression);
}

void FMaterialExpressionCollection::RemoveComment(UMaterialExpressionComment* InExpression)
{
	EditorComments.Remove(InExpression);
}

void FMaterialExpressionCollection::Empty()
{
	Expressions.Empty();
	EditorComments.Empty();
}

UMaterialExpression::UMaterialExpression(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, GraphNode(nullptr)
	, SubgraphExpression(nullptr)
#endif // WITH_EDITORONLY_DATA
{
#if WITH_EDITORONLY_DATA
	Outputs.Add(FExpressionOutput(TEXT("")));

	bShowInputs = true;
	bShowOutputs = true;
	bCollapsed = true;
	bShowMaskColorsOnPin = true;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Iterate over the properties of derived expression struct, searching for properties of type FExpressionInput, and add them to the list of cached inputs.
	for (TFieldIterator<FStructProperty> InputIt(GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
	{
		FStructProperty* StructProp = *InputIt;
		if (StructProp->Struct->GetFName() == NAME_ExpressionInput)
		{
			for (int32 ArrayIndex = 0; ArrayIndex < StructProp->ArrayDim; ArrayIndex++)
			{
				CachedInputs.Add(StructProp->ContainerPtrToValuePtr<FExpressionInput>(this, ArrayIndex));
			}
		}
	}
	CachedInputs.Shrink();

	// Initialize the input names from GetInputName())
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		It.Input->InputName = GetInputName(It.Index);
	}

	#endif
}

UObject* UMaterialExpression::GetAssetOwner() const
{
	return Function ? (UObject*)Function : (UObject*)Material;
}

FString UMaterialExpression::GetAssetPathName() const
{
	UObject* Asset = GetAssetOwner();
	return Asset ? Asset->GetPathName() : FString();
}

#if WITH_EDITOR

int32 UMaterialExpression::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (GetClass()->HasMetaData("NewMaterialTranslator"))
	{
		Compiler->Errorf(TEXT("Unsupported expression '%s'."), *GetName());
	}
	return INDEX_NONE;
}

void UMaterialExpression::CopyMaterialExpressions(const TArray<UMaterialExpression*>& SrcExpressions, const TArray<UMaterialExpressionComment*>& SrcExpressionComments, 
	UMaterial* Material, UMaterialFunction* EditFunction, TArray<UMaterialExpression*>& OutNewExpressions, TArray<UMaterialExpression*>& OutNewComments)
{
	OutNewExpressions.Empty();
	OutNewComments.Empty();

	UObject* ExpressionOuter = Material;
	if (EditFunction)
	{
		ExpressionOuter = EditFunction;
	}

	TMap<UMaterialExpression*,UMaterialExpression*> SrcToDestMap;

	// Duplicate source expressions into the editor's material copy buffer.
	for( int32 SrcExpressionIndex = 0 ; SrcExpressionIndex < SrcExpressions.Num() ; ++SrcExpressionIndex )
	{
		UMaterialExpression*	SrcExpression		= SrcExpressions[SrcExpressionIndex];
		UMaterialExpressionMaterialFunctionCall* FunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(SrcExpression);
		bool bIsValidFunctionExpression = true;

		if (EditFunction 
			&& FunctionExpression 
			&& FunctionExpression->MaterialFunction
			&& FunctionExpression->MaterialFunction->IsDependent(EditFunction))
		{
			bIsValidFunctionExpression = false;
		}

		if (bIsValidFunctionExpression && SrcExpression->IsAllowedIn(ExpressionOuter))
		{
			UMaterialExpression* NewExpression = Cast<UMaterialExpression>(StaticDuplicateObject( SrcExpression, ExpressionOuter, NAME_None, RF_Transactional ));
			NewExpression->Material = Material;
			// Make sure we remove any references to functions the nodes came from
			NewExpression->Function = nullptr;

			SrcToDestMap.Add( SrcExpression, NewExpression );

			// Add to list of material expressions associated with the copy buffer.
			Material->GetExpressionCollection().AddExpression( NewExpression );

			// There can be only one default mesh paint texture.
			UMaterialExpressionTextureBase* TextureSample = Cast<UMaterialExpressionTextureBase>( NewExpression );
			if( TextureSample )
			{
				TextureSample->IsDefaultMeshpaintTexture = false;
			}

			NewExpression->UpdateParameterGuid(true, true);
			NewExpression->UpdateMaterialExpressionGuid(true, true);

			UMaterialExpressionFunctionInput* FunctionInput = Cast<UMaterialExpressionFunctionInput>( NewExpression );
			if( FunctionInput )
			{
				FunctionInput->ConditionallyGenerateId(true);
				FunctionInput->ValidateName();
			}

			UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>( NewExpression );
			if( FunctionOutput )
			{
				FunctionOutput->ConditionallyGenerateId(true);
				FunctionOutput->ValidateName();
			}

			// Record in output list.
			OutNewExpressions.Add( NewExpression );
		}
	}

	// Fix up internal references.  Iterate over the inputs of the new expressions, and for each input that refers
	// to an expression that was duplicated, point the reference to that new expression.  Otherwise, clear the input.
	for( int32 NewExpressionIndex = 0 ; NewExpressionIndex < OutNewExpressions.Num() ; ++NewExpressionIndex )
	{
		UMaterialExpression* NewExpression = OutNewExpressions[NewExpressionIndex];
		for (FExpressionInputIterator It{ NewExpression }; It; ++It)
		{
			if (UMaterialExpression* InputExpression = It->Expression)
			{
				UMaterialExpression** NewInputExpression = SrcToDestMap.Find( InputExpression );
				if ( NewInputExpression )
				{
					check( *NewInputExpression );
					It->Expression = *NewInputExpression;
				}
				else
				{
					It->Expression = nullptr;
				}
			}
		}
	}

	// Copy Selected Comments
	for( int32 CommentIndex=0; CommentIndex<SrcExpressionComments.Num(); CommentIndex++)
	{
		UMaterialExpressionComment* ExpressionComment = SrcExpressionComments[CommentIndex];
		UMaterialExpressionComment* NewComment = Cast<UMaterialExpressionComment>(StaticDuplicateObject(ExpressionComment, ExpressionOuter));
		NewComment->Material = Material;

		// Add reference to the material
		Material->GetExpressionCollection().AddComment(NewComment);

		// Add to the output array.
		OutNewComments.Add(NewComment);
	}
}

EMaterialValueType UMaterialExpression::GetCompositeOutput(int32 OutputIndex)
{
	// Material attributes need to be tested first to work when plugged in main root node (to not return MCT_Substrate when Substrate mateiral is fed)
	if (IsResultMaterialAttributes(OutputIndex))
	{
		return MCT_MaterialAttributes;
	}

	if (IsResultSubstrateMaterial(OutputIndex))
	{
		return MCT_Substrate;
	}

	return MCT_Unknown;
}
#endif // WITH_EDITOR

void UMaterialExpression::Serialize(FStructuredArchive::FRecord Record)
{
	SCOPED_LOADTIMER(UMaterialExpression_Serialize);
	Super::Serialize(Record);

	FArchive& Archive = Record.GetUnderlyingArchive();

	Archive.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Archive.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	const FPackageFileVersion UEVer = Archive.UEVer();
	const int32 RenderVer = Archive.CustomVer(FRenderingObjectVersion::GUID);
	const int32 UE5Ver = Archive.CustomVer(FUE5MainStreamObjectVersion::GUID);

	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		DoMaterialAttributeReorder(It.Input, UEVer, RenderVer, UE5Ver);
	}
#endif // WITH_EDITORONLY_DATA
}

void UMaterialExpression::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		UpdateParameterGuid(false, false);
	
		UpdateMaterialExpressionGuid(false, false);
	}
}

void UMaterialExpression::PostLoad()
{
	SCOPED_LOADTIMER(UMaterialExpression_PostLoad);
	Super::PostLoad();

	if (!Material && GetOuter()->IsA(UMaterial::StaticClass()))
	{
		Material = CastChecked<UMaterial>(GetOuter());
	}

	if (!Function && GetOuter()->IsA(UMaterialFunction::StaticClass()))
	{
		Function = CastChecked<UMaterialFunction>(GetOuter());
	}

	UpdateParameterGuid(false, false);
	
	UpdateMaterialExpressionGuid(false, false);
}

void UMaterialExpression::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// We do not force a guid regen here because this function is used when the Material Editor makes a copy of a material to edit.
	// If we forced a GUID regen, it would cause all of the guids for a material to change everytime a material was edited.
	UpdateParameterGuid(false, true);
	UpdateMaterialExpressionGuid(false, true);
}

#if WITH_EDITOR

TArray<FProperty*> UMaterialExpression::GetInputPinProperty(int32 PinIndex)
{
	// Find all UPROPERTYs associated with this input pin
	TArray<FProperty*> Properties;

	// Explicit input pins are before property input pins
	TArray<FProperty*> PropertyInputs = GetPropertyInputs();

	
	if (FExpressionInput* Input = GetInput(PinIndex))
	{
		// Find the UPROPERTYs that have OverridingInputProperty meta data pointing to the expression input.
		// There can be multiple scalar entries together forming a vector parameter, e.g. DecalMipmapLevel node has FExpressionInput TextureSize <-> float ConstWidth/ConstHeight.
		static FName OverridingInputPropertyMetaData("OverridingInputProperty");
		for (TFieldIterator<FProperty> InputIt(GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
		{
			FProperty* Property = *InputIt;
			if (Property->HasMetaData(OverridingInputPropertyMetaData))
			{
				const FString& OverridingPropertyName = Property->GetMetaData(OverridingInputPropertyMetaData);
				FStructProperty* StructProp = FindFProperty<FStructProperty>(GetClass(), *OverridingPropertyName);
				if (ensure(StructProp != nullptr))
				{
					if (Input == StructProp->ContainerPtrToValuePtr<FExpressionInput>(this))
					{
						Properties.Add(Property);
					}
				}
			}
		}
	}
	else
	{
		int32 NumInputs = CountInputs();
		if (PinIndex < NumInputs + PropertyInputs.Num())
		{
			FName PropertyName = PropertyInputs[PinIndex - NumInputs]->GetFName();
			for (TFieldIterator<FProperty> InputIt(GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
			{
				FProperty* Property = *InputIt;
				if (PropertyName == Property->GetFName())
				{
					Properties.Add(Property);
				}
			}
		}
	}
	return Properties;
}

FName UMaterialExpression::GetInputPinSubCategory(int32 PinIndex)
{
	FName PinSubCategory;

	// Find the UPROPERTY associated with the pin
	TArray<FProperty*> Properties = GetInputPinProperty(PinIndex);
	if (Properties.Num() == 1)
	{
		// This is the UPROPERTY matching with the target input
		FProperty* Property = Properties[0];
		FFieldClass* PropertyClass = Property->GetClass();
		if (PropertyClass == FFloatProperty::StaticClass())
		{
			PinSubCategory = UMaterialGraphSchema::PSC_Red;
		}
		else if (PropertyClass == FDoubleProperty::StaticClass())
		{
			PinSubCategory = UMaterialGraphSchema::PSC_Red;
		}
		else if (PropertyClass == FIntProperty::StaticClass())
		{
			PinSubCategory = UMaterialGraphSchema::PSC_Int;
		}
		else if (PropertyClass == FUInt32Property::StaticClass())
		{
			PinSubCategory = UMaterialGraphSchema::PSC_Int;
		}
		else if (PropertyClass == FByteProperty::StaticClass())
		{
			PinSubCategory = UMaterialGraphSchema::PSC_Byte;
		}
		else if (PropertyClass == FBoolProperty::StaticClass())
		{
			PinSubCategory = UMaterialGraphSchema::PSC_Bool;
		}
		else if (PropertyClass == FStructProperty::StaticClass())
		{
			UScriptStruct* Struct = CastField<FStructProperty>(Property)->Struct;
			if (Struct == TBaseStructure<FLinearColor>::Get())
			{
				PinSubCategory = Property->HasMetaData(TEXT("HideAlphaChannel")) ? UMaterialGraphSchema::PSC_RGB : UMaterialGraphSchema::PSC_RGBA;
			}
			else if (Struct == TBaseStructure<FVector4>::Get() || Struct == TVariantStructure<FVector4d>::Get())
			{
				PinSubCategory = UMaterialGraphSchema::PSC_Vector4;
			}
			else if (Struct == TBaseStructure<FVector>::Get() || Struct == TVariantStructure<FVector3f>::Get())
			{
				PinSubCategory = UMaterialGraphSchema::PSC_RGB;
			}
			else if (Struct == TBaseStructure<FVector2D>::Get())
			{
				PinSubCategory = UMaterialGraphSchema::PSC_RG;
			}	
		}
	}
	// There can be multiple scalar entries together forming a vector2/3/4.
	else if (Properties.Num() == 2)
	{
		PinSubCategory = UMaterialGraphSchema::PSC_RG;
	}
	else if (Properties.Num() == 3)
	{
		PinSubCategory = UMaterialGraphSchema::PSC_RGB;
	}
	else if (Properties.Num() == 4)
	{
		PinSubCategory = UMaterialGraphSchema::PSC_Vector4;
	}

	return PinSubCategory;
}

UObject* UMaterialExpression::GetInputPinSubCategoryObject(int32 PinIndex)
{
	UObject* PinSubCategoryObject = nullptr;
	TArray<FProperty*> Properties = GetInputPinProperty(PinIndex);
	if (Properties.Num() > 0)
	{
		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Properties[0]))
		{
			PinSubCategoryObject = ByteProperty->GetIntPropertyEnum();
		}
	}
	return PinSubCategoryObject;
}

void UMaterialExpression::PinDefaultValueChanged(int32 PinIndex, const FString& DefaultValue)
{
	// Update the default value of the expression input when pin value changes
	// Find the UPROPERTYs that have OverridingInputProperty meta data pointing to the input, override their values.
	TArray<FProperty*> Properties = GetInputPinProperty(PinIndex);
	if (Properties.IsEmpty())
	{
		return;
	}

	Modify();

	TArray<FString> PropertyValues;
	if (Properties.Num() == 1)
	{
		PropertyValues.Add(DefaultValue);
	}
	else if (Properties.Num() == 2)
	{
		// Vector2 is formatted as (X=0.0, Y=0.0)
		FVector2D Value;
		Value.InitFromString(DefaultValue);
		PropertyValues.Add(FString::SanitizeFloat(Value.X));
		PropertyValues.Add(FString::SanitizeFloat(Value.Y));
	}
	else
	{
		// Vector3/4 are formatted as numbers separated by commas
		DefaultValue.ParseIntoArray(PropertyValues, TEXT(","), true);
		check(PropertyValues.Num() == Properties.Num());
	}

	for (int32 i = 0; i < Properties.Num(); ++i)
	{
		FProperty* Property = Properties[i];
		const FString& ClampMin = Property->GetMetaData(TEXT("ClampMin"));
		const FString& ClampMax = Property->GetMetaData(TEXT("ClampMax"));

		FString PropertyValue = PropertyValues[i];
		FFieldClass* PropertyClass = Property->GetClass();
		if (PropertyClass == FFloatProperty::StaticClass())
		{
			float Value = FCString::Atof(*PropertyValue);
			Value = ClampMin.Len() ? (FMath::Max<float>(FCString::Atof(*ClampMin), Value)) : Value;
			Value = ClampMax.Len() ? (FMath::Min<float>(FCString::Atof(*ClampMax), Value)) : Value;
			FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property);
			FloatProperty->SetPropertyValue_InContainer(this, Value);
		}
		else if (PropertyClass == FDoubleProperty::StaticClass())
		{
			double Value = FCString::Atod(*PropertyValue);
			Value = ClampMin.Len() ? (FMath::Max<double>(FCString::Atod(*ClampMin), Value)) : Value;
			Value = ClampMax.Len() ? (FMath::Min<double>(FCString::Atod(*ClampMax), Value)) : Value;
			FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property);
			DoubleProperty->SetPropertyValue_InContainer(this, Value);
		}
		else if (PropertyClass == FIntProperty::StaticClass())
		{
			int32 Value = FCString::Atoi(*PropertyValue);
			Value = ClampMin.Len() ? (FMath::Max<int32>(FCString::Atoi(*ClampMin), Value)) : Value;
			Value = ClampMax.Len() ? (FMath::Min<int32>(FCString::Atoi(*ClampMax), Value)) : Value;
			FIntProperty* IntProperty = CastField<FIntProperty>(Property);
			IntProperty->SetPropertyValue_InContainer(this, Value);
		}
		else if (PropertyClass == FUInt32Property::StaticClass())
		{
			int32 IntValue = FCString::Atoi(*PropertyValue);
			IntValue = ClampMin.Len() ? (FMath::Max<int32>(FCString::Atoi(*ClampMin), IntValue)) : IntValue;
			IntValue = ClampMax.Len() ? (FMath::Min<int32>(FCString::Atoi(*ClampMax), IntValue)) : IntValue;
			// Make sure the value is not negative
			uint32 Value = (uint32)FMath::Max(IntValue, 0);
			FUInt32Property* UInt32Property = CastField<FUInt32Property>(Property);
			UInt32Property->SetPropertyValue_InContainer(this, Value);
		}
		else if (PropertyClass == FByteProperty::StaticClass())
		{
			FByteProperty* ByteProperty = CastField<FByteProperty>(Property);
			uint8 Value;
			UEnum* Enum = ByteProperty->GetIntPropertyEnum();
			if (Enum)
			{
				Value = (uint8)Enum->GetValueByName(FName(PropertyValue));
			}
			else
			{
				int32 IntValue = FCString::Atoi(*PropertyValue);
				IntValue = ClampMin.Len() ? (FMath::Max<int32>(FCString::Atoi(*ClampMin), IntValue)) : IntValue;
				IntValue = ClampMax.Len() ? (FMath::Min<int32>(FCString::Atoi(*ClampMax), IntValue)) : IntValue;
				// Make sure the value doesn't exceed byte limit
				Value = (uint8)FMath::Max(FMath::Min(IntValue, 255), 0);
			}
			ByteProperty->SetPropertyValue_InContainer(this, Value);
		}
		else if (PropertyClass == FBoolProperty::StaticClass())
		{
			bool Value = FCString::ToBool(*PropertyValue);
			FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);
			BoolProperty->SetPropertyValue_InContainer(this, Value);
		}
		else if (PropertyClass == FStructProperty::StaticClass())
		{
			UScriptStruct* Struct = ((FStructProperty*)Property)->Struct;
			if (Struct == TBaseStructure<FLinearColor>::Get())
			{
				FLinearColor* ColorProperty = Property->ContainerPtrToValuePtr<FLinearColor>(this);
				if (Property->HasMetaData(TEXT("HideAlphaChannel")))
				{
					// This is a 3 element vector
					TArray<FString> Elements;
					PropertyValue.ParseIntoArray(Elements, TEXT(","), true);
					check(Elements.Num() == 3);
					ColorProperty->R = FCString::Atof(*Elements[0]);
					ColorProperty->G = FCString::Atof(*Elements[1]);
					ColorProperty->B = FCString::Atof(*Elements[2]);
				}
				else
				{
					// This is a 4 element vector
					ColorProperty->InitFromString(PropertyValue);
				}
			}
			else if (Struct == TBaseStructure<FVector4>::Get())
			{
				TArray<FString> Elements;
				PropertyValue.ParseIntoArray(Elements, TEXT(","), true);
				check(Elements.Num() == 4);
				FVector4* Value = Property->ContainerPtrToValuePtr<FVector4>(this);
				Value->X = FCString::Atod(*Elements[0]);
				Value->Y = FCString::Atod(*Elements[1]);
				Value->Z = FCString::Atod(*Elements[2]);
				Value->W = FCString::Atod(*Elements[3]);
			}
			else if (Struct == TVariantStructure<FVector4d>::Get())
			{
				TArray<FString> Elements;
				PropertyValue.ParseIntoArray(Elements, TEXT(","), true);
				check(Elements.Num() == 4);
				FVector4d* Value = Property->ContainerPtrToValuePtr<FVector4d>(this);
				Value->X = FCString::Atod(*Elements[0]);
				Value->Y = FCString::Atod(*Elements[1]);
				Value->Z = FCString::Atod(*Elements[2]);
				Value->W = FCString::Atod(*Elements[3]);
			}
			else if (Struct == TBaseStructure<FVector>::Get())
			{
				TArray<FString> Elements;
				PropertyValue.ParseIntoArray(Elements, TEXT(","), true);
				check(Elements.Num() == 3);
				FVector* Value = Property->ContainerPtrToValuePtr<FVector>(this);
				Value->X = FCString::Atod(*Elements[0]);
				Value->Y = FCString::Atod(*Elements[1]);
				Value->Z = FCString::Atod(*Elements[2]);
			}
			else if (Struct == TVariantStructure<FVector3f>::Get())
			{
				TArray<FString> Elements;
				PropertyValue.ParseIntoArray(Elements, TEXT(","), true);
				check(Elements.Num() == 3);
				FVector3f* Value = Property->ContainerPtrToValuePtr<FVector3f>(this);
				Value->X = FCString::Atof(*Elements[0]);
				Value->Y = FCString::Atof(*Elements[1]);
				Value->Z = FCString::Atof(*Elements[2]);
			}
			else if (Struct == TBaseStructure<FVector2D>::Get())
			{
				FVector2D* Value = Property->ContainerPtrToValuePtr<FVector2D>(this);
				Value->InitFromString(PropertyValue);
			}
		}

		FPropertyChangedEvent Event(Property);
		PostEditChangeProperty(Event);
	}

	RefreshNode();
}

void UMaterialExpression::ForcePropertyValueChanged(FProperty* Property, bool bUpdatePreview)
{
	Modify();

	FPropertyChangedEvent Event(Property);
	PostEditChangeProperty(Event);

	RefreshNode(bUpdatePreview);
}

void UMaterialExpression::RefreshNode(bool bUpdatePreview)
{
	const UMaterialGraphSchema* Schema = CastChecked<const UMaterialGraphSchema>(GraphNode->GetSchema());

	if (bUpdatePreview)
	{
		// Make sure that all other nodes also require a preview update.
		UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(GraphNode);
		if (Node)
		{
			Node->PropagatePropertyChange();
		}

		// Update the expression preview and the material to reflect the change
		GetDefault<UMaterialGraphSchema>()->ForceVisualizationCacheClear();

		Schema->UpdateMaterialOnDefaultValueChanged(GraphNode->GetGraph());
		// There might be other properties affected by this property change (e.g. propertyA determines if propertyB is read-only) so refresh the detail view
		Schema->UpdateDetailView(GraphNode->GetGraph());
	}
	else
	{
		Schema->MarkMaterialDirty(GraphNode->GetGraph());
	}
}

FString UMaterialExpression::GetInputPinDefaultValue(int32 PinIndex)
{
	TArray<FString> PropertyValues;

	// Find the UPROPERTYs for the input pin, retrieve their values.
	TArray<FProperty*> Properties = GetInputPinProperty(PinIndex);
	for (int32 i = 0; i < Properties.Num(); ++i)
	{
		// This is the UPROPERTY matching with the target input
		FProperty* Property = Properties[i];
		FString PropertyValue;
		FFieldClass* PropertyClass = Property->GetClass();
		if (PropertyClass == FFloatProperty::StaticClass())
		{
			FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property);
			float Value = FloatProperty->GetPropertyValue_InContainer(this);
			PropertyValue = FString::SanitizeFloat(Value);
		}
		else if (PropertyClass == FDoubleProperty::StaticClass())
		{
			FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property);
			double Value = DoubleProperty->GetPropertyValue_InContainer(this);
			PropertyValue = LexToString(Value);
		}
		else if (PropertyClass == FIntProperty::StaticClass())
		{
			FIntProperty* IntProperty = CastField<FIntProperty>(Property);
			int32 Value = IntProperty->GetPropertyValue_InContainer(this);
			PropertyValue = FString::FromInt(Value);
		}
		else if (PropertyClass == FUInt32Property::StaticClass())
		{
			FUInt32Property* UInt32Property = CastField<FUInt32Property>(Property);
			uint32 Value = UInt32Property->GetPropertyValue_InContainer(this);
			PropertyValue = FString::FromInt((int32)Value);
		}
		else if (PropertyClass == FByteProperty::StaticClass())
		{
			FByteProperty* ByteProperty = CastField<FByteProperty>(Property);
			uint8 Value = ByteProperty->GetPropertyValue_InContainer(this);
			PropertyValue = (ByteProperty->Enum ? ByteProperty->Enum->GetDisplayNameTextByValue(Value).ToString() : FString::FromInt(Value));
		}
		else if (PropertyClass == FBoolProperty::StaticClass())
		{
			FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);
			bool Value = BoolProperty->GetPropertyValue_InContainer(this);
			PropertyValue = (Value ? TEXT("true") : TEXT("false"));
		}
		else if (PropertyClass == FStructProperty::StaticClass())
		{
			UScriptStruct* Struct = ((FStructProperty*)Property)->Struct;
			if (Struct == TBaseStructure<FLinearColor>::Get())
			{
				FLinearColor Value = *Property->ContainerPtrToValuePtr<FLinearColor>(this);
				if (Property->HasMetaData(TEXT("HideAlphaChannel")))
				{
					// This is a 3 element vector
					PropertyValue = FString::SanitizeFloat(Value.R) + FString(",") + FString::SanitizeFloat(Value.G) + FString(",") + FString::SanitizeFloat(Value.B);
				}
				else
				{
					// This is a 4 element vector
					PropertyValue = Value.ToString();
				}
			}
			else if (Struct == TBaseStructure<FVector4>::Get())
			{
				FVector4 Value = *Property->ContainerPtrToValuePtr<FVector4>(this);		
				PropertyValue = FString::SanitizeFloat(Value.X) + FString(",") + FString::SanitizeFloat(Value.Y) + FString(",") + FString::SanitizeFloat(Value.Z) + FString(",") + FString::SanitizeFloat(Value.W);
			}
			else if (Struct == TVariantStructure<FVector4d>::Get())
			{
				FVector4d Value = *Property->ContainerPtrToValuePtr<FVector4d>(this);		
				PropertyValue = FString::SanitizeFloat(Value.X) + FString(",") + FString::SanitizeFloat(Value.Y) + FString(",") + FString::SanitizeFloat(Value.Z) + FString(",") + FString::SanitizeFloat(Value.W);
			}
			else if (Struct == TBaseStructure<FVector>::Get())
			{
				FVector Value = *Property->ContainerPtrToValuePtr<FVector>(this);
				PropertyValue = FString::SanitizeFloat(Value.X) + FString(",") + FString::SanitizeFloat(Value.Y) + FString(",") + FString::SanitizeFloat(Value.Z);
			}
			else if (Struct == TVariantStructure<FVector3f>::Get())
			{
				FVector3f Value = *Property->ContainerPtrToValuePtr<FVector3f>(this);
				PropertyValue = FString::SanitizeFloat(Value.X) + FString(",") + FString::SanitizeFloat(Value.Y) + FString(",") + FString::SanitizeFloat(Value.Z);
			}
			else if (Struct == TBaseStructure<FVector2D>::Get())
			{
				FVector2D* Value = Property->ContainerPtrToValuePtr<FVector2D>(this);
				PropertyValue = Value->ToString();
			}
		}

		PropertyValues.Add(PropertyValue);
	}

	check(PropertyValues.Num() == Properties.Num());
	if (Properties.Num() == 1)
	{
		return PropertyValues[0];
	}
	else if (Properties.Num() == 2)
	{
		// Vector2 is formatted as (X=0.0, Y=0.0)
		float X = FCString::Atof(*PropertyValues[0]);
		float Y = FCString::Atof(*PropertyValues[1]);
		FVector2D Value(X, Y);
		return Value.ToString();
	}
	// Vector3/4 are formatted as numbers separated by commas
	else if (Properties.Num() == 3)
	{
		return PropertyValues[0] + TEXT(",") + PropertyValues[1] + TEXT(",") + PropertyValues[2];
	}
	else if (Properties.Num() == 4)
	{
		return PropertyValues[0] + TEXT(",") + PropertyValues[1] + TEXT(",") + PropertyValues[2] + TEXT(",") + PropertyValues[3];
	}

	return TEXT("");
}

TArray<FProperty*> UMaterialExpression::GetPropertyInputs() const
{
	TArray<FProperty*> PropertyInputs;

	static FName OverridingInputPropertyMetaData("OverridingInputProperty");
	static FName ShowAsInputPinMetaData("ShowAsInputPin");
	for (TFieldIterator<FProperty> InputIt(GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
	{
		bool bCreateInput = false;
		FProperty* Property = *InputIt;
		// Don't create an expression input if the property is already associated with one explicitly declared
		bool bOverridingInputProperty = Property->HasMetaData(OverridingInputPropertyMetaData);
		// It needs to have the 'EditAnywhere' specifier
		const bool bEditAnywhere = Property->HasAnyPropertyFlags(CPF_Edit);
		// It needs to be marked with a valid pin category meta data
		const FString ShowAsInputPin = Property->GetMetaData(ShowAsInputPinMetaData);
		const bool bShowAsInputPin = ShowAsInputPin == TEXT("Primary") || ShowAsInputPin == TEXT("Advanced");

		if (!bOverridingInputProperty && bEditAnywhere && bShowAsInputPin)
		{
			// Check if the property type fits within the allowed widget types
			FFieldClass* PropertyClass = Property->GetClass();
			if (PropertyClass == FFloatProperty::StaticClass()
				|| PropertyClass == FDoubleProperty::StaticClass()
				|| PropertyClass == FIntProperty::StaticClass()
				|| PropertyClass == FUInt32Property::StaticClass()
				|| PropertyClass == FByteProperty::StaticClass()
				|| PropertyClass == FBoolProperty::StaticClass()
				)
			{
				bCreateInput = true;
			}
			else if (PropertyClass == FStructProperty::StaticClass())
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(Property);
				UScriptStruct* Struct = StructProperty->Struct;
				if (Struct == TBaseStructure<FLinearColor>::Get()
					|| Struct == TBaseStructure<FVector4>::Get()
					|| Struct == TVariantStructure<FVector4d>::Get()
					|| Struct == TBaseStructure<FVector>::Get()
					|| Struct == TVariantStructure<FVector3f>::Get()
					|| Struct == TBaseStructure<FVector2D>::Get()
					)
				{
					bCreateInput = true;
				}
			}
		}
		if (bCreateInput)
		{
			PropertyInputs.Add(Property);
		}
	}

	return PropertyInputs;
}

void UMaterialExpression::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!GIsImportingT3D && !GIsTransacting && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive )
	{
		FPropertyChangedEvent SubPropertyChangedEvent(nullptr, PropertyChangedEvent.ChangeType);

		// Don't recompile the outer material if we are in the middle of a transaction or interactively changing properties
		// as there may be many expressions in the transaction buffer and we would just be recompiling over and over again.
		if (Material && !(Material->bIsPreviewMaterial || Material->bIsFunctionPreviewMaterial))
		{
			Material->PreEditChange(nullptr);
			Material->PostEditChangeProperty(SubPropertyChangedEvent);
		}
		else if (Function)
		{
			Function->PreEditChange(nullptr);
			Function->PostEditChangeProperty(SubPropertyChangedEvent);
		}
	}

	// PropertyChangedEvent.MemberProperty is the owner of PropertyChangedEvent.Property so check for MemberProperty
	FProperty* MemberPropertyThatChanged = PropertyChangedEvent.MemberProperty;
	if (MemberPropertyThatChanged != nullptr && GraphNode)
	{
		int32 PinIndex = -1;

		// Find the expression input this UPROPERTY points to with OverridingInputProperty meta data
		static FName OverridingInputPropertyMetaData("OverridingInputProperty");
		if (MemberPropertyThatChanged->HasMetaData(OverridingInputPropertyMetaData))
		{
			const FString& OverridingPropertyName = MemberPropertyThatChanged->GetMetaData(OverridingInputPropertyMetaData);
			FStructProperty* StructProp = FindFProperty<FStructProperty>(GetClass(), *OverridingPropertyName);
			if (StructProp)
			{
				const FExpressionInput* TargetInput = StructProp->ContainerPtrToValuePtr<FExpressionInput>(this);
				for (FExpressionInputIterator It{ this }; It; ++It)
				{
					if (TargetInput == It.Input)
					{
						PinIndex = It.Index;
						break;
					}
				}
			}
		}
		else
		{
			// Not found in explicit expression inputs, so search in property inputs.
			const int32 NumInputs = CountInputs();
			TArray<FProperty*> PropertyInputs = GetPropertyInputs();
			for (int32 i = 0; i < PropertyInputs.Num(); ++i)
			{
				if (MemberPropertyThatChanged->GetFName() == PropertyInputs[i]->GetFName())
				{
					PinIndex = NumInputs + i;
				}
			}
		}

		if (PinIndex > -1)
		{
			const FString& NewDefaultValue = GetInputPinDefaultValue(PinIndex);

			// Update the pin value of the expression input
			if (UEdGraphPin* Pin = GraphNode->GetPinAt(PinIndex))
			{
				Pin->Modify();
				Pin->DefaultValue = NewDefaultValue;

				// If this expression refers to a parameter, we need to keep the pin state in sync with all other nodes of the same type as this node.
				if (IsA<UMaterialExpressionParameter>())
				{
					// Remember this expression parameter name.
					const FName& ParameterName = static_cast<UMaterialExpressionParameter*>(this)->ParameterName;

					// Fetch all nodes in the material that refer to a parameter.
					TArray<UMaterialExpressionParameter*> ParameterExpressions;
					Material->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionParameter>(ParameterExpressions);

					for (UMaterialExpressionParameter* ExpressionParameter : ParameterExpressions)
					{
						// If the other expression type and parameter name are the same as this expression's...
						if (ExpressionParameter->GraphNode && ExpressionParameter->GetArchetype() == GetArchetype() && ExpressionParameter->ParameterName == ParameterName)
						{
							// ...modify the pin on other parameter expression node with the new value.
							UEdGraphPin* OtherPin = ExpressionParameter->GraphNode->GetPinAt(PinIndex);
							if (ensure(OtherPin && Pin && OtherPin->GetName() == Pin->GetName()))
							{
								OtherPin->Modify();
								OtherPin->DefaultValue = NewDefaultValue;
							}
						}
					}

					// Propagate he parameter value change so that it updates the other caches.
					// Note: since this could create a transaction, avoid creating a secondary nested transition.
					if (!GIsTransacting)
					{
						Material->PropagateExpressionParameterChanges(this);
					}
				}
			}

			// If the property is linked as inline toggle to another property, both pins need updating to reflect the change.
			bool bInlineEditConditionToggle = MemberPropertyThatChanged->HasMetaData(TEXT("InlineEditConditionToggle"));
			bool bEditCondition = MemberPropertyThatChanged->HasMetaData(TEXT("EditCondition"));
			if (bInlineEditConditionToggle || bEditCondition)
			{
				CastChecked<UMaterialGraphNode>(GraphNode)->RecreateAndLinkNode();
			}
		}
	}

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if( PropertyThatChanged != nullptr )
	{
		// Update the preview for this node if we adjusted a property
		bNeedToUpdatePreview = true;

		const FName PropertyName = PropertyThatChanged->GetFName();

		const FName ParameterName = "ParameterName";
		if (PropertyName == ParameterName)
		{
			ValidateParameterName();
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpression, Desc) && !IsA(UMaterialExpressionComment::StaticClass()))
		{
			if (GraphNode)
			{
				GraphNode->Modify();
				GraphNode->NodeComment = Desc;
			}
			// Don't need to update preview after changing description
			bNeedToUpdatePreview = false;
		}
	}
}

void UMaterialExpression::PostEditImport()
{
	Super::PostEditImport();

	UpdateParameterGuid(true, true);
}

bool UMaterialExpression::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		// Automatically set property as non-editable if it has OverridingInputProperty metadata
		// pointing to an FExpressionInput property which is hooked up as an input.
		//
		// e.g. in the below snippet, meta=(OverridingInputProperty = "A") indicates that ConstA will
		// be overridden by an FExpressionInput property named 'A' if one is connected, and will thereby
		// be set as non-editable.
		//
		//	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstA' if not specified"))
		//	FExpressionInput A;
		//
		//	UPROPERTY(EditAnywhere, Category = MaterialExpressionAdd, meta = (OverridingInputProperty = "A"))
		//	float ConstA;
		//

		static FName OverridingInputPropertyMetaData("OverridingInputProperty");

		if (InProperty->HasMetaData(OverridingInputPropertyMetaData))
		{
			const FString& OverridingPropertyName = InProperty->GetMetaData(OverridingInputPropertyMetaData);

			FStructProperty* StructProp = FindFProperty<FStructProperty>(GetClass(), *OverridingPropertyName);
			if (ensure(StructProp != nullptr))
			{
				static FName RequiredInputMetaData("RequiredInput");

				// Must be a single FExpressionInput member, not an array, and must be tagged with metadata RequiredInput="false"
				if (ensure(	StructProp->Struct->GetFName() == NAME_ExpressionInput &&
							StructProp->ArrayDim == 1 &&
							StructProp->HasMetaData(RequiredInputMetaData) &&
							!StructProp->GetBoolMetaData(RequiredInputMetaData)))
				{
					const FExpressionInput* Input = StructProp->ContainerPtrToValuePtr<FExpressionInput>(this);

					if (Input->Expression != nullptr && Input->GetTracedInput().Expression != nullptr)
					{
						bIsEditable = false;
					}
				}
			}
		}

		if (bIsEditable)
		{
			// If the property has EditCondition metadata, then whether it's editable depends on the other EditCondition property
			const FString EditConditionPropertyName = InProperty->GetMetaData(TEXT("EditCondition"));
			if (!EditConditionPropertyName.IsEmpty())
			{
				FBoolProperty* EditConditionProperty = FindFProperty<FBoolProperty>(GetClass(), *EditConditionPropertyName);
				if (EditConditionProperty)
				{
					bIsEditable = *EditConditionProperty->ContainerPtrToValuePtr<bool>(this);
				}
			}
		}
	}

	return bIsEditable;
}

TArray<FExpressionOutput>& UMaterialExpression::GetOutputs() 
{
	return Outputs;
}

TArrayView<FExpressionInput*> UMaterialExpression::GetInputsView()
{
	return CachedInputs;
}

int32 UMaterialExpression::CountInputs() const
{
	int32 Index = 0;
	while (const FExpressionInput* Input = GetInput(Index))
	{
		Index += 1;
	}
	return Index;
}

FExpressionInput* UMaterialExpression::GetInput(int32 InputIndex)
{
	return InputIndex < CachedInputs.Num() ? CachedInputs[InputIndex] : nullptr;
}

FName UMaterialExpression::GetInputName(int32 InputIndex) const
{
	int32 Index = 0;
	for (TFieldIterator<FStructProperty> InputIt(GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
	{
		FStructProperty* StructProp = *InputIt;
		if (StructProp->Struct->GetFName() == NAME_ExpressionInput)
		{
			for (int32 ArrayIndex = 0; ArrayIndex < StructProp->ArrayDim; ArrayIndex++)
			{
				if (Index == InputIndex)
				{
					FExpressionInput const* Input = StructProp->ContainerPtrToValuePtr<FExpressionInput>(this, ArrayIndex);
		
					if (!Input->InputName.IsNone())
					{
						return Input->InputName;
					}
					else
					{
						FName StructName = StructProp->GetFName();

						if (StructProp->ArrayDim > 1)
						{
							StructName = *FString::Printf(TEXT("%s_%d"), *StructName.ToString(), ArrayIndex);
						}

						return StructName;
					}
				}
				Index++;
			}
		}
	}
	return NAME_None;
}

FText UMaterialExpression::GetCreationDescription() const
{
	return FText::GetEmpty();
}

FText UMaterialExpression::GetCreationName() const
{
	return FText::GetEmpty();
}

void UMaterialExpression::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	if (Desc.Len() > 0)
	{
		if (GraphNode)
		{
			GraphNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString().ParseIntoArrayLines(OutToolTip, false);
		}

		TArray<FString> Multiline = TArray<FString>();
		Desc.ParseIntoArrayLines(Multiline, false);

		TArray<FString> CurrentLines = TArray<FString>();
		for (FString Line : Multiline)
		{
			if (Line.IsEmpty())
			{
				OutToolTip.Add(Line);
			}
			else
			{
				ConvertToMultilineToolTip(Line, 40, CurrentLines);
				OutToolTip.Append(CurrentLines);
			}
		}
	}
}

bool UMaterialExpression::IsInputConnectionRequired(int32 InputIndex) const
{
	int32 Index = 0;
	for( TFieldIterator<FStructProperty> InputIt(GetClass(), EFieldIteratorFlags::IncludeSuper,  EFieldIteratorFlags::ExcludeDeprecated) ; InputIt ; ++InputIt )
	{
		FStructProperty* StructProp = *InputIt;
		if( StructProp->Struct->GetFName() == NAME_ExpressionInput)
		{
			for (int32 ArrayIndex = 0; ArrayIndex < StructProp->ArrayDim; ArrayIndex++)
			{
				if( Index == InputIndex )
				{
					FExpressionInput const* Input = StructProp->ContainerPtrToValuePtr<FExpressionInput>(this, ArrayIndex);
					const TCHAR* MetaKey = TEXT("RequiredInput");

					if( StructProp->HasMetaData(MetaKey) )
					{
						return StructProp->GetBoolMetaData(MetaKey);
					}
				}
				Index++;
			}
		}
	}
	return true;
}

uint32 UMaterialExpression::GetInputType(int32 InputIndex)
{
	// different inputs should be defined by sub classed expressions
	return MCT_Float;
}

uint32 UMaterialExpression::GetOutputType(int32 OutputIndex)
{
	// different outputs should be defined by sub classed expressions 

	EMaterialValueType CompositeResultOutput = GetCompositeOutput(OutputIndex);
	if (CompositeResultOutput != MCT_Unknown)
	{
		return CompositeResultOutput;
	}

	TArray<FExpressionOutput>& CurrentOutputs = GetOutputs();
	if (!CurrentOutputs.IsValidIndex(OutputIndex))
	{
		return MCT_Unknown;
	}

	FExpressionOutput& Output = CurrentOutputs[OutputIndex];
	if (Output.Mask)
	{
		int32 MaskChannelCount = (Output.MaskR ? 1 : 0)
								+ (Output.MaskG ? 1 : 0)
								+ (Output.MaskB ? 1 : 0)
								+ (Output.MaskA ? 1 : 0);
		switch (MaskChannelCount)
		{
		case 1:
			return MCT_Float;
		case 2:
			return MCT_Float2;
		case 3:
			return MCT_Float3;
		case 4:
			return MCT_Float4;
		default:
			return MCT_Unknown;
		}
	}
	else
	{
		return MCT_Float;
	}
}

int32 UMaterialExpression::GetWidth() const
{
	return ME_STD_THUMBNAIL_SZ+(ME_STD_BORDER*2);
}

int32 UMaterialExpression::GetHeight() const
{
	return FMath::Max(ME_CAPTION_HEIGHT + (Outputs.Num() * ME_STD_TAB_HEIGHT),ME_CAPTION_HEIGHT+ME_STD_THUMBNAIL_SZ+(ME_STD_BORDER*2));
}


bool UMaterialExpression::UsesLeftGutter() const
{
	return 0;
}



bool UMaterialExpression::UsesRightGutter() const
{
	return 0;
}

void UMaterialExpression::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Expression"));
}

FString UMaterialExpression::GetDescription() const
{
	// Combined captions sufficient for most expressions
	TArray<FString> Captions;
	GetCaption(Captions);

	// Guard against GetCaption() implementations not populating the array
	if (Captions.IsEmpty())
	{
		UMaterialExpression::GetCaption(Captions);
		check(!Captions.IsEmpty());
	}

	if (Captions.Num() > 1)
	{
		FString Result = Captions[0];
		for (int32 Index = 1; Index < Captions.Num(); ++Index)
		{
			Result += TEXT(" ");
			Result += Captions[Index];
		}

		return Result;
	}
	else
	{
		return Captions[0];
	}
}

void UMaterialExpression::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (InputIndex != INDEX_NONE)
	{
		for( TFieldIterator<FStructProperty> InputIt(GetClass()) ; InputIt ; ++InputIt )
		{
			FStructProperty* StructProp = *InputIt;
			if( StructProp->Struct->GetFName() == NAME_ExpressionInput )
			{
				for (int32 ArrayIndex = 0; ArrayIndex < StructProp->ArrayDim; ArrayIndex++)
				{
					if (!InputIndex)
					{
						if (StructProp->HasMetaData(TEXT("tooltip")))
						{
							// Set the tooltip from the .h comments
							ConvertToMultilineToolTip(StructProp->GetToolTipText().ToString(), 40, OutToolTip);
						}
						return;
					}
					InputIndex--;
				}
			}
		}
	}
}

int32 UMaterialExpression::CompilerError(FMaterialCompiler* Compiler, const TCHAR* pcMessage)
{
	TArray<FString> Captions;
	GetCaption(Captions);
	return Compiler->Errorf(TEXT("%s> %s"), Desc.Len() > 0 ? *Desc : *Captions[0], pcMessage);
}

bool UMaterialExpression::Modify( bool bAlwaysMarkDirty/*=true*/ )
{
	bNeedToUpdatePreview = true;
	
	return Super::Modify(bAlwaysMarkDirty);
}

bool UMaterialExpression::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if (FCString::Stristr(SearchQuery, TEXT("NAME=")) != nullptr)
	{
		FString SearchString(SearchQuery);
		SearchString.RightInline(SearchString.Len() - 5, EAllowShrinking::No);
		return (GetName().Contains(SearchString) );
	}
	return Desc.Contains(SearchQuery);
}

bool UMaterialExpression::DeepSearchQuery(const TCHAR* SearchQuery, TSharedPtr<FFindInMaterialResult> SearchResult)
{
	//Unlike the matches call above, we always want to check the expression names in a deep search.
	return GetName().Contains(SearchQuery) || MatchesSearchQuery(SearchQuery);
}

bool UMaterialExpression::IsExpressionConnected( FExpressionInput* Input, int32 OutputIndex )
{
	return Input->OutputIndex == OutputIndex && Input->Expression == this;
}

void UMaterialExpression::ConnectExpression( FExpressionInput* Input, int32 OutputIndex )
{
	if( Input && OutputIndex >= 0 && OutputIndex < Outputs.Num() )
	{
		FExpressionOutput& Output = Outputs[OutputIndex];
		Input->Expression = this;
		Input->OutputIndex = OutputIndex;
		Input->Mask = Output.Mask;
		Input->MaskR = Output.MaskR;
		Input->MaskG = Output.MaskG;
		Input->MaskB = Output.MaskB;
		Input->MaskA = Output.MaskA;
	}
}
#endif // WITH_EDITOR

void UMaterialExpression::UpdateMaterialExpressionGuid(bool bForceGeneration, bool bAllowMarkingPackageDirty)
{
	// If we are in the editor, and we don't have a valid GUID yet, generate one.
	if (GIsEditor && !IsRunningGame() && !GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		FGuid& Guid = GetMaterialExpressionId();

		if (bForceGeneration || !Guid.IsValid())
		{
			Guid = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::Expression);

			if (bAllowMarkingPackageDirty)
			{
				MarkPackageDirty();
			}
		}
	}
}


void UMaterialExpression::UpdateParameterGuid(bool bForceGeneration, bool bAllowMarkingPackageDirty)
{
	if (bIsParameterExpression)
	{
		// If we are in the editor, and we don't have a valid GUID yet, generate one.
		if (GIsEditor && !IsRunningGame())
		{
			if (GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
			{
				return;
			}

			FGuid& Guid = GetParameterExpressionId();

			if (bForceGeneration || !Guid.IsValid())
			{
				Guid = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::Parameter);

				if (bAllowMarkingPackageDirty)
				{
					MarkPackageDirty();
				}
			}
		}
	}
}

#if WITH_EDITOR

// A non public function to know if a material is a preview material that needs to be unlit.
// This is required for instance when previewing a material function using the UI domain. It needs unlit to fetch the right data and Substrate also uses unlit behind the scene to display that domain.
// Also this is used to maintain the correct MaterialDomain since Substrate can convert that otherwise in RebuildShadingModelField. 
// When in fact we must keep that for the ExpressionPreviewMaterial created from MaterialEditor.cpp for MAterialFunction asset preview for instance.
bool IsPreviewUnlitMaterial(const UMaterial* InMaterial)
{
	return InMaterial && InMaterial->MaterialDomain != MD_Surface && InMaterial->bIsPreviewMaterial && InMaterial->IsA(UPreviewMaterial::StaticClass());
}

void UMaterialExpression::ConnectToPreviewMaterial(UMaterial* InMaterial, int32 OutputIndex)
{
	// This is used when a node is right clicked and "Start previewing node" is used.
	if (InMaterial && OutputIndex >= 0 && OutputIndex < Outputs.Num())
	{
		if (Substrate::IsSubstrateEnabled())
		{
			if (IsResultSubstrateMaterial(0))
			{
				if (IsPreviewUnlitMaterial(InMaterial))
				{
					InMaterial->SetShadingModel(MSM_Unlit);
				}
				else
				{
					InMaterial->SetShadingModel(MSM_DefaultLit);
				}
				InMaterial->bUseMaterialAttributes = false;
				FExpressionInput* MaterialInput = InMaterial->GetExpressionInputForProperty(MP_FrontMaterial);
				check(MaterialInput);
				ConnectExpression(MaterialInput, OutputIndex);
			}
			else if (IsResultMaterialAttributes(0))
			{
				// Propagate material attributes to MaterialAttributes input
				if (!InMaterial->IsShadingModelFromMaterialExpression())
				{
					// Only set DefaultLit if we are not using the special FromMaterialExpression already as this is forbidden by SetShadingModel and Substrate deal with that case.
					InMaterial->SetShadingModel(MSM_DefaultLit);
					InMaterial->bUseMaterialAttributes = false;
				}
				else
				{
					InMaterial->bUseMaterialAttributes = true;
				}
				FExpressionInput* MaterialInput = InMaterial->GetExpressionInputForProperty(MP_MaterialAttributes);
				check(MaterialInput);
				ConnectExpression( MaterialInput, OutputIndex );

				// Converte material input into Substrate data
				UMaterialExpressionSubstrateConvertMaterialAttributes* ConvertAttributeNode = NewObject<UMaterialExpressionSubstrateConvertMaterialAttributes>(this);
				ConvertAttributeNode->Material = InMaterial;
				ConvertAttributeNode->MaterialAttributes.Connect(OutputIndex, this);
				if (IsPreviewUnlitMaterial(InMaterial))
				{
					ConvertAttributeNode->ShadingModelOverride = MSM_Unlit;
				}
				else
				{
					ConvertAttributeNode->ShadingModelOverride = MSM_DefaultLit;
				}

				// Connect substrate data into material FrontMaterial input
				if (UMaterialEditorOnlyData* MaterialEditorOnlyData = InMaterial->GetEditorOnlyData())
				{
					MaterialEditorOnlyData->FrontMaterial.Connect(0, ConvertAttributeNode);
				}
			}
			else
			{
				InMaterial->SetShadingModel(MSM_Unlit);
				UMaterialExpressionSubstrateUnlitBSDF* UnlitBSDF = NewObject<UMaterialExpressionSubstrateUnlitBSDF>(this);
				UnlitBSDF->EmissiveColor.Connect(OutputIndex, this);

				FExpressionInput* MaterialInput = InMaterial->GetExpressionInputForProperty(MP_FrontMaterial);
				check(MaterialInput);
				MaterialInput->Connect(0, UnlitBSDF);
			}
		}
		else if(IsResultMaterialAttributes(0))
		{
			InMaterial->SetShadingModel(MSM_DefaultLit);
			InMaterial->bUseMaterialAttributes = true;
			FExpressionInput* MaterialInput = InMaterial->GetExpressionInputForProperty(MP_MaterialAttributes);
			check(MaterialInput);
			ConnectExpression( MaterialInput, OutputIndex );
		}
		else
		{
			InMaterial->SetShadingModel(MSM_Unlit);
			InMaterial->bUseMaterialAttributes = false;

			// Connect the selected expression to the emissive node of the expression preview material.  The emissive material is not affected by light which is why its a good choice.
			FExpressionInput* MaterialInput = InMaterial->GetExpressionInputForProperty(MP_EmissiveColor);
			check(MaterialInput);
			ConnectExpression( MaterialInput, OutputIndex );
		}
	}
}
#endif // WITH_EDITOR

void UMaterialExpression::ValidateState()
{
}

#if WITH_EDITOR

bool UMaterialExpression::ShouldShowPreview() const
{
	return !bHidePreviewWindow && !bCollapsed;
}

bool UMaterialExpression::GetAllInputExpressions(TArray<UMaterialExpression*>& InputExpressions)
{
	// Make sure we don't end up in a loop
	if (!InputExpressions.Contains(this))
	{
		bool bFoundRepeat = false;
		InputExpressions.Add(this);

		for (FExpressionInputIterator It{ this }; It; ++It)
		{
			if (It->Expression && It->Expression->GetAllInputExpressions(InputExpressions))
			{
				bFoundRepeat = true;
			}
		}

		return bFoundRepeat;
	}
	else
	{
		return true;
	}
}

bool UMaterialExpression::CanRenameNode() const
{
	return false;
}

FString UMaterialExpression::GetEditableName() const
{
	// This function is only safe to call in a class that has implemented CanRenameNode() to return true
	check(false);
	return TEXT("");
}

void UMaterialExpression::SetEditableName(const FString& NewName)
{
	// This function is only safe to call in a class that has implemented CanRenameNode() to return true
	check(false);
}

EMaterialParameterType UMaterialExpression::GetParameterType() const
{
	FMaterialParameterMetadata Meta;
	if (GetParameterValue(Meta))
	{
		return Meta.Value.Type;
	}
	return EMaterialParameterType::None;
}

void UMaterialExpression::ValidateParameterName(const bool bAllowDuplicateName)
{
	// Incrementing the name is now handled in UMaterialExpressionParameter::ValidateParameterName
}

bool UMaterialExpression::HasClassAndNameCollision(UMaterialExpression* OtherExpression) const
{
	return GetClass() == OtherExpression->GetClass();
}


bool UMaterialExpression::HasConnectedOutputs() const
{
	bool bIsConnected = !GraphNode;
	if (GraphNode)
	{
		UMaterialGraphNode* MatGraphNode = Cast<UMaterialGraphNode>(GraphNode);
		if (MatGraphNode)
		{
			for (UEdGraphPin* Pin : MatGraphNode->Pins)
			{
				if (Pin->Direction == EGPD_Output &&
					Pin->LinkedTo.Num() > 0)
				{
					bIsConnected = true;
					break;
				}
			}
		}
	}
	return bIsConnected;
}

struct UMaterialExpression::FContainsInputLoopInternalExpressionStack
{
	const UMaterialExpression* Expression;
	const FContainsInputLoopInternalExpressionStack* Previous;

	FContainsInputLoopInternalExpressionStack(const UMaterialExpression* Expression, const FContainsInputLoopInternalExpressionStack* Previous)
		: Expression{ Expression }
		, Previous{ Previous }
	{}

	bool Contains(const UMaterialExpression* OtherExpression) const
	{
		const FContainsInputLoopInternalExpressionStack* Node = this;
		while (Node->Expression)
		{
			if (Node->Expression == OtherExpression)
			{
				return true;
			}
			Node = Node->Previous;
		}
		return false;
	}
};

bool UMaterialExpression::ContainsInputLoop(const bool bStopOnFunctionCall /*= true*/)
{
	FContainsInputLoopInternalExpressionStack ExpressionStack{ nullptr, nullptr };
	TSet<UMaterialExpression*> VisitedExpressions;
	return ContainsInputLoopInternal(ExpressionStack, VisitedExpressions, bStopOnFunctionCall);
}

bool UMaterialExpression::ContainsInputLoop(TSet<UMaterialExpression*>& VisitedExpressions, const bool bStopOnFunctionCall)
{
	if (VisitedExpressions.Contains(this))
	{
		return false;
	}
	FContainsInputLoopInternalExpressionStack ExpressionStack{ nullptr, nullptr };
	return ContainsInputLoopInternal(ExpressionStack, VisitedExpressions, bStopOnFunctionCall);
}

bool UMaterialExpression::ContainsInputLoopInternal(const FContainsInputLoopInternalExpressionStack& ExpressionStack, TSet<UMaterialExpression*>& VisitedExpressions, const bool bStopOnFunctionCall)
{
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		UMaterialExpression* InputExpression = It->Expression;
		if (!InputExpression)
		{
			continue;
		}

		// ContainsInputLoop primarily used to detect safe traversal path for IsResultMaterialAttributes.
		// In those cases we can bail on a function as the inputs are strongly typed
		UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(InputExpression);
		UMaterialExpressionMaterialAttributeLayers* Layers = Cast<UMaterialExpressionMaterialAttributeLayers>(InputExpression);
		if (bStopOnFunctionCall && (FunctionCall || Layers))
		{
			continue;
		}

		// A cycle is detected if one of this node's inputs leads back to a node we're coming from.
		if (ExpressionStack.Contains(InputExpression))
		{
			return true;
		}

		// Add this expression to the visited set. If it was already there, we do not need to explore it again.
		bool bAlreadyVisited = false;
		VisitedExpressions.Add(InputExpression, &bAlreadyVisited);
		if (bAlreadyVisited)
		{
			continue;
		}

		// Push this expression onto the stack and carry on crawling through this expression.
		FContainsInputLoopInternalExpressionStack ExpressionStackWithThisInput{ InputExpression, &ExpressionStack };
		if (InputExpression->ContainsInputLoopInternal(ExpressionStackWithThisInput, VisitedExpressions, bStopOnFunctionCall))
		{
			return true;
		}
	}

	return false;
}

bool UMaterialExpression::IsUsingNewHLSLGenerator() const
{
	if (Material)
	{
		return Material->IsUsingNewHLSLGenerator();
	}
	if (Function)
	{
		return Function->IsUsingNewHLSLGenerator();
	}
	return false;
}

FSubstrateOperator* UMaterialExpression::SubstrateGenerateMaterialTopologyTree(class FMaterialCompiler* Compiler, class UMaterialExpression* Parent, int32 OutputIndex)
{
	// Deprecated so call the general version.
	return SubstrateGenerateMaterialTopologyTree(Compiler->GetSubstrateTranslatorData(), Parent, OutputIndex);
}

FSubstrateOperator* UMaterialExpression::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	SubstrateTranslatorData.Errorf(FString::Printf(TEXT("%s nodes type does not support generating/processing/flowing Substrate data.\nPlease reach out to your development team for implementation, if it is supported and can be implemented."), *GetClass()->GetName()));
	return nullptr;
}

#endif // WITH_EDITOR

bool UMaterialExpression::IsAllowedIn(const UObject* MaterialOrFunction) const
{
	// Custom HLSL expressions are not allowed for client generated materials in certain UE editor configuration
	return IsExpressionClassPermitted(GetClass());
}

#if WITH_EDITOR
void UMaterialExpressionTextureBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty( PropertyChangedEvent );

	if (IsDefaultMeshpaintTexture && PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, IsDefaultMeshpaintTexture))
	{
		// Material is null when this expression lives inside a UMaterialFunction; the mesh paint default
		// only applies within a concrete UMaterial, so skip the duplicate-default check in that case.
		if (!Material)
		{
			return;
		}

		// Check for other defaulted textures in THIS material (does not search sub levels ie functions etc, as these are ignored in the texture painter).
		for (UMaterialExpression* Expression : this->Material->GetMaterial()->GetExpressions())
		{
			UMaterialExpressionTextureBase* TextureSample = Cast<UMaterialExpressionTextureBase>(Expression);
			if (TextureSample != nullptr && TextureSample != this)
			{
				if(TextureSample->IsDefaultMeshpaintTexture)
				{
					FText ErrorMessage = LOCTEXT("MeshPaintDefaultTextureErrorDefault","Only one texture can be set as the Mesh Paint Default Texture, disabling previous default");
					if (TextureSample->Texture != nullptr)
					{
						FFormatNamedArguments Args;
						Args.Add( TEXT("TextureName"), FText::FromString( TextureSample->Texture->GetName() ) );
						ErrorMessage = FText::Format(LOCTEXT("MeshPaintDefaultTextureErrorTextureKnown","Only one texture can be set as the Mesh Paint Default Texture, disabling {TextureName}"), Args );
					}
										
					// Launch notification to inform user of default change
					FNotificationInfo Info( ErrorMessage );
					Info.ExpireDuration = 5.0f;
					Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));

					FSlateNotificationManager::Get().AddNotification(Info);

					// Reset the previous default to false;
					TextureSample->IsDefaultMeshpaintTexture = false;
				}
			}
		}
	}
}

FString UMaterialExpressionTextureBase::GetDescription() const
{
	FString Result = Super::GetDescription();
	Result += TEXT(" (");
	Result += Texture ? Texture->GetName() : TEXT("None");
	Result += TEXT(")");

	return Result;
}

bool UMaterialExpressionTextureBase::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (Texture != nullptr && Texture->GetName().Contains(SearchQuery))
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

FText UMaterialExpressionTextureBase::GetPreviewOverlayText() const
{
	if (IsVirtualSamplerType(SamplerType))
	{
		return LOCTEXT("VT", "VT");
	}
	else
	{
		return FText();
	}
}

void UMaterialExpressionTextureBase::AutoSetSampleType()
{
	if (Texture)
	{
		SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture(Texture);
	}
}

EMaterialSamplerType UMaterialExpressionTextureBase::GetSamplerTypeForTexture(const UTexture* Texture, bool ForceNoVT)
{
	return MaterialExpressionUtils::GetSamplerTypeForTexture(Texture, ForceNoVT);
}

#endif // WITH_EDITOR

UObject* UMaterialExpressionTextureBase::GetReferencedTexture() const
{
	return Texture;
}

UMaterialExpressionTextureSample::UMaterialExpressionTextureSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("R"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("G"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("B"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));

	bShowOutputNameOnPin = true;
	bShowTextureInputPin = true;
	bCollapsed = false;

	MipValueMode = TMVM_None;
	ConstCoordinate = 0;
	ConstMipValue = INDEX_NONE;
	AutomaticViewMipBias = true;

	ApplyChannelNames();
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
bool UMaterialExpressionTextureSample::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		FName PropertyFName = InProperty->GetFName();

		if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSample, ConstMipValue))
		{
			bIsEditable = MipValueMode == TMVM_MipLevel || MipValueMode == TMVM_MipBias;
		}
		else if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSample, ConstCoordinate))
		{
			bIsEditable = !Coordinates.GetTracedInput().Expression;
		}
		else if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSample, Texture))
		{
			// The Texture property is overridden by a connection to TextureObject
			bIsEditable = TextureObject.GetTracedInput().Expression == nullptr;
		}
		else if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureSample, AutomaticViewMipBias))
		{
			bIsEditable = AutomaticViewMipBiasValue.GetTracedInput().Expression == nullptr;
		}
	}

	return bIsEditable;
}

void UMaterialExpressionTextureSample::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if ( PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Texture) )
	{
		if ( Texture )
		{
			AutoSetSampleType();
			FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
		}
	}

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, MipValueMode))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, ChannelNames))
	{
		ApplyChannelNames();

		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}
	
	// Need to update expression properties before super call (which triggers recompile)
	Super::PostEditChangeProperty( PropertyChangedEvent );	
}

void UMaterialExpressionTextureSample::PostLoad()
{
	Super::PostLoad();

	// Clear invalid input reference
	if (!bShowTextureInputPin && TextureObject.Expression)
	{
		TextureObject.Expression = nullptr;
	}
}

void UMaterialExpressionTextureSample::ApplyChannelNames()
{
	static const FName Red("R");
	static const FName Green("G");
	static const FName Blue("B");
	static const FName Alpha("A");
	if (GetOutputValueType(0) != MCT_Texture)
	{
		Outputs[1].OutputName = !ChannelNames.R.IsEmpty() ? FName(*ChannelNames.R.ToString()) : Red;
		Outputs[2].OutputName = !ChannelNames.G.IsEmpty() ? FName(*ChannelNames.G.ToString()) : Green;
		Outputs[3].OutputName = !ChannelNames.B.IsEmpty() ? FName(*ChannelNames.B.ToString()) : Blue;
		Outputs[4].OutputName = !ChannelNames.A.IsEmpty() ? FName(*ChannelNames.A.ToString()) : Alpha;
	}
}

TArrayView<FExpressionInput*> UMaterialExpressionTextureSample::GetInputsView()
{
	CachedInputs.Empty();
	uint32 InputIndex = 0;
	while (FExpressionInput* Ptr = GetInput(InputIndex++))
	{
		CachedInputs.Add(Ptr);
	}
	return CachedInputs;
}

// this define is only used for the following function
#define IF_INPUT_RETURN(Item) if(!InputIndex) return &Item; --InputIndex
FExpressionInput* UMaterialExpressionTextureSample::GetInput(int32 InputIndex)
{
	IF_INPUT_RETURN(Coordinates);

	if (bShowTextureInputPin)
	{
		IF_INPUT_RETURN(TextureObject);
	}

	if (MipValueMode == TMVM_Derivative)
	{
		IF_INPUT_RETURN(CoordinatesDX);
		IF_INPUT_RETURN(CoordinatesDY);
	}
	else if(MipValueMode != TMVM_None)
	{
		IF_INPUT_RETURN(MipValue);
	}

	IF_INPUT_RETURN(AutomaticViewMipBiasValue);

	return nullptr;
}
#undef IF_INPUT_RETURN

// this define is only used for the following function
#define IF_INPUT_RETURN(Name) if(!InputIndex) return Name; --InputIndex
FName UMaterialExpressionTextureSample::GetInputName(int32 InputIndex) const
{
	// Coordinates
	IF_INPUT_RETURN(TEXT("Coordinates"));

	if (bShowTextureInputPin)
	{
		// TextureObject
		IF_INPUT_RETURN(TEXT("TextureObject"));
	}

	if(MipValueMode == TMVM_MipLevel)
	{
		// MipValue
		IF_INPUT_RETURN(TEXT("MipLevel"));
	}
	else if(MipValueMode == TMVM_MipBias)
	{
		// MipValue
		IF_INPUT_RETURN(TEXT("MipBias"));
	}
	else if(MipValueMode == TMVM_Derivative)
	{
		// CoordinatesDX
		IF_INPUT_RETURN(TEXT("DDX(UVs)"));
		// CoordinatesDY
		IF_INPUT_RETURN(TEXT("DDY(UVs)"));
	}

	// AutomaticViewMipBiasValue
	IF_INPUT_RETURN(TEXT("Apply View MipBias"));

	return TEXT("");
}
#undef IF_INPUT_RETURN

bool UMaterialExpressionTextureBase::VerifySamplerType(const FString& TexturePathName, EMaterialSamplerType CorrectSamplerType, bool bSRGB, EMaterialSamplerType SamplerType, FString& OutErrorMessage)
{
	return MaterialExpressionUtils::VerifySamplerType(TexturePathName, CorrectSamplerType, bSRGB, SamplerType, OutErrorMessage);
}

bool UMaterialExpressionTextureBase::VerifySamplerType(EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform, const UTexture* Texture, EMaterialSamplerType SamplerType, FString& OutErrorMessage)
{
	return MaterialExpressionUtils::VerifySamplerType(ShaderPlatform, TargetPlatform, Texture, SamplerType, OutErrorMessage);
}

int32 UMaterialExpressionTextureSample::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	UMaterialExpression* InputExpression = TextureObject.GetTracedInput().Expression;

	if (Texture || InputExpression) // We deal with reroute textures later on in this function..
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		int32 TextureCodeIndex = INDEX_NONE;

		bool bDoAutomaticViewMipBias = AutomaticViewMipBias;
		if (AutomaticViewMipBiasValue.GetTracedInput().Expression)
		{
			bool bSucceeded;
			bool bValue = Compiler->GetStaticBoolValue(AutomaticViewMipBiasValue.Compile(Compiler), bSucceeded);

			if (bSucceeded)
			{
				bDoAutomaticViewMipBias = bValue;
			}
		}

		if (InputExpression)
		{
			TextureCodeIndex = TextureObject.Compile(Compiler);
		}
		else if (SamplerType == SAMPLERTYPE_External)
		{
			TextureCodeIndex = Compiler->ExternalTexture(Texture, TextureReferenceIndex);
		}
		else
		{
			TextureCodeIndex = Compiler->Texture(Texture, TextureReferenceIndex, SamplerType, SamplerSource, MipValueMode);
		}

		if (TextureCodeIndex == INDEX_NONE)
		{
			// Can't continue without a texture to sample
			return INDEX_NONE;
		}

		EMaterialValueType TextureType = Compiler->GetParameterType(TextureCodeIndex);

		auto CheckForMissingUVWInput = [this, Compiler, TextureType](int32 ExpressionInput) -> TOptional<int32>
		{
			const EMaterialValueType TypesToCheck = EMaterialValueType(MCT_TextureCube | MCT_VolumeTexture | MCT_Texture2DArray | MCT_TextureCubeArray);
			if (ExpressionInput != INDEX_NONE && (TextureType & TypesToCheck) != 0 && !Coordinates.GetTracedInput().Expression)
			{
				if (TextureType == MCT_TextureCube)
				{
					return CompilerError(Compiler, TEXT("UVW input required for cubemap sample"));
				}
				else if (TextureType == MCT_VolumeTexture)
				{
					return CompilerError(Compiler, TEXT("UVW input required for volume sample"));
				}
				else if (TextureType == MCT_Texture2DArray)
				{
					return CompilerError(Compiler, TEXT("UVW input required for texturearray sample"));
				}
				else if (TextureType == MCT_TextureCubeArray)
				{
					return CompilerError(Compiler, TEXT("UVWX input required for texturecubearray sample"));
				}
			}

			return TOptional<int32>();
		};

		auto GetCoordinateIndex = [this, Compiler](int32 ExpressionInput, EMaterialSamplerType EffectiveSamplerType, const TOptional<FName>& EffectiveParameterName)
		{
			int32 CoordinateIndex = Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);

			// If the sampler type is an external texture, we have might have a scale/bias to apply to the UV coordinates.
			// Generate that code for the TextureReferenceIndex here so we compile it using the correct texture based on possible reroute textures above
			if (EffectiveSamplerType == SAMPLERTYPE_External)
			{
				CoordinateIndex = CompileExternalTextureCoordinates(Compiler, CoordinateIndex, ExpressionInput, EffectiveParameterName);
			}

			return CoordinateIndex;
		};

		if ((TextureType & MCT_TextureCollection) != 0 && SamplerSource == SSM_FromTextureAsset)
		{
			return CompilerError(Compiler, TEXT("Texture Collections do not provide a sampler, please choose something other than 'From texture asset'"));
		}

		if (TextureType & (MCT_TextureCollection | MCT_TextureMeshPaint))
		{
			// There's no UTexture object to get here

			if (TOptional<int32> MissingError = CheckForMissingUVWInput(TextureCodeIndex))
			{
				return *MissingError;
			}

			const int32 CoordinateIndex = GetCoordinateIndex(TextureReferenceIndex, SamplerType, {});

			return Compiler->TextureSample(
				TextureCodeIndex,
				CoordinateIndex,
				SamplerType,
				CompileMipValue0(Compiler),
				CompileMipValue1(Compiler),
				MipValueMode,
				SamplerSource,
				GatherMode,
				TextureReferenceIndex,
				bDoAutomaticViewMipBias);
		}

		UTexture* EffectiveTexture = Texture;
		EMaterialSamplerType EffectiveSamplerType = SamplerType;
		TOptional<FName> EffectiveParameterName;
		if (InputExpression)
		{
			if (!Compiler->GetTextureForExpression(TextureCodeIndex, TextureReferenceIndex, EffectiveSamplerType, EffectiveParameterName))
			{
				return CompilerError(Compiler, TEXT("Tex input requires a texture value"));
			}
			if (TextureReferenceIndex != INDEX_NONE)
			{
				EffectiveTexture = Cast<UTexture>(Compiler->GetReferencedTexture(TextureReferenceIndex));
			}
		}

		FString SamplerTypeError;
		if (EffectiveTexture && MaterialExpressionUtils::VerifySamplerType(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform(), EffectiveTexture, EffectiveSamplerType, SamplerTypeError))
		{
			if (TOptional<int32> MissingError = CheckForMissingUVWInput(TextureCodeIndex))
			{
				return *MissingError;
			}

			const int32 CoordinateIndex = GetCoordinateIndex(TextureReferenceIndex, EffectiveSamplerType, EffectiveParameterName);

			return Compiler->TextureSample(
				TextureCodeIndex,
				CoordinateIndex,
				EffectiveSamplerType,
				CompileMipValue0(Compiler),
				CompileMipValue1(Compiler),
				MipValueMode,
				SamplerSource,
				GatherMode,
				TextureReferenceIndex,
				bDoAutomaticViewMipBias);
		}
		else
		{
			// TextureObject.Expression is responsible for generating the error message, since it had a nullptr texture value
			return Compiler->Errorf(TEXT("%s"), *SamplerTypeError);
		}
	}
	else
	{
		return CompilerError(Compiler, TEXT("Missing input texture"));
	}
}

int32 UMaterialExpressionTextureSample::GetWidth() const
{
	return ME_STD_THUMBNAIL_SZ+(ME_STD_BORDER*2);
}

#if WITH_EDITOR
#define IF_INPUT_RETURN(Value) if(!InputIndex) return (Value); --InputIndex
int32 GetAbsoluteIndex(int32 InputIndex, const bool bShowTextureInputPin, const TEnumAsByte<enum ETextureMipValueMode>& MipValueMode)
{
	// Coordinates
	IF_INPUT_RETURN(0);
	if (bShowTextureInputPin)
	{
		// TextureObject
		IF_INPUT_RETURN(1);
	}
	if(MipValueMode == TMVM_Derivative)
	{
		// CoordinatesDX
		IF_INPUT_RETURN(3);
		// CoordinatesDY
		IF_INPUT_RETURN(4);
	}
	else if(MipValueMode != TMVM_None)
	{
		// MipValue
		IF_INPUT_RETURN(2);
	}
	// AutomaticViewMipBiasValue
	IF_INPUT_RETURN(5);
	// If not found
	return INDEX_NONE;
}
#undef IF_INPUT_RETURN

void UMaterialExpressionTextureSample::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	const int32 AbsoluteIndex = GetAbsoluteIndex(InputIndex, bShowTextureInputPin, MipValueMode);
	Super::GetConnectorToolTip(AbsoluteIndex, OutputIndex, OutToolTip);
}
#endif // WITH_EDITOR

void UMaterialExpressionTextureSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Texture Sample"));
}

// this define is only used for the following function
#define IF_INPUT_RETURN(Type) if(!InputIndex) return (Type); --InputIndex

EMaterialValueType UMaterialExpressionTextureSample::GetInputValueType(int32 InputIndex)
{
	// Coordinates
	IF_INPUT_RETURN(MCT_Float);

	if (bShowTextureInputPin)
	{
		// TextureObject
		// TODO: Only show the TextureObject input inside a material function, since that's the only place it is useful
		IF_INPUT_RETURN(MCT_Texture);
	}
	
	if(MipValueMode == TMVM_MipLevel || MipValueMode == TMVM_MipBias)
	{
		// MipValue
		IF_INPUT_RETURN(MCT_Float);
	}
	else if(MipValueMode == TMVM_Derivative)
	{
		// CoordinatesDX
		IF_INPUT_RETURN(MCT_Float);
		// CoordinatesDY
		IF_INPUT_RETURN(MCT_Float);
	}

	// AutomaticViewMipBiasValue
	IF_INPUT_RETURN(MCT_StaticBool);

	return MCT_Unknown;
}
#undef IF_INPUT_RETURN

int32 UMaterialExpressionTextureSample::CompileMipValue0(class FMaterialCompiler* Compiler)
{
	if (MipValueMode == TMVM_Derivative)
	{
		if (CoordinatesDX.GetTracedInput().IsConnected())
		{
			return CoordinatesDX.Compile(Compiler);
		}
	}
	else if (MipValue.GetTracedInput().IsConnected())
	{
		return MipValue.Compile(Compiler);
	}
	else
	{
		return Compiler->Constant(ConstMipValue);
	}

	return INDEX_NONE;
}

int32 UMaterialExpressionTextureSample::CompileMipValue1(class FMaterialCompiler* Compiler)
{
	if (MipValueMode == TMVM_Derivative && CoordinatesDY.GetTracedInput().IsConnected())
	{
		return CoordinatesDY.Compile(Compiler);
	}

	return INDEX_NONE;
}
#endif // WITH_EDITOR

UMaterialExpressionRuntimeVirtualTextureOutput::UMaterialExpressionRuntimeVirtualTextureOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionRuntimeVirtualTextureOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 Result = INDEX_NONE;
	uint8 OutputAttributeMask = 0;

	Compiler->PushRuntimeVirtualTextureOutput();
	{
		int32 CodeInput = INDEX_NONE;

		// Order of outputs generates function names GetVirtualTextureOutput{index}
		// These must match the function names called in VirtualTextureMaterial.usf
		if (OutputIndex == 0)
		{
			CodeInput = BaseColor.IsConnected() ? BaseColor.Compile(Compiler) : Compiler->Constant3(0.f, 0.f, 0.f);
			OutputAttributeMask |= BaseColor.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::BaseColor) : 0;
		}
		else if (OutputIndex == 1)
		{
			CodeInput = Specular.IsConnected() ? Specular.Compile(Compiler) : Compiler->Constant(0.5f);
			OutputAttributeMask |= Specular.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::Specular) : 0;
		}
		else if (OutputIndex == 2)
		{
			CodeInput = Roughness.IsConnected() ? Roughness.Compile(Compiler) : Compiler->Constant(0.5f);
			OutputAttributeMask |= Roughness.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::Roughness) : 0;
		}
		else if (OutputIndex == 3)
		{
			CodeInput = Normal.IsConnected() ? Normal.Compile(Compiler) : Compiler->Constant3(0.f, 0.f, 1.f);
			OutputAttributeMask |= Normal.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::Normal) : 0;
		}
		else if (OutputIndex == 4)
		{
			CodeInput = WorldHeight.IsConnected() ? WorldHeight.Compile(Compiler) : Compiler->Constant(0.f);
			OutputAttributeMask |= WorldHeight.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::WorldHeight) : 0;
		}
		else if (OutputIndex == 5)
		{
			CodeInput = Opacity.IsConnected() ? Opacity.Compile(Compiler) : Compiler->Constant(1.f);
		}
		else if (OutputIndex == 6)
		{
			CodeInput = Mask.IsConnected() ? Mask.Compile(Compiler) : Compiler->Constant(1.f);
			OutputAttributeMask |= Mask.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::Mask) : 0;
		}
		else if (OutputIndex == 7)
		{
			CodeInput = Displacement.IsConnected() ? Displacement.Compile(Compiler) : Compiler->Constant(0.f);
			OutputAttributeMask |= Displacement.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::Displacement) : 0;
		}
		else if (OutputIndex == 8)
		{
			CodeInput = Mask4.IsConnected() ? Mask4.Compile(Compiler) : Compiler->Constant4(0.f, 0.f, 0.f, 0.f);
			OutputAttributeMask |= Mask4.IsConnected() ? (1 << (uint8)ERuntimeVirtualTextureAttributeType::Mask4) : 0;
		}

		Result = Compiler->CustomOutput(this, OutputIndex, CodeInput);
	}
	Compiler->PopRuntimeVirtualTextureOutput(OutputAttributeMask);

	return Result;
}

void UMaterialExpressionRuntimeVirtualTextureOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Runtime Virtual Texture Output"));
}

#endif // WITH_EDITOR

int32 UMaterialExpressionRuntimeVirtualTextureOutput::GetNumOutputs() const
{
	return 9; 
}

FString UMaterialExpressionRuntimeVirtualTextureOutput::GetFunctionName() const
{
	return TEXT("GetVirtualTextureOutput"); 
}

FString UMaterialExpressionRuntimeVirtualTextureOutput::GetDisplayName() const
{
	return TEXT("Runtime Virtual Texture"); 
}

UMaterialExpressionRuntimeVirtualTextureSample::UMaterialExpressionRuntimeVirtualTextureSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	InitOutputs();
	bShowOutputNameOnPin = true;
	bShowMaskColorsOnPin = false;
#endif
}

bool UMaterialExpressionRuntimeVirtualTextureSample::InitVirtualTextureDependentSettings()
{
	bool bChanged = false;
	if (VirtualTexture != nullptr)
	{
		bChanged |= MaterialType != VirtualTexture->GetMaterialType();
		MaterialType = VirtualTexture->GetMaterialType();
		bChanged |= bSinglePhysicalSpace != VirtualTexture->GetSinglePhysicalSpace();
		bSinglePhysicalSpace = VirtualTexture->GetSinglePhysicalSpace();
		bChanged |= bAdaptive != VirtualTexture->GetAdaptivePageTable();
		bAdaptive = VirtualTexture->GetAdaptivePageTable();
	}
	return bChanged;
}

ESamplerSourceMode UMaterialExpressionRuntimeVirtualTextureSample::GetSamplerSourceMode() const
{
	// Convert texture address mode to matching sampler source mode.
	// Would be better if ESamplerSourceMode had a Mirror enum that we could also use...
	switch (TextureAddressMode)
	{
	case RVTTA_Clamp:
		return SSM_Clamp_WorldGroupSettings;
	case RVTTA_Wrap:
		return SSM_Wrap_WorldGroupSettings;
	case RVTTA_MAX:
		break;
	}
	return SSM_Clamp_WorldGroupSettings;
}

void UMaterialExpressionRuntimeVirtualTextureSample::InitOutputs()
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	
	Outputs.Add(FExpressionOutput(TEXT("BaseColor"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Specular")));
	Outputs.Add(FExpressionOutput(TEXT("Roughness")));
	Outputs.Add(FExpressionOutput(TEXT("Normal")));
	Outputs.Add(FExpressionOutput(TEXT("WorldHeight")));
	Outputs.Add(FExpressionOutput(TEXT("Mask")));
	Outputs.Add(FExpressionOutput(TEXT("Displacement")));
	Outputs.Add(FExpressionOutput(TEXT("Mask4")));
#endif // WITH_EDITORONLY_DATA
}

UObject* UMaterialExpressionRuntimeVirtualTextureSample::GetReferencedTexture() const
{
	return VirtualTexture; 
}

#if WITH_EDITOR

FExpressionInput* UMaterialExpressionRuntimeVirtualTextureSample::GetInput(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0: 
		return &Coordinates;
	case 1: 
		return &WorldPosition;
	case 2:
		if (MipValueMode == RVTMVM_MipLevel || MipValueMode == RVTMVM_MipBias)
		{
			return &MipValue;
		}
		else if (MipValueMode == RVTMVM_DerivativeUV || MipValueMode == RVTMVM_DerivativeWorld)
		{
			return &DDX;
		}
		break;
	case 3:
		if (MipValueMode == RVTMVM_DerivativeUV || MipValueMode == RVTMVM_DerivativeWorld)
		{
			return &DDY;
		}
		break;
	}

	return nullptr;
}

FName UMaterialExpressionRuntimeVirtualTextureSample::GetInputName(int32 InputIndex) const
{
	switch (InputIndex)
	{
	case 1:
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	case 2:
		if (MipValueMode == RVTMVM_MipLevel)
		{
			return TEXT("Mip Level");
		}
		if (MipValueMode == RVTMVM_MipBias)
		{
			return TEXT("Mip Bias");
		}
		else if (MipValueMode == RVTMVM_DerivativeUV)
		{
			return TEXT("DDX (UV)");
		}
		else if (MipValueMode == RVTMVM_DerivativeWorld)
		{
			return TEXT("DDX (World)");
		}
		break;
	case 3:
		if (MipValueMode == RVTMVM_DerivativeUV)
		{
			return TEXT("DDY (UV)");
		}
		else if (MipValueMode == RVTMVM_DerivativeWorld)
		{
			return TEXT("DDY (World)");
		}
		break;
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionRuntimeVirtualTextureSample::PostLoad()
{
	Super::PostLoad();

	InitOutputs();
}

bool UMaterialExpressionRuntimeVirtualTextureSample::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		const FString PropertyName = InProperty->GetName();

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UMaterialExpressionRuntimeVirtualTextureSample, bEnableFeedback))
		{
			// We can support disabling feedback for MipLevel mode.
			// We could allow for other modes too, but it's not a good idea to freely expose this option since it makes it easy could break things by accicent.
			// Instead the user has to explicitly set the mip level mode before disabling feedback.
			bIsEditable &= MipValueMode == RVTMVM_MipLevel;
		}
	}
	return bIsEditable;
}

void UMaterialExpressionRuntimeVirtualTextureSample::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Update MaterialType setting to match VirtualTexture
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, VirtualTexture))
	{
		if (VirtualTexture != nullptr)
		{
			InitVirtualTextureDependentSettings();
			FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType) || PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MipValueMode))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UMaterialExpressionRuntimeVirtualTextureSample::IsParameter() const
{
	return HasAParameterName() && GetParameterName().IsValid() && !GetParameterName().IsNone();
}

bool UMaterialExpressionRuntimeVirtualTextureSample::ValidateVirtualTextureParameters(FString& OutError, bool& bOutIsVirtualTextureValid) const
{
	bOutIsVirtualTextureValid = VirtualTexture != nullptr;
	if (!bOutIsVirtualTextureValid)
	{
		if (IsParameter())
		{
			OutError = TEXT("Missing input Virtual Texture");
			return false;
		}
	}
	else if (VirtualTexture->GetMaterialType() != MaterialType)
	{
		UEnum const* Enum = StaticEnum<ERuntimeVirtualTextureMaterialType>();
		FString MaterialTypeDisplayName = Enum->GetDisplayNameTextByValue((int64)MaterialType).ToString();
		FString TextureTypeDisplayName = Enum->GetDisplayNameTextByValue((int64)VirtualTexture->GetMaterialType()).ToString();

		OutError = FString::Printf(TEXT("Material type is '%s', should be '%s' to match %s"),
			*MaterialTypeDisplayName,
			*TextureTypeDisplayName,
			*VirtualTexture->GetName());

		bOutIsVirtualTextureValid = false;
	}
	else if (VirtualTexture->GetSinglePhysicalSpace() != bSinglePhysicalSpace)
	{
		OutError = FString::Printf(TEXT("Page table packing is '%d', should be '%d' to match %s"),
			bSinglePhysicalSpace ? 1 : 0,
			VirtualTexture->GetSinglePhysicalSpace() ? 1 : 0,
			*VirtualTexture->GetName());

		bOutIsVirtualTextureValid = false;
	}
	else if ((VirtualTexture->GetAdaptivePageTable()) != bAdaptive)
	{
		OutError = FString::Printf(TEXT("Adaptive page table is '%d', should be '%d' to match %s"),
			bAdaptive ? 1 : 0,
			VirtualTexture->GetAdaptivePageTable() ? 1 : 0,
			*VirtualTexture->GetName());

		bOutIsVirtualTextureValid = false;
	}

	return true;
}

bool UMaterialExpressionRuntimeVirtualTextureSample::GetRVTUnpackProperties(int32 OutputIndex, bool bIsVirtualTextureValid, FRuntimeVirtualTextureUnpackProperties& Output) const
{
	Output = FRuntimeVirtualTextureUnpackProperties{};

	switch (MaterialType)
	{
	case ERuntimeVirtualTextureMaterialType::BaseColor: Output.bIsBaseColorValid = true; break;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness: Output.bIsBaseColorValid = Output.bIsNormalValid = Output.bIsRoughnessValid = true; break;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular: Output.bIsRoughnessValid = Output.bIsBaseColorValid = Output.bIsNormalValid = Output.bIsSpecularValid = true; break;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg: Output.bIsRoughnessValid = Output.bIsBaseColorValid = Output.bIsNormalValid = Output.bIsSpecularValid = true; break;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg: Output.bIsRoughnessValid = Output.bIsBaseColorValid = Output.bIsNormalValid = Output.bIsSpecularValid = Output.bIsMaskValid = true; break;
	case ERuntimeVirtualTextureMaterialType::Mask4: Output.bIsMask4Valid = true; break;
	case ERuntimeVirtualTextureMaterialType::WorldHeight: Output.bIsWorldHeightValid = true; break;
	case ERuntimeVirtualTextureMaterialType::Displacement: Output.bIsDisplacementValid = true; break;
	}

	switch (OutputIndex)
	{
	case 0:
		if (bIsVirtualTextureValid && Output.bIsBaseColorValid)
		{
			switch (MaterialType)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg: [[fallthrough]];
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg: Output.UnpackType = EVirtualTextureUnpackType::BaseColorYCoCg; break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness: Output.UnpackType = EVirtualTextureUnpackType::BaseColorSRGB; break;
			default: Output.UnpackTarget = 0; Output.UnpackMask = 0x7; break;
			}
		}
		else
		{
			Output.ConstantVector = { 0.f, 0.f, 0.f };
		}
		break;
	case 1:
		if (bIsVirtualTextureValid && Output.bIsSpecularValid)
		{
			switch (MaterialType)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular: Output.UnpackTarget = 1; Output.UnpackMask = 0x1; break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg: [[fallthrough]];
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg: Output.UnpackTarget = 2; Output.UnpackMask = 0x1; break;
			}
		}
		else
		{
			Output.ConstantVector = { 0.5f };
		}
		break;
	case 2:
		if (bIsVirtualTextureValid && Output.bIsRoughnessValid)
		{
			switch (MaterialType)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness: Output.UnpackTarget = 1; Output.UnpackMask = 0x2; break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular: Output.UnpackTarget = 1; Output.UnpackMask = 0x2; break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg: [[fallthrough]];
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg: Output.UnpackTarget = 2; Output.UnpackMask = 0x2; break;
			}
		}
		else
		{
			Output.ConstantVector = { 0.5f };
		}
		break;
	case 3:
		if (bIsVirtualTextureValid && Output.bIsNormalValid)
		{
			switch (MaterialType)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness: Output.UnpackType = EVirtualTextureUnpackType::NormalBGR565; break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular: Output.UnpackType = EVirtualTextureUnpackType::NormalBC3BC3; break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg: [[fallthrough]];
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg: Output.UnpackType = EVirtualTextureUnpackType::NormalBC5BC1; break;
			}
		}
		else
		{
			Output.ConstantVector = { 0.f, 0.f, 1.f };
		}
		break;
	case 4:
		if (bIsVirtualTextureValid && Output.bIsWorldHeightValid)
		{
			Output.UnpackType = EVirtualTextureUnpackType::HeightR16;
		}
		else
		{
			Output.ConstantVector = { 0.f };
		}
		break;
	case 5:
		if (bIsVirtualTextureValid && Output.bIsMaskValid)
		{
			Output.UnpackTarget = 2; Output.UnpackMask = 0x8;
		}
		else
		{
			Output.ConstantVector = { 1.f };
		}
		break;
	case 6:
		if (bIsVirtualTextureValid && Output.bIsDisplacementValid)
		{
			Output.UnpackType = EVirtualTextureUnpackType::DisplacementR16;
		}
		else
		{
			Output.ConstantVector = { 0.f };
		}
		break;
	case 7:
		if (bIsVirtualTextureValid && Output.bIsMask4Valid)
		{
			Output.UnpackTarget = 0; Output.UnpackMask = 0xf;
		}
		else
		{
			Output.ConstantVector = { 0.f, 0.f, 0.f, 0.f };
		}
		break;
	default:
		return false;
	}
	return true;
}

int32 UMaterialExpressionRuntimeVirtualTextureSample::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Is this a valid UMaterialExpressionRuntimeVirtualTextureSampleParameter?
	const bool bIsParameter = IsParameter();

	// Check validity of current virtual texture
	FString TextureValidityError;
	bool bIsVirtualTextureValid = false;
	if (!ValidateVirtualTextureParameters(TextureValidityError, bIsVirtualTextureValid))
	{
		Compiler->Error(*TextureValidityError);
		return INDEX_NONE;
	}
	else if (!TextureValidityError.IsEmpty())
	{
		Compiler->Error(*TextureValidityError);
	}

	// Calculate the virtual texture layer and sampling/unpacking functions for this output
	// Fallback to a sensible default value if the output isn't valid for the bound virtual texture
	FRuntimeVirtualTextureUnpackProperties UnpackProperties;
	if (!GetRVTUnpackProperties(OutputIndex, bIsVirtualTextureValid, UnpackProperties))
	{
		return INDEX_NONE;
	}

	if (!UnpackProperties.ConstantVector.IsEmpty())
	{
		switch (UnpackProperties.ConstantVector.Num())
		{
		case 1:
			return Compiler->Constant(UnpackProperties.ConstantVector[0]);
		case 3:
			return Compiler->Constant3(UnpackProperties.ConstantVector[0], UnpackProperties.ConstantVector[1], UnpackProperties.ConstantVector[2]);
		case 4:
			return Compiler->Constant4(UnpackProperties.ConstantVector[0], UnpackProperties.ConstantVector[1], UnpackProperties.ConstantVector[2], UnpackProperties.ConstantVector[3]);
		default:
			checkNoEntry();
		}
	}
	
	const uint32 UnpackTarget = UnpackProperties.UnpackTarget;
	const uint32 UnpackMask = UnpackProperties.UnpackMask;
	const EVirtualTextureUnpackType UnpackType = UnpackProperties.UnpackType;

	// Compile the texture object references
	const int32 TextureLayerCount = URuntimeVirtualTexture::GetLayerCount(MaterialType);
	check(TextureLayerCount <= RuntimeVirtualTexture::MaxTextureLayers);

	int32 TextureCodeIndex[RuntimeVirtualTexture::MaxTextureLayers] = { INDEX_NONE };
	int32 TextureReferenceIndex[RuntimeVirtualTexture::MaxTextureLayers] = { INDEX_NONE };
	for (int32 TexureLayerIndex = 0; TexureLayerIndex < TextureLayerCount; TexureLayerIndex++)
	{
		const int32 PageTableLayerIndex = bSinglePhysicalSpace ? 0 : TexureLayerIndex;

		if (bIsParameter)
		{
			TextureCodeIndex[TexureLayerIndex] = Compiler->VirtualTextureParameter(GetParameterName(), VirtualTexture, TexureLayerIndex, PageTableLayerIndex, TextureReferenceIndex[TexureLayerIndex], SAMPLERTYPE_VirtualMasks);
		}
		else
		{
			TextureCodeIndex[TexureLayerIndex] = Compiler->VirtualTexture(VirtualTexture, TexureLayerIndex, PageTableLayerIndex, TextureReferenceIndex[TexureLayerIndex], SAMPLERTYPE_VirtualMasks);
		}
	}

	// Compile the runtime virtual texture uniforms
	int32 Uniforms[ERuntimeVirtualTextureShaderUniform_Count];
	for (int32 UniformIndex = 0; UniformIndex < ERuntimeVirtualTextureShaderUniform_Count; ++UniformIndex)
	{
		const UE::Shader::EValueType Type = URuntimeVirtualTexture::GetUniformParameterType(UniformIndex);
		if (bIsParameter)
		{
			Uniforms[UniformIndex] = Compiler->VirtualTextureUniform(GetParameterName(), TextureReferenceIndex[0], UniformIndex, Type);
		}
		else
		{
			Uniforms[UniformIndex] = Compiler->VirtualTextureUniform(TextureReferenceIndex[0], UniformIndex, Type);
		}
	}

	// Compile the coordinates
	// We use the virtual texture world space transform by default
	int32 CoordinateIndex = INDEX_NONE;

	if (Coordinates.GetTracedInput().Expression != nullptr && WorldPosition.GetTracedInput().Expression != nullptr)
	{
		Compiler->Errorf(TEXT("Only one of 'Coordinates' and 'WorldPosition' can be used"));
	}

	if (Coordinates.GetTracedInput().Expression != nullptr)
	{
		CoordinateIndex = Coordinates.Compile(Compiler);
	}
	else
	{
		int32 WorldPositionIndex = INDEX_NONE;
		if (WorldPosition.GetTracedInput().Expression != nullptr)
		{
			WorldPositionIndex = WorldPosition.Compile(Compiler);
		}
		else
		{
			WorldPositionIndex = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
			ensure(WorldPositionIndex != INDEX_NONE);
		}
		
		if (WorldPositionIndex != INDEX_NONE)
		{
			if (WorldPositionOriginType == EPositionOrigin::Absolute)
			{
				const int32 P0 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0];
				const int32 P1 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1];
				const int32 P2 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2];
				CoordinateIndex = Compiler->VirtualTextureWorldToUV(WorldPositionIndex, P0, P1, P2, EPositionOrigin::Absolute);
			}
			else if (WorldPositionOriginType == EPositionOrigin::CameraRelative)
			{
				//TODO: optimize by calculating translated world to VT directly.
				//This requires some more work as the transform is currently fed in through a preshader variable, which is cached.
				const int32 AbsWorldPosIndex = Compiler->TransformPosition(EMaterialCommonBasis::MCB_TranslatedWorld, EMaterialCommonBasis::MCB_World, WorldPositionIndex);

				const int32 P0 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0];
				const int32 P1 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1];
				const int32 P2 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2];
				CoordinateIndex = Compiler->VirtualTextureWorldToUV(AbsWorldPosIndex, P0, P1, P2, EPositionOrigin::Absolute);
			}
			else
			{
				checkNoEntry();
			}
		}
	}
	
	// Compile the mip level for the current mip value mode
	ETextureMipValueMode TextureMipLevelMode = TMVM_None;
	int32 MipValue0Index = INDEX_NONE;
	int32 MipValue1Index = INDEX_NONE;
	const bool bMipValueExpressionValid = MipValue.GetTracedInput().Expression != nullptr;
	if (MipValueMode == RVTMVM_MipLevel)
	{
		TextureMipLevelMode = TMVM_MipLevel;
		MipValue0Index = bMipValueExpressionValid ? MipValue.Compile(Compiler) : Compiler->Constant(0);
	}
	else if (MipValueMode == RVTMVM_MipBias)
	{
		TextureMipLevelMode = TMVM_MipBias;
		MipValue0Index = bMipValueExpressionValid ? MipValue.Compile(Compiler) : Compiler->Constant(0);
	}
	else if (MipValueMode == RVTMVM_DerivativeUV || MipValueMode == RVTMVM_DerivativeWorld)
	{
		if (DDX.GetTracedInput().Expression == nullptr || DDY.GetTracedInput().Expression == nullptr)
		{
			Compiler->Errorf(TEXT("Derivative MipValueMode requires connected DDX and DDY pins."));
		}

		TextureMipLevelMode = TMVM_Derivative;
		const int32 Ddx = DDX.Compile(Compiler);
		const int32 Ddy = DDY.Compile(Compiler);

		if (MipValueMode == RVTMVM_DerivativeUV)
		{
			MipValue0Index = Ddx;
			MipValue1Index = Ddy;
		}
		else if (MipValueMode == RVTMVM_DerivativeWorld)
		{
 			const int32 UDdx = Compiler->Dot(Ddx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
 			const int32 VDdx = Compiler->Dot(Ddx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
 			MipValue0Index = Compiler->AppendVector(UDdx, VDdx);

			const int32 UDdy = Compiler->Dot(Ddy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
			const int32 VDdy = Compiler->Dot(Ddy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
			MipValue1Index = Compiler->AppendVector(UDdy, VDdy);
		}
	}
	else if (MipValueMode == RVTMVM_RecalculateDerivatives)
	{
		// Calculate derivatives from world position.
		// This is legacy/hidden, and is better implemented in the material graph using RVTMVM_DerivativeWorld.
		TextureMipLevelMode = TMVM_Derivative;
		const int32 WorldPos = Compiler->WorldPosition(WPT_CameraRelative);
		const int32 WorldPositionDdx = Compiler->DDX(WorldPos);
		const int32 UDdx = Compiler->Dot(WorldPositionDdx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
		const int32 VDdx = Compiler->Dot(WorldPositionDdx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
		MipValue0Index = Compiler->AppendVector(UDdx, VDdx);
		const int32 WorldPositionDdy = Compiler->DDY(WorldPos);
		const int32 UDdy = Compiler->Dot(WorldPositionDdy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
		const int32 VDdy = Compiler->Dot(WorldPositionDdy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
		MipValue1Index = Compiler->AppendVector(UDdy, VDdy);
	}

	// We can support disabling feedback for MipLevel mode.
	const bool bForceEnableFeedback = TextureMipLevelMode != TMVM_MipLevel;

	// Compile the texture sample code
	const ESamplerSourceMode SamplerSourceMode = GetSamplerSourceMode();
	const bool bAutomaticMipViewBias = true;
	int32 SampleCodeIndex[RuntimeVirtualTexture::MaxTextureLayers] = { INDEX_NONE };
	for (int32 TexureLayerIndex = 0; TexureLayerIndex < TextureLayerCount; TexureLayerIndex++)
	{
		SampleCodeIndex[TexureLayerIndex] = Compiler->TextureSample(
			TextureCodeIndex[TexureLayerIndex],
			CoordinateIndex, 
			SAMPLERTYPE_VirtualMasks,
			MipValue0Index, MipValue1Index, TextureMipLevelMode, SamplerSourceMode, TGM_None,
			TextureReferenceIndex[TexureLayerIndex],
			bAutomaticMipViewBias, bAdaptive, (bEnableFeedback || bForceEnableFeedback));
	}

	// Compile any unpacking code
	int32 UnpackCodeIndex = INDEX_NONE;
	if (UnpackType != EVirtualTextureUnpackType::None)
	{
		int32 P0 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack];
		UnpackCodeIndex = Compiler->VirtualTextureUnpack(SampleCodeIndex[0], SampleCodeIndex[1], SampleCodeIndex[2], P0, UnpackType);
	}
	else
	{
		UnpackCodeIndex = SampleCodeIndex[UnpackTarget] == INDEX_NONE ? INDEX_NONE : Compiler->ComponentMask(SampleCodeIndex[UnpackTarget], UnpackMask & 1, (UnpackMask >> 1) & 1, (UnpackMask >> 2) & 1, (UnpackMask >> 3) & 1);
	}
	return UnpackCodeIndex;
}

void UMaterialExpressionRuntimeVirtualTextureSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Runtime Virtual Texture Sample"));
}

#endif // WITH_EDITOR

UMaterialExpressionRuntimeVirtualTextureSampleParameter::UMaterialExpressionRuntimeVirtualTextureSampleParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsParameterExpression = true;
}

#if WITH_EDITOR

template<typename T>
static void SendPostEditChangeProperty(T* Object, const FName& Name)
{
	FProperty* Property = FindFProperty<FProperty>(T::StaticClass(), Name);
	FPropertyChangedEvent Event(Property);
	Object->PostEditChangeProperty(Event);
}

bool UMaterialExpressionRuntimeVirtualTextureSampleParameter::SetParameterValue(FName InParameterName, URuntimeVirtualTexture* InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (VirtualTexture != InValue)
		{
			VirtualTexture = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, VirtualTexture));
			}
		}
		return true;
	}

	return false;
}

void UMaterialExpressionRuntimeVirtualTextureSampleParameter::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}

FString UMaterialExpressionRuntimeVirtualTextureSampleParameter::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionRuntimeVirtualTextureSampleParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	ValidateParameterNameInternal(this, Material, bAllowDuplicateName);
}

void UMaterialExpressionRuntimeVirtualTextureSampleParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Runtime Virtual Texture Sample Param "));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionRuntimeVirtualTextureSampleParameter::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (ParameterName.ToString().Contains(SearchQuery))
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}
#endif

#if WITH_EDITOR

int32 UMaterialExpressionRuntimeVirtualTextureReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RuntimeVirtualTextureReplace input 'Default'"));
	}

	if (!VirtualTextureOutput.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RuntimeVirtualTextureReplace input 'VirtualTextureOutput'"));
	}

	return Compiler->IsInRuntimeVirtualTextureOutput() ? VirtualTextureOutput.Compile(Compiler) : Default.Compile(Compiler);}

bool UMaterialExpressionRuntimeVirtualTextureReplace::IsResultMaterialAttributes(int32 OutputIndex)
{
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		if (It->GetTracedInput().Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}
	return false;
}

void UMaterialExpressionRuntimeVirtualTextureReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("RuntimeVirtualTextureReplace"));
}

#endif // WITH_EDITOR

#if WITH_EDITOR

int32 UMaterialExpressionRuntimeVirtualTextureCustomData::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->VirtualTextureCustomData();
}

void UMaterialExpressionRuntimeVirtualTextureCustomData::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("RuntimeVirtualTextureCustomData"));
}

#endif // WITH_EDITOR

#if WITH_EDITOR

int32 UMaterialExpressionVirtualTextureFeatureSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Yes.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing VirtualTextureFeatureSwitch input 'Yes'"));
	}

	if (!No.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing VirtualTextureFeatureSwitch input 'No'"));
	}

	if (UseVirtualTexturing(Compiler->GetShaderPlatform()))
	{
		return Yes.Compile(Compiler);
	}
	
	return No.Compile(Compiler);
}

bool UMaterialExpressionVirtualTextureFeatureSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		if (It->GetTracedInput().Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}
	return false;
}

void UMaterialExpressionVirtualTextureFeatureSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("VirtualTextureFeatureSwitch"));
}

#endif // WITH_EDITOR

UMaterialExpressionMeshPaintTextureCoordinateIndex::UMaterialExpressionMeshPaintTextureCoordinateIndex(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("")));
}

#if WITH_EDITOR

void UMaterialExpressionMeshPaintTextureCoordinateIndex::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Mesh Paint Texture Coordinate Index"));
}

void UMaterialExpressionMeshPaintTextureCoordinateIndex::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Get the Mesh Paint Texture UV coordinate index."), 40, OutToolTip);
}

#endif // WITH_EDITOR

UMaterialExpressionMeshPaintTextureObject::UMaterialExpressionMeshPaintTextureObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("")));
}

#if WITH_EDITOR

int32 UMaterialExpressionExternalCodeBase::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Validate output index for given number of code identifiers
	if (ExternalCodeIdentifiers.Num() == 1)
	{
		OutputIndex = 0;
	}
	else if (OutputIndex < 0 || OutputIndex >= ExternalCodeIdentifiers.Num())
	{
		return Compiler->Errorf(TEXT("OutputIndex (%d) out of range, material expression has only %d external code entry/entries"), OutputIndex, ExternalCodeIdentifiers.Num());
	}
	
	// Find identifier in external code registry
	const FMaterialExternalCodeDeclaration* ExternalCode = MaterialExternalCodeRegistry::Get().FindExternalCode(ExternalCodeIdentifiers[OutputIndex]);
	if (!ExternalCode)
	{
		return Compiler->Errorf(TEXT("Missing external code declaration for '%s'"), *ExternalCodeIdentifiers[OutputIndex].ToString());
	}

	return Compiler->ExternalCode(*ExternalCode);
}

#endif

#if WITH_EDITOR

void UMaterialExpressionMeshPaintTextureObject::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Mesh Paint Texture Object"));
}

void UMaterialExpressionMeshPaintTextureObject::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Get the Mesh Paint Texture object for feeding to a Texture Sample node."), 40, OutToolTip);
}

EMaterialValueType UMaterialExpressionMeshPaintTextureObject::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Texture2D;
}

#endif // WITH_EDITOR

UMaterialExpressionMeshPaintTextureReplace::UMaterialExpressionMeshPaintTextureReplace(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("")));
}

#if WITH_EDITOR

void UMaterialExpressionMeshPaintTextureReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Mesh Paint Texture Replace"));
}

void UMaterialExpressionMeshPaintTextureReplace::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Switch between inputs according to whether there is a valid Mesh Paint Texture available to sample."), 40, OutToolTip);
}

int32 UMaterialExpressionMeshPaintTextureReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Default"));
	}
	else if (!MeshPaintTexture.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input MeshPaintTexture"));
	}
	else
	{
		const int32 Arg1 = Default.Compile(Compiler);
		const int32 Arg2 = MeshPaintTexture.Compile(Compiler);
		return Compiler->MeshPaintTextureReplace(Arg1, Arg2);
	}
}

#endif // WITH_EDITOR

//
//  UMaterialExpressionTextureSampleParameter
//
UMaterialExpressionTextureSampleParameter::UMaterialExpressionTextureSampleParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bIsParameterExpression = true;
	bShowTextureInputPin = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureSampleParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FString ErrorMessage;
	if (!TextureIsValid(Texture, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	FString SamplerTypeError;
	if (!MaterialExpressionUtils::VerifySamplerType(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform(), Texture, SamplerType, SamplerTypeError))
	{
		return Compiler->Errorf(TEXT("%s"), *SamplerTypeError);
	}

	if (!ParameterName.IsValid() || ParameterName.IsNone())
	{
		return UMaterialExpressionTextureSample::Compile(Compiler, OutputIndex);
	}

	if (Coordinates.GetTracedInput().Expression)
	{
		Compiler->SetPotentiallyManipulateTexCoords();
	}

	return CompileTextureSample(
		Compiler,
		Texture,
		Coordinates.GetTracedInput().Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false),
		SamplerType,
		ParameterName,
		CompileMipValue0(Compiler),
		CompileMipValue1(Compiler),
		MipValueMode,
		SamplerSource,
		AutomaticViewMipBias);
}

void UMaterialExpressionTextureSampleParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Texture Param")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

void UMaterialExpressionTextureSampleParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	ValidateParameterNameInternal(this, Material, bAllowDuplicateName);
}

bool UMaterialExpressionTextureSampleParameter::SetParameterValue(FName InParameterName, UTexture* InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (Texture != InValue)
		{
			Texture = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Texture));
			}
		}
		return true;
	}
	return false;
}

bool UMaterialExpressionTextureSampleParameter::TextureIsValid(UTexture* /*InTexture*/, FString& OutMessage)
{
	OutMessage = TEXT("Invalid texture type");
	return false;
}

void UMaterialExpressionTextureSampleParameter::SetDefaultTexture()
{
	// Does nothing in the base case...
}
#endif // WITH_EDITOR

//
//  UMaterialExpressionTextureObjectParameter
//
UMaterialExpressionTextureObjectParameter::UMaterialExpressionTextureObjectParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> DefaultTexture2D;
		FConstructorStatics()
			: DefaultTexture2D(TEXT("/Engine/EngineResources/DefaultTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Texture = ConstructorStatics.DefaultTexture2D.Object;

	Outputs.Empty();
	Outputs.Add(FExpressionOutput(TEXT("")));

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Hide the texture coordinate input
	CachedInputs.Empty();
#endif
}

#if WITH_EDITOR
bool UMaterialExpressionTextureObjectParameter::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	if (!InTexture)
	{
		OutMessage = TEXT("Requires valid texture");
		return false;
	}

	return true;
}

void UMaterialExpressionTextureObjectParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Param Tex Object")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

int32 UMaterialExpressionTextureObjectParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FString ErrorMessage;
	if (!TextureIsValid(Texture, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	// It seems like this error should be checked here, but this can break existing materials, see https://jira.it.epicgames.net/browse/UE-68862
	/*if (!VerifySamplerType(Compiler, (Desc.Len() > 0 ? *Desc : TEXT("TextureObjectParameter")), Texture, SamplerType))
	{
		return INDEX_NONE;
	}*/

	return SamplerType == SAMPLERTYPE_External ? Compiler->ExternalTextureParameter(ParameterName, Texture) : Compiler->TextureParameter(ParameterName, Texture, SamplerType);
}

int32 UMaterialExpressionTextureObjectParameter::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FString ErrorMessage;
	if (!TextureIsValid(Texture, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	// Preview the texture object by actually sampling it
	return CompileTextureSample(Compiler, Texture, Compiler->TextureCoordinate(0, false, false), SamplerType, ParameterName);
}
#endif // WITH_EDITOR

//
//  UMaterialExpressionTextureObject
//
UMaterialExpressionTextureObject::UMaterialExpressionTextureObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> Object0;
		FConstructorStatics()
			: Object0(TEXT("/Engine/EngineResources/DefaultTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Texture = ConstructorStatics.Object0.Object;

	Outputs.Empty();
	Outputs.Add(FExpressionOutput(TEXT("")));

	bCollapsed = false;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void UMaterialExpressionTextureObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty( PropertyChangedEvent );

	if ( PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Texture) )
	{
		if ( Texture )
		{
			AutoSetSampleType();
			FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
		}
	}
}

void UMaterialExpressionTextureObject::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Texture Object")); 
}

int32 UMaterialExpressionTextureObject::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Texture)
	{
		return CompilerError(Compiler, TEXT("Requires valid texture"));
	}

	// It seems like this error should be checked here, but this can break existing materials, see https://jira.it.epicgames.net/browse/UE-68862
	/*if (!VerifySamplerType(Compiler, (Desc.Len() > 0 ? *Desc : TEXT("TextureObject")), Texture, SamplerType))
	{
		return INDEX_NONE;
	}*/

	return SamplerType == SAMPLERTYPE_External ? Compiler->ExternalTexture(Texture) : Compiler->Texture(Texture, SamplerType);
}

int32 UMaterialExpressionTextureObject::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Texture)
	{
		return CompilerError(Compiler, TEXT("Requires valid texture"));
	}

	return CompileTextureSample(Compiler, Texture, Compiler->TextureCoordinate(0, false, false), MaterialExpressionUtils::GetSamplerTypeForTexture( Texture ));
}

EMaterialValueType UMaterialExpressionTextureObject::GetOutputValueType(int32 OutputIndex)
{
	if (Cast<UTextureCube>(Texture) != nullptr)
	{
		return MCT_TextureCube;
	}
	else if (Cast<UTexture2DArray>(Texture) != nullptr)
	{
		return MCT_Texture2DArray;
	}
	else if (Cast<UTextureCubeArray>(Texture) != nullptr)
	{
		return MCT_TextureCubeArray;
	}
	else if (Cast<UVolumeTexture>(Texture) != nullptr)
	{
		return MCT_VolumeTexture;
	}
	else
	{
		return MCT_Texture2D;
	}
}
#endif //WITH_EDITOR

//
//  UMaterialExpressionTextureProperty
//
UMaterialExpressionTextureProperty::UMaterialExpressionTextureProperty(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = false;
	
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("")));
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureProperty::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{	
	if (!TextureObject.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("TextureSample> Missing input texture"));
	}

	const int32 TextureCodeIndex = TextureObject.Compile(Compiler);
	if (TextureCodeIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->TextureProperty(TextureCodeIndex, Property);
}

void UMaterialExpressionTextureProperty::GetTexturesForceMaterialRecompile(TArray<UTexture *> &Textures) const
{
	UMaterialExpression *TextureObjectExpression = TextureObject.GetTracedInput().Expression;

	if (TextureObjectExpression && TextureObjectExpression->IsA(UMaterialExpressionTextureBase::StaticClass()))
	{
		UMaterialExpressionTextureBase *TextureExpressionBase = Cast<UMaterialExpressionTextureBase>(TextureObjectExpression);
		if (TextureExpressionBase->Texture)
		{
			Textures.Add(TextureExpressionBase->Texture);
		}
	}
}

void UMaterialExpressionTextureProperty::GetCaption(TArray<FString>& OutCaptions) const
{
#if WITH_EDITOR
	const UEnum* TexturePropertyEnum = StaticEnum<EMaterialExposedTextureProperty>();
	check(TexturePropertyEnum);

	const FString PropertyDisplayName = TexturePropertyEnum->GetDisplayNameTextByValue(Property).ToString();
#else
	const FString PropertyDisplayName;
#endif

	OutCaptions.Add(PropertyDisplayName);
}

// this define is only used for the following function
#define IF_INPUT_RETURN(Type) if(!InputIndex) return Type; --InputIndex

EMaterialValueType UMaterialExpressionTextureProperty::GetInputValueType(int32 InputIndex)
{
	// TextureObject
	IF_INPUT_RETURN(static_cast<EMaterialValueType>(MCT_Texture | MCT_SparseVolumeTexture));
	return MCT_Unknown;
}
#undef IF_INPUT_RETURN


bool UMaterialExpressionTextureProperty::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	TArray<FString> Captions;
	GetCaption(Captions);
	for (const FString& Caption : Captions)
	{
		if (Caption.Contains(SearchQuery))
		{
			return true;
		}
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

#endif

//
//  UMaterialExpressionTextureSampleParameter2D
//
UMaterialExpressionTextureSampleParameter2D::UMaterialExpressionTextureSampleParameter2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> DefaultTexture;
		FConstructorStatics()
			: DefaultTexture(TEXT("/Engine/EngineResources/DefaultTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Texture = ConstructorStatics.DefaultTexture.Object;
#endif
}

#if WITH_EDITOR
void UMaterialExpressionTextureSampleParameter2D::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Param2D")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionTextureSampleParameter2D::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	const bool bRequiresVirtualTexture = IsVirtualSamplerType(SamplerType);
	if (!InTexture)
	{
		OutMessage = TEXT("Found NULL, requires Texture2D");
		return false;
	}
	else if (!(InTexture->GetMaterialType() & (MCT_Texture2D | MCT_TextureExternal | MCT_TextureVirtual)))
	{
		OutMessage = FString::Printf(TEXT("Found %s, requires Texture2D"), *InTexture->GetClass()->GetName());
		return false;
	}
	else if (bRequiresVirtualTexture && !InTexture->VirtualTextureStreaming)
	{
		OutMessage = TEXT("Sampler requires VirtualTexture");
		return false;
	}
	else if (!bRequiresVirtualTexture && InTexture->VirtualTextureStreaming)
	{
		OutMessage = TEXT("Sampler requires non-VirtualTexture");
		return false;
	}

	return true;
}

void UMaterialExpressionTextureSampleParameter2D::SetDefaultTexture()
{
	Texture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineResources/DefaultTexture.DefaultTexture"), nullptr, LOAD_None, nullptr);
}

bool UMaterialExpressionTextureSampleParameter::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if( ParameterName.ToString().Contains(SearchQuery) )
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

FString UMaterialExpressionTextureSampleParameter::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionTextureSampleParameter::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}
#endif


//
//  UMaterialExpressionTextureSampleParameterCube
//
UMaterialExpressionTextureSampleParameterCube::UMaterialExpressionTextureSampleParameterCube(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTextureCube> DefaultTextureCube;
		FConstructorStatics()
			: DefaultTextureCube(TEXT("/Engine/EngineResources/DefaultTextureCube"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Texture = ConstructorStatics.DefaultTextureCube.Object;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureSampleParameterCube::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Coordinates.GetTracedInput().Expression)
	{
		return CompilerError(Compiler, TEXT("Cube sample needs UV input"));
	}

	return UMaterialExpressionTextureSampleParameter::Compile(Compiler, OutputIndex);
}

void UMaterialExpressionTextureSampleParameterCube::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ParamCube")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionTextureSampleParameterCube::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	if (!InTexture)
	{
		OutMessage = TEXT("Found NULL, requires TextureCube");
		return false;
	}
	else if (!(InTexture->GetMaterialType() & MCT_TextureCube))
	{
		OutMessage = FString::Printf(TEXT("Found %s, requires TextureCube"), *InTexture->GetClass()->GetName());
		return false;
	}

	return true;
}

void UMaterialExpressionTextureSampleParameterCube::SetDefaultTexture()
{
	Texture = LoadObject<UTextureCube>(nullptr, TEXT("/Engine/EngineResources/DefaultTextureCube.DefaultTextureCube"), nullptr, LOAD_None, nullptr);
}
#endif // WITH_EDITOR

//
//  UMaterialExpressionTextureSampleParameter2DArray
//
#if WITH_EDITOR
int32 UMaterialExpressionTextureSampleParameter2DArray::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
#if PLATFORM_ANDROID
	return CompilerError(Compiler, TEXT("Texture2DArrays not supported on selected platform."));
#endif

	if (!Coordinates.GetTracedInput().Expression)
	{
		return CompilerError(Compiler, TEXT("2D array sample needs UVW input"));
	}

	return UMaterialExpressionTextureSampleParameter::Compile(Compiler, OutputIndex);
}

void UMaterialExpressionTextureSampleParameter2DArray::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Param2DArray"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionTextureSampleParameter2DArray::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	if (!InTexture)
	{
		OutMessage = TEXT("Found NULL, requires Texture2DArray");
		return false;
	}
	else if (!(InTexture->GetMaterialType() & MCT_Texture2DArray))
	{
		OutMessage = FString::Printf(TEXT("Found %s, requires Texture2DArray"), *InTexture->GetClass()->GetName());
		return false;
	}

	return true;
}
#endif // WITH_EDITOR

bool UMaterialExpressionTextureSampleParameter2DArray::IsAllowedIn(const UObject* MaterialOrFunction) const
{
	static const auto AllowTextureArrayAssetCreationVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowTexture2DArrayCreation"));
	if (AllowTextureArrayAssetCreationVar->GetValueOnGameThread() == 0)
	{
		return false;
	}

	return Super::IsAllowedIn(MaterialOrFunction);
}

const TCHAR* UMaterialExpressionTextureSampleParameter2DArray::GetRequirements()
{
	return TEXT("Requires Texture2DArray");
}

//
//  UMaterialExpressionTextureSampleParameterCubeArray
//
#if WITH_EDITOR
int32 UMaterialExpressionTextureSampleParameterCubeArray::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Coordinates.GetTracedInput().Expression)
	{
		return CompilerError(Compiler, TEXT("Cube Array sample needs UV input"));
	}

	return UMaterialExpressionTextureSampleParameter::Compile(Compiler, OutputIndex);
}

void UMaterialExpressionTextureSampleParameterCubeArray::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ParamCubeArray"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionTextureSampleParameterCubeArray::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	if (!InTexture)
	{
		OutMessage = TEXT("Found NULL, requires TextureCubeArray");
		return false;
	}
	else if (!(InTexture->GetMaterialType() & MCT_TextureCubeArray))
	{
		OutMessage = FString::Printf(TEXT("Found %s, requires TextureCubeArray"), *InTexture->GetClass()->GetName());
		return false;
	}

	return true;
}
#endif // WITH_EDITOR

const TCHAR* UMaterialExpressionTextureSampleParameterCubeArray::GetRequirements()
{
	return TEXT("Requires TextureCubeArray");
}

//
//  UMaterialExpressionTextureSampleParameterVolume
//
UMaterialExpressionTextureSampleParameterVolume::UMaterialExpressionTextureSampleParameterVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UVolumeTexture> DefaultVolumeTexture;
		FConstructorStatics()
			: DefaultVolumeTexture(TEXT("/Engine/EngineResources/DefaultVolumeTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Texture = ConstructorStatics.DefaultVolumeTexture.Object;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureSampleParameterVolume::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Coordinates.GetTracedInput().Expression)
	{
		return CompilerError(Compiler, TEXT("Volume sample needs UVW input"));
	}

	return UMaterialExpressionTextureSampleParameter::Compile(Compiler, OutputIndex);
}

void UMaterialExpressionTextureSampleParameterVolume::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ParamVolume")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionTextureSampleParameterVolume::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	if (!InTexture)
	{
		OutMessage = TEXT("Found NULL, requires VolumeTexture");
		return false;
	}
	else if (!(InTexture->GetMaterialType() & MCT_VolumeTexture))
	{
		OutMessage = FString::Printf(TEXT("Found %s, requires VolumeTexture"), *InTexture->GetClass()->GetName());
		return false;
	}

	return true;
}

void UMaterialExpressionTextureSampleParameterVolume::SetDefaultTexture()
{
	Texture = LoadObject<UVolumeTexture>(nullptr, TEXT("/Engine/EngineResources/DefaultVolumeTexture.DefaultVolumeTexture"), nullptr, LOAD_None, nullptr);
}
#endif // WITH_EDITOR

/** 
 * Performs a SubUV operation, which is doing a texture lookup into a sub rectangle of a texture, and optionally blending with another rectangle.  
 * This supports both sprites and mesh emitters.
 */
static int32 ParticleSubUV(
	FMaterialCompiler* Compiler, 
	int32 TextureIndex, 
	EMaterialSamplerType SamplerType, 
	int32 MipValue0Index,
	int32 MipValue1Index,
	ETextureMipValueMode MipValueMode,
	bool bBlend)
{
	return Compiler->ParticleSubUV(TextureIndex, SamplerType, MipValue0Index, MipValue1Index, MipValueMode, bBlend);
}

/** 
 *	UMaterialExpressionTextureSampleParameterSubUV
 */
#if WITH_EDITOR
int32 UMaterialExpressionTextureSampleParameterSubUV::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FString ErrorMessage;
	if (!TextureIsValid(Texture, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	FString SamplerTypeError;
	if (! MaterialExpressionUtils::VerifySamplerType(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform(), Texture, SamplerType, SamplerTypeError))
	{
		return Compiler->Errorf(TEXT("%s"), *SamplerTypeError);
	}

	// while this expression does provide a TextureCoordinate input pin, it is, and has always been, ignored.  And only
	// supports using UV0.  Further, in order to support non-vertex fetch implementations we need to be sure to register
	// the use of the first texture slot
	Compiler->TextureCoordinate(0 /*Explit dependency on the 1st uv channel*/, false, false);

	int32 TextureCodeIndex = Compiler->TextureParameter(ParameterName, Texture, SamplerType);
	return ParticleSubUV(Compiler, TextureCodeIndex, SamplerType, CompileMipValue0(Compiler), CompileMipValue1(Compiler), MipValueMode,	bBlend);
}

void UMaterialExpressionTextureSampleParameterSubUV::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Parameter SubUV"));
}

bool UMaterialExpressionTextureSampleParameterSubUV::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	return UMaterialExpressionTextureSampleParameter2D::TextureIsValid(InTexture, OutMessage);
}

int32 UMaterialExpressionAdd::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Add(Arg1, Arg2);
}

void UMaterialExpressionAdd::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Add"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionMultiply
//
#if WITH_EDITOR
int32 UMaterialExpressionMultiply::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Mul(Arg1, Arg2);
}

void UMaterialExpressionMultiply::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Multiply"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionDivide::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Div(Arg1, Arg2);
}

void UMaterialExpressionDivide::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Divide"));
}

int32 UMaterialExpressionSubtract::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Sub(Arg1, Arg2);
}

void UMaterialExpressionSubtract::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Subtract"));
}

int32 UMaterialExpressionSmoothStep::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = Min.GetTracedInput().Expression ? Min.Compile(Compiler) : Compiler->Constant(ConstMin);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = Max.GetTracedInput().Expression ? Max.Compile(Compiler) : Compiler->Constant(ConstMax);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg3 = Value.GetTracedInput().Expression ? Value.Compile(Compiler) : Compiler->Constant(ConstValue);

	return Compiler->SmoothStep(Arg1, Arg2, Arg3);
}

void UMaterialExpressionSmoothStep::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SmoothStep"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionStep
//
#if WITH_EDITOR
int32 UMaterialExpressionStep::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = Y.GetTracedInput().Expression ? Y.Compile(Compiler) : Compiler->Constant(ConstY);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = X.GetTracedInput().Expression ? X.Compile(Compiler) : Compiler->Constant(ConstX);

	return Compiler->Step(Arg1, Arg2);
}

void UMaterialExpressionStep::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Step"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionInverseLerp
//
#if WITH_EDITOR
int32 UMaterialExpressionInverseLinearInterpolate::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg3 = Value.GetTracedInput().Expression ? Value.Compile(Compiler) : Compiler->Constant(ConstValue);

	int32 Result = Compiler->InvLerp(Arg1, Arg2, Arg3);
	return bClampResult ? Compiler->Clamp(Result, Compiler->Constant(0.0f), Compiler->Constant(1.0f)) : Result;
}

void UMaterialExpressionInverseLinearInterpolate::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("InvLerp"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionLinearInterpolate
//
#if WITH_EDITOR
int32 UMaterialExpressionLinearInterpolate::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg3 = Alpha.GetTracedInput().Expression ? Alpha.Compile(Compiler) : Compiler->Constant(ConstAlpha);

	return Compiler->Lerp(Arg1, Arg2, Arg3);
}

void UMaterialExpressionLinearInterpolate::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Lerp"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionColorRamp
//
UMaterialExpressionColorRamp::UMaterialExpressionColorRamp(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	ColorCurve = ObjectInitializer.CreateDefaultSubobject<UCurveLinearColor>(this, TEXT("ColorCurve"));

	// Initialize ColorCurve RGB value showing a white->black gradient
	for (int i = 0; i < 3; i++)
	{
		if (FRichCurve* Curve = &ColorCurve->FloatCurves[i])
		{
			Curve->Reset();
			Curve->AddKey(0.f, 1.f);
			Curve->AddKey(1.f, 0.f);
		}
	}
#endif // WITH_EDITOR
}

void UMaterialExpressionColorRamp::Serialize(FArchive& Ar)
{
	TOptional<TGuardValue<TObjectPtr<UCurveLinearColor>>> ColorCurveGuard;
	if (Ar.IsSaving() && Ar.IsCooking())
	{
		ColorCurveGuard.Emplace(ColorCurve, nullptr);
	}

	Super::Serialize(Ar);
}

#if WITH_EDITOR
void UMaterialExpressionColorRamp::HandleCurvePropertyChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	if (Material)
	{
		UMaterialExpression::RefreshNode(true);
	}
}

int32 UMaterialExpressionColorRamp::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!ColorCurve)
	{
		return Compiler->Errorf(TEXT("Missing ColorCurve"));
	}

	// If the input is constant, evaluate at compile time.
	if (!Input.GetTracedInput().Expression)
	{
		FLinearColor ColorValue = ColorCurve->GetLinearColorValue(ConstInput);
		return Compiler->Constant4(ColorValue.R, ColorValue.G, ColorValue.B, ColorValue.A);
	}

	// If the input is dynamic, evaluate with translator
	int32 InputCode = Compiler->ComponentMask(Input.Compile(Compiler), true, false, false, false);

	// Helper lambda to evaluate a curve using vectorized multiply–add operations
	auto EvaluateCurve = [Compiler, InputCode](const FRichCurve& Curve) -> int32
		{
			const int32 NumKeys = Curve.Keys.Num();
			switch (NumKeys)
			{
				case 0:
					return Compiler->Constant(0.0f);

				case 1:
					return Compiler->Constant(Curve.Keys[0].Value);

				case 2:
				{
					const float StartTime = Curve.Keys[0].Time;
					const float EndTime = Curve.Keys[1].Time;
					const float StartValue = Curve.Keys[0].Value;
					const float EndValue = Curve.Keys[1].Value;

					int32 TimeDelta = Compiler->Constant(EndTime - StartTime);
					int32 TimeDiff = Compiler->Sub(InputCode, Compiler->Constant(StartTime));
					int32 Fraction = Compiler->Div(TimeDiff, TimeDelta);

					return Compiler->Lerp(Compiler->Constant(StartValue), Compiler->Constant(EndValue), Fraction);
				}
			}

			// Turn input into a vector4
			int32 InValueVec = Compiler->AppendVector(Compiler->AppendVector(InputCode, InputCode), Compiler->AppendVector(InputCode, InputCode));

			int32 Result = Compiler->Constant(Curve.Keys[0].Value);
			int32 i = 0;

			// Use vector operations for segments of 4
			for (; i < NumKeys - 4; i += 4)
			{
				int32 StartTimeVec = Compiler->Constant4(
					Curve.Keys[i].Time,
					Curve.Keys[i + 1].Time,
					Curve.Keys[i + 2].Time,
					Curve.Keys[i + 3].Time
				);
				int32 EndTimeVec = Compiler->Constant4(
					Curve.Keys[i + 1].Time,
					Curve.Keys[i + 2].Time,
					Curve.Keys[i + 3].Time,
					Curve.Keys[i + 4].Time
				);
				int32 StartValueVec = Compiler->Constant4(
					Curve.Keys[i].Value,
					Curve.Keys[i + 1].Value,
					Curve.Keys[i + 2].Value,
					Curve.Keys[i + 3].Value
				);
				int32 EndValueVec = Compiler->Constant4(
					Curve.Keys[i + 1].Value,
					Curve.Keys[i + 2].Value,
					Curve.Keys[i + 3].Value,
					Curve.Keys[i + 4].Value
				);

				int32 TimeDeltaVec = Compiler->Sub(EndTimeVec, StartTimeVec);
				int32 ValueDeltaVec = Compiler->Sub(EndValueVec, StartValueVec);

				int32 TimeDiffVec = Compiler->Sub(InValueVec, StartTimeVec);
				int32 FractionVec = Compiler->Div(TimeDiffVec, TimeDeltaVec);
				int32 SatFractionVec = Compiler->Saturate(FractionVec);

				int32 ContributionVec = Compiler->Mul(ValueDeltaVec, SatFractionVec);
				int32 ContributionSum = Compiler->Dot(ContributionVec, Compiler->Constant4(1.0f, 1.0f, 1.0f, 1.0f));
				Result = Compiler->Add(Result, ContributionSum);
			}

			// Use scalar operations for the remaining keys
			for (; i < NumKeys - 1; i++)
			{
				const float StartTime = Curve.Keys[i].Time;
				const float EndTime = Curve.Keys[i + 1].Time;
				const float StartValue = Curve.Keys[i].Value;
				const float EndValue = Curve.Keys[i + 1].Value;

				int32 TimeDelta = Compiler->Constant(EndTime - StartTime);
				int32 ValueDelta = Compiler->Constant(EndValue - StartValue);

				int32 TimeDiff = Compiler->Sub(InputCode, Compiler->Constant(StartTime));
				int32 Fraction = Compiler->Div(TimeDiff, TimeDelta);
				int32 SatFraction = Compiler->Saturate(Fraction);

				int32 Contribution = Compiler->Mul(ValueDelta, SatFraction);
				Result = Compiler->Add(Result, Contribution);
			}
			return Result;
		};

	int32 Red = EvaluateCurve(ColorCurve->FloatCurves[0]);
	int32 Green = EvaluateCurve(ColorCurve->FloatCurves[1]);
	int32 Blue = EvaluateCurve(ColorCurve->FloatCurves[2]);
	int32 Alpha = EvaluateCurve(ColorCurve->FloatCurves[3]);
	int32 FinalVector = Compiler->AppendVector(Compiler->AppendVector(Red, Green), Compiler->AppendVector(Blue, Alpha));
	return FinalVector;
}

void UMaterialExpressionColorRamp::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Color Ramp"));
}
#endif // WITH_EDITOR

UMaterialExpressionGenericConstant::UMaterialExpressionGenericConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

UMaterialExpressionConstantDouble::UMaterialExpressionConstantDouble(const FObjectInitializer& ObjectInitializer) {}

#if WITH_EDITOR
int32 UMaterialExpressionGenericConstant::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->GenericConstant(GetConstantValue());
}

void UMaterialExpressionGenericConstant::GetCaption(TArray<FString>& OutCaptions) const
{
	TStringBuilder<1024> String;
	GetConstantValue().ToString(UE::Shader::EValueStringFormat::Description, String);
	OutCaptions.Add(FString(String.ToView()));
}

FString UMaterialExpressionGenericConstant::GetDescription() const
{
	FString Result = FString(*GetClass()->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
	Result += TEXT(" (");
	Result += Super::GetDescription();
	Result += TEXT(")");
	return Result;
}
#endif // WITH_EDITOR

UMaterialExpressionConstant::UMaterialExpressionConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionConstant::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->Constant(R);
}

void UMaterialExpressionConstant::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf( TEXT("%.4g"), R ));
}

FString UMaterialExpressionConstant::GetDescription() const
{
	FString Result = FString(*GetClass()->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
	Result += TEXT(" (");
	Result += Super::GetDescription();
	Result += TEXT(")");
	return Result;
}
#endif // WITH_EDITOR

UMaterialExpressionConstant2Vector::UMaterialExpressionConstant2Vector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 1, 0, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionConstant2Vector::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->Constant2(R,G);
}

void UMaterialExpressionConstant2Vector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf( TEXT("%.3g,%.3g"), R, G ));
}

FString UMaterialExpressionConstant2Vector::GetDescription() const
{
	FString Result = FString(*GetClass()->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
	Result += TEXT(" (");
	Result += Super::GetDescription();
	Result += TEXT(")");
	return Result;
}
#endif // WITH_EDITOR

UMaterialExpressionConstant3Vector::UMaterialExpressionConstant3Vector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 1, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionConstant3Vector::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->Constant3(Constant.R,Constant.G,Constant.B);
}

void UMaterialExpressionConstant3Vector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf( TEXT("%.3g,%.3g,%.3g"), Constant.R, Constant.G, Constant.B ));
}

FString UMaterialExpressionConstant3Vector::GetDescription() const
{
	FString Result = FString(*GetClass()->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
	Result += TEXT(" (");
	Result += Super::GetDescription();
	Result += TEXT(")");
	return Result;
}
#endif // WITH_EDITOR

UMaterialExpressionConstant4Vector::UMaterialExpressionConstant4Vector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("R"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("G"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("B"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionConstant4Vector::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->Constant4(Constant.R,Constant.G,Constant.B,Constant.A);
}


void UMaterialExpressionConstant4Vector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf( TEXT("%.2g,%.2g,%.2g,%.2g"), Constant.R, Constant.G, Constant.B, Constant.A ));
}

FString UMaterialExpressionConstant4Vector::GetDescription() const
{
	FString Result = FString(*GetClass()->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
	Result += TEXT(" (");
	Result += Super::GetDescription();
	Result += TEXT(")");
	return Result;
}
#endif // WITH_EDITOR

void UMaterialExpressionClamp::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	if (UnderlyingArchive.IsLoading() && UnderlyingArchive.UEVer() < VER_UE4_RETROFIT_CLAMP_EXPRESSIONS_SWAP)
	{
		if (ClampMode == CMODE_ClampMin)
		{
			ClampMode = CMODE_ClampMax;
		}
		else if (ClampMode == CMODE_ClampMax)
		{
			ClampMode = CMODE_ClampMin;
		}
	}
}

#if WITH_EDITOR
int32 UMaterialExpressionClamp::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Clamp input"));
	}
	else
	{
		const int32 MinIndex = Min.GetTracedInput().Expression ? Min.Compile(Compiler) : Compiler->Constant(MinDefault);
		const int32 MaxIndex = Max.GetTracedInput().Expression ? Max.Compile(Compiler) : Compiler->Constant(MaxDefault);

		if (ClampMode == CMODE_Clamp)
		{
			return Compiler->Clamp(Input.Compile(Compiler), MinIndex, MaxIndex);
		}
		else if (ClampMode == CMODE_ClampMin)
		{
			return Compiler->Max(Input.Compile(Compiler), MinIndex);
		}
		else if (ClampMode == CMODE_ClampMax)
		{
			return Compiler->Min(Input.Compile(Compiler), MaxIndex);
		}
		return INDEX_NONE;
	}
}

void UMaterialExpressionClamp::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Clamp"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionSaturate
//
#if WITH_EDITOR
int32 UMaterialExpressionSaturate::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Saturate input"));
	}
	
	return Compiler->Saturate(Input.Compile(Compiler));
}

void UMaterialExpressionSaturate::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Saturate"));
}

void UMaterialExpressionSaturate::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Clamps the value between 0 and 1. Saturate is free on most modern graphics hardware."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionMin
//
#if WITH_EDITOR
int32 UMaterialExpressionMin::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Min(Arg1, Arg2);
}

void UMaterialExpressionMin::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Min"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionMax
//
#if WITH_EDITOR
int32 UMaterialExpressionMax::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg1 = A.GetTracedInput().Expression ? A.Compile(Compiler) : Compiler->Constant(ConstA);
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 Arg2 = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	return Compiler->Max(Arg1, Arg2);
}

void UMaterialExpressionMax::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Max"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionTextureCoordinate
//

UMaterialExpressionTextureCoordinate::UMaterialExpressionTextureCoordinate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureCoordinate::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	Compiler->SetPotentiallyManipulateTexCoords();

	// Depending on whether we have U and V scale values that differ, we can perform a multiply by either
	// a scalar or a float2.  These tiling values are baked right into the shader node, so they're always
	// known at compile time.
	if( FMath::Abs( UTiling - VTiling ) > UE_SMALL_NUMBER )
	{
		return Compiler->Mul(Compiler->TextureCoordinate(CoordinateIndex, UnMirrorU, UnMirrorV),Compiler->Constant2(UTiling, VTiling));
	}
	else if(FMath::Abs(1.0f - UTiling) > UE_SMALL_NUMBER)
	{
		return Compiler->Mul(Compiler->TextureCoordinate(CoordinateIndex, UnMirrorU, UnMirrorV),Compiler->Constant(UTiling));
	}
	else
	{
		// Avoid emitting the multiply by 1.0f if possible
		// This should make generated HLSL a bit cleaner, but more importantly will help avoid generating redundant virtual texture stacks
		return Compiler->TextureCoordinate(CoordinateIndex, UnMirrorU, UnMirrorV);
	}
}

void UMaterialExpressionTextureCoordinate::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("TexCoord[%i]"), CoordinateIndex));
}


bool UMaterialExpressionTextureCoordinate::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	TArray<FString> Captions;
	GetCaption(Captions);
	for (const FString& Caption : Captions)
	{
		if (Caption.Contains(SearchQuery))
		{
			return true;
		}
	}

	return Super::MatchesSearchQuery(SearchQuery);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionDotProduct::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing DotProduct input A"));
	}
	else if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing DotProduct input B"));
	}
	else
	{
		int32 Arg1 = A.Compile(Compiler);
		int32 Arg2 = B.Compile(Compiler);
		return Compiler->Dot(
			Arg1,
			Arg2
			);
	}
}

void UMaterialExpressionDotProduct::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Dot"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionCrossProduct::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing CrossProduct input A"));
	}
	else if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing CrossProduct input B"));
	}
	else
	{
		int32 Arg1 = A.Compile(Compiler);
		int32 Arg2 = B.Compile(Compiler);
		return Compiler->Cross(
			Arg1,
			Arg2
			);
	}
}

void UMaterialExpressionCrossProduct::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Cross"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
static FString BuildStaticComponentMaskOutputText(const bool R, const bool G, const bool B, const bool A)
{
	FString Str("( ");
	if (R) Str += TEXT("R ");
	if (G) Str += TEXT("G ");
	if (B) Str += TEXT("B ");
	if (A) Str += TEXT("A ");
	Str += TEXT(")");

	return Str;
}

int32 UMaterialExpressionComponentMask::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing ComponentMask input"));
	}

	return Compiler->ComponentMask(
		Input.Compile(Compiler),
		R,
		G,
		B,
		A
		);
}

void UMaterialExpressionComponentMask::GetCaption(TArray<FString>& OutCaptions) const
{
	const FString Caption = FString("Mask ") + BuildStaticComponentMaskOutputText(R, G, B, A);
	OutCaptions.Add(Caption);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionStaticComponentMaskParameter
//
#if WITH_EDITOR
int32 UMaterialExpressionStaticComponentMaskParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing ComponentMaskParameter input"));
	}
	else
	{
		return Compiler->StaticComponentMask(
			Input.Compile(Compiler),
			ParameterName,
			DefaultR,
			DefaultG,
			DefaultB,
			DefaultA
			);
	}
}

void UMaterialExpressionStaticComponentMaskParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	const FString Caption = FString("Mask ") + BuildStaticComponentMaskOutputText(DefaultR, DefaultG, DefaultB, DefaultA);
	OutCaptions.Add(Caption);
}

void UMaterialExpressionStaticComponentMaskParameter::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	const FString MaskOutput = BuildStaticComponentMaskOutputText(DefaultR, DefaultG, DefaultB, DefaultA);

	OutToolTip.Add(TEXT("Outputs the selected channels."));
	OutToolTip.Add(TEXT("Note: This DOES create new permutations."));
	OutToolTip.Add(FString::Printf(TEXT("Current output: %s "), *MaskOutput));
}

bool  UMaterialExpressionStaticComponentMaskParameter::SetParameterValue(FName InParameterName, bool InR, bool InG, bool InB, bool InA, FGuid InExpressionGuid, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		const bool bHasRChanged = (DefaultR != InR);
		const bool bHasGChanged = (DefaultG != InG);
		const bool bHasBChanged = (DefaultB != InB);
		const bool bHasAChanged = (DefaultA != InA);

		if (bHasRChanged || bHasGChanged || bHasBChanged || bHasAChanged)
		{
			DefaultR = InR;
			DefaultG = InG;
			DefaultB = InB;
			DefaultA = InA;
			if (!EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::NoUpdateExpressionGuid))
			{
				ExpressionGUID = InExpressionGuid;
			}
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				if (bHasRChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultR));
				}
				if (bHasGChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultG));
				}
				if (bHasBChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultB));
				}
				if (bHasAChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultA));
				}
			}
		}
		return true;
	}

	return false;
}

//
//	UMaterialExpressionTime
//
int32 UMaterialExpressionTime::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return bIgnorePause ? Compiler->RealTime(bOverride_Period, Period) : Compiler->GameTime(bOverride_Period, Period);
}

void UMaterialExpressionTime::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bOverride_Period)
	{
		if (Period == 0.0f)
		{
			OutCaptions.Add(TEXT("Time (Stopped)"));
		}
		else
		{
			OutCaptions.Add(FString::Printf(TEXT("Time (Period of %.2f)"), Period));
		}
	}
	else
	{
		OutCaptions.Add(TEXT("Time"));
	}
}

int32 UMaterialExpressionCameraVectorWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->CameraVector();
}

void UMaterialExpressionCameraVectorWS::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Camera Vector"));
}

int32 UMaterialExpressionCameraPositionWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ViewProperty(MEVP_WorldSpaceCameraPosition);
}

void UMaterialExpressionCameraPositionWS::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Camera Position"));
}

//
//	UMaterialExpressionReflectionVectorWS
//
int32 UMaterialExpressionReflectionVectorWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 Result = CustomWorldNormal.Compile(Compiler);
	if (CustomWorldNormal.Expression) 
	{
		// Don't do anything special here in regards to if the Expression is a Reroute node, the compiler will handle properly internally and return INDEX_NONE if rerouted to nowhere.
		return Compiler->ReflectionAboutCustomWorldNormal(Result, bNormalizeCustomWorldNormal); 
	}
	else
	{
		return Compiler->ReflectionVector();
	}
}

void UMaterialExpressionReflectionVectorWS::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Reflection Vector"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionPanner
//
UMaterialExpressionPanner::UMaterialExpressionPanner(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionPanner::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 TimeArg = Time.GetTracedInput().Expression ? Time.Compile(Compiler) : Compiler->GameTime(false, 0.0f);
	bool bIsSpeedExpressionValid = Speed.GetTracedInput().Expression != nullptr;
	int32 SpeedVectorArg = bIsSpeedExpressionValid ? Speed.Compile(Compiler) : INDEX_NONE;
	int32 SpeedXArg = bIsSpeedExpressionValid ? Compiler->ComponentMask(SpeedVectorArg, true, false, false, false) : Compiler->Constant(SpeedX);
	int32 SpeedYArg = bIsSpeedExpressionValid ? Compiler->ComponentMask(SpeedVectorArg, false, true, false, false) : Compiler->Constant(SpeedY);
	int32 Arg1;
	int32 Arg2;
	if (bFractionalPart)
	{
		// Note: this is to avoid (delay) divergent accuracy issues as GameTime increases.
		// TODO: C++ to calculate its phase via per frame time delta.
		Arg1 = Compiler->PeriodicHint(Compiler->Frac(Compiler->Mul(TimeArg, SpeedXArg)));
		Arg2 = Compiler->PeriodicHint(Compiler->Frac(Compiler->Mul(TimeArg, SpeedYArg)));
	}
	else
	{
		Arg1 = Compiler->PeriodicHint(Compiler->Mul(TimeArg, SpeedXArg));
		Arg2 = Compiler->PeriodicHint(Compiler->Mul(TimeArg, SpeedYArg));
	}

	int32 Arg3 = Coordinate.GetTracedInput().Expression ? Coordinate.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);
	return Compiler->Add(
			Compiler->AppendVector(
				Arg1,
				Arg2
				),
			Arg3
			);
}

void UMaterialExpressionPanner::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Panner"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionRotator
//
UMaterialExpressionRotator::UMaterialExpressionRotator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionRotator::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32	Cosine = Compiler->Cosine(Compiler->Mul(Time.GetTracedInput().Expression ? Time.Compile(Compiler) : Compiler->GameTime(false, 0.0f),Compiler->Constant(Speed))),
		Sine = Compiler->Sine(Compiler->Mul(Time.GetTracedInput().Expression ? Time.Compile(Compiler) : Compiler->GameTime(false, 0.0f),Compiler->Constant(Speed))),
		RowX = Compiler->AppendVector(Cosine,Compiler->Mul(Compiler->Constant(-1.0f),Sine)),
		RowY = Compiler->AppendVector(Sine,Cosine),
		Origin = Compiler->Constant2(CenterX,CenterY),
		BaseCoordinate = Coordinate.GetTracedInput().Expression ? Coordinate.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);

	const int32 Arg1 = Compiler->Dot(RowX,Compiler->Sub(Compiler->ComponentMask(BaseCoordinate,1,1,0,0),Origin));
	const int32 Arg2 = Compiler->Dot(RowY,Compiler->Sub(Compiler->ComponentMask(BaseCoordinate,1,1,0,0),Origin));

	if(Compiler->GetType(BaseCoordinate) == MCT_Float3)
		return Compiler->AppendVector(
				Compiler->Add(
					Compiler->AppendVector(
						Arg1,
						Arg2
						),
					Origin
					),
				Compiler->ComponentMask(BaseCoordinate,0,0,1,0)
				);
	else
	{
		const int32 ArgOne = Compiler->Dot(RowX,Compiler->Sub(BaseCoordinate,Origin));
		const int32 ArgTwo = Compiler->Dot(RowY,Compiler->Sub(BaseCoordinate,Origin));

		return Compiler->Add(
				Compiler->AppendVector(
					ArgOne,
					ArgTwo
					),
				Origin
				);
	}
}

void UMaterialExpressionRotator::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Rotator"));
}

//
//	UMaterialExpressionSine
//
int32 UMaterialExpressionSine::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Sine input"));
	}

	return Compiler->Sine(Period > 0.0f ? Compiler->Mul(Input.Compile(Compiler),Compiler->Constant(2.0f * (float)UE_PI / Period)) : Input.Compile(Compiler));
}

void UMaterialExpressionSine::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Sine"));
}

//
//	UMaterialExpressionCosine
//
int32 UMaterialExpressionCosine::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Cosine input"));
	}

	return Compiler->Cosine(Compiler->Mul(Input.Compile(Compiler),Period > 0.0f ? Compiler->Constant(2.0f * (float)UE_PI / Period) : 0));
}

void UMaterialExpressionCosine::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Cosine"));
}

//
//	UMaterialExpressionTangent
//
int32 UMaterialExpressionTangent::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Tangent input"));
	}

	return Compiler->Tangent(Compiler->Mul(Input.Compile(Compiler),Period > 0.0f ? Compiler->Constant(2.0f * (float)UE_PI / Period) : 0));
}

void UMaterialExpressionTangent::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Tangent"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArcsine
//
#if WITH_EDITOR
int32 UMaterialExpressionArcsine::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Arcsine input"));
	}

	return Compiler->Arcsine(Input.Compile(Compiler));
}

void UMaterialExpressionArcsine::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Arcsine"));
}

void UMaterialExpressionArcsine::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Inverse sine function. This is an expensive operation not reflected by instruction count."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArcsineFast
//
#if WITH_EDITOR
int32 UMaterialExpressionArcsineFast::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing ArcsineFast input"));
	}

	return Compiler->ArcsineFast(Input.Compile(Compiler));
}

void UMaterialExpressionArcsineFast::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ArcsineFast"));
}

void UMaterialExpressionArcsineFast::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Approximate inverse sine function. Input must be between -1 and 1."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArccosine
//
#if WITH_EDITOR
int32 UMaterialExpressionArccosine::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Arccosine input"));
	}

	return Compiler->Arccosine(Input.Compile(Compiler));
}

void UMaterialExpressionArccosine::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Arccosine"));
}

void UMaterialExpressionArccosine::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Inverse cosine function. This is an expensive operation not reflected by instruction count."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArccosineFast
//
#if WITH_EDITOR
int32 UMaterialExpressionArccosineFast::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing ArccosineFast input"));
	}

	return Compiler->ArccosineFast(Input.Compile(Compiler));
}

void UMaterialExpressionArccosineFast::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ArccosineFast"));
}

void UMaterialExpressionArccosineFast::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Approximate inverse cosine function. Input must be between -1 and 1."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArctangent
//
#if WITH_EDITOR
int32 UMaterialExpressionArctangent::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Arctangent input"));
	}

	return Compiler->Arctangent(Input.Compile(Compiler));
}

void UMaterialExpressionArctangent::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Arctangent"));
}

void UMaterialExpressionArctangent::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Inverse tangent function. This is an expensive operation not reflected by instruction count."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArctangentFast
//
#if WITH_EDITOR
int32 UMaterialExpressionArctangentFast::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing ArctangentFast input"));
	}

	return Compiler->ArctangentFast(Input.Compile(Compiler));
}

void UMaterialExpressionArctangentFast::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ArctangentFast"));
}

void UMaterialExpressionArctangentFast::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Approximate inverse tangent function."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArctangent2
//
#if WITH_EDITOR
int32 UMaterialExpressionArctangent2::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Y.GetTracedInput().Expression || !X.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Arctangent2 input"));
	}

	int32 YResult = Y.Compile(Compiler);
	int32 XResult = X.Compile(Compiler);
	return Compiler->Arctangent2(YResult, XResult);
}

void UMaterialExpressionArctangent2::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Arctangent2"));
}

void UMaterialExpressionArctangent2::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Inverse tangent of X / Y where input signs are used to determine quadrant. This is an expensive operation not reflected by instruction count."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionArctangent2Fast
//
#if WITH_EDITOR
int32 UMaterialExpressionArctangent2Fast::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Y.GetTracedInput().Expression || !X.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Arctangent2Fast input"));
	}

	int32 YResult = Y.Compile(Compiler);
	int32 XResult = X.Compile(Compiler);
	return Compiler->Arctangent2Fast(YResult, XResult);
}

void UMaterialExpressionArctangent2Fast::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Arctangent2Fast"));
}

void UMaterialExpressionArctangent2Fast::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Approximate inverse tangent of X / Y where input signs are used to determine quadrant."), 40, OutToolTip);
}
#endif // WITH_EDITOR

UMaterialExpressionBumpOffset::UMaterialExpressionBumpOffset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionBumpOffset::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Height.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Height input"));
	}

	return Compiler->Add(
			Compiler->Mul(
				Compiler->ComponentMask(Compiler->TransformVector(MCB_World, MCB_Tangent, Compiler->CameraVector()),1,1,0,0),
				Compiler->Add(
					Compiler->Mul(
						HeightRatioInput.GetTracedInput().Expression ? Compiler->ForceCast(HeightRatioInput.Compile(Compiler),MCT_Float1) : Compiler->Constant(HeightRatio),
						Compiler->ForceCast(Height.Compile(Compiler),MCT_Float1)
						),
					HeightRatioInput.GetTracedInput().Expression ? Compiler->Mul(Compiler->Constant(-ReferencePlane), Compiler->ForceCast(HeightRatioInput.Compile(Compiler),MCT_Float1)) : Compiler->Constant(-ReferencePlane * HeightRatio)
					)
				),
			Coordinate.GetTracedInput().Expression ? Coordinate.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false)
			);
}

void UMaterialExpressionBumpOffset::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("BumpOffset"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionAppendVector::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing AppendVector input A"));
	}
	else if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing AppendVector input B"));
	}
	else
	{
		int32 Arg1 = A.Compile(Compiler);
		int32 Arg2 = B.Compile(Compiler);
		return Compiler->AppendVector(
			Arg1,
			Arg2
			);
	}
}

void UMaterialExpressionAppendVector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Append"));
}
#endif // WITH_EDITOR

// -----
FExpressionInput* UMaterialExpressionMakeMaterialAttributes::GetExpressionInput(EMaterialProperty InProperty)
{
	switch (InProperty)
	{
	case MP_BaseColor: return &BaseColor;
	case MP_Specular: return &Specular;
	case MP_Normal: return &Normal;
	case MP_Tangent: return &Tangent;
	case MP_Metallic: return &Metallic;
	case MP_Roughness: return &Roughness;
	case MP_Anisotropy: return &Anisotropy;
	case MP_AmbientOcclusion: return &AmbientOcclusion;
	case MP_EmissiveColor: return &EmissiveColor;
	case MP_Opacity: return &Opacity;
	case MP_OpacityMask: return &OpacityMask;
	case MP_SubsurfaceColor: return &SubsurfaceColor;
	case MP_WorldPositionOffset: return &WorldPositionOffset;
	case MP_Displacement: return &Displacement;
	case MP_ShadingModel: return &ShadingModel;
	case MP_Refraction: return &Refraction;
	case MP_PixelDepthOffset: return &PixelDepthOffset;
	case MP_CustomizedUVs0: return &CustomizedUVs[0];
	case MP_CustomizedUVs1: return &CustomizedUVs[1];
	case MP_CustomizedUVs2: return &CustomizedUVs[2];
	case MP_CustomizedUVs3: return &CustomizedUVs[3];
	case MP_CustomizedUVs4: return &CustomizedUVs[4];
	case MP_CustomizedUVs5: return &CustomizedUVs[5];
	case MP_CustomizedUVs6: return &CustomizedUVs[6];
	case MP_CustomizedUVs7: return &CustomizedUVs[7];
	case MP_CustomData0: return &ClearCoat;
	case MP_CustomData1: return &ClearCoatRoughness;
	default: break; // We don't support this property.
	}

	return nullptr;
}

void UMaterialExpressionMakeMaterialAttributes::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
#if WITH_EDITOR
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();
	UnderlyingArchive.UsingCustomVersion(FRenderingObjectVersion::GUID);
	if (UnderlyingArchive.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::FixedLegacyMaterialAttributeNodeTypes)
	{
		// Update the legacy masks else fail on vec3 to vec2 conversion
		Refraction.SetMask(1, 1, 1, 0, 0);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

// Return a conservative list of connected inputs
uint64 UMaterialExpressionMakeMaterialAttributes::GetConnectedInputs() const
{
	uint64 Out = 0ull;
	if (BaseColor.Expression != nullptr) 				Out |= (1ull << uint64(MP_BaseColor));
	if (Metallic.Expression != nullptr) 				Out |= (1ull << uint64(MP_Metallic));
	if (Specular.Expression != nullptr) 				Out |= (1ull << uint64(MP_Specular));
	if (Roughness.Expression != nullptr) 				Out |= (1ull << uint64(MP_Roughness));
	if (Anisotropy.Expression != nullptr) 				Out |= (1ull << uint64(MP_Anisotropy));
	if (EmissiveColor.Expression != nullptr) 			Out |= (1ull << uint64(MP_EmissiveColor));
	if (Opacity.Expression != nullptr) 					Out |= (1ull << uint64(MP_Opacity));
	if (OpacityMask.Expression != nullptr) 				Out |= (1ull << uint64(MP_OpacityMask));
	if (Normal.Expression != nullptr) 					Out |= (1ull << uint64(MP_Normal));
	if (Tangent.Expression != nullptr) 					Out |= (1ull << uint64(MP_Tangent));
	if (WorldPositionOffset.Expression != nullptr) 		Out |= (1ull << uint64(MP_WorldPositionOffset));
	if (Displacement.Expression != nullptr) 			Out |= (1ull << uint64(MP_Displacement));
	if (SubsurfaceColor.Expression != nullptr) 			Out |= (1ull << uint64(MP_SubsurfaceColor));
	if (ClearCoat.Expression != nullptr) 				Out |= (1ull << uint64(MP_CustomData0));
	if (ClearCoatRoughness.Expression != nullptr) 		Out |= (1ull << uint64(MP_CustomData1));
	if (AmbientOcclusion.Expression != nullptr) 		Out |= (1ull << uint64(MP_AmbientOcclusion));
	if (Refraction.Expression != nullptr) 				Out |= (1ull << uint64(MP_Refraction));
	if (PixelDepthOffset.Expression != nullptr) 		Out |= (1ull << uint64(MP_PixelDepthOffset));
	if (ShadingModel.Expression != nullptr) 			Out |= (1ull << uint64(MP_ShadingModel));
	return Out;
}

int32 UMaterialExpressionMakeMaterialAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) 
{
	int32 Ret = INDEX_NONE;
	UMaterialExpression* Expression = nullptr;

 	static_assert(MP_MAX == 35, 
		"New material properties should be added to the end of the inputs for this expression. \
		The order of properties here should match the material results pins, the make material attriubtes node inputs and the mapping of IO indices to properties in GetMaterialPropertyFromInputOutputIndex().\
		Insertions into the middle of the properties or a change in the order of properties will also require that existing data is fixed up in DoMaterialAttributeReorder().\
		");

	EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(Compiler->GetMaterialAttribute());
	// We don't worry about reroute nodes in the switch, as we have a test for their validity afterwards.
	switch (Property)
	{
	case MP_BaseColor: Ret = BaseColor.Compile(Compiler); Expression = BaseColor.Expression; break;
	case MP_Metallic: Ret = Metallic.Compile(Compiler); Expression = Metallic.Expression; break;
	case MP_Specular: Ret = Specular.Compile(Compiler); Expression = Specular.Expression; break;
	case MP_Roughness: Ret = Roughness.Compile(Compiler); Expression = Roughness.Expression; break;
	case MP_Anisotropy: Ret = Anisotropy.Compile(Compiler); Expression = Anisotropy.Expression; break;
	case MP_EmissiveColor: Ret = EmissiveColor.Compile(Compiler); Expression = EmissiveColor.Expression; break;
	case MP_Opacity: Ret = Opacity.Compile(Compiler); Expression = Opacity.Expression; break;
	case MP_OpacityMask: Ret = OpacityMask.Compile(Compiler); Expression = OpacityMask.Expression; break;
	case MP_Normal: Ret = Normal.Compile(Compiler); Expression = Normal.Expression; break;
	case MP_Tangent: Ret = Tangent.Compile(Compiler); Expression = Tangent.Expression; break;
	case MP_WorldPositionOffset: Ret = WorldPositionOffset.Compile(Compiler); Expression = WorldPositionOffset.Expression; break;
	case MP_Displacement: Ret = Displacement.Compile(Compiler); Expression = Displacement.Expression; break;
	case MP_SubsurfaceColor: Ret = SubsurfaceColor.Compile(Compiler); Expression = SubsurfaceColor.Expression; break;
	case MP_CustomData0: Ret = ClearCoat.Compile(Compiler); Expression = ClearCoat.Expression; break;
	case MP_CustomData1: Ret = ClearCoatRoughness.Compile(Compiler); Expression = ClearCoatRoughness.Expression; break;
	case MP_AmbientOcclusion: Ret = AmbientOcclusion.Compile(Compiler); Expression = AmbientOcclusion.Expression; break;
	case MP_Refraction: Ret = Refraction.Compile(Compiler); Expression = Refraction.Expression; break;
	case MP_PixelDepthOffset: Ret = PixelDepthOffset.Compile(Compiler); Expression = PixelDepthOffset.Expression; break;
	case MP_ShadingModel: Ret = ShadingModel.Compile(Compiler); Expression = ShadingModel.Expression; break;
	};

	if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
	{
		Ret = CustomizedUVs[Property - MP_CustomizedUVs0].Compile(Compiler); Expression = CustomizedUVs[Property - MP_CustomizedUVs0].Expression;
	}

	//If we've connected an expression but its still returned INDEX_NONE, flag the error. This also catches reroute nodes to nowhere.
	if (Expression && INDEX_NONE == Ret)
	{
		Compiler->Errorf(TEXT("Error on property %s"), *FMaterialAttributeDefinitionMap::GetAttributeName(Property));
	}

	return Ret;
}

void UMaterialExpressionMakeMaterialAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MakeMaterialAttributes"));
}

EMaterialValueType UMaterialExpressionMakeMaterialAttributes::GetInputValueType(int32 InputIndex)
{
	if (GetInputName(InputIndex).IsEqual("ShadingModel"))
	{
		return MCT_ShadingModel;
	}
	else
	{
		return UMaterialExpression::GetInputValueType(InputIndex);
	}
}

int32 UMaterialExpressionMakeMaterialAttributes::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Material attributes "Compile" will return the value of the current attribute at the top of the compiler stack. By default it is EmissiveColor, so can be just black.
	// Instead we use BaseColor as a better preview representative for material attributes.
	FExpressionInput BaseColorTraced = BaseColor.GetTracedInput();
	UMaterialExpression* BaseColorExpression = BaseColor.Expression;
	if (BaseColorExpression)
	{
		return BaseColor.Compile(Compiler); 
	}
	return Compiler->Constant3(0, 0, 0);
}
#endif // WITH_EDITOR

// -----

UMaterialExpressionBreakMaterialAttributes::UMaterialExpressionBreakMaterialAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	bShowMaskColorsOnPin = false;

 	static_assert(MP_MAX == 35, 
		"New material properties should be added to the end of the outputs for this expression. \
		The order of properties here should match the material results pins, the make material attributes node inputs and the mapping of IO indices to properties in GetMaterialPropertyFromInputOutputIndex().\
		Insertions into the middle of the properties or a change in the order of properties will also require that existing data is fixed up in DoMaterialAttributesReorder().\
		");

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("BaseColor"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Metallic"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Specular"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Roughness"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Anisotropy"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("EmissiveColor"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Opacity"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("OpacityMask"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Normal"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Tangent"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("WorldPositionOffset"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("SubsurfaceColor"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("ClearCoat"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("ClearCoatRoughness"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("AmbientOcclusion"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Refraction"), 1, 1, 1, 0, 0));

	for (int32 UVIndex = 0; UVIndex <= MP_CustomizedUVs7 - MP_CustomizedUVs0; UVIndex++)
	{
		Outputs.Add(FExpressionOutput(*FString::Printf(TEXT("CustomizedUV%u"), UVIndex), 1, 1, 1, 0, 0));
	}

	Outputs.Add(FExpressionOutput(TEXT("PixelDepthOffset"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("ShadingModel"), 0, 0, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Displacement"), 1, 1, 0, 0, 0));
#endif
}

void UMaterialExpressionBreakMaterialAttributes::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	UnderlyingArchive.UsingCustomVersion(FRenderingObjectVersion::GUID);

#if WITH_EDITOR
	if (UnderlyingArchive.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::FixedLegacyMaterialAttributeNodeTypes)
	{
		// Update the masks for legacy content
		int32 OutputIndex = 0;

		Outputs[OutputIndex].SetMask(1, 1, 1, 1, 0); ++OutputIndex; // BaseColor
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // Metallic
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // Specular
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // Roughness
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // Anisotropy
		Outputs[OutputIndex].SetMask(1, 1, 1, 1, 0); ++OutputIndex; // EmissiveColor
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // Opacity
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // OpacityMask
		Outputs[OutputIndex].SetMask(1, 1, 1, 1, 0); ++OutputIndex; // Normal
		Outputs[OutputIndex].SetMask(1, 1, 1, 1, 0); ++OutputIndex; // Tangent
		Outputs[OutputIndex].SetMask(1, 1, 1, 1, 0); ++OutputIndex; // WorldPositionOffset
		Outputs[OutputIndex].SetMask(1, 1, 1, 1, 0); ++OutputIndex; // SubsurfaceColor
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // ClearCoat
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // ClearCoatRoughness 
		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex; // AmbientOcclusion
		Outputs[OutputIndex].SetMask(1, 1, 1, 0, 0); ++OutputIndex; // Refraction
		
		for (int32 i = 0; i <= MP_CustomizedUVs7 - MP_CustomizedUVs0; ++i, ++OutputIndex)
		{
			Outputs[OutputIndex].SetMask(1, 1, 1, 0, 0);
		}

		Outputs[OutputIndex].SetMask(1, 1, 0, 0, 0); ++OutputIndex;// PixelDepthOffset
		Outputs[OutputIndex].SetMask(0, 0, 0, 0, 0); // ShadingModelFromMaterialExpression
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
TMap<EMaterialProperty, int32> UMaterialExpressionBreakMaterialAttributes::PropertyToIOIndexMap;

void UMaterialExpressionBreakMaterialAttributes::BuildPropertyToIOIndexMap()
{
	if (PropertyToIOIndexMap.Num() == 0)
	{
		PropertyToIOIndexMap.Add(MP_BaseColor,				0);
		PropertyToIOIndexMap.Add(MP_Metallic,				1);
		PropertyToIOIndexMap.Add(MP_Specular,				2);
		PropertyToIOIndexMap.Add(MP_Roughness,				3);
		PropertyToIOIndexMap.Add(MP_Anisotropy,				4);
		PropertyToIOIndexMap.Add(MP_EmissiveColor,			5);
		PropertyToIOIndexMap.Add(MP_Opacity,				6);
		PropertyToIOIndexMap.Add(MP_OpacityMask,			7);
		PropertyToIOIndexMap.Add(MP_Normal,					8);
		PropertyToIOIndexMap.Add(MP_Tangent,				9);
		PropertyToIOIndexMap.Add(MP_WorldPositionOffset,	10);
		PropertyToIOIndexMap.Add(MP_SubsurfaceColor,		11);
		PropertyToIOIndexMap.Add(MP_CustomData0,			12);
		PropertyToIOIndexMap.Add(MP_CustomData1,			13);
		PropertyToIOIndexMap.Add(MP_AmbientOcclusion,		14);
		PropertyToIOIndexMap.Add(MP_Refraction,				15);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs0,			16);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs1,			17);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs2,			18);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs3,			19);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs4,			20);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs5,			21);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs6,			22);
		PropertyToIOIndexMap.Add(MP_CustomizedUVs7,			23);
		PropertyToIOIndexMap.Add(MP_PixelDepthOffset,		24);
		PropertyToIOIndexMap.Add(MP_ShadingModel,			25);
		PropertyToIOIndexMap.Add(MP_Displacement,			26);
	}
}

int32 UMaterialExpressionBreakMaterialAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	BuildPropertyToIOIndexMap();

	// Here we don't care about any multiplex index coming in.
	// We pass through our output index as the multiplex index so the MakeMaterialAttriubtes node at the other end can send us the right data.
	const EMaterialProperty* Property = PropertyToIOIndexMap.FindKey(OutputIndex);

	if (!Property)
	{
		return Compiler->Errorf(TEXT("Tried to compile material attributes?"));
	}
	else
	{
		return MaterialAttributes.CompileWithDefault(Compiler, FMaterialAttributeDefinitionMap::GetID(*Property));
	}
}

void UMaterialExpressionBreakMaterialAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("BreakMaterialAttributes"));
}


FName UMaterialExpressionBreakMaterialAttributes::GetInputName(int32 InputIndex) const
{
	if( 0 == InputIndex )
	{
		return *NSLOCTEXT("BreakMaterialAttributes", "InputName", "Attr").ToString();
	}
	return NAME_None;
}

bool UMaterialExpressionBreakMaterialAttributes::IsInputConnectionRequired(int32 InputIndex) const
{
	return true;
}

EMaterialValueType UMaterialExpressionBreakMaterialAttributes::GetOutputValueType(int32 OutputIndex)
{
	BuildPropertyToIOIndexMap();

	const EMaterialProperty* Property = PropertyToIOIndexMap.FindKey(OutputIndex);

	if (Property && *Property == EMaterialProperty::MP_ShadingModel)
	{
		return MCT_ShadingModel;
	}
	else
	{
		return UMaterialExpression::GetOutputValueType(OutputIndex);
	}
}
#endif // WITH_EDITOR

// -----

#define GET_SET_MA_MATERIALATTRIBUTESINDEX 0
UMaterialExpressionGetMaterialAttributes::UMaterialExpressionGetMaterialAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

#if WITH_EDITOR
	// Add default output pins
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("MaterialAttributes"), 0, 0, 0, 0, 0));

	CachedInputs.Empty();
	CachedInputs.Push(&MaterialAttributes);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionGetMaterialAttributes::CreateOrGetOutputAttribute(EMaterialProperty Attribute)
{
	int32 OutputIndex = INDEX_NONE;
	if (Attribute == MP_MaterialAttributes)
	{
		OutputIndex = GET_SET_MA_MATERIALATTRIBUTESINDEX;
	}
	else
	{
		FGuid AttributeId = FMaterialAttributeDefinitionMap::GetID(Attribute);
		if (AttributeGetTypes.Find(AttributeId, OutputIndex))
		{
			/**
			* Add one to compensate for the AttributeGetTypes list not containing MP_MaterialAttributes
			* It's none trivial to iterate the Outputs list for the matching attribute so this is a simpler solution.
			*/
			OutputIndex++;
		}
		else
		{
			int32 GetTypesIndex = AttributeGetTypes.Add(AttributeId);
			if(GetTypesIndex != INDEX_NONE)
			{
				PreEditChange(nullptr);
				FString AttributeName = FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeGetTypes[GetTypesIndex], Material).ToString();
				OutputIndex = Outputs.Add(FExpressionOutput(*AttributeName, 0, 0, 0, 0, 0));
			}
		}
	}
	return OutputIndex;
}

int32 UMaterialExpressionGetMaterialAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Verify setup
	const int32 NumOutputPins = AttributeGetTypes.Num();
	for (int32 i = 0; i < NumOutputPins; ++i)
	{
		for (int j = i + 1; j < NumOutputPins; ++j)
		{
			if (AttributeGetTypes[i] == AttributeGetTypes[j])
			{
				return Compiler->Errorf(TEXT("Duplicate attribute types."));
			}
		}

		if (FMaterialAttributeDefinitionMap::GetProperty(AttributeGetTypes[i]) == MP_MAX)
		{
			return Compiler->Errorf(TEXT("Property type doesn't exist, needs re-mapping?"));
		}
	}

	// Compile attribute
	int32 Result = INDEX_NONE;

	if (OutputIndex == GET_SET_MA_MATERIALATTRIBUTESINDEX)
	{
		const FGuid AttributeID = Compiler->GetMaterialAttribute();
		Result = MaterialAttributes.CompileWithDefault(Compiler, AttributeID);
	}
	else if (OutputIndex > GET_SET_MA_MATERIALATTRIBUTESINDEX)
	{
		checkf(OutputIndex <= AttributeGetTypes.Num(), TEXT("Requested non-existent pin."));
		Result = MaterialAttributes.CompileWithDefault(Compiler, AttributeGetTypes[OutputIndex-1]);
	}

	return Result;
}

void UMaterialExpressionGetMaterialAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("GetMaterialAttributes"));
}

FName UMaterialExpressionGetMaterialAttributes::GetInputName(int32 InputIndex) const
{
	return NAME_None;
}

EMaterialValueType UMaterialExpressionGetMaterialAttributes::GetOutputValueType(int32 OutputIndex)
{
	// Call base class impl to get the type
	EMaterialValueType OutputType = Super::GetOutputValueType(OutputIndex);

	// Override the type if it's a ShadingModel type
	if (OutputIndex > GET_SET_MA_MATERIALATTRIBUTESINDEX) // "0th" place is the mandatory MaterialAttribute itself, skip it
	{
		ensure(OutputIndex < AttributeGetTypes.Num() + 1);
		EMaterialValueType PinType = FMaterialAttributeDefinitionMap::GetValueType(AttributeGetTypes[OutputIndex - 1]);
		if (PinType == MCT_ShadingModel)
		{
			OutputType = PinType;
		}
		else if (PinType == MCT_Substrate)
		{
			OutputType = PinType;
		}
	}

	return OutputType;
}

bool UMaterialExpressionGetMaterialAttributes::IsResultSubstrateMaterial(int32 OutputIndex)
{	
	if (MaterialAttributes.Expression)
	{
		EMaterialValueType OutputType = IsResultMaterialAttributes(OutputIndex) ? MCT_MaterialAttributes : FMaterialAttributeDefinitionMap::GetValueType(AttributeGetTypes[OutputIndex - 1]);
		switch (OutputType)
		{
			case MCT_Substrate: return true;
			case MCT_MaterialAttributes:
			{
				return MaterialAttributes.Expression->IsResultSubstrateMaterial(MaterialAttributes.OutputIndex);
			}
			default:break;
		}
	}
	return false;
}

void UMaterialExpressionGetMaterialAttributes::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (IsResultSubstrateMaterial(OutputIndex))
	{
		MaterialAttributes.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, MaterialAttributes.OutputIndex);
	}
}

FSubstrateOperator* UMaterialExpressionGetMaterialAttributes::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	if (IsResultSubstrateMaterial(OutputIndex))
	{
		return MaterialAttributes.Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, MaterialAttributes.OutputIndex);
	}
	return nullptr;
}

void UMaterialExpressionGetMaterialAttributes::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Backup attribute array so we can re-connect pins
	PreEditAttributeGetTypes.Empty();
	for (const FGuid& AttributeID : AttributeGetTypes)
	{
		PreEditAttributeGetTypes.Add(AttributeID);
	};

	Super::PreEditChange(PropertyAboutToChange);
}

void UMaterialExpressionGetMaterialAttributes::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty && GraphNode)
	{
		if (PreEditAttributeGetTypes.Num() < AttributeGetTypes.Num())
		{
			// Attribute type added so default out type
			AttributeGetTypes.Last() = FMaterialAttributeDefinitionMap::GetDefaultID();

			// Attempt to find a valid attribute that's not already listed
			const TArray<FGuid>& OrderedVisibleAttributes = FMaterialAttributeDefinitionMap::GetOrderedVisibleAttributeList();
			for (const FGuid& AttributeID : OrderedVisibleAttributes)
			{
				if (PreEditAttributeGetTypes.Find(AttributeID) == INDEX_NONE)
				{
					 AttributeGetTypes.Last() = AttributeID;
					 break;
				}
			}
		
			// Copy final defaults to new output
			FString AttributeName = FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeGetTypes.Last(), Material).ToString();
			Outputs.Add(FExpressionOutput(*AttributeName, 0, 0, 0, 0, 0));

			GraphNode->ReconstructNode();
		}	 
		else if (PreEditAttributeGetTypes.Num() > AttributeGetTypes.Num())
		{
			if (AttributeGetTypes.Num() == 0)
			{
				// All attribute types removed
				while (Outputs.Num() > 1)
				{
					Outputs.Pop();
					GraphNode->RemovePinAt(Outputs.Num(), EGPD_Output);
				}
			}
			else
			{
				// Attribute type removed
				int32 RemovedInputIndex = INDEX_NONE;

				for (int32 Attribute = 0; Attribute < AttributeGetTypes.Num(); ++Attribute)
				{
					// A mismatched attribute type means a middle pin was removed
					if (AttributeGetTypes[Attribute] != PreEditAttributeGetTypes[Attribute])
					{
						RemovedInputIndex = Attribute + 1;
						Outputs.RemoveAt(RemovedInputIndex);
						break;
					}
				};

				if (RemovedInputIndex == INDEX_NONE)
				{
					Outputs.Pop();
					RemovedInputIndex = Outputs.Num();
				}

				GraphNode->RemovePinAt(RemovedInputIndex, EGPD_Output);
			}
		}
		else
		{
			// Type changed, update pin names
			for (int i = 1; i < Outputs.Num(); ++i)
			{
				Outputs[i].OutputName = *FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeGetTypes[i-1], Material).ToString();
			}

			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty( PropertyChangedEvent );
}

void UMaterialExpressionGetMaterialAttributes::PostLoad()
{
	Super::PostLoad();

	// Verify serialized attributes
	check(Outputs.Num() == AttributeGetTypes.Num() + 1);

	// Make sure all outputs have up to date display names
	for (int i = 1; i < Outputs.Num(); ++i)
	{
		const FString DisplayName = FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeGetTypes[i - 1], Material).ToString();
		Outputs[i].OutputName = *DisplayName;
	}
}
#endif // WITH_EDITOR

// -----

UMaterialExpressionSetMaterialAttributes::UMaterialExpressionSetMaterialAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	// Add default input pins
	Inputs.Reset();
	Inputs.Add(FMaterialAttributesInput());
#endif
}

#if WITH_EDITOR
uint64 UMaterialExpressionSetMaterialAttributes::GetConnectedInputs() const
{
	uint64 Out = 0ull;
	const int32 NumInputPins = AttributeSetTypes.Num();
	for (int32 PinIndex = 0; PinIndex < NumInputPins; ++PinIndex)
	{
		const uint64 Bitmask = FMaterialAttributeDefinitionMap::GetBitmask(AttributeSetTypes[PinIndex]);
		const EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(AttributeSetTypes[PinIndex]);
		const FExpressionInput& AttributeInput = Inputs[PinIndex + 1];

		// If Anisotropy has been added to the SetMaterialAttributes list of attributes, but not connected, its input will be set to 0. 
		// This means that the material won't have any anisotropy effect. For this reason we by pass input marking.
		const bool bMarkInputAsConnected = Property != EMaterialProperty::MP_Anisotropy || (Property == EMaterialProperty::MP_Anisotropy && AttributeInput.GetTracedInput().Expression);
		if (bMarkInputAsConnected)
		{
			Out |= Bitmask;
		}
	}
	return Out;
}

int32 UMaterialExpressionSetMaterialAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) 
{
	// Verify setup
	const int32 NumInputPins = AttributeSetTypes.Num();
	for (int32 i = 0; i < NumInputPins; ++i)
	{
		for (int j = i + 1; j < NumInputPins; ++j)
		{
			if (AttributeSetTypes[i] == AttributeSetTypes[j])
			{
				return Compiler->Errorf(TEXT("Duplicate attribute types."));
			}
		}

		if (FMaterialAttributeDefinitionMap::GetProperty(AttributeSetTypes[i]) == MP_MAX)
		{
			return Compiler->Errorf(TEXT("Property type doesn't exist, needs re-mapping?"));
		}
	}

	// Compile attribute
	const FGuid CompilingAttributeID = Compiler->GetMaterialAttribute();
	if (CompilingAttributeID == FMaterialAttributeDefinitionMap::GetID(MP_MaterialAttributes))
	{
		int32 Result = INDEX_NONE;
		if (Inputs[GET_SET_MA_MATERIALATTRIBUTESINDEX].GetTracedInput().Expression)
		{
			Result = Inputs[GET_SET_MA_MATERIALATTRIBUTESINDEX].GetTracedInput().Compile(Compiler);
		}
		else
		{
			Result = Compiler->DefaultMaterialAttributes();
		}

		for (int32 PinIndex = 0; PinIndex < AttributeSetTypes.Num(); ++PinIndex)
		{
			const FExpressionInput& AttributeInput = Inputs[PinIndex + 1];
			if (AttributeInput.GetTracedInput().Expression)
			{
				const FGuid& AttributeID = AttributeSetTypes[PinIndex];
				// Only compile code to set attributes of the current shader frequency
				const EShaderFrequency AttributeFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(AttributeID);
				if (AttributeFrequency == Compiler->GetCurrentShaderFrequency())
				{
					const int32 AttributeResult = AttributeInput.GetTracedInput().Compile(Compiler);
					if (AttributeResult != INDEX_NONE)
					{
						Result = Compiler->SetMaterialAttribute(Result, AttributeResult, AttributeID);
					}
				}
			}
		}
		return Result;
	}
	else
	{
		FExpressionInput* AttributeInput = nullptr;

		int32 PinIndex;
		if (AttributeSetTypes.Find(CompilingAttributeID, PinIndex))
		{
			checkf(PinIndex + 1 < Inputs.Num(), TEXT("Requested non-existent pin."));
			AttributeInput = &Inputs[PinIndex + 1];
		}

		if (AttributeInput && AttributeInput->GetTracedInput().Expression)
		{
			EMaterialValueType ValueType = FMaterialAttributeDefinitionMap::GetValueType(CompilingAttributeID);
			return Compiler->ValidCast(AttributeInput->GetTracedInput().Compile(Compiler), ValueType);
		}
		else if (Inputs[GET_SET_MA_MATERIALATTRIBUTESINDEX].GetTracedInput().Expression)
		{
			return Inputs[GET_SET_MA_MATERIALATTRIBUTESINDEX].GetTracedInput().Compile(Compiler);
		}

		return FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, CompilingAttributeID);
	}
}

void UMaterialExpressionSetMaterialAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SetMaterialAttributes"));
}

TArrayView<FExpressionInput*> UMaterialExpressionSetMaterialAttributes::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(Inputs.Num());
	for (FExpressionInput& Input : Inputs)
	{
		CachedInputs.Push(&Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionSetMaterialAttributes::GetInput(int32 InputIndex)
{
	return Inputs.IsValidIndex(InputIndex) ? &Inputs[InputIndex] : nullptr;
}

FName UMaterialExpressionSetMaterialAttributes::GetInputName(int32 InputIndex) const
{
	FName Name;

	if (InputIndex == GET_SET_MA_MATERIALATTRIBUTESINDEX)
	{
		Name = *NSLOCTEXT("SetMaterialAttributes", "InputName", "MaterialAttributes").ToString();
	}
	else if (InputIndex > GET_SET_MA_MATERIALATTRIBUTESINDEX)
	{
		Name = *FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeSetTypes[InputIndex-1], Material).ToString();
	}

	return Name;
}

EMaterialValueType UMaterialExpressionSetMaterialAttributes::GetInputValueType(int32 InputIndex)
{
	EMaterialValueType InputType = MCT_Unknown;

	if (InputIndex == GET_SET_MA_MATERIALATTRIBUTESINDEX)
	{
		InputType = MCT_MaterialAttributes;
	}
	else if(InputIndex > GET_SET_MA_MATERIALATTRIBUTESINDEX && InputIndex < AttributeSetTypes.Num() + 1)
	{
		InputType = FMaterialAttributeDefinitionMap::GetValueType(AttributeSetTypes[InputIndex - 1]);
		if (InputType == MCT_ShadingModel)
		{
			InputType = MCT_ShadingModel;
		}
		else if (InputType == MCT_Substrate)
		{
			InputType = MCT_Substrate;
		}
		else
		{
			InputType = MCT_Float3;
		}
	}

	return InputType;
}

int32 UMaterialExpressionSetMaterialAttributes::CreateOrGetInputAttribute(EMaterialProperty Attribute)
{
	int32 InputsIndex = INDEX_NONE;
	if (Attribute == MP_MaterialAttributes)
	{
		InputsIndex = GET_SET_MA_MATERIALATTRIBUTESINDEX;
	}
	else
	{
		FGuid AttributeId = FMaterialAttributeDefinitionMap::GetID(Attribute);
		if (AttributeSetTypes.Find(AttributeId, InputsIndex))
		{
			/**
			* Add one to compensate for the AttributeGetTypes list not containing MP_MaterialAttributes
			* It's none trivial to iterate the Inputs list for the matching attribute so this is a simpler solution.
			*/
			InputsIndex++;
		}
		else
		{
			int32 SetTypesIndex = AttributeSetTypes.Add(AttributeId);
			if(SetTypesIndex != INDEX_NONE)
			{
				PreEditChange(nullptr);
				InputsIndex = Inputs.Add(FExpressionInput());
				if (Inputs.IsValidIndex(InputsIndex))
				{
					Inputs[InputsIndex].InputName = FName(*FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeSetTypes[SetTypesIndex], Material).ToString());
				}
			}
		}
	}
	return InputsIndex;
}

bool UMaterialExpressionSetMaterialAttributes::ConnectInputAttribute(EMaterialProperty Attribute, UMaterialExpression* Expression, int32 OutputIndex)
{
	int32 Index = CreateOrGetInputAttribute(Attribute);
	if(Expression && OutputIndex != INDEX_NONE && Inputs.IsValidIndex(Index))
	{
		Inputs[Index].Connect(OutputIndex, Expression);
		return Inputs[Index].IsConnected();
	}
	return false;
}

bool UMaterialExpressionSetMaterialAttributes::GetSubstrateMaterialInputIndex(int32 OutputIndex, int32& InputIndex)
{
	for (InputIndex = Inputs.Num() - 1; InputIndex > GET_SET_MA_MATERIALATTRIBUTESINDEX; InputIndex--)
	{
		if (GetInputValueType(InputIndex) == MCT_Substrate && Inputs[InputIndex].IsConnected())
		{
			return true;
		}
	}

	//MA input is always in position 0
	InputIndex = GET_SET_MA_MATERIALATTRIBUTESINDEX;
	if(GetInputValueType(InputIndex) == MCT_MaterialAttributes && Inputs[InputIndex].IsConnected())
	{
		return Inputs[InputIndex].Expression->IsResultSubstrateMaterial(Inputs[InputIndex].OutputIndex);
	}	

	return false;
}

bool UMaterialExpressionSetMaterialAttributes::IsResultSubstrateMaterial(int32 OutputIndex)
{
	int32 InputIndex = INDEX_NONE;
	return GetSubstrateMaterialInputIndex(OutputIndex, InputIndex);
}

void UMaterialExpressionSetMaterialAttributes::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	int32 InputIndex = INDEX_NONE;
	if(GetSubstrateMaterialInputIndex(OutputIndex, InputIndex))
	{
		Inputs[InputIndex].Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, Inputs[InputIndex].OutputIndex);
	}
}

FSubstrateOperator* UMaterialExpressionSetMaterialAttributes::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, UMaterialExpression* Parent, int32 OutputIndex)
{
	int32 InputIndex = INDEX_NONE;
	if (GetSubstrateMaterialInputIndex(OutputIndex, InputIndex))
	{
		return Inputs[InputIndex].Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, Inputs[InputIndex].OutputIndex);
	}
	return nullptr;
}

void UMaterialExpressionSetMaterialAttributes::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Allows assigning values to specific inputs on a material attributes pin. Any unconnected inputs will be unchanged."), 40, OutToolTip);
}

void UMaterialExpressionSetMaterialAttributes::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Backup attribute array so we can re-connect pins
	PreEditAttributeSetTypes.Empty();
	for (const FGuid& AttributeID : AttributeSetTypes)
	{
		PreEditAttributeSetTypes.Add(AttributeID);
	};

	Super::PreEditChange(PropertyAboutToChange);
}

void UMaterialExpressionSetMaterialAttributes::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty && GraphNode)
	{
		if (PreEditAttributeSetTypes.Num() < AttributeSetTypes.Num())
		{
			// Attribute type added so default out type
			AttributeSetTypes.Last() = FMaterialAttributeDefinitionMap::GetDefaultID();

			// Attempt to find a valid attribute that's not already listed
			const TArray<FGuid>& OrderedVisibleAttributes = FMaterialAttributeDefinitionMap::GetOrderedVisibleAttributeList();
			for (const FGuid& AttributeID : OrderedVisibleAttributes)
			{
				if (PreEditAttributeSetTypes.Find(AttributeID) == INDEX_NONE)
				{
					 AttributeSetTypes.Last() = AttributeID;
					 break;
				}
			}
		
			// Copy final defaults to new input
			Inputs.Add(FExpressionInput());
			Inputs.Last().InputName = FName(*FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeSetTypes.Last(), Material).ToString());
			GraphNode->ReconstructNode();
		}	 
		else if (PreEditAttributeSetTypes.Num() > AttributeSetTypes.Num())
		{
			if (AttributeSetTypes.Num() == 0)
			{
				// All attribute types removed
				while (Inputs.Num() > 1)
				{
					Inputs.Pop();
					GraphNode->RemovePinAt(Inputs.Num(), EGPD_Input);
				}
			}
			else
			{
				// Attribute type removed
				int32 RemovedInputIndex = INDEX_NONE;

				for (int32 Attribute = 0; Attribute < AttributeSetTypes.Num(); ++Attribute)
				{
					// A mismatched attribute type means a middle pin was removed
					if (AttributeSetTypes[Attribute] != PreEditAttributeSetTypes[Attribute])
					{
						RemovedInputIndex = Attribute + 1;
						Inputs.RemoveAt(RemovedInputIndex);
						break;
					}
				};

				if (RemovedInputIndex == INDEX_NONE)
				{
					Inputs.Pop();
					RemovedInputIndex = Inputs.Num();
				}

				GraphNode->RemovePinAt(RemovedInputIndex, EGPD_Input);
			}
		}
		else
		{
			// Type changed, update pin names
			for (int i = 1; i < Inputs.Num(); ++i)
			{
				Inputs[i].InputName = FName(*FMaterialAttributeDefinitionMap::GetDisplayNameForMaterial(AttributeSetTypes[i - 1], Material).ToString());
			}
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty( PropertyChangedEvent );
}

int32 UMaterialExpressionSetMaterialAttributes::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Use the new base color as preview if specified
	int32 InputIndex = 0;
	for (const FGuid& AttributeType : AttributeSetTypes)
	{
		EMaterialProperty MaterialProperty = FMaterialAttributeDefinitionMap::GetProperty(AttributeType);
		if (MaterialProperty == MP_BaseColor && Inputs.Num() > (InputIndex + 1))
		{
			FExpressionInput BaseColorInput = Inputs[InputIndex + 1]; // +1 because first element is always material attribute input

			// Trace the base color expression to use as preview
			FExpressionInput BaseColorTraced = BaseColorInput.GetTracedInput();
			UMaterialExpression* BaseColorExpression = BaseColorTraced.Expression;
			if (BaseColorExpression)
			{
				return BaseColorInput.Compile(Compiler);
			}
		}
		InputIndex++;
	}

	// Otherwise display the one from the material attribute input
	if(Inputs.Num() > 0)
	{
		FExpressionInput MaterialAttributeInput = Inputs[0];
		FExpressionInput MaterialAttributeTraced = MaterialAttributeInput.GetTracedInput();
		UMaterialExpression* MaterialAttributeExpression = MaterialAttributeTraced.Expression;
		if (MaterialAttributeExpression)
		{
			// Material attributes "Compile" will return the value of the current attribute at the top of the compiler stack. By default it is EmissiveColor, so can be just black.
			// Instead we use BaseColor as a better preview representative for material attributes.
			return MaterialAttributeExpression->CompilePreview(Compiler, MaterialAttributeInput.OutputIndex);
		}
	}

	// Otherwise black color
	return Compiler->Constant3(0, 0, 0);
}

#endif // WITH_EDITOR

// -----

UMaterialExpressionBlendMaterialAttributes::UMaterialExpressionBlendMaterialAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 0, 0, 0, 0, 0));
#endif
}

#if WITH_EDITOR
FExpressionInput* UMaterialExpressionBlendMaterialAttributes::GetInput(int32 InputIndex)
{
	switch (InputIndex)
	{
		case 0: return &A;
		case 1: return &B;
		case 2: return &Alpha;
		default: return nullptr;
	}
}

static inline bool AllowSkipAlphaComputationForMaterialBlend()
{
	return GAllowSkipAlphaComputationForMaterialBlend;
}

int32 UMaterialExpressionBlendMaterialAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const FGuid AttributeID = Compiler->GetMaterialAttribute();

	// Blending is optional, can skip on a per-node basis
	EMaterialAttributeBlend::Type BlendType;
	EShaderFrequency AttributeFrequency = FMaterialAttributeDefinitionMap::GetShaderFrequency(AttributeID);

	switch (AttributeFrequency)
	{
	case SF_Vertex:	BlendType = VertexAttributeBlendType;	break;
	case SF_Pixel:	BlendType = PixelAttributeBlendType;	break;
	default:
		return Compiler->Errorf(TEXT("Attribute blending for shader frequency %i not implemented."), AttributeFrequency);
	}

	switch (BlendType)
	{
	case EMaterialAttributeBlend::UseA:
	{
		int32 CodeChunk = A.CompileWithDefault(Compiler, AttributeID);
		return CodeChunk;
	}
	case EMaterialAttributeBlend::UseB:
	{
		int32 CodeChunk = B.CompileWithDefault(Compiler, AttributeID);
		return CodeChunk;
	}
	default:
		check(BlendType == EMaterialAttributeBlend::Blend);
	}

	// Allow custom blends or fallback to standard interpolation
	int32 ResultA = A.CompileWithDefault(Compiler, AttributeID);
	int32 ResultB = B.CompileWithDefault(Compiler, AttributeID);
	if (AllowSkipAlphaComputationForMaterialBlend() && (ResultA == ResultB))
	{
		return ResultA;
	}

	int32 ResultAlpha = Alpha.Compile(Compiler);

	MaterialAttributeBlendFunction BlendFunction = FMaterialAttributeDefinitionMap::GetBlendFunction(AttributeID);
	if (BlendFunction)
	{
		return BlendFunction(Compiler, ResultA, ResultB, ResultAlpha);
	}
	else
	{
		return Compiler->Lerp(ResultA, ResultB, ResultAlpha);
	}
}

void UMaterialExpressionBlendMaterialAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("BlendMaterialAttributes"));
}

FName UMaterialExpressionBlendMaterialAttributes::GetInputName(int32 InputIndex) const
{
	FName Name;

	switch (InputIndex)
	{
	case 0: Name = TEXT("A"); break;
	case 1: Name = TEXT("B"); break;
	case 2: Name = TEXT("Alpha"); break;
	};

	return Name;
}
#endif // WITH_EDITOR


UMaterialExpressionLegacyBlendMaterialAttributes::UMaterialExpressionLegacyBlendMaterialAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Outputs.Reset();
	bShowInputs = false;
	bShowOutputs = false;
	bCollapsed = true;
}

//
//	UMaterialExpressionMaterialAttributeLayers
//
UMaterialExpressionMaterialAttributeLayers::UMaterialExpressionMaterialAttributeLayers(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)	
{
#if WITH_EDITOR
	DefaultLayers.AddDefaultBackgroundLayer();
	
	CachedInputs.Empty();
	CachedInputs.Push(&Input);
#endif
}

void UMaterialExpressionMaterialAttributeLayers::PostLoad()
{
	Super::PostLoad();

	for (UMaterialFunctionInterface* Layer : DefaultLayers.Layers)
	{
		if (Layer)
		{
			Layer->ConditionalPostLoad();
		}
	}

	for (UMaterialFunctionInterface* Blend : DefaultLayers.Blends)
	{
		if (Blend)
		{
			Blend->ConditionalPostLoad();
		}
	}

#if WITH_EDITORONLY_DATA
	RebuildLayerGraph(false);
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void UMaterialExpressionMaterialAttributeLayers::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	RebuildLayerGraph(false);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterialExpressionMaterialAttributeLayers::RebuildLayerGraph(bool bReportErrors)
{
	{
		const TArray<UMaterialFunctionInterface*>& Layers = GetLayers();
		const TArray<UMaterialFunctionInterface*>& Blends = GetBlends();
		const TArray<bool>& LayerStates = GetLayerStates();
	
		// Pre-populate callers, we maintain these transient objects to avoid
		// heavy UObject recreation as the graphs are frequently rebuilt
		while (LayerCallers.Num() < Layers.Num())
		{
			LayerCallers.Add(NewObject<UMaterialExpressionMaterialFunctionCall>(GetTransientPackage()));
		}
		while (BlendCallers.Num() < Blends.Num())
		{
			BlendCallers.Add(NewObject<UMaterialExpressionMaterialFunctionCall>(GetTransientPackage()));
		}

		// Reset graph connectivity
		bIsLayerGraphBuilt = false;
		NumActiveLayerCallers = 0;
		NumActiveBlendCallers = 0;

		if (ValidateLayerConfiguration(nullptr, bReportErrors))
		{
			// Initialize layer function callers
			for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
			{
				if (Layers[LayerIndex] && LayerStates[LayerIndex])
				{
					int32 CallerIndex = NumActiveLayerCallers;
					LayerCallers[CallerIndex]->MaterialFunction = Layers[LayerIndex];
					LayerCallers[CallerIndex]->FunctionParameterInfo.Association = EMaterialParameterAssociation::LayerParameter;
					LayerCallers[CallerIndex]->FunctionParameterInfo.Index = LayerIndex;
					++NumActiveLayerCallers;

					Layers[LayerIndex]->GetInputsAndOutputs(LayerCallers[CallerIndex]->FunctionInputs, LayerCallers[CallerIndex]->FunctionOutputs);
					for (FFunctionExpressionOutput& FunctionOutput : LayerCallers[CallerIndex]->FunctionOutputs)
					{
						LayerCallers[CallerIndex]->Outputs.Add(FunctionOutput.Output);
					}

					// Optional: Single material attributes input, the base input to the stack
					if (LayerCallers[CallerIndex]->FunctionInputs.Num() > 0)
					{
						if (Input.GetTracedInput().Expression)
						{
							LayerCallers[CallerIndex]->FunctionInputs[0].Input = Input;
						}
					}

					// Recursively run through internal functions to allow connection of inputs/outputs
					LayerCallers[CallerIndex]->UpdateFromFunctionResource();
				}
			}

			for (int32 BlendIndex = 0; BlendIndex < Blends.Num(); ++BlendIndex)
			{
				const int32 LayerIndex = BlendIndex + 1;
				if (Layers[LayerIndex] && LayerStates[LayerIndex])
				{
					int32 CallerIndex = NumActiveBlendCallers;
					++NumActiveBlendCallers;

					if (Blends[BlendIndex])
					{
						BlendCallers[CallerIndex]->MaterialFunction = Blends[BlendIndex];
						BlendCallers[CallerIndex]->FunctionParameterInfo.Association = EMaterialParameterAssociation::BlendParameter;
						BlendCallers[CallerIndex]->FunctionParameterInfo.Index = BlendIndex;

						Blends[BlendIndex]->GetInputsAndOutputs(BlendCallers[CallerIndex]->FunctionInputs, BlendCallers[CallerIndex]->FunctionOutputs);
						for (FFunctionExpressionOutput& FunctionOutput : BlendCallers[CallerIndex]->FunctionOutputs)
						{
							BlendCallers[CallerIndex]->Outputs.Add(FunctionOutput.Output);
						}

						// Recursively run through internal functions to allow connection of inputs/ouputs
						BlendCallers[CallerIndex]->UpdateFromFunctionResource();
					}
					else
					{
						// Empty entries for opaque layers
						BlendCallers[CallerIndex]->MaterialFunction = nullptr;
					}
				}
			}

			// Empty out unused callers
			for (int32 CallerIndex = NumActiveLayerCallers; CallerIndex < LayerCallers.Num(); ++CallerIndex)
			{
				LayerCallers[CallerIndex]->MaterialFunction = nullptr;
			}

			for (int32 CallerIndex = NumActiveBlendCallers; CallerIndex < BlendCallers.Num(); ++CallerIndex)
			{
				BlendCallers[CallerIndex]->MaterialFunction = nullptr;
			}

			// Assemble function chain so each layer blends with the previous
			if (NumActiveLayerCallers >= 2 && NumActiveBlendCallers >= 1)
			{
				if (BlendCallers[0]->MaterialFunction)
				{
					BlendCallers[0]->FunctionInputs[0].Input.Connect(0, LayerCallers[0]);
					BlendCallers[0]->FunctionInputs[1].Input.Connect(0, LayerCallers[1]);
				}

				for (int32 LayerIndex = 2; LayerIndex < NumActiveLayerCallers; ++LayerIndex)
				{
					if (BlendCallers[LayerIndex - 1]->MaterialFunction)
					{
						// Active blend input is previous blend or direct layer if previous is opaque
						UMaterialExpressionMaterialFunctionCall* BlendInput = BlendCallers[LayerIndex - 2];
						BlendInput = BlendInput->MaterialFunction ? BlendInput : ToRawPtr(LayerCallers[LayerIndex - 1]);

						BlendCallers[LayerIndex - 1]->FunctionInputs[0].Input.Connect(0, BlendInput);
						BlendCallers[LayerIndex - 1]->FunctionInputs[1].Input.Connect(0, LayerCallers[LayerIndex]);
					}
				}
			}
			bIsLayerGraphBuilt = true;
		}
	}	
	
	if (!bIsLayerGraphBuilt && bReportErrors)
	{
		UE_LOGF(LogMaterial, Warning, "Failed to build layer graph for %ls.", Material ? *(Material->GetName()) : TEXT("Unknown"));
	}
}

void UMaterialExpressionMaterialAttributeLayers::OverrideLayerGraph(const FMaterialLayersFunctions* OverrideLayers)
{
	if (ParamLayers != OverrideLayers)
	{
		ParamLayers = OverrideLayers;
		RebuildLayerGraph(false);
	}
}


void UMaterialExpressionMaterialAttributeLayers::LogError(FMaterialCompiler* Compiler, const TCHAR* Format, ...)
{
	TStringBuilder<256> Builder;
	Builder.Append(TEXT("LayerValidation:"));
	va_list Args;
	va_start(Args, Format);
	const TCHAR* ErrorMsg = Builder.AppendV(Format, Args).GetData();
	va_end(Args);

	if (Compiler)
	{
		Compiler->Errorf(ErrorMsg);
	}
	else
	{
		UE_LOGF(LogMaterial, Warning, "%ls", ErrorMsg);
	}
}

bool UMaterialExpressionMaterialAttributeLayers::ValidateLayerConfiguration(FMaterialCompiler* Compiler, bool bReportErrors)
{
#define COMPILER_OR_LOG_ERROR(Format, ...)				\
	if (bReportErrors)									\
	{													\
		LogError(Compiler, Format, ##__VA_ARGS__);		\
	}

	const TArray<UMaterialFunctionInterface*>& Layers = GetLayers();
	const TArray<UMaterialFunctionInterface*>& Blends = GetBlends();
	const TArray<bool>& LayerStates = GetLayerStates();

	bool bIsValid = true;
	const int32 NumLayers = Layers.Num();
	const int32 NumBlends = Blends.Num();

	int32 NumActiveLayers = 0;
	int32 NumActiveBlends = 0;

	for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		UMaterialFunctionInterface* Layer = Layers[LayerIndex];
		
		if (Layer)
		{
			if (Layer->GetMaterialFunctionUsage() != EMaterialFunctionUsage::MaterialLayer)
			{
				COMPILER_OR_LOG_ERROR(TEXT("Layer %i, %s, not set for layer usage."), LayerIndex, *Layer->GetName());
				bIsValid = false;
			}
			else if (UMaterialFunctionInstance* InstanceLayer = Cast<UMaterialFunctionInstance>(Layer))
			{
				if (!InstanceLayer->Parent)
				{
					COMPILER_OR_LOG_ERROR(TEXT("Layer %i, %s, layer instance has no parent set."), LayerIndex, *Layer->GetName());
					bIsValid = false;
				}
			}
			else
			{
				TArray<UMaterialExpressionFunctionInput*> InputExpressions;
				Layer->GetAllExpressionsOfType<UMaterialExpressionFunctionInput>(InputExpressions, false);
				if (InputExpressions.Num() > AcceptableNumLayerMAInputs)
				{
					COMPILER_OR_LOG_ERROR(TEXT("Layer %i, %s, must have one MaterialAttributes input only."), LayerIndex, *Layer->GetName());
					bIsValid = false;
				}
			}

			if (LayerStates[LayerIndex])
			{
				++NumActiveLayers;
			}
		}
	}

	for (int32 BlendIndex = 0; BlendIndex < NumBlends; ++BlendIndex)
	{
		UMaterialFunctionInterface* Blend = Blends[BlendIndex];
		
		if (Blend)
		{
			if (Blend->GetMaterialFunctionUsage() != EMaterialFunctionUsage::MaterialLayerBlend)
			{
				COMPILER_OR_LOG_ERROR(TEXT("Blend %i, %s, not set for layer blend usage."), BlendIndex, *Blend->GetName());
				bIsValid = false;
			}
			else if (UMaterialFunctionInstance* InstanceBlend = Cast<UMaterialFunctionInstance>(Blend))
			{
				if (!InstanceBlend->Parent)
				{
					COMPILER_OR_LOG_ERROR(TEXT("Blend %i, %s, layer instance has no parent set."), BlendIndex, *Blend->GetName());
					bIsValid = false;
				}
			}
			else
			{
				TArray<UMaterialExpressionFunctionInput*> InputExpressions;
				Blend->GetAllExpressionsOfType<UMaterialExpressionFunctionInput>(InputExpressions, false);

				if (InputExpressions.Num() != AcceptableNumBlendMAInputs)
				{
					COMPILER_OR_LOG_ERROR(TEXT("Blend %i, %s, must have two MaterialAttributes inputs."), BlendIndex, *Blend->GetName());
					bIsValid = false;
				}
			}
		}
		
		// Null blends signify an opaque layer so count as valid for the sake of graph validation
		if (Layers[BlendIndex+1] && LayerStates[BlendIndex+1])
		{
			++NumActiveBlends;
		}
	}

	bool bValidGraphLayout = 
		(NumActiveLayers == 0 && NumActiveBlends == 0)		// Pass-through
		|| (NumActiveLayers == 1 && NumActiveBlends == 0)						// Single layer
		|| (NumActiveLayers >= 2 && NumActiveBlends == NumActiveLayers - 1);	// Blend graph

	if (!bValidGraphLayout)
	{
		COMPILER_OR_LOG_ERROR(TEXT("Invalid number of layers (%i) or blends (%i) assigned."), NumActiveLayers, NumActiveBlends);
		bIsValid = false;
	}

	if (Compiler && Compiler->GetCurrentFunctionStackDepth() > 1)
	{
		COMPILER_OR_LOG_ERROR(TEXT("Layer expressions cannot be used within a material function."));
		bIsValid = false;
	}

	return bIsValid;

#undef COMPILER_OR_LOG_ERROR
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UMaterialExpressionMaterialAttributeLayers::IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
{
	const TArray<UMaterialFunctionInterface*>& Layers = GetLayers();
	const TArray<UMaterialFunctionInterface*>& Blends = GetBlends();

	for (auto* Layer : Layers)
	{
		if (Layer)
		{		
			if (!Layer->IterateDependentFunctions(Predicate))
			{
				return false;
			}
			if (!Predicate(Layer))
			{
				return false;
			}
		}
	}

	for (auto* Blend : Blends)
	{
		if (Blend)
		{
			if (!Blend->IterateDependentFunctions(Predicate))
			{
				return false;
			}
			if (!Predicate(Blend))
			{
				return false;
			}
		}
	}
	return true;
}

void UMaterialExpressionMaterialAttributeLayers::GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
{
	IterateDependentFunctions([&DependentFunctions](UMaterialFunctionInterface* MaterialFunction) -> bool
	{
		DependentFunctions.AddUnique(MaterialFunction);
		return true;
	});
}
#endif // WITH_EDITORONLY_DATA

UMaterialFunctionInterface* UMaterialExpressionMaterialAttributeLayers::GetParameterAssociatedFunction(const FHashedMaterialParameterInfo& ParameterInfo) const
{
	check(ParameterInfo.Association != EMaterialParameterAssociation::GlobalParameter);

	// Grab the associated layer or blend
	UMaterialFunctionInterface* LayersFunction = nullptr;

	if (ParameterInfo.Association == EMaterialParameterAssociation::LayerParameter)
	{
		const TArray<UMaterialFunctionInterface*>& Layers = GetLayers();
		if (Layers.IsValidIndex(ParameterInfo.Index))
		{
			LayersFunction = Layers[ParameterInfo.Index];
		}
	}
	else if (ParameterInfo.Association == EMaterialParameterAssociation::BlendParameter)
	{
		const TArray<UMaterialFunctionInterface*>& Blends = GetBlends();
		if (Blends.IsValidIndex(ParameterInfo.Index))
		{
			LayersFunction = Blends[ParameterInfo.Index];
		}
	}

	return LayersFunction;
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialAttributeLayers::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 Result = INDEX_NONE;

	const FMaterialLayersFunctions* OverrideLayers = Compiler->GetMaterialLayers();
	OverrideLayerGraph(OverrideLayers);

	if (ValidateLayerConfiguration(Compiler, true) && bIsLayerGraphBuilt)
	{
		if (NumActiveBlendCallers > 0 && BlendCallers[NumActiveBlendCallers-1]->MaterialFunction)
		{
			// Multiple blended layers
			Result = BlendCallers[NumActiveBlendCallers-1]->Compile(Compiler, 0);
		}
		else if (NumActiveLayerCallers > 0 && LayerCallers[NumActiveLayerCallers-1]->MaterialFunction)
		{
			// Single layer
			Result = LayerCallers[NumActiveLayerCallers-1]->Compile(Compiler, 0);
		}
		else if (NumActiveLayerCallers == 0)
		{
			// Pass-through
			const FGuid AttributeID = Compiler->GetMaterialAttribute();
			if (Input.GetTracedInput().Expression)
			{
				Result = Input.CompileWithDefault(Compiler, AttributeID);
			}
			else
			{
				Result = FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, AttributeID);
			}
		}
		else
		{
			// Error on unknown mismatch
			Result = Compiler->Errorf(TEXT("Unknown error occured on validated layers."));
		}
	}
	else
	{
		// Error on unknown mismatch
		Result = Compiler->Errorf(TEXT("Failed to validate layer configuration."));
	}

	OverrideLayerGraph(nullptr);

	return Result;
}

void UMaterialExpressionMaterialAttributeLayers::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Material Attribute Layers"));
}

void UMaterialExpressionMaterialAttributeLayers::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Evaluates the active material layer stack and outputs the merged attributes."), 40, OutToolTip);
}

FName UMaterialExpressionMaterialAttributeLayers::GetInputName(int32 InputIndex) const
{
	return NAME_None;
}

EMaterialValueType UMaterialExpressionMaterialAttributeLayers::GetInputValueType(int32 InputIndex)
{
	return MCT_MaterialAttributes;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionFloatToUInt::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing FloatToUInt input"));
	}

	int32 Value = Input.Compile(Compiler);
	switch (Mode)
	{
		case EFloatToIntMode::Truncate: Value = Compiler->Truncate(Value); break;
		case EFloatToIntMode::Floor: Value = Compiler->Floor(Value); break;
		case EFloatToIntMode::Round: Value = Compiler->Round(Value); break;
		case EFloatToIntMode::Ceil: Value = Compiler->Ceil(Value); break;
		default: check(false);
	}
	
	EMaterialValueType Type = Compiler->GetParameterType(Value);
	int NumComponents = GetNumComponents(Type);
	if (NumComponents <= 0 || NumComponents > 4)
	{
		return Compiler->Errorf(TEXT("Input FloatToUInt is not a scalar or vector"));
	}

	static const EMaterialValueType UIntTypes[] = { MCT_UInt1, MCT_UInt2, MCT_UInt3, MCT_UInt4 };
	return Compiler->ForceCast(Value, UIntTypes[NumComponents - 1]);
}

void UMaterialExpressionFloatToUInt::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("FloatToUInt"));
}

EMaterialValueType UMaterialExpressionFloatToUInt::GetInputValueType(int32 InputIndex)
{
	return MCT_Float;
}

EMaterialValueType UMaterialExpressionFloatToUInt::GetOutputValueType(int32 OutputIndex)
{
	return MCT_UInt;
}


#endif // WITH_EDITOR


// -----
#if WITH_EDITOR
int32 UMaterialExpressionUIntToFloat::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
 	if (!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing UIntToFloat input"));
	}

	int32 Value = Input.Compile(Compiler);
	if (Value == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	EMaterialValueType Type = Compiler->GetParameterType(Value);
	int NumComponents = GetNumComponents(Type);
	if (NumComponents <= 0 || NumComponents > 4)
	{
		return Compiler->Errorf(TEXT("Input FloatToUInt is not a scalar or vector"));
	}

	static const EMaterialValueType FloatTypes[] = { MCT_Float1, MCT_Float2, MCT_Float3, MCT_Float4 };
	return Compiler->ForceCast(Value, FloatTypes[NumComponents - 1]);
}

void UMaterialExpressionUIntToFloat::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("UIntToFloat"));
}

EMaterialValueType UMaterialExpressionUIntToFloat::GetInputValueType(int32 InputIndex)
{
	return MCT_UInt;
}

EMaterialValueType UMaterialExpressionUIntToFloat::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Float;
}

#endif // WITH_EDITOR


// -----
#if WITH_EDITOR
int32 UMaterialExpressionFloor::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Floor input"));
	}

	return Compiler->Floor(Input.Compile(Compiler));
}

EMaterialValueType UMaterialExpressionFloor::GetOutputValueType(int32 OutputIndex)
{
	return static_cast<EMaterialValueType>(MCT_Float | MCT_UInt);
}

void UMaterialExpressionFloor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Floor"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionCeil::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Ceil input"));
	}
	return Compiler->Ceil(Input.Compile(Compiler));
}

EMaterialValueType UMaterialExpressionCeil::GetOutputValueType(int32 OutputIndex)
{
	return static_cast<EMaterialValueType>(MCT_Float | MCT_UInt);
}

void UMaterialExpressionCeil::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Ceil"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionRound
//
#if WITH_EDITOR
int32 UMaterialExpressionRound::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Round input"));
	}
	return Compiler->Round(Input.Compile(Compiler));
}

EMaterialValueType UMaterialExpressionRound::GetOutputValueType(int32 OutputIndex)
{
	return static_cast<EMaterialValueType>(MCT_Float | MCT_UInt);
}

void UMaterialExpressionRound::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Round"));
}

void UMaterialExpressionRound::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Rounds the value up to the next whole number if the fractional part is greater than or equal to half, else rounds down."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionTruncate
//
#if WITH_EDITOR
int32 UMaterialExpressionTruncate::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Truncate input"));
	}
	return Compiler->Truncate(Input.Compile(Compiler));
}

EMaterialValueType UMaterialExpressionTruncate::GetOutputValueType(int32 OutputIndex)
{
	return static_cast<EMaterialValueType>(MCT_Float | MCT_UInt);
}

void UMaterialExpressionTruncate::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Truncate"));
}

void UMaterialExpressionTruncate::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Truncates a value by discarding the fractional part."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionSign
//
#if WITH_EDITOR
int32 UMaterialExpressionSign::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Sign input"));
	}
	return Compiler->Sign(Input.Compile(Compiler));
}

void UMaterialExpressionSign::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Sign"));
}

void UMaterialExpressionSign::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns -1 if the input is less than 0, 1 if greater, or 0 if equal."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionFmod
//
#if WITH_EDITOR
int32 UMaterialExpressionFmod::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Fmod input A"));
	}
	if (!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Fmod input B"));
	}
	return Compiler->Fmod(A.Compile(Compiler), B.Compile(Compiler));
}

void UMaterialExpressionFmod::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Fmod"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionModulo
//
#if WITH_EDITOR
int32 UMaterialExpressionModulo::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Modulo input A"));
	}
	if (!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Modulo input B"));
	}
	return Compiler->Modulo(A.Compile(Compiler), B.Compile(Compiler));
}

EMaterialValueType UMaterialExpressionModulo::GetInputValueType(int32 InputIndex)
{
	return static_cast<EMaterialValueType>(MCT_Float | MCT_LWCType | MCT_UInt);
}

EMaterialValueType UMaterialExpressionModulo::GetOutputValueType(int32 OutputIndex)
{
	return static_cast<EMaterialValueType>(MCT_Float | MCT_LWCType | MCT_UInt);
}

void UMaterialExpressionModulo::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Modulo"));
}

FText UMaterialExpressionModulo::GetKeywords() const
{
	return FText::FromString(TEXT("%"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionFrac::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Frac input"));
	}

	return Compiler->Frac(Input.Compile(Compiler));
}

void UMaterialExpressionFrac::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Frac"));
}
#endif // WITH_EDITOR

UMaterialExpressionDesaturation::UMaterialExpressionDesaturation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LegacyLuminanceFactors"));
	if(CVar && CVar->GetInt() != 0)
	{
		LuminanceFactors = FLinearColor(0.3f, 0.59f, 0.11f, 0.0f); 
	}
	else
	{
		LuminanceFactors = UE::Color::FColorSpace::GetWorking().GetLuminanceFactors();
	}
}

#if WITH_EDITOR
int32 UMaterialExpressionDesaturation::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
		return Compiler->Errorf(TEXT("Missing Desaturation input"));

	int32 Color = Compiler->ForceCast(Input.Compile(Compiler), MCT_Float3, MFCF_ExactMatch|MFCF_ReplicateValue),
		Grey = Compiler->Dot(Color,Compiler->Constant3(LuminanceFactors.R,LuminanceFactors.G,LuminanceFactors.B));

	if(Fraction.GetTracedInput().Expression)
		return Compiler->Lerp(Color,Grey,Fraction.Compile(Compiler));
	else
		return Grey;
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionParameter
//
FName UMaterialExpressionParameter::ParameterDefaultName = TEXT("Param");

UMaterialExpressionParameter::UMaterialExpressionParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName ParameterName;
		FConstructorStatics()
			: ParameterName(UMaterialExpressionParameter::ParameterDefaultName)
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bIsParameterExpression = true;
	ParameterName = ConstructorStatics.ParameterName;

#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

#if WITH_EDITOR

bool UMaterialExpressionParameter::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if( ParameterName.ToString().Contains(SearchQuery) )
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}



FString UMaterialExpressionParameter::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionParameter::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}

void UMaterialExpressionParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	ValidateParameterNameInternal(this, Material, bAllowDuplicateName);
}
#endif

//
//	UMaterialExpressionVectorParameter
//
UMaterialExpressionVectorParameter::UMaterialExpressionVectorParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("R"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("G"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("B"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
	ApplyChannelNames();

	bShowOutputNameOnPin = true;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionVectorParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (bUseCustomPrimitiveData)
	{
		if (Material && Material->MaterialDomain == MD_UI)
		{
			return CompilerError(Compiler, CPD_UI_ErrorMessage);
		}

		return Compiler->CustomPrimitiveData(PrimitiveDataIndex, MCT_Float4);
	}
	else
	{
		return Compiler->VectorParameter(ParameterName,DefaultValue);
	}
}

void UMaterialExpressionVectorParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bUseCustomPrimitiveData)
	{
		FString IndexString = FString::Printf(TEXT("Index %d"), PrimitiveDataIndex);

		// Add info about remaining 3 components
		for (int i = 1; i < 4; i++)
		{
			// Append index if it's valid, otherwise append N/A
			if(PrimitiveDataIndex+i < FCustomPrimitiveData::NumCustomPrimitiveDataFloats)
			{
				IndexString.Append(FString::Printf(TEXT(", %d"), PrimitiveDataIndex+i));
			}
			else
			{
				IndexString.Append(FString::Printf(TEXT(", N/A")));
			}
		}

		OutCaptions.Add(IndexString); 
		OutCaptions.Add(FString::Printf(TEXT("Custom Primitive Data"))); 
	}
	else
	{
		OutCaptions.Add(FString::Printf(
			 TEXT("Param (%.3g,%.3g,%.3g,%.3g)"),
			 DefaultValue.R,
			 DefaultValue.G,
			 DefaultValue.B,
			 DefaultValue.A ));
	}

	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString())); 
}

bool UMaterialExpressionVectorParameter::SetParameterValue(FName InParameterName, FLinearColor InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (DefaultValue != InValue)
		{
			DefaultValue = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue));
			}
		}
		return true;
	}
	return false;
}

void UMaterialExpressionVectorParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue))
	{
		// Callback into the editor
		FEditorSupportDelegates::NumericParameterDefaultChanged.Broadcast(this, EMaterialParameterType::Vector, ParameterName, DefaultValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, PrimitiveDataIndex))
	{
		// Clamp value
		const int32 PrimDataIndex = PrimitiveDataIndex;
		PrimitiveDataIndex = (uint8)FMath::Clamp(PrimDataIndex, 0, FCustomPrimitiveData::NumCustomPrimitiveDataFloats-1);
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, ChannelNames)
		&& !IsUsedAsChannelMask())
	{
		ApplyChannelNames();

		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionVectorParameter::ApplyChannelNames()
{
	Outputs[1].OutputName = !ChannelNames.R.IsEmpty() ? FName(*ChannelNames.R.ToString()) : TEXT("R");
	Outputs[2].OutputName = !ChannelNames.G.IsEmpty() ? FName(*ChannelNames.G.ToString()) : TEXT("G");
	Outputs[3].OutputName = !ChannelNames.B.IsEmpty() ? FName(*ChannelNames.B.ToString()) : TEXT("B");
	Outputs[4].OutputName = !ChannelNames.A.IsEmpty() ? FName(*ChannelNames.A.ToString()) : TEXT("A");
}


void UMaterialExpressionVectorParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	bool bOverrideDuplicateBehavior = false;
	TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions;
	if (Material)
	{
		Expressions = Material->GetExpressions();
	}
	else if (Function)
	{
		Expressions = Function->GetExpressions();
	}

	for (UMaterialExpression* Expression : Expressions)
	{
		if (Expression != nullptr && Expression->HasAParameterName())
		{
			UMaterialExpressionVectorParameter* VectorExpression = Cast<UMaterialExpressionVectorParameter>(Expression);
			if (VectorExpression
				&& GetParameterName() == VectorExpression->GetParameterName()
				&& IsUsedAsChannelMask() != VectorExpression->IsUsedAsChannelMask())
			{
				bOverrideDuplicateBehavior = true;
				break;
			}
		}
	}
	Super::ValidateParameterName(bOverrideDuplicateBehavior ? false : bAllowDuplicateName);
}

bool UMaterialExpressionVectorParameter::HasClassAndNameCollision(UMaterialExpression* OtherExpression) const
{
	UMaterialExpressionVectorParameter* VectorExpression = Cast<UMaterialExpressionVectorParameter>(OtherExpression);
	if (VectorExpression
		&& GetParameterName() == VectorExpression->GetParameterName()
		&& IsUsedAsChannelMask() != VectorExpression->IsUsedAsChannelMask())
	{
		return true;
	}
	return Super::HasClassAndNameCollision(OtherExpression);
}
#endif

//
//	UMaterialExpressionDoubleVectorParameter
//
UMaterialExpressionDoubleVectorParameter::UMaterialExpressionDoubleVectorParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("XYZ"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("X"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Y"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Z"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("W"), 1, 0, 0, 0, 1));
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionDoubleVectorParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->NumericParameter(EMaterialParameterType::DoubleVector, ParameterName, DefaultValue);
}

void UMaterialExpressionDoubleVectorParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(
		TEXT("Param (%.3g,%.3g,%.3g,%.3g)"),
		DefaultValue.X,
		DefaultValue.Y,
		DefaultValue.Z,
		DefaultValue.W));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
bool UMaterialExpressionDoubleVectorParameter::SetParameterValue(FName InParameterName, FVector4d InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (DefaultValue != InValue)
		{
			DefaultValue = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue));
			}
		}
		return true;
	}
	return false;
}

void UMaterialExpressionDoubleVectorParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue))
	{
		// Callback into the editor
		FEditorSupportDelegates::NumericParameterDefaultChanged.Broadcast(this, EMaterialParameterType::DoubleVector, ParameterName, DefaultValue);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

//
//	UMaterialExpressionChannelMaskParameter
//
UMaterialExpressionChannelMaskParameter::UMaterialExpressionChannelMaskParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	bShowMaskColorsOnPin = false;
#endif

	// Default mask to red channel
	DefaultValue = FLinearColor(1.0f, 0.0f, 0.0f, 0.0f);
}

#if WITH_EDITOR
bool UMaterialExpressionChannelMaskParameter::SetParameterValue(FName InParameterName, FLinearColor InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (DefaultValue != InValue)
		{
			// Update value
			DefaultValue = InValue;

			// Update enum
			if (DefaultValue.R > 0.0f)
			{
				MaskChannel = EChannelMaskParameterColor::Red;
			}
			else if (DefaultValue.G > 0.0f)
			{
				MaskChannel = EChannelMaskParameterColor::Green;
			}
			else if (DefaultValue.B > 0.0f)
			{
				MaskChannel = EChannelMaskParameterColor::Blue;
			}
			else
			{
				MaskChannel = EChannelMaskParameterColor::Alpha;
			}

			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue));
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MaskChannel));
			}
		}

		return true;
	}

	return false;
}

void UMaterialExpressionChannelMaskParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MaskChannel))
	{
		// Update internal value
		switch (MaskChannel)
		{
		case EChannelMaskParameterColor::Red:
			DefaultValue = FLinearColor(1.0f, 0.0f, 0.0f, 0.0f); break;
		case EChannelMaskParameterColor::Green:
			DefaultValue = FLinearColor(0.0f, 1.0f, 0.0f, 0.0f); break;
		case EChannelMaskParameterColor::Blue:
			DefaultValue = FLinearColor(0.0f, 0.0f, 1.0f, 0.0f); break;
		default:
			DefaultValue = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f); break;
		}

		FEditorSupportDelegates::NumericParameterDefaultChanged.Broadcast(this, EMaterialParameterType::Vector, ParameterName, DefaultValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue))
	{
		// If the vector parameter was updated, the enum needs to match and we assert the values are valid
		if (DefaultValue.R > 0.0f)
		{
			MaskChannel = EChannelMaskParameterColor::Red;
			DefaultValue = FLinearColor(1.0f, 0.0f, 0.0f, 0.0f);
		}
		else if (DefaultValue.G > 0.0f)
		{
			MaskChannel = EChannelMaskParameterColor::Green;
			DefaultValue = FLinearColor(0.0f, 1.0f, 0.0f, 0.0f);
		}
		else if (DefaultValue.B > 0.0f)
		{
			MaskChannel = EChannelMaskParameterColor::Blue;
			DefaultValue = FLinearColor(0.0f, 0.0f, 1.0f, 0.0f);
		}
		else
		{
			MaskChannel = EChannelMaskParameterColor::Alpha;
			DefaultValue = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionChannelMaskParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing mask input"));
	}

	int32 Ret = Input.Compile(Compiler);
	Ret = Compiler->ForceCast(Ret, MCT_Float4, MFCF_ForceCast);

	if (Ret != INDEX_NONE)
	{
		// Internally this mask is a simple dot product, the mask is stored as a vector parameter
		int32 Param = Compiler->VectorParameter(ParameterName, DefaultValue);
		Ret = Compiler->Dot(Ret, Param);
	}
	else
	{
		Ret = Compiler->Errorf(TEXT("Failed to compile mask input"));
	}
	
	return Ret;
}

const TCHAR* GetChannelMaskParameterName(TEnumAsByte<EChannelMaskParameterColor::Type> InMaskChannel)
{
	const TCHAR* Output = nullptr;
	switch (InMaskChannel)
	{
	case EChannelMaskParameterColor::Red:
		Output = TEXT("Red"); break;
	case EChannelMaskParameterColor::Green:
		Output = TEXT("Green"); break;
	case EChannelMaskParameterColor::Blue:
		Output = TEXT("Blue"); break;
	case EChannelMaskParameterColor::Alpha:
		Output = TEXT("Alpha"); break;
	default:
		checkf(false, TEXT("Invalid MaskChannel %d"), InMaskChannel); break;
	}

	return Output;
}

void UMaterialExpressionChannelMaskParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	const TCHAR* MaskName = GetChannelMaskParameterName(MaskChannel);
	OutCaptions.Add(MaskName);
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

void UMaterialExpressionChannelMaskParameter::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	const TCHAR* MaskName = GetChannelMaskParameterName(MaskChannel);
	OutToolTip.Add(TEXT("Outputs the selected channel."));
	OutToolTip.Add(TEXT("Note: This does NOT create new permutations."));
	OutToolTip.Add(FString::Printf(TEXT("Current output: %s "), MaskName));
}

#endif // WITH_EDITOR

//
//	UMaterialExpressionScalarParameter
//
UMaterialExpressionScalarParameter::UMaterialExpressionScalarParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionScalarParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (bUseCustomPrimitiveData)
	{
		if (Material && Material->MaterialDomain == MD_UI)
		{
			return CompilerError(Compiler, CPD_UI_ErrorMessage);
		}

		return Compiler->CustomPrimitiveData(PrimitiveDataIndex, MCT_Float);
	}
	else
	{
		return Compiler->ScalarParameter(ParameterName,DefaultValue);
	}
}

void UMaterialExpressionScalarParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	if (bUseCustomPrimitiveData)
	{
		OutCaptions.Add(FString::Printf(TEXT("Index %d"), PrimitiveDataIndex)); 
		OutCaptions.Add(FString::Printf(TEXT("Custom Primitive Data"))); 
	}
	else
	{
		OutCaptions.Add(FString::Printf(
			 TEXT("Param (%.4g)"),
			DefaultValue )); 
	}
	 OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString())); 
}

bool UMaterialExpressionScalarParameter::GetParameterValue(FMaterialParameterMetadata& OutMeta) const
{
	OutMeta.Value = DefaultValue;
	OutMeta.PrimitiveDataIndex = INDEX_NONE;
	OutMeta.ScalarMin = 0.f;
	OutMeta.ScalarMin = 0.f;
	OutMeta.ScalarEnumeration = nullptr;
	OutMeta.ScalarEnumerationIndex = INDEX_NONE;

	if (bUseCustomPrimitiveData)
	{
		OutMeta.PrimitiveDataIndex = PrimitiveDataIndex;
	}
	else
	{
		if (ControlType == EMaterialScalarParameterControlType::Numeric && SliderMax > SliderMin)
		{
			OutMeta.ScalarMin = SliderMin;
			OutMeta.ScalarMax = SliderMax;
		}
		else if (ControlType == EMaterialScalarParameterControlType::Enumeration)
		{
			OutMeta.ScalarEnumeration = Enumeration;
		}
		else if (ControlType == EMaterialScalarParameterControlType::EnumerationIndex)
		{
			OutMeta.ScalarEnumerationIndex = EnumerationIndex;
		}
	}

	return Super::GetParameterValue(OutMeta);
}

bool UMaterialExpressionScalarParameter::SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (Meta.Value.Type == EMaterialParameterType::Scalar)
	{
		if (SetParameterValue(Name, Meta.Value.AsScalar(), Flags))
		{
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
			{
				Group = Meta.Group;
				SortPriority = Meta.SortPriority;
			}
			return true;
		}
	}
	return false;
}

bool UMaterialExpressionScalarParameter::SetParameterValue(FName InParameterName, float InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (DefaultValue != InValue)
		{
			DefaultValue = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue));
			}
		}
		return true;
	}
	return false;
}

void UMaterialExpressionScalarParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue))
	{
		// Callback into the editor
		FEditorSupportDelegates::NumericParameterDefaultChanged.Broadcast(this, EMaterialParameterType::Scalar, ParameterName, DefaultValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, PrimitiveDataIndex))
	{
		// Clamp value
		const int32 PrimDataIndex = PrimitiveDataIndex;
		PrimitiveDataIndex = (uint8)FMath::Clamp(PrimDataIndex, 0, FCustomPrimitiveData::NumCustomPrimitiveDataFloats-1);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionScalarParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	bool bOverrideDuplicateBehavior = false;
	TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions;
	if (Material)
	{
		Expressions = Material->GetExpressions();
	}
	else if (Function)
	{
		Expressions = Function->GetExpressions();
	}

	for (UMaterialExpression* Expression : Expressions)
	{
		if (Expression != nullptr && Expression->HasAParameterName())
		{
			UMaterialExpressionScalarParameter* ScalarExpression = Cast<UMaterialExpressionScalarParameter>(Expression);
			if (ScalarExpression 
				&& GetParameterName() == ScalarExpression->GetParameterName() 
				&& IsUsedAsAtlasPosition() != ScalarExpression->IsUsedAsAtlasPosition())
			{
				bOverrideDuplicateBehavior = true;
				break;
			}
		}
	}
	Super::ValidateParameterName(bOverrideDuplicateBehavior ? false : bAllowDuplicateName);
}

bool UMaterialExpressionScalarParameter::HasClassAndNameCollision(UMaterialExpression* OtherExpression) const
{
	UMaterialExpressionScalarParameter* ScalarExpression = Cast<UMaterialExpressionScalarParameter>(OtherExpression);
	if (ScalarExpression
		&& GetParameterName() == ScalarExpression->GetParameterName()
		&& IsUsedAsAtlasPosition() != ScalarExpression->IsUsedAsAtlasPosition())
	{
		return true;
	}
	return Super::HasClassAndNameCollision(OtherExpression);
}
#endif

//
//	UMaterialExpressionStaticSwitchParameter
//
#if WITH_EDITOR
bool UMaterialExpressionStaticSwitchParameter::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);

	const bool bAIsMaterialAttributes = (A.GetTracedInput().Expression && A.Expression->IsResultMaterialAttributes(A.OutputIndex));
	const bool bBIsMaterialAttributes = (B.GetTracedInput().Expression && B.Expression->IsResultMaterialAttributes(B.OutputIndex));

	return bAIsMaterialAttributes || bBIsMaterialAttributes;
}

EMaterialValueType GetRealInputValueType(const FExpressionInput& InExpression)
{
	const FExpressionInput& ExpressionInputRef = InExpression.GetTracedInput();
	if (ExpressionInputRef.Expression)
	{
		if (ExpressionInputRef.Expression->IsResultMaterialAttributes(ExpressionInputRef.OutputIndex))
		{
			return MCT_MaterialAttributes;
		}
		if (ExpressionInputRef.Expression->IsResultSubstrateMaterial(ExpressionInputRef.OutputIndex))
		{
			return MCT_Substrate;
		}
		return ExpressionInputRef.Expression->GetOutputValueType(ExpressionInputRef.OutputIndex);
	}

	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionStaticSwitchParameter::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		if (B.IsConnected())
		{
			return GetRealInputValueType(B);
		}
	}
	else if (InputIndex == 1)
	{
		if (A.IsConnected())
		{
			return GetRealInputValueType(A);
		}
	}

	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionStaticSwitchParameter::GetOutputValueType(int32 OutputIndex)
{
	return GetCompositeOutput(OutputIndex);
}

FExpressionInput* UMaterialExpressionStaticSwitchParameter::GetEffectiveInput(class FMaterialCompiler* Compiler)
{
	bool bSucceeded;
	const bool bValue = Compiler->GetStaticBoolValue(Compiler->StaticBoolParameter(ParameterName, DefaultValue), bSucceeded);

	//Both A and B must be connected in a parameter. 
	if (!A.GetTracedInput().IsConnected())
	{
		Compiler->Errorf(TEXT("Missing A input"));
		bSucceeded = false;
	}

	if (!B.GetTracedInput().IsConnected())
	{
		Compiler->Errorf(TEXT("Missing B input"));
		bSucceeded = false;
	}

	if (!bSucceeded)
	{
		return nullptr;
	}
	return bValue ? &A : &B;
}

int32 UMaterialExpressionStaticSwitchParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(DynamicBranch)
	{
		int32 V = Compiler->DynamicBoolParameter(ParameterName, DefaultValue);
		if (V != INDEX_NONE)
		{
			return Compiler->DynamicBranch(V, A.Compile(Compiler), B.Compile(Compiler));
		}
		else
		{
			if (DefaultValue)
			{
				return A.Compile(Compiler);
			}
			else
			{
				return B.Compile(Compiler);
			}
		}
	}

	FExpressionInput* EffectiveInput = GetEffectiveInput(Compiler);
	if (EffectiveInput)
	{
		return EffectiveInput->Compile(Compiler);			
	}
	return INDEX_NONE;
}

void UMaterialExpressionStaticSwitchParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("Switch Param (%s)"), (DefaultValue ? TEXT("True") : TEXT("False")))); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString())); 
}

FName UMaterialExpressionStaticSwitchParameter::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("True");
	}
	else
	{
		return TEXT("False");
	}
}

bool UMaterialExpressionStaticSwitchParameter::IsResultSubstrateMaterial(int32 OutputIndex)
{
	if (A.GetTracedInput().Expression && B.GetTracedInput().Expression)
	{
		return A.GetTracedInput().Expression->IsResultSubstrateMaterial(A.OutputIndex) && B.GetTracedInput().Expression->IsResultSubstrateMaterial(B.OutputIndex);
	}
	return false;
}

void UMaterialExpressionStaticSwitchParameter::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// SUBSTRATE_TODO: this is incorrect because we should only use A or B based on GetEffectiveInput, but we have no compiler at this stage so we just gather both.
	if (A.GetTracedInput().Expression)
	{
		A.GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, A.OutputIndex);
	}
	if (B.GetTracedInput().Expression)
	{
		B.GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, B.OutputIndex);
	}
}

FSubstrateOperator* UMaterialExpressionStaticSwitchParameter::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	if (DynamicBranch)
	{
		SubstrateTranslatorData.Errorf(TEXT("Static Switch nodes processing Substrate data do not support dynamic branching. The compiler must know the topology when translating HLSL (different branches could have different topologies)."));
		return nullptr;
	}

	FExpressionInput* EffectiveInput = nullptr;
	if (SubstrateTranslatorData.GetCompilerLegacy())
	{
		EffectiveInput = GetEffectiveInput(SubstrateTranslatorData.GetCompilerLegacy());
	}
	else
	{
		check(false);	// SUBSTRATE_TODO: Not implmenented. We need to do a selection of A or B based on static select evaluation.
		EffectiveInput = &A;
	}

	if (EffectiveInput && EffectiveInput->Expression)
	{
		return EffectiveInput->Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, EffectiveInput->OutputIndex);
	}
	return nullptr;
}
#endif // WITH_EDITOR


//
//	UMaterialExpressionStaticBoolParameter
//
UMaterialExpressionStaticBoolParameter::UMaterialExpressionStaticBoolParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bHidePreviewWindow = true;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionStaticBoolParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return DynamicBranch ? Compiler->DynamicBoolParameter(ParameterName, DefaultValue) : Compiler->StaticBoolParameter(ParameterName,DefaultValue);
}

int32 UMaterialExpressionStaticBoolParameter::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return INDEX_NONE;
}

void UMaterialExpressionStaticBoolParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("Static Bool Param (%s)"), (DefaultValue ? TEXT("True") : TEXT("False")))); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString())); 
}

bool UMaterialExpressionStaticBoolParameter::SetParameterValue(FName InParameterName, bool InValue, FGuid InExpressionGuid, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (DefaultValue != InValue)
		{
			DefaultValue = InValue;
			if (!EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::NoUpdateExpressionGuid))
			{
				ExpressionGUID = InExpressionGuid;
			}
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue));
			}
		}
		return true;
	}

	return false;
}
#endif

//
//	UMaterialExpressionStaticBool
//
UMaterialExpressionStaticBool::UMaterialExpressionStaticBool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bHidePreviewWindow = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionStaticBool::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->StaticBool(Value);
}

int32 UMaterialExpressionStaticBool::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return INDEX_NONE;
}

void UMaterialExpressionStaticBool::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Static Bool ") + (Value ? TEXT("(True)") : TEXT("(False)")));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionStaticSwitch
//
#if WITH_EDITOR
bool UMaterialExpressionStaticSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	// If there is a loop anywhere in this expression's inputs then we can't risk checking them. 
	// This one is a little tricky with respect to Reroute nodes. Since we are treating a dangling reroute as an empty expression, this
	// should early out, whereas IsResultMaterialAttributes on a reroute node will return false as the 
	// reroute node's input is dangling and therefore its type is unknown.
	check(OutputIndex == 0);
	if ((A.GetTracedInput().Expression && A.Expression->IsResultMaterialAttributes(A.OutputIndex)) ||
		(B.GetTracedInput().Expression && B.Expression->IsResultMaterialAttributes(B.OutputIndex)))
	{
		return true;
	}
	else
	{
		return false;
	}
}

EMaterialValueType UMaterialExpressionStaticSwitch::GetOutputValueType(int32 OutputIndex)
{
	return GetCompositeOutput(OutputIndex);
}

FExpressionInput* UMaterialExpressionStaticSwitch::GetEffectiveInput(class FMaterialCompiler* Compiler)
{
	bool bValue = DefaultValue;
	if (Value.GetTracedInput().Expression)
	{
		bool bSucceeded;
		bValue = Compiler->GetStaticBoolValue(Value.Compile(Compiler), bSucceeded);
		if (!bSucceeded)
		{
			return nullptr;
		}
	}

	return bValue ? &A : &B;
}

int32 UMaterialExpressionStaticSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 V = INDEX_NONE;
	if (Value.GetTracedInput().Expression)
	{
		V = Value.Compile(Compiler);
	}

	if(V != INDEX_NONE && Compiler->GetParameterType(V) == MCT_Bool)
	{
		return Compiler->DynamicBranch(V, A.Compile(Compiler), B.Compile(Compiler));
	}

	bool bValue = DefaultValue;
	if (V != INDEX_NONE)
	{
		bool bSucceeded;
		bValue = Compiler->GetStaticBoolValue(V, bSucceeded);
		if (!bSucceeded)
		{
			return INDEX_NONE;
		}
	}

	return bValue ? A.Compile(Compiler) : B.Compile(Compiler);
}

void UMaterialExpressionStaticSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Switch"));
}

FName UMaterialExpressionStaticSwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("True");
	}
	else if (InputIndex == 1)
	{
		return TEXT("False");
	}
	else
	{
		return TEXT("Value");
	}
}

EMaterialValueType UMaterialExpressionStaticSwitch::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0 || InputIndex == 1)
	{
		if (InputIndex == 0)
		{
			if (B.IsConnected())
			{
				return GetRealInputValueType(B);
			}
		}
		else if (InputIndex == 1)
		{
			if (A.IsConnected())
			{
				return GetRealInputValueType(A);
			}
		}

		return MCT_Unknown;
	}
	else
	{
		return MCT_Bool;
	}
}

bool UMaterialExpressionStaticSwitch::IsResultSubstrateMaterial(int32 OutputIndex)
{
	if (A.GetTracedInput().Expression && B.GetTracedInput().Expression)
	{
		return A.GetTracedInput().Expression->IsResultSubstrateMaterial(A.OutputIndex) && B.GetTracedInput().Expression->IsResultSubstrateMaterial(B.OutputIndex);
	}
	return false;
}

void UMaterialExpressionStaticSwitch::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// SUBSTRATE_TODO: this is incorrect because we should only use A or B based on GetEffectiveInput, but we have no compiler at this stage so we just gather both.
	if (A.GetTracedInput().Expression)
	{
		A.GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, A.OutputIndex);
	}
	if (B.GetTracedInput().Expression)
	{
		B.GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, B.OutputIndex);
	}
}

FSubstrateOperator* UMaterialExpressionStaticSwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FExpressionInput* EffectiveInput = nullptr;
	if (SubstrateTranslatorData.GetCompilerLegacy())
	{
		EffectiveInput = GetEffectiveInput(SubstrateTranslatorData.GetCompilerLegacy());
	}
	else
	{
		check(false);	// SUBSTRATE_TODO: Not implmenented. We need to do a selection of A or B based on static select evaluation.
		EffectiveInput = &A;
	}

	if (EffectiveInput && EffectiveInput->Expression)
	{
		return EffectiveInput->Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, EffectiveInput->OutputIndex);
	}
	return nullptr;
}
#endif

//
//	UMaterialExpressionPreviousFrameSwitch
//
#if WITH_EDITOR
bool UMaterialExpressionPreviousFrameSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	// If there is a loop anywhere in this expression's inputs then we can't risk checking them
	check(OutputIndex == 0);
	if ((CurrentFrame.Expression && CurrentFrame.Expression->IsResultMaterialAttributes(CurrentFrame.OutputIndex)) ||
		(PreviousFrame.Expression && PreviousFrame.Expression->IsResultMaterialAttributes(PreviousFrame.OutputIndex)))
	{
		return true;
	}
	else
	{
		return false;
	}
}

int32 UMaterialExpressionPreviousFrameSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Compiler->IsCurrentlyCompilingForPreviousFrame())
	{
		return PreviousFrame.Compile(Compiler);
	}
	return CurrentFrame.Compile(Compiler);
}

void UMaterialExpressionPreviousFrameSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("PreviousFrameSwitch"));
}

void UMaterialExpressionPreviousFrameSwitch::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Used to manually provide expressions for motion vector generation caused by changes in world position offset between frames."), 40, OutToolTip);
}

FName UMaterialExpressionPreviousFrameSwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Current Frame");
	}
	else
	{
		return TEXT("Previous Frame");
	}
}

EMaterialValueType UMaterialExpressionPreviousFrameSwitch::GetInputValueType(int32 InputIndex)
{
	return MCT_Unknown;
}
#endif

//
//	UMaterialExpressionQualitySwitch
//
#if WITH_EDITOR
FExpressionInput* UMaterialExpressionQualitySwitch::GetEffectiveInput(class FMaterialCompiler* Compiler)
{
	const EMaterialQualityLevel::Type QualityLevelToCompile = Compiler->GetQualityLevel();
	if (QualityLevelToCompile != EMaterialQualityLevel::Num)
	{
		check(QualityLevelToCompile < UE_ARRAY_COUNT(Inputs));
		FExpressionInput QualityInputTraced = Inputs[QualityLevelToCompile].GetTracedInput();
		if (QualityInputTraced.Expression)
		{
			return &Inputs[QualityLevelToCompile];
		}
	}
	return &Default;
}

int32 UMaterialExpressionQualitySwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FExpressionInput DefaultTraced = Default.GetTracedInput();
	if (!DefaultTraced.Expression)
	{
		return Compiler->Errorf(TEXT("Quality switch missing default input"));
	}

	// Evaluate all inputs during validation to minimize validation permutations.
	if (Compiler->IsValidating())
	{
		Default.Compile(Compiler);
		for (uint32 QualityLevelToCompile = 0; QualityLevelToCompile < EMaterialQualityLevel::Num; ++QualityLevelToCompile)
		{
			check(QualityLevelToCompile < UE_ARRAY_COUNT(Inputs));
			FExpressionInput QualityInputTraced = Inputs[QualityLevelToCompile].GetTracedInput();
			if (QualityInputTraced.Expression && QualityInputTraced.Expression != DefaultTraced.Expression)
			{
				Inputs[QualityLevelToCompile].Compile(Compiler);
			}
		}
	}

	FExpressionInput* EffectiveInput = GetEffectiveInput(Compiler);
	check(EffectiveInput);
	return EffectiveInput->Compile(Compiler);
}

void UMaterialExpressionQualitySwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Quality Switch"));
}

FName UMaterialExpressionQualitySwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Default");
	}

	return GetMaterialQualityLevelFName((EMaterialQualityLevel::Type)(InputIndex - 1));
}

bool UMaterialExpressionQualitySwitch::IsInputConnectionRequired(int32 InputIndex) const
{
	return InputIndex == 0;
}

bool UMaterialExpressionQualitySwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		// If there is a loop anywhere in this expression's inputs then we can't risk checking them
		if (It->Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}

	return false;
}

bool UMaterialExpressionQualitySwitch::IsResultSubstrateMaterial(int32 OutputIndex)
{
	// Return Substrate only if all inputs are Substrate
	bool bResultSubstrateMaterial = Default.GetTracedInput().Expression && Default.GetTracedInput().Expression->IsResultSubstrateMaterial(Default.OutputIndex);
	for (int i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		bResultSubstrateMaterial = bResultSubstrateMaterial && Inputs[i].GetTracedInput().Expression && Inputs[i].GetTracedInput().Expression->IsResultSubstrateMaterial(Inputs[i].OutputIndex);
	}
	return bResultSubstrateMaterial;
}

void UMaterialExpressionQualitySwitch::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// SUBSTRATE_TODO: this is incorrect because we should only use a single input based on GetEffectiveInput, but we have no compiler at this stage so we just gather all.
	if (Default.GetTracedInput().Expression)
	{
		Default.GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, Default.OutputIndex);
	}
	for (int i = 0; i < EMaterialQualityLevel::Num; ++i)
	{
		if (Inputs[i].GetTracedInput().Expression)
		{
			Inputs[i].GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, Inputs[i].OutputIndex);
		}
	}
}

FSubstrateOperator* UMaterialExpressionQualitySwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FExpressionInput DefaultTraced = Default.GetTracedInput();
	if (!DefaultTraced.Expression)
	{
		SubstrateTranslatorData.Errorf(TEXT("Quality switch missing default input"));
		return nullptr;
	}

	FExpressionInput* EffectiveInput = nullptr;
	if (SubstrateTranslatorData.GetCompilerLegacy())
	{
		EffectiveInput = GetEffectiveInput(SubstrateTranslatorData.GetCompilerLegacy());
	}
	else
	{
		check(false);	// SUBSTRATE_TODO: Not implmenented. We need to do a selection of A or B based on static select evaluation.
		EffectiveInput = &Default;
	}

	if (EffectiveInput && EffectiveInput->Expression)
	{
		return EffectiveInput->Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, EffectiveInput->OutputIndex);
	}
	return nullptr;
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionFeatureLevelSwitch
//
#if WITH_EDITOR
ERHIFeatureLevel::Type UMaterialExpressionFeatureLevelSwitch::GetFeatureLevelToCompile(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel)
{
	ERHIFeatureLevel::Type FeatureLevelToCompile = FeatureLevel;

	// PreviewPlatform can have a different feature level in order to support previewing the platform
	// But we still want to respect the material logic of the parent platform
	if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(ShaderPlatform))
	{
		const EShaderPlatform ParentShaderPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(ShaderPlatform);
		FeatureLevelToCompile = FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ParentShaderPlatform);
	}

	check(FeatureLevelToCompile < UE_ARRAY_COUNT(Inputs));
	return FeatureLevelToCompile;
}

int32 UMaterialExpressionFeatureLevelSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const FStaticShaderPlatform ShaderPlatform = Compiler->GetShaderPlatform();
	ERHIFeatureLevel::Type FeatureLevelToCompile = GetFeatureLevelToCompile(ShaderPlatform, Compiler->GetFeatureLevel());
	if (GetMaterialSimulateMobileForLowQuality(ShaderPlatform) &&
		(Compiler->GetQualityLevel() == EMaterialQualityLevel::Low) &&
		(FeatureLevelToCompile > ERHIFeatureLevel::ES3_1))
	{
		FeatureLevelToCompile = ERHIFeatureLevel::ES3_1;
	}
	FExpressionInput& FeatureInput = Inputs[FeatureLevelToCompile];

	if (!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Feature Level switch missing default input"));
	}

	if (FeatureInput.GetTracedInput().Expression)
	{
		return FeatureInput.Compile(Compiler);
	}

	return Default.Compile(Compiler);
}

void UMaterialExpressionFeatureLevelSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Feature Level Switch"));
}

FName UMaterialExpressionFeatureLevelSwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Default");
	}

	FName FeatureLevelName;
	GetFeatureLevelName((ERHIFeatureLevel::Type)(InputIndex - 1), FeatureLevelName);
	return FeatureLevelName;
}

bool UMaterialExpressionFeatureLevelSwitch::IsInputConnectionRequired(int32 InputIndex) const
{
	return InputIndex == 0;
}


bool UMaterialExpressionFeatureLevelSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		// If there is a loop anywhere in this expression's inputs then we can't risk checking them
		if (It->GetTracedInput().Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}

	return false;
}
#endif // WITH_EDITOR

void UMaterialExpressionFeatureLevelSwitch::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();
	UnderlyingArchive.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (UnderlyingArchive.IsLoading() && UnderlyingArchive.UEVer() < VER_UE4_RENAME_SM3_TO_ES3_1)
	{
		// Copy the ES2 input to SM3 (since SM3 will now become ES3_1 and we don't want broken content)
		Inputs[ERHIFeatureLevel::ES3_1] = Inputs[ERHIFeatureLevel::ES2_REMOVED];
	}

	if (UnderlyingArchive.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::RemovedSM4)
	{
		Inputs[ERHIFeatureLevel::SM4_REMOVED] = UMaterialExpressionFeatureLevelSwitch::Default;
	}
}

//
//	UMaterialExpressionDataDrivenShaderPlatformInfoSwitch
//

UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::UMaterialExpressionDataDrivenShaderPlatformInfoSwitch(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	FText NAME_Utility(LOCTEXT("Utility", "Utility"));
	MenuCategories.Add(NAME_Utility);
	bCollapsed = false;
	bContainsInvalidProperty = false;
#endif // WITH_EDITORONLY_DATA
}

TArray<FString> UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::GetNameOptions() const
{
	TArray<FString> Output;
#if WITH_EDITOR
	for (const auto& DDSPINames : FGenericDataDrivenShaderPlatformInfo::PropertyToShaderPlatformFunctionMap)
	{
		Output.Add(DDSPINames.Key);
	}
#endif
	return Output;
}

#if WITH_EDITOR
bool IsDataDrivenShaderPlatformInfoSwitchValid(TArray<FDataDrivenShaderPlatformInfoInput>& DDSPIPropertyNames, const UMaterial* Material)
{
	for (const FDataDrivenShaderPlatformInfoInput& DDSPIInput : DDSPIPropertyNames)
	{
		if (DDSPIInput.InputName == NAME_None)
		{
			continue;
		}

		bool PropertyExists = FGenericDataDrivenShaderPlatformInfo::PropertyToShaderPlatformFunctionMap.Find(DDSPIInput.InputName.ToString()) != nullptr;
		if (!PropertyExists)
		{
			return true;
		}
	}

	return false;
}
#endif

void UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	bContainsInvalidProperty = IsDataDrivenShaderPlatformInfoSwitchValid(DDSPIPropertyNames, Material);
#endif
}

#if WITH_EDITOR
FExpressionInput* UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::GetEffectiveInput(EShaderPlatform ShaderPlatform, FString& OutError)
{
	if (bContainsInvalidProperty || DDSPIPropertyNames.IsEmpty())
	{
		OutError = FString::Printf(TEXT("%s is using a DataDrivenShaderPlatformInfoSwitch whose condition is invalid. Default material will be used until this is fixed"), Material ? *(Material->GetName()) : TEXT("Unknown"));
		return nullptr;
	}

	check(FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform));

	bool bAllNamesAreNone = true;
	bool bCheck = true;
	for (const FDataDrivenShaderPlatformInfoInput& DDSPIInput : DDSPIPropertyNames)
	{
		if (DDSPIInput.InputName == NAME_None)
		{
			continue;
		}

		bAllNamesAreNone = false;
		bool bCheckProperty = FGenericDataDrivenShaderPlatformInfo::PropertyToShaderPlatformFunctionMap[DDSPIInput.InputName.ToString()](ShaderPlatform);
		if (DDSPIInput.PropertyCondition == EDataDrivenShaderPlatformInfoCondition::COND_True)
		{
			bCheck &= bCheckProperty;
		}
		else
		{
			bCheck &= !bCheckProperty;
		}
	}

	if (bAllNamesAreNone)
	{
		OutError = FString::Printf(TEXT("%s is using a DataDrivenShaderPlatformInfoSwitch whose condition is empty. Default material will be used until this is fixed"), Material ? *(Material->GetName()) : TEXT("Unknown"));
		return nullptr;
	}
	else if (bCheck)
	{
		return &InputTrue;
	}
	else
	{
		return &InputFalse;
	}
}

int32 UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FString Error;
	FExpressionInput* EffectiveInput = GetEffectiveInput(Compiler->GetShaderPlatform(), Error);

	if (!EffectiveInput)
	{
		return Compiler->Error(*Error);
	}

	return EffectiveInput->Compile(Compiler);
}

void UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("ShaderPlatformInfo Switch"));
}

bool UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::IsInputConnectionRequired(int32 InputIndex) const
{
	return true;
}

FName UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::GetInputName(int32 InputIndex) const
{
	TStringBuilder<128> Condition;
	bool bIsFirst = true;
	for (const FDataDrivenShaderPlatformInfoInput& DDSPIInput : DDSPIPropertyNames)
	{
		if (DDSPIInput.InputName == NAME_None)
		{
			continue;
		}

		if (!bIsFirst)
		{
			Condition.Append(TEXT(" && "));
		}

		if (DDSPIInput.PropertyCondition == EDataDrivenShaderPlatformInfoCondition::COND_False)
		{
			Condition.Append(TEXT("!"));
		}

		Condition.Append(DDSPIInput.InputName.ToString());
		bIsFirst = false;
	}

	const FString ConditionString = Condition.ToString();
	FString NegateConditionString = TEXT("!(") + ConditionString + TEXT(")");

	if (InputIndex == 0)
	{
		return *ConditionString;
	}
	else if (InputIndex == 1)
	{
		return *NegateConditionString;
	}
	return NAME_None;
}

bool UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		// If there is a loop anywhere in this expression's inputs then we can't risk checking them
		TObjectPtr<class UMaterialExpression> Expression = It->Expression;
		if (Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}

	return false;
}

void UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, DDSPIPropertyNames))
	{
		if (GraphNode)
		{
			bContainsInvalidProperty = IsDataDrivenShaderPlatformInfoSwitchValid(DDSPIPropertyNames, Material);
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FSubstrateOperator* UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	ThrowSubstrateUnsupportedSwitchNodeError(SubstrateTranslatorData, *GetClass()->GetName());
	return nullptr;
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionBindlessSwitch
//

#if WITH_EDITOR

bool IsBindlessEnabledForCompiler(EShaderPlatform ShaderPlatform, EShaderFrequency ShaderFrequency)
{
	const ERHIBindlessConfiguration BindlessConfiguration = FShaderPlatformConfig::GetBindlessConfiguration(ShaderPlatform);
	switch (BindlessConfiguration)
	{
		case ERHIBindlessConfiguration::All:        return true;
		case ERHIBindlessConfiguration::Minimal:    return true;
		case ERHIBindlessConfiguration::RayTracing: return IsRayTracingShaderFrequency(ShaderFrequency);
	}
	return false;
}

static bool IsBindlessEnabledForCompiler(FMaterialCompiler* Compiler)
{
	return IsBindlessEnabledForCompiler(Compiler->GetShaderPlatform(), Compiler->GetCurrentShaderFrequency());
}

int32 UMaterialExpressionBindlessSwitch::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (IsBindlessEnabledForCompiler(Compiler))
	{
		return Bindless.Compile(Compiler);
	}

	return Default.Compile(Compiler);
}

void UMaterialExpressionBindlessSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Bindless Switch"));
}

bool UMaterialExpressionBindlessSwitch::IsInputConnectionRequired(int32 InputIndex) const
{
	return true;
}

bool UMaterialExpressionBindlessSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		// If there is a loop anywhere in this expression's inputs then we can't risk checking them
		if (It->Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}

	return false;
}

void UMaterialExpressionBindlessSwitch::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Allows material to define specialized behavior when being rendered with bindless enabled."), 40, OutToolTip);
}
#endif

//
// UMaterialExpressionTextureCollection
//

UMaterialExpressionTextureCollection::UMaterialExpressionTextureCollection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Emplace(TEXT("TextureCollection"));
	Outputs.Emplace(TEXT("TextureCount"));
	
	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureCollection::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const int32 TextureCollectionCodeIndex = Compiler->TextureCollection(TextureCollection);

	if (OutputIndex == 1)
	{
		return Compiler->TextureCollectionCount(TextureCollectionCodeIndex);
	}

	return TextureCollectionCodeIndex;
}

void UMaterialExpressionTextureCollection::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Emplace(TEXT("Texture Collection"));
}

EMaterialValueType UMaterialExpressionTextureCollection::GetOutputValueType(int32 OutputIndex)
{
	if (OutputIndex == 1)
	{
		return MCT_UInt1;
	}

	return MCT_TextureCollection;
}
#endif

//
// UMaterialExpressionTextureCollectionParameter
//

UMaterialExpressionTextureCollectionParameter::UMaterialExpressionTextureCollectionParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bIsParameterExpression = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTextureCollectionParameter::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FString ErrorMessage;
	if (!TextureCollectionIsValid(TextureCollection, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	const int32 TextureCollectionCodeIndex = Compiler->TextureCollectionParameter(ParameterName, TextureCollection);

	if (OutputIndex == 1)
	{
		return Compiler->TextureCollectionCount(TextureCollectionCodeIndex);
	}

	return TextureCollectionCodeIndex;
}

void UMaterialExpressionTextureCollectionParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Emplace(TEXT("Texture Collection Parameter"));
}

bool UMaterialExpressionTextureCollectionParameter::CanRenameNode() const
{
	return true;
}

FString UMaterialExpressionTextureCollectionParameter::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionTextureCollectionParameter::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}

bool UMaterialExpressionTextureCollectionParameter::HasAParameterName() const
{
	return true;
}

FName UMaterialExpressionTextureCollectionParameter::GetParameterName() const
{
	return ParameterName;
}

void UMaterialExpressionTextureCollectionParameter::SetParameterName(const FName& Name)
{
	ParameterName = Name;
}

void UMaterialExpressionTextureCollectionParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	ValidateParameterNameInternal(this, Material, bAllowDuplicateName);
}

bool UMaterialExpressionTextureCollectionParameter::GetParameterValue(FMaterialParameterMetadata& OutMeta) const
{
	OutMeta.Value = TextureCollection;
	OutMeta.Description = Desc;
	OutMeta.ExpressionGuid = ExpressionGUID;
	OutMeta.Group = Group;
	OutMeta.SortPriority = SortPriority;
	return true;
}

bool UMaterialExpressionTextureCollectionParameter::SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (Meta.Value.Type == EMaterialParameterType::TextureCollection)
	{
		if (SetParameterValue(Name, Meta.Value.TextureCollection, Flags))
		{
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
			{
				Group = Meta.Group;
				SortPriority = Meta.SortPriority;
			}
			return true;
		}
	}
	return false;
}

bool UMaterialExpressionTextureCollectionParameter::TextureCollectionIsValid(UTextureCollection* InTextureCollection, FString& OutMessage)
{
	if (!InTextureCollection)
	{
		OutMessage = TEXT("Requires valid texture collection");
		return false;
	}

	return true;
}

bool UMaterialExpressionTextureCollectionParameter::SetParameterValue(const FName& InParameterName, UTextureCollection* InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (TextureCollection != InValue)
		{
			TextureCollection = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, TextureCollection));
			}
		}
		return true;
	}
	return false;
}
#endif

FGuid& UMaterialExpressionTextureCollectionParameter::GetParameterExpressionId()
{
	return ExpressionGUID;
}

//
// UMaterialExpressionTextureObjectFromCollection
//
#if WITH_EDITOR
int32 UMaterialExpressionTextureObjectFromCollection::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 TextureCollectionCodeIndex = TextureCollection.GetTracedInput().Expression ? TextureCollection.Compile(Compiler) : Compiler->TextureCollection(TextureCollectionObject);
	int32 IndexIntoCollectionCodeIndex = CollectionIndex.GetTracedInput().Expression ? CollectionIndex.Compile(Compiler) : Compiler->Constant(ConstCollectionIndex);
	int32 TextureFromCollectionCodeIndex = Compiler->TextureFromCollection(
		TextureCollectionCodeIndex,
		IndexIntoCollectionCodeIndex,
		MaterialValueTypeFromTextureCollectionMemberType(TextureType)
	);
	return TextureFromCollectionCodeIndex;
}

EMaterialValueType UMaterialExpressionTextureObjectFromCollection::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_TextureCollection;
	}

	return MCT_UInt1;
}

EMaterialValueType UMaterialExpressionTextureObjectFromCollection::GetOutputValueType(int32 OutputIndex)
{
	return MaterialValueTypeFromTextureCollectionMemberType(TextureType);
}

void UMaterialExpressionTextureObjectFromCollection::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Emplace(TEXT("Texture Object From Collection"));
}

//
//	UMaterialExpressionRequiredSamplersSwitch
//
int32 UMaterialExpressionRequiredSamplersSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const EShaderPlatform ShaderPlatform = Compiler->GetShaderPlatform();
	check(FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform));
	const bool bCheck = RequiredSamplers <= FDataDrivenShaderPlatformInfo::GetMaxSamplers(ShaderPlatform);
	if (bCheck)
	{
		return InputTrue.Compile(Compiler);
	}
	else
	{
		return InputFalse.Compile(Compiler);
	}
}

void UMaterialExpressionRequiredSamplersSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Required Samplers Switch"));
}

bool UMaterialExpressionRequiredSamplersSwitch::IsInputConnectionRequired(int32 InputIndex) const
{
	return true;
}

FName UMaterialExpressionRequiredSamplersSwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Within platform limit");
	}
	else if (InputIndex == 1)
	{
		return TEXT("Over platform limit");
	}
	return NAME_None;
}

bool UMaterialExpressionRequiredSamplersSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		// If there is a loop anywhere in this expression's inputs then we can't risk checking them
		if (It->Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}

	return false;
}

#endif // WITH_EDITOR

//
//	UMaterialExpressionShadingPathSwitch
//
#if WITH_EDITOR
ERHIShadingPath::Type UMaterialExpressionShadingPathSwitch::GetShadingPathToCompile(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, EMaterialQualityLevel::Type MaterialQualityLevel)
{
	ERHIShadingPath::Type ShadingPathToCompile = ERHIShadingPath::Deferred;

	static_assert(int(ERHIShadingPath::Deferred) == 0);

	int32 ShadingPathNodeOverride = GetMaterialShadingPathNodeOverride(ShaderPlatform);
	if (ShadingPathNodeOverride != -1)
	{
		ShadingPathToCompile = ERHIShadingPath::Type(ShadingPathNodeOverride);
	}
	else
	{
		if (GetMaterialSimulateMobileForLowQuality(ShaderPlatform) &&
			(MaterialQualityLevel == EMaterialQualityLevel::Low) &&
			(FeatureLevel > ERHIFeatureLevel::ES3_1))
		{
			ShadingPathToCompile = ERHIShadingPath::Mobile;
		}
		else if (IsForwardShadingEnabled(ShaderPlatform))
		{
			ShadingPathToCompile = ERHIShadingPath::Forward;
		}
		else if (FeatureLevel < ERHIFeatureLevel::SM5)
		{
			ShadingPathToCompile = ERHIShadingPath::Mobile;
		}
	}

	check(ShadingPathToCompile < UE_ARRAY_COUNT(Inputs));
	return ShadingPathToCompile;
}

FExpressionInput* UMaterialExpressionShadingPathSwitch::GetEffectiveInput(class FMaterialCompiler* Compiler)
{
	ERHIShadingPath::Type ShadingPathToCompile = GetShadingPathToCompile(Compiler->GetShaderPlatform(), Compiler->GetFeatureLevel(), Compiler->GetQualityLevel());
	FExpressionInput ShadingPathInput = Inputs[ShadingPathToCompile].GetTracedInput();
	if (ShadingPathInput.Expression)
	{
		return &Inputs[ShadingPathToCompile];
	}
	return &Default;
}

int32 UMaterialExpressionShadingPathSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FExpressionInput DefaultTraced = Default.GetTracedInput();
	if (!DefaultTraced.Expression)
	{
		return Compiler->Errorf(TEXT("Shading path switch missing default input"));
	}

	FExpressionInput* EffectiveInput = GetEffectiveInput(Compiler);
	check(EffectiveInput);
	return EffectiveInput->Compile(Compiler);
}

void UMaterialExpressionShadingPathSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Shading Path Switch"));
}

FName UMaterialExpressionShadingPathSwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		return TEXT("Default");
	}

	FName ShadingPathName;
	GetShadingPathName((ERHIShadingPath::Type)(InputIndex - 1), ShadingPathName);
	return ShadingPathName;
}

bool UMaterialExpressionShadingPathSwitch::IsInputConnectionRequired(int32 InputIndex) const
{
	return InputIndex == 0;
}

bool UMaterialExpressionShadingPathSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	check(OutputIndex == 0);
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		// If there is a loop anywhere in this expression's inputs then we can't risk checking them
		if (It->Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}

	return false;
}

bool UMaterialExpressionShadingPathSwitch::IsResultSubstrateMaterial(int32 OutputIndex)
{
	// Return Substrate only if all inputs are Substrate
	bool bResultSubstrateMaterial = Default.GetTracedInput().Expression && Default.GetTracedInput().Expression->IsResultSubstrateMaterial(Default.OutputIndex);
	for (int i = 0; i < ERHIShadingPath::Num; ++i)
	{
		bResultSubstrateMaterial = bResultSubstrateMaterial && Inputs[i].GetTracedInput().Expression && Inputs[i].GetTracedInput().Expression->IsResultSubstrateMaterial(Inputs[i].OutputIndex);
	}
	return bResultSubstrateMaterial;
}

void UMaterialExpressionShadingPathSwitch::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// SUBSTRATE_TODO: this is incorrect because we should only use a single input based on GetEffectiveInput, but we have no compiler at this stage so we just gather all.
	if (Default.GetTracedInput().Expression)
	{
		Default.GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, Default.OutputIndex);
	}
	for (int i = 0; i < ERHIShadingPath::Num; ++i)
	{
		if (Inputs[i].GetTracedInput().Expression)
		{
			Inputs[i].GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, Inputs[i].OutputIndex);
		}
	}
}

FSubstrateOperator* UMaterialExpressionShadingPathSwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FExpressionInput DefaultTraced = Default.GetTracedInput();
	if (!DefaultTraced.Expression)
	{
		SubstrateTranslatorData.Errorf(TEXT("Shading path switch missing default input"));
		return nullptr;
	}

	FExpressionInput* EffectiveInput = nullptr;
	if (SubstrateTranslatorData.GetCompilerLegacy())
	{
		EffectiveInput = GetEffectiveInput(SubstrateTranslatorData.GetCompilerLegacy());
	}
	else
	{
		check(false);	// SUBSTRATE_TODO: Not implmenented. We need to do a selection of A or B based on static select evaluation.
		EffectiveInput = &Default;
	}

	if (EffectiveInput && EffectiveInput->Expression)
	{
		return EffectiveInput->Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, EffectiveInput->OutputIndex);
	}
	return nullptr;
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionNormalize
//
#if WITH_EDITOR
int32 UMaterialExpressionNormalize::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!VectorInput.GetTracedInput().Expression)
		return Compiler->Errorf(TEXT("Missing Normalize input"));

	int32	V = VectorInput.Compile(Compiler);
	return Compiler->Normalize(V);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionTruncateLWC
//
#if WITH_EDITOR
int32 UMaterialExpressionTruncateLWC::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input"));
	}

	int32 CodeIndex = Input.Compile(Compiler);
	return Compiler->TruncateLWC(CodeIndex);
}
#endif // WITH_EDITOR

UMaterialExpressionVertexColor::UMaterialExpressionVertexColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 0, 1));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionVertexColor::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->VertexColor();
}

void UMaterialExpressionVertexColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Vertex Color"));
}
#endif // WITH_EDITOR

UMaterialExpressionFontSignedDistance::UMaterialExpressionFontSignedDistance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Signed Distance"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Smooth Signed Distance"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Pixel Distance Factor"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Implicit Opacity"), 1, 0, 0, 0, 1));

	bShowOutputNameOnPin = true;
	bShowMaskColorsOnPin = false;
#endif
}

#if WITH_EDITOR
void UMaterialExpressionFontSignedDistance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Font Signed Distance"));
}
#endif // WITH_EDITOR

UMaterialExpressionParticleColor::UMaterialExpressionParticleColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("R"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("G"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("B"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));

	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionParticleColor::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleColor();
}

void UMaterialExpressionParticleColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Color"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionParticlePositionWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticlePosition(OriginType);
}

void UMaterialExpressionParticlePositionWS::GetCaption(TArray<FString>& OutCaptions) const
{
	switch (OriginType)
	{
		case EPositionOrigin::Absolute:
		{
			OutCaptions.Add(TEXT("Particle Position (Absolute)"));
			break;
		}

		case EPositionOrigin::CameraRelative:
		{
			OutCaptions.Add(TEXT("Particle Position (Camera Relative)"));
			break;
		}

		default:
		{
			UE_LOGF(LogMaterial, Fatal, "Unknown position origin type");
			break;
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionParticleRadius::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleRadius();
}

void UMaterialExpressionParticleRadius::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Radius"));
}
#endif // WITH_EDITOR

UMaterialExpressionDynamicParameter::UMaterialExpressionDynamicParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	bHidePreviewWindow = true;
#endif // WITH_EDITORONLY_DATA

	ParamNames.Add(TEXT("Param1"));
	ParamNames.Add(TEXT("Param2"));
	ParamNames.Add(TEXT("Param3"));
	ParamNames.Add(TEXT("Param4"));

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 0, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionDynamicParameter::Compile( FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->DynamicParameter(DefaultValue, ParameterIndex);
}

TArray<FExpressionOutput>& UMaterialExpressionDynamicParameter::GetOutputs()
{
	Outputs[0].OutputName = *(ParamNames[0]);
	Outputs[1].OutputName = *(ParamNames[1]);
	Outputs[2].OutputName = *(ParamNames[2]);
	Outputs[3].OutputName = *(ParamNames[3]);
	return Outputs;
}

int32 UMaterialExpressionDynamicParameter::GetWidth() const
{
	return ME_STD_THUMBNAIL_SZ+(ME_STD_BORDER*2);
}

void UMaterialExpressionDynamicParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Dynamic Parameter"));
}

bool UMaterialExpressionDynamicParameter::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	for( int32 Index=0;Index<ParamNames.Num();Index++ )
	{
		if( ParamNames[Index].Contains(SearchQuery) )
		{
			return true;
		}
	}

	return Super::MatchesSearchQuery(SearchQuery);
}


void UMaterialExpressionDynamicParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, ParamNames))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}
}

#endif // WITH_EDITOR

void UMaterialExpressionDynamicParameter::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_DYNAMIC_PARAMETER_DEFAULT_VALUE)
	{
		DefaultValue = FLinearColor::Black;//Old data should default to 0.0f;
	}
}

#if WITH_EDITORONLY_DATA
void UMaterialExpressionDynamicParameter::UpdateDynamicParameterProperties()
{
	check(Material);
	for (UMaterialExpression* Expression : Material->GetExpressions())
	{
		const UMaterialExpressionDynamicParameter* DynParam = Cast<UMaterialExpressionDynamicParameter>(Expression);
		if (CopyDynamicParameterProperties(DynParam))
		{
			break;
		}
	}
}
#endif // WITH_EDITORONLY_DATA

bool UMaterialExpressionDynamicParameter::CopyDynamicParameterProperties(const UMaterialExpressionDynamicParameter* FromParam)
{
	if (FromParam && (FromParam != this) && ParameterIndex == FromParam->ParameterIndex)
	{
		for (int32 NameIndex = 0; NameIndex < 4; NameIndex++)
		{
			ParamNames[NameIndex] = FromParam->ParamNames[NameIndex];
		}
		DefaultValue = FromParam->DefaultValue;
		return true;
	}
	return false;
}

//
//	MaterialExpressionParticleSubUV
//
#if WITH_EDITOR
int32 UMaterialExpressionParticleSubUV::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Overriding texture with texture parameter
	TObjectPtr<class UTexture> TextureToCompile = Texture;
	TEnumAsByte<enum EMaterialSamplerType> SamplerTypeToUse = SamplerType; 
	if (TextureObject.GetTracedInput().Expression != nullptr)
	{
		if (UMaterialExpressionTextureObject* TextureObjectParameter = Cast<UMaterialExpressionTextureObject>(TextureObject.GetTracedInput().Expression))
		{
			TextureToCompile = TextureObjectParameter->Texture;
			SamplerTypeToUse = TextureObjectParameter->SamplerType;
		}
		else if (UMaterialExpressionTextureObjectParameter* TextureSampleParameter = Cast<UMaterialExpressionTextureObjectParameter>(TextureObject.GetTracedInput().Expression))
		{
			TextureToCompile = TextureSampleParameter->Texture;
			SamplerTypeToUse = TextureSampleParameter->SamplerType;
		}
	}

	if (TextureToCompile)
	{
		FString SamplerTypeError;
		if (!MaterialExpressionUtils::VerifySamplerType(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform(), TextureToCompile, SamplerTypeToUse, SamplerTypeError))
		{
			return Compiler->Errorf(TEXT("%s"), *SamplerTypeError);
		}
		int32 TextureCodeIndex = Compiler->Texture(TextureToCompile, SamplerTypeToUse);
        if (TextureObject.GetTracedInput().Expression)
		{
			TextureCodeIndex = TextureObject.Compile(Compiler);
		}
		return ParticleSubUV(Compiler, TextureCodeIndex, SamplerTypeToUse, CompileMipValue0(Compiler), CompileMipValue1(Compiler), MipValueMode, bBlend);
	}
	else
	{
		return Compiler->Errorf(TEXT("Missing ParticleSubUV input texture"));
	}
}

int32 UMaterialExpressionParticleSubUV::GetWidth() const
{
	return ME_STD_THUMBNAIL_SZ+(ME_STD_BORDER*2);
}

void UMaterialExpressionParticleSubUV::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle SubUV"));
}
#endif // WITH_EDITOR

//
//	MaterialExpressionParticleSubUVProperties
//
UMaterialExpressionParticleSubUVProperties::UMaterialExpressionParticleSubUVProperties(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("TextureCoordinate0"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("TextureCoordinate1"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Blend")));
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionParticleSubUVProperties::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleSubUVProperty(OutputIndex);
}

void UMaterialExpressionParticleSubUVProperties::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Provides direct access to properties used to implement particle UV frame animation."), 40, OutToolTip);
}

void UMaterialExpressionParticleSubUVProperties::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle SubUV Properties"));
}
#endif // WITH_EDITOR

//
//	MaterialExpressionParticleMacroUV
//
#if WITH_EDITOR
void UMaterialExpressionParticleMacroUV::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle MacroUV"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionLightVector::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->LightVector();
}

void UMaterialExpressionLightVector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Light Vector"));
}
#endif // WITH_EDITOR

UMaterialExpressionScreenPosition::UMaterialExpressionScreenPosition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("ViewportUV")));
	Outputs.Add(FExpressionOutput(TEXT("PixelPosition")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionScreenPosition::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (OutputIndex == 1)
	{
		return Compiler->GetPixelPosition();
	}
	return Compiler->GetViewportUV();
}

void UMaterialExpressionScreenPosition::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ScreenPosition"));
}
#endif // WITH_EDITOR

UMaterialExpressionViewProperty::UMaterialExpressionViewProperty(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Property")));
	Outputs.Add(FExpressionOutput(TEXT("InvProperty")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionViewProperty::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// To make sure any material that were correctly handling BufferUV != ViewportUV, we just lie to material
	// to make it believe ViewSize == BufferSize, so they are still compatible with SceneTextureLookup().
	// TODO: Remove MEVP_BufferSize, MEVP_ViewportOffset and do this at material load time. 
	if (Property == MEVP_BufferSize)
	{
		return Compiler->ViewProperty(MEVP_ViewSize, OutputIndex == 1);
	}
	else if (Property == MEVP_ViewportOffset)
	{
		// We don't care about OutputIndex == 1 because doesn't have any meaning and 
		// was already returning NaN on unconstrained unique view rendering.
		return Compiler->Constant2(0.0f, 0.0f);
	}

	return Compiler->ViewProperty(Property, OutputIndex == 1);
}

void UMaterialExpressionViewProperty::GetCaption(TArray<FString>& OutCaptions) const
{
	const UEnum* ViewPropertyEnum = StaticEnum<EMaterialExposedViewProperty>();
	check(ViewPropertyEnum);

	OutCaptions.Add(ViewPropertyEnum->GetDisplayNameTextByValue(Property).ToString());
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionViewSize::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ViewProperty(MEVP_ViewSize);
}

void UMaterialExpressionViewSize::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ViewSize"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterialExpressionIsOrthographic::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("IsOrthographic"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterialExpressionDeltaTime::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DeltaTime"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionSceneTexelSize::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// To make sure any material that were correctly handling BufferUV != ViewportUV, we just lie to material
	// to make it believe ViewSize == BufferSize, so they are still compatible with SceneTextureLookup().
	return Compiler->ViewProperty(MEVP_ViewSize, /* InvProperty = */ true);
}

void UMaterialExpressionSceneTexelSize::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SceneTexelSize"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionSquareRoot::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing square root input"));
	}
	return Compiler->SquareRoot(Input.Compile(Compiler));
}

void UMaterialExpressionSquareRoot::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Sqrt"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionSRGBColorToWorkingColorSpace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing sRGBColorToWorkingColorSpace input"));
	}

	int32 Color = Input.Compile(Compiler);
	int32 Result = Color;

	if (!UE::Color::FColorSpace::GetWorking().IsSRGB())
	{
		const UE::Color::FColorSpaceTransform& Transform = UE::Color::FColorSpaceTransform::GetSRGBToWorkingColorSpace();

		const int32 R = Compiler->Dot(Color, Compiler->Constant3((float)Transform.M[0][0], (float)Transform.M[1][0], (float)Transform.M[2][0]));
		const int32 G = Compiler->Dot(Color, Compiler->Constant3((float)Transform.M[0][1], (float)Transform.M[1][1], (float)Transform.M[2][1]));
		const int32 B = Compiler->Dot(Color, Compiler->Constant3((float)Transform.M[0][2], (float)Transform.M[1][2], (float)Transform.M[2][2]));
		Result = Compiler->AppendVector(Compiler->AppendVector(R, G), B);

		EMaterialValueType VectorType = Compiler->GetParameterType(Color);
		if (VectorType & MCT_Float4 || VectorType == MCT_LWCVector4)
		{
			// We preserve the original alpha when applicable
			Result = Compiler->AppendVector(Result, Compiler->ComponentMask(Color, false, false, false, true));
		}
	}

	return Result;
}

void UMaterialExpressionSRGBColorToWorkingColorSpace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("sRGBColorToWorkingColorSpace"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPixelDepth
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPixelDepth::UMaterialExpressionPixelDepth(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionPixelDepth::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// resulting index to compiled code chunk
	// add the code chunk for the pixel's depth     
	int32 Result = Compiler->PixelDepth();
	return Result;
}

void UMaterialExpressionPixelDepth::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PixelDepth"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSceneDepth
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionSceneDepth::UMaterialExpressionSceneDepth(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
#endif
}

void UMaterialExpressionSceneDepth::PostLoad()
{
	Super::PostLoad();

	if(GetLinkerUEVersion() < VER_UE4_REFACTOR_MATERIAL_EXPRESSION_SCENECOLOR_AND_SCENEDEPTH_INPUTS)
	{
		// Connect deprecated UV input to new expression input
		InputMode = EMaterialSceneAttributeInputMode::Coordinates;
		Input = Coordinates_DEPRECATED;
	}
}

#if WITH_EDITOR
int32 UMaterialExpressionSceneDepth::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{    
	int32 OffsetIndex = INDEX_NONE;
	int32 CoordinateIndex = INDEX_NONE;
	bool bUseOffset = false;

	if(InputMode == EMaterialSceneAttributeInputMode::OffsetFraction)
	{
		if (Input.GetTracedInput().Expression)
		{
			OffsetIndex = Input.Compile(Compiler);
		} 
		else
		{
			OffsetIndex = Compiler->Constant2(ConstInput.X, ConstInput.Y);
		}
		bUseOffset = true;
	}
	else if(InputMode == EMaterialSceneAttributeInputMode::Coordinates)
	{
		if (Input.GetTracedInput().Expression)
		{
			CoordinateIndex = Input.Compile(Compiler);
		} 
	}

	int32 Result = Compiler->SceneDepth(OffsetIndex, CoordinateIndex, bUseOffset);
	return Result;
}

void UMaterialExpressionSceneDepth::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Scene Depth"));
}

FName UMaterialExpressionSceneDepth::GetInputName(int32 InputIndex) const
{
	if(InputIndex == 0)
	{
		// Display the current InputMode enum's display name.
		FByteProperty* InputModeProperty = FindFProperty<FByteProperty>( UMaterialExpressionSceneDepth::StaticClass(), "InputMode" );
		// Can't use GetNameByValue as GetNameStringByValue does name mangling that GetNameByValue does not
		return *InputModeProperty->Enum->GetNameStringByValue((int64)InputMode.GetValue());
	}
	return NAME_None;
}

#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSceneTexture
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionSceneTexture::UMaterialExpressionSceneTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Color"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("Size")));
	Outputs.Add(FExpressionOutput(TEXT("InvSize")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSceneTexture::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{    
	int32 ViewportUV = INDEX_NONE;

	if (Coordinates.GetTracedInput().Expression)
	{
		ViewportUV = Coordinates.Compile(Compiler);
	}

	if(OutputIndex == 0 || OutputIndex == 3)
	{
		// Color.  Note that clamping support is not necessary for regular SceneTexture, because it's only useful when sampling from lower resolution
		// maps with filtering, where bilinear blending of a higher resolution UV sample can end up interpolating with pixels outside the valid UV
		// range on a lower resolution map.  All SceneTextures are full resolution, while UserSceneTextures can be lower resolution (see
		// UMaterialExpressionUserSceneTexture::Compile below), so those support a user specified clamp flag.  The special OutputIndex of 3 (not
		// user facing) indicates an input pin to custom HLSL that isn't used in the code, meaning the scene texture input should be compiled in,
		// but the input pin's expression should be dead stripped to avoid an unnecessary texture fetch.
		return Compiler->SceneTextureLookup(ViewportUV, SceneTextureId, bFiltered, /*bClamped=*/ false, /*bUnused=*/ OutputIndex == 3);
	}
	else if(OutputIndex == 1 || OutputIndex == 2)
	{
		return Compiler->GetSceneTextureViewSize(SceneTextureId, /* InvProperty = */ OutputIndex == 2);
	}

	return Compiler->Errorf(TEXT("Invalid input parameter"));
}

void UMaterialExpressionSceneTexture::GetCaption(TArray<FString>& OutCaptions) const
{
	UEnum* Enum = StaticEnum<ESceneTextureId>();

	check(Enum);

	FString Name = Enum->GetDisplayNameTextByValue(SceneTextureId).ToString();

	OutCaptions.Add(FString("SceneTexture:") + Name);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionUserSceneTexture
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionUserSceneTexture::UMaterialExpressionUserSceneTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Color"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("Size")));
	Outputs.Add(FExpressionOutput(TEXT("InvSize")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionUserSceneTexture::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (UserSceneTexture.IsNone())
	{
		return Compiler->Errorf(TEXT("UserSceneTexture missing name -- value must be set to something other than None"));
	}

	int32 SceneTextureId = Compiler->FindOrAddUserSceneTexture(UserSceneTexture);
	if (SceneTextureId == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Too many unique UserSceneTexture inputs in the post process material -- max allowed is %d"), kPostProcessMaterialInputCountMax);
	}

	int32 ViewportUV = INDEX_NONE;

	if (Coordinates.GetTracedInput().Expression)
	{
		ViewportUV = Coordinates.Compile(Compiler);
	}

	if (OutputIndex == 0 || OutputIndex == 3)
	{
		// Color.    The special OutputIndex of 3 (not user facing) indicates an input pin to custom HLSL that isn't used in the code, meaning the
		// scene texture input should be compiled in, but the input pin's expression should be dead stripped to avoid an unnecessary texture fetch.
		return Compiler->SceneTextureLookup(ViewportUV, SceneTextureId, bFiltered, bClamped, /*bUnused=*/ OutputIndex == 3);
	}
	else if (OutputIndex == 1 || OutputIndex == 2)
	{
		return Compiler->GetSceneTextureViewSize(SceneTextureId, /* InvProperty = */ OutputIndex == 2);
	}

	return Compiler->Errorf(TEXT("Invalid input parameter"));
}

void UMaterialExpressionUserSceneTexture::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("UserSceneTexture:") + UserSceneTexture.ToString());
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSceneColor
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionSceneColor::UMaterialExpressionSceneColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
}

void UMaterialExpressionSceneColor::PostLoad()
{
	Super::PostLoad();

	if(GetLinkerUEVersion() < VER_UE4_REFACTOR_MATERIAL_EXPRESSION_SCENECOLOR_AND_SCENEDEPTH_INPUTS)
	{
		// Connect deprecated UV input to new expression input
		InputMode = EMaterialSceneAttributeInputMode::OffsetFraction;
		Input = OffsetFraction_DEPRECATED;
	}
}

#if WITH_EDITOR
int32 UMaterialExpressionSceneColor::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 OffsetIndex = INDEX_NONE;
	int32 CoordinateIndex = INDEX_NONE;
	bool bUseOffset = false;


	if(InputMode == EMaterialSceneAttributeInputMode::OffsetFraction)
	{
		if (Input.GetTracedInput().Expression)
		{
			OffsetIndex = Input.Compile(Compiler);
		}
		else
		{
			OffsetIndex = Compiler->Constant2(ConstInput.X, ConstInput.Y);
		}

		bUseOffset = true;
	}
	else if(InputMode == EMaterialSceneAttributeInputMode::Coordinates)
	{
		if (Input.GetTracedInput().Expression)
		{
			CoordinateIndex = Input.Compile(Compiler);
		} 
	}	

	int32 Result = Compiler->SceneColor(OffsetIndex, CoordinateIndex, bUseOffset);
	return Result;
}

void UMaterialExpressionSceneColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Scene Color"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDBufferTexture
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionDBufferTexture::UMaterialExpressionDBufferTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDBufferTexture::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ViewportUV = INDEX_NONE;

	if (Coordinates.GetTracedInput().Expression)
	{
		ViewportUV = Coordinates.Compile(Compiler);
	}

	return Compiler->DBufferTextureLookup(ViewportUV, DBufferTextureId);
}

void UMaterialExpressionDBufferTexture::GetCaption(TArray<FString>& OutCaptions) const
{
	UEnum* Enum = StaticEnum<EDBufferTextureId>();
	check(Enum);

	FString Name = Enum->GetDisplayNameTextByValue(DBufferTextureId).ToString();
	OutCaptions.Add(Name);
}

int32 UMaterialExpressionPower::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Base.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Power Base input"));
	}

	int32 Arg1 = Base.Compile(Compiler);
	int32 Arg2 = Exponent.GetTracedInput().Expression ? Exponent.Compile(Compiler) : Compiler->Constant(ConstExponent);
	return Compiler->Power(
		Arg1,
		Arg2
		);
}

void UMaterialExpressionPower::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Power"));
}

void UMaterialExpressionPower::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns the Base value raised to the power of Exponent. Base value must be positive, values less than 0 will be clamped."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionLength::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Length input"));
	}

	int32 Index = Input.Compile(Compiler);
	if(Compiler->GetType(Index) == MCT_Float)
	{
		// optimized
		return Compiler->Abs(Index);
	}

	return Compiler->Length(Index);
}

void UMaterialExpressionLength::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Length"));
}

void UMaterialExpressionLength::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns the length of input."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionHsvToRgb::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing HSVToRGB input"));
	}

	int32 InputIndex = Input.Compile(Compiler);
	EMaterialValueType InputType = Compiler->GetType(InputIndex);
	if(InputType != EMaterialValueType::MCT_Float3 && InputType != EMaterialValueType::MCT_Float4)
	{
		return InputIndex;
	}

	int32 Result = Compiler->HsvToRgb(InputIndex);

	return Result;
}

void UMaterialExpressionHsvToRgb::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("HSVToRGB"));
}

void UMaterialExpressionHsvToRgb::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Convert an incoming color from HSV to RGB space."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionRgbToHsv::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RGBToHSV input"));
	}

	int32 InputIndex = Input.Compile(Compiler);
	EMaterialValueType InputType = Compiler->GetType(InputIndex);
	if(InputType != EMaterialValueType::MCT_Float3 && InputType != EMaterialValueType::MCT_Float4)
	{
		return InputIndex;
	}

	int32 Result = Compiler->RgbToHsv(InputIndex);

	return Result;
}

void UMaterialExpressionRgbToHsv::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("RGBToHSV"));
}

void UMaterialExpressionRgbToHsv::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Convert an incoming color from RGB to HSV space."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionExponential::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Exp input"));
	}

	return Compiler->Exponential(Input.Compile(Compiler));
}

void UMaterialExpressionExponential::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Exp"));
}

void UMaterialExpressionExponential::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns the base-e exponential, or e^x, of the input."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionExponential2::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Exp2 input"));
	}

	return Compiler->Exponential2(Input.Compile(Compiler));
}

void UMaterialExpressionExponential2::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Exp2"));
}

void UMaterialExpressionExponential2::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns the base 2 exponential, or 2^x, of the input."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionLogarithm::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Log input"));
	}

	return Compiler->Logarithm(Input.Compile(Compiler));
}

void UMaterialExpressionLogarithm::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Log"));
}

void UMaterialExpressionLogarithm::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns the base-e logarithm, or natural logarithm, of the input. Input should be greater than 0."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionLogarithm2::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!X.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Log2 X input"));
	}

	return Compiler->Logarithm2(X.Compile(Compiler));
}

void UMaterialExpressionLogarithm2::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Log2"));
}

void UMaterialExpressionLogarithm2::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns the base-2 logarithm of the input. Input should be greater than 0."), 40, OutToolTip);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionLogarithm10::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!X.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Log10 X input"));
	}

	return Compiler->Logarithm10(X.Compile(Compiler));
}

void UMaterialExpressionLogarithm10::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Log10"));
}

void UMaterialExpressionLogarithm10::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns the base-10 logarithm of the input. Input should be greater than 0."), 40, OutToolTip);
}

FName UMaterialExpressionSceneColor::GetInputName(int32 InputIndex) const
{
	if(InputIndex == 0)
	{
		// Display the current InputMode enum's display name.
		FByteProperty* InputModeProperty = FindFProperty<FByteProperty>( UMaterialExpressionSceneColor::StaticClass(), "InputMode" );
		// Can't use GetNameByValue as GetNameStringByValue does name mangling that GetNameByValue does not
		return *InputModeProperty->Enum->GetNameStringByValue((int64)InputMode.GetValue());
	}
	return NAME_None;
}

int32 UMaterialExpressionIf::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing If A input"));
	}
	if(!AGreaterThanB.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing If AGreaterThanB input"));
	}
	if(!ALessThanB.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing If ALessThanB input"));
	}

	int32 CompiledA = A.Compile(Compiler);
	int32 CompiledB = B.GetTracedInput().Expression ? B.Compile(Compiler) : Compiler->Constant(ConstB);

	if(!IsPrimitiveType(Compiler->GetType(CompiledA)))
	{
		return Compiler->Errorf(TEXT("If input A must be a primitive type."));
	}

	if(!IsPrimitiveType(Compiler->GetType(CompiledB)))
	{
		return Compiler->Errorf(TEXT("If input B must be a primitive type."));
	}

	int32 Arg3 = AGreaterThanB.Compile(Compiler);
	int32 Arg4 = AEqualsB.GetTracedInput().Expression ? AEqualsB.Compile(Compiler) : INDEX_NONE;
	int32 Arg5 = ALessThanB.Compile(Compiler);
	int32 ThresholdArg = Compiler->Constant(EqualsThreshold);

	if (Arg3 == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Failed to compile AGreaterThanB input."));
	}

	if (Arg5 == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Failed to compile ALessThanB input."));
	}

	return Compiler->If(CompiledA,CompiledB,Arg3,Arg4,Arg5,ThresholdArg);
}

void UMaterialExpressionIf::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("If"));
}

EMaterialValueType UMaterialExpressionIf::GetInputValueType(int32 InputIndex)
{
	// First two inputs are always float
	if (InputIndex == 0 || InputIndex == 1)
	{
		return static_cast<EMaterialValueType>(MCT_MaterialAttributes | MCT_Numeric | MCT_ShadingModel | MCT_StaticBool | MCT_Bool);
	}

	return MCT_Unknown;
}

bool UMaterialExpressionIf::IsResultMaterialAttributes(int32 OutputIndex)
{
	if ((AGreaterThanB.GetTracedInput().Expression && AGreaterThanB.Expression->IsResultMaterialAttributes(AGreaterThanB.OutputIndex))
		&& (!AEqualsB.GetTracedInput().Expression || AEqualsB.Expression->IsResultMaterialAttributes(AEqualsB.OutputIndex))
		&& (ALessThanB.GetTracedInput().Expression && ALessThanB.Expression->IsResultMaterialAttributes(ALessThanB.OutputIndex))
		)
	{
		return true;
	}
	else
	{
		return false;
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionOneMinus::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing 1-x input"));
	}
	return Compiler->Sub(Compiler->Constant(1.0f),Input.Compile(Compiler));
}

void UMaterialExpressionOneMinus::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("1-x"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionAbs::Compile( FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 Result=INDEX_NONE;

	if( !Input.GetTracedInput().Expression )
	{
		// an input expression must exist
		Result = Compiler->Errorf( TEXT("Missing Abs input") );
	}
	else
	{
		// evaluate the input expression first and use that as
		// the parameter for the Abs expression
		Result = Compiler->Abs( Input.Compile(Compiler) );
	}

	return Result;
}

void UMaterialExpressionAbs::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Abs"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionTransform
///////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
int32 UMaterialExpressionTransform::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	using namespace UE::MaterialTranslatorUtils;

	int32 Result = INDEX_NONE;

	if (!Input.GetTracedInput().Expression)
	{
		Result = Compiler->Errorf(TEXT("Missing Transform input vector"));
	}
	else
	{
		int32 VecInputIdx = Input.Compile(Compiler);
		const auto TransformSourceBasis = GetMaterialCommonBasis(TransformSourceType);
		const auto TransformDestBasis = GetMaterialCommonBasis(TransformType);
		Result = Compiler->TransformVector(TransformSourceBasis, TransformDestBasis, VecInputIdx);
	}

	return Result;
}

void UMaterialExpressionTransform::GetCaption(TArray<FString>& OutCaptions) const
{
#if WITH_EDITOR
	const UEnum* MVCTSEnum = StaticEnum<EMaterialVectorCoordTransformSource>();
	const UEnum* MVCTEnum = StaticEnum<EMaterialVectorCoordTransform>();
	check(MVCTSEnum);
	check(MVCTEnum);
	
	FString TransformDesc;
	TransformDesc += MVCTSEnum->GetDisplayNameTextByValue(TransformSourceType).ToString();
	TransformDesc += TEXT(" to ");
	TransformDesc += MVCTEnum->GetDisplayNameTextByValue(TransformType).ToString();
	OutCaptions.Add(TransformDesc);
#else
	OutCaptions.Add(TEXT(""));
#endif

	OutCaptions.Add(TEXT("TransformVector"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionTransformPosition
///////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
int32 UMaterialExpressionTransformPosition::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	using namespace UE::MaterialTranslatorUtils;

	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing Transform Position input vector"));
	}

	int32 VecInputIdx = Input.Compile(Compiler);

	FTransformParameters Parameters {};

	if (TransformSourceType == TRANSFORMPOSSOURCE_PeriodicWorld || TransformType == TRANSFORMPOSSOURCE_PeriodicWorld)
	{
		Parameters.PeriodicWorldTileSizeIndex = PeriodicWorldTileSize.IsConnected() ? PeriodicWorldTileSize.Compile(Compiler) : Compiler->Constant(ConstPeriodicWorldTileSize);
	}
	if (TransformSourceType == TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld || TransformType == TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld)
	{
		Parameters.FirstPersonInterpolationAlphaIndex = FirstPersonInterpolationAlpha.IsConnected() ? FirstPersonInterpolationAlpha.Compile(Compiler) : Compiler->Constant(ConstFirstPersonInterpolationAlpha);
	}
	
	const auto TransformSourceBasis = GetMaterialCommonBasis(TransformSourceType);
	const auto TransformDestBasis = GetMaterialCommonBasis(TransformType);

	return Compiler->TransformPosition(TransformSourceBasis, TransformDestBasis, Parameters, VecInputIdx);
}

void UMaterialExpressionTransformPosition::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	Super::GetConnectorToolTip(GetAbsoluteInputIndex(InputIndex), OutputIndex, OutToolTip);
}

void UMaterialExpressionTransformPosition::GetCaption(TArray<FString>& OutCaptions) const
{
#if WITH_EDITOR
	const UEnum* MPTSEnum = StaticEnum<EMaterialPositionTransformSource>();
	check(MPTSEnum);
	
	FString TransformDesc;
	TransformDesc += MPTSEnum->GetDisplayNameTextByValue(TransformSourceType).ToString();
	TransformDesc += TEXT(" to ");
	TransformDesc += MPTSEnum->GetDisplayNameTextByValue(TransformType).ToString();
	OutCaptions.Add(TransformDesc);
#else
	OutCaptions.Add(TEXT(""));
#endif
	
	OutCaptions.Add(TEXT("TransformPosition"));
}

TArrayView<FExpressionInput*> UMaterialExpressionTransformPosition::GetInputsView()
{
	CachedInputs.Empty();
	uint32 InputIndex = 0;
	while (FExpressionInput* Ptr = GetInput(InputIndex++))
	{
		CachedInputs.Add(Ptr);
	}
	return CachedInputs;
}

// this define is only used for the following function
#define IF_INPUT_RETURN(Item) if(!InputIndex) return &Item; --InputIndex
FExpressionInput* UMaterialExpressionTransformPosition::GetInput(int32 InputIndex)
{
	IF_INPUT_RETURN(Input);

	if (bUsesPeriodicWorldPosition)
	{
		IF_INPUT_RETURN(PeriodicWorldTileSize);
	}
	if (bUsesFirstPersonInterpolationAlpha)
	{
		IF_INPUT_RETURN(FirstPersonInterpolationAlpha);
	}

	return nullptr;
}
#undef IF_INPUT_RETURN

FName UMaterialExpressionTransformPosition::GetInputName(int32 InputIndex) const
{
	const FExpressionInput* FoundInput = static_cast<const UMaterialExpression*>(this)->GetInput(InputIndex);

	if (FoundInput == &PeriodicWorldTileSize)
	{
		return TEXT("Periodic World Tile Size");
	}
	else if (FoundInput == &FirstPersonInterpolationAlpha)
	{
		return TEXT("First Person Interpolation Alpha");
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionTransformPosition::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, TransformSourceType) ||
		PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, TransformType))
	{
		bUsesPeriodicWorldPosition = TransformSourceType == TRANSFORMPOSSOURCE_PeriodicWorld || TransformType == TRANSFORMPOSSOURCE_PeriodicWorld;
		bUsesFirstPersonInterpolationAlpha = TransformSourceType == TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld || TransformType == TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld;
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	// Need to update expression properties before super call (which triggers recompile)
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#define IF_INPUT_RETURN(Value) if(!InputIndex) return (Value); --InputIndex
int32 UMaterialExpressionTransformPosition::GetAbsoluteInputIndex(int32 InputIndex)
{
	IF_INPUT_RETURN(0);
	if (bUsesPeriodicWorldPosition)
	{
		IF_INPUT_RETURN(1);
	}
	if (bUsesFirstPersonInterpolationAlpha)
	{
		IF_INPUT_RETURN(2);
	}
	return INDEX_NONE;
}
#undef IF_INPUT_RETURN

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionComment
///////////////////////////////////////////////////////////////////////////////

void UMaterialExpressionComment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Text))
	{
		if (GraphNode)
		{
			GraphNode->Modify();
			GraphNode->NodeComment = Text;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, CommentColor))
	{
		if (GraphNode)
		{
			GraphNode->Modify();
			CastChecked<UMaterialGraphNode_Comment>(GraphNode)->CommentColor = CommentColor;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, FontSize))
	{
		if (GraphNode)
		{
			GraphNode->Modify();
			CastChecked<UMaterialGraphNode_Comment>(GraphNode)->FontSize = FontSize;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bColorCommentBubble))
	{
		if (GraphNode)
		{
			GraphNode->Modify();
			CastChecked<UMaterialGraphNode_Comment>(GraphNode)->bColorCommentBubble = bColorCommentBubble;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bCommentBubbleVisible_InDetailsPanel))
	{
		if (GraphNode)
		{
			GraphNode->Modify();
			UMaterialGraphNode_Comment* CommentNode = CastChecked<UMaterialGraphNode_Comment>(GraphNode);
			CommentNode->bCommentBubbleVisible_InDetailsPanel = bCommentBubbleVisible_InDetailsPanel;
			CommentNode->bCommentBubbleVisible = bCommentBubbleVisible_InDetailsPanel;
			CommentNode->bCommentBubblePinned = bCommentBubbleVisible_InDetailsPanel;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, bGroupMode))
	{
		if (GraphNode)
		{
			GraphNode->Modify();
			CastChecked<UMaterialGraphNode_Comment>(GraphNode)->MoveMode = bGroupMode ? ECommentBoxMode::GroupMovement : ECommentBoxMode::NoGroupMovement;
		}
	}

	// Don't need to update preview after changing comments
	bNeedToUpdatePreview = false;
}

bool UMaterialExpressionComment::Modify( bool bAlwaysMarkDirty/*=true*/ )
{
	bool bResult = Super::Modify(bAlwaysMarkDirty);

	// Don't need to update preview after changing comments
	bNeedToUpdatePreview = false;

	return bResult;
}

void UMaterialExpressionComment::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Comment"));
}

bool UMaterialExpressionComment::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if( Text.Contains(SearchQuery) )
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionComposite
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionComposite::UMaterialExpressionComposite(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SubgraphName = "Collapsed Nodes";

#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	Outputs.Reset();
#endif
}

#if WITH_EDITOR
TArray<UMaterialExpressionReroute*> UMaterialExpressionComposite::GetCurrentReroutes() const
{
	TArray<UMaterialExpressionReroute*> RerouteExpressions;
	if (InputExpressions)
	{
		for (const FCompositeReroute& InputReroute : InputExpressions->ReroutePins)
		{
			RerouteExpressions.Add(InputReroute.Expression);
		}
	}
	if (OutputExpressions)
	{
		for (const FCompositeReroute& OutputReroute : OutputExpressions->ReroutePins)
		{
			RerouteExpressions.Add(OutputReroute.Expression);
		}
	}
	return RerouteExpressions;
}

void UMaterialExpressionComposite::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(GraphNode))
	{
		if (CompositeNode->BoundGraph && CompositeNode->BoundGraph->GetName() != SubgraphName)
		{
			CompositeNode->BoundGraph->Rename(*SubgraphName);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UMaterialExpressionComposite::GetEditableName() const
{
	return SubgraphName;
}

void UMaterialExpressionComposite::SetEditableName(const FString& NewName)
{
	SubgraphName = NewName;

	if (UMaterialGraphNode_Composite* CompositeNode = Cast<UMaterialGraphNode_Composite>(GraphNode))
	{
		if (CompositeNode->BoundGraph && CompositeNode->BoundGraph->GetName() != SubgraphName)
		{
			CompositeNode->BoundGraph->Rename(*SubgraphName);
		}
	}
}

TArray<FExpressionOutput>& UMaterialExpressionComposite::GetOutputs()
{	
	Outputs.Reset();

	// OutputExpressions may be null if we are using the default object
	if (OutputExpressions)
	{
		for (const FCompositeReroute& ReroutePin : OutputExpressions->ReroutePins)
		{
			if (ReroutePin.Expression)
			{
				ReroutePin.Expression->GetOutputs()[0].OutputName = ReroutePin.Name;
				Outputs.Add(ReroutePin.Expression->GetOutputs()[0]);
			}
		}
	}
	return Outputs;
}

TArrayView<FExpressionInput*> UMaterialExpressionComposite::GetInputsView()
{
	// InputExpressions may be null if we are using the default object
	CachedInputs.Empty();
	if (InputExpressions)
	{
		CachedInputs.Reserve(InputExpressions->ReroutePins.Num());
		for (const FCompositeReroute& ReroutePin : InputExpressions->ReroutePins)
		{
			if (ReroutePin.Expression)
			{
				CachedInputs.Add(ReroutePin.Expression->GetInput(0));
			}
		}
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionComposite::GetInput(int32 InputIndex)
{
	if (InputExpressions && InputExpressions->ReroutePins.IsValidIndex(InputIndex))
	{
		if (InputExpressions->ReroutePins[InputIndex].Expression)
		{
			return InputExpressions->ReroutePins[InputIndex].Expression->GetInput(0);
		}
	}

	return nullptr;
}

FName UMaterialExpressionComposite::GetInputName(int32 InputIndex) const
{
	if (InputExpressions && InputExpressions->ReroutePins.IsValidIndex(InputIndex))
	{
		return InputExpressions->ReroutePins[InputIndex].Name;
	}

	return FName();
}

EMaterialValueType UMaterialExpressionComposite::GetInputValueType(int32 InputIndex)
{
	if (InputExpressions && InputExpressions->ReroutePins.IsValidIndex(InputIndex))
	{
		return InputExpressions->ReroutePins[InputIndex].Expression->GetInputValueType(0);
	}

	check(false);
	return MCT_Float;
}

EMaterialValueType UMaterialExpressionComposite::GetOutputValueType(int32 OutputIndex)
{
	if (OutputExpressions && OutputExpressions->ReroutePins.IsValidIndex(OutputIndex))
	{
		return OutputExpressions->ReroutePins[OutputIndex].Expression->GetOutputValueType(0);
	}

	check(false);
	return MCT_Float;
}

bool UMaterialExpressionComposite::IsExpressionConnected(FExpressionInput* Input, int32 OutputIndex)
{
	if (Input && OutputExpressions && OutputExpressions->ReroutePins.IsValidIndex(OutputIndex))
	{
		return OutputExpressions->ReroutePins[OutputIndex].Expression == Input->Expression;
	}

	return false;
}

void UMaterialExpressionComposite::ConnectExpression(FExpressionInput* Input, int32 OutputIndex)
{
	OutputExpressions->ReroutePins[OutputIndex].Expression->ConnectExpression(Input, 0);
}

void UMaterialExpressionComposite::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(SubgraphName);
}

bool UMaterialExpressionComposite::Modify(bool bAlwaysMarkDirty)
{
	// Modify pin bases so they can update the compilation graph
	if (InputExpressions)
	{
		InputExpressions->ModifyAsSubObject(bAlwaysMarkDirty);
	}

	if (OutputExpressions)
	{
		OutputExpressions->ModifyAsSubObject(bAlwaysMarkDirty);
	}

	return ModifyCompositeOnly(bAlwaysMarkDirty);
}

bool UMaterialExpressionComposite::ModifyCompositeOnly(bool bAlwaysMarkDirty)
{
	return Super::Modify(bAlwaysMarkDirty);
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPinBase
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPinBase::UMaterialExpressionPinBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
#endif
}

#if WITH_EDITOR

void UMaterialExpressionPinBase::DeleteReroutePins()
{
	Modify();
	for (FCompositeReroute& Reroute : ReroutePins)
	{
		if (Reroute.Expression)
		{
			Reroute.Expression->Modify();
			Material->GetExpressionCollection().RemoveExpression(Reroute.Expression);
			Reroute.Expression->MarkAsGarbage();
		}
		else
		{
			Material->GetExpressionCollection().RemoveExpression(Reroute.Expression);
		}
	}
	ReroutePins.Empty();
}

void UMaterialExpressionPinBase::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
	
	if (SubgraphExpression && SubgraphExpression->GraphNode)
	{
		SubgraphExpression->Modify();
	}

	PreEditRereouteExpresions.Empty();
	for (const FCompositeReroute& Reroute : ReroutePins)
	{
		PreEditRereouteExpresions.Add(Reroute.Expression);
	}
}

void UMaterialExpressionPinBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty && GraphNode && SubgraphExpression && SubgraphExpression->GraphNode)
	{
		Modify();
		Material->Modify();
		Material->GetEditorOnlyData()->Modify();

		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd || PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
		{
			uint32 AddedRerouteIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
			FCompositeReroute& AddedReroute = ReroutePins[AddedRerouteIndex];

			AddedReroute.Expression = NewObject<UMaterialExpressionReroute>(GetOuter(), UMaterialExpressionReroute::StaticClass(), NAME_None, RF_Transactional);
			AddedReroute.Expression->SubgraphExpression = SubgraphExpression;
			AddedReroute.Expression->Material = Material;
			AddedReroute.Name = AddedReroute.Name.IsNone() ? FName(FString::Printf(TEXT("Pin %u"), AddedRerouteIndex + 1)) : AddedReroute.Name;

			Material->GetExpressionCollection().AddExpression(AddedReroute.Expression);
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
		{
			uint32 RemovedRerouteIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());
			UMaterialExpression* RemovedReroute = PreEditRereouteExpresions[RemovedRerouteIndex];

			if (RemovedReroute)
			{
				RemovedReroute->Modify();
				Material->GetExpressionCollection().RemoveExpression(RemovedReroute);
				RemovedReroute->MarkAsGarbage();
			}
			else
			{
				Material->GetExpressionCollection().RemoveExpression(RemovedReroute);
			}
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
		{
			for (UMaterialExpressionReroute* RemovedReroute : PreEditRereouteExpresions)
			{
				if (RemovedReroute)
				{
					RemovedReroute->Modify();
					Material->GetExpressionCollection().RemoveExpression(RemovedReroute);
					RemovedReroute->MarkAsGarbage();
				}
				else
				{
					Material->GetExpressionCollection().RemoveExpression(RemovedReroute);
				}
			}
		}

		GraphNode->Modify();
		GraphNode->BreakAllNodeLinks();
		GraphNode->ReconstructNode();

		SubgraphExpression->GraphNode->Modify();
		SubgraphExpression->GraphNode->BreakAllNodeLinks();
		SubgraphExpression->GraphNode->ReconstructNode();

		Material->MaterialGraph->Modify();
		Material->MaterialGraph->LinkGraphNodesFromMaterial();
		Material->MaterialGraph->LinkMaterialExpressionsFromGraph();
	}

	PreEditRereouteExpresions.Empty();
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (Material && Material->GetExpressionCollection().Expressions.Remove(nullptr))
	{
		UE_LOGF(LogMaterial, Warning, "Null expressions detected in material expressions list after execution of UMaterialExpressionPinBase::PostEditChangeProperty.");
	}
}

TArray<FExpressionOutput>& UMaterialExpressionPinBase::GetOutputs()
{
	// Re-compute output expressions, since we can and do change via code.
	if (PinDirection == EGPD_Output)
	{
		Outputs.Reset();
		for (const FCompositeReroute& ReroutePin : ReroutePins)
		{
			if (ReroutePin.Expression)
			{
				ReroutePin.Expression->GetOutputs()[0].OutputName = ReroutePin.Name;
				Outputs.Add(ReroutePin.Expression->GetOutputs()[0]);
			}
		}
	}
	return Outputs;
}

TArrayView<FExpressionInput*> UMaterialExpressionPinBase::GetInputsView()
{
	CachedInputs.Empty();
	if (PinDirection == EGPD_Input)
	{
		CachedInputs.Reserve(ReroutePins.Num());
		for (const FCompositeReroute& ReroutePin : ReroutePins)
		{
			if (ReroutePin.Expression)
			{
				CachedInputs.Add(ReroutePin.Expression->GetInput(0));
			}
		}
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionPinBase::GetInput(int32 InputIndex)
{
	if (PinDirection == EGPD_Input)
	{
		if (InputIndex >= 0 && InputIndex < ReroutePins.Num())
		{
			if (ReroutePins[InputIndex].Expression)
			{
				return ReroutePins[InputIndex].Expression->GetInput(0);
			}
		}
	}

	return nullptr;
}

FName UMaterialExpressionPinBase::GetInputName(int32 InputIndex) const
{
	if (PinDirection == EGPD_Input)
	{
		if (InputIndex >= 0 && InputIndex < ReroutePins.Num())
		{
			return ReroutePins[InputIndex].Name;
		}
	}

	return FName();
}

EMaterialValueType UMaterialExpressionPinBase::GetInputValueType(int32 InputIndex)
{
	if (InputIndex >= 0 && InputIndex < ReroutePins.Num())
	{
		return PinDirection == EGPD_Input ? ReroutePins[InputIndex].Expression->GetInputValueType(0) : MCT_Float;
	}

	check(false);
	return MCT_Float;
}

EMaterialValueType UMaterialExpressionPinBase::GetOutputValueType(int32 OutputIndex)
{
	if (OutputIndex >= 0 && OutputIndex < ReroutePins.Num())
	{
		return PinDirection == EGPD_Output ? ReroutePins[OutputIndex].Expression->GetOutputValueType(0) : MCT_Float;
	}

	check(false);
	return MCT_Float;
}

bool UMaterialExpressionPinBase::IsExpressionConnected(FExpressionInput* Input, int32 OutputIndex)
{
	if (PinDirection == EGPD_Output && Input && OutputIndex >= 0 && OutputIndex < ReroutePins.Num())
	{
		return ReroutePins[OutputIndex].Expression == Input->Expression;
	}

	return false;
}

void UMaterialExpressionPinBase::ConnectExpression(FExpressionInput* Input, int32 OutputIndex)
{
	if (PinDirection == EGPD_Output && Input && OutputIndex >= 0 && OutputIndex < ReroutePins.Num())
	{
		ReroutePins[OutputIndex].Expression->ConnectExpression(Input, 0);
	}
}

void UMaterialExpressionPinBase::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(PinDirection == EGPD_Output ? "Input" : "Output");
}

bool UMaterialExpressionPinBase::Modify(bool bAlwaysMarkDirty)
{
	if (UMaterialExpressionComposite* Composite = Cast<UMaterialExpressionComposite>(SubgraphExpression))
	{
		Composite->ModifyCompositeOnly(bAlwaysMarkDirty);
	}
	return ModifyAsSubObject(bAlwaysMarkDirty);
}

bool UMaterialExpressionPinBase::ModifyAsSubObject(bool bAlwaysMarkDirty)
{
	// Modify reroute pins so they can update the compilation graph
	for (const FCompositeReroute& ReroutePin : ReroutePins)
	{
		// Reroute pin can not have an expression if just adding new pin.
		if (ReroutePin.Expression)
		{
			ReroutePin.Expression->Modify(bAlwaysMarkDirty);
		}
	}

	return Super::Modify(bAlwaysMarkDirty);
}

int32 UMaterialExpressionFresnel::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// pow(1 - max(0,Normal dot Camera),Exponent) * (1 - BaseReflectFraction) + BaseReflectFraction
	//
	int32 NormalArg = Normal.GetTracedInput().Expression ? Normal.Compile(Compiler) : Compiler->PixelNormalWS();
	int32 DotArg = Compiler->Dot(NormalArg,Compiler->CameraVector());
	int32 MaxArg = Compiler->Max(Compiler->Constant(0.f),DotArg);
	int32 MinusArg = Compiler->Sub(Compiler->Constant(1.f),MaxArg);
	int32 ExponentArg = ExponentIn.GetTracedInput().Expression ? ExponentIn.Compile(Compiler) : Compiler->Constant(Exponent);
	// Compiler->Power got changed to call PositiveClampedPow instead of ClampedPow
	// Manually implement ClampedPow to maintain backwards compatibility in the case where the input normal is not normalized (length > 1)
	int32 AbsBaseArg = Compiler->Max(Compiler->Abs(MinusArg), Compiler->Constant(UE_KINDA_SMALL_NUMBER));
	int32 PowArg = Compiler->Power(AbsBaseArg,ExponentArg);
	int32 BaseReflectFractionArg = BaseReflectFractionIn.GetTracedInput().Expression ? BaseReflectFractionIn.Compile(Compiler) : Compiler->Constant(BaseReflectFraction);
	int32 ScaleArg = Compiler->Mul(PowArg, Compiler->Sub(Compiler->Constant(1.f), BaseReflectFractionArg));
	
	return Compiler->Add(ScaleArg, BaseReflectFractionArg);
}
#endif // WITH_EDITOR

/*-----------------------------------------------------------------------------
UMaterialExpressionFontSample
-----------------------------------------------------------------------------*/
UMaterialExpressionFontSample::UMaterialExpressionFontSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 0, 1));

	bCollapsed = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionFontSample::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 Result = -1;
#if PLATFORM_EXCEPTIONS_DISABLED
	// if we can't throw the error below, attempt to thwart the error by using the default font
	if( !Font )
	{
		UE_LOGF(LogMaterial, Log, "Using default font instead of real font!");
		Font = GEngine->GetMediumFont();
		FontTexturePage = 0;
	}
	else if( !Font->Textures.IsValidIndex(FontTexturePage) )
	{
		UE_LOGF(LogMaterial, Log, "Invalid font page %d. Max allowed is %d",FontTexturePage,Font->Textures.Num());
		FontTexturePage = 0;
	}
#endif
	if( !Font )
	{
		Result = CompilerError(Compiler, TEXT("Missing input Font"));
	}
	else if( Font->FontCacheType == EFontCacheType::Runtime )
	{
		Result = CompilerError(Compiler, *FString::Printf(TEXT("Font '%s' is runtime cached, but only offline cached fonts can be sampled"), *Font->GetName()));
	}
	else if( !Font->Textures.IsValidIndex(FontTexturePage) )
	{
		Result = CompilerError(Compiler, *FString::Printf(TEXT("Invalid font page %d. Max allowed is %d"), FontTexturePage, Font->Textures.Num()));
	}
	else
	{
		auto [bSuccess, Texture, ExpectedSamplerType, ErrorOutput] = ValidateAndGetTextureSampler(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform());
		if (!bSuccess)
		{
			return Compiler->Error(*ErrorOutput);
		}

		int32 TextureCodeIndex = Compiler->Texture(Texture, ExpectedSamplerType);
		Result = Compiler->TextureSample(
			TextureCodeIndex,
			Compiler->TextureCoordinate(0, false, false),
			ExpectedSamplerType
		);
	}
	return Result;
}

TTuple<bool, UTexture*, EMaterialSamplerType, FString> UMaterialExpressionFontSample::ValidateAndGetTextureSampler(EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform)
{
	UTexture* Texture = Font->Textures[FontTexturePage];
	if (!Texture)
	{
		UE_LOGF(LogMaterial, Log, "Invalid font texture. Using default texture");
		Texture = GEngine->DefaultTexture;
	}
	check(Texture);

	EMaterialSamplerType ExpectedSamplerType;
	if (Texture->CompressionSettings == TC_DistanceFieldFont)
	{
		ExpectedSamplerType = SAMPLERTYPE_DistanceFieldFont;
	}
	else
	{
		ExpectedSamplerType = Texture->SRGB ? SAMPLERTYPE_Color : SAMPLERTYPE_LinearColor;
	}

	FString SamplerTypeError;
	if (!MaterialExpressionUtils::VerifySamplerType(ShaderPlatform, TargetPlatform, Texture, ExpectedSamplerType, SamplerTypeError))
	{
		return { false, nullptr, SAMPLERTYPE_MAX, SamplerTypeError };
	}

	return { true, Texture, ExpectedSamplerType, FString() };
}

int32 UMaterialExpressionFontSample::GetWidth() const
{
	return ME_STD_THUMBNAIL_SZ+(ME_STD_BORDER*2);
}

void UMaterialExpressionFontSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Font Sample"));
}

bool UMaterialExpressionFontSample::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if( Font != nullptr && Font->GetName().Contains(SearchQuery) )
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}
#endif // WITH_EDITOR

UObject* UMaterialExpressionFontSample::GetReferencedTexture() const
{
	if (Font && Font->Textures.IsValidIndex(FontTexturePage))
	{
		UTexture* Texture = Font->Textures[FontTexturePage];
		return Texture;
	}

	return nullptr;
}

/*-----------------------------------------------------------------------------
UMaterialExpressionFontSampleParameter
-----------------------------------------------------------------------------*/
UMaterialExpressionFontSampleParameter::UMaterialExpressionFontSampleParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsParameterExpression = true;
}

#if WITH_EDITOR
int32 UMaterialExpressionFontSampleParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 Result = -1;
	if( !ParameterName.IsValid() || 
		ParameterName.IsNone() || 
		!Font ||
		!Font->Textures.IsValidIndex(FontTexturePage) )
	{
		Result = UMaterialExpressionFontSample::Compile(Compiler, OutputIndex);
	}
	else 
	{
		auto [bSuccess, Texture, ExpectedSamplerType, ErrorOutput] = ValidateAndGetTextureSampler(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform());
		if (!bSuccess)
		{
			return Compiler->Error(*ErrorOutput);
		}

		int32 TextureCodeIndex = Compiler->TextureParameter(ParameterName, Texture, ExpectedSamplerType);
		Result = Compiler->TextureSample(
			TextureCodeIndex,
			Compiler->TextureCoordinate(0, false, false),
			ExpectedSamplerType
		);
	}
	return Result;
}

void UMaterialExpressionFontSampleParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Font Param")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

void UMaterialExpressionFontSampleParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	ValidateParameterNameInternal(this, Material, bAllowDuplicateName);
}

bool UMaterialExpressionFontSampleParameter::SetParameterValue(FName InParameterName, UFont* InFontValue, int32 InFontPage, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		const bool bFontValueChanged = (Font != InFontValue);
		const bool bFontPageChanged = (FontTexturePage != InFontPage);
		if (bFontValueChanged || bFontPageChanged)
		{
			Font = InFontValue;
			FontTexturePage = InFontPage;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				if (bFontValueChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Font));
				}
				if (bFontPageChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, FontTexturePage));
				}
			}
		}
		return true;
	}

	return false;
}
#endif

void UMaterialExpressionFontSampleParameter::SetDefaultFont()
{
	GEngine->GetMediumFont();
}

#if WITH_EDITOR

bool UMaterialExpressionFontSampleParameter::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if( ParameterName.ToString().Contains(SearchQuery) )
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

FString UMaterialExpressionFontSampleParameter::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionFontSampleParameter::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}
#endif

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLocalPosition
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionLocalPosition::UMaterialExpressionLocalPosition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("XYZ"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("XY"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Z"), 1, 0, 0, 1, 0));

	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionLocalPosition::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->LocalPosition(IncludedOffsets, LocalOrigin);
}

void UMaterialExpressionLocalPosition::GetCaption(TArray<FString>& OutCaptions) const
{
	if (LocalOrigin == ELocalPositionOrigin::InstancePreSkinning)
	{
		OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "LocalPositionPreSkinnedText", "Pre-Skinned Local Position").ToString());
	}
	else if (IncludedOffsets == EPositionIncludedOffsets::IncludeOffsets && LocalOrigin == ELocalPositionOrigin::Instance)
	{
		OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "LocalPositionInstanceIncludingOffsetsText", "Local Position").ToString());
	}
	else if (IncludedOffsets == EPositionIncludedOffsets::ExcludeOffsets && LocalOrigin == ELocalPositionOrigin::Instance)
	{
		OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "LocalPositionInstanceExcludingOffsetsText", "Local Position (Excluding Material Offsets)").ToString());
	}
	else if (IncludedOffsets == EPositionIncludedOffsets::IncludeOffsets && LocalOrigin == ELocalPositionOrigin::Primitive)
	{
		OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "LocalPositionComponentIncludingOffsetsText", "Component Local Position").ToString());
	}
	else if (IncludedOffsets == EPositionIncludedOffsets::ExcludeOffsets && LocalOrigin == ELocalPositionOrigin::Primitive)
	{
		OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "LocalPositionComponentExcludingOffsetsText", "Component Local Position (Excluding Material Offsets)").ToString());
	}
	else
	{
		checkNoEntry();
	}
}

void UMaterialExpressionLocalPosition::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Gets the local position of the mesh, based on the selected Local Origin"), 40, OutToolTip);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionWorldPosition
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionWorldPosition::UMaterialExpressionWorldPosition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("XYZ"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("XY"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Z"), 1, 0, 0, 1, 0));

	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionWorldPosition::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// TODO: should use a separate check box for Including/Excluding Material Shader Offsets
	return Compiler->WorldPosition(WorldPositionShaderOffset);
}

void UMaterialExpressionWorldPosition::GetCaption(TArray<FString>& OutCaptions) const
{
	switch (WorldPositionShaderOffset)
	{
	case WPT_Default:
		{
			OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "WorldPositonText", "Absolute World Position").ToString());
			break;
		}

	case WPT_ExcludeAllShaderOffsets:
		{
			OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "WorldPositonExcludingOffsetsText", "Absolute World Position (Excluding Material Offsets)").ToString());
			break;
		}

	case WPT_CameraRelative:
		{
			OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "CamRelativeWorldPositonText", "Camera Relative World Position").ToString());
			break;
		}

	case WPT_CameraRelativeNoOffsets:
		{
			OutCaptions.Add(NSLOCTEXT("MaterialExpressions", "CamRelativeWorldPositonExcludingOffsetsText", "Camera Relative World Position (Excluding Material Offsets)").ToString());
			break;
		}

	default:
		{
			UE_LOGF(LogMaterial, Fatal, "Unknown world position shader offset type");
			break;
		}
	}
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionObjectPositionWS
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionObjectPositionWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ObjectWorldPosition(OriginType);
}

void UMaterialExpressionObjectPositionWS::GetCaption(TArray<FString>& OutCaptions) const
{
	FString ObjectType = Material && Material->MaterialDomain == MD_LightFunction ? "Light" : "Object";
	switch (OriginType)
	{
		case EPositionOrigin::Absolute:
		{
			OutCaptions.Add(FString::Printf(TEXT("%s Position  (Absolute)"), *ObjectType));
			break;
		}

		case EPositionOrigin::CameraRelative:
		{
			OutCaptions.Add(FString::Printf(TEXT("%s Position  (Camera Relative)"), *ObjectType));
			break;
		}

		default:
		{
			UE_LOGF(LogMaterial, Fatal, "Unknown position origin type");
			break;
		}
	}
}

void UMaterialExpressionObjectPositionWS::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	if (!Material)
	{
		return;
	}
	FString ToolTipText;
	switch (Material->MaterialDomain)
	{
		case MD_LightFunction:
		{
			ToolTipText += "Gets the local position of the light, based on the selected Local Origin.\n";
			ToolTipText += "Note: Light Atlas cannot resolve positional data, so will always return 0.0f";
			break;
		}
		case MD_PostProcess:
		{
			ToolTipText += "PostProcess materials cannot resolve positional data, so will always return 0.0f";
			break;
		}
		default:
		{
			ToolTipText += "Gets the local position of the mesh, based on the selected Local Origin.\n";
			ToolTipText += "Note: Returns 0 if primitive data is not available to the material.";
			break;
		}
	}
	ConvertToMultilineToolTip(*ToolTipText, 40, OutToolTip);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionObjectRadius
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
void UMaterialExpressionObjectRadius::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Object Radius"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionObjectBoundingBox
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
void UMaterialExpressionObjectBounds::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Object Bounds"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionObjectLocalBounds
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionObjectLocalBounds::UMaterialExpressionObjectLocalBounds(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Half Extents"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Half the extent (width, depth and height) of the object bounding box. In local space.");
	Outputs.Add(FExpressionOutput(TEXT("Extents"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Full extent (width, depth and height) of the object bounding box. Same as 2x Half Extents. In local space.");
	Outputs.Add(FExpressionOutput(TEXT("Min"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Minimum 3D point of the object bounding box. In local space.");
	Outputs.Add(FExpressionOutput(TEXT("Max"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Maximum 3D point of the object bounding box. In local space.");
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionObjectLocalBounds::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Material && Material->MaterialDomain == MD_DeferredDecal)
	{
		return CompilerError(Compiler, TEXT("Expression not available in the deferred decal material domain."));
	}

	return Compiler->ObjectLocalBounds(OutputIndex);
}

void UMaterialExpressionObjectLocalBounds::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Object Local Bounds"));
}

void UMaterialExpressionObjectLocalBounds::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) 
{
#if WITH_EDITORONLY_DATA
	if (OutputIndex >= 0 && OutputIndex < OutputToolTips.Num())
	{
		ConvertToMultilineToolTip(OutputToolTips[OutputIndex], 40, OutToolTip);
	}
#endif // WITH_EDITORONLY_DATA
}

void UMaterialExpressionObjectLocalBounds::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns various info about the object local bounding box."
		"Usable in vertex or pixel shader (no need to pipe this through vertex interpolators)."
		"Hover the output pins for more information."), 40, OutToolTip);
}
#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionBounds
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionBounds::UMaterialExpressionBounds(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	check(BoundsHalfExtentOutputIndex == Outputs.Num());
	Outputs.Add(FExpressionOutput(TEXT("Half Extents"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Half the extent (width, depth and height) of the bounding box. In local space.");

	check(BoundsExtentOutputIndex == Outputs.Num());
	Outputs.Add(FExpressionOutput(TEXT("Extents"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Full extent (width, depth and height) of the bounding box. Same as 2x Half Extents. In local space.");
	
	check(BoundsMinOutputIndex == Outputs.Num());
	Outputs.Add(FExpressionOutput(TEXT("Min"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Minimum 3D point of the bounding box. In local space.");
	
	check(BoundsMaxOutputIndex == Outputs.Num());
	Outputs.Add(FExpressionOutput(TEXT("Max"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Maximum 3D point of the bounding box. In local space.");
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionBounds::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Material && Material->MaterialDomain == MD_DeferredDecal)
	{
		return CompilerError(Compiler, TEXT("Expression not available in the deferred decal material domain."));
	}

	switch (Type)
	{
	case MEILB_ObjectLocal: return Compiler->ObjectLocalBounds(OutputIndex);
	case MEILB_InstanceLocal: return Compiler->InstanceLocalBounds(OutputIndex);
	case MEILB_PreSkinnedLocal: return Compiler->PreSkinnedLocalBounds(OutputIndex);
	default: checkNoEntry();
	}
	return INDEX_NONE;
}

void UMaterialExpressionBounds::GetCaption(TArray<FString>& OutCaptions) const
{
	FString Caption;
	switch (Type)
	{
	case MEILB_ObjectLocal: Caption = TEXT("Bounds (Object Local)"); break;
	case MEILB_InstanceLocal: Caption = TEXT("Bounds (Instance Local)"); break;
	case MEILB_PreSkinnedLocal: Caption = TEXT("Bounds (Pre-Skinned Local)"); break;
	default: checkNoEntry();
	}

	OutCaptions.Add(MoveTemp(Caption));
}

void UMaterialExpressionBounds::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
#if WITH_EDITORONLY_DATA
	if (OutputIndex >= 0 && OutputIndex < OutputToolTips.Num())
	{
		ConvertToMultilineToolTip(OutputToolTips[OutputIndex], 40, OutToolTip);
	}
#endif // WITH_EDITORONLY_DATA
}

void UMaterialExpressionBounds::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns bounding box info of the specified type."
		"Usable in vertex or pixel shader (no need to pipe this through vertex interpolators)."
		"Hover the output pins for more information."), 40, OutToolTip);
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPreSkinnedLocalBounds
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPreSkinnedLocalBounds::UMaterialExpressionPreSkinnedLocalBounds(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Half Extents"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Half the extent (width, depth and height) of the pre-skinned bounding box. In local space.");
	Outputs.Add(FExpressionOutput(TEXT("Extents"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Full extent (width, depth and height) of the pre-skinned bounding box. Same as 2x Half Extents. In local space.");
	Outputs.Add(FExpressionOutput(TEXT("Min"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Minimum 3D point of the pre-skinned bounding box. In local space.");
	Outputs.Add(FExpressionOutput(TEXT("Max"), 1, 1, 1, 1, 0));
	OutputToolTips.Add("Maximum 3D point of the pre-skinned bounding box. In local space.");
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionPreSkinnedLocalBounds::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Material && Material->MaterialDomain == MD_DeferredDecal)
	{
		return CompilerError(Compiler, TEXT("Expression not available in the deferred decal material domain."));
	}

	return Compiler->PreSkinnedLocalBounds(OutputIndex);
}

void UMaterialExpressionPreSkinnedLocalBounds::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Pre-Skinned Local Bounds"));
}

void UMaterialExpressionPreSkinnedLocalBounds::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) 
{
#if WITH_EDITORONLY_DATA
	if (OutputIndex >= 0 && OutputIndex < OutputToolTips.Num())
	{
		ConvertToMultilineToolTip(OutputToolTips[OutputIndex], 40, OutToolTip);
	}
#endif // WITH_EDITORONLY_DATA
}

void UMaterialExpressionPreSkinnedLocalBounds::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns various info about the pre-skinned local bounding box for skeletal meshes."
		"Will return the regular local space bounding box for static meshes."
		"Usable in vertex or pixel shader (no need to pipe this through vertex interpolators)."
		"Hover the output pins for more information."), 40, OutToolTip);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDistanceCullFade
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionDistanceCullFade::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->DistanceCullFade();
}

void UMaterialExpressionDistanceCullFade::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Distance Cull Fade"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDistanceFieldsRenderingSwitch
///////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR

int32 UMaterialExpressionDistanceFieldsRenderingSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Yes.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing DistanceFieldsRenderingSwitch input 'Yes'"));
	}

	if (!No.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing DistanceFieldsRenderingSwitch input 'No'"));
	}

	if (!IsMobilePlatform(Compiler->GetShaderPlatform()))
	{
		return (IsUsingDistanceFields(Compiler->GetShaderPlatform()))? Yes.Compile(Compiler) : No.Compile(Compiler);
	}

    if (IsMobileDistanceFieldEnabled(Compiler->GetShaderPlatform()))
    {
        return Yes.Compile(Compiler);
    }

	return No.Compile(Compiler);
}

bool UMaterialExpressionDistanceFieldsRenderingSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		if (It->GetTracedInput().Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}
	return false;
}

void UMaterialExpressionDistanceFieldsRenderingSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DistanceFieldsRenderingSwitch"));
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionActorPositionWS
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionActorPositionWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Material != nullptr && (Material->MaterialDomain != MD_Surface) && (Material->MaterialDomain != MD_DeferredDecal) && (Material->MaterialDomain != MD_Volume))
	{
		return CompilerError(Compiler, TEXT("Expression only available in the Surface and Deferred Decal material domains."));
	}

	return Compiler->ActorWorldPosition(OriginType);
}

void UMaterialExpressionActorPositionWS::GetCaption(TArray<FString>& OutCaptions) const
{
	switch (OriginType)
	{
		case EPositionOrigin::Absolute:
		{
			OutCaptions.Add(TEXT("Actor Position (Absolute)"));
			break;
		}

		case EPositionOrigin::CameraRelative:
		{
			OutCaptions.Add(TEXT("Actor Position (Camera Relative)"));
			break;
		}

		default:
		{
			UE_LOGF(LogMaterial, Fatal, "Unknown position origin type");
			break;
		}
	}
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDeriveNormalZ
///////////////////////////////////////////////////////////////////////////////	
#if WITH_EDITOR
int32 UMaterialExpressionDeriveNormalZ::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!InXY.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input normal xy vector whose z should be derived."));
	}

	// z = sqrt(saturate(1 - ( x * x + y * y)));
	int32 InputVector = Compiler->ForceCast(InXY.Compile(Compiler), MCT_Float2);
	int32 DotResult = Compiler->Dot(InputVector, InputVector);
	int32 InnerResult = Compiler->Sub(Compiler->Constant(1), DotResult);
	int32 SaturatedInnerResult = Compiler->Saturate(InnerResult);
	int32 DerivedZ = Compiler->SquareRoot(SaturatedInnerResult);
	int32 AppendedResult = Compiler->ForceCast(Compiler->AppendVector(InputVector, DerivedZ), MCT_Float3);

	return AppendedResult;
}

void UMaterialExpressionDeriveNormalZ::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DeriveNormalZ"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionConstantBiasScale
///////////////////////////////////////////////////////////////////////////////

int32 UMaterialExpressionConstantBiasScale::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing ConstantBiasScale input"));
	}

	return Compiler->Mul(Compiler->Add(Compiler->Constant(Bias), Input.Compile(Compiler)), Compiler->Constant(Scale));
}


void UMaterialExpressionConstantBiasScale::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ConstantBiasScale"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionCustom
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionCustom::UMaterialExpressionCustom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Description = TEXT("Custom");
	Code = TEXT("// The below expression will get compiled\n// into the output of this node\nfloat3(1, 1, 1)");

	ShowCode = false;

	OutputType = CMOT_Float3;

	Inputs.Add(FCustomInput());
	Inputs[0].InputName = TEXT("");

#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR

int32 UMaterialExpressionCustom::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	TArray<int32> CompiledInputs;

	// We're not using the fixed up code here, just the SceneTextureInfo, which tracks whether the value of SceneTexture / UserSceneTexture input pins are
	// used in the custom HLSL code.  In many cases, the scene textures will be fetched from using SceneTextureLookup, SceneTextureFetch, or *.Fetch calls
	// in the custom HLSL, rather than using the input pin value.  The fixup function has a parser that is aware of HLSL syntax, and able to tokenize out
	// identifiers, and handle symbol sequences (such as Input.ID or Input.Fetch) that will be substituted into SceneTextureFetch calls.
	TArray<int8> SceneTextureInfo;
	UE::MaterialTranslatorUtils::CustomExpressionSceneTextureInputFixup(this, *Code, SceneTextureInfo);

	for( int32 i=0;i<Inputs.Num();i++ )
	{
		// skip over unnamed inputs
		if( Inputs[i].InputName.IsNone() )
		{
			CompiledInputs.Add(INDEX_NONE);
		}
		else
		{
			if(!Inputs[i].Input.GetTracedInput().Expression)
			{
				return Compiler->Errorf(TEXT("Custom material %s missing input %d (%s)"), *Description, i+1, *Inputs[i].InputName.ToString());
			}

			int32 InputCode;
			if (SceneTextureInfo.Num() && SceneTextureInfo[i] == -1)
			{
				// Scene texture reference, not actually used in the custom HLSL.  The special output index "3" (not present in the user interface) specifies
				// that the scene texture should be compiled into the shader for use by custom HLSL, but the input pin value is not actually used in code, so
				// its expression shouldn't be compiled in.  It's necessary to explicitly remove the input pin's SceneTexture expression, because the SceneColor
				// alpha propagation feature means the SceneTextureLookup function now has a side effect of caching propagated alpha, and the compiler can no
				// longer dead strip calls to that function.  This early removal of the fetch makes stats more accurate as a side bonus.
				FExpressionInput LocalInput = Inputs[i].Input;
				LocalInput.OutputIndex = 3;
				InputCode = LocalInput.Compile(Compiler);
			}
			else
			{
				InputCode = Inputs[i].Input.Compile(Compiler);
			}

			if( InputCode < 0 )
			{
				return InputCode;
			}
			CompiledInputs.Add( InputCode );
		}
	}

	return Compiler->CustomExpression(this, OutputIndex, CompiledInputs);
}


void UMaterialExpressionCustom::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(Description);
}


TArrayView<FExpressionInput*> UMaterialExpressionCustom::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(Inputs.Num());
	for( int32 i = 0; i < Inputs.Num(); i++ )
	{
		CachedInputs.Add(&Inputs[i].Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionCustom::GetInput(int32 InputIndex)
{
	return Inputs.IsValidIndex(InputIndex) ? &Inputs[InputIndex].Input : nullptr;
}

FName UMaterialExpressionCustom::GetInputName(int32 InputIndex) const
{
	if( InputIndex < Inputs.Num() )
	{
		return Inputs[InputIndex].InputName;
	}
	return NAME_None;
}

void UMaterialExpressionCustom::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// strip any spaces from input name
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if( PropertyThatChanged && PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(FCustomInput, InputName))
	{
		for( FCustomInput& Input : Inputs )
	{
			FString InputName = Input.InputName.ToString();
			if (InputName.ReplaceInline(TEXT(" "),TEXT("")) > 0)
		{
				Input.InputName = *InputName;
			}
		}
	}

	RebuildOutputs();

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, Inputs) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ThisClass, AdditionalOutputs))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionCustom::RebuildOutputs()
{
	Outputs.Reset(AdditionalOutputs.Num() + 1);
	if (AdditionalOutputs.Num() == 0)
	{
		bShowOutputNameOnPin = false;
		Outputs.Add(FExpressionOutput(TEXT("")));
	}
	else
	{
		bShowOutputNameOnPin = true;
		Outputs.Add(FExpressionOutput(TEXT("return")));
		for (const FCustomOutput& CustomOutput : AdditionalOutputs)
		{
			if (!CustomOutput.OutputName.IsNone())
			{
				Outputs.Add(FExpressionOutput(CustomOutput.OutputName));
			}
		}
	}
}

EMaterialValueType UMaterialExpressionCustom::GetOutputValueType(int32 OutputIndex)
{
	ECustomMaterialOutputType Type = CMOT_MAX;
	if (OutputIndex == 0)
	{
		Type = OutputType;
	}
	else if (OutputIndex >= 1 && OutputIndex - 1 < AdditionalOutputs.Num())
	{
		Type = AdditionalOutputs[OutputIndex - 1].OutputType;
	}

	switch (Type)
	{
	case CMOT_Float1:
		return MCT_Float;
	case CMOT_Float2:
		return MCT_Float2;
	case CMOT_Float3:
		return MCT_Float3;
	case CMOT_Float4:
		return MCT_Float4;
	case CMOT_MaterialAttributes:
		return MCT_MaterialAttributes;
	default:
		return MCT_Unknown;
	}
}

bool UMaterialExpressionCustom::IsResultMaterialAttributes(int32 OutputIndex)
{
	return GetOutputValueType(OutputIndex) == MCT_MaterialAttributes;
}

void UMaterialExpressionCustom::GetIncludeFilePaths(TSet<FString>& OutIncludeFilePaths) const
{
	OutIncludeFilePaths.Append(IncludeFilePaths);
}

bool UMaterialExpressionCustom::HasPixelDiscard() const
{
	return ContainsClipInstruction == CMCI_Yes ||
		(ContainsClipInstruction == CMCI_Search && (Code.Contains(TEXT("clip("), ESearchCase::CaseSensitive) || Code.Contains(TEXT("discard"), ESearchCase::CaseSensitive)));
}
#endif // WITH_EDITOR

void UMaterialExpressionCustom::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	UnderlyingArchive.UsingCustomVersion(FRenderingObjectVersion::GUID);
	UnderlyingArchive.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	UnderlyingArchive.UsingCustomVersion(FNaniteResearchStreamObjectVersion::GUID);

	// Make a copy of the current code before we change it
	const FString PreFixUp = Code;

	bool bDidUpdate = false;

	if (UnderlyingArchive.UEVer() < VER_UE4_INSTANCED_STEREO_UNIFORM_UPDATE)
	{
		// Look for WorldPosition rename
		if (Code.ReplaceInline(TEXT("Parameters.WorldPosition"), TEXT("Parameters.AbsoluteWorldPosition"), ESearchCase::CaseSensitive) > 0) //DF_TODO
		{
			bDidUpdate = true;
		}
	}
	// Fix up uniform references that were moved from View to Frame as part of the instanced stereo implementation
	else if (UnderlyingArchive.UEVer() < VER_UE4_INSTANCED_STEREO_UNIFORM_REFACTOR)
	{
		// Uniform members that were moved from View to Frame
		static const FString UniformMembers[] = {
			FString("FieldOfViewWideAngles"),
			FString("PrevFieldOfViewWideAngles"),
			FString("ViewRectMin"),
			FString("ViewSizeAndInvSize"),
			FString("BufferSizeAndInvSize"),
			FString("ExposureScale"),
			FString("DiffuseOverrideParameter"),
			FString("SpecularOverrideParameter"),
			FString("NormalOverrideParameter"),
			FString("RoughnessOverrideParameter"),
			FString("PrevFrameGameTime"),
			FString("PrevFrameRealTime"),
			FString("OutOfBoundsMask"),
			FString("WorldCameraMovementSinceLastFrame"),
			FString("CullingSign"),
			FString("NearPlane"),
			FString("GameTime"),
			FString("RealTime"),
			FString("Random"),
			FString("FrameNumber"),
			FString("CameraCut"),
			FString("UseLightmaps"),
			FString("UnlitViewmodeMask"),
			FString("DirectionalLightColor"),
			FString("DirectionalLightDirection"),
			FString("DirectionalLightShadowTransition"),
			FString("DirectionalLightShadowSize"),
			FString("DirectionalLightScreenToShadow"),
			FString("DirectionalLightShadowDistances"),
			FString("UpperSkyColor"),
			FString("LowerSkyColor"),
			FString("TranslucencyLightingVolumeMin"),
			FString("TranslucencyLightingVolumeInvSize"),
			FString("TemporalAAParams"),
			FString("CircleDOFParams"),
			FString("DepthOfFieldFocalDistance"),
			FString("DepthOfFieldScale"),
			FString("DepthOfFieldFocalLength"),
			FString("DepthOfFieldFocalRegion"),
			FString("DepthOfFieldNearTransitionRegion"),
			FString("DepthOfFieldFarTransitionRegion"),
			FString("MotionBlurNormalizedToPixel"),
			FString("GeneralPurposeTweak"),
			FString("DemosaicVposOffset"),
			FString("IndirectLightingColorScale"),
			FString("HDR32bppEncodingMode"),
			FString("AtmosphericFogSunDirection"),
			FString("AtmosphericFogSunPower"),
			FString("AtmosphericFogPower"),
			FString("AtmosphericFogDensityScale"),
			FString("AtmosphericFogDensityOffset"),
			FString("AtmosphericFogGroundOffset"),
			FString("AtmosphericFogDistanceScale"),
			FString("AtmosphericFogAltitudeScale"),
			FString("AtmosphericFogHeightScaleRayleigh"),
			FString("AtmosphericFogStartDistance"),
			FString("AtmosphericFogDistanceOffset"),
			FString("AtmosphericFogSunDiscScale"),
			FString("AtmosphericFogRenderMask"),
			FString("AtmosphericFogInscatterAltitudeSampleNum"),
			FString("AtmosphericFogSunColor"),
			FString("AmbientCubemapTint"),
			FString("AmbientCubemapIntensity"),
			FString("RenderTargetSize"),
			FString("SkyLightParameters"),
			FString("SceneFString(TEXTureMinMax"),
			FString("SkyLightColor"),
			FString("SkyIrradianceEnvironmentMap"),
			FString("MobilePreviewMode"),
			FString("HMDEyePaddingOffset"),
			FString("DirectionalLightShadowFString(TEXTure"),
			FString("SamplerState"),
		};

		const FString ViewUniformName("View.");
		const FString FrameUniformName("Frame.");
		for (const FString& Member : UniformMembers)
		{
			const FString SearchString = FrameUniformName + Member;
			const FString ReplaceString = ViewUniformName + Member;
			if (Code.ReplaceInline(*SearchString, *ReplaceString, ESearchCase::CaseSensitive) > 0)
			{
				bDidUpdate = true;
			}
		}
	}

	if (UnderlyingArchive.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::RemovedRenderTargetSize)
	{
		if (Code.ReplaceInline(TEXT("View.RenderTargetSize"), TEXT("View.BufferSizeAndInvSize.xy"), ESearchCase::CaseSensitive) > 0)
		{
			bDidUpdate = true;
		}
	}

	if (UnderlyingArchive.CustomVer(FNaniteResearchStreamObjectVersion::GUID) < FNaniteResearchStreamObjectVersion::LWCTypesInShaders)
	{
		static const TCHAR* UniformMembers[] =
		{
			TEXT("WorldToClip"),
			TEXT("ClipToWorld"),
			TEXT("ScreenToWorld"),
			TEXT("PrevClipToWorld"),
			TEXT("WorldCameraOrigin"),
			TEXT("WorldViewOrigin"),
			TEXT("PrevWorldCameraOrigin"),
			TEXT("PrevWorldViewOrigin"),
			TEXT("PreViewTranslation"),
			TEXT("PrevPreViewTranslation"),
		};

		for (const TCHAR* Member : UniformMembers)
		{
			const FString ViewSearchString = FString("View.") + Member;
			const FString ReplaceString = FString("DFDemote(ResolvedView.") + Member + FString(")");

			if (Code.ReplaceInline(*ViewSearchString, *ReplaceString, ESearchCase::CaseSensitive) > 0)
			{
				bDidUpdate = true;
			}
		}

		// We really want to replace all instances of 'View.Member' and 'ResolvedView.Member' with 'DFDemote(ResolvedView.Member)'
		// But since this is just dumb string processing and we're not really attempting to parse HLSL, replacing 'View.Member' will also match 'ResolvedVIEW.Member', and turn it into 'ResolvedDFDemote(ResolvedView.Member)'
		// So we just allow that to happen, and then fix up any instances of 'ResolvedDFDemote' here
		// This is admittedly pretty ugly...if this gets any worse probably need to just add a real HLSL parser here
		if (Code.ReplaceInline(TEXT("ResolvedDFDemote(ResolvedView."), TEXT("DFDemote(ResolvedView."), ESearchCase::CaseSensitive) > 0)
		{
			bDidUpdate = true;
		}

		static const TCHAR* GlobalExpressionsToReplace[] =
		{
			TEXT("GetPrimitiveData(Parameters).WorldToLocal"),
			TEXT("GetPrimitiveData(Parameters).LocalToWorld"),
			TEXT("GetPrimitiveData(Parameters.PrimitiveId).WorldToLocal"),
			TEXT("GetPrimitiveData(Parameters.PrimitiveId).LocalToWorld"),
			TEXT("Parameters.AbsoluteWorldPosition"),
		};
		static const TCHAR* GlobalExpressionsReplacement[] =
		{
			TEXT("GetWorldToLocal(Parameters)"),
			TEXT("GetLocalToWorld(Parameters)"),
			TEXT("GetWorldToLocal(Parameters)"),
			TEXT("GetLocalToWorld(Parameters)"),
			TEXT("GetWorldPosition(Parameters)"),
		};
		
		int NumGlobalExpressionsToReplace = sizeof(GlobalExpressionsToReplace)/sizeof(GlobalExpressionsToReplace[0]);
		for (int Index = 0; Index < NumGlobalExpressionsToReplace; Index++)
		{
			if (Code.ReplaceInline(GlobalExpressionsToReplace[Index], GlobalExpressionsReplacement[Index], ESearchCase::CaseSensitive) > 0)
			{
				bDidUpdate = true;
			}
		}

		static const TCHAR* GlobalExpressionsToDemote[] =
		{
			TEXT("GetWorldPosition(Parameters)"),
			TEXT("GetPrevWorldPosition(Parameters)"),
			TEXT("GetObjectWorldPosition(Parameters)"),
			TEXT("GetWorldToLocal(Parameters)"),
			TEXT("GetLocalToWorld(Parameters)")
		};

		for (const TCHAR* Expression : GlobalExpressionsToDemote)
		{
			const FString ReplaceString = FString::Printf(TEXT("WSDemote(%s)"), Expression);
			if (Code.ReplaceInline(Expression, *ReplaceString, ESearchCase::CaseSensitive) > 0)
			{
				bDidUpdate = true;
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// If we made changes, copy the original into the description just in case
	if (bDidUpdate)
	{
		Desc += TEXT("\n*** Original source before expression upgrade ***\n");
		Desc += PreFixUp;
		UE_LOGF(LogMaterial, Log, "Uniform references updated for custom material expression %ls.", *Description);
	}
#endif // WITH_EDITORONLY_DATA
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialFunctionInterfaceEditorOnlyData
///////////////////////////////////////////////////////////////////////////////
void UMaterialFunctionInterfaceEditorOnlyData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (!IsTemplate())
	{
		// If our owner material function isn't pointing to this EditorOnlyData it means this object's name
		// doesn't match the default created object name and we need to fix our pointer into the material function interface
		UMaterialFunctionInterface* MFInterface = CastChecked<UMaterialFunctionInterface>(GetOuter());
		if (MFInterface->EditorOnlyData != this)
		{
			MFInterface->EditorOnlyData = this;
		}
	}
#endif
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSwitch
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionSwitch::UMaterialExpressionSwitch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Inputs.Add(FSwitchCustomInput());
	Inputs[0].InputName = TEXT("");

#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
int32 UMaterialExpressionSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// if the input is hooked up, use it, otherwise use the internal constant
	int32 CompiledSwitchValue = SwitchValue.GetTracedInput().Expression ? SwitchValue.Compile(Compiler) : Compiler->Constant(ConstSwitchValue);
	int32 CompiledDefault = Default.GetTracedInput().Expression ? Default.Compile(Compiler) : Compiler->Constant(ConstDefault);

	TArray<int32> CompiledInputs;

	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		if (!Inputs[i].Input.GetTracedInput().Expression)
		{
			return Compiler->Errorf(TEXT("Texture Multiplexer missing input %d (%s)"), i + 1, *Inputs[i].InputName.ToString());
		}
		int32 InputCode = Inputs[i].Input.Compile(Compiler);
		if (InputCode < 0)
		{
			return InputCode;
		}
		CompiledInputs.Add(InputCode);
	}

	return Compiler->Switch(CompiledSwitchValue, CompiledDefault, CompiledInputs);
}


void UMaterialExpressionSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(Description);
}


TArrayView<FExpressionInput*> UMaterialExpressionSwitch::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(2 + Inputs.Num());
	CachedInputs.Add(&SwitchValue);
	CachedInputs.Add(&Default);
	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		CachedInputs.Add(&Inputs[i].Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionSwitch::GetInput(int32 InputIndex)
{
	if (InputIndex < 0 || InputIndex > Inputs.Num() + 1)
	{
		return nullptr;
	}
	switch (InputIndex)
	{
	case 0:
		return &SwitchValue;
	case 1:
		return &Default;
	default:
		return &Inputs[InputIndex - 2].Input;
	}
}

FName UMaterialExpressionSwitch::GetInputName(int32 InputIndex) const
{
	if (InputIndex < 0 || InputIndex > Inputs.Num() + 1)
	{
		return NAME_None;
	}
	switch (InputIndex)
	{
	case 0:
		return GET_MEMBER_NAME_STRING_CHECKED(ThisClass, SwitchValue);
	case 1:
		return GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Default);
	default:
		return Inputs[InputIndex - 2].InputName;
	}
}

void UMaterialExpressionSwitch::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// strip any spaces from input name
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(FCustomInput, InputName))
	{
		for (FSwitchCustomInput& Input : Inputs)
		{
			FString InputName = Input.InputName.ToString();
			if (InputName.ReplaceInline(TEXT(" "), TEXT("")) > 0)
			{
				Input.InputName = *InputName;
			}
		}
	}

	RebuildOutputs();

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, Inputs))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionSwitch::RebuildOutputs()
{
	Outputs.Reset(1);
	bShowOutputNameOnPin = false;
	Outputs.Add(FExpressionOutput(TEXT("")));
}

#endif // WITH_EDITOR

void UMaterialExpressionSwitch::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	UnderlyingArchive.UsingCustomVersion(FRenderingObjectVersion::GUID);
	UnderlyingArchive.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	UnderlyingArchive.UsingCustomVersion(FNaniteResearchStreamObjectVersion::GUID);
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialFunctionInterface
///////////////////////////////////////////////////////////////////////////////
namespace MaterialFunctionInterface
{
	FString GetEditorOnlyDataName(const TCHAR* InMaterialName)
	{
		return FString::Printf(TEXT("%sEditorOnlyData"), InMaterialName);
	}
}

void UMaterialFunctionInterface::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	// use the non-templated CreateEditorOnlyData because we need to use the virtual to get the class of the EOData
	// additionally, pass in an overridden name because we have existing EOData in the wild that must load correctly
	UObject* EOData = UE::EditorOptional::CreateEditorOptionalObject(this, GetEditorOnlyDataClass(), *MaterialFunctionInterface::GetEditorOnlyDataName(*GetName()));
	EditorOnlyData = CastChecked<UMaterialFunctionInterfaceEditorOnlyData>(EOData);
#endif
	Super::PostInitProperties();

	// Initialize StateId to something unique, in case this is a new function
	if (!IsTemplate())
	{
		StateId = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::InterfaceInit);
	}
}

void UMaterialFunctionInterface::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	static const auto CVarDuplicateVerbatim = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.MaterialsDuplicateVerbatim"));
	const bool bKeepStateId = StateId.IsValid() && HasAnyFlags(RF_WasLoaded) && CVarDuplicateVerbatim->GetValueOnAnyThread();
	if (!bKeepStateId)
	{
		// Initialize StateId to something unique, in case this is a new function
		StateId = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::InterfaceDuplicate);
	}
}

void UMaterialFunctionInterface::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (EditorOnlyData && !GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		// Test for badly named EditorOnlyData objects
		const FString EditorOnlyDataName = MaterialFunctionInterface::GetEditorOnlyDataName(*GetName());
		if (EditorOnlyData->GetName() != EditorOnlyDataName)
		{
			UMaterialFunctionInterfaceEditorOnlyData* CorrectEditorOnlyDataObj = Cast<UMaterialFunctionInterfaceEditorOnlyData>(StaticFindObject(EditorOnlyData->GetClass(), EditorOnlyData->GetOuter(), *EditorOnlyDataName, EFindObjectFlags::ExactClass));
			if (CorrectEditorOnlyDataObj)
			{
				// Copy data to correct EditorOnlyObject
				TArray<uint8> Data;
				FObjectWriter Ar(EditorOnlyData, Data);
				FObjectReader(CorrectEditorOnlyDataObj, Data);

				// Point EditorOnlyData to the right object
				EditorOnlyData = CorrectEditorOnlyDataObj;
			}
		}
	}
#endif

	
	if (!StateId.IsValid())
	{
		StateId = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::InterfacePostLoad);
	}
}

#if WITH_EDITORONLY_DATA
void UMaterialFunctionInterface::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMaterialFunctionInterfaceEditorOnlyData::StaticClass()));
}
#endif

void UMaterialFunctionInterface::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

#if WITH_EDITORONLY_DATA
	for (FName TagName : { GET_MEMBER_NAME_CHECKED(UMaterialFunctionInterface, CombinedInputTypes),
		GET_MEMBER_NAME_CHECKED(UMaterialFunctionInterface, CombinedOutputTypes)})
	{
		// Hide the combined input/output types as they are only needed in code
		if (FAssetRegistryTag* AssetTag = Context.FindTag(TagName); AssetTag)
		{
			AssetTag->Type = UObject::FAssetRegistryTag::TT_Hidden;
		}
	}
#endif
}

bool UMaterialFunctionInterface::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	bool bRenamed = Super::Rename(NewName, NewOuter, Flags);
#if WITH_EDITORONLY_DATA
	// if we have EditorOnlyData, also rename it if we are changing the material's name
	if (bRenamed && NewName && EditorOnlyData)
	{
		FString EditorOnlyDataName = MaterialFunctionInterface::GetEditorOnlyDataName(NewName);
		bRenamed = EditorOnlyData->Rename(*EditorOnlyDataName, nullptr, Flags);
	}
#endif
	return bRenamed;
}

UMaterialFunctionInterface* UMaterialFunctionInterface::GetBaseFunctionInterface()
{
	return GetBaseFunction();
}

const UMaterialFunctionInterface* UMaterialFunctionInterface::GetBaseFunctionInterface() const
{
	return GetBaseFunction();
}

#if WITH_EDITORONLY_DATA

TConstArrayView<TObjectPtr<UMaterialExpression>> UMaterialFunctionInterface::GetExpressions() const
{
	const UMaterialFunction* BaseFunction = GetBaseFunction();
	if (BaseFunction)
	{
		return BaseFunction->GetExpressions();
	}
	return TConstArrayView<TObjectPtr<UMaterialExpression>>();
}

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR

const FString& UMaterialFunctionInterface::GetDescription() const
{
	const UMaterialFunction* BaseFunction = GetBaseFunction();
	if (BaseFunction)
	{
		return BaseFunction->Description;
	}
	static const FString EmptyString;
	return EmptyString;
}

bool UMaterialFunctionInterface::GetReentrantFlag() const
{
	const UMaterialFunction* BaseFunction = GetBaseFunction();
	if (BaseFunction)
	{
		return BaseFunction->GetReentrantFlag();
	}
	return false;
}

void UMaterialFunctionInterface::SetReentrantFlag(bool bIsReentrant)
{
	UMaterialFunction* BaseFunction = GetBaseFunction();
	if (BaseFunction)
	{
		BaseFunction->SetReentrantFlag(bIsReentrant);
	}
}

void UMaterialFunctionInterface::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	#if 0 // temporary to unblock people, tracked as UE-109810
	{
		FMaterialUpdateContext UpdateContext;
		ForceRecompileForRendering(UpdateContext, nullptr);
	}
	#endif
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialFunctionInterface::ForceRecompileForRendering(FMaterialUpdateContext& UpdateContext, UMaterial* InPreviewMaterial)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialFunctionInterface::ForceRecompileForRendering)

	//@todo - recreate guid only when needed, not when a comment changes
	StateId = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::Compile);

	// Go through all materials in memory and recompile them if they use this function
	for (TObjectIterator<UMaterialInterface> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UMaterialInterface* CurrentMaterialInterface = *It;
		if (CurrentMaterialInterface == InPreviewMaterial)
		{
			continue;
		}

		bool bRecompile = false;

		// Preview materials often use expressions for rendering that are not in their Expressions array, 
		// And therefore their MaterialFunctionInfos are not up to date.
		// However we don't want to trigger this if the Material is a preview material itself. This can now be the case with thumbnail preview materials for material functions.
		if (InPreviewMaterial && !InPreviewMaterial->bIsPreviewMaterial)
		{
			UMaterial* CurrentMaterial = Cast<UMaterial>(CurrentMaterialInterface);
			if (CurrentMaterial && CurrentMaterial->bIsPreviewMaterial)
			{
				bRecompile = true;
			}
		}

		if (!bRecompile)
		{
			UMaterialFunctionInterface* Self = this;
			CurrentMaterialInterface->IterateDependentFunctions(
				[Self, &bRecompile](UMaterialFunctionInterface* InFunction)
				{
					if (InFunction == Self)
					{
						bRecompile = true;
						return false;
					}
					return true;
				});
		}

		if (bRecompile)
		{
			// Propagate the change to this material
			UpdateContext.AddMaterialInterface(CurrentMaterialInterface);
			CurrentMaterialInterface->ForceRecompileForRendering(EMaterialShaderPrecompileMode::None);
		}
	}
}

bool UMaterialFunctionInterface::GetParameterOverrideValue(EMaterialParameterType Type, const FName& ParameterName, FMaterialParameterMetadata& OutValue, FMFRecursionGuard RecursionGuard) const
{
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedScalarParameter(const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::Scalar, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.AsScalar();
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedVectorParameter(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::Vector, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.AsLinearColor();
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedTextureParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::Texture, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.Texture;
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedTextureCollectionParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::TextureCollection, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.TextureCollection;
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedRuntimeVirtualTextureParameter(const FHashedMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture*& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::RuntimeVirtualTexture, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.RuntimeVirtualTexture;
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedSparseVolumeTextureParameter(const FHashedMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture*& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::SparseVolumeTexture, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.SparseVolumeTexture;
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedFontParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UFont*& OutFontValue, int32& OutFontPage)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::Font, ParameterInfo.GetName(), Meta))
	{
		OutFontValue = Meta.Value.Font.Value;
		OutFontPage = Meta.Value.Font.Page;
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedStaticSwitchParameter(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, FGuid& OutExpressionGuid)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::StaticSwitch, ParameterInfo.GetName(), Meta))
	{
		OutExpressionGuid = Meta.ExpressionGuid;
		OutValue = Meta.Value.AsStaticSwitch();
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedParameterCollectionParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection*& OutValue)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::ParameterCollection, ParameterInfo.GetName(), Meta))
	{
		OutValue = Meta.Value.ParameterCollection;
		return true;
	}
	return false;
}

bool UMaterialFunctionInterface::OverrideNamedStaticComponentMaskParameter(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutR, bool& OutG, bool& OutB, bool& OutA, FGuid& OutExpressionGuid)
{
	FMaterialParameterMetadata Meta;
	if (GetParameterOverrideValue(EMaterialParameterType::Scalar, ParameterInfo.GetName(), Meta))
	{
		OutExpressionGuid = Meta.ExpressionGuid;
		OutR = Meta.Value.Bool[0];
		OutG = Meta.Value.Bool[1];
		OutB = Meta.Value.Bool[2];
		OutA = Meta.Value.Bool[3];
		return true;
	}
	return false;
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
const UClass* UMaterialFunctionInterface::GetEditorOnlyDataClass() const
{
	return UMaterialFunctionInterfaceEditorOnlyData::StaticClass();
}

#endif // WITH_EDITORONLY_DATA

///////////////////////////////////////////////////////////////////////////////
// UMaterialFunctionEditorOnlyData
///////////////////////////////////////////////////////////////////////////////
void UMaterialFunctionEditorOnlyData::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
#if WITH_EDITORONLY_DATA
	// If the collection of expressions got some null expressions remove them now, but warn the user about it.
	if (ExpressionCollection.Expressions.Remove(nullptr))
	{
		UE_LOGF(LogMaterial, Warning, "Material Function %ls editor only data contained null expression and some expressions may be missing. "
			"Please close and reopen this Material Function and verify it is still valid.",
			*GetFullName());
	}
#endif

	UMaterialFunctionInterfaceEditorOnlyData::PreSave(ObjectSaveContext);
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialFunction
///////////////////////////////////////////////////////////////////////////////
UMaterialFunction::UMaterialFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	LibraryCategoriesText.Add(LOCTEXT("Misc", "Misc"));
#endif
#if WITH_EDITORONLY_DATA
	PreviewMaterial = nullptr;
	ThumbnailInfo = nullptr;
	bAllExpressionsLoadedCorrectly = true;
#endif
}

#if WITH_EDITOR
UMaterialInterface* UMaterialFunction::GetPreviewMaterial()
{
	if( nullptr == PreviewMaterial )
	{
		PreviewMaterial = NewObject<UMaterial>(this, NAME_None, RF_Transient | RF_Public);
		PreviewMaterial->bIsPreviewMaterial = true;

		PreviewMaterial->AssignExpressionCollection(GetExpressionCollection());
		//Update cached expression data to ensure function calls are populated for resolving the preview
		PreviewMaterial->UpdateCachedExpressionData();

		// Find the output with bLastPreviewed set to true preferably expression.
		UMaterialExpressionFunctionOutput* PreviewOutput = nullptr;
		for (UMaterialExpression* Expression : GetExpressions())
		{
			UMaterialExpressionFunctionOutput* Output = Cast<UMaterialExpressionFunctionOutput>(Expression);
			if (Output && (!PreviewOutput || Output->bLastPreviewed))
			{
				PreviewOutput = Output;
				if (Output->bLastPreviewed)
				{
					break;
				}
			}
		}

		// Set the chosen output as preview.
		if (PreviewOutput != nullptr)
		{
			PreviewOutput->ConnectToPreviewMaterial(PreviewMaterial, 0);
		}

		PreviewMaterial->MaterialDomain = PreviewMaterialDomain;

		//Compile the material.
		PreviewMaterial->PreEditChange(nullptr);
		PreviewMaterial->PostEditChange();
	}
	return PreviewMaterial;
}

void UMaterialFunction::UpdateInputOutputTypes()
{
	CombinedInputTypes = {};
	CombinedOutputTypes = {};

	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		UMaterialExpressionFunctionOutput* OutputExpression = Cast<UMaterialExpressionFunctionOutput>(CurrentExpression);
		UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(CurrentExpression);

		if (InputExpression)
		{
			CombinedInputTypes |= InputExpression->GetInputValueType(0);
		}
		else if (OutputExpression)
		{
			CombinedInputTypes |= OutputExpression->GetOutputValueType(0);
		}
	}
}

#if WITH_EDITOR
void UMaterialFunction::UpdateDependentFunctionCandidates()
{
	DependentFunctionExpressionCandidates.Reset();
	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		if (UMaterialExpressionMaterialFunctionCall* MaterialFunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(CurrentExpression))
		{
			DependentFunctionExpressionCandidates.Add(MaterialFunctionExpression);
		}
	}
}
#endif

void UMaterialFunction::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMaterialFunction, bEnableNewHLSLGenerator))
	{
		if (EditorMaterial)
		{
			EditorMaterial->bEnableNewHLSLGenerator = bEnableNewHLSLGenerator;
		}
	}

	// many property changes can require rebuild of graph so always mark as changed
	// not interested in PostEditChange calls though as the graph may have instigated it
	if (PropertyChangedEvent.Property && MaterialGraph)
	{
		MaterialGraph->NotifyGraphChanged();
	}
}

void UMaterialFunction::ForceRecompileForRendering(FMaterialUpdateContext& UpdateContext, UMaterial* InPreviewMaterial)
{
#if WITH_EDITORONLY_DATA
	UpdateInputOutputTypes();
	UpdateDependentFunctionCandidates();
#endif

	Super::ForceRecompileForRendering(UpdateContext, InPreviewMaterial);
}

#endif // WITH_EDITOR

void UMaterialFunction::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	#if WITH_EDITORONLY_DATA
	if (DependentFunctionExpressionCandidates.Remove(nullptr))
	{
		UE_LOGF(LogMaterial, Warning, "Material Function %ls contained some null dependent function expression calls. "
			"Please close and reopen this Material Function and verify it is still valid.",
			   *GetFullName());
	}
	#endif

	UMaterialFunctionInterface::PreSave(ObjectSaveContext);
}

void UMaterialFunction::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.UEVer() < VER_UE4_FLIP_MATERIAL_COORDS)
	{
		GMaterialFunctionsThatNeedExpressionsFlipped.Set(this);
	}
	else if (Ar.UEVer() < VER_UE4_FIX_MATERIAL_COORDS)
	{
		GMaterialFunctionsThatNeedCoordinateCheck.Set(this);
	}
	else if (Ar.UEVer() < VER_UE4_FIX_MATERIAL_COMMENTS)
	{
		GMaterialFunctionsThatNeedCommentFix.Set(this);
	}

	if (Ar.UEVer() < VER_UE4_ADD_LINEAR_COLOR_SAMPLER)
	{
		GMaterialFunctionsThatNeedSamplerFixup.Set(this);
	}

	if (Ar.UEVer() < VER_UE4_LIBRARY_CATEGORIES_AS_FTEXT)
	{
		for (FString& Category : LibraryCategories_DEPRECATED)
		{
			LibraryCategoriesText.Add(FText::FromString(Category));
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::MaterialFeatureLevelNodeFixForSM6)
	{
		GMaterialFunctionsThatNeedFeatureLevelSM6Fix.Set(this);
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void UMaterialFunction::GetAllCustomOutputExpressions(TArray<class UMaterialExpressionCustomOutput*>& OutCustomOutputs) const
{
	for (UMaterialExpression* Expression : GetExpressions())
	{
		UMaterialExpressionCustomOutput* CustomOutput = Cast<UMaterialExpressionCustomOutput>(Expression);
		if (CustomOutput)
		{
			OutCustomOutputs.Add(CustomOutput);
		}
	}
}

#endif //WITH_EDITORONLY_DATA

void UMaterialFunction::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// Only clear when duplicating to a real package,
	// not when the material editor creates its transient editing copy
	if (!GetOutermost()->HasAnyFlags(RF_Transient))
	{
		bExposeToLibrary = false;
	}
}

void UMaterialFunction::PostLoad()
{
	LLM_SCOPE(ELLMTag::Materials);

	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	UMaterialFunctionEditorOnlyData* EditorOnly = GetEditorOnlyData();

	if (EditorOnly && FunctionExpressions_DEPRECATED.Num() > 0)
	{
		ensure(EditorOnly->ExpressionCollection.Expressions.Num() == 0);
		EditorOnly->ExpressionCollection.Expressions = MoveTemp(FunctionExpressions_DEPRECATED);
	}

	if (EditorOnly && FunctionEditorComments_DEPRECATED.Num() > 0)
	{
		ensure(EditorOnly->ExpressionCollection.EditorComments.Num() == 0);
		EditorOnly->ExpressionCollection.EditorComments = MoveTemp(FunctionEditorComments_DEPRECATED);
	}
	
	if (EditorOnly)
	{
		for (UMaterialExpression* Expression : EditorOnly->ExpressionCollection.Expressions)
		{
			// Expressions whose type was removed can be nullptr
			if (Expression)
			{
				Expression->ConditionalPostLoad();
			}
		}
	}

	if (!EditorOnly || (GetPackage()->HasAnyFlags(EObjectFlags::RF_WasLoaded) && !EditorOnly->HasAnyFlags(EObjectFlags::RF_WasLoaded)))
	{
		UE_LOGF(LogMaterial, Error, "Material function '%ls' was loaded, but it is missing editor-only data. Materials calling this function will not translate correctly.", *GetFullName());
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	if (CombinedOutputTypes == 0)
	{
		UpdateInputOutputTypes();
	}
	UpdateDependentFunctionCandidates();

	bAllExpressionsLoadedCorrectly = true;

	if (GIsEditor && EditorOnly)
	{
		// Go over all expressions in the collection and invalidate the material if a null expression is found. Then
		// remove the null expression from the array.
		for (int i = 0; i < EditorOnly->ExpressionCollection.Expressions.Num();)
		{
			UMaterialExpression* Expression = EditorOnly->ExpressionCollection.Expressions[i].Get();
			if (Expression)
			{
				++i;
				continue;
			}

			// Mark this function as invalid. This will cause the material containing an active call to it to fail translation.
			bAllExpressionsLoadedCorrectly = false;

			EditorOnly->ExpressionCollection.Expressions.RemoveAt(i);
		}

		if (!bAllExpressionsLoadedCorrectly)
		{
			// Dirty this function by deterministically changing its StateId.
			static FGuid NotAllExpressionsLoadedCorrectlyToken(TEXT("6B9D300E-ED9D-4E4A-A141-05DE059B5704"));
			StateId.A ^= NotAllExpressionsLoadedCorrectlyToken.A;
			StateId.B ^= NotAllExpressionsLoadedCorrectlyToken.B;
			StateId.C ^= NotAllExpressionsLoadedCorrectlyToken.C;
			StateId.D ^= NotAllExpressionsLoadedCorrectlyToken.D;

			UE_LOGF(LogMaterial, Log, "Some expression in Material Function %ls failed to load correctly. "
				"This will cause any material using this MF to fail translation. "
				"Please check open affected Material Function, make sure its expression graph is valid and resave it. "
				"Material Function's GUID was changed to %ls.", *GetFullName(), *StateId.ToString());
		}
	}

	if (GMaterialFunctionsThatNeedExpressionsFlipped.Get(this))
	{
		GMaterialFunctionsThatNeedExpressionsFlipped.Clear(this);
		if (EditorOnly)
		{
			UMaterial::FlipExpressionPositions(EditorOnly->ExpressionCollection.Expressions, EditorOnly->ExpressionCollection.EditorComments, true);
		}
	}
	else if (GMaterialFunctionsThatNeedCoordinateCheck.Get(this))
	{
		GMaterialFunctionsThatNeedCoordinateCheck.Clear(this);
		if (EditorOnly)
		{
			if (HasFlippedCoordinates())
			{
				UMaterial::FlipExpressionPositions(EditorOnly->ExpressionCollection.Expressions, EditorOnly->ExpressionCollection.EditorComments, false);
			}
			UMaterial::FixCommentPositions(EditorOnly->ExpressionCollection.EditorComments);
		}
	}
	else if (GMaterialFunctionsThatNeedCommentFix.Get(this))
	{
		GMaterialFunctionsThatNeedCommentFix.Clear(this);
		if (EditorOnly)
		{
			UMaterial::FixCommentPositions(EditorOnly->ExpressionCollection.EditorComments);
		}
	}

	if (GMaterialFunctionsThatNeedFeatureLevelSM6Fix.Get(this))
	{
		GMaterialFunctionsThatNeedFeatureLevelSM6Fix.Clear(this);
		if (EditorOnly)
		{
			UMaterial::FixFeatureLevelNodesForSM6(EditorOnly->ExpressionCollection.Expressions);
		}
	}

	if (GMaterialFunctionsThatNeedSamplerFixup.Get(this))
	{
		GMaterialFunctionsThatNeedSamplerFixup.Clear(this);
		if (EditorOnly)
		{
			for (UMaterialExpression* Expression : EditorOnly->ExpressionCollection.Expressions)
			{
				UMaterialExpressionTextureBase* TextureExpression = Cast<UMaterialExpressionTextureBase>(Expression);
				if (TextureExpression && TextureExpression->Texture)
				{
					switch (TextureExpression->Texture->CompressionSettings)
					{
					case TC_Normalmap:
						TextureExpression->SamplerType = SAMPLERTYPE_Normal;
						break;
					case TC_Grayscale:
						TextureExpression->SamplerType = TextureExpression->Texture->SRGB ? SAMPLERTYPE_Grayscale : SAMPLERTYPE_LinearGrayscale;
						break;

					case TC_Masks:
						TextureExpression->SamplerType = SAMPLERTYPE_Masks;
						break;

					case TC_Alpha:
						TextureExpression->SamplerType = SAMPLERTYPE_Alpha;
						break;
					default:
						TextureExpression->SamplerType = TextureExpression->Texture->SRGB ? SAMPLERTYPE_Color : SAMPLERTYPE_LinearColor;
						break;
					}
				}
			}
		}
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void UMaterialFunction::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMaterialFunctionEditorOnlyData::StaticClass()));
}
#endif


#if WITH_EDITORONLY_DATA
TConstArrayView<TObjectPtr<UMaterialExpression>> UMaterialFunction::GetExpressions() const
{
	return GetEditorOnlyData()->ExpressionCollection.Expressions;
}

TConstArrayView<TObjectPtr<UMaterialExpressionComment>> UMaterialFunction::GetEditorComments() const
{
	return GetEditorOnlyData()->ExpressionCollection.EditorComments;
}

const FMaterialExpressionCollection& UMaterialFunction::GetExpressionCollection() const
{
	return GetEditorOnlyData()->ExpressionCollection;
}

FMaterialExpressionCollection& UMaterialFunction::GetExpressionCollection()
{
	return GetEditorOnlyData()->ExpressionCollection;
}

void UMaterialFunction::AssignExpressionCollection(const FMaterialExpressionCollection& InCollection)
{
	GetEditorOnlyData()->ExpressionCollection = InCollection;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR

void UMaterialFunction::UpdateFromFunctionResource()
{
	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		UMaterialExpressionMaterialFunctionCall* MaterialFunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(CurrentExpression);
		if (MaterialFunctionExpression)
		{
			MaterialFunctionExpression->UpdateFromFunctionResource();
		}
	}
}

void UMaterialFunction::GetInputsAndOutputs(TArray<FFunctionExpressionInput>& OutInputs, TArray<FFunctionExpressionOutput>& OutOutputs) const
{
	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		UMaterialExpressionFunctionOutput* OutputExpression = Cast<UMaterialExpressionFunctionOutput>(CurrentExpression);
		UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(CurrentExpression);

		if (InputExpression)
		{
			// Create an input
			FFunctionExpressionInput NewInput;
			NewInput.ExpressionInput = InputExpression;
			NewInput.ExpressionInputId = InputExpression->Id;
			NewInput.Input.InputName = InputExpression->InputName;
			NewInput.Input.OutputIndex = INDEX_NONE;
			OutInputs.Add(NewInput);
		}
		else if (OutputExpression)
		{
			// Create an output
			FFunctionExpressionOutput NewOutput;
			NewOutput.ExpressionOutput = OutputExpression;
			NewOutput.ExpressionOutputId = OutputExpression->Id;
			NewOutput.Output.OutputName = OutputExpression->OutputName;
			OutOutputs.Add(NewOutput);
		}
	}

	// Sort by display priority
	struct FCompareInputSortPriority
	{
		FORCEINLINE bool operator()( const FFunctionExpressionInput& A, const FFunctionExpressionInput& B ) const 
		{ 
			return A.ExpressionInput->SortPriority < B.ExpressionInput->SortPriority; 
		}
	};
	OutInputs.Sort( FCompareInputSortPriority() );

	struct FCompareOutputSortPriority
	{
		FORCEINLINE bool operator()( const FFunctionExpressionOutput& A, const FFunctionExpressionOutput& B ) const 
		{ 
			return A.ExpressionOutput->SortPriority < B.ExpressionOutput->SortPriority; 
		}
	};
	OutOutputs.Sort( FCompareOutputSortPriority() );
}

/** Finds an input in the passed in array with a matching Id. */
static const FFunctionExpressionInput* FindInputById(const FGuid& Id, const TArray<FFunctionExpressionInput>& Inputs)
{
	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); InputIndex++)
	{
		const FFunctionExpressionInput& CurrentInput = Inputs[InputIndex];
		if (CurrentInput.ExpressionInputId == Id)
		{
			return &CurrentInput;
		}
	}
	return nullptr;
}

/** Finds an input in the passed in array with a matching name. */
static const FFunctionExpressionInput* FindInputByName(const FName& Name, const TArray<FFunctionExpressionInput>& Inputs)
{
	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); InputIndex++)
	{
		const FFunctionExpressionInput& CurrentInput = Inputs[InputIndex];
		if (CurrentInput.ExpressionInput && CurrentInput.ExpressionInput->InputName == Name)
		{
			return &CurrentInput;
		}
	}
	return nullptr;
}

/** Finds an input in the passed in array with a matching expression object. */
static const FExpressionInput* FindInputByExpression(UMaterialExpressionFunctionInput* InputExpression, const TArray<FFunctionExpressionInput>& Inputs)
{
	for (int32 InputIndex = 0; InputIndex < Inputs.Num(); InputIndex++)
	{
		const FFunctionExpressionInput& CurrentInput = Inputs[InputIndex];
		if (CurrentInput.ExpressionInput == InputExpression)
		{
			return &CurrentInput.Input;
		}
	}
	return nullptr;
}

/** Finds an output in the passed in array with a matching Id. */
static int32 FindOutputIndexById(const FGuid& Id, const TArray<FFunctionExpressionOutput>& Outputs)
{
	for (int32 OutputIndex = 0; OutputIndex < Outputs.Num(); OutputIndex++)
	{
		const FFunctionExpressionOutput& CurrentOutput = Outputs[OutputIndex];
		if (CurrentOutput.ExpressionOutputId == Id)
		{
			return OutputIndex;
		}
	}
	return INDEX_NONE;
}

/** Finds an output in the passed in array with a matching name. */
static int32 FindOutputIndexByName(const FName& Name, const TArray<FFunctionExpressionOutput>& Outputs)
{
	for (int32 OutputIndex = 0; OutputIndex < Outputs.Num(); OutputIndex++)
	{
		const FFunctionExpressionOutput& CurrentOutput = Outputs[OutputIndex];
		if (CurrentOutput.ExpressionOutput && CurrentOutput.ExpressionOutput->OutputName == Name)
		{
			return OutputIndex;
		}
	}
	return INDEX_NONE;
}
#endif

bool UMaterialFunction::ValidateFunctionUsage(FMaterialCompiler* Compiler, const FFunctionExpressionOutput& Output)
{
	bool bHasValidOutput = true;
	int32 NumInputs = 0;
	int32 NumOutputs = 0;

#if WITH_EDITOR
	if (GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayer)
	{
		bHasValidOutput = UMaterialExpressionLayerStack::ValidateFunctionForLayerUsage(Compiler, this);
	}
	else if (GetMaterialFunctionUsage() == EMaterialFunctionUsage::MaterialLayerBlend)
	{
		bHasValidOutput = UMaterialExpressionLayerStack::ValidateFunctionForBlendUsage(Compiler, this);
	}
#endif

	return bHasValidOutput;
}

#if WITH_EDITOR
int32 UMaterialFunction::Compile(FMaterialCompiler* Compiler, const FFunctionExpressionOutput& Output)
{
	int32 ReturnValue = INDEX_NONE;

	if (ValidateFunctionUsage(Compiler, Output))
	{
		if (Output.ExpressionOutput->A.GetTracedInput().Expression)
		{
			// Compile the given function output
			ReturnValue = Output.ExpressionOutput->A.Compile(Compiler);
		}
		else
		{
			ReturnValue = Compiler->Errorf(TEXT("Missing function output connection '%s'"), *Output.ExpressionOutput->OutputName.ToString());
		}
	}

	return ReturnValue;
}

void UMaterialFunction::LinkIntoCaller(const TArray<FFunctionExpressionInput>& CallerInputs)
{
	// Go through all the function's input expressions and hook their inputs up to the corresponding expression in the material being compiled.
	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		if(UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(CurrentExpression))
		{
			// Initialize for this function call
			FExpressionInput* CurrentPreview = InputExpression->AddNewEffectivePreviewDuringCompile(InputExpression->Preview);

			// Get the FExpressionInput which stores information about who this input node should be linked to in order to compile
			const FExpressionInput* MatchingInput = FindInputByExpression(InputExpression, CallerInputs);

			if (CurrentPreview && MatchingInput 
				// Only change the connection if the input has a valid connection,
				// Otherwise we will need what's connected to the Preview input
				&& (MatchingInput->Expression || !InputExpression->bUsePreviewValueAsDefault))
			{
				// Connect this input to the expression in the material that it should be connected to
				CurrentPreview->Expression = MatchingInput->Expression;
				CurrentPreview->OutputIndex = MatchingInput->OutputIndex;
				CurrentPreview->Mask = MatchingInput->Mask;
				CurrentPreview->MaskR = MatchingInput->MaskR;
				CurrentPreview->MaskG = MatchingInput->MaskG;
				CurrentPreview->MaskB = MatchingInput->MaskB;
				CurrentPreview->MaskA = MatchingInput->MaskA;
			}
		}
	}
}

void UMaterialFunction::UnlinkFromCaller()
{
	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		if(UMaterialExpressionFunctionInput* InputExpression = Cast<UMaterialExpressionFunctionInput>(CurrentExpression))
		{
			InputExpression->RemoveLastEffectivePreviewDuringCompile();
		}
	}
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UMaterialFunction::IsDependent(UMaterialFunctionInterface* OtherFunction)
{
	if (!OtherFunction)
	{
		return false;
	}

	bool bIsChild = false;
#if WITH_EDITORONLY_DATA
	UMaterialFunction* AsFunction = Cast<UMaterialFunction>(OtherFunction);
	bIsChild = AsFunction && AsFunction->ParentFunction == this;
#endif

	if (OtherFunction == this || bIsChild)
	{
		return true;
	}

#if WITH_EDITOR
	SetReentrantFlag(true);
#endif

	bool bIsDependent = false;
	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		UMaterialExpressionMaterialFunctionCall* MaterialFunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(CurrentExpression);
		if (MaterialFunctionExpression && MaterialFunctionExpression->MaterialFunction)
		{
			// Recurse to handle nesting
			bIsDependent = bIsDependent 
				|| MaterialFunctionExpression->MaterialFunction->GetReentrantFlag()
				|| MaterialFunctionExpression->MaterialFunction->IsDependent(OtherFunction);
		}
	}

#if WITH_EDITOR
	SetReentrantFlag(false);
#endif

	return bIsDependent;
}

bool UMaterialFunction::IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
{
#if WITH_EDITOR
	if (!ensure(!HasAnyFlags(RF_NeedPostLoad)))
	{
		return false;
	}
	for (UMaterialExpressionMaterialFunctionCall* MaterialFunctionExpression : DependentFunctionExpressionCandidates)
	{
		if(!MaterialFunctionExpression->IterateDependentFunctions(Predicate))
		{
			return false;
		}
	}
#else
	for (UMaterialExpression* CurrentExpression : FunctionExpressions)
	{
		if (UMaterialExpressionMaterialFunctionCall* MaterialFunctionExpression = Cast<UMaterialExpressionMaterialFunctionCall>(CurrentExpression))
		{
			if (!MaterialFunctionExpression->IterateDependentFunctions(Predicate))
			{
				return false;
			}
		}
	}
#endif
	return true;
}

void UMaterialFunction::GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
{
	IterateDependentFunctions([&DependentFunctions](UMaterialFunctionInterface* MaterialFunction) -> bool
	{
		DependentFunctions.AddUnique(MaterialFunction);
		return true;
	});
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
bool UMaterialFunction::HasFlippedCoordinates() const
{
	uint32 ReversedInputCount = 0;
	uint32 StandardInputCount = 0;

	for (UMaterialExpression* CurrentExpression : GetExpressions())
	{
		UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(CurrentExpression);
		if (FunctionOutput && FunctionOutput->A.Expression)
		{
			if (FunctionOutput->A.Expression->MaterialExpressionEditorX > FunctionOutput->MaterialExpressionEditorX)
			{
				++ReversedInputCount;
			}
			else
			{
				++StandardInputCount;
			}
		}
	}

	// Can't be sure coords are flipped if most are set out correctly
	return ReversedInputCount > StandardInputCount;
}

bool UMaterialFunction::SetParameterValueEditorOnly(const FName& ParameterName, const FMaterialParameterMetadata& Meta)
{
	bool bResult = false;
	for (UMaterialExpression* Expression : GetExpressions())
	{
		if (Expression->SetParameterValue(ParameterName, Meta))
		{
			bResult = true;
		}
		else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (FunctionCall->MaterialFunction)
			{
				TArray<UMaterialFunctionInterface*> Functions;
				Functions.Add(FunctionCall->MaterialFunction);
				FunctionCall->MaterialFunction->GetDependentFunctions(Functions);

				for (UMaterialFunctionInterface* Function : Functions)
				{
					for (const TObjectPtr<UMaterialExpression>& FunctionExpression : Function->GetExpressions())
					{
						if (FunctionExpression->SetParameterValue(ParameterName, Meta))
						{
							bResult = true;
						}
					}
				}
			}
		}
	}
	return bResult;
}

bool UMaterialFunction::SetVectorParameterValueEditorOnly(FName ParameterName, FLinearColor InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
}

bool UMaterialFunction::SetScalarParameterValueEditorOnly(FName ParameterName, float InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::SetTextureParameterValueEditorOnly(FName ParameterName, class UTexture* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::SetRuntimeVirtualTextureParameterValueEditorOnly(FName ParameterName, class URuntimeVirtualTexture* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::SetSparseVolumeTextureParameterValueEditorOnly(FName ParameterName, class USparseVolumeTexture* InValue)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = InValue;
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::SetFontParameterValueEditorOnly(FName ParameterName, class UFont* InFontValue, int32 InFontPage)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = FMaterialParameterValue(InFontValue, InFontPage);
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::SetStaticSwitchParameterValueEditorOnly(FName ParameterName, bool OutValue, FGuid OutExpressionGuid)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = OutValue;
	Meta.ExpressionGuid = OutExpressionGuid;
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::SetStaticComponentMaskParameterValueEditorOnly(FName ParameterName, bool R, bool G, bool B, bool A, FGuid OutExpressionGuid)
{
	FMaterialParameterMetadata Meta;
	Meta.Value = FMaterialParameterValue(R, G, B, A);
	Meta.ExpressionGuid = OutExpressionGuid;
	return SetParameterValueEditorOnly(ParameterName, Meta);
};

bool UMaterialFunction::IsUsingNewHLSLGenerator() const
{
	if (bEnableNewHLSLGenerator)
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.Material.Translator.EnableNew"));
		return CVar->GetValueOnAnyThread();
	}
	return false;
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialFunctionMaterialLayerBlend
///////////////////////////////////////////////////////////////////////////////
void UMaterialFunctionMaterialLayerBlend::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITORONLY_DATA
	const int32 FortMainVer = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (FortMainVer < FFortniteMainBranchObjectVersion::MaterialFunctionBlendTopBottomInputEnum)
	{
		/**
		* This code takes the legacy blend nodes which were based on an unreliable method of node generation
		* and converts it to assign designated blend input enum types which explicitly defines whether the input node
		* is relevant to the top or the bottom attributes input. No other inputs need updating as legacy blends have
		* a strict rule of only 2 inputs exactly.
		*/
		bool TopAssigned = false;
		bool BottomAssigned = false;

		TArray<TObjectPtr<UMaterialExpression>>& BlendExpressions = GetExpressionCollection().Expressions;
		for (UMaterialExpression* Expression : BlendExpressions)
		{
			UMaterialExpressionFunctionInput* InputNode = Cast<UMaterialExpressionFunctionInput>(Expression);
			if (!InputNode)
			{
				continue;
			}

			if (InputNode->InputType == FunctionInput_MaterialAttributes)
			{
				if (!TopAssigned && InputNode->InputName == TopMaterialBlendInputName)
				{
					InputNode->BlendInputRelevance = EBlendInputRelevance::Top;
					TopAssigned = true;
				}
				if (!BottomAssigned && InputNode->InputName == BottomMaterialBlendInputName)
				{
					InputNode->BlendInputRelevance = EBlendInputRelevance::Bottom;
					BottomAssigned = true;
				}

				//Historical Blends likely only have 2 inputs anyway so early out.
				if (TopAssigned && BottomAssigned)
				{
					break;
				}
			}
		}

		if (!TopAssigned || !BottomAssigned)
		{
			UE_LOGF(LogMaterial, Warning, "Legacy Material Layer Blend Function \"%ls\" has renamed Top and Bottom input nodes. Please manually set BlendInputRelevance on these nodes to their relative type.", *GetFullName());
		}
	}
#endif
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialFunctionInstance
///////////////////////////////////////////////////////////////////////////////

UMaterialFunctionInstance::UMaterialFunctionInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	PreviewMaterial = nullptr;
	ThumbnailInfo = nullptr;
#endif
}

void UMaterialFunctionInstance::SetParent(UMaterialFunctionInterface* NewParent)
{
	Parent = NewParent;
	MaterialFunctionUsage = NewParent->GetMaterialFunctionUsage();
	Base = GetBaseFunction();
}

EMaterialFunctionUsage UMaterialFunctionInstance::GetMaterialFunctionUsage()
{
	UMaterialFunctionInterface* BaseFunction = GetBaseFunction();
	return BaseFunction ? BaseFunction->GetMaterialFunctionUsage() : EMaterialFunctionUsage::Default;
}

UMaterialFunction* UMaterialFunctionInstance::GetBaseFunction(FMFRecursionGuard RecursionGuard)
{
	if (!Parent || RecursionGuard.Contains(this))
	{
		return nullptr;
	}

	RecursionGuard.Set(this);
	return Parent->GetBaseFunction(RecursionGuard);
}

const UMaterialFunction* UMaterialFunctionInstance::GetBaseFunction(FMFRecursionGuard RecursionGuard) const
{
	if (!Parent || RecursionGuard.Contains(this))
	{
		return nullptr;
	}

	RecursionGuard.Set(this);
	return Parent->GetBaseFunction(RecursionGuard);
}

#if WITH_EDITOR
void UMaterialFunctionInstance::UpdateParameterSet()
{
	if (UMaterialFunction* BaseFunction = GetBaseFunction())
	{
		TArray<UMaterialFunctionInterface*> Functions;
		BaseFunction->GetDependentFunctions(Functions);
		Functions.AddUnique(BaseFunction);

		// Loop through all contained parameters and update names as needed
		for (UMaterialFunctionInterface* Function : Functions)
		{
			for (UMaterialExpression* FunctionExpression : Function->GetExpressions())
			{
				if (const UMaterialExpressionScalarParameter* ScalarParameter = Cast<const UMaterialExpressionScalarParameter>(FunctionExpression))
				{
					for (FScalarParameterValue& ScalarParameterValue : ScalarParameterValues)
					{
						if (ScalarParameterValue.ExpressionGUID == ScalarParameter->ExpressionGUID)
						{
							ScalarParameterValue.ParameterInfo.Name = ScalarParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionVectorParameter* VectorParameter = Cast<const UMaterialExpressionVectorParameter>(FunctionExpression))
				{
					for (FVectorParameterValue& VectorParameterValue : VectorParameterValues)
					{
						if (VectorParameterValue.ExpressionGUID == VectorParameter->ExpressionGUID)
						{
							VectorParameterValue.ParameterInfo.Name = VectorParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionDoubleVectorParameter* DoubleVectorParameter = Cast<const UMaterialExpressionDoubleVectorParameter>(FunctionExpression))
				{
					for (FDoubleVectorParameterValue& DoubleVectorParameterValue : DoubleVectorParameterValues)
					{
						if (DoubleVectorParameterValue.ExpressionGUID == DoubleVectorParameter->ExpressionGUID)
						{
							DoubleVectorParameterValue.ParameterInfo.Name = DoubleVectorParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionTextureSampleParameter* TextureParameter = Cast<const UMaterialExpressionTextureSampleParameter>(FunctionExpression))
				{
					for (FTextureParameterValue& TextureParameterValue : TextureParameterValues)
					{
						if (TextureParameterValue.ExpressionGUID == TextureParameter->ExpressionGUID)
						{
							TextureParameterValue.ParameterInfo.Name = TextureParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionTextureCollectionParameter* TextureCollectionParameter = Cast<const UMaterialExpressionTextureCollectionParameter>(FunctionExpression))
				{
					for (FTextureCollectionParameterValue& TextureCollectionParameterValue : TextureCollectionParameterValues)
					{
						if (TextureCollectionParameterValue.ExpressionGUID == TextureCollectionParameter->ExpressionGUID)
						{
							TextureCollectionParameterValue.ParameterInfo.Name = TextureCollectionParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionRuntimeVirtualTextureSampleParameter* RuntimeVirtualTextureParameter = Cast<const UMaterialExpressionRuntimeVirtualTextureSampleParameter>(FunctionExpression))
				{
					for (FRuntimeVirtualTextureParameterValue& RuntimeVirtualTextureParameterValue : RuntimeVirtualTextureParameterValues)
					{
						if (RuntimeVirtualTextureParameterValue.ExpressionGUID == RuntimeVirtualTextureParameter->ExpressionGUID)
						{
							RuntimeVirtualTextureParameterValue.ParameterInfo.Name = RuntimeVirtualTextureParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionSparseVolumeTextureSampleParameter* SparseVolumeTextureParameter = Cast<const UMaterialExpressionSparseVolumeTextureSampleParameter>(FunctionExpression))
				{
					for (FSparseVolumeTextureParameterValue& SparseVolumeTextureParameterValue : SparseVolumeTextureParameterValues)
					{
						if (SparseVolumeTextureParameterValue.ExpressionGUID == SparseVolumeTextureParameter->ExpressionGUID)
						{
							SparseVolumeTextureParameterValue.ParameterInfo.Name = SparseVolumeTextureParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionFontSampleParameter* FontParameter = Cast<const UMaterialExpressionFontSampleParameter>(FunctionExpression))
				{
					for (FFontParameterValue& FontParameterValue : FontParameterValues)
					{
						if (FontParameterValue.ExpressionGUID == FontParameter->ExpressionGUID)
						{
							FontParameterValue.ParameterInfo.Name = FontParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionStaticBoolParameter* StaticSwitchParameter = Cast<const UMaterialExpressionStaticBoolParameter>(FunctionExpression))
				{
					for (FStaticSwitchParameter& StaticSwitchParameterValue : StaticSwitchParameterValues)
					{
						if (StaticSwitchParameterValue.ExpressionGUID == StaticSwitchParameter->ExpressionGUID)
						{
							StaticSwitchParameterValue.ParameterInfo.Name = StaticSwitchParameter->ParameterName;
							break;
						}
					}
				}
				else if (const UMaterialExpressionStaticComponentMaskParameter* StaticComponentMaskParameter = Cast<const UMaterialExpressionStaticComponentMaskParameter>(FunctionExpression))
				{
					for (FStaticComponentMaskParameter& StaticComponentMaskParameterValue : StaticComponentMaskParameterValues)
					{
						if (StaticComponentMaskParameterValue.ExpressionGUID == StaticComponentMaskParameter->ExpressionGUID)
						{
							StaticComponentMaskParameterValue.ParameterInfo.Name = StaticComponentMaskParameter->ParameterName;
							break;
						}
					}
				}
			}
		}
	}
}

void UMaterialFunctionInstance::OverrideMaterialInstanceParameterValues(UMaterialInstance* Instance)
{
	// Dynamic parameters
	Instance->ScalarParameterValues = ScalarParameterValues;
	Instance->VectorParameterValues = VectorParameterValues;
	Instance->DoubleVectorParameterValues = DoubleVectorParameterValues;
	Instance->TextureParameterValues = TextureParameterValues;
	Instance->TextureCollectionParameterValues = TextureCollectionParameterValues;
	Instance->RuntimeVirtualTextureParameterValues = RuntimeVirtualTextureParameterValues;
	Instance->SparseVolumeTextureParameterValues = SparseVolumeTextureParameterValues;
	Instance->FontParameterValues = FontParameterValues;

	// Static parameters
	FStaticParameterSet StaticParametersOverride = Instance->GetStaticParameters();
	StaticParametersOverride.StaticSwitchParameters = StaticSwitchParameterValues;
	StaticParametersOverride.EditorOnly.StaticComponentMaskParameters = StaticComponentMaskParameterValues;
	Instance->UpdateStaticPermutation(StaticParametersOverride);
}

void UMaterialFunctionInstance::UpdateFromFunctionResource()
{
	if (Parent)
	{
		Parent->UpdateFromFunctionResource();
	}
}

void UMaterialFunctionInstance::GetInputsAndOutputs(TArray<struct FFunctionExpressionInput>& OutInputs, TArray<struct FFunctionExpressionOutput>& OutOutputs) const
{
	if (Parent)
	{
		Parent->GetInputsAndOutputs(OutInputs, OutOutputs);
	}
}
#endif // WITH_EDITOR

bool UMaterialFunctionInstance::ValidateFunctionUsage(class FMaterialCompiler* Compiler, const FFunctionExpressionOutput& Output)
{
	return Parent ? Parent->ValidateFunctionUsage(Compiler, Output) : false;
}

#if WITH_EDITORONLY_DATA
void UMaterialFunctionInstance::Serialize(FArchive& Ar)
{
	FGuid OldStateId;
	// catch assets saved without proper StateId
	if (Ar.IsLoading() && Ar.IsPersistent())
	{
		OldStateId = StateId;
		StateId.Invalidate();
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading() && Ar.IsPersistent())
	{
		if (!StateId.IsValid())
		{
			UE_LOGF(LogMaterial, Warning, "%ls was saved without a valid StateId (old asset?). This will cause all materials using it to recompile their shaders on each load - please resave to fix.", *GetName());
			StateId = OldStateId;
		}
	}
}
#endif

void UMaterialFunctionInstance::PostLoad()
{
	Super::PostLoad();

	if (Parent)
	{
		Parent->ConditionalPostLoad();
	}

#if WITH_EDITORONLY_DATA
	for (const FScalarParameterValue& Param : ScalarParameterValues)
	{
		if (UCurveLinearColor* Curve = Param.AtlasData.Curve.Get())
		{
			Curve->ConditionalPostLoad();
		}

		if (UCurveLinearColorAtlas* Atlas = Param.AtlasData.Atlas.Get())
		{
			Atlas->ConditionalPostLoad();
		}
	}
#endif // WITH_EDITORONLY_DATA

	for (const FTextureParameterValue& Param : TextureParameterValues)
	{
		if (UTexture* Texture = Param.ParameterValue)
		{
			Texture->ConditionalPostLoad();
		}
	}

	for (const FTextureCollectionParameterValue& Param : TextureCollectionParameterValues)
	{
		if (UTextureCollection* TextureCollection = Param.ParameterValue)
		{
			TextureCollection->ConditionalPostLoad();
		}
	}

	for (const FFontParameterValue& Param : FontParameterValues)
	{
		if (UFont* Font = Param.FontValue)
		{
			Font->ConditionalPostLoad();
		}
	}

	for (const FRuntimeVirtualTextureParameterValue& Param : RuntimeVirtualTextureParameterValues)
	{
		if (URuntimeVirtualTexture* Texture = Param.ParameterValue)
		{
			Texture->ConditionalPostLoad();
		}
	}

	for (const FSparseVolumeTextureParameterValue& Param : SparseVolumeTextureParameterValues)
	{
		if (USparseVolumeTexture* Texture = Param.ParameterValue)
		{
			Texture->ConditionalPostLoad();
		}
	}
}

#if WITH_EDITOR
int32 UMaterialFunctionInstance::Compile(class FMaterialCompiler* Compiler, const struct FFunctionExpressionOutput& Output)
{
	return Parent ? Parent->Compile(Compiler, Output) : INDEX_NONE;
}

void UMaterialFunctionInstance::LinkIntoCaller(const TArray<FFunctionExpressionInput>& CallerInputs)
{
	if (Parent)
	{
		Parent->LinkIntoCaller(CallerInputs);
	}
}

void UMaterialFunctionInstance::UnlinkFromCaller()
{
	if (Parent)
	{
		Parent->UnlinkFromCaller();
	}
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UMaterialFunctionInstance::IsDependent(UMaterialFunctionInterface* OtherFunction)
{
	return Parent ? Parent->IsDependent(OtherFunction) : false;
}

bool UMaterialFunctionInstance::IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
{
	if (Parent)
	{
		if (!Parent->IterateDependentFunctions(Predicate))
		{
			return false;
		}
		if (!Predicate(Parent))
		{
			return false;
		}
	}
	return true;
}

void UMaterialFunctionInstance::GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
{
	if (Parent)
	{
		Parent->GetDependentFunctions(DependentFunctions);
		DependentFunctions.AddUnique(Parent);
	}
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
UMaterialInterface* UMaterialFunctionInstance::GetPreviewMaterial()
{
	if (nullptr == PreviewMaterial)
	{
		PreviewMaterial = NewObject<UMaterialInstanceConstant>((UObject*)GetTransientPackage(), FName("None"), RF_Transient);
	}

	// Update parameters in case they've changed so we get a live preview.
	// Once the preview material is created for a material function instance we never re-create it again
	// so it is important to take parameter updates so thumbnail and preview renders are correct.
	PreviewMaterial->SetParentEditorOnly(Parent ? Parent->GetPreviewMaterial() : nullptr);
	OverrideMaterialInstanceParameterValues(PreviewMaterial);
	PreviewMaterial->PreEditChange(nullptr);
	PreviewMaterial->PostEditChange();

	return PreviewMaterial;
}

void UMaterialFunctionInstance::UpdateInputOutputTypes()
{
	if (Parent)
	{
		Parent->UpdateInputOutputTypes();
	}
}

bool UMaterialFunctionInstance::HasFlippedCoordinates() const
{
	return Parent ? Parent->HasFlippedCoordinates() : false;
}

bool UMaterialFunctionInstance::GetParameterOverrideValue(EMaterialParameterType Type, const FName& ParameterName, FMaterialParameterMetadata& OutResult, FMFRecursionGuard RecursionGuard) const
{
	const FMemoryImageMaterialParameterInfo ParameterInfo(ParameterName);

	bool bResult = false;
	switch (Type)
	{
	case EMaterialParameterType::Scalar: bResult = GameThread_GetParameterValue(ScalarParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::Vector: bResult = GameThread_GetParameterValue(VectorParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::DoubleVector: bResult = GameThread_GetParameterValue(DoubleVectorParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::Texture: bResult = GameThread_GetParameterValue(TextureParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::TextureCollection: bResult = GameThread_GetParameterValue(TextureCollectionParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::RuntimeVirtualTexture: bResult = GameThread_GetParameterValue(RuntimeVirtualTextureParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::SparseVolumeTexture: bResult = GameThread_GetParameterValue(SparseVolumeTextureParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::Font: bResult = GameThread_GetParameterValue(FontParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::StaticSwitch: bResult = GameThread_GetParameterValue(StaticSwitchParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::ParameterCollection: bResult = GameThread_GetParameterValue(ParameterCollectionParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::StaticComponentMask: bResult = GameThread_GetParameterValue(StaticComponentMaskParameterValues, ParameterInfo, OutResult); break;
	default: checkNoEntry(); break;
	}

	if (!bResult && Parent && !RecursionGuard.Contains(this))
	{
		RecursionGuard.Set(this);
		bResult = Parent->GetParameterOverrideValue(Type, ParameterName, OutResult, RecursionGuard);
	}

	return bResult;
}

///////////////////////////////////////////////////////////////////////////////
// FMaterialLayersFunctionsID
///////////////////////////////////////////////////////////////////////////////

bool FMaterialLayersFunctionsID::operator==(const FMaterialLayersFunctionsID& Reference) const
{
	return LayerIDs == Reference.LayerIDs && BlendIDs == Reference.BlendIDs && LayerStates == Reference.LayerStates;
}


void FMaterialLayersFunctionsID::SerializeForDDC(FArchive& Ar)
{
	Ar << LayerIDs;
	Ar << BlendIDs;
	Ar << LayerStates;
}


void FMaterialLayersFunctionsID::UpdateHash(FXxHash64Builder& HashState) const
{
	for (const FGuid &Guid : LayerIDs)
	{
		HashState.Update((const uint8*)&Guid, sizeof(FGuid));
	}
	for (const FGuid &Guid : BlendIDs)
	{
		HashState.Update((const uint8*)&Guid, sizeof(FGuid));
	}
	HashState.Update((const uint8*)LayerStates.GetData(), LayerStates.Num()*LayerStates.GetTypeSize());
}

void FMaterialLayersFunctionsID::AppendKeyString(FString& KeyString) const
{
	FShaderKeyGenerator KeyGen(KeyString);
	Append(KeyGen);
}

void FMaterialLayersFunctionsID::Append(FShaderKeyGenerator& KeyGen) const
{
	for (const FGuid &Guid : LayerIDs)
	{
		KeyGen.Append(Guid);
	}
	for (const FGuid &Guid : BlendIDs)
	{
		KeyGen.Append(Guid);
	}
	for (bool State : LayerStates)
	{
		KeyGen.AppendBoolInt(State);
	}
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// FMaterialLayersFunctions
///////////////////////////////////////////////////////////////////////////////

const FGuid FMaterialLayersFunctions::BackgroundGuid(2u, 0u, 0u, 0u);

FMaterialLayersFunctionsRuntimeData::~FMaterialLayersFunctionsRuntimeData()
{
#if WITH_EDITORONLY_DATA
	// If this is destroyed while still holding 'LegacySerializedEditorOnlyData', that means it was serialized from an 'FMaterialLayersFunctions' in some unexpected context
	// Need to find where this is happening, and update that code to properly handle LegacySerializedEditorOnlyData
	checkf(!LegacySerializedEditorOnlyData, TEXT("LegacySerializedEditorOnlyData should have been acquired by FStaticParameterSet"));
#endif
}

bool FMaterialLayersFunctionsRuntimeData::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
#if WITH_EDITORONLY_DATA
	static const FName MaterialLayersFunctionsName("MaterialLayersFunctions");
	static const FName MaterialLayersPropertyName("MaterialLayers");
	if (Tag.GetType().IsStruct(MaterialLayersFunctionsName) &&
		Tag.Name == MaterialLayersPropertyName)
	{
		FMaterialLayersFunctions LocalMaterialLayers;
		FMaterialLayersFunctions::StaticStruct()->SerializeItem(Slot, &LocalMaterialLayers, nullptr);
		*this = MoveTemp(LocalMaterialLayers.GetRuntime());
		LegacySerializedEditorOnlyData.Reset(new FMaterialLayersFunctionsEditorOnlyData(MoveTemp(LocalMaterialLayers.EditorOnly)));
		return true;
	}
#endif // WITH_EDITORONLY_DATA
	return false;
}

#if WITH_EDITOR
const FMaterialLayersFunctionsID FMaterialLayersFunctionsRuntimeData::GetID(const FMaterialLayersFunctionsEditorOnlyData& EditorOnly) const
{
	FMaterialLayersFunctionsID Result;

	// Store the layer IDs in following format - stateID per function
	Result.LayerIDs.SetNum(Layers.Num());
	for (int i=0; i<Layers.Num(); ++i)
	{
		const UMaterialFunctionInterface* Layer = Layers[i];
		if (Layer)
		{
			check(Layer->StateId.IsValid());
			Result.LayerIDs[i] = Layer->StateId;
		}
		else
		{
			Result.LayerIDs[i] = FGuid();
		}
	}

	// Store the blend IDs in following format - stateID per function
	Result.BlendIDs.SetNum(Blends.Num());
	for (int i = 0; i < Blends.Num(); ++i)
	{
		const UMaterialFunctionInterface* Blend = Blends[i];
		if (Blend)
		{
			check(Blend->StateId.IsValid());
			Result.BlendIDs[i] = Blend->StateId;
		}
		else
		{
			Result.BlendIDs[i] = FGuid();
		}
	}

	// Store the states copy
	Result.LayerStates = EditorOnly.LayerStates;

	return Result;
}

FString FMaterialLayersFunctions::GetStaticPermutationString() const
{
	FString StaticKeyString;
	FShaderKeyGenerator KeyGen(StaticKeyString);
	AppendStaticPermutationKey(KeyGen);
	return StaticKeyString;
}

void FMaterialLayersFunctions::AppendStaticPermutationKey(FShaderKeyGenerator& KeyGen) const
{
	GetID().Append(KeyGen);
}

void FMaterialLayersFunctions::SerializeLegacy(FArchive& Ar)
{
	FString KeyString_DEPRECATED;
	Ar << KeyString_DEPRECATED;
}

#endif // WITH_EDITOR

void FMaterialLayersFunctions::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (LayerStates_DEPRECATED.Num() > 0)
		{
			EditorOnly.LayerStates = MoveTemp(LayerStates_DEPRECATED);
		}
		if (LayerNames_DEPRECATED.Num() > 0)
		{
			EditorOnly.LayerNames = MoveTemp(LayerNames_DEPRECATED);
		}
		if (RestrictToLayerRelatives_DEPRECATED.Num() > 0)
		{
			EditorOnly.RestrictToLayerRelatives = MoveTemp(RestrictToLayerRelatives_DEPRECATED);
		}
		if (RestrictToBlendRelatives_DEPRECATED.Num() > 0)
		{
			EditorOnly.RestrictToBlendRelatives = MoveTemp(RestrictToBlendRelatives_DEPRECATED);
		}
		if (LayerGuids_DEPRECATED.Num() > 0)
		{
			EditorOnly.LayerGuids = MoveTemp(LayerGuids_DEPRECATED);
		}
		if (LayerLinkStates_DEPRECATED.Num() > 0)
		{
			EditorOnly.LayerLinkStates = MoveTemp(LayerLinkStates_DEPRECATED);
		}
		if (DeletedParentLayerGuids_DEPRECATED.Num() > 0)
		{
			EditorOnly.DeletedParentLayerGuids = MoveTemp(DeletedParentLayerGuids_DEPRECATED);
		}

		const int32 NumLayers = Layers.Num();
		if (EditorOnly.LayerGuids.Num() != NumLayers ||
			EditorOnly.LayerLinkStates.Num() != NumLayers)
		{
			EditorOnly.LayerGuids.Empty(NumLayers);
			EditorOnly.LayerLinkStates.Empty(NumLayers);

			if (NumLayers > 0)
			{
				EditorOnly.LayerGuids.Add(BackgroundGuid);
				EditorOnly.LayerLinkStates.Add(EMaterialLayerLinkState::Uninitialized);

				for (int32 i = 1; i < NumLayers; ++i)
				{
					// Need to allocate deterministic guids for layers loaded from old data
					// These guids will be saved into any child material layers that have this material as their parent,
					// But it's possible *this* material may not actually be saved in that case
					// If that happens, need to ensure that the guids remain consistent if this material is loaded again;
					// otherwise they will no longer match the guids that were saved into the child material
					EditorOnly.LayerGuids.Add(FGuid(3u, 0u, 0u, i));
					EditorOnly.LayerLinkStates.Add(EMaterialLayerLinkState::Uninitialized);
				}
			}
		}
	} // Ar.IsLoading()

#endif // WITH_EDITORONLY_DATA	
}


#if WITH_EDITOR

void FMaterialLayersFunctions::AddDefaultBackgroundLayer()
{
	// Default to a non-blended "background" layer
	Layers.AddDefaulted();	
	EditorOnly.LayerStates.Add(true);
	FText LayerName = FText(LOCTEXT("Background", "Background"));
	EditorOnly.LayerNames.Add(LayerName);
	EditorOnly.RestrictToLayerRelatives.Add(false);
	// Use a consistent Guid for the background layer
	// Default constructor assigning different guids will break FStructUtils::AttemptToFindUninitializedScriptStructMembers
	EditorOnly.LayerGuids.Add(BackgroundGuid);
	EditorOnly.LayerLinkStates.Add(EMaterialLayerLinkState::NotFromParent);
}

int32 FMaterialLayersFunctions::AppendBlendedLayer()
{
	const int32 LayerIndex = Layers.AddDefaulted();
	Blends.AddDefaulted();

	EditorOnly.LayerStates.Add(true);
	FText LayerName = FText::Format(LOCTEXT("LayerPrefix", "Layer {0}"), Layers.Num() - 1);
	EditorOnly.LayerNames.Add(LayerName);
	EditorOnly.RestrictToLayerRelatives.Add(false);
	EditorOnly.RestrictToBlendRelatives.Add(false);
	CookDeterminism::LogWarningIfCooking(TEXT("AppendBlendedLayer creating NewGuid"));
	EditorOnly.LayerGuids.Add(FGuid::NewGuid());
	EditorOnly.LayerLinkStates.Add(EMaterialLayerLinkState::NotFromParent);

	return LayerIndex;
}

int32 FMaterialLayersFunctions::AddLayerCopy(const FMaterialLayersFunctionsRuntimeData& Source,
	const FMaterialLayersFunctionsEditorOnlyData& SourceEditorOnly,
	int32 SourceLayerIndex,
	bool bVisible,
	EMaterialLayerLinkState LinkState)
{
	check(LinkState != EMaterialLayerLinkState::Uninitialized);
	const int32 LayerIndex = Layers.Num();

	Layers.Add(Source.Layers[SourceLayerIndex]);

	if (LayerIndex > 0)
	{
		Blends.Add(Source.Blends[SourceLayerIndex - 1]);
	}

	EditorOnly.LayerStates.Add(bVisible);
	EditorOnly.LayerNames.Add(SourceEditorOnly.LayerNames[SourceLayerIndex]);
	EditorOnly.RestrictToLayerRelatives.Add(SourceEditorOnly.RestrictToLayerRelatives[SourceLayerIndex]);

	if (LayerIndex > 0)
	{
		EditorOnly.RestrictToBlendRelatives.Add(SourceEditorOnly.RestrictToBlendRelatives[SourceLayerIndex - 1]);
	}

	EditorOnly.LayerGuids.Add(SourceEditorOnly.LayerGuids[SourceLayerIndex]);
	EditorOnly.LayerLinkStates.Add(LinkState);

	return LayerIndex;
}

void FMaterialLayersFunctions::InsertLayerCopy(const FMaterialLayersFunctionsRuntimeData& Source,
	const FMaterialLayersFunctionsEditorOnlyData& SourceEditorOnly,
	int32 SourceLayerIndex,
	EMaterialLayerLinkState LinkState,
	int32 LayerIndex)
{
	check(LinkState != EMaterialLayerLinkState::Uninitialized);
	check(LayerIndex > 0);
	Layers.Insert(Source.Layers[SourceLayerIndex], LayerIndex);
	Blends.Insert(Source.Blends[SourceLayerIndex - 1], LayerIndex - 1);
	EditorOnly.RestrictToBlendRelatives.Insert(SourceEditorOnly.RestrictToBlendRelatives[SourceLayerIndex - 1], LayerIndex - 1);
	EditorOnly.LayerStates.Insert(SourceEditorOnly.LayerStates[SourceLayerIndex], LayerIndex);
	EditorOnly.LayerNames.Insert(SourceEditorOnly.LayerNames[SourceLayerIndex], LayerIndex);
	EditorOnly.RestrictToLayerRelatives.Insert(SourceEditorOnly.RestrictToLayerRelatives[SourceLayerIndex], LayerIndex);
	EditorOnly.LayerGuids.Insert(SourceEditorOnly.LayerGuids[SourceLayerIndex], LayerIndex);
	EditorOnly.LayerLinkStates.Insert(LinkState, LayerIndex);
}

void FMaterialLayersFunctions::RemoveBlendedLayerAt(int32 Index)
{
	if (Layers.IsValidIndex(Index))
	{
		check(Layers.IsValidIndex(Index) && Blends.IsValidIndex(Index - 1));
		Layers.RemoveAt(Index);
		Blends.RemoveAt(Index - 1);
		
		check(EditorOnly.LayerStates.IsValidIndex(Index) &&
			EditorOnly.LayerNames.IsValidIndex(Index) &&
			EditorOnly.RestrictToLayerRelatives.IsValidIndex(Index) &&
			EditorOnly.RestrictToBlendRelatives.IsValidIndex(Index - 1));

		EditorOnly.RestrictToBlendRelatives.RemoveAt(Index - 1);

		if (EditorOnly.LayerLinkStates[Index] != EMaterialLayerLinkState::NotFromParent)
		{
			// Save the parent guid as explicitly deleted, so it's not added back
			const FGuid& LayerGuid = EditorOnly.LayerGuids[Index];
			check(!EditorOnly.DeletedParentLayerGuids.Contains(LayerGuid));
			EditorOnly.DeletedParentLayerGuids.Add(LayerGuid);
		}

		EditorOnly.LayerStates.RemoveAt(Index);
		EditorOnly.LayerNames.RemoveAt(Index);
		EditorOnly.RestrictToLayerRelatives.RemoveAt(Index);
		EditorOnly.LayerGuids.RemoveAt(Index);
		EditorOnly.LayerLinkStates.RemoveAt(Index);
	}
}

void FMaterialLayersFunctions::MoveBlendedLayer(int32 SrcLayerIndex, int32 DstLayerIndex)
{
	check(SrcLayerIndex > 0);
	check(DstLayerIndex > 0);
	if (SrcLayerIndex != DstLayerIndex)
	{
		Layers.Swap(SrcLayerIndex, DstLayerIndex);
		Blends.Swap(SrcLayerIndex - 1, DstLayerIndex - 1);
		EditorOnly.RestrictToBlendRelatives.Swap(SrcLayerIndex - 1, DstLayerIndex - 1);
		EditorOnly.LayerStates.Swap(SrcLayerIndex, DstLayerIndex);
		EditorOnly.LayerNames.Swap(SrcLayerIndex, DstLayerIndex);
		EditorOnly.RestrictToLayerRelatives.Swap(SrcLayerIndex, DstLayerIndex);
		EditorOnly.LayerGuids.Swap(SrcLayerIndex, DstLayerIndex);
		EditorOnly.LayerLinkStates.Swap(SrcLayerIndex, DstLayerIndex);
	}
}

void FMaterialLayersFunctions::UnlinkLayerFromParent(int32 Index)
{
	if (EditorOnly.LayerLinkStates[Index] == EMaterialLayerLinkState::LinkedToParent)
	{
		EditorOnly.LayerLinkStates[Index] = EMaterialLayerLinkState::UnlinkedFromParent;
	}
}

bool FMaterialLayersFunctions::IsLayerLinkedToParent(int32 Index) const
{
	if (EditorOnly.LayerLinkStates.IsValidIndex(Index))
	{
		return EditorOnly.LayerLinkStates[Index] == EMaterialLayerLinkState::LinkedToParent;
	}
	return false;
}

void FMaterialLayersFunctions::RelinkLayersToParent()
{
	for (int32 Index = 0; Index < EditorOnly.LayerLinkStates.Num(); ++Index)
	{
		if (EditorOnly.LayerLinkStates[Index] == EMaterialLayerLinkState::UnlinkedFromParent)
		{
			EditorOnly.LayerLinkStates[Index] = EMaterialLayerLinkState::LinkedToParent;
		}
	}
	EditorOnly.DeletedParentLayerGuids.Empty();
}

bool FMaterialLayersFunctions::HasAnyUnlinkedLayers() const
{
	if (EditorOnly.DeletedParentLayerGuids.Num() > 0)
	{
		return true;
	}
	for (int32 Index = 0; Index < EditorOnly.LayerLinkStates.Num(); ++Index)
	{
		if (EditorOnly.LayerLinkStates[Index] == EMaterialLayerLinkState::UnlinkedFromParent)
		{
			return true;
		}
	}
	return false;
}

void FMaterialLayersFunctionsEditorOnlyData::LinkAllLayersToParent()
{
	for (int32 Index = 0; Index < LayerLinkStates.Num(); ++Index)
	{
		LayerLinkStates[Index] = EMaterialLayerLinkState::LinkedToParent;
	}
}

bool FMaterialLayersFunctions::MatchesParent(const FMaterialLayersFunctionsRuntimeData& Runtime,
	const FMaterialLayersFunctionsEditorOnlyData& EditorOnly,
	const FMaterialLayersFunctionsRuntimeData& ParentRuntime,
	const FMaterialLayersFunctionsEditorOnlyData& ParentEditorOnly)
{
	if (Runtime.Layers.Num() != ParentRuntime.Layers.Num())
	{
		return false;
	}

	for (int32 LayerIndex = 0; LayerIndex < Runtime.Layers.Num(); ++LayerIndex)
	{
		const EMaterialLayerLinkState LinkState = EditorOnly.LayerLinkStates[LayerIndex];
		if (LinkState != EMaterialLayerLinkState::LinkedToParent)
		{
			return false;
		}

		const FGuid& LayerGuid = EditorOnly.LayerGuids[LayerIndex];
		const int32 ParentLayerIndex = ParentEditorOnly.LayerGuids.Find(LayerGuid);
		if (ParentLayerIndex != LayerIndex)
		{
			return false;
		}

		// Possible for LayerStates arrays to be empty, if this is cooked data
		// We assume all cooked layers are visible
		const bool bLayerVisible = (EditorOnly.LayerStates.Num() > 0) ? EditorOnly.LayerStates[LayerIndex] : true;
		const bool bParentLayerVisible = (ParentEditorOnly.LayerStates.Num() > 0) ? ParentEditorOnly.LayerStates[ParentLayerIndex] : true;
		if (bLayerVisible != bParentLayerVisible)
		{
			return false;
		}
		if (Runtime.Layers[LayerIndex] != ParentRuntime.Layers[ParentLayerIndex])
		{
			return false;
		}

		if (LayerIndex > 0 && Runtime.Blends[LayerIndex - 1] == ParentRuntime.Blends[ParentLayerIndex - 1])
		{
			return false;
		}
	}

	return true;
}

bool FMaterialLayersFunctions::ResolveParent(const FMaterialLayersFunctionsRuntimeData& ParentRuntime,
	const FMaterialLayersFunctionsEditorOnlyData& ParentEditorOnly,
	FMaterialLayersFunctionsRuntimeData& Runtime,
	FMaterialLayersFunctionsEditorOnlyData& EditorOnly,
	TArray<int32>& OutRemapLayerIndices)
{
 	check(EditorOnly.LayerGuids.Num() == Runtime.Layers.Num());
	check(EditorOnly.LayerLinkStates.Num() == Runtime.Layers.Num());

	FMaterialLayersFunctions ResolvedLayers;
	TArray<int32> ParentLayerIndices;

	ResolvedLayers.Empty();
	
	//Ensure we retain the layer stack cache before we do anything else (only applies for LayerStack node usage, otherwise null and inconsequential).
	ResolvedLayers.LayerStackCache = ParentRuntime.LayerStackCache;

	bool bHasUninitializedLinks = false;
	for (int32 LayerIndex = 0; LayerIndex < Runtime.Layers.Num(); ++LayerIndex)
	{
		const FText& LayerName = EditorOnly.LayerNames[LayerIndex];
		const FGuid& LayerGuid = EditorOnly.LayerGuids[LayerIndex];
		const bool bLayerVisible = EditorOnly.LayerStates[LayerIndex];
		const EMaterialLayerLinkState LinkState = EditorOnly.LayerLinkStates[LayerIndex];

		int32 ParentLayerIndex = INDEX_NONE;
		if (LinkState == EMaterialLayerLinkState::Uninitialized)
		{
			bHasUninitializedLinks = true;
			if (LayerIndex == 0)
			{
				// Base layer must match against base layer
				if (ParentRuntime.Layers.Num() > 0)
				{
					ParentLayerIndex = 0;
				}
			}
			else
			{
				for (int32 CheckLayerIndex = 1; CheckLayerIndex < ParentRuntime.Layers.Num(); ++CheckLayerIndex)
				{
					// check if name matches, and if we haven't already linked to this parent layer
					if (LayerName.EqualTo(ParentEditorOnly.LayerNames[CheckLayerIndex]) &&
						!ParentLayerIndices.Contains(CheckLayerIndex))
					{
						ParentLayerIndex = CheckLayerIndex;
						break;
					}
				}
			}

			int32 ResolvedLayerIndex = INDEX_NONE;
			if (ParentLayerIndex == INDEX_NONE)
			{
				// Didn't find layer in the parent, assume it's local to this material
				ResolvedLayerIndex = ResolvedLayers.AddLayerCopy(Runtime, EditorOnly, LayerIndex, bLayerVisible, EMaterialLayerLinkState::NotFromParent);
				ParentLayerIndices.Add(INDEX_NONE);
			}
			else
			{
				// See if we match layer in parent
				if (Runtime.Layers[LayerIndex] == ParentRuntime.Layers[ParentLayerIndex]
					&& (LayerIndex == 0 || Runtime.Blends[LayerIndex - 1] == ParentRuntime.Blends[ParentLayerIndex - 1]))
				{
					// Parent layer matches, so link to parent
					ResolvedLayerIndex = ResolvedLayers.AddLayerCopy(ParentRuntime, ParentEditorOnly, ParentLayerIndex, bLayerVisible, EMaterialLayerLinkState::LinkedToParent);
				}
				else
				{
					// Parent layer does NOT match, so make the child overriden
					ResolvedLayerIndex = ResolvedLayers.AddLayerCopy(Runtime, EditorOnly, LayerIndex, bLayerVisible, EMaterialLayerLinkState::UnlinkedFromParent);
					ResolvedLayers.EditorOnly.LayerGuids[ResolvedLayerIndex] = ParentEditorOnly.LayerGuids[ParentLayerIndex]; // Still need to match guid to parent
				}


				check(!ParentLayerIndices.Contains(ParentLayerIndex));
				ParentLayerIndices.Add(ParentLayerIndex);
			}

			// If link state is Uninitialized, we *always* need to accept the layer in some way, otherwise we risk changing legacy data when loading in new engine
			check(ResolvedLayerIndex != INDEX_NONE);
		}
		else if (LinkState == EMaterialLayerLinkState::LinkedToParent)
		{
			check(LayerGuid.IsValid());
			ParentLayerIndex = ParentEditorOnly.LayerGuids.Find(LayerGuid);
			if (ParentLayerIndex != INDEX_NONE)
			{
				// Layer comes from parent
				ResolvedLayers.AddLayerCopy(ParentRuntime, ParentEditorOnly, ParentLayerIndex, bLayerVisible, EMaterialLayerLinkState::LinkedToParent);
				check(!ParentLayerIndices.Contains(ParentLayerIndex));
				ParentLayerIndices.Add(ParentLayerIndex);
			}
			// if we didn't find the layer in the parent, that means it was deleted from parent...so it's also deleted in the child
		}
		else
		{
			// layer not connected to parent
			check(LayerGuid.IsValid());
			check(LinkState == EMaterialLayerLinkState::UnlinkedFromParent || LinkState == EMaterialLayerLinkState::NotFromParent);

			if (LinkState == EMaterialLayerLinkState::UnlinkedFromParent)
			{
				// If we are unlinked from parent, track the layer index we were previously linked to
				ParentLayerIndex = ParentEditorOnly.LayerGuids.Find(LayerGuid);
			}
			check(ParentLayerIndex == INDEX_NONE || !ParentLayerIndices.Contains(ParentLayerIndex));

			// Update the link state, depending on if we can find this layer in the parent
			ResolvedLayers.AddLayerCopy(Runtime, EditorOnly, LayerIndex, bLayerVisible, (ParentLayerIndex == INDEX_NONE) ? EMaterialLayerLinkState::NotFromParent : EMaterialLayerLinkState::UnlinkedFromParent);
 			ParentLayerIndices.Add(ParentLayerIndex);
		}
	}

	check(ResolvedLayers.Layers.Num() == ParentLayerIndices.Num());

	// See if parent has any added layers
	for (int32 ParentLayerIndex = 1; ParentLayerIndex < ParentRuntime.Layers.Num(); ++ParentLayerIndex)
	{
		if (ParentLayerIndices.Contains(ParentLayerIndex))
		{
			// We already linked this layer to an existing child layer
			continue;
		}

		const FGuid& ParentLayerGuid = ParentEditorOnly.LayerGuids[ParentLayerIndex];
		if (EditorOnly.DeletedParentLayerGuids.Contains(ParentLayerGuid))
		{
			// Parent layer was previously explicitly overiden/deleted
			ResolvedLayers.EditorOnly.DeletedParentLayerGuids.Add(ParentLayerGuid);
			continue;
		}

		if (bHasUninitializedLinks)
		{
			// If we had any unitialized links, this means we're loading data saved by a previous version of UE
			// In this case, we have no way of determining if this layer was added to parent (and should therefore be kept),
			// or if this layer was explicitly deleted from child (and should therefore remain deleted).
			// In order to avoid needlessly changing legacy materials, we assume the layer was explicitly deleted, so we keep it deleted here
			// If desired, the relink functionality in the editor should allow it to be brought back
			ResolvedLayers.EditorOnly.DeletedParentLayerGuids.Add(ParentLayerGuid);
			continue;
		}

		// Find the layer before the newly inserted layer...we insert the new layer in the child at this same position
		int32 InsertLayerIndex = INDEX_NONE;
		{
			int32 CheckLayerIndex = ParentLayerIndex;
			while (InsertLayerIndex == INDEX_NONE)
			{
				--CheckLayerIndex;
				if (CheckLayerIndex == 0)
				{
					InsertLayerIndex = 0;
				}
				else
				{
					InsertLayerIndex = ParentLayerIndices.Find(CheckLayerIndex);
				}
			}
		}

		ParentLayerIndices.Insert(ParentLayerIndex, InsertLayerIndex + 1);
		ResolvedLayers.InsertLayerCopy(ParentRuntime, ParentEditorOnly, ParentLayerIndex, EMaterialLayerLinkState::LinkedToParent, InsertLayerIndex + 1);
	}

	bool bUpdatedLayerIndices = Runtime.Layers.Num() != ResolvedLayers.Layers.Num() ||
		EditorOnly.DeletedParentLayerGuids.Num() != ResolvedLayers.EditorOnly.DeletedParentLayerGuids.Num();

	OutRemapLayerIndices.SetNumUninitialized(Runtime.Layers.Num());
	for (int32 PrevLayerIndex = 0; PrevLayerIndex < Runtime.Layers.Num(); ++PrevLayerIndex)
	{
		const FGuid& LayerGuid = EditorOnly.LayerGuids[PrevLayerIndex];
		const int32 ResolvedLayerIndex = ResolvedLayers.EditorOnly.LayerGuids.Find(LayerGuid);
		OutRemapLayerIndices[PrevLayerIndex] = ResolvedLayerIndex;

		if (PrevLayerIndex != ResolvedLayerIndex)
		{
			bUpdatedLayerIndices = true;
		}
	}


	Runtime = MoveTemp(static_cast<FMaterialLayersFunctionsRuntimeData&>(ResolvedLayers));
	EditorOnly = MoveTemp(ResolvedLayers.EditorOnly);


	return bUpdatedLayerIndices;
}

void FMaterialLayersFunctions::Validate(const FMaterialLayersFunctionsRuntimeData& Runtime, const FMaterialLayersFunctionsEditorOnlyData& EditorOnly)
{
	if (Runtime.Layers.Num() > 0)
	{
		check(Runtime.Blends.Num() == Runtime.Layers.Num() - 1);
		check(Runtime.Layers.Num() == EditorOnly.LayerStates.Num());
		check(Runtime.Layers.Num() == EditorOnly.LayerNames.Num());
		check(Runtime.Layers.Num() == EditorOnly.LayerGuids.Num());
		check(Runtime.Layers.Num() == EditorOnly.LayerLinkStates.Num());
	}
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionMaterialFunctionCall
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionMaterialFunctionCall::UMaterialExpressionMaterialFunctionCall(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	bHidePreviewWindow = true;

	// Function calls created without a function should be pinless by default
	FunctionInputs.Empty();
	FunctionOutputs.Empty();
	Outputs.Empty();
#endif
}

void UMaterialExpressionMaterialFunctionCall::PostLoad()
{
	if (MaterialFunction)
	{
		MaterialFunction->ConditionalPreload();
		MaterialFunction->ConditionalPostLoad();
	}

	Super::PostLoad();
}

#if WITH_EDITORONLY_DATA
bool UMaterialExpressionMaterialFunctionCall::IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
{
	if (UMaterialFunctionInterface* MaterialFunctionInterface = MaterialFunction)
	{
		if (!MaterialFunctionInterface->IterateDependentFunctions(Predicate))
		{
			return false;
		}
		if (!Predicate(MaterialFunctionInterface))
		{
			return false;
		}
	}
	return true;
}

void UMaterialExpressionMaterialFunctionCall::GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
{
	IterateDependentFunctions([&DependentFunctions](UMaterialFunctionInterface* InMaterialFunction) -> bool
	{
		DependentFunctions.AddUnique(InMaterialFunction);
		return true;
	});
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void UMaterialExpressionMaterialFunctionCall::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MaterialFunction))
	{
		// Save off the previous MaterialFunction value
		SavedMaterialFunction = MaterialFunction;
	}
	Super::PreEditChange(PropertyAboutToChange);
}

void UMaterialExpressionMaterialFunctionCall::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, MaterialFunction))
	{
		// Set the new material function
		SetMaterialFunctionEx(SavedMaterialFunction, MaterialFunction);
		SavedMaterialFunction = nullptr;
	}
	else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Redirected)
	{
		// Refresh from the current material function as it may have been redirected to a different value
		UpdateFromFunctionResource();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionMaterialFunctionCall::LinkFunctionIntoCaller(FMaterialCompiler* Compiler)
{
	MaterialFunction->LinkIntoCaller(FunctionInputs);
	PushParameterOwner(Compiler);
}

void UMaterialExpressionMaterialFunctionCall::UnlinkFunctionFromCaller(FMaterialCompiler* Compiler)
{
	PopParameterOwner(Compiler);
	MaterialFunction->UnlinkFromCaller();
}

void UMaterialExpressionMaterialFunctionCall::PushParameterOwner(FMaterialCompiler* Compiler)
{
	// Update parameter owner when stepping into layer functions.
	// This is an optional step when we only want to march the material graph (e.g. to gather Substrate material topology)
	if (Compiler && FunctionParameterInfo.Association != EMaterialParameterAssociation::GlobalParameter)
	{
		Compiler->PushParameterOwner(FunctionParameterInfo);
	}
}

void UMaterialExpressionMaterialFunctionCall::PopParameterOwner(FMaterialCompiler* Compiler)
{
	if (Compiler && FunctionParameterInfo.Association != EMaterialParameterAssociation::GlobalParameter)
	{
		const FMaterialParameterInfo PoppedParameterInfo = Compiler->PopParameterOwner();
		check(PoppedParameterInfo == FunctionParameterInfo);
	}
}

int32 UMaterialExpressionMaterialFunctionCall::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!MaterialFunction)
	{
		return Compiler->Errorf(TEXT("Missing Material Function"));
	}

	// Verify that all function inputs and outputs are in a valid state to be linked into this material for compiling
	for (int32 i = 0; i < FunctionInputs.Num(); i++)
	{
		if (!FunctionInputs[i].ExpressionInput)
		{
			return Compiler->Errorf(TEXT("Function (%s) call input with index %d is unset."), *MaterialFunction->GetPathName(), i);
		}
	}

	for (int32 i = 0; i < FunctionOutputs.Num(); i++)
	{
		if (!FunctionOutputs[i].ExpressionOutput)
		{
			return Compiler->Errorf(TEXT("Function (%s) call output with index %d is unset."), *MaterialFunction->GetPathName(), i);
		}
	}

	if (!FunctionOutputs.IsValidIndex(OutputIndex))
	{
		return Compiler->Errorf(TEXT("Invalid function (%s) output"), *MaterialFunction->GetPathName());
	}

	// Link the function's inputs into the caller graph before entering
	LinkFunctionIntoCaller(Compiler);

	// Some functions (e.g. layers) don't benefit from re-using state so we locally create one as we did before sharing was added
	FMaterialFunctionCompileState LocalState(this);

	// Tell the compiler that we are entering a function
	const int32 ExpressionStackCheckSize = SharedCompileState ? SharedCompileState->ExpressionStack.Num() : 0;
	Compiler->PushFunction(SharedCompileState ? SharedCompileState : &LocalState);

	// Compile the requested output
	const int32 ReturnValue = MaterialFunction->Compile(Compiler, FunctionOutputs[OutputIndex]);

	// Tell the compiler that we are leaving a function
	FMaterialFunctionCompileState* CompileState = Compiler->PopFunction();
	check(!SharedCompileState || CompileState->ExpressionStack.Num() == ExpressionStackCheckSize);

	// Restore the function since we are leaving it
	UnlinkFunctionFromCaller(Compiler);

	return ReturnValue;
}

void UMaterialExpressionMaterialFunctionCall::GetCaption(TArray<FString>& OutCaptions) const
{
	if (MaterialFunction)
	{
		FString UserExposedCaption = MaterialFunction->GetUserExposedCaption();
		if (!UserExposedCaption.IsEmpty())
		{
			OutCaptions.Add(UserExposedCaption);
		}
		else
		{
			OutCaptions.Add(MaterialFunction->GetName());
		}
	}
	else
	{
		OutCaptions.Add(TEXT("Unspecified Function"));
	}
}

TArrayView<FExpressionInput*> UMaterialExpressionMaterialFunctionCall::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(FunctionInputs.Num());
	for (int32 i = 0; i < FunctionInputs.Num(); i++)
	{
		CachedInputs.Add(&FunctionInputs[i].Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionMaterialFunctionCall::GetInput(int32 InputIndex)
{
	return FunctionInputs.IsValidIndex(InputIndex) ? &FunctionInputs[InputIndex].Input : nullptr;
}


static const TCHAR* GetInputTypeName(uint8 InputType)
{
	const static TCHAR* TypeNames[FunctionInput_MAX] =
	{
		TEXT("S"),
		TEXT("V2"),
		TEXT("V3"),
		TEXT("V4"),
		TEXT("T2d"),
		TEXT("TCube"),
		TEXT("T2dArr"),
		TEXT("TVol"),
		TEXT("SB"),
		TEXT("MA"),
		TEXT("TExt"),
		TEXT("B"),
		TEXT("Stra")
	};

	check(InputType < FunctionInput_MAX);
	return TypeNames[InputType];
}

FName UMaterialExpressionMaterialFunctionCall::GetInputNameWithType(int32 InputIndex, bool bWithType) const
{
	if (InputIndex < FunctionInputs.Num())
	{
		if (FunctionInputs[InputIndex].ExpressionInput != nullptr && bWithType)
		{
			return *FString::Printf(TEXT("%s (%s)"), *FunctionInputs[InputIndex].Input.InputName.ToString(), GetInputTypeName(FunctionInputs[InputIndex].ExpressionInput->InputType));
		}
		else
		{
			return FunctionInputs[InputIndex].Input.InputName;
		}
	}
	return NAME_None;
}

FName UMaterialExpressionMaterialFunctionCall::GetInputName(int32 InputIndex) const
{
	return GetInputNameWithType(InputIndex, true);
}

bool UMaterialExpressionMaterialFunctionCall::IsInputConnectionRequired(int32 InputIndex) const
{
	if (InputIndex < FunctionInputs.Num() && FunctionInputs[InputIndex].ExpressionInput != nullptr)
	{
		return !FunctionInputs[InputIndex].ExpressionInput->bUsePreviewValueAsDefault;
	}
	return true;
}

static FString GetInputDefaultValueString(EFunctionInputType InputType, const FVector4f& PreviewValue)
{
	static_assert(FunctionInput_Scalar < FunctionInput_Vector4, "Enum values out of order.");
	check(InputType <= FunctionInput_Vector4);

	FString ValueString = FString::Printf(TEXT("DefaultValue = (%.2f"), PreviewValue.X);
	
	if (InputType >= FunctionInput_Vector2)
	{
		ValueString += FString::Printf(TEXT(", %.2f"), PreviewValue.Y);
	}

	if (InputType >= FunctionInput_Vector3)
	{
		ValueString += FString::Printf(TEXT(", %.2f"), PreviewValue.Z);
	}

	if (InputType >= FunctionInput_Vector4)
	{
		ValueString += FString::Printf(TEXT(", %.2f"), PreviewValue.W);
	}

	return ValueString + TEXT(")");
}

FString UMaterialExpressionMaterialFunctionCall::GetDescription() const
{
	FString Result = FString(*GetClass()->GetName()).Mid(FCString::Strlen(TEXT("MaterialExpression")));
	Result += TEXT(" (");
	Result += Super::GetDescription();
	Result += TEXT(")");
	return Result;
}

void UMaterialExpressionMaterialFunctionCall::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) 
{
	if (MaterialFunction)
	{
		if (InputIndex != INDEX_NONE)
		{
			if (FunctionInputs.IsValidIndex(InputIndex))
			{
				UMaterialExpressionFunctionInput* InputExpression = FunctionInputs[InputIndex].ExpressionInput;
				if (InputExpression)
				{
					ConvertToMultilineToolTip(InputExpression->Description, 40, OutToolTip);
					if (InputExpression->bUsePreviewValueAsDefault)
					{
						// Can't build a tooltip of an arbitrary expression chain
						if (InputExpression->Preview.Expression)
						{
							OutToolTip.Insert(FString("DefaultValue = Custom expressions"), 0);

							// Add a line after the default value string
							OutToolTip.Insert(FString(), 1);
						}
						else if (InputExpression->InputType <= FunctionInput_Vector4)
						{
							// Add a string for the default value at the top
							OutToolTip.Insert(GetInputDefaultValueString((EFunctionInputType)InputExpression->InputType, InputExpression->PreviewValue), 0);

							// Add a line after the default value string
							OutToolTip.Insert(FString(), 1);
						}
					}
				}
			}
		}
		else if (OutputIndex != INDEX_NONE)
		{
			if (FunctionOutputs.IsValidIndex(OutputIndex))
			{
				UMaterialExpressionFunctionOutput* OutputExpression = FunctionOutputs[OutputIndex].ExpressionOutput;
				if (OutputExpression)
				{
					ConvertToMultilineToolTip(OutputExpression->Description, 40, OutToolTip);
				}
			}
		}
	}
}

void UMaterialExpressionMaterialFunctionCall::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	if (MaterialFunction)
	{
		const FString& Description = MaterialFunction->GetDescription();
		ConvertToMultilineToolTip(Description, 40, OutToolTip);
	}
}

bool UMaterialExpressionMaterialFunctionCall::SetMaterialFunction(UMaterialFunctionInterface* NewMaterialFunction)
{
	// Remember the current material function
	UMaterialFunctionInterface* OldFunction = MaterialFunction;

	return SetMaterialFunctionEx(OldFunction, NewMaterialFunction);
}


bool UMaterialExpressionMaterialFunctionCall::SetMaterialFunctionEx(
	UMaterialFunctionInterface* OldFunctionResource, 
	UMaterialFunctionInterface* NewFunctionResource)
{
	// See if Outer is another material function
	UMaterialFunctionInterface* ThisFunctionResource = Cast<UMaterialFunction>(GetOuter());

	if (NewFunctionResource 
		&& ThisFunctionResource
		&& NewFunctionResource->IsDependent(ThisFunctionResource))
	{
		// Prevent recursive function call graphs
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("MaterialExpressions", "Error_CircularFunctionDependency", "Can't use that material function as it would cause a circular dependency.") );
		NewFunctionResource = nullptr;
	}

	MaterialFunction = NewFunctionResource;

	// Store the original inputs and outputs
	TArray<FFunctionExpressionInput> OriginalInputs = FunctionInputs;
	TArray<FFunctionExpressionOutput> OriginalOutputs = FunctionOutputs;

	FunctionInputs.Empty();
	FunctionOutputs.Empty();
	Outputs.Empty();

	if (NewFunctionResource)
	{
		// Get the current inputs and outputs
		NewFunctionResource->GetInputsAndOutputs(FunctionInputs, FunctionOutputs);

		for (int32 InputIndex = 0; InputIndex < FunctionInputs.Num(); InputIndex++)
		{
			FFunctionExpressionInput& CurrentInput = FunctionInputs[InputIndex];
			check(CurrentInput.ExpressionInput);
			const FFunctionExpressionInput* OriginalInput = FindInputByName(CurrentInput.ExpressionInput->InputName, OriginalInputs);

			if (OriginalInput)
			{
				// If there is an input whose name matches the original input, even if they are from different functions, maintain the connection
				CurrentInput.Input = OriginalInput->Input;
			}
		}

		for (int32 OutputIndex = 0; OutputIndex < FunctionOutputs.Num(); OutputIndex++)
		{
			Outputs.Add(FunctionOutputs[OutputIndex].Output);
		}
	}

	// Fixup even if NewFunctionResource is nullptr, because we have to clear old connections
	if (OldFunctionResource && OldFunctionResource != NewFunctionResource)
	{
		TArray<FExpressionInput*> MaterialInputs;
		if (Material)
		{
			MaterialInputs.Empty(MP_MAX);
			for (int32 InputIndex = 0; InputIndex < MP_MAX; InputIndex++)
			{
				auto Input = Material->GetExpressionInputForProperty((EMaterialProperty)InputIndex);

				if(Input)
				{
					MaterialInputs.Add(Input);
				}
			}

			// Fixup any references that the material or material inputs had to the function's outputs, maintaining links with the same output name
			FixupReferencingExpressions(FunctionOutputs, OriginalOutputs, Material->GetExpressions(), MaterialInputs, true);
		}
		else if (Function)
		{
			// Fixup any references that the material function had to the function's outputs, maintaining links with the same output name
			FixupReferencingExpressions(FunctionOutputs, OriginalOutputs, Function->GetExpressions(), MaterialInputs, true);
		}
	}

	if (GraphNode)
	{
		// Recreate the pins of this node after material function set
		CastChecked<UMaterialGraphNode>(GraphNode)->RecreateAndLinkNode();
	}

	return NewFunctionResource != nullptr;
}

void UMaterialExpressionMaterialFunctionCall::UpdateFromFunctionResource(bool bRecreateAndLinkNode)
{
	TArray<FFunctionExpressionInput> OriginalInputs = MoveTemp(FunctionInputs);
	TArray<FFunctionExpressionOutput> OriginalOutputs = MoveTemp(FunctionOutputs);

	FunctionInputs.Reserve(OriginalInputs.Num());
	FunctionOutputs.Reserve(OriginalOutputs.Num());
	Outputs.Reset();

	if (MaterialFunction)
	{
		// Recursively update any function call nodes in the function
		MaterialFunction->UpdateFromFunctionResource();

		// Get the function's current inputs and outputs
		MaterialFunction->GetInputsAndOutputs(FunctionInputs, FunctionOutputs);

		for (int32 InputIndex = 0; InputIndex < FunctionInputs.Num(); InputIndex++)
		{
			FFunctionExpressionInput& CurrentInput = FunctionInputs[InputIndex];
			check(CurrentInput.ExpressionInput);
			const FFunctionExpressionInput* OriginalInput = FindInputById(CurrentInput.ExpressionInputId, OriginalInputs);

			if (OriginalInput)
			{
				// Maintain the input connection if an input with matching Id is found, but propagate the new name
				// This way function inputs names can be changed without affecting material connections
				const FName TempInputName = CurrentInput.Input.InputName;
				CurrentInput.Input = OriginalInput->Input;
				CurrentInput.Input.InputName = TempInputName;
			}
		}

		for (int32 OutputIndex = 0; OutputIndex < FunctionOutputs.Num(); OutputIndex++)
		{
			Outputs.Add(FunctionOutputs[OutputIndex].Output);
		}

		TArray<FExpressionInput*> MaterialInputs;
		if (Material)
		{
			MaterialInputs.Empty(MP_MAX);
			for (int32 InputIndex = 0; InputIndex < MP_MAX; InputIndex++)
			{
				auto Input = Material->GetExpressionInputForProperty((EMaterialProperty)InputIndex);

				if(Input)
				{
					MaterialInputs.Add(Input);
				}
			}
			
			// Fixup any references that the material or material inputs had to the function's outputs
			FixupReferencingExpressions(FunctionOutputs, OriginalOutputs, Material->GetExpressions(), MaterialInputs, false);
		}
		else if (Function)
		{
			// Fixup any references that the material function had to the function's outputs
			FixupReferencingExpressions(FunctionOutputs, OriginalOutputs, Function->GetExpressions(), MaterialInputs, false);
		}
	}

	if (GraphNode && bRecreateAndLinkNode)
	{
		// Check whether number of input/outputs or transient pointers have changed
		bool bUpdatedFromFunction = false;
		if (OriginalInputs.Num() != FunctionInputs.Num()
			|| OriginalOutputs.Num() != FunctionOutputs.Num()
			|| OriginalOutputs.Num() != Outputs.Num())
		{
			bUpdatedFromFunction = true;
		}
		for (int32 Index = 0; Index < OriginalInputs.Num() && !bUpdatedFromFunction; ++Index)
		{
			if (OriginalInputs[Index].ExpressionInput != FunctionInputs[Index].ExpressionInput)
			{
				bUpdatedFromFunction = true;
			}
		}
		for (int32 Index = 0; Index < OriginalOutputs.Num() && !bUpdatedFromFunction; ++Index)
		{
			if (OriginalOutputs[Index].ExpressionOutput != FunctionOutputs[Index].ExpressionOutput)
			{
				bUpdatedFromFunction = true;
			}
		}
		if (bUpdatedFromFunction)
		{
			// Recreate the pins of this node after Expression links are made
			CastChecked<UMaterialGraphNode>(GraphNode)->RecreateAndLinkNode();
		}
	}
}

/** Fixes CurrentInput's OutputIndex, or breaks the connection if necessary. */
static void FixupReferencingInput(
	FExpressionInput* CurrentInput,
	const TArray<FFunctionExpressionOutput>& NewOutputs,
	const TArray<FFunctionExpressionOutput>& OriginalOutputs,
	UMaterialExpressionMaterialFunctionCall* FunctionExpression,
	bool bMatchByName)
{
	if (CurrentInput->Expression == FunctionExpression)
	{
		if (OriginalOutputs.IsValidIndex(CurrentInput->OutputIndex))
		{
			if (bMatchByName)
			{
				if (OriginalOutputs[CurrentInput->OutputIndex].ExpressionOutput)
				{
					CurrentInput->OutputIndex = FindOutputIndexByName(OriginalOutputs[CurrentInput->OutputIndex].ExpressionOutput->OutputName, NewOutputs);
				}
			}
			else
			{
				const FGuid OutputId = OriginalOutputs[CurrentInput->OutputIndex].ExpressionOutputId;
				CurrentInput->OutputIndex = FindOutputIndexById(OutputId, NewOutputs);
			}

			if (CurrentInput->OutputIndex == INDEX_NONE)
			{
				// The output that this input was connected to no longer exists, break the connection
				CurrentInput->Expression = nullptr;
			}
		}
		else
		{
			// The output that this input was connected to no longer exists, break the connection
			CurrentInput->OutputIndex = INDEX_NONE;
			CurrentInput->Expression = nullptr;
		}
	}
}

void UMaterialExpressionMaterialFunctionCall::FixupReferencingExpressions(
	const TArray<FFunctionExpressionOutput>& NewOutputs,
	const TArray<FFunctionExpressionOutput>& OriginalOutputs,
	TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions,
	TArray<FExpressionInput*>& MaterialInputs,
	bool bMatchByName)
{
	for (int32 ExpressionIndex = 0; ExpressionIndex < Expressions.Num(); ExpressionIndex++)
	{
		UMaterialExpression* CurrentExpression = Expressions[ExpressionIndex];
		if (CurrentExpression)
		{
			for (FExpressionInputIterator It{ CurrentExpression}; It; ++It)
			{
				FixupReferencingInput(It.Input, NewOutputs, OriginalOutputs, this, bMatchByName);
			}
		}
	}

	for (FExpressionInput* CurrentInput : MaterialInputs)
	{
		FixupReferencingInput(CurrentInput, NewOutputs, OriginalOutputs, this, bMatchByName);
	}
}

bool UMaterialExpressionMaterialFunctionCall::MatchesSearchQuery( const TCHAR* SearchQuery )
{
	if (MaterialFunction && MaterialFunction->GetName().Contains(SearchQuery) )
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

bool UMaterialExpressionMaterialFunctionCall::DeepSearchQuery(const TCHAR* SearchQuery, TSharedPtr<FFindInMaterialResult> SearchResult)
{
	//First check to see if this expression matches, if not then inspect the subfunction(s)
	if (Super::DeepSearchQuery(SearchQuery, SearchResult))
	{
		return true;
	}
	else if (SearchResult.IsValid() && MaterialFunction)
	{
		for (UMaterialExpression* Expression : MaterialFunction->GetExpressions())
		{
			SearchResult->bEmbeddedViaNode = Expression->DeepSearchQuery(SearchQuery, SearchResult);
			if (SearchResult->bEmbeddedViaNode)
			{			
				return true;
			}
		}
	}
	return false;
}


bool UMaterialExpressionMaterialFunctionCall::IsResultMaterialAttributes(int32 OutputIndex) 
{
	if( OutputIndex >= 0 && OutputIndex < FunctionOutputs.Num() && FunctionOutputs[OutputIndex].ExpressionOutput)
	{
		return FunctionOutputs[OutputIndex].ExpressionOutput->IsResultMaterialAttributes(0);
	}
	else
	{
		return false;
	}
}

bool UMaterialExpressionMaterialFunctionCall::IsResultSubstrateMaterial(int32 OutputIndex)
{
	if( OutputIndex >= 0 && OutputIndex < FunctionOutputs.Num() && FunctionOutputs[OutputIndex].ExpressionOutput)
	{
		return FunctionOutputs[OutputIndex].ExpressionOutput->IsResultSubstrateMaterial(0);
	}
	else
	{
		return false;
	}
}

void UMaterialExpressionMaterialFunctionCall::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (OutputIndex >= 0 && OutputIndex < FunctionOutputs.Num() && FunctionOutputs[OutputIndex].ExpressionOutput)
	{
		this->LinkFunctionIntoCaller(nullptr);
		FunctionOutputs[OutputIndex].ExpressionOutput->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, 0);
		this->UnlinkFunctionFromCaller(nullptr);
		return;
	}
}

FSubstrateOperator* UMaterialExpressionMaterialFunctionCall::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	if (OutputIndex >= 0 && OutputIndex < FunctionOutputs.Num() && FunctionOutputs[OutputIndex].ExpressionOutput)
	{
		this->LinkFunctionIntoCaller(nullptr);
		FMaterialFunctionCompileState LocalState(this);
		SubstrateTranslatorData.PushFunction(SharedCompileState ? SharedCompileState : &LocalState);

		FSubstrateOperator* ResultingOperator = FunctionOutputs[OutputIndex].ExpressionOutput->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, 0);

		SubstrateTranslatorData.PopFunction();
		this->UnlinkFunctionFromCaller(nullptr);
		return ResultingOperator;
	}
	return nullptr;
}

EMaterialValueType UMaterialExpressionMaterialFunctionCall::GetInputValueType(int32 InputIndex)
{
	if (InputIndex < FunctionInputs.Num())
	{
		if (FunctionInputs[InputIndex].ExpressionInput)
		{
			return FunctionInputs[InputIndex].ExpressionInput->GetInputValueType(0);
		}
	}
	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionMaterialFunctionCall::GetOutputValueType(int32 OutputIndex)
{
	if (OutputIndex >= 0 && OutputIndex < FunctionOutputs.Num() && FunctionOutputs[OutputIndex].ExpressionOutput)
	{
		return FunctionOutputs[OutputIndex].ExpressionOutput->GetOutputValueType(0);
	}
	return Super::GetOutputValueType(OutputIndex);
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionFunctionInput
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionFunctionInput::UMaterialExpressionFunctionInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InputType = FunctionInput_Vector3;
	InputName = TEXT("In");

#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

void UMaterialExpressionFunctionInput::PostLoad()
{
	Super::PostLoad();
	ConditionallyGenerateId(false);
}

void UMaterialExpressionFunctionInput::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	ConditionallyGenerateId(false);
}

#if WITH_EDITOR
FExpressionInput* UMaterialExpressionFunctionInput::AddNewEffectivePreviewDuringCompile(FExpressionInput& InEffectivePreview)
{
	PushEffectivePreviewDuringCompile(new FExpressionInput(InEffectivePreview));
	return EffectivePreviewDuringCompile.Last();
}

void UMaterialExpressionFunctionInput::PushEffectivePreviewDuringCompile(FExpressionInput* InEffectivePreview)
{
	EffectivePreviewDuringCompile.Push(InEffectivePreview);
}

FExpressionInput* UMaterialExpressionFunctionInput::PopEffectivePreviewDuringCompile()
{
	return EffectivePreviewDuringCompile.IsEmpty() ? nullptr : EffectivePreviewDuringCompile.Pop();
}

void UMaterialExpressionFunctionInput::RemoveLastEffectivePreviewDuringCompile()
{
	FExpressionInput* PoppedInput = PopEffectivePreviewDuringCompile();
	if (PoppedInput)
	{
		delete PoppedInput;
	}
}

void UMaterialExpressionFunctionInput::PostEditImport()
{
	Super::PostEditImport();
	ConditionallyGenerateId(true);
}

void UMaterialExpressionFunctionInput::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialExpressionFunctionInput, InputName))
	{
		InputNameBackup = InputName;
	}
	Super::PreEditChange(PropertyAboutToChange);
}

void UMaterialExpressionFunctionInput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, InputName))
	{
		if (Material)
		{
			for (UMaterialExpression* Expression : Material->GetExpressions())
			{
				UMaterialExpressionFunctionInput* OtherFunctionInput = Cast<UMaterialExpressionFunctionInput>(Expression);
				if (OtherFunctionInput && OtherFunctionInput != this && OtherFunctionInput->InputName == InputName)
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_InputNamesMustBeUnique", "Function input names must be unique"));
					InputName = InputNameBackup;
					break;
				}
			}
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
}

void UMaterialExpressionFunctionInput::GetCaption(TArray<FString>& OutCaptions) const
{
	const static TCHAR* TypeNames[FunctionInput_MAX] =
	{
		TEXT("Scalar"),
		TEXT("Vector2"),
		TEXT("Vector3"),
		TEXT("Vector4"),
		TEXT("Texture2D"),
		TEXT("TextureCube"),
		TEXT("Texture2DArray"),
		TEXT("VolumeTexture"),
		TEXT("StaticBool"),
		TEXT("MaterialAttributes"),
		TEXT("External"),
		TEXT("Bool"),
		TEXT("Substrate")
	};
	check(InputType < FunctionInput_MAX);
	OutCaptions.Add(FString("Input ") + InputName.ToString() + TEXT(" (") + TypeNames[InputType] + TEXT(")"));
}

void UMaterialExpressionFunctionInput::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(Description, 40, OutToolTip);
}

int32 UMaterialExpressionFunctionInput::CompilePreviewValue(FMaterialCompiler* Compiler)
{
	if (Preview.GetTracedInput().Expression)
	{
		int32 ExpressionResult;
		if (Preview.Expression->GetOuter() == GetOuter())
		{
			ExpressionResult = Preview.Compile(Compiler);
		}
		else
		{
			FMaterialFunctionCompileState* FunctionState = Compiler->PopFunction();
			ExpressionResult = Preview.Compile(Compiler);
			Compiler->PushFunction(FunctionState);
		}
		return ExpressionResult;
	}
	else
	{
		const FGuid AttributeID = Compiler->GetMaterialAttribute();

		// Compile PreviewValue if Preview was not connected
		switch (InputType)
		{
		case FunctionInput_Scalar:
			return Compiler->Constant(PreviewValue.X);
		case FunctionInput_Vector2:
			return Compiler->Constant2(PreviewValue.X, PreviewValue.Y);
		case FunctionInput_Vector3:
			return Compiler->Constant3(PreviewValue.X, PreviewValue.Y, PreviewValue.Z);
		case FunctionInput_Vector4:
			return Compiler->Constant4(PreviewValue.X, PreviewValue.Y, PreviewValue.Z, PreviewValue.W);
		case FunctionInput_MaterialAttributes:
		{
			if (AttributeID == FMaterialAttributeDefinitionMap::GetID(MP_EmissiveColor))
			{
				return Compiler->Constant3(PreviewValue.X, PreviewValue.Y, PreviewValue.Z);
			}

			if (!Substrate::IsSubstrateEnabled() || AttributeID != FMaterialAttributeDefinitionMap::GetID(MP_FrontMaterial))
			{
				return FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, AttributeID);
			}
		}
		case FunctionInput_Substrate:
		{
			return UMaterialExpressionSubstrateConvertMaterialAttributes::CompileDefaultSlab(Compiler, FVector3f(PreviewValue.X, PreviewValue.Y, PreviewValue.Z));
		}
		case FunctionInput_Texture2D:
		case FunctionInput_TextureCube:
		case FunctionInput_Texture2DArray:
		case FunctionInput_TextureExternal:
		case FunctionInput_StaticBool:
		case FunctionInput_Bool:
			return Compiler->Errorf(TEXT("Missing Preview connection for function input '%s'"), *InputName.ToString());
		default:
			return Compiler->Errorf(TEXT("Unknown input type"));
		}
	}
}

int32 UMaterialExpressionFunctionInput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const static EMaterialValueType FunctionTypeMapping[FunctionInput_MAX] =
	{
		MCT_Float1,
		MCT_Float2,
		MCT_Float3,
		MCT_Float4,
		MCT_Texture2D,
		MCT_TextureCube,
		MCT_Texture2DArray,
		MCT_VolumeTexture,
		MCT_StaticBool,
		MCT_MaterialAttributes,
		MCT_TextureExternal,
		MCT_Bool,
		MCT_Substrate
	};
	check(InputType < FunctionInput_MAX);

	// If we are being compiled as part of a material which calls this function
	FExpressionInput* LocalPreviewDuringCompile = PopEffectivePreviewDuringCompile();
	int32 ExpressionResult = INDEX_NONE;
	if (LocalPreviewDuringCompile && LocalPreviewDuringCompile->GetTracedInput().Expression)
	{
		FExpressionInput EffectivePreviewDuringCompileTracedInput = LocalPreviewDuringCompile->GetTracedInput();
		// Stay in this function if we are compiling an expression that is in the current function
		// This can happen if bUsePreviewValueAsDefault is true and the calling material didn't override the input
		if (bUsePreviewValueAsDefault && EffectivePreviewDuringCompileTracedInput.Expression->GetOuter() == GetOuter())
		{
			// Compile the function input
			ExpressionResult = EffectivePreviewDuringCompileTracedInput.Compile(Compiler);
		}
		else
		{
			// Tell the compiler that we are leaving the function
			FMaterialFunctionCompileState* FunctionState = Compiler->PopFunction();

			// Restore the function since we are leaving it
			FunctionState->FunctionCall->PopParameterOwner(Compiler);

			// Compile the function input
			ExpressionResult = EffectivePreviewDuringCompileTracedInput.Compile(Compiler);

			// Link the function's inputs into the caller graph before entering
			FunctionState->FunctionCall->PushParameterOwner(Compiler);

			// Tell the compiler that we are re-entering the function
			Compiler->PushFunction(FunctionState);
		}
	}
	else
	{
		if (!LocalPreviewDuringCompile || bUsePreviewValueAsDefault)
		{
			// If we are compiling the function in a preview material, such as when editing the function,
			// Compile the preview value or texture and output a texture object.
			ExpressionResult = CompilePreviewValue(Compiler);
		}
		else
		{
			ExpressionResult = Compiler->Errorf(TEXT("Missing function input '%s'"), *InputName.ToString());
		}
	}

	if (LocalPreviewDuringCompile)
	{
		PushEffectivePreviewDuringCompile(LocalPreviewDuringCompile);
	}

	if (ExpressionResult != INDEX_NONE)
	{
		// Cast to the type that the function author specified
		// This will truncate (float4 -> float3) but not add components (float2 -> float3)
		// Don't change the LWC status of the type
		EMaterialValueType ResultType = FunctionTypeMapping[InputType];
		if (IsLWCType(Compiler->GetParameterType(ExpressionResult)))
		{
			ResultType = MakeLWCType(ResultType);
		}

		ExpressionResult = Compiler->ValidCast(ExpressionResult, ResultType);
	}
	return ExpressionResult;
}

int32 UMaterialExpressionFunctionInput::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (InputType == FunctionInput_Substrate)
	{
		// Compile the SubstrateData output.
		int32 SubstrateDataCodeChunk = Compile(Compiler, OutputIndex);
		// Convert the SubstrateData to a preview color.
		int32 PreviewCodeChunk = Compiler->SubstrateCompilePreview(SubstrateDataCodeChunk);
		return PreviewCodeChunk;
	}

	// Compile the preview value, outputting a float type
	return Compiler->ValidCast(CompilePreviewValue(Compiler), MCT_Float3);
}
#endif // WITH_EDITOR

void UMaterialExpressionFunctionInput::ConditionallyGenerateId(bool bForce)
{
	if (bForce || !Id.IsValid())
	{
		Id = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::FunctionInput);
	}
}

FString UMaterialExpressionFunctionInput::GetInputTypeDisplayName(EFunctionInputType InputType)
{
	static const UEnum* FunctionInputEnum = StaticEnum<EFunctionInputType>();
	return FunctionInputEnum->GetDisplayNameTextByValue((int64)InputType).ToString();
}

EMaterialValueType UMaterialExpressionFunctionInput::GetMaterialTypeFromInputType(EFunctionInputType InputType)
{
	static_assert(FunctionInput_MAX == EXPECTED_NUM_INPUT_TYPES, "Number of expected Function Input types is incorrect. Need to update");
	switch (InputType)
	{
	case FunctionInput_Scalar:
		return MCT_Float;
	case FunctionInput_Vector2:
		return MCT_Float2;
	case FunctionInput_Vector3:
		return MCT_Float3;
	case FunctionInput_Vector4:
		return MCT_Float4;
	case FunctionInput_Texture2D:
		return MCT_Texture2D;
	case FunctionInput_TextureCube:
		return MCT_TextureCube;
	case FunctionInput_Texture2DArray:
		return MCT_Texture2DArray;
	case FunctionInput_TextureExternal:
		return MCT_TextureExternal;
	case FunctionInput_VolumeTexture:
		return MCT_VolumeTexture;
	case FunctionInput_StaticBool:
		return MCT_StaticBool;
	case FunctionInput_Bool:
		return MCT_Bool;
	case FunctionInput_MaterialAttributes:
		return MCT_MaterialAttributes;
	case FunctionInput_Substrate:
		return MCT_Substrate;
	default:
		return MCT_Unknown;
	}
}

EFunctionInputType UMaterialExpressionFunctionInput::GetInputTypeFromMaterialType(EMaterialValueType MaterialType)
{
	static_assert(FunctionInput_MAX == EXPECTED_NUM_INPUT_TYPES, "Number of expected Function Input types is incorrect. Need to update");
	switch (MaterialType)
	{
	case MCT_Float:
	case MCT_Float1:
		return FunctionInput_Scalar;
	case MCT_Float2:
		return FunctionInput_Vector2;
	case MCT_Float3:
		return FunctionInput_Vector3;
	case MCT_Float4:
		return FunctionInput_Vector4;
	case MCT_Texture:
	case MCT_Texture2D:
		return FunctionInput_Texture2D;
	case MCT_TextureCube:
		return FunctionInput_TextureCube;
	case MCT_Texture2DArray:
		return FunctionInput_Texture2DArray;
	case MCT_TextureExternal:
		return FunctionInput_TextureExternal;
	case MCT_VolumeTexture:
		return FunctionInput_VolumeTexture;
	case MCT_StaticBool:
		return FunctionInput_StaticBool;
	case MCT_Bool:
		return FunctionInput_Bool;
	case MCT_MaterialAttributes:
		return FunctionInput_MaterialAttributes;
	case MCT_Substrate:
		return FunctionInput_Substrate;
	default:
		return FunctionInput_MAX;
	}
}

#if WITH_EDITOR
void UMaterialExpressionFunctionInput::ValidateName()
{
	if (Material)
	{
		int32 InputNameIndex = 1;
		bool bResultNameIndexValid = true;
		FName PotentialInputName;

		// Find an available unique name
		do 
		{
			PotentialInputName = InputName;
			if (InputNameIndex != 1)
			{
				PotentialInputName.SetNumber(InputNameIndex);
			}

			bResultNameIndexValid = true;
			for (UMaterialExpression* Expression : Material->GetExpressions())
			{
				UMaterialExpressionFunctionInput* OtherFunctionInput = Cast<UMaterialExpressionFunctionInput>(Expression);
				if (OtherFunctionInput && OtherFunctionInput != this && OtherFunctionInput->InputName == PotentialInputName)
				{
					bResultNameIndexValid = false;
					break;
				}
			}

			InputNameIndex++;
		} 
		while (!bResultNameIndexValid);

		InputName = PotentialInputName;
	}
}

bool UMaterialExpressionFunctionInput::IsResultMaterialAttributes(int32 OutputIndex)
{
	if( FunctionInput_MaterialAttributes == InputType )
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool UMaterialExpressionFunctionInput::IsResultSubstrateMaterial(int32 OutputIndex)
{
	bool bResult = false;
	if (FunctionInput_Substrate == InputType)
	{
		bResult = true;
	}
	else if(IsResultMaterialAttributes(OutputIndex))
	{
		FExpressionInput* LocalPreviewDuringCompile = PopEffectivePreviewDuringCompile();
		if (LocalPreviewDuringCompile && LocalPreviewDuringCompile->GetTracedInput().Expression)
		{
			bResult = LocalPreviewDuringCompile->GetTracedInput().Expression->IsResultSubstrateMaterial(LocalPreviewDuringCompile->OutputIndex);
		}
		else if(bUsePreviewValueAsDefault)
		{
			if (Preview.Expression)
			{
				bResult = Preview.Expression->IsResultSubstrateMaterial(Preview.OutputIndex);
			}
			else
			{
				//Ensure default values force slab generation for MA type inputs.
				bResult = true;
			}
		}

		if (LocalPreviewDuringCompile)
		{
			PushEffectivePreviewDuringCompile(LocalPreviewDuringCompile);
		}
	}
	return bResult;
}

void UMaterialExpressionFunctionInput::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	if (IsResultSubstrateMaterial(OutputIndex))
	{
		FExpressionInput* LocalPreviewDuringCompile = PopEffectivePreviewDuringCompile();
		if (LocalPreviewDuringCompile && LocalPreviewDuringCompile->GetTracedInput().Expression)
		{
			LocalPreviewDuringCompile->GetTracedInput().Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, LocalPreviewDuringCompile->OutputIndex);
		}
		else if(Preview.Expression && bUsePreviewValueAsDefault)
		{
			Preview.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, Preview.OutputIndex);
		}

		if (LocalPreviewDuringCompile)
		{
			PushEffectivePreviewDuringCompile(LocalPreviewDuringCompile);
		}

		if (SubstrateMaterialInfo.IsValid())
		{
			return;
		}
	}
	SubstrateMaterialInfo.AddShadingModel(SSM_DefaultLit);
	SubstrateMaterialInfo.AddGuid(MaterialExpressionGuid);
}

FSubstrateOperator* UMaterialExpressionFunctionInput::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	FSubstrateOperator* SlabOperator = nullptr;
	if (IsResultSubstrateMaterial(OutputIndex))
	{
		FExpressionInput* LocalPreviewDuringCompile = PopEffectivePreviewDuringCompile();
		if (LocalPreviewDuringCompile && LocalPreviewDuringCompile->GetTracedInput().Expression)
		{
			// Backup EffectivePreviewDuringCompile which will be modified by UnlinkFromCaller and LinkIntoCaller of any potential chained function calls to the same function
			FExpressionInput LocalPreviewTracedInput = LocalPreviewDuringCompile->GetTracedInput();
			SlabOperator = LocalPreviewTracedInput.Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, LocalPreviewTracedInput.OutputIndex);
		}
		else if(Preview.Expression && bUsePreviewValueAsDefault)
		{
			SlabOperator = Preview.Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, Preview.OutputIndex);
		}

		if (LocalPreviewDuringCompile)
		{
			PushEffectivePreviewDuringCompile(LocalPreviewDuringCompile);
		}
	}

	// If we are parsing for a material function input we always needs to return a default valid BSDF operator at least 
	// If the material requires an input then the UI will force the user to proviced one.
	// If this is not the case however, we need to compile a default material to compile the shader, or to preview the material function.
	if(!SlabOperator)
	{
		SlabOperator = &SubstrateTranslatorData.SubstrateGetDefaultBSDFOperator(this->MaterialExpressionGuid, Parent);
	}
	return SlabOperator;
}

EMaterialValueType UMaterialExpressionFunctionInput::GetInputValueType(int32 InputIndex)
{
	return GetMaterialTypeFromInputType(InputType);
}

EMaterialValueType UMaterialExpressionFunctionInput::GetOutputValueType(int32 OutputIndex)
{
	return GetInputValueType(0);
}
#endif // WITH_EDITOR

bool UMaterialExpressionFunctionInput::IsAllowedIn(const UObject* MaterialOrFunction) const
{
	return MaterialOrFunction && MaterialOrFunction->IsA<UMaterialFunction>() && Super::IsAllowedIn(MaterialOrFunction);
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionFunctionOutput
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionFunctionOutput::UMaterialExpressionFunctionOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputs = false;
#endif

	OutputName = TEXT("Result");

#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

void UMaterialExpressionFunctionOutput::PostLoad()
{
	Super::PostLoad();
	ConditionallyGenerateId(false);
}

void UMaterialExpressionFunctionOutput::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	// Ideally we would like to regenerate the Id here, but this is used when propagating 
	// To the preview material function when editing a material function and back
	// So instead we regenerate the Id when copy pasting in the material editor, see UMaterialExpression::CopyMaterialExpressions
	ConditionallyGenerateId(false);
}

#if WITH_EDITOR
void UMaterialExpressionFunctionOutput::PostEditImport()
{
	Super::PostEditImport();
	ConditionallyGenerateId(true);
}
#endif	//#if WITH_EDITOR

#if WITH_EDITOR
void UMaterialExpressionFunctionOutput::PreEditChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialExpressionFunctionOutput, OutputName))
	{
		OutputNameBackup = OutputName;
	}
	Super::PreEditChange(PropertyAboutToChange);
}

void UMaterialExpressionFunctionOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMaterialExpressionFunctionOutput, OutputName))
	{
		if (Material)
		{
			for (UMaterialExpression* Expression : Material->GetExpressions())
			{
				UMaterialExpressionFunctionOutput* OtherFunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);
				if (OtherFunctionOutput && OtherFunctionOutput != this && OtherFunctionOutput->OutputName == OutputName)
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OutputNamesMustBeUnique", "Function output names must be unique"));
					OutputName = OutputNameBackup;
					break;
				}
			}
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionFunctionOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Output ") + OutputName.ToString());
}

void UMaterialExpressionFunctionOutput::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(Description, 40, OutToolTip);
}

EMaterialValueType UMaterialExpressionFunctionOutput::GetInputValueType(int32 InputIndex)
{
	// Acceptable types for material function outputs
	return static_cast<EMaterialValueType>(MCT_Float | MCT_MaterialAttributes | MCT_Substrate);
}

EMaterialValueType UMaterialExpressionFunctionOutput::GetOutputValueType(int32 OutputIndex)
{
	const EMaterialValueType RealInputValueType = GetRealInputValueType(A);

	if (RealInputValueType != MCT_Unknown)
	{
		return RealInputValueType;
	}

	return Super::GetOutputValueType(OutputIndex);
}

int32 UMaterialExpressionFunctionOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing function output '%s'"), *OutputName.ToString());
	}
	return A.Compile(Compiler);
}

int32 UMaterialExpressionFunctionOutput::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FExpressionInput ATraced = A.GetTracedInput();
	UMaterialExpression* AExpression = ATraced.Expression;

	// If this logic is changed, one need to test 
	//  - material function preview on nodes in materials, 
	//  - and in material functions with output node being SubstrateData (a slab) or a color,
	//  - and for material with Surface or UI domain preview.
	//  - This for r.substrate 1 and r.Substrate.ProjectGBufferFormat 0/1.
	if (Substrate::IsSubstrateEnabled() && AExpression && AExpression->IsResultSubstrateMaterial(ATraced.OutputIndex))
	{
		// Compile the SubstrateData output.
		int32 SubstrateDataCodeChunk = Compile(Compiler, ATraced.OutputIndex);
		// Convert the SubstrateData to a preview color.
		return Compiler->SubstrateCompilePreview(SubstrateDataCodeChunk);
	}

	// If this is a material attribute node, let's have it decide of the preview 
	// (instead of using current attribute on the compiler stack which is not a good representative, default to EmissiveColor).
	if (AExpression && AExpression->IsResultMaterialAttributes(ATraced.OutputIndex))
	{
		return AExpression->CompilePreview(Compiler, ATraced.OutputIndex);
	}

	// Default preview value compilation.
	return Compile(Compiler, OutputIndex);
}
#endif // WITH_EDITOR

void UMaterialExpressionFunctionOutput::ConditionallyGenerateId(bool bForce)
{
	if (bForce || !Id.IsValid())
	{
		Id = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::FunctionOutput);
	}
}

#if WITH_EDITOR
void UMaterialExpressionFunctionOutput::ValidateName()
{
	if (Material)
	{
		int32 OutputNameIndex = 1;
		bool bResultNameIndexValid = true;
		FName PotentialOutputName;

		// Find an available unique name
		do 
		{
			PotentialOutputName = OutputName;
			if (OutputNameIndex != 1)
			{
				PotentialOutputName.SetNumber(OutputNameIndex);
			}

			bResultNameIndexValid = true;
			for (UMaterialExpression* Expression : Material->GetExpressions())
			{
				UMaterialExpressionFunctionOutput* OtherFunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);
				if (OtherFunctionOutput && OtherFunctionOutput != this && OtherFunctionOutput->OutputName == PotentialOutputName)
				{
					bResultNameIndexValid = false;
					break;
				}
			}

			OutputNameIndex++;
		} 
		while (!bResultNameIndexValid);

		OutputName = PotentialOutputName;
	}
}

bool UMaterialExpressionFunctionOutput::IsResultMaterialAttributes(int32 OutputIndex)
{
	// If there is a loop anywhere in this expression's inputs then we can't risk checking them
	if( A.GetTracedInput().Expression )
	{
		return A.Expression->IsResultMaterialAttributes(A.OutputIndex);
	}
	else
	{
		return false;
	}
}

bool UMaterialExpressionFunctionOutput::IsResultSubstrateMaterial(int32 OutputIndex)
{
	// If there is a loop anywhere in this expression's inputs then we can't risk checking them
	if( A.GetTracedInput().Expression )
	{
		return A.Expression->IsResultSubstrateMaterial(A.OutputIndex);
	}
	else
	{
		return false;
	}
}

void UMaterialExpressionFunctionOutput::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	// If there is a loop anywhere in this expression's inputs then we can't risk checking them
	if( A.GetTracedInput().Expression )
	{
		A.Expression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, A.OutputIndex);
	}
}

FSubstrateOperator* UMaterialExpressionFunctionOutput::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	if (A.GetTracedInput().Expression)
	{
		return A.Expression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, A.OutputIndex);
	}
	return nullptr;
}
#endif // WITH_EDITOR

bool UMaterialExpressionFunctionOutput::IsAllowedIn(const UObject* MaterialOrFunction) const
{
	return MaterialOrFunction && MaterialOrFunction->IsA<UMaterialFunction>() && Super::IsAllowedIn(MaterialOrFunction);
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionMaterialLayerOutput
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionMaterialLayerOutput::UMaterialExpressionMaterialLayerOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	OutputName = TEXT("Material Attributes");
}


//
//	UMaterialExpressionCollectionParameter
//
UMaterialExpressionCollectionParameter::UMaterialExpressionCollectionParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}


void UMaterialExpressionCollectionParameter::PostLoad()
{
	if (Collection)
	{
		Collection->ConditionalPostLoad();
		ParameterName = Collection->GetParameterName(ParameterId);
	}

	Super::PostLoad();
}

#if WITH_EDITOR
void UMaterialExpressionCollectionParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	if (Collection)
	{
		ParameterId = Collection->GetParameterId(ParameterName);
	}
	else
	{
		ParameterId = FGuid();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionCollectionParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ParameterIndex = -1;
	int32 ComponentIndex = -1;

	if (Collection)
	{
		Collection->GetParameterIndex(ParameterId, ParameterIndex, ComponentIndex);
	}

	if (ParameterIndex != -1)
	{
		return Compiler->AccessCollectionParameter(Collection, ParameterIndex, ComponentIndex);
	}
	else
	{
		if (!Collection)
		{
			return Compiler->Errorf(TEXT("CollectionParameter has invalid Collection!"));
		}
		else
		{
			return Compiler->Errorf(TEXT("CollectionParameter has invalid parameter %s"), *ParameterName.ToString());
		}
	}
}

void UMaterialExpressionCollectionParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	FString TypePrefix;

	if (Collection)
	{
		int32 ParameterIndex = -1;
		int32 ComponentIndex = -1;
		Collection->GetParameterIndex(ParameterId, ParameterIndex, ComponentIndex);

		if (ComponentIndex == -1)
		{
			TypePrefix = TEXT("(float4) ");
		}
		else
		{
			TypePrefix = TEXT("(float1) ");
		}
	}

	OutCaptions.Add(TypePrefix + TEXT("Collection Param"));

	if (Collection)
	{
		OutCaptions.Add(Collection->GetName());
		OutCaptions.Add(FString("'") + ParameterName.ToString() + TEXT("'"));
	}
	else
	{
		OutCaptions.Add(TEXT("Unspecified"));
	}
}

bool UMaterialExpressionCollectionParameter::GetParameterValue(FMaterialParameterMetadata& OutMeta) const
{
	OutMeta.Value = FMaterialParameterValue(Collection.Get());
	OutMeta.Description = Desc;
	OutMeta.ExpressionGuid = ExpressionGUID;
	OutMeta.Group = Group;
	OutMeta.SortPriority = SortPriority;
	return true;
}

bool UMaterialExpressionCollectionParameter::SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (Name == ParameterName && Meta.Value.Type == EMaterialParameterType::ParameterCollection)
	{
		if (UMaterialParameterCollection* ParameterCollection = Cast<UMaterialParameterCollection>(Meta.Value.ParameterCollection))
		{
			if (Collection != ParameterCollection)
			{
				Collection = ParameterCollection;
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Collection));
				}
			}
			return true;
		}
	}
	return false;
}

bool UMaterialExpressionCollectionParameter::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (ParameterName.ToString().Contains(SearchQuery))
	{
		return true;
	}

	if (Collection && Collection->GetName().Contains(SearchQuery))
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}
#endif // WITH_EDITOR


//
//	UMaterialExpressionCollectionTransform
//
UMaterialExpressionCollectionTransform::UMaterialExpressionCollectionTransform(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
	bHidePreviewWindow = true;
#endif
}


void UMaterialExpressionCollectionTransform::PostLoad()
{
	if (Collection)
	{
		Collection->ConditionalPostLoad();
		ParameterName = Collection->GetParameterName(ParameterId);
	}

	Super::PostLoad();
}

#if WITH_EDITOR
void UMaterialExpressionCollectionTransform::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	if (Collection)
	{
		ParameterId = Collection->GetParameterId(ParameterName);
	}
	else
	{
		ParameterId = FGuid();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionCollectionTransform::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Input.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("CollectionTransform missing input"));
	}

	int32 InputIndex = Input.Compile(Compiler);
	EMaterialValueType InputType = Compiler->GetType(InputIndex);
	if (InputType != MCT_Float3 && InputType != MCT_Float4 && InputType != MCT_LWCVector3 && InputType != MCT_LWCVector4)
	{
		return Compiler->Errorf(TEXT("CollectionTransform requires vector input"));
	}

	if (!Collection)
	{
		return Compiler->Errorf(TEXT("CollectionTransform has invalid Collection!"));
	}

	int32 ParameterIndex = -1;
	int32 ComponentIndex = -1;
	Collection->GetParameterIndex(ParameterId, ParameterIndex, ComponentIndex);

	if (ParameterIndex == -1)
	{
		return Compiler->Errorf(TEXT("CollectionTransform has invalid parameter %s"), *ParameterName.ToString());
	}

	if (ComponentIndex != -1)
	{
		return Compiler->Errorf(TEXT("CollectionTransform parameter %s is scalar, vectors are required"), *ParameterName.ToString());
	}

	TStaticArray<int32, 5> CollectionParameters;

	if (TransformType == EParameterCollectionTransformType::Position || TransformType == EParameterCollectionTransformType::Projection)
	{
		if (ParameterIndex + 4 > Collection->GetTotalVectorStorage())
		{
			return Compiler->Errorf(TEXT("CollectionTransform parameter %s requires 4 vectors for Position or Projection matrix"), *ParameterName.ToString());
		}
		else
		{
			for (int32 ParameterOffset = 0; ParameterOffset < 4; ParameterOffset++)
			{
				CollectionParameters[ParameterOffset] = Compiler->AccessCollectionParameter(Collection, ParameterIndex + ParameterOffset, -1);
			}
			CollectionParameters[4] = -1;
		}
	}
	else if (TransformType == EParameterCollectionTransformType::Vector)
	{
		if (ParameterIndex + 3 > Collection->GetTotalVectorStorage())
		{
			return Compiler->Errorf(TEXT("CollectionTransform parameter %s requires 3 vectors for Vector matrix"), *ParameterName.ToString());
		}
		else
		{
			for (int32 ParameterOffset = 0; ParameterOffset < 3; ParameterOffset++)
			{
				CollectionParameters[ParameterOffset] = Compiler->AccessCollectionParameter(Collection, ParameterIndex + ParameterOffset, -1);
			}
			CollectionParameters[3] = -1;
			CollectionParameters[4] = -1;
		}
	}
	else
	{
		if (ParameterIndex + 5 > Collection->GetTotalVectorStorage())
		{
			return Compiler->Errorf(TEXT("CollectionTransform parameter %s requires 5 vectors for LWC Matrix"), *ParameterName.ToString());
		}
		else
		{
			for (int32 ParameterOffset = 0; ParameterOffset < 5; ParameterOffset++)
			{
				CollectionParameters[ParameterOffset] = Compiler->AccessCollectionParameter(Collection, ParameterIndex + ParameterOffset, -1);
			}
			check(TransformType == EParameterCollectionTransformType::LocalToWorld || TransformType == EParameterCollectionTransformType::WorldToLocal);
		}
	}

	return Compiler->CollectionTransform(InputIndex, CollectionParameters, TransformType);
}

void UMaterialExpressionCollectionTransform::GetCaption(TArray<FString>& OutCaptions) const
{
	FString TypePrefix;

	if (Collection)
	{
		int32 ParameterIndex = -1;
		int32 ComponentIndex = -1;
		Collection->GetParameterIndex(ParameterId, ParameterIndex, ComponentIndex);

		if (ParameterIndex == -1)
		{
			TypePrefix = TEXT("(Needs name)");
		}
		else if (ComponentIndex != -1)
		{
			TypePrefix = TEXT("(Needs vector type)");
		}
		else if (TransformType == EParameterCollectionTransformType::Position || TransformType == EParameterCollectionTransformType::Projection)
		{
			if (ParameterIndex + 4 > Collection->GetTotalVectorStorage())
			{
				TypePrefix = TEXT("(Needs 4 vectors)");
			}
			else if (TransformType == EParameterCollectionTransformType::Position)
			{
				TypePrefix = TEXT("(Position)");
			}
			else
			{
				TypePrefix = TEXT("(Projection)");
			}
		}
		else if (TransformType == EParameterCollectionTransformType::Vector)
		{
			if (ParameterIndex + 3 > Collection->GetTotalVectorStorage())
			{
				TypePrefix = TEXT("(Needs 3 vectors)");
			}
			else
			{
				TypePrefix = TEXT("(Vector)");
			}
		}
		else
		{
			if (ParameterIndex + 5 > Collection->GetTotalVectorStorage())
			{
				TypePrefix = TEXT("(Needs 5 vectors)");
			}
			else if (TransformType == EParameterCollectionTransformType::LocalToWorld)
			{
				TypePrefix = TEXT("(Local to World)");
			}
			else
			{
				check(TransformType == EParameterCollectionTransformType::WorldToLocal);
				TypePrefix = TEXT("(World to Local)");
			}
		}
	}
	else
	{
		TypePrefix = TEXT("(Needs collection)");
	}

	OutCaptions.Add(TypePrefix + TEXT(" Collection Transform"));

	if (Collection)
	{
		OutCaptions.Add(Collection->GetName());
		OutCaptions.Add(FString("'") + ParameterName.ToString() + TEXT("'"));
	}
	else
	{
		OutCaptions.Add(TEXT("Unspecified"));
	}
}


bool UMaterialExpressionCollectionTransform::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (ParameterName.ToString().Contains(SearchQuery))
	{
		return true;
	}

	if (Collection && Collection->GetName().Contains(SearchQuery))
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionLightmapUVs
//
UMaterialExpressionLightmapUVs::UMaterialExpressionLightmapUVs(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	bHidePreviewWindow = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 0, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionLightmapUVs::Compile( FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->LightmapUVs();
}

	
void UMaterialExpressionLightmapUVs::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("LightmapUVs"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionAOMaterialMask
//
UMaterialExpressionPrecomputedAOMask::UMaterialExpressionPrecomputedAOMask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	bHidePreviewWindow = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionPrecomputedAOMask::Compile( FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->PrecomputedAOMask();
}

	
void UMaterialExpressionPrecomputedAOMask::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PrecomputedAOMask"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionLightmassReplace
//
#if WITH_EDITOR
int32 UMaterialExpressionLightmassReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Realtime.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing LightmassReplace input Realtime"));
	}
	else if (!Lightmass.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing LightmassReplace input Lightmass"));
	}
	else
	{
		const int32 Arg2 = Lightmass.Compile(Compiler);
		if (Compiler->IsLightmassCompiler())
		{
			return Arg2;
		}
		const int32 Arg1 = Realtime.Compile(Compiler);
		//only when both of these are real expressions do the actual code.  otherwise various output pins will
		//end up considered 'set' when really we just want a default.  This can cause us to force depth output when we don't want it for example.
		if (Arg1 != INDEX_NONE && Arg2 != INDEX_NONE)
		{
			return Compiler->LightmassReplace(Arg1, Arg2);
		}
		else if (Arg1 != INDEX_NONE)
		{
			return Arg1;
		}
		else if (Arg2 != INDEX_NONE)
		{
			return Arg2;
		}
		return INDEX_NONE;
	}
}

void UMaterialExpressionLightmassReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("LightmassReplace"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionShadowReplace
//
#if WITH_EDITOR
int32 UMaterialExpressionShadowReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Default"));
	}
	else if (!Shadow.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Shadow"));
	}
	else
	{
		const int32 Arg1 = Default.Compile(Compiler);
		const int32 Arg2 = Shadow.Compile(Compiler);
		return Compiler->ShadowReplace(Arg1, Arg2);
	}
}

void UMaterialExpressionShadowReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Shadow Pass Switch"));
}

void UMaterialExpressionShadowReplace::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Allows material to define specialized behavior when being rendered into ShadowMap."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionShaderStageSwitch
//
#if WITH_EDITOR
int32 UMaterialExpressionShaderStageSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!PixelShader.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input PixelShader"));
	}
	else if (!VertexShader.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input VertexShader"));
	}
	else
	{
		const EShaderFrequency ShaderFrequency = Compiler->GetCurrentShaderFrequency();
		return ShouldUsePixelShaderInput(ShaderFrequency) ? PixelShader.Compile(Compiler) : VertexShader.Compile(Compiler);
	}
}

void UMaterialExpressionShaderStageSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Shader Stage Switch"));
}

void UMaterialExpressionShaderStageSwitch::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Allows material to define specialized behavior for certain shader stages."), 40, OutToolTip);
}
#endif // WITH_EDITOR


//
//	UMaterialExpressionMaterialProxy
//
#if WITH_EDITOR
int32 UMaterialExpressionMaterialProxyReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Realtime.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialProxyReplace input Realtime"));
	}
	else if (!MaterialProxy.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing MaterialProxyReplace input MaterialProxy"));
	}
	else
	{
		return Compiler->IsMaterialProxyCompiler() ? MaterialProxy.Compile(Compiler) : Realtime.Compile(Compiler);
	}
}

bool UMaterialExpressionMaterialProxyReplace::IsResultMaterialAttributes(int32 OutputIndex)
{
	for (FExpressionInputIterator It{ this }; It; ++It)
	{
		if (It->GetTracedInput().Expression && It->Expression->IsResultMaterialAttributes(It->OutputIndex))
		{
			return true;
		}
	}
	return false;
}

void UMaterialExpressionMaterialProxyReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MaterialProxyReplace"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionGIReplace
//
#if WITH_EDITOR
int32 UMaterialExpressionGIReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	FExpressionInput& LocalStaticIndirect = StaticIndirect.GetTracedInput().Expression ? StaticIndirect : Default;
	FExpressionInput& LocalDynamicIndirect = DynamicIndirect.GetTracedInput().Expression ? DynamicIndirect : Default;

	if(!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing GIReplace input 'Default'"));
	}
	else
	{
		int32 Arg1 = Default.Compile(Compiler);
		int32 Arg2 = LocalStaticIndirect.Compile(Compiler);
		int32 Arg3 = LocalDynamicIndirect.Compile(Compiler);
		return Compiler->GIReplace(Arg1, Arg2, Arg3);
	}
}

void UMaterialExpressionGIReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("GIReplace"));
}
#endif // WITH_EDITOR
//
// UMaterialExpressionRayTracingQualitySwitch
//
#if WITH_EDITOR
int32 UMaterialExpressionRayTracingQualitySwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Normal.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RayTracingQualitySwitch input 'Normal'"));
	}
	else if (!RayTraced.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RayTracingQualitySwitch input 'RayTraced'"));
	}
	else
	{
		int32 Arg1 = Normal.Compile(Compiler);
		int32 Arg2 = FDataDrivenShaderPlatformInfo::GetSupportsRayTracing(Compiler->GetShaderPlatform()) ? RayTraced.Compile(Compiler) : INDEX_NONE;

		//only when both of these are real expressions do the actual code.  otherwise various output pins will
		//end up considered 'set' when really we just want a default.  This can cause us to force depth output when we don't want it for example.
		if (Arg1 != INDEX_NONE && Arg2 != INDEX_NONE)
		{
			return Compiler->RayTracingQualitySwitchReplace(Arg1, Arg2);
		}
		else if (Arg1 != INDEX_NONE)
		{
			return Arg1;
		}
		else if (Arg2 != INDEX_NONE)
		{
			return Arg2;
		}
		return INDEX_NONE;
	}
}

bool UMaterialExpressionRayTracingQualitySwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	if (Normal.Expression)
	{
		return Normal.Expression->IsResultMaterialAttributes(OutputIndex);
	}
	return false;
}

FSubstrateOperator* UMaterialExpressionRayTracingQualitySwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	ThrowSubstrateUnsupportedSwitchNodeError(SubstrateTranslatorData, *GetClass()->GetName());
	return nullptr;
}

void UMaterialExpressionRayTracingQualitySwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("RayTracingQualitySwitchReplace"));
}

EMaterialValueType UMaterialExpressionRayTracingQualitySwitch::GetInputValueType(int32 InputIndex)
{
	return MCT_Unknown;
}
#endif // WITH_EDITOR

//
// UMaterialExpressionPathTracingQualitySwitch
//
#if WITH_EDITOR
int32 UMaterialExpressionPathTracingQualitySwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Normal.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing PathTracingQualitySwitch input 'Normal'"));
	}
	else if (!PathTraced.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing PathTracingQualitySwitch input 'PathTraced'"));
	}
	else
	{
		int32 Arg1 = Normal.Compile(Compiler);
		int32 Arg2 = PathTraced.Compile(Compiler);

		//only when both of these are real expressions do the actual code.  otherwise various output pins will
		//end up considered 'set' when really we just want a default.  This can cause us to force depth output when we don't want it for example.
		if (Arg1 != INDEX_NONE && Arg2 != INDEX_NONE)
		{
			return Compiler->PathTracingQualitySwitchReplace(Arg1, Arg2);
		}
		else if (Arg1 != INDEX_NONE)
		{
			return Arg1;
		}
		else if (Arg2 != INDEX_NONE)
		{
			return Arg2;
		}
		return INDEX_NONE;
	}
}

bool UMaterialExpressionPathTracingQualitySwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	if (Normal.Expression)
	{
		return Normal.Expression->IsResultMaterialAttributes(OutputIndex);
	}
	return false;
}

void UMaterialExpressionPathTracingQualitySwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PathTracingQualitySwitchReplace"));
}

EMaterialValueType UMaterialExpressionPathTracingQualitySwitch::GetInputValueType(int32 InputIndex)
{
	return MCT_Unknown;
}

FSubstrateOperator* UMaterialExpressionPathTracingQualitySwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	ThrowSubstrateUnsupportedSwitchNodeError(SubstrateTranslatorData, *GetClass()->GetName());
	return nullptr;
}
#endif // WITH_EDITOR


//
// UMaterialExpressionPathTracingRayTypeSwitch
//
#if WITH_EDITOR
int32 UMaterialExpressionPathTracingRayTypeSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Main.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing PathTracingRayTypeSwitch input 'Main'"));
	}
	else
	{
		// compile all arguments (its ok if some of these are not connected, the will default to using Main)
		int32 ArgMain = Main.Compile(Compiler);
		int32 ArgShadow = Shadow.Compile(Compiler);
		int32 ArgDiffuse = IndirectDiffuse.Compile(Compiler);
		int32 ArgSpecular = IndirectSpecular.Compile(Compiler);
		int32 ArgVolume = IndirectVolume.Compile(Compiler);

		return Compiler->PathTracingRayTypeSwitch(ArgMain, ArgShadow, ArgDiffuse, ArgSpecular, ArgVolume);
	}
}

bool UMaterialExpressionPathTracingRayTypeSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	// Only check the Main expression since it must be connected. If the other plugs have something else connected that is not compatible,
	// we would get an error during translation therefore it is sufficient to check the main plug only.
	if (Main.Expression)
	{
		return Main.Expression->IsResultMaterialAttributes(OutputIndex);
	}
	return false;
}

void UMaterialExpressionPathTracingRayTypeSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PathTracingRayTypeSwitch"));
}

FSubstrateOperator* UMaterialExpressionPathTracingRayTypeSwitch::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	ThrowSubstrateUnsupportedSwitchNodeError(SubstrateTranslatorData, *GetClass()->GetName());
	return nullptr;
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPathTracingBufferTexture
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPathTracingBufferTexture::UMaterialExpressionPathTracingBufferTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionPathTracingBufferTexture::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ViewportUV = INDEX_NONE;

	if (Coordinates.GetTracedInput().Expression)
	{
		ViewportUV = Coordinates.Compile(Compiler);
	}

	return Compiler->PathTracingBufferTextureLookup(ViewportUV, PathTracingBufferTextureId);
}

void UMaterialExpressionPathTracingBufferTexture::GetCaption(TArray<FString>& OutCaptions) const
{
	UEnum* Enum = StaticEnum<EPathTracingBufferTextureId>();
	check(Enum);

	FString Name = Enum->GetDisplayNameTextByValue(PathTracingBufferTextureId).ToString();
	OutCaptions.Add(Name);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionObjectOrientation::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ObjectOrientation();
}

void UMaterialExpressionObjectOrientation::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("ObjectOrientation"));
}
#endif // WITH_EDITOR

UMaterialExpression* UMaterialExpressionRerouteBase::TraceInputsToRealExpression(int32& OutputIndex) const
{
#if WITH_EDITORONLY_DATA
	TSet<FMaterialExpressionKey> VisitedExpressions;
	FExpressionInput RealInput = TraceInputsToRealExpressionInternal(VisitedExpressions);
	OutputIndex = RealInput.OutputIndex;
	return RealInput.Expression;
#else
	OutputIndex = 0;
	return nullptr;
#endif
}

FExpressionInput UMaterialExpressionRerouteBase::TraceInputsToRealInput() const
{
	TSet<FMaterialExpressionKey> VisitedExpressions;
	FExpressionInput RealInput = TraceInputsToRealExpressionInternal(VisitedExpressions);
	return RealInput;
}

FExpressionInput UMaterialExpressionRerouteBase::TraceInputsToRealExpressionInternal(TSet<FMaterialExpressionKey>& VisitedExpressions) const
{
#if WITH_EDITORONLY_DATA
	FExpressionInput Input;
	if (GetRerouteInput(Input))
	{
		// First check to see if this is a terminal node, if it is then we have a reroute to nowhere.
		if (Input.Expression != nullptr)
		{
			// Now check to see if we're also connected to another reroute. If we are, then keep going unless we hit a loop condition.
			UMaterialExpressionRerouteBase* RerouteInput = Cast<UMaterialExpressionRerouteBase>(Input.Expression);
			if (RerouteInput != nullptr)
			{
				FMaterialExpressionKey InputExpressionKey(Input.Expression, Input.OutputIndex);
				// prevent recurring visits to expressions we've already checked
				if (VisitedExpressions.Contains(InputExpressionKey))
				{
					// We have a loop! This should result in not finding the value!
					return FExpressionInput();
				}
				else
				{
					VisitedExpressions.Add(InputExpressionKey);
					FExpressionInput OutputExpressionInput = RerouteInput->TraceInputsToRealExpressionInternal(VisitedExpressions);
					return OutputExpressionInput;
				}
			}
			else
			{
				// We aren't connected to another Reroute, so we are good.
				return Input;
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
	// We went to nowhere, so bail out.
	return FExpressionInput();
}

#if WITH_EDITOR
EMaterialValueType UMaterialExpressionRerouteBase::GetInputValueType(int32 InputIndex)
{
	FExpressionInput Input;
	if (GetRerouteInput(Input))
	{
		// Our input type should match the node that we are ultimately connected to, no matter how many reroute nodes lie between us.
		if (InputIndex == 0 && Input.IsConnected() && Input.Expression != nullptr)
		{
			int32 RealExpressionOutputIndex = -1;
			UMaterialExpression* RealExpression = TraceInputsToRealExpression(RealExpressionOutputIndex);

			// If we found a valid connection to a real output, then our type becomes that type.
			if (RealExpression != nullptr && RealExpressionOutputIndex != -1 && RealExpression->Outputs.Num() > RealExpressionOutputIndex && RealExpressionOutputIndex >= 0)
			{
				return RealExpression->GetOutputValueType(RealExpressionOutputIndex);
			}
		}
	}
	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionRerouteBase::GetOutputValueType(int32 OutputIndex)
{
	// Our node is a passthrough so input and output types must match.
	return GetInputValueType(0);
}

UMaterialExpression* UMaterialExpressionRerouteBase::ResolveRerouteTarget(int32 OutputIndex, int32& OutRealOutputIndex)
{
	FExpressionInput Input;
	if (GetRerouteInput(Input))
	{
		if (Input.IsConnected() && Input.Expression != nullptr && OutputIndex == 0)
		{
			// Guard against cycles that pass through non-reroute expressions (e.g. Reroute -> StaticSwitch -> Reroute).
			if (ContainsInputLoop())
			{
				return nullptr;
			}

			OutRealOutputIndex = -1;
			return TraceInputsToRealExpression(OutRealOutputIndex);
		}
	}
	return nullptr;
}

bool UMaterialExpressionRerouteBase::IsResultMaterialAttributes(int32 OutputIndex)
{
	int32 RealOutputIndex = -1;
	if (UMaterialExpression* RealExpression = ResolveRerouteTarget(OutputIndex, RealOutputIndex))
	{
		return RealExpression->IsResultMaterialAttributes(RealOutputIndex);
	}
	return false;
}

bool UMaterialExpressionRerouteBase::IsResultSubstrateMaterial(int32 OutputIndex)
{
	int32 RealOutputIndex = -1;
	if (UMaterialExpression* RealExpression = ResolveRerouteTarget(OutputIndex, RealOutputIndex))
	{
		return RealExpression->IsResultSubstrateMaterial(RealOutputIndex);
	}
	return false;
}

void UMaterialExpressionRerouteBase::GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex)
{
	int32 RealOutputIndex = -1;
	if (UMaterialExpression* RealExpression = ResolveRerouteTarget(OutputIndex, RealOutputIndex))
	{
		RealExpression->GatherSubstrateMaterialInfo(SubstrateMaterialInfo, RealOutputIndex);
	}
}

FSubstrateOperator* UMaterialExpressionRerouteBase::SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex)
{
	int32 RealOutputIndex = -1;
	if (UMaterialExpression* RealExpression = ResolveRerouteTarget(OutputIndex, RealOutputIndex))
	{
		return RealExpression->SubstrateGenerateMaterialTopologyTree(SubstrateTranslatorData, Parent, RealOutputIndex);
	}
	return nullptr;
}

#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionReroute::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Because we don't want to generate *any* additional instructions, we just forward this request
	// to the node that this input is connected to. If it isn't connected, then the compile will return INDEX_NONE.
	int32 Result = Input.Compile(Compiler);
	return Result;
}

int32 UMaterialExpressionReroute::CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ResultCodeChunk = Compile(Compiler, OutputIndex);

	if (Input.Expression && Input.Expression->IsResultSubstrateMaterial(Input.OutputIndex))
	{
		ResultCodeChunk = Compiler->SubstrateCompilePreview(ResultCodeChunk);
	}
	return ResultCodeChunk;
}

void UMaterialExpressionReroute::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Reroute Node (reroutes wires)"));
}


FText UMaterialExpressionReroute::GetCreationDescription() const 
{
	return LOCTEXT("RerouteNodeCreationDesc", "This node looks like a single pin and can be used to tidy up your graph by adding a movable control point to the connection spline.");
}

FText UMaterialExpressionReroute::GetCreationName() const
{
	return LOCTEXT("RerouteNodeCreationName", "Add Reroute Node...");
}
#endif // WITH_EDITOR

bool UMaterialExpressionReroute::GetRerouteInput(FExpressionInput& OutInput) const
{
	OutInput = Input;
	return true;
}

UMaterialExpressionNamedRerouteDeclaration* UMaterialExpressionNamedRerouteBase::FindDeclarationInMaterial(const FGuid& VariableGuid) const
{
	UMaterialExpressionNamedRerouteDeclaration* Declaration = nullptr;
#if WITH_EDITORONLY_DATA
	if (Material)
	{
		Declaration = FindDeclarationInArray(VariableGuid, Material->GetExpressions());
	}
	else if (Function) // Material should always be valid, but just in case also check Function
	{
		Declaration = FindDeclarationInArray(VariableGuid, Function->GetExpressions());
	}
#endif // WITH_EDITORONLY_DATA
	return Declaration;
}

UMaterialExpressionNamedRerouteDeclaration::UMaterialExpressionNamedRerouteDeclaration(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	NodeColor = CookDeterminism::MakeRandomColor(GetPathName(), CookDeterminism::ESeed::RerouteColor);
#endif
	Name = TEXT("Name");
}

void UMaterialExpressionNamedRerouteDeclaration::PostInitProperties()
{
	Super::PostInitProperties();
	// Init the GUID
	UpdateVariableGuid(false, false);
}

void UMaterialExpressionNamedRerouteDeclaration::PostLoad()
{
	Super::PostLoad();
	// Init the GUID
	UpdateVariableGuid(false, false);
}

void UMaterialExpressionNamedRerouteDeclaration::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// We do not force a guid regen here because this function is used when the Material Editor makes a copy of a material to edit.
	// If we forced a GUID regen, it would cause all of the guids for a material to change every time a material was edited.
	UpdateVariableGuid(false, true);
}

#if WITH_EDITOR
void UMaterialExpressionNamedRerouteDeclaration::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, Name))
	{
		MakeNameUnique();
	}
}

int32 UMaterialExpressionNamedRerouteDeclaration::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Just forward to the input
	return Input.Compile(Compiler);
}

int32 UMaterialExpressionNamedRerouteDeclaration::CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ResultCodeChunk = Compile(Compiler, OutputIndex);

	if (Input.Expression && Input.Expression->IsResultSubstrateMaterial(Input.OutputIndex))
	{
		ResultCodeChunk = Compiler->SubstrateCompilePreview(ResultCodeChunk);
	}
	return ResultCodeChunk;
}

void UMaterialExpressionNamedRerouteDeclaration::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(Name.ToString());
}


FText UMaterialExpressionNamedRerouteDeclaration::GetCreationDescription() const 
{
	return LOCTEXT("NamedRerouteDeclCreationDesc", "Captures the value of an input, may be used at multiple other points in the graph without requiring connecting wires, allows tiding up of complex graphs");
}

FText UMaterialExpressionNamedRerouteDeclaration::GetCreationName() const
{
	return LOCTEXT("NamedRerouteDeclCreationName", "Add Named Reroute Declaration Node...");
}

bool UMaterialExpressionNamedRerouteDeclaration::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (Name.ToString().Contains(SearchQuery))
	{
		return true;
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

bool UMaterialExpressionNamedRerouteDeclaration::CanRenameNode() const
{
	return true;
}

FString UMaterialExpressionNamedRerouteDeclaration::GetEditableName() const
{
	return Name.ToString();
}

void UMaterialExpressionNamedRerouteDeclaration::SetEditableName(const FString& NewName)
{
	Name = *NewName;
	MakeNameUnique();

	// Refresh usage names
	if (Material || Function)
	{
		auto Expressions = Material ? Material->GetExpressions() : Function->GetExpressions();
		for (UMaterialExpression* Expression : Expressions)
		{
			auto* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(Expression);
			if (Usage && Usage->Declaration == this && Usage->GraphNode)
			{
				Usage->GraphNode->ReconstructNode();
			}
		}
	}
}

void UMaterialExpressionNamedRerouteDeclaration::PostCopyNode(const TArray<UMaterialExpression*>& CopiedExpressions)
{
	Super::PostCopyNode(CopiedExpressions);

	// Only force regeneration of Guid if there's already a variable with the same one
	if (FindDeclarationInMaterial(VariableGuid))
	{
		// Update Guid, and update the copied usages accordingly
		FGuid OldGuid = VariableGuid;
		UpdateVariableGuid(true, true);
		for (UMaterialExpression* Expression : CopiedExpressions)
		{
			auto* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(Expression);
			if (Usage && Usage->DeclarationGuid == OldGuid)
			{
				Usage->Declaration = this;
				Usage->DeclarationGuid = VariableGuid;
			}
		}

		// Find a new name
		MakeNameUnique();
	}
	else
	{
		// If there's no existing variable with this GUID, only create it if needed
		UpdateVariableGuid(false, true);
	}
}
#endif // WITH_EDITOR

bool UMaterialExpressionNamedRerouteDeclaration::GetRerouteInput(FExpressionInput& OutInput) const
{
	OutInput = Input;
	return true;
}

void UMaterialExpressionNamedRerouteDeclaration::UpdateVariableGuid(bool bForceGeneration, bool bAllowMarkingPackageDirty)
{
	// If we are in the editor, and we don't have a valid GUID yet, generate one.
	if (GIsEditor && !IsRunningGame() && !GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor) && !IsTemplate())
	{
		if (bForceGeneration || !VariableGuid.IsValid())
		{
			VariableGuid = CookDeterminism::NewGuid(GetPathName(), CookDeterminism::ESeed::RerouteVariable);

			if (bAllowMarkingPackageDirty)
			{
				MarkPackageDirty();
			}
		}
	}
}

void UMaterialExpressionNamedRerouteDeclaration::MakeNameUnique()
{
#if WITH_EDITORONLY_DATA
	if (Material || Function)
	{
		auto Expressions = Material ? Material->GetExpressions() : Function->GetExpressions();

		int32 NameIndex = 1;
		bool bResultNameIndexValid = true;
		FName PotentialName;

		// Find an available unique name
		do
		{
			PotentialName = Name;
			if (NameIndex != 1)
			{
				PotentialName.SetNumber(NameIndex);
			}

			bResultNameIndexValid = true;
			for (UMaterialExpression* Expression : Expressions)
			{
				auto* OtherDeclaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(Expression);
				if (OtherDeclaration && OtherDeclaration != this && OtherDeclaration->Name == PotentialName)
				{
					bResultNameIndexValid = false;
					break;
				}
			}

			NameIndex++;
		} while (!bResultNameIndexValid);

		Name = PotentialName;
	}
#endif // WITH_EDITORONLY_DATA
}

bool UMaterialExpressionNamedRerouteUsage::IsDeclarationValid() const
{
	// Deleted expressions are marked as pending kill (see FMaterialEditor::DeleteNodes)
	return IsValid(Declaration);
}

#if WITH_EDITOR

int32 UMaterialExpressionNamedRerouteUsage::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!IsDeclarationValid())
	{
		return Compiler->Errorf(TEXT("Invalid named reroute variable"));
	}
	return Declaration->Compile(Compiler, OutputIndex);
}
int32 UMaterialExpressionNamedRerouteUsage::CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ResultCodeChunk = Compile(Compiler, OutputIndex);

	if (IsDeclarationValid())
	{
		FExpressionInput Input = Declaration->TraceInputsToRealInput();
		if (Input.Expression && Input.Expression->IsResultSubstrateMaterial(Input.OutputIndex))
		{
			ResultCodeChunk = Compiler->SubstrateCompilePreview(ResultCodeChunk);
		}
	}

	return ResultCodeChunk;
}

void UMaterialExpressionNamedRerouteUsage::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(IsDeclarationValid() ? Declaration->Name.ToString() : TEXT("Invalid named reroute"));
}

EMaterialValueType UMaterialExpressionNamedRerouteUsage::GetOutputValueType(int32 OutputIndex)
{
	if (IsDeclarationValid())
	{
		return Declaration->GetInputValueType(OutputIndex);
	}
	else
	{
		return Super::GetOutputValueType(OutputIndex);
	}
}

bool UMaterialExpressionNamedRerouteUsage::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (IsDeclarationValid())
	{
		return Declaration->MatchesSearchQuery(SearchQuery);
	}
	return Super::MatchesSearchQuery(SearchQuery);
}

void UMaterialExpressionNamedRerouteUsage::PostCopyNode(const TArray<UMaterialExpression*>& CopiedExpressions)
{
	Super::PostCopyNode(CopiedExpressions);

	// First try to find the declaration in the copied expressions
	Declaration = FindDeclarationInArray(DeclarationGuid, CopiedExpressions);
	if (!Declaration)
	{
		// If unsuccessful, try to find it in the whole material
		Declaration = FindDeclarationInMaterial(DeclarationGuid);
	}

	// Keep GUID in sync. In case this is pasted by itself into another graph, we don't want this node to connect up to a previously connected declaration. 
	if (Declaration)
	{
		DeclarationGuid = Declaration->VariableGuid;
	}

	// Save that Declaration change
	MarkPackageDirty();
}
#endif // WITH_EDITOR

bool UMaterialExpressionNamedRerouteUsage::GetRerouteInput(FExpressionInput& OutInput) const
{
	if (IsDeclarationValid())
	{
		// Forward to the declaration input
		OutInput = Declaration->Input;
		return true;
	}
	else
	{
		return false;
	}
}

#if WITH_EDITOR
int32 UMaterialExpressionRotateAboutAxis::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!NormalizedRotationAxis.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RotateAboutAxis input NormalizedRotationAxis"));
	}
	else if (!RotationAngle.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RotateAboutAxis input RotationAngle"));
	}
	else if (!PivotPoint.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RotateAboutAxis input PivotPoint"));
	}
	else if (!Position.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing RotateAboutAxis input Position"));
	}
	else
	{
		const int32 AngleIndex = Compiler->Mul(RotationAngle.Compile(Compiler), Compiler->Constant(2.0f * (float)UE_PI / Period));
		const int32 RotationIndex = Compiler->AppendVector(
			Compiler->ForceCast(NormalizedRotationAxis.Compile(Compiler), MCT_Float3), 
			Compiler->ForceCast(AngleIndex, MCT_Float1));

		return Compiler->RotateAboutAxis(
			RotationIndex, 
			PivotPoint.Compile(Compiler), 
			Position.Compile(Compiler));
	}
}

void UMaterialExpressionRotateAboutAxis::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("RotateAboutAxis"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// Static functions so it can be used other material expressions.
///////////////////////////////////////////////////////////////////////////////

/** Does not use length() to allow optimizations. */
static int32 CompileHelperLength( FMaterialCompiler* Compiler, int32 A, int32 B )
{
	int32 Delta = Compiler->Sub(A, B);
	if(Compiler->GetType(A) == MCT_Float && Compiler->GetType(B) == MCT_Float)
	{
		// optimized
		return Compiler->Abs(Delta);
	}
	return Compiler->Length(Delta);
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSphereMask
///////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
int32 UMaterialExpressionSphereMask::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input A"));
	}
	else if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input B"));
	}
	else
	{
		int32 Arg1 = A.Compile(Compiler);
		int32 Arg2 = B.Compile(Compiler);
		int32 Distance = CompileHelperLength(Compiler, Arg1, Arg2);

		int32 ArgInvRadius;
		if(Radius.GetTracedInput().Expression)
		{
			// if the radius input is hooked up, use it
			ArgInvRadius = Compiler->Div(Compiler->Constant(1.0f), Compiler->Max(Compiler->Constant(0.00001f), Radius.Compile(Compiler)));
		}
		else
		{
			// otherwise use the internal constant
			ArgInvRadius = Compiler->Constant(1.0f / FMath::Max(0.00001f, AttenuationRadius));
		}

		int32 NormalizeDistance = Compiler->Mul(Distance, ArgInvRadius);

		int32 ArgInvHardness;
		if(Hardness.GetTracedInput().Expression)
		{
			int32 Softness = Compiler->Sub(Compiler->Constant(1.0f), Hardness.Compile(Compiler));

			// if the radius input is hooked up, use it
			ArgInvHardness = Compiler->Div(Compiler->Constant(1.0f), Compiler->Max(Softness, Compiler->Constant(0.00001f)));
		}
		else
		{
			// Hardness is in percent 0%:soft .. 100%:hard
			// Max to avoid div by 0
			float InvHardness = 1.0f / FMath::Max(1.0f - HardnessPercent * 0.01f, 0.00001f);

			// otherwise use the internal constant
			ArgInvHardness = Compiler->Constant(InvHardness);
		}

		int32 NegNormalizedDistance = Compiler->Sub(Compiler->Constant(1.0f), NormalizeDistance);
		int32 MaskUnclamped = Compiler->Mul(NegNormalizedDistance, ArgInvHardness);
		return Compiler->Saturate(MaskUnclamped);
	}
}

void UMaterialExpressionSphereMask::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SphereMask"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSobol
///////////////////////////////////////////////////////////////////////////////

int32 UMaterialExpressionSobol::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CellInput = Cell.GetTracedInput().Expression ? Cell.Compile(Compiler) : Compiler->Constant2(0.f, 0.f);
	int32 IndexInput = Index.GetTracedInput().Expression ? Index.Compile(Compiler) : Compiler->Constant(ConstIndex);
	int32 SeedInput = Seed.GetTracedInput().Expression ? Seed.Compile(Compiler) : Compiler->Constant2(ConstSeed.X, ConstSeed.Y);
	return Compiler->Sobol(CellInput, IndexInput, SeedInput);
}

void UMaterialExpressionSobol::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Sobol"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionTemporalSobol
///////////////////////////////////////////////////////////////////////////////

int32 UMaterialExpressionTemporalSobol::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 IndexInput = Index.GetTracedInput().Expression ? Index.Compile(Compiler) : Compiler->Constant(ConstIndex);
	int32 SeedInput = Seed.GetTracedInput().Expression ? Seed.Compile(Compiler) : Compiler->Constant2(ConstSeed.X, ConstSeed.Y);
	return Compiler->TemporalSobol(IndexInput, SeedInput);
}

void UMaterialExpressionTemporalSobol::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Temporal Sobol"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionNaniteReplace
//
#if WITH_EDITOR
int32 UMaterialExpressionNaniteReplace::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Default"));
	}
	else if (!Nanite.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Nanite"));
	}
	else
	{
		const int32 Arg1 = Default.Compile(Compiler);
		const int32 Arg2 = FDataDrivenShaderPlatformInfo::GetSupportsNanite(Compiler->GetShaderPlatform()) ? Nanite.Compile(Compiler) : Arg1;
		return Compiler->NaniteReplace(Arg1, Arg2);
	}
}

void UMaterialExpressionNaniteReplace::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Nanite Pass Switch"));
}

void UMaterialExpressionNaniteReplace::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Allows material to define specialized behavior when being rendered with Nanite."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionMaterialCache
//
UMaterialExpressionMaterialCache::UMaterialExpressionMaterialCache(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	bShowOutputNameOnPin = true;
	bShowMaskColorsOnPin = false;
	ConstructFromTag();
#endif // WITH_EDITOR
}

#if WITH_EDITOR
int32 UMaterialExpressionMaterialCache::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Outputs.IsValidIndex(OutputIndex))
	{
		return CompilerError(Compiler, TEXT("Invalid Output Index"));
	}

	// Empty tags are not valid
	if (TagLayout.Layers.IsEmpty())
	{
		return CompilerError(Compiler, TEXT("Invalid Tag Definition"));
	}
		
	// If we're compiling the output's graph, emit the input attributes
	if (Compiler->GetTopCustomOutput() == this)
	{
		// We're sort of mixing concepts here, the "outputs" of this node isn't exactly the same as the number of custom outputs
		// But the translator doesn't differentiate, so, just ignore all custom outputs beyond 0.
		if (bIsSample || OutputIndex != 0)
		{
			return INDEX_NONE;
		}
		
		int32 AttributeSet = Compiler->DefaultMaterialCacheAttribute(TagLayout);

		for (int32 i = 0; i < Attributes.Num(); i++)
		{
			FMaterialExpressionMaterialCacheAttribute& Attribute = Attributes[i];

			// Unbound inputs is supported, albeit a waste
			if (!Attribute.Input.GetTracedInput().Expression)
			{
				UE_LOGF(LogMaterial, Warning, "Material Cache expression with unbound inputs in '%ls', consider a minimal tag.", Material ? *(Material->GetName()) : TEXT("Unknown"));
				continue;
			}
			
			int32 AttributeIndex = Attribute.Input.Compile(Compiler);

			// May fail for unrelated reasons
			if (AttributeIndex == INDEX_NONE)
			{
				return Compiler->Errorf(TEXT("Failed to compile attribute '%s"), *Attribute.Decoration);
			}

			AttributeIndex = Compiler->ForceCast(AttributeIndex, static_cast<EMaterialValueType>(Attribute.ValueType));

			Compiler->SetMaterialCacheAttribute(TagLayout, AttributeSet, GetPackedAttributeIndex(TagAttributes[i]), AttributeIndex);
		}

		// Emit entire set
		return Compiler->MaterialCacheOutput(this, TagLayout, AttributeSet);
	}

	// Primitive is optional
	int32 PrimitiveIDIndex = Primitive.IsConnected() ? Primitive.Compile(Compiler) : INDEX_NONE;

	// UV is optionally overridden
	int32 UVIndex = INDEX_NONE;
	if (UV.IsConnected())
	{
		UVIndex = UV.Compile(Compiler);
	}
	else
	{
		UVIndex = Compiler->TextureCoordinate(0, false, false);
	}

	// Sample the cached attributes
	int32 AttributeIndex = Compiler->SampleMaterialCache(TagLayout, PrimitiveIDIndex, UVIndex);
	if (AttributeIndex == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Failed to sample"));
	}

	// Get the specific attribute
	return Compiler->GetMaterialCacheAttribute(TagLayout, AttributeIndex, GetPackedAttributeIndex(TagAttributes[OutputIndex]));
}

void UMaterialExpressionMaterialCache::ConstructFromTag()
{
	Attributes.Reset();
	ConstructLayout();

	// Create output pins
	for (EMaterialCacheAttribute Attribute : TagAttributes)
	{
		FMaterialExpressionMaterialCacheAttribute ExpressionAttribute;
		ExpressionAttribute.Decoration = GetMaterialCacheAttributeDecoration(Attribute);
		ExpressionAttribute.ValueType = static_cast<uint8>(GetMaterialCacheAttributeValueType(Attribute));
		ExpressionAttribute.Input.InputName = *ExpressionAttribute.Decoration;
		Attributes.Add(ExpressionAttribute);
	}
}

void UMaterialExpressionMaterialCache::ConstructLayout()
{
	// If there's a tag, assume its layers, otherwise default
	if (Tag)
	{
		TagAttributes = Tag->Attributes;
		TagLayout = Tag->GetRuntimeLayout();
	}
	else
	{
		TagAttributes = TArrayView<EMaterialCacheAttribute>(DefaultMaterialCacheAttributes);
		TagLayout.Guid = FGuid();
		TagLayout.Layers.Reset();
		
		PackMaterialCacheAttributeLayers(DefaultMaterialCacheAttributes, TagLayout.Layers);
	}
}

void UMaterialExpressionMaterialCache::ConstructOutputs()
{
	Outputs.Reset();
	
	for (EMaterialCacheAttribute Attribute : TagAttributes)
	{
		const uint8 ComponentCount = GetMaterialCacheAttributeComponentCount(Attribute, false);
			
		Outputs.Add(FExpressionOutput(
			*GetMaterialCacheAttributeDecoration(Attribute),
			1,
			ComponentCount > 0, //-V547
			ComponentCount > 1,
			ComponentCount > 2,
			ComponentCount > 3 //-V547
		));
	}
}

TArray<FExpressionOutput>& UMaterialExpressionMaterialCache::GetOutputs()
{
	ConstructOutputs();
	return Outputs;
}

bool UMaterialExpressionMaterialCache::AllowMultipleCustomOutputs()
{
	return true;
}

void UMaterialExpressionMaterialCache::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Tag))
	{
		ConstructFromTag();
		
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, bIsSample))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionMaterialCache::PostLoad()
{
	Super::PostLoad();
	ConstructLayout();
}

uint32 UMaterialExpressionMaterialCache::GetPackedAttributeIndex(EMaterialCacheAttribute Attribute)
{
	uint32_t AttributeIndex = 0;
	
	for (const FMaterialCacheLayer& Layer : TagLayout.Layers)
	{
		for (EMaterialCacheAttribute PackedAttribute : Layer.Attributes)
		{
			if (PackedAttribute == Attribute)
			{
				return AttributeIndex;
			}
			
			AttributeIndex++;
		}
	}
	
	checkf(false, TEXT("Invalid attribute index"));
	return 0;
}

void UMaterialExpressionMaterialCache::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Material Cache Value"));
}

void UMaterialExpressionMaterialCache::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Evaluate an expression in cache-space, subsequent (base pass, shadow, etc.) fetches are a virtual sample"), 40, OutToolTip);
}

EShaderFrequency UMaterialExpressionMaterialCache::GetShaderFrequency(uint32 OutputIndex)
{
	return SF_Pixel;
}

EMaterialValueType UMaterialExpressionMaterialCache::GetInputValueType(int32 InputIndex)
{
	if (bIsSample)
	{
		switch (InputIndex)
		{
		default:
			return MCT_Unknown;
		case 0:
			return MCT_UInt;
		case 1:
			return MCT_Float2;
		}
	}
	
	if (Attributes.IsValidIndex(InputIndex))
	{
		return static_cast<EMaterialValueType>(Attributes[InputIndex].ValueType);
	}

	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionMaterialCache::GetOutputValueType(int32 OutputIndex)
{
	if (!Attributes.IsValidIndex(OutputIndex))
	{
		return MCT_Unknown;
	}
	
	return static_cast<EMaterialValueType>(Attributes[OutputIndex].ValueType);
}

FExpressionInput* UMaterialExpressionMaterialCache::GetInput(int32 InputIndex)
{
	if (bIsSample)
	{
		switch (InputIndex)
		{
		default:
			return nullptr;
		case 0:
			return &Primitive;
		case 1:
			return &UV;
		}
	}

	if (Attributes.IsValidIndex(InputIndex))
	{
		return &Attributes[InputIndex].Input;
	}

	return nullptr;
}

bool UMaterialExpressionMaterialCache::IsInputConnectionRequired(int32 InputIndex) const
{
	return false;
}

bool UMaterialExpressionMaterialCache::IsResultMaterialAttributes(int32 OutputIndex)
{
	return false;
}

FName UMaterialExpressionMaterialCache::GetInputName(int32 InputIndex) const
{
	if (bIsSample)
	{
		switch (InputIndex)
		{
		default:
			return NAME_None;
		case 0:
			return TEXT("Primitive");
		case 1:
			return TEXT("UV");
		}
	}
	
	if (Attributes.IsValidIndex(InputIndex))
	{
		return *Attributes[InputIndex].Decoration;
	}

	return NAME_None;
}

int32 UMaterialExpressionMaterialCache::GetMaxOutputs() const
{
	return Outputs.Num();
}

int32 UMaterialExpressionMaterialCache::GetNumOutputs() const
{
	return Outputs.Num();
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionBlackBody
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionBlackBody::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 TempInput = INDEX_NONE;

	if( Temp.GetTracedInput().Expression )
	{
		TempInput = Temp.Compile(Compiler);
	}

	if( TempInput == INDEX_NONE )
	{
		return INDEX_NONE;
	}

	return Compiler->BlackBody( TempInput );
}

void UMaterialExpressionBlackBody::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("BlackBody"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDistanceToNearestSurface
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
FName UMaterialExpressionDistanceToNearestSurface::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &Position)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionDistanceToNearestSurface::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionDistanceToNearestSurface::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (Position.GetTracedInput().Expression)
	{
		PositionArg = Position.Compile(Compiler);
	}
	else 
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->DistanceToNearestSurface(PositionArg, WorldPositionOriginType);
}

void UMaterialExpressionDistanceToNearestSurface::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DistanceToNearestSurface"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDistanceFieldGradient
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
FName UMaterialExpressionDistanceFieldGradient::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &Position)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionDistanceFieldGradient::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionDistanceFieldGradient::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (Position.GetTracedInput().Expression)
	{
		PositionArg = Position.Compile(Compiler);
	}
	else 
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->DistanceFieldGradient(PositionArg, WorldPositionOriginType);
}

void UMaterialExpressionDistanceFieldGradient::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DistanceFieldGradient"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDistanceFieldApproxAO
///////////////////////////////////////////////////////////////////////////////

FName UMaterialExpressionDistanceFieldApproxAO::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &Position)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionDistanceFieldApproxAO::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionDistanceFieldApproxAO::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (Position.GetTracedInput().Expression)
	{
		PositionArg = Position.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	int32 NormalArg = INDEX_NONE;

	if (Normal.GetTracedInput().Expression)
	{
		NormalArg = Normal.Compile(Compiler);
	}
	else
	{
		NormalArg = Compiler->VertexNormal();
	}

	int32 BaseDistanceArg = BaseDistance.GetTracedInput().Expression ? BaseDistance.Compile(Compiler) : Compiler->Constant(BaseDistanceDefault);
	int32 RadiusArg = Radius.GetTracedInput().Expression ? Radius.Compile(Compiler) : Compiler->Constant(RadiusDefault);

	return Compiler->DistanceFieldApproxAO(PositionArg, WorldPositionOriginType, NormalArg, BaseDistanceArg, RadiusArg, NumSteps, StepScaleDefault);
}

void UMaterialExpressionDistanceFieldApproxAO::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("DistanceFieldApproxAO"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSamplePhysicsVectorField
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
FName UMaterialExpressionSamplePhysicsVectorField::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &WorldPosition)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionSamplePhysicsVectorField::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionSamplePhysicsVectorField::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (WorldPosition.GetTracedInput().Expression)
	{
		PositionArg = WorldPosition.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->ProcessPhysicsField(PositionArg, WorldPositionOriginType, EFieldOutputType::Field_Output_Vector, static_cast<uint8>(FieldTarget), true);
}

void UMaterialExpressionSamplePhysicsVectorField::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SamplePhysicsVectorField"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionEvalPhysicsVectorField
///////////////////////////////////////////////////////////////////////////////

int32 UMaterialExpressionEvalPhysicsVectorField::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (WorldPosition.GetTracedInput().Expression)
	{
		PositionArg = WorldPosition.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->ProcessPhysicsField(PositionArg, WorldPositionOriginType, EFieldOutputType::Field_Output_Vector, static_cast<uint8>(FieldTarget), false);
}

void UMaterialExpressionEvalPhysicsVectorField::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("EvalPhysicsVectorField"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSamplePhysicsScalarField
///////////////////////////////////////////////////////////////////////////////

FName UMaterialExpressionSamplePhysicsScalarField::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &WorldPosition)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionSamplePhysicsScalarField::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionSamplePhysicsScalarField::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (WorldPosition.GetTracedInput().Expression)
	{
		PositionArg = WorldPosition.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->ProcessPhysicsField(PositionArg, WorldPositionOriginType, EFieldOutputType::Field_Output_Scalar, static_cast<uint8>(FieldTarget), true);
}

void UMaterialExpressionSamplePhysicsScalarField::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SamplePhysicsScalarField"));
}

//////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionEvalPhysicsScalarField
///////////////////////////////////////////////////////////////////////////////

int32 UMaterialExpressionEvalPhysicsScalarField::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (WorldPosition.GetTracedInput().Expression)
	{
		PositionArg = WorldPosition.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->ProcessPhysicsField(PositionArg, WorldPositionOriginType, EFieldOutputType::Field_Output_Scalar, static_cast<uint8>(FieldTarget), false);
}

void UMaterialExpressionEvalPhysicsScalarField::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("EvalPhysicsScalarField"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSamplePhysicsIntegerField
///////////////////////////////////////////////////////////////////////////////

FName UMaterialExpressionSamplePhysicsIntegerField::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &WorldPosition)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionSamplePhysicsIntegerField::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionSamplePhysicsIntegerField::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (WorldPosition.GetTracedInput().Expression)
	{
		PositionArg = WorldPosition.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->ProcessPhysicsField(PositionArg, WorldPositionOriginType, EFieldOutputType::Field_Output_Integer, static_cast<uint8>(FieldTarget), true);
}

void UMaterialExpressionSamplePhysicsIntegerField::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SamplePhysicsIntegerField"));
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionEvalPhysicsIntegerField
///////////////////////////////////////////////////////////////////////////////

int32 UMaterialExpressionEvalPhysicsIntegerField::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 PositionArg = INDEX_NONE;

	if (WorldPosition.GetTracedInput().Expression)
	{
		PositionArg = WorldPosition.Compile(Compiler);
	}
	else
	{
		PositionArg = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}

	return Compiler->ProcessPhysicsField(PositionArg, WorldPositionOriginType, EFieldOutputType::Field_Output_Integer, static_cast<uint8>(FieldTarget), false);
}

void UMaterialExpressionEvalPhysicsIntegerField::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("EvalPhysicsIntegerField"));
}
#endif // WITH_EDITOR



///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDistance
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionDistance::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!A.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input A"));
	}
	else if(!B.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input B"));
	}
	else
	{
		int32 Arg1 = A.Compile(Compiler);
		int32 Arg2 = B.Compile(Compiler);
		return CompileHelperLength(Compiler, Arg1, Arg2);
	}
}

void UMaterialExpressionDistance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Distance"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterialExpressionTwoSidedSign::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("TwoSidedSign"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionVertexNormalWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->VertexNormal();
}

void UMaterialExpressionVertexNormalWS::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("VertexNormalWS"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionVertexTangentWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->VertexTangent();
}

void UMaterialExpressionVertexTangentWS::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("VertexTangentWS"));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionPixelNormalWS::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->PixelNormalWS();
}

void UMaterialExpressionPixelNormalWS::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PixelNormalWS"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPerInstanceRandom
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionPerInstanceRandom::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->PerInstanceRandom();
}

void UMaterialExpressionPerInstanceRandom::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PerInstanceRandom"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPerInstanceFadeAmount
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionPerInstanceFadeAmount::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->PerInstanceFadeAmount();
}

void UMaterialExpressionPerInstanceFadeAmount::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("PerInstanceFadeAmount"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPerInstanceCustomData
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionPerInstanceCustomData::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 DefaultArgument = DefaultValue.GetTracedInput().Expression ? DefaultValue.Compile(Compiler) : Compiler->Constant(ConstDefaultValue);
	return Compiler->PerInstanceCustomData(DataIndex, DefaultArgument);
}

void UMaterialExpressionPerInstanceCustomData::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("PerInstanceCustomData[%d]"), DataIndex));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPerInstanceCustomData3Vector
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR

int32 UMaterialExpressionPerInstanceCustomData3Vector::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 DefaultArgument = DefaultValue.GetTracedInput().Expression ? DefaultValue.Compile(Compiler) : Compiler->Constant3(ConstDefaultValue.R, ConstDefaultValue.G, ConstDefaultValue.B);
	return Compiler->PerInstanceCustomData3Vector(DataIndex, DefaultArgument);
}

void UMaterialExpressionPerInstanceCustomData3Vector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("PerInstanceCustomData[%d, %d, %d]"), DataIndex, DataIndex + 1, DataIndex + 2));
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionAntialiasedTextureMask
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionAntialiasedTextureMask::UMaterialExpressionAntialiasedTextureMask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> DefaultTexture;
		FConstructorStatics()
			: DefaultTexture(TEXT("/Engine/EngineResources/DefaultTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Texture = ConstructorStatics.DefaultTexture.Object;
#endif

	ParameterName = NAME_None;

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionAntialiasedTextureMask::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if(!Texture)
	{
		return Compiler->Errorf(TEXT("UMaterialExpressionAntialiasedTextureMask> Missing input texture"));
	}

	if (Texture->GetMaterialType() == MCT_TextureVirtual)
	{
		return Compiler->Errorf(TEXT("UMaterialExpressionAntialiasedTextureMask> Virtual textures are not supported"));
	}

	int32 ArgCoord = Coordinates.Expression ? Coordinates.Compile(Compiler) : Compiler->TextureCoordinate(ConstCoordinate, false, false);

	FString ErrorMessage;
	if (!TextureIsValid(Texture, ErrorMessage))
	{
		return CompilerError(Compiler, *ErrorMessage);
	}

	int32 TextureCodeIndex;

	if (!ParameterName.IsValid() || ParameterName.IsNone())
	{
		TextureCodeIndex = Compiler->Texture(Texture, SamplerType);
	}
	else
	{
		TextureCodeIndex = Compiler->TextureParameter(ParameterName, Texture, SamplerType);
	}

	FString SamplerTypeError;
	if (! MaterialExpressionUtils::VerifySamplerType(Compiler->GetShaderPlatform(), Compiler->GetTargetPlatform(), Texture, SamplerType, SamplerTypeError))
	{
		return Compiler->Errorf(TEXT("%s"), *SamplerTypeError);
	}

	return Compiler->AntialiasedTextureMask(TextureCodeIndex,ArgCoord,Threshold,Channel);
}

void UMaterialExpressionAntialiasedTextureMask::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("AAMasked Param2D")); 
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionAntialiasedTextureMask::TextureIsValid(UTexture* InTexture, FString& OutMessage)
{
	if (!InTexture)
	{
		OutMessage = TEXT("Found NULL, requires Texture2D");
		return false;
	}
	// Doesn't allow virtual/external textures here
	else if (!(InTexture->GetMaterialType() & MCT_Texture2D))
	{
		OutMessage = FString::Printf(TEXT("Found %s, requires Texture2D"), *InTexture->GetClass()->GetName());
		return false;
	}
	
	return true;
}

void UMaterialExpressionAntialiasedTextureMask::SetDefaultTexture()
{
	Texture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineResources/DefaultTexture.DefaultTexture"), nullptr, LOAD_None, nullptr);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionDecalDerivative
//
UMaterialExpressionDecalDerivative::UMaterialExpressionDecalDerivative(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("DDX")));
	Outputs.Add(FExpressionOutput(TEXT("DDY")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDecalDerivative::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->TextureDecalDerivative(OutputIndex == 1);
}

void UMaterialExpressionDecalDerivative::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Decal Derivative"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionDecalColor
//
UMaterialExpressionDecalColor::UMaterialExpressionDecalColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGB"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("R"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("G"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("B"), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("A"), 1, 0, 0, 0, 1));
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionDecalColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Decal Color"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionDecalLifetimeOpacity
//
UMaterialExpressionDecalLifetimeOpacity::UMaterialExpressionDecalLifetimeOpacity(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Opacity")));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionDecalLifetimeOpacity::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Decal Lifetime Opacity"));
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionDecalMipmapLevel
//
UMaterialExpressionDecalMipmapLevel::UMaterialExpressionDecalMipmapLevel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ConstWidth(256.0f)
	, ConstHeight(ConstWidth)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDecalMipmapLevel::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Material && Material->MaterialDomain != MD_DeferredDecal)
	{
		return CompilerError(Compiler, TEXT("Node only works for the deferred decal material domain."));
	}

	int32 TextureSizeInput = INDEX_NONE;

	if (TextureSize.GetTracedInput().Expression)
	{
		TextureSizeInput = TextureSize.Compile(Compiler);
	}
	else
	{
		TextureSizeInput = Compiler->Constant2(ConstWidth, ConstHeight);
	}

	if (TextureSizeInput == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->TextureDecalMipmapLevel(TextureSizeInput);
}

void UMaterialExpressionDecalMipmapLevel::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Decal Mipmap Level"));
}
#endif // WITH_EDITOR

UMaterialExpressionDepthFade::UMaterialExpressionDepthFade(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDepthFade::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Scales Opacity by a Linear fade based on SceneDepth, from 0 at PixelDepth to 1 at FadeDistance
	// Result = Opacity * saturate((SceneDepth - PixelDepth) / max(FadeDistance, DELTA))
	const int32 OpacityIndex = InOpacity.GetTracedInput().Expression ? InOpacity.Compile(Compiler) : Compiler->Constant(OpacityDefault);
	const int32 FadeDistanceIndex = Compiler->Max(FadeDistance.GetTracedInput().Expression ? FadeDistance.Compile(Compiler) : Compiler->Constant(FadeDistanceDefault), Compiler->Constant(UE_DELTA));

	int32 PixelDepthIndex = -1; 
	// On mobile scene depth is limited to 65500 
	// to avoid false fading on objects that are close or exceed this limit we clamp pixel depth to (65500 - FadeDistance)
	if (Compiler->GetFeatureLevel() <= ERHIFeatureLevel::ES3_1)
	{
		PixelDepthIndex = Compiler->Min(Compiler->PixelDepth(), Compiler->Sub(Compiler->Constant(65500.f), FadeDistanceIndex));
	}
	else
	{
		PixelDepthIndex = Compiler->PixelDepth();
	}
	
	const int32 FadeIndex = Compiler->Saturate(Compiler->Div(Compiler->Sub(Compiler->SceneDepth(INDEX_NONE, INDEX_NONE, false), PixelDepthIndex), FadeDistanceIndex));
	
	return Compiler->Mul(OpacityIndex, FadeIndex);
}
#endif // WITH_EDITOR

//
//	UMaterialExpressionSphericalParticleOpacity
//
UMaterialExpressionSphericalParticleOpacity::UMaterialExpressionSphericalParticleOpacity(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionSphericalParticleOpacity::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const int32 DensityIndex = Density.GetTracedInput().Expression ? Density.Compile(Compiler) : Compiler->Constant(ConstantDensity);
	return Compiler->SphericalParticleOpacity(DensityIndex);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDepthOfFieldFunction
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionDepthOfFieldFunction::UMaterialExpressionDepthOfFieldFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDepthOfFieldFunction::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 DepthInput;

	if(Depth.GetTracedInput().Expression)
	{
		// using the input allows more custom behavior
		DepthInput = Depth.Compile(Compiler);
	}
	else
	{
		// no input means we use the PixelDepth
		DepthInput = Compiler->PixelDepth();
	}

	if(DepthInput == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->DepthOfFieldFunction(DepthInput, FunctionValue);
}


void UMaterialExpressionDepthOfFieldFunction::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("DepthOfFieldFunction"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPostVolumeUserFlagTest
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPostVolumeUserFlagTest::UMaterialExpressionPostVolumeUserFlagTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionPostVolumeUserFlagTest::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 BitIndexCompiled;

	if (BitIndex.GetTracedInput().Expression)
	{
		BitIndexCompiled = BitIndex.Compile(Compiler);
	}
	else
	{
		BitIndexCompiled = Compiler->Constant((float)ConstBitIndex);
	}

	if (BitIndexCompiled == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->PostVolumeUserFlagTestFunction(BitIndexCompiled);
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDDX
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionDDX::UMaterialExpressionDDX(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDDX::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ValueInput = INDEX_NONE;

	if(Value.GetTracedInput().Expression)
	{
		ValueInput = Value.Compile(Compiler);
	}

	if(ValueInput == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->DDX(ValueInput);
}


void UMaterialExpressionDDX::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("DDX"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionDDY
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionDDY::UMaterialExpressionDDY(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionDDY::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ValueInput = INDEX_NONE;

	if(Value.GetTracedInput().Expression)
	{
		ValueInput = Value.Compile(Compiler);
	}

	if(ValueInput == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->DDY(ValueInput);
}


void UMaterialExpressionDDY::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("DDY"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle relative time material expression.
------------------------------------------------------------------------------*/
#if WITH_EDITOR
int32 UMaterialExpressionParticleRelativeTime::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleRelativeTime();
}

void UMaterialExpressionParticleRelativeTime::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Relative Time"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle motion blur fade material expression.
------------------------------------------------------------------------------*/
#if WITH_EDITOR
int32 UMaterialExpressionParticleMotionBlurFade::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleMotionBlurFade();
}

void UMaterialExpressionParticleMotionBlurFade::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Motion Blur Fade"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle motion blur fade material expression.
------------------------------------------------------------------------------*/
#if WITH_EDITOR
int32 UMaterialExpressionParticleRandom::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleRandom();
}

void UMaterialExpressionParticleRandom::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Random Value"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle direction material expression.
------------------------------------------------------------------------------*/
#if WITH_EDITOR
int32 UMaterialExpressionParticleDirection::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleDirection();
}

void UMaterialExpressionParticleDirection::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Direction"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle speed material expression.
------------------------------------------------------------------------------*/
#if WITH_EDITOR
int32 UMaterialExpressionParticleSpeed::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleSpeed();
}

void UMaterialExpressionParticleSpeed::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Speed"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle size material expression.
------------------------------------------------------------------------------*/
#if WITH_EDITOR
int32 UMaterialExpressionParticleSize::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleSize();
}

void UMaterialExpressionParticleSize::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Size"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Particle sprite rotation material expression.
------------------------------------------------------------------------------*/
UMaterialExpressionParticleSpriteRotation::UMaterialExpressionParticleSpriteRotation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Emplace(TEXT("Rad"), 1, 1, 0, 0, 0);
	Outputs.Emplace(TEXT("Deg"), 1, 0, 1, 0, 0);

	bShowOutputNameOnPin = true;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionParticleSpriteRotation::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->ParticleSpriteRotation();
}

void UMaterialExpressionParticleSpriteRotation::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Particle Sprite Rotation"));
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	Atmospheric fog material expression.
------------------------------------------------------------------------------*/
UMaterialExpressionAtmosphericFogColor::UMaterialExpressionAtmosphericFogColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = false;
#endif
}

#if WITH_EDITOR
FName UMaterialExpressionAtmosphericFogColor::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &WorldPosition)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionAtmosphericFogColor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionAtmosphericFogColor::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 WorldPositionInput = INDEX_NONE;

	if( WorldPosition.GetTracedInput().Expression )
	{
		WorldPositionInput = WorldPosition.Compile(Compiler);
	}

	return Compiler->AtmosphericFogColor( WorldPositionInput, WorldPositionOriginType );
}

void UMaterialExpressionAtmosphericFogColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Atmospheric Fog Color (deprecated)"));
}

/*------------------------------------------------------------------------------
	SpeedTree material expression.
------------------------------------------------------------------------------*/

int32 UMaterialExpressionSpeedTree::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 GeometryArg = (GeometryInput.GetTracedInput().Expression ? GeometryInput.Compile(Compiler) : Compiler->Constant(GeometryType));
	int32 WindArg = (WindInput.GetTracedInput().Expression ? WindInput.Compile(Compiler) : Compiler->Constant(WindType));
	int32 LODArg = (LODInput.GetTracedInput().Expression ? LODInput.Compile(Compiler) : Compiler->Constant(LODType));
	
	bool bExtraBend = (ExtraBendWS.GetTracedInput().Expression != nullptr);
	int32 ExtraBendArg = (ExtraBendWS.GetTracedInput().Expression ? ExtraBendWS.Compile(Compiler) : Compiler->Constant3(0.0f, 0.0f, 0.0f));
	 
	return Compiler->SpeedTree(GeometryArg, WindArg, LODArg, BillboardThreshold, bAccurateWindVelocities, bExtraBend, ExtraBendArg);
}

void UMaterialExpressionSpeedTree::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SpeedTree"));
}
#endif // WITH_EDITOR

void UMaterialExpressionSpeedTree::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);

	if (Record.GetUnderlyingArchive().UEVer() < VER_UE4_SPEEDTREE_WIND_V7)
	{
		// update wind presets for speedtree v7
		switch (WindType)
		{
		case STW_Fastest:
			WindType = STW_Better;
			break;
		case STW_Fast:
			WindType = STW_Palm;
			break;
		case STW_Better:
			WindType = STW_Best;
			break;
		default:
			break;
		}
	}
}

#if WITH_EDITOR

bool UMaterialExpressionSpeedTree::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);

	if (GeometryType == STG_Billboard)
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, LODType))
		{
			bIsEditable = false;
		}
	}
	else
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, BillboardThreshold))
		{
			bIsEditable = false;
		}
	}

	return bIsEditable;
}

#endif

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionEyeAdaptation
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionEyeAdaptation::UMaterialExpressionEyeAdaptation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("EyeAdaptation")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionEyeAdaptation::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{    
	return Compiler->EyeAdaptation();
}

void UMaterialExpressionEyeAdaptation::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("EyeAdaptation"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionEyeAdaptationInverse
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionEyeAdaptationInverse::UMaterialExpressionEyeAdaptationInverse(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("EyeAdaptationInverse")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionEyeAdaptationInverse::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{    
	int32 LightValueArg = (LightValueInput.GetTracedInput().Expression ? LightValueInput.Compile(Compiler) : Compiler->Constant3(1.0f,1.0f,1.0f));
	int32 AlphaArg = (AlphaInput.GetTracedInput().Expression ? AlphaInput.Compile(Compiler) : Compiler->Constant(1.0f));

	return Compiler->EyeAdaptationInverse(LightValueArg,AlphaArg);
}

void UMaterialExpressionEyeAdaptationInverse::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("EyeAdaptationInverse"));
}
#endif // WITH_EDITOR

//
// UMaterialExpressionTangentOutput
//
UMaterialExpressionTangentOutput::UMaterialExpressionTangentOutput(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// No outputs
	Outputs.Reset();
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionTangentOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if( Input.GetTracedInput().Expression )
	{
		return Compiler->CustomOutput(this, OutputIndex, Input.Compile(Compiler));
	}
	else
	{
		return CompilerError(Compiler, TEXT("Input missing"));
	}
}

void UMaterialExpressionTangentOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Tangent output"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// Clear Coat Custom Normal Input
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionClearCoatNormalCustomOutput::UMaterialExpressionClearCoatNormalCustomOutput(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;

	// No outputs
	Outputs.Reset();
#endif
}

#if WITH_EDITOR
int32  UMaterialExpressionClearCoatNormalCustomOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Input.GetTracedInput().Expression)
	{
		return Compiler->CustomOutput(this, OutputIndex, Input.Compile(Compiler));
	}
	return CompilerError(Compiler, TEXT("Input missing"));
}


void UMaterialExpressionClearCoatNormalCustomOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("ClearCoatBottomNormal"));
}

FExpressionInput* UMaterialExpressionClearCoatNormalCustomOutput::GetInput(int32 InputIndex)
{
	return InputIndex == 0 ? &Input : nullptr;
}
#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// Bent Normal Output
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionBentNormalCustomOutput::UMaterialExpressionBentNormalCustomOutput(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;

	// No outputs
	Outputs.Reset();
#endif
}

#if WITH_EDITOR
int32  UMaterialExpressionBentNormalCustomOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Input.GetTracedInput().Expression)
	{
		return Compiler->CustomOutput(this, OutputIndex, Input.Compile(Compiler));
	}
	else
	{
		return CompilerError(Compiler, TEXT("Input missing"));
	}
}

void UMaterialExpressionBentNormalCustomOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("BentNormal"));
}

FExpressionInput* UMaterialExpressionBentNormalCustomOutput::GetInput(int32 InputIndex)
{
	return InputIndex == 0 ? &Input : nullptr;
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// Vertex to pixel interpolated data handler
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionVertexInterpolator::UMaterialExpressionVertexInterpolator(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("PS"), 0, 0, 0, 0, 0));
	bShowOutputNameOnPin = true;
#endif

	InterpolatorIndex = INDEX_NONE;
	InterpolatedType = MCT_Unknown;
	InterpolatorOffset = INDEX_NONE;
}

#if WITH_EDITOR
int32 UMaterialExpressionVertexInterpolator::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Input.GetTracedInput().Expression)
	{
		if (Compiler->IsVertexInterpolatorBypass())
		{
			// Certain types of compilers don't support vertex interpolators, just evaluate the input directly in that case
			return Input.Compile(Compiler);
		}
		else if (InterpolatorIndex == INDEX_NONE || CompileErrors.Num() > 0)
		{
			// Now this node is confirmed part of the graph, append all errors from the input compilation
			check(CompileErrors.Num() == CompileErrorExpressions.Num());
			for (int32 Error = 0; Error < CompileErrors.Num(); ++Error)
			{
				if (CompileErrorExpressions[Error])
				{
					Compiler->AppendExpressionError(CompileErrorExpressions[Error], *CompileErrors[Error]);
				}
				else
				{
					Compiler->Errorf(*CompileErrors[Error]);
				}
			}
			
			return Compiler->Errorf(TEXT("Failed to compile interpolator input."));
		}
		else
		{
			return Compiler->VertexInterpolator(InterpolatorIndex);
		}
	}
	else
	{
		return CompilerError(Compiler, TEXT("Input missing"));
	}
}

int32 UMaterialExpressionVertexInterpolator::CompileInput(class FMaterialCompiler* Compiler, int32 AssignedInterpolatorIndex)
{
	int32 Ret = INDEX_NONE;
	InterpolatorIndex = INDEX_NONE;
	InterpolatedType = MCT_Unknown;
	InterpolatorOffset = INDEX_NONE;

	ensure(!Compiler->IsVertexInterpolatorBypass());

	CompileErrors.Empty();
	CompileErrorExpressions.Empty();

	if (Input.GetTracedInput().Expression)
	{
		int32 InternalCode = Input.Compile(Compiler);
		EMaterialValueType Type = Compiler->GetType(InternalCode);
		if (IsLWCType(Type))
		{
			// Don't support LWC interpolators for now
			// Possible to do this with more complex allocation scheme, if we interpolate tile coordinate along with offset
			Type = MakeNonLWCType(Type);
			InternalCode = Compiler->ValidCast(InternalCode, Type);
		}

		Compiler->CustomOutput(this, AssignedInterpolatorIndex, InternalCode);
		InterpolatorIndex = AssignedInterpolatorIndex;
		InterpolatedType = Type;
		Ret = InternalCode;
	}

	return Ret;
}

void UMaterialExpressionVertexInterpolator::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("VertexInterpolator"));
}

FExpressionInput* UMaterialExpressionVertexInterpolator::GetInput(int32 InputIndex)
{
	return InputIndex == 0 ? &Input : nullptr;
}

EMaterialValueType UMaterialExpressionVertexInterpolator::GetInputValueType(int32 InputIndex)
{
	// New HLSL generator allows struct interpolators
	return IsUsingNewHLSLGenerator() ? MCT_Unknown : MCT_Float4;
}

EMaterialValueType UMaterialExpressionVertexInterpolator::GetOutputValueType(int32 OutputIndex)
{
	return IsUsingNewHLSLGenerator() ? MCT_Unknown : UMaterialExpression::GetOutputValueType(OutputIndex);
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionrAtmosphericLightVector
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
void UMaterialExpressionAtmosphericLightVector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Atmosphere Sun Light Vector"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionrAtmosphericLightColor
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
void UMaterialExpressionAtmosphericLightColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Atmosphere Sun Light Illuminance On Ground (not per pixel)"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereLightIlluminance
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
FName UMaterialExpressionSkyAtmosphereLightIlluminance::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &WorldPosition)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionSkyAtmosphereLightIlluminance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionSkyAtmosphereLightIlluminance::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 WorldPositionInput;
	if (WorldPosition.GetTracedInput().Expression)
	{
		WorldPositionInput = WorldPosition.Compile(Compiler);
	}
	else
	{
		WorldPositionInput = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}
	return Compiler->SkyAtmosphereLightIlluminance(WorldPositionInput, WorldPositionOriginType, LightIndex);
}

void UMaterialExpressionSkyAtmosphereLightIlluminance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("SkyAtmosphereLightIlluminance[%i]"), LightIndex));
}

void UMaterialExpressionSkyAtmosphereLightIlluminance::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns the light illuminance at the specified world position. This node samples the transmitance texture once."), 80, OutToolTip);

}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereLightIlluminanceOnGround
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionSkyAtmosphereLightIlluminanceOnGround::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->SkyAtmosphereLightIlluminanceOnGround(LightIndex);
}

void UMaterialExpressionSkyAtmosphereLightIlluminanceOnGround::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("SkyAtmosphereLightIlluminanceOnGround[%i]"), LightIndex));
}

void UMaterialExpressionSkyAtmosphereLightIlluminanceOnGround::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Returns the light illuminance at level = 0 meter on the ground at the top of the planet. This node is cheap: it does not sample the transmitance texture."), 80, OutToolTip);

}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereLightDirection
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionSkyAtmosphereLightDirection::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->SkyAtmosphereLightDirection(LightIndex);
}

void UMaterialExpressionSkyAtmosphereLightDirection::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("SkyAtmosphereLightDirection[%i]"), LightIndex));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereLightDiskLuminance
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionSkyAtmosphereLightDiskLuminance::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CosHalfDiskRadiusCodeChunk = INDEX_NONE;
	if (DiskAngularDiameterOverride.GetTracedInput().Expression)
	{
		// Convert from apex angle (angular diameter) to cosine of the disk radius.
		CosHalfDiskRadiusCodeChunk = Compiler->Cosine(Compiler->Mul(Compiler->Constant(0.5f * float(UE_PI) / 180.0f), DiskAngularDiameterOverride.Compile(Compiler)));
	}
	return Compiler->SkyAtmosphereLightDiskLuminance(LightIndex, CosHalfDiskRadiusCodeChunk);
}

void UMaterialExpressionSkyAtmosphereLightDiskLuminance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString::Printf(TEXT("SkyAtmosphereLightDiskLuminance[%i]"), LightIndex));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereViewLuminance
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionSkyAtmosphereViewLuminance::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 WorldDirectionCodeChunk = INDEX_NONE;
	if (WorldDirection.GetTracedInput().Expression)
	{
		WorldDirectionCodeChunk = WorldDirection.Compile(Compiler);
	}
	return Compiler->SkyAtmosphereViewLuminance(WorldDirectionCodeChunk);
}

void UMaterialExpressionSkyAtmosphereViewLuminance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SkyAtmosphereViewLuminance"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereAerialPerpsective
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
FName UMaterialExpressionSkyAtmosphereAerialPerspective::GetInputName(int32 InputIndex) const
{
	if (GetInput(InputIndex) == &WorldPosition)
	{
		return UE::MaterialTranslatorUtils::GetWorldPositionInputName(WorldPositionOriginType);
	}

	return Super::GetInputName(InputIndex);
}

void UMaterialExpressionSkyAtmosphereAerialPerspective::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, WorldPositionOriginType))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionSkyAtmosphereAerialPerspective::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 WorldPositionInput;
	if (WorldPosition.GetTracedInput().Expression)
	{
		WorldPositionInput = WorldPosition.Compile(Compiler);
	}
	else
	{
		WorldPositionInput = Compiler->WorldPosition(UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
	}
	return Compiler->SkyAtmosphereAerialPerspective(WorldPositionInput, WorldPositionOriginType);
}

void UMaterialExpressionSkyAtmosphereAerialPerspective::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SkyAtmosphereAerialPerspective"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyAtmosphereAerialPerpsective
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionSkyAtmosphereDistantLightScatteredLuminance::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->SkyAtmosphereDistantLightScatteredLuminance();
}

void UMaterialExpressionSkyAtmosphereDistantLightScatteredLuminance::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SkyAtmosphereDistantLightScatteredLuminance"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionMainDirectionalLight
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionMainDirectionalLight::UMaterialExpressionMainDirectionalLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FConstructorStatics
	{
		FText NAME_MainDirectionalLight;
		FConstructorStatics() : NAME_MainDirectionalLight(LOCTEXT("Forward Selected Directional Light", "Forward Selected Directional Light")) {}
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_MainDirectionalLight);
#endif

	bShowOutputNameOnPin = true;

#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Illuminance")));
	Outputs.Add(FExpressionOutput(TEXT("Direction")));
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionMainDirectionalLight::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (OutputIndex == 0)
	{
		return Compiler->MainDirectionalLightIlluminance();
	}
	else if (OutputIndex == 1)
	{
		return Compiler->MainDirectionalLightDirection();
	}
	check(false);
	return INDEX_NONE;
}

void UMaterialExpressionMainDirectionalLight::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Forward Selected Directional Light"));
}

EMaterialValueType UMaterialExpressionMainDirectionalLight::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Float3; // Direction and illuminance are float3
}

void UMaterialExpressionMainDirectionalLight::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	OutToolTip.Add(FText(LOCTEXT("MainDirectionalLightTT", "This node returns attributes from the selected main directional light used for the forward shading and lighting of translucent, fog or single layer water. It accounts for ForwardShadingPriority then intensity.")).ToString());
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPreSkinnedPosition
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPreSkinnedPosition::UMaterialExpressionPreSkinnedPosition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 1, 0));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionPreSkinnedPosition::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Pre-Skinned Local Position (deprecated)"));
}

void UMaterialExpressionPreSkinnedPosition::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Deprecated, has been merged into 'Local Position'."
	                               "Returns pre-skinned local position for skeletal meshes, usable in vertex shader only."
	                               "Returns the local position for non-skeletal meshes. Incompatible with GPU skin cache feature."), 40, OutToolTip);
}

FText UMaterialExpressionPreSkinnedPosition::GetCreationDescription() const
{
	return LOCTEXT("PreSkinnedPositionCreationDesc", "Deprecated, has been merged into 'Local Position' node.");
}

FText UMaterialExpressionPreSkinnedPosition::GetCreationName() const
{
	return LOCTEXT("PreSkinnedPositionCreationName", "PreSkinnedPosition (Deprecated)");
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionPreSkinnedNormal
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionPreSkinnedNormal::UMaterialExpressionPreSkinnedNormal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 1, 0));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionPreSkinnedNormal::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Pre-Skinned Local Normal"));
}

void UMaterialExpressionPreSkinnedNormal::GetExpressionToolTip(TArray<FString>& OutToolTip) 
{
	ConvertToMultilineToolTip(TEXT("Returns pre-skinned local normal for skeletal meshes, usable in vertex shader only."
		"Returns the local normal for non-skeletal meshes. Incompatible with GPU skin cache feature."), 40, OutToolTip);
}
#endif // WITH_EDITOR

//
//  UMaterialExpressionCurveAtlasRowParameter
//
UMaterialExpressionCurveAtlasRowParameter::UMaterialExpressionCurveAtlasRowParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bCollapsed = true;
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 0, 0, 0, 1));
#endif // WITH_EDITORONLY_DATA

}

UObject* UMaterialExpressionCurveAtlasRowParameter::GetReferencedTexture() const
{
	return Atlas;
};

#if WITH_EDITOR

void UMaterialExpressionCurveAtlasRowParameter::GetTexturesForceMaterialRecompile(TArray<UTexture *> &Textures) const
{	
	if (Atlas)
	{
		Textures.AddUnique(Atlas);
	}
}

int32 UMaterialExpressionCurveAtlasRowParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Some error checking. Note that when bUseCustomPrimitiveData is true we don't rely on the Curve at all
	if (Atlas == nullptr && Curve == nullptr)
	{
		if (bUseCustomPrimitiveData)
		{
			return CompilerError(Compiler, TEXT("The atlas are not currently set."));
		}
		else
		{
			return CompilerError(Compiler, TEXT("The curve and atlas are not currently set."));
		}
	}
	else if (Curve == nullptr && !bUseCustomPrimitiveData)
	{
		return CompilerError(Compiler, TEXT("The curve is not currently set."));
	}
	else if (Atlas == nullptr)
	{
		return CompilerError(Compiler, TEXT("The atlas is not currently set."));
	}

	int32 CurveIndex = 0;
	int32 Slot = INDEX_NONE;

	// Support for using the Custom Primitive Data to fetch an atlas index if that is chosen
	if (bUseCustomPrimitiveData)
	{
		if (Material && Material->MaterialDomain == MD_UI)
		{
			return CompilerError(Compiler,CPD_UI_ErrorMessage);
		}

		Slot = Compiler->CustomPrimitiveData(PrimitiveDataIndex, MCT_Float);

		if (Slot == INDEX_NONE)
		{
			return CompilerError(Compiler, TEXT("Failed to compile the Custom Primitive Data index code."));
		}
	}
	else if (Curve && Atlas->GetCurveIndex(Curve, CurveIndex))
	{
		// Retrieve the curve index directly from the atlas rather than relying on the scalar parameter defaults
		DefaultValue = (float)CurveIndex;
		Slot = Compiler->ScalarParameter(ParameterName, DefaultValue);

		if (Slot == INDEX_NONE)
		{
			return CompilerError(Compiler, TEXT("The curve is not contained within the atlas."));
		}
	}

	// Get Atlas texture object and texture size
	int32 AtlasRef = INDEX_NONE;
	int32 AtlasCode = Compiler->Texture(Atlas, AtlasRef, SAMPLERTYPE_LinearColor, SSM_Clamp_WorldGroupSettings, TMVM_None);
	if (AtlasCode != INDEX_NONE)
	{
		int32 AtlasSize = Compiler->TextureProperty(AtlasCode, TMTM_TextureSize);
		int32 AtlasWidth  = Compiler->ComponentMask(AtlasSize, true, false, false, false);
		int32 AtlasHeight = Compiler->ComponentMask(AtlasSize, false, true, false, false);

		// Calculate UVs from height and slot
		// if the input is hooked up, use it, otherwise use the internal constant
		int32 ArgT;
		if ( InputTime.GetTracedInput().Expression )
		{
			ArgT = InputTime.Compile(Compiler);
		}
		else
		{
			ArgT = Compiler->Constant(0);
		}

		// ArgT is [t] the parameter on the curve
		//	adjust so that [0,1] in t corresponds to the first to last pixel center
		//		(CurveLinearColorAtlas has no mips, so that's not an issue)
		//	Arg1 = (t*(width-1) + 0.5)/width
		int32 Arg1 = Compiler->Div(Compiler->Add( Compiler->Mul(ArgT,Compiler->Sub(AtlasWidth,Compiler->Constant(1.0))) , Compiler->Constant(0.5)),AtlasWidth);
		
		// Arg2 is the curve index
		//	each row of the atlas texture can be a different curve
		//	Arg2 = (index + 0.5)/height so it is centered vertically on a pixel row
		int32 Arg2 = Compiler->Div(Compiler->Add(Slot, Compiler->Constant(0.5)), AtlasHeight);

		int32 UV = Compiler->AppendVector(Arg1, Arg2);

		// Sample texture
		return Compiler->TextureSample(AtlasCode, UV, SAMPLERTYPE_LinearColor, INDEX_NONE, INDEX_NONE, TMVM_None, SSM_Clamp_WorldGroupSettings, TGM_None, AtlasRef, false);
	}
	else
	{
		return CompilerError(Compiler, TEXT("There was an error when compiling the texture."));
	}
}

bool UMaterialExpressionCurveAtlasRowParameter::GetParameterValue(FMaterialParameterMetadata& OutMeta) const
{
	float Value = 0.0f;
	if (Atlas && Curve)
	{
		const int32 SlotIndex = Atlas->GradientCurves.Find(Curve);
		if (SlotIndex != INDEX_NONE)
		{
			Value = (float)SlotIndex;
		}
		OutMeta.bUsedAsAtlasPosition = true;
		OutMeta.ScalarCurve = Curve;
		OutMeta.ScalarAtlas = Atlas;
	}

	OutMeta.Value = Value;
	OutMeta.ExpressionGuid = ExpressionGUID;
	OutMeta.Group = Group;
	OutMeta.SortPriority = SortPriority;
	return true;
}

bool UMaterialExpressionCurveAtlasRowParameter::SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (Name == ParameterName &&
		Meta.Value.Type == EMaterialParameterType::Scalar &&
		Meta.bUsedAsAtlasPosition)
	{
		const bool bHasCurveChanged = (Curve != Meta.ScalarCurve.Get());
		const bool bHasAtlasChanged = (Atlas != Meta.ScalarAtlas.Get());
		const bool bHasDefaultValueChanged = (DefaultValue != Meta.Value.AsScalar());
		if (bHasCurveChanged || bHasAtlasChanged || bHasDefaultValueChanged)
		{
			Curve = Meta.ScalarCurve.Get();
			Atlas = Meta.ScalarAtlas.Get();
			DefaultValue = Meta.Value.AsScalar();
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				if (bHasCurveChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Curve));
				}
				if (bHasAtlasChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Atlas));
				}
				if (bHasDefaultValueChanged)
				{
					SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, DefaultValue));
				}
			}
		}

		if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
		{
			Group = Meta.Group;
			SortPriority = Meta.SortPriority;
		}
		return true;
	}
	return false;
}

void UMaterialExpressionCurveAtlasRowParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Atlas) || PropertyName == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, Curve))
	{
		int32 SlotIndex = INDEX_NONE;
		if (Atlas && Curve)
		{
			SlotIndex = Atlas->GradientCurves.Find(Curve);
		}
		if (SlotIndex != INDEX_NONE)
		{
			DefaultValue = (float)SlotIndex;
		}
		else
		{
			DefaultValue = 0.0f;
		}
	}

	// Need to update expression properties before super call (which triggers recompile)
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif


//
// Hair attributes
//

UMaterialExpressionHairAttributes::UMaterialExpressionHairAttributes(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("U"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("V"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Length"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Radius"), 1, 0, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Seed"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Tangent"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Root UV"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("BaseColor"), 1, 1, 1, 1, 0));
	Outputs.Add(FExpressionOutput(TEXT("Roughness"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Depth"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Coverage"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("AuxilaryData"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("AtlasUVs"), 1, 1, 1, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Group Index"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("AO"), 1, 1, 0, 0, 0));
	Outputs.Add(FExpressionOutput(TEXT("Clump ID"), 1, 1, 0, 0, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionHairAttributes::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (OutputIndex == 0 || OutputIndex == 1)
	{
		return Compiler->GetHairUV();
	}
	else if (OutputIndex == 2 || OutputIndex == 3)
	{
		return Compiler->GetHairDimensions();
	}
	else if (OutputIndex == 4)
	{
		return Compiler->GetHairSeed();
	}
	else if (OutputIndex == 5)
	{
		return Compiler->GetHairTangent(bUseTangentSpace);
	}
	else if (OutputIndex == 6)
	{
		return Compiler->GetHairRootUV();
	}
	else if (OutputIndex == 7)
	{
		return Compiler->GetHairBaseColor();
	}
	else if (OutputIndex == 8)
	{
		return Compiler->GetHairRoughness();
	}
	else if (OutputIndex == 9)
	{
		return Compiler->GetHairDepth();
	}
	else if (OutputIndex == 10)
	{
		return Compiler->GetHairCoverage();
	}
	else if (OutputIndex == 11)
	{
		return Compiler->GetHairAuxilaryData();
	}
	else if (OutputIndex == 12)
	{
		return Compiler->GetHairAtlasUVs();
	}
	else if (OutputIndex == 13)
	{
		return Compiler->GetHairGroupIndex();
	}
	else if (OutputIndex == 14)
	{
		return Compiler->GetHairAO();
	}
	else if (OutputIndex == 15)
	{
		return Compiler->GetHairClumpID();
	}

	return Compiler->Errorf(TEXT("Invalid input parameter"));
}

void UMaterialExpressionHairAttributes::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Hair Attributes"));
}

void UMaterialExpressionHairAttributes::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (OutputIndex == 0)
	{
		OutToolTip.Add(TEXT("The U coordinates of hair. Coordinate along the hair, with 0 being the root and 1 the tip."));
		return;
	}
	else if (OutputIndex == 1)
	{
		OutToolTip.Add(TEXT("The V coordinates of hair. Coordinate across the hair, with 0 being the left and 1 the right."));
		return;
	}
	else if (OutputIndex == 2)
	{
		OutToolTip.Add(TEXT("The length of the current curve. Length from the root to the shading point, in cm."));
		return;
	}
	else if (OutputIndex == 3)
	{
		OutToolTip.Add(TEXT("Radius of the curve at the current position."));
		return;
	}
	else if (OutputIndex == 4)
	{
		OutToolTip.Add(TEXT("Random value in 0 to 1, and uniform along the curve."));
		return;
	}
	else if (OutputIndex == 5)
	{
		OutToolTip.Add(TEXT("Tangent vector aligned in the direction of the curve."));
		return;
	}
	else if (OutputIndex == 6)
	{
		OutToolTip.Add(TEXT("UV of the underlying mesh at the curve's root position."));
		return;
	}
	else if (OutputIndex == 7)
	{
		OutToolTip.Add(TEXT("Per curve's point color."));
		return;
	}
	else if (OutputIndex == 8)
	{
		OutToolTip.Add(TEXT("Per curve's point roughness."));
		return;
	}
	else if (OutputIndex == 9)
	{
		OutToolTip.Add(TEXT("Depth offset. Only used for cards and mesh geometry."));
		return;
	}
	else if (OutputIndex == 10)
	{
		OutToolTip.Add(TEXT("Coverage mask value. Only used for cards and mesh geometry."));
		return;
	}
	else if (OutputIndex == 11)
	{
		OutToolTip.Add(TEXT("Auxiliary data that is only used for cards and mesh geometry."));
		return;
	}
	else if (OutputIndex == 12)
	{
		OutToolTip.Add(TEXT("Cards UVs that are only used for cards and mesh geometry."));
		return;
	}
	else if (OutputIndex == 13)
	{
		OutToolTip.Add(TEXT("The group index of the curve."));
		return;
	}
	else if (OutputIndex == 14)
	{
		OutToolTip.Add(TEXT("Per curve's ambient occlusion."));
		return;
	}
	else if (OutputIndex == 15)
	{
		OutToolTip.Add(TEXT("The clump ID of the curve."));
		return;
	}

	Super::GetConnectorToolTip(InputIndex, INDEX_NONE, OutToolTip);
}
#endif // WITH_EDITOR

//
// Hair Color
//

UMaterialExpressionHairColor::UMaterialExpressionHairColor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Color"), 1, 1, 1, 1, 0));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionHairColor::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 MelaninInput = Melanin.GetTracedInput().Expression ? Melanin.Compile(Compiler) : Compiler->Constant(0.5f);
	int32 RednessInput = Redness.GetTracedInput().Expression ? Redness.Compile(Compiler) : Compiler->Constant(0.0f);
	int32 DyeColorInput = DyeColor.GetTracedInput().Expression ? DyeColor.Compile(Compiler) : Compiler->Constant3(1.f,1.f, 1.f);

	return Compiler->GetHairColorFromMelanin(
		MelaninInput,
		RednessInput,
		DyeColorInput);
}

void UMaterialExpressionHairColor::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Hair Color"));
}
#endif // WITH_EDITOR

//
//  UMaterialExpressionARPassthroughCameraUVs
//
#if WITH_EDITOR
int32 UMaterialExpressionMapARPassthroughCameraUV::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Coordinates.GetTracedInput().Expression)
	{
		return CompilerError(Compiler, TEXT("UV input missing"));
	}
	else
	{
		int32 Index = Coordinates.Compile(Compiler);
		return Compiler->MapARPassthroughCameraUV(Index);
	}
}

void UMaterialExpressionMapARPassthroughCameraUV::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Map AR Passthrough Camera UVs"));
}
#endif // WITH_EDITOR
//
//	UMaterialExpressionShadingModel
//
UMaterialExpressionShadingModel::UMaterialExpressionShadingModel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA

	MenuCategories.Add(LOCTEXT("Shading Model", "Shading Model"));

	bShowOutputNameOnPin = true;
	
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionShadingModel::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Maybe add the shading model to the material here. Don't forget to clear the material shading model list before compilation though
	return Compiler->ShadingModel(ShadingModel);
}

int32 UMaterialExpressionShadingModel::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	return Compiler->Constant(0.f);
}

void UMaterialExpressionShadingModel::GetCaption(TArray<FString>& OutCaptions) const
{
	const UEnum* ShadingModelEnum = FindObject<UEnum>(nullptr, TEXT("Engine.EMaterialShadingModel"));
	check(ShadingModelEnum);

	const FString ShadingModelDisplayName = ShadingModelEnum->GetDisplayNameTextByValue(ShadingModel).ToString();

	// Add as a stack, last caption to be added will be the main (bold) caption
	OutCaptions.Add(ShadingModelDisplayName);
	OutCaptions.Add(TEXT("Shading Model"));
}

EMaterialValueType UMaterialExpressionShadingModel::GetOutputValueType(int32 OutputIndex)
{
	return MCT_ShadingModel;
}
#endif // WITH_EDITOR

UMaterialExpressionSingleLayerWaterMaterialOutput::UMaterialExpressionSingleLayerWaterMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionSingleLayerWaterMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	const bool bSubstrate = Substrate::IsSubstrateEnabled();

	if (!ScatteringCoefficients.IsConnected() && !AbsorptionCoefficients.IsConnected() && !PhaseG.IsConnected()
		&& !bSubstrate)
	{
		Compiler->Error(TEXT("No inputs to Single Layer Water Material."));
	}

	// Generates function names GetSingleLayerWaterMaterialOutput{index} used in BasePixelShader.usf.
	if (OutputIndex == 0)
	{
		CodeInput = ScatteringCoefficients.IsConnected() ? ScatteringCoefficients.Compile(Compiler) : Compiler->Constant3(0.f, 0.f, 0.f);
	}
	else if (OutputIndex == 1)
	{
		CodeInput = AbsorptionCoefficients.IsConnected() ? AbsorptionCoefficients.Compile(Compiler) : Compiler->Constant3(0.f, 0.f, 0.f);
	}
	else if (OutputIndex == 2)
	{
		CodeInput = PhaseG.IsConnected() ? PhaseG.Compile(Compiler) : Compiler->Constant(0.f);
	}
	else if (OutputIndex == 3)
	{
		CodeInput = ColorScaleBehindWater.IsConnected() ? ColorScaleBehindWater.Compile(Compiler) : Compiler->Constant(1.f);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionSingleLayerWaterMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Single Layer Water Material"));
}

#endif // WITH_EDITOR

int32 UMaterialExpressionSingleLayerWaterMaterialOutput::GetNumOutputs() const
{
	return 4;
}

FString UMaterialExpressionSingleLayerWaterMaterialOutput::GetFunctionName() const
{
	return TEXT("GetSingleLayerWaterMaterialOutput");
}

FString UMaterialExpressionSingleLayerWaterMaterialOutput::GetDisplayName() const
{
	return TEXT("Single Layer Water Material");
}


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionVolumetricAdvancedMaterialOutput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionVolumetricAdvancedMaterialOutput::UMaterialExpressionVolumetricAdvancedMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionVolumetricAdvancedMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	// Generates function names GetVolumetricAdvanceMaterialOutput{index} used in BasePixelShader.usf.
	if (OutputIndex == 0)
	{
		CodeInput = PhaseG.IsConnected() ? PhaseG.Compile(Compiler) : Compiler->Constant(ConstPhaseG);
	}
	else if (OutputIndex == 1)
	{
		CodeInput = PhaseG2.IsConnected() ? PhaseG2.Compile(Compiler) : Compiler->Constant(ConstPhaseG2);
	}
	else if (OutputIndex == 2)
	{
		CodeInput = PhaseBlend.IsConnected() ? PhaseBlend.Compile(Compiler) : Compiler->Constant(ConstPhaseBlend);
	}
	else if (OutputIndex == 3)
	{
		CodeInput = MultiScatteringContribution.IsConnected() ? MultiScatteringContribution.Compile(Compiler) : Compiler->Constant(ConstMultiScatteringContribution);
	}
	else if (OutputIndex == 4)
	{
		CodeInput = MultiScatteringOcclusion.IsConnected() ? MultiScatteringOcclusion.Compile(Compiler) : Compiler->Constant(ConstMultiScatteringOcclusion);
	}
	else if (OutputIndex == 5)
	{
		CodeInput = MultiScatteringEccentricity.IsConnected() ? MultiScatteringEccentricity.Compile(Compiler) : Compiler->Constant(ConstMultiScatteringEccentricity);
	}
	else if (OutputIndex == 6)
	{
		CodeInput = ConservativeDensity.IsConnected() ? ConservativeDensity.Compile(Compiler) : Compiler->Constant3(1.0f, 1.0f, 1.0f);

		// We force a cast to a float4 because that is what we use behind the scene and is needed by artists in some cases.
		CodeInput = Compiler->ForceCast(CodeInput, MCT_Float4);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionVolumetricAdvancedMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Volumetric Advanced Output"));
}

bool UMaterialExpressionVolumetricAdvancedMaterialOutput::GetEvaluatePhaseOncePerSample() const
{
	return PerSamplePhaseEvaluation && (PhaseG.IsConnected() || PhaseG2.IsConnected() || PhaseBlend.IsConnected());
}

uint32 UMaterialExpressionVolumetricAdvancedMaterialOutput::GetMultiScatteringApproximationOctaveCount() const
{
	return MultiScatteringApproximationOctaveCount;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionVolumetricAdvancedMaterialOutput::GetNumOutputs() const
{
	return 7;
}

FString UMaterialExpressionVolumetricAdvancedMaterialOutput::GetFunctionName() const
{
	return TEXT("GetVolumetricAdvancedMaterialOutput");
}

FString UMaterialExpressionVolumetricAdvancedMaterialOutput::GetDisplayName() const
{
	return TEXT("Volumetric Advanced Ouput");
}


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionVolumetricAdvancedMaterialInput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionVolumetricAdvancedMaterialInput::UMaterialExpressionVolumetricAdvancedMaterialInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("ConservativeDensity as Float3")));
	Outputs.Add(FExpressionOutput(TEXT("ConservativeDensity as Float4")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionVolumetricAdvancedMaterialInput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (OutputIndex == 0)
	{
		return Compiler->ForceCast(Compiler->GetVolumeSampleConservativeDensity(), MCT_Float3);
	}
	else if (OutputIndex == 1)
	{
		return Compiler->GetVolumeSampleConservativeDensity();
	}

	return Compiler->Errorf(TEXT("Invalid input parameter"));
}

void UMaterialExpressionVolumetricAdvancedMaterialInput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Volumetric Advanced Input"));
}
#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	if (OutputIndex == 0)
	{
		CodeInput = ContainsMatter.IsConnected() ? ContainsMatter.Compile(Compiler) : Compiler->Constant(1.0f);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Volumetric Cloud Empty Space Skipping Output <Experimental>"));
}

EMaterialValueType UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Float1;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::GetNumOutputs() const
{
	return 1;
}

FString UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::GetFunctionName() const
{
	return TEXT("GetVolumetricCloudEmptySpaceSkippingOutput");
}

FString UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput::GetDisplayName() const
{
	return TEXT("Volumetric Cloud Empty Space Skipping Output");
}


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionVolumetricCloudEmptySpaceSkippingInput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionVolumetricCloudEmptySpaceSkippingInput::UMaterialExpressionVolumetricCloudEmptySpaceSkippingInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Sphere Center")));
	Outputs.Add(FExpressionOutput(TEXT("Sphere Radius")));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionVolumetricCloudEmptySpaceSkippingInput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Volumetric Cloud Empty Space Skipping Input <Experimental>"));
}

void UMaterialExpressionVolumetricCloudEmptySpaceSkippingInput::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (OutputIndex == 0)
	{
		OutToolTip.Add(TEXT("The position of the center of the spherical area considered for empty space skipping. When evaluating cloud for empty space skipping, this output returns the same as the WorldPosition node."));
		return;
	}
	else if (OutputIndex == 1)
	{
		OutToolTip.Add(TEXT("The radius of the spherical area considered for empty space skipping."));
		return;
	}
	Super::GetConnectorToolTip(InputIndex, OutputIndex, OutToolTip);
}

EMaterialValueType UMaterialExpressionVolumetricCloudEmptySpaceSkippingInput::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_Float3;
	}
	else if (InputIndex == 1)
	{
		return MCT_Float1;
	}
	return MCT_Unknown;
}
#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionReflectionCapturePassSwitch
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionReflectionCapturePassSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Default.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Default"));
	}
	else if (!Reflection.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input Reflection"));
	}
	else
	{
		const int32 Arg1 = Default.Compile(Compiler);
		const int32 Arg2 = Reflection.Compile(Compiler);

		return Compiler->ReflectionCapturePassSwitch(Arg1, Arg2);
	}
}

void UMaterialExpressionReflectionCapturePassSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Reflection Capture Pass Switch"));
}

void UMaterialExpressionReflectionCapturePassSwitch::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Allows material to define specialized behavior when being rendered into reflection capture views."), 40, OutToolTip);
}

#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionCloudSampleAttribute
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionCloudSampleAttribute::UMaterialExpressionCloudSampleAttribute(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Altitude")));
	Outputs.Add(FExpressionOutput(TEXT("AltitudeInLayer")));
	Outputs.Add(FExpressionOutput(TEXT("NormAltitudeInLayer")));
	Outputs.Add(FExpressionOutput(TEXT("ShadowSampleDistance")));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionCloudSampleAttribute::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Cloud Sample Attributes"));
}

void UMaterialExpressionCloudSampleAttribute::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(TEXT("Cloud sample attributes.\n- Altitude is the sample atlitude relative to the planet ground (centimeters).\n- AltitudeInLayer is the sample atlitude relative to the cloud layer bottom altitude (centimeters).\n- NormAltitudeInLayer is the normalised sample altitude within the cloud layer (0=bottom, 1=top).\n- ShadowSampleDistance is 0.0 if the sample is used to trace the cloud in view (primary view ray sample). If it is used to trace volumetric shadows, then it is greater than 0.0 and it represents the shadow sample distance in centimeter from the primary view ray sample (during secondary ray marching or Beer shadow map generation): This can help tweaking the shadow strength, skip some code using dynamic branching or sample texture lower mipmaps."), 80, OutToolTip);
}

#endif // WITH_EDITOR


/////////////////////////// THIN TRANSLUCENT


UMaterialExpressionThinTranslucentMaterialOutput::UMaterialExpressionThinTranslucentMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionThinTranslucentMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	// Generates function names GetSingleLayerWaterMaterialOutput{index} used in BasePixelShader.usf.
	if (OutputIndex == 0)
	{
		CodeInput = TransmittanceColor.IsConnected() ? TransmittanceColor.Compile(Compiler) : Compiler->Constant3(0.5f, 0.5f, 0.5f);
	}
	if (OutputIndex == 1)
	{
		CodeInput = SurfaceCoverage.IsConnected() ? SurfaceCoverage.Compile(Compiler) : Compiler->Constant(1.0f);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionThinTranslucentMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Thin Translucent Material"));
}

EMaterialValueType UMaterialExpressionThinTranslucentMaterialOutput::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float1;
	}

	check(false);
	return MCT_Float1;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionThinTranslucentMaterialOutput::GetNumOutputs() const
{
	return 2;
}

FString UMaterialExpressionThinTranslucentMaterialOutput::GetFunctionName() const
{
	return TEXT("GetThinTranslucentMaterialOutput");
}

FString UMaterialExpressionThinTranslucentMaterialOutput::GetDisplayName() const
{
	return TEXT("Thin Translucent Material");
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionAbsorptionMediumMaterialOutput
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionAbsorptionMediumMaterialOutput::UMaterialExpressionAbsorptionMediumMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionAbsorptionMediumMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	if (OutputIndex == 0)
	{
		CodeInput = TransmittanceColor.IsConnected() ? TransmittanceColor.Compile(Compiler) : Compiler->Constant3(1.0f, 1.0f, 1.0f);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionAbsorptionMediumMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Absorption Medium (Path Tracer Only)"));
}

#endif // WITH_EDITOR

int32 UMaterialExpressionAbsorptionMediumMaterialOutput::GetNumOutputs() const
{
	return 1;
}

FString UMaterialExpressionAbsorptionMediumMaterialOutput::GetFunctionName() const
{
	return TEXT("GetAbsorptionMediumMaterialOutput");
}

FString UMaterialExpressionAbsorptionMediumMaterialOutput::GetDisplayName() const
{
	return TEXT("Absorption Medium");
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSceneDepthWithoutWater
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionSceneDepthWithoutWater::UMaterialExpressionSceneDepthWithoutWater()
{
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(LOCTEXT("Water", "Water"));

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT(""), 1, 1, 0, 0, 0));
#endif // WITH_EDITORONLY_DATA

	ConstInput = FVector2D(0.f, 0.f);
}

#if WITH_EDITOR
int32 UMaterialExpressionSceneDepthWithoutWater::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 OffsetIndex = INDEX_NONE;
	int32 CoordinateIndex = INDEX_NONE;
	bool bUseOffset = false;

	if (InputMode == EMaterialSceneAttributeInputMode::OffsetFraction)
	{
		if (Input.GetTracedInput().Expression)
		{
			OffsetIndex = Input.Compile(Compiler);
		}
		else
		{
			OffsetIndex = Compiler->Constant2(ConstInput.X, ConstInput.Y);
		}
		bUseOffset = true;
	}
	else if (InputMode == EMaterialSceneAttributeInputMode::Coordinates)
	{
		if (Input.GetTracedInput().Expression)
		{
			CoordinateIndex = Input.Compile(Compiler);
		}
	}

	int32 Result = Compiler->SceneDepthWithoutWater(OffsetIndex, CoordinateIndex, bUseOffset, FallbackDepth);
	return Result;
}

void UMaterialExpressionSceneDepthWithoutWater::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Scene Depth Without Water"));
}

FName UMaterialExpressionSceneDepthWithoutWater::GetInputName(int32 InputIndex) const
{
	if (InputIndex == 0)
	{
		// Display the current InputMode enum's display name.
		FByteProperty* InputModeProperty = FindFProperty<FByteProperty>(UMaterialExpressionSceneDepthWithoutWater::StaticClass(), GET_MEMBER_NAME_STRING_CHECKED(ThisClass, InputMode));
		// Can't use GetNameByValue as GetNameStringByValue does name mangling that GetNameByValue does not
		return *InputModeProperty->Enum->GetNameStringByValue((int64)InputMode.GetValue());
	}
	return NAME_None;
}

#endif // WITH_EDITOR

#if WITH_EDITOR
int32 UMaterialExpressionIfThenElse::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!Condition.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing IfThenElse Condition input"));
	}
	if (!True.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing IfThenElse True input"));
	}
	if (!False.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing IfThenElse False input"));
	}

	const int32 ConditionCode = Condition.Compile(Compiler);
	const int32 TrueCode = True.Compile(Compiler);
	const int32 FalseCode = False.Compile(Compiler);

	if (ConditionCode == INDEX_NONE || TrueCode == INDEX_NONE || FalseCode == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return Compiler->DynamicBranch(ConditionCode, TrueCode, FalseCode);
}

void UMaterialExpressionIfThenElse::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("If"));
}

EMaterialValueType UMaterialExpressionIfThenElse::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
		case 0: return MCT_Bool;
		case 1: return MCT_Unknown;
		case 2: return MCT_Unknown;
		default: checkNoEntry(); return {};
	}
}
	
EMaterialValueType UMaterialExpressionIfThenElse::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Unknown;
}

#endif

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSkyLightEnvMapSample
///////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
int32 UMaterialExpressionSkyLightEnvMapSample::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 DirectionCodeChunk = Direction.GetTracedInput().Expression ? Direction.Compile(Compiler) : Compiler->Constant3(0.0f, 0.0f, 1.0f);
	int32 RoughnessCodeChunk = Roughness.GetTracedInput().Expression ? Roughness.Compile(Compiler) : Compiler->Constant(0.0f);

	if (DirectionCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Direction input graph could not be evaluated for SkyLightEnvMapSample."));
	}
	if (RoughnessCodeChunk == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Roughness input graph could not be evaluated for SkyLightEnvMapSample."));
	}

	return Compiler->SkyLightEnvMapSample(DirectionCodeChunk, RoughnessCodeChunk);
}

void UMaterialExpressionSkyLightEnvMapSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Sky Light Env Map Sample"));
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSparseVolumeTextureBase
///////////////////////////////////////////////////////////////////////////////
UObject* UMaterialExpressionSparseVolumeTextureBase::GetReferencedTexture() const
{
	return SparseVolumeTexture;
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSparseVolumeTextureObject
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionSparseVolumeTextureObject::UMaterialExpressionSparseVolumeTextureObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Empty();
	Outputs.Add(FExpressionOutput(TEXT("")));

	bCollapsed = true;
	bHidePreviewWindow = true;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR

void UMaterialExpressionSparseVolumeTextureObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Update what needs to be when the texture is changed
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, SparseVolumeTexture))
	{
		if (SparseVolumeTexture != nullptr)
		{
			FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionSparseVolumeTextureObject::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!SparseVolumeTexture)
	{
		return CompilerError(Compiler, TEXT("Requires valid SparseVolumeTexture"));
	}

	int32 TextureReferenceIndex = INDEX_NONE;
	return Compiler->SparseVolumeTexture(SparseVolumeTexture, TextureReferenceIndex, SAMPLERTYPE_LinearColor);
}

void UMaterialExpressionSparseVolumeTextureObject::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("SparseVolumeTexture Object"));
}

EMaterialValueType UMaterialExpressionSparseVolumeTextureObject::GetOutputValueType(int32 OutputIndex)
{
	return MCT_SparseVolumeTexture;
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSparseVolumeTextureObjectParameter
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionSparseVolumeTextureObjectParameter::UMaterialExpressionSparseVolumeTextureObjectParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	Outputs.Empty();
	Outputs.Add(FExpressionOutput(TEXT("")));
#endif

#if WITH_EDITOR
	// Hide the texture coordinate input
	CachedInputs.Empty();
#endif
}

#if WITH_EDITOR

void UMaterialExpressionSparseVolumeTextureObjectParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Param Tex Object"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

int32 UMaterialExpressionSparseVolumeTextureObjectParameter::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!SparseVolumeTexture)
	{
		return CompilerError(Compiler, TEXT("Requires valid SparseVolumeTexture"));
	}

	int32 TextureReferenceIndex = INDEX_NONE;
	return Compiler->SparseVolumeTextureParameter(ParameterName, SparseVolumeTexture, TextureReferenceIndex, SAMPLERTYPE_LinearColor);
}

EMaterialValueType UMaterialExpressionSparseVolumeTextureObjectParameter::GetOutputValueType(int32 OutputIndex)
{
	return MCT_SparseVolumeTexture;
}

#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSparseVolumeTextureSample
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionSparseVolumeTextureSample::UMaterialExpressionSparseVolumeTextureSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Attributes A"), 1, 1, 1, 1, 1));
	Outputs.Add(FExpressionOutput(TEXT("Attributes B"), 1, 1, 1, 1, 1));
	bShowOutputNameOnPin = true;
	bShowMaskColorsOnPin = false;
#endif
}

#if WITH_EDITOR

bool UMaterialExpressionSparseVolumeTextureSample::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		FName PropertyFName = InProperty->GetFName();

		if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionSparseVolumeTextureSample, ConstMipValue))
		{
			bIsEditable = MipValueMode == TMVM_MipLevel || MipValueMode == TMVM_MipBias;
		}
		else if (PropertyFName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionSparseVolumeTextureSample, SparseVolumeTexture))
		{
			// The Texture property is overridden by a connection to TextureObject
			bIsEditable = TextureObject.GetTracedInput().Expression == nullptr;
		}
	}

	return bIsEditable;
}

void UMaterialExpressionSparseVolumeTextureSample::PostLoad()
{
	Super::PostLoad();
}

TArrayView<FExpressionInput*> UMaterialExpressionSparseVolumeTextureSample::GetInputsView()
{
	CachedInputs.Empty();
	uint32 InputIndex = 0;
	while (FExpressionInput* Ptr = GetInput(InputIndex++))
	{
		CachedInputs.Add(Ptr);
	}
	return CachedInputs;
}

void UMaterialExpressionSparseVolumeTextureSample::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Update what needs to be when the texture is changed
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(ThisClass, SparseVolumeTexture))
	{
		if (SparseVolumeTexture != nullptr)
		{
			FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
		}
	}

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, MipValueMode))
	{
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

EMaterialValueType UMaterialExpressionSparseVolumeTextureSample::GetOutputValueType(int32 OutputIndex)
{
	switch (OutputIndex)
	{
	case 0:
	case 1:
		return MCT_Float;	// Attributes A and B defined in constructor for now
	}

	check(false);
	return MCT_Float1;
}

// this define is only used for the following function
#define IF_INPUT_RETURN(Item) if(!InputIndex) return &Item; --InputIndex
FExpressionInput* UMaterialExpressionSparseVolumeTextureSample::GetInput(int32 InputIndex)
{
	IF_INPUT_RETURN(Coordinates);

	IF_INPUT_RETURN(TextureObject);

	if (MipValueMode == TMVM_Derivative)
	{
		IF_INPUT_RETURN(CoordinatesDX);
		IF_INPUT_RETURN(CoordinatesDY);
	}
	else if (MipValueMode != TMVM_None)
	{
		IF_INPUT_RETURN(MipValue);
	}

	return nullptr;
}
#undef IF_INPUT_RETURN

// this define is only used for the following function
#define IF_INPUT_RETURN(Name) if(!InputIndex) return Name; --InputIndex
FName UMaterialExpressionSparseVolumeTextureSample::GetInputName(int32 InputIndex) const
{
	// Coordinates
	IF_INPUT_RETURN(TEXT("Coordinates"));

	// TextureObject
	IF_INPUT_RETURN(TEXT("TextureObject"));

	if (MipValueMode == TMVM_MipLevel)
	{
		// MipValue
		IF_INPUT_RETURN(TEXT("MipLevel"));
	}
	else if (MipValueMode == TMVM_MipBias)
	{
		// MipValue
		IF_INPUT_RETURN(TEXT("MipBias"));
	}
	else if (MipValueMode == TMVM_Derivative)
	{
		// CoordinatesDX
		IF_INPUT_RETURN(TEXT("DDX(UVs)"));
		// CoordinatesDY
		IF_INPUT_RETURN(TEXT("DDY(UVs)"));
	}

	return TEXT("");
}
#undef IF_INPUT_RETURN

// this define is only used for the following function
#define IF_INPUT_RETURN(Type) if(!InputIndex) return (Type); --InputIndex

EMaterialValueType UMaterialExpressionSparseVolumeTextureSample::GetInputValueType(int32 InputIndex)
{
	// Coordinates
	IF_INPUT_RETURN(MCT_Float3);

	// TextureObject
	IF_INPUT_RETURN(MCT_SparseVolumeTexture);

	if (MipValueMode == TMVM_MipLevel || MipValueMode == TMVM_MipBias)
	{
		// MipValue
		IF_INPUT_RETURN(MCT_Float);
	}
	else if (MipValueMode == TMVM_Derivative)
	{
		// CoordinatesDX
		IF_INPUT_RETURN(MCT_Float);
		// CoordinatesDY
		IF_INPUT_RETURN(MCT_Float);
	}

	return MCT_Unknown;
}
#undef IF_INPUT_RETURN

int32 UMaterialExpressionSparseVolumeTextureSample::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	UMaterialExpression* InputExpression = TextureObject.GetTracedInput().Expression;

	if (SparseVolumeTexture || InputExpression)
	{
		const bool bIsParameter = HasAParameterName() && GetParameterName().IsValid() && !GetParameterName().IsNone();

		int32 SparseVolumeTextureIndex = INDEX_NONE;
		if (InputExpression)
		{
			SparseVolumeTextureIndex = TextureObject.Compile(Compiler);
		}
		else
		{
			int32 TextureReferenceIndex = INDEX_NONE;
			if (bIsParameter)
			{
				SparseVolumeTextureIndex = Compiler->SparseVolumeTextureParameter(GetParameterName(), SparseVolumeTexture, TextureReferenceIndex, SAMPLERTYPE_LinearColor);
			}
			else
			{
				SparseVolumeTextureIndex = Compiler->SparseVolumeTexture(SparseVolumeTexture, TextureReferenceIndex, SAMPLERTYPE_LinearColor);
			}
		}

		if (SparseVolumeTextureIndex == INDEX_NONE)
		{
			// Can't continue without a texture to sample
			return INDEX_NONE;
		}

		// Get sampling coordinates/UVWs. Fall back to computing them as (LocalPosition - LocalBoundsMin) / BoundsSize.
		int32 CoordinateIndex = INDEX_NONE;
		if (Coordinates.GetTracedInput().Expression)
		{
			CoordinateIndex = Coordinates.Compile(Compiler);
		}
		else
		{
			int32 WorldPositionLWCIndex = Compiler->WorldPosition(WPT_Default);
			int32 LocalPositionLWCIndex = Compiler->TransformPosition(MCB_World, MCB_Local, WorldPositionLWCIndex);
			int32 LocalPositionIndex = Compiler->ValidCast(LocalPositionLWCIndex, MCT_Float3); // LocalPosition is likely of LWC type. Float precision is fine past this point, so cast the LWC-ness away.
			int32 BoundsSizeIndex = Compiler->ObjectLocalBounds(1);
			int32 BoundsMinIndex = Compiler->ObjectLocalBounds(2);
			int32 RelativeLocalPositionIndex = Compiler->Sub(LocalPositionIndex, BoundsMinIndex);
			CoordinateIndex = Compiler->Div(RelativeLocalPositionIndex, BoundsSizeIndex);

			if (CoordinateIndex == INDEX_NONE)
			{
				return CompilerError(Compiler, TEXT("Failed to generate fallback UVW input for sparse volume texture"));
			}
		}

		int32 MipValue0Index = INDEX_NONE;
		int32 MipValue1Index = INDEX_NONE;
		if (MipValueMode == TMVM_Derivative)
		{
			if (CoordinatesDX.GetTracedInput().IsConnected())
			{
				MipValue0Index = CoordinatesDX.Compile(Compiler);
			}
			if (CoordinatesDY.GetTracedInput().IsConnected())
			{
				MipValue1Index = CoordinatesDY.Compile(Compiler);
			}
		}
		else if (MipValue.GetTracedInput().IsConnected())
		{
			MipValue0Index = MipValue.Compile(Compiler);
		}
		else
		{
			MipValue0Index = Compiler->Constant(ConstMipValue);
		}

		int32 PhysicalTileDataIdxIndex = Compiler->Constant(OutputIndex);

		return Compiler->SparseVolumeTextureSample(SparseVolumeTextureIndex, CoordinateIndex, MipValue0Index, MipValue1Index, PhysicalTileDataIdxIndex, MipValueMode, SamplerSource);
	}
	else
	{
		return CompilerError(Compiler, TEXT("Missing input texture"));
	}
}

void UMaterialExpressionSparseVolumeTextureSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Sparse Volume Texture Sample"));
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSparseVolumeTextureSampleParameter
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionSparseVolumeTextureSampleParameter::UMaterialExpressionSparseVolumeTextureSampleParameter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsParameterExpression = true;
}

#if WITH_EDITOR

bool UMaterialExpressionSparseVolumeTextureSampleParameter::SetParameterValue(FName InParameterName, USparseVolumeTexture* InValue, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (InParameterName == ParameterName)
	{
		if (SparseVolumeTexture != InValue)
		{
			SparseVolumeTexture = InValue;
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::SendPostEditChangeProperty))
			{
				SendPostEditChangeProperty(this, GET_MEMBER_NAME_STRING_CHECKED(ThisClass, SparseVolumeTexture));
			}
		}
		return true;
	}

	return false;
}

void UMaterialExpressionSparseVolumeTextureSampleParameter::ValidateParameterName(const bool bAllowDuplicateName)
{
	ValidateParameterNameInternal(this, Material, bAllowDuplicateName);
}

void UMaterialExpressionSparseVolumeTextureSampleParameter::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Sparse Volume Texture Sample Param "));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionSparseVolumeTextureSampleParameter::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	if (ParameterName.ToString().Contains(SearchQuery))
	{
		return true;
	}
	return Super::MatchesSearchQuery(SearchQuery);
}

bool UMaterialExpressionSparseVolumeTextureSampleParameter::GetParameterValue(FMaterialParameterMetadata& OutMeta) const
{
	OutMeta.Value = SparseVolumeTexture;
	OutMeta.Description = Desc;
	OutMeta.ExpressionGuid = ExpressionGUID;
	OutMeta.Group = Group;
	OutMeta.SortPriority = SortPriority;
	return true;
}

bool UMaterialExpressionSparseVolumeTextureSampleParameter::SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags)
{
	if (Meta.Value.Type == EMaterialParameterType::SparseVolumeTexture)
	{
		if (SetParameterValue(Name, Meta.Value.SparseVolumeTexture, Flags))
		{
			if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
			{
				Group = Meta.Group;
				SortPriority = Meta.SortPriority;
			}
			return true;
		}
	}
	return false;
}

#endif

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionSubsurfaceMediumMaterialOutput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionSubsurfaceMediumMaterialOutput::UMaterialExpressionSubsurfaceMediumMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionSubsurfaceMediumMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{

	int32 CodeInput = INDEX_NONE;
	
	if (OutputIndex == 0)
	{
		CodeInput = MeanFreePath.IsConnected() ? MeanFreePath.Compile(Compiler) : INDEX_NONE;
	}
	else if (OutputIndex == 1)
	{
		CodeInput = ScatteringDistribution.IsConnected() ? ScatteringDistribution.Compile(Compiler) : INDEX_NONE;
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionSubsurfaceMediumMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Subsurface Medium (Path Tracer Only)"));
}

EMaterialValueType UMaterialExpressionSubsurfaceMediumMaterialOutput::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0:
		return MCT_Float3;
	case 1:
		return MCT_Float1;
	default:
		break;
	}

	check(false);
	return MCT_Float1;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionSubsurfaceMediumMaterialOutput::GetNumOutputs() const
{
	return 2;
}

FString UMaterialExpressionSubsurfaceMediumMaterialOutput::GetFunctionName() const
{
	return TEXT("GetSubsurfaceMediumMaterialOutput");
}

FString UMaterialExpressionSubsurfaceMediumMaterialOutput::GetDisplayName() const
{
	return TEXT("Subsurface Medium");
}


///////////////////////////Neural network nodes

UMaterialExpressionNeuralNetworkInput::UMaterialExpressionNeuralNetworkInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionNeuralNetworkInput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;
	const bool bUseTextureAsInput = NeuralIndexType == ENeuralIndexType::NIT_TextureIndex;

	if (OutputIndex == 0)
	{
		if (Coordinates.IsConnected())
		{
			CodeInput = Coordinates.Compile(Compiler);
			if (bUseTextureAsInput)
			{
				CodeInput = Compiler->ComponentMask(CodeInput, false, true, true, true);
				CodeInput = Compiler->AppendVector(Compiler->Constant(-1.0f), CodeInput);
			}
		}
		else
		{
			int32 ViewportUV = Compiler->GetViewportUV();
			float BatchIndex = bUseTextureAsInput ? -1.0f : 0.0f;
				
			CodeInput = Compiler->AppendVector(Compiler->Constant2(BatchIndex, 0.0f), ViewportUV);
		}
	}
	else if (OutputIndex == 1)
	{
		CodeInput = Input0.IsConnected() ? Input0.Compile(Compiler) : Compiler->Constant3(0.5f, 0.5f, 0.5f);
	}
	else if (OutputIndex == 2)
	{
		CodeInput = Mask.IsConnected() ? Mask.Compile(Compiler) : Compiler->Constant(1.0f);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionNeuralNetworkInput::GetCaption(TArray<FString>& OutCaptions) const
{
	if (NeuralIndexType == ENeuralIndexType::NIT_TextureIndex)
	{
		OutCaptions.Add(TEXT("Neural Input (Texture)"));
	}
	else if (NeuralIndexType == ENeuralIndexType::NIT_BufferIndex)
	{
		OutCaptions.Add(TEXT("Neural Input (Buffer)"));
	}
}

EMaterialValueType UMaterialExpressionNeuralNetworkInput::GetInputValueType(int32 InputIndex)
{

	switch (InputIndex)
	{
	case 0:return MCT_Float4;
	case 1:return MCT_Float3;
	case 2:return MCT_Float1;
	default: checkNoEntry(); break;
	}

	return MCT_Float1;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionNeuralNetworkInput::GetNumOutputs() const
{
	return 3;
}

FString UMaterialExpressionNeuralNetworkInput::GetFunctionName() const
{
	return TEXT("GetNeuralInput");
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionNeuralNetworkOutput
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionNeuralNetworkOutput::UMaterialExpressionNeuralNetworkOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;
#endif

#if WITH_EDITOR
	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("RGBA"), 1, 1, 1, 1, 1));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionNeuralNetworkOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 ViewportUV = INDEX_NONE;

	if (Coordinates.GetTracedInput().Expression)
	{
		ViewportUV = Coordinates.Compile(Compiler);
	}

	return Compiler->NeuralOutput(ViewportUV, NeuralIndexType);
}

void UMaterialExpressionNeuralNetworkOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	if (NeuralIndexType == ENeuralIndexType::NIT_TextureIndex)
	{
		OutCaptions.Add(TEXT("Neural Output (Texture)"));
	}
	else if (NeuralIndexType == ENeuralIndexType::NIT_BufferIndex)
	{
		OutCaptions.Add(TEXT("Neural Output (Buffer)"));
	}
}

EMaterialValueType UMaterialExpressionNeuralNetworkOutput::GetInputValueType(int32 InputIndex)
{
	if (NeuralIndexType == ENeuralIndexType::NIT_TextureIndex)
	{
		return MCT_Float2;
	}
	else if (NeuralIndexType == ENeuralIndexType::NIT_BufferIndex)
	{
		return MCT_Float4;
	}

	checkNoEntry();
	return MCT_Float2;
}
#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// First Person Output
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionFirstPersonOutput::UMaterialExpressionFirstPersonOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionFirstPersonOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	if (OutputIndex == 0)
	{
		CodeInput = FirstPersonInterpolationAlpha.IsConnected() ? FirstPersonInterpolationAlpha.Compile(Compiler) : Compiler->Constant(ConstFirstPersonInterpolationAlpha);
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionFirstPersonOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("First Person Output"));
}

#endif // WITH_EDITOR

int32 UMaterialExpressionFirstPersonOutput::GetNumOutputs() const
{
	return 1;
}

FString UMaterialExpressionFirstPersonOutput::GetFunctionName() const
{
	return TEXT("GetFirstPersonOutput");
}

FString UMaterialExpressionFirstPersonOutput::GetDisplayName() const
{
	return TEXT("First Person Output");
}

#if WITH_EDITOR
void UMaterialExpressionIsFirstPerson::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("IsFirstPerson"));
}
#endif // WITH_EDITOR


///////////////////////////////////////////////////////////////////////////////
// Convert
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionConvert::UMaterialExpressionConvert(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bHidePreviewWindow = true;
	MenuCategories.Add(LOCTEXT("Convert", "Convert"));
#endif
}

#if WITH_EDITOR
void UMaterialExpressionConvert::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const bool bNeedsNodeUpdate = 
			PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, ConvertInputs)
		|| 	PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, ConvertMappings)
		|| 	PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, ConvertOutputs)
		||	PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, NodeName);

	if (bNeedsNodeUpdate)
	{
		RecreateOutputs();
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMaterialExpressionConvert::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static FProperty* ConvertInputsProperty = UMaterialExpressionConvert::StaticClass()->FindPropertyByName(
		GET_MEMBER_NAME_CHECKED(ThisClass, ConvertInputs)
	);

	static FProperty* ConvertOutputsProperty = UMaterialExpressionConvert::StaticClass()->FindPropertyByName(
		GET_MEMBER_NAME_CHECKED(ThisClass, ConvertOutputs)
	);

	const bool bNeedsNodeUpdate = 
			PropertyChangedEvent.PropertyChain.Contains(ConvertInputsProperty)
		||	PropertyChangedEvent.PropertyChain.Contains(ConvertOutputsProperty);

	if (bNeedsNodeUpdate)
	{
		RecreateOutputs();
		if (GraphNode)
		{
			GraphNode->ReconstructNode();
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionConvert::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!ConvertOutputs.IsValidIndex(OutputIndex))
	{
		return CompilerError(Compiler, TEXT("Invalid Output Index"));
	}

	const FMaterialExpressionConvertOutput& ConvertOutput = ConvertOutputs[OutputIndex];
	const EMaterialExpressionConvertType OutputType = ConvertOutput.Type;
	const int32 OutputComponentCount = MaterialExpressionConvertType::GetComponentCount(OutputType);

	int32 Components[4] = { INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE };

	TArray<int32> CachedCompiledInputs;
	CachedCompiledInputs.Init(INDEX_NONE, ConvertInputs.Num());

	// Determine value of any connected outputs
	for (const FMaterialExpressionConvertMapping& ConvertMapping : ConvertMappings)
	{
		const int32 CurrentOutputIndex = ConvertMapping.OutputIndex;

		// We only care about mapping relevant to this output
		if (CurrentOutputIndex != OutputIndex)
		{
			continue;
		}

		const int32 OutputComponentIndex = ConvertMapping.OutputComponentIndex;
		if (!IsValidComponentIndex(OutputComponentIndex, OutputType))
		{
			return CompilerError(Compiler, TEXT("Invalid Mapping Output Component Index"));
		}

		const int32 InputIndex = ConvertMapping.InputIndex;
		if (!ConvertInputs.IsValidIndex(InputIndex))
		{
			return CompilerError(Compiler, TEXT("Invalid Mapping Input Index"));
		}

		FMaterialExpressionConvertInput& ConvertInput = ConvertInputs[InputIndex];
		const int32 InputComponentIndex = ConvertMapping.InputComponentIndex;
		if (!IsValidComponentIndex(InputComponentIndex, ConvertInput.Type))
		{
			return CompilerError(Compiler, TEXT("Invalid Mapping Input Component Index"));
		}

		// Search CachedCompiledInputs for previously compiled value. If it isn't found, compile that expression input now
		int32& CompiledInput = CachedCompiledInputs[InputIndex];
		if (CompiledInput == INDEX_NONE)
		{
			const FLinearColor& DefaultValue = ConvertInput.DefaultValue;
			CompiledInput = ConvertInput.ExpressionInput.GetTracedInput().Expression 
				? ConvertInput.ExpressionInput.Compile(Compiler)
				: Compiler->Constant4(DefaultValue.R, DefaultValue.G, DefaultValue.B, DefaultValue.A);
		}

		// Mask out the specific value we want
		const int32 MaskedCompiledInput = Compiler->ComponentMask(
			CompiledInput, 
			InputComponentIndex == 0,	// R
			InputComponentIndex == 1,	// G
			InputComponentIndex == 2,	// B
			InputComponentIndex == 3	// A
		);

		Components[OutputComponentIndex] = MaskedCompiledInput;
	}

	// For any ComponentCompileResults that are still INDEX_NONE, set to ConvertOutput's default value
	for (int32 OutputComponentIndex = 0; OutputComponentIndex < OutputComponentCount; ++OutputComponentIndex)
	{
		int32& ComponentCompileResult = Components[OutputComponentIndex];
		// If we don't have a compile result here, default it to a that component's default value
		if (ComponentCompileResult == INDEX_NONE)
		{
			ComponentCompileResult = Compiler->Constant(ConvertOutput.DefaultValue.Component(OutputComponentIndex));
		}
	}

	// Init FinalCompileResult to first component result and then chain appends to form final vector
	int32 FinalCompileResult = Components[0];
	for (int32 AppendIndex = 1; AppendIndex < OutputComponentCount; ++AppendIndex)
	{
		FinalCompileResult = Compiler->AppendVector(
			FinalCompileResult, 
			Components[AppendIndex]
		);
	}
	return FinalCompileResult;
}

void UMaterialExpressionConvert::GetCaption(TArray<FString>& OutCaptions) const 
{
	OutCaptions.Add(NodeName);
}

void UMaterialExpressionConvert::GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip)
{
	if (ConvertInputs.IsValidIndex(InputIndex))
	{
		OutToolTip.Add(MaterialExpressionConvertType::ToText(ConvertInputs[InputIndex].Type).ToString());
	}
	else if (ConvertOutputs.IsValidIndex(OutputIndex))
	{
		OutToolTip.Add(MaterialExpressionConvertType::ToText(ConvertOutputs[OutputIndex].Type).ToString());
	}
}

int32 UMaterialExpressionConvert::CountInputs() const
{
	return ConvertInputs.Num();
}

EMaterialValueType UMaterialExpressionConvert::GetInputValueType(int32 InputIndex)
{
	if (ConvertInputs.IsValidIndex(InputIndex))
	{
		return MaterialExpressionConvertType::ToMaterialValueType(ConvertInputs[InputIndex].Type);
	}

	checkNoEntry();
	return MCT_Float;
}

FExpressionInput* UMaterialExpressionConvert::GetInput(int32 InputIndex)
{
	if (ConvertInputs.IsValidIndex(InputIndex))
	{
		return &ConvertInputs[InputIndex].ExpressionInput;
	}

	return nullptr;
}

EMaterialValueType UMaterialExpressionConvert::GetOutputValueType(int32 OutputIndex)
{
	if (ConvertOutputs.IsValidIndex(OutputIndex))
	{
		return MaterialExpressionConvertType::ToMaterialValueType(ConvertOutputs[OutputIndex].Type);
	}

	checkNoEntry();
	return MCT_Float;
}

TArray<FExpressionOutput>& UMaterialExpressionConvert::GetOutputs()
{
	RecreateOutputs();
	return Outputs;
}

bool UMaterialExpressionConvert::CanDeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) const
{
	return true;
}

void UMaterialExpressionConvert::DeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex)
{
	switch (PinDirection)
	{
		case EEdGraphPinDirection::EGPD_Input:
		{
			if (ConvertInputs.IsValidIndex(PinIndex))
			{
				ConvertInputs.RemoveAt(PinIndex);
			}
			break;
		}
		case EEdGraphPinDirection::EGPD_Output:
		{
			if (ConvertOutputs.IsValidIndex(PinIndex))
			{
				ConvertOutputs.RemoveAt(PinIndex);
				RecreateOutputs();
			}
			break;
		}
		default: break;
	}

	// Clean up any stale ConvertMappings
	for (auto ConvertMappingIter = ConvertMappings.CreateIterator(); ConvertMappingIter; ++ConvertMappingIter)
	{
		if (!ConvertInputs.IsValidIndex(ConvertMappingIter->InputIndex) ||
			!ConvertOutputs.IsValidIndex(ConvertMappingIter->OutputIndex))
		{
			ConvertMappingIter.RemoveCurrent();
			continue;
		}
	}

	if (GraphNode)
	{
		GraphNode->ReconstructNode();
	}
}

TSharedPtr<class SGraphNodeMaterialBase> UMaterialExpressionConvert::CreateCustomGraphNodeWidget()
{
	if (UMaterialGraphNode* MaterialGraphNode = Cast<UMaterialGraphNode>(GraphNode))
	{
		return SNew(SGraphNodeMaterialConvert, MaterialGraphNode);
	}

	return nullptr;
}

void UMaterialExpressionConvert::RegisterAdditionalMenuActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FText& CategoryName)
{
	// Helper Lambda to Create a Make Node of a specified ConvertType
	auto CreateMakeNode = [this, &ActionMenuBuilder, &CategoryName](EMaterialExpressionConvertType ConvertType)
	{
		const int32 NumComponents = MaterialExpressionConvertType::GetComponentCount(ConvertType);
		const FText ActionName = FText::Format(LOCTEXT("Convert_MakeNode_Name", "Make Vector{0}"), NumComponents);
		const FText ActionTooltip = FText::Format(LOCTEXT("Convert_MakeNode_Tooltip", "Adds a Make Vector{0} Node"), NumComponents);

		TSharedPtr<FMaterialGraphSchemaAction_NewNode> Action_MakeNode(new FMaterialGraphSchemaAction_NewNode(
			CategoryName,
			ActionName,
			ActionTooltip,
			0,
			GetKeywords()
		));
		Action_MakeNode->MaterialExpressionClass = GetClass();
		Action_MakeNode->PostCreationDelegate = TDelegate<void(UMaterialExpression*)>::CreateLambda(
			[ConvertType, NumComponents, ActionName](UMaterialExpression* NewExpression)
			{
				UMaterialExpressionConvert* NewConvertExpression = CastChecked<UMaterialExpressionConvert>(NewExpression);
				NewConvertExpression->NodeName = ActionName.ToString();
			
				// Initialize NumComponents Float1 Inputs
				FMaterialExpressionConvertInput Float1Input;
				Float1Input.Type = EMaterialExpressionConvertType::Scalar;
				NewConvertExpression->ConvertInputs.Init(Float1Input, NumComponents);

				// Add a single Output of our desired Convert Type
				FMaterialExpressionConvertOutput ConvertOutput;
				ConvertOutput.Type = ConvertType;
				NewConvertExpression->ConvertOutputs = { ConvertOutput };

				// Create the relevant mappings
				for (int32 MappingIndex = 0; MappingIndex < NumComponents; ++MappingIndex)
				{
					FMaterialExpressionConvertMapping NewMapping(MappingIndex, 0, 0, MappingIndex);
					NewConvertExpression->ConvertMappings.Add(NewMapping);
				}

				if (NewExpression->GraphNode)
				{
					NewExpression->GraphNode->ReconstructNode();
				}
			}
		);
		ActionMenuBuilder.AddAction(Action_MakeNode);
	};

	CreateMakeNode(EMaterialExpressionConvertType::Vector2);
	CreateMakeNode(EMaterialExpressionConvertType::Vector3);
	CreateMakeNode(EMaterialExpressionConvertType::Vector4);

	// Helper Lambda to Create a Break Node of a specified ConvertType
	auto CreateBreakNode = [this, &ActionMenuBuilder, &CategoryName](EMaterialExpressionConvertType ConvertType)
	{
		const int32 NumComponents = MaterialExpressionConvertType::GetComponentCount(ConvertType);
		const FText ActionName = FText::Format(LOCTEXT("Convert_BreakNode_Name", "Break Float{0}"), NumComponents);
		const FText ActionTooltip = FText::Format(LOCTEXT("Convert_BreakNode_Tooltip", "Adds a Break Float{0} Node"), NumComponents);

		TSharedPtr<FMaterialGraphSchemaAction_NewNode> Action_BreakNode(new FMaterialGraphSchemaAction_NewNode(
			CategoryName,
			ActionName,
			ActionTooltip,
			0,
			GetKeywords()
		));
		Action_BreakNode->MaterialExpressionClass = GetClass();
		Action_BreakNode->PostCreationDelegate = TDelegate<void(UMaterialExpression*)>::CreateLambda(
			[ConvertType, NumComponents, ActionName](UMaterialExpression* NewExpression)
			{
				UMaterialExpressionConvert* NewConvertExpression = CastChecked<UMaterialExpressionConvert>(NewExpression);
				NewConvertExpression->NodeName = ActionName.ToString();
			
				// Add a single Input of our desired Convert Type
				FMaterialExpressionConvertInput ConvertInput;
				ConvertInput.Type = ConvertType;
				NewConvertExpression->ConvertInputs = { ConvertInput };

				// Initialize NumComponents Float1 Outputs
				FMaterialExpressionConvertOutput ScalarOutput;
				ScalarOutput.Type = EMaterialExpressionConvertType::Scalar;
				NewConvertExpression->ConvertOutputs.Init(ScalarOutput, NumComponents);

				// Create the relevant mappings
				for (int32 MappingIndex = 0; MappingIndex < NumComponents; ++MappingIndex)
				{
					FMaterialExpressionConvertMapping NewMapping(0, MappingIndex, MappingIndex, 0);
					NewConvertExpression->ConvertMappings.Add(NewMapping);
				}

				if (NewExpression->GraphNode)
				{
					NewExpression->GraphNode->ReconstructNode();
				}
			}
		);
		ActionMenuBuilder.AddAction(Action_BreakNode);
	};

	CreateBreakNode(EMaterialExpressionConvertType::Vector2);
	CreateBreakNode(EMaterialExpressionConvertType::Vector3);
	CreateBreakNode(EMaterialExpressionConvertType::Vector4);
}

void UMaterialExpressionConvert::RecreateOutputs()
{
	Outputs.Reset();
	Outputs.AddDefaulted(ConvertOutputs.Num());
}

bool UMaterialExpressionConvert::IsValidComponentIndex(int32 InComponentIndex, EMaterialExpressionConvertType InType) const
{
	return InComponentIndex < MaterialExpressionConvertType::GetComponentCount(InType);
}
#endif // WITH_EDITOR


UMaterialExpressionRecordTextureStreamingInfo::UMaterialExpressionRecordTextureStreamingInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Tex")));
#endif
}

#if WITH_EDITOR

FExpressionInput* UMaterialExpressionRecordTextureStreamingInfo::GetInput(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0: return &TextureObject;
	case 1: return &Coordinates;
	}

	return nullptr;
}

EMaterialValueType UMaterialExpressionRecordTextureStreamingInfo::GetInputValueType(int32 InputIndex)
{
	switch (InputIndex)
	{
	case 0: return MCT_Texture2D;
	case 1: return MCT_Float2;
	}

	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionRecordTextureStreamingInfo::GetOutputValueType(int32 OutputIndex)
{
	if (TextureObject.GetTracedInput().Expression)
	{
		return TextureObject.GetTracedInput().Expression->GetOutputValueType(0);
	}
	return MCT_Texture2D;
}

void UMaterialExpressionRecordTextureStreamingInfo::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Record Texture Streaming"));
}

void UMaterialExpressionRecordTextureStreamingInfo::GetExpressionToolTip(TArray<FString>& OutToolTip)
{
	ConvertToMultilineToolTip(
		TEXT("Adds functionality to record the material UV scales for use by the automatic texture streaming system. If this material uses Texture ")
		TEXT("Sample nodes then this node isn't required. But if any texture sampling is done inside Custom HLSL nodes then this node should be added ")
		TEXT("so that the texture streaming system can record the UV scales of the texture being sampled."), 40, OutToolTip);
}

int32 UMaterialExpressionRecordTextureStreamingInfo::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (!TextureObject.GetTracedInput().Expression)	
	{
		return Compiler->Errorf(TEXT("Missing input TextureObject"));
	}
	const int32 TextureCodeIndex = TextureObject.Compile(Compiler);
	if (TextureCodeIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (!Coordinates.GetTracedInput().Expression)
	{
		return Compiler->Errorf(TEXT("Missing input UV Coordinates"));
	}
	const int32 CoordinatesCodeIndex = Coordinates.Compile(Compiler);

	// Streaming stat collection for texture collections and mesh paint textures are not supported.
	EMaterialValueType TextureType = Compiler->GetParameterType(TextureCodeIndex);
	if (TextureType & (MCT_TextureCollection | MCT_TextureMeshPaint))
	{
		return TextureCodeIndex;
	}

	int32 TextureReferenceIndex = INDEX_NONE;
	EMaterialSamplerType EffectiveSamplerType = SAMPLERTYPE_Color;
	TOptional<FName> EffectiveParameterName;
	if (!Compiler->GetTextureForExpression(TextureCodeIndex, TextureReferenceIndex, EffectiveSamplerType, EffectiveParameterName))
	{
		return CompilerError(Compiler, TEXT("TextureObject input requires a texture value"));
	}

	Compiler->TextureStreamingInfo(TextureReferenceIndex, TextureCodeIndex, CoordinatesCodeIndex);

	return TextureCodeIndex;
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// MaterialExpressionOperator
///////////////////////////////////////////////////////////////////////////////

// defined in MaterialExpressionsIR.cpp
uint32 GetMaterialExpressionOperatorArity(EMaterialExpressionOperatorKind Operator);

/** Array of Operators that allow adding inputs */
static const EMaterialExpressionOperatorKind VariableInputsExpressionOperators[] = {
	EMaterialExpressionOperatorKind::Add,
	EMaterialExpressionOperatorKind::Multiply
};

UMaterialExpressionOperator::UMaterialExpressionOperator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Initialize default inputs
	for (uint32 i = 0; i < Arity; i++)
	{
		DynamicInputs.Add(FMaterialExpressionOperatorInput());
	}
}

#if WITH_EDITOR

static bool OperatorSupportsVariablePinCount(EMaterialExpressionOperatorKind InOperator)
{
	auto begin = std::begin(VariableInputsExpressionOperators);
	auto end = std::end(VariableInputsExpressionOperators);
	return std::find(begin, end, InOperator) != end;
}

void UMaterialExpressionOperator::AddInputPin()
{
	DynamicInputs.Add(FMaterialExpressionOperatorInput());
	PreEditChange(nullptr);
	PostEditChange();
}

bool UMaterialExpressionOperator::CanDeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) const
{
	return (uint32(DynamicInputs.Num()) > Arity);
}

void UMaterialExpressionOperator::DeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex)
{
	if (DynamicInputs.IsValidIndex(PinIndex))
	{
		DynamicInputs.RemoveAt(PinIndex);
	}
	if (GraphNode)
	{
		GraphNode->ReconstructNode();
		this->RefreshNode();
	}
}

int32 UMaterialExpressionOperator::CountInputs() const
{
	return DynamicInputs.Num();
}

FExpressionInput* UMaterialExpressionOperator::GetInput(int32 InputIndex) 
{
	return (DynamicInputs.IsValidIndex(InputIndex)) ? &DynamicInputs[InputIndex].ExpressionInput : nullptr;
}

FName UMaterialExpressionOperator::GetInputName(int32 InputIndex) const
{
	// if InputIndex is 0-25, return name: A-Z
	if (InputIndex < 26)
	{
		TCHAR Letter = TCHAR('A' + InputIndex);
		return FName(*FString::Printf(TEXT("%c"), Letter));
	}
	// return name: Input N
	return FName(*FString::Printf(TEXT("Input %d"), InputIndex + 1));
}

void UMaterialExpressionOperator::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(UEnum::GetDisplayValueAsText(Operator).ToString());
}

FText UMaterialExpressionOperator::GetKeywords() const
{
	return FText::FromString(TEXT("operator"));
}

FText UMaterialExpressionOperator::GetCreationName() const
{
	return FText::FromString(TEXT("Operator"));
}

void UMaterialExpressionOperator::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UMaterialExpression::PostEditChangeProperty(PropertyChangedEvent);

	bool bNeedsUpdate = false;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, DynamicInputs))
	{
		bNeedsUpdate = true;
	}	
	
	if (PropertyChangedEvent.GetPropertyName() == TEXT("Operator"))
	{
		const uint32 ArityNew = GetMaterialExpressionOperatorArity(Operator);
		const bool bAllowAddPinNew = OperatorSupportsVariablePinCount(Operator);

		// Resize if switching between fixed and flexible operators, OR if the arity change
		if (Arity != ArityNew || bAllowAddPin != bAllowAddPinNew)
		{
			// If the operator remains flexible before and after, and DynamicInputs don't need to be updated according to Arity
			// Otherwise, resize
			if (!bAllowAddPinNew || !bAllowAddPin)
			{
				DynamicInputs.SetNum(ArityNew, EAllowShrinking::Yes);
			}
			Arity = ArityNew;
			bAllowAddPin = bAllowAddPinNew;
			bNeedsUpdate = true;
		}
	}

	if (bNeedsUpdate)
	{
		GraphNode->ReconstructNode();
	}
}

void UMaterialExpressionOperator::PostLoad()
{
	Arity = GetMaterialExpressionOperatorArity(Operator);
	UMaterialExpression::PostLoad();
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionAggregate
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionAggregate::UMaterialExpressionAggregate(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	Outputs.Empty();
	Outputs.Emplace();
}

#if WITH_EDITOR

const UMaterialAggregate* UMaterialExpressionAggregate::GetAggregate() const
{
	switch (Kind)
	{
		case EMaterialExpressionMakeAggregateKind::MaterialAttributes: return MaterialAttributesAggregate::Get();
		case EMaterialExpressionMakeAggregateKind::UserDefined: return UserAggregate;
		default: checkNoEntry(); return nullptr;
	}
}

const FMaterialAggregateAttribute* UMaterialExpressionAggregate::GetAggregateAttribute(int32 Index) const
{
	check(Index >= 0);
	if (const UMaterialAggregate* Aggregate = GetAggregate())
	{
		if (Index < Aggregate->Attributes.Num())
		{
			return &Aggregate->Attributes[Index];
		}
	}
	return nullptr;
}

static bool IsAggregateAttributeEnabled(const UMaterial* Material, const UMaterialAggregate* Aggregate, int AttributeIndex, const TArray<FName>& UsedAttributeNames)
{
	if (Aggregate == MaterialAttributesAggregate::Get())
	{
		EMaterialProperty Property = MaterialAttributesAggregate::AttributeIndexToMaterialProperty(AttributeIndex);
		if (Property <= MP_CustomizedUVs7 && Property >= MP_CustomizedUVs0 + Material->NumCustomizedUVs)
		{
			return false;
		}
	}
	if (UsedAttributeNames.Contains(Aggregate->Attributes[AttributeIndex].Name))
	{
		return false;
	}
	return true;
}

TArray<FString> UMaterialExpressionAggregate::GetPossibleAttributeNames() const
{
	TArray<FString> Values;
	if (const UMaterialAggregate* Aggregate = GetAggregate())
	{
		for (int32 i = 0; i < Aggregate->Attributes.Num(); ++i)
		{
			if (IsAggregateAttributeEnabled(Material, Aggregate, i, AttributeNames))
			{
				Values.Add(Aggregate->Attributes[i].Name.ToString());
			}
		}
	}
	return Values;
}

void UMaterialExpressionAggregate::GetCaption(TArray<FString>& OutCaptions) const
{
	const UMaterialAggregate* Aggregate = GetAggregate();
	if (Aggregate)
	{
		OutCaptions.Add(*Aggregate->GetName());
	}
	else
	{
		OutCaptions.Add(TEXT("Aggregate")); 
	}
}

FExpressionInput* UMaterialExpressionAggregate::GetInput(int32 InputIndex)
{
	check(InputIndex >= 0);
	if (InputIndex == 0)
	{
		return &PrototypeInput;
	}

	InputIndex -= 1;
	return InputIndex < Entries.Num() ? &Entries[InputIndex].Input : nullptr;
}

FName UMaterialExpressionAggregate::GetInputName(int32 InputIndex) const
{
	check(InputIndex >= 0);
	if (InputIndex == 0)
	{
		if (const UMaterialAggregate* Aggregate = GetAggregate())
		{
			return *GetAggregate()->GetName();
		}
		return NAME_None;
	}

	InputIndex -= 1;
	return InputIndex < AttributeNames.Num() ? AttributeNames[InputIndex] : NAME_None;
}

static EMaterialValueType GetPinValueType(const UMaterialExpressionAggregate* Expr, int32 PinIndex)
{
	check(PinIndex >= 0);
	if (PinIndex == 0)
	{
		return Expr->GetAggregate() == MaterialAttributesAggregate::Get() ? MCT_MaterialAttributes : MCT_Unknown;
	}
	if (const FMaterialAggregateAttribute* Attribute = Expr->GetAggregateAttribute(PinIndex - 1))
	{
		return Attribute->ToMaterialValueType();
	}
	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionAggregate::GetInputValueType(int32 InputIndex)
{
	return GetPinValueType(this, InputIndex);
}

EMaterialValueType UMaterialExpressionAggregate::GetOutputValueType(int32 OutputIndex)
{
	return GetPinValueType(this, OutputIndex);
}

bool UMaterialExpressionAggregate::IsResultMaterialAttributes(int32 OutputIndex)
{
	return GetOutputValueType(OutputIndex) == MCT_MaterialAttributes;
}

void UMaterialExpressionAggregate::PreEditChange(FProperty* PropertyAboutToChange)
{
	PreEditData.Aggregate = GetAggregate();
	PreEditData.AttributeNames = AttributeNames;
	PreEditData.Entries = Entries;
}

static FName PickAvailableAttribute(const UMaterial* Material, const UMaterialAggregate* Aggregate, const TArray<FName>& AttributeNames)
{
	if (Aggregate)
	{
		for (int i = 0; i < Aggregate->Attributes.Num(); ++i)
		{
			if (IsAggregateAttributeEnabled(Material, Aggregate, i, AttributeNames))
			{
				return Aggregate->Attributes[i].Name;
			}
		}
	}
	return {};
}
void UMaterialExpressionAggregate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
    // Early-out if there’s no graph node or no property change to handle
    if (!GraphNode || !PropertyChangedEvent.MemberProperty)
    {
        Super::PostEditChangeProperty(PropertyChangedEvent);
        return;
    }

    const UMaterialAggregate* Aggregate = GetAggregate();

    // If the underlying aggregate asset changed, reset all attribute selections
    if (PreEditData.Aggregate != Aggregate)
    {
        AttributeNames.Empty();
    }
    
    // Handle newly added attributes
    if (AttributeNames.Num() > PreEditData.AttributeNames.Num())
    {
        for (int32 i = 0; i < AttributeNames.Num(); )
        {
            if (AttributeNames[i] == NAME_None)
            {
                AttributeNames[i] = PickAvailableAttribute(Material, Aggregate, AttributeNames);
                if (AttributeNames[i] == NAME_None)
                {
                    AttributeNames.RemoveAt(i);
                    continue;
                }
                Entries.EmplaceAt(i);
                Outputs.Emplace();
            }
            ++i;
        }
    }
    // Handle removed attributes
    else if (AttributeNames.Num() < PreEditData.AttributeNames.Num())
    {
        int32 CurrIndex = 0, PrevIndex = 0;

        // Remove entries/pins for any attributes dropped from the list
        for (; CurrIndex < AttributeNames.Num(); ++PrevIndex)
        {
            if (PrevIndex >= PreEditData.AttributeNames.Num() ||
                AttributeNames[CurrIndex] != PreEditData.AttributeNames[PrevIndex])
            {
                Entries.RemoveAt(CurrIndex);

                int PinIndex = CurrIndex + 1;
                Outputs.RemoveAt(PinIndex);
                GraphNode->RemovePinAt(PinIndex, EGPD_Input);
                GraphNode->RemovePinAt(PinIndex, EGPD_Output);

                continue;
            }
            ++CurrIndex;
        }

        // Clean up any leftover old entries beyond the new list size
        for (; PrevIndex < PreEditData.AttributeNames.Num(); ++PrevIndex)
        {
            Entries.Pop();
            Outputs.Pop();
            GraphNode->RemovePinAt(Outputs.Num(), EGPD_Input);
            GraphNode->RemovePinAt(Outputs.Num(), EGPD_Output);
        }
    }

    // Update each entry’s index to match the aggregate’s attribute ordering
    for (int i = 0; i < AttributeNames.Num(); ++i)
    {
        Entries[i].AttributeIndex = Aggregate->FindAttributeIndexByName(AttributeNames[i]);
    }

    // Rebuild the node to reflect any pin/entry changes
    GraphNode->ReconstructNode();

    Super::PostEditChangeProperty(PropertyChangedEvent);
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionBlend
///////////////////////////////////////////////////////////////////////////////

void UMaterialExpressionBlend::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Blend")); 
}

EMaterialValueType UMaterialExpressionBlend::GetInputValueType(int32 InputIndex)
{
	return InputIndex == 2 ? MCT_Float1 : MCT_Unknown;
}

EMaterialValueType UMaterialExpressionBlend::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Unknown;
}

#endif // WITH_EDITOR

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionTemporalResponsivenessOutput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionTemporalResponsivenessOutput::UMaterialExpressionTemporalResponsivenessOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionTemporalResponsivenessOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	if (OutputIndex == 0)
	{
		CodeInput = Input.IsConnected() ? Input.Compile(Compiler) : INDEX_NONE;
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionTemporalResponsivenessOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Temporal Responsiveness"));
}

EMaterialValueType UMaterialExpressionTemporalResponsivenessOutput::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_Float1;
	}

	check(false);
	return MCT_Float1;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionTemporalResponsivenessOutput::GetNumOutputs() const
{
	return 1;
}

FString UMaterialExpressionTemporalResponsivenessOutput::GetFunctionName() const
{
	return TEXT("GetTemporalResponsivenessOutput");
}

FString UMaterialExpressionTemporalResponsivenessOutput::GetDisplayName() const
{
	return TEXT("TemporalResponsiveness");
}

///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionMotionVectorWorldOffsetOutput
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionMotionVectorWorldOffsetOutput::UMaterialExpressionMotionVectorWorldOffsetOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

int32 UMaterialExpressionMotionVectorWorldOffsetOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	int32 CodeInput = INDEX_NONE;

	if (OutputIndex == 0)
	{
		CodeInput = Input.IsConnected() ? Input.Compile(Compiler) : INDEX_NONE;
	}

	return Compiler->CustomOutput(this, OutputIndex, CodeInput);
}

void UMaterialExpressionMotionVectorWorldOffsetOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Motion Vector World Offset (Per-Pixel)"));
}

EMaterialValueType UMaterialExpressionMotionVectorWorldOffsetOutput::GetInputValueType(int32 InputIndex)
{
	if (InputIndex == 0)
	{
		return MCT_Float3;
	}

	check(false);
	return MCT_Float1;
}

#endif // WITH_EDITOR

int32 UMaterialExpressionMotionVectorWorldOffsetOutput::GetNumOutputs() const
{
	return 1;
}

FString UMaterialExpressionMotionVectorWorldOffsetOutput::GetFunctionName() const
{
	return TEXT("GetMotionVectorWorldOffsetOutput");
}

FString UMaterialExpressionMotionVectorWorldOffsetOutput::GetDisplayName() const
{
	return TEXT("MotionVectorWorldOffset");
}

#undef LOCTEXT_NAMESPACE
