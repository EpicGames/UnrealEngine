// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Curves/CurveLinearColorAtlas.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "MaterialIRInternal.h"
#include "MaterialDomain.h"
#include "MaterialExpressionIO.h"
#include "Materials/Material.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionAbs.h"
#include "Materials/MaterialExpressionAbsorptionMediumMaterialOutput.h"
#include "Materials/MaterialExpressionActorPositionWS.h"
#include "Materials/MaterialExpressionAdd.h"
#include "Materials/MaterialExpressionAntialiasedTextureMask.h"
#include "Materials/MaterialExpressionAppendVector.h"
#include "Materials/MaterialExpressionArccosine.h"
#include "Materials/MaterialExpressionArccosineFast.h"
#include "Materials/MaterialExpressionArcsine.h"
#include "Materials/MaterialExpressionArcsineFast.h"
#include "Materials/MaterialExpressionArctangent.h"
#include "Materials/MaterialExpressionArctangent2.h"
#include "Materials/MaterialExpressionArctangent2Fast.h"
#include "Materials/MaterialExpressionArctangentFast.h"
#include "Materials/MaterialExpressionAtmosphericLightColor.h"
#include "Materials/MaterialExpressionAtmosphericLightVector.h"
#include "Materials/MaterialExpressionBentNormalCustomOutput.h"
#include "Materials/MaterialExpressionBlackBody.h"
#include "Materials/MaterialExpressionBlend.h"
#include "Materials/MaterialExpressionBlendMaterialAttributes.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionBumpOffset.h"
#include "Materials/MaterialExpressionCameraPositionWS.h"
#include "Materials/MaterialExpressionCameraVectorWS.h"
#include "Materials/MaterialExpressionCeil.h"
#include "Materials/MaterialExpressionChannelMaskParameter.h"
#include "Materials/MaterialExpressionClamp.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionFirstPersonOutput.h"
#include "Materials/MaterialExpressionCloudLayer.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionCollectionTransform.h"
#include "Materials/MaterialExpressionColorRamp.h"
#include "Materials/MaterialExpressionComponentMask.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionConstantBiasScale.h"
#include "Materials/MaterialExpressionConvert.h"
#include "Materials/MaterialExpressionCosine.h"
#include "Materials/MaterialExpressionCrossProduct.h"
#include "Materials/MaterialExpressionCurveAtlasRowParameter.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionDataDrivenShaderPlatformInfoSwitch.h"
#include "Materials/MaterialExpressionDBufferTexture.h"
#include "Materials/MaterialExpressionDDX.h"
#include "Materials/MaterialExpressionDDY.h"
#include "Materials/MaterialExpressionDecalColor.h"
#include "Materials/MaterialExpressionDecalDerivative.h"
#include "Materials/MaterialExpressionDecalLifetimeOpacity.h"
#include "Materials/MaterialExpressionDecalMipmapLevel.h"
#include "Materials/MaterialExpressionDeltaTime.h"
#include "Materials/MaterialExpressionDepthFade.h"
#include "Materials/MaterialExpressionDepthOfFieldFunction.h"
#include "Materials/MaterialExpressionDeriveNormalZ.h"
#include "Materials/MaterialExpressionDesaturation.h"
#include "Materials/MaterialExpressionDistance.h"
#include "Materials/MaterialExpressionDistanceCullFade.h"
#include "Materials/MaterialExpressionDistanceFieldApproxAO.h"
#include "Materials/MaterialExpressionDistanceFieldGradient.h"
#include "Materials/MaterialExpressionDistanceFieldsRenderingSwitch.h"
#include "Materials/MaterialExpressionDistanceToNearestSurface.h"
#include "Materials/MaterialExpressionDivide.h"
#include "Materials/MaterialExpressionDotProduct.h"
#include "Materials/MaterialExpressionDoubleVectorParameter.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Materials/MaterialExpressionExponential.h"
#include "Materials/MaterialExpressionExponential2.h"
#include "Materials/MaterialExpressionEyeAdaptation.h"
#include "Materials/MaterialExpressionEyeAdaptationInverse.h"
#include "Materials/MaterialExpressionFeatureLevelSwitch.h"
#include "Materials/MaterialExpressionFloor.h"
#include "Materials/MaterialExpressionFmod.h"
#include "Materials/MaterialExpressionAtmosphericFogColor.h"
#include "Materials/MaterialExpressionFontSample.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionFrac.h"
#include "Materials/MaterialExpressionFresnel.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionGenericConstant.h"
#include "Materials/MaterialExpressionGetMaterialAttributes.h"
#include "Materials/MaterialExpressionGIReplace.h"
#include "Materials/MaterialExpressionHairAttributes.h"
#include "Materials/MaterialExpressionHairColor.h"
#include "Materials/MaterialExpressionHsvToRgb.h"
#include "Materials/MaterialExpressionIf.h"
#include "Materials/MaterialExpressionIfThenElse.h"
#include "Materials/MaterialExpressionInverseLinearInterpolate.h"
#include "Materials/MaterialExpressionIsOrthographic.h"
#include "Materials/MaterialExpressionLength.h"
#include "Materials/MaterialExpressionLightmapUVs.h"
#include "Materials/MaterialExpressionLightVector.h"
#include "Materials/MaterialExpressionLinearInterpolate.h"
#include "Materials/MaterialExpressionLightmassReplace.h"
#include "Materials/MaterialExpressionLocalPosition.h"
#include "Materials/MaterialExpressionLogarithm.h"
#include "Materials/MaterialExpressionLogarithm10.h"
#include "Materials/MaterialExpressionLogarithm2.h"
#include "Materials/MaterialExpressionMakeMaterialAttributes.h"
#include "Materials/MaterialExpressionMapARPassthroughCameraUV.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionMaterialProxyReplace.h"
#include "Materials/MaterialExpressionMax.h"
#include "Materials/MaterialExpressionMeshPaintTextureReplace.h"
#include "Materials/MaterialExpressionMin.h"
#include "Materials/MaterialExpressionModulo.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionNaniteReplace.h"
#include "Materials/MaterialExpressionNoise.h"
#include "Materials/MaterialExpressionNormalize.h"
#include "Materials/MaterialExpressionNeuralPostProcessNode.h"
#include "Materials/MaterialExpressionObjectBounds.h"
#include "Materials/MaterialExpressionObjectLocalBounds.h"
#include "Materials/MaterialExpressionBounds.h"
#include "Materials/MaterialExpressionObjectOrientation.h"
#include "Materials/MaterialExpressionObjectPositionWS.h"
#include "Materials/MaterialExpressionObjectRadius.h"
#include "Materials/MaterialExpressionOneMinus.h"
#include "Materials/MaterialExpressionPanner.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionParticleColor.h"
#include "Materials/MaterialExpressionParticleDirection.h"
#include "Materials/MaterialExpressionParticleMacroUV.h"
#include "Materials/MaterialExpressionParticleMotionBlurFade.h"
#include "Materials/MaterialExpressionParticlePositionWS.h"
#include "Materials/MaterialExpressionParticleRadius.h"
#include "Materials/MaterialExpressionParticleRandom.h"
#include "Materials/MaterialExpressionParticleRelativeTime.h"
#include "Materials/MaterialExpressionParticleSize.h"
#include "Materials/MaterialExpressionParticleSpeed.h"
#include "Materials/MaterialExpressionParticleSubUV.h"
#include "Materials/MaterialExpressionParticleSubUVProperties.h"
#include "Materials/MaterialExpressionPathTracingBufferTexture.h"
#include "Materials/MaterialExpressionPathTracingQualitySwitch.h"
#include "Materials/MaterialExpressionPathTracingRayTypeSwitch.h"
#include "Materials/MaterialExpressionPerInstanceCustomData.h"
#include "Materials/MaterialExpressionPerInstanceFadeAmount.h"
#include "Materials/MaterialExpressionPerInstanceRandom.h"
#include "Materials/MaterialExpressionPixelDepth.h"
#include "Materials/MaterialExpressionPixelNormalWS.h"
#include "Materials/MaterialExpressionPower.h"
#include "Materials/MaterialExpressionPrecomputedAOMask.h"
#include "Materials/MaterialExpressionPreSkinnedLocalBounds.h"
#include "Materials/MaterialExpressionPreSkinnedNormal.h"
#include "Materials/MaterialExpressionPreSkinnedPosition.h"
#include "Materials/MaterialExpressionPreviousFrameSwitch.h"
#include "Materials/MaterialExpressionSamplePhysicsField.h"
#include "Materials/MaterialExpressionSphericalParticleOpacity.h"
#include "Materials/MaterialExpressionQualitySwitch.h"
#include "Materials/MaterialExpressionRayTracingQualitySwitch.h"
#include "Materials/MaterialExpressionReflectionCapturePassSwitch.h"
#include "Materials/MaterialExpressionReflectionVectorWS.h"
#include "Materials/MaterialExpressionRequiredSamplersSwitch.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionRerouteBase.h"
#include "Materials/MaterialExpressionRgbToHsv.h"
#include "Materials/MaterialExpressionRotateAboutAxis.h"
#include "Materials/MaterialExpressionRotator.h"
#include "Materials/MaterialExpressionRound.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureOutput.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureReplace.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionSaturate.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionSceneColor.h"
#include "Materials/MaterialExpressionSceneDepth.h"
#include "Materials/MaterialExpressionSceneDepthWithoutWater.h"
#include "Materials/MaterialExpressionSceneTexelSize.h"
#include "Materials/MaterialExpressionSceneTexture.h"
#include "Materials/MaterialExpressionScreenPosition.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionShaderStageSwitch.h"
#include "Materials/MaterialExpressionShadingModel.h"
#include "Materials/MaterialExpressionShadingPathSwitch.h"
#include "Materials/MaterialExpressionShadowReplace.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionSign.h"
#include "Materials/MaterialExpressionSine.h"
#include "Materials/MaterialExpressionSkyAtmosphereLightDirection.h"
#include "Materials/MaterialExpressionSkyAtmosphereLightIlluminance.h"
#include "Materials/MaterialExpressionSkyAtmosphereViewLuminance.h"
#include "Materials/MaterialExpressionSkyLightEnvMapSample.h"
#include "Materials/MaterialExpressionSmoothStep.h"
#include "Materials/MaterialExpressionSobol.h"
#include "Materials/MaterialExpressionSpeedTree.h"
#include "Materials/MaterialExpressionSphereMask.h"
#include "Materials/MaterialExpressionSquareRoot.h"
#include "Materials/MaterialExpressionSRGBColorToWorkingColorSpace.h"
#include "Materials/MaterialExpressionStaticBool.h"
#include "Materials/MaterialExpressionStaticComponentMaskParameter.h"
#include "Materials/MaterialExpressionStaticSwitch.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"
#include "Materials/MaterialExpressionStep.h"
#include "Materials/MaterialExpressionSubtract.h"
#include "Materials/MaterialExpressionSwitch.h"
#include "Materials/MaterialExpressionTangent.h"
#include "Materials/MaterialExpressionTangentOutput.h"
#include "Materials/MaterialExpressionTemporalSobol.h"
#include "Materials/MaterialExpressionTextureCollection.h"
#include "Materials/MaterialExpressionTextureCollectionParameter.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionTextureObjectFromCollection.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter2DArray.h"
#include "Materials/MaterialExpressionTextureSampleParameterCube.h"
#include "Materials/MaterialExpressionTextureSampleParameterSubUV.h"
#include "Materials/MaterialExpressionTextureSampleParameterVolume.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionTime.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionTruncate.h"
#include "Materials/MaterialExpressionTruncateLWC.h"
#include "Materials/MaterialExpressionTwoSidedSign.h"
#include "Materials/MaterialExpressionUserSceneTexture.h"
#include "Materials/MaterialExpressionVectorNoise.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionVertexColor.h"
#include "Materials/MaterialExpressionVertexInterpolator.h"
#include "Materials/MaterialExpressionVertexNormalWS.h"
#include "Materials/MaterialExpressionVertexTangentWS.h"
#include "Materials/MaterialExpressionViewProperty.h"
#include "Materials/MaterialExpressionViewSize.h"
#include "Materials/MaterialExpressionVirtualTextureFeatureSwitch.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialInput.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionFloatToUInt.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialExpressionOperator.h"
#include "Materials/MaterialExpressionAggregate.h"
#include "Materials/MaterialSharedPrivate.h"
#include "Materials/MaterialExpressionMotionVectorWorldOffsetOutput.h"
#include "Materials/MaterialExpressionSubsurfaceMediumMaterialOutput.h"
#include "Materials/MaterialExpressionPostVolumeUserFlagTest.h"
#include "Materials/MaterialExpressionBindlessSwitch.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialExpressionsToMIRCommon.h"
#include "MaterialShared.h"
#include "Materials/MaterialIREmitter.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRExtern.h"
#include "Misc/MemStackUtility.h"
#include "RenderUtils.h"
#include "Shader/Preshader2.h"
#include "ShaderPlatformConfig.h"
#include "TextureResource.h"
#include "ColorManagement/ColorSpace.h"
#include "SubstrateTranslatorCommon.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "Engine/ToonProfile.h"

#include "PostProcess/PostProcessMaterialInputs.h"

using FValueRef = MIR::FValueRef;

/* Material builtin values emission */

/* Helper functions to emit inline HLSL instructions */

static void CheckExternalCodeIdentifierIsValid(const UMaterialExpressionExternalCodeBase& InExpression, int32 InExternalCodeIdentifierIndex)
{
	checkf(InExternalCodeIdentifierIndex >= 0 && InExternalCodeIdentifierIndex < InExpression.ExternalCodeIdentifiers.Num(),
		   TEXT("External code identifier index (%d) out of bounds; Upper bound is %d"), InExternalCodeIdentifierIndex, InExpression.ExternalCodeIdentifiers.Num());
}

template <typename... TArguments>
requires (std::same_as<TArguments, FValueRef> && ...)
static FValueRef MaterialExpressionExternalCodeBase_EmitExtern(MIR::FEmitter& Em, const UMaterialExpressionExternalCodeBase& InExpression, int32 InExternalCodeIdentifierIndex, TArguments... InArguments)
{
	CheckExternalCodeIdentifierIsValid(InExpression, InExternalCodeIdentifierIndex);

	const FName ExternalCodeIdentifier = InExpression.ExternalCodeIdentifiers[InExternalCodeIdentifierIndex];
	return Em.Extern<MIR::FExternFromMaterialDecl>({ ExternalCodeIdentifier }, InArguments...);
}

template <typename... TArguments>
requires (std::same_as<TArguments, FValueRef> && ...)
static void BuildMaterialExpressionExternalCodeBase(MIR::FEmitter& Em, UMaterialExpressionExternalCodeBase& InExpression, TArguments... InArguments)
{
	// If the external code expression has only one identifier, evaluate it and flow its result to all expression outputs.
	if (InExpression.ExternalCodeIdentifiers.Num() == 1)
	{
		Em.OutputsWithComponentMask(MaterialExpressionExternalCodeBase_EmitExtern(Em, InExpression, 0, InArguments...));
	}
	else 
	{
		// Make sure the number of identifiers perfectly matches the number of expression outputs.
		if (InExpression.ExternalCodeIdentifiers.Num() != InExpression.GetOutputs().Num())
		{
			Em.Errorf(TEXT("Internal Error: The number of external code identifiers (%d) does not match the number of expression outputs (%d) in expression '%s'."),
					  InExpression.ExternalCodeIdentifiers.Num(), InExpression.GetOutputs().Num(), *InExpression.GetName());
			return;
		}

		// Evaluate each external code identifier separately and flow it to its own matching expression output only.
		for (int32 OutputIndex = 0; OutputIndex < InExpression.ExternalCodeIdentifiers.Num(); ++OutputIndex)
		{
			Em.Output(OutputIndex, MaterialExpressionExternalCodeBase_EmitExtern(Em, InExpression, OutputIndex, InArguments...));
		}
	}
}


static FName NAME_CameraVector("CameraVector");

void UMaterialExpression::Build(MIR::FEmitter& Em)
{
	Em.Error(TEXT("Unsupported material expression."));
} 

void UMaterialExpressionMaterialFunctionCall::Build(MIR::FEmitter& Em)
{
	MIR::TTemporaryArray<MIR::FSubgraphOutputMapping> OutputMappings{ FunctionOutputs.Num() };
	for (int i = 0; i < FunctionOutputs.Num(); ++i)
	{
		OutputMappings[i].Expression  = FunctionOutputs[i].ExpressionOutput.Get();
		OutputMappings[i].OutputIndex = 0; // Note: UMaterialExpressionFunctionOutput have a single bypass expression output
	}
	Em.Subgraph(FunctionInputs.Num(), OutputMappings);
}

void UMaterialExpressionFunctionInput::Build(MIR::FEmitter& Em)
{
	// When inside a function call, find our input index and fetch the caller-provided value.
	if (UMaterialExpressionMaterialFunctionCall* Call = Cast<UMaterialExpressionMaterialFunctionCall>(Em.GetSubgraphExpression()))
	{
		int32 InputIndex = Call->FunctionInputs.IndexOfByPredicate([this](const FFunctionExpressionInput& FI) { return FI.ExpressionInput.Get() == this; });
		if (InputIndex != INDEX_NONE)
		{
			FValueRef CallerValue = bUsePreviewValueAsDefault ? Em.TrySubgraphInput(InputIndex) : Em.SubgraphInput(InputIndex);
			UE_MIR_CHECKPOINT(Em);

			if (CallerValue)
			{
				MIR::FType Type = MIR::FType::FromMaterialValueType(GetInputValueType(0));
				Em.Output(0, Em.Cast(CallerValue, Type));
				return;
			}
			// Optional input not connected, fall through to preview/default.
		}
		else
		{
			Em.Errorf(TEXT("Function input '%s' not found in call '%s'."), *InputName.ToString(), Call->MaterialFunction ? *Call->MaterialFunction->GetName() : TEXT("<null>"));
			return;
		}
	}

	// Not inside a call, or function input not found. Use the preview connection or a default constant.
	FValueRef OutputValue = Em.TryInput(&Preview);
	if (OutputValue)
	{
		Em.Output(0, OutputValue);
		return;
	}

	switch (InputType)
	{
		case FunctionInput_Scalar:
			OutputValue = Em.ConstantFloat(PreviewValue.X);
			break;

		case FunctionInput_Vector2:
			OutputValue = Em.ConstantFloat2({ PreviewValue.X, PreviewValue.Y });
			break;

		case FunctionInput_Vector3:
			OutputValue = Em.ConstantFloat3({ PreviewValue.X, PreviewValue.Y, PreviewValue.Z });
			break;

		case FunctionInput_Vector4:
			OutputValue = Em.ConstantFloat4(PreviewValue);
			break;

		case FunctionInput_Bool:
		case FunctionInput_StaticBool:
			OutputValue = Em.ConstantBool(PreviewValue.X != 0.0f);
			break;

		case FunctionInput_MaterialAttributes:
			OutputValue = Em.Aggregate(MaterialAttributesAggregate::Get());
			break;

		case FunctionInput_Texture2D:
		case FunctionInput_TextureCube:
		case FunctionInput_Texture2DArray:
		case FunctionInput_VolumeTexture:
		case FunctionInput_TextureExternal:
		case FunctionInput_Substrate:
			Em.Error(TEXT("Function input of object type requires preview input to be provided."));
			return;

		default:
			UE_MIR_UNREACHABLE();
	}

	Em.Output(0, OutputValue);
}

void UMaterialExpressionFunctionOutput::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Input(&A));
}

void UMaterialExpressionConstant::Build(MIR::FEmitter& Em)
{
	FValueRef Value = Em.ConstantFloat(R);
	Em.Output(0, Value);
}

void UMaterialExpressionConstant2Vector::Build(MIR::FEmitter& Em)
{
	FValueRef Value = Em.ConstantFloat2({ R, G });
	Em.Output(0, Value);
	for (int i = 0; i < 2; ++i)
	{
		Em.Output(i + 1, Em.Subscript(Value, i));
	}
}

void UMaterialExpressionConstant3Vector::Build(MIR::FEmitter& Em)
{
	FValueRef Value = Em.ConstantFloat3({ Constant.R, Constant.G, Constant.B });
	Em.Output(0, Value);
	for (int i = 0; i < 3; ++i)
	{
		Em.Output(i + 1, Em.Subscript(Value, i));
	}
}

void UMaterialExpressionConstant4Vector::Build(MIR::FEmitter& Em)
{
	FValueRef Value = Em.ConstantFloat4(Constant);
	Em.Output(0, Value);
	for (int i = 0; i < 4; ++i)
	{
		Em.Output(i + 1, Em.Subscript(Value, i));
	}
}

void UMaterialExpressionGenericConstant::Build(MIR::FEmitter& Em)
{
	FValueRef Value = Em.ConstantFromShaderValue(GetConstantValue());
	Em.Output(0, Value);
}

void UMaterialExpressionConstantBiasScale::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Multiply(Em.Add(Em.ConstantFloat(Bias), Em.Input(&Input)), Em.ConstantFloat(Scale)));
}

void UMaterialExpressionStaticBool::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.ConstantBool(Value));
}

static FMaterialParameterMetadata GetMaterialExpressionParameterMetadata(MIR::FEmitter& Em, UMaterialExpression* Expr)
{
	FMaterialParameterMetadata ParameterMetadata;
	if (!Expr->GetParameterValue(ParameterMetadata))
	{
		Em.Errorf(TEXT("Could not get parameter value from expression '%s'."), *Expr->GetName());
		return {};
	}

	const FName ParameterName = Expr->GetParameterName();

	// Apply eventual parameter overrides
	switch (ParameterMetadata.Value.Type)
	{
		case EMaterialParameterType::StaticSwitch:
			for (const FStaticSwitchParameter& Param : Em.GetStaticParameterSet()->GetRuntime().StaticSwitchParameters)
			{
				if (Param.IsOverride() && Param.ParameterInfo.Name == ParameterName)
				{
					ParameterMetadata.Value.Bool[0] = Param.Value;
					break;
				}
			}
			break;

		
		case EMaterialParameterType::StaticComponentMask:
			for (const FStaticComponentMaskParameter& Param : Em.GetStaticParameterSet()->EditorOnly.StaticComponentMaskParameters)
			{
				if (Param.IsOverride() && Param.ParameterInfo.Name == ParameterName)
				{
					ParameterMetadata.Value.Bool[0] = Param.R;
					ParameterMetadata.Value.Bool[1] = Param.G;
					ParameterMetadata.Value.Bool[2] = Param.B;
					ParameterMetadata.Value.Bool[3] = Param.A;
					break;
				}
			}
			break;

		default:
			break;
	}

	return ParameterMetadata;
}

static FValueRef BuildPrimitiveUniformMaterialParameter(MIR::FEmitter& Em, UMaterialExpressionParameter* ParameterExpr)
{
	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, ParameterExpr);
	if (Em.IsInvalid())
	{
		return Em.Poison();
	}

	if (Metadata.PrimitiveDataIndex != INDEX_NONE)
	{
		switch (Metadata.Value.Type)
		{
			case EMaterialParameterType::Scalar:
				return MaterialToMIR::EmitCustomPrimitiveDataFloat1(Em, Metadata.PrimitiveDataIndex);
			
			case EMaterialParameterType::Vector:
				return MaterialToMIR::EmitCustomPrimitiveDataFloat4(Em, Metadata.PrimitiveDataIndex);

			case EMaterialParameterType::DoubleVector:
				Em.Error(TEXT("DoubleVector material parameters do not support custom primitive data."));
				return Em.Poison();

			default:
				UE_MIR_UNREACHABLE();
		}
	}

	MIR::FValueRef DefaultValue;
	switch (Metadata.Value.Type)
	{
		case EMaterialParameterType::Scalar:       DefaultValue = Em.ConstantFloat(Metadata.Value.AsScalar()); break;
		case EMaterialParameterType::Vector:       DefaultValue = Em.ConstantFloat4(UE::Math::TVector4<float>{ Metadata.Value.AsLinearColor().R, Metadata.Value.AsLinearColor().G, Metadata.Value.AsLinearColor().B, Metadata.Value.AsLinearColor().A }); break;
		case EMaterialParameterType::DoubleVector: DefaultValue = Em.ConstantDouble4(Metadata.Value.AsVector4d()); break;
		default: UE_MIR_UNREACHABLE();
	}

	return Em.NamedPrimitiveUniform(ParameterExpr->GetParameterName(), DefaultValue);
}

void UMaterialExpressionScalarParameter::Build(MIR::FEmitter& Em)
{
	Em.Output(0, BuildPrimitiveUniformMaterialParameter(Em, this));
}

void UMaterialExpressionVectorParameter::Build(MIR::FEmitter& Em)
{
	FValueRef Value = BuildPrimitiveUniformMaterialParameter(Em, this);
	Em.Output(0, Em.Swizzle(Value, MIR::FSwizzleMask::XYZ()));
	Em.Output(1, Em.Subscript(Value, 0));
	Em.Output(2, Em.Subscript(Value, 1));
	Em.Output(3, Em.Subscript(Value, 2));
	Em.Output(4, Em.Subscript(Value, 3));
	Em.Output(5, Value);
}

void UMaterialExpressionDoubleVectorParameter::Build(MIR::FEmitter& Em)
{
	FValueRef Value = BuildPrimitiveUniformMaterialParameter(Em, this);
	Em.Output(0, Em.Cast(Value, MIR::FType::MakeDoubleVector(3)));
	Em.Output(1, Em.Subscript(Value, 0));
	Em.Output(2, Em.Subscript(Value, 1));
	Em.Output(3, Em.Subscript(Value, 2));
	Em.Output(4, Em.Subscript(Value, 3));
}

void UMaterialExpressionChannelMaskParameter::Build(MIR::FEmitter& Em)
{
	FValueRef Value = BuildPrimitiveUniformMaterialParameter(Em, this);
	Em.Output(0, Em.Dot(Em.CastToFloat(Em.Input(&Input), 4), Value));
}

void UMaterialExpressionStaticBoolParameter::Build(MIR::FEmitter& Em)
{
	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, this);
	UE_MIR_CHECKPOINT(Em);

	Em.Output(0, Em.ConstantBool(Metadata.Value.Bool[0]));
}

void UMaterialExpressionStaticComponentMaskParameter::Build(MIR::FEmitter& Em)
{
	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, this);
	UE_MIR_CHECKPOINT(Em);

	check(Metadata.Value.Type == EMaterialParameterType::StaticComponentMask);

	MIR::FSwizzleMask Mask;
	if (Metadata.Value.Bool[0])
	{
		Mask.Append(MIR::EVectorComponent::X);
	}
	if (Metadata.Value.Bool[1])
	{
		Mask.Append(MIR::EVectorComponent::Y);
	}
	if (Metadata.Value.Bool[2])
	{
		Mask.Append(MIR::EVectorComponent::Z);
	}
	if (Metadata.Value.Bool[3])
	{
		Mask.Append(MIR::EVectorComponent::W);
	}
	
	Em.Output(0, Em.Swizzle(Em.Input(&Input), Mask));
}

void UMaterialExpressionStaticSwitch::Build(MIR::FEmitter& Em)
{
	bool bCondition = Em.ToConstantBool(Em.InputDefaultBool(&Value, DefaultValue));
	UE_MIR_CHECKPOINT(Em); // Make sure that the bCondition value was available and didn't raise an error

	Em.Output(0, Em.Input(bCondition ? &A : &B));
}

void UMaterialExpressionStaticSwitchParameter::Build(MIR::FEmitter& Em)
{
	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, this);
	UE_MIR_CHECKPOINT(Em);

	check(Metadata.Value.Type == EMaterialParameterType::StaticSwitch);

	// Make sure both input pins are always connected to some expression even if we don't build one.
	MaterialToMIR::CheckInputIsConnected(Em, A, TEXT("A"));
	MaterialToMIR::CheckInputIsConnected(Em, B, TEXT("B"));
	
	Em.Output(0, Em.Input(Metadata.Value.Bool[0] ? &A : &B));
}

void UMaterialExpressionDataDrivenShaderPlatformInfoSwitch::Build(MIR::FEmitter& Em)
{
	FString Error;
	FExpressionInput* EffectiveInput = GetEffectiveInput(Em.GetShaderPlatform(), Error);
	if (!EffectiveInput)
	{
		Em.Error(Error);
		return;
	}

	Em.Output(0, Em.Input(EffectiveInput));
}

void UMaterialExpressionDistanceFieldsRenderingSwitch::Build(MIR::FEmitter& Em)
{
	EShaderPlatform ShaderPlatform = Em.GetShaderPlatform();
	bool bDistanceFieldsEnabled = IsMobilePlatform(ShaderPlatform) ? IsMobileDistanceFieldEnabled(ShaderPlatform) : IsUsingDistanceFields(ShaderPlatform);

	// Make sure both input pins are always connected to some expression even if we don't build one.
	MaterialToMIR::CheckInputIsConnected(Em, Yes, TEXT("Yes"));
	MaterialToMIR::CheckInputIsConnected(Em, No, TEXT("No"));
	
	Em.Output(0, Em.Input(bDistanceFieldsEnabled ? &Yes : &No));
}

void UMaterialExpressionFeatureLevelSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef Result = Em.TryInput(&Inputs[GetFeatureLevelToCompile(Em.GetShaderPlatform(), Em.GetFeatureLevel())]);
	UE_MIR_CHECKPOINT(Em);

	if (!Result)
	{
		Result = Em.Input(&Default);
	}
	else
	{
		// Make sure the default pin is always connected to some expression even if we don't build it.
		MaterialToMIR::CheckInputIsConnected(Em, Default, TEXT("Default"));
	}

	Em.Output(0, Result);
}

void UMaterialExpressionQualitySwitch::Build(MIR::FEmitter& Em)
{
	EMaterialQualityLevel::Type QualityLevelToCompile = Em.GetQualityLevel();
	if (QualityLevelToCompile != EMaterialQualityLevel::Num)
	{
		check(QualityLevelToCompile < UE_ARRAY_COUNT(Inputs));

		FValueRef Result = Em.TryInput(&Inputs[QualityLevelToCompile]);
		if (!Result)
		{
			Result = Em.Input(&Default);
		}
		else
		{
			// Make sure the default pin is always connected to some expression even if we don't build it.
			MaterialToMIR::CheckInputIsConnected(Em, Default, TEXT("Default"));
		}
		Em.Output(0, Result);
	}
	else
	{
		Em.Output(0, Em.Input(&Default));
	}
}

void UMaterialExpressionRequiredSamplersSwitch::Build(MIR::FEmitter& Em)
{
	const EShaderPlatform ShaderPlatform = Em.GetShaderPlatform();
	const bool bCheck = RequiredSamplers <= FDataDrivenShaderPlatformInfo::GetMaxSamplers(ShaderPlatform);

	Em.Output(0, Em.Input(bCheck ? &InputTrue : &InputFalse));
}

void UMaterialExpressionShaderStageSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef PixelValue = Em.Input(&PixelShader);
	FValueRef VertexValue = Em.Input(&VertexShader);
	UE_MIR_CHECKPOINT(Em);

	MIR::FType CommonType = Em.GetCommonType(PixelValue->Type, VertexValue->Type);
	UE_MIR_CHECKPOINT(Em);

	PixelValue = Em.Cast(PixelValue, CommonType);
	VertexValue = Em.Cast(VertexValue, CommonType);

	static_assert(MIR::EStage::NumStages == 3);
	TStaticArray<FValueRef, MIR::EStage::NumStages> ValuePerStage;
	ValuePerStage[MIR::EStage::Stage_Vertex] = VertexValue;
	ValuePerStage[MIR::EStage::Stage_Pixel] = PixelValue;
	ValuePerStage[MIR::EStage::Stage_Compute] = PixelValue;

	Em.Output(0, Em.StageSwitch(VertexValue->Type, ValuePerStage));
}

void UMaterialExpressionShadingPathSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef Result = Em.TryInput(&Inputs[GetShadingPathToCompile(Em.GetShaderPlatform(), Em.GetFeatureLevel(), Em.GetQualityLevel())]);
	if (!Result)
	{
		Result = Em.Input(&Default);
	}
	else
	{
		// Make sure the default pin is always connected to some expression even if we don't build it.
		MaterialToMIR::CheckInputIsConnected(Em, Default, TEXT("Default"));
	}
	Em.Output(0, Result);
}

void UMaterialExpressionVirtualTextureFeatureSwitch::Build(MIR::FEmitter& Em)
{
	// Make sure both input pins are always connected to some expression even if we don't build one.
	MaterialToMIR::CheckInputIsConnected(Em, Yes, TEXT("Yes"));
	MaterialToMIR::CheckInputIsConnected(Em, No, TEXT("No"));

	Em.Output(0, Em.Input(UseVirtualTexturing(Em.GetShaderPlatform()) ? &Yes : &No));
}

static void EmitExternalCodeConditionalReplace(MIR::FEmitter& Em, FName ExternalCodeIdentifier, FExpressionInput& Default, FExpressionInput& Replace)
{
	FValueRef DefaultValue = Em.Input(&Default);
	FValueRef ReplaceValue = Em.Input(&Replace);
	UE_MIR_CHECKPOINT(Em);

	MIR::FType CommonType = Em.GetCommonType(DefaultValue->Type, ReplaceValue->Type);
	UE_MIR_CHECKPOINT(Em);

	DefaultValue = Em.Cast(DefaultValue, CommonType);
	ReplaceValue = Em.Cast(ReplaceValue, CommonType);

	Em.Output(0, Em.Branch(Em.Extern<MIR::FExternFromMaterialDecl>(ExternalCodeIdentifier), ReplaceValue, DefaultValue));
}

void UMaterialExpressionLightmassReplace::Build(MIR::FEmitter& Em)
{
	// @massimo.tristano Missing check whether this is a "lightmass compiler" and optimize out to Lightmass
	EmitExternalCodeConditionalReplace(Em, FName("LightmassReplace"), Realtime, Lightmass);
}

void UMaterialExpressionMeshPaintTextureReplace::Build(MIR::FEmitter& Em)
{
	EmitExternalCodeConditionalReplace(Em, FName("MeshPaintTextureReplace"), Default, MeshPaintTexture);
}

void UMaterialExpressionNaniteReplace::Build(MIR::FEmitter& Em)
{
	EmitExternalCodeConditionalReplace(Em, FName("NaniteReplace"), Default, Nanite);
}

void UMaterialExpressionReflectionCapturePassSwitch::Build(MIR::FEmitter& Em)
{
	EmitExternalCodeConditionalReplace(Em, FName("ReflectionCapturePassSwitch"), Default, Reflection);
}

void UMaterialExpressionShadowReplace::Build(MIR::FEmitter& Em)
{
	EmitExternalCodeConditionalReplace(Em, FName("ShadowReplace"), Default, Shadow);
}

void UMaterialExpressionAppendVector::Build(MIR::FEmitter& Em)
{
	FValueRef AVal = Em.CheckIsScalarOrVector(Em.Input(&A));
	FValueRef BVal = Em.CheckIsScalarOrVector(Em.TryInput(&B));

	UE_MIR_CHECKPOINT(Em);

	MIR::FPrimitive AType = AVal->Type.GetPrimitive();
	MIR::FPrimitive BType = BVal ? BVal->Type.GetPrimitive() : MIR::FPrimitive{};

	int Dimensions = AType.NumColumns + (BVal ? BType.NumColumns : 0);
	if (Dimensions > 4)
	{
		Em.Errorf(TEXT("The resulting vector would have %d component (it can have at most 4)."), Dimensions);
		return;
	}

	check(Dimensions >= 2 && Dimensions <= 4);

	// Construct the output vector type.
	MIR::EScalarKind ResultKind = (AType.IsDouble() || (BVal && BType.IsDouble())) ? MIR::EScalarKind::Double : MIR::EScalarKind::Float;
	MIR::FType ResultType = MIR::FType::MakeVector(ResultKind, Dimensions);

	// Set up each output vector component.  These need CastToScalarKind in case we are appending LWC and non-LWC.
	FValueRef Components[4] = { nullptr, nullptr, nullptr, nullptr };
	int ComponentIndex = 0;
	for (int i = 0; i < AType.NumColumns; ++i, ++ComponentIndex)
	{
		Components[ComponentIndex] = Em.CastToScalarKind(Em.Subscript(AVal, i), ResultKind);
	}

	if (BVal)
	{
		for (int i = 0; i < BType.NumColumns; ++i, ++ComponentIndex)
		{
			Components[ComponentIndex] = Em.CastToScalarKind(Em.Subscript(BVal, i), ResultKind);
		}
	}

	// Create the vector value and output it.
	FValueRef Output{};
	if (Dimensions == 2)
	{
		Output = Em.Vector2(Components[0], Components[1]);
	}
	else if (Dimensions == 3)
	{
		Output = Em.Vector3(Components[0], Components[1], Components[2]);
	}
	else
	{
		Output = Em.Vector4(Components[0], Components[1], Components[2], Components[3]);
	}

	Em.Output(0, Output);
}

/* Unary Operators */

void UMaterialExpressionAbs::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Abs(Em.Input(&Input)));
}

void UMaterialExpressionCeil::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Ceil(Em.Input(&Input)));
}

void UMaterialExpressionFloor::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Floor(Em.Input(&Input)));
}

void UMaterialExpressionFrac::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Frac(Em.Input(&Input)));
}

void UMaterialExpressionLength::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Length(Em.Input(&Input)));
}

void UMaterialExpressionNormalize::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloatKind(Em.Input(&VectorInput));
	if (InputValue->Type.IsScalar())
	{
		Em.Output(0, Em.ConstantOne(MIR::EScalarKind::Float));
	}
	else
	{
		Em.Output(0, Em.Multiply(InputValue, Em.Rsqrt(Em.Dot(InputValue, InputValue))));
	}
}

void UMaterialExpressionRound::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Round(Em.Input(&Input)));
}

void UMaterialExpressionExponential::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Exponential(Em.Input(&Input)));
}

void UMaterialExpressionExponential2::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Exponential2(Em.Input(&Input)));
}

void UMaterialExpressionLogarithm::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Logarithm(Em.Input(&Input)));
}

void UMaterialExpressionLogarithm2::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Logarithm2(Em.Input(&X)));
}

void UMaterialExpressionLogarithm10::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Logarithm10(Em.Input(&X)));
}

void UMaterialExpressionTruncate::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Truncate(Em.Input(&Input)));
}

void UMaterialExpressionArccosine::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.ACos(Em.Input(&Input)));
}

void UMaterialExpressionArcsine::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.ASin(Em.Input(&Input)));
}

void UMaterialExpressionArctangent::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.ATan(Em.Input(&Input)));
}

void UMaterialExpressionArccosineFast::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Operator(MIR::UO_ACosFast, Em.Input(&Input)));
}

void UMaterialExpressionArcsineFast::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Operator(MIR::UO_ASinFast, Em.Input(&Input)));
}

void UMaterialExpressionArctangentFast::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Operator(MIR::UO_ATanFast, Em.Input(&Input)));
}

void UMaterialExpressionComponentMask::Build(MIR::FEmitter& Em)
{
	FValueRef Value = Em.Input(&Input);

	MIR::FSwizzleMask Mask;
	if (R)
	{
		Mask.Append(MIR::EVectorComponent::X);
	}
	if (G)
	{
		Mask.Append(MIR::EVectorComponent::Y);
	}
	if (B)
	{
		Mask.Append(MIR::EVectorComponent::Z);
	}
	if (A)
	{
		Mask.Append(MIR::EVectorComponent::W);
	}

	Em.Output(0, Em.Swizzle(Value, Mask));
}

static FValueRef PositiveClampedPow(MIR::FEmitter& Em, FValueRef Base, FValueRef Exponent)
{
	FValueRef PrimitiveBase = Em.CheckIsPrimitive(Base);
	if (!PrimitiveBase.IsValid())
	{
		return PrimitiveBase.ToPoison();
	}

	TOptional<MIR::FPrimitive> ValuePrimitiveType = Base->Type.AsPrimitive();
	return Em.Select(
				Em.LessThanOrEquals(Base, Em.ConstantFloat(2.980233e-8f)),
				Em.ConstantZero(ValuePrimitiveType->ScalarKind),
				Em.Pow(Base, Exponent));
}

void UMaterialExpressionPower::Build(MIR::FEmitter& Em)
{
	Em.Output(0, PositiveClampedPow(Em, 
					 				Em.Input(&Base),
					 				Em.InputDefaultFloat(&Exponent, ConstExponent)));
}

static FValueRef GetTrigonometricInputWithPeriod(MIR::FEmitter& Em, const FExpressionInput* Input, float Period)
{
	// Get input after checking it has primitive type.
	FValueRef Value = Em.CheckIsArithmetic(Em.Input(Input));
	if (Period > 0.0f)
	{
		Value = Em.Multiply(Value, Em.ConstantFloat(2.0f * (float)UE_PI / Period));
	}
	return Value;
}

void UMaterialExpressionCosine::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Cos(GetTrigonometricInputWithPeriod(Em, &Input, Period)));
}

void UMaterialExpressionSine::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Sin(GetTrigonometricInputWithPeriod(Em, &Input, Period)));
}

void UMaterialExpressionTangent::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Tan(GetTrigonometricInputWithPeriod(Em, &Input, Period)));
}

void UMaterialExpressionSaturate::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Saturate(Em.Input(&Input)));
}

void UMaterialExpressionSign::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Sign(Em.Input(&Input)));
}

void UMaterialExpressionSquareRoot::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Sqrt(Em.Input(&Input)));
}

void UMaterialExpressionExternalCodeBase::Build(MIR::FEmitter& Em)
{
	BuildMaterialExpressionExternalCodeBase(Em, *this);
}

/* Binary Operators */

void UMaterialExpressionDesaturation::Build(MIR::FEmitter& Em)
{
	FValueRef ColorValue = Em.CastToFloat(Em.Input(&Input), 3);
	FValueRef GreyOrLerpValue = Em.Dot(ColorValue, Em.ConstantFloat3(FVector3f(LuminanceFactors))); // todo: check
	FValueRef FractionValue = Em.TryInput(&Fraction);
	if (FractionValue)
	{
		GreyOrLerpValue = Em.Lerp(ColorValue, GreyOrLerpValue, FractionValue);
	}
	Em.Output(0, GreyOrLerpValue);
}

void UMaterialExpressionDistance::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Length(Em.Subtract(Em.Input(&A), Em.Input(&B))));
}

void UMaterialExpressionFmod::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Fmod(Em.Input(&A), Em.Input(&B)));
}

static void BuildBinaryOperatorWithDefaults(MIR::FEmitter& Em, MIR::EOperator Op, const FExpressionInput* A, float ConstA, const FExpressionInput* B, float ConstB)
{
	FValueRef AVal = Em.InputDefaultFloat(A, ConstA);
	FValueRef BVal = Em.InputDefaultFloat(B, ConstB);
	Em.Output(0, Em.Operator(Op, AVal, BVal));
}

void UMaterialExpressionAdd::Build(MIR::FEmitter& Em)
{ 
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Add, &A, ConstA, &B, ConstB);
}

void UMaterialExpressionSubtract::Build(MIR::FEmitter& Em)
{
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Subtract, &A, ConstA, &B, ConstB);
}

void UMaterialExpressionMultiply::Build(MIR::FEmitter& Em)
{
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Multiply, &A, ConstA, &B, ConstB);
}

void UMaterialExpressionDivide::Build(MIR::FEmitter& Em)
{
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Divide, &A, ConstA, &B, ConstB);
}

void UMaterialExpressionMax::Build(MIR::FEmitter& Em)
{
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Max, &A, ConstA, &B, ConstB);
}

void UMaterialExpressionMin::Build(MIR::FEmitter& Em)
{ 
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Min, &A, ConstA, &B, ConstB);
}

void UMaterialExpressionModulo::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Operator(MIR::BO_Modulo, Em.Input(&A), Em.Input(&B)));
}

void UMaterialExpressionStep::Build(MIR::FEmitter& Em)
{
	BuildBinaryOperatorWithDefaults(Em, MIR::BO_Step, &Y, ConstY, &X, ConstX);
}

void UMaterialExpressionDotProduct::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Dot(Em.Input(&A), Em.Input(&B)));
}

void UMaterialExpressionCrossProduct::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Cross(Em.Input(&A), Em.Input(&B)));
}

void UMaterialExpressionArctangent2::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Atan2(Em.Input(&Y), Em.Input(&X)));
}

void UMaterialExpressionArctangent2Fast::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.Operator(MIR::BO_ATan2Fast, Em.Input(&Y), Em.Input(&X)));
}

void UMaterialExpressionEyeAdaptationInverse::Build(MIR::FEmitter& Em)
{
	check(ExternalCodeIdentifiers.Num() == 1);
	FValueRef LightValue = Em.CastToFloat(Em.InputDefaultFloat(&LightValueInput, 1.0f), 3);
	FValueRef AlphaValue = Em.CastToFloat(Em.InputDefaultFloat(&AlphaInput, 1.0f), 1);
	FValueRef MultiplierValue = MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, AlphaValue);
	Em.Output(0, Em.Multiply(LightValue, MultiplierValue));
}

void UMaterialExpressionOneMinus::Build(MIR::FEmitter& Em)
{
	// Default input to zero if not connected, then get it as a primitive.
	FValueRef Value = Em.InputDefaultFloat(&Input, 0.0f);

	UE_MIR_CHECKPOINT(Em); // verify the value is valid

	// Make a "One" value of the same type and dimension as input's.
	FValueRef One = Em.ConstantOne(Value->Type.AsPrimitive()->ScalarKind);

	// And flow the subtraction out of the expression's only output.
	Em.Output(0, Em.Subtract(One, Value));
}

void UMaterialExpressionIfThenElse::Build(MIR::FEmitter& Em)
{
	// Get the condition value checking it is a bool scalar
	FValueRef ConditionValue = Em.CastToBool(Em.InputDefaultBool(&Condition, false), 1);

	UE_MIR_CHECKPOINT(Em); // Make sure the condition value is valid
	
	// If condition boolean is constant, select which input is active and simply
	// bypass its value to our output.
	if (MIR::FConstant* Constant = ConditionValue->As<MIR::FConstant>())
	{
		FExpressionInput* ActiveInput = Constant->Boolean ? &True : &False;
		Em.Output(0, Em.Input(ActiveInput));
		return;
	}

	// The condition isn't static; Get the true and false values.
	// If any is disconnected, the emitter will report an error.
	FValueRef ThenValue = Em.Input(&True);
	FValueRef ElseValue = Em.Input(&False);
	
	UE_MIR_CHECKPOINT(Em); // Make sure both true and false inputs are defined

	MIR::FType CommonType = Em.GetCommonType(ThenValue->Type, ElseValue->Type);

	UE_MIR_CHECKPOINT(Em); // Make sure the common type is valid

	// Cast the "then" and "else" values to the common type.
	ThenValue = Em.Cast(ThenValue, CommonType);
	ElseValue = Em.Cast(ElseValue, CommonType);

	// Emit the branch instruction
	FValueRef OutputValue = Em.Branch(ConditionValue, ThenValue, ElseValue);

	Em.Output(0, OutputValue);
}

static FValueRef EmitAlmostEquals(MIR::FEmitter& Em, FValueRef A, FValueRef B, float Threshold)
{
	// abs(A - B) <= Threshold
	return Em.LessThanOrEquals(Em.Abs(Em.Subtract(A, B)), Em.ConstantFloat(Threshold));
}

void UMaterialExpressionIf::Build(MIR::FEmitter& Em)
{
	FValueRef AValue = Em.InputDefaultFloat(&A, 0.f);
	FValueRef BValue = Em.InputDefaultFloat(&B, ConstB);
	FValueRef AGreaterThanBValue = Em.InputDefaultFloat(&AGreaterThanB, 0.f);
	FValueRef AEqualsBValue = Em.TryInput(&AEqualsB);
	FValueRef ALessThanBValue = Em.InputDefaultFloat(&ALessThanB, 0.f);

	// Less than comparison -- if equals value isn't present (see below), AGreaterThanBValue will also be returned for the equal case.
	FValueRef ALessThanBConditionValue = Em.LessThan(AValue, BValue);
	FValueRef OutputValue = Em.Branch(ALessThanBConditionValue, ALessThanBValue, AGreaterThanBValue);

	// Equals value is optional -- if present, generate an additional conditional.
	if (AEqualsBValue.IsValid())
	{
		FValueRef AEqualsBConditionValue = EmitAlmostEquals(Em, AValue, BValue, EqualsThreshold);
		OutputValue = Em.Branch(AEqualsBConditionValue, AEqualsBValue, OutputValue);
	}

	Em.Output(0, OutputValue);
}

static void ErrorUnlessFeatureLevelSupported(FMaterialIRModule* Module, ERHIFeatureLevel::Type RequiredFeatureLevel, const TCHAR* Message)
{
	ERHIFeatureLevel::Type FeatureLevel = Module->GetFeatureLevel();
	if (FeatureLevel < RequiredFeatureLevel)
	{
		FString FeatureLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);
		Module->AddError(nullptr, FString::Printf(TEXT("%s  Current feature level is %s."), Message, *FeatureLevelName));
	}
}

enum class EScreenTexture : uint8
{
	SceneTexture,
	UserSceneTexture,
	SceneColor,
	SceneDepth,
	SceneDepthWithoutWater,
	DBufferTexture,
};

// Reference to various types of full screen textures accessible by expressions.  These
// generally involve custom code to set data in the compilation output, extensive validation
// logic, and HLSL generation deferred to the analyzer for the case of UserSceneTextures.
struct FScreenTextureExtern
{
	EScreenTexture Kind;

	union
	{
		ESceneTextureId SceneTextureId;
		EDBufferTextureId DBufferTextureId;
	};

	FName UserSceneTexture;

	FScreenTextureExtern(ESceneTextureId InId)
	: Kind{ EScreenTexture::SceneTexture }
	, SceneTextureId{ InId }
	{
	}

	FScreenTextureExtern(FName InUserSceneTexture)
	: Kind{ EScreenTexture::UserSceneTexture }
	, UserSceneTexture{ InUserSceneTexture }
	{
	}

	FScreenTextureExtern(EDBufferTextureId DBufferTextureId)
	: Kind{ EScreenTexture::DBufferTexture }
	, DBufferTextureId { DBufferTextureId }
	{
	}

	FScreenTextureExtern(EScreenTexture TextureKind)
	: Kind(TextureKind)
	{
	}

	MIR::FExternInfo GetInfo() const
	{
		return {
			.Name = TEXTVIEW("ScreenTexture"),
			.Type = MIR::FType::MakeIntScalar(),
			.Flags = MIR::EExternFlags::Inline,
		};
	}

	void Analyze(MIR::FExternAnalysisContext& Context)
	{
		const UMaterial* Material = Context.MaterialInterface->GetMaterial();
		FMaterialCompilationOutput& CompilationOutput = Context.Module->GetCompilationOutput();
		const EMaterialDomain MaterialDomain = Material->MaterialDomain;

		switch (Kind)
		{
			case EScreenTexture::SceneTexture:
			case EScreenTexture::UserSceneTexture:
			{
				// TODO: If referenced from custom HLSL, this can be false. Revisit when custom HLSL support is added to the new translator.
				const bool bTextureLookup = true;

				// Add defines and compilation outputs.
				Context.Module->AddIntegerEnvironmentDefine(FName("NEEDS_SCENE_TEXTURES"), 1);

				if (Kind == EScreenTexture::UserSceneTexture)
				{
					if (UserSceneTexture.IsNone())
					{
						Context.Module->AddError(nullptr, TEXT("UserSceneTexture missing name. Value must be set to something other than None."));
						return;
					}
					else
					{
						// Allocate value during Analyze and check for failure. FindUserSceneTexture is later called during HLSL generation.
						SceneTextureId = (ESceneTextureId)CompilationOutput.FindOrAddUserSceneTexture(UserSceneTexture);
						if ((int32)SceneTextureId == INDEX_NONE)
						{
							Context.Module->AddError(nullptr, FString::Printf(TEXT("Too many unique UserSceneTexture inputs in the post process material. Max allowed is %d."), kPostProcessMaterialInputCountMax));
							return;
						}
					}
				}

				CompilationOutput.bNeedsSceneTextures = true;
				CompilationOutput.SetIsSceneTextureUsed(SceneTextureId);

				// Substrate TODO:
				// When SceneTexture lookup is used, simple and single paths are disabled to ensure correct decoding.
				// Reading SceneTexture with Substrate implies unpacking material buffer data.
				// To avoid compiling out unpacking paths, force simple and single versions to be disabled.
				// FSubstrateCompilationContext& SubstrateContext = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
				// SubstrateContext.SubstrateMaterialComplexity.bIsSimple = false;
				// SubstrateContext.SubstrateMaterialComplexity.bIsSingle = false;

				const EShaderPlatform Platform = Context.Module->GetShaderPlatform();

				const bool bHasSingleLayerWaterSM =
					Context.MaterialInterface->GetShadingModels().HasShadingModel(MSM_SingleLayerWater);

				if (bHasSingleLayerWaterSM && SceneTextureId != PPI_CustomDepth && SceneTextureId != PPI_CustomStencil)
				{
					Context.Module->AddError(nullptr, TEXT("Only CustomDepth and CustomStencil can be sampled with SceneTexture when using the Single Layer Water shading model."));
				}

				if (SceneTextureId == PPI_DecalMask)
				{
					Context.Module->AddError(nullptr, TEXT("Decal mask bit was moved from the GBuffer to the stencil buffer and is no longer available."));
				}

				if (MaterialDomain == MD_DeferredDecal)
				{
					const bool bSceneTextureSupportsDecal = SceneTextureId == PPI_SceneDepth || SceneTextureId == PPI_WorldNormal || SceneTextureId == PPI_CustomDepth || SceneTextureId == PPI_CustomStencil;

					if (!bSceneTextureSupportsDecal)
					{
						Context.Module->AddError(nullptr, TEXT("Decals can only access SceneDepth, CustomDepth, CustomStencil, and WorldNormal."));
					}

					if (SceneTextureId == PPI_WorldNormal)
					{
						ErrorUnlessFeatureLevelSupported(Context.Module, ERHIFeatureLevel::SM5, TEXT("Deferred decals require SM5 for world normal access."));
					}

					if (SceneTextureId == PPI_WorldNormal && !IsUsingDBuffers(Platform))
					{
						const bool bHasNormalConnected = Substrate::IsSubstrateEnabled() ? Material->IsPropertyConnected(MP_Normal) : Material->HasNormalConnected();
						if (bHasNormalConnected)
						{
							Context.Module->AddError(nullptr, TEXT("Decals that read WorldNormal cannot output to Normal at the same time. Enable DBuffer to support this."));
						}
					}
				}

				bool bNeedsSceneTexturePostProcessInputs = false;
				if (bTextureLookup)
				{
					bNeedsSceneTexturePostProcessInputs =
						(SceneTextureId >= PPI_PostProcessInput0 && SceneTextureId <= PPI_PostProcessInput6)
						|| (SceneTextureId >= PPI_UserSceneTexture0 && SceneTextureId <= PPI_UserSceneTexture6)
						|| SceneTextureId == PPI_Velocity
						|| SceneTextureId == PPI_SceneColor;
				}

				if (SceneTextureId == PPI_Velocity && MaterialDomain != MD_PostProcess)
				{
					Context.Module->AddError(nullptr, TEXT("Velocity scene textures are only available in post process materials."));
				}

				if (MaterialDomain != MD_DeferredDecal && MaterialDomain != MD_PostProcess)
				{
					if (!bHasSingleLayerWaterSM && IsOpaqueOrMaskedBlendMode(Material->BlendMode))
					{
						Context.Module->AddError(nullptr, TEXT("SceneTexture expressions cannot be used in opaque materials except with Single Layer Water."));
					}
					else if (bNeedsSceneTexturePostProcessInputs)
					{
						Context.Module->AddError(nullptr, TEXT("SceneTexture expressions cannot use post process inputs or SceneColor outside post process materials."));
					}
				}

				if (SceneTextureId == PPI_SceneDepth && bTextureLookup && MaterialDomain != MD_PostProcess && !IsTranslucentBlendMode(Material->BlendMode))
				{
					Context.Module->AddError(nullptr, TEXT("Only transparent or post process materials can read from SceneDepth."));
				}
				break;
			}

			case EScreenTexture::SceneColor:
			{
				CompilationOutput.SetIsSceneTextureUsed(PPI_SceneColor);

				if (MaterialDomain != MD_Surface)
				{
					Context.Module->AddError(nullptr, TEXT("SceneColor lookups are only available when MaterialDomain is Surface."));
				}

				ErrorUnlessFeatureLevelSupported(Context.Module, ERHIFeatureLevel::SM5, TEXT("SceneColor access requires SM5."));
				break;
			}

			case EScreenTexture::SceneDepth:
			{
				CompilationOutput.SetIsSceneTextureUsed(PPI_SceneDepth);

				if (Context.MaterialInterface->IsTranslucencyWritingVelocity())
				{
					Context.Module->AddError(nullptr, TEXT("Translucent materials writing velocity cannot read from SceneDepth."));
				}

				if (MaterialDomain != MD_PostProcess && !IsTranslucentBlendMode(Material->BlendMode))
				{
					Context.Module->AddError(nullptr, TEXT("Only transparent or post process materials can read from SceneDepth."));
				}
				break;
			}

			case EScreenTexture::SceneDepthWithoutWater:
			{
				if (MaterialDomain != MD_PostProcess)
				{
					if (!Context.MaterialInterface->GetShadingModels().HasShadingModel(MSM_SingleLayerWater))
					{
						Context.Module->AddError(nullptr, TEXT("Reading scene depth below water requires Single Layer Water or PostProcess domain."));
					}

					if (MaterialDomain != MD_Surface)
					{
						Context.Module->AddError(nullptr, TEXT("SceneDepthWithoutWater requires Surface or PostProcess material domain."));
					}

					if (IsTranslucentBlendMode(Context.Module->GetBlendMode()))
					{
						Context.Module->AddError(nullptr, TEXT("SceneDepthWithoutWater cannot be used with translucent blend mode."));
					}
				}
				break;
			}

			case EScreenTexture::DBufferTexture:
			{
				Context.Module->AddEnvironmentDefine(FName("Material_Uses_Decal_Lookup"));

				CompilationOutput.SetIsDBufferTextureUsed(DBufferTextureId);
				CompilationOutput.SetIsDBufferTextureLookupUsed(true);

				if (MaterialDomain != MD_Surface || IsTranslucentBlendMode(Context.Module->GetBlendMode()))
				{
					Context.Module->AddError(nullptr, TEXT("DBuffer scene textures are only available on opaque or masked surfaces."));
				}
				break;
			}
		}
	}

	void AnalyzeInStage(MIR::FExternAnalysisContext& Context, MIR::EStage Stage)
	{
		switch (Kind)
		{
			case EScreenTexture::SceneColor:
				if (Stage == MIR::Stage_Vertex)
				{
					Context.Module->AddError(nullptr, TEXT("SceneColor is only supported in pixel shader input."));
				}
				break;

			case EScreenTexture::SceneDepth:
				if (Stage == MIR::Stage_Vertex)
				{
					ErrorUnlessFeatureLevelSupported(Context.Module, ERHIFeatureLevel::SM5, TEXT("Reading SceneDepth from the vertex shader requires SM5."));
				}
				break;

			case EScreenTexture::SceneDepthWithoutWater:
				if (Stage == MIR::Stage_Vertex)
				{
					Context.Module->AddError(nullptr, TEXT("Cannot read SceneDepthWithoutWater from the vertex shader."));
				}
				break;

			default:
				break;
		}
	}

	void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
	{
		check(Printer.Differential == MIR::EExternDifferential::None);

		switch (Kind)
		{
			case EScreenTexture::SceneTexture:
			case EScreenTexture::UserSceneTexture:
				Printer << UE::MaterialTranslatorUtils::SceneTextureIdToHLSLString(SceneTextureId);
				break;

			case EScreenTexture::DBufferTexture:
				Printer << (int32)DBufferTextureId;
				break;

			default:
				break;
		}
	}

	void CopyTo(FScreenTextureExtern& Other) const
	{
		Other.Kind = Kind;
		Other.SceneTextureId = SceneTextureId;
		Other.UserSceneTexture = UserSceneTexture;
	}
};

// If DefaultOffset is not null, Coordinates are treated as an offset (or DefaultOffset if unset), rather than absolute coordinates.  Clamping is
// automatically applied for custom or offset fetches -- the "bClamped" parameter only controls clamping for default texture coordinate fetches,
// and is only needed when fetching from lower resolution User Scene Textures.  A zero constant can be passed in for "SceneTextureInput" for
// cases where the default view rect should be used for UV calculations.
static FValueRef SceneTextureExpressionTexCoords(MIR::FEmitter& Em, FValueRef SceneTextureInput, const FExpressionInput& Coordinates, const FVector2D* DefaultOffset = nullptr, bool bClamped = false)
{
	FValueRef TexCoords;
	if (DefaultOffset)
	{
		TexCoords = Em.Cast(Em.InputDefaultFloat2(&Coordinates, { (float)DefaultOffset->X, (float)DefaultOffset->Y }), MIR::FType::MakeFloatVector(2));
		TexCoords = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("CalcScreenUVFromOffsetFraction(GetScreenPosition(Parameters), $0)") }, TexCoords);
	}
	else
	{
		TexCoords = Em.Cast(Em.TryInput(&Coordinates), MIR::FType::MakeFloatVector(2));
		if (TexCoords)
		{
			// Convert from viewport to scene texture space
			TexCoords = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("ClampSceneTextureUV(ViewportUVToSceneTextureUV($0, $1), $1)") }, TexCoords, SceneTextureInput);
		}
		else
		{
			const TCHAR* Code = bClamped ? TEXT("ClampSceneTextureUV(GetDefaultSceneTextureUV(Parameters, $0), $0)") : TEXT("GetDefaultSceneTextureUV(Parameters, $0)");
			TexCoords = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, Code }, SceneTextureInput);
		}
	}
	return TexCoords;
}

static void SceneTextureExpressionBuild(MIR::FEmitter& Em, FValueRef SceneTextureInput, const FExpressionInput& Coordinates, const FVector2D* DefaultOffset, bool bClamped, bool bFiltered)
{
	FValueRef TexCoords = SceneTextureExpressionTexCoords(Em, SceneTextureInput, Coordinates, DefaultOffset, bClamped);
	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("SceneTextureLookup(Parameters, $0, $1, $2)") }, TexCoords, SceneTextureInput, Em.ConstantBool(bFiltered)));

	FValueRef SceneTextureViewSize = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("GetSceneTextureViewSize($0)") }, SceneTextureInput);
	Em.Output(1, Em.Swizzle(SceneTextureViewSize, MIR::FSwizzleMask::XY()));
	Em.Output(2, Em.Swizzle(SceneTextureViewSize, MIR::FSwizzleMask::ZW()));
}

void UMaterialExpressionSceneTexture::Build(MIR::FEmitter& Em)
{
	const bool bClamped = false;
	SceneTextureExpressionBuild(Em, Em.Extern<FScreenTextureExtern>({ SceneTextureId.GetValue() }), Coordinates, nullptr, bClamped, bFiltered);
}

void UMaterialExpressionUserSceneTexture::Build(MIR::FEmitter& Em)
{
	SceneTextureExpressionBuild(Em, Em.Extern<FScreenTextureExtern>({ UserSceneTexture }), Coordinates, nullptr, bClamped, bFiltered);
}

void UMaterialExpressionSceneColor::Build(MIR::FEmitter& Em)
{
	FValueRef TexCoords = SceneTextureExpressionTexCoords(Em, Em.ConstantInt(0), Input, this->InputMode == EMaterialSceneAttributeInputMode::OffsetFraction ? &ConstInput : nullptr);

	// We need a dependency on ScreenTexture as a second argument, so the value analyzer can see it, even though it's technically not used in the code.
	FValueRef ScreenTexture = Em.Extern<FScreenTextureExtern>({ EScreenTexture::SceneColor });
	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("DecodeSceneColorAndAlpharForMaterialNode($0)") }, TexCoords, ScreenTexture));
}

void UMaterialExpressionSceneDepth::Build(MIR::FEmitter& Em)
{
	FValueRef TexCoords = SceneTextureExpressionTexCoords(Em, Em.ConstantInt(0), Input, this->InputMode == EMaterialSceneAttributeInputMode::OffsetFraction ? &ConstInput : nullptr);

	// We need a dependency on ScreenTexture as a second argument, so the value analyzer can see it, even though it's technically not used in the code.
	FValueRef ScreenTexture = Em.Extern<FScreenTextureExtern>({ EScreenTexture::SceneDepth });
	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("CalcSceneDepth($0)") }, TexCoords, ScreenTexture));
}

void UMaterialExpressionSceneDepthWithoutWater::Build(MIR::FEmitter& Em)
{
	FValueRef TexCoords = SceneTextureExpressionTexCoords(Em, Em.ConstantInt(0), Input, this->InputMode == EMaterialSceneAttributeInputMode::OffsetFraction ? &ConstInput : nullptr);

	// We need a dependency on ScreenTexture as a third argument, so the value analyzer can see it, even though it's technically not used in the code.
	FValueRef ScreenTexture = Em.Extern<FScreenTextureExtern>({ EScreenTexture::SceneDepthWithoutWater });
	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionSceneDepthWithoutWater($0, $1)")}, TexCoords, Em.ConstantFloat(FallbackDepth), ScreenTexture));
}

void UMaterialExpressionDBufferTexture::Build(MIR::FEmitter& Em)
{
	FValueRef TexCoords = SceneTextureExpressionTexCoords(Em, Em.ConstantInt(0), Coordinates);

	FValueRef ScreenTexture = Em.Extern<FScreenTextureExtern>({ DBufferTextureId.GetValue() });
	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("MaterialExpressionDBufferTextureLookup(Parameters, $0, $1)") }, TexCoords, ScreenTexture));
}

void UMaterialExpressionSphericalParticleOpacity::Build(MIR::FEmitter& Em)
{
	FValueRef DensityValue = Em.InputDefaultFloat(&Density, ConstantDensity);
	UE_MIR_CHECKPOINT(Em); // Early out in case of errors
	BuildMaterialExpressionExternalCodeBase(Em, *this, DensityValue);
}

void UMaterialExpressionShadingModel::Build(MIR::FEmitter& Em)
{
	struct FShadingModel
	{
		EMaterialShadingModel Id;

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name  = TEXTVIEW("ShadingModel"),
				.Type  = MIR::FType::MakeIntScalar(),
				.Flags = MIR::EExternFlags::Inline,
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			check(Id < MSM_NUM);
			Context.Module->AddShadingModel(Id);
		}
	
		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << (uint32)Id;
		}
	};

	EMaterialShadingModel Id = ShadingModel;

	// If the shading model is masked out, fallback to default shading model
	uint32 PlatformShadingModelsMask = GetPlatformShadingModelsMask(Em.GetShaderPlatform());
	if ((PlatformShadingModelsMask & (1u << (uint32)Id)) == 0)
	{
		Id = MSM_DefaultLit;
	}

	Em.Output(0, Em.Extern<FShadingModel>({ Id }));
}

void UMaterialExpressionTextureObject::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.TextureUniform(Texture->GetFName(), Texture, SamplerType));
}

static MIR::ETextureReadMode TextureGatherModeToMIR(::ETextureGatherMode Mode)
{
	switch (Mode)
	{
		case TGM_Red: return MIR::ETextureReadMode::GatherRed;
		case TGM_Green: return MIR::ETextureReadMode::GatherGreen;
		case TGM_Blue: return MIR::ETextureReadMode::GatherBlue;
		case TGM_Alpha: return MIR::ETextureReadMode::GatherAlpha;
		default: UE_MIR_UNREACHABLE();
	}
}

static FValueRef BuildTextureSample(MIR::FEmitter& Em, UMaterialExpressionTextureSample* Expr, FValueRef Texture, EMaterialValueType TextureType, FValueRef TexCoords, bool bAutomaticViewMipBias)
{
	FValueRef TextureRead{};
	if (Expr->GatherMode != TGM_None)
	{
		if (Expr->MipValueMode != TMVM_None)
		{
			Em.Errorf(TEXT("Texture gather does not support mipmap overrides (it implicitly accesses a specific mip)."));
			return TextureRead;
		}

		TextureRead = Em.TextureGather(Texture, TexCoords, TextureGatherModeToMIR(Expr->GatherMode), { Expr->SamplerSource });
	}
	else
	{
		// If not 2D texture, disable AutomaticViewMipBias.
		if (!(TextureType & (MCT_Texture2D | MCT_TextureVirtual | MCT_TextureMeshPaint)))
		{
			bAutomaticViewMipBias = false;
		}

		// Get the mip value level (either through the expression input or using the given constant if disconnected).
		FValueRef MipValue;
		if (Expr->MipValueMode == TMVM_MipLevel || Expr->MipValueMode == TMVM_MipBias)
		{
			MipValue = Em.CheckIsScalar(Em.InputDefaultInt(&Expr->MipValue, Expr->ConstMipValue));
		}

		switch (Expr->MipValueMode)
		{
			case TMVM_None:
				TextureRead = Em.TextureSample(Texture, TexCoords, bAutomaticViewMipBias, { Expr->SamplerSource });
				break;
		
			case TMVM_MipBias:
				TextureRead = Em.TextureSampleBias(Texture, TexCoords, MipValue, bAutomaticViewMipBias, { Expr->SamplerSource });
				break;
		
			case TMVM_MipLevel:
				TextureRead = Em.TextureSampleLevel(Texture, TexCoords, MipValue, bAutomaticViewMipBias, { Expr->SamplerSource });
				break;

			case TMVM_Derivative:
			{
				FValueRef TexCoordsDdx = Em.Cast(Em.Input(&Expr->CoordinatesDX), TexCoords->Type);
				FValueRef TexCoordsDdy = Em.Cast(Em.Input(&Expr->CoordinatesDY), TexCoords->Type);
				TextureRead = Em.TextureSampleGrad(Texture, TexCoords, TexCoordsDdx, TexCoordsDdy, bAutomaticViewMipBias, { Expr->SamplerSource });
				break;
			}
		}
	}

	return TextureRead;
}

static FValueRef BuildTextureValue(MIR::FEmitter& Em, UMaterialExpressionTextureSample* Expr)
{
	FValueRef TextureValue = Em.TryInput(&Expr->TextureObject);
	if (!TextureValue)
	{
		if (!Expr->Texture)
		{
			Em.Error(TEXT("No texture specified for this expression."));
			return Em.Poison();
		}

		TextureValue = Em.TextureUniform(Expr->Texture->GetFName(), Expr->Texture.Get(), Expr->SamplerType);
	}
	return TextureValue;
}

static void BuildTextureSampleExpression(MIR::FEmitter& Em, UMaterialExpressionTextureSample* Expr, FValueRef Texture, EMaterialValueType TextureType)
{
	FValueRef TexCoords = Em.TryInput(&Expr->Coordinates);
	if (!TexCoords)
	{
		TexCoords = MaterialToMIR::EmitTexCoord(Em, Expr->ConstCoordinate);
	}

	// Determine if automatic view mip bias should be used, by trying to acquire its input as a static boolean.
	const bool bAutomaticViewMipBias = Em.ToConstantBool(Em.InputDefaultBool(&Expr->AutomaticViewMipBiasValue, Expr->AutomaticViewMipBias));

	FValueRef TextureRead = BuildTextureSample(Em, Expr, Texture, TextureType, TexCoords, bAutomaticViewMipBias);

	Em.Output(0, Em.Swizzle(TextureRead, MIR::FSwizzleMask::XYZ()));
	Em.Output(1, Em.Subscript(TextureRead, 0));
	Em.Output(2, Em.Subscript(TextureRead, 1));
	Em.Output(3, Em.Subscript(TextureRead, 2));
	Em.Output(4, Em.Subscript(TextureRead, 3));
	Em.Output(5, TextureRead);
}

void UMaterialExpressionTextureSample::Build(MIR::FEmitter& Em)
{
	FValueRef TextureValue = Em.TryInput(&TextureObject);
	if (!TextureValue)
	{
		if (!Texture)
		{
			Em.Error(TEXT("No texture specified for this expression."));
			return;
		}

		TextureValue = Em.TextureUniform(Texture->GetFName(), Texture.Get(), SamplerType);
	}

	UE_MIR_CHECKPOINT(Em);

	UObject* DefaultTexture = TextureValue->GetTextureObject();
	if (!DefaultTexture)
	{
		Em.Error(TEXT("Missing texture object from input"));
		return;
	}

	BuildTextureSampleExpression(Em, this, TextureValue, MIR::Internal::GetTextureMaterialValueType(DefaultTexture));
}

static void EmitParticleSubUV(MIR::FEmitter& Em, UMaterialExpressionTextureSample* Expr, FValueRef TextureValue, bool bBlend, FValueRef DummyDependency)
{
	EMaterialValueType TextureType = MIR::Internal::GetTextureMaterialValueType(TextureValue->GetTextureObject());

	// Although the parent UMaterialExpressionTextureSample class includes an automatic view mip bias flag, it is specifically ignored by ParticleSubUV.
	const bool bAutomaticViewMipBias = false;

	static FName NAME_ParticleSubUVCoords0("ParticleSubUVCoords0");
	static FName NAME_ParticleSubUVCoords1("ParticleSubUVCoords1");
	static FName NAME_ParticleSubUVLerp("ParticleSubUVLerp");

	FValueRef TexCoords0 = DummyDependency
		? Em.Extern<MIR::FExternFromMaterialDecl>(NAME_ParticleSubUVCoords0, DummyDependency)
		: Em.Extern<MIR::FExternFromMaterialDecl>(NAME_ParticleSubUVCoords0);
	
	FValueRef Sample0 = BuildTextureSample(Em, Expr, TextureValue, TextureType, TexCoords0, bAutomaticViewMipBias);

	if (bBlend)
	{
		FValueRef TexCoords1 = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_ParticleSubUVCoords1);
		FValueRef Sample1 = BuildTextureSample(Em, Expr, TextureValue, TextureType, TexCoords1, bAutomaticViewMipBias);

		FValueRef SubImageLerp = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_ParticleSubUVLerp);

		Sample0 = Em.Lerp(Sample0, Sample1, SubImageLerp);
	}

	// Same outputs as UMaterialExpressionTextureSample
	Em.Output(0, Em.Swizzle(Sample0, MIR::FSwizzleMask::XYZ()));
	Em.Output(1, Em.Subscript(Sample0, 0));
	Em.Output(2, Em.Subscript(Sample0, 1));
	Em.Output(3, Em.Subscript(Sample0, 2));
	Em.Output(4, Em.Subscript(Sample0, 3));
	Em.Output(5, Sample0);
}

// Inherits from UMaterialExpressionTextureSample, but uses different particle specific UVs, and optionally supports blending two different texture samples.
void UMaterialExpressionParticleSubUV::Build(MIR::FEmitter& Em)
{
	FValueRef TextureValue = BuildTextureValue(Em, this);
	UE_MIR_CHECKPOINT(Em);

	EmitParticleSubUV(Em, this, TextureValue, bBlend, {});
}

// Similar to above, but texture comes from a parameter, rather than a local or object texture reference.
void UMaterialExpressionTextureSampleParameterSubUV::Build(MIR::FEmitter& Em)
{
	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, this);
	UE_MIR_CHECKPOINT(Em);

	FValueRef ParameterValue = Em.TextureUniform(GetParameterName(), Metadata.Value.Texture, SamplerType);
	UE_MIR_CHECKPOINT(Em);

	// while this expression does provide a TextureCoordinate input pin, it is, and has always been, ignored.  And only
	// supports using UV0.  Further, in order to support non-vertex fetch implementations we need to be sure to register
	// the use of the first texture slot
	FValueRef DummyDependency = MaterialToMIR::EmitTexCoord(Em, 0);

	EmitParticleSubUV(Em, this, ParameterValue, bBlend, DummyDependency);
}

// Inherits from UMaterialExpressionTextureSample, but does extra math on the sample afterwards.  Note that this was an HLSL utility function in the
// original translator, but uses ops here.  The main advantage of the ops version is that it uses the standard texture sampling code path, rather than
// sampling the texture in the utility function, meaning it supports all sampling features (the original would break if using non-standard sampling).
void UMaterialExpressionAntialiasedTextureMask::Build(MIR::FEmitter& Em)
{
	// Check if a texture is assigned and the right type.
	FString ErrorMessage;
	if (!TextureIsValid(Texture, ErrorMessage))
	{
		Em.Errorf(TEXT("%s"), *ErrorMessage);
		return;
	}

	FValueRef TexCoords = Em.TryInput(&Coordinates);
	if (!TexCoords)
	{
		TexCoords = MaterialToMIR::EmitTexCoord(Em, ConstCoordinate);
	}

	FValueRef TextureValue = BuildTextureValue(Em, this);
	UE_MIR_CHECKPOINT(Em);
	FValueRef Sample1 = BuildTextureSample(Em, this, TextureValue, MCT_Texture2D, TexCoords, /*bAutomaticViewMipBias=*/ false);

	FValueRef ThresholdConst = Em.ConstantFloat(Threshold);

	// Logic below ported from the AntialiasedTextureMask HLSL function.
	Sample1 = Em.Subscript(Sample1, FMath::Clamp(Channel, 0, 3));

	FValueRef TexDDLength = Em.Max(Em.Abs(Em.PartialDerivative(Sample1, MIR::EDerivativeAxis::X)), Em.Abs(Em.PartialDerivative(Sample1, MIR::EDerivativeAxis::Y)));
	FValueRef Top = Em.Subtract(Sample1, ThresholdConst);
	Em.Output(0, Em.Add(Em.Divide(Top, TexDDLength), ThresholdConst));
}

static void BuildTextureSampleParameter(MIR::FEmitter& Em, UMaterialExpressionTextureSampleParameter* Expr)
{
	FString ErrorMessage;
	if (!Expr->TextureIsValid(Expr->Texture, ErrorMessage))
	{
		Em.Error(ErrorMessage);
		return;
	}

	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, Expr);
	UE_MIR_CHECKPOINT(Em);

	FValueRef ParameterValue = Em.TextureUniform(Expr->GetParameterName(), Metadata.Value.Texture, Expr->SamplerType);
	UE_MIR_CHECKPOINT(Em);

	BuildTextureSampleExpression(Em, Expr, ParameterValue, Expr->Texture->GetMaterialType());
}

void UMaterialExpressionTextureSampleParameter::Build(MIR::FEmitter& Em)
{
	BuildTextureSampleParameter(Em, this);
}

static void BuildTextureSampleParameterWithCoordinatesInput(MIR::FEmitter& Em, UMaterialExpressionTextureSampleParameter* Expr)
{
	Em.Input(&Expr->Coordinates); // Cubemap, 2DArray, and Volume sampling requires coordinates input specified
	UE_MIR_CHECKPOINT(Em);
	
	BuildTextureSampleParameter(Em, Expr);
}

void UMaterialExpressionTextureSampleParameterCube::Build(MIR::FEmitter& Em)
{
	BuildTextureSampleParameterWithCoordinatesInput(Em, this);
}

void UMaterialExpressionTextureSampleParameter2DArray::Build(MIR::FEmitter& Em)
{
	BuildTextureSampleParameterWithCoordinatesInput(Em, this);
}

void UMaterialExpressionTextureSampleParameterVolume::Build(MIR::FEmitter& Em)
{
	BuildTextureSampleParameterWithCoordinatesInput(Em, this);
}

void UMaterialExpressionTextureObjectParameter::Build(MIR::FEmitter& Em)
{
	FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, this);
	UE_MIR_CHECKPOINT(Em);

	Em.Output(0, Em.TextureUniform(GetParameterName(), Metadata.Value.Texture, SamplerType));
}

void UMaterialExpressionTextureCoordinate::Build(MIR::FEmitter& Em)
{
	if (UnMirrorU || UnMirrorV)
	{
		Em.Error(TEXT("Unmirroring unsupported"));
		return;
	}

	FValueRef OutputValue = MaterialToMIR::EmitTexCoord(Em, CoordinateIndex);
	
	// Multiply the UV input by the UV tiling constants
	OutputValue = Em.Multiply(OutputValue, Em.ConstantFloat2({ UTiling, VTiling }));
	
	Em.Output(0, OutputValue);
}

void UMaterialExpressionTextureProperty::Build(MIR::FEmitter& Em)
{
	MIR::FValueRef TextureValue = Em.Input(&TextureObject);
	UE_MIR_CHECKPOINT(Em);

	const bool bTexelSizeInUVSpace = Property == TMTM_TexelSize;
	const EPreshader2Opcode PreshaderOpcode = bTexelSizeInUVSpace ? EPreshader2Opcode::TexelSize : EPreshader2Opcode::TextureSize;

	UObject* SourceParameterTexture = TextureValue->GetTextureObject();
	if (!SourceParameterTexture)
	{
		Em.Error(TEXT("Missing default texture from source parameter"));
		return;
	}

	const EMaterialValueType TextureType = MIR::Internal::GetTextureMaterialValueType(SourceParameterTexture);
	const EMaterialValueType PropertyType = UE::MaterialTranslatorUtils::GetTexturePropertyValueType(TextureType);

	Em.Output(0, Em.PreshaderParameter(MIR::FType::FromMaterialValueType(PropertyType), PreshaderOpcode, TextureValue));
}

void UMaterialExpressionFontSample::Build(MIR::FEmitter& Em)
{
#if PLATFORM_EXCEPTIONS_DISABLED
	// if we can't throw the error below, attempt to thwart the error by using the default font
	if (!Font)
	{
		UE_LOGF(LogMaterial, Log, "Using default font instead of real font!");
		Font = GEngine->GetMediumFont();
		FontTexturePage = 0;
	}
	else if (!Font->Textures.IsValidIndex(FontTexturePage))
	{
		UE_LOGF(LogMaterial, Log, "Invalid font page %d. Max allowed is %d", FontTexturePage, Font->Textures.Num());
		FontTexturePage = 0;
	}
#endif
	if (!Font)
	{
		Em.Error(TEXT("Missing input Font"));
	}
	else if (Font->FontCacheType == EFontCacheType::Runtime)
	{
		Em.Errorf(TEXT("Font '%s' is runtime cached, but only offline cached fonts can be sampled"), *Font->GetName());
	}
	else if (!Font->Textures.IsValidIndex(FontTexturePage))
	{
		Em.Errorf(TEXT("Invalid font page %d. Max allowed is %d"), FontTexturePage, Font->Textures.Num());
	}
	else
	{
		auto [bSuccess, Texture, ExpectedSamplerType, ErrorOutput] = ValidateAndGetTextureSampler(Em.GetShaderPlatform(), Em.GetTargetPlatform());
		if (!bSuccess)
		{
			Em.Error(*ErrorOutput);
			return;
		}

		FValueRef OutputValue = Em.TextureSample(
			Em.TextureUniform(Texture->GetFName(), Texture, ExpectedSamplerType),
			MaterialToMIR::EmitTexCoord(Em, 0),
			false,
			{ SSM_FromTextureAsset, ExpectedSamplerType }
		);

		Em.OutputsWithComponentMask(OutputValue);
	}
}
void UMaterialExpressionFontSampleParameter::Build(MIR::FEmitter& Em)
{
	if (!ParameterName.IsValid() || ParameterName.IsNone() || !Font || !Font->Textures.IsValidIndex(FontTexturePage))
	{
		UMaterialExpressionFontSample::Build(Em);
	}
	else
	{
		auto [bSuccess, Texture, ExpectedSamplerType, ErrorOutput] = ValidateAndGetTextureSampler(Em.GetShaderPlatform(), Em.GetTargetPlatform());
		if (!bSuccess)
		{
			Em.Error(*ErrorOutput);
			return;
		}

		FMaterialParameterMetadata ParameterMetaData;
		GetParameterValue(ParameterMetaData);

		FValueRef TextureParameter = Em.TextureUniform(ParameterName, Texture, ExpectedSamplerType);
		if (!TextureParameter->Type.IsTexture())
		{
			Em.Error(TEXT("Parameter is not a texture"));
			return;
		}

		FValueRef OutputValue = Em.TextureSample( TextureParameter, MaterialToMIR::EmitTexCoord(Em, 0), false, { SSM_FromTextureAsset, ExpectedSamplerType });

		Em.OutputsWithComponentMask(OutputValue);
	}
}

void UMaterialExpressionCurveAtlasRowParameter::Build(MIR::FEmitter& Em)
{
	// Some error checking. Note that when bUseCustomPrimitiveData is true we don't rely on the Curve at all
	if (!Atlas && !Curve && !bUseCustomPrimitiveData)
	{
		Em.Error(TEXT("The curve and atlas are not currently set."));
	}
	else if (!Atlas)
	{
		Em.Error(TEXT("The atlas is not currently set."));
	}
	else if (!Curve && !bUseCustomPrimitiveData)
	{
		Em.Error(TEXT("The curve is not currently set."));
	}

	UE_MIR_CHECKPOINT(Em);

	FValueRef Slot;
	int32 CurveIndex = 0;

	// Support for using the Custom Primitive Data to fetch an atlas index if that is chosen
	if (bUseCustomPrimitiveData)
	{
		Slot = MaterialToMIR::EmitCustomPrimitiveDataFloat1(Em, PrimitiveDataIndex);
	}
	else if (Curve && Atlas->GetCurveIndex(Curve, CurveIndex))
	{
		// Retrieve the curve index directly from the atlas rather than relying on the scalar parameter defaults
		Slot = Em.NamedPrimitiveUniform(ParameterName, Em.ConstantFloat((float)CurveIndex));
	} 

	UE_MIR_CHECKPOINT(Em);

	// Emit the atlas texture parameter
	FValueRef AtlasTexture = Em.TextureUniform(GetParameterName(), Atlas, SAMPLERTYPE_LinearColor);

	// Query texture size.
	FValueRef AtlasSize   = Em.PreshaderParameter(MIR::FType::MakeFloatVector(2), EPreshader2Opcode::TextureSize, AtlasTexture);
	FValueRef AtlasWidth  = Em.Subscript(AtlasSize, 0);
	FValueRef AtlasHeight = Em.Subscript(AtlasSize, 1);

	UE_MIR_CHECKPOINT(Em);

	FValueRef ArgT = Em.CastToFloat(Em.InputDefaultFloat(&InputTime, 0.0f), 1);

	// ArgT is [t] the parameter on the curve
	//	adjust so that [0,1] in t corresponds to the first to last pixel center
	//		(CurveLinearColorAtlas has no mips, so that's not an issue)
	//	Arg1 = (t*(width-1) + 0.5)/width
	FValueRef Arg1 = Em.Divide(
		Em.Add(
			Em.Multiply(ArgT, Em.Subtract(AtlasWidth, Em.ConstantFloat(1.0f))),
			Em.ConstantFloat(0.5f)
		),
		AtlasWidth
	);

	// Arg2 is the curve index
	//	each row of the atlas texture can be a different curve
	//	Arg2 = (index + 0.5)/height so it is centered vertically on a pixel row
	FValueRef Arg2 = Em.Divide(Em.Add(Slot, Em.ConstantFloat(0.5f)), AtlasHeight);

	// Calculate UVs from height and slot.
	FValueRef UV = Em.Vector2(Arg1, Arg2);

	UE_MIR_CHECKPOINT(Em);

	// Sample the atlas texture.
	FValueRef Sample = Em.TextureSample(AtlasTexture, UV, false, {
		.SamplerSourceMode = SSM_Clamp_WorldGroupSettings,
		.SamplerType       = SAMPLERTYPE_LinearColor,
	});

	Em.Output(0, Em.Swizzle(Sample, MIR::FSwizzleMask::XYZ()));
	Em.Output(1, Em.Subscript(Sample, 0));
	Em.Output(2, Em.Subscript(Sample, 1));
	Em.Output(3, Em.Subscript(Sample, 2));
	Em.Output(4, Em.Subscript(Sample, 3));
	Em.Output(5, Sample);
}

static MIR::FValueRef BuildVirtualTextureWorldToUV(MIR::FEmitter& Em, MIR::FValueRef WorldPositionValue, MIR::FValueRef P0, MIR::FValueRef P1, MIR::FValueRef P2, EPositionOrigin PositionOrigin)
{
	return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("VirtualTextureWorldToUV($0, $1, $2, $3)") }, WorldPositionValue, P0, P1, P2);
}

static MIR::FValueRef EmitFloatArrayToConstant(MIR::FEmitter& Em, TArrayView<float> InConstants)
{
	switch (InConstants.Num())
	{
		case 1: return Em.ConstantFloat(InConstants[0]);
		case 2: return Em.ConstantFloat2(FVector2f{ InConstants[0], InConstants[1] });
		case 3: return Em.ConstantFloat3(FVector3f{ InConstants[0], InConstants[1], InConstants[2] });
		case 4: return Em.ConstantFloat4(FVector4f{ InConstants[0], InConstants[1], InConstants[2], InConstants[3] });
		default: UE_MIR_UNREACHABLE();
	}
}

static FValueRef EmitWorldPosition(MIR::FEmitter& Em, EWorldPositionIncludedOffsets WorldPositionShaderOffset)
{
	struct FWorldPosition
	{
		EWorldPositionIncludedOffsets WorldPositionShaderOffset;

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXTVIEW("WorldPosition"),
				.Type = (WorldPositionShaderOffset == WPT_Default || WorldPositionShaderOffset == WPT_ExcludeAllShaderOffsets)
				            ? MIR::FType::MakeDoubleVector(3)
				            : MIR::FType::MakeFloatVector(3),
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			Context.Module->GetCompilationOutput().SetIsSceneTextureUsed(PPI_SceneDepth);
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			switch (Printer.Differential)
			{
				case MIR::EExternDifferential::None:
					// Various function permutations exist to fetch world position.  "Prev" permutations are only available in the vertex shader, while
					// "MoMaterialOffsets" permutations are only available in the pixel shader, so we need to cobble together the permutation string
					// factoring in those limitations.  Logic adapted from FHLSLMaterialTranslator::WorldPosition.
					// 
					// Format:  Get[Prev][Translated]WorldPosition[_NoMaterialOffsets](Parameters)
					//
					Printer << TEXTVIEW("Get") << MIR::ExternTag_Prev;

					if (WorldPositionShaderOffset == WPT_CameraRelative || WorldPositionShaderOffset == WPT_CameraRelativeNoOffsets)
					{
						Printer << TEXTVIEW("Translated");
					}

					Printer << TEXTVIEW("WorldPosition");

					if (Printer.Stage == MIR::Stage_Pixel && (WorldPositionShaderOffset == WPT_ExcludeAllShaderOffsets || WorldPositionShaderOffset == WPT_CameraRelativeNoOffsets))
					{
						Printer << TEXTVIEW("_NoMaterialOffsets");
					}

					Printer << TEXTVIEW("(Parameters)");
					break;

				default:
					// Camera-relative variants use float vectors others require promotion to LWC.
					if (WorldPositionShaderOffset == WPT_Default || WorldPositionShaderOffset == WPT_ExcludeAllShaderOffsets)
					{
						Printer << TEXTVIEW("WSPromote(Parameters.WorldPosition_") << MIR::ExternTag_DD << TEXT(')');
					}
					else
					{
						Printer << TEXTVIEW("Parameters.WorldPosition_") << MIR::ExternTag_DD;
					}
					break;
			}
		}
	};

	FValueRef WorldPosition = Em.Extern<FWorldPosition>({ WorldPositionShaderOffset });

	// CastToNonLWCIfDisabled
	if (!UE::MaterialTranslatorUtils::IsLWCEnabled())
	{
		WorldPosition = Em.CastToFloatKind(WorldPosition);
	}

	return WorldPosition;
}

static FValueRef EmitLocalPosition(MIR::FEmitter& Em, ELocalPositionOrigin LocalOrigin, EPositionIncludedOffsets LocalShaderOffset)
{
	struct FLocalPosition
	{
		bool bLocalOriginIsInstance;
		bool bIncludeOffsets;

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXTVIEW("LocalPosition"),
				.Type = MIR::FType::MakeFloatVector(3),
				.Flags = MIR::EExternFlags::Inline,
			};
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			switch (Printer.Differential)
			{
				case MIR::EExternDifferential::None:
					// Various function permutations exist to fetch local position.  "Prev" permutations are only available in the vertex shader, while
					// "MoMaterialOffsets" permutations are only available in the pixel shader, so we need to cobble together the permutation string
					// factoring in those limitations.  Logic adapted from FHLSLMaterialTranslator::LocalPosition (and similar to WorldPosition above).
					// 
					// Format:  Get[Prev]Position[Instance|Primitive]Space[_NoMaterialOffsets](Parameters)
					//
					Printer << TEXTVIEW("Get") << MIR::ExternTag_Prev << TEXTVIEW("Position");
					Printer << (bLocalOriginIsInstance ? TEXTVIEW("Instance") : TEXTVIEW("Primitive"));
					Printer << TEXTVIEW("Space");

					if (Printer.Stage == MIR::Stage_Pixel && !bIncludeOffsets)
					{
						Printer << TEXTVIEW("_NoMaterialOffsets");
					}

					Printer << TEXTVIEW("(Parameters)");
					break;

				default:
					Printer << TEXTVIEW("mul(Parameters.WorldPosition_") << MIR::ExternTag_DD;
					// Local position derivatives reuse the world position derivatives, transforming them to local space
					if (bLocalOriginIsInstance)
					{
						Printer << TEXTVIEW(", DFToFloat3x3(GetWorldToInstanceDF(Parameters)))");
					}
					else
					{
						Printer << (Printer.bIsPreviousFrame ? TEXTVIEW(", DFToFloat3x3(GetPrevWorldToLocalDF(Parameters)))") : TEXTVIEW(", DFToFloat3x3(GetWorldToLocalDF(Parameters)))"));
					}
					break;
			}
		}
	};

	// ELocalPositionOrigin::InstancePreSkinning just uses an external code declaration, and doesn't have variations for offsets.
	if (LocalOrigin == ELocalPositionOrigin::InstancePreSkinning)
	{
		return Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("PreSkinnedPosition") });
	}

	// Make sure ELocalPositionOrigin / EPositionIncludedOffsets have the number of entries we expect.
	if ((int)LocalOrigin >= 3)
	{
		Em.Errorf(TEXT("Unsupported LocalPositionOrigin with value '%s'."), *UEnum::GetValueAsString(LocalOrigin));
		return Em.Poison();
	}
	else if ((int)LocalShaderOffset >= 2)
	{
		Em.Errorf(TEXT("Unsupported PositionIncludedOffsets with value '%s'."), *UEnum::GetValueAsString(LocalShaderOffset));
		return Em.Poison();
	}

	return Em.Extern<FLocalPosition>({ LocalOrigin == ELocalPositionOrigin::Instance, LocalShaderOffset == EPositionIncludedOffsets::IncludeOffsets });
}

static FValueRef EmitTransformVector(
	MIR::FEmitter& Em, FValueRef InputValue, EMaterialCommonBasis TransformSourceBasis, EMaterialCommonBasis TransformDestBasis,
	bool bIsPositionTransform, MIR::FValueRef PeriodicWorldTileSizeValue, MIR::FValueRef FirstPersonInterpolationAlphaValue);

static FValueRef BuildVirtualTextureUnpack(MIR::FEmitter& Em, FValueRef SampleCode0, FValueRef SampleCode1, FValueRef SampleCode2, FValueRef P0, EVirtualTextureUnpackType UnpackType)
{
	switch (UnpackType)
	{
		case EVirtualTextureUnpackType::BaseColorYCoCg:		return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackBaseColorYCoCg($0)") }, SampleCode0);
		case EVirtualTextureUnpackType::NormalBC3:			return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackNormalBC3($0)") }, SampleCode1);
		case EVirtualTextureUnpackType::NormalBC5:			return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackNormalBC5($0)") }, SampleCode1);
		case EVirtualTextureUnpackType::NormalBC3BC3:		return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackNormalBC3BC3($0, $1)") }, SampleCode0, SampleCode1);
		case EVirtualTextureUnpackType::NormalBC5BC1:		return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackNormalBC5BC1($0, $1)") }, SampleCode1, SampleCode2);
		case EVirtualTextureUnpackType::HeightR16:			return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("VirtualTextureUnpackHeight($0, $1)") }, SampleCode0, P0);
		case EVirtualTextureUnpackType::DisplacementR16:	return Em.Swizzle(SampleCode0, MIR::EVectorComponent::X);
		case EVirtualTextureUnpackType::NormalBGR565:		return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackNormalBGR565($0)") }, SampleCode1);
		case EVirtualTextureUnpackType::BaseColorSRGB:		return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("VirtualTextureUnpackBaseColorSRGB($0)") }, SampleCode0);
		default:											UE_MIR_UNREACHABLE();
	}
}

void UMaterialExpressionRuntimeVirtualTextureSample::Build(MIR::FEmitter& Em)
{
	if (!UseVirtualTexturing(Em.GetShaderPlatform()))
	{
		Em.Errorf(TEXT("Virtual texturing not supported on platform '%s'"), *ShaderPlatformToPlatformName(Em.GetShaderPlatform()).ToString());
		return;
	}

	// Check validity of current texture
	FString TextureValidityError;
	bool bIsVirtualTextureValid = false;
	if (!ValidateVirtualTextureParameters(TextureValidityError, bIsVirtualTextureValid))
	{
		Em.Error(*TextureValidityError);
		return;
	}
	else if (!TextureValidityError.IsEmpty())
	{
		Em.Error(*TextureValidityError);
	}

	// Compile the texture object references
	const int32 TextureLayerCount = URuntimeVirtualTexture::GetLayerCount(MaterialType);
	check(TextureLayerCount <= RuntimeVirtualTexture::MaxTextureLayers);

	MIR::FValueRef TextureObjects[RuntimeVirtualTexture::MaxTextureLayers];
	for (int32 TextureLayerIndex = 0; TextureLayerIndex < TextureLayerCount; TextureLayerIndex++)
	{
		const int32 PageTableLayerIndex = bSinglePhysicalSpace ? 0 : TextureLayerIndex;

		FName UniformName;
		URuntimeVirtualTexture* TheVirtualTexture;

		if (IsParameter())
		{
			FMaterialParameterMetadata Metadata = GetMaterialExpressionParameterMetadata(Em, this);
			UE_MIR_CHECKPOINT(Em);

			UniformName = GetParameterName();
			TheVirtualTexture = Metadata.Value.RuntimeVirtualTexture;
		}
		else if (VirtualTexture)
		{
			UniformName = VirtualTexture->GetFName();
			TheVirtualTexture = VirtualTexture;
		}
		else
		{
			Em.Error(TEXT("No virtual texture object specified."));
			return;
		}

		TextureObjects[TextureLayerIndex] = Em.VirtualTextureUniform(UniformName, TheVirtualTexture, TextureLayerIndex, PageTableLayerIndex);
	}

	UE_MIR_CHECKPOINT(Em);

	// Compile the runtime texture uniforms
	MIR::FValueRef Uniforms[ERuntimeVirtualTextureShaderUniform_Count];

	for (int32 UniformIndex = 0; UniformIndex < ERuntimeVirtualTextureShaderUniform_Count; ++UniformIndex)
	{
		const UE::Shader::EValueType UniformType = URuntimeVirtualTexture::GetUniformParameterType(UniformIndex);
		Uniforms[UniformIndex] = Em.PreshaderParameter(
			MIR::FType::FromShaderType(UniformType), EPreshader2Opcode::RuntimeVirtualTextureUniform,
			TextureObjects[0], MIR::FPreshaderParameterPayload{ .UniformIndex = UniformIndex });
	}

	// Compile the coordinates
	// We use the texture world space transform by default
	if (Coordinates.GetTracedInput().Expression != nullptr && WorldPosition.GetTracedInput().Expression != nullptr)
	{
		Em.Error(TEXT("Only one of 'Coordinates' and 'WorldPosition' can be used"));
	}

	MIR::FValueRef CoordinateValue = Em.TryInput(&Coordinates);
	if (!CoordinateValue)
	{
		MIR::FValueRef WorldPositionValue;
		if (WorldPosition.GetTracedInput().Expression != nullptr)
		{
			WorldPositionValue = Em.Input(&WorldPosition);
		}
		else
		{
			WorldPositionValue = EmitWorldPosition(Em, UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(WorldPositionOriginType));
			ensure(WorldPositionValue);
		}

		if (WorldPositionValue)
		{
			if (WorldPositionOriginType == EPositionOrigin::Absolute)
			{
				MIR::FValueRef P0 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0];
				MIR::FValueRef P1 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1];
				MIR::FValueRef P2 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2];
				CoordinateValue = BuildVirtualTextureWorldToUV(Em, WorldPositionValue, P0, P1, P2, EPositionOrigin::Absolute);
			}
			else if (WorldPositionOriginType == EPositionOrigin::CameraRelative)
			{
				//TODO: optimize by calculating translated world to VT directly.
				//This requires some more work as the transform is currently fed in through a preshader variable, which is cached.
				MIR::FValueRef AbsWorldPosIndex = EmitTransformVector(Em, WorldPositionValue, EMaterialCommonBasis::MCB_TranslatedWorld, EMaterialCommonBasis::MCB_World, true, {}, {});

				MIR::FValueRef P0 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0];
				MIR::FValueRef P1 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1];
				MIR::FValueRef P2 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2];
				CoordinateValue = BuildVirtualTextureWorldToUV(Em, AbsWorldPosIndex, P0, P1, P2, EPositionOrigin::Absolute);
			}
			else
			{
				checkNoEntry();
			}
		}
	}

	// Compile the mip level for the current mip value mode
	ETextureMipValueMode TextureMipLevelMode = TMVM_None;
	MIR::FValueRef MipValue0Value;
	MIR::FValueRef MipValue1Value;
	const bool bMipValueExpressionValid = MipValue.GetTracedInput().Expression != nullptr;

	if (MipValueMode == RVTMVM_MipLevel)
	{
		TextureMipLevelMode = TMVM_MipLevel;
		MipValue0Value = bMipValueExpressionValid ? Em.Input(&MipValue) : Em.ConstantFloat(0.0f);
	}
	else if (MipValueMode == RVTMVM_MipBias)
	{
		TextureMipLevelMode = TMVM_MipBias;
		MipValue0Value = bMipValueExpressionValid ? Em.Input(&MipValue) : Em.ConstantFloat(0.0f);
	}
	else if (MipValueMode == RVTMVM_DerivativeUV || MipValueMode == RVTMVM_DerivativeWorld)
	{
		if (DDX.GetTracedInput().Expression == nullptr || DDY.GetTracedInput().Expression == nullptr)
		{
			Em.Error(TEXT("Derivative MipValueMode requires connected DDX and DDY pins."));
		}

		TextureMipLevelMode = TMVM_Derivative;
		MIR::FValueRef Ddx = Em.Input(&DDX);
		MIR::FValueRef Ddy = Em.Input(&DDY);

		if (MipValueMode == RVTMVM_DerivativeUV)
		{
			MipValue0Value = Ddx;
			MipValue1Value = Ddy;
		}
		else if (MipValueMode == RVTMVM_DerivativeWorld)
		{
			MIR::FValueRef UDdx = Em.Dot(Ddx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
			MIR::FValueRef VDdx = Em.Dot(Ddx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
			MipValue0Value = Em.Vector2(UDdx, VDdx);

			MIR::FValueRef UDdy = Em.Dot(Ddy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
			MIR::FValueRef VDdy = Em.Dot(Ddy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
			MipValue1Value = Em.Vector2(UDdy, VDdy);
		}
	}
	else if (MipValueMode == RVTMVM_RecalculateDerivatives)
	{
		// Calculate derivatives from world position.
		// This is legacy/hidden, and is better implemented in the material graph using RVTMVM_DerivativeWorld.
		TextureMipLevelMode = TMVM_Derivative;

		MIR::FValueRef WorldPos = EmitWorldPosition(Em, WPT_CameraRelative);
		MIR::FValueRef WorldPositionDdx = Em.AnalyticalPartialDerivative(WorldPos, MIR::EDerivativeAxis::X);
		MIR::FValueRef UDdx = Em.Dot(WorldPositionDdx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
		MIR::FValueRef VDdx = Em.Dot(WorldPositionDdx, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
		MipValue0Value = Em.Vector2(UDdx, VDdx);

		MIR::FValueRef WorldPositionDdy = Em.AnalyticalPartialDerivative(WorldPos, MIR::EDerivativeAxis::Y);
		MIR::FValueRef UDdy = Em.Dot(WorldPositionDdy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1]);
		MIR::FValueRef VDdy = Em.Dot(WorldPositionDdy, Uniforms[ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2]);
		MipValue1Value = Em.Vector2(UDdy, VDdy);
	}

	// We can support disabling feedback for MipLevel mode.
	const bool bForceEnableFeedback = TextureMipLevelMode != TMVM_MipLevel;

	// Compile the texture sample code
	constexpr bool bAutomaticMipViewBias = true;

	const MIR::FTextureSampleAttributes SampleAttributes
	{
		.SamplerSourceMode = GetSamplerSourceMode(),
		.SamplerType = SAMPLERTYPE_VirtualMasks,
		.bEnableFeedback = bEnableFeedback || bForceEnableFeedback,
		.bIsAdaptive = bAdaptive
	};

	MIR::FValueRef SampleCodeValues[RuntimeVirtualTexture::MaxTextureLayers];
	for (int32 TexureLayerIndex = 0; TexureLayerIndex < TextureLayerCount; TexureLayerIndex++)
	{
		switch (TextureMipLevelMode)
		{
			case TMVM_None:
				SampleCodeValues[TexureLayerIndex] = Em.TextureSample(TextureObjects[TexureLayerIndex], CoordinateValue, bAutomaticMipViewBias, SampleAttributes);
				break;

			case TMVM_MipBias:
				SampleCodeValues[TexureLayerIndex] = Em.TextureSampleBias(TextureObjects[TexureLayerIndex], CoordinateValue, MipValue0Value, bAutomaticMipViewBias, SampleAttributes);
				break;

			case TMVM_MipLevel:
				SampleCodeValues[TexureLayerIndex] = Em.TextureSampleLevel(TextureObjects[TexureLayerIndex], CoordinateValue, MipValue0Value, bAutomaticMipViewBias, SampleAttributes);
				break;

			case TMVM_Derivative:
				SampleCodeValues[TexureLayerIndex] = Em.TextureSampleGrad(TextureObjects[TexureLayerIndex], CoordinateValue, MipValue0Value, MipValue1Value, bAutomaticMipViewBias, SampleAttributes);
				break;

			default:
				UE_MIR_UNREACHABLE();
		}
	}

	UE_MIR_CHECKPOINT(Em);

	// Compile unpacking code
	for (int32 OutputIndex = 0; OutputIndex < 8; ++OutputIndex)
	{
		// Calculate the texture layer and sampling/unpacking functions for this output
		// Fallback to a sensible default value if the output isn't valid for the bound texture
		FRuntimeVirtualTextureUnpackProperties UnpackProperties;
		if (!GetRVTUnpackProperties(OutputIndex, bIsVirtualTextureValid, UnpackProperties))
		{
			Em.Errorf(TEXT("Failed to retrieve unpack properties from RuntimeVirtualTexture for output pin %d"), OutputIndex);
			return;
		}

		if (UnpackProperties.ConstantVector.IsEmpty())
		{
			if (UnpackProperties.UnpackType != EVirtualTextureUnpackType::None)
			{
				MIR::FValueRef P0 = Uniforms[ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack];
				Em.Output(OutputIndex,
					BuildVirtualTextureUnpack(Em, SampleCodeValues[0], SampleCodeValues[1], SampleCodeValues[2], P0, UnpackProperties.UnpackType));
			}
			else
			{
				Em.Output(OutputIndex,
					Em.Swizzle(SampleCodeValues[UnpackProperties.UnpackTarget], MIR::FSwizzleMask(
						(UnpackProperties.UnpackMask     ) & 1,
						(UnpackProperties.UnpackMask >> 1) & 1,
						(UnpackProperties.UnpackMask >> 2) & 1,
						(UnpackProperties.UnpackMask >> 3) & 1)));
			}
		}
		else
		{
			Em.Output(OutputIndex, EmitFloatArrayToConstant(Em, UnpackProperties.ConstantVector));
		}
	}
}

void UMaterialExpressionRuntimeVirtualTextureOutput::Build(MIR::FEmitter& Em)
{
	// Get RVT Values
	FValueRef BaseColorValue	= Em.TryInput(&BaseColor);
	FValueRef SpecularValue		= Em.TryInput(&Specular);
	FValueRef RoughnessValue	= Em.TryInput(&Roughness);
	FValueRef NormalValue		= Em.TryInput(&Normal);
	FValueRef WorldHeightValue	= Em.TryInput(&WorldHeight);
	FValueRef OpacityValue		= Em.TryInput(&Opacity);
	FValueRef MaskValue			= Em.TryInput(&Mask);
	FValueRef DisplacementValue	= Em.TryInput(&Displacement);
	FValueRef Mask4Value		= Em.TryInput(&Mask4);

	// Generate output mask (Note opacity does not have a type bit)
	uint8 AttributeMask = 0;
	if (BaseColorValue)    AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::BaseColor;
	if (SpecularValue)     AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::Specular;
	if (RoughnessValue)    AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::Roughness;
	if (NormalValue)       AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::Normal;
	if (WorldHeightValue)  AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::WorldHeight;
	if (MaskValue)         AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::Mask;
	if (DisplacementValue) AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::Displacement;
	if (Mask4Value)        AttributeMask |= 1 << (uint8)ERuntimeVirtualTextureAttributeType::Mask4;

	// Generate output value array to match VirtualTextureMaterial.usf GetVirtualTextureOutputX order
	FValueRef OutputValues[] =
	{
		BaseColorValue		? Em.CastToFloat(BaseColorValue, 3)		: Em.ConstantFloat3({0.f, 0.f, 0.f}),		// 0: BaseColor
		SpecularValue		? Em.CastToFloat(SpecularValue, 1)		: Em.ConstantFloat(0.5f),					// 1: Specular
		RoughnessValue		? Em.CastToFloat(RoughnessValue, 1)		: Em.ConstantFloat(0.5f),					// 2: Roughness
		NormalValue			? Em.CastToFloat(NormalValue, 3)		: Em.ConstantFloat3({0.f, 0.f, 1.f}),		// 3: Normal
		WorldHeightValue	? Em.CastToFloat(WorldHeightValue, 1)	: Em.ConstantFloat(0.f),					// 4: WorldHeight
		OpacityValue		? Em.CastToFloat(OpacityValue, 1)		: Em.ConstantFloat(1.f),					// 5: Opacity
		MaskValue			? Em.CastToFloat(MaskValue, 1)			: Em.ConstantFloat(1.f),					// 6: Mask
		DisplacementValue	? Em.CastToFloat(DisplacementValue, 1)	: Em.ConstantFloat(0.f),					// 7: Displacement
		Mask4Value			? Em.CastToFloat(Mask4Value, 4)			: Em.ConstantFloat4(FLinearColor(0.f, 0.f, 0.f, 0.f)),	// 8: Mask4
	};

	Em.SetCustomOutputs(
		TEXTVIEW("VirtualTextureOutput"),
		OutputValues,
		MIR::EMaterialOutputFrequency::PerPixel,
		[AttributeMask](FMaterialIRModule& Module)
		{
			FMaterialCompilationOutput& Out = Module.GetCompilationOutput();
			Out.bHasRuntimeVirtualTextureOutputNode |= (AttributeMask != 0);
			Out.RuntimeVirtualTextureOutputAttributeMask |= AttributeMask;
		}
	);
}

void UMaterialExpressionTime::Build(MIR::FEmitter& Em)
{
	// When pausing the game is ignored for this time expression, use real-time instead of game-time.
	if (!bOverride_Period)
	{
		Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, bIgnorePause ? TEXT("View.<PREVFRAME>RealTime") : TEXT("View.<PREVFRAME>GameTime") }));
	}
	else if (Period == 0.0f)
	{
		Em.Output(0, Em.ConstantFloat(0.0f));
	}
	else
	{
		// Note: Don't use IR intrinsic for Fmod() here to avoid conversion to fp16 on mobile.
		// We want full 32 bit float precision until the fmod when using a period.
		FValueRef PeriodValue = Em.ConstantFloat(Period);
		Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, bIgnorePause ? TEXT("fmod(View.<PREVFRAME>RealTime, $0)") : TEXT("fmod(View.<PREVFRAME>GameTime, $0)") }, PeriodValue));
	}
}

// Returns true if the specified value is a constant power of two (scalar or vector).
static bool IsConstFloatOfPow2Expression(FValueRef TileScaleIndexValue)
{
	using namespace UE::MaterialTranslatorUtils;
	if (MIR::FConstant* ConstIndex = TileScaleIndexValue->As<MIR::FConstant>())
	{
		return IsFloatPowerOfTwo(ConstIndex->Float);
	}
	else if (MIR::FComposite* Composite = TileScaleIndexValue->As<MIR::FComposite>())
	{
		for (MIR::FValue* Component : Composite->GetComponents())
		{
			MIR::FConstant* ConstComponent = Component->As<MIR::FConstant>();
			if (!ConstComponent || !IsFloatPowerOfTwo(ConstComponent->Float))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

static FValueRef EmitPeriodicWorldPositionOrigin(MIR::FEmitter& Em, FValueRef TileScaleIndexValue)
{
	return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, IsConstFloatOfPow2Expression(TileScaleIndexValue) ? TEXT("GetPeriodicWorldOrigin_Pow2($0)") : TEXT("GetPeriodicWorldOrigin($0)") }, TileScaleIndexValue);
}

static FValueRef EmitFixedExternalCode(MIR::FEmitter& Em, FName InExternalCodeIdentifier)
{
	return Em.Extern<MIR::FExternFromMaterialDecl>(InExternalCodeIdentifier);
}

static FValueRef EmitMatrixCastTo3x3(MIR::FEmitter& Em, FValueRef MatrixValue)
{
	return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3x3, MatrixValue->Type.IsDouble() ? TEXT("DFToFloat3x3($0)") : TEXT("(float3x3)$0") }, MatrixValue);
}

static FValueRef EmitMatrixMultiply(MIR::FEmitter& Em, FValueRef VectorValue, FValueRef MatrixValue, bool bHasWComponent)
{
	return bHasWComponent
		? Em.Swizzle(Em.MatrixMultiply(VectorValue, MatrixValue), MIR::FSwizzleMask::XYZ())  // mul(Float4(V, 1.0), V).xyz
		: Em.MatrixMultiply(VectorValue, EmitMatrixCastTo3x3(Em, MatrixValue)); // mul(V, (Float3x3)M)
}

static FValueRef EmitMultiplyTransposeMatrix(MIR::FEmitter& Em, FValueRef MatrixValue, FValueRef VectorValue, bool bHasWComponent)
{
	// TODO: this should be removed when the Transpose operator is added to the translator.
	return bHasWComponent
		? Em.Swizzle(Em.MatrixMultiply(MatrixValue, VectorValue), MIR::FSwizzleMask::XYZ())  // mul(M, Float4(V, 1.0)).xyz
		: Em.MatrixMultiply(EmitMatrixCastTo3x3(Em, MatrixValue), VectorValue); // mul((Float3x3)M, V)
}

static FValueRef EmitMultiplyTranslatedMatrix(MIR::FEmitter& Em, FValueRef VectorValue, FValueRef MatrixPreTranslation, bool bHasWComponent)
{
	if (bHasWComponent)
	{
		// mul(Float4(V, 1.0), DFFastToTranslatedWorld(M, ResolvedView.PreViewTranslation))
		MatrixPreTranslation = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4x4, TEXT("DFFastToTranslatedWorld($0, ResolvedView.<PREV>PreViewTranslation)") }, MatrixPreTranslation);
		return Em.MatrixMultiply(Em.Vector4(VectorValue, Em.ConstantOne(MIR::EScalarKind::Float)), MatrixPreTranslation);
	}
	else
	{
		// mul(V, DFToFloat3x3(M))
		return Em.MatrixMultiply(VectorValue, EmitMatrixCastTo3x3(Em, MatrixPreTranslation));
	}
}

static FValueRef EmitMultiplyLWCMatrix(MIR::FEmitter& Em, FValueRef VectorValue, FValueRef MatrixValue, bool bHasWComponent, bool bDemote)
{
	static const MIR::FExternInlineDeclaration DeclA{ .Type = MIR::FType::MakeFloatVector(3), .CodeHLSL = TEXT("WSMultiplyDemote($0, $1)") };
	static const MIR::FExternInlineDeclaration DeclB{ .Type = MIR::FType::MakeDoubleVector(3), .CodeHLSL = TEXT("WSMultiply($0, $1)") };
	static const MIR::FExternInlineDeclaration DeclC{ .Type = MIR::FType::MakeFloatVector(3), .CodeHLSL = TEXT("WSMultiplyVector($0, $1)") };

	const MIR::FExternInlineDeclaration& Decl = bHasWComponent
		? ( bDemote ? DeclA : DeclB )
		: DeclC;

	return Em.Extern<MIR::FExternFromInlineDecl>(&Decl, VectorValue, MatrixValue);
}

static FValueRef EmitTransformVector(MIR::FEmitter& Em, FValueRef InputValue, EMaterialCommonBasis TransformSourceBasis, EMaterialCommonBasis TransformDestBasis,
	bool bIsPositionTransform, MIR::FValueRef PeriodicWorldTileSizeValue, MIR::FValueRef FirstPersonInterpolationAlphaValue)
{
	// Construct float3(0,0,x) out of the input if it is a scalar
	// This way artists can plug in a scalar and it will be treated as height, or a vector displacement
	if (TransformSourceBasis == MCB_Tangent && InputValue->Type.IsScalar())
	{
		FValueRef Zero = Em.ConstantZero(MIR::EScalarKind::Float);
		InputValue = Em.Vector3(Zero, Zero, InputValue);
	}
	
	// Cast the input value to a float3 then make sure it's valid.
	InputValue = Em.CastToVector(InputValue, 3);
	if (!InputValue.IsValid())
	{
		return InputValue.ToPoison();
	}

	MIR::FType ResultType = (TransformDestBasis == MCB_World && bIsPositionTransform) ? MIR::FType::MakeDoubleVector(3) : MIR::FType::MakeFloatVector(3);

	EMaterialCommonBasis IntermediaryBasis = MCB_World;

	switch (TransformSourceBasis)
	{
		case MCB_Tangent:
			check(!bIsPositionTransform);
			if (TransformDestBasis == MCB_World)
			{
				return Em.MatrixMultiply(InputValue, Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("TangentToWorld") }));
			}
			// else use MCB_World as intermediary basis
			break;

		case MCB_Local:
			switch (TransformDestBasis)
			{
				case MCB_World:
				{
					return Em.Extern<MIR::FExternSimpleHLSL>(
						{
							MIR::EExternSimpleType::Float3,
							bIsPositionTransform ? TEXT("TransformLocalPositionTo<PREV>World(Parameters, $0)") : TEXT("TransformLocalVectorTo<PREV>World(Parameters, $0)")
						}, InputValue);
				}

				case MCB_TranslatedWorld:
					if (bIsPositionTransform)
					{
						return EmitMultiplyTranslatedMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetLocalToWorldDF")), bIsPositionTransform);
					}
					break;

				case MCB_PeriodicWorld:
				case MCB_FirstPerson:
					IntermediaryBasis = MCB_TranslatedWorld;
					break;

				default:
					// else use MCB_World as intermediary basis
					break;
			}
			break;

		case MCB_TranslatedWorld:
			switch (TransformDestBasis)
			{
				case MCB_World:
					return bIsPositionTransform
						? Em.Subscript(InputValue, EmitFixedExternalCode(Em, TEXT("GetPreViewTranslation")))
						: InputValue;

				case MCB_Camera:
					return EmitMatrixMultiply(Em, InputValue, EmitFixedExternalCode(Em, TEXT("TranslatedWorldToCameraView")), bIsPositionTransform);

				case MCB_View:
					return EmitMatrixMultiply(Em, InputValue, EmitFixedExternalCode(Em, TEXT("TranslatedWorldToView")), bIsPositionTransform);

				case MCB_Tangent:
					return EmitMultiplyTransposeMatrix(Em, EmitFixedExternalCode(Em, TEXT("TangentToWorld")), InputValue, bIsPositionTransform);

				case MCB_Local:
					return EmitMultiplyTranslatedMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetWorldToLocalDF")), bIsPositionTransform);

				case MCB_MeshParticle:
					return EmitMultiplyTranslatedMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("WorldToParticle")), bIsPositionTransform);

				case MCB_Instance:
					return EmitMultiplyTranslatedMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetWorldToInstanceDF")), bIsPositionTransform);

				case MCB_PeriodicWorld:
					if (!PeriodicWorldTileSizeValue.IsValid())
					{
						Em.Error(TEXT("Missing periodic world tile size"));
						return Em.Poison();
					}
					return Em.Subtract(InputValue, EmitPeriodicWorldPositionOrigin(Em, PeriodicWorldTileSizeValue));

				case MCB_FirstPerson:
				{
					if (!FirstPersonInterpolationAlphaValue.IsValid())
					{
						Em.Error(TEXT("Missing first person interpolation alpha"));
						return Em.Poison();
					}
					// The first person transform is actually a 3x3 matrix and can therefore be used for derivatives as well.
					MIR::FValueRef LerpAlphaClampedIndexValue = Em.Saturate(Em.CastToFloat(FirstPersonInterpolationAlphaValue, 1));
					return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("TransformTo<PREVIOUS>FirstPerson($0, $1)") }, InputValue, LerpAlphaClampedIndexValue);
				}

				default:
					break; // else use MCB_World as intermediary basis
			}
			break;

		case MCB_World:
			switch (TransformDestBasis)
			{
				case MCB_Tangent:
					return EmitMultiplyTransposeMatrix(Em, EmitFixedExternalCode(Em, TEXT("TangentToWorld")), InputValue, bIsPositionTransform);

				case MCB_Local:
					return EmitMultiplyLWCMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetWorldToLocal")), bIsPositionTransform, true);

				case MCB_TranslatedWorld:
					return bIsPositionTransform
						? Em.Add(InputValue, EmitFixedExternalCode(Em, TEXT("GetPreViewTranslation")))
						: InputValue;

				case MCB_MeshParticle:
					return EmitMultiplyLWCMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetWorldToParticle")), bIsPositionTransform, true);

				case MCB_Instance:
					return EmitMultiplyLWCMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetWorldToInstance")), bIsPositionTransform, true);

				default:
					// else use MCB_TranslatedWorld as intermediary basis
					IntermediaryBasis = MCB_TranslatedWorld;
					break;
			}
			break;

		case MCB_Camera:
			if (TransformDestBasis == MCB_TranslatedWorld)
			{
				return EmitMatrixMultiply(Em, InputValue, EmitFixedExternalCode(Em, TEXT("CameraViewToTranslatedWorld")), bIsPositionTransform);
			}
			
			// else use MCB_TranslatedWorld as intermediary basis
			IntermediaryBasis = MCB_TranslatedWorld;
			break;

		case MCB_View:
			if (TransformDestBasis == MCB_TranslatedWorld)
			{
				return EmitMatrixMultiply(Em, InputValue, EmitFixedExternalCode(Em, TEXT("ViewToTranslatedWorld")), bIsPositionTransform);
			}

			// else use MCB_TranslatedWorld as intermediary basis
			IntermediaryBasis = MCB_TranslatedWorld;
			break;

		case MCB_MeshParticle:
			switch (TransformDestBasis)
			{
				case MCB_World:
					return EmitMultiplyLWCMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetParticleToWorld")), bIsPositionTransform, false);

				case MCB_TranslatedWorld:
					return EmitMultiplyTranslatedMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("ParticleToWorld")), bIsPositionTransform);

				case MCB_PeriodicWorld:
				case MCB_FirstPerson:
					IntermediaryBasis = MCB_TranslatedWorld;
					break;

				default:
					break; // use World as an intermediary base
			}
			break;

		case MCB_Instance:
			switch(TransformDestBasis)
			{
				case MCB_World:
					return EmitMultiplyLWCMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetInstanceToWorld")), bIsPositionTransform, false);

				case MCB_TranslatedWorld:
					return EmitMultiplyTranslatedMatrix(Em, InputValue, EmitFixedExternalCode(Em, TEXT("GetInstanceToWorldDF")), bIsPositionTransform);

				case MCB_PeriodicWorld:
				case MCB_FirstPerson:
					IntermediaryBasis = MCB_TranslatedWorld;
					break;

				default:
					break; // use World as an intermediary base
			}
			break;

		case MCB_PeriodicWorld:
			switch (TransformDestBasis)
			{
				case MCB_TranslatedWorld:
					if (!PeriodicWorldTileSizeValue.IsValid())
					{
						Em.Error(TEXT("Missing periodic world tile size"));
						return Em.Poison();
					}
					return Em.Add(InputValue, EmitPeriodicWorldPositionOrigin(Em, PeriodicWorldTileSizeValue));

				default:
					// else use MCB_TranslatedWorld as intermediary basis
					IntermediaryBasis = MCB_TranslatedWorld;
					break;
			}
			break;

		case MCB_FirstPerson:
			UE_MIR_UNREACHABLE(); // MCB_FirstPerson is not supported as a source basis. This should've been caught earlier in validation.

		default:
			UE_MIR_UNREACHABLE();
	}

	// Check intermediary basis so we don't have infinite recursion
	check(IntermediaryBasis != TransformSourceBasis);
	check(IntermediaryBasis != TransformDestBasis);

	// Use intermediary basis
	FValueRef IntermediaryBasisA = EmitTransformVector(Em, InputValue, TransformSourceBasis, IntermediaryBasis, bIsPositionTransform, PeriodicWorldTileSizeValue, FirstPersonInterpolationAlphaValue);
	FValueRef IntermediaryBasisB = EmitTransformVector(Em, IntermediaryBasisA, IntermediaryBasis, TransformDestBasis, bIsPositionTransform, PeriodicWorldTileSizeValue, FirstPersonInterpolationAlphaValue);

	return IntermediaryBasisB;
}

static void BuildTransformVector(
	MIR::FEmitter& Em, const FExpressionInput* Input, EMaterialCommonBasis TransformSourceBasis, EMaterialCommonBasis TransformDestBasis,
	bool bIsPositionTransform, MIR::FValueRef PeriodicWorldTileSizeValue, MIR::FValueRef FirstPersonInterpolationAlphaValue)
{
	FValueRef InputValue = Em.CheckIsPrimitive(Em.Input(Input));
	UE_MIR_CHECKPOINT(Em);

	FValueRef OutputValue = EmitTransformVector(Em, InputValue, TransformSourceBasis, TransformDestBasis, bIsPositionTransform, PeriodicWorldTileSizeValue, FirstPersonInterpolationAlphaValue);

	if (TransformSourceBasis == MCB_World && bIsPositionTransform)
	{
		if (!OutputValue->Type.IsDouble())
		{
			OutputValue = Em.CastToScalarKind(OutputValue, MIR::EScalarKind::Double);
		}
	}
	else if (OutputValue->Type.IsDouble())
	{
		OutputValue = Em.CastToFloatKind(OutputValue);
	}

	Em.Output(0, OutputValue);
}

void UMaterialExpressionTransform::Build(MIR::FEmitter& Em)
{
	const EMaterialCommonBasis TransformSourceBasis = UE::MaterialTranslatorUtils::GetMaterialCommonBasis(TransformSourceType);
	const EMaterialCommonBasis TransformDestBasis = UE::MaterialTranslatorUtils::GetMaterialCommonBasis(TransformType);

	constexpr bool bIsPositionTransform = false;
	BuildTransformVector(Em, &Input, TransformSourceBasis, TransformDestBasis, bIsPositionTransform, {}, {});
}

void UMaterialExpressionTransformPosition::Build(MIR::FEmitter& Em)
{
	MIR::FValueRef PeriodicWorldTileSizeValue, FirstPersonInterpolationAlphaValue;
	if (TransformSourceType == TRANSFORMPOSSOURCE_PeriodicWorld || TransformType == TRANSFORMPOSSOURCE_PeriodicWorld)
	{
		PeriodicWorldTileSizeValue = Em.InputDefaultFloat(&PeriodicWorldTileSize, ConstPeriodicWorldTileSize);
	}
	if (TransformSourceType == TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld || TransformType == TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld)
	{
		FirstPersonInterpolationAlphaValue = Em.InputDefaultFloat(&FirstPersonInterpolationAlpha, ConstFirstPersonInterpolationAlpha);
	}

	const EMaterialCommonBasis TransformSourceBasis = UE::MaterialTranslatorUtils::GetMaterialCommonBasis(TransformSourceType);
	const EMaterialCommonBasis TransformDestBasis = UE::MaterialTranslatorUtils::GetMaterialCommonBasis(TransformType);

	constexpr bool bIsPositionTransform = true;
	BuildTransformVector(Em, &Input, TransformSourceBasis, TransformDestBasis, bIsPositionTransform, PeriodicWorldTileSizeValue, FirstPersonInterpolationAlphaValue);
}

void UMaterialExpressionReroute::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.TryInput(&Input));
}

void UMaterialExpressionNamedRerouteDeclaration::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.TryInput(&Input));
}

void UMaterialExpressionNamedRerouteUsage::Build(MIR::FEmitter& Em)
{
	if (!IsDeclarationValid())
	{
		Em.Error(TEXT("Named reroute expression does not have a valid declaration."));
		return;
	}
	Em.Output(0, Em.TryInput(&Declaration->Input));
}

void UMaterialExpressionClamp::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.Input(&Input);
	FValueRef MinValue = Em.InputDefaultFloat(&Min, MinDefault);
	FValueRef MaxValue = Em.InputDefaultFloat(&Max, MaxDefault);

	FValueRef OutputValue = nullptr;
	if (ClampMode == CMODE_Clamp)
	{
		OutputValue = Em.Clamp(InputValue, MinValue, MaxValue);
	}
	else if (ClampMode == CMODE_ClampMin)
	{
		OutputValue = Em.Max(InputValue, MinValue);
	}
	else if (ClampMode == CMODE_ClampMax)
	{
		OutputValue = Em.Min(InputValue, MaxValue);
	}

	Em.Output(0, OutputValue);
}


void BuildTernaryArithmeticOperator(MIR::FEmitter& Em, MIR::EOperator Op, FExpressionInput* A, float ConstA, FExpressionInput* B, float ConstB, FExpressionInput* C, float ConstC)
{
	FValueRef ValueA = Em.InputDefaultFloat(A, ConstA);
	FValueRef ValueB = Em.InputDefaultFloat(B, ConstB);
	FValueRef ValueC = Em.InputDefaultFloat(C, ConstC);
	Em.Output(0, Em.Operator(Op, ValueA, ValueB, ValueC));
}

void UMaterialExpressionColorRamp::Build(MIR::FEmitter& Em)
{	
	// Check that the ColorCurve is set
	if (!ColorCurve)
	{
		Em.Errorf(TEXT("Missing ColorCurve"));
		return;
	}

	FValueRef InputValue = Em.CastToFloat(Em.InputDefaultFloat(&Input, ConstInput), 1);

	// If the input is constant, evaluate at compile time.
	if (const MIR::FConstant* Constant = MIR::As<MIR::FConstant>(InputValue))
	{
		FLinearColor ColorValue = ColorCurve->GetLinearColorValue(Constant->Float);
		Em.Output(0, Em.ConstantFloat4(ColorValue));
		return;
	}

	// Helper lambda to evaluate a curve
	auto EvaluateCurve = [&Em, InputValue](const FRichCurve& Curve) -> FValueRef
		{
			const int32 NumKeys = Curve.Keys.Num();

			switch (NumKeys)
			{
				case 0:
					return Em.ConstantFloat(0.0f);

				case 1:
					return Em.ConstantFloat(Curve.Keys[0].Value);

				case 2:
				{
					float StartTime = Curve.Keys[0].Time;
					float EndTime = Curve.Keys[1].Time;
					float StartValue = Curve.Keys[0].Value;
					float EndValue = Curve.Keys[1].Value;

					FValueRef TimeDelta = Em.ConstantFloat(EndTime - StartTime);
					FValueRef TimeDiff = Em.Subtract(InputValue, Em.ConstantFloat(StartTime));
					FValueRef Fraction = Em.Divide(TimeDiff, TimeDelta);

					return Em.Lerp(Em.ConstantFloat(StartValue), Em.ConstantFloat(EndValue), Fraction);
				}
			}

			FValueRef InValueVec = Em.Vector4(InputValue, InputValue, InputValue, InputValue);

			FValueRef Result = Em.ConstantFloat(Curve.Keys[0].Value);
			int32 i = 0;

			// Use vector operations for segments of 4
			for (; i < NumKeys - 4; i += 4)
			{
				FVector4f StartTimeVector(
					Curve.Keys[i].Time,
					Curve.Keys[i + 1].Time,
					Curve.Keys[i + 2].Time,
					Curve.Keys[i + 3].Time
				);
				FValueRef StartTimeVec = Em.ConstantFloat4(StartTimeVector);

				FVector4f EndTimeVector(
					Curve.Keys[i + 1].Time,
					Curve.Keys[i + 2].Time,
					Curve.Keys[i + 3].Time,
					Curve.Keys[i + 4].Time
				);
				FValueRef EndTimeVec = Em.ConstantFloat4(EndTimeVector);

				FVector4f StartValueVector(
					Curve.Keys[i].Value,
					Curve.Keys[i + 1].Value,
					Curve.Keys[i + 2].Value,
					Curve.Keys[i + 3].Value
				);
				FValueRef StartValueVec = Em.ConstantFloat4(StartValueVector);

				FVector4f EndValueVector(
					Curve.Keys[i + 1].Value,
					Curve.Keys[i + 2].Value,
					Curve.Keys[i + 3].Value,
					Curve.Keys[i + 4].Value
				);
				FValueRef EndValueVec = Em.ConstantFloat4(EndValueVector);

				FValueRef TimeDeltaVec = Em.Subtract(EndTimeVec, StartTimeVec);
				FValueRef ValueDeltaVec = Em.Subtract(EndValueVec, StartValueVec);

				FValueRef TimeDiffVec = Em.Subtract(InValueVec, StartTimeVec);
				FValueRef FractionVec = Em.Divide(TimeDiffVec, TimeDeltaVec);
				FValueRef SatFractionVec = Em.Saturate(FractionVec);
				FValueRef ContributionVec = Em.Multiply(ValueDeltaVec, SatFractionVec);

				FVector4f Ones(1.0f, 1.0f, 1.0f, 1.0f);
				FValueRef OnesVec = Em.ConstantFloat4(Ones);
				FValueRef ContributionSum = Em.Dot(ContributionVec, OnesVec);

				Result = Em.Add(Result, ContributionSum);
			}
			
			// Use scalar operations for the remaining keys
			for (; i < NumKeys - 1; i++)
			{
				float StartTime = Curve.Keys[i].Time;
				float EndTime = Curve.Keys[i + 1].Time;
				float StartValue = Curve.Keys[i].Value;
				float EndValue = Curve.Keys[i + 1].Value;

				FValueRef TimeDelta = Em.ConstantFloat(EndTime - StartTime);
				FValueRef ValueDelta = Em.ConstantFloat(EndValue - StartValue);
				FValueRef TimeDiff = Em.Subtract(InputValue, Em.ConstantFloat(StartTime));
				FValueRef Fraction = Em.Divide(TimeDiff, TimeDelta);
				FValueRef SatFraction = Em.Saturate(Fraction);
				FValueRef Contribution = Em.Multiply(ValueDelta, SatFraction);
				Result = Em.Add(Result, Contribution);
			}
			return Result;
		};

	FValueRef Red = EvaluateCurve(ColorCurve->FloatCurves[0]);
	FValueRef Green = EvaluateCurve(ColorCurve->FloatCurves[1]);
	FValueRef Blue = EvaluateCurve(ColorCurve->FloatCurves[2]);
	FValueRef Alpha = EvaluateCurve(ColorCurve->FloatCurves[3]);

	FValueRef FinalVector = Em.Vector4(Red, Green, Blue, Alpha);
	Em.Output(0, FinalVector);
}

void UMaterialExpressionInverseLinearInterpolate::Build(MIR::FEmitter& Em)
{
	FValueRef ValueA = Em.InputDefaultFloat(&A, ConstA);
	FValueRef ValueB = Em.InputDefaultFloat(&B, ConstB);
	FValueRef ValueC = Em.InputDefaultFloat(&Value, ConstValue);
	FValueRef Result = Em.Divide(Em.CastToFloatKind(Em.Subtract(ValueC, ValueA)), Em.CastToFloatKind(Em.Subtract(ValueB, ValueA)));
	if (bClampResult)
	{
		Result = Em.Saturate(Result);
	}
	Em.Output(0, Result);
}

void UMaterialExpressionLinearInterpolate::Build(MIR::FEmitter& Em)
{
	BuildTernaryArithmeticOperator(Em, MIR::TO_Lerp, &A, ConstA, &B, ConstB, &Alpha, ConstAlpha);
}

void UMaterialExpressionSmoothStep::Build(MIR::FEmitter& Em)
{
	BuildTernaryArithmeticOperator(Em, MIR::TO_Smoothstep, &Min, ConstMin, &Max, ConstMax, &Value, ConstValue);
}

void UMaterialExpressionConvert::Build(MIR::FEmitter& Em)
{
	TArray<FValueRef, TInlineAllocator<8>> InputValues;
	InputValues.Init(nullptr, ConvertInputs.Num());

	for (int32 OutputIndex = 0; OutputIndex < ConvertOutputs.Num(); ++OutputIndex)
	{
		const FMaterialExpressionConvertOutput& ConvertOutput = ConvertOutputs[OutputIndex];
		FValueRef OutComponents[4] = { nullptr, nullptr, nullptr, nullptr };

		for (const FMaterialExpressionConvertMapping& Mapping : ConvertMappings)
		{
			// We only care about mappings relevant to this output
			if (Mapping.OutputIndex != OutputIndex)
			{
				continue;
			}

			const int32 OutputComponentIndex = Mapping.OutputComponentIndex;
			if (!IsValidComponentIndex(OutputComponentIndex, ConvertOutput.Type))
			{
				Em.Errorf(TEXT("Convert mapping's output component `%d` is invalid."), OutputComponentIndex);
				continue;
			}

			const int32 InputIndex = Mapping.InputIndex;
			if (!ConvertInputs.IsValidIndex(InputIndex))
			{
				Em.Errorf(TEXT("Convert mapping's input `%d` is invalid."), InputIndex);
				continue;
			}

			FMaterialExpressionConvertInput& ConvertInput = ConvertInputs[InputIndex];
			const int32 InputComponentIndex = Mapping.InputComponentIndex;
			if (!IsValidComponentIndex(InputComponentIndex, ConvertInput.Type))
			{
				Em.Errorf(TEXT("Convert mapping's input component `%d` is invalid."), InputComponentIndex);
				continue;
			}

			// If not already emitted, read the input value, cast it to the specified input
			// type and cache it into an array, as each input could be used multiple times
			// by output values.
			if (!InputValues[InputIndex])
			{
				// Read the input's value (or read float zero if disconnected).
				InputValues[InputIndex] = Em.InputDefaultFloat4(&ConvertInput.ExpressionInput, ConvertInput.DefaultValue);

				// Expect type to be primitive.
				TOptional<MIR::FPrimitive> InputPrimitiveType = InputValues[InputIndex]->Type.AsPrimitive();
				if (!InputPrimitiveType)
				{
					Em.Errorf(TEXT("Input `%d` of type `%s` is not primitive."), InputComponentIndex, *InputValues[InputIndex]->Type.GetSpelling());
					continue;
				}

				// Determine the target type.
				MIR::FType InputType = MIR::FType::MakeVector(InputPrimitiveType->ScalarKind, MaterialExpressionConvertType::GetComponentCount(ConvertInput.Type));

				// Cast the input value to the target type.
				InputValues[InputIndex] = Em.Cast(InputValues[InputIndex], InputType);
			}

			// Subscript the input value to the specified component index.
			OutComponents[OutputComponentIndex] = Em.Subscript(InputValues[InputIndex], InputComponentIndex);
		}

		const int32 OutputNumComponents = MaterialExpressionConvertType::GetComponentCount(ConvertOutput.Type);

		// For any component still unset, give assign it to the default value.
		for (int32 OutputComponentIndex = 0; OutputComponentIndex < OutputNumComponents; ++OutputComponentIndex)
		{
			// If we don't have a compile result here, default it to a that component's default value
			if (!OutComponents[OutputComponentIndex])
			{
				OutComponents[OutputComponentIndex] = Em.ConstantFloat(ConvertOutput.DefaultValue.Component(OutputComponentIndex));
			}
		}

		// Finally create the output dimensional value by combining the output components.
		FValueRef OutValue;
		switch (OutputNumComponents)
		{
			case 1: OutValue = OutComponents[0]; break;
			case 2: OutValue = Em.Vector2(OutComponents[0], OutComponents[1]); break;
			case 3: OutValue = Em.Vector3(OutComponents[0], OutComponents[1], OutComponents[2]); break;
			case 4: OutValue = Em.Vector4(OutComponents[0], OutComponents[1], OutComponents[2], OutComponents[3]); break;
			
			default:
				OutValue = Em.Poison();
				Em.Errorf(TEXT("Convert node has an invalid component count of %d"), OutputNumComponents);
				break;
		}

		Em.Output(OutputIndex, OutValue);
	}
}

static FValueRef BuildViewProperty(MIR::FEmitter& Em, EMaterialExposedViewProperty InProperty, bool bInvProperty = false)
{
	check(InProperty < MEVP_MAX);

	struct FViewPropertyExtern
	{
		const FMaterialExposedViewPropertyMeta* PropertyMeta;
		bool bInvProperty;

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXT("ViewProperty"),
				.Type = MIR::FType::FromMaterialValueType(PropertyMeta->Type),
				.Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials,
			};
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << (bInvProperty ? PropertyMeta->InvPropertyCode : PropertyMeta->PropertyCode);
		}

		void CopyTo(FViewPropertyExtern& Other) const
		{
			Other.PropertyMeta = PropertyMeta;
			Other.bInvProperty = bInvProperty;
		}
	};

	const FMaterialExposedViewPropertyMeta& PropertyMeta = MaterialExternalCodeRegistry::Get().GetExternalViewPropertyCode(InProperty);
	const bool bHasCustomInverseCode = PropertyMeta.InvPropertyCode != nullptr;

	FValueRef Result = Em.Extern<FViewPropertyExtern>({ &PropertyMeta, bInvProperty && bHasCustomInverseCode });

	// CastToNonLWCIfDisabled
	TOptional<MIR::FPrimitive> PrimitiveType = MIR::FType::FromMaterialValueType(PropertyMeta.Type).AsPrimitive();
	if (PrimitiveType && PrimitiveType->IsDouble() && !UE::MaterialTranslatorUtils::IsLWCEnabled())
	{
		Result = Em.CastToFloatKind(Result);
	}

	// Fall back to compute the property's inverse from PropertyCode, if no custom inverse
	if (bInvProperty && !bHasCustomInverseCode)
	{
		Result = Em.Divide(Em.ConstantFloat(1.0f), Result);
	}

	return Result;
}

void UMaterialExpressionViewProperty::Build(MIR::FEmitter& Em)
{
	for (int32 OutputIndex = 0; OutputIndex < 2; ++OutputIndex)
	{
		const bool bInvProperty = OutputIndex == 1;
		Em.Output(OutputIndex, BuildViewProperty(Em, Property, bInvProperty));
	}
}

void UMaterialExpressionViewSize::Build(MIR::FEmitter& Em)
{
	Em.Output(0, BuildViewProperty(Em, MEVP_ViewSize));
}

void UMaterialExpressionSceneTexelSize::Build(MIR::FEmitter& Em)
{
	// To make sure any material that were correctly handling BufferUV != ViewportUV, we just lie to material
	// to make it believe ViewSize == BufferSize, so they are still compatible with SceneTextureLookup().
	Em.Output(0, BuildViewProperty(Em, MEVP_ViewSize, true));
}

void UMaterialExpressionCameraPositionWS::Build(MIR::FEmitter& Em)
{
	Em.Output(0, BuildViewProperty(Em, MEVP_WorldSpaceCameraPosition));
}

void UMaterialExpressionPixelNormalWS::Build(MIR::FEmitter& Em)
{
	Em.Output(0, MaterialToMIR::EmitPixelNormalWS(Em));
}

void UMaterialExpressionDDX::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.PartialDerivative(Em.Input(&Value), MIR::EDerivativeAxis::X));
}

void UMaterialExpressionDDY::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.PartialDerivative(Em.Input(&Value), MIR::EDerivativeAxis::Y));
}

static constexpr MIR::EOperator MaterialExpressionOperatorToMIR(EMaterialExpressionOperatorKind Operator)
{
	return static_cast<MIR::EOperator>(static_cast<uint32>(Operator) + 1);
}

// Checks to make sure the two enums are aligned.
static_assert(MaterialExpressionOperatorToMIR(EMaterialExpressionOperatorKind::BitwiseNot) == MIR::UO_BitwiseNot);
static_assert(MaterialExpressionOperatorToMIR(EMaterialExpressionOperatorKind::Sign) == MIR::UO_Sign);
static_assert(MaterialExpressionOperatorToMIR(EMaterialExpressionOperatorKind::BitwiseAnd) == MIR::BO_BitwiseAnd);
static_assert(MaterialExpressionOperatorToMIR(EMaterialExpressionOperatorKind::Smoothstep) == MIR::TO_Smoothstep);

uint32 GetMaterialExpressionOperatorArity(EMaterialExpressionOperatorKind Operator)
{
	return MIR::GetOperatorArity(MaterialExpressionOperatorToMIR(Operator));
}

void UMaterialExpressionOperator::Build(MIR::FEmitter& Em)
{
	FValueRef AValue = Em.InputDefaultFloat(&DynamicInputs[0].ExpressionInput, DynamicInputs[0].ConstValue);
	if (bAllowAddPin)
	{
		MIR::EOperator OpMIR = MaterialExpressionOperatorToMIR(Operator);

		// Apply operation to iteratively to all input values
		FValueRef Value = AValue;
		for (int32 i = 1; i < DynamicInputs.Num(); i++)
		{
			FValueRef CurValue = Em.InputDefaultFloat(&DynamicInputs[i].ExpressionInput, DynamicInputs[i].ConstValue);
			Value = Em.Operator(OpMIR, Value, CurValue);
		}

		Em.Output(0, Value);
	}
	else
	{
		MIR::EOperator OpMIR = MaterialExpressionOperatorToMIR(Operator);
		int32 OperatorArity = MIR::GetOperatorArity(OpMIR);

		FValueRef BValue = OperatorArity >= 2 ? Em.InputDefaultFloat(&DynamicInputs[1].ExpressionInput, DynamicInputs[1].ConstValue) : FValueRef{};
		FValueRef CValue = OperatorArity >= 3 ? Em.InputDefaultFloat(&DynamicInputs[2].ExpressionInput, DynamicInputs[2].ConstValue) : FValueRef{};

		Em.Output(0, Em.Operator(OpMIR, AValue, BValue, CValue));
	}
}

void UMaterialExpressionFloatToUInt::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.CastToIntKind(Em.Input(&Input)));
}

void UMaterialExpressionUIntToFloat::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.CastToFloatKind(Em.Input(&Input)));
}

void UMaterialExpressionTruncateLWC::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.Input(&Input);
	int32 LWCTruncateMode = UE::MaterialTranslatorUtils::GetLWCTruncateMode();

	if (LWCTruncateMode == 1 || LWCTruncateMode == 2)
	{
		TOptional<MIR::FPrimitive> Primitive = InputValue->Type.AsPrimitive();
		if (Primitive && Primitive->ScalarKind == MIR::EScalarKind::Double)
		{
			Em.Output(0, Em.CastToFloatKind(Em.Input(&Input)));
			return;
		}
	}

	Em.Output(0, InputValue);
}

enum class EActorOrObjectPosition
{
	Actor, Object,
};

static FValueRef EmitActorOrObjectPosition(MIR::FEmitter& Em, EActorOrObjectPosition Mode, EPositionOrigin Origin)
{
	struct FActorAndObjectPositionExtern
	{
		EActorOrObjectPosition Mode;
		bool bIsAbsolute;

		FActorAndObjectPositionExtern(EActorOrObjectPosition Mode, bool bIsAbsolute)
		: Mode { Mode }
		, bIsAbsolute { bIsAbsolute }
		{}

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXTVIEW("ActorAndObjectPosition"),
				.Type = bIsAbsolute ? MIR::FType::MakeDoubleVector(3) : MIR::FType::MakeFloatVector(3),
			};
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			if (Printer.Differential != MIR::EExternDifferential::None)
			{
				Printer << (bIsAbsolute ? TEXTVIEW("WSPromote((MaterialFloat3)0)") : TEXTVIEW("(MaterialFloat3)0"));
				return;
			}
			// Get<Previous><Actor|Object><Translated>WorldPosition(Parameters)
			Printer << TEXTVIEW("Get");
			if (Mode == EActorOrObjectPosition::Actor && Printer.bIsPreviousFrame)
			{
				// Object doesn't have previous
				Printer << TEXTVIEW("Previous");
			}
			Printer << (Mode == EActorOrObjectPosition::Actor ? TEXTVIEW("Actor") : TEXTVIEW("Object"));
			if (!bIsAbsolute)
			{
				Printer << TEXTVIEW("Translated");
			}
			Printer << TEXTVIEW("WorldPosition(Parameters)");
		}

		void EmitDebugInfo(FString& Out) const
		{
			Out.Appendf(TEXT("IsActorPosition=%d, bIsAbsolute=%d"), Mode == EActorOrObjectPosition::Actor, bIsAbsolute);
		}

		void CopyTo(FActorAndObjectPositionExtern& Other) const
		{
			Other.Mode = Mode;
			Other.bIsAbsolute = bIsAbsolute;
		}
	};

	FValueRef Value = Em.Extern<FActorAndObjectPositionExtern>({ Mode, Origin == EPositionOrigin::Absolute });
	if (Origin == EPositionOrigin::Absolute && !UE::MaterialTranslatorUtils::IsLWCEnabled())
	{
		Value = Em.CastToFloatKind(Value);
	}

	return Value;
}

void UMaterialExpressionActorPositionWS::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitActorOrObjectPosition(Em, EActorOrObjectPosition::Actor, OriginType));
}

void UMaterialExpressionObjectPositionWS::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitActorOrObjectPosition(Em, EActorOrObjectPosition::Object, OriginType));
}

// Adapted from FHLSLMaterialTranslator::GetWorldPositionOrDefault.  WorldPosition input is optional, a default is provided if not set.
static FValueRef EmitWorldPositionOrDefault(MIR::FEmitter& Em, FValueRef WorldPosition, EPositionOrigin PositionOrigin)
{
	if (PositionOrigin != EPositionOrigin::Absolute && PositionOrigin != EPositionOrigin::CameraRelative)
	{
		Em.Error(TEXT("Invalid EPositionOrigin enum value."));
		return Em.Poison();
	}
	if (WorldPosition)
	{
		// Sanitize the explicitly provided input to the correct vector type if needed.
		return Em.Cast(WorldPosition, PositionOrigin == EPositionOrigin::Absolute ? MIR::FType::MakeDoubleVector(3) : MIR::FType::MakeFloatVector(3));
	}
	else
	{
		// Return default world position.
		return EmitWorldPosition(Em, PositionOrigin == EPositionOrigin::CameraRelative ? EWorldPositionIncludedOffsets::WPT_CameraRelative : EWorldPositionIncludedOffsets::WPT_Default);
	}
}

void UMaterialExpressionWorldPosition::Build(MIR::FEmitter& Em)
{
	FValueRef WorldPosition = EmitWorldPosition(Em, WorldPositionShaderOffset);

	Em.Output(0, WorldPosition);
	Em.Output(1, Em.Swizzle(WorldPosition, MIR::FSwizzleMask::XY()));
	Em.Output(2, Em.Subscript(WorldPosition, 2));
}

void UMaterialExpressionLocalPosition::Build(MIR::FEmitter& Em)
{
	FValueRef LocalPosition = EmitLocalPosition(Em, LocalOrigin, IncludedOffsets);

	Em.Output(0, LocalPosition);
	Em.Output(1, Em.Swizzle(LocalPosition, MIR::FSwizzleMask::XY()));
	Em.Output(2, Em.Subscript(LocalPosition, 2));
}

void UMaterialExpressionMakeMaterialAttributes::Build(MIR::FEmitter& Em)
{
	MIR::TTemporaryArray<MIR::FAttributeAssignment> Assignments{ MP_MAX };
	int32 NumAssignments = 0;

	auto PushAttributeAssignment = [&](EMaterialProperty Property, FExpressionInput* Input)
	{
		if (FValueRef Value = Em.TryInput(Input))
		{
			Assignments[NumAssignments++] = { *FMaterialAttributeDefinitionMap::GetAttributeName(Property), Value };
		}
	};

	PushAttributeAssignment(MP_BaseColor, &BaseColor);
	PushAttributeAssignment(MP_Metallic, &Metallic);
	PushAttributeAssignment(MP_Specular, &Specular);
	PushAttributeAssignment(MP_Roughness, &Roughness);
	PushAttributeAssignment(MP_Anisotropy, &Anisotropy);
	PushAttributeAssignment(MP_EmissiveColor, &EmissiveColor);
	PushAttributeAssignment(MP_Opacity, &Opacity);
	PushAttributeAssignment(MP_OpacityMask, &OpacityMask);
	PushAttributeAssignment(MP_Normal, &Normal);
	PushAttributeAssignment(MP_Tangent, &Tangent);
	PushAttributeAssignment(MP_WorldPositionOffset, &WorldPositionOffset);
	PushAttributeAssignment(MP_SubsurfaceColor, &SubsurfaceColor);
	PushAttributeAssignment(MP_CustomData0, &ClearCoat);
	PushAttributeAssignment(MP_CustomData1, &ClearCoatRoughness);
	PushAttributeAssignment(MP_AmbientOcclusion, &AmbientOcclusion);
	PushAttributeAssignment(MP_Refraction, &Refraction);
	PushAttributeAssignment(MP_PixelDepthOffset, &PixelDepthOffset);
	PushAttributeAssignment(MP_ShadingModel, &ShadingModel);
	PushAttributeAssignment(MP_Displacement, &Displacement);

	for (int32 i = 0; i < 8; ++i)
	{
		PushAttributeAssignment(EMaterialProperty(MP_CustomizedUVs0 + i), &CustomizedUVs[i]);
	}

	Em.Output(0, Em.Aggregate(MaterialAttributesAggregate::Get(), {}, Assignments.Left(NumAssignments)));
}

void UMaterialExpressionBreakMaterialAttributes::Build(MIR::FEmitter& Em)
{
	FValueRef Prototype = Em.CheckIsAggregate(Em.Input(&MaterialAttributes), MaterialAttributesAggregate::Get());
	UE_MIR_CHECKPOINT(Em);

	static const EMaterialProperty Properties[] = {
		MP_BaseColor,
		MP_Metallic,
		MP_Specular,
		MP_Roughness,
		MP_Anisotropy,
		MP_EmissiveColor,
		MP_Opacity,
		MP_OpacityMask,
		MP_Normal,
		MP_Tangent,
		MP_WorldPositionOffset,
		MP_SubsurfaceColor,
		MP_CustomData0, // ClearColor
		MP_CustomData1, // ClearColorRoughness
		MP_AmbientOcclusion,
		MP_Refraction,
		MP_PixelDepthOffset,
		MP_ShadingModel,
		MP_Displacement
	};

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Properties); ++Index)
	{
		Em.Output(Index, Em.Subscript(Prototype, MaterialAttributesAggregate::MaterialPropertyToAttributeIndex(Properties[Index])));
	}
}

// Verifies that the attribute ids in the MaterialAttributes expression are valid (e.g. no duplicates, proper mapping).
static void CheckMaterialAttributesExpression(MIR::FEmitter& Em, TConstArrayView<FGuid> AttributeIds)
{
	for (int32 i = 0; i < AttributeIds.Num(); ++i)
	{
		for (int j = i + 1; j < AttributeIds.Num(); ++j)
		{
			if (AttributeIds[i] == AttributeIds[j])
			{
				Em.Error(TEXT("Duplicate attribute types."));
				return;
			}
		}

		if (FMaterialAttributeDefinitionMap::GetProperty(AttributeIds[i]) == MP_MAX)
		{
			Em.Error(TEXT("Property type doesn't exist, needs re-mapping?"));
			return;
		}
	}
}

void UMaterialExpressionGetMaterialAttributes::Build(MIR::FEmitter& Em)
{
	CheckMaterialAttributesExpression(Em, AttributeGetTypes);

	FValueRef Prototype = Em.CheckIsAggregate(Em.TryInput(&MaterialAttributes), MaterialAttributesAggregate::Get());
	
	UE_MIR_CHECKPOINT(Em);
	
	Em.Output(0, Prototype);

	const UMaterialAggregate* MaterialAttributesAggregate = MaterialAttributesAggregate::Get();
	for (int32 i = 0; i < AttributeGetTypes.Num(); ++i)
	{
		EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(AttributeGetTypes[i]);
		check(Property != MP_MAX);

		int32 AttributeIndex = MaterialAttributesAggregate->FindAttributeIndexByName(*FMaterialAttributeDefinitionMap::GetAttributeName(Property));

		Em.Output(i + 1, Em.Subscript(Prototype, AttributeIndex));
	}
}

void UMaterialExpressionSetMaterialAttributes::Build(MIR::FEmitter& Em)
{
	CheckMaterialAttributesExpression(Em, AttributeSetTypes);
	
	FValueRef Prototype = Em.CheckIsAggregate(Em.TryInput(&Inputs[0]), MaterialAttributesAggregate::Get());
	
	UE_MIR_CHECKPOINT(Em);

	const UMaterialAggregate* MaterialAttributesAggregate = MaterialAttributesAggregate::Get();
	MIR::TTemporaryArray<MIR::FAttributeAssignment> Assignments{ AttributeSetTypes.Num() };
	int32 NumAssignments = 0;

	for (int32 i = 0; i < AttributeSetTypes.Num(); ++i)
	{
		EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(AttributeSetTypes[i]);
		check(Property != MP_MAX);

		if (FValueRef Value = Em.TryInput(&Inputs[i + 1]))
		{
			Assignments[NumAssignments++] = { *FMaterialAttributeDefinitionMap::GetAttributeName(Property), Value };
		}
	}

	Prototype = Em.Aggregate(MaterialAttributesAggregate::Get(), Prototype, Assignments.Left(NumAssignments));

	Em.Output(0, Prototype);
}

// Utility to input a MaterialAttributes value, or return the default instance (with each attribute set to zero). 
static FValueRef InputDefaultMaterialAttributes(MIR::FEmitter& Em, FExpressionInput* Input)
{
	const UMaterialAggregate* MaterialAttributes = MaterialAttributesAggregate::Get();
	FValueRef Value = Em.CheckIsAggregate(Em.TryInput(Input), MaterialAttributes);
	return Value ? Value : Em.Aggregate(MaterialAttributes);
}

// Converts old EMaterialAttributeBlend::Type to EMaterialExpressionBlendMode.
static EMaterialExpressionBlendMode ConvertMaterialAttributeBlend(EMaterialAttributeBlend::Type InBlend)
{
	switch (InBlend)
	{
		case EMaterialAttributeBlend::Blend: return EMaterialExpressionBlendMode::Blend;
		case EMaterialAttributeBlend::UseA: return EMaterialExpressionBlendMode::UseA;
		case EMaterialAttributeBlend::UseB: return EMaterialExpressionBlendMode::UseB;
		default: UE_MIR_UNREACHABLE();
	}
}

// Forward declaration.
static FValueRef Blend(MIR::FEmitter& Em, EMaterialExpressionBlendMode PixelAttributeBlendMode, EMaterialExpressionBlendMode VertexAttributeBlendMode, FValueRef A, FValueRef B, FValueRef Alpha);

// Blends two argument aggregate values based on [0-1] alpha value. See Blend() below for more info.
static FValueRef BlendAggregate(MIR::FEmitter& Em,
						   EMaterialExpressionBlendMode PixelAttributesBlendMode,
						   EMaterialExpressionBlendMode VertexAttributesBlendMode,
						   FValueRef A,
						   FValueRef B,
						   FValueRef Alpha)
{
	const UMaterialAggregate* MaterialAggregate = A->Type.AsAggregate();
	MIR::TTemporaryArray<MIR::FValueRef> AttributeValues{ MaterialAggregate->Attributes.Num() };

	for (int32 i = 0; i < MaterialAggregate->Attributes.Num(); ++i)
	{
		EMaterialExpressionBlendMode BlendMode = PixelAttributesBlendMode;
		if (MaterialAggregate == MaterialAttributesAggregate::Get())
		{
			EMaterialProperty Property = MaterialAttributesAggregate::AttributeIndexToMaterialProperty(i);
			BlendMode = (Property == MP_WorldPositionOffset) ? VertexAttributesBlendMode : PixelAttributesBlendMode;
		}

		if (BlendMode == EMaterialExpressionBlendMode::UseA)
		{
			AttributeValues[i] = Em.Subscript(A, i);
		}
		if (BlendMode == EMaterialExpressionBlendMode::UseB)
		{
			AttributeValues[i] = Em.Subscript(B, i);
		}
		else if (BlendMode == EMaterialExpressionBlendMode::Blend)
		{
			AttributeValues[i] = Blend(Em, PixelAttributesBlendMode, VertexAttributesBlendMode, Em.Subscript(A, i), Em.Subscript(B, i), Alpha);
		}
	}

	return Em.Aggregate(MaterialAggregate, {}, AttributeValues);
}

// Blends two argument values based on [0-1] alpha value. If argument values are or contain MaterialAttributes aggregates,
// PixelAttributeBlendMode and VertexAttributeBlendMode instruct on how to blend the attributes depending on whether
// they're evaluated in pixel or vertex shaders.
// Note: VertexAttributeBlendMode are only used when blending MaterialAttributes. Otherwise, PixelAttributeBlendMode is used.
static FValueRef Blend(MIR::FEmitter& Em, EMaterialExpressionBlendMode PixelAttributeBlendMode, EMaterialExpressionBlendMode VertexAttributeBlendMode, FValueRef A, FValueRef B, FValueRef Alpha)
{
	// Find the common type between arguments
	if (MIR::FType CommonType = Em.GetCommonType(A->Type, B->Type); !CommonType.IsPoison())
	{
		// And cast both arguments to the common type
		A = Em.Cast(A, CommonType);
		B = Em.Cast(B, CommonType);
	}
	else
	{
		return Em.Poison();
	}

	if (A->Type.IsAnyFloat())
	{
		// Blend floating point values using linear interpolation
		return Em.Lerp(A, B, Alpha);
	}
	else if (A->Type.IsInteger())
	{
		// "Blend" integer values by selecting A or B based on whether alpha is less than 0.5.
		return Em.Select(Em.LessThan(Alpha, Em.ConstantFloat(0.5f)), A, B);
	}
	else if (A->Type.AsAggregate())
	{
		// Arguments are aggregates, so recursively blend each attribute pair.
		return BlendAggregate(Em, PixelAttributeBlendMode, VertexAttributeBlendMode, A, B, Alpha);
	}
	else if (A->Type.IsSubstrateData())
	{
		// Ignore this attribute for now. FrontMaterial should be removed from MaterialAttributes. @massimo.tristano, @charles.derousiers, @sebastien.hillaire
		return Em.SubstrateDefaultSlab();
	}
	else
	{
		Em.Errorf(TEXT("Cannot blend values of type '%s'."), *A->Type.GetSpelling());
		return Em.Poison();
	}
}

void UMaterialExpressionBlendMaterialAttributes::Build(MIR::FEmitter& Em)
{
	const UMaterialAggregate* MaterialAttributes = MaterialAttributesAggregate::Get();
	
	FValueRef AValue = InputDefaultMaterialAttributes(Em, &A);
	FValueRef BValue = InputDefaultMaterialAttributes(Em, &B);
	FValueRef AlphaValue = Em.CastToFloat(Em.InputDefaultFloat(&Alpha, 0.0f), 1);

	UE_MIR_CHECKPOINT(Em);

	FValueRef Result = BlendAggregate(
		Em, 
		ConvertMaterialAttributeBlend(PixelAttributeBlendType),
		ConvertMaterialAttributeBlend(VertexAttributeBlendType),
		AValue,
		BValue,
		AlphaValue);

	Em.Output(0, Result);
}

void UMaterialExpressionAggregate::Build(MIR::FEmitter& Em)
{
	// Get the material aggregate definition.
	const UMaterialAggregate* Aggregate = GetAggregate();
	if (!Aggregate)
	{
		Em.Error(TEXT("Unspecified material aggregate."));
		return;
	}

	// Read the aggregate prototype value, if present, and make sure it is of the right type.
	FValueRef Prototype = Em.CheckIsAggregate(Em.TryInput(&PrototypeInput), Aggregate);
	UE_MIR_CHECKPOINT(Em);

	// Collect the attribute assignments from the input pins.
	MIR::TTemporaryArray<MIR::FAttributeAssignment> Assignments{ Entries.Num() };
	int32 NumAssignments = 0;

	for (const FMaterialExpressionAggregateEntry& Entry : Entries)
	{
		// If value is present, push this attribute assignment.
		if (FValueRef AttributeValue = Em.TryInput(&Entry.Input))
		{
			Assignments[NumAssignments++] = { Aggregate->Attributes[Entry.AttributeIndex].Name, AttributeValue };
		}
	}
	
	// Make the aggregate value using the optional prototype and assignments.
	Prototype = Em.Aggregate(Aggregate, Prototype, Assignments.Left(NumAssignments));

	// Output the aggregate value
	Em.Output(0, Prototype);

	// And output each individual aggregate attribute through the invidual output pins
	for (int i = 0; i < Entries.Num(); ++i)
	{
		Em.Output(i + 1, Em.Subscript(Prototype, Entries[i].AttributeIndex));
	}
}

void UMaterialExpressionBlend::Build(MIR::FEmitter& Em)
{
	// Try reading the input values (could be null).
	FValueRef AValue = Em.TryInput(&A);
	FValueRef BValue = Em.TryInput(&B);

	if (!AValue && !BValue)
	{
		Em.Error(TEXT("No input value provided."));
		return;
	}

	// Create a default value from the other input's type if any input is missing.
	if (!AValue)
	{
		AValue = Em.ConstantDefault(BValue->Type);
	}
	else if (!BValue)
	{
		BValue = Em.ConstantDefault(AValue->Type);
	}

	// Read the alpha value (defaulting it to 0.0f)
	FValueRef AlphaValue = Em.CastToFloat(Em.InputDefaultFloat(&Alpha, 0.0f), 1);

	// Make sure all previous operations went well.
	UE_MIR_CHECKPOINT(Em);

	// Blend the input values.
	FValueRef Result = Blend(Em, PixelAttributesBlendMode, VertexAttributesBlendMode, AValue, BValue, AlphaValue);
	
	Em.Output(0, Result);
}

static FValueRef EmitParameterCollectionVectorExtern(MIR::FEmitter& Em, MIR::FValueRef CollectionValue, int32 ParameterIndex)
{
	return Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("MaterialCollection$0.Vectors[$1]") }, CollectionValue, Em.ConstantInt(ParameterIndex));
}

static bool GetExpressionCollectionParameter(MIR::FEmitter& Em, UMaterialParameterCollection* Collection, FName ParameterName, const FGuid& ParameterId, int32& OutParamIndex, int32& OutComponentIndex)
{
	if (!Collection)
	{
		Em.Errorf(TEXT("CollectionParameter has invalid Collection!"));
		return false;
	}

	Collection->GetParameterIndex(ParameterId, OutParamIndex, OutComponentIndex);
	if (OutParamIndex == INDEX_NONE)
	{
		Em.Errorf(TEXT("CollectionParameter has invalid parameter %s"), *ParameterName.ToString());
		return false;
	}

	return true;
}

void UMaterialExpressionCollectionParameter::Build(MIR::FEmitter& Em)
{
	int32 ParameterIndex = INDEX_NONE;
	int32 ComponentIndex = INDEX_NONE;
	if (!GetExpressionCollectionParameter(Em, Collection, ParameterName, ParameterId, ParameterIndex, ComponentIndex))
	{
		return;
	}

	FValueRef Result = EmitParameterCollectionVectorExtern(Em, Em.MaterialParameterCollection(Collection), ParameterIndex);
	if (ComponentIndex != INDEX_NONE)
	{
		Result = Em.Subscript(Result, ComponentIndex % 4);
	}
	Em.Output(0, Result);
}

void UMaterialExpressionCollectionTransform::Build(MIR::FEmitter& Em)
{
	int32 ParameterIndex = INDEX_NONE;
	int32 ComponentIndex = INDEX_NONE;
	if (!GetExpressionCollectionParameter(Em, Collection, ParameterName, ParameterId, ParameterIndex, ComponentIndex))
	{
		return;
	}

	if (ComponentIndex != INDEX_NONE)
	{
		Em.Errorf(TEXT("CollectionTransform parameter %s is scalar, vectors are required"), *ParameterName.ToString());
		return;
	}

	FValueRef Value = Em.CheckIsPrimitive(Em.Input(&Input));
	
	UE_MIR_CHECKPOINT(Em);

	if (!Value->Type.IsAnyFloat() || Value->Type.GetPrimitive().NumRows != 1 || Value->Type.GetPrimitive().NumColumns < 3)
	{
		Em.Error(TEXT("CollectionTransform requires float3 vector input"));
		return;
	}

	int32 NumVectors = 0;
	if (TransformType == EParameterCollectionTransformType::Position || TransformType == EParameterCollectionTransformType::Projection)
	{
		if (ParameterIndex + 4 > Collection->GetTotalVectorStorage())
		{
			Em.Errorf(TEXT("CollectionTransform parameter %s requires 4 vectors for Position or Projection matrix"), *ParameterName.ToString());
			return;
		}
		NumVectors = 4;
	}
	else if (TransformType == EParameterCollectionTransformType::Vector)
	{
		if (ParameterIndex + 3 > Collection->GetTotalVectorStorage())
		{
			Em.Errorf(TEXT("CollectionTransform parameter %s requires 3 vectors for Vector matrix"), *ParameterName.ToString());
			return;
		}
		NumVectors = 3;
	}
	else
	{
		check(TransformType == EParameterCollectionTransformType::LocalToWorld || TransformType == EParameterCollectionTransformType::WorldToLocal);
		if (ParameterIndex + 5 > Collection->GetTotalVectorStorage())
		{
			Em.Errorf(TEXT("CollectionTransform parameter %s requires 5 vectors for LWC Matrix"), *ParameterName.ToString());
			return;
		}
		NumVectors = 5;
	}

	FValueRef CollectionValue = Em.MaterialParameterCollection(Collection);
	TArray<FValueRef, TFixedAllocator<5>> CollectionParameters;

	for (int32 i = 0; i < NumVectors; i++)
	{
		CollectionParameters.Add(EmitParameterCollectionVectorExtern(Em, CollectionValue, ParameterIndex + i));
	}

	MIR::FValueRef Result;

	// Matrix transforms cobbled together from primitive ops (rather than using mul or LWCMultiply), so analytic derivatives are supported for free
	if (TransformType == EParameterCollectionTransformType::Vector)
	{
		// Treat input as a direction vector (w = 0)
		Value = Em.Cast(Value, MIR::FType::MakeFloatVector(3));

		Result =        Em.Multiply(Em.Subscript(Value, 0), Em.Swizzle(CollectionParameters[0], MIR::FSwizzleMask::XYZ()));
		Result = Em.Add(Em.Multiply(Em.Subscript(Value, 1), Em.Swizzle(CollectionParameters[1], MIR::FSwizzleMask::XYZ())), Result);
		Result = Em.Add(Em.Multiply(Em.Subscript(Value, 2), Em.Swizzle(CollectionParameters[2], MIR::FSwizzleMask::XYZ())), Result);
	}
	else if (TransformType == EParameterCollectionTransformType::Projection)
	{
		// Optimized to save many ALU for a standard perspective or orthographic projection matrix, where most of the elements of the matrix are zero.
		Result = Em.Vector4(
			Em.Multiply(Em.Subscript(Value, 0), Em.Subscript(CollectionParameters[0], 0)),														// Value.x * Matrix._00
			Em.Multiply(Em.Subscript(Value, 1), Em.Subscript(CollectionParameters[1], 1)),														// Value.y * Matrix._11
			Em.Add(Em.Multiply(Em.Subscript(Value, 2), Em.Subscript(CollectionParameters[2], 2)), Em.Subscript(CollectionParameters[3], 2)),	// Value.z * Matrix._22 + Matrix._32
			Em.Add(Em.Multiply(Em.Subscript(Value, 2), Em.Subscript(CollectionParameters[2], 3)), Em.Subscript(CollectionParameters[3], 3)));	// Value.z * Matrix._23 + Matrix._33
	}
	else
	{
		// Position, LocalToWorld, WorldToLocal
		if (TransformType == EParameterCollectionTransformType::WorldToLocal)
		{
			// Pre subtract tile value, to convert this to float (LWC inverse matrices have their tile negated, so adding means we are subtracting the tile value).
			// The tile value only applies to XYZ -- if the input Value has a fourth component, the Add operation will pad the tile argument with zero if needed.
			Value = Em.Add(Em.CastToScalarKind(Value, MIR::EScalarKind::Double), Em.LWCTile(Em.Swizzle(CollectionParameters[4], MIR::FSwizzleMask::XYZ())));
		}

		Value = Em.CastToFloatKind(Value);

		// If a 3-element vector is provided as input, we want to generate a 3-element vector as output.  Swizzle the collection parameters to achieve this.
		if (Value->Type.GetPrimitive().NumColumns == 3)
		{
			for (int32 i = 0; i < 4; i++)
			{
				CollectionParameters[i] = Em.Swizzle(CollectionParameters[i], MIR::FSwizzleMask::XYZ());
			}
		}

		Result =        Em.Multiply(Em.Subscript(Value, 0), CollectionParameters[0]);
		Result = Em.Add(Em.Multiply(Em.Subscript(Value, 1), CollectionParameters[1]), Result);
		Result = Em.Add(Em.Multiply(Em.Subscript(Value, 2), CollectionParameters[2]), Result);

		if (Value->Type.GetPrimitive().NumColumns == 3)
		{
			// Treat input as a translation vector (w = 1)
			Result = Em.Add(CollectionParameters[3], Result);
		}
		else
		{
			// Treat input as a homogenous vector (w = user specified)
			Result = Em.Add(Em.Multiply(Em.Subscript(Value, 3), CollectionParameters[3]), Result);
		}

		if (TransformType == EParameterCollectionTransformType::LocalToWorld)
		{
			// Post add tile value, to convert this to LWC
			Result = Em.Add(Em.CastToScalarKind(Result, MIR::EScalarKind::Double), Em.LWCTile(Em.Swizzle(CollectionParameters[4], MIR::FSwizzleMask::XYZ())));
		}
	}

	Em.Output(0, Result);
}

void UMaterialExpressionAtmosphericFogColor::Build(MIR::FEmitter& Em)
{
	// This node is deprecated in favor of UMaterialExpressionSkyAtmosphereAerialPerspective, and falls through to the newer expression
	FValueRef PositionValue = EmitWorldPositionOrDefault(Em, Em.TryInput(&WorldPosition), WorldPositionOriginType);
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, PositionValue));
}

void UMaterialExpressionBlackBody::Build(MIR::FEmitter& Em)
{
	FValueRef TempValue = Em.CastToFloat(Em.InputDefaultFloat(&Temp, 0.f), 1);
	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("MaterialExpressionBlackBody($0)") }, TempValue));
}

void UMaterialExpressionDepthFade::Build(MIR::FEmitter& Em)
{
	// Scales Opacity by a Linear fade based on SceneDepth, from 0 at PixelDepth to 1 at FadeDistance
	// Result = Opacity * saturate((SceneDepth - PixelDepth) / max(FadeDistance, DELTA))
	FValueRef OpacityValue = Em.InputDefaultFloat(&InOpacity, OpacityDefault);
	FValueRef FadeDistanceValue = Em.Max(Em.InputDefaultFloat(&FadeDistance, FadeDistanceDefault), Em.ConstantFloat(UE_DELTA));

	static FName NAME_PixelDepth("PixelDepth");
	FValueRef PixelDepth = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_PixelDepth);
	// On mobile scene depth is limited to 65500 
	// to avoid false fading on objects that are close or exceed this limit we clamp pixel depth to (65500 - FadeDistance)
	if (Em.GetFeatureLevel() <= ERHIFeatureLevel::ES3_1)
	{
		PixelDepth = Em.Min(PixelDepth, Em.Subtract(Em.ConstantFloat(65500.f), FadeDistanceValue));
	}

	// We need a dependency on EScreenTexture::SceneDepth, so the value analyzer can see it, even though it's technically not used in the code.
	FValueRef SceneDepth = Em.Nop(Em.Extern<FScreenTextureExtern>({ EScreenTexture::SceneDepth }));
	SceneDepth = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("CalcSceneDepth(ScreenAlignedPosition(GetScreenPosition(Parameters)))") }, SceneDepth);

	Em.Output(0, Em.Multiply(OpacityValue, Em.Saturate(Em.Divide(Em.Subtract(SceneDepth, PixelDepth), FadeDistanceValue))));
}

void UMaterialExpressionDeriveNormalZ::Build(MIR::FEmitter& Em)
{
	// z = sqrt(saturate(1 - ( x * x + y * y)));
	FValueRef InputVector = Em.Cast(Em.Input(&InXY), MIR::FType::MakeFloatVector(2));
	FValueRef DotResult = Em.Dot(InputVector, InputVector);
	FValueRef InnerResult = Em.Subtract(Em.ConstantFloat(1.0f), DotResult);
	FValueRef SaturatedInnerResult = Em.Saturate(InnerResult);
	FValueRef DerivedZ = Em.Sqrt(SaturatedInnerResult);
	
	Em.Output(0, Em.Vector3(Em.Subscript(InputVector, 0), Em.Subscript(InputVector, 1), DerivedZ));
}
	
void UMaterialExpressionDistanceFieldApproxAO::Build(MIR::FEmitter& Em)
{
	FValueRef PositionValue = EmitWorldPositionOrDefault(Em, Em.TryInput(&Position), WorldPositionOriginType);

	FValueRef NormalValue = Em.TryInput(&Normal);
	if (!NormalValue)
	{
		static FName NAME_VertexNormal("VertexNormal");
		NormalValue = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_VertexNormal);
	}

	FValueRef BaseDistanceValue = Em.InputDefaultFloat(&BaseDistance, BaseDistanceDefault);

	int32 NumStepsClamped = FMath::Clamp(NumSteps, 1, 4);
	float StepScaleClamped = FMath::Max(StepScaleDefault, 1.0f);

	FValueRef NumStepsConst = Em.ConstantInt(NumStepsClamped);
	FValueRef NumStepsMinus1Const = Em.ConstantInt(NumStepsClamped - 1);
	FValueRef StepScaleConst = Em.ConstantFloat(StepScaleClamped);

	FValueRef StepDistance;
	FValueRef DistanceBias;
	FValueRef MaxDistance;

	if (NumSteps == 1)
	{
		StepDistance = Em.ConstantFloat(0);
		DistanceBias = BaseDistanceValue;
		MaxDistance = BaseDistanceValue;
	}
	else
	{
		FValueRef RadiusValue = Em.InputDefaultFloat(&Radius, RadiusDefault);

		StepDistance = Em.Divide(Em.Subtract(RadiusValue, BaseDistanceValue), Em.Subtract(Em.Pow(StepScaleConst, NumStepsMinus1Const), Em.ConstantFloat(1.0f)));
		DistanceBias = Em.Subtract(BaseDistanceValue, StepDistance);
		MaxDistance = RadiusValue;
	}

	struct FGlobalDistanceFieldExtern
	{
		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name  = TEXTVIEW("GlobalDistanceField"),
				.Type  = MIR::FType::MakeFloatScalar(),
				.Flags = MIR::EExternFlags::ZeroDifferentials,
			};
		}
	
		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			if (!FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields(Context.Module->GetShaderPlatform()))
			{
				FString ShaderPlatformName = FDataDrivenShaderPlatformInfo::GetName(Context.Module->GetShaderPlatform()).ToString();
				Context.Module->AddError(nullptr, FString::Printf(TEXT("Node not supported in shader platform %s. The node requires DistanceField support."), *ShaderPlatformName));
				return;
			}

			Context.Module->GetCompilationOutput().bUsesGlobalDistanceField = true;
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << TEXT("CalculateDistanceFieldApproxAO($0, $1, $2, $3, $4, $5, $6)");
		}
	};

	// Last input tells value analyzer that this expression uses the global distance field
	Em.Output(0, Em.Extern<FGlobalDistanceFieldExtern>({},
		PositionValue,
		Em.Cast(NormalValue, MIR::FType::MakeFloatVector(3)),
		NumStepsConst,
		Em.Cast(StepDistance, MIR::FType::MakeFloatScalar()),
		StepScaleConst,
		Em.Cast(DistanceBias, MIR::FType::MakeFloatScalar()),
		Em.Cast(MaxDistance, MIR::FType::MakeFloatScalar())
	));
}

static void EmitDistanceFieldExpression(MIR::FEmitter& Em, FExpressionInput& Position, EPositionOrigin WorldPositionOriginType, EMaterialValueType Type,  const TCHAR* FunctionName)
{
	struct FDistanceFieldHelper
	{
		EMaterialValueType Type;
		const TCHAR* FunctionName;

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXTVIEW("DistanceFieldHelper"),
				.Type = MIR::FType::FromMaterialValueType(Type),
				.Flags = MIR::EExternFlags::Inline,
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			if (!FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields(Context.Module->GetShaderPlatform()))
			{
				FString ShaderPlatformName = FDataDrivenShaderPlatformInfo::GetName(Context.Module->GetShaderPlatform()).ToString();
				Context.Module->AddError(nullptr, FString::Printf(TEXT("Node not supported in shader platform %s. The node requires DistanceField support."), *ShaderPlatformName));
				return;
			}

			Context.Module->GetCompilationOutput().bUsesGlobalDistanceField = true;
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << FunctionName << TEXT('(') << Printer.Arg(0) << TEXT(')');
		}
	};

	FValueRef PositionValue = EmitWorldPositionOrDefault(Em, Em.TryInput(&Position), WorldPositionOriginType);
	FValueRef Extern = Em.Extern<FDistanceFieldHelper>({ Type, FunctionName }, PositionValue);
	Em.Output(0, Extern);
}

void UMaterialExpressionDistanceFieldGradient::Build(MIR::FEmitter& Em)
{
	EmitDistanceFieldExpression(Em, Position, WorldPositionOriginType, MCT_Float3, TEXT("GetDistanceFieldGradientGlobal"));
}

void UMaterialExpressionDistanceToNearestSurface::Build(MIR::FEmitter& Em)
{
	EmitDistanceFieldExpression(Em, Position, WorldPositionOriginType, MCT_Float, TEXT("GetDistanceToNearestSurfaceGlobal"));
}

void UMaterialExpressionFresnel::Build(MIR::FEmitter& Em)
{
	// pow(1 - max(0,Normal dot Camera),Exponent) * (1 - BaseReflectFraction) + BaseReflectFraction
	//
	FValueRef NormalArg = Em.TryInput(&Normal);
	if (NormalArg)
	{
		NormalArg = Em.Cast(NormalArg, MIR::FType::MakeFloatVector(3));
	}
	else
	{
		NormalArg = MaterialToMIR::EmitPixelNormalWS(Em);
	}

	FValueRef DotArg = Em.Dot(NormalArg, Em.Extern<MIR::FExternFromMaterialDecl>({ NAME_CameraVector }));
	FValueRef MaxArg = Em.Max(Em.ConstantFloat(0.f), DotArg);
	FValueRef MinusArg = Em.Subtract(Em.ConstantFloat(1.f), MaxArg);
	FValueRef ExponentArg = Em.InputDefaultFloat(&ExponentIn, Exponent);
	
	// Compiler->Power got changed to call PositiveClampedPow instead of ClampedPow
	// Manually implement ClampedPow to maintain backwards compatibility in the case where the input normal is not normalized (length > 1)
	FValueRef AbsBaseArg = Em.Max(Em.Abs(MinusArg), Em.ConstantFloat(UE_KINDA_SMALL_NUMBER));
	FValueRef PowArg = Em.Pow(AbsBaseArg, ExponentArg);
	FValueRef BaseReflectFractionArg = Em.InputDefaultFloat(&BaseReflectFractionIn, BaseReflectFraction);
	FValueRef ScaleArg = Em.Multiply(PowArg, Em.Subtract(Em.ConstantFloat(1.f), BaseReflectFractionArg));
	
	Em.Output(0, Em.Add(ScaleArg, BaseReflectFractionArg));
}

void UMaterialExpressionReflectionVectorWS::Build(MIR::FEmitter& Em)
{
	FValueRef NormalValue = Em.TryInput(&CustomWorldNormal);

	if (NormalValue)
	{
		NormalValue = Em.Cast(NormalValue, MIR::FType::MakeFloatVector(3));

		// Ported from HLSL utility function ReflectionAboutCustomWorldNormal
		if (bNormalizeCustomWorldNormal)
		{
			NormalValue = Em.Multiply(NormalValue, Em.Rsqrt(Em.Dot(NormalValue, NormalValue)));
		}

		// Normal * dot(Normal, CameraVector) * 2.0 - CameraVector;
		FValueRef CameraVector = Em.Extern<MIR::FExternFromMaterialDecl>({ NAME_CameraVector });
		Em.Output(0, Em.Subtract(Em.Multiply(NormalValue, Em.Multiply(Em.Dot(NormalValue, CameraVector), Em.ConstantFloat(2.0f))), CameraVector));
	}
	else
	{
		static FName NAME_ReflectionVector("ReflectionVector");
		Em.Output(0, Em.Extern<MIR::FExternFromMaterialDecl>({ NAME_ReflectionVector }));
	}
}

void UMaterialExpressionRgbToHsv::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloatKind(Em.Input(&Input));
	UE_MIR_CHECKPOINT(Em);

	MIR::FPrimitive Primitive = *InputValue->Type.AsPrimitive();
	if ((!Primitive.IsRowVector() && !Primitive.IsColumnVector()) || Primitive.NumComponents() < 3)
	{
		Em.Error(InputValue, TEXT("Expected a 3D or 4D vector."));
		return;
	}

	FValueRef R = Em.Subscript(InputValue, 0);
	FValueRef G = Em.Subscript(InputValue, 1);
	FValueRef B = Em.Subscript(InputValue, 2);

	FValueRef NegOne		= Em.ConstantFloat(-1.0f);
	FValueRef TwoThirds		= Em.Divide(Em.ConstantFloat(2.0f), Em.ConstantFloat(3.0f));
	FValueRef Zero			= Em.ConstantFloat(0.0f);
	FValueRef NegOneThird	= Em.Divide(Em.ConstantFloat(-1.0f), Em.ConstantFloat(3.0f));
	FValueRef Six			= Em.ConstantFloat(6.0f);
	FValueRef Epsilon		= Em.ConstantFloat(1e-10f);

	// P = (G < B) ? float4(B, G, -1.0, 2.0/3.0) : float4(G, B, 0.0, -1.0/3.0)
	FValueRef P = Em.Branch(Em.LessThan(G, B), Em.Vector4(B, G, NegOne, TwoThirds), Em.Vector4(G, B, Zero, NegOneThird));

	FValueRef Px = Em.Subscript(P, 0);
	FValueRef Py = Em.Subscript(P, 1);
	FValueRef Pz = Em.Subscript(P, 2);
	FValueRef Pw = Em.Subscript(P, 3);

	// Q = (R < P.x) ? float4(P.xyw, R) : float4(R, P.yzx)
	FValueRef Q = Em.Branch(Em.LessThan(R, Px), Em.Vector4(Px, Py, Pw, R), Em.Vector4(R, Py, Pz, Px));

	FValueRef Qx = Em.Subscript(Q, 0);
	FValueRef Qy = Em.Subscript(Q, 1);
	FValueRef Qz = Em.Subscript(Q, 2);
	FValueRef Qw = Em.Subscript(Q, 3);

	// Chroma = Q.x - min(Q.w, Q.y)
	FValueRef Chroma = Em.Subtract(Qx, Em.Min(Qw, Qy));

	// Hue = abs((Q.w - Q.y) / (6.0 * Chroma + 1e-10) + Q.z)
	FValueRef Hue = Em.Abs(Em.Add( Em.Divide(Em.Subtract(Qw, Qy), Em.Add(Em.Multiply(Six, Chroma), Epsilon)), Qz));

	// S = Chroma / (Q.x + 1e-10)
	FValueRef S = Em.Divide(Chroma, Em.Add(Qx, Epsilon));

	// Result = float3(Hue, S, Q.x) or float4(Hue, S, Q.x, input.w)
	FValueRef Result;
	if (Primitive.NumComponents() == 4)
	{
		Result = Em.Vector4(Hue, S, Qx, Em.Subscript(InputValue, 3));
	}
	else
	{
		Result = Em.Vector3(Hue, S, Qx);
	}

	if (Primitive.IsColumnVector())
	{
		Result = Em.Transpose(Result);
	}

	Em.Output(0, Result);
}

void UMaterialExpressionRotateAboutAxis::Build(MIR::FEmitter& Em)
{
	FValueRef Angle = Em.Multiply(Em.Subscript(Em.Input(&RotationAngle), 0), Em.ConstantFloat(2.0f * (float)UE_PI / Period));
	FValueRef Axis = Em.Cast(Em.Input(&NormalizedRotationAxis), MIR::FType::MakeFloatVector(3));
	FValueRef PosOnAxis = Em.Input(&PivotPoint);
	FValueRef Pos = Em.Input(&Position);

	// Math adapted from RotateAboutAxis, but simplified and optimized slightly.  Note that the function returns an offset to
	// the rotated position, not an absolute position, and so the offset will be non-LWC.  This initial subtraction is LWC aware,
	// but we can then use float operations for the remainder (the LWC RotateAboutAxis HLSL function does the same).
	FValueRef PosOffset = Em.Cast(Em.Subtract(Pos, PosOnAxis), MIR::FType::MakeFloatVector(3));

	// Construct orthogonal axes in the plane of rotation.  The UAxis is computed by subtracting the projection of
	// PosOffset along the Axis vector.
	FValueRef UAxis = Em.Subtract(PosOffset, Em.Multiply(Axis, Em.Dot(Axis, PosOffset)));
	FValueRef VAxis = Em.Cross(Axis, UAxis);

	// Rotate the orthogonal axes
	FValueRef CosAngle = Em.Cos(Angle);
	FValueRef SinAngle = Em.Sin(Angle);
	FValueRef R = Em.Add(Em.Multiply(UAxis, CosAngle), Em.Multiply(VAxis, SinAngle));

	// Return the offset from the original position to the rotated position.  The original position in this context
	// is the pre-rotation axis vector.
	Em.Output(0, Em.Subtract(R, UAxis));
}

void UMaterialExpressionRotator::Build(MIR::FEmitter& Em)
{
	FValueRef TimeValue = Em.TryInput(&Time);
	if (!TimeValue)
	{
		TimeValue = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("View.<PREVFRAME>GameTime") });
	}
	TimeValue = Em.Multiply(TimeValue, Em.ConstantFloat(Speed));

	FValueRef BaseCoordinate = Em.TryInput(&Coordinate);
	if (!BaseCoordinate)
	{
		BaseCoordinate = MaterialToMIR::EmitTexCoord(Em, ConstCoordinate);
	}
	BaseCoordinate = Em.Subtract(BaseCoordinate, Em.ConstantFloat2({ CenterX, CenterY }));

	FValueRef CosValue = Em.Cos(TimeValue);
	FValueRef SinValue = Em.Sin(TimeValue);

	FValueRef Arg1 = Em.Add(Em.Subtract(Em.Multiply(CosValue, Em.Subscript(BaseCoordinate, 0)), Em.Multiply(SinValue, Em.Subscript(BaseCoordinate, 1))), Em.ConstantFloat(CenterX));		// cos*U - sin*V + CenterX
	FValueRef Arg2 = Em.Add(Em.Add     (Em.Multiply(SinValue, Em.Subscript(BaseCoordinate, 0)), Em.Multiply(CosValue, Em.Subscript(BaseCoordinate, 1))), Em.ConstantFloat(CenterY));		// sin*U + cos*V + CenterY

	TOptional<MIR::FPrimitive> BaseType = BaseCoordinate->Type.AsPrimitive();
	if (BaseType && BaseType->NumColumns >= 3)
	{
		Em.Output(0, Em.Vector3(Arg1, Arg2, Em.Subscript(BaseCoordinate, 2)));
	}
	else
	{
		Em.Output(0, Em.Vector2(Arg1, Arg2));
	}
}

void UMaterialExpressionSkyAtmosphereAerialPerspective::Build(MIR::FEmitter& Em)
{
	FValueRef PositionValue = EmitWorldPositionOrDefault(Em, Em.TryInput(&WorldPosition), WorldPositionOriginType);
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, PositionValue));
}

void UMaterialExpressionSkyAtmosphereLightDirection::Build(MIR::FEmitter& Em)
{
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, Em.ConstantInt(LightIndex)));
}

void UMaterialExpressionSkyAtmosphereLightDiskLuminance::Build(MIR::FEmitter& Em)
{
	FValueRef CosHalfDiskRadius = Em.TryInput(&DiskAngularDiameterOverride);
	if (CosHalfDiskRadius)
	{
		// Convert from apex angle (angular diameter) to cosine of the disk radius.
		CosHalfDiskRadius = Em.Cos(Em.Multiply(Em.ConstantFloat(0.5f * float(UE_PI) / 180.0f), CosHalfDiskRadius));
	}
	else
	{
		CosHalfDiskRadius = Em.ConstantFloat(-1.0f);
	}
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, Em.ConstantInt(LightIndex), CosHalfDiskRadius));
}

void UMaterialExpressionSkyAtmosphereLightIlluminance::Build(MIR::FEmitter& Em)
{
	FValueRef PositionValue = EmitWorldPositionOrDefault(Em, Em.TryInput(&WorldPosition), WorldPositionOriginType);
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, PositionValue, Em.ConstantInt(LightIndex)));
}

void UMaterialExpressionSkyAtmosphereLightIlluminanceOnGround::Build(MIR::FEmitter& Em)
{
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, Em.ConstantInt(LightIndex)));
}

void UMaterialExpressionSkyAtmosphereViewLuminance::Build(MIR::FEmitter& Em)
{
	FValueRef WorldDirectionValue = Em.TryInput(&WorldDirection);
	if (!WorldDirectionValue)
	{
		WorldDirectionValue = Em.Multiply(Em.ConstantFloat(-1.0f), Em.Extern<MIR::FExternFromMaterialDecl>({ NAME_CameraVector }));
	}
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, WorldDirectionValue));
}

void UMaterialExpressionSkyLightEnvMapSample::Build(MIR::FEmitter& Em)
{
	if (Material->bIsSky)
	{
		UE_LOGF(LogMaterial, Warning, "Using SkyLightEnvMapSample from a IsSky material can result in visual artifact. For instance, if the previous frame capture was super bright, it might leak onto a new frame, e.g. transtion from menu to game.");
	}

	FValueRef DirectionValue = Em.InputDefaultFloat3(&Direction, FVector3f(0.0f, 0.0f, 1.0f));
	FValueRef RoughnessValue = Em.InputDefaultFloat(&Roughness, 0.0f);

	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, DirectionValue, RoughnessValue));
}

void UMaterialExpressionSphereMask::Build(MIR::FEmitter& Em)
{
	FValueRef Arg1 = Em.Input(&A);
	FValueRef Arg2 = Em.Input(&B);
	UE_MIR_CHECKPOINT(Em);

	// 1.0f / max(0.00001f, Radius)
	FValueRef ArgInvRadius = Em.Divide(Em.ConstantFloat(1.0f), Em.Max(Em.ConstantFloat(0.00001f), Em.InputDefaultFloat(&Radius, AttenuationRadius)));

	// 1.0f / max(0.00001fp4, 1.0f - Hardness)
	FValueRef ArgInvHardness = Em.Divide(Em.ConstantFloat(1.0f), Em.Max(Em.ConstantFloat(0.00001f), Em.Subtract(Em.ConstantFloat(1.0f), Em.InputDefaultFloat(&Hardness, HardnessPercent * 0.01f))));

	FValueRef Distance = Em.Length(Em.Subtract(Arg1, Arg2));
	FValueRef NormalizeDistance = Em.Multiply(Distance, ArgInvRadius);
	FValueRef NegNormalizedDistance = Em.Subtract(Em.ConstantFloat(1.0f), NormalizeDistance);
	FValueRef MaskUnclamped = Em.Multiply(NegNormalizedDistance, ArgInvHardness);
	Em.Output(0, Em.Saturate(MaskUnclamped));
}

// Takes a description user string and turns it into a valid C/HLSL identifier.
static FString DescriptionToIdentifier(FStringView Source)
{
	FString Out;
	Out.Reserve(Source.Len());
	// Append an underscore if the source starts by a digit
	if (!Source.IsEmpty() && FChar::IsDigit(Source[0]))
	{
		Out.AppendChar('_');
	}
	for (TCHAR Ch : Source)
	{
		Out.AppendChar(FChar::IsAlnum(Ch) ? Ch : '_');
	}
	return Out;
}

// Custom material output to MIR type conversion.
static MIR::FType CustomMaterialOutputTypeToMIR(ECustomMaterialOutputType Type)
{
	switch (Type) 
	{
		case CMOT_Float1: return MIR::FType::MakeFloatScalar();
		case CMOT_Float2: return MIR::FType::MakeFloatVector(2);
		case CMOT_Float3: return MIR::FType::MakeFloatVector(3);
		case CMOT_Float4: return MIR::FType::MakeFloatVector(4);
		case CMOT_MaterialAttributes: return MIR::FType::MakeAggregate(MaterialAttributesAggregate::Get());
		default: UE_MIR_UNREACHABLE();
	}
}

void UMaterialExpressionCustom::Build(MIR::FEmitter& Em)
{
	// Convert the description to a valid HLSL identifier
	FString Name = DescriptionToIdentifier(Description);

	MIR::TTemporaryArray<MIR::FValueRef> InputArgs { Inputs.Num() };

	// Prepare a description of the user-defined HLSL function for the emitter.
	MIR::FFunctionHLSLDesc FuncDesc;
	FuncDesc.Name = Name;
	FuncDesc.ReturnType = CustomMaterialOutputTypeToMIR(OutputType);
	
	// Fixup the scene texture identifiers in the source string
	TArray<int8> SceneTextureInfo;
	FString FixedCode = UE::MaterialTranslatorUtils::CustomExpressionSceneTextureInputFixup(this, *Code, SceneTextureInfo);
	FuncDesc.Code = FixedCode;
	if (FuncDesc.Code.IsEmpty())
	{
		FuncDesc.Code = Code;
	}

	// Turn each expression input into an input-only parameter.
	for (int32 i = 0; i < Inputs.Num(); ++i)
	{
		FCustomInput const& Input = Inputs[i];

		if (Input.InputName.IsNone())
		{
			// Ignore this input parameters with "None" name.
			continue;
		}
		
		// Read the input argument
		InputArgs[FuncDesc.NumInputOnlyParams] = Em.Input(&Input.Input);

		// Is this argument an unused scene texture sample?
		if (SceneTextureInfo.IsValidIndex(i) && SceneTextureInfo[i] == -1)
		{
			// If this parameter samples an unused scene texture, skip the parameter, but still
			// make sure the scene-texture sample is analyzed.
			InputArgs[FuncDesc.NumInputOnlyParams] = Em.Nop(InputArgs[FuncDesc.NumInputOnlyParams]);
		}
		
		// Declare an input-only parameter
		if (!FuncDesc.PushInputOnlyParameter(Input.InputName, InputArgs[FuncDesc.NumInputOnlyParams]->Type))
		{
			Em.Errorf(TEXT("Too many inputs. Custom expressions can have at most %d input/output pins."), MIR::MaxNumFunctionParameters);
			return;
		}
	}

	// Some Input() call might have generated an error
	UE_MIR_CHECKPOINT(Em);

	// Turn each expression additional output into a output-only parameter.
	for (FCustomOutput const& AdditionalOutput : AdditionalOutputs)
	{
		// Ignore output parameters with "None" name.
		if (AdditionalOutput.OutputName.IsNone())
		{
			continue;
		}

		if (!FuncDesc.PushOutputOnlyParameter(AdditionalOutput.OutputName, CustomMaterialOutputTypeToMIR(AdditionalOutput.OutputType)))
		{
			Em.Errorf(TEXT("Too many input/outputs. Custom expressions can have at most %d input/output pins."), MIR::MaxNumFunctionParameters);
			return;
		}
	}

	// Generate the array of additional defines
	MIR::TTemporaryArray<MIR::FFunctionHLSLDefine> Defines{ AdditionalDefines.Num() };
	for (int32 i = 0; i < AdditionalDefines.Num(); ++i)
	{
		if (AdditionalDefines[i].DefineName.IsEmpty())
		{
			Em.Errorf(TEXT("Define with index '%d' has no valid name."), i);
		}

		if (AdditionalDefines[i].DefineValue.IsEmpty())
		{
			Em.Errorf(TEXT("Define with index '%d' has no valid value."), i);
		}

		Defines[i] = { AdditionalDefines[i].DefineName, AdditionalDefines[i].DefineValue };
	}
	FuncDesc.Defines = Defines;

	// Generate the array of additional includes
	MIR::TTemporaryArray<FStringView> Includes{ IncludeFilePaths.Num() };
	for (int32 i = 0; i < IncludeFilePaths.Num(); ++i)
	{
		if (IncludeFilePaths[i].IsEmpty())
		{
			Em.Errorf(TEXT("Include with index '%d' has no valid value."), i);
		}

		Includes[i] = IncludeFilePaths[i];
	}

	FuncDesc.Includes = Includes;

	UE_MIR_CHECKPOINT(Em); // Make sure checks above did not fail

	// Declare the HLSL function with the description we generated
	MIR::FFunction* Func = Em.FunctionHLSL(FuncDesc);
	
	UE_MIR_CHECKPOINT(Em); // To guarantee a function was succesfully emitted.

	FValueRef Call = Em.Call(Func, { InputArgs.GetData(), (int32)FuncDesc.NumInputOnlyParams });

	// Output the call return value through the first output pin
	Em.Output(0, Call);

	// Output the additional outputs through subsequent output pins
	for (uint32 i = 0; i < Func->GetNumOutputParameters(); ++i)
	{
		Em.Output(i + 1, Em.CallParameterOutput(Call, i));
	}
}

void UMaterialExpressionBounds::Build(MIR::FEmitter& Em)
{
	// Select between 3 different sets of 4 outputs (half, full, min, max), depending on bounds type.  Check that enum matches order in BaseMaterialExpressions.ini.
	static_assert(MEILB_InstanceLocal == 0);
	static_assert(MEILB_ObjectLocal == 1);
	static_assert(MEILB_PreSkinnedLocal == 2);

	int32 OutputOffset = Type * 4;
	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, OutputOffset + 0));
	Em.Output(1, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, OutputOffset + 1));
	Em.Output(2, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, OutputOffset + 2));
	Em.Output(3, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, OutputOffset + 3));
}

void UMaterialExpressionBumpOffset::Build(MIR::FEmitter& Em)
{
	FValueRef HeightRatioArg = Em.Cast(Em.InputDefaultFloat(&HeightRatioInput, HeightRatio), MIR::FType::MakeFloatScalar());

	FValueRef TexCoordArg = Em.TryInput(&Coordinate);
	if (!TexCoordArg)
	{
		TexCoordArg = MaterialToMIR::EmitTexCoord(Em, ConstCoordinate);
	}
	
	Em.Output(0,
		Em.Add(
			Em.Multiply(
				Em.Swizzle(EmitTransformVector(Em, Em.Extern<MIR::FExternFromMaterialDecl>({ NAME_CameraVector }), MCB_World, MCB_Tangent, false, nullptr, nullptr), MIR::FSwizzleMask(MIR::EVectorComponent::X, MIR::EVectorComponent::Y)),
				Em.Add(
					Em.Multiply(
						HeightRatioArg,
						Em.Cast(Em.Input(&Height), MIR::FType::MakeFloatScalar())
					),
					Em.Multiply(Em.ConstantFloat(-ReferencePlane), HeightRatioArg)
				)
			),
			TexCoordArg
		)
	);
}

void UMaterialExpressionDynamicParameter::Build(MIR::FEmitter& Em)
{
	struct FDynamicParameter
	{
		FVector4f Default;
		uint32 ParameterIndex;
			
		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXTVIEW("DynamicParameter"),
				.Type = MIR::FType::MakeFloatVector(4),
				.Flags = MIR::EExternFlags::Inline,
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			const uint32 DynamicParametersMask = 1 << ParameterIndex;
			Context.Module->GetStatistics().DynamicParticleParameterMask |= DynamicParametersMask;
			Context.Module->AddIntegerEnvironmentDefine(TEXT("USE_DYNAMIC_PARAMETERS"), 1);
			Context.Module->AddIntegerEnvironmentDefine(TEXT("DYNAMIC_PARAMETERS_MASK"), DynamicParametersMask);
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			if (Printer.Differential != MIR::EExternDifferential::None)
			{
				Printer << TEXT("0");
				return;
			}
			Printer << TEXTVIEW("GetDynamicParameter(Parameters.Particle, MaterialFloat4(")
				    << Default.X << TEXTVIEW(", ") << Default.Y << TEXTVIEW(", ") << Default.Z << TEXTVIEW(", ") << Default.W << TEXTVIEW("), ")
				    << ParameterIndex << TEXTVIEW(")");
		}

		void EmitDebugInfo(FString& Out) const
		{
			Out.Appendf(TEXT("Default=(%f, %f, %f, %f) ParameterIndex=%d"), Default.X, Default.Y, Default.Z, Default.W, ParameterIndex);
		}

		void CopyTo(FDynamicParameter& Other) const
		{
			Other.Default = Default;
			Other.ParameterIndex = ParameterIndex;
		}
	};

	FValueRef Result = Em.Extern<FDynamicParameter>({
		FVector4f(DefaultValue.R, DefaultValue.G, DefaultValue.B, DefaultValue.A),
		ParameterIndex
	});

	Em.Output(0, Em.Subscript(Result, 0));
	Em.Output(1, Em.Subscript(Result, 1));
	Em.Output(2, Em.Subscript(Result, 2));
	Em.Output(3, Em.Subscript(Result, 3));
	Em.Output(4, Em.Swizzle(Result, MIR::FSwizzleMask::XYZ()));		// RGB
	Em.Output(5, Result);											// RGBA
}

void UMaterialExpressionNoise::Build(MIR::FEmitter& Em)
{
	FValueRef PositionInput = EmitWorldPositionOrDefault(Em, Em.TryInput(&Position), WorldPositionOriginType);

	if (WorldPositionOriginType == EPositionOrigin::CameraRelative)
	{
		// LWC_TODO: add support for translated world positions in the corresponding HLSL function
		PositionInput = EmitTransformVector(Em, PositionInput, MCB_TranslatedWorld, MCB_World, true, {}, {});
	}

	FValueRef FilterWidthInput = Em.InputDefaultFloat(&FilterWidth, 0.0f);
	FValueRef ScaleValue = Em.ConstantFloat(Scale);
	FValueRef QualityValue = Em.ConstantInt(Quality);
	FValueRef NoiseFunctionValue = Em.ConstantInt(NoiseFunction);
	FValueRef TurbulenceValue = Em.ConstantBool(bTurbulence);
	FValueRef LevelsValue = Em.ConstantInt(FMath::Clamp(Levels, 1, 10));		// to limit performance problems due to values outside reasonable range
	FValueRef OutputMinValue = Em.ConstantFloat(OutputMin);
	FValueRef OutputMaxValue = Em.ConstantFloat(OutputMax);
	FValueRef LevelScaleValue = Em.ConstantFloat(LevelScale);
	FValueRef TilingValue = Em.ConstantBool(bTiling);
	FValueRef RepeatSizeValue = Em.ConstantFloat(RepeatSize);

	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>(
		{ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionNoise($0,$1,$2,$3,$4,$5,$6,$7,$8,$9,$10,$11)") },
		PositionInput,
		ScaleValue,
		QualityValue,
		NoiseFunctionValue,
		TurbulenceValue,
		LevelsValue,
		OutputMinValue,
		OutputMaxValue,
		LevelScaleValue,
		FilterWidthInput,
		TilingValue,
		RepeatSizeValue
	));
}

void UMaterialExpressionVectorNoise::Build(MIR::FEmitter& Em)
{
	FValueRef PositionInput = EmitWorldPositionOrDefault(Em, Em.TryInput(&Position), WorldPositionOriginType);

	if (WorldPositionOriginType == EPositionOrigin::CameraRelative)
	{
		// LWC_TODO: add support for translated world positions in the corresponding HLSL function
		PositionInput = EmitTransformVector(Em, PositionInput, MCB_TranslatedWorld, MCB_World, true, {}, {});
	}

	// LWC_TODO - maybe possible/useful to add LWC-aware noise functions
	PositionInput = Em.Cast(PositionInput, MIR::FType::MakeFloatVector(3));

	FValueRef QualityValue = Em.ConstantInt(Quality);
	FValueRef NoiseFunctionValue = Em.ConstantInt(NoiseFunction);
	FValueRef TilingValue = Em.ConstantBool(bTiling);
	FValueRef TileSizeValue = Em.ConstantFloat(TileSize);

	FValueRef NoiseResult = Em.Extern<MIR::FExternSimpleHLSL>(
		{ MIR::EExternSimpleType::Float4, TEXT("MaterialExpressionVectorNoise($0,$1,$2,$3,$4)") },
		PositionInput,
		QualityValue,
		NoiseFunctionValue,
		TilingValue,
		TileSizeValue
	);

	// Function returns float4, but only certain noise functions fill in all four elements, so downcast to float3 if not those cases.
	if (NoiseFunction != VNF_GradientALU && NoiseFunction != VNF_VoronoiALU)
	{
		NoiseResult = Em.Cast(NoiseResult, MIR::FType::MakeFloatVector(3));
	}

	Em.Output(0, NoiseResult);
}

void UMaterialExpressionPanner::Build(MIR::FEmitter& Em)
{
	FValueRef TimeArg = Em.TryInput(&Time);
	if (TimeArg)
	{
		TimeArg = Em.Cast(TimeArg, MIR::FType::MakeFloatScalar());
	}
	else
	{
		TimeArg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("View.<PREVFRAME>GameTime") });
	}
	
	FValueRef SpeedVectorArg = Em.InputDefaultFloat2(&Speed, FVector2f(SpeedX, SpeedY));

	// TODO:  When preshaders get implemented, the original translator generates a unique "PeriodicHint" preshader op for this expression,
	//        which attempts to do math at higher precision to avoid accuracy issues as GameTime increases.  We'll want to add that logic here,
	//        or consider making preshader math involving game time automatically run at high precision across the board (naturally solving
	//        precision issues even outside this specific expression).

	SpeedVectorArg = Em.Multiply(TimeArg, SpeedVectorArg);
	if (bFractionalPart)
	{
		SpeedVectorArg = Em.Frac(SpeedVectorArg);
	}

	FValueRef TexCoordArg = Em.TryInput(&Coordinate);
	if (!TexCoordArg)
	{
		TexCoordArg = MaterialToMIR::EmitTexCoord(Em, 0);
	}

	Em.Output(0, Em.Add(SpeedVectorArg, TexCoordArg));
}

void UMaterialExpressionParticlePositionWS::Build(MIR::FEmitter& Em)
{
	int32 ExternalCodeIndex = OriginType == EPositionOrigin::Absolute ? 0 : 1;
	MIR::FType ResultType = OriginType == EPositionOrigin::Absolute ? MIR::FType::MakeDoubleVector(3) : MIR::FType::MakeFloatVector(3);

	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, ExternalCodeIndex));
}

void UMaterialExpressionPerInstanceCustomData::Build(MIR::FEmitter& Em)
{
	FValueRef DataIndexArgument = Em.ConstantInt(DataIndex);
	FValueRef DefaultArgument = Em.InputDefaultFloat(&DefaultValue, ConstDefaultValue);

	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, DataIndexArgument, DefaultArgument));
}

void UMaterialExpressionPerInstanceCustomData3Vector::Build(MIR::FEmitter& Em)
{
	FValueRef DataIndexArgument = Em.ConstantInt(DataIndex);
	FValueRef DefaultArgument = Em.InputDefaultFloat3(&DefaultValue, FVector3f(ConstDefaultValue.R, ConstDefaultValue.G, ConstDefaultValue.B));

	Em.Output(0, MaterialExpressionExternalCodeBase_EmitExtern(Em, *this, 0, DataIndexArgument, DefaultArgument));
}

void UMaterialExpressionPreviousFrameSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef CurrentFrameValue = Em.Input(&CurrentFrame);
	FValueRef PreviousFrameValue = Em.Input(&PreviousFrame);
	UE_MIR_CHECKPOINT(Em);

	struct FPreviousFrame
	{
		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name  = TEXTVIEW("PreviousFrameExtern"),
				.Type  = MIR::FType::MakeBoolScalar(),
				.Flags = MIR::EExternFlags::Inline,
			};
		}
	
		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << (Printer.bIsPreviousFrame ? TEXTVIEW("true") : TEXTVIEW("false"));
		}
	};

	Em.Output(0, Em.Branch(Em.Extern<FPreviousFrame>({}), PreviousFrameValue, CurrentFrameValue));
}

void UMaterialExpressionHairAttributes::Build(MIR::FEmitter& Em)
{
	{
		FValueRef HairUV = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("MaterialExpressionGetHairUV(Parameters)") });
		Em.Output(0, Em.Subscript(HairUV, 0));
		Em.Output(1, Em.Subscript(HairUV, 1));
	}

	{
		FValueRef HairDimensions = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("MaterialExpressionGetHairDimensions(Parameters)") });
		Em.Output(2, Em.Subscript(HairDimensions, 0));		// Length
		Em.Output(3, Em.Subscript(HairDimensions, 1));		// Radius
	}

	Em.Output(4, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionGetHairSeed(Parameters)") }));
	Em.Output(5, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("MaterialExpressionGetHairTangent(Parameters, $0)") }, Em.ConstantBool(bUseTangentSpace)));
	Em.Output(6, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("MaterialExpressionGetHairRootUV(Parameters)") }));
	Em.Output(7, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("MaterialExpressionGetHairBaseColor(Parameters)") }));
	Em.Output(8, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionGetHairRoughness(Parameters)") }));
	Em.Output(9, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionGetHairDepth(Parameters)") }));
	Em.Output(10, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionGetHairCoverage(Parameters)") }));
	Em.Output(11, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("MaterialExpressionGetHairAuxilaryData(Parameters)") }));
	Em.Output(12, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("MaterialExpressionGetAtlasUVs(Parameters)") }));
	Em.Output(13, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionGetHairGroupIndex(Parameters)") }));
	Em.Output(14, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("MaterialExpressionGetHairAO(Parameters)") }));
	Em.Output(15, Em.Subscript(Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("MaterialExpressionGetHairClumpID(Parameters)") }), 0));
}

void UMaterialExpressionHairColor::Build(MIR::FEmitter& Em)
{
	FValueRef MelaninInput = Em.InputDefaultFloat(&Melanin, 0.5f);
	FValueRef RednessInput = Em.InputDefaultFloat(&Redness, 0.0f);
	FValueRef DyeColorInput = Em.InputDefaultFloat3(&DyeColor, FVector3f(1.f, 1.f, 1.f));

	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("MaterialExpressionGetHairColorFromMelanin($0, $1, $2)") }, MelaninInput, RednessInput, DyeColorInput));
}

void UMaterialExpressionMapARPassthroughCameraUV::Build(MIR::FEmitter& Em)
{
	FValueRef UV = Em.Input(&Coordinates);
	UE_MIR_CHECKPOINT(Em);

	FValueRef UVPair0 = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("ResolvedView.XRPassthroughCameraUVs[0]") });
	FValueRef UVPair1 = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float4, TEXT("ResolvedView.XRPassthroughCameraUVs[1]") });

	FValueRef ULerp = Em.Lerp(UVPair0, UVPair1, Em.Subscript(UV, 0));
	Em.Output(0, Em.Lerp(Em.Swizzle(ULerp, MIR::FSwizzleMask(MIR::EVectorComponent::X, MIR::EVectorComponent::Y)), Em.Swizzle(ULerp, MIR::FSwizzleMask(MIR::EVectorComponent::Z, MIR::EVectorComponent::W)), Em.Subscript(UV, 1)));
}

void UMaterialExpressionSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef CompiledDefault = Em.InputDefaultFloat(&Default, ConstDefault);

	// If no other inputs, just return the default
	if (Inputs.Num() == 0)
	{
		Em.Output(0, CompiledDefault);
		return;
	}

	// Only the "x" component of the switch value is used.
	FValueRef CompiledSwitchValue = Em.InputDefaultFloat(&SwitchValue, ConstSwitchValue);
	if (CompiledSwitchValue->Type.IsVector())
	{
		CompiledSwitchValue = Em.Subscript(CompiledSwitchValue, 0);
	}

	// Compile the inputs.
	TArray<FValueRef> CompiledInputs;
	CompiledInputs.SetNumUninitialized(Inputs.Num());

	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		CompiledInputs[i] = Em.Input(&Inputs[i].Input);
	}
	UE_MIR_CHECKPOINT(Em);		// Make sure inputs are connected.

	// Get common type of inputs.  Done as a separate loop, to avoid spurious errors for unconnected inputs, which otherwise also produce "No common type" errors.
	MIR::FType CommonType = CompiledDefault->Type;
	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		CommonType = Em.GetCommonType(CommonType, CompiledInputs[i]->Type);
	}
	UE_MIR_CHECKPOINT(Em);		// Make sure inputs have a valid common type.

	// If the switch value is a constant, we can directly pass the corresponding input as the result.
	const MIR::FConstant* CompiledSwitchValueConstant = CompiledSwitchValue->As<MIR::FConstant>();
	if (CompiledSwitchValueConstant)
	{
		int32 InputIndex = 0;
		switch (CompiledSwitchValue->Type.AsPrimitive()->ScalarKind)
		{
		case MIR::EScalarKind::Boolean:	InputIndex = CompiledSwitchValueConstant->Boolean ? 1 : 0;  break;
		case MIR::EScalarKind::Integer:		InputIndex = CompiledSwitchValueConstant->Integer;  break;
		case MIR::EScalarKind::Float:	InputIndex = FMath::FloorToInt(CompiledSwitchValueConstant->Float);  break;
		case MIR::EScalarKind::Double:	InputIndex = (int32)FMath::FloorToInt(CompiledSwitchValueConstant->Double);  break;
		default:  UE_MIR_UNREACHABLE();
		}

		if (Inputs.IsValidIndex(InputIndex))
		{
			Em.Output(0, Em.Cast(CompiledInputs[InputIndex], CommonType));
		}
		else
		{
			Em.Output(0, Em.Cast(CompiledDefault, CommonType));
		}
		return;
	}

	// Floor the switch value if it's a float, to prepare for comparisons.
	if (CompiledSwitchValue->Type.IsAnyFloat())
	{
		CompiledSwitchValue = Em.Floor(CompiledSwitchValue);
	}

	// Generate a switch statement as a chain of if..else branches.  We scan backwards, so the comparisons end up in order,
	// factoring in that each Branch is a parent of the previous Branch, and so the last Branch added is the first that gets
	// executed.  The first previous Branch (final else case) starts out as the default.
	FValueRef PreviousBranch = Em.Cast(CompiledDefault, CommonType);

	for (int32 i = Inputs.Num() - 1; i >= 0; i--)
	{
		PreviousBranch = Em.Branch(Em.Equals(CompiledSwitchValue, Em.ConstantInt(i)), Em.Cast(CompiledInputs[i], CommonType), PreviousBranch);
	}

	Em.Output(0, PreviousBranch);
}

void UMaterialExpressionTangentOutput::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloat(Em.Input(&Input), 3);
	Em.SetCustomOutputs(TEXTVIEW("TangentOutput"), { &InputValue, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionBentNormalCustomOutput::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloat(Em.Input(&Input), 3);
	Em.SetCustomOutputs(TEXTVIEW("BentNormal"), { &InputValue, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionClearCoatNormalCustomOutput::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloat(Em.Input(&Input), 3);
	Em.SetCustomOutputs(TEXTVIEW("ClearCoatBottomNormal"), { &InputValue, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionFirstPersonOutput::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloat(Em.InputDefaultFloat(&FirstPersonInterpolationAlpha, ConstFirstPersonInterpolationAlpha), 1);
	Em.SetCustomOutputs(TEXTVIEW("FirstPersonOutput"), { &InputValue, 1 }, MIR::EMaterialOutputFrequency::PerVertex, [] (FMaterialIRModule& Module)
	{
		Module.GetCompilationOutput().bModifiesMeshPosition |= true;
	});
}

void UMaterialExpressionMotionVectorWorldOffsetOutput::Build(MIR::FEmitter& Em)
{
	FValueRef InputValue = Em.CastToFloat(Em.Input(&Input), 3);
	Em.SetCustomOutputs(TEXTVIEW("MotionVectorWorldOffsetOutput"), { &InputValue, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionSingleLayerWaterMaterialOutput::Build(MIR::FEmitter& Em)
{
	FValueRef ScatteringCoefficientsValue = Em.TryInput(&ScatteringCoefficients);
	FValueRef AbsorptionCoefficientsValue = Em.TryInput(&AbsorptionCoefficients);
	FValueRef PhaseGValue = Em.TryInput(&PhaseG);

	if (!ScatteringCoefficientsValue && !AbsorptionCoefficientsValue && !PhaseGValue && !Substrate::IsSubstrateEnabled())
	{
		Em.Error(TEXTVIEW("No inputs to Single Layer Water Material."));
		return;
	}

	// Generates function names GetSingleLayerWaterMaterialOutput{index} used in BasePixelShader.usf.
	FValueRef OutputValues[] = 
	{
		ScatteringCoefficientsValue ? Em.CastToFloat(ScatteringCoefficientsValue, 3) : Em.ConstantFloat3({ 0.f, 0.f, 0.f }),
		AbsorptionCoefficientsValue ? Em.CastToFloat(AbsorptionCoefficientsValue, 3) : Em.ConstantFloat3({ 0.f, 0.f, 0.f }),
		PhaseGValue ? Em.CastToFloat(PhaseGValue, 1) : Em.ConstantFloat(0.f),
		Em.CastToFloat(Em.InputDefaultFloat(&ColorScaleBehindWater, 1.f), 1),
	};
	Em.SetCustomOutputs(TEXTVIEW("SingleLayerWaterMaterialOutput"), OutputValues, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionSubsurfaceMediumMaterialOutput::Build(MIR::FEmitter& Em)
{
	FValueRef OutputValues[] = 
	{
		Em.CastToFloat(Em.Input(&MeanFreePath), 3),
		Em.CastToFloat(Em.Input(&ScatteringDistribution), 1),
	};
	Em.SetCustomOutputs(TEXTVIEW("SubsurfaceMediumMaterialOutput"), OutputValues, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionThinTranslucentMaterialOutput::Build(MIR::FEmitter& Em)
{
	FValueRef OutputValues[] =
	{
		Em.CastToFloat(Em.InputDefaultFloat3(&TransmittanceColor, {0.5f, 0.5f, 0.5f}), 3),
		Em.CastToFloat(Em.InputDefaultFloat(&SurfaceCoverage, 1.0f), 1),
	};
	Em.SetCustomOutputs(TEXTVIEW("ThinTranslucentMaterialOutput"), OutputValues, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionVolumetricAdvancedMaterialInput::Build(MIR::FEmitter& Em)
{
	FValueRef VolumeSampleConservativeDensity = Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("GetVolumeSampleConservativeDensity") });
	Em.Output(0, Em.CastToFloat(VolumeSampleConservativeDensity, 3));
	Em.Output(1, VolumeSampleConservativeDensity);
}

void UMaterialExpressionVolumetricAdvancedMaterialOutput::Build(MIR::FEmitter& Em)
{
	FValueRef OutputValues[] =
	{
		Em.CastToFloat(Em.InputDefaultFloat(&PhaseG, ConstPhaseG), 1),
		Em.CastToFloat(Em.InputDefaultFloat(&PhaseG2, ConstPhaseG2), 1),
		Em.CastToFloat(Em.InputDefaultFloat(&PhaseBlend, ConstPhaseBlend), 1),
		Em.CastToFloat(Em.InputDefaultFloat(&MultiScatteringContribution, ConstMultiScatteringContribution), 1),
		Em.CastToFloat(Em.InputDefaultFloat(&MultiScatteringOcclusion, ConstMultiScatteringOcclusion), 1),
		Em.CastToFloat(Em.InputDefaultFloat(&MultiScatteringEccentricity, ConstMultiScatteringEccentricity), 1),
		Em.CastToFloat(Em.InputDefaultFloat4(&ConservativeDensity, FVector4f{ 1.0f, 1.0f, 1.0f, 1.0f }), 4),
	};
	Em.SetCustomOutputs(TEXTVIEW("VolumetricAdvancedMaterialOutput"), OutputValues, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionAbsorptionMediumMaterialOutput::Build(MIR::FEmitter& Em)
{
	FValueRef OutputValues[] =
	{
		Em.CastToFloat(Em.InputDefaultFloat3(&TransmittanceColor, { 1.0f, 1.0f, 1.0f }), 3),
	};
	Em.SetCustomOutputs(TEXTVIEW("AbsorptionMediumMaterialOutput"), OutputValues, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionDecalDerivative::Build(MIR::FEmitter& Em)
{
	Em.CheckMaterialDomainIsAnyOf({ MD_DeferredDecal });

	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("ComputeDecalDDX(Parameters)") }));
	Em.Output(1, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("ComputeDecalDDY(Parameters)") }));
}

void UMaterialExpressionDecalMipmapLevel::Build(MIR::FEmitter& Em)
{
	Em.CheckMaterialDomainIsAnyOf({ MD_DeferredDecal });

	FValueRef TextureSizeValue = Em.CastToFloat(Em.InputDefaultFloat2(&TextureSize, { ConstWidth, ConstHeight }), 2);
	BuildMaterialExpressionExternalCodeBase(Em, *this, TextureSizeValue);
}

void UMaterialExpressionDepthOfFieldFunction::Build(MIR::FEmitter& Em)
{
	FValueRef DepthValue = Em.CastToFloat(Em.TryInput(&Depth), 1);
	if (!DepthValue)
	{
		DepthValue = Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("PixelDepth") });
	}

	UE_MIR_CHECKPOINT(Em);
	BuildMaterialExpressionExternalCodeBase(Em, *this, DepthValue, Em.ConstantInt((MIR::TInteger)FunctionValue));
}

void UMaterialExpressionDistanceCullFade::Build(MIR::FEmitter& Em)
{
	struct FDistanceCullFade
	{
		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXTVIEW("DistanceCullFade"),
				.Type = MIR::FType::MakeFloatScalar(),
				.Flags = MIR::EExternFlags::Inline,
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			Context.Module->GetCompilationOutput().bUsesDistanceCullFade = true;
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << TEXTVIEW("GetDistanceCullFade()");
		}
	};

	Em.Output(0, Em.Extern<FDistanceCullFade>({}));
}

void UMaterialExpressionGIReplace::Build(MIR::FEmitter& Em)
{
	FValueRef DefaultValue = Em.Input(&Default);
	UE_MIR_CHECKPOINT(Em);

	// Note: StaticIndirect is unused.
	FValueRef LocalDynamicIndirectValue = Em.InputDefault(&DynamicIndirect, DefaultValue);
	FValueRef GIReplaceStateValue = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Bool1, TEXT("GetGIReplaceState()") });

	Em.Output(0, Em.Branch(GIReplaceStateValue, LocalDynamicIndirectValue, DefaultValue));
}

void UMaterialExpressionHsvToRgb::Build(MIR::FEmitter& Em)
{
    FValueRef InputValue = Em.CastToFloatKind(Em.Input(&Input));
    UE_MIR_CHECKPOINT(Em);

    MIR::FPrimitive Primitive = *InputValue->Type.AsPrimitive();
    if ((!Primitive.IsRowVector() && !Primitive.IsColumnVector()) || Primitive.NumComponents() < 3)
    {
        Em.Error(InputValue, TEXT("Expected a 3D or 4D vector."));
        return;
    }

    FValueRef H = Em.Subscript(InputValue, 0);
    FValueRef S = Em.Subscript(InputValue, 1);
    FValueRef V = Em.Subscript(InputValue, 2);

    FValueRef One   = Em.ConstantFloat(1.0f);
    FValueRef Two   = Em.ConstantFloat(2.0f);
    FValueRef Three = Em.ConstantFloat(3.0f);
    FValueRef Four  = Em.ConstantFloat(4.0f);
    FValueRef Six   = Em.ConstantFloat(6.0f);

    FValueRef HTimesSix = Em.Multiply(H, Six);

    // R = abs(H * 6 - 3) - 1;
    FValueRef R = Em.Subtract(Em.Abs(Em.Subtract(HTimesSix, Three)), One);

    // G = 2 - abs(H * 6 - 2);
    FValueRef G = Em.Subtract(Two, Em.Abs(Em.Subtract(HTimesSix, Two)));

    // B = 2 - abs(H * 6 - 4);
    FValueRef B = Em.Subtract(Two, Em.Abs(Em.Subtract(HTimesSix, Four)));

    // RGB = saturate(float3(R, G, B));
    FValueRef RGB = Em.Saturate(Em.Vector3(R, G, B));

    // RGB = ((RGB - 1) * S + 1) * V;
    RGB = Em.Multiply(
        Em.Add(Em.Multiply(Em.Subtract(RGB, One), S), One),
        V
    );

    if (Primitive.NumComponents() == 4)
    {
        RGB = Em.Vector4(RGB, Em.Subscript(InputValue, 3));
    }

    if (Primitive.IsColumnVector())
    {
        RGB = Em.Transpose(RGB);
    }

    Em.Output(0, RGB);
}

void UMaterialExpressionMaterialProxyReplace::Build(MIR::FEmitter& Em)
{
	// New translator does not have the concept of "proxy". Simply pass through the realtime input.
	Em.Output(0, Em.Input(&Realtime));
}

void UMaterialExpressionPathTracingBufferTexture::Build(MIR::FEmitter& Em)
{
	Em.CheckMaterialDomainIsAnyOf({ MD_PostProcess });
	UE_MIR_CHECKPOINT(Em);

	FValueRef CoordinatesValue = Em.TryInput(&Coordinates);
	if (!CoordinatesValue)
	{
		CoordinatesValue = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float2, TEXT("GetDefaultPathTracingBufferTextureUV(Parameters, 0)") });
	}
	CoordinatesValue = Em.CastToFloat(CoordinatesValue, 2);

	FValueRef PathTracingBufferTextureIndexValue = Em.ConstantInt(PathTracingBufferTextureId);

	struct FPathTracingBufferTextureExtern
	{
		EPathTracingBufferTextureId PathTracingBufferTextureId;

		MIR::FExternInfo GetInfo() const
		{
			return
				{
					.Name = TEXT("PathTracingBufferTexture"),
					.Type  = MIR::FType::MakeFloatVector(4),
					.Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials, 
				};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			Context.Module->GetCompilationOutput().SetIsPathTracingBufferTextureUsed((int32)PathTracingBufferTextureId);
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << TEXT("MaterialExpressionPathTracingBufferTextureLookup(Parameters, $0, $1)");
		}
	};

	Em.Output(0, Em.Extern<FPathTracingBufferTextureExtern>({ PathTracingBufferTextureId }, CoordinatesValue, PathTracingBufferTextureIndexValue));
}

void UMaterialExpressionPostVolumeUserFlagTest::Build(MIR::FEmitter& Em)
{
	FValueRef BitIndexValue = Em.TryInput(&BitIndex); 
	if (!BitIndexValue)
	{
		if (ConstBitIndex < 0)
		{
			Em.Error(TEXTVIEW("Invalid constant bit index provided."));
			return;
		}
		BitIndexValue = Em.ConstantInt(ConstBitIndex);
	}

	Em.Output(0, Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("PostVolumeUserFlagTest($0)") }, BitIndexValue));
}

void UMaterialExpressionPreSkinnedLocalBounds::Build(MIR::FEmitter& Em)
{
	Em.CheckMaterialDomainIsAnyOf( { MD_Surface, MD_Volume });
	UE_MIR_CHECKPOINT(Em);

	Em.Output(0, Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("PreSkinnedLocalBoundsHalf") })); // Half extents
	Em.Output(1, Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("PreSkinnedLocalBoundsFull") })); // Full extents
	Em.Output(2, Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("PreSkinnedLocalBoundsMin") }));  // Min point
	Em.Output(3, Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("PreSkinnedLocalBoundsMax") }));  // Max point
}

static void EmitRayOrPathTracingQualitySwitch(MIR::FEmitter& Em, const TCHAR* ConditionHLSL, FValueRef NormalValue, FValueRef TracedValue)
{
	if (!NormalValue && !TracedValue)
	{
		Em.Error(TEXT("Neither Normal nor PathTraced inputs are provided."));
		return;
	}

	FValueRef OutputValue;
	if (!NormalValue)
	{
		OutputValue = TracedValue;
	}
	else if (!TracedValue)
	{
		OutputValue = NormalValue;
	}
	else
	{
		FValueRef ConditionValue = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Bool1, ConditionHLSL });
		OutputValue = Em.Branch(ConditionValue, TracedValue, NormalValue);
	}

	Em.Output(0, OutputValue);
}

void UMaterialExpressionRayTracingQualitySwitch::Build(MIR::FEmitter& Em)
{
	FValueRef NormalValue = Em.TryInput(&Normal);
	FValueRef RayTracedValue = FDataDrivenShaderPlatformInfo::GetSupportsRayTracing(Em.GetShaderPlatform()) ? Em.TryInput(&RayTraced) : FValueRef{};
	EmitRayOrPathTracingQualitySwitch(Em, TEXT("GetRayTracingQualitySwitch()"), NormalValue, RayTracedValue);
}

void UMaterialExpressionPathTracingQualitySwitch::Build(MIR::FEmitter& Em)
{
	FValueRef NormalValue = Em.TryInput(&Normal);
	FValueRef PathTracedValue = Em.TryInput(&PathTraced);
	EmitRayOrPathTracingQualitySwitch(Em, TEXT("GetPathTracingQualitySwitch()"), NormalValue, PathTracedValue);
}

void UMaterialExpressionPathTracingRayTypeSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef MainValue = Em.Input(&Main);
	UE_MIR_CHECKPOINT(Em);

	// compile all arguments (its ok if some of these are not connected, the will default to using Main)
	FValueRef ShadowValue = Em.TryInput(&Shadow);
	FValueRef DiffuseValue = Em.TryInput(&IndirectDiffuse);
	FValueRef SpecularValue = Em.TryInput(&IndirectSpecular);
	FValueRef VolumeValue = Em.TryInput(&IndirectVolume);

	// return Compiler->PathTracingRayTypeSwitch(ArgMain, ArgShadow, ArgDiffuse, ArgSpecular, ArgVolume);
		
	FValueRef OutputValue;
	OutputValue = Em.Branch(Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Bool1, TEXT("GetPathTracingIsShadow()") }), ShadowValue ? ShadowValue : MainValue, MainValue);
	OutputValue = Em.Branch(Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Bool1, TEXT("GetPathTracingIsIndirectDiffuse()") }), DiffuseValue ? DiffuseValue : MainValue, OutputValue);
	OutputValue = Em.Branch(Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Bool1, TEXT("GetPathTracingIsIndirectSpecular()") }), SpecularValue ? SpecularValue : MainValue, OutputValue);
	OutputValue = Em.Branch(Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Bool1, TEXT("GetPathTracingIsIndirectVolume()") }), VolumeValue ? VolumeValue : MainValue, OutputValue);
	
	Em.Output(0, OutputValue);
}

void UMaterialExpressionBindlessSwitch::Build(MIR::FEmitter& Em)
{
	FValueRef OutputValue;
	switch (FShaderPlatformConfig::GetBindlessConfiguration(Em.GetShaderPlatform()))
	{
		case ERHIBindlessConfiguration::Disabled:
			OutputValue = Em.Input(&Default);
			break;
		case ERHIBindlessConfiguration::Minimal: 
		case ERHIBindlessConfiguration::All:
			OutputValue = Em.Input(&Bindless);
			break;
		case ERHIBindlessConfiguration::RayTracing:
			OutputValue = Em.Input(&Bindless); // @massimo.tristano todo
			break;
	}
	Em.Output(0, OutputValue);
}

static FValueRef EmitProcessPhysicsField(MIR::FEmitter& Em, FExpressionInput* PositionInput, EPositionOrigin PositionOriginType, EFieldOutputType OutputType, const int32 TargetIndex, const bool bSampleField)
{
	FValueRef PositionValue = Em.TryInput(PositionInput);
	if (!PositionValue)
	{
		PositionValue = EmitWorldPosition(Em, UE::MaterialTranslatorUtils::GetWorldPositionTypeWithOrigin(PositionOriginType));
	}

	if (Em.GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
	{
		// TODO: add physics field sampling to mobile
		if (OutputType == Field_Output_Vector)
		{
			return Em.ConstantFloat3({ 0.0f, 0.0f, 0.0f });
		}
		else if (OutputType == Field_Output_Scalar || OutputType == Field_Output_Integer)
		{
			return Em.ConstantFloat(0.0f);
		}
	}

	Em.CheckFeatureLevelIsAtLeast(ERHIFeatureLevel::SM5);
	if (Em.IsInvalid())
	{
		return Em.Poison();
	}

	if (PositionOriginType == EPositionOrigin::CameraRelative)
	{
		// LWC_TODO: support translated-world coordinates
		PositionValue = EmitTransformVector(Em, PositionValue, MCB_TranslatedWorld, MCB_World, true, {}, {});
	}

	// Cast the position to al float3
	PositionValue = Em.Cast(PositionValue, MIR::FType::MakeFloatVector(3));

	// Make a constant integer value from the target index.
	FValueRef TargetIndexValue = Em.ConstantInt(TargetIndex);

	MIR::EExternSimpleType ResultType = MIR::EExternSimpleType::Float1;
	const TCHAR* ResultCode;

	// LWC_TODO: LWC aware physics field
	if (OutputType == Field_Output_Vector)
	{
		ResultType = MIR::EExternSimpleType::Float3;
		ResultCode = bSampleField ? TEXT("MatPhysicsField_SamplePhysicsVectorField($0, $1)") : TEXT("MatPhysicsField_EvalPhysicsVectorField($0, $1)");
	}
	else if (OutputType == Field_Output_Scalar)
	{
		ResultCode = bSampleField ? TEXT("MatPhysicsField_SamplePhysicsScalarField($0, $1)") : TEXT("MatPhysicsField_EvalPhysicsScalarField($0, $1)");
	}
	else if (OutputType == Field_Output_Integer)
	{
		ResultCode = bSampleField ? TEXT("MatPhysicsField_SamplePhysicsIntegerField($0, $1)") : TEXT("MatPhysicsField_EvalPhysicsIntegerField($0, $1)");
	}
	else
	{
		checkf(false, TEXT("Unsupported output type."));
		return Em.Poison();
	}

	return Em.Extern<MIR::FExternSimpleHLSL>({ ResultType, ResultCode }, PositionValue, TargetIndexValue);
}

void UMaterialExpressionSamplePhysicsIntegerField::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitProcessPhysicsField(Em, &WorldPosition, WorldPositionOriginType, Field_Output_Integer, FieldTarget, true));
}

void UMaterialExpressionSamplePhysicsScalarField::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitProcessPhysicsField(Em, &WorldPosition, WorldPositionOriginType, Field_Output_Scalar, FieldTarget, true));
}

void UMaterialExpressionSamplePhysicsVectorField::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitProcessPhysicsField(Em, &WorldPosition, WorldPositionOriginType, Field_Output_Vector, FieldTarget, true));
}

void UMaterialExpressionEvalPhysicsIntegerField::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitProcessPhysicsField(Em, &WorldPosition, WorldPositionOriginType, Field_Output_Integer, FieldTarget, false));
}

void UMaterialExpressionEvalPhysicsScalarField::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitProcessPhysicsField(Em, &WorldPosition, WorldPositionOriginType, Field_Output_Scalar, FieldTarget, false));
}

void UMaterialExpressionEvalPhysicsVectorField::Build(MIR::FEmitter& Em)
{
	Em.Output(0, EmitProcessPhysicsField(Em, &WorldPosition, WorldPositionOriginType, Field_Output_Vector, FieldTarget, false));
}

void UMaterialExpressionSobol::Build(MIR::FEmitter& Em)
{
	// Construct the following expression:
	// floor(Cell) + float2(SobolIndex(SobolPixel(Cell), Index) ^ (Seed * 0x10000) & 0xffff) / float(0x10000)

	FValueRef CellValue   = Em.CastToFloat(Em.InputDefaultFloat2(&Cell, { 0.f, 0.f }), 2);   // float2
	FValueRef IndexValue  = Em.InputDefaultInt(&Index, ConstIndex);                          // int
	FValueRef SeedValue   = Em.CastToFloat(Em.InputDefaultFloat2(&Seed, { (float)ConstSeed.X, (float)ConstSeed.Y }), 2); // float2
	FValueRef SobolBits   = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Int2, TEXT("SobolIndex(SobolPixel(uint2($0)), uint($1))") }, CellValue, IndexValue);
	FValueRef SeedUint    = Em.CastToInt(Em.Multiply(SeedValue,Em.ConstantFloat(65536.0f)), 2);
	FValueRef MaskedBits  = Em.BitwiseAnd(Em.BitwiseXor(SobolBits, SeedUint), Em.ConstantInt(0xffff));
	FValueRef Normalized  = Em.Divide(Em.CastToFloat(MaskedBits, 2), Em.ConstantFloat(65536.0f));
	FValueRef Result      = Em.Add(Em.Floor(CellValue), Normalized);

	Em.Output(0, Result);
}

void UMaterialExpressionSpeedTree::Build(MIR::FEmitter& Em)
{
	if (Em.GetMaterialInterface()->GetUsageByFlag(MATUSAGE_SkeletalMesh))
	{
		Em.Error(TEXT("SpeedTree node not currently supported for Skeletal Meshes, please disable usage flag."));
		return;
	}

	FValueRef GeometryValue = Em.CastToInt(Em.InputDefaultInt(&GeometryInput, GeometryType), 1);
	FValueRef WindValue = Em.CastToInt(Em.InputDefaultInt(&WindInput, WindType), 1);
	FValueRef LODValue = Em.CastToInt(Em.InputDefaultInt(&LODInput, LODType), 1);
	FValueRef ExtraBendValue = Em.CastToVector(Em.TryInput(&ExtraBendWS), 3);
	FValueRef UseExtraBendValue = Em.ConstantBool(ExtraBendValue);
	FValueRef BillboardThresholdValue = Em.ConstantFloat(BillboardThreshold);

	if (!ExtraBendValue)
	{
		ExtraBendValue = Em.ConstantFloat3({ 0.f, 0.f, 0.f });
	}
 
	struct FSpeedTreeExtern
	{
		bool bAccurateWindVelocities;

		MIR::FExternInfo GetInfo() const
		{
			return
			{
				.Name = TEXT("SpeedTree"),
				.Type = MIR::FType::MakeFloatVector(3),
				.Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials,
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			Context.Module->AddEnvironmentDefine(TEXT("USES_SPEEDTREE"));
			Context.Module->GetStatistics().NumInterpolatedTexCoords = FMath::Max(Context.Module->GetStatistics().NumInterpolatedTexCoords, 8);
		}

		void AnalyzeInStage(MIR::FExternAnalysisContext& Context, MIR::EStage Stage)
		{
			Context.CheckStage(Stage, MIR::EStageMask::Vertex, TEXT("MaterialExpressionSpeedTree"));
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			// Only generate previous frame's computations if required and opted-in
			Printer << (bAccurateWindVelocities
				? TEXTVIEW("GetSpeedTreeVertexOffset(Parameters, $0, $1, $2, $3, $4, $5)")
				: TEXTVIEW("Get<PREV>SpeedTreeVertexOffset(Parameters, $0, $1, $2, $3, $4, $5)"));
		}
	};

	Em.Output(0, Em.Extern<FSpeedTreeExtern>({ bAccurateWindVelocities }, GeometryValue, WindValue, LODValue, BillboardThresholdValue, UseExtraBendValue, ExtraBendValue));
}

void UMaterialExpressionSRGBColorToWorkingColorSpace::Build(MIR::FEmitter& Em)
{
	FValueRef ColorValue = Em.CheckIsVector(Em.Input(&Input), 3, 4);
	FValueRef ResultValue = ColorValue;

	UE_MIR_CHECKPOINT(Em);

	if (!UE::Color::FColorSpace::GetWorking().IsSRGB())
	{
		const UE::Color::FColorSpaceTransform& Transform = UE::Color::FColorSpaceTransform::GetSRGBToWorkingColorSpace();

		FValueRef R = Em.Dot(ColorValue, Em.ConstantFloat3({ (float)Transform.M[0][0], (float)Transform.M[1][0], (float)Transform.M[2][0] }));
		FValueRef G = Em.Dot(ColorValue, Em.ConstantFloat3({ (float)Transform.M[0][1], (float)Transform.M[1][1], (float)Transform.M[2][1] }));
		FValueRef B = Em.Dot(ColorValue, Em.ConstantFloat3({ (float)Transform.M[0][2], (float)Transform.M[1][2], (float)Transform.M[2][2] }));
		
		ResultValue = ColorValue->Type.IsVector(4)
			? Em.Vector4(R, G, B, Em.Subscript(ColorValue, 3)) // We preserve the original alpha when applicable
			: Em.Vector3(R, G, B);
	}

	Em.Output(0, ResultValue);
}

void UMaterialExpressionTemporalSobol::Build(MIR::FEmitter& Em)
{
	// The logic in this function implements this expression
	// (float2(SobolIndex(SobolPixel(uint2(Parameters.SvPosition.xy)), uint(View.StateFrameIndexMod8 + 8 * $0)) ^ (uint2($1 * 0x10000) & 0xffff)) / 0x10000)

    FValueRef IndexValue = Em.CastToInt(Em.InputDefaultInt(&Index, ConstIndex), 1);
    FValueRef SeedValue  = Em.CastToFloat(Em.InputDefaultFloat2(&Seed, { (float)ConstSeed.X, (float)ConstSeed.Y }), 2);
	FValueRef FrameMod8  = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Int1, TEXT("int(View.StateFrameIndexMod8)") });

    // View.StateFrameIndexMod8 + 8 * Index
    FValueRef FrameIndex  = Em.Add(FrameMod8, Em.Multiply(IndexValue, Em.ConstantInt(8)));

    // SobolIndex(SobolPixel(uint2(Parameters.SvPosition.xy)), FrameIndexU)
	FValueRef Sobol = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Int2, TEXT("SobolIndex(SobolPixel(uint2(Parameters.SvPosition.xy)), uint($0))") }, FrameIndex);

	// uint2(SeedValue * 0x10000) & 0xffff)
	FValueRef Scale      = Em.ConstantFloat(65536.0f); // 0x1000
    FValueRef SeedScaled = Em.Multiply(SeedValue, Scale);
    FValueRef SeedMasked = Em.BitwiseAnd(Em.CastToInt(SeedScaled, 2), Em.ConstantInt2({ 0xffff, 0xffff }));

	Sobol = Em.CastToFloat(Em.BitwiseXor(Sobol, SeedMasked), 2);
	Sobol = Em.Divide(Sobol, Scale);

    Em.Output(0, Sobol);
}

void UMaterialExpressionNeuralNetworkInput::Build(MIR::FEmitter& Em)
{
	int32 CodeInput = INDEX_NONE;
	bool bUseTextureAsInput = NeuralIndexType == ENeuralIndexType::NIT_TextureIndex;

	FValueRef CoordinatesValue = Em.TryInput(&Coordinates);
	if (CoordinatesValue)
	{
		if (bUseTextureAsInput)
		{
			CoordinatesValue = Em.Swizzle(CoordinatesValue, MIR::FSwizzleMask { MIR::EVectorComponent::Y, MIR::EVectorComponent::Z, MIR::EVectorComponent::W });
			CoordinatesValue = Em.Vector4(Em.ConstantFloat(-1.0f), CoordinatesValue);
		}
	}
	else
	{
		FValueRef ViewportUV = Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("ViewportUV") });
		float     BatchIndex = bUseTextureAsInput ? -1.0f : 0.0f;
		CoordinatesValue     = Em.Vector4(Em.ConstantFloat2({ BatchIndex, 0.0f }), ViewportUV);
	}

	FValueRef OutputValues[] =
	{
		CoordinatesValue, 
		Em.CastToFloat(Em.InputDefaultFloat3(&Input0, { 0.5f, 0.5f, 0.5f }), 3), 
		Em.CastToFloat(Em.InputDefaultFloat(&Mask, 1.0f), 1)
	};

	Em.SetCustomOutputs(TEXTVIEW("NeuralInput"), OutputValues, MIR::EMaterialOutputFrequency::PerPixel);
}

void UMaterialExpressionNeuralNetworkOutput::Build(MIR::FEmitter& Em)
{
	Em.CheckMaterialDomainIsAnyOf({ MD_PostProcess });
	UE_MIR_CHECKPOINT(Em);

	FValueRef ViewportUV = Em.CastToFloat(Em.TryInput(&Coordinates), 2);
	UE_MIR_CHECKPOINT(Em);

	FValueRef BufferIndex = ViewportUV;

	if (!ViewportUV)
	{
		ViewportUV = Em.Extern<MIR::FExternFromMaterialDecl>({ TEXT("ViewportUV") });
		BufferIndex = Em.Vector(Em.ConstantFloat2({ 0.0f, 0.0f }), ViewportUV);
	}

	BufferIndex = Em.CastToFloat(BufferIndex, 4);

	// @massimo.tristano another note on this. do we need this in the new translator?
	// AddEstimatedTextureSample();

	struct FNeuralNetworkOutputExtern
	{
		const TCHAR* Code;

		MIR::FExternInfo GetInfo() const
		{
			return {
				.Name = TEXT("NeuralNetworkOutputExtern"),
				.Type = MIR::FType::MakeFloatVector(4),
				.Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials,
			};
		}

		void Analyze(MIR::FExternAnalysisContext& Context)
		{
			Context.Module->GetCompilationOutput().bUsedWithNeuralNetworks = true;
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << Code;
		}
	};
	
	Em.Output(0, Em.Extern<FNeuralNetworkOutputExtern>({ TEXT("NeuralTextureOutput(Parameters, $0)") }, ViewportUV));
	Em.Output(1, Em.Extern<FNeuralNetworkOutputExtern>({ TEXT("NeuralBufferOutput(Parameters, $0)") }, BufferIndex));
}

void UMaterialExpressionVertexInterpolator::Build(MIR::FEmitter& Em)
{
	Em.Output(0, Em.VertexInterpolator(Em.Input(&Input)));
}

#endif // WITH_EDITOR



///////////////////////////////////////////////////////////////////////////////
//                       Substrate MIR implementation
///////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR

using namespace Substrate;

struct FSubstrateRegisteredSharedLocalBasisSearchResult
{
	bool bFound = false;
	FSubstrateSharedLocalBasesInfo BasisInfo;
};

FSubstrateRegisteredSharedLocalBasisSearchResult SubstrateCompilationFindExistingSharedLocalBasis(Substrate::FSubstrateTranslatorData* SubstrateTranslatorData, uint32 NormalCodeChunkHash, uint32 TangentCodeChunkHash)
{
	FSubstrateRegisteredSharedLocalBasisSearchResult Result;

	FSubstrateCompilationContext& SubstrateCtx = SubstrateTranslatorData->SubstrateCompilationContext[SubstrateTranslatorData->CurrentSubstrateCompilationContext];

	// Find a basis which matches both the Normal & the Tangent code chunks
	TArray<FSubstrateSharedLocalBasesInfo*> NormalInfos;
	SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.MultiFindPointer(NormalCodeChunkHash, NormalInfos);
	for (FSubstrateSharedLocalBasesInfo* NormalInfo : NormalInfos)
	{
		// * Either we find a perfect match (normal & tangent matches)
		// * Or we find a normal which doesn't have a tangent associated with, and we set the tangent for code
		if (TangentCodeChunkHash == INDEX_NONE || TangentCodeChunkHash == NormalInfo->SharedData.TangentCodeChunk)
		{
			Result.BasisInfo = *NormalInfo;
			Result.bFound = true;
		}
	}

	return Result;
}

FSubstrateRegisteredSharedLocalBasis MIRSubstrateCompilationInfoCreateSharedLocalBasis2(Substrate::FSubstrateTranslatorData* SubstrateTranslatorData, FValueRef* NormalArg, FValueRef* TangentArg)
{
	// Specific implementation for IR emitter on the common FSubstrateTranslatorData
	// Note that:
	//  - We do not have generated code or unique identifier yet at this stage.
	//  - And we do not have any unique hash for the generated IR tree.
	//  - So there is no local basis deduplication right now with the new translator based on IR.

	MIR::FEmitter* Em = SubstrateTranslatorData->GetCompilerNew();
	check(Em); // This function is meant to be used with the new translator only.

	FString SubstrateSharedNormalParamCode;
	FString SubstrateSharedTangentParamCode;
	const uint32 FinalSharedLocalBasisIndex = SubstrateTranslatorData->GetFreeSharedLocalBasesIndex();

	FString SubstrateSharedNormalParamName;
	FString SubstrateSharedTangentParamName;

	SubstrateSharedNormalParamName = FString(TEXT("SubstrateNormal"));
	SubstrateSharedNormalParamName.AppendInt(FinalSharedLocalBasisIndex);
	Em->SetCustomOutputs(SubstrateSharedNormalParamName, { NormalArg, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
	SubstrateSharedNormalParamCode = FString::Printf(TEXT("Parameters.%s0"), *SubstrateSharedNormalParamName);

	// Always write tangent even if not used later for now.
	SubstrateSharedTangentParamName = FString(TEXT("SubstrateTangent"));
	SubstrateSharedTangentParamName.AppendInt(FinalSharedLocalBasisIndex);
	FValueRef DefaultTangent = MaterialToMIR::EmitVertexTangent(*Em);
	Em->SetCustomOutputs(SubstrateSharedTangentParamName, { TangentArg ? TangentArg : &DefaultTangent, 1}, MIR::EMaterialOutputFrequency::PerPixel);
	SubstrateSharedTangentParamCode = FString::Printf(TEXT("Parameters.%s0"), *SubstrateSharedTangentParamName);

	FSubstrateCompilationContext& SubstrateCtx = SubstrateTranslatorData->SubstrateCompilationContext[SubstrateTranslatorData->CurrentSubstrateCompilationContext];
	check(NormalArg != nullptr);
	check(SubstrateCtx.NextFreeSubstrateShaderNormalIndex < 255);	// Out of shared local basis slots

	static const FString NullSubstrateIRNormalTangentCode(TEXT("Parameters.TangentToWorld[2]"));	//

	FSubstrateRegisteredSharedLocalBasis SubstrateRegisteredSharedLocalBasis;
	SubstrateRegisteredSharedLocalBasis.NormalCodeChunk = FinalSharedLocalBasisIndex;
	SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash = FinalSharedLocalBasisIndex;
	SubstrateRegisteredSharedLocalBasis.TangentCodeChunk = FinalSharedLocalBasisIndex;
	SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash = FinalSharedLocalBasisIndex;
	SubstrateRegisteredSharedLocalBasis.GraphSharedLocalBasisIndex = SubstrateCtx.NextFreeSubstrateShaderNormalIndex++;

	// Find a basis which matches both the Normal & the Tangent code chunks
	TArray<FSubstrateSharedLocalBasesInfo*> NormalInfos;
	SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.MultiFindPointer(SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash, NormalInfos);
	for (FSubstrateSharedLocalBasesInfo* NormalInfo : NormalInfos)
	{
		// * Either we find a perfect match (normal & tangent matches)
		// * Or we find a normal which doesn't have a tangent associated with, and we set the tangent for code
		if (SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash == NormalInfo->SharedData.TangentCodeChunk)
		{
			return NormalInfo->SharedData;
		}
		else if (NormalInfo->SharedData.TangentCodeChunk == INDEX_NONE)
		{
			NormalInfo->SharedData.TangentCodeChunk = FinalSharedLocalBasisIndex;
			NormalInfo->SharedData.TangentCodeChunkHash = SubstrateRegisteredSharedLocalBasis.TangentCodeChunkHash;
			NormalInfo->TangentCode = SubstrateSharedTangentParamCode;
			NormalInfo->TangentArg = TangentArg != nullptr ? *TangentArg : MIR::FValueRef();
			return NormalInfo->SharedData;
		}
	}

	// Allocate a new slot for a new shared local basis
	SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.Add(SubstrateRegisteredSharedLocalBasis.NormalCodeChunkHash, { SubstrateRegisteredSharedLocalBasis, *SubstrateSharedNormalParamCode, *SubstrateSharedTangentParamCode, *NormalArg, TangentArg != nullptr ? *TangentArg : MIR::FValueRef()});
	return SubstrateRegisteredSharedLocalBasis;
}

static bool IsTangentSpaceNormal(UMaterial* Material)
{
	check(Material); // SUBSTRATE_TODO We should never have a null material here, but it seems to happen in some cases. Figure out why and fix if this can result in bad tangent space normal.
	if (!Material)
	{
		return true; // This is here to fall back to using tangent space when debugging and working with the new translator.
	}
	return Material->MaterialDomain == MD_DeferredDecal || Material->bTangentSpaceNormal;
}

static FValueRef TransformNormalFromRequestedBasisToWorld(MIR::FEmitter& Em, UMaterial* Material, FValueRef NormalArg)
{
	// When feeding tangent space or world space, we want to have the final normal/tangent normalized when stored in SharedLocalBases.
	// So that we do not have to normalise it later in any forward processes and avoid overhead in instructions.
	if (IsTangentSpaceNormal(Material))
	{
		// See TransformTangentNormalToWorld definitions in MaterialTemplate.ush
		return EmitTransformVector(Em, NormalArg, MCB_Tangent, MCB_World, false, {}, {});
	}
	// Normalize
	return NormalArg = Em.Multiply(NormalArg, Em.Rsqrt(Em.Dot(NormalArg, NormalArg)));
}

// The compilation of an expression can sometimes lead to a INDEX_NONE code chunk when editing material graphs 
// or when the node is inside a material function, linked to an input pin of the material function and that input is not plugged in to anything.
// But for normals or tangents, Substrate absolutely need a valid code chunk to de-duplicate when stored in memory. 
// Also, we want all our nodes to have default, as that is needed when creating BSDF, when registering code chunk representing material topology.

static FValueRef CompileWithDefaultFloat1(MIR::FEmitter& Em, FExpressionInput& Input, float X, const FScalarMaterialInput* RootNodeInput = nullptr)
{
	FValueRef DefaultResultArg = Em.ConstantFloat(X);
	if (RootNodeInput && RootNodeInput->UseConstant)
	{
		DefaultResultArg = Em.ConstantFloat(RootNodeInput->Constant);
	}
	FValueRef ResultArg = Input.GetTracedInput().Expression ? Em.TryInput(&Input) : DefaultResultArg;
	ResultArg = Em.CastToFloatKind(ResultArg);	// Move to non LWC
	return !ResultArg.IsValid() ? DefaultResultArg : ResultArg;
}

static FValueRef CompileWithDefaultFloat2(MIR::FEmitter& Em, FExpressionInput& Input, float X, float Y, const FVector2MaterialInput* RootNodeInput = nullptr)
{
	FValueRef DefaultResultArg = Em.ConstantFloat2(FVector2f(X, Y));
	if (RootNodeInput && RootNodeInput->UseConstant)
	{
		DefaultResultArg = Em.ConstantFloat2(FVector2f(RootNodeInput->Constant.X, RootNodeInput->Constant.Y));
	}
	FValueRef ResultArg = Input.GetTracedInput().Expression ? Em.TryInput(&Input) : DefaultResultArg;
	ResultArg = Em.CastToFloatKind(ResultArg);	// Move to non LWC
	return !ResultArg.IsValid() ? DefaultResultArg : ResultArg;
}

static FValueRef CompileWithDefaultFloat3(MIR::FEmitter& Em, FExpressionInput& Input, float X, float Y, float Z, const FColorMaterialInput* RootNodeInput = nullptr)
{
	FValueRef DefaultResultArg = Em.ConstantFloat3(FVector3f(X, Y, Z));
	if (RootNodeInput && RootNodeInput->UseConstant)
	{
		DefaultResultArg = Em.ConstantFloat3(FVector3f(RootNodeInput->Constant.R, RootNodeInput->Constant.G, RootNodeInput->Constant.B));
	}
	FValueRef ResultArg = Input.GetTracedInput().Expression ? Em.TryInput(&Input) : DefaultResultArg;
	ResultArg = Em.CastToFloatKind(ResultArg);	// Move to non LWC
	return !ResultArg.IsValid() ? DefaultResultArg : ResultArg;
}

enum EDefaultDirection
{
	EDefaultDirectionNormal,
	EDefaultDirectionTangent
};

static FValueRef CompileWithDefaultDirectionWS(MIR::FEmitter& Em, UMaterial* Material, FExpressionInput& DirectionInput, bool bConvertToRequestedSpace, EDefaultDirection DefaultDirection)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	auto EmitDefaultDirection = [&]()
	{
		switch(DefaultDirection)
		{
		case EDefaultDirectionNormal:
			return MaterialToMIR::EmitVertexNormal(Em);
		case EDefaultDirectionTangent:
			return MaterialToMIR::EmitVertexTangent(Em);
		}
		Em.Error(TEXT("Unkown EmitDefaultDirection parameter."));
		return MaterialToMIR::EmitVertexNormal(Em);
	};

	if (DirectionInput.GetTracedInput().Expression != nullptr)
	{
		FValueRef DirectionArg = Em.TryInput(&DirectionInput);
		if (!DirectionArg)
		{
			// Nothing is plug in from the linked input, so specify world space normal the BSDF node expects.
			DirectionArg = EmitDefaultDirection();
		}

		// Ensure the normal has always a valid float3 type
		DirectionArg = Em.Cast(DirectionArg, MIR::FType::MakeFloatVector(3));
		DirectionArg = Em.CastToFloatKind(DirectionArg);	// Move to non LWC
		if (!DirectionArg.IsValid())
		{
			DirectionArg = EmitDefaultDirection();
		}

		// Transform into world space normal if needed. BSDF nodes always expects world space normal as input.
		if (bConvertToRequestedSpace)
		{
			DirectionArg = TransformNormalFromRequestedBasisToWorld(Em, Material, DirectionArg);
		}
		return DirectionArg;
	}
	// Nothing is plug in on the BSDF node, so specify world space normal the node expects.
	return EmitDefaultDirection();
}

static FValueRef CompileWithDefaultNormalWS(MIR::FEmitter& Em, UMaterial* Material, FExpressionInput& NormalInput, bool bConvertToRequestedSpace = true)
{
	return CompileWithDefaultDirectionWS(Em, Material, NormalInput, bConvertToRequestedSpace, EDefaultDirection::EDefaultDirectionNormal);
}

static FValueRef CompileWithDefaultTangentWS(MIR::FEmitter& Em, UMaterial* Material, FExpressionInput& TangentInput, bool bConvertToRequestedSpace = true)
{
	return CompileWithDefaultDirectionWS(Em, Material, TangentInput, bConvertToRequestedSpace, EDefaultDirection::EDefaultDirectionTangent);
}

FValueRef EmitSubstrateSharedLocalBasis(MIR::FEmitter& Em, const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis, ESubstrateCompilationContext Context)
{
	struct FSubstrateSharedLocalBasisExtern
	{
		uint8 GraphSharedLocalBasisIndex;
		ESubstrateCompilationContext Context;

		MIR::FExternInfo GetInfo() const
		{
			return
				{
					.Name = TEXT("SubstrateSharedLocalBasis"),
					.Type = MIR::FType::MakeIntScalar(),
					.Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::NoDifferentials,
				};
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			Printer << TEXT("SHAREDLOCALBASIS_INDEX_") << (uint32)GraphSharedLocalBasisIndex << TEXT('_') << (uint32)Context;
		}
	};
	return Em.Extern<FSubstrateSharedLocalBasisExtern>({ SharedLocalBasis.GraphSharedLocalBasisIndex, Context });
}

static FValueRef FinalizeBSDFAndPromoteToOperatorIfNeeded(MIR::FEmitter& Em, FSubstrateOperator& SubstrateOperator, FValueRef BSDF)
{
	FSubstrateOperator* PromoteToOperator = !SubstrateOperator.bUseParameterBlending || (SubstrateOperator.bUseParameterBlending && SubstrateOperator.bRootOfParameterBlendingSubTree) ? &SubstrateOperator : nullptr;
	if (PromoteToOperator)
	{
		if (PromoteToOperator->Index == INDEX_NONE || PromoteToOperator->BSDFIndex == INDEX_NONE)
		{
			Em.Errorf(TEXT("Invalid SubstrateSlabBSDF operator and BSDF indices during promotion.\r\n"));
			FValueRef DefaultSlab = Em.SubstrateDefaultSlab();
			Em.Output(0, DefaultSlab);
			return DefaultSlab;
		}

		FValueRef PromototedSlab = Em.SubstratePromoteToOperator(
			BSDF,
			Em.ConstantInt(PromoteToOperator->Index),
			Em.ConstantInt(PromoteToOperator->BSDFIndex),
			Em.ConstantInt(PromoteToOperator->LayerDepth),
			Em.ConstantInt(PromoteToOperator->bIsBottom ? 1 : 0));
		Em.Output(0, PromototedSlab);
		return PromototedSlab;
	}

	Em.Output(0, BSDF);
	return BSDF;
}

static FValueRef CreateSubsurfaceProfileParameter(MIR::FEmitter& Em, USubsurfaceProfile* InProfile)
{
	check(InProfile);
	const FName SubsurfaceProfileParameterName = SubsurfaceProfile::CreateSubsurfaceProfileParameterName(InProfile);
	const FValueRef SSSProfileCodeArg = Em.CastToFloat(Em.NamedPrimitiveUniform(SubsurfaceProfileParameterName, Em.ConstantFloat(0.0f)), 1);
	return SSSProfileCodeArg;
}

static FValueRef CreateDefaultSubsurfaceProfileParameter(MIR::FEmitter& Em)
{
	const FValueRef SSSProfileCodeArg = Em.CastToFloat(Em.NamedPrimitiveUniform(SubsurfaceProfile::GetSubsurfaceProfileParameterName(), Em.ConstantFloat(0.0f)), 1);
	return SSSProfileCodeArg;
}

void UMaterialExpressionSubstrateSlabBSDF::Build(MIR::FEmitter& Em)
{
	// SUBSTRATE_TODO make this similar to UMaterialExpressionSubstrateSlabBSDF::Compile
	// SUBSTRATE_TODO put SubstrateTranslatorCommon on Em.

	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef SubsurfaceProfileArg;
	if (SubstrateOperator.Has(ESubstrateBsdfFeature::SSS) && SubsurfaceProfile)
	{
		SubsurfaceProfileArg = CreateSubsurfaceProfileParameter(Em, SubsurfaceProfile);
	}
	else
	{
		SubsurfaceProfileArg = CreateDefaultSubsurfaceProfileParameter(Em);
	}

	FValueRef SpecularProfileArg = Em.ConstantFloat(0.0f);
	if (SubstrateOperator.Has(ESubstrateBsdfFeature::SpecularProfile))
	{
		const FName SpecularProfileParameterName = SpecularProfile::CreateSpecularProfileParameterName(SpecularProfile);
		SpecularProfileArg = Em.CastToFloat(Em.NamedPrimitiveUniform(SpecularProfileParameterName, Em.ConstantFloat(0.0f)), 1);
	}
	else
	{
		SpecularProfileArg = Em.ConstantFloat(0.0f);
	}

	const float DefaultSpecular = 0.5f;
	const float DefaultF0 = DielectricSpecularToF0(DefaultSpecular);

	// We also cannot ignore the tangent when using the default Tangent because GetTangentBasis
	// used in SubstrateGetBSDFSharedBasis cannot be relied on for smooth tangent used for lighting on any mesh.
	const bool bHasAnisotropy = SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy);
	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, Normal);
	FValueRef TangentArg = CompileWithDefaultTangentWS(Em, Material, Tangent);
	const bool bHasValidTangent = bHasAnisotropy && TangentArg.IsValid();

	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, bHasValidTangent ? &TangentArg : nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;

	FValueRef ThicknessArg;
	if (SubstrateOperator.ThicknessIndex != INDEX_NONE)
	{
		FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression* ThicknessExpression = SubstrateTranslatorData->SubstrateThicknessStackGetExpression(SubstrateOperator.ThicknessIndex);

		if (ThicknessExpression == nullptr)
		{
			Em.Errorf(TEXT(" SubstrateThichkness: %i could not be found)"), SubstrateOperator.ThicknessIndex);
			ThicknessArg = Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
		}
		else if (ThicknessExpression->ExpressionInput)
		{
			FExpressionInput* ExpressionInput = ThicknessExpression->ExpressionInput;
			ThicknessArg = ExpressionInput->GetTracedInput().Expression ? Em.TryInput(ExpressionInput) : Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
			ThicknessArg = Em.CastToFloatKind(ThicknessArg); // remove LWC
		}
		else if (ThicknessExpression->MaterialInput)
		{
			FScalarMaterialInput* MaterialInput = ThicknessExpression->MaterialInput;
			ThicknessArg = MaterialInput->UseConstant ? Em.ConstantFloat(MaterialInput->Constant) : Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
			ThicknessArg = Em.CastToFloatKind(ThicknessArg); // remove LWC
		}
		if (!ThicknessArg.IsValid())
		{
			ThicknessArg = Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
		}
	}
	else
	{
		// Thickness is not tracked properly, this can happen when opening a material function in editor
		ThicknessArg = Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);
	}
	check(ThicknessArg.IsValid());

	FValueRef EmissiveColorArg				= CompileWithDefaultFloat3(Em, EmissiveColor, 0.0f, 0.0f, 0.0f);
	FValueRef DiffuseAlbedoArg				= CompileWithDefaultFloat3(Em, DiffuseAlbedo, 0.18f, 0.18f, 0.18f);
	FValueRef F0Arg							= CompileWithDefaultFloat3(Em, F0, DefaultF0, DefaultF0, DefaultF0);
	FValueRef RoughnessArg					= CompileWithDefaultFloat1(Em, Roughness, 0.5f);
	FValueRef AnisotropyArg					= CompileWithDefaultFloat1(Em, Anisotropy, 0.0f);
	FValueRef F90Arg						= CompileWithDefaultFloat3(Em, F90, 1.0f, 1.0f, 1.0f);
	FValueRef SSSMFPArg						= CompileWithDefaultFloat3(Em, SSSMFP, 0.0f, 0.0f, 0.0f);
	FValueRef SSSMFPScaleArg				= CompileWithDefaultFloat1(Em, SSSMFPScale, 1.0f);
	FValueRef SSSPhaseAnisotropyArg			= CompileWithDefaultFloat1(Em, SSSPhaseAnisotropy, 0.0f);
	FValueRef SecondRoughnessArg			= CompileWithDefaultFloat1(Em, SecondRoughness, 0.0f);
	FValueRef SecondRoughnessWeightArg		= CompileWithDefaultFloat1(Em, SecondRoughnessWeight, 0.0f);
	FValueRef FuzzAmountArg					= CompileWithDefaultFloat1(Em, FuzzAmount, 0.0f);
	FValueRef FuzzColorArg					= CompileWithDefaultFloat3(Em, FuzzColor, 0.0f, 0.0f, 0.0f);
	FValueRef FuzzRoughnessArg				= HasFuzzRoughness() ? CompileWithDefaultFloat1(Em, FuzzRoughness, 0.5f) : RoughnessArg;
	FValueRef GlintValueArg					= CompileWithDefaultFloat1(Em, GlintValue, 1.0f);
	FValueRef GlintUVArg					= CompileWithDefaultFloat2(Em, GlintUV, 0.0f, 0.0f);

	// Disable some features if requested by the simplification process
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::MFPPluggedIn) || SubSurfaceType == EMaterialSubSurfaceType::MSS_None)
	{
		SSSMFPArg = Em.ConstantFloat3(FVector3f(0.0f, 0.0f, 0.0f));
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::EdgeColor))
	{
		F90Arg = Em.ConstantFloat3(FVector3f(1.0f, 1.0f, 1.0));
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::Fuzz))
	{
		FuzzAmountArg = Em.ConstantFloat(0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat))
	{
		SecondRoughnessWeightArg = Em.ConstantFloat(0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy))
	{
		AnisotropyArg = Em.ConstantFloat(0.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::Glint))
	{
		GlintValueArg = Em.ConstantFloat(1.0f);
	}
	if (!SubstrateOperator.Has(ESubstrateBsdfFeature::SpecularProfile))
	{
		SpecularProfileArg = Em.ConstantFloat(0.0f);
	}

	const bool bIsAtTheBottomOfTopology = SubstrateOperator.bIsBottom > 0;
	const bool bIsThinSurface = Material ? Material->IsThinSurface() : false;
	float ClearCoatUseSecondNormal = 0.0f; // Constant(ClearCoatBottomNormal != Normal ? 1.0f : 0.0f);SUBSTRATE_TODO

	FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	MIR::FSubstrateSlabDesc SubstrateSlabDesc = {
			NormalArg,
			DiffuseAlbedoArg,
			F0Arg,
			F90Arg,
			RoughnessArg,
			AnisotropyArg,
			SubsurfaceProfileArg,
			SSSMFPArg,
			SSSMFPScaleArg,
			SSSPhaseAnisotropyArg,
			Em.ConstantFloat(float(SubSurfaceType)),
			EmissiveColorArg,
			SecondRoughnessArg,
			SecondRoughnessWeightArg,
			Em.ConstantFloat(0.0f),												//		SecondRoughnessAsSimpleClearCoatSUBSTRATE_TODO
			Em.ConstantFloat(ClearCoatUseSecondNormal),							//		*SubstrateGetCastParameterCode(ClearCoatUseSecondNormal, MCT_Float),SUBSTRATE_TODO
			Em.ConstantFloat3(FVector3f::Zero()),								//		*SubstrateGetCastParameterCode(ClearCoatBottomNormal, MCT_Float3),SUBSTRATE_TODO
			FuzzAmountArg,
			FuzzColorArg,
			FuzzRoughnessArg,
			GlintValueArg,
			GlintUVArg,
			SpecularProfileArg,
			ThicknessArg,
			bIsThinSurface ? Em.ConstantTrue() : Em.ConstantFalse(),
			bIsAtTheBottomOfTopology ? Em.ConstantTrue() : Em.ConstantFalse(),
			SharedLocalBasisIndexMacroArg };

	FValueRef Slab = Em.SubstrateSlab(SubstrateSlabDesc);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, Slab);
}

MIR::FValueRef UMaterialExpressionSubstrateShadingModels::BuildCommon(MIR::FEmitter& Em, UMaterial* Material,
	FExpressionInput& BaseColor, FExpressionInput& Specular, FExpressionInput& Metallic, FExpressionInput& Roughness, FExpressionInput& EmissiveColor,
	FExpressionInput& Opacity, FExpressionInput& SubSurfaceColor, FExpressionInput& ClearCoat, FExpressionInput& ClearCoatRoughness,
	FExpressionInput& ShadingModel, TEnumAsByte<enum EMaterialShadingModel> ShadingModelOverride,
	FExpressionInput& TransmittanceColor, FExpressionInput& ThinTranslucentSurfaceCoverage,
	FExpressionInput& WaterScatteringCoefficients, FExpressionInput& WaterAbsorptionCoefficients, FExpressionInput& WaterPhaseG, FExpressionInput& ColorScaleBehindWater,
	const bool bHasAnisotropy, FExpressionInput& Anisotropy,
	FExpressionInput& Normal, FExpressionInput& Tangent,
	FExpressionInput& ClearCoatNormal, FExpressionInput& CustomTangent,
	const bool bHasSSS, USubsurfaceProfile* SSSProfile,
	const class UMaterialEditorOnlyData* EditorOnlyData)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef SubsurfaceProfileIdArg;
	if (bHasSSS && SSSProfile != nullptr)
	{
		SubsurfaceProfileIdArg = CreateSubsurfaceProfileParameter(Em, SSSProfile);
	}
	else
	{
		SubsurfaceProfileIdArg = CreateDefaultSubsurfaceProfileParameter(Em);
	}


	// We also cannot ignore the tangent when using the default Tangent because GetTangentBasis
	// used in SubstrateGetBSDFSharedBasis cannot be relied on for smooth tangent used for lighting on any mesh.
//	const bool bHasAnisotropy = SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy);
	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, Normal);
	FValueRef TangentArg = CompileWithDefaultTangentWS(Em, Material, Tangent);
	const bool bHasValidTangent = bHasAnisotropy && TangentArg.IsValid();

	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, bHasValidTangent ? &TangentArg : nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef BasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	FValueRef EmissiveColorArg = CompileWithDefaultFloat3(Em, EmissiveColor, 0.0f, 0.0f, 0.0f, EditorOnlyData ? &EditorOnlyData->EmissiveColor : nullptr);
	FValueRef BaseColorArg = CompileWithDefaultFloat3(Em, BaseColor, 0.18f, 0.18f, 0.18f, EditorOnlyData ? &EditorOnlyData->BaseColor : nullptr);
	FValueRef SpecularArg = CompileWithDefaultFloat1(Em, Specular, 0.5f, EditorOnlyData ? &EditorOnlyData->Specular : nullptr);
	FValueRef MetallicArg = CompileWithDefaultFloat1(Em, Metallic, 0.0f, EditorOnlyData ? &EditorOnlyData->Metallic : nullptr);
	FValueRef RoughnessArg = CompileWithDefaultFloat1(Em, Roughness, 0.5f, EditorOnlyData ? &EditorOnlyData->Roughness : nullptr);
	FValueRef AnisotropyArg = CompileWithDefaultFloat1(Em, Anisotropy, 0.0f, EditorOnlyData ? &EditorOnlyData->Anisotropy : nullptr);
	FValueRef SubSurfaceColorArg = CompileWithDefaultFloat3(Em, SubSurfaceColor, 1.0f, 1.0f, 1.0f, EditorOnlyData ? &EditorOnlyData->SubsurfaceColor : nullptr);
	FValueRef ClearCoatArg = CompileWithDefaultFloat1(Em, ClearCoat, 1.0f, EditorOnlyData ? &EditorOnlyData->ClearCoat : nullptr);
	FValueRef ClearCoatRoughnessArg = CompileWithDefaultFloat1(Em, ClearCoatRoughness, 0.1f, EditorOnlyData ? &EditorOnlyData->ClearCoatRoughness : nullptr);
	FValueRef ThinTranslucentTransmittanceColorArg = CompileWithDefaultFloat3(Em, TransmittanceColor, 1.0f, 1.0f, 1.0f);
	FValueRef WaterScatteringCoefficientsArg = CompileWithDefaultFloat3(Em, WaterScatteringCoefficients, 0.0f, 0.0f, 0.0f);
	FValueRef WaterAbsorptionCoefficientsArg = CompileWithDefaultFloat3(Em, WaterAbsorptionCoefficients, 0.0f, 0.0f, 0.0f);
	FValueRef WaterPhaseGArg = CompileWithDefaultFloat1(Em, WaterPhaseG, 0.0f);
	FValueRef ColorScaleBehindWaterArg = CompileWithDefaultFloat3(Em, ColorScaleBehindWater, 1.0f, 1.0f, 1.0f);
	FValueRef CustomTangentArg = CompileWithDefaultTangentWS(Em, Material, CustomTangent);
	FValueRef ThinTranslucentSurfaceCoverageArg = CompileWithDefaultFloat1(Em, ThinTranslucentSurfaceCoverage, 1.0f);


	const bool bHasCoatNormal = ClearCoatNormal.IsConnected();
	// Clear coat normal basis
	FValueRef ClearCoatNormalArg = NormalArg;
	FValueRef ClearCoatTangentArg = TangentArg;
	FValueRef ClearCoat_BasisIndexMacroArg = BasisIndexMacroArg;
	FSubstrateRegisteredSharedLocalBasis ClearCoat_NewRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	if (bHasCoatNormal)
	{
		ClearCoatNormalArg = CompileWithDefaultNormalWS(Em, Material, ClearCoatNormal);
		ClearCoat_NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &ClearCoatNormalArg, bHasValidTangent ? &ClearCoatTangentArg : nullptr);
		//SubstrateCompilationInfoCreateSharedLocalBasis(Compiler, ClearCoat_NormalCodeChunk, ClearCoat_TangentCodeChunk);
		ClearCoat_BasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, ClearCoat_NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);
	}

	FValueRef OpacityArg = FValueRef();
	const bool bSubstrateSkipsOpacityEvaluation = false; // SUBSTRATE_TODO implemente the below code shared with legacy trnaslator
	/*
	return !IsTranslucentBlendMode(Material)
		&& GetSubstrateMaterialExportType() == ESubstrateMaterialExport::SME_None	// We never skip when exporting information
		&& Material->GetShadingModels().CountShadingModels() == 1
		&& !Material->GetShadingModels().HasShadingModel(MSM_SingleLayerWater)
		&& !Material->GetShadingModels().HasShadingModel(MSM_Subsurface)
		&& !Material->GetShadingModels().HasShadingModel(MSM_SubsurfaceProfile)
		&& !Material->GetShadingModels().HasShadingModel(MSM_TwoSidedFoliage)
		&& !Material->GetShadingModels().HasShadingModel(MSM_PreintegratedSkin);
	*/
	const float DefaultOpacity = 1.0f;
	if (!bSubstrateSkipsOpacityEvaluation)
	{
		OpacityArg = CompileWithDefaultFloat1(Em, Opacity, DefaultOpacity, EditorOnlyData ? &EditorOnlyData->Opacity : nullptr);
	}
	if (!OpacityArg.IsValid())
	{
		OpacityArg = Em.ConstantFloat(DefaultOpacity);
	}

	FValueRef ShadingModelArg = ShadingModel.IsConnected() ? CompileWithDefaultFloat1(Em, ShadingModel, float(MSM_DefaultLit)) : Em.ConstantFloat(float(ShadingModelOverride));
	int32 ShadingModelCount = Material ? Material->GetShadingModels().CountShadingModels() : 1;
	bool bHasDynamicShadingModels = ShadingModelCount > 1;

	check(Material);
	UMaterial* BaseMaterial = Material->GetBaseMaterial();
	// Material is probably an instanced material, so we check that this is the case before potentially overriding the ShadingModel from the material instance.
	UMaterialInterface* BaseMaterialInterface = static_cast<UMaterialInterface*>(BaseMaterial);
	UMaterialInterface* MaterialInterface = Material;
	if (BaseMaterial && (BaseMaterialInterface != MaterialInterface))
	{
		FMaterialShadingModelField BaseMaterialShadingModels = BaseMaterial->GetShadingModels();
		FMaterialShadingModelField MaterialShadingModels = Material->GetShadingModels();

		// If the potentially instanced material does not have the same shading model as the instance material, this means it would have overridden the shading model.
		// From the UI, only a single shading model is selectable, so we simply apply the one coming form the material instance.
		if (MaterialShadingModels.IsValid() && MaterialShadingModels.CountShadingModels() == 1 && MaterialShadingModels != BaseMaterialShadingModels)
		{
			bHasDynamicShadingModels = false;	// No need to go dynamic when there is only a single shading model selected.
			ShadingModelArg = Em.ConstantFloat(float(MaterialShadingModels.GetFirstShadingModel()));
		}
	}

	MIR::FSubstrateShadingModelsDesc SubstrateShadingModelsDesc = {
		BaseColorArg,
		SpecularArg,
		MetallicArg,
		RoughnessArg,
		AnisotropyArg,
		SubSurfaceColorArg,
		SubsurfaceProfileIdArg,
		ClearCoatArg,
		ClearCoatRoughnessArg,
		EmissiveColorArg,
		OpacityArg,
		ThinTranslucentTransmittanceColorArg,
		ThinTranslucentSurfaceCoverageArg,
		WaterScatteringCoefficientsArg,
		WaterAbsorptionCoefficientsArg,
		WaterPhaseGArg,
		ColorScaleBehindWaterArg,
		ShadingModelArg,
		NormalArg,
		TangentArg,
		ClearCoatNormalArg,
		CustomTangentArg,
		BasisIndexMacroArg,
		ClearCoat_BasisIndexMacroArg,
		bHasDynamicShadingModels };

	FValueRef Slab = Em.SubstrateShadingModels(SubstrateShadingModelsDesc);

	return FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, Slab);
}

void UMaterialExpressionSubstrateShadingModels::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();
	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	const bool bHasAnisotropy = SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy);
	const bool bHasSSS = HasSSS();

	BuildCommon(Em, Material.Get(),
		BaseColor, Specular, Metallic, Roughness, EmissiveColor,
		Opacity,  SubSurfaceColor, ClearCoat, ClearCoatRoughness,
		ShadingModel, ShadingModelOverride,
		TransmittanceColor, ThinTranslucentSurfaceCoverage,
		WaterScatteringCoefficients, WaterAbsorptionCoefficients, WaterPhaseG, ColorScaleBehindWater,
		bHasAnisotropy, Anisotropy,
		Normal, Tangent,
		ClearCoatNormal, CustomTangent,
		bHasSSS, SubsurfaceProfile/*, EditorOnlyData = nullptr*/);
}

void UMaterialExpressionSubstrateConvertMaterialAttributes::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FValueRef AttrValue = Em.CheckIsAggregate(Em.Input(&MaterialAttributes), MaterialAttributesAggregate::Get());
	UE_MIR_CHECKPOINT(Em);

	// Subscript a standard property from the MaterialAttributes aggregate.
	auto GetAttr = [&](EMaterialProperty Property) -> FValueRef
	{
		return Em.Subscript(AttrValue, MaterialAttributesAggregate::MaterialPropertyToAttributeIndex(Property));
	};

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	const bool bHasAnisotropy = SubstrateOperator.Has(ESubstrateBsdfFeature::Anisotropy);

	// Normal in world space
	FValueRef NormalArg = GetAttr(MP_Normal);
	NormalArg = Em.Cast(NormalArg, MIR::FType::MakeFloatVector(3));
	NormalArg = Em.CastToFloatKind(NormalArg);
	if (!NormalArg.IsValid())
	{
		NormalArg = MaterialToMIR::EmitVertexNormal(Em);
	}
	NormalArg = TransformNormalFromRequestedBasisToWorld(Em, Material.Get(), NormalArg);

	// Tangent in world space. Always computed (FSubstrateShadingModelsDesc.Tangent must not be null),
	// though bHasValidTangent gates whether the shared local basis actually uses it.
	FValueRef TangentArg = GetAttr(MP_Tangent);
	TangentArg = Em.Cast(TangentArg, MIR::FType::MakeFloatVector(3));
	TangentArg = Em.CastToFloatKind(TangentArg);
	if (!TangentArg.IsValid())
	{
		TangentArg = MaterialToMIR::EmitVertexTangent(Em);
	}
	TangentArg = TransformNormalFromRequestedBasisToWorld(Em, Material.Get(), TangentArg);
	const bool bHasValidTangent = bHasAnisotropy && TangentArg.IsValid();

	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, bHasValidTangent ? &TangentArg : nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef BasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	// SSS profile
	FValueRef SubsurfaceProfileIdArg = HasSSS() && SubsurfaceProfile
		? CreateSubsurfaceProfileParameter(Em, SubsurfaceProfile)
		: CreateDefaultSubsurfaceProfileParameter(Em);

	// Opacity
	FValueRef OpacityArg = GetAttr(MP_Opacity);
	if (!OpacityArg.IsValid())
	{
		OpacityArg = Em.ConstantFloat(1.0f);
	}

	// Shading model: use the override constant unless the material has dynamic shading models
	// or the override requests per-pixel selection from the expression.
	const int32 ShadingModelCount = Material ? Material->GetShadingModels().CountShadingModels() : 1;
	const bool bHasDynamicShadingModels = ShadingModelCount > 1;
	FValueRef ShadingModelArg = (bHasDynamicShadingModels || ShadingModelOverride == MSM_FromMaterialExpression)
		? GetAttr(MP_ShadingModel)
		: Em.ConstantFloat(float(ShadingModelOverride));

	// Water parameters from the node's own expression inputs
	FValueRef WaterScatteringCoefficientsArg = CompileWithDefaultFloat3(Em, WaterScatteringCoefficients, 0.0f, 0.0f, 0.0f);
	FValueRef WaterAbsorptionCoefficientsArg = CompileWithDefaultFloat3(Em, WaterAbsorptionCoefficients, 0.0f, 0.0f, 0.0f);
	FValueRef WaterPhaseGArg                 = CompileWithDefaultFloat1(Em, WaterPhaseG, 0.0f);
	FValueRef ColorScaleBehindWaterArg       = CompileWithDefaultFloat3(Em, ColorScaleBehindWater, 1.0f, 1.0f, 1.0f);

	// Transmittance/thin-translucent: these come from custom attribute GUIDs not present in the
	// standard aggregate property map, so fall back to the same defaults as the legacy translator.
	FValueRef TransmittanceColorArg          = Em.ConstantFloat3(FVector3f(1.0f, 1.0f, 1.0f));
	FValueRef ThinTranslucentSurfaceCoverageArg = Em.ConstantFloat(1.0f);

	MIR::FSubstrateShadingModelsDesc SubstrateShadingModelsDesc = {
		GetAttr(MP_BaseColor),
		GetAttr(MP_Specular),
		GetAttr(MP_Metallic),
		GetAttr(MP_Roughness),
		GetAttr(MP_Anisotropy),
		GetAttr(MP_SubsurfaceColor),
		SubsurfaceProfileIdArg,
		GetAttr(MP_CustomData0),    // ClearCoat
		GetAttr(MP_CustomData1),    // ClearCoatRoughness
		GetAttr(MP_EmissiveColor),
		OpacityArg,
		TransmittanceColorArg,
		ThinTranslucentSurfaceCoverageArg,
		WaterScatteringCoefficientsArg,
		WaterAbsorptionCoefficientsArg,
		WaterPhaseGArg,
		ColorScaleBehindWaterArg,
		ShadingModelArg,
		NormalArg,
		TangentArg,
		NormalArg,              // ClearCoatNormal: no separate coat normal from aggregate in MIR
		NormalArg,              // CustomTangent: no separate custom tangent from aggregate in MIR
		BasisIndexMacroArg,
		BasisIndexMacroArg,     // ClearCoat_BasisIndexMacro: same basis as base layer
		bHasDynamicShadingModels };

	FValueRef Slab = Em.SubstrateShadingModels(SubstrateShadingModelsDesc);
	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, Slab);
}

void UMaterialExpressionSubstrateSimpleClearCoatBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	const float DefaultSpecular = 0.5f;
	const float DefaultF0 = DielectricSpecularToF0(DefaultSpecular);
	
	FValueRef EmissiveColorArg = CompileWithDefaultFloat3(Em, EmissiveColor, 0.0f, 0.0f, 0.0f);
	FValueRef DiffuseAlbedoArg = CompileWithDefaultFloat3(Em, DiffuseAlbedo, 0.18f, 0.18f, 0.18f);
	FValueRef F0Arg = CompileWithDefaultFloat3(Em, F0, DefaultF0, DefaultF0, DefaultF0);
	FValueRef RoughnessArg = CompileWithDefaultFloat1(Em, Roughness, 0.5f);
	FValueRef ClearCoatArg = CompileWithDefaultFloat1(Em, ClearCoatCoverage, 1.0f);
	FValueRef ClearCoatRoughnessArg = CompileWithDefaultFloat1(Em, ClearCoatRoughness, 0.1f);
	FValueRef ThicknessArg = Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);

	FValueRef AnisotropyArg = Em.ConstantFloat(0.0f);
	FValueRef F90Arg = Em.ConstantFloat3(FVector3f(1.0f, 1.0f, 1.0f));
	FValueRef SubsurfaceProfileArg = Em.ConstantFloat(0.0f);
	FValueRef SSSMFPArg = Em.ConstantFloat3(FVector3f(0.0f, 0.0f, 0.0f));
	FValueRef SSSMFPScaleArg = Em.ConstantFloat(1.0f);
	FValueRef SSSPhaseAnisotropyArg = Em.ConstantFloat(0.0f);
	FValueRef FuzzAmountArg = Em.ConstantFloat(0.0f);
	FValueRef FuzzColorArg = Em.ConstantFloat3(FVector3f(0.0f, 0.0f, 0.0f));
	FValueRef FuzzRoughnessArg = Em.ConstantFloat(0.5f);
	FValueRef GlintValueArg = Em.ConstantFloat(1.0f);
	FValueRef GlintUVArg = Em.ConstantFloat2(FVector2f(0.0f, 0.0f));
	FValueRef SpecularProfileArg = Em.ConstantFloat(0.0);
	FValueRef SubSurfaceTypeArg = Em.ConstantFloat(float(MSS_None));

	const bool bIsAtTheBottomOfTopology = SubstrateOperator.bIsBottom > 0;
	const bool bIsThinSurface = Material ? Material->IsThinSurface() : false;
	float ClearCoatUseSecondNormal = 0.0f; // Constant(ClearCoatBottomNormal != Normal ? 1.0f : 0.0f);SUBSTRATE_TODO

	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, Normal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	MIR::FSubstrateSlabDesc SubstrateSlabDesc = {
			NormalArg,
			DiffuseAlbedoArg,
			F0Arg,
			F90Arg,
			RoughnessArg,
			AnisotropyArg,
			SubsurfaceProfileArg,
			SSSMFPArg,
			SSSMFPScaleArg,
			SSSPhaseAnisotropyArg,
			SubSurfaceTypeArg,
			EmissiveColorArg,
			ClearCoatRoughnessArg,												//		Second roughness
			ClearCoatArg,														//		Second roughness weight
			Em.ConstantFloat(1.0f),												//		SecondRoughnessAsSimpleClearCoat = true
			Em.ConstantFloat(ClearCoatUseSecondNormal),							//		*SubstrateGetCastParameterCode(ClearCoatUseSecondNormal, MCT_Float),SUBSTRATE_TODO
			Em.ConstantFloat3(FVector3f::Zero()),								//		*SubstrateGetCastParameterCode(ClearCoatBottomNormal, MCT_Float3),SUBSTRATE_TODO
			FuzzAmountArg,
			FuzzColorArg,
			FuzzRoughnessArg,
			GlintValueArg,
			GlintUVArg,
			SpecularProfileArg,
			ThicknessArg,
			bIsThinSurface ? Em.ConstantTrue() : Em.ConstantFalse(),
			bIsAtTheBottomOfTopology ? Em.ConstantTrue() : Em.ConstantFalse(),
			SharedLocalBasisIndexMacroArg };

	FValueRef Slab = Em.SubstrateSlab(SubstrateSlabDesc);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, Slab);
}

void UMaterialExpressionSubstrateEyeBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef DiffuseColorArg		= CompileWithDefaultFloat3(Em, DiffuseColor,	0.18f, 0.18f, 0.18f);
	FValueRef RoughnessArg			= CompileWithDefaultFloat1(Em, Roughness,		0.5f);
	FValueRef EmissiveColorArg		= CompileWithDefaultFloat3(Em, EmissiveColor,	0.0f, 0.0f, 0.0f);
	FValueRef IrisMaskArg			= CompileWithDefaultFloat1(Em, IrisMask,		0.0f);
	FValueRef IrisDistanceArg		= CompileWithDefaultFloat1(Em, IrisDistance,	0.0f);
	FValueRef IrisNormalArg			= CompileWithDefaultNormalWS(Em, Material,		IrisNormal);
	FValueRef IrisPlaneNormalArg	= CompileWithDefaultNormalWS(Em, Material,		IrisPlaneNormal);

	FValueRef SubsurfaceProfileArg	= Em.ConstantFloat(0.0f);	// SUBSTRATE_TODO when profile is supported with NMT

	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, CorneaNormal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	FValueRef EyeBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateEyeBSDF($0, $1, $2, $3, $4, $5, ExtractSubsurfaceProfileInt($6), $7, $8)")
		},
		DiffuseColorArg,
		RoughnessArg,
		IrisMaskArg,
		IrisDistanceArg,
		IrisNormalArg,
		IrisPlaneNormalArg,
		SubsurfaceProfileArg,
		EmissiveColorArg,
		SharedLocalBasisIndexMacroArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, EyeBSDF);
}

void UMaterialExpressionSubstrateHairBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef EmissiveColorArg	= CompileWithDefaultFloat3(Em, EmissiveColor,	0.0f, 0.0f, 0.0f);
	FValueRef BaseColorArg		= CompileWithDefaultFloat3(Em, BaseColor,		0.18f, 0.18f, 0.18f);
	FValueRef RoughnessArg		= CompileWithDefaultFloat1(Em, Roughness,		0.5f);
	FValueRef SpecularArg		= CompileWithDefaultFloat1(Em, Specular,		0.5f);
	FValueRef ScatterArg		= CompileWithDefaultFloat1(Em, Scatter,			0.0f);
	FValueRef BacklitArg		= CompileWithDefaultFloat1(Em, Backlit,			0.0f);

	FValueRef TangentArg		= CompileWithDefaultTangentWS(Em, Material, Tangent);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &TangentArg, nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	FValueRef HairBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateHairBSDF($0, $1, $2, $3, $4, $5, false/*bComplexTransmittance*/, $6)")
		},
		BaseColorArg,
		ScatterArg,
		SpecularArg,
		RoughnessArg,
		BacklitArg,
		EmissiveColorArg,
		SharedLocalBasisIndexMacroArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, HairBSDF);
}

void UMaterialExpressionSubstrateVolumetricFogCloudBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);
	
	FValueRef EmissiveColorArg		= CompileWithDefaultFloat3(Em, EmissiveColor,		0.0f, 0.0f, 0.0f);
	FValueRef AlbedoArg				= CompileWithDefaultFloat3(Em, Albedo,				0.0f, 0.0f, 0.0f);
	FValueRef ExtinctionArg			= CompileWithDefaultFloat3(Em, Extinction,			0.0f, 0.0f, 0.0f);
	FValueRef AmbientOcclusionArg	= CompileWithDefaultFloat1(Em, AmbientOcclusion,	1.0f);

	bool bActualEmissiveOnly = bEmissiveOnly;
	UMaterial* BaseMaterial = Material->GetBaseMaterial();
	// Material is probably an instanced material, so we check that this is the case before potentially overriding the ShadingModel from the material instance.
	UMaterialInterface* BaseMaterialInterface = static_cast<UMaterialInterface*>(BaseMaterial);
	UMaterialInterface* MaterialInterface = Material;
	if (BaseMaterial && (BaseMaterialInterface != MaterialInterface))
	{
		FMaterialShadingModelField BaseMaterialShadingModels = BaseMaterial->GetShadingModels();
		FMaterialShadingModelField MaterialShadingModels = Material->GetShadingModels();

		// If the potentially instanced material does not have the same shading model as the instance material, this means it would have overridden the shading model.
		// From the UI, only a single shading model is selectable, so we simply apply the one coming form the material instance.
		if (MaterialShadingModels.IsValid() && MaterialShadingModels.CountShadingModels() == 1 && MaterialShadingModels != BaseMaterialShadingModels)
		{
			// Use the mode required by the overriden shading model.
			bActualEmissiveOnly = MaterialShadingModels.GetFirstShadingModel() == MSM_Unlit ? true : false;
		}
	}

	// Override to EmissiveOnly if that is the final shading model.
	if (bActualEmissiveOnly)
	{
		AlbedoArg = ExtinctionArg = Em.ConstantFloat3(FVector3f::ZeroVector);
		AmbientOcclusionArg = Em.ConstantFloat(1.0f);
	}

	FValueRef VolumetricFogCloudBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateVolumeFogCloudBSDF($0, $1, $2, $3)"),
		},
		AlbedoArg,
		ExtinctionArg,
		EmissiveColorArg,
		AmbientOcclusionArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, VolumetricFogCloudBSDF);
}

void UMaterialExpressionSubstrateSingleLayerWaterBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	if (SubstrateOperator.bUseParameterBlending)
	{
		return Em.Errorf(TEXT("Substrate SingleLayerWater BSDF node cannot be used with parameter blending."));
	}
	else if (SubstrateOperator.bRootOfParameterBlendingSubTree)
	{
		return Em.Errorf(TEXT("Substrate SingleLayerWater BSDF node cannot be the root of a parameter blending sub tree."));
	}

	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, Normal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	FValueRef BaseColorArg				= CompileWithDefaultFloat3(Em, BaseColor,				0.0f, 0.0f, 0.0f);
	FValueRef MetallicArg				= CompileWithDefaultFloat1(Em, Metallic,				0.0f);
	FValueRef SpecularArg				= CompileWithDefaultFloat1(Em, Specular,				0.5f);
	FValueRef RoughnessArg				= CompileWithDefaultFloat1(Em, Roughness,				0.5f);
	FValueRef EmissiveColorArg			= CompileWithDefaultFloat3(Em, EmissiveColor,			0.0f, 0.0f, 0.0f);
	FValueRef TopMaterialOpacityArg		= CompileWithDefaultFloat1(Em, TopMaterialOpacity,		0.0f);
	FValueRef WaterAlbedoArg			= CompileWithDefaultFloat3(Em, WaterAlbedo,				0.0f, 0.0f, 0.0f);
	FValueRef WaterExtinctionArg		= CompileWithDefaultFloat3(Em, WaterExtinction,			0.0f, 0.0f, 0.0f);
	FValueRef WaterPhaseGArg			= CompileWithDefaultFloat1(Em, WaterPhaseG,				0.0f);
	FValueRef ColorScaleBehindWaterArg	= CompileWithDefaultFloat3(Em, ColorScaleBehindWater,	1.0f, 1.0f, 1.0f);

	FValueRef SingleLayerWaterBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateSingleLayerWaterBSDF($0, $1, $2, $3, $4, $5, $6, $7, $8, $9, $10)"),
		},
		BaseColorArg,
		MetallicArg,
		SpecularArg,
		RoughnessArg,
		EmissiveColorArg,
		TopMaterialOpacityArg,
		WaterAlbedoArg,
		WaterExtinctionArg,
		WaterPhaseGArg,
		ColorScaleBehindWaterArg,
		SharedLocalBasisIndexMacroArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, SingleLayerWaterBSDF);
}

void UMaterialExpressionSubstrateUnlitBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef EmissiveColorArg = CompileWithDefaultFloat3(Em, EmissiveColor, 0.0f, 0.0f, 0.0f);
	FValueRef TransmittanceColorArg = CompileWithDefaultFloat3(Em, TransmittanceColor, 1.0f, 1.0f, 1.0f);
	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, Normal);

	FValueRef UnlitBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateUnlitBSDF($0, $1, $2)"),
		},
		EmissiveColorArg,
		TransmittanceColorArg,
		NormalArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, UnlitBSDF);
}

void UMaterialExpressionSubstrateToonBSDF::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef NormalArg = CompileWithDefaultNormalWS(Em, Material, Normal);
	const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NormalArg, nullptr);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

	FValueRef ToonProfileIdArg;
	if (ToonProfile != nullptr)
	{
 		const FName ToonProfileParameterName = ToonProfile::CreateToonProfileParameterName(ToonProfile);
		ToonProfileIdArg = Em.CastToFloat(Em.NamedPrimitiveUniform(ToonProfileParameterName, Em.ConstantFloat(0.0f)), 1);
	}
	else
	{
		ToonProfileIdArg = Em.ConstantFloat(0.0f);
	}

	FValueRef BaseColorArg = CompileWithDefaultFloat3(Em, BaseColor, 0.18f, 0.18f, 0.18f);
	FValueRef MetallicArg = CompileWithDefaultFloat1(Em, Metallic, 0.0f);
	FValueRef SpecularArg = CompileWithDefaultFloat1(Em, Specular, 0.5f);
	FValueRef RoughnessArg = CompileWithDefaultFloat1(Em, Roughness, 0.5f);
	FValueRef EmissiveColorArg = CompileWithDefaultFloat3(Em, EmissiveColor, 0.0f, 0.0f, 0.0f);

	// Unplugged PatternUVs result in UV0 being used, scaled by 10 to make offset/hatching textures visible by default.
	FValueRef PatternUVsArg = PatternUVs.GetTracedInput().Expression ? CompileWithDefaultFloat2(Em, PatternUVs, 0.0f, 0.0f) : Em.Multiply(MaterialToMIR::EmitTexCoord(Em, 0), Em.ConstantFloat(10.0f));

	const bool bIsAtTheBottomOfTopology = SubstrateOperator.bIsBottom > 0;
	MIR::FSubstrateToonDesc SubstrateToonDesc = {
			NormalArg,
			ToonProfileIdArg,
			BaseColorArg,
			MetallicArg,
			SpecularArg,
			RoughnessArg,
			EmissiveColorArg,
			PatternUVsArg,
			bIsAtTheBottomOfTopology ? Em.ConstantTrue() : Em.ConstantFalse(),
			SharedLocalBasisIndexMacroArg };

	FValueRef ToonBSDF = Em.SubstrateToon(SubstrateToonDesc);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, ToonBSDF);

}

void UMaterialExpressionSubstrateLightFunction::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef ColorArg = CompileWithDefaultFloat3(Em, Color, 0.0f, 0.0f, 0.0f);				// Light function color
	FValueRef TransmittanceColorArg = Em.ConstantFloat3(FVector3f(1.0f, 1.0f, 1.0f));		// Opacity / Transmittance is ignored by light functions
	FValueRef NormalArg = Em.ConstantFloat3(FVector3f(0.0f, 0.0f, 1.0f));					// place holder normal

	FValueRef UnlitBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateUnlitBSDF($0, $1, $2)"),
		},
		ColorArg,
		TransmittanceColorArg,
		NormalArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, UnlitBSDF);
}

void UMaterialExpressionSubstratePostProcess::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	FValueRef OpacityArg = CompileWithDefaultFloat1(Em, Opacity, 1.0f);
	FValueRef TransmittanceColorArg = Em.Saturate(Em.Subtract(Em.ConstantFloat(1.0f), OpacityArg));	// Transmittance from Opacity

	FValueRef ColorArg = CompileWithDefaultFloat3(Em, Color, 0.0f, 0.0f, 0.0f);						// Post process color
	FValueRef NormalArg = Em.ConstantFloat3(FVector3f(0.0f, 0.0f, 1.0f));							// place holder normal

	FValueRef UnlitBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::SubstrateData,
			TEXT("GetSubstrateUnlitBSDF($0, $1, $2)"),
		},
		ColorArg,
		TransmittanceColorArg,
		NormalArg);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, UnlitBSDF);
}

void UMaterialExpressionSubstrateUI::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();
	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	// Compile inputs with defaults like the legacy code did.
	FValueRef ColorArg = CompileWithDefaultFloat3(Em, Color, 0.0f, 0.0f, 0.0f);		// Color
	FValueRef OpacityArg = CompileWithDefaultFloat1(Em, Opacity, 1.0f);				// Opacity

	// Call the Substrate UI BSDF helper and finalize/promote the resulting BSDF as needed.
	FValueRef UIBSDF = Em.Extern<MIR::FExternSimpleHLSL>(
		{ MIR::EExternSimpleType::SubstrateData, TEXT("SubstrateCreateUIMaterial($0, $1)") },
		ColorArg,
		OpacityArg
	);

	FinalizeBSDFAndPromoteToOperatorIfNeeded(Em, SubstrateOperator, UIBSDF);
}

void UMaterialExpressionSubstrateConvertToDecal::Build(MIR::FEmitter& Em)
{
	if (!DecalMaterial.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing ConvertToDecal node input."));
	}

	FValueRef CoverageArg = CompileWithDefaultFloat1(Em, Coverage, 1.0f);

	// Note: we no longer do SubstrateTreeStackPush/Pop since the path guid is regenerated for every substrate node from BuildTopMaterialExpression.
	FValueRef DecalMaterialArg = Em.TryInput(&DecalMaterial);

	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();
	FGuid PathUniqueId = SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(PathUniqueId);

	if (!SubstrateOperator.bUseParameterBlending)
	{
		return Em.Errorf(TEXT("Substrate Convert To Decal node must receive SubstrateData a parameter blended Substrate material sub tree."));
	}
	if (!SubstrateOperator.bRootOfParameterBlendingSubTree)
	{
		return Em.Errorf(TEXT("Substrate Convert To Decal node must be the root of a parameter blending sub tree: no more Substrate operations can be applied a over its output."));
	}

	// Propagate the parameter blended normal
	FSubstrateOperator* Operator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
	SubstrateOperator.BSDFRegisteredSharedLocalBasis = Operator->BSDFRegisteredSharedLocalBasis;

	// Now emit the WEight operator acting as conversion to decal with coverage.
	if (!DecalMaterialArg.IsValid())
	{
		return Em.Errorf(TEXT("A input input graphs could not be evaluated for parameter blending."));
	}
	if (!CoverageArg.IsValid())
	{
		return Em.Errorf(TEXT("Weight input graphs could not be evaluated for parameter blending."));
	}

	check(SubstrateOperator.Index != INDEX_NONE);
	check(SubstrateOperator.BSDFIndex != INDEX_NONE);

	FValueRef DecalMaterialWeightOpArg = Em.SubstrateCoverageWeightParameterBlending(
		DecalMaterialArg,
		CoverageArg);

	FValueRef PromotedDecalMaterialWeightOpArg = Em.SubstratePromoteToOperator(
		DecalMaterialWeightOpArg,
		Em.ConstantInt(SubstrateOperator.Index),
		Em.ConstantInt(SubstrateOperator.BSDFIndex),
		Em.ConstantInt(SubstrateOperator.LayerDepth),
		Em.ConstantInt(SubstrateOperator.bIsBottom ? 1 : 0));

	Em.Output(0, PromotedDecalMaterialWeightOpArg);
}

static FValueRef SubstrateBlendNormal(MIR::FEmitter& Em, FValueRef NormalCodeChunk0, FValueRef NormalCodeChunk1, FValueRef MixCodeChunk)
{
	FValueRef SafeMixCodeChunk = Em.Saturate(MixCodeChunk);
	FValueRef LerpedNormal = Em.Lerp(NormalCodeChunk0, NormalCodeChunk1, SafeMixCodeChunk);
	FValueRef BlendedNormalCodeChunk = Em.Divide(LerpedNormal, Em.Sqrt(Em.Dot(LerpedNormal, LerpedNormal)));
	return BlendedNormalCodeChunk;
}

void UMaterialExpressionSubstrateHorizontalMixing::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	if (!Foreground.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing Foreground input"));
	}
	if (!Background.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing Background input"));
	}

	// Note: we no longer do SubstrateTreeStackPush/Pop since the path guid is regenerated for every substrate node from BuildTopMaterialExpression.
	FValueRef BackgroundArg = Em.TryInput(&Background);
	FValueRef ForegroundArg = Em.TryInput(&Foreground);

	FValueRef HorizontalMixArg = CompileWithDefaultFloat1(Em, Mix, 0.5f);

	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		if (!ForegroundArg.IsValid())
		{
			return Em.Errorf(TEXT("Foreground input graphs could not be evaluated for parameter blending."));
		}
		if (!BackgroundArg.IsValid())
		{
			return Em.Errorf(TEXT("Background input graphs could not be evaluated for parameter blending."));
		}

		FValueRef NormalMixArg = Em.Extern<MIR::FExternSimpleHLSL>(
			{
				MIR::EExternSimpleType::Float1,
				TEXT("HorizontalMixingParameterBlendingBSDFCoverageToNormalMix($0, $1, $2)")
			},
			BackgroundArg,
			ForegroundArg,
			HorizontalMixArg);

		FSubstrateOperator* BackgroundBSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* ForegroundBSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!BackgroundBSDFOperator || !ForegroundBSDFOperator)
		{
			return Em.Errorf(TEXT("Missing input on Horizontal blending node."));
		}

		FSubstrateRegisteredSharedLocalBasisSearchResult ForegroundBasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, ForegroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);
		FSubstrateRegisteredSharedLocalBasisSearchResult BackgroundBasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, BackgroundBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);

		if (!ForegroundBasis.bFound || !BackgroundBasis.bFound)
		{
			Em.Errorf(TEXT("Foreground & Background shared local basis could not be found."));
			return;
		}

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		FValueRef NewNormalArg = SubstrateBlendNormal(Em, BackgroundBasis.BasisInfo.NormalArg, ForegroundBasis.BasisInfo.NormalArg, NormalMixArg);

		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		FValueRef NewTangentArg;
		if (ForegroundBasis.BasisInfo.TangentArg && BackgroundBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = SubstrateBlendNormal(Em, BackgroundBasis.BasisInfo.TangentArg, ForegroundBasis.BasisInfo.TangentArg, NormalMixArg);
		}
		else if (ForegroundBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = ForegroundBasis.BasisInfo.TangentArg;
		}
		else if (BackgroundBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = BackgroundBasis.BasisInfo.TangentArg;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NewNormalArg, NewTangentArg.IsValid() ? &NewTangentArg : nullptr);

		FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

		if (!ForegroundArg.IsValid() || !BackgroundArg.IsValid() || !HorizontalMixArg.IsValid() || !NormalMixArg.IsValid())
		{
			Em.Errorf(TEXT("Invalid Horizontal operator Foreground, Background, Mix or NormalMix input."));
		}

		FValueRef CameraVectorArg = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_CameraVector);
		FValueRef BackgroundNoV = Em.Saturate(Em.Dot(BackgroundBasis.BasisInfo.NormalArg, CameraVectorArg));
		FValueRef ForegroundNoV = Em.Saturate(Em.Dot(ForegroundBasis.BasisInfo.NormalArg, CameraVectorArg));

		const FSubstrateOperator* PromoteToOperator = SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr;
		if (PromoteToOperator)
		{
			check(PromoteToOperator->Index != INDEX_NONE);
			check(PromoteToOperator->BSDFIndex != INDEX_NONE);

			FValueRef MixedSlab = Em.SubstrateHorizontalMixingParameterBlending(
				BackgroundArg,
				ForegroundArg,
				HorizontalMixArg,
				NormalMixArg,
				SharedLocalBasisIndexMacroArg,
				BackgroundNoV,
				ForegroundNoV);

			FValueRef PromotedSlab = Em.SubstratePromoteToOperator(
				MixedSlab,
				Em.ConstantInt(PromoteToOperator->Index),
				Em.ConstantInt(PromoteToOperator->BSDFIndex),
				Em.ConstantInt(PromoteToOperator->LayerDepth),
				Em.ConstantInt(PromoteToOperator->bIsBottom ? 1 : 0));

			Em.Output(0, PromotedSlab);
		}
		else
		{
			FValueRef MixedSlab = Em.SubstrateHorizontalMixingParameterBlending(
				BackgroundArg,
				ForegroundArg,
				HorizontalMixArg,
				NormalMixArg,
				SharedLocalBasisIndexMacroArg,
				BackgroundNoV,
				ForegroundNoV);

			Em.Output(0, MixedSlab);
		}

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		if (!ForegroundArg.IsValid() || !BackgroundArg.IsValid() || !HorizontalMixArg.IsValid())
		{
			Em.Errorf(TEXT("Invalid Horizontal operator Foreground, Background or Mix input."));
		}

		FValueRef MixedSlabs = Em.SubstrateHorizontalMixing(
				BackgroundArg,
				ForegroundArg,
				HorizontalMixArg,
				Em.ConstantInt(SubstrateOperator.Index),
				Em.ConstantInt(SubstrateOperator.MaxDistanceFromLeaves));

		Em.Output(0, MixedSlabs);
	}
}


void UMaterialExpressionSubstrateVerticalLayering::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	if (!Top.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing Foreground input"));
	}
	if (!Base.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing Background input"));
	}

	// Note: we no longer do SubstrateTreeStackPush/Pop since the path guid is regenerated for every substrate node from BuildTopMaterialExpression.
	FValueRef TopArg = Em.TryInput(&Top);
	FValueRef BaseArg = Em.TryInput(&Base);

	FValueRef ThicknessArg = CompileWithDefaultFloat1(Em, Thickness, SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);

	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId());

	if (SubstrateOperator.bUseParameterBlending)
	{
		if (!TopArg.IsValid())
		{
			return Em.Errorf(TEXT("Foreground input graphs could not be evaluated for parameter blending."));
		}
		if (!BaseArg.IsValid())
		{
			return Em.Errorf(TEXT("Background input graphs could not be evaluated for parameter blending."));
		}

		FValueRef NormalMixArg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("VerticalLayeringParameterBlendingBSDFCoverageToNormalMix($0)") }, TopArg);

		FSubstrateOperator* TopBSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* BaseBSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!TopBSDFOperator || !BaseBSDFOperator)
		{
			return Em.Errorf(TEXT("Missing input on Vertical blending node."));
		}

		FSubstrateRegisteredSharedLocalBasisSearchResult BaseBasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, BaseBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);
		FSubstrateRegisteredSharedLocalBasisSearchResult TopBasis  = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, TopBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash,  TopBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);

		if (!BaseBasis.bFound || !TopBasis.bFound)
		{
			Em.Errorf(TEXT("Top & Base shared local basis could not be found."));
			return;
		}

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		FValueRef NewNormalArg = SubstrateBlendNormal(Em, BaseBasis.BasisInfo.NormalArg, TopBasis.BasisInfo.NormalArg, NormalMixArg);

		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		FValueRef NewTangentArg;
		if (TopBasis.BasisInfo.TangentArg && BaseBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = SubstrateBlendNormal(Em, BaseBasis.BasisInfo.TangentArg, TopBasis.BasisInfo.TangentArg, NormalMixArg);
		}
		else if (TopBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = TopBasis.BasisInfo.TangentArg;
		}
		else if (BaseBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = BaseBasis.BasisInfo.TangentArg;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NewNormalArg, NewTangentArg.IsValid() ? &NewTangentArg : nullptr);

		FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

		if (!TopArg.IsValid() || !BaseArg.IsValid() || !ThicknessArg.IsValid() || !NormalMixArg.IsValid())
		{
			Em.Errorf(TEXT("Invalid Vertical operator Base, Top, Thickness or NormalMix input."));
		}

		FValueRef CameraVectorArg = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_CameraVector);
		FValueRef TopNoV = Em.Saturate(Em.Dot(TopBasis.BasisInfo.NormalArg, CameraVectorArg));
		FValueRef BaseNoV = Em.Saturate(Em.Dot(BaseBasis.BasisInfo.NormalArg, CameraVectorArg));

		const FSubstrateOperator* PromoteToOperator = SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr;
		if (PromoteToOperator)
		{
			check(PromoteToOperator->Index != INDEX_NONE);
			check(PromoteToOperator->BSDFIndex != INDEX_NONE);

			FValueRef LayeredSlab = Em.SubstrateVerticalLayeringParameterBlending(
				TopArg,
				BaseArg,
				SharedLocalBasisIndexMacroArg,
				TopNoV,
				BaseNoV);

			FValueRef PromotedSlab = Em.SubstratePromoteToOperator(
				LayeredSlab,
				Em.ConstantInt(PromoteToOperator->Index),
				Em.ConstantInt(PromoteToOperator->BSDFIndex),
				Em.ConstantInt(PromoteToOperator->LayerDepth),
				Em.ConstantInt(PromoteToOperator->bIsBottom ? 1 : 0));

			Em.Output(0, PromotedSlab);
		}
		else
		{
			FValueRef LayeredSlab = Em.SubstrateVerticalLayeringParameterBlending(
				TopArg,
				BaseArg,
				SharedLocalBasisIndexMacroArg,
				TopNoV,
				BaseNoV);

			Em.Output(0, LayeredSlab);
		}

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		if (!TopArg.IsValid() || !BaseArg.IsValid())
		{
			Em.Errorf(TEXT("Invalid Vertical operator Top, Base or Thickness input."));
		}

		FValueRef LayeredSlabs = Em.SubstrateVerticalLayering(
				TopArg,
				BaseArg,
				Em.ConstantInt(SubstrateOperator.Index),
				Em.ConstantInt(SubstrateOperator.MaxDistanceFromLeaves));

		Em.Output(0, LayeredSlabs);
	}
}

void UMaterialExpressionSubstrateWeight::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();
	if (!A.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing A input"));
	}

	// Note: we no longer do SubstrateTreeStackPush/Pop since the path guid is regenerated for every substrate node from BuildTopMaterialExpression.
	FValueRef AInputArg = Em.TryInput(&A);
	FValueRef WeightArg = CompileWithDefaultFloat1(Em, Weight, 1.0f);

	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId());

	if (!AInputArg.IsValid())
	{
		return Em.Errorf(TEXT("A input input graphs could not be evaluated for parameter blending."));
	}
	if (!WeightArg.IsValid())
	{
		return Em.Errorf(TEXT("Weight input graphs could not be evaluated for parameter blending."));
	}

	if (SubstrateOperator.bUseParameterBlending)
	{
		FSubstrateOperator* ABSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = ABSDFOperator->BSDFRegisteredSharedLocalBasis;

		const FSubstrateOperator* PromoteToOperator = SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr;
		if (PromoteToOperator)
		{
			check(PromoteToOperator->Index != INDEX_NONE);
			check(PromoteToOperator->BSDFIndex != INDEX_NONE);

			FValueRef LayeredSlab = Em.SubstrateCoverageWeightParameterBlending(
				AInputArg,
				WeightArg);

			FValueRef PromotedSlab = Em.SubstratePromoteToOperator(
				LayeredSlab,
				Em.ConstantInt(PromoteToOperator->Index),
				Em.ConstantInt(PromoteToOperator->BSDFIndex),
				Em.ConstantInt(PromoteToOperator->LayerDepth),
				Em.ConstantInt(PromoteToOperator->bIsBottom ? 1 : 0));

			Em.Output(0, PromotedSlab);
		}
		else
		{
			FValueRef LayeredSlab = Em.SubstrateCoverageWeightParameterBlending(
				AInputArg,
				WeightArg);

			Em.Output(0, LayeredSlab);
		}

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		FValueRef LayeredSlabs = Em.SubstrateCoverageWeight(
			AInputArg,
			WeightArg,
			Em.ConstantInt(SubstrateOperator.Index),
			Em.ConstantInt(SubstrateOperator.MaxDistanceFromLeaves));

		Em.Output(0, LayeredSlabs);
	}
}

void UMaterialExpressionSubstrateAdd::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	if (!A.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing A input"));
	}
	if (!B.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing B input"));
	}

	// Note: we no longer do SubstrateTreeStackPush/Pop since the path guid is regenerated for every substrate node from BuildTopMaterialExpression.
	FValueRef BArg = Em.TryInput(&B);
	FValueRef AArg = Em.TryInput(&A);

	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		if (!AArg.IsValid())
		{
			return Em.Errorf(TEXT("A input graphs could not be evaluated for parameter blending."));
		}
		if (!BArg.IsValid())
		{
			return Em.Errorf(TEXT("B input graphs could not be evaluated for parameter blending."));
		}

		FValueRef NormalMixArg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("AddParameterBlendingBSDFCoverageToNormalMix($0, $1)") },
			BArg,
			AArg);

		FSubstrateOperator* BBSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* ABSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!BBSDFOperator || !ABSDFOperator)
		{
			return Em.Errorf(TEXT("Missing input on Add blending node."));
		}

		FSubstrateRegisteredSharedLocalBasisSearchResult ABasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, ABSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, ABSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);
		FSubstrateRegisteredSharedLocalBasisSearchResult BBasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, BBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, BBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);

		if (!ABasis.bFound || !BBasis.bFound)
		{
			Em.Errorf(TEXT("A & B shared local basis could not be found."));
			return;
		}

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		FValueRef NewNormalArg = SubstrateBlendNormal(Em, BBasis.BasisInfo.NormalArg, ABasis.BasisInfo.NormalArg, NormalMixArg);

		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		FValueRef NewTangentArg;
		if (ABasis.BasisInfo.TangentArg && BBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = SubstrateBlendNormal(Em, BBasis.BasisInfo.TangentArg, ABasis.BasisInfo.TangentArg, NormalMixArg);
		}
		else if (ABasis.BasisInfo.TangentArg)
		{
			NewTangentArg = ABasis.BasisInfo.TangentArg;
		}
		else if (BBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = BBasis.BasisInfo.TangentArg;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NewNormalArg, NewTangentArg.IsValid() ? &NewTangentArg : nullptr);

		FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

		if (!AArg.IsValid() || !BArg.IsValid() || !NormalMixArg.IsValid())
		{
			Em.Errorf(TEXT("Invalid Add operator A, B, Mix or NormalMix input."));
		}

		FValueRef CameraVectorArg = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_CameraVector);
		FValueRef BNoV = Em.Saturate(Em.Dot(BBasis.BasisInfo.NormalArg, CameraVectorArg));
		FValueRef ANoV = Em.Saturate(Em.Dot(ABasis.BasisInfo.NormalArg, CameraVectorArg));

		const FSubstrateOperator* PromoteToOperator = SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr;
		if (PromoteToOperator)
		{
			check(PromoteToOperator->Index != INDEX_NONE);
			check(PromoteToOperator->BSDFIndex != INDEX_NONE);

			FValueRef AddedSlabs = Em.SubstrateAddParameterBlending(
				BArg,
				AArg,
				NormalMixArg,
				SharedLocalBasisIndexMacroArg,
				BNoV,
				ANoV);

			FValueRef PromotedSlab = Em.SubstratePromoteToOperator(
				AddedSlabs,
				Em.ConstantInt(PromoteToOperator->Index),
				Em.ConstantInt(PromoteToOperator->BSDFIndex),
				Em.ConstantInt(PromoteToOperator->LayerDepth),
				Em.ConstantInt(PromoteToOperator->bIsBottom ? 1 : 0));

			Em.Output(0, PromotedSlab);
		}
		else
		{
			FValueRef AddedSlabs = Em.SubstrateAddParameterBlending(
				BArg,
				AArg,
				NormalMixArg,
				SharedLocalBasisIndexMacroArg,
				BNoV,
				ANoV);

			Em.Output(0, AddedSlabs);
		}

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		if (!AArg.IsValid() || !BArg.IsValid())
		{
			Em.Errorf(TEXT("Invalid Horizontal operator A, B or Mix input."));
		}

		FValueRef AddedSlabs = Em.SubstrateAdd(
			BArg,
			AArg,
			Em.ConstantInt(SubstrateOperator.Index),
			Em.ConstantInt(SubstrateOperator.MaxDistanceFromLeaves));

		Em.Output(0, AddedSlabs);
	}
}

void UMaterialExpressionSubstrateSelect::Build(MIR::FEmitter& Em)
{
	if (!A.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing A input"));
	}
	if (!B.GetTracedInput().Expression)
	{
		return Em.Errorf(TEXT("Missing B input"));
	}

	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();
	FSubstrateOperator& SubstrateOperator = SubstrateTranslatorData->SubstrateCompilationGetOperator(SubstrateTranslatorData->SubstrateTreeStackGetPathUniqueId());
	if (SubstrateOperator.bUseParameterBlending)
	{
		// Note: we no longer do SubstrateTreeStackPush/Pop since the path guid is regenerated for every substrate node from BuildTopMaterialExpression.
		FValueRef BArg = Em.TryInput(&B);
		FValueRef AArg = Em.TryInput(&A);

		if (!AArg.IsValid())
		{
			return Em.Errorf(TEXT("A input graphs could not be evaluated for parameter blending."));
		}
		if (!BArg.IsValid())
		{
			return Em.Errorf(TEXT("B input graphs could not be evaluated for parameter blending."));
		}

		// if SelectValue is not pluggedin, need to be 0. Otherwise it must be a float value.
		FValueRef SelectValueArg = CompileWithDefaultFloat1(Em, SelectValue, 0.0f);

		FValueRef ThresholdArg = Em.ConstantFloat(Threshold);
		FValueRef FinalSelectValueArg = Em.Select(Em.LessThanOrEquals(SelectValueArg, ThresholdArg), Em.ConstantZero(MIR::EScalarKind::Float), Em.ConstantOne(MIR::EScalarKind::Float));

		if (!AArg.IsValid() || !BArg.IsValid() || !ThresholdArg.IsValid())
		{
			return Em.Errorf(TEXT("Invalid Select operator A, B or Threshold input."));
		}

		auto SubstrateSelectNormal = [&](FValueRef NormalA, FValueRef NormalB)
			{
				return Em.Select(Em.LessThanOrEquals(SelectValueArg, ThresholdArg), NormalA, NormalB);
			};

		FSubstrateOperator* BBSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.LeftIndex);
		FSubstrateOperator* ABSDFOperator = SubstrateTranslatorData->SubstrateCompilationGetOperatorFromIndex(SubstrateOperator.RightIndex);
		if (!BBSDFOperator || !ABSDFOperator)
		{
			return Em.Errorf(TEXT("Missing input on Select node."));
		}

		FSubstrateRegisteredSharedLocalBasisSearchResult ABasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, ABSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, ABSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);
		FSubstrateRegisteredSharedLocalBasisSearchResult BBasis = SubstrateCompilationFindExistingSharedLocalBasis(SubstrateTranslatorData, BBSDFOperator->BSDFRegisteredSharedLocalBasis.NormalCodeChunkHash, BBSDFOperator->BSDFRegisteredSharedLocalBasis.TangentCodeChunkHash);

		if (!ABasis.bFound || !BBasis.bFound)
		{
			Em.Errorf(TEXT("A & B shared local basis could not be found."));
			return;
		}

		// Compute the new Normal and Tangent resulting from the blending using code chunk
		FValueRef NewNormalArg = SubstrateSelectNormal(BBasis.BasisInfo.NormalArg, ABasis.BasisInfo.NormalArg);

		// The tangent is optional so we treat it differently if INDEX_NONE is specified
		FValueRef NewTangentArg;
		if (ABasis.BasisInfo.TangentArg && BBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = SubstrateSelectNormal(BBasis.BasisInfo.TangentArg, ABasis.BasisInfo.TangentArg);
		}
		else if (ABasis.BasisInfo.TangentArg)
		{
			NewTangentArg = ABasis.BasisInfo.TangentArg;
		}
		else if (BBasis.BasisInfo.TangentArg)
		{
			NewTangentArg = BBasis.BasisInfo.TangentArg;
		}
		const FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis = MIRSubstrateCompilationInfoCreateSharedLocalBasis2(SubstrateTranslatorData, &NewNormalArg, NewTangentArg.IsValid() ? &NewTangentArg : nullptr);

		FValueRef SharedLocalBasisIndexMacroArg = EmitSubstrateSharedLocalBasis(Em, NewRegisteredSharedLocalBasis, SubstrateTranslatorData->CurrentSubstrateCompilationContext);

		const FSubstrateOperator* PromoteToOperator = SubstrateOperator.bRootOfParameterBlendingSubTree ? &SubstrateOperator : nullptr;
		if (PromoteToOperator)
		{
			check(PromoteToOperator->Index != INDEX_NONE);
			check(PromoteToOperator->BSDFIndex != INDEX_NONE);

			FValueRef SelectedSlab = Em.SubstrateSelectParameterBlending(
				AArg,
				BArg,
				FinalSelectValueArg,
				SharedLocalBasisIndexMacroArg);

			FValueRef PromotedSlab = Em.SubstratePromoteToOperator(
				SelectedSlab,
				Em.ConstantInt(PromoteToOperator->Index),
				Em.ConstantInt(PromoteToOperator->BSDFIndex),
				Em.ConstantInt(PromoteToOperator->LayerDepth),
				Em.ConstantInt(PromoteToOperator->bIsBottom ? 1 : 0));

			Em.Output(0, PromotedSlab);
		}
		else
		{
			FValueRef SelectedSlab = Em.SubstrateSelectParameterBlending(
				AArg,
				BArg,
				FinalSelectValueArg,
				SharedLocalBasisIndexMacroArg);

			Em.Output(0, SelectedSlab);
		}

		// Propagate the parameter blended normal
		SubstrateOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	}
	else
	{
		return Em.Errorf(TEXT("The Select node can only use parameter blending to only select between one of two BSDF."));
	}
}

void UMaterialExpressionSubstrateTransmittanceToMFP::Build(MIR::FEmitter& Em)
{
	FSubstrateTranslatorData* SubstrateTranslatorData = Em.GetSubstrateTranslatorData();

	FValueRef TransmittanceColorArg	= TransmittanceColor.GetTracedInput().Expression	? Em.TryInput(&TransmittanceColor)	: Em.ConstantFloat(0.5f);
	FValueRef ThicknessArg			= Thickness.GetTracedInput().Expression				? Em.TryInput(&Thickness)			: Em.ConstantFloat(SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM);

	if (!TransmittanceColorArg.IsValid())
	{
		Em.Errorf(TEXT("TransmittanceColor input graph could not be evaluated for TransmittanceToMFP."));
		return;
	}
	if (!ThicknessArg.IsValid())
	{
		Em.Errorf(TEXT("Thickness input graph could not be evaluated for TransmittanceToMFP."));
		return;
	}

	MIR::FValueRef MFPArg = Em.Extern<MIR::FExternSimpleHLSL>(
		{
			MIR::EExternSimpleType::Float3,
			TEXT("(TransmittanceToMeanFreePath($0, $1 * CENTIMETER_TO_METER) * METER_TO_CENTIMETER)")
		}, TransmittanceColorArg, ThicknessArg);

	Em.Output(0, MFPArg);
	Em.Output(1, ThicknessArg);
}

void UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0::Build(MIR::FEmitter& Em)
{
	FValueRef BaseColorArg	= BaseColor.GetTracedInput().Expression	? Em.TryInput(&BaseColor)	: Em.ConstantFloat(0.18f);
	FValueRef SpecularArg	= Specular.GetTracedInput().Expression	? Em.TryInput(&Specular)	: Em.ConstantFloat(0.5f);
	FValueRef MetallicArg	= Metallic.GetTracedInput().Expression	? Em.TryInput(&Metallic)	: Em.ConstantFloat(0.0f);

	MIR::FValueRef DiffuseAlbedoArg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("ComputeDiffuseAlbedo($0, saturate($1))") }, BaseColorArg, MetallicArg);
	MIR::FValueRef F0Arg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("ComputeF0($0, $1, saturate($2))") }, SpecularArg, BaseColorArg, MetallicArg);

	Em.Output(0, DiffuseAlbedoArg);
	Em.Output(1, F0Arg);
}

void UMaterialExpressionSubstrateHazinessToSecondaryRoughness::Build(MIR::FEmitter& Em)
{
	FValueRef BaseRoughnessArg	= BaseRoughness.GetTracedInput().Expression ?	Em.TryInput(&BaseRoughness)	: Em.ConstantFloat(0.1f);
	FValueRef HazinessArg		= Haziness.GetTracedInput().Expression ?		Em.TryInput(&Haziness)		: Em.ConstantFloat(0.5f);

	MIR::FValueRef HazeRoughnessArg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("SubstrateComputeHazeRoughness(saturate($0))") }, BaseRoughnessArg);
	MIR::FValueRef HazeWeightArg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float1, TEXT("SubstrateComputeHazeWeight(saturate($0), saturate($1))") }, BaseRoughnessArg, HazinessArg);

	Em.Output(0, HazeRoughnessArg);
	Em.Output(1, HazeWeightArg);
}

void UMaterialExpressionSubstrateThinFilm::Build(MIR::FEmitter& Em)
{
	UMaterial* DummyMaterial = nullptr;
	bool bConvertToRequestedSpace = false;
	FValueRef NormalArg		= CompileWithDefaultNormalWS(Em, DummyMaterial, Normal, bConvertToRequestedSpace);

	FValueRef F0Arg			= F0.GetTracedInput().Expression ?			Em.TryInput(&F0)		: Em.ConstantFloat(0.04f);
	FValueRef F90Arg		= F90.GetTracedInput().Expression ?			Em.TryInput(&F90)		: Em.ConstantFloat(1.0f);
	FValueRef ThicknessArg	= Thickness.GetTracedInput().Expression ?	Em.TryInput(&Thickness)	: Em.ConstantFloat(1.0f);
	FValueRef IORArg		= IOR.GetTracedInput().Expression ?			Em.TryInput(&IOR)		: Em.ConstantFloat(1.44f);

	FValueRef CameraVector = Em.Extern<MIR::FExternFromMaterialDecl>(NAME_CameraVector);

	MIR::FValueRef OutF0Arg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("SubstrateGetThinFilmF0F90($0, $1, $2, $3, $4).F0") },
		Em.Dot(NormalArg, CameraVector),
		F0Arg,
		F90Arg,
		ThicknessArg,
		IORArg);

	// Calling SubstrateGetThinFilmF0F90 a second time. SUBSTRATE_TODO: create a structure to pull the data from the first call instead
	MIR::FValueRef OutF90Arg = Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("SubstrateGetThinFilmF0F90($0, $1, $2, $3, $4).F90") },
		Em.Dot(NormalArg, CameraVector),
		F0Arg,
		F90Arg,
		ThicknessArg,
		IORArg);

	Em.Output(0, OutF0Arg);
	Em.Output(1, OutF90Arg);
}

#endif // WITH_EDITOR
