// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Materials/MaterialExpressionsToMIRCommon.h"
#include "Materials/MaterialIREmitter.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRExtern.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionNamedReroute.h"

namespace MaterialToMIR
{

MIR::FValueRef EmitTexCoord(MIR::FEmitter& Em, int32 Index)
{
	check(Index >= 0);
	if (Index >= 8)
	{
		Em.Errorf(TEXT("Texture coordinate %d out of bounds (max is 8)"), Index);
		return Em.Poison();
	}

	struct FTexCoord
	{
		int32 Index;

		MIR::FExternInfo GetInfo() const
		{
			return
			{
				.Name  = TEXTVIEW("TexCoord"),
				.Type  = MIR::FType::MakeFloatVector(2),
				.Flags = MIR::EExternFlags::Inline,
			};
		}

		void AnalyzeInStage(MIR::FExternAnalysisContext& Context, MIR::EStage Stage)
		{
			// @massimo.tristano I think we can avoid fully interpolating texture coordinates when they're
			// only used inside a vertex stage. We should make a different #define to store the number of
			// texture coordinates we need in the MaterialVertexParameter, so that it is not automatically
			// interpolated for the pixel shader, saving perf. 
			// 	 if (Stage == MIR::EStage::Stage_Vertex)
			// 	 {
			//       <here bump the number of texcoords only used in vertex shader>
			//       return;
			// 	 }

			// @massimo.tristano this follows the logic in the old translator. A better way would instead be
			// to **allocate** an index to a texture coordinate based on usage and map that index here. 
			FMaterialIRModule::FStatistics& Statistics = Context.Module->GetStatistics();
			Statistics.NumInterpolatedTexCoords = FMath::Max(Statistics.NumInterpolatedTexCoords, Index + 1);
			Statistics.InterpolatedTexCoordsMask |= 1 << Index;
		}
	
		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			// Parameters.TexCoords<_DD?>[<Index>]
			Printer << TEXTVIEW("Parameters.TexCoords")
					<< MIR::FExternTag_IfDifferential(TEXTVIEW("_"))
				    << MIR::ExternTag_DD
					<< TEXT('[') << Index << TEXT(']');
		}
	};

	return Em.Extern<FTexCoord>({ Index });
}

MIR::FValueRef EmitPixelNormalWS(MIR::FEmitter& Em)
{
	static const MIR::FExternInlineDeclaration Decl{ .Type =MIR::FType::MakeFloatVector(3), .CodeHLSL = TEXT("Parameters.WorldNormal"), .GraphProperties = MIR::EGraphProperties::ReadsPixelNormal };
	return Em.Extern<MIR::FExternFromInlineDecl>(&Decl);
}

MIR::FValueRef EmitVertexNormal(MIR::FEmitter& Em)
{
	static const FName NAME_VertexNormal("VertexNormal");
	return Em.Extern<MIR::FExternFromMaterialDecl>(NAME_VertexNormal);
}

MIR::FValueRef EmitVertexTangent(MIR::FEmitter& Em)
{
	static const FName NAME_VertexTangent("VertexTangent");
	return Em.Extern<MIR::FExternFromMaterialDecl>(NAME_VertexTangent);
}

MIR::FValueRef EmitCustomPrimitiveDataFloat1(MIR::FEmitter& Em, int32 Index)
{
	struct FCustomPrimitiveExtern
	{
		int32 Index;

		MIR::FExternInfo GetInfo() const
		{
			return
				{
					.Name = TEXT("CustomPrimitive"),
					.Type = MIR::FType::MakeFloatScalar(),
					.Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials,
				};
		}

		void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
		{
			if (Index >= FCustomPrimitiveData::NumCustomPrimitiveDataFloats)
			{
				Printer << TEXT("0.0f");
				return;
			}

			const int32 CustomDataIndex = Index / 4;
			const int32 ElementIndex    = Index % 4; // Index x, y, z or w

			Printer.Appendf(TEXT("GetPrimitiveData(Parameters).CustomPrimitiveData[%d][%d]"), CustomDataIndex, ElementIndex);
		}

		void EmitDebugInfo(FString& Out) const
		{
			Out.Appendf(TEXT("Index=%d"), Index);
		}
	};

	if (Em.GetMaterialInterface()->GetMaterial()->MaterialDomain == MD_UI)
	{
		Em.Error(TEXT("Custom Primitive Data cannot be usd in materials with the UI domain."));
		return Em.Poison();
	}

	check(Index >= 0);
	return Em.Extern<FCustomPrimitiveExtern>({ Index });
}

MIR::FValueRef EmitCustomPrimitiveDataFloat4(MIR::FEmitter& Em, int32 Index)
{
	return Em.Vector4(
		EmitCustomPrimitiveDataFloat1(Em, Index + 0),
		EmitCustomPrimitiveDataFloat1(Em, Index + 1),
		EmitCustomPrimitiveDataFloat1(Em, Index + 2),
		EmitCustomPrimitiveDataFloat1(Em, Index + 3));
}

static bool TryGetRerouteInput(UMaterialExpressionRerouteBase* Reroute, FExpressionInput& OutInput)
{
	if (UMaterialExpressionReroute* SimpleReroute = Cast<UMaterialExpressionReroute>(Reroute))
	{
		OutInput = SimpleReroute->Input;
		return true;
	}
	if (UMaterialExpressionNamedRerouteDeclaration* Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(Reroute))
	{
		OutInput = Declaration->Input;
		return true;
	}
	if (UMaterialExpressionNamedRerouteUsage* Usage = Cast<UMaterialExpressionNamedRerouteUsage>(Reroute))
	{
		if (Usage->IsDeclarationValid())
		{
			OutInput = Usage->Declaration->Input;
			return true;
		}
	}
	return false;
}

void CheckInputIsConnected(MIR::FEmitter& Em, FExpressionInput& Input, const TCHAR* InputName)
{
	// Iteratively trace through reroute nodes to find the real connected expression.
	FExpressionInput TracedInput = Input;
	TArray<const UMaterialExpression*, TInlineAllocator<4>> Visited;
	while (UMaterialExpressionRerouteBase* Reroute = Cast<UMaterialExpressionRerouteBase>(TracedInput.Expression))
	{
		if (Visited.Contains(Reroute))
		{
			TracedInput.Expression = nullptr;
			break;
		}
		
		Visited.Add(Reroute);

		if (!TryGetRerouteInput(Reroute, TracedInput))
		{
			TracedInput.Expression = nullptr;
			break;
		}
	}

	if (!TracedInput.Expression)
	{
		Em.Errorf(TEXT("Input '%s' must be connected."), InputName);
	}
}

} // MaterialToMIR
#endif // WITH_EDITOR
