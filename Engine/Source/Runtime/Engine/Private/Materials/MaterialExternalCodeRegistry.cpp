// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExternalCodeRegistry.h"
#include "Misc/Parse.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExternalCodeRegistry)


/*
 * FMaterialExternalCodeEnvironmentDefine
 */

void FMaterialExternalCodeEnvironmentDefine::UpdateHash(FXxHash64Builder& Hasher) const
{
	FString NameString = Name.ToString();
	Hasher.Update(*NameString, NameString.NumBytesWithoutNull());
	Hasher.Update(&ShaderFrequency, sizeof(ShaderFrequency));
}


/*
 * FMaterialExternalCodeDeclaration 
 */

void FMaterialExternalCodeDeclaration::UpdateHash(FXxHash64Builder& Hasher) const
{
	const uint8 bIsInlinedByte = bIsInlined ? 1 : 0;
	Hasher.Update(&bIsInlinedByte, sizeof(bIsInlinedByte));

	Hasher.Update(&ReturnType, sizeof(ReturnType));

	FString NameString = Name.ToString();
	Hasher.Update(*NameString, NameString.NumBytesWithoutNull());

	Hasher.Update(*Definition, Definition.NumBytesWithoutNull());
	Hasher.Update(&Derivative, sizeof(Derivative));
	Hasher.Update(&ShaderFrequency, sizeof(ShaderFrequency));

	for (const TEnumAsByte<EMaterialDomain>& InDomain : Domains)
	{
		EMaterialDomain Domain = InDomain.GetValue();
		Hasher.Update(&Domain, sizeof(Domain));
	}

	for (const FMaterialExternalCodeEnvironmentDefine& InEnvironmentDefine : EnvironmentDefines)
	{
		InEnvironmentDefine.UpdateHash(Hasher);
	}
}


/*
 * UMaterialExternalCodeCollection
 */

void UMaterialExternalCodeCollection::PostInitProperties()
{
	Super::PostInitProperties();

	TStringBuilder<2048> InvalidPropertiesList;

	auto AppendInvalidProperty = [&InvalidPropertiesList](const TCHAR* PropertyDescription) -> void
		{
			if (InvalidPropertiesList.Len() > 0)
			{
				InvalidPropertiesList.Append(TEXT(", "));
			}
			InvalidPropertiesList.Append(PropertyDescription);
		};

	for (const FMaterialExternalCodeDeclaration& ExternalCode : ExternalCodeDeclarations)
	{
		InvalidPropertiesList.Reset();

		// Validate basic properties
		if (ExternalCode.Definition.IsEmpty())
		{
			AppendInvalidProperty(TEXT("Definition is empty"));
		}

		// Validate derivative configuration properties
		if (ExternalCode.Derivative == EDerivativeStatus::Valid)
		{
			if (ExternalCode.DefinitionDDX.IsEmpty())
			{
				AppendInvalidProperty(TEXT("DefinitionDDX is empty"));
			}
			if (ExternalCode.DefinitionDDY.IsEmpty())
			{
				AppendInvalidProperty(TEXT("DefinitionDDY is empty"));
			}
		}

		if (InvalidPropertiesList.Len() > 0)
		{
			UE_LOGF(LogMaterial, Error, "External HLSL code declaration '%ls' is invalid: %ls", *ExternalCode.Name.ToString(), *InvalidPropertiesList);
		}
	}
}


/*
 * MaterialExternalCodeRegistry
 */

MaterialExternalCodeRegistry& MaterialExternalCodeRegistry::Get()
{
	static MaterialExternalCodeRegistry Instance;
	return Instance;
}

MaterialExternalCodeRegistry::MaterialExternalCodeRegistry()
{
	BuildMapToExternalDeclarations();
}

void MaterialExternalCodeRegistry::BuildMapToExternalDeclarations()
{
	if (const UMaterialExternalCodeCollection* ExternalCodeCollection = GetDefault<UMaterialExternalCodeCollection>())
	{
		ExternalCodeDeclarationMap.Reserve(ExternalCodeCollection->ExternalCodeDeclarations.Num());

		for (const FMaterialExternalCodeDeclaration& InExternalCode : ExternalCodeCollection->ExternalCodeDeclarations)
		{
			if (ExternalCodeDeclarationMap.Contains(InExternalCode.Name))
			{
				UE_LOGF(LogMaterial, Fatal, "External HLSL code declarations for materials must not be overloaded, but '%ls' is defined more than once", *InExternalCode.Name.ToString());
			}
			ExternalCodeDeclarationMap.Add(InExternalCode.Name) = &InExternalCode;
		}
	}
}

const FMaterialExternalCodeDeclaration* MaterialExternalCodeRegistry::FindExternalCode(const FName& InExternalCodeIdentifier) const
{
	if (const FMaterialExternalCodeDeclaration* const* Entry = ExternalCodeDeclarationMap.Find(InExternalCodeIdentifier))
	{
		return *Entry;
	}
	return nullptr;
}

const FMaterialExposedViewPropertyMeta& MaterialExternalCodeRegistry::GetExternalViewPropertyCode(const EMaterialExposedViewProperty InViewProperty) const
{
	// Compile time struct storing all EMaterialExposedViewProperty's enumerations' HLSL compilation specific meta information
	static const FMaterialExposedViewPropertyMeta ViewPropertyMetaArray[] =
	{
		{ MEVP_BufferSize, MCT_Float2, TEXTVIEW("View.BufferSizeAndInvSize.xy"), TEXTVIEW("View.BufferSizeAndInvSize.zw")},
		{ MEVP_FieldOfView, MCT_Float2, TEXTVIEW("View.<PREV>FieldOfViewWideAngles"), {} },
		{ MEVP_TanHalfFieldOfView, MCT_Float2, TEXTVIEW("Get<PREV>TanHalfFieldOfView()"), TEXTVIEW("Get<PREV>CotanHalfFieldOfView()")},
		{ MEVP_ViewSize, MCT_Float2, TEXTVIEW("View.ViewSizeAndInvSize.xy"), TEXTVIEW("View.ViewSizeAndInvSize.zw")},
		{ MEVP_WorldSpaceViewPosition, MCT_LWCVector3, TEXTVIEW("Get<PREV>WorldViewOrigin(Parameters)"), {} },
		{ MEVP_WorldSpaceCameraPosition, MCT_LWCVector3, TEXTVIEW("Get<PREV>WorldCameraOrigin(Parameters)"), {} },
		{ MEVP_ViewportOffset, MCT_Float2, TEXTVIEW("View.ViewRectMin.xy"), {} },
		{ MEVP_TemporalSampleCount, MCT_Float1, TEXTVIEW("View.TemporalAAParams.y"), {} },
		{ MEVP_TemporalSampleIndex, MCT_Float1, TEXTVIEW("View.TemporalAAParams.x"), {} },
		{ MEVP_TemporalSampleOffset, MCT_Float2, TEXTVIEW("View.TemporalAAParams.zw"), {} },
		{ MEVP_RuntimeVirtualTextureOutputLevel, MCT_Float1, TEXTVIEW("GetRuntimeVirtualTextureMipLevel().x"), {} },
		{ MEVP_RuntimeVirtualTextureOutputDerivative, MCT_Float2, TEXTVIEW("GetRuntimeVirtualTextureMipLevel().zw"), {} },
		{ MEVP_PreExposure, MCT_Float1, TEXTVIEW("View.PreExposure.x"), TEXTVIEW("View.OneOverPreExposure.x")},
		{ MEVP_RuntimeVirtualTextureMaxLevel, MCT_Float1, TEXTVIEW("GetRuntimeVirtualTextureMipLevel().y"), {} },
		{ MEVP_ResolutionFraction, MCT_Float1, TEXTVIEW("View.ResolutionFractionAndInv.x"), TEXTVIEW("View.ResolutionFractionAndInv.y")},
		{ MEVP_PostVolumeUserFlags, MCT_Float1, TEXTVIEW("View.PostVolumeUserFlags"), {} },
		{ MEVP_FirstPersonFieldOfView, MCT_Float2, TEXTVIEW("View.<PREV>FirstPersonFieldOfViewWideAngles"), {} },
		{ MEVP_FirstPersonTanHalfFieldOfView, MCT_Float2, TEXTVIEW("View.<PREV>FirstPersonTanAndInvTanHalfFOV.xy"), TEXTVIEW("View.<PREV>FirstPersonTanAndInvTanHalfFOV.zw")},
		{ MEVP_FirstPersonScale, MCT_Float, TEXTVIEW("View.<PREV>FirstPersonScale"), {} },
		{ MEVP_NearPlane, MCT_Float, TEXTVIEW("View.NearPlane"), {} },
	};
	static_assert((sizeof(ViewPropertyMetaArray) / sizeof(ViewPropertyMetaArray[0])) == MEVP_MAX, "incoherency between EMaterialExposedViewProperty and ViewPropertyMetaArray");

	check(InViewProperty < MEVP_MAX);

	const FMaterialExposedViewPropertyMeta& PropertyMeta = ViewPropertyMetaArray[InViewProperty];
	check(InViewProperty == PropertyMeta.EnumValue);

	return PropertyMeta;
}
