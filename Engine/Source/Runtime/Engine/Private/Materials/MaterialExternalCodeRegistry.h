// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialExternalCodeRegistry.h: External HLSL code declarations.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Materials/HLSLMaterialDerivativeAutogen.h"
#include "Materials/MaterialExpressionViewProperty.h"
#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"

#include "MaterialExternalCodeRegistry.generated.h"

UENUM(Flags)
enum class EMaterialShaderFrequency : uint8
{
	None    = 0,
	Vertex  = 1 << SF_Vertex,
	Pixel   = 1 << SF_Pixel,
	Compute = 1 << SF_Compute,
	Any     = Vertex | Pixel | Compute,
};
ENUM_CLASS_FLAGS(EMaterialShaderFrequency);

UENUM()
enum class EMaterialFeatureLevel : uint8
{
	ES2_REMOVED = ERHIFeatureLevel::ES2_REMOVED,
	ES3_1 = ERHIFeatureLevel::ES3_1,
	SM4_REMOVED = ERHIFeatureLevel::SM4_REMOVED,
	SM5 = ERHIFeatureLevel::SM5,
	SM6 = ERHIFeatureLevel::SM6,
	Num
};
static_assert((uint8)ERHIFeatureLevel::Num == (uint8)EMaterialFeatureLevel::Num);

/**
 * This has to be kept in sync with EMaterialValueType.
 * To consolidate those enums, EMaterialValueType must be made an 'enum class' but it has to be deprecated first.
 * NOTE: Remove UMETA(Hidden) markers once the respective entry is used in BaseMaterialExpressions.ini file.
 */
UENUM(Flags)
enum class EMaterialValueTypeBridge : uint64
{
	Float1              = MCT_Float1,
	Float2              = MCT_Float2,
	Float3              = MCT_Float3,
	Float4              = MCT_Float4,
	Texture2D           = MCT_Texture2D UMETA(Hidden),
	TextureCube         = MCT_TextureCube UMETA(Hidden),
	Texture2DArray      = MCT_Texture2DArray UMETA(Hidden),
	TextureCubeArray    = MCT_TextureCubeArray UMETA(Hidden),
	VolumeTexture       = MCT_VolumeTexture UMETA(Hidden),
	StaticBool          = MCT_StaticBool UMETA(Hidden),
	Unknown             = MCT_Unknown UMETA(Hidden),
	MaterialAttributes  = MCT_MaterialAttributes UMETA(Hidden),
	TextureExternal     = MCT_TextureExternal UMETA(Hidden),
	TextureVirtual      = MCT_TextureVirtual UMETA(Hidden),
	SparseVolumeTexture = MCT_SparseVolumeTexture UMETA(Hidden),
	VTPageTableResult   = MCT_VTPageTableResult UMETA(Hidden),
	ShadingModel        = MCT_ShadingModel UMETA(Hidden),
	Substrate           = MCT_Substrate UMETA(Hidden),
	LWCScalar           = MCT_LWCScalar UMETA(Hidden),
	LWCVector2          = MCT_LWCVector2 UMETA(Hidden),
	LWCVector3          = MCT_LWCVector3 UMETA(Hidden),
	LWCVector4          = MCT_LWCVector4 UMETA(Hidden),
	Execution           = MCT_Execution UMETA(Hidden),
	VoidStatement       = MCT_VoidStatement UMETA(Hidden),
	Bool                = MCT_Bool UMETA(Hidden),
	UInt1               = MCT_UInt1 UMETA(Hidden),
	UInt2               = MCT_UInt2 UMETA(Hidden),
	UInt3               = MCT_UInt3 UMETA(Hidden),
	UInt4               = MCT_UInt4 UMETA(Hidden),
	TextureCollection   = MCT_TextureCollection UMETA(Hidden),
	TextureMeshPaint    = MCT_TextureMeshPaint,
	TextureMaterialCache= MCT_TextureMaterialCache,
	Texture             = MCT_Texture UMETA(Hidden),
	Float               = MCT_Float,
	UInt                = MCT_UInt UMETA(Hidden),
	LWCType             = MCT_LWCType UMETA(Hidden),
	Numeric             = MCT_Numeric UMETA(Hidden),
	Float3x3  			= MCT_Float3x3,
	Float4x4  			= MCT_Float4x4,
	LWCMatrix 			= MCT_LWCMatrix,
};

/**
 * Structure to enable shader environment defines per external code declaration.
 * E.g. the external code declaration for "ParticleColor" enables the define "NEEDS_PARTICLE_COLOR" when used in the pixel stage.
 */
USTRUCT()
struct FMaterialExternalCodeEnvironmentDefine
{
	GENERATED_BODY();

	/** Name of the environment define to enable. */
	UPROPERTY(Config)
	FName Name;

	/** Optional shader frequency to further restrict this environment define besides its code declaration shader frequency. */
	UPROPERTY(Config)
	EMaterialShaderFrequency ShaderFrequency = EMaterialShaderFrequency::Any;

	/** Updates the input hasher state with the content of this environemnt define. */
	void UpdateHash(FXxHash64Builder& Hasher) const;
};

/** Declaration of external HLSL code. Such code expressions can be emitted as part of a material translation. */
USTRUCT()
struct FMaterialExternalCodeDeclaration
{
	GENERATED_BODY()

public:
	UPROPERTY(Config)
	uint32 bIsInlined:1 = false;

	UPROPERTY(Config)
	EMaterialValueTypeBridge ReturnType = EMaterialValueTypeBridge::Unknown;

	UPROPERTY(Config)
	FName Name;

	/** Primary external code definition provided as HLSL shader code expression. */
	UPROPERTY(Config)
	FString Definition;

	/** Secondary external code definition for custom DDX derivatives. */
	UPROPERTY(Config)
	FString DefinitionDDX;

	/** Secondary external code definition for custom DDY derivatives. */
	UPROPERTY(Config)
	FString DefinitionDDY;

	/**
	 * Specifies the kind of derivative this code declaration provides.
	 * If this is EDerivativeStatus::Valid, DefinitionDDX and DefinitionDDY provides the code definitions for the DDX and DDY derivatives respectively.
	 */
	UPROPERTY(Config)
	EDerivativeStatus Derivative = EDerivativeStatus::NotAware;

	UPROPERTY(Config)
	EMaterialShaderFrequency ShaderFrequency = EMaterialShaderFrequency::Any;

	/** List of material domains this external code can be used with. If this is empty, all material domains are accepted. */
	UPROPERTY(Config)
	TArray<TEnumAsByte<EMaterialDomain>> Domains;

	/** List of shader environment defines to enable for this external code declaration. */
	UPROPERTY(Config)
	TArray<FMaterialExternalCodeEnvironmentDefine> EnvironmentDefines;

	/** Minimum required feature level for this external code. */
	UPROPERTY(Config)
	EMaterialFeatureLevel MinimumFeatureLevel = (EMaterialFeatureLevel)0;		// Default is lowest feature level (zero), whatever that is in the enum

public:
	/**
	 * Function return type as material value type.
	 * Deduced from ReturnType string to maintain compatibility with other systems that rely on EMaterialValueType *not* being a UENUM().
	 */
	inline EMaterialValueType GetReturnTypeValue() const
	{
		return (EMaterialValueType)ReturnType;
	}

	/** Updates the input hasher state with the content of this external code declaration. */
	void UpdateHash(FXxHash64Builder& Hasher) const;
};

UCLASS(Config=MaterialExpressions)
class UMaterialExternalCodeCollection : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Config)
	TArray<FMaterialExternalCodeDeclaration> ExternalCodeDeclarations;

public:
	virtual void PostInitProperties() override;

};

/** Helper struct for hard coded external code expressions for view properties. */
struct FMaterialExposedViewPropertyMeta
{
	EMaterialExposedViewProperty EnumValue;
	EMaterialValueType Type;
	FStringView PropertyCode;
	FStringView InvPropertyCode;
};

/** Singleton class to register external HLSL function and input declarations for material IR modules. */
class MaterialExternalCodeRegistry
{
public:
	MaterialExternalCodeRegistry(const MaterialExternalCodeRegistry&) = delete;
	MaterialExternalCodeRegistry& operator = (const MaterialExternalCodeRegistry&) = delete;

	static MaterialExternalCodeRegistry& Get();

	/** Returns the external code declaration for the specified name. */
	const FMaterialExternalCodeDeclaration* FindExternalCode(const FName& InExternalCodeIdentifier) const;

	/** Returns the external code declaration for the specified view property. */
	const FMaterialExposedViewPropertyMeta& GetExternalViewPropertyCode(const EMaterialExposedViewProperty InViewProperty) const;

private:
	MaterialExternalCodeRegistry();

	/** Builds the name-to-declaration map for all serialized function declarations. */
	void BuildMapToExternalDeclarations();

	TMap<FName, const FMaterialExternalCodeDeclaration*> ExternalCodeDeclarationMap;

};
