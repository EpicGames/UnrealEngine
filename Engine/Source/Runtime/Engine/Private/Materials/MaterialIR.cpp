// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialAggregate.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "Engine/Texture.h"
#include "VT/RuntimeVirtualTexture.h"

#if WITH_EDITOR

namespace MIR
{

const TCHAR* LexToString(EValueKind Kind)
{
	#define CASE(Name) case VK_##Name: return TEXT(#Name);
	switch (Kind)
	{
		case VK_Poison: return TEXT("Poison");
		case VK_AnalysisError: return TEXT("AnalysisError");
		case VK_Constant: return TEXT("Constant");
		case VK_Builtin: return TEXT("Builtin");
		case VK_MaterialParameterCollection: return TEXT("MaterialParameterCollection");
		case VK_ShadingModel: return TEXT("ShadingModel");
		case VK_PrimitiveUniform: return TEXT("PrimitiveUniform");
		case VK_TextureUniform: return TEXT("TextureUniform");
		case VK_VirtualTextureUniform: return TEXT("VirtualTextureUniform");
		case VK_GetVertexInterpolator: return TEXT("GetVertexInterpolator");
		case VK_Nop: return TEXT("Nop");
		case VK_Composite: return TEXT("Composite");
		case VK_SetMaterialOutput: return TEXT("SetMaterialOutput");
		case VK_Operator: return TEXT("Operator");
		case VK_Branch: return TEXT("Branch");
		case VK_Subscript: return TEXT("Subscript");
		case VK_Scalar: return TEXT("Scalar");
		case VK_TextureRead: return TEXT("TextureRead");
		case VK_VTPageTableRead: return TEXT("VTPageTableRead");
		case VK_Extern: return TEXT("Extern");
		case VK_SubstrateDefaultSlab: return TEXT("SubstrateDefaultSlab");
		case VK_SubstrateSlab: return TEXT("SubstrateSlab");
		case VK_SubstrateShadingModels: return TEXT("SubstrateShadingModel");
		case VK_SubstrateToon: return TEXT("SubstrateToon");
		case VK_SubstrateHorizontalMixing: return TEXT("SubstrateHorizontalMixing");
		case VK_SubstrateVerticalLayering: return TEXT("SubstrateVerticalLayering");
		case VK_SubstrateCoverageWeight: return TEXT("SubstrateCoverageWeight");
		case VK_SubstrateAdd: return TEXT("SubstrateAdd");
		case VK_SubstrateSelect: return TEXT("SubstrateSelect");
		case VK_SubstratePromoteToOperator: return TEXT("SubstratePromoteToOperator");
		case VK_StageSwitch: return TEXT("StageSwitch");
		case VK_PartialDerivative: return TEXT("HardwarePartialDerivative");
		case VK_SetVertexInterpolator: return TEXT("SetVertexInterpolator");
		case VK_Call: return TEXT("Call");
		case VK_CallParameterOutput: return TEXT("CallParameterOutput");
		case VK_PreshaderParameter: return TEXT("PreshaderParameter");
		/* invalid entries */
		case VK_InstructionBegin:
		case VK_InstructionEnd:
			UE_MIR_UNREACHABLE();
	}
	return nullptr;
	#undef CASE
}

bool FValue::IsAnalyzed(EStage Stage) const
{
	return (Flags & EValueFlags(1 << Stage)) != EValueFlags::None;
}

bool FValue::HasFlags(EValueFlags InFlags) const
{
	return (Flags & InFlags) == InFlags;
}

void FValue::SetFlags(EValueFlags InFlags)
{
	Flags |= InFlags;
}

void FValue::ClearFlags(EValueFlags InFlags)
{
	Flags &= ~InFlags;
}

bool FValue::HasSubgraphProperties(EGraphProperties Properties) const
{
	return (GraphProperties & Properties) == Properties;
}

void FValue::SetSubgraphProperties(EGraphProperties Properties)
{
	GraphProperties |= Properties;
}

uint32 FValue::GetSizeInBytes() const
{
	switch (Kind)
	{
		case VK_Poison: return sizeof(FPoison);
		case VK_AnalysisError: return sizeof(FAnalysisError);
		case VK_Constant: return sizeof(FConstant);
		case VK_Builtin: return sizeof(FBuiltin);
		case VK_MaterialParameterCollection: return sizeof(FMaterialParameterCollection);
		case VK_ShadingModel: return sizeof(FShadingModel);
		case VK_PrimitiveUniform: return sizeof(FPrimitiveUniform);
		case VK_TextureUniform: return sizeof(FTextureUniform);
		case VK_VirtualTextureUniform: return sizeof(FVirtualTextureUniform);
		case VK_GetVertexInterpolator: return sizeof(FGetVertexInterpolator);
		case VK_Nop: return sizeof(FNop);
		case VK_Composite: return sizeof(FComposite) + sizeof(FValue*) * static_cast<const FComposite*>(this)->GetComponents().Num();
		case VK_SetMaterialOutput: return sizeof(FSetMaterialOutput);
		case VK_Operator: return sizeof(FOperator);
		case VK_Branch: return sizeof(FBranch);
		case VK_Subscript: return sizeof(FSubscript);
		case VK_Scalar: return sizeof(FScalar);
		case VK_TextureRead: return sizeof(FTextureRead);
		case VK_VTPageTableRead: return sizeof(FVTPageTableRead);
		case VK_SubstrateDefaultSlab: return sizeof(FSubstrateDefaultSlab);
		case VK_SubstrateSlab: return sizeof(FSubstrateSlab);
		case VK_SubstrateShadingModels: return sizeof(FSubstrateShadingModels);
		case VK_SubstrateToon: return sizeof(FSubstrateToon);
		case VK_SubstrateHorizontalMixing: return sizeof(FSubstrateHorizontalMixing);
		case VK_SubstrateVerticalLayering: return sizeof(FSubstrateVerticalLayering);
		case VK_SubstrateCoverageWeight: return sizeof(FSubstrateCoverageWeight);
		case VK_SubstrateAdd: return sizeof(FSubstrateAdd);
		case VK_SubstrateSelect: return sizeof(FSubstrateSelect);
		case VK_SubstratePromoteToOperator: return sizeof(FSubstratePromoteToOperator);
		case VK_StageSwitch: return sizeof(FStageSwitch);
		case VK_PartialDerivative: return sizeof(FPartialDerivative);
		case VK_SetVertexInterpolator: return sizeof(FSetVertexInterpolator);
		case VK_Call: return sizeof(FCall);
		case VK_CallParameterOutput: return sizeof(FCallParameterOutput);
		case VK_PreshaderParameter: return sizeof(FPreshaderParameter);
		case VK_Extern: return static_cast<const FExtern*>(this)->Impl->ByteSize;

			/* invalid entries */
		case VK_InstructionBegin:
		case VK_InstructionEnd:
			UE_MIR_UNREACHABLE();
	}
	return 0;
}

TConstArrayView<FValue*> FValue::GetUses() const
{
	// Values have no uses by definition.
	if (Kind < VK_InstructionBegin)
	{
		return {};
	}

	switch (Kind)
	{
		case VK_Nop:
		{
			auto This = static_cast<const FNop*>(this);
			return { &This->Arg, FNop::NumStaticUses };
		}

		case VK_Composite:
		{
			auto This = static_cast<const FComposite*>(this);
			return This->GetComponents();
		}

		case VK_SetMaterialOutput:
		{
			auto This = static_cast<const FSetMaterialOutput*>(this);
			return { &This->Arg, FSetMaterialOutput::NumStaticUses };
		}

		case VK_Operator:
		{
			auto This = static_cast<const FOperator*>(this);
			return { &This->AArg, FOperator::NumStaticUses };
		}

		case VK_Branch:
		{
			auto This = static_cast<const FBranch*>(this);
			return { &This->ConditionArg, FBranch::NumStaticUses };
		}

		case VK_Subscript:
		{
			auto This = static_cast<const FSubscript*>(this);
			return { &This->Arg, FSubscript::NumStaticUses };
		}

		case VK_Scalar:
		{
			auto This = static_cast<const FScalar*>(this);
			return { &This->Arg, FScalar::NumStaticUses };
		}
			
		case VK_TextureRead:
		{
			auto This = static_cast<const FTextureRead*>(this);
			return { &This->TextureUniform, FTextureRead::NumStaticUses };
		}

		case VK_VTPageTableRead:
		{
			auto This = static_cast<const FVTPageTableRead*>(this);
			return { &This->VirtualTextureUniform, FVTPageTableRead::NumStaticUses };
		}

		case VK_Extern:
		{
			auto This = static_cast<const FExtern*>(this);
			return This->GetArguments();
		}

		case VK_SubstrateDefaultSlab:
		{
			auto This = static_cast<const FSubstrateDefaultSlab*>(this);
			return { &This->Dummy, FSubstrateDefaultSlab::NumStaticUses };
		}

		case VK_SubstrateSlab:
		{
			auto This = static_cast<const FSubstrateSlab*>(this);
			return { &This->Normal, FSubstrateSlab::NumStaticUses };
		}

		case VK_SubstrateShadingModels:
		{
			auto This = static_cast<const FSubstrateShadingModels*>(this);
			return { &This->BaseColor, FSubstrateShadingModels::NumStaticUses };
		}

		case VK_SubstrateToon:
		{
			auto This = static_cast<const FSubstrateToon*>(this);
			return { &This->Normal, FSubstrateToon::NumStaticUses };
		}

		case VK_SubstrateHorizontalMixing:
		{
			auto This = static_cast<const FSubstrateHorizontalMixing*>(this);
			return { &This->Background, FSubstrateHorizontalMixing::NumStaticUses };
		}

		case VK_SubstrateVerticalLayering:
		{
			auto This = static_cast<const FSubstrateVerticalLayering*>(this);
			return { &This->Top, FSubstrateVerticalLayering::NumStaticUses };
		}

		case VK_SubstrateCoverageWeight:
		{
			auto This = static_cast<const FSubstrateCoverageWeight*>(this);
			return { &This->A, FSubstrateCoverageWeight::NumStaticUses };
		}

		case VK_SubstrateAdd:
		{
			auto This = static_cast<const FSubstrateAdd*>(this);
			return { &This->A, FSubstrateAdd::NumStaticUses };
		}

		case VK_SubstrateSelect:
		{
			auto This = static_cast<const FSubstrateSelect*>(this);
			return { &This->A, FSubstrateSelect::NumStaticUses };
		}

		case VK_SubstratePromoteToOperator:
		{
			auto This = static_cast<const FSubstratePromoteToOperator*>(this);
			return { &This->SubstrateDataInput, FSubstratePromoteToOperator::NumStaticUses };
		}

		case VK_StageSwitch:
		{
			auto This = static_cast<const FStageSwitch*>(this);
			return { This->Args, FStageSwitch::NumStaticUses * NumStages };
		}

		case VK_PartialDerivative:
		{
			auto This = static_cast<const FPartialDerivative*>(this);
			return { &This->Arg, FPartialDerivative::NumStaticUses };
		}

		case VK_SetVertexInterpolator:
		{
			auto This = static_cast<const FSetVertexInterpolator*>(this);
			return { &This->Arg, FSetVertexInterpolator::NumStaticUses };
		}

		case VK_Call:
		{
			auto This = static_cast<const FCall*>(this);
			return { This->Arguments, This->NumArguments };
		}

		case VK_CallParameterOutput:
		{
			auto This = static_cast<const FCallParameterOutput*>(this);
			return { &This->Call, FCallParameterOutput::NumStaticUses };
		}

		case VK_PreshaderParameter:
		{
			auto This = static_cast<const FPreshaderParameter*>(this);
			return { &This->SourceParameter, FPreshaderParameter::NumStaticUses };
		}

		default: UE_MIR_UNREACHABLE();
	}
}

TConstArrayView<FValue*> FValue::GetUsesForStage(MIR::EStage Stage) const
{
	if (const FStageSwitch* This = As<FStageSwitch>())
	{
		return { &This->Args[int32(Stage)], FStageSwitch::NumStaticUses };
	}
	return GetUses();
}

bool FValue::IsTrue() const
{
	const MIR::FConstant* Constant = As<MIR::FConstant>();
	return Constant && Constant->Type.IsBoolean() && Constant->Boolean == true;
}

bool FValue::IsFalse() const
{
	const MIR::FConstant* Constant = As<MIR::FConstant>();
	return Constant && Constant->Type.IsBoolean() && Constant->Boolean == false;
}

bool FValue::AreAllTrue() const
{
	if (const MIR::FComposite* Composite = As<MIR::FComposite>())
	{
		for (const MIR::FValue* Component : Composite->GetComponents())
		{
			if (!Component->IsTrue())
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		return IsTrue();
	}
}

bool FValue::AreAllFalse() const
{
	if (const MIR::FComposite* Composite = As<MIR::FComposite>())
	{
		for (const MIR::FValue* Component : Composite->GetComponents())
		{
			if (!Component->IsFalse())
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		return IsFalse();
	}
}

bool FValue::AreAllExactlyZero() const
{
	if (const MIR::FComposite* Composite = As<MIR::FComposite>())
	{
		for (const MIR::FValue* Component : Composite->GetComponents())
		{
			if (!Component->AreAllExactlyZero())
			{
				return false;
			}
		}
		return true;
	}
	else if (const MIR::FConstant* Constant = As<MIR::FConstant>())
	{
		return (Constant->Type.IsInteger() && Constant->Integer == 0)
			|| (Constant->Type.IsFloat() && Constant->Float == 0.0f);
	}
	return false;
}

bool FValue::AreAllNearlyZero() const
{
	if (const MIR::FComposite* Composite = As<MIR::FComposite>())
	{
		for (const MIR::FValue* Component : Composite->GetComponents())
		{
			if (!Component->AreAllNearlyZero())
			{
				return false;
			}
		}
		return true;
	}
	else if (const MIR::FConstant* Constant = As<MIR::FConstant>())
	{
		return (Constant->Type.IsInteger() && Constant->Integer == 0)
			|| (Constant->Type.IsFloat() && FMath::IsNearlyZero(Constant->Float));
	}
	return false;
}

bool FValue::AreAllExactlyOne() const
{
	if (const MIR::FComposite* Composite = As<MIR::FComposite>())
	{
		for (const MIR::FValue* Component : Composite->GetComponents())
		{
			if (!Component->AreAllExactlyOne())
			{
				return false;
			}
		}
		return true;
	}
	else if (const MIR::FConstant* Constant = As<MIR::FConstant>())
	{
		return (Constant->Type.IsInteger() && Constant->Integer == 1)
			|| (Constant->Type.IsFloat() && Constant->Float == 1.0f);
	}
	return false;
}

bool FValue::AreAllNearlyOne() const
{
	if (const MIR::FComposite* Composite = As<MIR::FComposite>())
	{
		for (const MIR::FValue* Component : Composite->GetComponents())
		{
			if (!Component->AreAllNearlyOne())
			{
				return false;
			}
		}
		return true;
	}
	else if (const MIR::FConstant* Constant = As<MIR::FConstant>())
	{
		return (Constant->Type.IsInteger() && Constant->Integer == 1)
			|| (Constant->Type.IsFloat() && FMath::IsNearlyEqual(Constant->Float, 1.0f));
	}
	return false;
}

bool FValue::IsConstant() const
{
	if (As<FConstant>())
	{
		return true;
	}
	if (const FComposite* AsComposite = As<FComposite>())
	{
		return AsComposite->AreComponentsConstant();
	}
	return false;
}

bool FValue::Equals(const FValue* Other) const
{
	if (this == Other)
	{
		return true;
	}

	// Get the size of this value in bytes. It should match that of Other, since the value kinds are the same.
	uint32 SizeInBytes = GetSizeInBytes();
	if (SizeInBytes != Other->GetSizeInBytes())
	{
		return false;
	}

	// Values are PODs by design, therefore simply comparing bytes is sufficient.
	return FMemory::Memcmp(this, Other, SizeInBytes) == 0;
}

bool FValue::EqualsConstant(float TestValue) const
{
	const MIR::FConstant* ValueConstant = this->As<MIR::FConstant>();

	if (ValueConstant)
	{
		switch (ValueConstant->Type.GetPrimitive().ScalarKind)
		{
			case MIR::EScalarKind::Boolean:	return ValueConstant->Boolean == !!TestValue;
			case MIR::EScalarKind::Integer:	return ValueConstant->Integer == (TInteger)TestValue;
			case MIR::EScalarKind::Float:	return ValueConstant->Float == TestValue;
			case MIR::EScalarKind::Double:	return ValueConstant->Double == TestValue;
			default: UE_MIR_UNREACHABLE();
		}
	}
	return false;
}

bool FValue::EqualsConstant(FVector4f TestValue) const
{
	if (this->Kind == MIR::VK_Constant)
	{
		return EqualsConstant(TestValue.X);
	}
	else if (this->Kind == MIR::VK_Composite)
	{
		TConstArrayView<FValue*> Components = static_cast<const FComposite*>(this)->GetComponents();
		int32 NumComponents = FMath::Min(Components.Num(), 4);

		for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			if (!Components[ComponentIndex]->EqualsConstant(TestValue[ComponentIndex]))
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		return false;
	}
}

UObject* FValue::GetTextureObject() const
{
	if (const FTextureUniform* TextureObject = As<FTextureUniform>())
	{
		return TextureObject->Texture;
	}
	if (const FVirtualTextureUniform* RVTextureObject = As<FVirtualTextureUniform>())
	{
		return RVTextureObject->VirtualTexture;
	}
	return nullptr;
}

int32 FValue::GetUniformIndex() const
{
	if (const MIR::FPrimitiveUniform* Uniform = As<MIR::FPrimitiveUniform>())
	{
		return Uniform->Analysis_UniformIndex;
	}
	if (const MIR::FTextureUniform* Uniform = As<MIR::FTextureUniform>())
	{
		return Uniform->Analysis_UniformIndex;
	}
	if (const MIR::FVirtualTextureUniform* Uniform = As<MIR::FVirtualTextureUniform>())
	{
		return Uniform->Analysis_UniformIndex;
	}
	return INDEX_NONE;
}

FInstruction* AsInstruction(FValue* Value)
{
	return Value && (Value->Kind > VK_InstructionBegin && Value->Kind < VK_InstructionEnd) ? static_cast<FInstruction*>(Value) : nullptr;
}

const FInstruction* AsInstruction(const FValue* Value)
{
	return AsInstruction(const_cast<FValue*>(Value));
}

FPoison* FPoison::Get()
{
	static FPoison Poison = [] {
		FPoison P;
		P.Kind = VK_Poison;
		P.Type = FType::MakePoison();
		return P;
	} ();
	return &Poison;
}

const TCHAR* LexToString(EBuiltin ID)
{
	switch (ID)
	{
		case EBuiltin::TextureMipBias: return TEXT("TextureMipBias");
		case EBuiltin::TextureDerivativeMultiply: return TEXT("TextureDerivativeMultiply");
		default: UE_MIR_UNREACHABLE();
	}
}

FBlock* FBlock::FindCommonParentWith(MIR::FBlock* Other)
{
	FBlock* A = this;
	FBlock* B = Other;

	if (A == B)
	{
		return A;
	}

	while (A->Level > B->Level)
	{
		A = A->Parent;
	}

	while (B->Level > A->Level)
	{
		B = B->Parent;
	}

	while (A != B)
	{
		A = A->Parent;
		B = B->Parent;
	}

	return A;
}

// This global is needed to allow the Visual Studio debugger (and subsequently Natvis) to have access to the "TComposite<1>" type identifer, as used in the function below.
TComposite<1> GCompositeNatvisPrototype;

TConstArrayView<FValue*> FComposite::GetComponents() const
{
	int32 NumComponents;
	if (TOptional<FPrimitive> Primitive = Type.AsPrimitive())
	{
		NumComponents = Primitive->NumComponents();
	}
	else if (const UMaterialAggregate* Aggregate = Type.AsAggregate())
	{
		NumComponents = Aggregate->Attributes.Num();
	}
	else
	{
		UE_MIR_UNREACHABLE();
	}

	auto Ptr = static_cast<const TComposite<1>*>(this)->Components;
	return { Ptr, NumComponents };
}

TArrayView<FValue*> FComposite::GetMutableComponents()
{
	// const_cast is okay here as it's used only to get the array of components
	TConstArrayView<FValue*> Components = static_cast<const FComposite*>(this)->GetComponents();
	return { const_cast<FValue**>(Components.GetData()), Components.Num() };
}

bool FComposite::AreComponentsConstant() const
{
	for (FValue const* Component : GetComponents())
	{
		if (Component && !Component->As<FConstant>())
		{
			return false;
		}
	}
	return true;
}

FBlock* FInstruction::GetTargetBlockForUse(int32 EntryPointIndex, int32 UseIndex)
{
	if (auto Branch = As<FBranch>())
	{
		switch (UseIndex)
		{
			case 0: return Linkage[EntryPointIndex].Block; 		// ConditionArg goes into the same block as this instruction's
			case 1: return &Branch->TrueBlock[EntryPointIndex]; 	// TrueArg
			case 2: return &Branch->FalseBlock[EntryPointIndex]; // FalseArg
			default: UE_MIR_UNREACHABLE();
		}
	}

	// By default, dependencies can go in the same block as this instruction
	return Linkage[EntryPointIndex].Block;
}

FStringView FInstruction::GetName() const
{
	if (const FSetMaterialOutput* SetMaterialOutput = MIR::As<FSetMaterialOutput>(this))
	{
		return SetMaterialOutput->Name;
	}
	return {};
}

int32 FInstruction::GetOutputPriority() const
{
	if (const FSetMaterialOutput* SetMaterialOutput = MIR::As<FSetMaterialOutput>(this))
	{
		return SetMaterialOutput->Priority;
	}
	// Otherwise use a large priority based on this instruction kind
	return 1024 + Kind;
}

bool IsUnaryOperator(EOperator Op)
{
	return Op >= UO_FirstUnaryOperator && Op < BO_FirstBinaryOperator;
}

bool IsBinaryOperator(EOperator Op)
{
	return Op >= BO_FirstBinaryOperator && Op < TO_FirstTernaryOperator;
}

bool IsTernaryOperator(EOperator Op)
{
	return Op >= TO_FirstTernaryOperator;
}

int GetOperatorArity(EOperator Op)
{
	return IsUnaryOperator(Op) ? 1
		: IsBinaryOperator(Op) ? 2
		: 3;
}

const TCHAR* LexToString(EOperator Op)
{
	// Note: sorted alphabetically
	switch (Op)
	{
		/* Unary operators */
		case UO_Abs: return TEXT("Abs");
		case UO_ACos: return TEXT("ACos");
		case UO_ACosFast: return TEXT("ACosFast");
		case UO_ACosh: return TEXT("ACosh");
		case UO_ASin: return TEXT("ASin");
		case UO_ASinFast: return TEXT("ASinFast");
		case UO_ASinh: return TEXT("ASinh");
		case UO_ATan: return TEXT("ATan");
		case UO_ATanFast: return TEXT("ATanFast");
		case UO_ATanh: return TEXT("ATanh");
		case UO_BitwiseNot: return TEXT("BitwiseNot");
		case UO_Ceil: return TEXT("Ceil");
		case UO_Cos: return TEXT("Cos");
		case UO_Cosh: return TEXT("Cosh");
		case UO_Exponential: return TEXT("Exponential");
		case UO_Exponential2: return TEXT("Exponential2");
		case UO_Floor: return TEXT("Floor");
		case UO_Frac: return TEXT("Frac");
		case UO_IsFinite: return TEXT("IsFinite");
		case UO_IsInf: return TEXT("IsInf");
		case UO_IsNan: return TEXT("IsNan");
		case UO_Length: return TEXT("Length");
		case UO_Logarithm: return TEXT("Logarithm");
		case UO_Logarithm10: return TEXT("Logarithm10");
		case UO_Logarithm2: return TEXT("Logarithm2");
		case UO_LWCTile: return TEXT("LWCTile");
		case UO_Negate: return TEXT("Negate");
		case UO_Not: return TEXT("Not");
		case UO_Reciprocal: return TEXT("Reciprocal");
		case UO_Round: return TEXT("Round");
		case UO_Rsqrt: return TEXT("Rsqrt");
		case UO_Saturate: return TEXT("Saturate");
		case UO_Sign: return TEXT("Sign");
		case UO_Sin: return TEXT("Sin");
		case UO_Sinh: return TEXT("Sinh");
		case UO_Sqrt: return TEXT("Sqrt");
		case UO_Tan: return TEXT("Tan");
		case UO_Tanh: return TEXT("Tanh");
		case UO_Transpose: return TEXT("Transpose");
		case UO_Truncate: return TEXT("Truncate");

		/* Binary operators */
		case BO_Add: return TEXT("Add");
		case BO_And: return TEXT("And");
		case BO_ATan2: return TEXT("ATan2");
		case BO_ATan2Fast: return TEXT("ATan2Fast");
		case BO_BitShiftLeft: return TEXT("BitShiftLeft");
		case BO_BitShiftRight: return TEXT("BitShiftRight");
		case BO_BitwiseAnd: return TEXT("BitwiseAnd");
		case BO_BitwiseOr: return TEXT("BitwiseOr");
		case BO_BitwiseXor: return TEXT("BitwiseXor");
		case BO_Cross: return TEXT("Cross");
		case BO_Distance: return TEXT("Distance");
		case BO_Divide: return TEXT("Divide");
		case BO_Dot: return TEXT("Dot");
		case BO_Equals: return TEXT("Equals");
		case BO_Fmod: return TEXT("Fmod");
		case BO_GreaterThan: return TEXT("GreaterThan");
		case BO_GreaterThanOrEquals: return TEXT("GreaterThanOrEquals");
		case BO_LessThan: return TEXT("LessThan");
		case BO_LessThanOrEquals: return TEXT("LessThanOrEquals");
		case BO_Max: return TEXT("Max");
		case BO_Min: return TEXT("Min");
		case BO_Modulo: return TEXT("Modulo");
		case BO_Multiply: return TEXT("Multiply");
		case BO_MatrixMultiply: return TEXT("MatrixMultiply");
		case BO_NotEquals: return TEXT("NotEquals");
		case BO_Or: return TEXT("Or");
		case BO_Pow: return TEXT("Pow");
		case BO_Step: return TEXT("Step");
		case BO_Subtract: return TEXT("Subtract");

		/* Ternary operators */
		case TO_Clamp: return TEXT("Clamp");
		case TO_Lerp: return TEXT("Lerp");
		case TO_Select: return TEXT("Select");
		case TO_Smoothstep: return TEXT("Smoothstep");
		
		case O_Invalid: return TEXT("Invalid");
		case OperatorCount: UE_MIR_UNREACHABLE();
	}

	return TEXT("???");
}

EPreshader2Opcode ToPreshader2Opcode(EOperator Op, bool& bOutIsCommutative)
{
	bOutIsCommutative = false;

	switch (Op)
	{
	case UO_Abs: return EPreshader2Opcode::Abs;
	case UO_ACos: return EPreshader2Opcode::ACos;
	case UO_ACosFast: return EPreshader2Opcode::ACos;
	case UO_ASin: return EPreshader2Opcode::ASin;
	case UO_ASinFast: return EPreshader2Opcode::ASin;
	case UO_ATan: return EPreshader2Opcode::ATan;
	case UO_ATanFast: return EPreshader2Opcode::ATan;
	case UO_Ceil: return EPreshader2Opcode::Ceil;
	case UO_Cos: return EPreshader2Opcode::Cos;
	case UO_Exponential: return EPreshader2Opcode::Exponential;
	case UO_Exponential2: return EPreshader2Opcode::Exponential2;
	case UO_Floor: return EPreshader2Opcode::Floor;
	case UO_Frac: return EPreshader2Opcode::Frac;
	case UO_Length: return EPreshader2Opcode::Length;
	case UO_Logarithm: return EPreshader2Opcode::Logarithm;
	case UO_Logarithm10: return EPreshader2Opcode::Logarithm10;
	case UO_Logarithm2: return EPreshader2Opcode::Logarithm2;
	case UO_Negate: return EPreshader2Opcode::Negate;
	case UO_Reciprocal: return EPreshader2Opcode::Reciprocal;
	case UO_Round: return EPreshader2Opcode::Round;
	case UO_Saturate: return EPreshader2Opcode::Saturate;
	case UO_Sign: return EPreshader2Opcode::Sign;
	case UO_Sin: return EPreshader2Opcode::Sin;
	case UO_Sqrt: return EPreshader2Opcode::Sqrt;
	case UO_Tan: return EPreshader2Opcode::Tan;
	case UO_Truncate: return EPreshader2Opcode::Truncate;

	case BO_Add: bOutIsCommutative = true; return EPreshader2Opcode::Add;
	case BO_ATan2: return EPreshader2Opcode::ATan2;
	case BO_ATan2Fast: return EPreshader2Opcode::ATan2;
	case BO_Cross: return EPreshader2Opcode::Cross;
	case BO_Divide: return EPreshader2Opcode::Divide;
	case BO_Dot: bOutIsCommutative = true; return EPreshader2Opcode::Dot;
	case BO_Fmod: return EPreshader2Opcode::Fmod;
	case BO_Max: bOutIsCommutative = true; return EPreshader2Opcode::Max;
	case BO_Min: bOutIsCommutative = true; return EPreshader2Opcode::Min;
	case BO_Modulo: return EPreshader2Opcode::Fmod;
	case BO_Multiply: bOutIsCommutative = true; return EPreshader2Opcode::Multiply;
	case BO_Subtract: return EPreshader2Opcode::Subtract;
	}

	return EPreshader2Opcode::Invalid;
}

const TCHAR* LexToString(ETextureReadMode Mode)
{
	switch (Mode)
	{
		case ETextureReadMode::GatherRed: return TEXT("GatherRed");
		case ETextureReadMode::GatherGreen: return TEXT("GatherGreen");
		case ETextureReadMode::GatherBlue: return TEXT("GatherBlue");
		case ETextureReadMode::GatherAlpha: return TEXT("GatherAlpha");
		case ETextureReadMode::MipAuto: return TEXT("MipAuto");
		case ETextureReadMode::MipLevel: return TEXT("MipLevel");
		case ETextureReadMode::MipBias: return TEXT("MipBias");
		case ETextureReadMode::Derivatives: return TEXT("Derivatives");
		default: UE_MIR_UNREACHABLE();
	}
}

void FStageSwitch::SetArgs(FValue* PixelStageArg, FValue* OtherStagesArg)
{
	for (uint32 i = 0; i < NumStages; ++i)
	{
		Args[i]= (i == Stage_Pixel) ? PixelStageArg : OtherStagesArg;
	}
}

uint32 GetTypeHash(const FFunctionParameter& F)
{
	return HashCombine(GetTypeHash(F.Name), GetTypeHash(F.Type));
}

bool FFunction::Equals(const FFunction* Other) const
{
	return Kind == Other->Kind
		&& Name == Other->Name
		&& ReturnType == Other->ReturnType
		&& NumInputOnlyParams == Other->NumInputOnlyParams
		&& NumInputAndOutputParams == Other->NumInputAndOutputParams
		&& NumParameters == Other->NumParameters
		&& CompareItems(Parameters, Other->Parameters, NumParameters);
}

uint32 GetTypeHash(const FFunction& F)
{
	uint32 Hash = GetTypeHash(F.Name);
	Hash = HashCombine(Hash, ::GetTypeHash((uint8)F.Kind));
	Hash = HashCombine(Hash, ::GetTypeHash(F.NumInputOnlyParams));
	Hash = HashCombine(Hash, ::GetTypeHash(F.NumInputAndOutputParams));
	Hash = HashCombine(Hash, ::GetTypeHash(F.NumParameters));
	Hash = HashCombine(Hash, GetTypeHash(F.ReturnType));
	Hash = HashCombine(Hash, ArrayViewHash(TConstArrayView<FFunctionParameter>{ F.Parameters, F.NumParameters }));
	return Hash;
}

bool FFunctionHLSL::Equals(const FFunctionHLSL* Other) const
{
	return FFunction::Equals(Other)
		&& Code == Other->Code
		&& MIR::ArrayViewEquals(Defines, Other->Defines)
		&& MIR::ArrayViewEquals(Includes, Other->Includes);
}

uint32 GetTypeHash(const FFunctionHLSL& F)
{
	uint32 Hash = GetTypeHash(static_cast<const FFunction&>(F));
	Hash = HashCombine(Hash, GetTypeHash(F.Code));
	Hash = HashCombine(Hash, ArrayViewHash(F.Defines));
	Hash = HashCombine(Hash, ArrayViewHash(F.Includes));
	return Hash;
}

int32 FPrimitiveUniform::Analysis_NumComponents() const
{
	check(Analysis_ComponentMask);

	return 32 - FMath::CountLeadingZeros(Analysis_ComponentMask);
}

} // namespace MIR

#endif // #if WITH_EDITOR
