<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

<!-- FType -->
<Type Name="MIR::FType">
	<DisplayString Condition="Kind == MIR::ETypeKind::Poison">Poison</DisplayString>
	<DisplayString Condition="Kind == MIR::ETypeKind::Void">Void</DisplayString>
	<DisplayString Condition="Kind == MIR::ETypeKind::Primitive">{Primitive.ScalarKind,en}{(int)Primitive.NumRows,d}x{(int)Primitive.NumColumns,d}</DisplayString>
	<DisplayString Condition="Kind == MIR::ETypeKind::Aggregate">Aggregate {Aggregate,na}</DisplayString>
	<DisplayString>{Kind,en}</DisplayString>
	<Expand>
		<Item Name="Kind">Kind,en</Item>
		<ExpandedItem Condition="Kind == MIR::ETypeKind::Primitive">Primitive</ExpandedItem>
		<ExpandedItem Condition="Kind == MIR::ETypeKind::Aggregate">Aggregate</ExpandedItem>
	</Expand>
</Type>

<!-- FValue -->
<Type Name="MIR::FValue">
	<!-- Constant -->
	<DisplayString Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Boolean">Bool "{((MIR::FConstant*)this)->Boolean}"</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Integer">Int "{((MIR::FConstant*)this)->Integer}"</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Float">Float "{((MIR::FConstant*)this)->Float}"</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Double">Double "{((MIR::FConstant*)this)->Double}"</DisplayString>

	<!-- Composite -->
	<DisplayString Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Primitive.NumRows == 3 &amp;&amp; Type.Primitive.NumColumns == 1">Composite [{((MIR::TComposite&lt;1&gt;*)this)->Components[0],na}, {((MIR::TComposite&lt;1&gt;*)this)->Components[1],na}, {((MIR::TComposite&lt;1&gt;*)this)->Components[2],na}] </DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Primitive.NumRows == 4 &amp;&amp; Type.Primitive.NumColumns == 1">Composite [{((MIR::TComposite&lt;1&gt;*)this)->Components[0],na}, {((MIR::TComposite&lt;1&gt;*)this)->Components[1],na}, {((MIR::TComposite&lt;1&gt;*)this)->Components[2],na}, {((MIR::TComposite&lt;1&gt;*)this)->Components[3],na}] </DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Composite">Composite</DisplayString>

	<DisplayString Condition="Kind.Value == MIR::VK_AnalysisError">AnalysisError "{((MIR::FAnalysisError*)this)->Message}"</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Branch">Branch</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Builtin">Builtin</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Call">Call</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_CallParameterOutput">CallParameterOutput [{((MIR::FCallParameterOutput*)this)->Index,d}]</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Extern">Extern</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_GetVertexInterpolator">GetVertexInterpolator</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_MaterialParameterCollection">MaterialParameterCollection</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Nop">Nop</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Operator">Operator   {((MIR::FOperator*)this)->Op,en}</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_PartialDerivative">PartialDerivative</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Poison">Poison</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_ShadingModel">ShadingModel</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SetVertexInterpolator">SetVertexInterpolator</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateAdd">SubstrateAdd</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight">SubstrateCoverageWeight</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateDefaultSlab">SubstrateDefaultSlab</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing">SubstrateHorizontalMixing</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator">SubstratePromoteToOperator</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateSelect">SubstrateSelect</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateSlab">SubstrateSlab</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateShadingModels">SubstrateShadingModel</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateToon">SubstrateToon</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering">SubstrateVerticalLayering</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Scalar">Scalar</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_SetMaterialOutput">SetMaterialOutput</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_StageSwitch">StageSwitch</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_Subscript">Subscript</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_TextureUniform">TextureUniform</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_TextureRead">TextureRead</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_VirtualTextureUniform">VirtualTextureUniform</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_PrimitiveUniform">PrimitiveUniform {((MIR::FPrimitiveUniform*)this)->Analysis_UniformIndex,d}</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_PreshaderParameter">PreshaderParameter</DisplayString>
	<DisplayString Condition="Kind.Value == MIR::VK_VTPageTableRead">VTPageTableRead</DisplayString>
	<DisplayString>{Kind}</DisplayString>

	<Expand>
		<!-- AnalysisError -->
		<Item Name="Expression" Condition="Kind.Value == MIR::VK_AnalysisError">((MIR::FAnalysisError*)this)->Expression</Item>
		<Item Name="Message" Condition="Kind.Value == MIR::VK_AnalysisError">((MIR::FAnalysisError*)this)->Message</Item>

		<!-- Branch -->
		<Item Name="ConditionArg" Condition="Kind.Value == MIR::VK_Branch">((MIR::FBranch*)this)->ConditionArg</Item>
		<Item Name="TrueArg" Condition="Kind.Value == MIR::VK_Branch">((MIR::FBranch*)this)->TrueArg</Item>
		<Item Name="FalseArg" Condition="Kind.Value == MIR::VK_Branch">((MIR::FBranch*)this)->FalseArg</Item>
		<Item Name="Type" Condition="Kind.Value == MIR::VK_Branch">((MIR::FBranch*)this)->Type,na</Item>

		<!-- Builtin TODO -->

		<!-- Call -->
		<Item Condition="Kind.Value == MIR::VK_Call" Name="Function">((MIR::FCall*)this)->Function</Item>
		<ArrayItems Condition="Kind.Value == MIR::VK_Call">
			<Size>((MIR::FCall*)this)->NumArguments</Size>
			<ValuePointer>((MIR::FCall*)this)->Arguments</ValuePointer>
		</ArrayItems>

		<!-- CallParameterOutput -->
		<Item Condition="Kind.Value == MIR::VK_CallParameterOutput" Name="Call">((MIR::FCallParameterOutput*)this)->Call</Item>
		<Item Condition="Kind.Value == MIR::VK_CallParameterOutput" Name="Index">((MIR::FCallParameterOutput*)this)->Index</Item>

		<!-- Composite -->
		<Item Name="[X]" Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Kind == MIR::ETypeKind::Primitive &amp;&amp; Type.Primitive.NumRows == 1 &amp;&amp; Type.Primitive.NumColumns >= 1">
			((MIR::TComposite&lt;1&gt;*)this)->Components[0]
		</Item>
		<Item Name="[Y]" Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Kind == MIR::ETypeKind::Primitive &amp;&amp; Type.Primitive.NumRows == 1 &amp;&amp; Type.Primitive.NumColumns >= 2">
			((MIR::TComposite&lt;1&gt;*)this)->Components[1]
		</Item>
		<Item Name="[Z]" Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Kind == MIR::ETypeKind::Primitive &amp;&amp; Type.Primitive.NumRows == 1 &amp;&amp; Type.Primitive.NumColumns >= 3">
			((MIR::TComposite&lt;1&gt;*)this)->Components[2]
		</Item>
		<Item Name="[W]" Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Kind == MIR::ETypeKind::Primitive &amp;&amp; Type.Primitive.NumRows == 1 &amp;&amp; Type.Primitive.NumColumns >= 4">
			((MIR::TComposite&lt;1&gt;*)this)->Components[3]
		</Item>
		<Item Name="Type" Condition="Kind.Value == MIR::VK_Composite">Type,na</Item>
		<CustomListItems Condition="Kind.Value == MIR::VK_Composite &amp;&amp; Type.Kind == MIR::ETypeKind::Aggregate">
			<Variable Name="Index" InitialValue="0" />
			<Size>Type.Aggregate->Attributes.ArrayNum</Size>
			<Loop Condition="Index &lt; Type.Aggregate->Attributes.ArrayNum">
				<Item Name="{(Type.Aggregate->Attributes.AllocatorInstance.Data)[Index]}">
					((MIR::TComposite&lt;1&gt;*)this)->Components[Index]
				</Item>
				<Exec>Index++</Exec>
			</Loop>
		</CustomListItems>

		<!-- Constant -->
		<Item Name="Boolean" Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Boolean">((MIR::FConstant*)this)->Boolean</Item>
		<Item Name="Integer" Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Integer">((MIR::FConstant*)this)->Integer</Item>
		<Item Name="Float" Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Float">((MIR::FConstant*)this)->Float</Item>
		<Item Name="Double" Condition="Kind.Value == MIR::VK_Constant &amp;&amp; Type.Primitive.ScalarKind == MIR::EScalarKind::Double">((MIR::FConstant*)this)->Double</Item>

		<!-- Extern TODO -->

		<!-- GetVertexInterpolator -->
		<Item Condition="Kind.Value == MIR::VK_GetVertexInterpolator" Name="SetInstr">((MIR::FGetVertexInterpolator*)this)->SetInstr</Item>

		<!-- MaterialParameterCollection -->
		<Item Name="Collection" Condition="Kind.Value == MIR::VK_MaterialParameterCollection">((MIR::FMaterialParameterCollection*)this)->Collection</Item>

		<!-- Nop -->
		<Item Condition="Kind.Value == MIR::VK_Nop" Name="Arg">((MIR::FNop*)this)->Arg</Item>

		<!-- Operator -->
		<Item Condition="Kind.Value == MIR::VK_Operator" Name="AArg">((MIR::FOperator*)this)->AArg</Item>
		<Item Condition="Kind.Value == MIR::VK_Operator &amp;&amp; ((MIR::FOperator*)this)->Op >= MIR::BO_FirstBinaryOperator" Name="BArg">((MIR::FOperator*)this)->BArg</Item>
		<Item Condition="Kind.Value == MIR::VK_Operator &amp;&amp; ((MIR::FOperator*)this)->Op >= MIR::TO_FirstTernaryOperator" Name="CArg">((MIR::FOperator*)this)->CArg</Item>
		<Item Condition="Kind.Value == MIR::VK_Operator" Name="Type">((MIR::FOperator*)this)->Type</Item>

		<!-- PartialDerivative -->
		<Item Condition="Kind.Value == MIR::VK_PartialDerivative" Name="Arg">((MIR::FPartialDerivative*)this)->Arg</Item>
		<Item Condition="Kind.Value == MIR::VK_PartialDerivative" Name="Axis">((MIR::FPartialDerivative*)this)->Axis</Item>
		<Item Condition="Kind.Value == MIR::VK_PartialDerivative" Name="bHardwareDerivative">((MIR::FPartialDerivative*)this)->bHardwareDerivative</Item>

		<!-- Scalar -->
		<Item Condition="Kind.Value == MIR::VK_Scalar" Name="Arg">((MIR::FScalar*)this)->Arg</Item>
		<Item Condition="Kind.Value == MIR::VK_Scalar" Name="Type">((MIR::FScalar*)this)->Type,na</Item>

		<!-- ShadingModel -->
		<Item Name="Id" Condition="Kind.Value == MIR::VK_ShadingModel">*((EMaterialShadingModel*)this),na</Item>

		<!-- SetMaterialOutput -->
		<Item Condition="Kind.Value == MIR::VK_SetMaterialOutput" Name="Arg">((MIR::FSetMaterialOutput*)this)->Arg</Item>
		<Item Condition="Kind.Value == MIR::VK_SetMaterialOutput" Name="Property">((MIR::FSetMaterialOutput*)this)->Property</Item>

		<!-- SetVertexInterpolator -->
		<Item Condition="Kind.Value == MIR::VK_SetVertexInterpolator" Name="Arg">((MIR::FSetVertexInterpolator*)this)->Arg</Item>
		<Item Condition="Kind.Value == MIR::VK_SetVertexInterpolator" Name="Analysis_BaseInterpolatorFloatRegister">((MIR::FSetVertexInterpolator*)this)->Analysis_BaseInterpolatorFloatRegister</Item>

		<!-- StageSwitch -->
		<Item Condition="Kind.Value == MIR::VK_StageSwitch" Name="Args">((MIR::FStageSwitch*)this)->Args</Item>

		<!-- SubstrateAdd -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="A">((MIR::FSubstrateAdd*)this)->A</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="B">((MIR::FSubstrateAdd*)this)->B</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="OperatorIndex">((MIR::FSubstrateAdd*)this)->OperatorIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="MaxDistanceFromLeaves">((MIR::FSubstrateAdd*)this)->MaxDistanceFromLeaves</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="NormalMix">((MIR::FSubstrateAdd*)this)->NormalMix</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="SharedLocalBasisIndexMacro">((MIR::FSubstrateAdd*)this)->SharedLocalBasisIndexMacro</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="ANoV">((MIR::FSubstrateAdd*)this)->ANoV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="BNoV">((MIR::FSubstrateAdd*)this)->BNoV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="bParameterBlendingEnabled">((MIR::FSubstrateAdd*)this)->bParameterBlendingEnabled</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateAdd" Name="bFullySimplifiedInstruction">((MIR::FSubstrateAdd*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateCoverageWeight -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight" Name="A">((MIR::FSubstrateCoverageWeight*)this)->A</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight" Name="Weight">((MIR::FSubstrateCoverageWeight*)this)->Weight</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight" Name="OperatorIndex">((MIR::FSubstrateCoverageWeight*)this)->OperatorIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight" Name="MaxDistanceFromLeaves">((MIR::FSubstrateCoverageWeight*)this)->MaxDistanceFromLeaves</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight" Name="bParameterBlendingEnabled">((MIR::FSubstrateCoverageWeight*)this)->bParameterBlendingEnabled</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateCoverageWeight" Name="bFullySimplifiedInstruction">((MIR::FSubstrateCoverageWeight*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateHorizontalMixing -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="Background">((MIR::FSubstrateHorizontalMixing*)this)->Background</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="Foreground">((MIR::FSubstrateHorizontalMixing*)this)->Foreground</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="Mix">((MIR::FSubstrateHorizontalMixing*)this)->Mix</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="OperatorIndex">((MIR::FSubstrateHorizontalMixing*)this)->OperatorIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="MaxDistanceFromLeaves">((MIR::FSubstrateHorizontalMixing*)this)->MaxDistanceFromLeaves</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="NormalMix">((MIR::FSubstrateHorizontalMixing*)this)->NormalMix</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="SharedLocalBasisIndexMacro">((MIR::FSubstrateHorizontalMixing*)this)->SharedLocalBasisIndexMacro</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="BackgroundNoV">((MIR::FSubstrateHorizontalMixing*)this)->BackgroundNoV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="ForegroundNoV">((MIR::FSubstrateHorizontalMixing*)this)->ForegroundNoV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="bParameterBlendingEnabled">((MIR::FSubstrateHorizontalMixing*)this)->bParameterBlendingEnabled</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateHorizontalMixing" Name="bFullySimplifiedInstruction">((MIR::FSubstrateHorizontalMixing*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstratePromoteToOperator -->
		<Item Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator" Name="SubstrateDataInput">((MIR::FSubstratePromoteToOperator*)this)->SubstrateDataInput</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator" Name="OperatorIndex">((MIR::FSubstratePromoteToOperator*)this)->OperatorIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator" Name="BSDFIndex">((MIR::FSubstratePromoteToOperator*)this)->BSDFIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator" Name="LayerDepth">((MIR::FSubstratePromoteToOperator*)this)->LayerDepth</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator" Name="bIsBottom">((MIR::FSubstratePromoteToOperator*)this)->bIsBottom</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstratePromoteToOperator" Name="bFullySimplifiedInstruction">((MIR::FSubstratePromoteToOperator*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateSelect -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="A">((MIR::FSubstrateSelect*)this)->A</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="B">((MIR::FSubstrateSelect*)this)->B</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="SelectValue">((MIR::FSubstrateSelect*)this)->SelectValue</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="OperatorIndex">((MIR::FSubstrateSelect*)this)->OperatorIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="MaxDistanceFromLeaves">((MIR::FSubstrateSelect*)this)->MaxDistanceFromLeaves</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="SharedLocalBasisIndexMacro">((MIR::FSubstrateSelect*)this)->SharedLocalBasisIndexMacro</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="bParameterBlendingEnabled">((MIR::FSubstrateSelect*)this)->bParameterBlendingEnabled</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSelect" Name="bFullySimplifiedInstruction">((MIR::FSubstrateSelect*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateShadingModels -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="BaseColor">((MIR::FSubstrateShadingModels*)this)->BaseColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Specular">((MIR::FSubstrateShadingModels*)this)->Specular</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Metallic">((MIR::FSubstrateShadingModels*)this)->Metallic</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Roughness">((MIR::FSubstrateShadingModels*)this)->Roughness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Anisotropy">((MIR::FSubstrateShadingModels*)this)->Anisotropy</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="SubSurfaceColor">((MIR::FSubstrateShadingModels*)this)->SubSurfaceColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="SubSurfaceProfileId">((MIR::FSubstrateShadingModels*)this)->SubSurfaceProfileId</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ClearCoat">((MIR::FSubstrateShadingModels*)this)->ClearCoat</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ClearCoatRoughness">((MIR::FSubstrateShadingModels*)this)->ClearCoatRoughness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="EmissiveColor">((MIR::FSubstrateShadingModels*)this)->EmissiveColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Opacity">((MIR::FSubstrateShadingModels*)this)->Opacity</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ThinTranslucentTransmittanceColor">((MIR::FSubstrateShadingModels*)this)->ThinTranslucentTransmittanceColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ThinTranslucentSurfaceCoverage">((MIR::FSubstrateShadingModels*)this)->ThinTranslucentSurfaceCoverage</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="WaterScatteringCoefficients">((MIR::FSubstrateShadingModels*)this)->WaterScatteringCoefficients</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="WaterAbsorptionCoefficients">((MIR::FSubstrateShadingModels*)this)->WaterAbsorptionCoefficients</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="WaterPhaseG">((MIR::FSubstrateShadingModels*)this)->WaterPhaseG</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ColorScaleBehindWater">((MIR::FSubstrateShadingModels*)this)->ColorScaleBehindWater</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ShadingModel">((MIR::FSubstrateShadingModels*)this)->ShadingModel</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Normal">((MIR::FSubstrateShadingModels*)this)->Normal</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="Tangent">((MIR::FSubstrateShadingModels*)this)->Tangent</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ClearCoatNormal">((MIR::FSubstrateShadingModels*)this)->ClearCoatNormal</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="CustomTangent">((MIR::FSubstrateShadingModels*)this)->CustomTangent</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="BasisIndexMacro">((MIR::FSubstrateShadingModels*)this)->BasisIndexMacro</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="ClearCoat_BasisIndexMacro">((MIR::FSubstrateShadingModels*)this)->ClearCoat_BasisIndexMacro</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="bHasDynamicShadingModels">((MIR::FSubstrateShadingModels*)this)->bHasDynamicShadingModels</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateShadingModels" Name="bFullySimplifiedInstruction">((MIR::FSubstrateShadingModels*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateSlab -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="Normal">((MIR::FSubstrateSlab*)this)->Normal</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="DiffuseAlbedo">((MIR::FSubstrateSlab*)this)->DiffuseAlbedo</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="F0">((MIR::FSubstrateSlab*)this)->F0</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="F90">((MIR::FSubstrateSlab*)this)->F90</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="Roughness">((MIR::FSubstrateSlab*)this)->Roughness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="Anisotropy">((MIR::FSubstrateSlab*)this)->Anisotropy</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SSSProfileId">((MIR::FSubstrateSlab*)this)->SSSProfileId</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SSSMFP">((MIR::FSubstrateSlab*)this)->SSSMFP</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SSSMFPScale">((MIR::FSubstrateSlab*)this)->SSSMFPScale</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SSSPhaseAniso">((MIR::FSubstrateSlab*)this)->SSSPhaseAniso</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SSSType">((MIR::FSubstrateSlab*)this)->SSSType</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="EmissiveColor">((MIR::FSubstrateSlab*)this)->EmissiveColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SecondRoughness">((MIR::FSubstrateSlab*)this)->SecondRoughness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SecondRoughnessWeight">((MIR::FSubstrateSlab*)this)->SecondRoughnessWeight</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SecondRoughnessAsSimpleClearCoat">((MIR::FSubstrateSlab*)this)->SecondRoughnessAsSimpleClearCoat</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="ClearCoatUseSecondNormal">((MIR::FSubstrateSlab*)this)->ClearCoatUseSecondNormal</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="ClearCoatBottomNormal">((MIR::FSubstrateSlab*)this)->ClearCoatBottomNormal</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="FuzzAmount">((MIR::FSubstrateSlab*)this)->FuzzAmount</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="FuzzColor">((MIR::FSubstrateSlab*)this)->FuzzColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="FuzzRoughness">((MIR::FSubstrateSlab*)this)->FuzzRoughness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="GlintValue">((MIR::FSubstrateSlab*)this)->GlintValue</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="GlintUV">((MIR::FSubstrateSlab*)this)->GlintUV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="SpecularProfileId">((MIR::FSubstrateSlab*)this)->SpecularProfileId</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="Thickness">((MIR::FSubstrateSlab*)this)->Thickness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="IsThin">((MIR::FSubstrateSlab*)this)->IsThin</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="IsAtBottom">((MIR::FSubstrateSlab*)this)->IsAtBottom</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="LocalBasisIndex">((MIR::FSubstrateSlab*)this)->LocalBasisIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateSlab" Name="bFullySimplifiedInstruction">((MIR::FSubstrateSlab*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateToon -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="Normal">((MIR::FSubstrateToon*)this)->Normal</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="ToonProfileId">((MIR::FSubstrateToon*)this)->ToonProfileId</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="BaseColor">((MIR::FSubstrateToon*)this)->BaseColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="Metallic">((MIR::FSubstrateToon*)this)->Metallic</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="Specular">((MIR::FSubstrateToon*)this)->Specular</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="Roughness">((MIR::FSubstrateToon*)this)->Roughness</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="EmissiveColor">((MIR::FSubstrateToon*)this)->EmissiveColor</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="PatternUVs">((MIR::FSubstrateToon*)this)->PatternUVs</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="IsAtBottom">((MIR::FSubstrateToon*)this)->IsAtBottom</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="LocalBasisIndex">((MIR::FSubstrateToon*)this)->LocalBasisIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateToon" Name="bFullySimplifiedInstruction">((MIR::FSubstrateToon*)this)->bFullySimplifiedInstruction</Item>

		<!-- SubstrateVerticalLayering -->
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="Top">((MIR::FSubstrateVerticalLayering*)this)->Top</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="Base">((MIR::FSubstrateVerticalLayering*)this)->Base</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="OperatorIndex">((MIR::FSubstrateVerticalLayering*)this)->OperatorIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="MaxDistanceFromLeaves">((MIR::FSubstrateVerticalLayering*)this)->MaxDistanceFromLeaves</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="SharedLocalBasisIndexMacro">((MIR::FSubstrateVerticalLayering*)this)->SharedLocalBasisIndexMacro</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="TopNoV">((MIR::FSubstrateVerticalLayering*)this)->TopNoV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="BaseNoV">((MIR::FSubstrateVerticalLayering*)this)->BaseNoV</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="bParameterBlendingEnabled">((MIR::FSubstrateVerticalLayering*)this)->bParameterBlendingEnabled</Item>
		<Item Condition="Kind.Value == MIR::VK_SubstrateVerticalLayering" Name="bFullySimplifiedInstruction">((MIR::FSubstrateVerticalLayering*)this)->bFullySimplifiedInstruction</Item>

		<!-- Subscript -->
		<Item Condition="Kind.Value == MIR::VK_Subscript" Name="Arg">((MIR::FSubscript*)this)->Arg</Item>
		<Item Condition="Kind.Value == MIR::VK_Subscript" Name="Index">((MIR::FSubscript*)this)->Index</Item>

		<!-- TextureUniform -->
		<Item Condition="Kind.Value == MIR::VK_TextureUniform" Name="Texture">((MIR::FTextureUniform*)this)->Texture</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureUniform" Name="SamplerType">((MIR::FTextureUniform*)this)->SamplerType</Item>

		<!-- VirtualTextureUniform -->
		<Item Condition="Kind.Value == MIR::VK_VirtualTextureUniform" Name="Name">((MIR::FVirtualTextureUniform*)this)->Name</Item>
		<Item Condition="Kind.Value == MIR::VK_VirtualTextureUniform" Name="VirtualTexture">((MIR::FVirtualTextureUniform*)this)->VirtualTexture</Item>
		<Item Condition="Kind.Value == MIR::VK_VirtualTextureUniform" Name="VTLayerIndex">((MIR::FVirtualTextureUniform*)this)->VTLayerIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_VirtualTextureUniform" Name="VTPageTableIndex">((MIR::FVirtualTextureUniform*)this)->VTPageTableIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_VirtualTextureUniform" Name="Analysis_UniformIndex">((MIR::FVirtualTextureUniform*)this)->Analysis_UniformIndex</Item>

		<!-- TextureRead -->
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="TextureUniform">((MIR::FTextureRead*)this)->TextureUniform</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="TexCoord">((MIR::FTextureRead*)this)->TexCoord</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="MipValue">((MIR::FTextureRead*)this)->MipValue</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="TexCoordDdx">((MIR::FTextureRead*)this)->TexCoordDdx</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="TexCoordDdy">((MIR::FTextureRead*)this)->TexCoordDdy</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="VTPageTable">((MIR::FTextureRead*)this)->VTPageTable</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="Mode">((MIR::FTextureRead*)this)->Mode</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="SamplerSourceMode">((MIR::FTextureRead*)this)->SamplerSourceMode</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="SamplerType">((MIR::FTextureRead*)this)->SamplerType</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="bUseAnisoSampler">((MIR::FTextureRead*)this)->bUseAnisoSampler</Item>
		<Item Condition="Kind.Value == MIR::VK_TextureRead" Name="Type">((MIR::FTextureRead*)this)->Type</Item>

		<!-- PreshaderParameter -->
		<Item Condition="Kind.Value == MIR::VK_PreshaderParameter" Name="SourceParameter">((MIR::FPreshaderParameter*)this)->SourceParameter</Item>
		<Item Condition="Kind.Value == MIR::VK_PreshaderParameter" Name="Opcode">((MIR::FPreshaderParameter*)this)->Opcode</Item>
		<Item Condition="Kind.Value == MIR::VK_PreshaderParameter" Name="TextureIndex">((MIR::FPreshaderParameter*)this)->TextureIndex</Item>

		<!-- PrimitiveUniform -->
		<!-- TODO:  to improve visualization of this, we need to have the uniform parameter point to its parent FMaterialIRModule to fetch metadata... -->
		<Item Condition="Kind.Value == MIR::VK_PrimitiveUniform" Name="Name">((MIR::FPrimitiveUniform*)this)->Name</Item>
		<Item Condition="Kind.Value == MIR::VK_PrimitiveUniform" Name="DefaultConstant">((MIR::FPrimitiveUniform*)this)->DefaultConstant</Item>
		<Item Condition="Kind.Value == MIR::VK_PrimitiveUniform" Name="Analysis_UniformIndex">((MIR::FPrimitiveUniform*)this)->Analysis_UniformIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_PrimitiveUniform" Name="Analysis_ComponentMask">((MIR::FPrimitiveUniform*)this)->Analysis_ComponentMask</Item>

		<!-- VTPageTableRead -->
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="VirtualTextureUniform">((MIR::FVTPageTableRead*)this)->VirtualTextureUniform</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="TexCoord">((MIR::FVTPageTableRead*)this)->TexCoord</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="TexCoordDdx">((MIR::FVTPageTableRead*)this)->TexCoordDdx</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="TexCoordDdy">((MIR::FVTPageTableRead*)this)->TexCoordDdy</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="MipValue">((MIR::FVTPageTableRead*)this)->MipValue</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="VTStackIndex">((MIR::FVTPageTableRead*)this)->VTStackIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="VTPageTableIndex">((MIR::FVTPageTableRead*)this)->VTPageTableIndex</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="AddressU">((MIR::FVTPageTableRead*)this)->AddressU</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="AddressV">((MIR::FVTPageTableRead*)this)->AddressV</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="MipValueMode">((MIR::FVTPageTableRead*)this)->MipValueMode</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="bEnableFeedback">((MIR::FVTPageTableRead*)this)->bEnableFeedback</Item>
		<Item Condition="Kind.Value == MIR::VK_VTPageTableRead" Name="bIsAdaptive">((MIR::FVTPageTableRead*)this)->bIsAdaptive</Item>

		<!-- FValue common -->
		<Item Name="GraphProperties">GraphProperties</Item>

	</Expand>
</Type>

</AutoVisualizer>
