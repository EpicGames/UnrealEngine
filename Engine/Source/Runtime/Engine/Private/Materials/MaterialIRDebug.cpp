// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRDebug.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Engine/Texture.h"

#if WITH_EDITOR

static TAutoConsoleVariable<bool> CVarDumpMaterialIRUseGraph_EnableNext(
	TEXT("r.Material.Translator.DumpUseGraphOpts.EnableSuccessors"),
	false,
	TEXT("Whether the Material Module IR 'Uses' graph should also display 'Instruction Next' edges."),
	ECVF_RenderThreadSafe);

namespace MIR
{

struct FDebugDumpIRUseGraphState
{
	FString Out;
	TSet<const FValue*> Crawled;
	TArray<const FValue*> ValueStack;

	void DumpModule(const FMaterialIRModule& Module)
	{
		Out.Appendf(TEXT(
			"digraph G {\n\n"
			"rankdir=LR\n"
			"node [shape=box,fontname=\"Consolas\"]\n"
			"edge [fontname=\"Consolas\"]\n\n"
		));

		for (int32 EntryPointIndex = 0; EntryPointIndex < Module.GetNumEntryPoints(); ++EntryPointIndex)
		{
			const FMaterialIRModule::FEntryPoint& EntryPoint = Module.GetEntryPoint(EntryPointIndex);

			for (const FValue* Output : EntryPoint.Outputs)
			{
				if (Output)
				{
					ValueStack.Push(Output);
				}
			}

			while (!ValueStack.IsEmpty())
			{
				DumpValue(EntryPointIndex, EntryPoint.Stage, ValueStack.Pop());
			}
		}

		Out.Appendf(TEXT("\n}\n"));
	}

	void DumpValue(uint32 EntryPointIndex, MIR::EStage Stage, const FValue* Value)
	{
		const bool bDumpInstructionSequence = CVarDumpMaterialIRUseGraph_EnableNext.GetValueOnAnyThread();

		// Begin the node declaration
		Out.Appendf(TEXT("\"%p\" [label=< <b>%s</b>  (%s) <br/> "),
						Value,
						LexToString(Value->Kind),
						!Value->Type.IsPoison() ? *Value->Type.GetSpelling() : TEXT("???"));

		DumpValueInfo(Value);

		// End the node declaration
		Out.Append(TEXT(">]\n"));

		const FInstruction* Instr = AsInstruction(Value);
		if (bDumpInstructionSequence && Instr && Instr->Linkage[EntryPointIndex].Next)
		{
			Out.Appendf(TEXT("\"%p\" -> \"%p\" [color=\"red\"]\n"), Instr, Instr->Linkage[EntryPointIndex].Next);
		}

		int32 UseIndex = -1;
		for (const FValue* Use : Value->GetUsesForStage(Stage))
		{
			++UseIndex;

			if (!Use)
			{
				continue;
			}
			
			Out.Appendf(TEXT("\"%p\" -> \"%p\" [label=\""), Value, Use);

			DumpUseInfo(Value, Use, UseIndex);

			Out.Appendf(TEXT("\"]\n"));

			if (!Crawled.Contains(Use))
			{
				Crawled.Add(Use);
				ValueStack.Push(Use);
			}

			if (bDumpInstructionSequence && Instr)
			{
				const FInstruction* UseInstr = AsInstruction(Use);
				if (UseInstr && UseInstr->Linkage[EntryPointIndex].Block != Instr->Linkage[EntryPointIndex].Block)
				{
					Out.Appendf(TEXT("\"%p\" -> \"%p\" [color=\"red\", style=\"dashed\"]\n"), UseInstr, Instr);
				}
			}
		}
	}

	void DumpValueInfo(const FValue* Value)
	{
		if (const FConstant* Constant = Value->As<FConstant>())
		{
			switch (Constant->Type.GetPrimitive().ScalarKind)
			{
				case EScalarKind::Boolean:  Out.Append(Constant->Boolean ? TEXT("true") : TEXT("false")); break;
				case EScalarKind::Integer:	 Out.Appendf(TEXT("%lld"), Constant->Integer); break;
				case EScalarKind::Float: Out.Appendf(TEXT("%f"), Constant->Float); break;
				case EScalarKind::Double:   Out.Appendf(TEXT("%f"), Constant->Double); break;
				default: UE_MIR_UNREACHABLE();
			}
		}
		else if (const FBuiltin* ExternalInput = Value->As<FBuiltin>())
		{
			Out.Append(LexToString(ExternalInput->Id));
		}
		else if (const FSetMaterialOutput* SetMaterailOutput = Value->As<FSetMaterialOutput>())
		{
			const FString& PropertyName = (SetMaterailOutput->Property == MP_SubsurfaceColor)
				? TEXT("Subsurface")
				: FMaterialAttributeDefinitionMap::GetAttributeName(SetMaterailOutput->Property);
		
			Out.Append(PropertyName);
		}
		else if (const FSubscript* Subscript = Value->As<FSubscript>())
		{
			if (Subscript->Arg->Type.IsVector())
			{
				static const TCHAR* Suffix[] = { TEXT(".x"), TEXT(".y"), TEXT(".z"), TEXT(".w") };
				check(Subscript->Index < 4); 
				Out.Append(Suffix[Subscript->Index]);
			}
			else
			{
				Out.Appendf(TEXT("Index: %d"), Subscript->Index);
			}
		}
		else if (const FOperator* Operator = Value->As<FOperator>())
		{
			Out.Append(LexToString(Operator->Op));
		}
		else if (const FExtern* Extern = Value->As<FExtern>())
		{
			Out.Append(Extern->GetInfo().Name);
		}
	}

	void DumpUseInfo(const FValue* Value, const FValue* Use, int32 UseIndex)
	{
		if (const FComposite* Composite = Value->As<FComposite>())
		{
			if (Composite->Type.IsVector())
			{
				check(UseIndex < 4);
				Out.AppendChar(TEXT("xyzw")[UseIndex]);
			}
			else
			{
				Out.AppendInt(UseIndex);
			}
		}
		else if (const FBranch* If = Value->As<FBranch>())
		{
			static const TCHAR* Uses[] = { TEXT("condition"), TEXT("true"), TEXT("false") };
			Out.Append(Uses[UseIndex]);
		}
		else if (const FOperator* Operator = Value->As<FOperator>())
		{
			static const TCHAR* Uses[] = { TEXT("a"), TEXT("b"), TEXT("c") };
			Out.Append(Uses[UseIndex]);
		}
	}
};

void DebugDumpIRUseGraph(const FMaterialIRModule& Module)
{
	FDebugDumpIRUseGraphState State;
	State.DumpModule(Module);

	FString FilePath = FPaths::Combine(FPaths::ProjectSavedDir(), "Materials", TEXT("IRDumpUseGraph.dot"));
	FFileHelper::SaveStringToFile(State.Out, *FilePath);
}

/* Module IR to textual representation dumping */

/**
 * Returns whether given instruction kind has a dynamic number of arguments, such as the
 * Operator instruction which can have one, two or three arguments.
 */
static bool InstrHasVariableArgCount(MIR::EValueKind Kind)
{
	return Kind == MIR::VK_Operator;
}

// Helper struct to wrap the state used during IR to text dumping.
struct FDebugDumpIRState
{
	// The module we are printing the IR for.
	const FMaterialIRModule* Module{};

	// Output string containing the generated result.
	FString Out{}; 

	// String used for temporary operations. Clear before use.
	FString Temp{};

	// Maps values to an incrementing id. Used to give values a "name" for future referencing (e.g. "%6")
	TMap<const FValue*, uint32> ValueToIdMap{};

	// Counter used to assign an id to encountered values.
	uint32 InstrIdCounter{};

	// Array of encountered parameters. Used later on to generate the a recap of all referenced parameters.
	TArray<TPair<uint32, const FValue*>> ReferencedUniforms{};
	
	// Array of encountered user HLSL functions.
	TArray<const MIR::FFunctionHLSL*> ReferencedFunctionHLSLs{};

	int32 CurrentEntryPointIndex{};

	// Stage we're currently emitting to.
	MIR::EStage CurrentStage{};

	// Prints a block of instructions to `Out`. Indentation indicates how many levels of indentation to put
	// to the left of printed instruction.
	void AppendBlock(const FBlock& Block, int32 Indentation)
	{
		for (FInstruction* Instr = Block.Instructions; Instr; Instr = Instr->GetNext(CurrentEntryPointIndex))
		{
			// Format the left column (e.g. "%4 = ") string if this instruction is referenceable.
			Temp.Empty();
			if (Instr->Kind != VK_SetMaterialOutput)
			{
				Temp.Appendf(TEXT("%%%u = "), ReferenceInstruction(Instr));
			}

			// Print indentation, then Temp aligned to the right.
			AppendLeftColumn(Indentation, Temp);

			// Print the kind of the instruction (the opcode, e.g. "Operator")
			Out.Append(LexToString(Instr->Kind));

			// Begin printing the arguments (used values)
			Out.Append(TEXT(" ("));

			bool bAddComma = false;
			TConstArrayView<FValue*> Uses = Instr->GetUsesForStage(CurrentStage);
			for (int32 UseIndex = 0; UseIndex < Uses.Num(); ++UseIndex)
			{
				const FValue* Use = Uses[UseIndex];
				if (!Use && InstrHasVariableArgCount(Instr->Kind))
				{
					continue;
				}

				if (bAddComma)
				{
					Out.Append(TEXT(", "));
				}
				bAddComma = true;
				
				if (!Use)
				{
					Out.Append(TEXT("null"));
					continue;
				}

				// First the type...
				Out.Appendf(TEXT("%s "), *Use->Type.GetSpelling());

				// If this use is in a block different from current's, dump the block in "{}" first.
				const FBlock* UseBlock = Instr->GetTargetBlockForUse(CurrentEntryPointIndex, UseIndex);
				if (UseBlock != Instr->GetBlock(CurrentEntryPointIndex) && UseBlock->Instructions)
				{
					Out.Append(TEXT("{\n"));
					AppendBlock(*UseBlock, Indentation + 1);

					AppendLeftColumn(Indentation, TEXT(""));
					Out.Append(TEXT("} "));
				}
				
				// Finally, reference the used value (this will print "%x" if it's an
				// instruction, or inline its information otherwise, like in constants).
				AppendValueReference(Use);
			}

			Out.Append(TEXT(")"));

			// Dump the instruction properties.
			AppendInstructionProperties(Instr);
			Out.Appendf(TEXT(" <Users=%d>"), Instr->Linkage[CurrentEntryPointIndex].NumUsers);

			Out.Append(TEXT("\n"));
		}
	}

	// Appends extra information regarding the instruction.
	void AppendInstructionProperties(const FInstruction* Instr)
	{
		if (const FSetMaterialOutput* SetMaterialOutput = Instr->As<FSetMaterialOutput>())
		{
			Out.Appendf(TEXT(" \"%s\""), *SetMaterialOutput->Name);
		}
		else if (const FOperator* Operator = Instr->As<FOperator>())
		{
			Out.Appendf(TEXT(" \"%s\""), LexToString(Operator->Op));
		}
		else if (const FSubscript* Subscript = Instr->As<FSubscript>())
		{
			if (Subscript->Arg->Type.IsVector())
			{
				check(Subscript->Index < 4);
				Out.Appendf(TEXT(" .%c"), TEXT("XYZW")[Subscript->Index]);
			}
			else
			{
				Out.Appendf(TEXT(" Index=%d"), Subscript->Index);
			}
		}
		else if (const FTextureRead* TextureRead = Instr->As<FTextureRead>())
		{
			Out.Appendf(TEXT(" Mode=\"%s\""), MIR::LexToString(TextureRead->Mode));
			Out.Appendf(TEXT(" SamplerSourceMode=\"%s\""), *StaticEnum<ESamplerSourceMode>()->GetDisplayNameTextByValue(TextureRead->SamplerSourceMode).ToString());
			Out.Appendf(TEXT(" SamplerType=\"%s\""), *StaticEnum<EMaterialSamplerType>()->GetDisplayNameTextByValue(TextureRead->SamplerType).ToString());
		}
		else if (const FPreshaderParameter* PreshaderParameter = Instr->As<FPreshaderParameter>())
		{
			Out.Appendf(TEXT(" TextureIndex=%d"), PreshaderParameter->TextureIndex);
			Out.Appendf(TEXT(" PreshaderOffset=%d"), PreshaderParameter->Analysis_PreshaderOffset - 1);			// One is added to preshader offsets, so zero can represent none, subtract that here.
		}
		else if (const FPartialDerivative* Derivative = Instr->As<FPartialDerivative>())
		{
			Out.Append(Derivative->Axis == MIR::EDerivativeAxis::X ? TEXT(" \"ddx\"") : TEXT(" \"ddy\""));
		}
		else if (const FCall* Call = Instr->As<FCall>())
		{
			if (Call->Function->Kind == MIR::FFunctionKind::HLSL)
			{
				ReferencedFunctionHLSLs.AddUnique(static_cast<const MIR::FFunctionHLSL*>(Call->Function));
				Out.Appendf(TEXT(" FunctionHLSL=\"%s\""), Call->Function->Name.GetData());
			}
		}
		else if (const FCallParameterOutput* CallOutput = Instr->As<FCallParameterOutput>())
		{
			Out.Appendf(TEXT(" Output=\"%s\""), *CallOutput->Call->As<FCall>()->Function->GetOutputParameter(CallOutput->Index).Name.ToString());
		}
		else if (const FExtern* Extern = Instr->As<FExtern>())
		{
			Out.Appendf(TEXT(" \"%s\" "), Extern->GetInfo().Name.GetData());
			Extern->EmitDebugInfo(Out);
		}

		if (Instr->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut))
		{
			Out.Appendf(TEXT(" PreshaderOut"));
		}
		else if (Instr->HasSubgraphProperties(MIR::EGraphProperties::PreshaderTemp))
		{
			Out.Appendf(TEXT(" PreshaderTemp"));
		}
	}

	// Appends a reference to the specified value. This will look like "%x" if Value is an
	// instruction, otherwise it will inline information regarding the value.
	void AppendValueReference(const FValue* Value)
	{
		if (ValueToIdMap.Contains(Value))
		{
			Out.Appendf(TEXT("%%%u"), ValueToIdMap[Value]);
			return;
		}

		if (Value->As<FPoison>())
		{
			Out.Append(TEXT("Poison"));
		}
		else if (const FConstant* Constant = Value->As<FConstant>())
		{
			if (Constant->Type.IsBoolScalar())
			{
				Out.Append(Constant->Boolean ? TEXT("true") : TEXT("false"));
			}
			else if (Constant->Type.IsInteger())
			{
				Out.Appendf(TEXT("%lld"), Constant->Integer);
			}
			else if (Constant->Type.IsFloat())
			{
				Out.Appendf(TEXT("%.5ff"), Constant->Float);
			}
			else if (Constant->Type.IsDouble())
			{
				Out.Appendf(TEXT("%.8f"), Constant->Double);
			}
			else
			{
				UE_MIR_UNREACHABLE();
			}
		}
		else if (const FBuiltin* ExternalInput = Value->As<FBuiltin>())
		{
			Out.Appendf(TEXT("[ExternalInput \"%s\"]"), LexToString(Value->As<FBuiltin>()->Id));
		}
		else if (const FPrimitiveUniform* PrimitiveUniform = Value->As<FPrimitiveUniform>())
		{
			const TCHAR* PreshaderText = TEXT("");
			if (Value->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut))
			{
				PreshaderText = TEXT(" PreshaderOut");
			}
			else if (Value->HasSubgraphProperties(MIR::EGraphProperties::PreshaderTemp))
			{
				PreshaderText = TEXT(" PreshaderTemp");
			}

			Out.Appendf(TEXT("[Parameter #%d \"%s\"%s]"), PrimitiveUniform->Analysis_UniformIndex, *PrimitiveUniform->Name.ToString(), PreshaderText);
			ReferencedUniforms.AddUnique({ PrimitiveUniform->Analysis_UniformIndex, PrimitiveUniform });
		}
		else if (const FTextureUniform* TextureUniform = Value->As<FTextureUniform>())
		{
			Out.Appendf(TEXT("[UniformIndex #%d SamplerType=\"%s\"]"), TextureUniform->Analysis_UniformIndex, *StaticEnum<EMaterialSamplerType>()->GetDisplayNameTextByValue(TextureUniform->SamplerType).ToString());
			ReferencedUniforms.AddUnique({ TextureUniform->Analysis_UniformIndex, TextureUniform });
		}
		else if (const FVirtualTextureUniform* VirtualTextureUniform = Value->As<FVirtualTextureUniform>())
		{
			Out.Appendf(TEXT("[UniformIndex #%d]"), VirtualTextureUniform->Analysis_UniformIndex);
			ReferencedUniforms.AddUnique({ VirtualTextureUniform->Analysis_UniformIndex, VirtualTextureUniform });
		}
		else
		{
			Out.Appendf(TEXT("[%s]"), LexToString(Value->Kind));
		}
	}

	// Gets the instruction reference.
	uint32 ReferenceInstruction(const FInstruction* Instr)
	{
		uint32 Id;
		if (!MIR::Find(ValueToIdMap, static_cast<const FValue*>(Instr), Id))
		{
			Id = InstrIdCounter++;
			ValueToIdMap.Add(Instr, Id);
		}
		return Id;
	}
	
	void AppendLeftColumn(int32 Indentation, FStringView LeftColumn)
	{
		for (int32 i = 0; i < Indentation; ++i)
		{
			Out.Append("        ");
		}
		
		// Put some padding so that all '=' are aligned to the right.
		for (int32 i = 0, Spaces = 8 - LeftColumn.Len(); i < Spaces; ++i)
		{
			Out.Append(TEXT(" "));
		}

		Out += LeftColumn;
	}

	// Adapted from the AppendUniformLocation utility function in Preshader2.cpp, but applied to MIR type.
	void AppendUniformLocation(const FValue* Value, FString& InoutResult)
	{
		check(Value->Analysis_PreshaderOffset);

		static const TCHAR* LinearSwizzles[4][5] =
		{
			{ TEXT(".?"), TEXT(".x"), TEXT(".xy"), TEXT(".xyz"), TEXT(".xyzw") },
			{ TEXT(".?"), TEXT(".y"), TEXT(".yz"), TEXT(".yzw"), TEXT(".????") },
			{ TEXT(".?"), TEXT(".z"), TEXT(".zw"), TEXT(".???"), TEXT(".????") },
			{ TEXT(".?"), TEXT(".w"), TEXT(".??"), TEXT(".???"), TEXT(".????") },
		};

		FPrimitive Primitive = Value->Type.GetPrimitive();
		EScalarKind ScalarKind = Primitive.ScalarKind;
		int32 Dimension = Primitive.NumComponents();
		int32 Offset = Value->Analysis_PreshaderOffset - 1;			// One is added to preshader offsets, so zero can represent none, subtract that here.

		int32 WordCount;
		if (const FPrimitiveUniform* Parameter = Value->As<FPrimitiveUniform>())
		{
			WordCount = Parameter->Analysis_NumComponents();
		}
		else
		{
			WordCount = Primitive.NumComponents();
		}

		if (ScalarKind == EScalarKind::Double)
		{
			WordCount *= 2;
		}

		// Note that the Preshader system supports locations that span float4 buffer elements, such as LWC/Double.  So we need to wrap across buffer
		// elements in order to communicate that.  For example, a 3-element double would display as [1].xyzw, [2].xy, indicating that
		// it uses 6 words across two buffer elements.
		bool bNeedComma = false;
		while (WordCount > 0)
		{
			if (bNeedComma)
			{
				InoutResult.Append(TEXT(", "));
			}
			bNeedComma = true;

			// Figure out how many words we need to print at the current offset
			int32 WordsToPrint = FMath::Min(4 - Offset % 4, WordCount);
			InoutResult.Appendf(TEXT("[%d]%s"), Offset / 4, LinearSwizzles[Offset % 4][WordsToPrint]);

			WordCount -= WordsToPrint;
			Offset += WordsToPrint;
		}
	}

	// Prints the parameter recap section.
	void DumpReferencedParameters()
	{
		if (ReferencedUniforms.IsEmpty())
		{
			return;
		}

		// Dump the list of referenced parameters.
		ReferencedUniforms.Sort([](const auto& Param1, const auto& Param2)
		{
			return (Param1.Value->Analysis_PreshaderOffset != Param2.Value->Analysis_PreshaderOffset)
				? Param1.Value->Analysis_PreshaderOffset < Param2.Value->Analysis_PreshaderOffset
				: Param1.Key < Param2.Key;
		});

		Out.Append(TEXT("\n; Referenced material parameters\n"));
		for (const auto& ParamPair : ReferencedUniforms)
		{
			const FValue* Value = ParamPair.Value;
			if (const FPrimitiveUniform* PrimitiveUniform = Value->As<FPrimitiveUniform>())
			{
				AppendUniformLocation(PrimitiveUniform, Out);

				Out.Appendf(TEXT(" = #%d Name=\"%s\" Type=\"%s\" Mask=.%s%s%s%s\n"),
					PrimitiveUniform->Analysis_UniformIndex,
					*PrimitiveUniform->Name.ToString(),
					*PrimitiveUniform->Type.GetSpelling(),
					(PrimitiveUniform->Analysis_ComponentMask & (1 << 0)) ? TEXT("x") : TEXT(""),
					(PrimitiveUniform->Analysis_ComponentMask & (1 << 1)) ? TEXT("y") : TEXT(""),
					(PrimitiveUniform->Analysis_ComponentMask & (1 << 2)) ? TEXT("z") : TEXT(""),
					(PrimitiveUniform->Analysis_ComponentMask & (1 << 3)) ? TEXT("w") : TEXT(""));
			}
			else if (const FTextureUniform* TextureUniform = Value->As<FTextureUniform>())
			{
				Out.Appendf(TEXT("#%d = Name=\"%s\" Type=\"%s\"\n"), TextureUniform->Analysis_UniformIndex, *TextureUniform->Name.ToString(), TEXT("Texture"));
			}
			else if (const FVirtualTextureUniform* VirtualTextureUniform = Value->As<FVirtualTextureUniform>())
			{
				Out.Appendf(TEXT("#%d = Name=\"%s\" Type=\"%s\"\n"), VirtualTextureUniform->Analysis_UniformIndex, *VirtualTextureUniform->Name.ToString(), TEXT("VirtualTexture"));
			}
		}
	}

	// Prints Preshader data
	void DumpPreshader()
	{
		const UE::Shader::FPreshaderData& PreshaderData = Module->GetCompilationOutput().UniformExpressionSet.GetPreshaderData();
		if (PreshaderData.Data.IsEmpty())
		{
			return;
		}

		TConstArrayView<FMaterialUniformParameterEvaluation> ParameterEvaluations = Module->GetCompilationOutput().UniformExpressionSet.UniformParameterEvaluations;
		TArray<UE::Preshader2::FPrintParameterInfo> ParameterInfos;

		ParameterInfos.SetNum(ParameterEvaluations.Num());
		for (int32 ParameterIndex = 0; ParameterIndex < ParameterEvaluations.Num(); ParameterIndex++)
		{
			const FMaterialNumericParameterInfo& NumericParameterInfo = Module->GetCompilationOutput().UniformExpressionSet.GetNumericParameter(ParameterEvaluations[ParameterIndex].ParameterIndex);

			ParameterInfos[ParameterIndex].Name = NumericParameterInfo.ParameterInfo.Name;
			ParameterInfos[ParameterIndex].Offset = ParameterEvaluations[ParameterIndex].BufferOffset;
			ParameterInfos[ParameterIndex].ComponentCount = NumericParameterInfo.ParameterType == EMaterialParameterType::Scalar ? 1 : 4;
			ParameterInfos[ParameterIndex].bDouble = NumericParameterInfo.ParameterType == EMaterialParameterType::DoubleVector;
		}

		Out.Append(TEXT("\n; Preshader Instructions\n"));

		TArray<FString> Instructions;
		UE::Preshader2::Print(PreshaderData, ParameterInfos, Instructions);

		for (const FString& Instruction : Instructions)
		{
			Out.Append(Instruction);
			Out.Append(TEXT("\n"));
		}

		Out.Append(TEXT("\n"));
	}

	void DumpFunctionHLSLs()
	{
		if (ReferencedFunctionHLSLs.IsEmpty())
		{
			return;
		}

		Out.Append(TEXT("\n; Referenced user HLSL functions\n"));

		for (const MIR::FFunctionHLSL* Function : ReferencedFunctionHLSLs)
		{
			Out.Appendf(TEXT("FunctionHLSL Name=\"%s\" ReturnType=\"%s\"\n"), Function->Name.GetData(), *Function->ReturnType.GetSpelling());
			for (int i = 0; i < Function->NumParameters; ++i)
			{
				const TCHAR* Keyword = (i < Function->NumInputOnlyParams) ? TEXT("In")
					: (i < Function->NumInputAndOutputParams) ? TEXT("InOut")
					: TEXT("Out");
				
				Out.Appendf(TEXT("\tParam %s Name=\"%s\" Type=\"%s\"\n"), Keyword, *Function->Parameters[i].Name.ToString(), *Function->Parameters[i].Type.GetSpelling());
			}
		}
	}
};

FString DebugDumpIR(FStringView MaterialName, const FMaterialIRModule& Module)
{
	FDebugDumpIRState State{};
	State.Module = &Module;
	State.Out.Append(TEXT("; Material IR module dump.\n"));
	State.Out.Appendf(TEXT(";    Material: %s\n"), MaterialName.GetData());

	// Dump the IR instructions in the root block.
	static const TCHAR* Header = TEXT("; Material \"%s\" translated IR dump\n\n");
	for (int32 EntryPointIndex = 0; EntryPointIndex < Module.GetNumEntryPoints(); ++EntryPointIndex)
	{
		const FMaterialIRModule::FEntryPoint& EntryPoint = Module.GetEntryPoint(EntryPointIndex);
		State.Out.Appendf(TEXT("\n; Entry Point %d \"%s\" (stage \"%s\")\n"), EntryPointIndex, EntryPoint.Name.GetData(), MIR::LexToString(EntryPoint.Stage));

		State.CurrentEntryPointIndex = EntryPointIndex;
		State.CurrentStage = EntryPoint.Stage;
		State.AppendBlock(EntryPoint.RootBlock, 0);
	}

	// Print referenced material parameters recap if any
	State.DumpReferencedParameters();
	State.DumpPreshader();
	State.DumpFunctionHLSLs();
	return State.Out;
}

} // namespace MIR

#endif // #if WITH_EDITOR
