// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIREmitter.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionUtils.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRModuleBuilder.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialAggregate.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialIRExtern.h"
#include "Shader/Preshader2.h"
#include "Shader/ShaderTypes.h"
#include "MaterialShared.h"
#include "MaterialSharedPrivate.h"
#include "MaterialExpressionIO.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "RenderUtils.h"
#include "Materials/SubstrateTranslatorCommon.h"
#include "MaterialDomain.h"

#if WITH_EDITOR

static TAutoConsoleVariable<bool> CVarMaterialIRDebugBreakOnPoison(
	TEXT("r.Material.Translator.DebugBreakOnPoison"),
	0,
	TEXT("Whether the material translator break in the debugger when hitting a poison value in the IR.\n"),
	ECVF_Default);

static bool GAllowPreshaderOperators = true;
static FAutoConsoleVariableRef CVarAllowPreshaderOperators(
	TEXT("r.Material.Translator.AllowPreshaderOperators"),
	GAllowPreshaderOperators,
	TEXT("Chicken switch to disable preshader operator support, in case of stability issues."),
	ECVF_Default);

namespace MIR
{

// Converts a vector component enum to its string representation ("x", "y", "z", or "w").
const TCHAR* VectorComponentToString(EVectorComponent Component)
{
	static const TCHAR* Strings[] = { TEXT("x"), TEXT("y"), TEXT("z"), TEXT("w") };
	return Strings[(int32)Component];
}

FSwizzleMask::FSwizzleMask(EVectorComponent X)
: NumComponents{ 1 }
{
	Components[0] = X;
}

FSwizzleMask::FSwizzleMask(EVectorComponent X, EVectorComponent Y)
: NumComponents{ 2 }
{
	Components[0] = X;
	Components[1] = Y;
}

FSwizzleMask::FSwizzleMask(EVectorComponent X, EVectorComponent Y, EVectorComponent Z)
: NumComponents{ 3 }
{
	Components[0] = X;
	Components[1] = Y;
	Components[2] = Z;
}

FSwizzleMask::FSwizzleMask(EVectorComponent X, EVectorComponent Y, EVectorComponent Z, EVectorComponent W)
: NumComponents{ 4 }
{
	Components[0] = X;
	Components[1] = Y;
	Components[2] = Z;
	Components[3] = W;
}

FSwizzleMask::FSwizzleMask(bool bMaskX, bool bMaskY, bool bMaskZ, bool bMaskW)
{
	if (bMaskX)
	{
		Append(MIR::EVectorComponent::X);
	}
	if (bMaskY)
	{
		Append(MIR::EVectorComponent::Y);
	}
	if (bMaskZ)
	{
		Append(MIR::EVectorComponent::Z);
	}
	if (bMaskW)
	{
		Append(MIR::EVectorComponent::W);
	}
}

void FSwizzleMask::Append(EVectorComponent Component)
{
	check(NumComponents < 4);
	Components[NumComponents++] = Component;
}

bool FSwizzleMask::IsXYZW() const
{
	return
		NumComponents == 4 &&
		Components[0] == EVectorComponent::X &&
		Components[1] == EVectorComponent::Y &&
		Components[2] == EVectorComponent::Z &&
		Components[3] == EVectorComponent::W
		;
}

struct FEmitter::FPrivate
{
	// Searches the emitter’s value set for an existing FValue matching Prototype, or returns nullptr.
	static FValue* FindValue(FEmitter& Emitter, const FValue* Prototype)
	{
		FValue** Value = Emitter.ValueSet.Find(Prototype);
		return Value ? *Value : nullptr;
	}

	// Allocates zero-initialized memory of given size and alignment in the module’s arena.
	static void* Allocate(FEmitter& Emitter, int32 Size, int32 Alignment)
	{
		return Emitter.Module->Allocator.PushBytes(Size, Alignment);
	}

	// Registers a new value in the module.
	static void PushValueToModule(FEmitter& Emitter, FValue* Value)
	{
		Emitter.Module->AddValue(Value);
		Emitter.ValueSet.Add(Value);
	}
};

// Creates a copy of specified array using the Module allocator and returns it.
template <typename T>
static TConstArrayView<T> MakeArrayCopy(FEmitter& Emitter, TConstArrayView<T> Array)
{
	T* Data = (T*)FEmitter::FPrivate::Allocate(Emitter, sizeof(T) * Array.Num(), alignof(T));
	FMemory::Memcpy(Data, Array.GetData(), sizeof(T) * Array.Num());
	return { Data, Array.Num() };
}

// Creates a “prototype” value of type T with the specified MIR type.
// Emit this value later with a matching EmitPrototype() call, which returns the actual value instance
// after potential deduplication.
template <typename T>
static T MakePrototype(FType InType)
{
	 static_assert(std::is_trivially_destructible_v<T> && std::is_trivially_copy_constructible_v<T> && std::is_trivially_copy_assignable_v<T>,
		"FValues are expected to be trivial types.");

	T Value;
	FMemory::Memzero(Value);
	Value.Kind = T::TypeKind;
	Value.Type = InType;

	return Value;
}

// Allocates and initializes a temporary composite value with the given number of components.
static FComposite* MakeCompositePrototype(FEmitter& Emitter, FType Type, int NumComponents)
{
	// Compute the total size of this composite value.
	int32 SizeInBytes = sizeof(FComposite) + sizeof(FValue*) * NumComponents;

	// Allocate a temporary buffer for it.
	FComposite* Value = (FComposite*)FMemStack::Get().Alloc(SizeInBytes, alignof(FComposite));

	// Zero its memory and set it up
	FMemory::Memzero(Value, SizeInBytes);
	Value->Kind = VK_Composite;
	Value->Type = Type;
	if (Type.IsPrimitive() && !Type.IsDouble())
	{
		Value->SetSubgraphProperties(EGraphProperties::Uniform);
	}

	return Value;
}

// Emits a new value into the module, performing no de-duplication with existing values in the module.
// NOTE: This function should **only** be used when emitting a "terminal" instruction that is known
// to not have duplicates, condition checked externally by the caller.
template <typename T>
static T* EmitNew(FEmitter& Emitter, FType Type)
{
	// Otherwise, create a new value allocating the necessary memory in the module's arena.
	T* Value = (T*)FEmitter::FPrivate::Allocate(Emitter, sizeof(T), alignof(T));
	FMemory::Memzero(Value, sizeof(T));

	// Set up the basics
	Value->Kind = T::TypeKind;
	Value->Type = Type;

	FEmitter::FPrivate::PushValueToModule(Emitter, Value);

	return Value;
}

// Emits a prototype value into the module, deduplicating if an identical value was already emitted.
static FValueRef EmitPrototype(FEmitter& Emitter, const FValue& Prototype)
{
	// Optimization: See if we emitted this value before, and if so, since MIR is SSA, with
	// instructions having being the equivalent of "pure functions" with no side effects,
	// simply return the existing value which holds the already computed result.
	if (FValue* Existing = FEmitter::FPrivate::FindValue(Emitter, &Prototype))
	{
		return Existing;
	}

	// Otherwise, create a new value allocating the necessary memory in the module's arena.
	FValue* Value = (FValue*)FEmitter::FPrivate::Allocate(Emitter, Prototype.GetSizeInBytes(), alignof(FValue));

	// Bitcopy the prototype into the value memory.
	FMemory::Memcpy(Value, &Prototype, Prototype.GetSizeInBytes());

	// Push the value to the module.
	FEmitter::FPrivate::PushValueToModule(Emitter, Value);

	// Verify that value hashing is deterministic.
	checkSlow(Value == FEmitter::FPrivate::FindValue(Emitter, &Prototype));

	return Value;
}

// Finds the expression input index. Although the implementation has O(n) complexity, it is only used for error reporting.
static int32 SlowFindExpressionInputIndex(UMaterialExpression* Expression, const FExpressionInput* InInput)
{
	for (FExpressionInputIterator It{ Expression }; It; ++It)
	{
		if (It.Input == InInput)
		{
			return It.Index;
		}
	}
	return -1;
}

// Finds the expression input name. Although the implementation has O(n) complexity, it is only used for error reporting.
static FName SlowFindInputName(UMaterialExpression* Expression, const FExpressionInput* InInput)
{
	int32 InputIndex = SlowFindExpressionInputIndex(Expression, InInput);
	return InputIndex != INDEX_NONE ? Expression->GetInputName(InputIndex) : FName{};
}

/*------------------------------------- FValueRef --------------------------------------*/

static inline bool IsAnyNotValid()
{
	return false;
}

// Returns whether any of the values is invalid (null or poison).
template <typename... TTail>
static bool IsAnyNotValid(FValueRef Head, TTail... Tail)
{
	return !Head.IsValid() || IsAnyNotValid(Tail...);
}

// Returns whether any of the values is invalid (null or poison).
static inline bool IsAnyNotValid(TConstArrayView<FValueRef> Values)
{
	for (FValueRef Value : Values)
	{
		if (!Value.IsValid())
		{
			return true;
		}
	}
	return false;
}

bool FValueRef::IsValid() const
{
	return Value && !Value->IsPoison();
}

bool FValueRef::IsPoison() const
{
	return Value && Value->IsPoison();
}

FValueRef FValueRef::To(FValue* InValue) const
{
	return { InValue, Input };
}

FValueRef FValueRef::ToPoison() const
{
	return To(FPoison::Get());
}

/*----------------------------------- FFunctionDesc -----------------------------------*/

bool FFunctionDesc::PushInputOnlyParameter(FName InName, FType InType)
{
	// You must input-only parameters first, before the others.
	check(!NumInputOutputParams && !NumOutputOnlyParams);

	if (GetNumParameters() == MIR::MaxNumFunctionParameters)
	{
		return false;
	}

	Parameters[NumInputOnlyParams++] = { InName, InType };
	return true;
}
	
bool FFunctionDesc::PushInputOutputParameter(FName InName, FType InType)
{
	checkf(!NumOutputOnlyParams, TEXT("You must input-output parameters before you push any output-only parameters."));

	if (GetNumParameters() == MIR::MaxNumFunctionParameters)
	{
		return false;
	}

	Parameters[NumInputOnlyParams + NumInputOutputParams] = { InName, InType };
	++NumInputOutputParams;
	return true;
}
	
bool FFunctionDesc::PushOutputOnlyParameter(FName InName, FType InType)
{
	if (GetNumParameters() == MIR::MaxNumFunctionParameters)
	{
		return false;
	}

	Parameters[NumInputOnlyParams + NumInputOutputParams + NumOutputOnlyParams] = { InName, InType };
	++NumOutputOnlyParams;
	return true;
}

/*----------------------------------- Error handling -----------------------------------*/

void FEmitter::Error(FValueRef Source, FStringView Message)
{
	Source.Input
		? Error(FString::Printf(TEXT("From expression input '%s': %s"), *SlowFindInputName(Expression, Source.Input).ToString(),Message.GetData()))
		: Error(Message);
}

void FEmitter::Error(FStringView Message)
{
	FMaterialIRModule::FError Error;
	Error.Expression = Expression;

	// Add the node type to the error message
	if (Expression)
	{
		const int32 ChopCount = FCString::Strlen(TEXT("MaterialExpression"));
		const FString& ErrorClassName = Expression->GetClass()->GetName();

		Error.Message.Appendf(TEXT("(Node %s) %s"), *ErrorClassName + ChopCount, Message.GetData());
	}
	else
	{
		// Error without any associated expression being built.
		Error.Message.Appendf(TEXT("%s"), Message.GetData());
	}
	
	Module->Errors.Push(Error);

	State = EState::Error;
}

/*--------------------------------- Type handling ----------------------------------*/

FType FEmitter::TryGetCommonType(FType A, FType B)
{
	// Trivial case: types are equal
	if (A == B)
	{
		return A;
	}

	// Primitive types can only be constructed from other primitive types.
	TOptional<FPrimitive> PrimitiveA = A.AsPrimitive();
	TOptional<FPrimitive> PrimitiveB = B.AsPrimitive();
	if (!PrimitiveA || !PrimitiveB)
	{
		return FType::MakePoison();
	}
	
	// No common type between row and column vectors
	if ((PrimitiveA->IsRowVector() && PrimitiveB->IsColumnVector()) || (PrimitiveA->IsColumnVector() && PrimitiveB->IsRowVector()))
	{
		return FType::MakePoison();
	}

	// Can't cast a vector to a matrix.
	if ((PrimitiveA->IsRowVector() && PrimitiveB->IsMatrix()) || (PrimitiveA->IsMatrix() && PrimitiveB->IsRowVector()))
	{
		return FType::MakePoison();
	}
	
	// Return the primitive type with the maximum number of rows and columns between the two types.
	EScalarKind ScalarKind = FMath::Max(PrimitiveA->ScalarKind, PrimitiveB->ScalarKind);
	int32 NumRows = FMath::Max(PrimitiveA->NumRows, PrimitiveB->NumRows);
	int32 NumColumns = FMath::Max(PrimitiveA->NumColumns, PrimitiveB->NumColumns);
	return FType::MakePrimitive(ScalarKind, NumRows, NumColumns);
}

FType FEmitter::GetCommonType(FType A, FType B)
{
	if (FType CommonType = TryGetCommonType(A, B); !CommonType.IsPoison())
	{
		return CommonType;
	}

	Errorf(TEXT("No common type between '%s' and '%s'."), *A.GetSpelling(), *B.GetSpelling());
	return FType::MakePoison();
}

FType FEmitter::GetCommonType(TConstArrayView<FValueRef> Values)
{
	check(!Values.IsEmpty() && Values[0]);

	// Find the common type among non-null values
	FType CommonType = Values[0]->Type;
	for (int32 i = 1; i < Values.Num(); ++i)
	{
		if (Values[i].IsPoison())
		{
			return FType::MakePoison();
		}
		if (Values[i])
		{
			CommonType = TryGetCommonType(CommonType, Values[i]->Type);
		}
	}

	// If common type is valid, return it
	if (!CommonType.IsPoison())
	{
		return CommonType;
	}

	// ...otherwise generate an error. This error message prints the input the values
	// come from, if available.

	// Search for the last valid index in the values array, so that we know when to print " and "
	int32 LastIndex = Values.Num() - 1;
	for (; LastIndex >= 0; --LastIndex)
	{
		if (Values[LastIndex])
		{
			break;
		}
	}

	// Whether some value has already been reported (used to print the comma ", ")
	bool bSomeValueAlreadyPrinted = false;

	FString ErrorMsg{ TEXTVIEW("No common type between ") };
	for (int32 i = 0; i < Values.Num(); ++i)
	{
		if (!Values[i])
		{
			continue;
		}

		FValueRef Value = Values[i];

		if (i == LastIndex)
		{
			ErrorMsg.Append(TEXTVIEW(" and "));
		}
		else if (bSomeValueAlreadyPrinted)
		{
			ErrorMsg.Append(TEXTVIEW(", "));
		}

		ErrorMsg.Appendf(TEXT("'%s'"), *Value->Type.GetSpelling());
		
		if (Value.Input)
		{
			ErrorMsg.Appendf(TEXT(" (from input '%s')"), *SlowFindInputName(Expression, Value.Input).ToString());
		}

		bSomeValueAlreadyPrinted = true;
	}

	ErrorMsg.AppendChar('.');
	Error(ErrorMsg);

	return FType::MakePoison();
}

FType FEmitter::GetMaterialAggregateAttributeType(const UMaterialAggregate* Aggregate, int32 AttributeIndex)
{
	check(AttributeIndex >= 0);

	if (AttributeIndex >= Aggregate->Attributes.Num())
	{
		Errorf(TEXT("Invalid attribute index %d for material aggregate '%s'. Index is out of range (Num = %d)."),
			   AttributeIndex, *Aggregate->GetName(), Aggregate->Attributes.Num());
		return FType::MakePoison();
	}

	switch (Aggregate->Attributes[AttributeIndex].Type)
	{
		case EMaterialAggregateAttributeType::Bool1: return FType::MakeVector(EScalarKind::Boolean, 1);
		case EMaterialAggregateAttributeType::Bool2: return FType::MakeVector(EScalarKind::Boolean, 2);
		case EMaterialAggregateAttributeType::Bool3: return FType::MakeVector(EScalarKind::Boolean, 3);
		case EMaterialAggregateAttributeType::Bool4: return FType::MakeVector(EScalarKind::Boolean, 4);
		case EMaterialAggregateAttributeType::ShadingModel: return FType::MakeIntScalar();
		case EMaterialAggregateAttributeType::UInt1: return FType::MakeVector(EScalarKind::Integer, 1);
		case EMaterialAggregateAttributeType::UInt2: return FType::MakeVector(EScalarKind::Integer, 2);
		case EMaterialAggregateAttributeType::UInt3: return FType::MakeVector(EScalarKind::Integer, 3);
		case EMaterialAggregateAttributeType::UInt4: return FType::MakeVector(EScalarKind::Integer, 4);
		case EMaterialAggregateAttributeType::Float1: return FType::MakeVector(EScalarKind::Float, 1);
		case EMaterialAggregateAttributeType::Float2: return FType::MakeVector(EScalarKind::Float, 2);
		case EMaterialAggregateAttributeType::Normal:
		case EMaterialAggregateAttributeType::Float3: return FType::MakeVector(EScalarKind::Float, 3);
		case EMaterialAggregateAttributeType::Float4: return FType::MakeVector(EScalarKind::Float, 4);
		case EMaterialAggregateAttributeType::MaterialAttributes: return FType::MakeAggregate(MaterialAttributesAggregate::Get());
		case EMaterialAggregateAttributeType::SubstrateData: return FType::MakeSubstrateData();
		case EMaterialAggregateAttributeType::Aggregate: return FType::MakeAggregate(Aggregate->Attributes[AttributeIndex].Aggregate.Get());
		default: UE_MIR_UNREACHABLE();
	}
}

/*-------------------------------- Input management --------------------------------*/

FValueRef FEmitter::TryInput(const FExpressionInput* InInput)
{
	TOptional<FValue*> Result = Internal::RequestInputValue(BuilderImpl, InInput);
	if (!Result.IsSet())
	{
		check(bExpressionRequestsInputsOnDemand);
		State = EState::Pending;
		return { FPoison::Get(), InInput };
	}
	return FValueRef{ Result.GetValue(), InInput };
}

FValueRef FEmitter::Input(const FExpressionInput* InInput)
{
	FValueRef Value = TryInput(InInput);
	if (!Value)
	{
		Errorf(TEXT("Missing '%s' input value."), *SlowFindInputName(Expression, InInput).ToString());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::InputDefaultBool(const FExpressionInput* Input, bool Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantBool(Default));
}

FValueRef FEmitter::InputDefaultInt(const FExpressionInput* Input, TInteger Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantInt(Default));
}

FValueRef FEmitter::InputDefaultInt2(const FExpressionInput* Input, UE::Math::TIntVector2<TInteger> Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantInt2(Default));
}

FValueRef FEmitter::InputDefaultInt3(const FExpressionInput* Input, UE::Math::TIntVector3<TInteger> Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantInt3(Default));
}

FValueRef FEmitter::InputDefaultInt4(const FExpressionInput* Input, UE::Math::TIntVector4<TInteger> Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantInt4(Default));
}

FValueRef FEmitter::InputDefaultFloat(const FExpressionInput* Input, TFloat Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantFloat(Default));
}

FValueRef FEmitter::InputDefaultFloat2(const FExpressionInput* Input, UE::Math::TVector2<TFloat> Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantFloat2(Default));
}

FValueRef FEmitter::InputDefaultFloat3(const FExpressionInput* Input, UE::Math::TVector<TFloat> Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantFloat3(Default));
}

FValueRef FEmitter::InputDefaultFloat4(const FExpressionInput* Input, UE::Math::TVector4<TFloat> Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Value.To(ConstantFloat4(Default));
}

FValueRef FEmitter::InputDefault(const FExpressionInput* Input, FValueRef Default)
{
	FValueRef Value = TryInput(Input);
	return Value ? Value : Default;
}

FValueRef FEmitter::CheckTypeIsKind(FValueRef Value, ETypeKind Kind)
{
	if (Value.IsValid() && !Value->Type.Is(Kind))
	{
		Errorf(Value, TEXT("Expected a '%s' value, got a '%s' instead."), TypeKindToString(Kind), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsPrimitive(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.AsPrimitive())
	{
		Errorf(Value, TEXT("Expected a primitive value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsArithmetic(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.IsArithmetic())
	{
		Errorf(Value, TEXT("Expected an arithmetic value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsBoolean(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.IsBoolean())
	{
		Errorf(Value, TEXT("Expected a boolean value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsInteger(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.IsInteger())
	{
		Errorf(Value, TEXT("Expected an integer value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsScalar(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.IsScalar())
	{
		Errorf(Value, TEXT("Expected a scalar value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsVector(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.IsVector())
	{
		Errorf(Value, TEXT("Expected a vector value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsVector(FValueRef Value, int32 MinNumComponents, int32 MaxNumComponents)
{
	Value = CheckIsVector(Value);
	if (Value.IsValid())
	{
		int32 NumComponents = Value->Type.AsPrimitive()->NumComponents();
		if (NumComponents < MinNumComponents)
		{
			(MinNumComponents == MaxNumComponents) 
			? Errorf(Value, TEXT("Expected a vector value of exactly %d components, got a '%s' instead."), MinNumComponents, *Value->Type.GetSpelling())
			: Errorf(Value, TEXT("Expected a vector value of at least %d components, got a '%s' instead."), MinNumComponents, *Value->Type.GetSpelling());
			return Value.ToPoison();
		}
		if (NumComponents > MaxNumComponents)
		{
			Errorf(Value, TEXT("Expected a vector value of at most %d components, got a '%s' instead."), MaxNumComponents, *Value->Type.GetSpelling());
			return Value.ToPoison();
		}
	}
	return Value;
}

FValueRef FEmitter::CheckIsScalarOrVector(FValueRef Value)
{
	if (Value.IsValid() && (!Value->Type.AsPrimitive() || Value->Type.GetPrimitive().IsMatrix()))
	{
		Errorf(Value, TEXT("Expected a scalar or vector value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsMatrix(FValueRef Value)
{
	if (Value.IsValid() && (!Value->Type.AsPrimitive() || !Value->Type.GetPrimitive().IsMatrix()))
	{
		Errorf(Value, TEXT("Expected a matrix value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsTexture(FValueRef Value)
{
	if (Value.IsValid() && !Value->Type.IsTexture())
	{
		Errorf(Value, TEXT("Expected a texture value, got a '%s' instead."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	return Value;
}

FValueRef FEmitter::CheckIsAggregate(FValueRef Value, const UMaterialAggregate* Aggregate)
{
	if (Value.IsValid())
	{
		const UMaterialAggregate* ValueAggregate = Value->Type.AsAggregate();
		if (!ValueAggregate)
		{
			Errorf(Value, TEXT("Expected an aggregate value, got a '%s' instead."), *Value->Type.GetSpelling());
			return Value.ToPoison();
		}
		if (Aggregate && ValueAggregate != Aggregate)
		{
			Errorf(Value, TEXT("Expected a value of aggregate type '%s', got a value of aggregate type '%s' instead."), *Aggregate->GetName(), *Value->Type.AsAggregate()->GetName());
			return Value.ToPoison();
		}
	}
	return Value;
}

void FEmitter::CheckMaterialDomainIsAnyOf(TConstArrayView<EMaterialDomain> Domains)
{
	if (Domains.Contains(BaseMaterial->MaterialDomain))
	{
		return;
	}

	// Report the error message.
	FString ErrorMessage = TEXT("Expression does not support material domain '");
	ErrorMessage += StaticEnum<EMaterialDomain>()->GetValueAsString(BaseMaterial->MaterialDomain);
	ErrorMessage += TEXTVIEW("' (supported are ");

	for (int32 i = 0; i < Domains.Num(); ++i)
	{
		if (i > 0)
		{
			ErrorMessage += TEXTVIEW(", ");
		}
		ErrorMessage += StaticEnum<EMaterialDomain>()->GetValueAsString(Domains[i]);
	}
	ErrorMessage += TEXTVIEW(")");

	Error(ErrorMessage);
}

void FEmitter::CheckMaterialHasUsageFlag(EMaterialUsage UsageFlag)
{
	if (!MaterialInterface->GetUsageByFlag(UsageFlag))
	{
		Errorf(TEXT("This expression requires the material to enable usage flags %s."), *StaticEnum<EMaterialUsage>()->GetValueAsString(UsageFlag));
	}
}

void FEmitter::CheckFeatureLevelIsAtLeast(ERHIFeatureLevel::Type MinFeatureLevel)
{
	if (GetFeatureLevel() < MinFeatureLevel)
	{
		FString FeatureLevelName, MinFeatureLevelName;
		GetFeatureLevelName(GetFeatureLevel(), FeatureLevelName);
		GetFeatureLevelName(MinFeatureLevel, MinFeatureLevelName);
		Errorf(TEXT("Expression not supported with feature level %s. %s or higher required."), *FeatureLevelName, *MinFeatureLevelName);
	}
}

bool FEmitter::ToConstantBool(FValueRef Value)
{
	if (!Value.IsValid())
	{
		return false;
	}
	const FConstant* Constant = Value->As<FConstant>();
    if (!Constant)
    {
        Errorf(Value, TEXT("Expected a constant bool value, got a non-constant value instead."));
        return false;
    }
	if (Constant->Type != FType::MakeBoolScalar())
    {
        Errorf(Value, TEXT("Expected a constant bool value, got a '%s' instead."), *Constant->Type.GetSpelling());
        return false;
    }
	return Constant->Boolean;
 }

/*-------------------------------- Output management -------------------------------*/

FEmitter& FEmitter::Output(int32 OutputIndex, FValueRef Value)
{
	if (Value.IsValid())
	{
		Output(Expression->GetOutput(OutputIndex), Value);
	}
	return *this;
}

static MIR::FSwizzleMask GetSwizzleMaskFromExpressionOutputMask(const FExpressionOutput& ExpressionOutput)
{
	MIR::FSwizzleMask SwizzleMask;
	if (ExpressionOutput.MaskR)
	{
		SwizzleMask.Append(MIR::EVectorComponent::X);
	}
	if (ExpressionOutput.MaskG)
	{
		SwizzleMask.Append(MIR::EVectorComponent::Y);
	}
	if (ExpressionOutput.MaskB)
	{
		SwizzleMask.Append(MIR::EVectorComponent::Z);
	}
	if (ExpressionOutput.MaskA)
	{
		SwizzleMask.Append(MIR::EVectorComponent::W);
	}
	return SwizzleMask;
}

FEmitter& FEmitter::Output(const FExpressionOutput* ExpressionOutput, FValueRef Value)
{
	if (Value.IsValid())
	{
		Internal::BindValueToExpressionOutput(BuilderImpl, ExpressionOutput, Value);
	}
	return *this;
}

FEmitter& FEmitter::OutputsWithComponentMask(FValueRef Value)
{
	for (const FExpressionOutput& ExpressionOutput : Expression->GetOutputs())
	{
		// Get the swizzle mask from this expression output component mask
		FSwizzleMask SwizzleMask = GetSwizzleMaskFromExpressionOutputMask(ExpressionOutput);

		// Apply the output's swizzle mask to the value
		FValueRef OutputValue = SwizzleMask.NumComponents > 0 ? Swizzle(Value, SwizzleMask) : Value;

		// Bind the resulting value to the output
		Output(&ExpressionOutput, OutputValue);
	}
	return *this;
}

/*------------------------------- Subgraph management ------------------------------*/

void FEmitter::Subgraph(int32 NumInputs, TArrayView<FSubgraphOutputMapping> Outputs)
{
	Internal::DeclareSubgraph(BuilderImpl, NumInputs, Outputs);
}

FValueRef FEmitter::TrySubgraphInput(int32 InputIndex)
{
	TOptional<FValue*> Result = Internal::RequestSubgraphInputValue(BuilderImpl, InputIndex);
	if (!Result.IsSet())
	{
		check(bExpressionRequestsInputsOnDemand);
		State = EState::Pending;
		return { FPoison::Get(), nullptr };
	}
	return FValueRef{ Result.GetValue(), nullptr };
}

FValueRef FEmitter::SubgraphInput(int32 InputIndex)
{
	FValueRef Value = TrySubgraphInput(InputIndex);
	if (!Value)
	{
		Errorf(TEXT("Missing subgraph input %d."), InputIndex);
		return Value.ToPoison();
	}
	return Value;
}

UMaterialExpression* FEmitter::GetSubgraphExpression()
{
	return Internal::GetSubgraphExpression(BuilderImpl);
}

/*------------------------------- Constants emission -------------------------------*/

FValueRef FEmitter::ConstantFromShaderValue(const UE::Shader::FValue& InValue)
{
	using namespace UE::Shader;

	switch (InValue.Type.ValueType)
	{
		case UE::Shader::EValueType::Int1: return ConstantInt(InValue.AsFloatScalar());
		case UE::Shader::EValueType::Int2: return ConstantInt2({ InValue.Component[0].Int, InValue.Component[1].Int });
		case UE::Shader::EValueType::Int3: return ConstantInt3({ InValue.Component[0].Int, InValue.Component[1].Int, InValue.Component[2].Int });
		case UE::Shader::EValueType::Int4: return ConstantInt4({ InValue.Component[0].Int, InValue.Component[1].Int, InValue.Component[2].Int, InValue.Component[3].Int });
	
		case UE::Shader::EValueType::Float1: return ConstantFloat(InValue.AsFloatScalar());
		case UE::Shader::EValueType::Float2: return ConstantFloat2(UE::Math::TVector2<TFloat>{ InValue.Component[0].Float, InValue.Component[1].Float });
		case UE::Shader::EValueType::Float3: return ConstantFloat3(UE::Math::TVector<TFloat>{ InValue.Component[0].Float, InValue.Component[1].Float, InValue.Component[2].Float });
		case UE::Shader::EValueType::Float4: return ConstantFloat4(UE::Math::TVector4<TFloat>{ InValue.Component[0].Float, InValue.Component[1].Float, InValue.Component[2].Float, InValue.Component[3].Float });

		case UE::Shader::EValueType::Double1: return ConstantDouble(InValue.Component[0].Double);
		case UE::Shader::EValueType::Double2: return Vector2(ConstantDouble(InValue.Component[0].Double), ConstantDouble(InValue.Component[1].Double));
		case UE::Shader::EValueType::Double3: return Vector3(ConstantDouble(InValue.Component[0].Double), ConstantDouble(InValue.Component[1].Double), ConstantDouble(InValue.Component[2].Double));
		case UE::Shader::EValueType::Double4: return Vector4(ConstantDouble(InValue.Component[0].Double), ConstantDouble(InValue.Component[1].Double), ConstantDouble(InValue.Component[2].Double), ConstantDouble(InValue.Component[3].Double));
	}

	UE_MIR_UNREACHABLE();
}

FValueRef FEmitter::ConstantDefault(FType Type)
{
	if (TOptional<FPrimitive> PrimitiveType = Type.AsPrimitive())
	{
		FValueRef Zero = ConstantZero(PrimitiveType->ScalarKind);
		if (PrimitiveType->IsScalar())
		{
			return Zero;
		}
		else
		{
			FComposite* Composite = MakeCompositePrototype(*this, Type, PrimitiveType->NumComponents());
			for (FValue*& Component : Composite->GetMutableComponents())
			{
				Component = Zero;
			}
			return EmitPrototype(*this, *Composite);
		}
	}
	else if (const UMaterialAggregate* TypeAggregate = Type.AsAggregate())
	{
		return Aggregate(TypeAggregate);
	}
	else
	{
		Errorf(TEXT("Type '%s' has no default. Expected primitive or aggregate type."), *Type.GetSpelling());
		return Poison();
	}
}

FValueRef FEmitter::ConstantZero(EScalarKind Kind)
{
	switch (Kind)
	{
		case EScalarKind::Boolean: return ConstantFalse();
		case EScalarKind::Integer: return ConstantInt(0);
		case EScalarKind::Float: return ConstantFloat(0.0f);
		case EScalarKind::Double: return ConstantDouble(0.0);
		default: UE_MIR_UNREACHABLE();
	}
}

FValueRef FEmitter::ConstantOne(EScalarKind Kind)
{
	switch (Kind)
	{
		case EScalarKind::Boolean: return ConstantTrue();
		case EScalarKind::Integer: return ConstantInt(1);
		case EScalarKind::Float: return ConstantFloat(1.0f);
		case EScalarKind::Double: return ConstantDouble(1.0);
		default: UE_MIR_UNREACHABLE();
	}
}

FValueRef FEmitter::ConstantScalar(EScalarKind Kind, TDouble Value)
{
	switch (Kind)
	{
		case EScalarKind::Boolean: return ConstantBool(Value != 0.0f);
		case EScalarKind::Integer: return ConstantInt(TInteger(Value));
		case EScalarKind::Float: return ConstantFloat(TFloat(Value));
		case EScalarKind::Double: return ConstantDouble(Value);
		default: UE_MIR_UNREACHABLE();
	}
}

FValueRef FEmitter::ConstantTrue()
{
	return TrueConstant;
}

FValueRef FEmitter::ConstantFalse()
{
	return FalseConstant;
}

FValueRef FEmitter::ConstantBool(bool InX)
{
	return InX ? ConstantTrue() : ConstantFalse();
}

FValueRef FEmitter::ConstantBool2(bool InX, bool InY)
{
	FValueRef X = ConstantBool(InX);
	FValueRef Y = ConstantBool(InY);
	return Vector2(X, Y);
}

FValueRef FEmitter::ConstantBool3(bool InX, bool InY, bool InZ)
{
	FValueRef X = ConstantBool(InX);
	FValueRef Y = ConstantBool(InY);
	FValueRef Z = ConstantBool(InZ);
	return Vector3(X, Y, Z);
}

FValueRef FEmitter::ConstantBool4(bool InX, bool InY, bool InZ, bool InW)
{
	FValueRef X = ConstantBool(InX);
	FValueRef Y = ConstantBool(InY);
	FValueRef Z = ConstantBool(InZ);
	FValueRef W = ConstantBool(InW);
	return Vector4(X, Y, Z, W);
}

FValueRef FEmitter::ConstantInt(TInteger InX)
{
	FConstant Scalar = MakePrototype<FConstant>(FType::MakeScalar(EScalarKind::Integer));
	Scalar.Integer = InX;
	return EmitPrototype(*this, Scalar);
}

FValueRef FEmitter::ConstantInt2(UE::Math::TIntVector2<TInteger> InValue)
{
	FValueRef X = ConstantInt(InValue.X);
	FValueRef Y = ConstantInt(InValue.Y);
	return Vector2(X, Y);
}

FValueRef FEmitter::ConstantInt3(UE::Math::TIntVector3<TInteger> InValue)
{
	FValueRef X = ConstantInt(InValue.X);
	FValueRef Y = ConstantInt(InValue.Y);
	FValueRef Z = ConstantInt(InValue.Z);
	return Vector3(X, Y, Z);
}

FValueRef FEmitter::ConstantInt4(UE::Math::TIntVector4<TInteger> InValue)
{
	FValueRef X = ConstantInt(InValue.X);
	FValueRef Y = ConstantInt(InValue.Y);
	FValueRef Z = ConstantInt(InValue.Z);
	FValueRef W = ConstantInt(InValue.W);
	return Vector4(X, Y, Z, W);
}

FValueRef FEmitter::ConstantFloat(TFloat InX)
{
	FConstant Scalar = MakePrototype<FConstant>(FType::MakeScalar(EScalarKind::Float));
	Scalar.Float = InX;
	Scalar.SetSubgraphProperties(EGraphProperties::Uniform);
	return EmitPrototype(*this, Scalar);
}

FValueRef FEmitter::ConstantFloat2(UE::Math::TVector2<TFloat> InValue)
{
	FValueRef X = ConstantFloat(InValue.X);
	FValueRef Y = ConstantFloat(InValue.Y);
	return Vector2(X, Y);
}

FValueRef FEmitter::ConstantFloat3(UE::Math::TVector<TFloat> InValue)
{
	FValueRef X = ConstantFloat(InValue.X);
	FValueRef Y = ConstantFloat(InValue.Y);
	FValueRef Z = ConstantFloat(InValue.Z);
	return Vector3(X, Y, Z);
}

FValueRef FEmitter::ConstantFloat4(UE::Math::TVector4<TFloat> InValue)
{
	FValueRef X = ConstantFloat(InValue.X);
	FValueRef Y = ConstantFloat(InValue.Y);
	FValueRef Z = ConstantFloat(InValue.Z);
	FValueRef W = ConstantFloat(InValue.W);
	return Vector4(X, Y, Z, W);
}

FValueRef FEmitter::ConstantDouble(TDouble InX)
{
	FConstant Scalar = MakePrototype<FConstant>(FType::MakeDoubleScalar());
	Scalar.Double = InX;
	return EmitPrototype(*this, Scalar);
}

FValueRef FEmitter::ConstantDouble2(UE::Math::TVector2<TDouble> InValue)
{
	FValueRef X = ConstantDouble(InValue.X);
	FValueRef Y = ConstantDouble(InValue.Y);
	return Vector2(X, Y);
}

FValueRef FEmitter::ConstantDouble3(UE::Math::TVector<TDouble> InValue)
{
	FValueRef X = ConstantDouble(InValue.X);
	FValueRef Y = ConstantDouble(InValue.Y);
	FValueRef Z = ConstantDouble(InValue.Z);
	return Vector3(X, Y, Z);
}

FValueRef FEmitter::ConstantDouble4(UE::Math::TVector4<TDouble> InValue)
{
	FValueRef X = ConstantDouble(InValue.X);
	FValueRef Y = ConstantDouble(InValue.Y);
	FValueRef Z = ConstantDouble(InValue.Z);
	FValueRef W = ConstantDouble(InValue.W);
	return Vector4(X, Y, Z, W);
}

/*--------------------- Other non-instruction values emission ---------------------*/

FValueRef FEmitter::Poison()
{
	if (CVarMaterialIRDebugBreakOnPoison.GetValueOnGameThread())
	{
		UE_DEBUG_BREAK();
	}
	return FPoison::Get();
}

FValueRef FEmitter::AnalysisError(FType Type, FStringView Message)
{
	FAnalysisError* Error = EmitNew<FAnalysisError>(*this, Type);
	Error->Expression = Expression;
	Error->Message = Module->InternString(Message);
	return Error;
}

FValueRef FEmitter::Builtin(EBuiltin Id)
{
	FType Type = FType::MakeFloatScalar();
	FBuiltin Prototype = MakePrototype<FBuiltin>(Type);
	Prototype.Id = Id;
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::MaterialParameterCollection(UMaterialParameterCollection* Collection)
{
	FMaterialParameterCollection Prototype = MakePrototype<FMaterialParameterCollection>(FType::MakeParameterCollection());
	Prototype.Collection = Collection;
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::NamedPrimitiveUniform(FName Name, FValueRef DefaultConstant)
{
	if (!DefaultConstant->Type.AsPrimitive())
	{
		Error(TEXT("Default value must have primitive type."));
		return Poison();
	}

	FPrimitive Primitive = DefaultConstant->Type.GetPrimitive();
	if (Primitive.IsMatrix())
	{
		Error(TEXT("Matrix primitive uniforms are unsupported."));
		return Poison();
	}

	if (!DefaultConstant->IsConstant())
	{
		Error(TEXT("Default value is expected to be constant."));
		return Poison();
	}

	FPrimitiveUniform Proto = MakePrototype<FPrimitiveUniform>(DefaultConstant->Type);
	Proto.Name = FScriptName{ Name };
	Proto.DefaultConstant = DefaultConstant; 
	Proto.Analysis_UniformIndex = -1;
	Proto.SetSubgraphProperties(EGraphProperties::HasParameter | EGraphProperties::Uniform);

	return EmitPrototype(*this, Proto);
}

FValueRef FEmitter::TextureUniform(FName Name, UTexture* Texture, EMaterialSamplerType SamplerType)
{
	check(Texture);

	// @massimo.tristano move this to ExpressionsIR.cpp
	FString SamplerTypeError;
	if (!MaterialExpressionUtils::VerifySamplerType(GetShaderPlatform(), GetTargetPlatform(), Texture, SamplerType, SamplerTypeError))
	{
		Errorf(TEXT("%s"), *SamplerTypeError);
		return Poison();
	}

	// When virtual texturing is disabled we need to fall back to texture 2D.
	FType TextureType = FType::FromMaterialValueType(Texture->GetMaterialType());
	if (TextureType.IsVirtualTexture() && !UseVirtualTexturing(GetShaderPlatform()))
	{
		TextureType = FType::MakeTexture2D();
	}

	FTextureUniform Proto = MakePrototype<FTextureUniform>(TextureType);
	Proto.Name = FScriptName{ Name };
	Proto.Texture = Texture;
	Proto.SamplerType = SamplerType;
	Proto.Analysis_UniformIndex = INDEX_NONE;
	
	// This needs to flagged as Uniform, because it may be referenced as a child of FPreshaderParameter, which we want to be Uniform.
	// Uniform status on a graph node requires all children of the node to also be Uniformn.
	Proto.SetSubgraphProperties(EGraphProperties::Uniform);
	
	return EmitPrototype(*this, Proto);
}

// Maps a sampler type from standard texture (ST) to virtual texture (VT).
static EMaterialSamplerType PromoteSamplerTypeFromSTtoVT(EMaterialSamplerType InSamplerType)
{
	switch (InSamplerType)
	{
		case SAMPLERTYPE_Color: return SAMPLERTYPE_VirtualColor;
		case SAMPLERTYPE_Grayscale: return SAMPLERTYPE_VirtualGrayscale;
		case SAMPLERTYPE_Alpha: return SAMPLERTYPE_VirtualAlpha;
		case SAMPLERTYPE_Normal: return SAMPLERTYPE_VirtualNormal;
		case SAMPLERTYPE_Masks: return SAMPLERTYPE_VirtualMasks;
		case SAMPLERTYPE_LinearColor: return SAMPLERTYPE_VirtualLinearColor;
		case SAMPLERTYPE_LinearGrayscale: return SAMPLERTYPE_VirtualLinearGrayscale;
	}
	return InSamplerType;
}

// Maps a sampler type from virtual texture (VT) to standard texture (ST).
static EMaterialSamplerType DemoteSamplerTypeFromVTtoST(EMaterialSamplerType InSamplerType)
{
	switch (InSamplerType)
	{
		case SAMPLERTYPE_VirtualColor: return SAMPLERTYPE_Color;
		case SAMPLERTYPE_VirtualGrayscale: return SAMPLERTYPE_Grayscale;
		case SAMPLERTYPE_VirtualAlpha: return SAMPLERTYPE_Alpha;
		case SAMPLERTYPE_VirtualNormal: return SAMPLERTYPE_Normal;
		case SAMPLERTYPE_VirtualMasks: return SAMPLERTYPE_Masks;
		case SAMPLERTYPE_VirtualLinearColor: return SAMPLERTYPE_LinearColor;
		case SAMPLERTYPE_VirtualLinearGrayscale: return SAMPLERTYPE_LinearGrayscale;
	}
	return InSamplerType;
}

FValueRef FEmitter::VirtualTextureUniform(FName Name, URuntimeVirtualTexture* VirtualTexture, int32 VTLayerIndex, int32 VTPageTableIndex)
{
	check(VirtualTexture);
	check(VTLayerIndex >= INT16_MIN && VTLayerIndex <= INT16_MAX);
	check(VTPageTableIndex >= 0 && VTPageTableIndex <= UINT16_MAX);

	FVirtualTextureUniform Proto = MakePrototype<FVirtualTextureUniform>(FType::MakeRuntimeVirtualTexture());
	Proto.Name = FScriptName{ Name };
	Proto.VirtualTexture = VirtualTexture;
	Proto.VTLayerIndex = VTLayerIndex;
	Proto.VTPageTableIndex = VTPageTableIndex;
	Proto.Analysis_UniformIndex = INDEX_NONE;

	// This needs to flagged as Uniform, because it may be referenced as a child of FPreshaderParameter, which we want to be Uniform.
	// Uniform status on a graph node requires all children of the node to also be Uniformn.
	Proto.SetSubgraphProperties(EGraphProperties::Uniform);

	return EmitPrototype(*this, Proto);
}

// For now, only a small subset of opcodes are supported: TextureSize, TexelSize, and RuntimeVirtualTextureUniform.
FValueRef FEmitter::PreshaderParameter(FType Type, EPreshader2Opcode Opcode, FValueRef SourceParameter, FPreshaderParameterPayload Payload)
{
	checkf(
		Opcode == EPreshader2Opcode::TextureSize ||
		Opcode == EPreshader2Opcode::TexelSize ||
		Opcode == EPreshader2Opcode::RuntimeVirtualTextureUniform,
		TEXT("Preshader opcode (0x%X) not supported for parameters in new material translator"), (int32)Opcode
	);

	UObject* SourceParameterTexture = SourceParameter->GetTextureObject();
	if (!SourceParameterTexture)
	{
		Error(TEXT("Missing default texture from source parameter"));
		return Poison();
	}

	FPreshaderParameter Prototype = MakePrototype<FPreshaderParameter>(Type);
	Prototype.SourceParameter = SourceParameter;
	Prototype.Opcode = Opcode;
	Prototype.TextureIndex = MaterialInterface->GetDefaultTextureIdx(SourceParameterTexture);
	Prototype.Payload = MoveTemp(Payload);
	Prototype.SetSubgraphProperties(EGraphProperties::HasParameter | EGraphProperties::Uniform);

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::CustomPrimitiveData(uint32 PrimitiveDataIndex)
{
	// UE_MIR_TODO();
	return {};
}

/*------------------------------ Instruction emission ------------------------------*/

// Helper function to create a single SetMaterialOutput instruction. It is used to create
// the final instruction for both base material attributes and custom outputs.
static FSetMaterialOutput* CreateSetMaterialOutput(FEmitter& Em, FMaterialIRModule* Module, FStringView Name, EMaterialProperty Property, EMaterialOutputFrequency Frequency, FValueRef Arg)
{
	// Material attributes is not a real material attribute, the translator should never create a SetMaterialOutput on this property.
	check(Property != MP_MaterialAttributes); 

	// Create the set material output instruction.
	FSetMaterialOutput* Instr = EmitNew<FSetMaterialOutput>(Em, FType::MakeVoid());
	Instr->Property           = Property;
	Instr->Frequency	      = Frequency;
	Instr->Name               = Module->InternString(Name);
	Instr->Arg                = Arg;

	// Set the instruction priority according to this output property.
	Instr->Priority = Property == MP_Normal ? 0 		// Normal goes first
                    : Property == MP_CustomOutput ? 2 	// Custom outputs go last
                    : 1; 								// All other attributes (BaseColor, Emissive, etc)

	// Add the instruction to list of outputs of the stages it is evaluated in.
	if (Frequency == EMaterialOutputFrequency::PerVertex)
	{
		Module->GetEntryPoint(MIR::Stage_Vertex).Outputs.Push(Instr);
	}
	else
	{
		Module->GetEntryPoint(MIR::Stage_Pixel).Outputs.Push(Instr);
		Module->GetEntryPoint(MIR::Stage_Compute).Outputs.Push(Instr);
	}

	return Instr;
}

static bool IsVertexStageMaterialProperty(EMaterialProperty Property)
{
	return Property == MP_WorldPositionOffset
		|| (Property >= MP_CustomizedUVs0 && Property <= MP_LastCustomizedUVs);
}

void FEmitter::Private_SetMaterialOutput(EMaterialProperty Property, FValueRef Arg)
{
	checkf(Property != MP_CustomOutput, TEXT("This API does not support CustomOutputs, use SetCustomOutput() instead."));

	const FString Name          	 	= MaterialAttributesAggregate::GetAttribute(Property)->Name.ToString();
	const EMaterialOutputFrequency Freq = IsVertexStageMaterialProperty(Property) ? MIR::EMaterialOutputFrequency::PerVertex : MIR::EMaterialOutputFrequency::PerPixel;

	CreateSetMaterialOutput(*this, Module, Name, Property, Freq, Arg);
}

void FEmitter::SetCustomOutputs(FStringView Name, TConstArrayView<FValueRef> Args, EMaterialOutputFrequency Frequency)
{
	// If this is a custom output, check that its name is unique.
	for (const FMaterialIRModule::FCustomOutputGroup& Group : Module->GetCustomOutputGroups())
	{
		if (Group.Name == Name)
		{
			Errorf(TEXT("A custom output with name '%.*s' already exists."), Name.Len(), Name.GetData());
			return;
		}
	}

	// Avoid work if some error occurred.
	if (IsInvalid() || IsAnyNotValid(Args))
	{
		return;
	}

	// Declare a new custom output group to include all generated individual outputs
	Module->BeginCustomOutputGroup(Module->InternString(Name), Args.Num());

	// Allocate a small buffer to make room for the custom output name plus index
	MIR::TTemporaryArray<TCHAR> TempNameBuffer{ Name.Len() + 8 };

	// Copy the custom output base name to the temp buffer. 
	FCString::Snprintf(TempNameBuffer.GetData(), TempNameBuffer.Num(), TEXT("%.*s"), Name.Len(), Name.GetData());

	// Iterate over all the arguments and create a custom output SetMaterialOutput for each
	for (int32 i = 0; i < Args.Num(); ++i)
	{
		// Append the index to the base name to make this custom output name, e.g. "TangentOutput0"
		int32 NameLen = Name.Len() + FCString::Snprintf(TempNameBuffer.GetData() + Name.Len(), TempNameBuffer.Num() - Name.Len(), TEXT("%d"), i);

		// Create the SetMaterialOutput instruction
		FSetMaterialOutput* Instr = CreateSetMaterialOutput(*this, Module, { TempNameBuffer.GetData(), NameLen }, MP_CustomOutput, Frequency, Args[i]);
	
		// Push the instruction to the custom outputs in the module
		Module->AddCustomOutput(Instr);
	}
}

void FEmitter::SetCustomOutputs(FStringView Name, TConstArrayView<FValueRef> Args, EMaterialOutputFrequency Frequency, TFunctionRef<void(FMaterialIRModule&)> Func)
{
	SetCustomOutputs(Name, Args, Frequency);
	Func(*Module);
}

static void InvalidVectorConstructionError(FEmitter& Em, int32 DesiredVectorDimensions, FValueRef A, FValueRef B, FValueRef C = {}, FValueRef D = {})
{
	if (C.IsValid() && D.IsValid())
	{
		Em.Errorf(TEXT("Cannot construct a %dD vector from concatenating primitive values of %d, %d, %d and %d dimensions"), DesiredVectorDimensions,
				  A->Type.GetPrimitive().NumComponents(), 
				  B->Type.GetPrimitive().NumComponents(), 
				  C->Type.GetPrimitive().NumComponents(), 
				  D->Type.GetPrimitive().NumComponents());
	}
	else if (C.IsValid())
	{
		Em.Errorf(TEXT("Cannot construct a %dD vector from concatenating primitive values of %d, %d and %d dimensions"), DesiredVectorDimensions,
				  A->Type.GetPrimitive().NumComponents(), 
				  B->Type.GetPrimitive().NumComponents(), 
				  C->Type.GetPrimitive().NumComponents());
	}
	else
	{
		Em.Errorf(TEXT("Cannot construct a %dD vector from concatenating primitive values of %d and %d dimensions"), DesiredVectorDimensions,
				  A->Type.GetPrimitive().NumComponents(), 
				  B->Type.GetPrimitive().NumComponents());
	}
}

FValueRef FEmitter::Vector2(FValueRef X, FValueRef Y)
{
	// Vector2 has only one constructor from two scalars, for better error reporting, check arguments are scalars first.
	X = CheckIsScalar(X);
	Y = CheckIsScalar(Y);

	FValueRef Result = Vector(X, Y);
	if (Result.IsValid() && Result->Type.AsPrimitive()->NumComponents() != 2)
	{
		InvalidVectorConstructionError(*this, 2, X, Y);
		return Poison();
	}
	return Result;
}

FValueRef FEmitter::Vector3(FValueRef A, FValueRef B, FValueRef C)
{
	FValueRef Result = Vector(A, B, C);
	if (Result.IsValid() && Result->Type.AsPrimitive()->NumComponents() != 3)
	{
		InvalidVectorConstructionError(*this, 3, A, B, C);
		return Poison();
	}
	return Result;
}

FValueRef FEmitter::Vector4(FValueRef A, FValueRef B, FValueRef C, FValueRef D)
{
	FValueRef Result = Vector(A, B, C, D);
	if (Result.IsValid() && Result->Type.AsPrimitive()->NumComponents() != 4)
	{
		InvalidVectorConstructionError(*this, 4, A, B, C, D);
		return Poison();
	}
	return Result;
}

FValueRef FEmitter::Vector(FValueRef A, FValueRef B, FValueRef C, FValueRef D)
{
	check(!D || C); // If D is provided, then C must be provided too.

	// First verify that all given arguments are scalars or vectors
	if (CheckIsScalarOrVector(A).IsPoison() || CheckIsScalarOrVector(B).IsPoison() || CheckIsScalarOrVector(C).IsPoison() || CheckIsScalarOrVector(D).IsPoison())
	{
		return Poison();
	}

	FPrimitive APrimType = *A->Type.AsPrimitive();
	FPrimitive BPrimType = *B->Type.AsPrimitive();
	TOptional<FPrimitive> CPrimType = C.IsValid() ? C->Type.AsPrimitive() : TOptional<FPrimitive>{};
	TOptional<FPrimitive> DPrimType = D.IsValid() ? D->Type.AsPrimitive() : TOptional<FPrimitive>{};

	// Then compute the common scalar kind
	EScalarKind CommonScalarKind = GetCommonScalarKind(APrimType.ScalarKind, BPrimType.ScalarKind);
	if (CPrimType)
	{
		CommonScalarKind = GetCommonScalarKind(CommonScalarKind, CPrimType->ScalarKind);
		if (DPrimType)
		{
			CommonScalarKind = GetCommonScalarKind(CommonScalarKind, DPrimType->ScalarKind);
		}
	}

	// Cast all given arguments to the common scalar kind
	A = CastToScalarKind(A, CommonScalarKind);
	B = CastToScalarKind(B, CommonScalarKind);
	C = CastToScalarKind(C, CommonScalarKind);
	D = CastToScalarKind(D, CommonScalarKind);

	if (A.IsPoison() || B.IsPoison() || C.IsPoison() || D.IsPoison())
	{
		return Poison();
	}

	int32 NumComponents = APrimType.NumComponents() + BPrimType.NumComponents();
	if (CPrimType)
	{
		NumComponents += CPrimType->NumComponents();
		if (DPrimType)
		{
			NumComponents += DPrimType->NumComponents();
		}
	}
	
	if (NumComponents > 4)
	{
		Errorf(TEXT("Cannot create a vector of more than 4 components (%d provided)"), NumComponents);
		return Poison();
	}

	// Make the result vector
	FType ResultType = FType::MakeVector(CommonScalarKind, NumComponents);

	// Allocate the result composite prototype
	FComposite* Result = MakeCompositePrototype(*this, ResultType, NumComponents);

	// Copy arguments components over to the result composite
	int32 ComponentIndex = 0;
	for (int32 i = 0; i < APrimType.NumComponents(); ++i)
	{
		Result->GetMutableComponents()[ComponentIndex++] = Subscript(A, i);
	}
	
	for (int32 i = 0; i < BPrimType.NumComponents(); ++i)
	{
		Result->GetMutableComponents()[ComponentIndex++] = Subscript(B, i);
	}

	if (C.IsValid())
	{
		for (int32 i = 0; i < CPrimType->NumComponents(); ++i)
		{
			Result->GetMutableComponents()[ComponentIndex++] = Subscript(C, i);
		}
		if (D.IsValid())
		{
			for (int32 i = 0; i < DPrimType->NumComponents(); ++i)
			{
				Result->GetMutableComponents()[ComponentIndex++] = Subscript(D, i);
			}
		}

	}

	FValueRef ResultRef = EmitPrototype(*this, *Result);

	// If all arguments come from the same expression input, propagate the input to the result value ref.
	if (A.Input == B.Input && (!C || A.Input == C.Input) && (!D || A.Input == D.Input))
	{
		ResultRef.Input = A.Input;
	}

	return ResultRef;
}

FValueRef FEmitter::Aggregate(const UMaterialAggregate* InAggregate)
{
	return Aggregate(InAggregate, {}, TConstArrayView<FValueRef>{});
}

// This is a necessary, sad special case. See 'EMaterialAggregateAttributeType::Normal' for more info.
static FValueRef EmitNormalDefaultValue(FEmitter& Em)
{
	if (Em.GetBaseMaterial()->bTangentSpaceNormal)
	{
		return Em.ConstantFloat3({ 0.0f, 0.f, 1.f });
	}
	else
	{
		static const FName NAME_VertexNormal = TEXT("VertexNormal");
		return Em.Extern<MIR::FExternFromMaterialDecl>(NAME_VertexNormal);
	}
}

static FValueRef EmitAttributeDefaultValue(FEmitter& Em, UMaterialInterface* Material, const FMaterialAggregateAttribute& Attribute)
{
	switch (Attribute.Type)
	{
		case EMaterialAggregateAttributeType::Bool1: return Em.ConstantBool((bool)Attribute.DefaultValue.X);
		case EMaterialAggregateAttributeType::Bool2: return Em.ConstantBool2((bool)Attribute.DefaultValue.X, (bool)Attribute.DefaultValue.Y);
		case EMaterialAggregateAttributeType::Bool3: return Em.ConstantBool3((bool)Attribute.DefaultValue.X, (bool)Attribute.DefaultValue.Y, (bool)Attribute.DefaultValue.Z);
		case EMaterialAggregateAttributeType::Bool4: return Em.ConstantBool4((bool)Attribute.DefaultValue.X, (bool)Attribute.DefaultValue.Y, (bool)Attribute.DefaultValue.Z, (bool)Attribute.DefaultValue.W);
		case EMaterialAggregateAttributeType::UInt1: return Em.ConstantInt((TInteger)Attribute.DefaultValue.X);
		case EMaterialAggregateAttributeType::UInt2: return Em.ConstantInt2({ (TInteger)Attribute.DefaultValue.X, (TInteger)Attribute.DefaultValue.Y });
		case EMaterialAggregateAttributeType::UInt3: return Em.ConstantInt3({ (TInteger)Attribute.DefaultValue.X, (TInteger)Attribute.DefaultValue.Y, (TInteger)Attribute.DefaultValue.Z });
		case EMaterialAggregateAttributeType::UInt4: return Em.ConstantInt4({ (TInteger)Attribute.DefaultValue.X, (TInteger)Attribute.DefaultValue.Y, (TInteger)Attribute.DefaultValue.Z, (TInteger)Attribute.DefaultValue.W });
		case EMaterialAggregateAttributeType::Float1: return Em.ConstantFloat((TFloat)Attribute.DefaultValue.X);
		case EMaterialAggregateAttributeType::Float2: return Em.ConstantFloat2({ (TFloat)Attribute.DefaultValue.X, (TFloat)Attribute.DefaultValue.Y });
		case EMaterialAggregateAttributeType::Float3: return Em.ConstantFloat3({ (TFloat)Attribute.DefaultValue.X, (TFloat)Attribute.DefaultValue.Y, (TFloat)Attribute.DefaultValue.Z });
		case EMaterialAggregateAttributeType::Float4: return Em.ConstantFloat4(UE::Math::TVector4<TFloat>{ Attribute.DefaultValue.X, Attribute.DefaultValue.Y, Attribute.DefaultValue.Z, Attribute.DefaultValue.W });
		case EMaterialAggregateAttributeType::Normal: return EmitNormalDefaultValue(Em);
		case EMaterialAggregateAttributeType::ShadingModel: return Em.ConstantInt((TInteger)Material->GetShadingModels().GetFirstShadingModel());
		case EMaterialAggregateAttributeType::MaterialAttributes: return Em.Aggregate(MaterialAttributesAggregate::Get());
		case EMaterialAggregateAttributeType::SubstrateData: return Em.SubstrateDefaultSlab();
		case EMaterialAggregateAttributeType::Aggregate: return Em.Aggregate(Attribute.Aggregate);
		default: UE_MIR_UNREACHABLE();
	}
}

FValueRef FEmitter::Aggregate(const UMaterialAggregate* InAggregate, FValueRef InPrototype, TConstArrayView<FValueRef> AttributeValues)
{
	// Check that the specified prototype (if any) aggregate matches the one provided.
	InPrototype = CheckIsAggregate(InPrototype, InAggregate);

	// If a prototype is provided and there are no attribute assignments, this is a no-op, return the prototype.
	if (InPrototype.IsValid() && AttributeValues.IsEmpty())
	{
		return InPrototype;
	}

	// Create the new composite to store the aggregate attribute values.
	FComposite* AggregateValue = MakeCompositePrototype(*this, FType::MakeAggregate(InAggregate), InAggregate->Attributes.Num());
	
	// Assign all components of the new composite value.
	TArrayView<FValue*> Components = AggregateValue->GetMutableComponents();
	for (int32 i = 0; i < Components.Num(); ++i)
	{
		// Get the the ith aggregate aggregate MIR type.
		FType AttributeType = GetMaterialAggregateAttributeType(InAggregate, i);

		if (i < AttributeValues.Num() && AttributeValues[i])
		{
			// Set the this aggregate component to the specified value cast to the attribute type, if present...
			Components[i] = Cast(AttributeValues[i], AttributeType);
		}
		else if (InPrototype)
		{
			// ... otherwise use the component value as in the prototype if provided
			Components[i] = Subscript(InPrototype, i);
		}
		else
		{
			// ...otherwise construct the default value as indicated in the attribute.
			Components[i] = EmitAttributeDefaultValue(*this, MaterialInterface, InAggregate->Attributes[i]);
		}
	}

	return EmitPrototype(*this, *AggregateValue);
}

FValueRef FEmitter::Aggregate(const UMaterialAggregate* InAggregate, FValueRef InPrototype, TConstArrayView<FAttributeAssignment> AttributeAssignments)
{
	// Check that the specified prototype (if any) aggregate matches the one provided.
	InPrototype = CheckIsAggregate(InPrototype, InAggregate);

	// If prototype was provided and is poison, return it..
	if (InPrototype.IsPoison())
	{
		return InPrototype.ToPoison();
	}

	// Allocate temporary storage to hold the attribute values
	TTemporaryArray<FValueRef> AttributeValues { InAggregate->Attributes.Num() };
	AttributeValues.Zero();

	// Linearize the assignments into the array of attribute values.
	for (const FAttributeAssignment& Assignment : AttributeAssignments)
	{
		// Find the attribute index by name
		int32 AttrIndex = InAggregate->FindAttributeIndexByName(Assignment.Name);
		if (AttrIndex != INDEX_NONE)
		{
			// If found, set the slot to the assignment value.
			AttributeValues[AttrIndex] = Assignment.Value;
		}
	}

	return Aggregate(InAggregate, InPrototype, AttributeValues);
}

/*--------------------------------- Operator emission ---------------------------------*/
	
template <typename T>
static bool FoldFloatToBoolOperatorScalar(EOperator Operator, T A)
{
	static_assert(std::is_floating_point_v<T>);
	switch (Operator)
	{
		case UO_IsFinite: return FGenericPlatformMath::IsFinite(A);
		case UO_IsInf:    return !FGenericPlatformMath::IsFinite(A);
		case UO_IsNan:    return FGenericPlatformMath::IsNaN(A);
		default: UE_MIR_UNREACHABLE();
	}
}

template <typename T>
static bool FoldComparisonOperatorScalar(EOperator Operator, T A, T B)
{
	switch (Operator)
	{
		case BO_GreaterThan: return A > B;
		case BO_GreaterThanOrEquals: return A >= B;
		case BO_LessThan: return A < B;
		case BO_LessThanOrEquals: return A <= B;
		case BO_Equals: return A == B;
		case BO_NotEquals: return A != B;
		default: UE_MIR_UNREACHABLE();
	}
}

template <typename T>
static T ACosh(T x)
{
	static_assert(std::is_floating_point_v<T>);
	check(x >= 1);
	return FGenericPlatformMath::Loge(x + FGenericPlatformMath::Sqrt(x * x - 1));
}

template <typename T>
static T ASinh(T x)
{
	static_assert(std::is_floating_point_v<T>);
	return FGenericPlatformMath::Loge(x + FGenericPlatformMath::Sqrt(x * x + 1));
}

template <typename T>
static T ATanh(T x)
{
	static_assert(std::is_floating_point_v<T>);
    check(x > -1 && x < 1);
    return T(0.5) * FGenericPlatformMath::Loge((1 + x) / (1 - x));
}

template <typename T>
static T FoldScalarOperator(FEmitter& Emitter, EOperator Operator, T A, T B, T C)
{
	if constexpr (std::is_floating_point_v<T>)
	{
		switch (Operator)
		{
			case UO_ACos: return FGenericPlatformMath::Acos(A);
			case UO_ACosFast: return FGenericPlatformMath::Acos(A);
			case UO_ACosh: return ACosh(A);
			case UO_ASin: return FGenericPlatformMath::Asin(A);
			case UO_ASinFast: return FGenericPlatformMath::Asin(A);
			case UO_ASinh: return ASinh(A);
			case UO_ATan: return FGenericPlatformMath::Atan(A);
			case UO_ATanFast: return FGenericPlatformMath::Atan(A);
			case UO_ATanh: return ATanh(A);
			case UO_Ceil:
				if constexpr (std::is_same<T, float>::value)
				{
					return FGenericPlatformMath::CeilToFloat(A);
				}
				else
				{
					return FGenericPlatformMath::CeilToDouble(A);
				}
			case UO_Cos: return FGenericPlatformMath::Cos(A);
			case UO_Cosh: return FGenericPlatformMath::Cosh(A);
			case UO_Exponential: return FGenericPlatformMath::Pow(UE_EULERS_NUMBER, A);
			case UO_Exponential2: return FGenericPlatformMath::Pow(2.0f, A);
			case UO_Floor:
				if constexpr (std::is_same<T, float>::value)
				{
					return FGenericPlatformMath::FloorToFloat(A);
				}
				else
				{
					return FGenericPlatformMath::FloorToDouble(A);
				}
			case UO_Frac:
			{
				return FGenericPlatformMath::Frac(A);
			}
			case UO_Logarithm:
			case UO_Logarithm2:
			case UO_Logarithm10:
			{
				const T Base = (Operator == UO_Logarithm) ? T(UE_EULERS_NUMBER)
					: Operator == UO_Logarithm2 ? T(2)
					: T(10);
				return FGenericPlatformMath::LogX(Base, A);
			}
			case UO_Reciprocal: return 1 / A;
			case UO_Round:
				if constexpr (std::is_same<T, float>::value)
				{
					return FGenericPlatformMath::RoundToFloat(A);
				}
				else
				{
					return FGenericPlatformMath::RoundToDouble(A);
				}
			case UO_Rsqrt: return FGenericPlatformMath::InvSqrt(A);
			case UO_Sin: return FGenericPlatformMath::Sin(A);
			case UO_Sinh: return FGenericPlatformMath::Sinh(A);
			case UO_Sqrt: return FGenericPlatformMath::Sqrt(A);
			case UO_Tan: return FGenericPlatformMath::Tan(A);
			case UO_Tanh: return FGenericPlatformMath::Tanh(A);
			case UO_Truncate:
				if constexpr (std::is_same<T, float>::value)
				{
					return FGenericPlatformMath::TruncToFloat(A);
				}
				else
				{
					return FGenericPlatformMath::TruncToDouble(A);
				}
			case BO_ATan2: return FGenericPlatformMath::Atan2(A, B);
			case BO_ATan2Fast: return FGenericPlatformMath::Atan2(A, B);
			case BO_Fmod: return FGenericPlatformMath::Fmod(A, B);
			// truncated division (A-B*(trunc(A/B))) where the result takes on the sign of operand 1 the dividend
			case BO_Modulo: return A - B * FGenericPlatformMath::TruncToFloat(A / B);
			case BO_Pow: return FGenericPlatformMath::Pow(A, B);
			case TO_Lerp: return FMath::Lerp<T>(A, B, C);
			case TO_Smoothstep: return FMath::SmoothStep<T>(A, B, C);
			default: break;
		}
	}

	if constexpr (std::is_integral_v<T>)
	{
		switch (Operator)
		{
			case UO_Not: return !A;
			case UO_BitwiseNot: return ~A;
			case BO_And: return A & B;
			case BO_Or: return A | B;
			case BO_BitwiseAnd: return A & B;
			case BO_BitwiseOr: return A | B;
			case BO_BitwiseXor: return A ^ B;
			case BO_BitShiftLeft: return A << B;
			case BO_BitShiftRight: return A >> B;
			case BO_Modulo: return A % B;
			default: break;
		}
	}

	switch (Operator)
	{
		case UO_Abs: return FGenericPlatformMath::Abs<T>(A);
		case UO_Negate: return -A;
		case UO_Saturate: return FMath::Clamp(A, T(0), T(1));
		case BO_Add: return A + B;
		case BO_Subtract: return A - B;
		case BO_Multiply: return A * B;
		case BO_MatrixMultiply: return A * B; // mul() is also supported for scalars
		case BO_Divide: return A / B;
		case BO_Min: return FMath::Min<T>(A, B);
		case BO_Max: return FMath::Max<T>(A, B);
		case BO_Step: return B >= A ? 1.0f : 0.0f;
		case TO_Clamp: return FMath::Clamp(A, B, C);
		default: UE_MIR_UNREACHABLE();
	}
}

// It tries to apply a known identity of specified operator, e.g. "x + 0 = x ? x ? R".
// If it returns a value, the operation has been "folded" and the returned value is the
// result (in the example above, it would return "x").
// If it returns null, the end result could not be inferred, but the operator could have
// still been changed to some other (with lower complexity). For example "clamp(x, 0, 1)"
// will change to "saturate(x)".
static FValueRef TrySimplifyOperator(FEmitter& Emitter, EOperator& Op, FValueRef& A, FValueRef& B, FValueRef& C, FType ResultType)
{
	switch (Op)
	{
		/* Unary Operators */
		case UO_Length:
			if (A->Type.GetPrimitive().IsScalar())
			{
				Op = UO_Abs;
			}
			break;

		/* Binary Comparisons */
		case BO_GreaterThan:
		case BO_LessThan:
		case BO_NotEquals:
			if (A->Equals(B))
			{
				return Emitter.ConstantFalse();
			}
			break;

		case BO_GreaterThanOrEquals:
		case BO_LessThanOrEquals:
		case BO_Equals:
			if (A->Equals(B))
			{
				return Emitter.ConstantTrue();
			}
			break;

		/* Binary Arithmetic */
		case BO_Add:
			if (A->AreAllNearlyZero())
			{
				return B;
			}
			else if (B->AreAllNearlyZero())
			{
				return A;
			}
			break;

		case BO_Subtract:
			if (B->AreAllNearlyZero())
			{
				return A;
			}
			else if (A->AreAllNearlyZero())
			{
				return Emitter.Negate(B);
			}
			else if (A == B)
			{
				return Emitter.ConstantZero(ResultType.GetPrimitive().ScalarKind);
			}
			break;

		case BO_Multiply:
			if (A->AreAllNearlyZero() || B->AreAllNearlyOne())
			{
				return A;
			}
			else if (A->AreAllNearlyOne() || B->AreAllNearlyZero())
			{
				return B;
			}
			break;
		
		case BO_MatrixMultiply:
			if (ResultType.IsScalar())
			{
				Op = BO_Dot;

				// The dot could be simplified further, from dot to multiply if A and B are scalars.
				return TrySimplifyOperator(Emitter, Op, A, B, C, ResultType);
			}
			break;

		case BO_Divide:
			if (A->AreAllNearlyZero() || B->AreAllNearlyOne())
			{
				return A;
			}
			else if (A == B)
			{
				return Emitter.ConstantOne(ResultType.GetPrimitive().ScalarKind);
			}
			break;

		case BO_Modulo:
			if (A->AreAllNearlyZero() || B->AreAllNearlyOne())
			{
				return Emitter.ConstantZero(ResultType.GetPrimitive().ScalarKind);
			}
			break;
			
		case BO_BitwiseAnd:
			if (A->AreAllExactlyZero())
			{
				return A;
			}
			else if (B->AreAllExactlyZero())
			{
				return B;
			}
			else if (A == B)
			{
				return A;
			}
			break;

		case BO_BitwiseOr:
			if (A->AreAllExactlyZero())
			{
				return B;
			}
			else if (B->AreAllExactlyZero())
			{
				return A;
			}
			else if (A == B)
			{
				return A;
			}
			break;

		case BO_BitwiseXor:
			if (A->AreAllExactlyZero())
			{
				return B;
			}
			else if (B->AreAllExactlyZero())
			{
				return A;
			}
			else if (A == B)
			{
				return Emitter.ConstantZero(ResultType.GetPrimitive().ScalarKind);
			}
			break;

		case BO_BitShiftLeft:
		case BO_BitShiftRight:
			if (A->AreAllExactlyZero() || B->AreAllExactlyZero())
			{
				return A;
			}
			break;
		
		case BO_Dot:
			if (A->Type.IsScalar())
			{
				check(B->Type.IsScalar());
				Op = BO_Multiply;
				return TrySimplifyOperator(Emitter, Op, A, B, C, ResultType);
			}
			if (A->AreAllNearlyZero() || B->AreAllNearlyZero())
			{
				return Emitter.ConstantZero(ResultType.GetPrimitive().ScalarKind);
			}
			break;

		case BO_Pow:
			if (A->AreAllNearlyZero())
			{
				// If the base is 0.
				return A;
			}
			else if (B->AreAllNearlyZero())
			{
				// If the exponent is 0.
				return Emitter.ConstantOne(ResultType.GetPrimitive().ScalarKind);
			}
			else if (B->AreAllExactlyOne())
			{
				// If the exponent is 1.
				return A;
			}
			break;

		case TO_Clamp:
			if (B->AreAllNearlyZero() && C->AreAllNearlyOne())
			{
				Op = UO_Saturate;
				B = {};
				C = {};
			}
			else if (B == C)
			{
				return B;
			}
			break;

		case TO_Lerp:
			if (C->AreAllNearlyZero())
			{
				return A;
			}
			else if (C->AreAllNearlyOne())
			{
				return B;
			}
			else if (A == B)
			{
				return A;
			}
			break;

		case TO_Select:
			if (A->AreAllTrue())
			{
				return B;
			}
			else if (A->AreAllFalse())
			{
				return C;
			}
			else if (B == C)
			{
				return B;
			}
			break;

		default:
			break;
	}

	return {};
}

// Whether the given operator is unary and returns a boolean.
static bool IsUnaryOperatorToBool(EOperator Op)
{
	switch (Op)
	{
		case UO_IsFinite:
		case UO_IsInf:
		case UO_IsNan:
			return true;
		default:
			return false;
	}
}

// Whether the given operator identifies a comparison operation (e.g., ">=", "==").
static bool IsComparisonOperator(EOperator Op)
{
	switch (Op)
	{
		case BO_Equals:
		case BO_GreaterThan:
		case BO_GreaterThanOrEquals:
		case BO_LessThan:
		case BO_LessThanOrEquals:
		case BO_NotEquals:
			return true;
		default:
			return false;
	}
}

// Tries to fold (statically evaluate) the operator, assuming that the arguments are all scalar.
// It returns either the result of the operator or null if it could not be folded.
static FValueRef TryFoldOperatorScalar(FEmitter& Emitter, EOperator Op, FValueRef A, FValueRef B, FValueRef C, FType ResultType)
{
	TOptional<FPrimitive> PrimitiveType = A->Type.AsPrimitive();

	// Try to simplify the operator. This could potentially change Op, A, B and C.
	if (FValueRef Simplified = TrySimplifyOperator(Emitter, Op, A, B, C, ResultType))
	{
		return Simplified;
	}

	// If TrySimplifyOperator did not already fold the `select` operator, there is nothing else to do.
	if (Op == TO_Select)
	{
		return {};
	}

	// Verify that both lhs and rhs are constants, otherwise we cannot fold the operation.
	const FConstant* AConstant = As<FConstant>(A.Value);
	const FConstant* BConstant = As<FConstant>(B.Value);
	const FConstant* CConstant = As<FConstant>(C.Value);
	if (!AConstant || (IsBinaryOperator(Op) && !BConstant) || (IsTernaryOperator(Op) && (!BConstant || !CConstant)))
	{
		return {};
	}

	// Call the appropriate helper function depending on what type of operator this is
	if (Op == UO_Not)
	{
		return Emitter.ConstantBool(!AConstant->Boolean);
	}
	else if (IsUnaryOperatorToBool(Op))
	{
		bool Result;
		switch (PrimitiveType->ScalarKind)
		{
			case EScalarKind::Float:
				Result = FoldFloatToBoolOperatorScalar<TFloat>(Op, AConstant->Float);
				break;

			case EScalarKind::Double:
				Result = FoldFloatToBoolOperatorScalar<TDouble>(Op, AConstant->Double);
				break;
			
			default:
				UE_MIR_UNREACHABLE();
		}
		return Emitter.ConstantBool(Result);
	}
	else if (IsComparisonOperator(Op))
	{
		bool Result;
		switch (PrimitiveType->ScalarKind)
		{
			case EScalarKind::Integer:
				Result = FoldComparisonOperatorScalar<TInteger>(Op, AConstant->Integer, BConstant->Integer);
				break;

			case EScalarKind::Float:
				Result = FoldComparisonOperatorScalar<TFloat>(Op, AConstant->Float, BConstant->Float);
				break;

			case EScalarKind::Double:
				Result = FoldComparisonOperatorScalar<TDouble>(Op, AConstant->Double, BConstant->Double);
				break;

			default:
				UE_MIR_UNREACHABLE();
		}
		return Emitter.ConstantBool(Result);
	}
	else
	{
		switch (PrimitiveType->ScalarKind)
		{
			case EScalarKind::Boolean:
			{
				bool Result = FoldScalarOperator<TInteger>(Emitter, Op, AConstant->Boolean, BConstant ? BConstant->Boolean : 0, 0) & 0x1;
				return Emitter.ConstantBool(Result);
			}

			case EScalarKind::Integer:
			{
				TInteger Result = FoldScalarOperator<TInteger>(Emitter, Op, AConstant->Integer, BConstant ? BConstant->Integer : 0, CConstant ? CConstant->Integer : 0);
				return Emitter.ConstantInt(Result);
			}

			case EScalarKind::Float:
			{
				TFloat Result = FoldScalarOperator<TFloat>(Emitter, Op, AConstant->Float, BConstant ? BConstant->Float : 0, CConstant ? CConstant->Float : 0);
				return Emitter.ConstantFloat(Result);
			}

			case EScalarKind::Double:
			{
				TDouble Result = FoldScalarOperator<TDouble>(Emitter, Op, AConstant->Double, BConstant ? BConstant->Double : 0, CConstant ? CConstant->Double : 0);
				return Emitter.ConstantDouble(Result);
			}

			default:
				UE_MIR_UNREACHABLE();
		}
	}
}

// Used to filter what parameter *primitive* types operators can take.
enum EOperatorParameterFlags
{
	OPF_Unknown = 0xff,                                         // Unspecified
	OPF_Any = 0,                                                // Any primitive type
	OPF_CheckIsBoolean = 1 << 1,                                // Check the type is boolean primitive of any dimension
	OPF_CheckIsInteger = 1 << 2,                                // Check the type is integer primitive of any dimension
	OPF_CheckIsArithmetic = 1 << 3,                             // Check the type is arithmetic primitive of any dimension (i.e. that supports arithmetic operations)
	OPF_CheckIsMatrix = 1 << 4,                                 // Check the type is any matrix type
	OPF_CheckIsNotMatrix = 1 << 5,                              // Check the type is any primitive type except matrices
	OPF_CheckIsVector3 = 1 << 6,                                // Check the type is a 3D vector of any scalar type
	OPF_CheckIsNonNegativeFloatConst = 1 << 7,                  // Check that if the argument is a constant float, it is not negative (x >= 0)
	OPF_CheckIsNonZeroFloatConst = 1 << 8,                      // Check that if the argument is a constant float, it is not zero    (x != 0)
	OPF_CheckIsOneOrGreaterFloatConst = 1 << 9,                 // Check that if the argument is a constant float, it is 1 or greater (xFloat >= 1)
	OPF_CheckIsBetweenMinusOneAndPlusOneFloatConst = 1 << 10,   // Check that if the argument is a constant float, it is between -1 and 1 (-1 < x < 1)
	OPF_CastToFirstArgumentType = 1 << 11,                      // Cast the argument to the first argument's type
	OPF_CastToAnyFloat = 1 << 12,                               // Cast the argument to the floating point primitive type of any dimension
	OPF_AllowDouble = 1 << 13,                                  // This argument is allowed to be a double
	OPF_CastToCommonScalarKind = 1 << 14,                       // Casts the argument to have the scalar kind in common with other arguments
	OPF_CastToCommonType = 1 << 15,                             // Cast the argument to the common arguments type
	OPF_CastToCommonArithmeticType = OPF_CheckIsArithmetic | OPF_CastToCommonType,
	OPF_CastToCommonArithmeticTypeAllowDouble = OPF_CheckIsArithmetic | OPF_CastToCommonType | OPF_AllowDouble,
	OPF_CastToCommonFloatType = OPF_CastToAnyFloat | OPF_CastToCommonType,
};

inline EOperatorParameterFlags operator|(EOperatorParameterFlags A, EOperatorParameterFlags B)
{
	return EOperatorParameterFlags(uint32(A) | uint32(B));
}

// Used to determine the operator result type based on argument types
enum EOperatorResult
{
	OR_Unknown,							    // Unspecified
	OR_FirstArgumentType,					// The same type as the first argument, LWC input produces float result
	OR_BooleanWithFirstArgumentDimensions,	// A boolean primitive type with the same dimensions (rows and columns) as the first argument type
	OR_FirstArgumentTypeToScalarLWC,		// A scalar primitive type with the same kind as the scalar type of the first argument, LWC results allowed
	OR_SecondArgumentType,					// The same type as the second argument
	OR_FirstArgumentTypeAllowDouble,		// The same type as the first argument, LWC results allowed
	OR_MatrixMultiplyResult,				// The result type of the matrix multiplication of first two arguments
};

enum EOperatorUniform
{
	OU_False = 0,							// Operator does not support uniform evaluation
	OU_True = 1,							// Operator does support uniform evaluation
};

// The signature of an operator consisting of its parameter and return type information.
struct FOperatorSignature
{
	EOperatorParameterFlags ParameterFlags[3] = { OPF_Unknown, OPF_Unknown, OPF_Unknown };
	EOperatorResult 		Result = OR_Unknown;
	EOperatorUniform		Uniform = OU_False;
};

// Returns the signature of an operator.
static const FOperatorSignature* GetOperatorSignature(EOperator Op)
{
	// For reference, these Preshader2 ops are not supported in MIR:
	//		Fractional, Normalize
	//		NOTE:  Fractional is slightly different than Frac, in that it returns signed fractions for negative numbers.
	// 
	// And these MIR ops are not supported in Preshader2:
	//		UO_Rsqrt, TO_Clamp == good candidates to add support for, used in existing preshaders.  Clamp would be min plus max ops.
	//		BO_Distance, BO_Pow, BO_Step
	//		Bitwise ops, hyperbolics, logical ops, number categorizers, comparisons, ternary ops
	//
	// Cast is supported by Preshader2, but not implemented yet.
	//

	static const FOperatorSignature* Signatures = [] ()
	{
		const FOperatorSignature UnaryFloat = { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat }, OR_FirstArgumentType };
		const FOperatorSignature UnaryFloatUniform = { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat }, OR_FirstArgumentType, OU_True };
		const FOperatorSignature UnaryFloatOrDouble = { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat | OPF_AllowDouble }, OR_FirstArgumentTypeAllowDouble, OU_True };
		const FOperatorSignature UnaryFloatLWCDemote = { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat | OPF_AllowDouble }, OR_FirstArgumentType, OU_True };
		const FOperatorSignature UnaryFloatToBoolean = { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat }, OR_BooleanWithFirstArgumentDimensions };
		const FOperatorSignature BinaryArithmetic = { { OPF_CastToCommonArithmeticType, OPF_CastToCommonArithmeticType }, OR_FirstArgumentType };
		const FOperatorSignature BinaryArithmeticAllowDouble = { { OPF_CastToCommonArithmeticTypeAllowDouble, OPF_CastToCommonArithmeticTypeAllowDouble }, OR_FirstArgumentTypeAllowDouble, OU_True };
		const FOperatorSignature BinaryArithmeticLWCDemote = { { OPF_CastToCommonArithmeticTypeAllowDouble, OPF_CastToCommonArithmeticTypeAllowDouble }, OR_FirstArgumentType };
		const FOperatorSignature BinaryInteger = { { OPF_CheckIsInteger | OPF_CastToCommonArithmeticType, OPF_CheckIsInteger | OPF_CastToCommonArithmeticType }, OR_FirstArgumentType };
		const FOperatorSignature BinaryFloat = { { OPF_CastToCommonFloatType, OPF_CastToCommonFloatType }, OR_FirstArgumentType };
		const FOperatorSignature BinaryFloatUniform = { { OPF_CastToCommonFloatType, OPF_CastToCommonFloatType }, OR_FirstArgumentType, OU_True };
		const FOperatorSignature BinaryArithmeticComparisonAllowDouble = { { OPF_CastToCommonArithmeticType | OPF_AllowDouble, OPF_CastToCommonArithmeticType | OPF_AllowDouble }, OR_BooleanWithFirstArgumentDimensions };
		const FOperatorSignature BinaryLogical = { { OPF_CheckIsBoolean | OPF_CastToCommonType, OPF_CheckIsBoolean | OPF_CastToCommonType }, OR_FirstArgumentType };
		const FOperatorSignature TernaryArithmeticDouble = { { OPF_CastToCommonArithmeticTypeAllowDouble, OPF_CastToCommonArithmeticTypeAllowDouble, OPF_CastToCommonArithmeticTypeAllowDouble }, OR_FirstArgumentTypeAllowDouble };
		const FOperatorSignature TernaryFloatDoubleDemote = { { OPF_CastToCommonArithmeticTypeAllowDouble | OPF_CastToAnyFloat, OPF_CastToCommonArithmeticTypeAllowDouble, OPF_CastToCommonArithmeticTypeAllowDouble }, OR_FirstArgumentType };

		static FOperatorSignature S[OperatorCount];

		/* unary operators */
		S[UO_BitwiseNot] 				= { { OPF_CheckIsInteger }, OR_FirstArgumentType };
		S[UO_Negate] 					= { { OPF_CheckIsArithmetic | OPF_AllowDouble }, OR_FirstArgumentTypeAllowDouble, OU_True };
		S[UO_Not] 						= { { OPF_CheckIsBoolean }, OR_FirstArgumentType };

		S[UO_Abs] 						= UnaryFloatOrDouble;
		S[UO_ACos] 						= UnaryFloatLWCDemote;
		S[UO_ACosFast]					= UnaryFloatUniform;
		S[UO_ACosh] 					= { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat | OPF_CheckIsOneOrGreaterFloatConst }, OR_FirstArgumentType };
		S[UO_ASin] 						= UnaryFloatLWCDemote;
		S[UO_ASinFast]					= UnaryFloatUniform;
		S[UO_ASinh] 					= UnaryFloat;
		S[UO_ATan] 						= UnaryFloatLWCDemote;
		S[UO_ATanFast]					= UnaryFloatUniform;
		S[UO_ATanh] 					= { { OPF_CheckIsArithmetic | OPF_CastToAnyFloat | OPF_CheckIsBetweenMinusOneAndPlusOneFloatConst }, OR_FirstArgumentType };
		S[UO_Ceil] 						= UnaryFloatOrDouble;
		S[UO_Cos] 						= UnaryFloatLWCDemote;
		S[UO_Exponential]				= UnaryFloatUniform;
		S[UO_Exponential2]				= UnaryFloatUniform;
		S[UO_Floor] 					= UnaryFloatOrDouble;
		S[UO_Frac]						= UnaryFloatLWCDemote;
		S[UO_IsFinite]					= UnaryFloatToBoolean;
		S[UO_IsInf]						= UnaryFloatToBoolean;
		S[UO_IsNan]						= UnaryFloatToBoolean;
		S[UO_Length]					= { { OPF_CheckIsArithmetic | OPF_CheckIsNotMatrix | OPF_CastToAnyFloat | OPF_AllowDouble }, OR_FirstArgumentTypeToScalarLWC, OU_True };
		S[UO_Logarithm] 				= { { OPF_CheckIsArithmetic | OPF_CheckIsNonZeroFloatConst | OPF_CheckIsNonNegativeFloatConst | OPF_CastToAnyFloat }, OR_FirstArgumentType, OU_True };
		S[UO_Logarithm10]				= { { OPF_CheckIsArithmetic | OPF_CheckIsNonZeroFloatConst | OPF_CheckIsNonNegativeFloatConst | OPF_CastToAnyFloat }, OR_FirstArgumentType, OU_True };
		S[UO_Logarithm2] 				= { { OPF_CheckIsArithmetic | OPF_CheckIsNonZeroFloatConst | OPF_CheckIsNonNegativeFloatConst | OPF_CastToAnyFloat }, OR_FirstArgumentType, OU_True };
		S[UO_LWCTile]					= { };		// UNUSED
		S[UO_Reciprocal]				= { { OPF_CheckIsArithmetic | OPF_CheckIsNonZeroFloatConst | OPF_CastToAnyFloat }, OR_FirstArgumentType, OU_True };
		S[UO_Round] 					= UnaryFloatOrDouble;
		S[UO_Rsqrt]						= { { OPF_CheckIsArithmetic | OPF_CheckIsNonZeroFloatConst | OPF_CastToAnyFloat }, OR_FirstArgumentType };
		S[UO_Saturate]					= UnaryFloatLWCDemote;
		S[UO_Sign]						= UnaryFloatLWCDemote;
		S[UO_Sin] 						= UnaryFloatLWCDemote;
		S[UO_Sqrt]						= { { OPF_CheckIsArithmetic | OPF_CheckIsNonNegativeFloatConst | OPF_CastToAnyFloat | OPF_AllowDouble }, OR_FirstArgumentType, OU_True };
		S[UO_Tan] 						= UnaryFloatLWCDemote;
		S[UO_Tanh] 						= UnaryFloat;
		S[UO_Truncate]					= UnaryFloatOrDouble;

		/* binary operators */
		S[BO_Equals]					= { { OPF_CastToCommonType | OPF_AllowDouble, OPF_CastToCommonType | OPF_AllowDouble }, OR_BooleanWithFirstArgumentDimensions };
		S[BO_GreaterThan]				= BinaryArithmeticComparisonAllowDouble;
		S[BO_GreaterThanOrEquals]		= BinaryArithmeticComparisonAllowDouble;
		S[BO_LessThan]					= BinaryArithmeticComparisonAllowDouble;
		S[BO_LessThanOrEquals]			= BinaryArithmeticComparisonAllowDouble;
		S[BO_NotEquals]					= { { OPF_CastToCommonType | OPF_AllowDouble, OPF_CastToCommonType | OPF_AllowDouble }, OR_BooleanWithFirstArgumentDimensions };
		
		S[BO_And] 						= BinaryLogical;
		S[BO_Or] 						= BinaryLogical;
		S[BO_Add] 						= BinaryArithmeticAllowDouble;
		S[BO_Subtract] 					= BinaryArithmeticAllowDouble;
		S[BO_Multiply] 					= BinaryArithmeticAllowDouble;
		S[BO_MatrixMultiply]			= { { OPF_CheckIsArithmetic | OPF_CastToCommonScalarKind, OPF_CheckIsArithmetic | OPF_CastToCommonScalarKind }, OR_MatrixMultiplyResult };
		S[BO_Divide] 					= BinaryArithmeticAllowDouble;
		S[BO_Modulo] 					= BinaryArithmetic;
		S[BO_BitwiseAnd]				= BinaryInteger;
		S[BO_BitwiseOr]					= BinaryInteger;
		S[BO_BitwiseXor]				= BinaryInteger;
		S[BO_BitShiftLeft]				= BinaryInteger;
		S[BO_BitShiftRight]				= BinaryInteger;


		S[BO_ATan2]						= BinaryFloatUniform;
		S[BO_ATan2Fast]					= BinaryFloatUniform;
		S[BO_Cross] 					= { { OPF_CheckIsArithmetic | OPF_CheckIsVector3, OPF_CastToFirstArgumentType }, OR_FirstArgumentType, OU_True };
		S[BO_Distance]					= { { OPF_CastToCommonFloatType | OPF_AllowDouble, OPF_CastToCommonFloatType | OPF_AllowDouble }, OR_FirstArgumentTypeToScalarLWC };
		S[BO_Dot] 						= { { OPF_CheckIsArithmetic | OPF_CheckIsNotMatrix | OPF_AllowDouble, OPF_CastToFirstArgumentType | OPF_AllowDouble }, OR_FirstArgumentTypeToScalarLWC, OU_True };
		S[BO_Fmod] 						= { { OPF_CastToCommonFloatType | OPF_AllowDouble, OPF_CastToCommonFloatType }, OR_FirstArgumentType, OU_True };		// First input can be LWC, but second and output are always demoted

		S[BO_Max] 						= BinaryArithmeticAllowDouble;
		S[BO_Min] 						= BinaryArithmeticAllowDouble;
		S[BO_Pow] 						= BinaryFloat;
		S[BO_Step] 						= BinaryArithmeticLWCDemote;

		/* ternary operators -- note lerp doesn't support LWC for the third argument! */
		S[TO_Clamp]						= TernaryArithmeticDouble;
		S[TO_Lerp] 						= { { OPF_CastToCommonArithmeticTypeAllowDouble | OPF_CastToAnyFloat, OPF_CastToCommonArithmeticTypeAllowDouble, OPF_CastToCommonArithmeticType }, OR_FirstArgumentTypeAllowDouble };;
		S[TO_Select]					= { { OPF_CheckIsBoolean | OPF_CheckIsNotMatrix, OPF_CheckIsNotMatrix | OPF_AllowDouble, OPF_CheckIsNotMatrix | OPF_AllowDouble}, OR_SecondArgumentType }; // Note: this is a special operator, which is handled manually in the Validate function
		S[TO_Smoothstep]				= TernaryFloatDoubleDemote;
		return S;
	} ();
	return &Signatures[Op];
}
 
// Validates that the types of the arguments are valid for specified operator.
// If valid, it returns the type of the result. Otherwise if it is not valid, it returns nullptr.
static FType ValidateOperatorAndGetResultType(FEmitter& Emitter, EOperator Op, FValueRef& A, FValueRef& B, FValueRef& C)
{
	// Argument A must have always been provided.
	check(A);

	// Assert that if C is specified, B must too.
	check(!C || B);

	// Verify that B argument has been provided if operator is binary.
	check(!IsBinaryOperator(Op) || B);

	// Verify that C argument has been provided if operator is ternary.
	check(!IsTernaryOperator(Op) || C);

	// Make sure the first argument has primitive type first, since the following operations assume this.
	if (!Emitter.CheckIsPrimitive(A).IsValid())
	{
		return FType::MakePoison();
	}

	// SPECIAL CASE: Given input float3, generates double3 type with the given tile value and zero offset. Caller must pass in float3.
	if (Op == UO_LWCTile)
	{
		if (!A->Type.GetPrimitive().IsFloat() || !A->Type.GetPrimitive().IsRowVector() || A->Type.GetPrimitive().NumComponents() != 3)
		{
			Emitter.Errorf(A, TEXT("Argument of LWCTile operator expected to be a 3D float vector."));
			return FType::MakePoison();
		}
		return FType::MakeDoubleVector(3);
	}

	// SPECIAL CASE: For Clamp, we do a special case and demote the first LWC argument if the second and third arguments (min / max) are non-LWC.
	// We want to do this before fetching FirstArgumentPrimitiveType.
	if (Op == TO_Clamp && A->Type.GetPrimitive().IsDouble() && !B->Type.IsDouble() && !C->Type.IsDouble())
	{
		A = Emitter.CastToFloatKind(A);
	}

	// Get the operator signature information.
	const FOperatorSignature* Signature = GetOperatorSignature(Op);

	// Handle automatic cast to float for operators that don't support LWC inputs
	FType FirstArgumentType = A->Type;
	if (FirstArgumentType.GetPrimitive().IsDouble() && !(Signature->ParameterFlags[0] & OPF_AllowDouble))
	{
		FirstArgumentType = A->Type.GetPrimitive().ToScalarKind(EScalarKind::Float);
	}

	// Verify that the first argument type is primitive.
	FValueRef Arguments[] = { A, B, C, nullptr };
	static const TCHAR* ArgumentsStr[] = { TEXT("first"), TEXT("second"), TEXT("third") };

	for (int32 i = 0; Arguments[i]; ++i)
	{
		// Check this argument type i primitive.
		Arguments[i] = Emitter.CheckIsPrimitive(Arguments[i]);
		if (!Arguments[i].IsValid())
		{
			return FType::MakePoison();
		}

		EOperatorParameterFlags Filter = Signature->ParameterFlags[i];
		check(Filter != OPF_Unknown); // No signature specified for this operator.

		if (Filter & OPF_CastToFirstArgumentType)
		{
			check(i > 0); // This check can't apply to the first argument.
			Arguments[i] = Emitter.Cast(Arguments[i], FirstArgumentType);
		}
		else if ( 
			// Cast argument to float when...
			// ...the argument should be cast to any float (and it's not a float already)
			((Filter & OPF_CastToAnyFloat) && !Arguments[i]->Type.GetPrimitive().IsAnyFloat())
			// ...or the argument is not allowed to be a double and it is.
			|| (!(Filter & OPF_AllowDouble) && Arguments[i]->Type.GetPrimitive().IsDouble()))
		{
			Arguments[i] = Emitter.CastToFloatKind(Arguments[i]);
			check(!Arguments[i].IsPoison());
		}
		
		if (Filter & OPF_CheckIsBoolean)
		{
			Arguments[i] = Emitter.CheckIsBoolean(Arguments[i]); 
		}
			
		if (Filter & OPF_CheckIsArithmetic)
		{
			Arguments[i] = Emitter.CheckIsArithmetic(Arguments[i]); 
		}

		if (Filter & OPF_CheckIsInteger)
		{
			Arguments[i] = Emitter.CheckIsInteger(Arguments[i]); 
		}
		
		if (Filter & OPF_CheckIsMatrix)
		{
			Arguments[i] = Emitter.CheckIsMatrix(Arguments[i]); 
		}

		if (Filter & OPF_CheckIsNotMatrix)
		{
			Arguments[i] = Emitter.CheckIsScalarOrVector(Arguments[i]); 
		}

		if (Filter & OPF_CheckIsVector3)
		{
			if (!Arguments[i]->Type.GetPrimitive().IsRowVector() || Arguments[i]->Type.GetPrimitive().NumComponents() != 3)
			{
				Emitter.Errorf(Arguments[i], TEXT("Expected a 3D vector."));
				Arguments[i] = Arguments[i].ToPoison();
			}
		}

		// The following checks are only applicable if the argument is constant.
		if (FConstant* Constant = Arguments[i]->As<FConstant>())
		{
			if (Filter & OPF_CheckIsNonZeroFloatConst)
			{
				check((Filter & OPF_CastToAnyFloat) || (Filter & OPF_CastToCommonFloatType));
				if (Constant->Float == 0)
				{
					Emitter.Errorf(Arguments[i], TEXT("Expected non-zero value."));
					Arguments[i] = Arguments[i].ToPoison();
				}
			}

			if (Filter & OPF_CheckIsNonNegativeFloatConst)
			{
				check((Filter & OPF_CastToAnyFloat) || (Filter & OPF_CastToCommonFloatType));
				if (Constant->Float < 0)
				{
					Emitter.Errorf(Arguments[i], TEXT("Expected non-negative value."));
					Arguments[i] = Arguments[i].ToPoison();
				}
			}

			if (Filter & OPF_CheckIsOneOrGreaterFloatConst)
			{
				check((Filter & OPF_CastToAnyFloat) || (Filter & OPF_CastToCommonFloatType));
				if (Constant->Float < 1)
				{
					Emitter.Errorf(Arguments[i], TEXT("Expected a value equal or greater than 1."));
					Arguments[i] = Arguments[i].ToPoison();
				}
			}

			if (Filter & OPF_CheckIsBetweenMinusOneAndPlusOneFloatConst)
			{
				check((Filter & OPF_CastToAnyFloat) || (Filter & OPF_CastToCommonFloatType));
				if (Constant->Float < -1 || Constant->Float > 1)
				{
					Emitter.Errorf(Arguments[i], TEXT("Expected a value greater than -1 and lower than 1."));
					Arguments[i] = Arguments[i].ToPoison();
				}
			}
		}
	}

	if (Arguments[0].IsPoison() || Arguments[1].IsPoison() || Arguments[2].IsPoison())
	{
		return FType::MakePoison();
	}
	
	// Whether any argument will require to be cast to common type/scalar kind.
	bool bRequiresArgumentsCommonType = (Signature->ParameterFlags[0] & OPF_CastToCommonType) || (Signature->ParameterFlags[1] & OPF_CastToCommonType) || (Signature->ParameterFlags[2] & OPF_CastToCommonType);
	bool bRequiresArgumentsCommonScalarKind = (Signature->ParameterFlags[0] & OPF_CastToCommonScalarKind) || (Signature->ParameterFlags[1] & OPF_CastToCommonScalarKind) || (Signature->ParameterFlags[2] & OPF_CastToCommonScalarKind);

	// SPECIAL CASE:  The select operator is special insofar as its first argument is a boolean, while the second and third can be any primitive type.
	if (Op == TO_Select)
	{
		// Cast the second and third argument types to primitive. This is safe as it was already checked earlier.
		const FPrimitive& BPrimitiveType = Arguments[1]->Type.GetPrimitive();
		const FPrimitive& CPrimitiveType = Arguments[2]->Type.GetPrimitive();

		// Compute the maximum number of vector components between all arguments. We know they're scalar or vectors, as it was checked before.
		int32 MaxNumComponents = FMath::Max3(FirstArgumentType.GetPrimitive().NumComponents(), BPrimitiveType.NumComponents(), CPrimitiveType.NumComponents());

		// Cast the first argument (the boolean condition) to a bool vector of the maximum number of components.
		Arguments[0] = Emitter.Cast(Arguments[0], FType::MakeVector(EScalarKind::Boolean, MaxNumComponents));

		// Compute the common type between the second and third argument types with a number of components equal to the max of all three.
		FType CommonTypeBetweenSecondAndThirdArguments = Emitter.TryGetCommonType(
			FType::MakeVector(BPrimitiveType.ScalarKind, MaxNumComponents),
			FType::MakeVector(CPrimitiveType.ScalarKind, MaxNumComponents));
		
		// Getting the common type should always be possible.
		check(!CommonTypeBetweenSecondAndThirdArguments.IsPoison());

		// Cast second and third arguments to their common type.
		Arguments[1] = Emitter.Cast(Arguments[1], CommonTypeBetweenSecondAndThirdArguments);
		Arguments[2] = Emitter.Cast(Arguments[2], CommonTypeBetweenSecondAndThirdArguments);
	}
	else if (bRequiresArgumentsCommonType || bRequiresArgumentsCommonScalarKind)
	{
		// Determine the common type and scalar kind (if needeed)
		// Note: these two cannot be unified, because we can always determine the common (biggest) scalar kind between two
		// primitive types, but not always can be determined a common type (e.g. a float3 with a float4x4)
		FType ArgumentsCommonType{};
		if (bRequiresArgumentsCommonType)
		{
			ArgumentsCommonType = Emitter.GetCommonType({ Arguments[0], Arguments[1], Arguments[2] });
			if (ArgumentsCommonType.IsPoison())
			{
				return FType::MakePoison();
			}
		}

		EScalarKind ArgumentsCommonScalarKind = EScalarKind::Boolean;
		if (bRequiresArgumentsCommonScalarKind)
		{
			for (int32 i = 0; Arguments[i]; ++i)
			{
				ArgumentsCommonScalarKind = (EScalarKind)FMath::Max((int)ArgumentsCommonScalarKind, (int)Arguments[i]->Type.GetPrimitive().ScalarKind);
			}
		}

		// Cast every argument with the `CastToCommon` to the common type, if necessary.
		for (int32 i = 0; Arguments[i]; ++i)
		{
			EOperatorParameterFlags Filter = Signature->ParameterFlags[i];
			if (Filter & OPF_CastToCommonScalarKind)
			{
				Arguments[i] = Emitter.CastToScalarKind(Arguments[i], ArgumentsCommonScalarKind);
			}
			else if (Filter & OPF_CastToCommonType)
			{
				check(bRequiresArgumentsCommonType);

				// Lerp doesn't accept double for its third input, so we need to check if double is allowed per input when casting to the common type.
				FType ToType = (!(Filter & OPF_AllowDouble) && ArgumentsCommonType.IsDouble()) 
					? ArgumentsCommonType.GetPrimitive().ToScalarKind(EScalarKind::Float)
					: ArgumentsCommonType;

				Arguments[i] = Emitter.Cast(Arguments[i], ToType);
			}
		}
	}
	
	if (Arguments[0].IsPoison() || Arguments[1].IsPoison() || Arguments[2].IsPoison())
	{
		return FType::MakePoison();
	}

	// Arguments might have changed, update the references.
	A = Arguments[0];
	B = Arguments[1];
	C = Arguments[2];

	// Finally, determine operator result type.
	const FPrimitive& FirstArgumentPrimitiveType = Arguments[0]->Type.GetPrimitive();
	switch (Signature->Result)
	{
		case OR_Unknown:
			UE_MIR_UNREACHABLE(); // missing operator signature declaration

		case OR_FirstArgumentType:
			// Some operators accept LWC as input, but always output non-LWC (examples: modulo, sign, sine, cosine, saturate, frac). Anything that may output LWC should use OR_FirstArgumentTypeLWC.
			return FirstArgumentPrimitiveType.IsDouble() ? FirstArgumentPrimitiveType.ToScalarKind(EScalarKind::Float) : Arguments[0]->Type;

		case OR_FirstArgumentTypeAllowDouble:
			return Arguments[0]->Type;

		case OR_BooleanWithFirstArgumentDimensions:
			return FType::MakePrimitive(EScalarKind::Boolean, FirstArgumentPrimitiveType.NumRows, FirstArgumentPrimitiveType.NumColumns);

 		case OR_FirstArgumentTypeToScalarLWC:
			return FirstArgumentPrimitiveType.ToScalar();

		case OR_SecondArgumentType:
			return B->Type;

		case OR_MatrixMultiplyResult:
		{
			FType SecondArgumentType = Arguments[1]->Type;

			FPrimitive LhsPrimitiveType = FirstArgumentType.GetPrimitive();
			FPrimitive RhsPrimitiveType = SecondArgumentType.GetPrimitive();

			int32 RhsRows = RhsPrimitiveType.NumRows;
			int32 RhsColumns = RhsPrimitiveType.NumColumns;
			int32 OutputRows = LhsPrimitiveType.NumRows;
			int32 OutputColumns = RhsPrimitiveType.NumColumns;

			// When multiplying matrix * vector, we reinterpret the input as a column vector (NumColumns == 1), even though by default our vectors are row vectors.
			// And the output is reinterpreted back as a row vector.
			if (SecondArgumentType.IsVector())
			{
				RhsRows = RhsPrimitiveType.NumColumns;
				RhsColumns = 1;
				OutputRows = 1;
				OutputColumns = LhsPrimitiveType.NumRows;
			}

			if (FirstArgumentType.GetPrimitive().NumColumns != RhsRows)
			{
				Emitter.Errorf({}, TEXT("Cannot matrix multiply a '%s' with a '%s'."), *FirstArgumentType.GetSpelling(), *SecondArgumentType.GetSpelling());
				return FType::MakePoison();
			}

			return FType::MakePrimitive(LhsPrimitiveType.ScalarKind, OutputRows, OutputColumns);
		}
	}

	UE_MIR_UNREACHABLE();
}

// Conditionally mark the given operator as uniform if it's supported in preshaders.
static void MarkUniformIfSupportedPreshaderOperator(FOperator& Proto)
{
	// Only allow operators on float types now -- need to implement casting logic for other types to work, but it's low priority,
	// given that 99.9% of preshaders use purely float math.  There is currently no user facing integer parameter type, and math
	// on doubles is rarely uniform, because interesting double typed values (such as camera, world, or object position) are all
	// non-uniform.
	if (GAllowPreshaderOperators && GetOperatorSignature(Proto.Op)->Uniform && Proto.AArg->Type.IsFloat() && (!Proto.BArg || Proto.BArg->Type.IsFloat()))
	{
		Proto.SetSubgraphProperties(EGraphProperties::Uniform);
	}
}

// Returns whether the operator supports componentwise application. In other words, if the following is true:
// 	op(v, w) == [op(v_0, w_0), ..., op(v_n, w_n)]
static bool IsComponentwiseOperator(EOperator Op)
{
	return Op != BO_Dot && Op != BO_Cross && Op != BO_MatrixMultiply;
}

// Tries to fold the operator by applying the operator componentwise on arguments components.
// If a value is returned, it will be a composite with some component folded to a constant. If some argument
// isn't a composite, or all arguments components are non-constant, the folding will not be carried out.
// If no folding is carried out, this function simply returns nullptr.
static FValue* TryFoldComponentwiseOperator(FEmitter& Emitter, EOperator Op, FValue* A, FValue* B, FValue* C, FType ResultType)
{
	// Check that at least one component of the resulting composite value would folded.
	// If all components of resulting composite value are not folded, then instead of emitting
	// an individual operator instruction for each component, simply emit a single binary operator
	// instruction applied between lhs and rhs as a whole. (v1 + v2 rather than float2(v1.x + v2.x, v1.y + v2.y)
	bool bSomeResultComponentWasFolded = false;
	bool bResultIsIdenticalToA = true;
	bool bResultIsIdenticalToB = true;
	bool bResultIsIdenticalToC = true;

	// Allocate the temporary array to store the folded component results
	TTemporaryArray<FValue*> TempResultComponents{ ResultType.GetPrimitive().NumComponents() };
	
	for (int32 i = 0; i < TempResultComponents.Num(); ++i)
	{
		// Extract the arguments individual components
		FValue* AComponent = Emitter.Subscript(A, i);
		FValue* BComponent = B ? Emitter.Subscript(B, i).Value : nullptr;
		FValue* CComponent = C ? Emitter.Subscript(C, i).Value : nullptr;

		// Try folding the operation, it may return null
		FValue* ResultComponent = TryFoldOperatorScalar(Emitter, Op, AComponent, BComponent, CComponent, ResultType);

		// Update the flags
		bSomeResultComponentWasFolded |= (bool)ResultComponent;
		bResultIsIdenticalToA &= ResultComponent && ResultComponent->Equals(AComponent);
		bResultIsIdenticalToB &= BComponent && ResultComponent && ResultComponent->Equals(BComponent);
		bResultIsIdenticalToC &= CComponent && ResultComponent && ResultComponent->Equals(CComponent);

		// Cache the results
		TempResultComponents[i] = ResultComponent;
	}

	// If result is identical to either lhs or rhs, simply return it
	if (bResultIsIdenticalToA)
	{
		return A;
	}
	else if (bResultIsIdenticalToB)
	{
		return B;
	}
	else if (bResultIsIdenticalToC)
	{
		return C;
	}

	// If some component was folded (it is either constant or the operation was a NOP), it is worth
	// build the operation as a separate operation for each component, that is like
	//    float2(a.x + b.x, a.y + b.y)
	// rather than
	//    a + b
	// so that we retain as much compile-time information as possible.
	if (bSomeResultComponentWasFolded)
	{
		// If result type is scalar, simply return the single folded result (instead of creating a composite value)
		if (ResultType.GetPrimitive().IsScalar())
		{
			check(TempResultComponents[0]);
			return TempResultComponents[0];
		}

		// Make the new composite value
		FComposite* Result = MakeCompositePrototype(Emitter, ResultType, TempResultComponents.Num());

		// Fetch the components array from the result composite
		TArrayView<FValue*> ResultComponents = Result->GetMutableComponents();

		// Also cache the type of a single component
		FType ComponentType = ResultType.GetPrimitive().ToScalar();

		// Create the operator instruction for each component pair
		for (int32 i = 0; i < TempResultComponents.Num(); ++i)
		{
			// Reuse cached result if possible
			ResultComponents[i] = TempResultComponents[i];

			// Otherwise emit the binary operation between the two components (this will create a new instruction)
			if (!ResultComponents[i])
			{
				FOperator Proto = MakePrototype<FOperator>(ComponentType);
				Proto.Op = Op;
				Proto.AArg = Emitter.Subscript(A, i);
				Proto.BArg = B ? Emitter.Subscript(B, i).Value : nullptr;
				Proto.CArg = C ? Emitter.Subscript(C, i).Value : nullptr;

				MarkUniformIfSupportedPreshaderOperator(Proto);

				ResultComponents[i] = EmitPrototype(Emitter, Proto);
			}
		}
		
		return EmitPrototype(Emitter, *Result);
	}

	return {};
}

// If V is a composite and all its components are constants, it unpacks the components into OutComponents and returns true.
// If this is not possible for any reason, it returns false.
static bool TryUnpackConstantScalarOrVector(FValue* V, TArrayView<FConstant*> OutComponents, int32& OutNumComponents)
{
	// V not specified? Or not a scalar/vector?
	FComposite* Composite = As<FComposite>(V);
	if (!Composite || V->Type.AsPrimitive()->IsMatrix())
	{
		return false;
	}

	TConstArrayView<FValue*> Components = Composite->GetComponents();
	for (int32 i = 0; i < Components.Num(); ++i)
	{
		OutComponents[i] = As<FConstant>(Components[i]);
		if (!OutComponents[i])
		{
			return false;
		}
	}

	OutNumComponents = Components.Num();
	return true;
}

// Computes the dot product on two arrays of constant float components.
static TFloat ConstantDotFloat(TArrayView<FConstant*> AComponents, TArrayView<FConstant*> BComponents, int32 NumComponents)
{
	float Result = 0.0f;
	for (int32 i = 0; i < NumComponents; ++i)
	{
		Result += AComponents[i]->Float * BComponents[i]->Float;
	}
	return Result;
}

static MIR::TDouble ConstantDotDouble(TArrayView<FConstant*> AComponents, TArrayView<FConstant*> BComponents, int32 NumComponents)
{
	double Result = 0.0;
	for (int32 i = 0; i < NumComponents; ++i)
	{
		Result += AComponents[i]->Double * BComponents[i]->Double;
	}
	return Result;
}

// Tries to fold the operator, that is to evaluate its result now at translation time if its arguments are constant.
// If the operator could not be folded in any way, it returns nullptr.
static FValueRef TryFoldOperator(FEmitter& Emitter, EOperator Op, FValueRef A, FValueRef B, FValueRef C, FType ResultType)
{
	// First, try to apply some operator identity to simplify the operator.
	if (FValueRef Simplified = TrySimplifyOperator(Emitter, Op, A, B, C, ResultType))
	{
		return Simplified;
	}

	FConstant* AComponents[4];
	int32      ANumComponents;
	
	// CASE 1: Some operations like Length, Dot and Cross are not defined on individual scalar components.
	// For instance length(V) is not the same as [length(V.x), ..., length(V.z)]. These operations
	// folding is handled here as special cases.
	// First, try to unpack the first argument to an array of constants.
	if (TryUnpackConstantScalarOrVector(A, AComponents, ANumComponents))
	{
		FConstant* BComponents[4];
		int32      BNumComponents;
	
		if (Op == UO_Length)
		{
			if (ResultType.GetPrimitive().IsFloat())
			{
				float Result = FMath::Sqrt(ConstantDotFloat(AComponents, AComponents, ANumComponents));
				return Emitter.ConstantFloat(Result);
			}
			else if (ResultType.GetPrimitive().IsDouble())
			{
				double Result = FMath::Sqrt(ConstantDotDouble(AComponents, AComponents, ANumComponents));
				return Emitter.ConstantDouble(Result);
			}
			else
			{
				UE_MIR_UNREACHABLE();
			}
		}
		else if ((Op == BO_Dot || Op == BO_Cross) && TryUnpackConstantScalarOrVector(B, BComponents, BNumComponents))
		{
			// Verified before the operation is folded, here as a safety check.
			check(ANumComponents == BNumComponents);

			if (Op == BO_Dot)
			{
				if (ResultType.GetPrimitive().IsFloat())
				{
					float Result = ConstantDotFloat(AComponents, BComponents, ANumComponents);
					return Emitter.ConstantFloat(Result);
				}
				else if (ResultType.GetPrimitive().IsDouble())
				{
					double Result = ConstantDotDouble(AComponents, BComponents, ANumComponents);
					return Emitter.ConstantDouble(Result);
				}
				else
				{
					UE_MIR_UNREACHABLE();
				}
			}
			else
			{
				check(Op == BO_Cross);
				if (ResultType.GetPrimitive().IsFloat())
				{
					FVector3f AVector{ AComponents[0]->Float, AComponents[1]->Float, AComponents[2]->Float };
					FVector3f BVector{ BComponents[0]->Float, BComponents[1]->Float, BComponents[2]->Float };
					FVector3f Result = AVector.Cross(BVector);
					return Emitter.ConstantFloat3(Result);
				}
				else if (ResultType.GetPrimitive().IsDouble())
				{
					UE_MIR_TODO();
				}
				else
				{
					UE_MIR_UNREACHABLE();
				}
			}
		}
	}

	// CASE 2: If the operation supports componentwise application, try folding the operator componentwise.
	if (IsComponentwiseOperator(Op))
	{
		return TryFoldComponentwiseOperator(Emitter, Op, A, B, C, ResultType);
	}

	// No folding was possible, simply return null to indicate this.
	return {};
}

FValueRef FEmitter::Operator(EOperator Op, FValueRef A, FValueRef B, FValueRef C)
{
	// Transpose is a translation-time operation only that never creates a runtime Operator instruction.
	if (Op == UO_Transpose)
	{
		return Transpose(A);
	}

	if (!A.IsValid() || (B && !B.IsValid()) || (C && !C.IsValid()))
	{
		return Poison();
	}

	// Validate the operation and retrieve the result type.
	FType ResultType = ValidateOperatorAndGetResultType(*this, Op, A, B, C);
	if (ResultType.IsPoison())
	{
		return Poison();
	}

	FValueRef Result;

	// Try folding the operator first.
	if (FValueRef FoldedValue = TryFoldOperator(*this, Op, A, B, C, ResultType))
	{
		Result = FoldedValue;
	}
	else
	{
		// Otherwise, we must emit a new instruction that executes the operator.
		FOperator Proto = MakePrototype<FOperator>(ResultType);
		Proto.Op = Op;
		Proto.AArg = A;
		Proto.BArg = B;
		Proto.CArg = C;

		MarkUniformIfSupportedPreshaderOperator(Proto);

		Result = EmitPrototype(*this, Proto);
	}

	// Subtract has a special case option to automatically truncate when subtracting two double-precision inputs from each other, assuming this is a
	// transition from double-precision space to relative space, and no longer needs to be double-precision.
	// We need to check that all arguments are double-precision before the call to ValidateOperatorAndGetResultType, as that may cast the inputs, changing them.
	if (Op == BO_Subtract && UE::MaterialTranslatorUtils::GetLWCTruncateMode() == 2 && A->Type.IsDouble() && B->Type.IsDouble())
	{
		Result = CastToFloatKind(Result);
	}

	return Result;
} 

FValueRef FEmitter::Branch(FValueRef Condition, FValueRef True, FValueRef False)
{
	if (IsAnyNotValid(Condition, True, False))
	{
		return Poison();
	}

	// Condition must be of type bool
	Condition = Cast(Condition, FType::MakeBoolScalar());
	if (!Condition)
	{
		return Poison();
	}

	// If the condition is a scalar constant, then simply evaluate the result now.
	if (const FConstant* ConstCondition = As<FConstant>(Condition))
	{
		return ConstCondition->Boolean ? True : False;
	}

	// If the condition is not static, make both true and false arguments have the same type,
	// by casting false argument into the true's type.
	FType CommonType = GetCommonType({ True, False });
	if (CommonType.IsPoison())
	{
		return Poison();
	}

	True = Cast(True, CommonType);
	False = Cast(False, CommonType);
	if (!True || !False)
	{
		return Poison();
	}

	// Constants don't require deferred evaluation inside scoped blocks. If so instead emit a lighter "select" instead.
	if (CommonType.IsPrimitive() && True->IsConstant() && False->IsConstant())
	{
		return Select(Condition, True, False);
	}
	
	// Create the branch instruction. For aggregate types, if both branches share some identical
	// attribute values (pointer-equal due to deduplication), we can extract those directly and only
	// branch on the attributes that actually differ. When the branch result is an aggregate, since branch
	// instruction emission is deferred, it is skipped entirely when all aggregate attributes are identical.
	FBranch Proto = MakePrototype<FBranch>(CommonType);
	Proto.ConditionArg = Condition;
	Proto.TrueArg = True;
	Proto.FalseArg = False;

	// When both branches are composite aggregates, build an aggregate result where identical
	// attributes are extracted directly (bypassing the branch) and differing attributes are
	// subscripted from the branch result.
	if (const UMaterialAggregate* Aggregate = CommonType.AsAggregate())
	{
		const MIR::FComposite* TrueAsComposite = True->As<MIR::FComposite>();
		const MIR::FComposite* FalseAsComposite = False->As<MIR::FComposite>();
		if (TrueAsComposite && FalseAsComposite)
		{
			const int32 NumAttributes = Aggregate->Attributes.Num();
			TConstArrayView<FValue*> TrueComponents = TrueAsComposite->GetComponents();
			TConstArrayView<FValue*> FalseComponents = FalseAsComposite->GetComponents();

			// First pass: check if any attribute can be collapsed.
			bool bHasCollapsed = false;
			for (int32 i = 0; i < NumAttributes; ++i)
			{
				if (TrueComponents[i] == FalseComponents[i])
				{
					bHasCollapsed = true;
					break;
				}
			}

			if (bHasCollapsed)
			{
				// Create stripped composites for TrueArg/FalseArg: null out identical
				// attributes so the translator skips them, emitting only what differs.
				FComposite* StrippedTrue = MakeCompositePrototype(*this, CommonType, NumAttributes);
				FComposite* StrippedFalse = MakeCompositePrototype(*this, CommonType, NumAttributes);
				TArrayView<FValue*> StrippedTrueComponents = StrippedTrue->GetMutableComponents();
				TArrayView<FValue*> StrippedFalseComponents = StrippedFalse->GetMutableComponents();

				TTemporaryArray<FValueRef> AttributeValues(NumAttributes);

				for (int32 i = 0; i < NumAttributes; ++i)
				{
					if (TrueComponents[i] == FalseComponents[i])
					{
						// Both branches produce the same value — use it directly.
						AttributeValues[i] = TrueComponents[i];
					}
					else
					{
						StrippedTrueComponents[i] = TrueComponents[i];
						StrippedFalseComponents[i] = FalseComponents[i];
					}
				}

				// Replace the branch args with the stripped composites.
				Proto.TrueArg = EmitPrototype(*this, *StrippedTrue);
				Proto.FalseArg = EmitPrototype(*this, *StrippedFalse);

				// Emit the branch, then build the result aggregate.
				FValue* BranchResult = EmitPrototype(*this, Proto);
				for (int32 i = 0; i < NumAttributes; ++i)
				{
					if (TrueComponents[i] != FalseComponents[i])
					{
						AttributeValues[i] = Subscript(BranchResult, i);
					}
				}

				return this->Aggregate(Aggregate, FValueRef(), AttributeValues);
			}
		}
	}

	return EmitPrototype(*this, Proto);
}

FValueRef FEmitter::Subscript(FValueRef Value, int32 Index)
{
	check(Index >= 0);

	if (!Value.IsValid())
	{
		return Value;
	}

	// Subscripting a composite by index is always possible and simply yields the i-th component.
	if (FComposite* Composite = As<FComposite>(Value))
	{
		check(Index < Composite->GetComponents().Num());
		return Value.To(Composite->GetComponents()[Index]);
	}

	// The final value of the subscript instruction should it be emitted 
	FType SubscriptType;

	if (const UMaterialAggregate* Aggregate = Value->Type.AsAggregate())
	{
		// Make sure the attribute index is valid
		if (!Aggregate->Attributes.IsValidIndex(Index))
		{
			Errorf(Value, TEXT("Attribute index %d is out of range for material aggregate '%s'."), Index, *Aggregate->GetName());
			return Value.ToPoison();
		}

		// Get the type of the material 
		SubscriptType = GetMaterialAggregateAttributeType(Aggregate, Index);
	}
	else if (TOptional<FPrimitive> PrimitiveType = Value->Type.AsPrimitive())
	{
		SubscriptType = PrimitiveType->ToScalar();

		// Getting first component and Value is already a scalar, just return itself.
		if (Index == 0 && PrimitiveType->IsScalar())
		{
			return Value;
		}

		if (Index >= PrimitiveType->NumComponents())
		{
			Errorf(Value, TEXT("Value of type '%s' has fewer dimensions than subscript index `%d`."), *Value->Type.GetSpelling(), Index);
			return Value.ToPoison();
		}

		if (PrimitiveType->IsMatrix() && PrimitiveType->IsDouble())
		{
			Errorf(Value, TEXT("Cannot subscript a double-precision matrix."));
			return Value.ToPoison();
		}

		// Avoid double subscripting a primitive value (e.g. no value.xy.x)
		if (FSubscript* Subscript = As<FSubscript>(Value); Subscript && Subscript->Arg->Type.AsPrimitive())
		{
			Value = Value.To(Subscript->Arg);
		}
	}
	else
	{
		Errorf(Value, TEXT("Value of type '%s' cannot be subscripted."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}

	// We can't resolve it at compile time: emit subscript value.
	FSubscript Prototype = MakePrototype<FSubscript>(SubscriptType);
	Prototype.Arg = Value;
	Prototype.Index = Index;

	// Don't allow preshader code to be generated for subscripts of double types.  These must be accessed from the original
	// full source vector, and subscripted on the GPU, since the data format is Structure of Arrays.  High and low components
	// of values are not interleaved, and thus there's no way to encode an offset to a specific component without the full
	// context of the size of vector it came from, which adds major complexity.  We currently don't support Preshader math
	// on doubles anyway, so this works fine, but if we do want to support it, we will need to change the new translator to
	// use AoS, so subsets or components of a vector can be trivially addressed.
	if (!Value->Type.IsDouble())
	{
		Prototype.SetSubgraphProperties(EGraphProperties::Uniform);
	}

	return Value.To(EmitPrototype(*this, Prototype));
}

FValueRef FEmitter::Swizzle(FValueRef Value, FSwizzleMask Mask)
{
	if (!Value.IsValid())
	{
		return Value;
	}

	// At least one component must have been specified.
	if (Mask.NumComponents <= 0)
	{
		Errorf(Value, TEXT("Swizzle mask has no components."));
		return Value.ToPoison();
	}

	// We can only swizzle on non-matrix primitive types.
	if (!Value->Type.AsPrimitive() || Value->Type.GetPrimitive().IsMatrix())
	{
		Errorf(Value, TEXT("Cannot swizzle a '%s' value."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}
	
	// For brevity
	FPrimitive PrimitiveType = Value->Type.GetPrimitive();
	int NumComponents = PrimitiveType.NumComponents();

	// Make sure each component in the mask fits the number of components in Value.
	for (EVectorComponent Component : Mask)
	{
		if ((int32)Component >= NumComponents)
		{
			Errorf(Value, TEXT("Value of type '%s' has no component '%s'."), *Value->Type.GetSpelling(), VectorComponentToString(Component));
			return Value.ToPoison();
		}
	}

	// If the requested number of components is the same as Value and the order in which the components
	// are specified in the mask is sequential (e.g. x, y, z) then this is a no op, simply return Value as is.
	if (Mask.NumComponents == NumComponents)
	{
		bool InOrder = true;
		for (int32 i = 0; i < Mask.NumComponents; ++i)
		{
			if (Mask.Components[i] != (EVectorComponent)i)
			{
				InOrder = false;
				break;
			}
		}
		if (InOrder)
		{
			return Value;
		}
	}
	
	// If only one component is requested, we can use Subscript() to return the single component.
	if (Mask.NumComponents == 1)
	{
		return Value.To(Subscript(Value, (int32)Mask.Components[0]));
	}

	// Make the result vector type.
	FType ResultType = FType::MakeVector(PrimitiveType.ScalarKind, Mask.NumComponents);
	FComposite* Result = MakeCompositePrototype(*this, ResultType, Mask.NumComponents);
	for (int32 i = 0; i < Mask.NumComponents; ++i)
	{
		Result->GetMutableComponents()[i] = Subscript(Value, (int32)Mask.Components[i]);
	}

	return Value.To(EmitPrototype(*this, *Result));
}

FValueRef FEmitter::Transpose(FValueRef Value)
{
	if (!Value.IsValid())
	{
		return Value;
	}

	TOptional<FPrimitive> PrimitiveType = Value->Type.AsPrimitive();
	if (!PrimitiveType)
	{
		Errorf(Value, TEXT("Cannot transpose a non primitive value of type '%s'."), *Value->Type.GetSpelling());
		return Value.ToPoison();
	}

	// A transposed scalar is itself.
	if (PrimitiveType->IsScalar())
	{
		return Value;
	}

	// Build the result type (swap rows and columns)
	const int32 OrigRows = PrimitiveType->NumRows;
    const int32 OrigColumns = PrimitiveType->NumColumns;
    FType ResultType = FType::MakePrimitive(PrimitiveType->ScalarKind, OrigColumns, OrigRows);

    // Make a composite prototype with space for all components
    FComposite* Prototype = MakeCompositePrototype(*this, ResultType, OrigRows * OrigColumns);

	// Transpose the components
	TArrayView<FValue*> Components = Prototype->GetMutableComponents();
	for (int32 i = 0, NumComponents = OrigRows * OrigColumns; i < NumComponents; ++i) {
		int32 OrigRow = i % OrigRows;
		int32 OrigColumn = i / OrigRows;
		Components[i] = Subscript(Value, OrigRow * OrigColumns + OrigColumn);
	}

	return EmitPrototype(*this, *Prototype);
}

static FValue* CastConstant(FEmitter& Emitter, FConstant* Constant, EScalarKind ConstantScalarKind, EScalarKind TargetKind)
{
	if (ConstantScalarKind == TargetKind)
	{
		return Constant;
	}

	switch (ConstantScalarKind)
	{
		case EScalarKind::Boolean:
		{
			switch (TargetKind)
			{
				case EScalarKind::Integer: return Emitter.ConstantInt(Constant->Boolean ? 1 : 0);
				case EScalarKind::Float: return Emitter.ConstantFloat(Constant->Boolean ? 1.0f : 0.0f);
				case EScalarKind::Double: return Emitter.ConstantDouble(Constant->Boolean ? 1.0 : 0.0);
				default: UE_MIR_UNREACHABLE();
			}
		}

		case EScalarKind::Integer:
		{
			switch (TargetKind)
			{
				case EScalarKind::Boolean: return Emitter.ConstantBool(Constant->Integer != 0);
				case EScalarKind::Float: return Emitter.ConstantFloat((TFloat)Constant->Integer);
				case EScalarKind::Double: return Emitter.ConstantDouble(Constant->Integer);
				default: UE_MIR_UNREACHABLE();
			}
		}

		case EScalarKind::Float:
		{
			switch (TargetKind)
			{
				case EScalarKind::Boolean: return Emitter.ConstantBool(Constant->Float != 0.0f);
				case EScalarKind::Integer: return Emitter.ConstantInt((int32)Constant->Float);
				case EScalarKind::Double: return Emitter.ConstantDouble(Constant->Float);
				default: UE_MIR_UNREACHABLE();
			}
		}

		case EScalarKind::Double:
		{
			switch (TargetKind)
			{
				case EScalarKind::Boolean: return Emitter.ConstantBool(Constant->Double != 0.0);
				case EScalarKind::Integer: return Emitter.ConstantInt((int32)Constant->Double);
				case EScalarKind::Float: return Emitter.ConstantFloat((float)Constant->Double);
				default: UE_MIR_UNREACHABLE();
			}
		}

		default: break;
	}

	UE_MIR_UNREACHABLE();
}

static FValueRef CastToPrimitive(FEmitter& Emitter, FValueRef Value, FType TargetType)
{
	if (!Value->Type.AsPrimitive())
	{
		Emitter.Errorf(Value, TEXT("Cannot construct a '%s' from non primitive type '%s'."), *Value->Type.GetSpelling(), *TargetType.GetSpelling());
		return Value.ToPoison();
	}

	FPrimitive ValuePrimitiveType = Value->Type.GetPrimitive();
	FPrimitive TargetPrimitiveType = TargetType.GetPrimitive();

	// Construct a scalar from another scalar.
	if (TargetPrimitiveType.IsScalar())
	{
		// Get the first component of value. We already know value's type is primitive, so this will return a scalar.
		Value = Emitter.Subscript(Value, 0);

		ValuePrimitiveType = Value->Type.GetPrimitive();
		check(ValuePrimitiveType.IsScalar());
		
		if (ValuePrimitiveType == TargetPrimitiveType)
		{
			// If types are identical, return the component value as is.
			return Value;
		}
		else if (FConstant* ConstantInitializer = As<FConstant>(Value))
		{
			// If value is a constant, we can cast the constant now.
			return CastConstant(Emitter, ConstantInitializer, ValuePrimitiveType.ScalarKind, TargetPrimitiveType.ScalarKind);
		}
		else
		{
			// Otherwise emit a cast instruction of the subscript value to the target type.
			FScalar Prototype = MakePrototype<FScalar>(TargetType);
			Prototype.Arg = Value;
			// Prototype.SetSubgraphProperties(EGraphProperties::Uniform);		// TODO:  Add preshader cast support
			return EmitPrototype(Emitter, Prototype);
		}
	}

	// Construct a vector or matrix from a scalar. E.g. 3.14f -> float3(3.14f, 3.14f, 3.14f)
	// Note: we know target isn't scalar as it's been handled above.
	if (ValuePrimitiveType.IsScalar())
	{
		// Create the result composite value.
		FComposite* Result = MakeCompositePrototype(Emitter, TargetType, TargetPrimitiveType.NumComponents());

		// Create a composite and initialize each of its components to the conversion
		// of initializer value to the single component type.
		FValue* Component = Emitter.Cast(Value, TargetPrimitiveType.ToScalar());

		// Get the mutable array of components.
		TArrayView<FValue*> ResultComponents = Result->GetMutableComponents();

		// Initialize all result components to the same scalar.
		for (int32 i = 0; i < TargetPrimitiveType.NumComponents(); ++i)
		{
			ResultComponents[i] = Component;
		}
		
		return EmitPrototype(Emitter, *Result);
	}

	// Construct a vector from another vector. If constructed vector is larger, initialize
	// remaining components to zero. If it's smaller, truncate initializer vector and only use
	// the necessary components.
	if (TargetPrimitiveType.IsRowVector() && ValuePrimitiveType.IsRowVector())
	{
		// #todo-massimo.tristano Use swizzle when scalartypes are the same, and target num components is less than initializer's.

		int32 TargetNumComponents = TargetPrimitiveType.NumComponents();
		int32 InitializerNumComponents = ValuePrimitiveType.NumComponents();

		// Create the result composite value.
		FComposite* Result = MakeCompositePrototype(Emitter, TargetType, TargetPrimitiveType.NumComponents());
		TArrayView<FValue*> ResultComponents = Result->GetMutableComponents();

		// Determine the result component type (scalar).
		FType ResultComponentType = TargetPrimitiveType.ToScalar();

		// For iterating over the components of the result composite value.
		int32 Index = 0;
		
		// Convert components from the initializer vector.
		const int32 MinNumComponents = FMath::Min(TargetNumComponents, InitializerNumComponents);
		for (; Index < MinNumComponents; ++Index)
		{
			ResultComponents[Index] = Emitter.Cast(Emitter.Subscript(Value, Index), ResultComponentType);
		}

		// Initialize remaining result composite components to zero.
		for (; Index < TargetNumComponents; ++Index)
		{
			ResultComponents[Index] = Emitter.ConstantZero(ResultComponentType.GetPrimitive().ScalarKind);
		}

		return EmitPrototype(Emitter, *Result);
	}
	
	// The two primitive types are identical matrices that differ only by their scalar type.
	if (TargetPrimitiveType.NumRows == ValuePrimitiveType.NumRows &&
		TargetPrimitiveType.NumColumns == ValuePrimitiveType.NumColumns)
	{
		check(TargetPrimitiveType.IsMatrix());

		// Create the result composite value.
		FComposite* Result = MakeCompositePrototype(Emitter, TargetType, TargetPrimitiveType.NumComponents());
		TArrayView<FValue*> ResultComponents = Result->GetMutableComponents();
			
		// Determine the result component type (scalar).
		FType ResultComponentType = TargetPrimitiveType.ToScalar();

		// Convert components from the initializer vector.
		for (int32 Index = 0, Num = ResultComponents.Num(); Index < Num; ++Index)
		{
			ResultComponents[Index] = Emitter.Cast(Emitter.Subscript(Value, Index), ResultComponentType);
		}

		return EmitPrototype(Emitter, *Result);
	}

	// Initializer value cannot be used to construct this primitive type.
	return Value.ToPoison();
}

FValueRef FEmitter::Cast(FValueRef Value, FType TargetType)
{
	if (!Value.IsValid())
	{
		return Value;
	}

	// If target type matches initializer's, simply return the same value.
	FType InitializerType = Value->Type;
	if (InitializerType == TargetType)
	{
		return Value;
	}
	
	FValueRef Result = FPoison::Get();
	if (TargetType.AsPrimitive())
	{
		Result = CastToPrimitive(*this, Value, TargetType);
	}

	if (Result->IsPoison())
	{
		// No other legal conversions applicable. Report error if we haven't converted the value.
		Errorf(Value, TEXT("Cannot construct a '%s' from a '%s'."), *TargetType.GetSpelling(), *Value->Type.GetSpelling());
		return Poison();
	}
	
	return Result;
}

FValueRef FEmitter::CastToScalar(FValueRef Value)
{
	Value = CheckIsPrimitive(Value);
	return Value.IsValid()
		? Cast(Value, FType::MakeScalar(Value->Type.GetPrimitive().ScalarKind))
		: Value;
}

FValueRef FEmitter::CastToVector(FValueRef Value, int32 NumColumns)
{
	Value = CheckIsPrimitive(Value);
	return Value.IsValid()
		? Cast(Value, FType::MakeVector(Value->Type.GetPrimitive().ScalarKind, NumColumns))
		: Value;
}

FValueRef FEmitter::CastToScalarKind(FValueRef Value, EScalarKind ToScalarKind)
{
	Value = CheckIsPrimitive(Value);
	return Value.IsValid()
		? Cast(Value, Value->Type.GetPrimitive().ToScalarKind(ToScalarKind))
		: Value;
}

FValueRef FEmitter::CastToBoolKind(FValueRef Value)
{
	return CastToScalarKind(Value, EScalarKind::Boolean);
}

FValueRef FEmitter::CastToIntKind(FValueRef Value)
{
	return CastToScalarKind(Value, EScalarKind::Integer);
}

FValueRef FEmitter::CastToFloatKind(FValueRef Value)
{
	return CastToScalarKind(Value, EScalarKind::Float);
}

FValueRef FEmitter::CastToBool(FValueRef Value, int NumColumns)
{
	return Cast(Value, FType::MakeVector(EScalarKind::Boolean, NumColumns));
}

FValueRef FEmitter::CastToInt(FValueRef Value, int NumColumns)
{
	return Cast(Value, FType::MakeVector(EScalarKind::Integer, NumColumns));
}

FValueRef FEmitter::CastToFloat(FValueRef Value, int NumColumns)
{
	return Cast(Value, FType::MakeVector(EScalarKind::Float, NumColumns));
}

FValueRef FEmitter::StageSwitch(FType Type, TConstArrayView<FValueRef> ValuePerStage)
{
	check(ValuePerStage.Num() <= NumStages);
	FStageSwitch Prototype = MakePrototype<FStageSwitch>(Type);
	for (int i = 0; i < ValuePerStage.Num(); ++i)
	{
		Prototype.Args[i] = ValuePerStage[i];

		// If no value was specified for this stage, create an analysis error so that if used, it will report an error accordingly.
		if (!Prototype.Args[i])
		{
			Prototype.Args[i] = AnalysisError(Type, FString::Printf(TEXT("This operation is not supported in shader stage '%s'."), Module->GetEntryPoint(i).Name.GetData()));
		}
	}
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::Nop(FValueRef Arg)
{
	// Nop can only have primitive arguments
	Arg = CheckIsPrimitive(Arg);

	if (!Arg.IsValid())
	{
		return Arg;
	}

	MIR::FNop Prototype = MakePrototype<MIR::FNop>(Arg->Type);
	Prototype.Arg = Arg;

	return EmitPrototype(*this, Prototype);
}

static EMaterialSamplerType MapSamplerTypeForTexture(FValueRef TextureValue, EMaterialSamplerType InSamplerType, EShaderPlatform ShaderPlatform)
{
	// Can't sample with virtual texturing if input texture is not a virtual texture
	if (InSamplerType == SAMPLERTYPE_MAX)
	{
		if (MIR::FTextureUniform* TextureUniform = MIR::As<MIR::FTextureUniform>(TextureValue))
		{
			InSamplerType = TextureUniform->SamplerType;
		}
		else if (MIR::As<MIR::FVirtualTextureUniform>(TextureValue))
		{
			InSamplerType = SAMPLERTYPE_VirtualMasks;
		}
	}

	const bool bIsVirtualTexture = TextureValue->Type.IsRuntimeVirtualTexture() || TextureValue->Type.IsVirtualTexture();
	return (bIsVirtualTexture && UseVirtualTexturing(ShaderPlatform)) ? PromoteSamplerTypeFromSTtoVT(InSamplerType) : DemoteSamplerTypeFromVTtoST(InSamplerType);
}

static FValueRef VTPageTableLoadFromSamplerSource(FEmitter& Em, FValueRef Texture, const FTextureSampleAttributes& BaseAttributes, FValueRef TexCoord,
	FValueRef TexCoordDdx = {}, FValueRef TexCoordDdy = {}, ETextureMipValueMode MipValueMode = TMVM_None, FValueRef MipValue = {})
{
	// Cast input texture to UTexture. If it's a URuntimeVirtualTexture, we accept the cast to be null when passed to GetTextureAddressForSamplerSource().
	TextureAddress StaticAddressX = TA_Wrap;
	TextureAddress StaticAddressY = TA_Wrap;
	TextureAddress StaticAddressZ = TA_Wrap;
	UE::MaterialTranslatorUtils::GetTextureAddressForSamplerSource(Cast<UTexture>(Texture->GetTextureObject()), BaseAttributes.SamplerSourceMode, StaticAddressX, StaticAddressY, StaticAddressZ);

	// Emits the virtual texture page load instruction. VT stack and layer indices are initializes during IR analysis
	FVTPageTableRead Prototype = MakePrototype<FVTPageTableRead>(FType::MakeVTPageTableResult());
	Prototype.VirtualTextureUniform = Texture;
	Prototype.TexCoord = TexCoord;
	Prototype.TexCoordDdx = TexCoordDdx;
	Prototype.TexCoordDdy = TexCoordDdy;
	Prototype.MipValue = MipValue;
	Prototype.AddressU = StaticAddressX;
	Prototype.AddressV = StaticAddressY;
	Prototype.MipValueMode = MipValueMode;
	Prototype.bEnableFeedback = BaseAttributes.bEnableFeedback;
	Prototype.bIsAdaptive = BaseAttributes.bIsAdaptive;

	float AspectRatio = 1.0f;
	if (UTexture* TextureObj = Cast<UTexture>(Texture->GetTextureObject()))
	{
		const int32 SizeX = TextureObj->Source.GetSizeX();
		const int32 SizeY = TextureObj->Source.GetSizeY();
		if (SizeX > 0 && SizeY > 0)
		{
			AspectRatio = (float)SizeX / (float)SizeY;
		}
	}
	Prototype.AspectRatio = AspectRatio;

	return EmitPrototype(Em, Prototype);
}

// Cast the given texture coordinate to the type necessary for sampling given texture.
static void CastTexCoord(FEmitter& Em, FValueRef Texture, FValueRef& TexCoord)
{
	if (IsAnyNotValid(Texture, TexCoord))
	{
		return;
	}

	MIR::ETypeKind Kind = Texture->Type.GetKind();
	if (Kind == MIR::ETypeKind::TextureCube || Kind == MIR::ETypeKind::Texture2DArray || Kind == MIR::ETypeKind::TextureCubeArray || Kind == MIR::ETypeKind::Texture3D)
	{
		TexCoord = Em.CastToFloat(TexCoord, 3);
	}
	else
	{
		TexCoord = Em.CastToFloat(TexCoord, 2);
	}
}

FValueRef FEmitter::TextureGather(FValueRef Texture, FValueRef TexCoord, ETextureReadMode GatherMode, const FTextureSampleAttributes& BaseAttributes)
{
	check(GatherMode >= ETextureReadMode::GatherRed && GatherMode <= ETextureReadMode::GatherAlpha);

	// Cast the texture coordinate to the type necessary for sampling given texture.
	CastTexCoord(*this, Texture, TexCoord);
	
	if (IsAnyNotValid(Texture, TexCoord))
	{
		return Poison();
	}

	const EMaterialSamplerType SamplerType = MapSamplerTypeForTexture(Texture, BaseAttributes.SamplerType, GetShaderPlatform());

	FTextureRead Prototype = MakePrototype<FTextureRead>(FType::MakeFloatVector(4));
	Prototype.TextureUniform = Texture;
	Prototype.TexCoord = TexCoord;
	Prototype.Mode = GatherMode;
	Prototype.SamplerSourceMode = BaseAttributes.SamplerSourceMode;
	Prototype.SamplerType = SamplerType;

	if (IsVirtualSamplerType(SamplerType))
	{
		Prototype.VTPageTable = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, Prototype.TexCoord);
	}

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::TextureSample(FValueRef Texture, FValueRef TexCoord, bool bAutomaticViewMipBias, const FTextureSampleAttributes& BaseAttributes)
{
	// Cast the texture coordinate to the type necessary for sampling given texture.
	CastTexCoord(*this, Texture, TexCoord);
	
	if (IsAnyNotValid(Texture, TexCoord))
	{
		return Poison();
	}

	const EMaterialSamplerType SamplerType = MapSamplerTypeForTexture(Texture, BaseAttributes.SamplerType, GetShaderPlatform());

	FTextureRead PrototypePixel = MakePrototype<FTextureRead>(FType::MakeFloatVector(4));
	PrototypePixel.TextureUniform = Texture;
	PrototypePixel.TexCoord = TexCoord;
	PrototypePixel.Mode = ETextureReadMode::MipAuto;
	PrototypePixel.SamplerSourceMode = BaseAttributes.SamplerSourceMode;
	PrototypePixel.SamplerType = SamplerType;

	FTextureRead PrototypeCompute = PrototypePixel;
	PrototypeCompute.Mode = ETextureReadMode::Derivatives;
	PrototypeCompute.TexCoordDdx = AnalyticalPartialDerivative(TexCoord, EDerivativeAxis::X);
	PrototypeCompute.TexCoordDdy = AnalyticalPartialDerivative(TexCoord, EDerivativeAxis::Y);

	FTextureRead PrototypeVertex = PrototypePixel;
	PrototypeVertex.Mode = ETextureReadMode::MipLevel;
	PrototypeVertex.MipValue = ConstantZero(EScalarKind::Float);

	if (bAutomaticViewMipBias)
	{
		FValueRef ViewMaterialTextureMipBias = Builtin(EBuiltin::TextureMipBias);
		PrototypePixel.Mode = ETextureReadMode::MipBias;
		PrototypePixel.MipValue = ViewMaterialTextureMipBias;

		FValueRef Exp2ViewMaterialTextureMipBias = Operator(UO_Exponential2, ViewMaterialTextureMipBias);
		PrototypeCompute.TexCoordDdx = Operator(BO_Multiply, PrototypeCompute.TexCoordDdx, Exp2ViewMaterialTextureMipBias);
		PrototypeCompute.TexCoordDdy = Operator(BO_Multiply, PrototypeCompute.TexCoordDdy, Exp2ViewMaterialTextureMipBias);
	}

	if (IsVirtualSamplerType(SamplerType))
	{
		PrototypePixel.VTPageTable   = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, PrototypePixel.TexCoord);
		PrototypeCompute.VTPageTable = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, PrototypeCompute.TexCoord, PrototypeCompute.TexCoordDdx, PrototypeCompute.TexCoordDdy);
		PrototypeVertex.VTPageTable  = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, PrototypeVertex.TexCoord);
	}

	FStageSwitch StageSwitch = MakePrototype<FStageSwitch>(PrototypePixel.Type);
	StageSwitch.Args[Stage_Vertex] = EmitPrototype(*this, PrototypeVertex);
	StageSwitch.Args[Stage_Pixel] = EmitPrototype(*this, PrototypePixel);
	StageSwitch.Args[Stage_Compute] = EmitPrototype(*this, PrototypeCompute);

	return EmitPrototype(*this, StageSwitch);
}

FValueRef FEmitter::TextureSampleLevel(FValueRef Texture, FValueRef TexCoord, FValueRef MipLevel, bool bAutomaticViewMipBias, const FTextureSampleAttributes& BaseAttributes)
{
	// Cast the texture coordinate to the type necessary for sampling given texture.
	CastTexCoord(*this, Texture, TexCoord);
	
	if (IsAnyNotValid(Texture, TexCoord))
	{
		return Poison();
	}

	const EMaterialSamplerType SamplerType = MapSamplerTypeForTexture(Texture, BaseAttributes.SamplerType, GetShaderPlatform());

	FTextureRead Prototype = MakePrototype<FTextureRead>(FType::MakeFloatVector(4));
	Prototype.TextureUniform = Texture;
	Prototype.TexCoord = TexCoord;
	Prototype.MipValue = MipLevel;
	Prototype.Mode = ETextureReadMode::MipLevel;
	Prototype.SamplerSourceMode = BaseAttributes.SamplerSourceMode;
	Prototype.SamplerType = SamplerType;

	if (bAutomaticViewMipBias)
	{
		Prototype.MipValue = Operator(BO_Add, MipLevel, Builtin(EBuiltin::TextureMipBias));
	}

	if (IsVirtualSamplerType(SamplerType))
	{
		Prototype.VTPageTable = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, Prototype.TexCoord, {}, {}, TMVM_MipLevel, Prototype.MipValue);
	}

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::TextureSampleBias(FValueRef Texture, FValueRef TexCoord, FValueRef MipBias, bool bAutomaticViewMipBias, const FTextureSampleAttributes& BaseAttributes)
{
	// Cast the texture coordinate to the type necessary for sampling given texture.
	CastTexCoord(*this, Texture, TexCoord);
	
	if (IsAnyNotValid(Texture, TexCoord))
	{
		return Poison();
	}

	if (bAutomaticViewMipBias)
	{
		MipBias = Operator(BO_Add, MipBias, Builtin(EBuiltin::TextureMipBias));
	}

	const EMaterialSamplerType SamplerType = MapSamplerTypeForTexture(Texture, BaseAttributes.SamplerType, GetShaderPlatform());

	FTextureRead PrototypePixel = MakePrototype<FTextureRead>(FType::MakeFloatVector(4));
	PrototypePixel.TextureUniform = Texture;
	PrototypePixel.TexCoord = TexCoord;
	PrototypePixel.MipValue = MipBias;
	PrototypePixel.Mode = ETextureReadMode::MipBias;
	PrototypePixel.SamplerSourceMode = BaseAttributes.SamplerSourceMode;
	PrototypePixel.SamplerType = SamplerType;

	FTextureRead PrototypeCompute = PrototypePixel;
	PrototypeCompute.Mode = ETextureReadMode::Derivatives;

	FValueRef Exp2MipBias = Operator(UO_Exponential2, MipBias);
	PrototypeCompute.TexCoordDdx = Operator(BO_Multiply, AnalyticalPartialDerivative(TexCoord, EDerivativeAxis::X), Exp2MipBias);
	PrototypeCompute.TexCoordDdy = Operator(BO_Multiply, AnalyticalPartialDerivative(TexCoord, EDerivativeAxis::Y), Exp2MipBias);

	if (IsVirtualSamplerType(SamplerType))
	{
		PrototypePixel.VTPageTable   = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, PrototypePixel.TexCoord, {}, {}, TMVM_MipBias, PrototypePixel.MipValue);
		PrototypeCompute.VTPageTable = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, PrototypeCompute.TexCoord, PrototypeCompute.TexCoordDdx, PrototypeCompute.TexCoordDdy, TMVM_MipBias, PrototypeCompute.MipValue);
	}

	FStageSwitch StageSwitch = MakePrototype<FStageSwitch>(PrototypePixel.Type);
	StageSwitch.Args[Stage_Vertex] = AnalysisError(PrototypePixel.Type, TEXTVIEW("TextureSampleMipBias is only supported in the pixel shader"));
	StageSwitch.Args[Stage_Pixel] = EmitPrototype(*this, PrototypePixel);
	StageSwitch.Args[Stage_Compute] = EmitPrototype(*this, PrototypeCompute);

	return EmitPrototype(*this, StageSwitch);
}

FValueRef FEmitter::TextureSampleGrad(FValueRef Texture, FValueRef TexCoord, FValueRef TexCoordDdx, FValueRef TexCoordDdy, bool bAutomaticViewMipBias, const FTextureSampleAttributes& BaseAttributes)
{
	// Cast the texture coordinate to the type necessary for sampling given texture.
	CastTexCoord(*this, Texture, TexCoord);
	
	if (IsAnyNotValid(Texture, TexCoord))
	{
		return Poison();
	}

	const EMaterialSamplerType SamplerType = MapSamplerTypeForTexture(Texture, BaseAttributes.SamplerType, GetShaderPlatform());

	FTextureRead Prototype = MakePrototype<FTextureRead>(FType::MakeFloatVector(4));
	Prototype.TextureUniform = Texture;
	Prototype.TexCoord = TexCoord;
	Prototype.TexCoordDdx = TexCoordDdx;
	Prototype.TexCoordDdy = TexCoordDdy;
	Prototype.Mode = ETextureReadMode::Derivatives;
	Prototype.SamplerSourceMode = BaseAttributes.SamplerSourceMode;
	Prototype.SamplerType = SamplerType;

	if (bAutomaticViewMipBias)
	{
		FValueRef ViewMaterialTextureDerivativeMultiply = Builtin(EBuiltin::TextureDerivativeMultiply);
		Prototype.TexCoordDdx = Operator(BO_Multiply, Prototype.TexCoordDdx, ViewMaterialTextureDerivativeMultiply);
		Prototype.TexCoordDdy = Operator(BO_Multiply, Prototype.TexCoordDdy, ViewMaterialTextureDerivativeMultiply);
	}

	if (IsVirtualSamplerType(SamplerType))
	{
		Prototype.VTPageTable = VTPageTableLoadFromSamplerSource(*this, Texture, BaseAttributes, Prototype.TexCoord, Prototype.TexCoordDdx, Prototype.TexCoordDdy, TMVM_Derivative);
	}

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::PartialDerivative(FValueRef Value, EDerivativeAxis Axis)
{
	// Any operation on poison arguments is a poison.
	if (!Value.IsValid())
	{
		return Value;
	}

	// Differentiation is only valid on primitive types.
	if (!Value->Type.IsAnyFloat())
	{
		Errorf(Value, TEXT("Trying to differentiate a value of type `%s` is invalid. Expected a float type."), *Value->Type.GetSpelling());
		return Poison();
	}

	// Make the hardware derivative instruction.
	FPartialDerivative HwDerivativeProto = MakePrototype<FPartialDerivative>(Value->Type);
	HwDerivativeProto.Arg = Value;
	HwDerivativeProto.Axis = Axis;
	HwDerivativeProto.bHardwareDerivative = true;
	FValueRef HwDerivative = EmitPrototype(*this, HwDerivativeProto);

	// Compute the analytical derivative for stages that don't support hardware derivatives.
	FValueRef AnalyticalDerivative = AnalyticalPartialDerivative(Value, Axis);

	// Emit the stage switch instruction so that analytical derivatives are used on stages that support it and hardware
	// derivatives in the other stages.  Note that hardware derivatives throw errors on the vertex stage downstream, but
	// we don't know if the expression is reached in that stage until the ValueAnalyzer runs.
	FValueRef StageValues[NumStages] = {};
	for (int i = 0; i < NumStages; ++i)
	{
		StageValues[i] = (i == Stage_Compute) ? AnalyticalDerivative : HwDerivative;
	}
	return StageSwitch(Value->Type, StageValues);
}

static FValue* DifferentiateOperator(FEmitter& E, FOperator* Op, EDerivativeAxis Axis)
{
	EScalarKind OpScalarKind = Op->Type.GetPrimitive().ScalarKind;

	// Considering an operator acting on f(x), g(x) and h(x) arguments (e.g. "f(x) + g(x)"),
	// calculate base terms and their partial derivatives.
	FValue* F = Op->AArg;
	FValue* G = Op->BArg;
	FValue* H = Op->CArg;
	FValue* dF = F && !F->Type.IsBoolean() ? E.AnalyticalPartialDerivative(F, Axis).Value : nullptr; // Note: select's first argument is a boolean, avoid making the derivative then
	FValue* dG = E.AnalyticalPartialDerivative(G, Axis);
	FValue* dH = E.AnalyticalPartialDerivative(H, Axis);

	// Convenience local functions as multiplications and division operations are common in derivatives.
	auto Zero = [&E, OpScalarKind] () { return E.ConstantZero(OpScalarKind); };
	auto One = [&E, OpScalarKind] () { return E.ConstantOne(OpScalarKind); };
	auto Constant = [&E, OpScalarKind] (TDouble Scalar) { return E.ConstantScalar(OpScalarKind, Scalar); };

	// Some constants
	constexpr TDouble Ln2 = 0.69314718055994530941723212145818;
	constexpr TDouble Ln10 = 2.3025850929940456840179914546844;
	
	switch (Op->Op)
	{
		// d/dx -f(x) = -f'(x)
		case UO_Negate:
			return E.Negate(dF);

		// d/dx |f(x)| = f(x) f'(x) / |f(x)|
		case UO_Abs:
			return E.Divide(E.Multiply(F, dF), Op);

		// d/dx arccos(f(x)) = -1 / sqrt(1 - f(x)^2) * f'(x)
		case UO_ACos:
		case UO_ACosFast:
			return E.Negate(E.Divide(dF, E.Sqrt(E.Subtract(One(), E.Multiply(F, F)))));
			
		// d/dx acosh(f(x)) = 1 / sqrt(f(x)^2 - 1) * f'(x)
		case UO_ACosh:
			return E.Divide(dF, E.Sqrt(E.Subtract(E.Multiply(F, F), One())));
			
		// d/dx arcsin(f(x)) = 1 / sqrt(1 - f(x)^2) * f'(x)
		case UO_ASin:
		case UO_ASinFast:
			return E.Divide(dF, E.Sqrt(E.Subtract(One(), E.Multiply(F, F))));
		
		// d/dx asinh(f(x)) = 1 / sqrt(f(x)^2 + 1) * f'(x)
		case UO_ASinh:
			return E.Divide(dF, E.Sqrt(E.Add(E.Multiply(F, F), One())));

		// d/dx arctan(f(x)) = 1 / (1 + f(x)^2) * f'(x)
		case UO_ATan:
		case UO_ATanFast:
			return E.Divide(dF, E.Add(One(), E.Multiply(F, F)));

    	// d/dx atanh(f(x)) = f'(x) / (1 - f(x)^2)
		case UO_ATanh:
    		return E.Divide(dF, E.Subtract(One(), E.Multiply(F, F)));

		// d/dx cos(f(x)) = -sin(f(x)) * f'(x)
		case UO_Cos:
			return E.Negate(E.Multiply(E.Sin(F), dF));

		// d/dx cosh(f(x)) = sinh(f(x)) * f'(x)
		case UO_Cosh:
			return E.Multiply(E.Sinh(F), dF);

		// d/dx e^f(x) = e^f(x) * f'(x)
		case UO_Exponential:
			return E.Multiply(Op, dF);

		// d/dx 2^f(x) = ln(2) * 2^f(x) * f'(x)
		case UO_Exponential2:
			return E.Multiply(E.Multiply(Constant(Ln2), Op), dF);

		// d/dx frac(f(x)) = f'(x), since frac(x) = x - floor(x)
		case UO_Frac:
			return dF;

		// d/dx |f(x)| (length in vector case) = f(x) f'(x) / |f(x)|
		case UO_Length:
			return E.Divide(E.Multiply(F, dF), Op);

		// d/dx log(f(x)) = 1 / f(x) * f'(x)
		case UO_Logarithm:
			return E.Divide(dF, F);

		// d/dx log2(f(x)) = 1 / (f(x) * ln(2)) * f'(x)
		case UO_Logarithm2:
			return E.Divide(dF, E.Multiply(F, Constant(Ln2)));

		// d/dx log10(f(x)) = 1 / (f(x) * ln(10)) * f'(x)
		case UO_Logarithm10:
			return E.Divide(dF, E.Multiply(F, Constant(Ln10)));

		// d/dx saturate(f(x)) = f'(x) if f(x) is inside (0-1) range, 0 otherwise
		case UO_Saturate:
			return E.Select(E.And(
							E.LessThan(Zero(), F), // 0 < f(x)
							E.LessThan(F, One())), // f(x) < 1
						dF, Zero());

		// d/dx sin(f(x)) = cos(f(x)) * f'(x)
		case UO_Sin:
			return E.Multiply(E.Cos(F), dF);
		
		// d/dx sinh(f(x)) = cosh(f(x)) * f'(x)
		case UO_Sinh:
			return E.Multiply(E.Cosh(F), dF);

		// d/dx sqrt(f(x)) = 1 / (2 * sqrt(f(x))) * f'(x)
		case UO_Sqrt:
			return E.Divide(dF, E.Multiply(Constant(2), E.Sqrt(F)));

		// d/dx rcp(f(x)) = -1 / (f(x)^2) * f'(x) 
		case UO_Reciprocal:
		{
			FValueRef rcp = E.Reciprocal(F);
			return E.Multiply(E.Multiply(E.Negate(rcp), rcp), dF);
		}

		// d/dx rsqrt(f(x)) = -1 / (2 * sqrt(f(x)) * f(x)) * f'(x)
		case UO_Rsqrt:
			return E.Multiply(E.Multiply(E.ConstantFloat(-0.5f), E.Multiply(E.Rsqrt(F), E.Reciprocal(F))), dF);

		// d/dx tan(f(x)) = 1 / cos^2(f(x)) * f'(x)
		case UO_Tan:
		{
			FValue* CosVal = E.Cos(F);
			return E.Divide(dF, E.Multiply(CosVal, CosVal));
		}
		
		// d/dx tanh(f(x)) = (1 - tanh(f(x))^2) * f'(x)
		case UO_Tanh:
			return E.Multiply(E.Subtract(One(), E.Multiply(Op, Op)), dF);

		// These functions are piecewise constant, that is mostly constant with some
		// discontinuities. We assume they're always constant, as they're not differentiable
		// at the discontinuities.
		case UO_Ceil:
		case UO_Floor:
		case UO_Round:
		case UO_Truncate:
			return E.Cast(Zero(), Op->Type);

		// d/dx (f(x) + g(x)) = f'(x) + g'(x)
		case BO_Add:
			return E.Add(dF, dG);

		// d/dx (f(x) - g(x)) = f'(x) - g'(x)
		case BO_Subtract:
			return E.Subtract(dF, dG);

		// d/dx (f(x) * g(x)) = f'(x) * g(x) + f(x) * g'(x)
		case BO_Multiply:
			return E.Add(E.Multiply(dF, G), E.Multiply(F, dG));

		// d/dx matmul(f(x), g(x)) = matmul(f'(x), g(x)) + matmul(f(x), g'(x))
		case BO_MatrixMultiply:
			return E.Add(E.MatrixMultiply(dF, G), E.MatrixMultiply(F, dG));

		// d/dx (f(x) / g(x)) = (f'(x) * g(x) - f(x) * g'(x)) / g(x)^2
		case BO_Divide:
			return E.Divide(E.Subtract(E.Multiply(dF, G), E.Multiply(F, dG)), E.Multiply(G, G));

		// fmod(f(x), g(x)) = f(x) - g(x) * floor(f(x) / g(x)).
		// Thus:
		//     d/dx fmod(f(x), g(x)) = f'(x) - g(x) * floor(f(x) / g(x))
		// since `floor` is piecewise constant.
		case BO_Fmod:
			return E.Subtract(dF, E.Multiply(dG, E.Operator(UO_Floor, E.Divide(F, G))));

		// d/dx max(f(x), g(x)) = f'(x) if f(x) > g(x), else g'(x)
		case BO_Max:
			return E.Select(E.Operator(BO_GreaterThan, F, G), dF, dG);

		// d/dx min(f(x), g(x)) = f'(x) if f(x) < g(x), else g'(x)
		case BO_Min:
			return E.Select(E.LessThan( F, G), dF, dG);

		// d/dx pow(f(x), g(x)) = f(x)^g(x) * (g'(x) * ln(f(x)) + g(x) * f'(x) / f(x))
		case BO_Pow:
		{
			FValue* Term1 = E.Multiply(dG, E.Logarithm(F)); // g'(x) * ln(f(x))
			FValue* Term2 = E.Divide(E.Multiply(G, dF), F); // g(x) * f'(x) / f(x)
			return E.Multiply(Op, E.Add(Term1, Term2));
		}

		// d/dx atan2(f(x), g(x)) = g(x) / (f(x)^2 + g(x)^2) * f'(x)  -  f(x) / (f(x)^2 + g(x)^2) * g'(x)
		case BO_ATan2:
		case BO_ATan2Fast:
		{
			FValue* Magnitude = E.Divide(One(), E.Add(E.Multiply(F, F), E.Multiply(G, G)));		// 1 / (f(x)^2 + g(x)^2)
			return E.Subtract(E.Multiply(E.Multiply(G, Magnitude), dF), E.Multiply(E.Multiply(F, Magnitude), dG));
		}

		// The multiplication rule applies for the dot product too.
		// d/dx (f(x) • g(x)) = f'(x) • g(x) + f(x) • g'(x)
		case BO_Dot:
			return E.Add(E.Operator(BO_Dot, dF, G), E.Operator(BO_Dot, F, dG));

		// The multiplication rule applies for the cross product too.
		// d/dx (f(x) × g(x)) = f'(x) × g(x) + f(x) × g'(x)
		case BO_Cross:
			return E.Add(E.Operator(BO_Cross, dF, G), E.Operator(BO_Cross, F, dG));

		// clamp(x, min, max) (F=x, min=G, max=H)
		// The derivative is defined when x is between min and max (f'(x)). At and outside
		// bounds, the clamp result is constant and thus the derivative is zero.
		case TO_Clamp:
			return E.Select(E.And(
					E.LessThan(G, F), 
					E.LessThan(F, H)),
				dF, Zero());

		// lerp(a, b, t) = a + t * (b - a)
		// d/dx lerp(f(x), g(x), h(x)) = f'(x) + d/dx (h(x) * ((g(x) - f(x)))
		// d/dx (h(x) * ((g(x) - f(x))) = h'(x) * ((g(x) - f(x))) + h(x) * (g'(x) - f'(x))
		case TO_Lerp:
			return E.Add(dF, 
					   E.Add(E.Multiply(dH, E.Subtract(G, F)), 
						   E.Multiply(H, E.Subtract(dG, dF))));

		// d/dx select(F, g(x), h(x)) = select(F, g'(x), h'(x))
		case TO_Select:
			return E.Select(F, dG, dH);

		// smoothstep(f(x), g(x), h(x)) = 3 z^2 - 2 z^3  with z = saturate((h - f) / (g - f))
		case TO_Smoothstep:
		{
			FValue* Z  = E.Saturate(E.Divide(E.Subtract(H, F), E.Subtract(G, F)));
			FValue* dZ = E.AnalyticalPartialDerivative(Z, Axis);
			// d/dx 3 z(x)^2 - 2 z(x)^3 = 6 * z(x) * z'(x) - 6 * z(x)^2 * z'(x) = 6 * (z(x) - z(x)^2) * z'(x)
			return E.Multiply(dZ, E.Multiply(Constant(6), E.Subtract(Z, E.Multiply(Z, Z))));
		}

		// these are either invalid or constant
		case UO_BitwiseNot:
		case UO_IsFinite:
		case UO_IsInf:
		case UO_IsNan:
		case UO_LWCTile:
		case UO_Sign:
		case BO_Modulo:
		case BO_BitwiseAnd:
		case BO_BitwiseOr:
		case BO_BitwiseXor:
		case BO_BitShiftLeft:
		case BO_BitShiftRight:
		case BO_Step:
			return E.Cast(Zero(), Op->Type);

		default:
			UE_MIR_UNREACHABLE();
	}
}

FValueRef FEmitter::AnalyticalPartialDerivative(FValueRef Value, EDerivativeAxis Axis)
{
	// Any operation on poison arguments is a poison.
	if (!Value.IsValid())
	{
		return Value;
	}

	// Differentiation is only valid on primitive types.
	if (!Value->Type.IsAnyFloat())
	{
		Errorf(Value, TEXT("Trying to differentiate a value of type `%s` is invalid. Expected a float type."), *Value->Type.GetSpelling());
		return Poison();
	}

	switch (Value->Kind)
	{
		case VK_Builtin:
		{
			return Cast(ConstantZero(Value->Type.GetPrimitive().ScalarKind), Value->Type);
		}

		case VK_Composite:
		{
			// Make a prototype composite to hold the derivatives of all its components
			FComposite* Derivative = MakeCompositePrototype(*this, Value->Type, Value->Type.GetPrimitive().NumComponents());
			TArrayView<FValue*> DerivativeComponents = Derivative->GetMutableComponents();

			// Compute the derivative of each component
			TConstArrayView<FValue*> ValueComponents = Value->As<FComposite>()->GetComponents();
			for (int32 i = 0; i < ValueComponents.Num(); ++i)
			{
				DerivativeComponents[i] = AnalyticalPartialDerivative(ValueComponents[i], Axis);
			}

			return EmitPrototype(*this, *Derivative);
		}

		case VK_Operator:
			return DifferentiateOperator(*this, Value->As<FOperator>(), Axis);
		
		case VK_Branch:
		{
			FBranch* AsBranch = Value->As<FBranch>();
			return Branch(AsBranch->ConditionArg,
				AnalyticalPartialDerivative(AsBranch->TrueArg, Axis),
				AnalyticalPartialDerivative(AsBranch->FalseArg, Axis));
		}

		case VK_StageSwitch:
		{
			// For StageSwitch, we want to pass through and generate derivatives for its input.  We only need to do this for
			// the compute stage, because the analytic derivative code path is unreachable for the pixel and vertex stages.
			// To reach the analytic derivative code path in the first place, there will have been a higher level stage switch,
			// which will already have chosen a different hardware derivative path for the pixel shader, or thrown an error for
			// the vertex shader, where explicit derivatives are disallowed completely.
			// 
			// For the other stages, we can pass a poison value, to detect if this assumption is violated in the future.
			// Because this is a non-error unreachable poison, we don't call FEmitter::Poison, as that will trigger an unwanted
			// breakpoint when using the debug feature that breaks on poison values.
			FStageSwitch* AsStageSwitch = Value->As<FStageSwitch>();
			TStaticArray<FValueRef, NumStages> StageDerivatives;
			for (int32 StageIndex = 0; StageIndex < NumStages; StageIndex++)
			{
				StageDerivatives[StageIndex] = StageIndex == Stage_Compute ? AnalyticalPartialDerivative(AsStageSwitch->Args[StageIndex], Axis) : FPoison::Get();
			}
			return StageSwitch(Value->Type, StageDerivatives);
		}

		case VK_Subscript:
		{
			FSubscript* AsSubscript = Value->As<FSubscript>();
			return Subscript(AnalyticalPartialDerivative(AsSubscript->Arg, Axis), AsSubscript->Index);
		}

		case VK_Scalar: 
		{
			FScalar* AsScalar = Value->As<FScalar>();
			return Cast(AnalyticalPartialDerivative(AsScalar->Arg, Axis), AsScalar->Type);
		}

		case VK_Extern:
		{
			FExtern* AsExtern = Value->As<FExtern>();
			const FExternInfo& Info = AsExtern->GetInfo();
			const FType DerivativeType = !Info.DifferentialType.IsPoison() ? Info.DifferentialType : Info.Type;

			if ((Info.Flags & MIR::EExternFlags::ZeroDifferentials) != MIR::EExternFlags::None)
			{
				return ConstantDefault(DerivativeType);
			}
			else if ((Info.Flags & MIR::EExternFlags::NoDifferentials) != MIR::EExternFlags::None)
			{
				Errorf(TEXT("Extern '%s' does not support analytical partial derivatives."), Info.Name.GetData());
				return Value.ToPoison();
			}

			FPartialDerivative DerivativeProto = MakePrototype<FPartialDerivative>(DerivativeType);
			DerivativeProto.Arg = Value;
			DerivativeProto.Axis = Axis;

			return EmitPrototype(*this, DerivativeProto);
		}

		case VK_GetVertexInterpolator:
		{
			FPartialDerivative DerivativeProto = MakePrototype<FPartialDerivative>(Value->Type);
			DerivativeProto.Arg = Value;
			DerivativeProto.Axis = Axis;
			return EmitPrototype(*this, DerivativeProto);
		}

		// These values don't work with analytic derivatives, and force hardware derivatives (or zero if the shader model doesn't support compute shader derivatives).
		case VK_TextureRead:
		case VK_Call:
		{
			if (GetFeatureLevel() >= ERHIFeatureLevel::Type::SM6)
			{
				FPartialDerivative DerivativeProto = MakePrototype<FPartialDerivative>(Value->Type);
				DerivativeProto.Arg = Value;
				DerivativeProto.Axis = Axis;
				DerivativeProto.bHardwareDerivative = true;
				return EmitPrototype(*this, DerivativeProto);
			}
			else
			{
				return Cast(ConstantZero(Value->Type.GetPrimitive().ScalarKind), Value->Type);
			}
		}

		// Finally, these values are uniform (constant) or their differential is not provided, thus their value is always zero.
		case VK_Constant:
		case VK_PrimitiveUniform:
		case VK_PreshaderParameter:
		case VK_PartialDerivative:
			return Cast(ConstantZero(Value->Type.GetPrimitive().ScalarKind), Value->Type);

		default:
			UE_MIR_UNREACHABLE();
	}
}

FValueRef FEmitter::VertexInterpolator(FValueRef VertexValue)
{
	// Cast the incoming vertex value to a float scalar or vector
	VertexValue = CastToFloatKind(CheckIsScalarOrVector(VertexValue));
	if (!VertexValue.IsValid())
	{
		return VertexValue;
	}

	// Create the set/get pair of vertex interpolator instructions
	FSetVertexInterpolator SetInstrProto = MakePrototype<FSetVertexInterpolator>(FType::MakeVoid());
	SetInstrProto.Arg = VertexValue;

	FGetVertexInterpolator GetInstrProto = MakePrototype<FGetVertexInterpolator>(VertexValue->Type);
	GetInstrProto.SetSubgraphProperties(EGraphProperties::PixelStageOnly);
	GetInstrProto.SetInstr = EmitPrototype(*this, SetInstrProto)->As<FSetVertexInterpolator>();

	return EmitPrototype(*this, GetInstrProto);
}

static void SetFunctionFromDesc(FFunction& Function, FFunctionKind Kind, const FFunctionDesc& Desc)
{
	Function.Kind = Kind;
	Function.Name = Desc.Name;
	Function.NumInputOnlyParams = Desc.NumInputOnlyParams;
	Function.NumInputAndOutputParams = Desc.NumInputOnlyParams + Desc.NumInputOutputParams;
	Function.NumParameters = Function.NumInputAndOutputParams + Desc.NumOutputOnlyParams;
	Function.ReturnType = Desc.ReturnType;

	// Copy over the parameter declarations from the description to the function prototype
	for (uint32 i = 0; i < Function.NumParameters; ++i)
	{
		Function.Parameters[i] = Desc.Parameters[i];
	} 
}

FFunction* FEmitter::FunctionHLSL(const FFunctionHLSLDesc& Desc)
{
	check(Desc.GetNumParameters() <= MIR::MaxNumFunctionParameters);

	FFunctionHLSL Prototype = {};
	SetFunctionFromDesc(Prototype, FFunctionKind::HLSL, Desc);
	Prototype.Code = Desc.Code;
	Prototype.Defines = Desc.Defines;
	Prototype.Includes = Desc.Includes;

	// Return existing function if an equivalent one was already emitted.
	if (FFunctionHLSL* const* Found = FunctionHLSLSet.Find(&Prototype))
	{
		return *Found;
	}

	// Create the new MIR HLSL function instance and set it up
	FFunctionHLSL* Function = new (FPrivate::Allocate(*this, sizeof(FFunctionHLSL), alignof(FFunctionHLSL))) FFunctionHLSL { Prototype };
	Function->Name = Module->InternString(Function->Name);
	Function->Code = Module->InternString(Desc.Code);
	Function->Defines = MakeArrayCopy(*this, Desc.Defines);
	Function->Includes = MakeArrayCopy(*this, Desc.Includes);

	FunctionHLSLSet.Add(Function);
	return Function;
}

uint32 FEmitter::FFunctionHLSLKeyFuncs::GetKeyHash(const FFunctionHLSL* Key)
{
	return GetTypeHash(*Key);
}

FValueRef FEmitter::Call(FFunction* Function, TConstArrayView<FValueRef> InputArguments)
{
	if (!Function)
	{
		return Poison();
	}

	if (InputArguments.Num() != Function->NumInputAndOutputParams)
	{
		Errorf(TEXT("Function called with incorrect number of arguments. Expected %d but got %d."), Function->NumInputAndOutputParams, InputArguments.Num());
		return Poison();
	}

	FCall Call = MakePrototype<FCall>(Function->ReturnType);
	Call.Function = Function;
	Call.NumArguments = InputArguments.Num();

	for (int i = 0; i < InputArguments.Num(); ++i)
	{
		Call.Arguments[i] = InputArguments[i];
	}

	return EmitPrototype(*this, Call);
}

FValueRef FEmitter::CallParameterOutput(FValueRef InCall, uint32 ParameterIndex)
{
	if (IsAnyNotValid(InCall))
	{
		return InCall.ToPoison();
	}

	FCall* Call = InCall->As<FCall>();
	if (!Call)
	{
		Errorf(TEXT("Expected function call, found a '%s' value instead."), MIR::LexToString(InCall->Kind));
		return InCall.ToPoison();
	}
	
	if (ParameterIndex >= Call->Function->GetNumOutputParameters())
	{
		Errorf(InCall, TEXT("Invalid output index %d. Function has %d outputs."), ParameterIndex, Call->Function->GetNumOutputParameters());
		return InCall.ToPoison();
	}

	FCallParameterOutput Proto = MakePrototype<FCallParameterOutput>(Call->Function->GetOutputParameter(ParameterIndex).Type);
	Proto.Call = Call;
	Proto.Index = ParameterIndex;

	return EmitPrototype(*this, Proto);
}

FValueRef FEmitter::SubstrateDefaultSlab()
{
	FSubstrateDefaultSlab Prototype = MakePrototype<FSubstrateDefaultSlab>(FType::MakeSubstrateData());
	Prototype.Dummy = nullptr;
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstratePromoteToOperator(FValueRef SubstrateData, FValueRef OperatorIndex, FValueRef BSDFIndex, FValueRef LayerDepth, FValueRef bIsBottom)
{
	FSubstratePromoteToOperator Prototype = MakePrototype<FSubstratePromoteToOperator>(FType::MakeSubstrateData());
	Prototype.SubstrateDataInput = SubstrateData;
	Prototype.OperatorIndex = OperatorIndex;
	Prototype.BSDFIndex = BSDFIndex;
	Prototype.LayerDepth = LayerDepth;
	Prototype.bIsBottom = bIsBottom;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateShadingModels(const FSubstrateShadingModelsDesc Desc)
{
	FSubstrateShadingModels Prototype = MakePrototype<FSubstrateShadingModels>(FType::MakeSubstrateData());

	Prototype.BaseColor = Desc.BaseColor;
	Prototype.Specular = Desc.Specular;
	Prototype.Metallic = Desc.Metallic;
	Prototype.Roughness = Desc.Roughness;
	Prototype.Anisotropy = Desc.Anisotropy;
	Prototype.SubSurfaceColor = Desc.SubSurfaceColor;
	Prototype.SubSurfaceProfileId = Desc.SubSurfaceProfileId;
	Prototype.ClearCoat = Desc.ClearCoat;
	Prototype.ClearCoatRoughness = Desc.ClearCoatRoughness;
	Prototype.EmissiveColor = Desc.EmissiveColor;
	Prototype.Opacity = Desc.Opacity;
	Prototype.ThinTranslucentTransmittanceColor = Desc.ThinTranslucentTransmittanceColor;
	Prototype.ThinTranslucentSurfaceCoverage = Desc.ThinTranslucentSurfaceCoverage;
	Prototype.WaterScatteringCoefficients = Desc.WaterScatteringCoefficients;
	Prototype.WaterAbsorptionCoefficients = Desc.WaterAbsorptionCoefficients;
	Prototype.WaterPhaseG = Desc.WaterPhaseG;
	Prototype.ColorScaleBehindWater = Desc.ColorScaleBehindWater;
	Prototype.ShadingModel = Desc.ShadingModel;
	Prototype.Normal = Desc.Normal;
	Prototype.Tangent = Desc.Tangent;
	Prototype.ClearCoatNormal = Desc.ClearCoatNormal;
	Prototype.CustomTangent = Desc.CustomTangent;
	Prototype.BasisIndexMacro = Desc.BasisIndexMacro;
	Prototype.ClearCoat_BasisIndexMacro = Desc.ClearCoat_BasisIndexMacro;

	Prototype.bHasDynamicShadingModels = Desc.bHasDynamicShadingModels;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateSlab(const FSubstrateSlabDesc& Desc)
{
	FSubstrateSlab Prototype = MakePrototype<FSubstrateSlab>(FType::MakeSubstrateData());
	
	Prototype.Normal = Desc.Normal;
	Prototype.DiffuseAlbedo = Desc.DiffuseAlbedo;
	Prototype.F0 = Desc.F0;
	Prototype.F90 = Desc.F90;
	Prototype.Roughness = Desc.Roughness;
	Prototype.Anisotropy = Desc.Anisotropy;
	Prototype.SSSProfileId = Desc.SSSProfileId;
	Prototype.SSSMFP = Desc.SSSMFP;
	Prototype.SSSMFPScale = Desc.SSSMFPScale;
	Prototype.SSSPhaseAniso = Desc.SSSPhaseAniso;
	Prototype.SSSType = Desc.SSSType;
	Prototype.EmissiveColor = Desc.EmissiveColor;
	Prototype.SecondRoughness = Desc.SecondRoughness;
	Prototype.SecondRoughnessWeight = Desc.SecondRoughnessWeight;
	Prototype.SecondRoughnessAsSimpleClearCoat = Desc.SecondRoughnessAsSimpleClearCoat;
	Prototype.ClearCoatUseSecondNormal = Desc.ClearCoatUseSecondNormal;
	Prototype.ClearCoatBottomNormal = Desc.ClearCoatBottomNormal;
	Prototype.FuzzAmount = Desc.FuzzAmount;
	Prototype.FuzzColor = Desc.FuzzColor;
	Prototype.FuzzRoughness = Desc.FuzzRoughness;
	Prototype.GlintValue = Desc.GlintValue;
	Prototype.GlintUV = Desc.GlintUV;
	Prototype.SpecularProfileId = Desc.SpecularProfileId;
	Prototype.Thickness = Desc.Thickness;
	Prototype.IsThin = Desc.IsThin;
	Prototype.IsAtBottom = Desc.IsAtBottom;
	Prototype.LocalBasisIndex = Desc.LocalBasisIndex;

	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateToon(const FSubstrateToonDesc& Desc)
{
	FSubstrateToon Prototype = MakePrototype<FSubstrateToon>(FType::MakeSubstrateData());

	Prototype.Normal = Desc.Normal;
	Prototype.ToonProfileId = Desc.ToonProfileId;
	Prototype.BaseColor = Desc.BaseColor;
	Prototype.Metallic = Desc.Metallic;
	Prototype.Specular = Desc.Specular;
	Prototype.Roughness = Desc.Roughness;
	Prototype.EmissiveColor = Desc.EmissiveColor;
	Prototype.PatternUVs = Desc.PatternUVs;
	Prototype.IsAtBottom = Desc.IsAtBottom;
	Prototype.LocalBasisIndex = Desc.LocalBasisIndex;

	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;
	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateHorizontalMixing(FValueRef Background, FValueRef Foreground, FValueRef Mix, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves)
{
	FSubstrateHorizontalMixing Prototype = MakePrototype<FSubstrateHorizontalMixing>(FType::MakeSubstrateData());
	Prototype.Background = Background;
	Prototype.Foreground = Foreground;
	Prototype.Mix = Mix;

	// ParameterBlending = 0
	Prototype.OperatorIndex = OperatorIndex;
	Prototype.MaxDistanceFromLeaves = MaxDistanceFromLeaves;

	// ParameterBlending = 1
	Prototype.NormalMix = nullptr;
	Prototype.SharedLocalBasisIndexMacro = nullptr;
	Prototype.BackgroundNoV = nullptr;
	Prototype.ForegroundNoV = nullptr;

	// Settings
	Prototype.bParameterBlendingEnabled = false;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateHorizontalMixingParameterBlending(FValueRef Background, FValueRef Foreground, FValueRef Mix, FValueRef NormalMix, FValueRef SharedLocalBasisIndexMacro, FValueRef BackgroundNoV, FValueRef ForegroundNoV)
{
	FSubstrateHorizontalMixing Prototype = MakePrototype<FSubstrateHorizontalMixing>(FType::MakeSubstrateData());
	Prototype.Background = Background;
	Prototype.Foreground = Foreground;
	Prototype.Mix = Mix;

	// ParameterBlending = 0
	Prototype.OperatorIndex = nullptr;
	Prototype.MaxDistanceFromLeaves = nullptr;

	// ParameterBlending = 1
	Prototype.NormalMix = NormalMix;
	Prototype.SharedLocalBasisIndexMacro = SharedLocalBasisIndexMacro;
	Prototype.BackgroundNoV = BackgroundNoV;
	Prototype.ForegroundNoV = ForegroundNoV;

	// Settings
	Prototype.bParameterBlendingEnabled = true;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateVerticalLayering(FValueRef Top, FValueRef Base, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves)
{
	FSubstrateVerticalLayering Prototype = MakePrototype<FSubstrateVerticalLayering>(FType::MakeSubstrateData());
	Prototype.Top = Top;
	Prototype.Base = Base;

	// ParameterBlending = 0
	Prototype.OperatorIndex = OperatorIndex;
	Prototype.MaxDistanceFromLeaves = MaxDistanceFromLeaves;

	// ParameterBlending = 1
	Prototype.SharedLocalBasisIndexMacro = nullptr;
	Prototype.TopNoV = nullptr;
	Prototype.BaseNoV = nullptr;

	// Settings
	Prototype.bParameterBlendingEnabled = false;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateVerticalLayeringParameterBlending(FValueRef Top, FValueRef Base, FValueRef SharedLocalBasisIndexMacro, FValueRef TopNoV, FValueRef BaseNoV)
{
	FSubstrateVerticalLayering Prototype = MakePrototype<FSubstrateVerticalLayering>(FType::MakeSubstrateData());
	Prototype.Top = Top;
	Prototype.Base = Base;

	// ParameterBlending = 0
	Prototype.OperatorIndex = nullptr;
	Prototype.MaxDistanceFromLeaves = nullptr;

	// ParameterBlending = 1
	Prototype.SharedLocalBasisIndexMacro = SharedLocalBasisIndexMacro;
	Prototype.TopNoV = TopNoV;
	Prototype.BaseNoV = BaseNoV;

	// Settings
	Prototype.bParameterBlendingEnabled = true;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateCoverageWeight(FValueRef A, FValueRef Weight, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves)
{
	FSubstrateCoverageWeight Prototype = MakePrototype<FSubstrateCoverageWeight>(FType::MakeSubstrateData());
	Prototype.A = A;
	Prototype.Weight = Weight;

	// ParameterBlending = 0
	Prototype.OperatorIndex = OperatorIndex;
	Prototype.MaxDistanceFromLeaves = MaxDistanceFromLeaves;

	// Settings
	Prototype.bParameterBlendingEnabled = false;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateCoverageWeightParameterBlending(FValueRef A, FValueRef Weight)
{
	FSubstrateCoverageWeight Prototype = MakePrototype<FSubstrateCoverageWeight>(FType::MakeSubstrateData());
	Prototype.A = A;
	Prototype.Weight = Weight;

	// ParameterBlending = 0
	Prototype.OperatorIndex = nullptr;
	Prototype.MaxDistanceFromLeaves = nullptr;

	// Settings
	Prototype.bParameterBlendingEnabled = true;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateAdd(FValueRef A, FValueRef B, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves)
{
	FSubstrateAdd Prototype = MakePrototype<FSubstrateAdd>(FType::MakeSubstrateData());
	Prototype.A = A;
	Prototype.B = B;

	// ParameterBlending = 0
	Prototype.OperatorIndex = OperatorIndex;
	Prototype.MaxDistanceFromLeaves = MaxDistanceFromLeaves;

	// ParameterBlending = 1
	Prototype.NormalMix = nullptr;
	Prototype.SharedLocalBasisIndexMacro = nullptr;
	Prototype.ANoV = nullptr;
	Prototype.BNoV = nullptr;

	// Settings
	Prototype.bParameterBlendingEnabled = false;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateAddParameterBlending(FValueRef A, FValueRef B, FValueRef NormalMix, FValueRef SharedLocalBasisIndexMacro, FValueRef ANoV, FValueRef BNoV)
{
	FSubstrateAdd Prototype = MakePrototype<FSubstrateAdd>(FType::MakeSubstrateData());
	Prototype.A = A;
	Prototype.B = B;

	// ParameterBlending = 0
	Prototype.OperatorIndex = nullptr;
	Prototype.MaxDistanceFromLeaves = nullptr;

	// ParameterBlending = 1
	Prototype.NormalMix = NormalMix;
	Prototype.SharedLocalBasisIndexMacro = SharedLocalBasisIndexMacro;
	Prototype.ANoV = ANoV;
	Prototype.BNoV = BNoV;

	// Settings
	Prototype.bParameterBlendingEnabled = true;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

FValueRef FEmitter::SubstrateSelectParameterBlending(FValueRef A, FValueRef B, FValueRef SelectValue, FValueRef SharedLocalBasisIndexMacro)
{
	FSubstrateSelect Prototype = MakePrototype<FSubstrateSelect>(FType::MakeSubstrateData());
	Prototype.A = A;
	Prototype.B = B;
	Prototype.SelectValue = SelectValue;

	// ParameterBlending = 0
	Prototype.OperatorIndex = nullptr;
	Prototype.MaxDistanceFromLeaves = nullptr;

	// ParameterBlending = 1
	Prototype.SharedLocalBasisIndexMacro = SharedLocalBasisIndexMacro;

	// Settings
	Prototype.bParameterBlendingEnabled = true;
	Prototype.bFullySimplifiedInstruction = SubstrateTranslatorData->CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

	return EmitPrototype(*this, Prototype);
}

EShaderPlatform FEmitter::GetShaderPlatform() const
{
	return Module->GetShaderPlatform();
}

const ITargetPlatform* FEmitter::GetTargetPlatform() const
{
	return Module->GetTargetPlatform();
}

ERHIFeatureLevel::Type FEmitter::GetFeatureLevel() const
{
	return Module->GetFeatureLevel();
}

EMaterialQualityLevel::Type FEmitter::GetQualityLevel() const
{
	return Module->GetQualityLevel();
}

Substrate::FSubstrateTranslatorData* FEmitter::GetSubstrateTranslatorData() const
{
	return SubstrateTranslatorData;
}

void FEmitter::Initialize()
{
	// Create and reference the true/false constants.
	FConstant Temp = MakePrototype<FConstant>(FType::MakeBoolScalar());

	Temp.Boolean = true;
	TrueConstant = EmitPrototype(*this, Temp);

	Temp.Boolean = false;
	FalseConstant = EmitPrototype(*this, Temp);
}

FValueRef FEmitter::EmitPrototype_Internal(const FValue& Value)
{
	return EmitPrototype(*this, Value);
}

bool FEmitter::FValueKeyFuncs::Matches(KeyInitType A, KeyInitType B)
{
	return A->Equals(B);
}

uint32 FEmitter::FValueKeyFuncs::GetKeyHash(KeyInitType Key)
{
	return MIR::HashBytes(Key, Key->GetSizeInBytes());
}

} // namespace MIR

#endif // #if WITH_EDITOR
