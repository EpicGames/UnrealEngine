// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialIREmitter.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "Engine/Texture.h"
#include "VT/RuntimeVirtualTexture.h"

#if WITH_EDITOR

namespace MIR::Internal {

EMaterialValueType GetTextureMaterialValueType(const UObject* TextureObject)
{
	if (TextureObject)
	{
		if (TextureObject->IsA<UTexture>())
		{
			return Cast<UTexture>(TextureObject)->GetMaterialType();
		}
		if (TextureObject->IsA<URuntimeVirtualTexture>())
		{
			return MCT_TextureVirtual;
		}
	}
	return MCT_Unknown;
}


} // namespace MIR::Internal

#endif
