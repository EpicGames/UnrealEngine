// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIRCommon.h"
#include "MaterialShared.h"

#if WITH_EDITOR

namespace MIR::Internal {

// Returns the material value type of the specified UTexture or URuntimeVirtualTexture.
EMaterialValueType GetTextureMaterialValueType(const UObject* TextureObject);

// Resolves the value for a given expression input, following through call scope boundaries.
// Returns an empty TOptional if a dependency was pushed (caller should defer).
// Returns TOptional with the value (possibly nullptr for disconnected inputs) when available.
TOptional<FValue*> RequestInputValue(FMaterialIRModuleBuilderImpl* Builder, const FExpressionInput* Input);

// Flows a value into a given expression output.
void BindValueToExpressionOutput(FMaterialIRModuleBuilderImpl* Builder, const FExpressionOutput* Output, FValue* Value);

// Registers the current expression as a subgraph with its own scope.
// NumInputs is the number of input expressions in this subgraph.
// Outputs[i] is the inner expression that produces output i.
void DeclareSubgraph(FMaterialIRModuleBuilderImpl* Builder, int32 NumInputs, TConstArrayView<FSubgraphOutputMapping> Outputs);

// Fetches the value from the subgraph expression's input pin at the given index, resolving in the parent scope.
// Returns an empty TOptional if a dependency was pushed (caller should defer).
// Returns nullptr if not inside a subgraph scope or if the index is invalid.
TOptional<FValue*> RequestSubgraphInputValue(FMaterialIRModuleBuilderImpl* Builder, int32 InputIndex);

// Returns the expression that declared the current subgraph scope, or null if at root level.
UMaterialExpression* GetSubgraphExpression(FMaterialIRModuleBuilderImpl* Builder);

} // namespace MIR::Internal

#endif // #if WITH_EDITOR

