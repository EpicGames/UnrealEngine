// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/Material.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "ParameterCollection.h"

#if WITH_EDITOR

FMaterialIRModule::FMaterialIRModule()
{
}

FMaterialIRModule::~FMaterialIRModule()
{
	Empty();
}

void FMaterialIRModule::Empty()
{
	Errors.Empty();
	ShadingModelsFromCompilation = {};
	FunctionHLSLs.Empty();
	ParameterCollections.Empty();
	EnvironmentDefines.Empty();
	IntegerEnvironmentDefines.Empty();
	InternedStrings.Empty();
	ParameterIdToData.Empty();
	ParameterInfoToId.Empty();
	Statistics = {};
	MIR::ZeroArray(TArrayView<MIR::FValue*>{ PropertyValues, UE_ARRAY_COUNT(PropertyValues) });
	EntryPoints.Empty();
	Values.Empty();
	CompilationOutput = {};
	CustomOutputs.Empty();
	CustomOutputGroups.Empty();
	SubstrateData = {};
	Allocator.Flush();
}

int32 FMaterialIRModule::AddEntryPoint(FStringView Name, MIR::EStage Stage, int32 NumOutputsHint)
{
	uint32 Index = EntryPoints.Num();

	EntryPoints.Push({
		.Name = InternString(Name),
		.Stage = Stage,
		.RootBlock = {},
	});

	EntryPoints.Last().Outputs.Reserve(NumOutputsHint);

	return Index;
}

void FMaterialIRModule::BeginCustomOutputGroup(FStringView Name, int32 NumOutputs)
{
	CustomOutputGroups.Push(FCustomOutputGroup{ Name, NumOutputs, CustomOutputs.Num() });
}

FStringView FMaterialIRModule::InternString(FStringView InString)
{
	// Precompute the hash first, to be used in both finding and inserting the string into the set of interned strings
	uint32 KeyHash = GetTypeHash(InString);

	// Look for the string in the set of interned strings first, and return existing if found.
	const FStringView* ExistingString = InternedStrings.FindByHash(KeyHash, InString);
	if (ExistingString)
	{
		return *ExistingString;
	}

	// Allocate the buffer to contain the string
	FStringView::ElementType* String = (FStringView::ElementType*)Allocator.PushBytes(InString.NumBytes() + sizeof(FStringView::ElementType), alignof(FStringView::ElementType));

	// Copy the string over to the interned location
	FMemory::Memcpy(String, InString.GetData(), InString.NumBytes());

	// Mark the null-character at the end
	String[InString.Len()] = 0;

	// Make the view
	FStringView View = { String, InString.Len() };

	// Add the new string into the set of interned strings so future fetches of this string will result in the same unique string
	InternedStrings.AddByHash(KeyHash, View);

	return View;
}

void FMaterialIRModule::AddError(UMaterialExpression* Source, FString Message)
{
	Errors.Push({ Source, MoveTemp(Message) });
}

int32 FMaterialIRModule::FindOrAddParameterCollection(UMaterialParameterCollection* ParameterCollection)
{
	int32 CollectionIndex = ParameterCollections.Find(ParameterCollection);

	if (CollectionIndex == INDEX_NONE)
	{
		if (ParameterCollections.Num() >= MaxNumParameterCollectionsPerMaterial)
		{
			return INDEX_NONE;
		}
		else
		{
			ParameterCollections.Add(ParameterCollection);
			CollectionIndex = ParameterCollections.Num() - 1;
		}
	}

	return CollectionIndex;
}

void FMaterialIRModule::AddFunctionHLSL(const MIR::FFunctionHLSL* Function)
{
	FunctionHLSLs.Add(Function);
}

void FMaterialIRModule::AddShadingModel(EMaterialShadingModel InShadingModel)
{
	ShadingModelsFromCompilation.AddShadingModel(InShadingModel);
}

bool FMaterialIRModule::IsMaterialPropertyUsed(EMaterialProperty InProperty) const
{
	return PropertyValues[InProperty] && !PropertyValues[InProperty]->EqualsConstant(FMaterialAttributeDefinitionMap::GetDefaultValue(InProperty));
}


#endif // #if WITH_EDITOR
