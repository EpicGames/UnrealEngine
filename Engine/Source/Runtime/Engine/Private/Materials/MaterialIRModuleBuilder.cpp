// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRModuleBuilder.h"

#if WITH_EDITOR

#include "Materials/MaterialIRValueAnalyzer.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRDebug.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIREmitter.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialAggregate.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionAbsorptionMediumMaterialOutput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionSingleLayerWaterMaterialOutput.h"
#include "Materials/MaterialExpressionThinTranslucentMaterialOutput.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionVertexInterpolator.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialSharedPrivate.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "MaterialExpressionIO.h"
#include "MaterialShared.h"
#include "ShaderCompiler.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialInsights.h"
#include "Math/Color.h"
#include "Engine/Texture.h"
#include "Misc/FileHelper.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "RenderUtils.h"
#include "SubstrateTranslatorCommon.h"
#include "Engine/SubsurfaceProfile.h"

static TAutoConsoleVariable<int32> CVarMaterialIRDebugDumpLevel(
	TEXT("r.Material.Translator.DebugDump"),
	0,
	TEXT("Whether the material translator should dump debug information about the translated module IR.\n")
	TEXT("0 (Default): No debug dump generated.\n")
	TEXT("1: Dump the material IR instructions to readable a human readable textual representation (to '{SavedDir}/Materials/IRDump.txt').\n")
	TEXT("2: Everything above, plus dump the 'Uses' graph in Graphviz Dot syntax (to '{SavedDir}/Materials/IRDumpUseGraph.dot').\n"),
	ECVF_RenderThreadSafe);

// Returns whether given property is a customized UV.
static inline bool IsMaterialPropertyCustomizedUV(EMaterialProperty Property)
{
	return Property >= MP_CustomizedUVs0 && Property <= MP_LastCustomizedUVs;
}

// Identifies a subgraph scope. The (SubgraphExpression, Parent) pair uniquely identifies
// every instantiation, including nested and duplicate subgraphs.
struct FSubgraphContext
{
	// The expression that declared this subgraph scope (null for root).
	UMaterialExpression* SubgraphExpression{};

	// Parent scope from which this subgraph was instantiated (null for root).
	FSubgraphContext* Parent{};

	// Number of inputs.
	int32 NumInputs;

	// One bit per input, tracks which parent-side connection insights have already been pushed.
	uint64* InputInsightsBitField;

	// Length of 'Outputs[]'.
	int32 NumOutputs;

	// The expressions and their output indices generating this subgraph outputs.
	MIR::FSubgraphOutputMapping Outputs[];

	// Allocates a context into the given allocator.
	static FSubgraphContext* Create(FMemStackBase& Allocator, UMaterialExpression* InSubgraphExpression, FSubgraphContext* InParent, int32 NumInputs, TConstArrayView<MIR::FSubgraphOutputMapping> Outputs)
	{
		const int32 NumInputBitFieldPages = (NumInputs + 63) / 64;

		FSubgraphContext* Ctx = (FSubgraphContext*) Allocator.Alloc(sizeof(FSubgraphContext) + Outputs.NumBytes(), alignof(FSubgraphContext));
		Ctx->SubgraphExpression = InSubgraphExpression;
		Ctx->Parent = InParent;
		Ctx->NumInputs = NumInputs;
		Ctx->InputInsightsBitField = NumInputs ? (uint64*) Allocator.Alloc(sizeof(uint64) * NumInputBitFieldPages, alignof(uint64)) : nullptr;
		FMemory::Memzero(Ctx->InputInsightsBitField, sizeof(uint64) * NumInputBitFieldPages);
		Ctx->NumOutputs = Outputs.Num();
		FMemory::Memcpy(Ctx->Outputs, Outputs.GetData(), Outputs.NumBytes());
		return Ctx;
	}
};

// Main implementation struct for the MIR module builder.
struct FMaterialIRModuleBuilderImpl
{
	// Public builder interface that owns this implementation.
	FMaterialIRModuleBuilder* Builder;

	// Target module being constructed.
	FMaterialIRModule* Module;

	// IR value emitter used by expression Build() methods.
	MIR::FEmitter Emitter;

	// Preshader manager for CPU-side uniform expressions.
	FMaterialIRPreshader Preshader;

	// Semantic analyzer for the IR value graph.
	FMaterialIRValueAnalyzer ValueAnalyzer;

	// Default zero-valued material attributes aggregate.
	MIR::FValue* DefaultMaterialAggregate;

	// Preview expression input for the material editor preview.
	FColorMaterialInput PreviewInput;

	// Bump allocator for FSubgraphContext instances (stable pointers, bulk deallocation).
	FMemStackBase ContextAllocator;

	// Root context (no subgraph, no parent), used for top-level expressions.
	FSubgraphContext* RootContext{};

	// Context currently being processed.
	FSubgraphContext* CurrentContext{};

	// Expression build stack, entries from any context interleave naturally.
	TArray<TPair<UMaterialExpression*, FSubgraphContext*>> ExpressionStack;

	// Set of expressions already built in their respective contexts.
	TSet<TPair<UMaterialExpression*, FSubgraphContext*>> BuiltExpressions;

	// Maps expression outputs to their produced IR values, keyed by context.
	TMap<TPair<const FExpressionOutput*, FSubgraphContext*>, MIR::FValue*> OutputValues;

	// Maps (SubgraphExpression, ParentContext) to the subgraph context for that instantiation.
	TMap<TPair<UMaterialExpression*, FSubgraphContext*>, FSubgraphContext*> SubgraphContexts;

	// Substrate material system translator data.
	Substrate::FSubstrateTranslatorData SubstrateTranslatorData;

	// Sets up everything needed before expression graph traversal can begin.
	void Step_Initialize(FMaterialIRModuleBuilder* InBuilder, FMaterialIRModule* InModule)
	{
		this->Builder = InBuilder;
		this->Module = InModule;

		// Setup the Builder implementation
		ValueAnalyzer.Setup(Builder->MaterialInterface, Module, &Preshader, Builder->TargetInsights);

		// Empty the module and set it up
		Module->Empty();
		Module->ShaderPlatform = Builder->ShaderPlatform;
		Module->TargetPlatform = Builder->TargetPlatform;
		Module->FeatureLevel = Builder->FeatureLevel;
		Module->QualityLevel = Builder->QualityLevel;
		Module->BlendMode = Builder->BlendMode;

		// Declare a entry point to evaluate the vertex stage.
		Module->AddEntryPoint(TEXTVIEW("VertexStage"), MIR::Stage_Vertex, 1);

		// Declare the entry points to evaluate both the pixel and compute stages.
		int32 Num = MaterialAttributesAggregate::GetMaterialProperties().Num();
		Module->AddEntryPoint(TEXTVIEW("PixelStage"), MIR::Stage_Pixel, Num);
		Module->AddEntryPoint(TEXTVIEW("ComputeStage"), MIR::Stage_Compute, Num);

		// Setup the emitter and initialize it
		Emitter.BuilderImpl = this;
		Emitter.MaterialInterface = Builder->MaterialInterface;
		Emitter.BaseMaterial = Builder->MaterialInterface->GetBaseMaterial();
		Emitter.Module = Module;
		Emitter.StaticParameterSet = Builder->StaticParameters;
		Emitter.SubstrateTranslatorData = &SubstrateTranslatorData;
		Emitter.Initialize();

		// Create an IR value to hold the material attributes aggregate default
		DefaultMaterialAggregate = Emitter.Aggregate(MaterialAttributesAggregate::Get());

		// Create the root subgraph context
		RootContext = FSubgraphContext::Create(ContextAllocator, nullptr, nullptr, 0, {});
		CurrentContext = RootContext;

		// Set the preview input expression
		PreviewInput.Expression = Builder->PreviewExpression;


		// SUBSTRATE INITIALIZATION

		// Setup Substrate translator data to process the Substrate tree and simplification.
		FSubstrateCompilationConfig* SubstrateCompilationConfig = nullptr;	// SUBSTRATE_TODO: forward SubstrateCompilationConfig from compilation step to make substrate tab simplification preview work.
		if (SubstrateCompilationConfig)
		{
			SubstrateTranslatorData.SubstrateCompilationConfig = *SubstrateCompilationConfig;
		}
		SubstrateTranslatorData.Initialize(Builder->MaterialInterface, Module->CompilationOutput, nullptr, &Emitter);

		FMaterialShadingModelField MaterialShadingModels;
		// If the material gets its shading model from material expressions and we have compiled one or more shading model expressions already, 
		// then use that shading model field instead. It's the most optimal set of shading models
		if (Builder->MaterialInterface->IsShadingModelFromMaterialExpression())
		{
			// This also correctly fetches all shading models form the graphs since material rebuild the "ShadingModels" field.
			// SUBSTRATE_TODO It would be better to use Module->ShadingModelsFromCompilation gathered from the graph using static evaluation/switches but it is evaluated later.
			MaterialShadingModels = Builder->MaterialInterface->GetShadingModels();
		}
		else
		{
			MaterialShadingModels = Builder->MaterialInterface->GetShadingModels();
			UMaterialInterface::FilterOutPlatformShadingModels(Module->ShaderPlatform, MaterialShadingModels);
		}

		bool bDoClearFunctionStack = false;

		UMaterialExpression* MaterialGraphNodePreviewExpression = nullptr; // SUBSTRATE_TODO use Builder->Material->GetMaterialGraphNodePreviewExpression();
		SubstrateTranslatorData.ProcessSubstrateTreeAndTopology(MaterialShadingModels, MaterialGraphNodePreviewExpression, bDoClearFunctionStack);
	}

	// Main build loop, converts the material expression graph into IR values.
	bool Step_BuildMaterialExpressionsToIRGraph()
	{
		// Identify the material property output pins and push their value-producing
		// expressions onto the expression stack to start crawling from them.
		PushRootExpressionDependencies();

		do
		{
			while (!ExpressionStack.IsEmpty())
			{
				FMemMark MemMark { FMemStack::Get() };
				BuildTopMaterialExpression();
			}

			// Re-seeds root dependencies after each drain because DeclareSubgraph may have registered
			// new subgraph contexts that change how root inputs resolve (two-pass resolution).
			CurrentContext = RootContext;
			PushRootExpressionDependencies();
		}
		while (Module->IsValid() && !ExpressionStack.IsEmpty());

		return Module->IsValid();
	}
	
	// Seeds the expression stack with the expressions linked to final material outputs.
	void PushRootExpressionDependencies()
	{
		// If we are processing from the Material Attributes output node, push that dependency here.
		if (Builder->MaterialInterface->GetMaterial()->bUseMaterialAttributes)
		{
			FMaterialInputDescription Input;
			GetExpressionInputDescription(MP_MaterialAttributes, Input);
			RequestInputValue(Input.Input);
		}
		else
		{
			// Otherwise iterate over each individual material attribute pin and push its connected expression as a dependency.
			for (EMaterialProperty Property : MaterialAttributesAggregate::GetMaterialProperties())
			{
				// Ignore deactivated properties.
				if (!PropertyIsActiveInMaterial(Property))
				{
					continue;
				}

				// Read the material input associated to this property
				FMaterialInputDescription Input;
				GetExpressionInputDescription(Property, Input);

				// Push the connected expression to this material attribute input as a dependency, if any.
				RequestInputValue(Input.Input);
			}
		}

		// Push the custom output expressions to the dependency queue too.
		// Note: unfortunately the vertex interpolator expression derives from custom output (even if it isn't a custom output)
		// 	     as an artifact of the old translator. We're only interested in actual custom outputs, as vertex interpolators are
		//       handled differently. When the old translator is removed, make UMaterialExpressionVertexInterpolator a simple UMaterialExpression.
		for (UMaterialExpression* Expression : Builder->MaterialInterface->GetMaterial()->GetExpressions())
		{
			if (Cast<UMaterialExpressionCustomOutput>(Expression) && !Cast<UMaterialExpressionVertexInterpolator>(Expression))
			{
				PushDependencyOnStack(Expression);
			}
		}
	}

	// Produces the Substrate front material custom output, then re-processes BSDFs for the fully-simplified pass.
	bool Step_BuildSubstrateFrontMaterialOutput()
	{
		if (SubstrateTranslatorData.FrontMaterialExpr)
		{
			// Save the the FrontMaterial to a custom output.
			MIR::FValue** SubstrateValuePtr = OutputValues.Find( { SubstrateTranslatorData.FrontMaterialExpr->GetOutput(0), RootContext }); // SUBSTRATE_TODO always assuming output 0 for now but this can be wrong for some material function
			MIR::FValueRef SubstrateFrontMaterial = MIR::FValueRef(SubstrateValuePtr ? *SubstrateValuePtr : nullptr);
			Emitter.SetCustomOutputs(TEXTVIEW("SubstrateFrontMaterial"), { &SubstrateFrontMaterial, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
		}

		// Un-mark Substrate BSDF nodes so they can be re-processed for the fully simplified pass.
		bool bNeedSubstrateExpressionProcessing = false;
		{
			// Collect entries to un-mark (can't modify TSet while iterating)
			TArray<TPair<UMaterialExpression*, FSubgraphContext*>> ToUnmark;
			for (const TPair<UMaterialExpression*, FSubgraphContext*>& Entry : BuiltExpressions)
			{
				if (Entry.Key->IsA(UMaterialExpressionSubstrateBSDF::StaticClass()))
				{
					ToUnmark.Add(Entry);
					bNeedSubstrateExpressionProcessing = SubstrateTranslatorData.FrontMaterialExpr != nullptr;
				}
			}
			for (const TPair<UMaterialExpression*, FSubgraphContext*>& Entry : ToUnmark)
			{
				BuiltExpressions.Remove(Entry);
			}
		}

		// Process the graph once again to generate IR for the FullySimplifiedFrontMaterial output.
		if (bNeedSubstrateExpressionProcessing)
		{
			SubstrateTranslatorData.CurrentSubstrateCompilationContext = Substrate::ESubstrateCompilationContext::SCC_FullySimplified;

			// Clear the expression stack and push the front material expression in the root context.
			ExpressionStack.Empty();
			ExpressionStack.Push( { SubstrateTranslatorData.FrontMaterialExpr, RootContext });
			CurrentContext = RootContext;

			while (!ExpressionStack.IsEmpty())
			{
				FMemMark MemMark { FMemStack::Get() };
				BuildTopMaterialExpression();
			}

			if (SubstrateTranslatorData.CurrentSubstrateCompilationContext == Substrate::ESubstrateCompilationContext::SCC_FullySimplified)
			{
				// Save the output to the FrontMaterial, now fully simplified, to the custom output representing fully simplified.
				MIR::FValue** SubstrateSimplifiedPtr = OutputValues.Find( { SubstrateTranslatorData.FrontMaterialExpr->GetOutput(0), RootContext });
				MIR::FValueRef SubstrateFrontMaterial = MIR::FValueRef(SubstrateSimplifiedPtr ? *SubstrateSimplifiedPtr : nullptr);
				Emitter.SetCustomOutputs(TEXTVIEW("SubstrateFrontMaterialFullySimplified"), { &SubstrateFrontMaterial, 1 }, MIR::EMaterialOutputFrequency::PerPixel);

				SubstrateTranslatorData.CurrentSubstrateCompilationContext = Substrate::ESubstrateCompilationContext::SCC_Default;
			}
		}
		else
		{
			// SUBSTRATE_TODO copy to fully simplified the first input?
		}

		return Module->IsValid();
	}

	// Builds one expression from the top of the stack, or defers if its dependencies aren't ready.
	void BuildTopMaterialExpression()
	{
		static const FName OnDemandInputRequestName(TEXT("MIR_OnDemandInputRequest"));

		const auto& Top = ExpressionStack.Last();
		Emitter.Expression = Top.Key;
		CurrentContext = Top.Value;

		// If expression is already built in this context, nothing to be done.
		if (BuiltExpressions.Contains( { Emitter.Expression, CurrentContext }))
		{
			ExpressionStack.Pop(EAllowShrinking::No);
			return;
		}

		// Set the current expression to the emitter
		Emitter.State = MIR::FEmitter::EState::None;
		Emitter.bExpressionRequestsInputsOnDemand = Emitter.Expression->GetClass()->HasMetaData(OnDemandInputRequestName);

		// Auto-request all inputs unless the expression opts out via MIR_OnDemandInputRequest meta flag.
		if (!Emitter.bExpressionRequestsInputsOnDemand)
		{
			for (FExpressionInputIterator It { Emitter.Expression }; It; ++It)
			{
				RequestInputValue(It.Input);
			}

			// If top expression on the stack changed, some new dependency was pushed. Defer.
			if (ExpressionStack.Last().Key != Emitter.Expression || ExpressionStack.Last().Value != CurrentContext)
			{
				return;
			}
		}

		UpdateSubstrateNodeIdentifierStack();

		// Invoke the expression build function. This will perform semantic analysis, error reporting and
		// emit IR values for its outputs (which will flow into connected expressions inputs).
		Emitter.Expression->Build(Emitter);

		// If the expression has a pending dependency, defer building it.
		if (Emitter.State == MIR::FEmitter::EState::Pending)
		{
			return;
		}

		// Populate the insight information about this expression pins.
		PushExpressionConnectionInsights(Emitter.Expression);

		// Take the top expression out of the stack as ready for analysis. Also mark it as built.
		ExpressionStack.Pop();
		BuiltExpressions.Add( { Emitter.Expression, CurrentContext });
	}

	// Maintains the Substrate node identity chain needed for unique BSDF GUIDs.
	void UpdateSubstrateNodeIdentifierStack()
	{
		// Build the Substrate expressions unique GUID as if we had a SubstrateTreeStack.
		// It needs to be done each time because expression are appended breath first while we need position Top-1 to be the parent
		if (UMaterialExpressionSubstrateBSDF* NewSubstrateExpr = Cast<UMaterialExpressionSubstrateBSDF>(Emitter.Expression))
		{
			Substrate::FSubstrateCompilationContext& SubstrateCtx = SubstrateTranslatorData.SubstrateCompilationContext[Substrate::ESubstrateCompilationContext::SCC_Default];
			SubstrateCtx.SubstrateNodeIdentifierStack.Empty();

			FSubgraphContext* SearchCtx = CurrentContext;

			UMaterialExpression* TrackedNode = NewSubstrateExpr;
			struct FWalkedTree
			{
				UMaterialExpression* Expression = nullptr;
				uint32 InputIndex = 0;
			};
			TArray<FWalkedTree> WalkedNodeStack;
			// Walk the unified expression stack, filtering entries by the current search context.
			// If WalkedNodeStack is empty then we are only dealing with a single BSDF, otherwise we chain all operator from the BSDF to the root
			for (int32 Index = ExpressionStack.Num() - 1; Index >= 0; Index--)
			{
				UMaterialExpression* Expression = ExpressionStack[Index].Key;
				FSubgraphContext* Context = ExpressionStack[Index].Value;
				if (Context != SearchCtx)
				{
					continue;
				}

				if (UMaterialExpressionSubstrateHorizontalMixing* SubstrateHorizontalExpr = Cast<UMaterialExpressionSubstrateHorizontalMixing>(Expression))
				{
					if (SubstrateHorizontalExpr->Background.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateHorizontalExpr , 0 });
						TrackedNode = SubstrateHorizontalExpr;
					}
					else if (SubstrateHorizontalExpr->Foreground.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateHorizontalExpr , 1 });
						TrackedNode = SubstrateHorizontalExpr;
					}
					// Else, cannot error since it could be another mixing from another branch we never walk from there
				}
				else if (UMaterialExpressionSubstrateConvertToDecal* SubstrateDecalExpr = Cast<UMaterialExpressionSubstrateConvertToDecal>(Expression))
				{
					if (SubstrateDecalExpr->DecalMaterial.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateDecalExpr , 0 });
						TrackedNode = SubstrateDecalExpr;
					}
				}
				else if (UMaterialExpressionSubstrateVerticalLayering* SubstrateVerticalExpr = Cast<UMaterialExpressionSubstrateVerticalLayering>(Expression))
				{
					if (SubstrateVerticalExpr->Top.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateVerticalExpr , 0 });
						TrackedNode = SubstrateVerticalExpr;
					}
					else if (SubstrateVerticalExpr->Base.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateVerticalExpr , 1 });
						TrackedNode = SubstrateVerticalExpr;
					}
				}
				else if (UMaterialExpressionSubstrateAdd* SubstrateAddExpr = Cast<UMaterialExpressionSubstrateAdd>(Expression))
				{
					if (SubstrateAddExpr->A.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateAddExpr , 0 });
						TrackedNode = SubstrateAddExpr;
					}
					else if (SubstrateAddExpr->B.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateAddExpr , 1 });
						TrackedNode = SubstrateAddExpr;
					}
				}
				else if (UMaterialExpressionSubstrateWeight* SubstrateWeightExpr = Cast<UMaterialExpressionSubstrateWeight>(Expression))
				{
					if (SubstrateWeightExpr->A.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateWeightExpr , 0 });
						TrackedNode = SubstrateWeightExpr;
					}
				}
				else if (UMaterialExpressionSubstrateSelect* SubstrateSelectExpr = Cast<UMaterialExpressionSubstrateSelect>(Expression))
				{
					if (SubstrateSelectExpr->A.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateSelectExpr , 0 });
						TrackedNode = SubstrateSelectExpr;
					}
					else if (SubstrateSelectExpr->B.GetTracedInput().Expression == TrackedNode)
					{
						WalkedNodeStack.Push( { SubstrateSelectExpr , 1 });
						TrackedNode = SubstrateSelectExpr;
					}
				}
				else if (UMaterialExpressionFunctionOutput* FunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression))
				{
					// We need to select another context
					if (SearchCtx->SubgraphExpression == nullptr)
					{
						break; // we are done
					}
					else
					{
						// Update the tracked node to the subgraph expression and search in the parent context
						TrackedNode = SearchCtx->SubgraphExpression;
						SearchCtx = SearchCtx->Parent;
					}
				}
			}

			// Now update the stack on the context
			SubstrateCtx.SubstrateNodeIdentifierStack.Push(FGuid(0x7AEE, 0xBAD, 0xDEAD, 0xBEEF));
			for (int32 Index = WalkedNodeStack.Num() - 1; Index >= 0; Index--)
			{
				FWalkedTree& SubstrateExpr = WalkedNodeStack[Index];

				// Create an md5 hash for the parent, its input pin index and current node to represent the path.
				uint32 IntputHashBuffer[9];
				FGuid PreviousNodeGuid = SubstrateCtx.SubstrateNodeIdentifierStack.Top();
				FGuid NodeGuid = SubstrateExpr.Expression->MaterialExpressionGuid;
				IntputHashBuffer[0] = PreviousNodeGuid.A;
				IntputHashBuffer[1] = PreviousNodeGuid.B;
				IntputHashBuffer[2] = PreviousNodeGuid.C;
				IntputHashBuffer[3] = PreviousNodeGuid.D;
				IntputHashBuffer[4] = SubstrateExpr.InputIndex;
				IntputHashBuffer[5] = NodeGuid.A;
				IntputHashBuffer[6] = NodeGuid.B;
				IntputHashBuffer[7] = NodeGuid.C;
				IntputHashBuffer[8] = NodeGuid.D;

				uint32 OutputHashBuffer[] { 0, 0, 0, 0 };
				FMD5 IdentifierStringHash;
				IdentifierStringHash.Update((uint8*)IntputHashBuffer, sizeof(IntputHashBuffer));
				IdentifierStringHash.Final((uint8*)&OutputHashBuffer);

				SubstrateCtx.SubstrateNodeIdentifierStack.Push(FGuid(OutputHashBuffer[0], OutputHashBuffer[1], OutputHashBuffer[2], OutputHashBuffer[3]));
			}

			// Simply copy for now to the fully simplified as it should be the same. SUBSTRATE_TODO: simplify, move the stack onto the SubstrateTranslatorData instead of context.
			check(Substrate::ESubstrateCompilationContext::SCC_MAX == 2);
			SubstrateTranslatorData.SubstrateCompilationContext[Substrate::ESubstrateCompilationContext::SCC_FullySimplified].SubstrateNodeIdentifierStack = SubstrateTranslatorData.SubstrateCompilationContext[Substrate::ESubstrateCompilationContext::SCC_Default].SubstrateNodeIdentifierStack;
		}
	}
	
	// Resolves the value for a given expression input. If the expression has declared a subgraph
	// scope (via Subgraph), resolves through to the mapped inner expression in the
	// subgraph context. Pushes unbuilt dependencies and returns an empty TOptional (caller should
	// defer). Returns TOptional with the value (possibly nullptr for disconnected inputs) when
	// available.
	TOptional<MIR::FValue*> RequestInputValue(const FExpressionInput* Input)
	{
		if (!Input->Expression)
		{
			return nullptr;
		}

		UMaterialExpression* Expression = Input->Expression;
		FExpressionOutput* Output = Input->GetConnectedOutput();
		FSubgraphContext* Context = CurrentContext;

		// If this expression has a registered subgraph scope, resolve to the mapped inner expression.
		FSubgraphContext* SubgraphCtx;
		if (MIR::Find(SubgraphContexts, { Expression, Context }, SubgraphCtx))
		{
			if (Input->OutputIndex < SubgraphCtx->NumOutputs)
			{
				const MIR::FSubgraphOutputMapping& OutputMapping = SubgraphCtx->Outputs[Input->OutputIndex];
				if (OutputMapping.Expression)
				{
					Expression = OutputMapping.Expression;
					Output     = OutputMapping.Expression->GetOutput(OutputMapping.OutputIndex);
					Context    = SubgraphCtx;
				}
			}
			else
			{
				// Subgraph scope exists but output index is out of range, stale connection.
				Emitter.Errorf(TEXT("Subgraph expression '%s': invalid output index %d."), *Expression->GetName(), Input->OutputIndex);
			}
		}

		// If the dependency hasn't been built yet, push it and defer.
		if (!BuiltExpressions.Contains({Expression, Context}))
		{
			ExpressionStack.Push({Expression, Context});
			return {};
		}

		// Read the value directly from the producing expression's output.
		MIR::FValue** ValuePtr = OutputValues.Find({Output, Context});
		return ValuePtr ? *ValuePtr : nullptr;
	}
	
	// Registers the current expression as a subgraph with its own scope.
	void DeclareSubgraph(int32 NumInputs, TConstArrayView<MIR::FSubgraphOutputMapping> Outputs)
	{
		FSubgraphContext* Ctx = FSubgraphContext::Create(ContextAllocator, Emitter.Expression, CurrentContext, NumInputs, Outputs);
		SubgraphContexts.Add({Emitter.Expression, CurrentContext}, Ctx);
	}

	// Fetches the value from the subgraph expression's input pin at the given index, resolving in the parent scope.
	TOptional<MIR::FValue*> RequestSubgraphInputValue(int32 InputIndex)
	{
		FSubgraphContext* SubgraphCtx = CurrentContext;
		if (!SubgraphCtx->SubgraphExpression)
		{
			return nullptr; // Not inside a subgraph scope.
		}

		const FExpressionInput* SubgraphInput = SubgraphCtx->SubgraphExpression->GetInput(InputIndex);
		if (!SubgraphInput)
		{
			return nullptr; // Invalid index.
		}

		// Request the value in the parent scope.
		CurrentContext = SubgraphCtx->Parent;
		TOptional<MIR::FValue*> Result = RequestInputValue(SubgraphInput);
		CurrentContext = SubgraphCtx;

		// Push a parent-side connection insight so the edge into the subgraph expression gets colored.
		// Only for top-level subgraphs (parent is root), and only once per input (tracked via bitfield).
		if (SubgraphCtx->Parent == RootContext && Result.IsSet() && Result.GetValue() && SubgraphInput->Expression)
		{
			// Test and set the bit for this input to avoid pushing duplicate insights
			// when multiple inner expressions consume the same subgraph input.
			uint64& BitFieldPage = SubgraphCtx->InputInsightsBitField[InputIndex / 64];
			int32 BitFieldPageOffset = InputIndex % 64;
			if ((BitFieldPage & (1ull << BitFieldPageOffset)) == 0)
			{
				BitFieldPage |= (1ull << BitFieldPageOffset);
				PushConnectionInsight(SubgraphCtx->SubgraphExpression, InputIndex, SubgraphInput->Expression, SubgraphInput->OutputIndex, Result.GetValue()->Type);
			}
		}

		return Result;
	}

	// Pushes the given expression onto the build stack if it hasn't been built yet.
	// Returns true if already resolved (null or already built), false if pushed.
	bool PushDependencyOnStack(UMaterialExpression* Expression)
	{
		if (Expression && !BuiltExpressions.Contains({Expression, CurrentContext}))
		{
			ExpressionStack.Push({Expression, CurrentContext});
			return false;
		}
		return true;
	}

	// Bridges the expression graph to the shader, assigns built values to material output properties.
	bool Step_EmitSetMaterialPropertyInstructions()
	{
		// First, if the material is flagged to use the material attributes aggregate, read its value now
		// so that we can extract its individual attributes later.
		MIR::FValue* MaterialAttributesValue = nullptr;
		if (Emitter.BaseMaterial->bUseMaterialAttributes)
		{
			FMaterialInputDescription InputDesc;
			GetExpressionInputDescription(MP_MaterialAttributes, InputDesc);

			// Fetch the value from the material attributes input.
			MaterialAttributesValue = RequestInputValue(InputDesc.Input).Get(nullptr);
			if (!MaterialAttributesValue)
			{
				MaterialAttributesValue = DefaultMaterialAggregate;
			}

			// Make sure a valid value is present and it is of the correct type.
			check(!MaterialAttributesValue->IsPoison());
			
			if (!MaterialAttributesValue->Type.AsAggregate() || MaterialAttributesValue->Type.AsAggregate() != MaterialAttributesAggregate::Get())
			{
				Module->AddError(nullptr, FString::Printf(TEXT("Expression connected to the MaterialAttributes material output does is not a MaterialAttributes value (it is a '%s' instead)."), *MaterialAttributesValue->Type.GetSpelling()));
			}

			// Push insight for the MaterialAttributes connection itself so the edge colorizes.
			PushConnectionInsight(Builder->MaterialInterface->GetMaterial(), (int32)MP_MaterialAttributes, InputDesc.Input->Expression, InputDesc.Input->OutputIndex, MaterialAttributesValue->Type);
		}

		for (EMaterialProperty Property : MaterialAttributesAggregate::GetMaterialProperties())
		{
			// Skip CustomizedUV slots that aren't enabled as there is no output to emit for these.
			if (IsMaterialPropertyCustomizedUV(Property) && Property >= MP_CustomizedUVs0 + Emitter.BaseMaterial->NumCustomizedUVs)
			{
				continue;
			}

			// Get the input description of this material property (input, type, default value, etc).
			FMaterialInputDescription InputDesc;
			GetExpressionInputDescription(Property, InputDesc);

			// This holds the value being set to this property.
			MIR::FValue* PropertyValue = nullptr;

			// Only evaluate expressions for properties that are active for the current blend mode.
			// Inactive properties will skip expression evaluation and leave PropertyValue as nullptr. Then we fall through to the default value path below. 
			// SetMaterialOutput is still emitted so the shader variable is always initialized.
			bool const bIsPropertyActive = (Property == MP_ShadingModel) || Emitter.MaterialInterface->IsPropertyActive(Property);
			if (bIsPropertyActive)
			{
				// If the material attributes value is valid, extract this property attribute from the material attributes aggregate value
				// and manually flow it into the this property material expression input pin.
				if (MaterialAttributesValue)
				{
					PropertyValue = Emitter.Subscript(MaterialAttributesValue, MaterialAttributesAggregate::MaterialPropertyToAttributeIndex(Property));
				}
				else
				{
					// Otherwise grab the value from the individual attribute pin.
					PropertyValue = RequestInputValue(InputDesc.Input).Get(nullptr);
				}
			}
			
			// Determine the value to assign to this material attribute "SetMaterialOutput" instruction
			if (Property == MP_ShadingModel && !Emitter.MaterialInterface->IsShadingModelFromMaterialExpression())
			{
				// Don't use shading model coming from expressions if the material states a shading model explicitly, intead of setting it "FromMaterialExpression".
				PropertyValue = Emitter.ConstantInt(Emitter.MaterialInterface->GetShadingModels().GetFirstShadingModel());
			}
			else if (PropertyValue)
			{
				// Case when the value to assigne to the material attribute is given.

				// If this property is the emissive color and we're previewing the material, apply gamma correction to the previewed value.
				if (Property == MP_EmissiveColor && PreviewInput.IsConnected())
				{
					MIR::FValue* Zero = Emitter.ConstantZero(MIR::EScalarKind::Float);

					// Get preview expression back into gamma corrected space, as DrawTile does not do this adjustment.
					PropertyValue = Emitter.Pow(Emitter.Max(PropertyValue, Zero), Emitter.ConstantFloat(1.0f / 2.2f));

					// Preview should display scalars as red, so if this is a scalar, create a vector padded with zeroes
					if (PropertyValue->Type.IsScalar())
					{
						PropertyValue = Emitter.Vector3(PropertyValue, Zero, Zero);
					}
				}
				else if (Property == MP_FrontMaterial)
				{
					// Just check that we get a struct for now.
					// SUBSTRATE_TODO We should probably add a proper SubstrateData struct enum?
					if (InputDesc.Type != UE::Shader::EValueType::Struct)
					{
						Emitter.Error(TEXT("FrontMaterial must be fed with a Struct"));
					}
				}
				else
				{
					// If a value is flowing in through the connection, cast it to this material attribute type and assign it.
					MIR::FType OutputArgType = MIR::FType::FromShaderType(InputDesc.Type);
					PropertyValue = Emitter.Cast(PropertyValue, OutputArgType);
				}
			}
			else if (IsMaterialPropertyCustomizedUV(Property))
			{
				// To keep backwards compatibility with the old translator, a CustomUV attribute pin with no given value
				// does not default to (0,0) (as it should) but to the uncustomized texture coordinate. If we find a customized UV
				// attribute with no value, simply don't emit it entirely. This will default to the default texture coordinates.
				continue;
			}
			else if (Property == MP_FrontMaterial && PropertyValue == nullptr && SubstrateTranslatorData.bMaterialUsesRootNodeToSubstrateHiddenConversion)
			{
				// SUBSTRATE_TODO Explain this case
				PropertyValue = BuildSubstrateFrontMaterialIRFromRootNodeHiddenConversion();
			}
			else if (Property != MP_Normal && InputDesc.bUseConstant)
			{
				// If non-Normal input is marked to use constant, assign this output to the specified constant value.
				// The Normal attribute should never use the constant as they have special default values.
				PropertyValue = Emitter.ConstantFromShaderValue(InputDesc.ConstantValue);
			}
			else
			{
				// Otherwise, fallback to assigning this material output to its default value.
				PropertyValue = Emitter.Subscript(DefaultMaterialAggregate, MaterialAttributesAggregate::MaterialPropertyToAttributeIndex(Property));
			}
			
			// Quit if some error occurred in the operations above.
			if (!Module->IsValid())
			{
				return false;
			}

			// The value being set to this material output is now valid.
			check(PropertyValue);

			// Add support for lerp to selection color for PC development builds.
			if (Property == MP_EmissiveColor &&
				Builder->MaterialInterface->GetMaterial()->MaterialDomain != MD_Volume &&
				UE::MaterialTranslatorUtils::IsDevelopmentFeatureEnabled(NAME_SelectionColor, Module->GetShaderPlatform(), Builder->MaterialInterface->GetMaterial()))
			{
				MIR::FValue* SelectionColor = Emitter.NamedPrimitiveUniform(NAME_SelectionColor, Emitter.ConstantFloat4(UE::Math::TVector4<float>{ 0.f, 0.f, 0.f, 0.f }));
				PropertyValue = Emitter.Lerp(PropertyValue, Emitter.Swizzle(SelectionColor, MIR::FSwizzleMask::XYZ()), Emitter.Subscript(SelectionColor, 3));
			}
			// Append refraction depth bias to refraction for translucent and single layer water materials
			else if (Property == MP_Refraction && (IsTranslucentBlendMode(Builder->BlendMode) || Builder->MaterialInterface->GetShadingModels().HasShadingModel(MSM_SingleLayerWater)))
			{
				MIR::FValueRef RefractionDepthBias = Emitter.NamedPrimitiveUniform(FName(TEXT("RefractionDepthBias")), Emitter.ConstantFloat(Emitter.BaseMaterial->RefractionDepthBias));
				PropertyValue = Emitter.Vector3(Emitter.CastToFloat(PropertyValue, 2), RefractionDepthBias);
			}
			// Pack SubsurfaceColor (float3) with SubsurfaceProfile ID (float1) into float4,
			// matching the old translator's behavior (HLSLMaterialTranslator.cpp:1623-1636).
			else if (Property == MP_SubsurfaceColor)
			{
				MIR::FValueRef ColorRGB = Emitter.Swizzle(PropertyValue, MIR::FSwizzleMask::XYZ());
				MIR::FValueRef ProfileParam = Emitter.CastToFloat(Emitter.NamedPrimitiveUniform(SubsurfaceProfile::GetSubsurfaceProfileParameterName(), Emitter.ConstantFloat(1.0f)), 1);
				PropertyValue = Emitter.Vector4(ColorRGB, ProfileParam);
			}
			
			// Quit if some error occurred in the operations above.
			if (!Module->IsValid())
			{
				return false;
			}

			// Emit the SetMaterialProperty instruction
			Emitter.Private_SetMaterialOutput(Property, PropertyValue);

			// Finally, push this connection insight
			PushConnectionInsight(Builder->MaterialInterface->GetMaterial(), (int32)Property, InputDesc.Input->Expression, InputDesc.Input->OutputIndex, PropertyValue->Type);
		}

		return Module->IsValid();
	}

	// Synthesizes a Substrate slab for non-Substrate materials by converting legacy root properties.
	MIR::FValue* BuildSubstrateFrontMaterialIRFromRootNodeHiddenConversion()
	{
		MIR::FValue* FrontMaterialPropertyValue = Emitter.SubstrateDefaultSlab();

		{
			UMaterialInterface* MaterialInterface = SubstrateTranslatorData.GetMaterialInterface();
			UMaterial* BaseMaterial = MaterialInterface ? MaterialInterface->GetMaterial() : nullptr;
			UMaterialEditorOnlyData* EditorOnlyData = BaseMaterial ? BaseMaterial->GetEditorOnlyData() : nullptr;

			if (BaseMaterial && BaseMaterial->bUseMaterialAttributes)
			{
				check(false); // SUBSTRATE_TODO
				return FrontMaterialPropertyValue;
				//	// Need to root subsurface profile for the uniform scalar numeric parameter name to be correct.
				//	USubsurfaceProfile* SSSProfile = MaterialInterface ? MaterialInterface->GetSubsurfaceProfileRoot_Internal() : nullptr;
				//	const bool bHasSSS = SSSProfile != nullptr;
				//
				//	return UMaterialExpressionSubstrateConvertMaterialAttributes::CompileCommon(this, 0,
				//		FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(BaseMaterial->GetExpressions()), EditorOnlyData->MaterialAttributes, MaterialInterface->GetShadingModels().GetFirstShadingModel(),
				//		*SubstrateTranslatorData.GetWaterScatteringCoefficients(), *SubstrateTranslatorData.GetWaterAbsorptionCoefficients(), *SubstrateTranslatorData.GetWaterPhaseG(), *SubstrateTranslatorData.GetColorScaleBehindWater(),
				//		bHasSSS, SSSProfile,
				//		*SubstrateTranslatorData.GetClearCoatNormal(), *SubstrateTranslatorData.GetCustomTangent());
			}
			else if (BaseMaterial && (BaseMaterial->MaterialDomain == MD_Surface || BaseMaterial->MaterialDomain == MD_RuntimeVirtualTexture || BaseMaterial->MaterialDomain == MD_DeferredDecal) && EditorOnlyData)
			{
				USubsurfaceProfile* SSSProfile = MaterialInterface ? MaterialInterface->GetSubsurfaceProfileRoot_Internal() : nullptr;
				const bool bHasSSS = SSSProfile != nullptr;
				const bool bHasAnisotropy = EditorOnlyData->Anisotropy.IsConnected();

				FrontMaterialPropertyValue = UMaterialExpressionSubstrateShadingModels::BuildCommon(Emitter, Emitter.BaseMaterial,
					EditorOnlyData->BaseColor, EditorOnlyData->Specular, EditorOnlyData->Metallic, EditorOnlyData->Roughness, EditorOnlyData->EmissiveColor,
					EditorOnlyData->Opacity, EditorOnlyData->SubsurfaceColor, EditorOnlyData->ClearCoat, EditorOnlyData->ClearCoatRoughness,
					EditorOnlyData->ShadingModelFromMaterialExpression, MaterialInterface->GetShadingModels().GetFirstShadingModel(),
					*SubstrateTranslatorData.GetThinTranslucentTransmittanceColor(), *SubstrateTranslatorData.GetThinTranslucentSurfaceCoverage(),
					*SubstrateTranslatorData.GetWaterScatteringCoefficients(), *SubstrateTranslatorData.GetWaterAbsorptionCoefficients(), *SubstrateTranslatorData.GetWaterPhaseG(), *SubstrateTranslatorData.GetColorScaleBehindWater(),
					bHasAnisotropy, EditorOnlyData->Anisotropy,
					EditorOnlyData->Normal, EditorOnlyData->Tangent,
					*SubstrateTranslatorData.GetClearCoatNormal(), *SubstrateTranslatorData.GetCustomTangent(),
					bHasSSS, SSSProfile,
					EditorOnlyData);
			}
			else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_Volume && EditorOnlyData)
			{
				check(false); // SUBSTRATE_TODO
				return FrontMaterialPropertyValue;
				//	return UMaterialExpressionSubstrateVolumetricFogCloudBSDF::CompileCommon(
				//		this,
				//		EditorOnlyData->BaseColor,
				//		EditorOnlyData->SubsurfaceColor,
				//		EditorOnlyData->EmissiveColor,
				//		EditorOnlyData->AmbientOcclusion,
				//		MaterialInterface->GetShadingModels().HasOnlyShadingModel(MSM_Unlit),
				//		EditorOnlyData);
			}
			else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_LightFunction && EditorOnlyData)
			{
				check(false); // SUBSTRATE_TODO
				return FrontMaterialPropertyValue;
				//	return UMaterialExpressionSubstrateLightFunction::CompileCommon(this, EditorOnlyData->EmissiveColor, EditorOnlyData);
			}
			else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_PostProcess && EditorOnlyData)
			{
				check(false); // SUBSTRATE_TODO
				return FrontMaterialPropertyValue;
				//	return UMaterialExpressionSubstratePostProcess::CompileCommon(this, EditorOnlyData->EmissiveColor, EditorOnlyData->Opacity, EditorOnlyData);
			}
			else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_UI && EditorOnlyData)
			{
				check(false); // SUBSTRATE_TODO
				return FrontMaterialPropertyValue;
				//	return UMaterialExpressionSubstrateUI::CompileCommon(this, EditorOnlyData->EmissiveColor, EditorOnlyData->Opacity, EditorOnlyData);
			}
			else
			{
				Emitter.Error(TEXT("Substrate could not compile the material."));
				return FrontMaterialPropertyValue;
			}


			// When we are running hidden conversion (bMaterialUsesRootNodeToSubstrateHiddenConversion),
			// We know that we will only ever have a single BSDF and shared local bases. So simply use the same non-fully-simplified as the fully-simplified version of the material too.
			MIR::FValueRef PropertyValueRef = MIR::FValueRef(FrontMaterialPropertyValue);
			Emitter.SetCustomOutputs(TEXTVIEW("SubstrateFrontMaterialFullySimplified"), { &PropertyValueRef, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
			Emitter.SetCustomOutputs(TEXTVIEW("SubstrateFrontMaterial"), { &PropertyValueRef, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
			// We also copy the copy the shared local bases structure from within GeneratePixelInputInitializerHLSLCode.
		}

		return FrontMaterialPropertyValue;
	}

	// Records Substrate tree metadata so the shader compiler knows what BSDFs and features are in play.
	void GenerateSubstrateMaterialCompilationOutputAndDefines()
	{
		// Write substrate tree specific code.
		if (SubstrateTranslatorData.bSubstrateEnabled)
		{
			const bool bSubstrateFrontMaterialProvided = Module->IsMaterialPropertyUsed(MP_FrontMaterial);
			const bool bValid = SubstrateTranslatorData.GenerateSubstrateTreeHLSLCode(bSubstrateFrontMaterialProvided);
			Module->SubstrateData.TreeHLSL = SubstrateTranslatorData.SubstrateTreeHLSL;

			// TODO add that back
			// if (!bValid)
			// {
			// 	TranslationResult = EHLSLMaterialTranslatorResult::Failure;
			// 	return;
			// }
		}

		// SUBSTRATE_TODO export context
		ESubstrateMaterialExport SubstrateMaterialExportType = ESubstrateMaterialExport::SME_None;
		ESubstrateMaterialExportContext SubstrateMaterialExportContext = ESubstrateMaterialExportContext::SMEC_Opaque;
		uint8 SubstrateMaterialExportLegacyBlendMode = BLEND_Opaque;

		const bool bOpacityPropertyIsUsed = Module->IsMaterialPropertyUsed(MP_Opacity);
		FString DescriptionStringCommentForDebug;
		Substrate::FSubstrateDefines OutSubstrateDefines;
		SubstrateTranslatorData.GenerateMaterialCompilationOutputAndDefines(
			bOpacityPropertyIsUsed,
			(int32)SubstrateMaterialExportType, (int32)SubstrateMaterialExportContext, (int32)SubstrateMaterialExportLegacyBlendMode,
			DescriptionStringCommentForDebug,
			&OutSubstrateDefines);


		if (SubstrateTranslatorData.bSubstrateEnabled)
		{
			FString HLSLSubstrateTypeString = TEXT("FSubstrateData");	// TODO we would need to abstract that away only when lowering the IR
			SubstrateTranslatorData.GeneratePixelMemberDeclarationHLSLCode(Module->SubstrateData.PixelMembersDeclarationHLSL, HLSLSubstrateTypeString);
			SubstrateTranslatorData.GeneratePixelInputInitializerHLSLCode(Module->SubstrateData.PixelInputInitializerValuesHLSL);
			SubstrateTranslatorData.GeneratePixelNormalInitializerHLSLCode(Module->SubstrateData.PixelInputInitializerValuesHLSL);
		}

		// Copie Substrate defines
		// SUBSTRATE_TODO: have SubstrateTranslatorData and both translator use the same define format to just add to the existing array one
		for (auto& Define : OutSubstrateDefines)
		{
			Module->AddIntegerEnvironmentDefine(*Define.Key, Define.Value);
		}
	}

	// Maps a material property to its source expression input, honoring preview expression overrides.
	bool GetExpressionInputDescription(EMaterialProperty Property, FMaterialInputDescription& Input)
	{
		if (PreviewInput.IsConnected() && Property == MP_EmissiveColor)
		{
			Input.Type = UE::Shader::EValueType::Float3;
			Input.Input = &PreviewInput;
			return true;
		}
		else
		{
			bool bResult = Builder->MaterialInterface->GetMaterial()->GetExpressionInputDescription(Property, Input);
			/**
			* MP_SubsurfaceColor is currently hacked in the old translator to float4, 
			* but we rely on default types (i.e.float3) for default values in the material editor output.
			* 
			* This hack resolves the default value to use float4 rather than float3 until we can 
			* implement a permanent float4 alternative method in the new translator 
			* (i.e. work towards deprecating the MP_SubsurfaceColor hacks scattered throughout UE).
			*/
			if (Property == MP_SubsurfaceColor)
			{
				Input.Type = UE::Shader::EValueType::Float4;
				Input.ConstantValue = UE::Shader::FValue(Input.ConstantValue.AsLinearColor());
			}

			return bResult;
		}
	}

	// Semantic pass, propagates liveness, stage assignment, and side-effect info through the IR graph.
	bool Step_AnalyzeIRGraph()
	{
		TArray<MIR::FValue*> ValueStack{};
		TSet<MIR::FValue*> VisitedValues{};

		// The analysis of several vertex values (e.g. interpolators, customized UVs) depend on other non-vertex values
		// values analysis results. For this reason, analyze stages in reverse order of execution (compute/pixel stages first,
		// vertex stage last). For example, the entire subgraph evaluating a CustomizedUV can be optimized out if it is found
		// that during pixel/compute stages that relative texture coordinate isn't actually used.
		MIR::TTemporaryArray<int32> SortedEntryPointIndices { Module->GetNumEntryPoints() };
		for (int32 i = 0; i < Module->GetNumEntryPoints(); ++i)
		{
			SortedEntryPointIndices[i] = i;
		}
		SortedEntryPointIndices.StableSort([this](int32 A, int32 B)
		{
			return Module->GetEntryPoint(A).Stage > Module->GetEntryPoint(B).Stage;
		});

		// Analyze all the nodes in each entry point starting from their root outputs and walking backwards along the Use edges.
		for (int32 EntryPointIndex : SortedEntryPointIndices)
		{
			// Reset bookkeeping to process new output subgraph
			ValueStack.Empty(ValueStack.Max());
			VisitedValues.Empty(Module->Values.Num());

			FMaterialIRModule::FEntryPoint& EntryPoint = Module->GetEntryPoint(EntryPointIndex);

			// If the analysis of previous stages resulted in some outputs in this stage to be actually unnecessary, remove
			// these optimized out root outputs now.
			RemoveDisabledEntryPointsOutputs(EntryPoint);

			// Stable sort the outputs in this entry point by their indicated priority, then name.
			// Note: this is needed to support situations where a final entry-point instruction needs to execute
			//    before another, for instance we want to set the Normal attribute before all others, because the
			//    others may read back the data. This shouldn't really be done this way, but it is currently a
			//    restriction of how the material shader works traditionally.
			Algo::StableSort(EntryPoint.Outputs, [](const MIR::FInstruction* A, const MIR::FInstruction* B)
			{
				int32 PriorityA = A->GetOutputPriority();
				int32 PriorityB = B->GetOutputPriority();
				return PriorityA != PriorityB ? PriorityA < PriorityB : A->GetName() < B->GetName();
			});

			// Push this SetOutput instruction the value stack for processing.
			for (MIR::FValue* Output : EntryPoint.Outputs)
			{
				ValueStack.Push(Output);
			}

			// Process until the value stack is empty.
			while (!ValueStack.IsEmpty())
			{
				MIR::FValue* Value = ValueStack.Last();
				
				// Module building should have interrupted before if poison values were generated.
				check(!Value->IsPoison());

				// If this instruction has already been analyzed for this entry point, nothing else is left to do for it. Continue.
				if (VisitedValues.Contains(Value))
				{
					ValueStack.Pop();
					continue;
				}

				// Before analyzing this value, make sure all used values are analyzed first.
				for (MIR::FValue* Use : Value->GetUsesForStage(EntryPoint.Stage))
				{
					if (Use && !VisitedValues.Contains(Use))
					{
						ValueStack.Push(Use);
					}
				}

				// If any other value has been pushed to the stack, it means we have a dependency to analyze first.
				if (ValueStack.Last() != Value)
				{
					continue;
				}

				// All dependencies of this value has been analyzed, we can proceed analyzing this value now.
				ValueStack.Pop();

				// Go through each use instruction and increment its counter of users (this instruction).
				for (MIR::FValue* Use : Value->GetUsesForStage(EntryPoint.Stage))
				{
					// If this used value is an instruction, update its counter of users (in current stage).
					if (MIR::FInstruction* UseInstr = MIR::AsInstruction(Use))
					{
						UseInstr->Linkage[EntryPointIndex].NumUsers += 1;
					}
				}

				// If this is the first time this value is analyzed in any entry point, let the analyzer process it.
				// Note that individual value processing is independent from the stage it runs on so we can perform it only once.
				if ((Value->Flags & MIR::EValueFlags::AnalyzedInAllStagesMask) == MIR::EValueFlags::None)
				{
					// Allocate the entry points linkage information for this instruction.
					if (MIR::FInstruction* Instr = MIR::AsInstruction(Value))
					{
						Instr->Linkage = Module->AllocateArray<MIR::FInstructionLinkage>(Module->GetNumEntryPoints());
						MIR::ZeroArray(Instr->Linkage);
					}

					// Then analyze the instruction based on its kind.
					ValueAnalyzer.Analyze(Value);
				}

				// Analyze this instruction in this entry point's stage if it's the first time it's encountered.
				MIR::EValueFlags StageFlag = MIR::EValueFlags(1 << (int32)EntryPoint.Stage);
				if (!Value->HasFlags(StageFlag))
				{
					Value->SetFlags(StageFlag);
					ValueAnalyzer.AnalyzeInStage(Value, EntryPoint.Stage);
				}

				// Mark the used instruction as analyzed for this entry point.
				VisitedValues.Add(Value);
			}
		}

		return Module->IsValid();
	}

	// Remove instructions from the given entry point outputs that, due to semantic analysis of previous other, are found not to contribute to the end result.
	void RemoveDisabledEntryPointsOutputs(FMaterialIRModule::FEntryPoint& EntryPoint)
	{
		if (EntryPoint.Stage == MIR::Stage_Vertex)
		{
			EntryPoint.Outputs.RemoveAllSwap([this](const MIR::FInstruction* Instr)
			{
				const MIR::FSetMaterialOutput* SetMaterialOutput = MIR::As<MIR::FSetMaterialOutput>(Instr);
				return SetMaterialOutput
					&& IsMaterialPropertyCustomizedUV(SetMaterialOutput->Property)
					&& (Module->GetStatistics().InterpolatedTexCoordsMask & (1 << (SetMaterialOutput->Property - MP_CustomizedUVs0))) == 0;
			});
		}
	}

	// Post-analysis fixups that require the full IR graph to be finalized (buffer layout, preshader codegen).
	bool Step_PerformModuleWideAnalysis()
	{
		// After the entire tree has been analyzed, we can run fixup, which allocates output buffer offsets and fixes up
		// parameter evaluations and preshader bytecode with those.
		UE::Shader::FPreshaderData& PreshaderData = Module->GetCompilationOutput().UniformExpressionSet.GetPreshaderData();
		Preshader.PreshaderFixup(PreshaderData, Module, ValueAnalyzer.Insights);

		// If the material doesn't use expression shading models, or they aren't valid, initialize the shading models to the ones from the material.
		if (!Builder->MaterialInterface->IsShadingModelFromMaterialExpression() || !Module->ShadingModelsFromCompilation.IsValid())
		{
			Module->ShadingModelsFromCompilation = Builder->MaterialInterface->GetShadingModels();
			UMaterialInterface::FilterOutPlatformShadingModels(Module->GetShaderPlatform(), Module->ShadingModelsFromCompilation);
		}

		// Check the number of post process input is not too large
		int32 NumPostProcessInputs = Module->CompilationOutput.GetNumPostProcessInputsUsed();
		if (NumPostProcessInputs > kPostProcessMaterialInputCountMax)
		{
			Module->AddError(nullptr, FString::Printf(TEXT("Maximum Scene Texture post process inputs exceeded (%d > %d), between SceneTexture nodes with PostProcessInputs or UserSceneTexture nodes."), NumPostProcessInputs, kPostProcessMaterialInputCountMax));
		}

		// Generate specific code and define.
		GenerateSubstrateMaterialCompilationOutputAndDefines();

		// Consolidate and validate environment defines
		ConsolidateEnvironmentDefines();
		ValidateEnvironmentDefines();

		// Complete the setup of the MaterialCompilationOutput instance
		FinishSetupMaterialCompilationOutput();

		return Module->IsValid();
	}

	// Prunes environment defines that don't apply to this material's actual configuration.
	void ConsolidateEnvironmentDefines()
	{
		// Keep defines if a combined condition is met. Otherwise, remove them from the environemnt defines set.
		auto KeepDefineConditionally = [this](FName Name, bool bConditionToKeepDefine) -> void
		{
			if (!bConditionToKeepDefine)
			{
				ValueAnalyzer.EnvironmentDefines.Remove(Name);
			}
		};

		// todo: We want the usage on the MaterialInstance if that exists. 
		// See FMaterialResource::IsUsedWithInstancedStaticMeshes() which is used at the same point in the old translator.
		// This is one case of several where the old translator works by having a FMaterial instead of just the UMaterial.
		const bool bUsedWithInstancedStaticMeshes = Builder->MaterialInterface->GetUsageByFlag(MATUSAGE_InstancedStaticMeshes);

		KeepDefineConditionally(TEXT("USES_PER_INSTANCE_CUSTOM_DATA"), bUsedWithInstancedStaticMeshes);
		KeepDefineConditionally(TEXT("NEEDS_PER_INSTANCE_RANDOM_PS"), bUsedWithInstancedStaticMeshes);
		KeepDefineConditionally(TEXT("USES_PER_INSTANCE_FADE_AMOUNT"), bUsedWithInstancedStaticMeshes);

		// Merge defines from analyzer into module (don't discard defines added during AnalyzeInStage)
		for (const FName& Define : ValueAnalyzer.EnvironmentDefines)
		{
			Module->EnvironmentDefines.Add(Define);
		}
	}

	// Catches environment defines that conflict with the material's blend mode or feature set.
	void ValidateEnvironmentDefines()
	{
		// Match various defines against the material configuration
		if (Module->EnvironmentDefines.Contains(TEXT("MIR.SceneDepth")))
		{
			// @todo-jason.hoerner - Support for material instance blend mode overrides needed.
			if (Builder->MaterialInterface->GetMaterial()->MaterialDomain != MD_PostProcess && !IsTranslucentBlendMode(Builder->MaterialInterface->GetMaterial()->BlendMode))
			{
				Module->AddError(nullptr, TEXT("Only transparent or postprocess materials can read from scene depth."));
			}
		}

		// Remove all environment defines that have the "MIR." prefix as they are not meant to propagate into the set of compiler environment defines.
		TCHAR DefineMIRPrefix[5] = {};
		for (auto Iter = Module->EnvironmentDefines.CreateIterator(); Iter; ++Iter)
		{
			Iter->ToStringTruncate(DefineMIRPrefix, UE_ARRAY_COUNT(DefineMIRPrefix));
			if (FCString::Strncmp(DefineMIRPrefix, TEXT("MIR."), UE_ARRAY_COUNT(DefineMIRPrefix)) == 0)
			{
				Iter.RemoveCurrent();
			}
		}
	}
	
	// Fills in the compilation output that downstream systems (shader compiler, renderer) depend on.
	void FinishSetupMaterialCompilationOutput()
	{
		FMaterialCompilationOutput& CompilationOutput = Module->CompilationOutput;

		// UniformExpressionSet
		CompilationOutput.UniformExpressionSet.SetParameterCollections(Module->ParameterCollections);

		// Create CompactUniformsVS so the MaterialVS uniform buffer struct gets declared in the shader.
		// VS-frequency preshader outputs were allocated first in PreshaderFixup, so the compact VS buffer
		// can reference the contiguous prefix of the PreshaderBuffer.
		if (UseMaterialVSUniformBuffer())
		{
			FCompactUniformExpressionSet* CompactUniformsVS = CompilationOutput.UniformExpressionSet.CreateCompactUniformsVS();
			check(CompactUniformsVS);
			CompactUniformsVS->UniformPreshaderBufferSize = Preshader.VSPreshaderFloat4s;
		}
		else
		{
			CompilationOutput.UniformExpressionSet.EmptyCompactUniformsVS();
		}

		// NumUsedUVScalars
		CompilationOutput.NumUsedUVScalars = Module->Statistics.NumInterpolatedTexCoords * 2;

		// NumVirtualTextureFeedbackRequests / EstimatedNumVirtualTextureLookups
		int32 NumVirtualTextureFeedbackRequests = 0;
		for (const FMaterialIRValueAnalyzer::FVTStackEntry& VTStack : ValueAnalyzer.VTStacks)
		{
			NumVirtualTextureFeedbackRequests += VTStack.bGenerateFeedback ? 1 : 0;
		}
		CompilationOutput.NumVirtualTextureFeedbackRequests = NumVirtualTextureFeedbackRequests;
		CompilationOutput.EstimatedNumVirtualTextureLookups = ValueAnalyzer.VTStacks.Num();

		// Simple on/off flags depending on whether a non-default value is set to some material property
		CompilationOutput.bUsesWorldPositionOffset  = MaterialPropertyHasNonZeroValue(MP_WorldPositionOffset);
		CompilationOutput.bUsesPixelDepthOffset     = MaterialPropertyHasNonZeroValue(MP_PixelDepthOffset);
		CompilationOutput.bUsesAnisotropy           = MaterialPropertyHasNonZeroValue(MP_Anisotropy);
		CompilationOutput.bUsesOpacityMask          = Module->IsMaterialPropertyUsed(MP_OpacityMask);
		CompilationOutput.bUsesOpacity              = Module->IsMaterialPropertyUsed(MP_Opacity);
		
		// Check if displacement is actually used (disable if it was left at the default, invalid constant value of -1)
		CompilationOutput.bUsesDisplacement 	 |= DoesPlatformSupportNanite(Module->GetShaderPlatform()) && Builder->MaterialInterface->IsTessellationEnabled() && Module->IsMaterialPropertyUsed(MP_Displacement);
		CompilationOutput.bModifiesMeshPosition  |= CompilationOutput.bUsesPixelDepthOffset || CompilationOutput.bUsesWorldPositionOffset || CompilationOutput.bUsesDisplacement;
		
		// Run a final material compilation output validation and report any error reported.
		TArray<FString> ValidationErrors;
		UE::MaterialTranslatorUtils::FinalCompileValidation(
			Emitter.BaseMaterial,
			CompilationOutput,
			Module->GetCompiledShadingModels(),
			Module->GetBlendMode(),
			Module->IsMaterialPropertyUsed(MP_FrontMaterial),
			Module->GetShaderPlatform(),
			ValidationErrors);

		for (const FString& ValidationError : ValidationErrors)
		{
			Module->AddError(nullptr, *ValidationError);
		}
	}

	// Linearizes the IR graph into ordered instruction lists, one per entry point.
	void Step_LinkInstructions()
	{
		TArray<MIR::FInstruction*> InstructionStack{};

		for (int32 EntryPointIndex = 0; EntryPointIndex < Module->GetNumEntryPoints(); ++EntryPointIndex)
		{
			// This function walks the instruction graph and puts each instruction into the inner most possible block.
			InstructionStack.Empty(InstructionStack.Max());

			FMaterialIRModule::FEntryPoint& EntryPoint = Module->GetEntryPoint(EntryPointIndex);

			// Push all entry point final outputs onto the instruction stack to begin.
			// Note: the first output on the stack will be the first to be evaluated in the entry point root block.
			for (MIR::FValue* Output : EntryPoint.Outputs)
			{
				if (Output->Kind != MIR::EValueKind::VK_Nop)
				{
					if (MIR::FInstruction* Instr = MIR::AsInstruction(Output))
					{
						Instr->Linkage[EntryPointIndex].Block = &EntryPoint.RootBlock;
						InstructionStack.Push(Instr);
					}
				}
			}

			while (!InstructionStack.IsEmpty())
			{
				MIR::FInstruction* Instr = InstructionStack.Pop();
				MIR::FBlock* InstrBlock = Instr->Linkage[EntryPointIndex].Block;

				// Push the instruction to its block in reverse order (push front)
				Instr->Linkage[EntryPointIndex].Next = InstrBlock->Instructions;
				InstrBlock->Instructions = Instr;

				// NOP instructions uses are only analyzed but not evaluated. Ignore them.
				if (Instr->As<MIR::FNop>())
				{
					continue;
				}

				TConstArrayView<MIR::FValue*> Uses = Instr->GetUsesForStage(EntryPoint.Stage);
				for (int32 UseIndex = 0; UseIndex < Uses.Num(); ++UseIndex)
				{
					MIR::FInstruction* UseInstr = MIR::AsInstruction(Uses[UseIndex]);
					if (!UseInstr)
					{
						continue;
					}

					// Get the block into which the dependency instruction should go.
					MIR::FBlock* TargetBlock = Instr->GetTargetBlockForUse(EntryPointIndex, UseIndex);

					// Update dependency's block to be a child of current instruction's block.
					if (TargetBlock != InstrBlock)
					{
						TargetBlock->Parent = InstrBlock;
						TargetBlock->Level = InstrBlock->Level + 1;
					}

					// Set the dependency's block to the common block betwen its current block and this one.
					MIR::FInstructionLinkage& UseLinkage = UseInstr->Linkage[EntryPointIndex];

					UseLinkage.Block = UseLinkage.Block
						? UseLinkage.Block->FindCommonParentWith(TargetBlock)
						: TargetBlock;

					// Increase the number of times this dependency instruction has been considered.
					++UseLinkage.NumProcessedUsers;
					check(UseLinkage.NumProcessedUsers <= UseLinkage.NumUsers);

					// If all dependants have been processed, we can carry the processing from this dependency.
					if (UseLinkage.NumProcessedUsers == UseLinkage.NumUsers)
					{
						InstructionStack.Push(UseInstr);
					}
				}
			}
		}
	}

	// Last step, produces debug/editor artifacts from the completed module.
	void Step_FinalizeArtifacts()
	{
		// Finalize the material insights debug information
		GenerateDebugInsights();
	}

	// Produces the debug insight payload consumed by the material editor for diagnostics display.
	void GenerateDebugInsights()
	{
		if (!Builder->TargetInsights)
		{
			return;
		}

		// Dump the module IR to string and store it inside the material insights.
		Builder->TargetInsights->IRString = MIR::DebugDumpIR(Builder->MaterialInterface->GetFullName(), *Module);
		
		// Dump the requested debugging information
		switch (CVarMaterialIRDebugDumpLevel.GetValueOnGameThread())
		{
			case 2:
			{
				MIR::DebugDumpIRUseGraph(*Module);
				// fallthrough
			}

			case 1:
			{
				// Save the dump to file
				FString FilePath = FPaths::Combine(FPaths::ProjectSavedDir(), "Materials", TEXT("IRDump.txt"));
				FFileHelper::SaveStringToFile(Builder->TargetInsights->IRString, *FilePath);
				// fallthrough
			}
		}
	}
	
	/* Auxiliary functions */

	// This function returns whether a given property is active in the translating material.
	// Some properties may be deactivated in the material, such as CustomizedUVx properties, or properties that are inactive for the material's blend mode.
	bool PropertyIsActiveInMaterial(const EMaterialProperty Property)
	{
		if (!Emitter.MaterialInterface->IsPropertyActive(Property))
		{
			return false;
		}
		if (IsMaterialPropertyCustomizedUV(Property) && Property >= MP_CustomizedUVs0 + Emitter.BaseMaterial->NumCustomizedUVs)
		{
			return false;
		}
		return true;
	}

	// Adds an expression connection insight to the MaterialInsights instance, if any.
	void PushExpressionConnectionInsights(UMaterialExpression* Expression)
	{
		if (!Builder->TargetInsights || CurrentContext != RootContext)
		{
			return;
		}

		// Subgraph expressions (e.g. function calls) get their input pin insights from
		// RequestSubgraphInputValue, ignore them here as they haven't been built yet.
		bool bIsExpressionSubgraph = SubgraphContexts.Contains({ Expression, CurrentContext });
		if (bIsExpressionSubgraph)
		{
			return;
		}

		// Go over all expression inputs and emit the connection insights for those that have a value coming in.
		for (FExpressionInputIterator It{ Expression }; It; ++It)
		{
			FExpressionOutput* Output = It.Input->GetConnectedOutput();
			if (!Output)
			{
				continue;
			}

			// Resolve through subgraph output mappings if the connected expression declares a subgraph.
			FSubgraphContext* Context = CurrentContext;
			FSubgraphContext* SubgraphCtx;
			if (MIR::Find(SubgraphContexts, {It->Expression, Context}, SubgraphCtx)
				&& It->OutputIndex < SubgraphCtx->NumOutputs
				&& SubgraphCtx->Outputs[It->OutputIndex].Expression)
			{
				const MIR::FSubgraphOutputMapping& Mapping = SubgraphCtx->Outputs[It->OutputIndex];
				Output  = Mapping.Expression->GetOutput(Mapping.OutputIndex);
				Context = SubgraphCtx;
			}

			MIR::FValue** ValuePtr = OutputValues.Find({Output, Context});
			if (ValuePtr)
			{
				PushConnectionInsight(Expression, It.Index, It->Expression, It->OutputIndex, (*ValuePtr)->Type);
			}
		}
	}
	
	// Adds a connection insight to the MaterialInsights instance, if any.
	void PushConnectionInsight(const UObject* InputObject, int InputIndex, const UMaterialExpression* OutputExpression, int OutputIndex, MIR::FType Type)
	{
		if (!Builder->TargetInsights || Type.IsPoison() || !OutputExpression)
		{
			return;
		}

		FMaterialInsights::FConnectionInsight Insight {
			.InputObject = InputObject,
			.OutputExpression = OutputExpression,
			.InputIndex = InputIndex,
			.OutputIndex = OutputIndex,
			.ValueType = Type.ToValueType(),
		};
		
		Builder->TargetInsights->ConnectionInsights.Push(Insight);
	}

	// Returns whether a material property (e.g. MP_BaseColor) has a value assigned that isn't a constant zero.
	// Used to determine if a property is being used.
	bool MaterialPropertyHasNonZeroValue(EMaterialProperty InProperty)
	{
		return Module->GetPropertyValue(InProperty) && !Module->GetPropertyValue(InProperty)->AreAllExactlyZero();
	}
};

namespace MIR::Internal {

	TOptional<FValue*> RequestInputValue(FMaterialIRModuleBuilderImpl* Builder, const FExpressionInput* Input)
	{
		return Builder->RequestInputValue(Input);
	}
	
	void BindValueToExpressionOutput(FMaterialIRModuleBuilderImpl* Builder, const FExpressionOutput* Output, FValue* Value)
	{
		Builder->OutputValues.Add({Output, Builder->CurrentContext}, Value);
	}

	void DeclareSubgraph(FMaterialIRModuleBuilderImpl* Builder, int32 NumInputs, TConstArrayView<FSubgraphOutputMapping> Outputs)
	{
		Builder->DeclareSubgraph(NumInputs, Outputs);
	}
	
	TOptional<FValue*> RequestSubgraphInputValue(FMaterialIRModuleBuilderImpl* Builder, int32 InputIndex)
	{
		return Builder->RequestSubgraphInputValue(InputIndex);
	}

	UMaterialExpression* GetSubgraphExpression(FMaterialIRModuleBuilderImpl* Builder)
	{
		return Builder->CurrentContext->SubgraphExpression;
	}

} // namespace MIR::Internal

bool FMaterialIRModuleBuilder::Build(FMaterialIRModule* TargetModule)
{
	FMaterialIRModuleBuilderImpl Impl;

	FMemMark MemMark { FMemStack::Get() };

	// Initialize the module to a blank slate, initialize the builder auxiliary data
	// and the emitter for IR values emission.
	Impl.Step_Initialize(this, TargetModule);
	
	// Main step. It crawls the expression graph and calls the Build() function on each
	// visited expression in order to emit the IR values that implement that expression
	// semantics. At the end of this step the IR values graph has been built, but is still
	// missing the root SetMaterialProperty instructions.
	if (!Impl.Step_BuildMaterialExpressionsToIRGraph())
	{
		return false;
	}

	// New that we have generated the IR for the front material output, we store it on a custom output used explicitely later in materials.
	// We also generate the IR for the fully simplified version of the front material output (reprocess all the substrate nodes with the FullySimplified context).
	// Then a material shader can decide to use the normal of fully simplified materials based on SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL from this single translated shader file.
	if (!Impl.Step_BuildSubstrateFrontMaterialOutput())
	{
		return false;
	}

	// Materials dont have a final "output expression", so this step grabs the values flowing
	// into the material property output pins (if any) and generates SetMaterialProperty instructions
    // handling details such as default values, preview material expression, etc.
	if (!Impl.Step_EmitSetMaterialPropertyInstructions())
	{
		return false;
	}

	// Now that the full IR graph has been produced, starting from the output instructions of each
	// entry point, crawl the IR graph backwards in order to let each value analyze itself. A value
	// is analyzed only after all its dependencies (its uses) have been analyzed first, so that when
	// a value is analyzed it is guaranteed to have all the information to properly analyze itself.
	// In this step is performed semantic analysis, where a value can potentially throw new errors depending
	// on the semantic context it is placed in (for instance, an instruction can be executed only in
	// specific stages will throw an error if it finds itself being executed in an incorrect stage.
	if (!Impl.Step_AnalyzeIRGraph())
	{
		return false;
	}
	
	// Once the module individual IR values have been fully analyzed succesfully, we can do module-wide
	// analysis and validation, for instance checking that max limits haven't exceeded, or that an
	// incompatible set of material features are turned on all at once.
	if (!Impl.Step_PerformModuleWideAnalysis())
	{
		return false;
	}
	
	// The IR graph has now been been fully produced and is valid. Proceed to link instructions together,
	// placing each instruction into its own parent block. This is done in a way to put instructions in the
	// narrowest possible scope that still puts them in an execution order that will occur after its dependencies
	// have occurred.
	Impl.Step_LinkInstructions();
	
	// Populate all other non IR-graph artifacts such as the CompilationOutput and the EnvironmentDefines
	// data structures.
	Impl.Step_FinalizeArtifacts();

	return true;
}

#endif // #if WITH_EDITOR
