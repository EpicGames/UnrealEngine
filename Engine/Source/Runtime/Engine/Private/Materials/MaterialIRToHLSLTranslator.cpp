// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRToHLSLTranslator.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Font.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "VT/RuntimeVirtualTexture.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialAggregate.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionVolumetricAdvancedMaterialOutput.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialParameterCollection.h"
#include "MaterialShared.h"
#include "MaterialSharedPrivate.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "ParameterCollection.h"
#include "RenderUtils.h"
#include "ShaderCore.h"
#include "ShaderCompiler.h"

#if WITH_EDITOR

static const FStringView GVector4SwizzleSubset[4] = { TEXTVIEW(".x"), TEXTVIEW(".xy"), TEXTVIEW(".xyz"), {} };

#define TAB "    "

enum ENoOp { NoOp };
enum ENewLine { NewLine };
enum EEndOfStatement { EndOfStatement };
enum EOpenBlock { OpenBlock };
enum ECloseBlock { CloseBlock };
enum EIndentation { Indentation };
enum EBeginArgs { BeginArgs };
enum EEndArgs { EndArgs };
enum EListSeparator { ListSeparator };

struct FPrinter
{
	FString Buffer;
	bool bFirstListItem = false;
	int32 Tabs = 0;
	TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration;
	
	FPrinter(TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration, int32 Tabs = 0)
		: Tabs{ Tabs }
		, MaterialAggregatesRequiringDeclaration { MaterialAggregatesRequiringDeclaration }
	{
		*this << Indentation;
	}

	FPrinter& operator<<(FStringView Text)
	{
		Buffer.Append(Text);
		return *this;
	}

	FPrinter& operator<<(int32 Value)
	{
		Buffer.Appendf(TEXT("%d"), Value);
		return *this;
	}

	FPrinter& operator<<(uint32 Value)
	{
		Buffer.Appendf(TEXT("%u"), Value);
		return *this;
	}

	FPrinter& operator<<(float Value)
	{
		if (FGenericPlatformMath::IsNaN(Value))
		{
			Buffer.Append(TEXTVIEW("(0.0f / 0.0f)"));
		}
		else if (!FGenericPlatformMath::IsFinite(Value))
		{
			Buffer.Append(TEXTVIEW("INFINITE_FLOAT"));
		}
		else
		{
			Buffer.Appendf(TEXT("%#.8gf"), Value);
		}
		return *this;
	}

    FPrinter& operator<<(ENoOp)
	{
		return *this;
	}

    FPrinter& operator<<(ENewLine)
    {
		Buffer.AppendChar(TEXT('\n'));
		operator<<(Indentation);
        return *this;
    }

	FPrinter& operator<<(EIndentation)
	{
		for (int32 i = 0; i < Tabs; ++i)
		{
			Buffer.AppendChar(TEXT('\t'));
		}
		return *this;
	}

	FPrinter& operator<<(EEndOfStatement)
	{
		// Find the last non-whitespace character to determine how to terminate.
		int32 LastNonWS = Buffer.Len() - 1;
		while (LastNonWS >= 0 && FChar::IsWhitespace(Buffer[LastNonWS]))
		{
			--LastNonWS;
		}

		if (LastNonWS >= 0 && Buffer[LastNonWS] == TEXT(';'))
		{
			return *this; // already terminated, nothing to do
		}

		if (LastNonWS < 0 || Buffer[LastNonWS] != TEXT('}'))
		{
			Buffer.AppendChar(TEXT(';'));
		}

		*this << NewLine;
		return *this;
	}

    FPrinter& operator<<(EOpenBlock)
    {
		*this << NewLine;
		Buffer.AppendChar(TEXT('{'));
        ++Tabs;
        *this << NewLine;
        return *this;
    }

    FPrinter& operator<<(ECloseBlock)
    {
        --Tabs;
        Buffer.LeftChopInline(1); // undo tab
        Buffer.AppendChar(TEXT('}'));
        return *this;
    }
	
    FPrinter& operator<<(EBeginArgs)
    {
        Buffer.AppendChar(TEXT('('));
		BeginList();
        return *this;
    }

    FPrinter& operator<<(EEndArgs)
    {
        Buffer.AppendChar(TEXT(')'));
        return *this;
    }

	FPrinter& operator<<(EListSeparator)
    {
		PrintListSeparator();
        return *this;
    }

	FPrinter& operator<<(TCHAR Ch)
    {
        Buffer.AppendChar(Ch);
        return *this;
    }

	void BeginList()
	{
		bFirstListItem = true;
	}

	void PrintListSeparator()
	{
		if (!bFirstListItem)
		{
			Buffer.Append(TEXTVIEW(", "));
		}
		bFirstListItem = false;
	}
};

static void PrintUserIdentifier(FPrinter& Printer, FStringView Identifier)
{
	if (!Identifier.IsEmpty() && FChar::IsDigit(Identifier[0]))
	{
		Printer << TEXT('_');
	}
	for (TCHAR Char : Identifier)
	{
		Printer << (FChar::IsAlnum(Char) || Char == TEXT('_') ? Char : TEXT('_'));
	}
}

// Prints the MIR type HLSL declaration to the specified printer.
static FPrinter& operator<<(FPrinter& Printer, const MIR::FType& Type)
{
	if (TOptional<MIR::FPrimitive> PrimitiveType = Type.AsPrimitive())
	{
		if (PrimitiveType->IsDouble())
		{
			if (PrimitiveType->IsScalar())
			{
				Printer << TEXTVIEW("FWSScalar");
			}
			else if (PrimitiveType->IsRowVector())
			{
				Printer << TEXTVIEW("FWSVector") << PrimitiveType->NumComponents();
			}
			else if (PrimitiveType->bIsLWCInverseMatrix)
			{
				Printer << TEXTVIEW("FWSInverseMatrix");
			}
			else
			{
				Printer << TEXTVIEW("FWSMatrix");
			}
		}
		else
		{
			switch (PrimitiveType->ScalarKind)
			{
				case MIR::EScalarKind::Boolean:  Printer << TEXTVIEW("bool"); break;
				case MIR::EScalarKind::Integer:   Printer << TEXTVIEW("int"); break;
				case MIR::EScalarKind::Float: Printer << TEXTVIEW("MaterialFloat"); break;
			}

			if (PrimitiveType->NumRows == 1 && PrimitiveType->NumColumns > 1)
			{
				Printer << PrimitiveType->NumColumns;
			}
			else if (PrimitiveType->IsMatrix())
			{
				Printer << PrimitiveType->NumRows << TEXTVIEW("x") << PrimitiveType->NumColumns;
			}
		}
	}
	else if (const UMaterialAggregate* Aggregate = Type.AsAggregate())
	{
		Printer << TEXT('F');
		PrintUserIdentifier(Printer, Aggregate->GetName());
		Printer.MaterialAggregatesRequiringDeclaration.AddUnique(Aggregate);
	}
	else
	{
		switch (Type.GetKind())
		{
			case MIR::ETypeKind::Void: Printer << TEXTVIEW("void"); break;
			case MIR::ETypeKind::Texture2D: Printer << TEXTVIEW("Texture2D"); break;
			case MIR::ETypeKind::TextureCube: Printer << TEXTVIEW("TextureCube"); break;
			case MIR::ETypeKind::Texture2DArray: Printer << TEXTVIEW("Texture2DArray"); break;
			case MIR::ETypeKind::TextureCubeArray: Printer << TEXTVIEW("TextureCubeArray"); break;
			case MIR::ETypeKind::Texture3D: Printer << TEXTVIEW("Texture3D"); break;
			case MIR::ETypeKind::TextureExternal: Printer << TEXTVIEW("TextureExternal"); break;
			case MIR::ETypeKind::VirtualTexture: Printer << TEXTVIEW("VirtualTexturePhysical"); break;
			case MIR::ETypeKind::RuntimeVirtualTexture: Printer << TEXTVIEW("VirtualTexturePhysical"); break;
			case MIR::ETypeKind::SubstrateData: Printer << TEXTVIEW("FSubstrateData"); break;
			case MIR::ETypeKind::VTPageTableResult: Printer << TEXTVIEW("VTPageTableResult"); break;
			default: UE_MIR_UNREACHABLE();
		}
	}
	return Printer;
}

// Prints the name of the property as it is called in the HLSL shader template.
static void PrintMaterialPropertyName(FPrinter& Printer, EMaterialProperty Property)
{
	if (Property == MP_SubsurfaceColor)
	{
		// Because, for some reason, this is called "Subsurface" in the shader.
		Printer << TEXTVIEW("Subsurface");
	}
	else
	{
		Printer << FMaterialAttributeDefinitionMap::GetAttributeName(Property);
	}
}

// This struct encapsulates the full state and logic required to translate a single Material IR entry point into HLSL. It outputs
// a the generated HLSL that evaluates a given entry point as described in the module.
struct FEntryPointCodeGen
{
	// The resulting generated HLSL code snippets.
	struct FResult
	{
		// The code generated to evalute normal attribute needs to be placed in a separate slot than the other attributes (before).
		// If the Normal SetMaterialOutput instruction is encountered, the code generated so far will be saved here and Printer.Buffer
		// will contain the remaining code of this entry point.
		FString NormalAttributeHLSL;

		// The code generated to evaluate all other non-Normal attribute outputs.
		FString NonNormalHLSL;
	};

	// The Material IR module, which contains the intermediate representation of the material graph.
	const FMaterialIRModule* Module{};

	// Utility for emitting HLSL code
	FPrinter Printer;

	// Used internally. See FResult::OutputNormalAttributeCode
	FString OutputNormalAttributeCode;

	// Number of local variables generated during translation.
	int32 NumLocals{};
	
	// Mapping from instructions to their corresponding local variable index
	// Used to map instructions that have more than one use to a local identifier like "_42".
	TMap<const MIR::FInstruction*, uint32> InstrToLocalIndex{};

	// The index of the current entry point being generated.
	int32 CurrentEntryPointIndex{};

	// Current material stage being translated (e.g., vertex, pixel)
	MIR::EStage CurrentStage{};

	// Set when generating world position offset HLSL using previous frame data.
	// Affects token replacement for <PREV> in external code, and code generation for certain external inputs.
	bool bCompilingPreviousFrame{};
	
	// Used to increase generated HLSL readability, so that we add a newline before the first instruction that follows a SetMateiralOutput.
	bool bLastInstructionWasSetMaterialOutput{};

	FEntryPointCodeGen(const FMaterialIRModule* Module, TArray<const UMaterialAggregate*>& MaterialAggregateRequiringDeclaration)
	: Module { Module }
	, Printer{ MaterialAggregateRequiringDeclaration, 1 }
	{}

	// Generates the full HLSL of the specified entry point.
	FResult GenerateEntryPoint(int32 ModuleEntryPointIndex, bool bPreviousFrame = false)
	{
		const FMaterialIRModule::FEntryPoint& EntryPoint = Module->GetEntryPoint(ModuleEntryPointIndex);

		CurrentEntryPointIndex = ModuleEntryPointIndex;
		CurrentStage = EntryPoint.Stage;
		bCompilingPreviousFrame = bPreviousFrame;
		InstrToLocalIndex.Empty(InstrToLocalIndex.Num());

		LowerBlock(EntryPoint.RootBlock);

		return { MoveTemp(OutputNormalAttributeCode), MoveTemp(Printer.Buffer) };
	}

	ENoOp LowerBlock(const MIR::FBlock& Block)
	{
		int32 OldNumLocals = NumLocals;
		for (MIR::FInstruction* Instr = Block.Instructions; Instr; Instr = Instr->GetNext(CurrentEntryPointIndex))
		{
			// @massimo.tristano This works but it isn't ideal. We should instead omit linking instruction
			// in the Step_LinkInstructions (skip them) that do not satisfy this condition. Only instructions that truly need
			// evaluation in the various stages should be linked so that here we never skip any.

			// PreshaderTemp values don't need to be lowered to HLSL, since they are only used by other preshaders, and can
			// just be skipped.  TrivialTailSwizzle values are an exception, and should be lowered, even if also marked as
			// PreshaderTemp.  TrivialTailSwizzle indicates this value is used by HLSL, but isn't a full fledged PreshaderOut
			// written to an explicit buffer location, and is instead cobbled together with HLSL swizzles from an underlying
			// value that will have been marked as PreshaderOut.  PreshaderOut values never need to be emitted, since those
			// just become uniform buffer fetches.
			if ((Instr->HasSubgraphProperties(MIR::EGraphProperties::PreshaderTemp) && !Instr->HasSubgraphProperties(MIR::EGraphProperties::TrivialTailSwizzle))
				|| Instr->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut))
			{
				continue;
			}

			// If instruction should be inlined, don't lower it to HLSL now, but when it is referenced.
			if (InstructionShouldBeInlined(Instr))
			{
				continue;
			}

			// If this isn't a set material output instruction but last one was, add a newline for better readability.
			const MIR::FSetMaterialOutput* SetMaterialOutput = MIR::As<MIR::FSetMaterialOutput>(Instr);
			if (!SetMaterialOutput && bLastInstructionWasSetMaterialOutput)
			{
				Printer << NewLine;
				bLastInstructionWasSetMaterialOutput = false;
			}

			// Declare a local to hold this instruction's result, e.g. "float _4 = <expression>;"
			// Not all instructions need a local (e.g. Call, SetMaterialOutput, void-typed ones are excluded — see InstructionNeedsLocal).
			if (InstructionNeedsLocal(Instr) && !InstrToLocalIndex.Find(Instr))
			{
				// Allocate a new local index for this instruction
				uint32 LocalIndex = NumLocals++;
				
				// Remember the mapping between this instruction and its local index
				InstrToLocalIndex.Add(Instr, LocalIndex);

				// Print the local declaration "<Type> _<LocalIndex>", e.g. "float4 _3"
				Printer << Instr->Type << TEXTVIEW(" _") << LocalIndex;
				
				// If this instruction doesn't use a phi value, we'll immediately assign its local to its result so output the "="
				// E.g., if Instr is a branch, we will set its value inside the generate "if" {} scopes, so no need for a "=" now.
				if (InstructionUsesPhiValue(Instr))
				{
					Printer << EndOfStatement;
				}
				else
				{
					Printer << TEXTVIEW(" = ");
				}
			}

			LowerInstruction(Instr);
			
			Printer << EndOfStatement;

			// Store the code needed to evaluate the normal in a separate chunk than the other material attributes
			// since this needs to be emitted before the others in the material template.
			if (SetMaterialOutput && SetMaterialOutput->Property == MP_Normal)
			{
				// "Normal" is treated in a special way because the rest of the attributes may lead back to reading it.
				// Therefore, in the way MaterialTemplate.ush is structured, it needs to be evaluated before other attributes.
				OutputNormalAttributeCode = MoveTemp(Printer.Buffer);
				bLastInstructionWasSetMaterialOutput = false;
			}
		}

		NumLocals = OldNumLocals;

		return NoOp;
	}
	
	ENoOp LowerInstruction(const MIR::FInstruction* Instr)
	{
		switch (Instr->Kind)
		{
			case MIR::VK_Nop: LowerNop(static_cast<const MIR::FNop*>(Instr)); break;
			case MIR::VK_Composite: LowerComposite(static_cast<const MIR::FComposite*>(Instr)); break;
			case MIR::VK_SetMaterialOutput: LowerSetMaterialOutput(static_cast<const MIR::FSetMaterialOutput*>(Instr)); break;
			case MIR::VK_Operator: LowerOperator(static_cast<const MIR::FOperator*>(Instr)); break;
			case MIR::VK_Branch: LowerBranch(static_cast<const MIR::FBranch*>(Instr)); break;
			case MIR::VK_Subscript: LowerSubscript(static_cast<const MIR::FSubscript*>(Instr)); break;
			case MIR::VK_Scalar: LowerScalar(static_cast<const MIR::FScalar*>(Instr)); break;
			case MIR::VK_TextureRead: LowerTextureRead(static_cast<const MIR::FTextureRead*>(Instr)); break;
			case MIR::VK_VTPageTableRead: LowerVTPageTableRead(static_cast<const MIR::FVTPageTableRead*>(Instr)); break;
			case MIR::VK_Extern: LowerExtern(static_cast<const MIR::FExtern*>(Instr)); break;
			case MIR::VK_StageSwitch: LowerStageSwitch(static_cast<const MIR::FStageSwitch*>(Instr)); break;
			case MIR::VK_PartialDerivative: LowerPartialDerivative(static_cast<const MIR::FPartialDerivative*>(Instr)); break;
			case MIR::VK_SetVertexInterpolator: LowerSetVertexInterpolator(static_cast<const MIR::FSetVertexInterpolator*>(Instr)); break;
			case MIR::VK_Call: LowerCall(static_cast<const MIR::FCall*>(Instr)); break;
			case MIR::VK_CallParameterOutput: LowerCallOutput(static_cast<const MIR::FCallParameterOutput*>(Instr)); break;
			case MIR::VK_SubstrateDefaultSlab: LowerSubstrateDefaultSlab(static_cast<const MIR::FSubstrateDefaultSlab*>(Instr)); break;
			case MIR::VK_SubstrateSlab: LowerSubstrateSlab(static_cast<const MIR::FSubstrateSlab*>(Instr)); break;
			case MIR::VK_SubstrateShadingModels: LowerSubstrateShadingModel(static_cast<const MIR::FSubstrateShadingModels*>(Instr)); break;
			case MIR::VK_SubstrateToon: LowerSubstrateToon(static_cast<const MIR::FSubstrateToon*>(Instr)); break;
			case MIR::VK_SubstrateHorizontalMixing: LowerSubstrateHorizontalMixing(static_cast<const MIR::FSubstrateHorizontalMixing*>(Instr)); break;
			case MIR::VK_SubstrateVerticalLayering: LowerSubstrateVerticalLayering(static_cast<const MIR::FSubstrateVerticalLayering*>(Instr)); break;
			case MIR::VK_SubstrateCoverageWeight: LowerSubstrateCoverageWeight(static_cast<const MIR::FSubstrateCoverageWeight*>(Instr)); break;
			case MIR::VK_SubstrateAdd: LowerSubstrateAdd(static_cast<const MIR::FSubstrateAdd*>(Instr)); break;
			case MIR::VK_SubstrateSelect: LowerSubstrateSelect(static_cast<const MIR::FSubstrateSelect*>(Instr)); break;
			case MIR::VK_SubstratePromoteToOperator: LowerSubstratePromoteToOperator(static_cast<const MIR::FSubstratePromoteToOperator*>(Instr)); break;
			default: UE_MIR_UNREACHABLE();
		}
	
		return NoOp;
	}

	ENoOp LowerValue(const MIR::FValue* Value)
	{
		// @jason.hoerner this is okay as this is simply emitting the instruction that reads the preshader result.  
		if (Value->Analysis_PreshaderOffset && Value->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut))
		{
			EmitPreshaderBufferRead(Value);
			return NoOp;
		}

		// Instruction results may be shared among other dependent instructions. No need to lower it here, only reference its local.
		if (const MIR::FInstruction* Instr = MIR::AsInstruction(Value))
		{
			if (InstructionShouldBeInlined(Instr))
			{
				LowerInstruction(Instr);
			}
			else
			{
				check(InstrToLocalIndex.Contains(Instr));
				Printer << TEXTVIEW("_") << InstrToLocalIndex[Instr];
			}
			return NoOp;
		}

		switch (Value->Kind)
		{
			case MIR::VK_Constant: LowerConstant(static_cast<const MIR::FConstant*>(Value)); break;
			case MIR::VK_Builtin: LowerBuiltin(static_cast<const MIR::FBuiltin*>(Value)); break;
			case MIR::VK_MaterialParameterCollection: LowerMaterialParameterCollection(static_cast<const MIR::FMaterialParameterCollection*>(Value)); break;
			case MIR::VK_TextureUniform: LowerTextureUniform(static_cast<const MIR::FTextureUniform*>(Value)); break;
			case MIR::VK_VirtualTextureUniform: LowerVirtualTextureUniform(static_cast<const MIR::FVirtualTextureUniform*>(Value)); break;
			case MIR::VK_GetVertexInterpolator: LowerGetVertexInterpolator(static_cast<const MIR::FGetVertexInterpolator*>(Value)); break;
			default: UE_MIR_UNREACHABLE();
		}

		return NoOp;
	}

	void LowerConstant(const MIR::FConstant* Constant)
	{
		TOptional<MIR::FPrimitive> Primitive = Constant->Type.AsPrimitive();
		check(Primitive && Primitive->IsScalar());

		switch (Primitive->ScalarKind)
		{
			case MIR::EScalarKind::Boolean:
				Printer << (Constant->Boolean ? TEXTVIEW("true") : TEXTVIEW("false"));
				break;
				
			case MIR::EScalarKind::Integer:
				Printer.Buffer.Appendf(TEXT("%lld"), Constant->Integer);
				break;
			
			case MIR::EScalarKind::Float:
				Printer << Constant->Float;
				break;
			
			case MIR::EScalarKind::Double:
			{
				FLargeWorldRenderScalar ConstantLWC(Constant->Double);
				Printer << TEXTVIEW("MakeLWCScalar(") << ConstantLWC.GetTile() << TEXTVIEW(", ") << ConstantLWC.GetOffset() << TEXTVIEW(")");
				break;
			}
		}
	}

	void LowerBuiltin(const MIR::FBuiltin* ExternalInput)
	{
		int32 ExternalInputIndex = (int32)ExternalInput->Id;
	
		FStringView Code;
		switch (ExternalInput->Id)
		{
			case MIR::EBuiltin::TextureMipBias: Code = TEXTVIEW("View.MaterialTextureMipBias"); break;
			case MIR::EBuiltin::TextureDerivativeMultiply: Code = TEXTVIEW("View.MaterialTextureDerivativeMultiply"); break;
			default: UE_MIR_UNREACHABLE();
		}
		Printer << Code;
	}

	void LowerMaterialParameterCollection(const MIR::FMaterialParameterCollection* MaterialParameterCollection)
	{
		Printer << MaterialParameterCollection->Analysis_CollectionIndex;
	}

	void LowerGetVertexInterpolator(const MIR::FGetVertexInterpolator* GetVertexInterpolator, const TCHAR* Suffix = TEXT(""))
	{
		if (GetVertexInterpolator->Type.GetPrimitive().NumComponents() > 1)
		{
			Printer << GetVertexInterpolator->Type << TEXT('(');
		}

		ComputeUserVertexInterpolatorAllocation(GetVertexInterpolator->SetInstr, [this, Suffix](int32 Index, int32 RegisterIndex, FStringView RegisterSwizzle, FStringView)
		{
			if (Index > 0)
			{
				Printer << TEXTVIEW(", ");
			}
			Printer << TEXTVIEW("Parameters.TexCoords") << Suffix << TEXT('[') << RegisterIndex << TEXTVIEW("].") << RegisterSwizzle;
		});

		if (GetVertexInterpolator->Type.GetPrimitive().NumComponents() > 1)
		{
			Printer << TEXT(')');
		}
	}

	MIR::FExternPrinterHLSL MakeExternPrinter(MIR::EExternDifferential Differential, MIR::FExternArgs Arguments)
	{
		return {
			.Buffer             = Printer.Buffer,
			.Differential       = Differential,
			.Stage              = CurrentStage,
			.bIsPreviousFrame   = bCompilingPreviousFrame,
			.Arguments          = Arguments,
			.LowerValueFunction = +[](const MIR::FValue* Value, void* UserData)
			{
				static_cast<FEntryPointCodeGen*>(UserData)->LowerValue(Value);
			},
			.LowerValueUserData = this,
		};
	}

	void LowerExtern(const MIR::FExtern* Extern)
	{
		MIR::FExternPrinterHLSL ExternPrinter = MakeExternPrinter(MIR::EExternDifferential::None, Extern->GetArguments());
		Extern->ToHLSL(ExternPrinter);
	}

	void LowerTextureUniform(const MIR::FTextureUniform* TextureUniform)
	{
		Printer << GetMaterialUBName() << TEXTVIEW(".") << LowerTextureTypeKind(TextureUniform->Type, true) << TEXTVIEW("_") << TextureUniform->Analysis_UniformIndex;
	}

	void LowerVirtualTextureUniform(const MIR::FVirtualTextureUniform* VirtualTextureUniform)
	{
		Printer << GetMaterialUBName() << TEXTVIEW(".") << LowerTextureTypeKind(MIR::FType::MakeRuntimeVirtualTexture(), true) << TEXTVIEW("_") << VirtualTextureUniform->Analysis_UniformIndex;
	}

	void EmitPreshaderBufferReadDoubleVector(uint32 GlobalComponentOffset, int32 SourceComponents, int32 UsedComponents)
	{
		// Must be a multiple of 2 if a single component, and multiple of 4 if more than one component.
		check(!(GlobalComponentOffset & (SourceComponents == 1 ? 0x1 : 0x3)));
		check(UsedComponents <= SourceComponents);

		// Index of the float4 slot.
		const uint32 BufferSlotIndex = GlobalComponentOffset / 4;
		const TCHAR* UBName = GetMaterialUBName();

		if (UsedComponents == 1)
		{
			Printer << TEXTVIEW("DFToWS(MakeDFScalar");
		}
		else
		{
			Printer << TEXTVIEW("DFToWS(MakeDFVector") << UsedComponents;
		}

		static const TCHAR* Components = TEXT("xyzw");
		static const TCHAR* ComponentSubsets[4] = { TEXT("x"), TEXT("xy"), TEXT("xyz"), TEXT("xyzw") };

		switch (SourceComponents)
		{
		case 1:
			// Emit two individual components
			Printer << TEXTVIEW("(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].") << Components[GlobalComponentOffset % 4];
			Printer << TEXTVIEW(", ") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].") << Components[GlobalComponentOffset % 4 + 1] << TEXTVIEW("))");
			break;
		case 2:
			// Emit XY and ZW, or just X and Z
			Printer << TEXTVIEW("(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << (UsedComponents == 2 ? TEXTVIEW("].xy") : TEXTVIEW("].x"));
			Printer << TEXTVIEW(", ") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << (UsedComponents == 2 ? TEXTVIEW("].zw))") : TEXTVIEW("].z))"));
			break;
		case 3:
			if (UsedComponents == 3)
			{
				// Emit XYZ and float3(W, XY)
				Printer << TEXTVIEW("(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].xyz");
				Printer << TEXTVIEW(", float3(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].w");
				Printer << TEXTVIEW(", ") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex + 1 << TEXTVIEW("].xy)))");
			}
			else if (UsedComponents == 2)
			{
				// Emit XY and float2(W, X)
				Printer << TEXTVIEW("(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].xy");
				Printer << TEXTVIEW(", float2(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].w");
				Printer << TEXTVIEW(", ") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex + 1 << TEXTVIEW("].x)))");
			}
			else
			{
				// Emit X and W
				Printer << TEXTVIEW("(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].x");
				Printer << TEXTVIEW(", ") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].w))");
			}
			break;
		case 4:
			// Emit used components
			Printer << TEXTVIEW("(") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("].") << ComponentSubsets[UsedComponents - 1];
			Printer << TEXTVIEW(", ") << UBName << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex + 1 << TEXTVIEW("].") << ComponentSubsets[UsedComponents - 1] << TEXTVIEW("))");
			break;
		}
	}

	void EmitPreshaderBufferReadFloatVector(const MIR::FPrimitive& PrimitiveType, uint32 GlobalComponentOffset)
	{
		const int32 NumComponents = PrimitiveType.NumComponents();

		// Index of the float4 slot
		uint32 BufferSlotIndex = GlobalComponentOffset / 4;

		// Starting component of the float4 slot
		uint32 BufferSlotOffset = GlobalComponentOffset % 4;

		if (PrimitiveType.IsInteger())
		{
			Printer << TEXTVIEW("asint(");
		}

		Printer << GetMaterialUBName() << TEXTVIEW(".PreshaderBuffer[") << BufferSlotIndex << TEXTVIEW("]");

		if (NumComponents < 4)
		{
			Printer << TEXTVIEW(".");

			const TCHAR* Components = TEXT("xyzw");
			for (int32 i = 0; i < NumComponents; ++i)
			{
				check(BufferSlotOffset + i < 4);
				Printer.Buffer.AppendChar(Components[BufferSlotOffset + i]);
			}
		}

		if (PrimitiveType.IsInteger())
		{
			Printer << TEXTVIEW(")"); // close the "asint(" bracket
		}
	}

	void EmitPreshaderBufferRead(const MIR::FValue* Value)
	{
		check(Value->Analysis_PreshaderOffset);

		uint32 GlobalComponentOffset = Value->Analysis_PreshaderOffset - 1;
		MIR::FPrimitive PrimitiveType = Value->Type.GetPrimitive();

		check(PrimitiveType.IsScalar() || PrimitiveType.IsRowVector()); // no matrices yet

		// LWC parameters are handled differently, they have their own dedicated function.
		if (PrimitiveType.IsDouble())
		{
			// For double uniform parameters, we need to get the analysis component count, because it affects the stride
			// between the high and low portion in the DF format.  We can be lazier for float vectors below, because
			// there's no stride to worry about, and it doesn't matter how many components were in the original vector.
			int32 NumComponents = PrimitiveType.NumComponents();
			if (const MIR::FPrimitiveUniform* PrimitiveUniform = Value->As<MIR::FPrimitiveUniform>())
			{
				NumComponents = PrimitiveUniform->Analysis_NumComponents();
			}

			EmitPreshaderBufferReadDoubleVector(GlobalComponentOffset, NumComponents, NumComponents);
		}
		else
		{
			EmitPreshaderBufferReadFloatVector(PrimitiveType, GlobalComponentOffset);
		}
	}

	bool HasMatchingScalarComponentCastChain(MIR::FValue* FirstComponent, MIR::FValue* CurrentComponent, int32 Index, int32 NumComponents)
	{
		const MIR::FSubscript* FirstComponentAsSubscript = FirstComponent->As<MIR::FSubscript>();
		const MIR::FSubscript* CurrentComponentAsSubscript = CurrentComponent->As<MIR::FSubscript>();
		if (FirstComponentAsSubscript && CurrentComponentAsSubscript)
		{
			if (FirstComponentAsSubscript->Arg == CurrentComponentAsSubscript->Arg // Both subscripts have the same argument
				&& FirstComponentAsSubscript->Arg->Type.IsVector()
				&& FirstComponentAsSubscript->Index == 0 && CurrentComponentAsSubscript->Index == Index)
			{
				MIR::FPrimitive ArgPrimitiveType = FirstComponentAsSubscript->Arg->Type.GetPrimitive();

				// Exact match?
				if (ArgPrimitiveType.NumColumns == NumComponents)
				{
					return true;
				}

				// Subset of components? See if we can generate a swizzled vector.
				if (ArgPrimitiveType.NumColumns >= NumComponents)
				{
					// Allow swizzle for non-LWC values or uniform parameters (the latter having a special case for LWC)
					return !ArgPrimitiveType.IsDouble() || FirstComponentAsSubscript->Arg->As<MIR::FPrimitiveUniform>();
				}
			}
		}
		else
		{
			const MIR::FScalar* FirstComponentAsScalar = FirstComponent->As<MIR::FScalar>();
			const MIR::FScalar* CurrentComponentAsScalar = CurrentComponent->As<MIR::FScalar>();
			if (FirstComponentAsScalar && CurrentComponentAsScalar && FirstComponentAsScalar->Type == CurrentComponentAsScalar->Type)
			{
				return HasMatchingScalarComponentCastChain(FirstComponentAsScalar->Arg, CurrentComponentAsScalar->Arg, Index, NumComponents);
			}
		}
		return false;
	}

	ENoOp LowerVectorCastChain(MIR::FValue* FirstComponent, int32 NumComponents)
	{
		if (const MIR::FScalar* FirstComponentAsScalar = FirstComponent->As<MIR::FScalar>())
		{
			MIR::FValue* Arg = FirstComponentAsScalar->Arg;
			MIR::FPrimitive PrimitiveType = FirstComponent->Type.GetPrimitive();
			MIR::FPrimitive ArgPrimitiveType = Arg->Type.GetPrimitive();

			if (ArgPrimitiveType.IsDouble())
			{
				// Cast from LWC
				if (PrimitiveType.IsBoolean())
				{
					// Cast to bool requires a comparison with zero, outside the WSDemote
					Printer << TEXTVIEW("(WSDemote(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(") != 0)");
				}
				else if (!PrimitiveType.IsFloat())
				{
					// Cast to non-float (integer) requires a cast to the type, outside the WSDemote
					MIR::FType VectorType = MIR::FType::MakeVector(PrimitiveType.ScalarKind, NumComponents);
					Printer << VectorType << TEXTVIEW("(WSDemote(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW("))");
				}
				else
				{
					// Cast to float
					Printer << TEXTVIEW("WSDemote(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(")");
				}
			}
			else if (PrimitiveType.IsDouble())
			{
				// Cast to LWC
				if (ArgPrimitiveType.IsBoolean())
				{
					// Cast from bool requires a select between 1.0f and 0.0f, inside the WSPromote
					Printer << TEXTVIEW("WSPromote(select(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(", (float") << NumComponents << TEXTVIEW(")1.0f, (float") << NumComponents << TEXTVIEW(")0.0f))");
				}
				else if (!ArgPrimitiveType.IsFloat())
				{
					// Cast from non-float (integer) requires a cast to float, inside the WSPromote
					Printer << TEXTVIEW("WSPromote(float") << NumComponents << TEXTVIEW("(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW("))");
				}
				else
				{
					// Cast from float
					Printer << TEXTVIEW("WSPromote(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(")");
				}
			}
			else
			{
				// Cast between intrinsic types
				if (PrimitiveType.IsBoolean())
				{
					// Cast to bool requires a comparison with zero
					Printer << TEXTVIEW("(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(" != 0)");
				}
				else if (ArgPrimitiveType.IsBoolean())
				{
					// Cast from bool requires a select between 1 and 0
					MIR::FType VectorType = MIR::FType::MakeVector(PrimitiveType.ScalarKind, NumComponents);
					Printer << TEXTVIEW("select(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(", (") << VectorType << TEXTVIEW(")1, (") << VectorType << TEXTVIEW(")0)");
				}
				else
				{
					// Cast between arithmetic types
					MIR::FType VectorType = MIR::FType::MakeVector(PrimitiveType.ScalarKind, NumComponents);
					Printer << VectorType << TEXTVIEW("(") << LowerVectorCastChain(Arg, NumComponents) << TEXTVIEW(")");
				}
			}
		}
		else if (auto FirstComponentAsSubscript = FirstComponent->As<MIR::FSubscript>())
		{
			// Finally reached the inner subscript, print its vector argument
			const MIR::FValue* Arg = FirstComponentAsSubscript->Arg;
			const MIR::FPrimitiveUniform* ArgAsPrimitiveUniform = Arg->As<MIR::FPrimitiveUniform>();
			if (Arg->Type.IsVector() && Arg->Type.IsDouble() && ArgAsPrimitiveUniform)
			{
				// LWC types support special case swizzling logic for uniform parameters on initial fetch. See HasMatchingScalarComponentCastChain above.
				check(NumComponents >= 2 && NumComponents <= 4);
				check(ArgAsPrimitiveUniform->Type.IsDouble());

				// Get the global float4 component index (e.g. if this is 13, it refer to PreshaderBuffer[3].y)
				const uint32 GlobalComponentOffset = ArgAsPrimitiveUniform->Analysis_PreshaderOffset - 1;

				EmitPreshaderBufferReadDoubleVector(GlobalComponentOffset, ArgAsPrimitiveUniform->Analysis_NumComponents(), NumComponents);
			}
			else
			{
				Printer << LowerValue(Arg);
				if (Arg->Type.GetPrimitive().NumColumns > NumComponents)
				{
					Printer << GVector4SwizzleSubset[NumComponents - 1];
				}
			}
		}
		else
		{
			UE_MIR_UNREACHABLE();
		}
		return NoOp;
	}

	// Check if we can tidy up casts from non-LWC scalar to LWC vectors (assuming we already know all components are the same and LWC)
	static bool IsScalarToLWCVectorCast(const MIR::FValue* FirstComponent)
	{
		if (const MIR::FScalar* AsScalar = FirstComponent->As<MIR::FScalar>())
		{
			// Check if this is a cast from an arithmetic non-LWC scalar type. If so, we can cast the scalar to a float vector, and then to LWC.
			return !AsScalar->Arg->Type.IsBoolean() && !AsScalar->Arg->Type.IsDouble();
		}
		else if (const MIR::FConstant* AsConstant = FirstComponent->As<MIR::FConstant>())
		{
			// Check if this is a cast from a constant representable exactly as a non-LWC float
			return AsConstant->Type.IsDouble() && FLargeWorldRenderScalar(AsConstant->Double).GetTile() == 0.0f;
		}
		return false;
	}

	void LowerNop(const MIR::FNop* Nop)
	{
		// NOP instructions are only used to analyze their argument, but have no effect, thus we compile it to a default
		// value based on its type.
		checkf(!Nop->Type.IsDouble(), TEXT("NOPs do not support LWC primitive type yet"));
		Printer << TEXTVIEW("((") << Nop->Type << TEXTVIEW(")0)");
	}

	void LowerComposite(const MIR::FComposite* Composite)
	{
		if (TOptional<MIR::FPrimitive> PrimitiveType = Composite->Type.AsPrimitive())
		{
			LowerPrimitiveComposite(Composite, *PrimitiveType);
		}
		else if (const UMaterialAggregate* Aggregate = Composite->Type.AsAggregate())
		{
			LowerAggregateComposite(Composite, Aggregate, InstrToLocalIndex[Composite]);
		}
		else
		{
			UE_MIR_UNREACHABLE();
		}
	}

	void LowerPrimitiveComposite(const MIR::FComposite* Composite, MIR::FPrimitive PrimitiveType)
	{
		check(!PrimitiveType.IsScalar());
		TConstArrayView<MIR::FValue*> Components = Composite->GetComponents();
		
		// In order to generate smaller and tidier HLSL, first check whether all components
		// of this dimensional are actually the same. If so, we can simply emit the 
		// component and cast it to the type.  LWC doesn't support casting, and always needs
		// to call a function to convert types.
		bool bSameComponents = true;

		// Track if all components are part of a constant vector
		bool bConstantVector = Components[0]->As<MIR::FConstant>() && PrimitiveType.IsRowVector();
		
		// We can also generate tidier HLSL for cases where casts are done for whole vectors.  This is a frequent case for LWC, where casts happen
		// in both directions (LWC to float and back), due to operations automatically casting their inputs or outputs.  For example, BO_Sub
		// automatically downcasts LWC to float, but if you feed that into a BO_Add that has LWC for its other input, it will immediately cast it
		// back to LWC -- two consecutive casts.  These then get expanded into individual scalar casts, which we would like to do as whole vector
		// casts for readability.  We only need to consider this special case if the first component is a Scalar, and the type is a Vector.
		bool bWholeVectorCast = PrimitiveType.IsRowVector() && (Components[0]->As<MIR::FScalar>() || Components[0]->As<MIR::FSubscript>());

		for (int32 i = 1; i < Components.Num(); ++i)
		{
			bSameComponents &= (Components[i] == Components[0]);
			bConstantVector &= Components[i]->As<MIR::FConstant>() != nullptr;
			bWholeVectorCast = bWholeVectorCast && HasMatchingScalarComponentCastChain(Components[0], Components[i], i, Components.Num());
		}
		
		if (bSameComponents && !PrimitiveType.IsDouble())
		{
			Printer << TEXTVIEW("(") << Composite->Type << TEXTVIEW(")") << LowerValue(Components[0]);
		}
		else if (bSameComponents && PrimitiveType.IsRowVector() && IsScalarToLWCVectorCast(Components[0]))
		{
			// Cast scalar to float vector, then promote, for example:  "WSPromote((float3)1.0f)"
			Printer << TEXTVIEW("WSPromote") << TEXTVIEW("((float") << PrimitiveType.NumColumns << TEXTVIEW(")");
			if (const MIR::FScalar* Scalar = Components[0]->As<MIR::FScalar>())
			{
				// Print the inner non-LWC scalar value
				LowerValue(Scalar->Arg);
			}
			else if (const MIR::FConstant* Constant = Components[0]->As<MIR::FConstant>())
			{
				// Print the inner double constant in its non-LWC form
				Printer << (float)Constant->Double;
			}
			else
			{
				UE_MIR_UNREACHABLE();
			}
			Printer << TEXTVIEW(")");
		}
		else if (bWholeVectorCast)
		{
			LowerVectorCastChain(Components[0], Components.Num());
		}
		else if (bConstantVector && PrimitiveType.IsDouble())
		{
			// Special case for LWC constant vectors
			bool bAllTileValuesZero = true;
			float TileValues[4];
			float OffsetValues[4];
			for (int32 Index = 0; Index < Components.Num(); Index++)
			{
				FLargeWorldRenderScalar ConstantLWC(Components[Index]->As<MIR::FConstant>()->Double);
				TileValues[Index] = ConstantLWC.GetTile();
				OffsetValues[Index] = ConstantLWC.GetOffset();
				bAllTileValuesZero &= TileValues[Index] == 0.0f;
			}

			if (bAllTileValuesZero)
			{
				// Vector representable as regular floats (all tile values zero), call WSPromote on the offset vector
				Printer << TEXTVIEW("WSPromote(");
			}
			else
			{
				// Vector needs tile values, call LWC constructor and generate tile vector, before generating offset vector below
				Printer << TEXTVIEW("MakeLWCVector") << Components.Num() << TEXTVIEW("(float") << Components.Num() << BeginArgs;
				for (int32 Index = 0; Index < Components.Num(); Index++)
				{
					Printer << ListSeparator << TileValues[Index];
				}
				Printer << EndArgs << TEXTVIEW(", ");
			}

			// Generate offset vector, plus extra parentheses to close WSPromote or MakeLWCVector call
			Printer << TEXTVIEW("float") << Components.Num() << BeginArgs;
			for (int32 Index = 0; Index < Components.Num(); Index++)
			{
				Printer << ListSeparator << OffsetValues[Index];
			}
			Printer << EndArgs << TEXTVIEW(")");
		}
		else
		{
			if (PrimitiveType.IsDouble())
			{
				if (PrimitiveType.IsRowVector())
				{
					Printer << TEXTVIEW("MakeWSVector") << BeginArgs;
				}
				else
				{
					// @todo-jason.hoerner - LWC Matrix support.  I don't think there are any nodes that can build LWC matrices from scratch, these
					// generally can only come from external inputs or material parameter collections, so this isn't necessary at the moment.
					UE_MIR_UNREACHABLE();
				}
			}
			else
			{
				Printer << Composite->Type << BeginArgs;
			}

			for (const MIR::FValue* Component : Components)
			{
				Printer << ListSeparator << LowerValue(Component);
			}

			Printer << EndArgs;
		}
	}

	void LowerAggregateComposite(const MIR::FComposite* Composite, const UMaterialAggregate* Aggregate, int32 TargetLocal)
	{
		auto Components = Composite->GetComponents();
		for (int i = 0; i < Components.Num(); ++i)
		{
			if (!Components[i])
			{
				continue;
			}

			Printer << TEXT('_') << TargetLocal << TEXT('.');
			PrintUserIdentifier(Printer, Aggregate->Attributes[i].Name.ToString());
			Printer << TEXT(" = ") << LowerValue(Components[i]) << EndOfStatement;
		}
	}

	void LowerSetMaterialOutput(const MIR::FSetMaterialOutput* SetMaterialOutput)
	{
		if (SetMaterialOutput->Property >= MP_CustomizedUVs0 && SetMaterialOutput->Property <= MP_LastCustomizedUVs)
		{
			Printer << TEXTVIEW("Parameters.CustomizedUVs[") << (int)(SetMaterialOutput->Property - MP_CustomizedUVs0) << TEXT(']');
		}
		else if (SetMaterialOutput->Property == MP_CustomOutput)
		{
			Printer << TEXTVIEW("Parameters.") << *SetMaterialOutput->Name;
		}
		else
		{
			Printer << (CurrentStage != MIR::Stage_Vertex ? TEXTVIEW("PixelMaterialInputs.") : TEXTVIEW("Parameters."));
			PrintMaterialPropertyName(Printer, SetMaterialOutput->Property);
		}

		Printer << TEXTVIEW(" = ") << LowerValue(SetMaterialOutput->Arg);
		
		bLastInstructionWasSetMaterialOutput = true;
	}

	void LowerOperator(const MIR::FOperator* Operator)
	{
		// LWCTile operator is special in that it has an extra zero parameter, so it can't go through the normal operator code path
		if (Operator->Op == MIR::UO_LWCTile)
		{
			// Given input float3, generate LWC3 type with the given tile value and zero offset.
			Printer << TEXTVIEW("MakeLWCVector3(") << LowerValue(Operator->AArg) << TEXTVIEW(", 0)");
			return;
		}

		// Whether any of this operator's arguments has double type which requires special handling in the shader.
		bool bIsDouble = Operator->AArg->Type.IsDouble() || (Operator->BArg && Operator->BArg->Type.IsDouble()) || (Operator->CArg && Operator->CArg->Type.IsDouble());

		// Whether the operator in HLSL is infix between its arguments, e.g. "4 + 4"
		bool bOperatorIsInfix = false;
		if (!bIsDouble)
		{
			switch (Operator->Op)
			{
				case MIR::BO_GreaterThan:
				case MIR::BO_GreaterThanOrEquals:
				case MIR::BO_LessThan:
				case MIR::BO_LessThanOrEquals:
				case MIR::BO_Equals:
				case MIR::BO_NotEquals:
				case MIR::BO_And:
				case MIR::BO_Or:
				case MIR::BO_Add:
				case MIR::BO_Multiply:
				case MIR::BO_Subtract:
				case MIR::BO_Divide:
				case MIR::BO_Modulo:
				case MIR::BO_BitwiseAnd:
				case MIR::BO_BitwiseOr:
				case MIR::BO_BitwiseXor:
				case MIR::BO_BitShiftLeft:
				case MIR::BO_BitShiftRight:
					bOperatorIsInfix = true;
					break;

				default: break;
			}
		}

		if (bOperatorIsInfix)
		{
			FStringView OpString;
			switch (Operator->Op)
			{
				case MIR::BO_And: OpString = TEXTVIEW("&&"); break;
				case MIR::BO_Or: OpString = TEXTVIEW("||"); break;
				case MIR::BO_Add: OpString = TEXTVIEW("+"); break;
				case MIR::BO_Divide: OpString = TEXTVIEW("/"); break;
				case MIR::BO_Modulo: OpString = TEXTVIEW("%"); break;
				case MIR::BO_Equals: OpString = TEXTVIEW("=="); break;
				case MIR::BO_GreaterThan: OpString = TEXTVIEW(">"); break;
				case MIR::BO_GreaterThanOrEquals: OpString = TEXTVIEW(">="); break;
				case MIR::BO_LessThan: OpString = TEXTVIEW("<"); break;
				case MIR::BO_LessThanOrEquals: OpString = TEXTVIEW("<="); break;
				case MIR::BO_Multiply: OpString = TEXTVIEW("*"); break;
				case MIR::BO_NotEquals: OpString = TEXTVIEW("!="); break;
				case MIR::BO_Subtract: OpString = TEXTVIEW("-"); break;
				case MIR::BO_BitwiseAnd: OpString = TEXTVIEW("&"); break;
				case MIR::BO_BitwiseOr: OpString = TEXTVIEW("|"); break;
				case MIR::BO_BitwiseXor: OpString = TEXTVIEW("^"); break;
				case MIR::BO_BitShiftLeft: OpString = TEXTVIEW("<<"); break;
				case MIR::BO_BitShiftRight: OpString = TEXTVIEW(">>"); break;
				default: UE_MIR_UNREACHABLE();
			}

			Printer << TEXTVIEW("(") << LowerValue(Operator->AArg) << TEXT(' ') << OpString << TEXT(' ') << LowerValue(Operator->BArg) << TEXTVIEW(")");
		}
		else
		{
			FStringView OpString;
			if (bIsDouble)
			{
				// "Demotes LWC" indicates the given operator returns a non-LWC float, even if the input is LWC.  Besides that, comparison
				// operators all return bool, instead of LWC, and a couple specific operators require specific inputs to always be non-LWC
				// (second argument of Fmod, and third argument of Lerp).
				switch (Operator->Op)
				{
					case MIR::UO_Abs: OpString = TEXTVIEW("WSAbs"); break;
					case MIR::UO_ACos: OpString = TEXTVIEW("WSACos"); break;						// Demotes LWC
					case MIR::UO_ASin: OpString = TEXTVIEW("WSASin"); break;						// Demotes LWC
					case MIR::UO_ATan: OpString = TEXTVIEW("WSATan"); break;						// Demotes LWC
					case MIR::UO_Ceil: OpString = TEXTVIEW("WSCeil"); break;
					case MIR::UO_Cos: OpString = TEXTVIEW("WSCos"); break;							// Demotes LWC
					case MIR::UO_Floor: OpString = TEXTVIEW("WSFloor"); break;
					case MIR::UO_Frac: OpString = TEXTVIEW("WSFracDemote"); break;					// Demotes LWC
					case MIR::UO_Length: OpString = TEXTVIEW("WSLength"); break;
					case MIR::UO_Negate: OpString = TEXTVIEW("WSNegate"); break;
					case MIR::UO_Round: OpString = TEXTVIEW("WSRound"); break;
					case MIR::UO_Saturate: OpString = TEXTVIEW("WSSaturateDemote"); break;			// Demotes LWC
					case MIR::UO_Sign: OpString = TEXTVIEW("WSSign"); break;						// Demotes LWC
					case MIR::UO_Sin: OpString = TEXTVIEW("WSSin"); break;							// Demotes LWC
					case MIR::UO_Sqrt: OpString = TEXTVIEW("WSSqrtDemote"); break;					// Demotes LWC
					case MIR::UO_Tan: OpString = TEXTVIEW("WSTan"); break;							// Demotes LWC
					case MIR::UO_Truncate: OpString = TEXTVIEW("WSTrunc"); break;

					case MIR::BO_Add: OpString = TEXTVIEW("WSAdd"); break;
					case MIR::BO_Distance: OpString = TEXTVIEW("WSDistance"); break;
					case MIR::BO_Divide: OpString = TEXTVIEW("WSDivide"); break;
					case MIR::BO_Dot: OpString = TEXTVIEW("WSDot"); break;
					case MIR::BO_Equals: OpString = TEXTVIEW("WSEquals"); break;					// Bool output
					case MIR::BO_Fmod: OpString = TEXTVIEW("WSFmodDemote"); break;					// Demotes LWC, second input must be float (not LWC)!
					case MIR::BO_GreaterThan: OpString = TEXTVIEW("WSGreater"); break;				// Bool output
					case MIR::BO_GreaterThanOrEquals: OpString = TEXTVIEW("WSGreaterEqual"); break;	// Bool output
					case MIR::BO_LessThan: OpString = TEXTVIEW("WSLess"); break;					// Bool output
					case MIR::BO_LessThanOrEquals: OpString = TEXTVIEW("WSLessEqual"); break;		// Bool output
					case MIR::BO_Max: OpString = TEXTVIEW("WSMax"); break;
					case MIR::BO_Min: OpString = TEXTVIEW("WSMin"); break;
					case MIR::BO_Multiply: OpString = TEXTVIEW("WSMultiply"); break;
					case MIR::BO_MatrixMultiply: OpString = TEXTVIEW("WSMultiply"); break;			// TODO: implement WSMultiplyVector, WSMultiplyDemote through tiling
					case MIR::BO_NotEquals: OpString = TEXTVIEW("WSNotEquals"); break;				// Bool output
					case MIR::BO_Step: OpString = TEXTVIEW("WSStep"); break;						// Demotes LWC
					case MIR::BO_Subtract: OpString = TEXTVIEW("WSSubtract"); break;

					case MIR::TO_Clamp: OpString = TEXTVIEW("WSClamp"); break;
					case MIR::TO_Lerp: OpString = TEXTVIEW("WSLerp"); break;						// Third input must be float (not LWC)!
					case MIR::TO_Select: OpString = TEXTVIEW("WSSelect"); break;
					case MIR::TO_Smoothstep: OpString = TEXTVIEW("WSSmoothStepDemote"); break;		// Demotes LWC

					default: UE_MIR_UNREACHABLE();
				}
			}
			else
			{
				switch (Operator->Op)
				{
					case MIR::UO_Abs: OpString = TEXTVIEW("abs"); break;
					case MIR::UO_ACos: OpString = TEXTVIEW("acos"); break;
					case MIR::UO_ACosFast: OpString = TEXTVIEW("acosFast"); break;
					case MIR::UO_ACosh: OpString = TEXTVIEW("acosh"); break;
					case MIR::UO_ASin: OpString = TEXTVIEW("asin"); break;
					case MIR::UO_ASinFast: OpString = TEXTVIEW("asinFast"); break;
					case MIR::UO_ASinh: OpString = TEXTVIEW("asinh"); break;
					case MIR::UO_ATan: OpString = TEXTVIEW("atan"); break;
					case MIR::UO_ATanFast: OpString = TEXTVIEW("atanFast"); break;
					case MIR::UO_ATanh: OpString = TEXTVIEW("atanh"); break;
					case MIR::UO_BitwiseNot: OpString = TEXTVIEW("~"); break;
					case MIR::UO_Ceil: OpString = TEXTVIEW("ceil"); break;
					case MIR::UO_Cos: OpString = TEXTVIEW("cos"); break;
					case MIR::UO_Cosh: OpString = TEXTVIEW("cosh"); break;
					case MIR::UO_Exponential: OpString = TEXTVIEW("exp"); break;
					case MIR::UO_Exponential2: OpString = TEXTVIEW("exp2"); break;
					case MIR::UO_Floor: OpString = TEXTVIEW("floor"); break;
					case MIR::UO_Frac: OpString = TEXTVIEW("frac"); break;
					case MIR::UO_IsFinite: OpString = TEXTVIEW("isfinite"); break;
					case MIR::UO_IsInf: OpString = TEXTVIEW("isinf"); break;
					case MIR::UO_IsNan: OpString = TEXTVIEW("isnan"); break;
					case MIR::UO_Length: OpString = TEXTVIEW("length"); break;
					case MIR::UO_Logarithm: OpString = TEXTVIEW("log"); break;
					case MIR::UO_Logarithm10: OpString = TEXTVIEW("log10"); break;
					case MIR::UO_Logarithm2: OpString = TEXTVIEW("log2"); break;
					case MIR::UO_Negate: OpString = TEXTVIEW("-"); break;
					case MIR::UO_Not: OpString = TEXTVIEW("!"); break;
					case MIR::UO_Reciprocal: OpString = TEXTVIEW("rcp"); break;
					case MIR::UO_Round: OpString = TEXTVIEW("round"); break;
					case MIR::UO_Rsqrt: OpString = TEXTVIEW("rsqrt"); break;
					case MIR::UO_Saturate: OpString = TEXTVIEW("saturate"); break;
					case MIR::UO_Sign: OpString = TEXTVIEW("sign"); break;
					case MIR::UO_Sin: OpString = TEXTVIEW("sin"); break;
					case MIR::UO_Sinh: OpString = TEXTVIEW("sinh"); break;
					case MIR::UO_Sqrt: OpString = TEXTVIEW("sqrt"); break;
					case MIR::UO_Tan: OpString = TEXTVIEW("tan"); break;
					case MIR::UO_Tanh: OpString = TEXTVIEW("tanh"); break;
					case MIR::UO_Truncate: OpString = TEXTVIEW("trunc"); break;

					case MIR::BO_And: OpString = TEXTVIEW("and"); break;
					case MIR::BO_ATan2: OpString = TEXTVIEW("atan2"); break;
					case MIR::BO_ATan2Fast: OpString = TEXTVIEW("atan2Fast"); break;
					case MIR::BO_Cross: OpString = TEXTVIEW("cross"); break;
					case MIR::BO_Distance: OpString = TEXTVIEW("distance"); break;
					case MIR::BO_Dot: OpString = TEXTVIEW("dot"); break;
					case MIR::BO_Fmod: OpString = TEXTVIEW("fmod"); break;
					case MIR::BO_Max: OpString = TEXTVIEW("max"); break;
					case MIR::BO_MatrixMultiply: OpString = TEXTVIEW("mul"); break;
					case MIR::BO_Min: OpString = TEXTVIEW("min"); break;
					case MIR::BO_Or: OpString = TEXTVIEW("or"); break;
					case MIR::BO_Pow: OpString = TEXTVIEW("pow"); break;
					case MIR::BO_Step: OpString = TEXTVIEW("step"); break;

					case MIR::TO_Clamp: OpString = TEXTVIEW("clamp"); break;
					case MIR::TO_Lerp: OpString = TEXTVIEW("lerp"); break;
					case MIR::TO_Select: OpString = TEXTVIEW("select"); break;
					case MIR::TO_Smoothstep: OpString = TEXTVIEW("smoothstep"); break;
			
					default: UE_MIR_UNREACHABLE();
				}
			}

			// Unary
			Printer << OpString << TEXTVIEW("(") << LowerValue(Operator->AArg);

			// Binary
			if (Operator->BArg)
			{
				check(MIR::IsBinaryOperator(Operator->Op) || MIR::IsTernaryOperator(Operator->Op));
				Printer << TEXTVIEW(", ") << LowerValue(Operator->BArg);
			}

			// Ternary
			if (Operator->CArg)
			{
				check(MIR::IsTernaryOperator(Operator->Op));
				Printer << TEXTVIEW(", ") << LowerValue(Operator->CArg);
			}

			Printer << TEXTVIEW(")");
		}
	}

	void LowerBranch(const MIR::FBranch* Branch)
	{
		const int32 Local = InstrToLocalIndex[Branch];

		// Lowers a branch arm's block.
		auto LowerBranchArm = [this, Local](MIR::FBlock& Block, const MIR::FValue* Arg)
		{
			// If the arg is a composite inside the block, have it write directly to the branch's phi local instead of declaring an intermediate.
			const MIR::FComposite* ArgComposite = Arg->As<MIR::FComposite>();
			bool bAggregateComposite = ArgComposite && Arg->Type.AsAggregate();
			bool bOptimized = bAggregateComposite && ArgComposite->Linkage[CurrentEntryPointIndex].Block == &Block;
			if (bOptimized)
			{
				InstrToLocalIndex.Add(ArgComposite, Local);
			}

			Printer << LowerBlock(Block);

			if (!bOptimized)
			{
				if (bAggregateComposite)
				{
					LowerAggregateComposite(ArgComposite, ArgComposite->Type.AsAggregate(), Local);
				}
				else
				{
					Printer << TEXTVIEW("_") << Local << TEXTVIEW(" = ") << LowerValue(Arg) << EndOfStatement;
				}
			}
		};

		Printer << TEXTVIEW("if (") << LowerValue(Branch->ConditionArg) << TEXTVIEW(")") << OpenBlock;
		LowerBranchArm(Branch->TrueBlock[CurrentEntryPointIndex], Branch->TrueArg);
		Printer << CloseBlock << NewLine;

		Printer << TEXTVIEW("else") << OpenBlock;
		LowerBranchArm(Branch->FalseBlock[CurrentEntryPointIndex], Branch->FalseArg);
		Printer << CloseBlock;
	}
			
	void LowerSubscript(const MIR::FSubscript* Subscript)
	{
		if (TOptional<MIR::FPrimitive> ArgVectorType = Subscript->Arg->Type.AsVector())
		{
			if (ArgVectorType->IsDouble())
			{
				FStringView LwcComponentsStr[] = { TEXTVIEW("WSGetX("), TEXTVIEW("WSGetY("), TEXTVIEW("WSGetZ("), TEXTVIEW("WSGetW(") };
				check(Subscript->Index < ArgVectorType->NumComponents() && Subscript->Index < 4);

				Printer << LwcComponentsStr[Subscript->Index] << LowerValue(Subscript->Arg) << TEXTVIEW(")");
			}
			else
			{
				LowerValue(Subscript->Arg);
				
				FStringView ComponentsStr[] = { TEXTVIEW(".x"), TEXTVIEW(".y"), TEXTVIEW(".z"), TEXTVIEW(".w") };
				check(Subscript->Index < ArgVectorType->NumComponents() && Subscript->Index < 4);

				Printer << ComponentsStr[Subscript->Index];
			}
		}
		else if (TOptional<MIR::FPrimitive> MatrixType = Subscript->Arg->Type.AsMatrix())
		{
			check(!MatrixType->IsDouble()); // The emitter should have checked this

			LowerValue(Subscript->Arg);

			// Print matrix component swizzle, e.g. `M._m00`
			check(Subscript->Index < MatrixType->NumComponents());
			Printer << TEXTVIEW("._m") << (Subscript->Index % MatrixType->NumRows) << (Subscript->Index / MatrixType->NumRows);
		}
		else if (const UMaterialAggregate* Aggregate = Subscript->Arg->Type.AsAggregate())
		{
			LowerValue(Subscript->Arg);
			Printer << TEXT('.');
			PrintUserIdentifier(Printer, Aggregate->Attributes[Subscript->Index].Name.ToString());
		}
		else
		{
			// The builder should never emit subscripts of scalar types.
			UE_MIR_UNREACHABLE();
		}
	}

	void LowerScalar(const MIR::FScalar* Scalar)
	{
		MIR::FPrimitive PrimitiveType = Scalar->Type.GetPrimitive();
		MIR::FPrimitive ArgPrimitiveType = Scalar->Arg->Type.GetPrimitive();

		if (ArgPrimitiveType.IsDouble())
		{
			// Cast from LWC
			if (PrimitiveType.IsBoolean())
			{
				// Cast to bool requires a comparison with zero, outside the WSDemote
				Printer << TEXTVIEW("(WSDemote(") << LowerValue(Scalar->Arg) << TEXTVIEW(") != 0)");
			}
			else if (!PrimitiveType.IsFloat())
			{
				// Cast to non-float (integer) requires a cast to the type, outside the WSDemote
				Printer << Scalar->Type << TEXTVIEW("(WSDemote(") << LowerValue(Scalar->Arg) << TEXTVIEW("))");
			}
			else
			{
				// Cast to float
				Printer << TEXTVIEW("WSDemote(") << LowerValue(Scalar->Arg) << TEXTVIEW(")");
			}
		}
		else if (PrimitiveType.IsDouble())
		{
			// Cast to LWC
			if (ArgPrimitiveType.IsBoolean())
			{
				// Cast from bool requires a select between 1.0f and 0.0f, inside the WSPromote
				Printer << TEXTVIEW("WSPromote(") << LowerValue(Scalar->Arg) << TEXTVIEW(" ? 1.0f : 0.0f)");
			}
			else if (!ArgPrimitiveType.IsFloat())
			{
				// Cast from non-float (integer) requires a cast to float, inside the WSPromote
				Printer << TEXTVIEW("WSPromote(float(") << LowerValue(Scalar->Arg) << TEXTVIEW("))");
			}
			else
			{
				// Cast from float
				Printer << TEXTVIEW("WSPromote(") << LowerValue(Scalar->Arg) << TEXTVIEW(")");
			}
		}
		else
		{
			// Cast between intrinsic types
			if (PrimitiveType.IsBoolean())
			{
				// Cast to bool requires a comparison with zero
				Printer << TEXTVIEW("(") << LowerValue(Scalar->Arg) << TEXTVIEW(" != 0)");
			}
			else if (ArgPrimitiveType.IsBoolean())
			{
				// Cast from bool requires a select between 1 and 0
				Printer << Scalar->Type << TEXTVIEW("(") << LowerValue(Scalar->Arg) << TEXTVIEW(" ? 1 : 0)");
			}
			else
			{
				// Cast between arithmetic types
				Printer << Scalar->Type << TEXTVIEW("(") << LowerValue(Scalar->Arg) << TEXTVIEW(")");
			}
		}
	}

	ENoOp LowerTextureTypeKind(const MIR::FType& Type, bool bForResourceDeclarations)
	{
	    // @massimo.tristano Double check we need this.
		if (bForResourceDeclarations && Type.Is(MIR::ETypeKind::Texture3D))
		{
			Printer << TEXTVIEW("VolumeTexture");
		}
		else if (bForResourceDeclarations && Type.Is(MIR::ETypeKind::TextureExternal))
		{
			Printer << TEXTVIEW("ExternalTexture");
		}
		else
		{
			Printer << Type;
		}
		return NoOp;
	}

	void LowerStandardTextureRead(const MIR::FTextureRead* TextureRead)
	{
		LowerTextureTypeKind(TextureRead->TextureUniform->Type, false);

		switch (TextureRead->Mode)
		{
			case MIR::ETextureReadMode::GatherRed: Printer << TEXTVIEW("GatherRed"); break;
			case MIR::ETextureReadMode::GatherGreen: Printer << TEXTVIEW("GatherGreen"); break;
			case MIR::ETextureReadMode::GatherBlue: Printer << TEXTVIEW("GatherBlue"); break;
			case MIR::ETextureReadMode::GatherAlpha: Printer << TEXTVIEW("GatherAlpha"); break;
			case MIR::ETextureReadMode::MipAuto: Printer << TEXTVIEW("Sample"); break;
			case MIR::ETextureReadMode::MipLevel: Printer << TEXTVIEW("SampleLevel"); break;
			case MIR::ETextureReadMode::MipBias: Printer << TEXTVIEW("SampleBias"); break;
			case MIR::ETextureReadMode::Derivatives: Printer << TEXTVIEW("SampleGrad"); break;
			default: UE_MIR_UNREACHABLE();
		}

		Printer << BeginArgs
				<< ListSeparator << LowerValue(TextureRead->TextureUniform)
				<< ListSeparator << LowerTextureSamplerReference(TextureRead->TextureUniform, TextureRead->SamplerSourceMode)
				<< ListSeparator << LowerValue(TextureRead->TexCoord);

		switch (TextureRead->Mode)
		{
			case MIR::ETextureReadMode::MipLevel: Printer << ListSeparator << LowerValue(TextureRead->MipValue); break;
			case MIR::ETextureReadMode::MipBias: Printer << ListSeparator << LowerValue(TextureRead->MipValue); break;
			case MIR::ETextureReadMode::Derivatives: Printer << ListSeparator << LowerValue(TextureRead->TexCoordDdx) << ListSeparator << LowerValue(TextureRead->TexCoordDdy); break;
			default: break;
		}

		Printer << EndArgs;
	}

	void LowerVirtualTextureRead(const MIR::FTextureRead* TextureRead)
	{
		checkf(TextureRead->VTPageTable, TEXT("Missing page table for virtual texture read instruction"));
		const MIR::FVTPageTableRead* VTPageTableResult = TextureRead->VTPageTable->As<MIR::FVTPageTableRead>();

		const int32 VirtualTextureIndex = TextureRead->TextureUniform->GetUniformIndex();
		check(VirtualTextureIndex >= 0);

		// Sampling function
		Printer << TEXTVIEW("TextureVirtualSample");

		// 'Texture name/sampler', 'PageTableResult', 'LayerIndex', 'PackedUniform'
		Printer
			<< BeginArgs
			<< ListSeparator << LowerValue(TextureRead->TextureUniform)
			<< ListSeparator;
		
		// Sampling function argument list
		if (TextureRead->SamplerSourceMode != SSM_FromTextureAsset)
		{
			// VT doesn't care if the shared sampler is wrap or clamp. It only cares if it is aniso or not.
			// The wrap/clamp/mirror operation is handled in the shader explicitly.
			// This generates: GetMaterialSharedSampler(Material.VirtualTexturePhysical_<VirtualTextureIndex>Sampler, <SharedSamplerName>)
			FStringView SharedSamplerName = TextureRead->bUseAnisoSampler ? TEXTVIEW("View.SharedBilinearAnisoClampedSampler") : TEXTVIEW("View.SharedBilinearClampedSampler");
			Printer << TEXTVIEW("GetMaterialSharedSampler(") << GetMaterialUBName() << TEXTVIEW(".VirtualTexturePhysical_") << VirtualTextureIndex << TEXTVIEW("Sampler, ") << SharedSamplerName << TEXTVIEW(")");
		}
		else
		{
			Printer << GetMaterialUBName() << TEXTVIEW(".VirtualTexturePhysical_") << VirtualTextureIndex << TEXTVIEW("Sampler");
		}

		Printer << ListSeparator << LowerValue(TextureRead->VTPageTable)
			<< ListSeparator << VTPageTableResult->VTPageTableIndex
			<< ListSeparator << TEXTVIEW("VTUniform_Unpack(") << GetMaterialUBName() << TEXTVIEW(".VTPackedUniform[") << VirtualTextureIndex << TEXTVIEW("]") << TEXTVIEW(")")
			<< EndArgs;
	}

	void LowerTextureRead(const MIR::FTextureRead* TextureRead)
	{
		bool bSamplerNeedsBrackets = false;
		LowerSamplerType(TextureRead->SamplerType, bSamplerNeedsBrackets);
		if (bSamplerNeedsBrackets)
		{
			Printer << TEXTVIEW("(");
		}

		checkf(
			TextureRead->TextureUniform->Type.IsTexture() ||
			TextureRead->TextureUniform->Type.IsRuntimeVirtualTexture(),
			TEXT("Invalid texture object type")
		);

		if (IsVirtualSamplerType(TextureRead->SamplerType))
		{
			LowerVirtualTextureRead(TextureRead);
		}
		else
		{
			LowerStandardTextureRead(TextureRead);
		}

		if (bSamplerNeedsBrackets)
		{
			Printer << TEXTVIEW(")");
		}
	}

	void LowerVTPageTableRead(const MIR::FVTPageTableRead* VTPageTableRead)
	{
		using namespace UE::MaterialTranslatorUtils;

		const bool bHasDerivativeTexCoords = VTPageTableRead->TexCoordDdx != nullptr && VTPageTableRead->TexCoordDdy != nullptr;

		// Construct VT page table load function name 'TextureLoadVirtualPageTable [ Adaptive ] [ * | Grad | Level ]'
		Printer << TEXTVIEW("TextureLoadVirtualPageTable");
		if (VTPageTableRead->bIsAdaptive)
		{
			Printer << TEXTVIEW("Adaptive");
		}

		switch (VTPageTableRead->MipValueMode)
		{
			case TMVM_None: [[fallthrough]];
			case TMVM_MipBias: if (bHasDerivativeTexCoords) { Printer << TEXTVIEW("Grad"); } break;
			case TMVM_MipLevel: Printer << TEXTVIEW("Level"); break;
			case TMVM_Derivative: Printer << TEXTVIEW("Grad"); break;
			default: UE_MIR_UNREACHABLE();
		}

		// Lower common parameters that are shared across all VT page table load functions
		Printer
			<< BeginArgs
			<< ListSeparator << TEXTVIEW("VIRTUALTEXTURE_PAGETABLE_") << VTPageTableRead->VTStackIndex
			<< ListSeparator << TEXTVIEW("VTPageTableUniform_Unpack(VIRTUALTEXTURE_PAGETABLE_UNIFORM_") << VTPageTableRead->VTStackIndex << TEXTVIEW(")")
			<< ListSeparator << LowerValue(VTPageTableRead->TexCoord)
			<< ListSeparator << FStringView{ GetVTAddressMode(VTPageTableRead->AddressU) }
			<< ListSeparator << FStringView{ GetVTAddressMode(VTPageTableRead->AddressV) }
			;

		// Lower additional parameters depening on VT page table load function
		switch (VTPageTableRead->MipValueMode)
		{
			case TMVM_None:
				if (bHasDerivativeTexCoords)
				{
					Printer
						<< ListSeparator << LowerValue(VTPageTableRead->TexCoordDdx)
						<< ListSeparator << LowerValue(VTPageTableRead->TexCoordDdy);
				}
				else
				{
					constexpr float kZeroMipBias = 0.0f;
					Printer << ListSeparator << kZeroMipBias;
				}
				break;

			case TMVM_MipBias:
				if (bHasDerivativeTexCoords)
				{
					Printer
						<< ListSeparator << LowerValue(VTPageTableRead->TexCoordDdx)
						<< ListSeparator << LowerValue(VTPageTableRead->TexCoordDdy);
				}
				else
				{
					check(VTPageTableRead->MipValue);
					Printer << ListSeparator << LowerValue(VTPageTableRead->MipValue);
				}
				break;

			case TMVM_MipLevel:
				check(VTPageTableRead->MipValue);
				Printer << ListSeparator << LowerValue(VTPageTableRead->MipValue);
				break;

			case TMVM_Derivative:
				check(bHasDerivativeTexCoords);
				Printer
					<< ListSeparator << LowerValue(VTPageTableRead->TexCoordDdx)
					<< ListSeparator << LowerValue(VTPageTableRead->TexCoordDdy);
				break;
		}

		Printer << ListSeparator << TEXTVIEW("Parameters.SvPosition.xy");

		// Lower final arguments for VT feedback
		if (VTPageTableRead->bEnableFeedback && CurrentStage == MIR::EStage::Stage_Pixel)
		{
			Printer	<< ListSeparator << TEXTVIEW("Parameters.VirtualTextureFeedback");
		}

		Printer << EndArgs;
	}

	void LowerSamplerType(EMaterialSamplerType SamplerType, bool& bOutSamplerNeedsBrackets)
	{
		bOutSamplerNeedsBrackets = true;

		switch (SamplerType)
		{
			case SAMPLERTYPE_External:
				Printer << TEXTVIEW("ProcessMaterialExternalTextureLookup");
				break;

			case SAMPLERTYPE_Color:
				Printer << TEXTVIEW("ProcessMaterialColorTextureLookup");
				break;
			case SAMPLERTYPE_VirtualColor:
				// has a mobile specific workaround
				Printer << TEXTVIEW("ProcessMaterialVirtualColorTextureLookup");
				break;

			case SAMPLERTYPE_LinearColor:
			case SAMPLERTYPE_VirtualLinearColor:
				Printer << TEXTVIEW("ProcessMaterialLinearColorTextureLookup");
				break;

			case SAMPLERTYPE_Alpha:
			case SAMPLERTYPE_VirtualAlpha:
			case SAMPLERTYPE_DistanceFieldFont:
				Printer << TEXTVIEW("ProcessMaterialAlphaTextureLookup");
				break;

			case SAMPLERTYPE_Grayscale:
			case SAMPLERTYPE_VirtualGrayscale:
				Printer << TEXTVIEW("ProcessMaterialGreyscaleTextureLookup");
				break;

			case SAMPLERTYPE_LinearGrayscale:
			case SAMPLERTYPE_VirtualLinearGrayscale:
				Printer << TEXTVIEW("ProcessMaterialLinearGreyscaleTextureLookup");
				break;
				
				// Normal maps need to be unpacked in the pixel shader.
			case SAMPLERTYPE_Normal:
				Printer << TEXTVIEW("UnpackNormalMap");
				break;
			case SAMPLERTYPE_VirtualNormal:
				Printer << TEXTVIEW("UnpackNormalMapVT");
				break;

			case SAMPLERTYPE_Masks:
			case SAMPLERTYPE_VirtualMasks:
			case SAMPLERTYPE_Data:
				bOutSamplerNeedsBrackets = false;
				break;

			default:
				UE_MIR_UNREACHABLE();
		}
	}

	ENoOp LowerTextureSamplerReference(MIR::FValue* TextureValue, ESamplerSourceMode SamplerSource)
	{
		if (SamplerSource != SSM_FromTextureAsset)
		{
			Printer << TEXTVIEW("GetMaterialSharedSampler(");
		}
		
		Printer << LowerValue(TextureValue) << TEXTVIEW("Sampler");
		
		if (SamplerSource == SSM_Wrap_WorldGroupSettings)
		{
			Printer << TEXTVIEW(", View.MaterialTextureBilinearWrapedSampler)");
		}
		else if (SamplerSource == SSM_Clamp_WorldGroupSettings)
		{
			Printer << TEXTVIEW(", View.MaterialTextureBilinearClampedSampler)");
		}
		else
		{
			// SSM_TerrainWeightmapGroupSettings unsupported yet
			check(SamplerSource == SSM_FromTextureAsset);
		}

		return NoOp;
	}

	void LowerStageSwitch(const MIR::FStageSwitch* StageSwitch)
	{
		LowerValue(StageSwitch->Args[CurrentStage]);
	}

	void LowerPartialDerivative(const MIR::FPartialDerivative* PartialDerivative)
	{
		if (PartialDerivative->bHardwareDerivative)
		{
			// This is differentiating a generic expression.
			if (PartialDerivative->Arg->Type.IsDouble())
			{
				// Expression emitter assumes these to be LWC for analytic derivative evaluation, so we promote them on load
				Printer << (PartialDerivative->Axis == MIR::EDerivativeAxis::X ? TEXTVIEW("WSPromote(WSDdxDemote(") : TEXTVIEW("WSPromote(WSDdyDemote(")) << LowerValue(PartialDerivative->Arg) << TEXTVIEW("))");
			}
			else
			{
				Printer << (PartialDerivative->Axis == MIR::EDerivativeAxis::X ? TEXTVIEW("DDX(") : TEXTVIEW("DDY(")) << LowerValue(PartialDerivative->Arg) << TEXTVIEW(")");
			}
		}
		else if (const MIR::FExtern* ExternArg = PartialDerivative->Arg->As<MIR::FExtern>())
		{
			// Otherwise invoke the extern to generate the differential code by itself
			const MIR::EExternDifferential Differential = PartialDerivative->Axis == MIR::EDerivativeAxis::X ? MIR::EExternDifferential::ScreenSpaceX : MIR::EExternDifferential::ScreenSpaceY;
			MIR::FExternPrinterHLSL ExternPrinter = MakeExternPrinter(Differential, ExternArg->GetArguments());
			ExternArg->ToHLSL(ExternPrinter);
		}
		else if (const MIR::FGetVertexInterpolator* InterpolatorArg = PartialDerivative->Arg->As<MIR::FGetVertexInterpolator>())
		{
			LowerGetVertexInterpolator(InterpolatorArg, PartialDerivative->Axis == MIR::EDerivativeAxis::X ? TEXT("_DDX") : TEXT("_DDY"));
		}
		else
		{
			UE_MIR_UNREACHABLE();
		}
	}

	void LowerSetVertexInterpolator(const MIR::FSetVertexInterpolator* SetVertexInterpolator)
	{
		// If the argument is foldable, first assign it to a HLSL local variable to avoid getting it evaluated more than once should
		// the interpolator spill its base TexCoords register to the next.
		uint32 Local;
		const MIR::FInstruction* ArgAsInstr = MIR::AsInstruction(SetVertexInterpolator->Arg);
		if (!ArgAsInstr || !MIR::Find(InstrToLocalIndex, ArgAsInstr, Local))
		{
			Local = NumLocals++;
			Printer << SetVertexInterpolator->Arg->Type << TEXT(" _") << Local << TEXTVIEW(" = ") << LowerValue(SetVertexInterpolator->Arg) << EndOfStatement;
		}

		ComputeUserVertexInterpolatorAllocation(SetVertexInterpolator, [this, SetVertexInterpolator, Local](int32 Index, int32 RegisterIndex, FStringView RegisterSwizzle, FStringView ArgumentSwizzle)
		{
			if (Index > 0)
			{
				Printer << EndOfStatement;
			}
			Printer << TEXTVIEW("Parameters.TexCoords[") << RegisterIndex << TEXTVIEW("].") << RegisterSwizzle << TEXTVIEW(" = _") << Local;
			if (!ArgumentSwizzle.IsEmpty())
			{
				Printer << TEXTVIEW(".") << ArgumentSwizzle;
			}
		});
	}

	void LowerCall(const MIR::FCall* Call)
	{
		int32 ParamLocal = NumLocals;
		
		// Generate locals to store the output and input-output parameters
		for (int32 i = Call->Function->NumInputOnlyParams; i < Call->Function->NumParameters; ++i)
		{
			Printer << Call->Function->Parameters[i].Type << TEXTVIEW(" _") << (NumLocals++);
			
			if (i < Call->NumArguments)
			{
				Printer << TEXTVIEW(" = ") << LowerValue(Call->Arguments[i]);
			}

			Printer << EndOfStatement;
		}

		// Print the local that will store the result and assign it to the call to the custom function.
		Printer << Call->Function->ReturnType << TEXTVIEW(" _") << NumLocals << TEXTVIEW(" = ") << TEXT('C') << Call->Function->UniqueId << TEXT('_') << Call->Function->Name;
		
		Printer << BeginArgs << TEXTVIEW("Parameters");

		// Print function call arguments
		for (int32 i = 0; i < Call->Function->NumParameters; ++i)
		{
			Printer << TEXTVIEW(", ");

			// Outputs and input-output parameters are stored in special locals. Refer to them
			if (i >= Call->Function->NumInputOnlyParams)
			{
				Printer << TEXT('_') << ParamLocal++;
			}
			else
			{
				// Input-only parameters instead can inline their value instead
				const MIR::FType& ParamType = Call->Function->Parameters[i].Type;

				if (ParamType.IsDouble())
				{
					// LWC types get two values: first the value demoted to float, then the raw LWC value
					Printer << TEXTVIEW("WSDemote(") << LowerValue(Call->Arguments[i]) << TEXTVIEW("), ");
				}

				Printer << LowerValue(Call->Arguments[i]);

				// Texture parameters require an accompanying sampler argument
				if (ParamType.IsTexture())
				{
					Printer << TEXTVIEW(", ") << LowerValue(Call->Arguments[i]) << TEXTVIEW("Sampler");
				}
			}
		}
		Printer << EndArgs;

		// Assign a local to the call result
		InstrToLocalIndex.Add(Call, NumLocals++);
	}
	
	void LowerCallOutput(const MIR::FCallParameterOutput* CallOutput)
	{
		const MIR::FCall* Call = CallOutput->Call->As<MIR::FCall>();
		const int32 NumOutputParams = Call->Function->GetNumOutputParameters();
		// Compute the index of the local that stores the additional output parameter 
		const int32 ParamIndex = InstrToLocalIndex[Call] - NumOutputParams + CallOutput->Index;

		Printer << TEXT('_') << ParamIndex;
	}

	// Returns the name of the material uniform buffer for the current stage.
	const TCHAR* GetMaterialUBName() const
	{
		return CurrentStage == MIR::Stage_Vertex ? TEXT("__Material_UB_VS__") : TEXT("__Material_UB__");
	}

	void LowerSubstrateDefaultSlab(const MIR::FSubstrateDefaultSlab* SubstrateParameter)
	{
		Printer << TEXTVIEW("GetInitialisedSubstrateData()");
	}

	static FStringView GetSharedLocalBasesTypes(bool bFullySimplifiedInstruction)
	{
		return bFullySimplifiedInstruction ? TEXTVIEW("Parameters.SharedLocalBasesFullySimplified.Types") : TEXTVIEW("Parameters.SharedLocalBases.Types");
	}

	static FStringView GetSubstrateTree(bool bFullySimplifiedInstruction)
	{
		return bFullySimplifiedInstruction ? TEXTVIEW("Parameters.SubstrateTreeFullySimplified") : TEXTVIEW("Parameters.SubstrateTree");
	}

	void LowerSubstrateSlab(const MIR::FSubstrateSlab* SubstrateParameter)
	{
		Printer << TEXTVIEW("GetSubstrateSlabBSDF");
		Printer << BeginArgs;
		Printer << TEXTVIEW("Parameters.SubstratePixelFootprint") << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Normal) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->DiffuseAlbedo) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->F0) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->F90) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Roughness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Anisotropy) << TEXTVIEW(",");
		Printer << TEXT("ExtractSubsurfaceProfileInt(") << LowerValue(SubstrateParameter->SSSProfileId) << TEXTVIEW("),");
		Printer << TEXTVIEW("false") << TEXTVIEW(","); // bSupportDefaultSSSProfile
		Printer << LowerValue(SubstrateParameter->SSSMFP) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SSSMFPScale) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SSSPhaseAniso) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SSSType) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->EmissiveColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SecondRoughness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SecondRoughnessWeight) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SecondRoughnessAsSimpleClearCoat) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ClearCoatUseSecondNormal) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ClearCoatBottomNormal) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->FuzzAmount) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->FuzzColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->FuzzRoughness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->GlintValue) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->GlintUV) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SpecularProfileId) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Thickness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->IsThin) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->IsAtBottom) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->LocalBasisIndex) << TEXTVIEW(",");
		Printer << GetSharedLocalBasesTypes(SubstrateParameter->bFullySimplifiedInstruction);
		Printer << EndArgs;
	}

	void LowerSubstrateShadingModel(const MIR::FSubstrateShadingModels* SubstrateParameter)
	{
		Printer << TEXTVIEW("SubstrateConvertLegacyMaterial") << (SubstrateParameter->bHasDynamicShadingModels ? TEXTVIEW("Dynamic") : TEXTVIEW("Static"));
		Printer << BeginArgs;
		Printer << TEXTVIEW("Parameters.SubstratePixelFootprint") << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->BaseColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Specular) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Metallic) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Roughness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Anisotropy) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->SubSurfaceColor) << TEXTVIEW(",");
		Printer << TEXT("ExtractSubsurfaceProfileInt(") << LowerValue(SubstrateParameter->SubSurfaceProfileId) << TEXTVIEW("),");
		Printer << LowerValue(SubstrateParameter->ClearCoat) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ClearCoatRoughness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->EmissiveColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Opacity) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ThinTranslucentTransmittanceColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ThinTranslucentSurfaceCoverage) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->WaterScatteringCoefficients) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->WaterAbsorptionCoefficients) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->WaterPhaseG) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ColorScaleBehindWater) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ShadingModel) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Normal) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Tangent) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ClearCoatNormal) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->CustomTangent) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->BasisIndexMacro) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->ClearCoat_BasisIndexMacro) << TEXTVIEW(",");
		Printer << GetSharedLocalBasesTypes(SubstrateParameter->bFullySimplifiedInstruction);
		Printer << EndArgs;
	}

	void LowerSubstrateToon(const MIR::FSubstrateToon* SubstrateParameter)
	{
		Printer << TEXTVIEW("GetSubstrateToonBSDF");
		Printer << BeginArgs;
		Printer << LowerValue(SubstrateParameter->ToonProfileId) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->BaseColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Metallic) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Specular) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->Roughness) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->EmissiveColor) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->PatternUVs) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->LocalBasisIndex) << TEXTVIEW(",");
		Printer << GetSharedLocalBasesTypes(SubstrateParameter->bFullySimplifiedInstruction);
		Printer << EndArgs;
	}

	void LowerSubstrateHorizontalMixing(const MIR::FSubstrateHorizontalMixing* SubstrateParameter)
	{
		if (SubstrateParameter->bParameterBlendingEnabled)
		{
			Printer << TEXTVIEW("SubstrateHorizontalMixingParameterBlending");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->Background) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Foreground) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Mix) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->NormalMix) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->SharedLocalBasisIndexMacro) << TEXTVIEW(",");
			Printer << GetSharedLocalBasesTypes(SubstrateParameter->bFullySimplifiedInstruction) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->BackgroundNoV) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->ForegroundNoV);
			Printer << EndArgs;
		}
		else
		{
			Printer << GetSubstrateTree(SubstrateParameter->bFullySimplifiedInstruction) << TEXTVIEW(".SubstrateHorizontalMixing");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->Background) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Foreground) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Mix) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->OperatorIndex) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->MaxDistanceFromLeaves);
			Printer << EndArgs;
		}
	}

	void LowerSubstrateVerticalLayering(const MIR::FSubstrateVerticalLayering* SubstrateParameter)
	{
		if (SubstrateParameter->bParameterBlendingEnabled)
		{
			Printer << TEXTVIEW("SubstrateVerticalLayeringParameterBlending");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->Top) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Base) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->SharedLocalBasisIndexMacro) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->TopNoV) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->BaseNoV);
			Printer << EndArgs;
		}
		else
		{
			Printer << GetSubstrateTree(SubstrateParameter->bFullySimplifiedInstruction) << TEXTVIEW(".SubstrateVerticalLayering");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->Top) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Base) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->OperatorIndex) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->MaxDistanceFromLeaves);
			Printer << EndArgs;
		}
	}

	void LowerSubstrateCoverageWeight(const MIR::FSubstrateCoverageWeight* SubstrateParameter)
	{
		if (SubstrateParameter->bParameterBlendingEnabled)
		{
			Printer << TEXTVIEW("SubstrateWeightParameterBlending");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->A) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Weight);
			Printer << EndArgs;
		}
		else
		{
			Printer << GetSubstrateTree(SubstrateParameter->bFullySimplifiedInstruction) << TEXTVIEW(".SubstrateWeight");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->A) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->Weight) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->OperatorIndex) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->MaxDistanceFromLeaves);
			Printer << EndArgs;
		}
	}

	void LowerSubstrateAdd(const MIR::FSubstrateAdd* SubstrateParameter)
	{
		if (SubstrateParameter->bParameterBlendingEnabled)
		{
			Printer << TEXTVIEW("SubstrateAddParameterBlending");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->A) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->B) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->NormalMix) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->SharedLocalBasisIndexMacro) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->ANoV) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->BNoV);
			Printer << EndArgs;
		}
		else
		{
			Printer << GetSubstrateTree(SubstrateParameter->bFullySimplifiedInstruction) << TEXTVIEW(".SubstrateAdd");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->A) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->B) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->OperatorIndex) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->MaxDistanceFromLeaves);
			Printer << EndArgs;
		}
	}

	void LowerSubstrateSelect(const MIR::FSubstrateSelect* SubstrateParameter)
	{
		if (SubstrateParameter->bParameterBlendingEnabled)
		{
			Printer << TEXTVIEW("SubstrateSelectParameterBlending");
			Printer << BeginArgs;
			Printer << LowerValue(SubstrateParameter->A) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->B) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->SelectValue) << TEXTVIEW(",");
			Printer << LowerValue(SubstrateParameter->SharedLocalBasisIndexMacro) << TEXTVIEW(",");
			Printer << GetSharedLocalBasesTypes(SubstrateParameter->bFullySimplifiedInstruction);
			Printer << EndArgs;
		}
		else
		{
			Printer << TEXTVIEW("SubstrateSelect - Parameter Blending is required for the select node.");
		}
	}

	void LowerSubstratePromoteToOperator(const MIR::FSubstratePromoteToOperator* SubstrateParameter)
	{
		Printer << GetSubstrateTree(SubstrateParameter->bFullySimplifiedInstruction) << TEXTVIEW(".PromoteParameterBlendedBSDFToOperator");
		Printer << BeginArgs;
		Printer << LowerValue(SubstrateParameter->SubstrateDataInput) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->OperatorIndex) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->BSDFIndex) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->LayerDepth) << TEXTVIEW(",");
		Printer << LowerValue(SubstrateParameter->bIsBottom);
		Printer << EndArgs;
	}

	// Helper function that generates the mappings between the TexCoords float parameters and given VertexInterpolators, taking care of spilling to subsequent registers and appropriate swizzling.
	//  - Index is the assignment index
	//  - RegisterIndex is the TexCoords register this mapping
	//  - RegisterSwizzle is the TexCoords register swizzle for this mapping
	//  - ArgumentSwizzle is the components of SetVertexInterpolator->Arg for this mapping
	// For instance, this may generate mappings like so (Arg is a float3)
	//    0) TexCoords[0].zw <==> Arg.xy
	//    1) TexCoord[1].x <==> Arg.z
	void ComputeUserVertexInterpolatorAllocation(const MIR::FSetVertexInterpolator* SetVertexInterpolator, TFunctionRef<void(int32 MappingIndex, int32 RegisterIndex, FStringView RegisterSwizzle, FStringView ArgumentSwizzle)> Function)
	{
		static const TCHAR* SwizzleComponents = TEXT("xyzw");

		int MappingIndex = 0;

		const int BaseSlot = SetVertexInterpolator->Analysis_BaseInterpolatorFloatRegister;
		const int NumComponents = SetVertexInterpolator->Arg->Type.GetPrimitive().NumComponents();

		int SlotBegin = BaseSlot;
		const int SlotEnd = BaseSlot + NumComponents;

		while (SlotBegin < SlotEnd)
		{
			const int RegisterIndex = Module->GetStatistics().NumInterpolatedTexCoords + SlotBegin / 2;
			const int RegisterSlotOffset = SlotBegin % 2;

			// How many components fit in the current float2 from this offset. The minimum between the remaining
			// slots in the register and the those remaining to write
			const int PlaceableSlots = FMath::Min(2 - RegisterSlotOffset, SlotEnd - SlotBegin);

			FStringView RegisterSwizzle{ &SwizzleComponents[RegisterSlotOffset], PlaceableSlots };
			FStringView ArgumentSwizzle{};

			// If we are writing a subset, swizzle the RHS to match this chunk.
			const int ArgComponentOffset = SlotBegin - BaseSlot; // 0..NumComponents-1
			if (!(ArgComponentOffset == 0 && PlaceableSlots == NumComponents))
			{
				ArgumentSwizzle = { &SwizzleComponents[ArgComponentOffset], PlaceableSlots };
			}

			Function(MappingIndex, RegisterIndex, RegisterSwizzle, ArgumentSwizzle);

			SlotBegin += PlaceableSlots;
			++MappingIndex;
		}
	}

	/* helpers */

	static bool InstructionShouldBeInlined(const MIR::FInstruction* Instr)
	{
		if (Instr->Type.IsPrimitive() && Instr->IsConstant())
		{
			return true;
		}
		if (Instr->Kind == MIR::VK_Extern)
		{
			return (static_cast<const MIR::FExtern*>(Instr)->GetInfo().Flags & MIR::EExternFlags::Inline) != MIR::EExternFlags::None;
		}
		return Instr->Kind == MIR::VK_Subscript
			|| Instr->Kind == MIR::VK_StageSwitch;
	}

	static bool InstructionNeedsLocal(const MIR::FInstruction* Instr)
	{
		if (Instr->Type.IsVoid())
		{
			return false;
		}
		switch (Instr->Kind)
		{
			case MIR::VK_Call:
			case MIR::VK_SetMaterialOutput:
				return false;
			default:
				return true;
		}
	}
	
	static bool InstructionUsesPhiValue(const MIR::FInstruction* Instr)
	{
		switch (Instr->Kind)
		{
			case MIR::VK_Branch:
				return true;
			case MIR::VK_Composite:
				return Instr->Type.AsAggregate() != nullptr;
			default:
				return false;
		}
	}
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static void GenerateStageEntryPoints(const FMaterialIRModule* Module, TMap<FString, FString>& OutParameters, TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration)
{
	FEntryPointCodeGen CodeGen { Module, MaterialAggregatesRequiringDeclaration };
	FEntryPointCodeGen::FResult Result;

	// Generate current frame vertex stage HLSL
	Result = CodeGen.GenerateEntryPoint(MIR::Stage_Vertex, false);
	OutParameters.Add(TEXT("evaluate_vertex_material_attributes"), MoveTemp(Result.NonNormalHLSL));

	// Generate previous frame vertex stage HLSL
	Result = CodeGen.GenerateEntryPoint(MIR::Stage_Vertex, true);
	OutParameters.Add(TEXT("evaluate_vertex_material_attributes_prev"), MoveTemp(Result.NonNormalHLSL));

	// Generate pixel stage HLSL
	Result = CodeGen.GenerateEntryPoint(MIR::Stage_Pixel);
	Result.NonNormalHLSL += CodeGen.Module->GetSubstrateData().PixelInputInitializerValuesHLSL;
	OutParameters.Add(TEXT("calc_pixel_material_inputs_normal"), MoveTemp(Result.NormalAttributeHLSL));
	OutParameters.Add(TEXT("calc_pixel_material_inputs_other_inputs"), MoveTemp(Result.NonNormalHLSL));

	// Generate compute stage HLSL
	Result = CodeGen.GenerateEntryPoint(MIR::Stage_Compute);
	Result.NonNormalHLSL += CodeGen.Module->GetSubstrateData().PixelInputInitializerValuesHLSL;
	OutParameters.Add(TEXT("calc_pixel_material_inputs_analytic_derivatives_normal"), MoveTemp(Result.NormalAttributeHLSL));
	OutParameters.Add(TEXT("calc_pixel_material_inputs_analytic_derivatives_other_inputs"), MoveTemp(Result.NonNormalHLSL));
}

static void GenerateCustomOutputDeclarations(const FMaterial* Material, const FMaterialIRModule* Module, TMap<FString, FString>& OutParameters, TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration)
{
	// Generate the custom output fields of the struct FMaterialPixelParameters. These fields will be computed and 
	// set by the function that calculates all material outputs (e.g. BaseColor).
	FPrinter VertexPrinter{ MaterialAggregatesRequiringDeclaration, 1 };
	FPrinter PixelPrinter{ MaterialAggregatesRequiringDeclaration, 1 };

	const int NumCustomizedUVs = Material->GetNumCustomizedUVs();
	if (NumCustomizedUVs > 0)
	{
		VertexPrinter << TEXTVIEW("MaterialFloat2 CustomizedUVs[") << NumCustomizedUVs << TEXT(']') << EndOfStatement;
	}

	for (const MIR::FSetMaterialOutput* Output : Module->GetCustomOutputs())
	{
		FPrinter& ThePrinter = Output->Frequency == MIR::EMaterialOutputFrequency::PerVertex ? VertexPrinter : PixelPrinter;
		ThePrinter << Output->Arg->Type << TEXT(' ') << *Output->Name << EndOfStatement;
	}
		
	OutParameters.Add(TEXT("material_vertex_parameter_decls"), MoveTemp(VertexPrinter.Buffer));
	OutParameters.Add(TEXT("material_pixel_parameter_decls"), MoveTemp(PixelPrinter.Buffer));

	// The material shader expects a getter function for each custom output. Generate these getter functions
	// for each FMaterialPixelParameters field, generated above, matching each custom output in each group.
	// Development Note: this is really just cruft necessary for backwards compatibility with the way the
	// 	  old translator supported custom outputs and could be removed entirely with a little refactoring
	//    of MaterialTemplate.ush.
	FPrinter Printer = { MaterialAggregatesRequiringDeclaration };
	for (const FMaterialIRModule::FCustomOutputGroup& Group : Module->GetCustomOutputGroups())
	{
		Printer << TEXTVIEW("#define NUM_MATERIAL_OUTPUTS_GET") << *FString(Group.Name).ToUpper() << TEXT(' ') << Group.NumOutputs << NewLine;

		for (int i = 0; i < Group.NumOutputs; ++i)
		{
			const MIR::FSetMaterialOutput* Instr = Module->GetCustomOutputs()[i + Group.CustomOutputInstructionsBegin];
			Printer << TEXTVIEW("#define HAVE_Get") << *Instr->Name << TEXTVIEW(" 1") << NewLine << NewLine;
				
			// Select the right XXXParameters HLSL struct used by this output based on its 
			FStringView Signature = Instr->Frequency == MIR::EMaterialOutputFrequency::PerVertex
				? TEXTVIEW("(inout FMaterialVertexParameters Parameters)")
				: TEXTVIEW("(inout FMaterialPixelParameters Parameters)");

			// Print the non-LWC get function.
			Printer << Instr->Arg->Type << TEXTVIEW(" Get") << Instr->Name << Signature << OpenBlock << TEXTVIEW("return ");

			Instr->Arg->Type.IsDouble()
				? Printer << TEXTVIEW("WSDemote(Parameters.") << Instr->Name << TEXT(')')
				: Printer << TEXTVIEW("Parameters.") << Instr->Name;

			Printer << EndOfStatement << CloseBlock << NewLine;
				
			// Print the LWC get function.
			if (!Instr->Arg->Type.IsSubstrateData())
			{
				MIR::FType ReturnType = Instr->Arg->Type.GetPrimitive().ToScalarKind(MIR::EScalarKind::Double);
				Printer << ReturnType << TEXTVIEW(" Get") << Instr->Name << TEXTVIEW("_LWC") << Signature << OpenBlock << TEXTVIEW("return ");
				
				Instr->Arg->Type.IsDouble()
					? Printer << TEXTVIEW("Parameters.") << Instr->Name
					: Printer << TEXTVIEW("WSPromote(Parameters.") << Instr->Name << TEXT(')');
					
				Printer << EndOfStatement << CloseBlock << NewLine;
			}
		}
	}

	OutParameters.Add(TEXT("custom_outputs"), MoveTemp(Printer.Buffer));
}

// Generates the definitions of all the custom HLSL functions in the module and puts the resulting string
// into the "custom_functions" source template parameter.
static void GenerateCustomFunctionsHLSL(const FMaterialIRModule* Module, TMap<FString, FString>& OutParameters, TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration)
{
	FPrinter Printer = { MaterialAggregatesRequiringDeclaration };

	for (const MIR::FFunctionHLSL* Function : Module->GetFunctionHLSLs())
	{
		// Print the user-specified defines
		for (const MIR::FFunctionHLSLDefine& Define : Function->Defines)
		{
			Printer << TEXTVIEW("#ifndef ") << Define.Name << NewLine;
			Printer << TEXTVIEW("\t#define ") << Define.Name << TEXT(' ') << Define.Value << NewLine;
			Printer << TEXTVIEW("#endif") << NewLine;
		}

		// Print the user-specified include directives
		for (FStringView Include : Function->Includes)
		{
			Printer << TEXTVIEW("#include \"") << Include << TEXT('\"') << NewLine;
		}

		// Emit one overload per stage so that the correct Parameters type (FMaterialVertexParameters vs
		// FMaterialPixelParameters) is used. When a function is called from multiple stages, HLSL overload
		// resolution at the call site picks the right overload based on the type of 'Parameters' in scope.
		const MIR::EStage Stages[] = { MIR::Stage_Vertex, MIR::Stage_Pixel }; // Compute still uses FMaterialPixelParameters
		for (int StageIndex = 0; StageIndex < UE_ARRAY_COUNT(Stages); ++StageIndex)
		{
			MIR::EStage Stage = Stages[StageIndex];
			if (!MIR::IsStageInMask(Stage, Function->Analysis_CallStageMask))
			{
				continue;
			}

			// Write the custom function signature, for example "C5_MyCustomNode".
			// - 'C' is only a "namespace" for custom functions
			// - '5' is a unique id used to disambiguate distinct custom functions with the same name.
			Printer << Function->ReturnType << TEXTVIEW(" C") << Function->UniqueId << TEXT('_') << Function->Name << BeginArgs;
			Printer << (Stage == MIR::Stage_Vertex ? TEXTVIEW("FMaterialVertexParameters") : TEXTVIEW("FMaterialPixelParameters"));
			Printer << TEXTVIEW(" Parameters");

			// Write the parameter declarations
			for (uint16 i = 0; i < Function->NumParameters; ++i)
			{
				Printer << TEXTVIEW(", ");

				// Print the io keyword
				Printer << (i < Function->NumInputOnlyParams ? TEXTVIEW("")
				: i < Function->NumInputAndOutputParams ? TEXTVIEW("inout ")
				: TEXTVIEW("out "));

				// Type and name
				const MIR::FType& ParamType = Function->Parameters[i].Type;
				const FString& ParamName = Function->Parameters[i].Name.ToString();

				if (ParamType.IsDouble())
				{
					// LWC types get two parameters: first a demoted float version, then the raw LWC value.
					// This way legacy custom expressions can continue to operate on regular float values.
					int32 NumComp = ParamType.GetPrimitive().NumComponents();
					Printer << TEXTVIEW("float");
					if (NumComp > 1)
					{
						Printer << NumComp;
					}
					Printer << TEXT(' ') << *ParamName;
					Printer << TEXTVIEW(", ") << ParamType << TEXTVIEW(" LWC") << *ParamName;
				}
				else
				{
					Printer << ParamType << TEXT(' ') << *ParamName;

					// Texture parameters require an accompanying sampler argument
					if (ParamType.IsTexture())
					{
						Printer << TEXTVIEW(", SamplerState ") << *ParamName << TEXTVIEW("Sampler");
					}
				}
			}

			Printer << EndArgs << OpenBlock;

			// If the function does not contain a "return" keyword, add one.
			bool bContainsReturn = Function->Code.Contains(TEXT("return"));
			if (!bContainsReturn)
			{
				Printer << TEXTVIEW("return") << NewLine;
			}

			// Write the function code
			Printer << Function->Code << NewLine;

			if (!bContainsReturn)
			{
				Printer << TEXT(';');
			}

			Printer << NewLine << CloseBlock << NewLine << NewLine;
		}
	}

	OutParameters.Add(TEXT("custom_functions"), MoveTemp(Printer.Buffer));
}
	
static void GenerateOtherTemplateStringParameters(const FMaterial* Material, const FMaterialIRModule* Module, TMap<FString, FString>& OutParameters, TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration)
{
	const FMaterialIRModule::FStatistics ModuleStatistics = Module->GetStatistics();
	const FMaterialIRModule::FSubstrateData& SubstrateData = Module->GetSubstrateData();

	// pixel_material_inputs (field declarations of the FPixelMaterialInputs struct)
	{
		FPrinter Printer { MaterialAggregatesRequiringDeclaration, 1 };
		for (EMaterialProperty Property : MaterialAttributesAggregate::GetMaterialProperties())
		{
			if (Property == MP_WorldPositionOffset || (Property >= MP_CustomizedUVs0 && Property <= MP_LastCustomizedUVs)) // todo: fixme
			{
				continue;
			}
			if (const FMaterialAggregateAttribute* PropertyAggregate = MaterialAttributesAggregate::GetAttribute(Property))
			{
				Printer << MIR::FType::FromMaterialValueType(PropertyAggregate->ToMaterialValueType()) << TEXT(' ');
				PrintMaterialPropertyName(Printer, Property);
				Printer << EndOfStatement;
			}
		}

		// Add Substrate pixel members declaration code
		Printer.Buffer.Append(SubstrateData.PixelMembersDeclarationHLSL);

		OutParameters.Add(TEXT("pixel_material_inputs"), MoveTemp(Printer.Buffer));
	}

	const int32 NumUserVertexInterpolators = (ModuleStatistics.NumUsedVertexInterpolatorFloatRegisters + 1) / 2; // packed into float2s
	const int32 ShaderTexCoordsArrayLength = ModuleStatistics.NumInterpolatedTexCoords + NumUserVertexInterpolators;

	// get_material_customized_u_vs
	{
		FPrinter Printer { MaterialAggregatesRequiringDeclaration, 1 };
		for (int32 i = 0; i < ShaderTexCoordsArrayLength; i++)
		{
			const bool bIsCustomizedUV = i < MP_NumCustomizedUVs && Module->IsMaterialPropertyUsed((EMaterialProperty)(MP_CustomizedUVs0 + i));
			Printer << TEXTVIEW("OutTexCoords[") << i << TEXTVIEW("] = Parameters.");
			Printer << (bIsCustomizedUV ? TEXTVIEW("CustomizedUVs[") : TEXTVIEW("TexCoords["));
			Printer << i << TEXT(']') << EndOfStatement;
		}
		OutParameters.Add(TEXT("get_material_customized_u_vs"), MoveTemp(Printer.Buffer));
	}
		
	// texcoords parameters
	{
		OutParameters.Add(TEXT("num_material_texcoords"), FString::FromInt(ModuleStatistics.NumInterpolatedTexCoords));
		OutParameters.Add(TEXT("num_material_texcoords_vertex"), FString::FromInt(ModuleStatistics.NumInterpolatedTexCoords));
		OutParameters.Add(TEXT("num_custom_vertex_interpolators"), FString::FromInt(NumUserVertexInterpolators));
		OutParameters.Add(TEXT("num_tex_coord_interpolators"), FString::FromInt(ShaderTexCoordsArrayLength));
	}

	// material properties
	{
		#define SET_PARAM_RETURN_FLOAT(ParamName, Value) OutParameters.Add(TEXT(ParamName), FString::Printf(TEXT(TAB "return %.5f"), Value))
		SET_PARAM_RETURN_FLOAT("get_material_emissive_for_cs", 0.f);
		SET_PARAM_RETURN_FLOAT("get_material_translucency_directional_lighting_intensity", Material->GetTranslucencyDirectionalLightingIntensity());
		SET_PARAM_RETURN_FLOAT("get_material_translucent_shadow_density_scale", Material->GetTranslucentShadowDensityScale());
		SET_PARAM_RETURN_FLOAT("get_material_translucent_self_shadow_density_scale", Material->GetTranslucentSelfShadowDensityScale());
		SET_PARAM_RETURN_FLOAT("get_material_translucent_self_shadow_second_density_scale", Material->GetTranslucentSelfShadowSecondDensityScale());
		SET_PARAM_RETURN_FLOAT("get_material_translucent_self_shadow_second_opacity", Material->GetTranslucentSelfShadowSecondOpacity());
		SET_PARAM_RETURN_FLOAT("get_material_translucent_backscattering_exponent", Material->GetTranslucentBackscatteringExponent());
		SET_PARAM_RETURN_FLOAT("get_material_opacity_mask_clip_value", Material->GetOpacityMaskClipValue());

		FLinearColor Extinction = Material->GetTranslucentMultipleScatteringExtinction();
		OutParameters.Add(TEXT("get_material_translucent_multiple_scattering_extinction"), FString::Printf(TEXT(TAB "return MaterialFloat3(%.5f, %.5f, %.5f)"), Extinction.R, Extinction.G, Extinction.B));
	}

	// user_scene_texture_remap
	OutParameters.Add(TEXT("user_scene_texture_remap"), UE::MaterialTranslatorUtils::GenerateUserSceneTextureRemapHLSLDefines(Module->GetCompilationOutput()));
	
	{
		FString UniformMaterialExpressions = TEXT("#define __Material_UB__ Material" HLSL_LINE_TERMINATOR);
		if (UseMaterialVSUniformBuffer())
		{
			UniformMaterialExpressions += TEXT(
				"#if (VERTEXSHADER || MESHSHADER || (SHADER_USES_ONLY_VERTEX_DATA && SHADER_SUPPORTS_MATERIAL_VS_UB))" HLSL_LINE_TERMINATOR
				"#define __Material_UB_VS__ MaterialVS" HLSL_LINE_TERMINATOR
				"#define USE_MATERIAL_VS_UB 1" HLSL_LINE_TERMINATOR
				"#else" HLSL_LINE_TERMINATOR
				"#define __Material_UB_VS__ Material" HLSL_LINE_TERMINATOR
				"#define USE_MATERIAL_VS_UB 0" HLSL_LINE_TERMINATOR
				"#endif" HLSL_LINE_TERMINATOR);
		}
		else
		{
			// When we want to compile a Nanite compute shader which is also calling SF_Vertex level 
			// functions, we actually remap __Material_UB_VS__ to the generic Material UB. 
			// Since the indices are the same for VS-accessed uniforms, we only need to replace the UB name
			UniformMaterialExpressions += TEXT("#define __Material_UB_VS__ Material" HLSL_LINE_TERMINATOR);
			UniformMaterialExpressions += TEXT("#define USE_MATERIAL_VS_UB 0" HLSL_LINE_TERMINATOR);
		}

		if (Substrate::IsSubstrateEnabled())
		{
			UniformMaterialExpressions += SubstrateData.TreeHLSL;
			// SUBSTRATE_TODO also add in there DescriptionStringCommentForDebug
		}

		OutParameters.Add(TEXT("uniform_material_expressions"), MoveTemp(UniformMaterialExpressions));
	}
}

static const TCHAR* GetShadingModelParameterName(EMaterialShadingModel InModel)
{
	switch (InModel)
	{
		case MSM_Unlit: return TEXT("MATERIAL_SHADINGMODEL_UNLIT");
		case MSM_DefaultLit: return TEXT("MATERIAL_SHADINGMODEL_DEFAULT_LIT");
		case MSM_Subsurface: return TEXT("MATERIAL_SHADINGMODEL_SUBSURFACE");
		case MSM_PreintegratedSkin: return TEXT("MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN");
		case MSM_ClearCoat: return TEXT("MATERIAL_SHADINGMODEL_CLEAR_COAT");
		case MSM_SubsurfaceProfile: return TEXT("MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE");
		case MSM_TwoSidedFoliage: return TEXT("MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE");
		case MSM_Hair: return TEXT("MATERIAL_SHADINGMODEL_HAIR");
		case MSM_Cloth: return TEXT("MATERIAL_SHADINGMODEL_CLOTH");
		case MSM_Eye: return TEXT("MATERIAL_SHADINGMODEL_EYE");
		case MSM_SingleLayerWater: return TEXT("MATERIAL_SHADINGMODEL_SINGLELAYERWATER");
		case MSM_ThinTranslucent: return TEXT("MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT");
		default: UE_MIR_UNREACHABLE();
	}
}

static void GenerateMaterialAggregatesDeclarations(TMap<FString, FString>& OutParameters, TArray<const UMaterialAggregate*>& MaterialAggregatesRequiringDeclaration)
{
	FPrinter Printer { MaterialAggregatesRequiringDeclaration };

	// Recursively push material aggregates referenced by attributes of existing aggregates
	for (int32 i = 0; i < MaterialAggregatesRequiringDeclaration.Num(); ++i)
	{
		for (const FMaterialAggregateAttribute& Attribute : MaterialAggregatesRequiringDeclaration[i]->Attributes)
		{
			if (Attribute.Type == EMaterialAggregateAttributeType::Aggregate)
			{
				MaterialAggregatesRequiringDeclaration.AddUnique(Attribute.Aggregate.Get());
			}
		}
	}

	// Generate the aggregates "struct XXX {};" declarations
	for (int32 i = MaterialAggregatesRequiringDeclaration.Num() - 1; i >= 0; --i)
	{
		Printer << TEXTVIEW("\nstruct F");
		PrintUserIdentifier(Printer, MaterialAggregatesRequiringDeclaration[i]->GetName());
		Printer << OpenBlock;

		for (const FMaterialAggregateAttribute& Attribute : MaterialAggregatesRequiringDeclaration[i]->Attributes)
		{
			switch (Attribute.Type)
			{
				case EMaterialAggregateAttributeType::Bool1: Printer << TEXTVIEW("bool"); break;
				case EMaterialAggregateAttributeType::Bool2: Printer << TEXTVIEW("bool2"); break;
				case EMaterialAggregateAttributeType::Bool3: Printer << TEXTVIEW("bool3"); break;
				case EMaterialAggregateAttributeType::Bool4: Printer << TEXTVIEW("bool4"); break;

				case EMaterialAggregateAttributeType::UInt1: Printer << TEXTVIEW("uint"); break;
				case EMaterialAggregateAttributeType::UInt2: Printer << TEXTVIEW("uint2"); break;
				case EMaterialAggregateAttributeType::UInt3: Printer << TEXTVIEW("uint3"); break;
				case EMaterialAggregateAttributeType::UInt4: Printer << TEXTVIEW("uint4"); break;

				case EMaterialAggregateAttributeType::Float1: Printer << TEXTVIEW("MaterialFloat"); break;
				case EMaterialAggregateAttributeType::Float2: Printer << TEXTVIEW("MaterialFloat2"); break;
				case EMaterialAggregateAttributeType::Normal:
				case EMaterialAggregateAttributeType::Float3: Printer << TEXTVIEW("MaterialFloat3"); break;
				case EMaterialAggregateAttributeType::Float4: Printer << TEXTVIEW("MaterialFloat4"); break;

				case EMaterialAggregateAttributeType::ShadingModel: Printer << TEXTVIEW("int"); break;
				case EMaterialAggregateAttributeType::MaterialAttributes: Printer << TEXTVIEW("FMaterialAttributes"); break;
				case EMaterialAggregateAttributeType::SubstrateData: Printer << TEXTVIEW("FSubstrateData"); break;
				
				case EMaterialAggregateAttributeType::Aggregate:
					Printer << TEXT('F');
					PrintUserIdentifier(Printer, Attribute.Aggregate->GetName());
					break;

				default:
					UE_MIR_UNREACHABLE();
			}

			Printer << TEXT(' ');
			
			PrintUserIdentifier(Printer, Attribute.Name.ToString());

			Printer << EndOfStatement;
		}

		Printer << CloseBlock << TEXT(";\n");
	}

	OutParameters.Add(TEXT("material_aggregates_declarations"), MoveTemp(Printer.Buffer));
}

static void GetShaderCompilerEnvironment(const FMaterial* Material, const FMaterialIRModule* Module, const ITargetPlatform* TargetPlatform, FShaderCompilerEnvironment& OutEnvironment)
{
	const FMaterialCompilationOutput& CompilationOutput = Module->GetCompilationOutput();
	EShaderPlatform ShaderPlatform = Module->GetShaderPlatform();

	OutEnvironment.TargetPlatform = TargetPlatform;
	OutEnvironment.SetDefine(TEXT("ENABLE_NEW_HLSL_GENERATOR"), 1);
	OutEnvironment.SetDefine(TEXT("MATERIAL_ATMOSPHERIC_FOG"), false);
	OutEnvironment.SetDefine(TEXT("MATERIAL_SKY_ATMOSPHERE"), false);
	OutEnvironment.SetDefine(TEXT("INTERPOLATE_VERTEX_COLOR"), false);
	OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_COLOR"), false);
	OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_LOCAL_TO_WORLD"), false);
	OutEnvironment.SetDefine(TEXT("NEEDS_PARTICLE_WORLD_TO_LOCAL"), false);
	OutEnvironment.SetDefine(TEXT("NEEDS_PER_INSTANCE_RANDOM_PS"), false);
	OutEnvironment.SetDefine(TEXT("USES_EYE_ADAPTATION"), false);
	OutEnvironment.SetDefine(TEXT("USES_PER_INSTANCE_CUSTOM_DATA"), false);
	OutEnvironment.SetDefine(TEXT("USES_PER_INSTANCE_FADE_AMOUNT"), false);
	OutEnvironment.SetDefine(TEXT("USES_TRANSFORM_VECTOR"), false);
	OutEnvironment.SetDefine(TEXT("WANT_PIXEL_DEPTH_OFFSET"), CompilationOutput.bUsesPixelDepthOffset);
	OutEnvironment.SetDefineAndCompileArgument(TEXT("USES_WORLD_POSITION_OFFSET"), (bool)CompilationOutput.bUsesWorldPositionOffset);
	OutEnvironment.SetDefineAndCompileArgument(TEXT("USES_DISPLACEMENT"), (bool)CompilationOutput.bUsesDisplacement);
	OutEnvironment.SetDefineAndCompileArgument(TEXT("SHADER_SUPPORTS_MATERIAL_VS_UB"), !CompilationOutput.bUsesDisplacement);
	OutEnvironment.SetDefine(TEXT("USES_EMISSIVE_COLOR"), false);
	OutEnvironment.SetDefine(TEXT("USES_DISTORTION"), Material->IsDistorted());
	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENCY_FOGGING"), Material->ShouldApplyFogging());
	OutEnvironment.SetDefine(TEXT("MATERIAL_ENABLE_TRANSLUCENCY_CLOUD_FOGGING"), Material->ShouldApplyCloudFogging());
	OutEnvironment.SetDefine(TEXT("MATERIAL_IS_SKY"), Material->IsSky());
	OutEnvironment.SetDefine(TEXT("MATERIAL_COMPUTE_FOG_PER_PIXEL"), Material->ComputeFogPerPixel());
	OutEnvironment.SetDefine(TEXT("MATERIAL_FULLY_ROUGH"), false);
	OutEnvironment.SetDefine(TEXT("MATERIAL_USES_ANISOTROPY"), CompilationOutput.bUsesAnisotropy);
	OutEnvironment.SetDefine(TEXT("MATERIAL_USES_SPECULAR_PROFILE"), EnumHasAnyFlags(CompilationOutput.SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SpecularProfile) && Substrate::IsSpecularProfileEnabled(ShaderPlatform));
	OutEnvironment.SetDefine(TEXT("MATERIAL_NEURAL_POST_PROCESS"), (CompilationOutput.bUsedWithNeuralNetworks || Material->IsUsedWithNeuralNetworks()) && Material->IsPostProcessMaterial());
	OutEnvironment.SetDefine(TEXT("NUM_VIRTUALTEXTURE_SAMPLES"), CompilationOutput.UniformExpressionSet.GetVTStacks().Num());
	OutEnvironment.SetDefine(TEXT("NUM_VIRTUALTEXTURE_FEEDBACK_REQUESTS"), CompilationOutput.NumVirtualTextureFeedbackRequests);
	OutEnvironment.SetDefine(TEXT("MATERIAL_VIRTUALTEXTURE_FEEDBACK"), CompilationOutput.NumVirtualTextureFeedbackRequests > 0);
	OutEnvironment.SetDefine(TEXT("IS_MATERIAL_SHADER"), true);
	OutEnvironment.SetDefine(TEXT("VIRTUAL_TEXTURE_OUTPUT"), CompilationOutput.bHasRuntimeVirtualTextureOutputNode != 0);

	// Set all defines that are defined by the module.
	// Any conditional exemption via material properties, such as 'Material->IsUsedWithInstancedStaticMeshes()', are handled during the material IR analysis.
	// @massimo.tristano refactor this
	for (const FName& EnvironmentDefine : Module->GetEnvironmentDefines())
	{
		OutEnvironment.SetDefine(EnvironmentDefine, true);
	}

	// Temporary solution. Define the integer environment defines
	// @massimo.tristano refactor this
	for (const auto& Pair : Module->GetIntegerEnvironmentDefines())
	{
		OutEnvironment.SetDefine(Pair.Key, Pair.Value);
	}

	FMaterialShadingModelField ShadingModels = Module->GetCompiledShadingModels();
	ensure(ShadingModels.IsValid());

	// Logic from FHLSLMaterialTranslator::TranslateMaterial
	bool bOpacityPropertyIsUsed = Module->IsMaterialPropertyUsed(MP_Opacity);

	bool bUsesCurvature = Module->GetFeatureLevel() == ERHIFeatureLevel::ES3_1 &&
		((ShadingModels.HasShadingModel(MSM_SubsurfaceProfile) && Module->IsMaterialPropertyUsed(MP_CustomData0))
		|| (ShadingModels.HasShadingModel(MSM_Eye) && bOpacityPropertyIsUsed));

	int32 NumActiveShadingModels = 0;
	if (ShadingModels.IsLit())
	{
		// This is to have platforms use the simple single layer water shading similar to mobile: no dynamic lights, only sun and sky, no distortion, no colored transmittance on background, no custom depth read.
		const bool bSingleLayerWaterUsesSimpleShading = FDataDrivenShaderPlatformInfo::GetWaterUsesSimpleForwardShading(ShaderPlatform) && IsForwardShadingEnabled(ShaderPlatform);

		for (int32 i = 0; i < MSM_NUM; ++i)
		{
			// SUBSTRATE_TODO handle that using EnumHasAnyFlags(SubstrateTranslatorData.SubstrateMaterialBsdfFeatures, ESubstrateBsdfFeature::SSS)
			EMaterialShadingModel Model = (EMaterialShadingModel)i;
			if (Model == MSM_Strata || !ShadingModels.HasShadingModel(Model))
			{
				continue;
			}

			if (Model == MSM_SingleLayerWater && bSingleLayerWaterUsesSimpleShading)
			{
				// Value must match SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE in SingleLayerWaterCommon.ush!
				OutEnvironment.SetDefine(TEXT("SINGLE_LAYER_WATER_SHADING_QUALITY"), true);
			}

			OutEnvironment.SetDefine(GetShadingModelParameterName(Model), true);
			NumActiveShadingModels += 1;
		}

		if (ShadingModels.HasShadingModel(MSM_SubsurfaceProfile) && bUsesCurvature)
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SUBSURFACE_PROFILE_USE_CURVATURE"), true);
		}

		if (ShadingModels.HasShadingModel(MSM_Eye) && bUsesCurvature)
		{
			OutEnvironment.SetDefine(TEXT("MATERIAL_SHADINGMODEL_EYE_USE_CURVATURE"), true);
		}

		if (ShadingModels.HasShadingModel(MSM_SingleLayerWater) && FDataDrivenShaderPlatformInfo::GetRequiresDisableForwardLocalLights(ShaderPlatform))
		{
			OutEnvironment.SetDefine(TEXT("DISABLE_FORWARD_LOCAL_LIGHTS"), true);
		}

		const bool bIsWaterDistanceFieldShadowEnabled = IsWaterDistanceFieldShadowEnabled(ShaderPlatform);
		const bool bIsWaterVSMFilteringEnabled = IsWaterVirtualShadowMapFilteringEnabled(ShaderPlatform);
		if (ShadingModels.HasShadingModel(MSM_SingleLayerWater) && (bIsWaterDistanceFieldShadowEnabled || bIsWaterVSMFilteringEnabled))
		{
			OutEnvironment.SetDefine(TEXT("SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT"), TEXT("1"));
		}
	}
	else
	{
		// Unlit shading model can only exist by itself
		OutEnvironment.SetDefine(GetShadingModelParameterName(MSM_Unlit), true);
		NumActiveShadingModels += 1;
	}

	if (NumActiveShadingModels == 1)
	{
		OutEnvironment.SetDefine(TEXT("MATERIAL_SINGLE_SHADINGMODEL"), true);
	}
	else if (!ensure(NumActiveShadingModels > 0))
	{
		UE_LOGF(LogMaterial, Warning, "Unknown material shading model(s). Setting to MSM_DefaultLit");
		OutEnvironment.SetDefine(GetShadingModelParameterName(MSM_DefaultLit), true);
	}

	OutEnvironment.SetDefine(TEXT("MATERIAL_LWC_ENABLED"), UE::MaterialTranslatorUtils::IsLWCEnabled() ? 1 : 0);
	OutEnvironment.SetDefine(TEXT("WSVECTOR_IS_TILEOFFSET"), true);
	OutEnvironment.SetDefine(TEXT("WSVECTOR_IS_DOUBLEFLOAT"), false);

	if (Material->GetMaterialDomain() == MD_Volume)
	{
		TArray<const UMaterialExpressionVolumetricAdvancedMaterialOutput*> VolumetricAdvancedExpressions;
		Material->GetMaterialInterface()->GetMaterial()->GetAllExpressionsOfType(VolumetricAdvancedExpressions);
		if (VolumetricAdvancedExpressions.Num() > 0)
		{
			if (VolumetricAdvancedExpressions.Num() > 1)
			{
				UE_LOGF(LogMaterial, Fatal, "Only a single UMaterialExpressionVolumetricAdvancedMaterialOutput node is supported.");
			}

			const UMaterialExpressionVolumetricAdvancedMaterialOutput* VolumetricAdvancedNode = VolumetricAdvancedExpressions[0];
			const TCHAR* Param = VolumetricAdvancedNode->GetEvaluatePhaseOncePerSample() ? TEXT("MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERSAMPLE") : TEXT("MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERPIXEL");
			OutEnvironment.SetDefine(Param, true);

			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED"), true);
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_GRAYSCALE_MATERIAL"), VolumetricAdvancedNode->bGrayScaleMaterial);
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_RAYMARCH_VOLUME_SHADOW"), VolumetricAdvancedNode->bRayMarchVolumeShadow);
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_CLAMP_MULTISCATTERING_CONTRIBUTION"), VolumetricAdvancedNode->bClampMultiScatteringContribution);
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT"), VolumetricAdvancedNode->GetMultiScatteringApproximationOctaveCount());
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY"), VolumetricAdvancedNode->ConservativeDensity.IsConnected());
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_OVERRIDE_AMBIENT_OCCLUSION"), Material->HasAmbientOcclusionConnected());
			OutEnvironment.SetDefine(TEXT("MATERIAL_VOLUMETRIC_ADVANCED_GROUND_CONTRIBUTION"), VolumetricAdvancedNode->bGroundContribution);
		}
	}

	OutEnvironment.SetDefine(TEXT("DUAL_SOURCE_COLOR_BLENDING_ENABLED"), Material->IsDualBlendingEnabled() ? 1 : 0);
	OutEnvironment.SetDefine(TEXT("TEXTURE_SAMPLE_DEBUG"), false);

	for (int32 VTStackIndex = 0; VTStackIndex < CompilationOutput.UniformExpressionSet.GetVTStacks().Num(); ++VTStackIndex)
	{
		const FMaterialVirtualTextureStack& VTStack = CompilationOutput.UniformExpressionSet.GetVTStack(VTStackIndex);
		const TCHAR* UniformUBName = (VTStack.ShaderFrequencyMask & (1ULL << ((uint64)SF_Vertex))) ? TEXT("__Material_UB_VS__") : TEXT("__Material_UB__");

		// Setup page table defines to map each VT stack to either 1 or 2 page table textures, depending on how many layers it uses
		FString PageTableValue = FString::Printf(TEXT("%s.VirtualTexturePageTable0_%d"), UniformUBName, VTStackIndex);
		OutEnvironment.SetDefine(*FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_%d"), VTStackIndex), *PageTableValue);

		// Setup page table uniform defines.
		FString PageTableUniformValue = FString::Printf(TEXT("%s.VTPackedPageTableUniform[%d*2], %s.VTPackedPageTableUniform[%d*2+1]"), UniformUBName, VTStackIndex, UniformUBName, VTStackIndex);
		OutEnvironment.SetDefine(*FString::Printf(TEXT("VIRTUALTEXTURE_PAGETABLE_UNIFORM_%d"), VTStackIndex), *PageTableUniformValue);
	}

	TConstArrayView<UMaterialParameterCollection*> ParameterCollections = Module->GetParameterCollections();
	for (int32 CollectionIndex = 0; CollectionIndex < ParameterCollections.Num(); CollectionIndex++)
	{
		// Add uniform buffer declarations for any parameter collections referenced
		static_assert(MaxNumParameterCollectionsPerMaterial == 2);
		static const TCHAR* CollectionNames[MaxNumParameterCollectionsPerMaterial] =
		{
			TEXT("MaterialCollection0"),
			TEXT("MaterialCollection1"),
		};

		// Check that the parameter collection loaded successfully.
		UMaterialParameterCollection* ParameterCollection = ParameterCollections[CollectionIndex];
		if (!ParameterCollection)
		{
			UE_LOGF(LogMaterial, Warning, "Null parameter collection found in environment defines while translating material.");
			continue;
		}

		// Ensure PostLoad is called so the uniform buffers are created in case the parameter collection was loaded async 
		ParameterCollection->ConditionalPostLoad();

		// Check that the parameter collection uniform buffer structure is valid
		if (!ParameterCollection->HasValidUniformBufferStruct())
		{
			UE_LOGF(LogMaterial, Warning, "Invalid parameter collection uniform buffer struct found in environment defines while translating material.");
			continue;
		}

		// This can potentially become an issue for MaterialCollection Uniform Buffers if they ever get non-numeric resources (eg Textures), as
		// OutEnvironment.ResourceTableMap has a map by name, and the N ParameterCollection Uniform Buffers ALL are names "MaterialCollection"
		// (and the hlsl cbuffers are named MaterialCollection0, etc, so the names don't match the layout)
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(CollectionNames[CollectionIndex], ParameterCollection->GetUniformBufferStruct(), Module->GetShaderPlatform(), OutEnvironment);
	}
}

void FMaterialIRToHLSLTranslation::Run(TMap<FString, FString>& OutShaderParameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutShaderParameters.Empty();

	TArray<const UMaterialAggregate*> MaterialAggregatesRequiringDeclaration;
	MaterialAggregatesRequiringDeclaration.Add(MaterialAttributesAggregate::Get());

	GenerateStageEntryPoints(Module, OutShaderParameters, MaterialAggregatesRequiringDeclaration);
	GenerateCustomOutputDeclarations(Material, Module, OutShaderParameters, MaterialAggregatesRequiringDeclaration);
	GenerateCustomFunctionsHLSL(Module, OutShaderParameters, MaterialAggregatesRequiringDeclaration);
	GenerateOtherTemplateStringParameters(Material, Module, OutShaderParameters, MaterialAggregatesRequiringDeclaration);
	GenerateMaterialAggregatesDeclarations(OutShaderParameters, MaterialAggregatesRequiringDeclaration);
	GetShaderCompilerEnvironment(Material, Module, TargetPlatform, OutEnvironment);
}

#undef TAB
#endif // #if WITH_EDITOR
