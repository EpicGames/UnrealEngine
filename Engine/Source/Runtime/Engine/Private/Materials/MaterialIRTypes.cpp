// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialAggregate.h"

#if WITH_EDITOR

namespace MIR
{

const TCHAR* TypeKindToString(ETypeKind Kind)
{
    #define CASE(Name) case ETypeKind::Name: return TEXT(#Name); 
    
	switch (Kind)
	{
		CASE(Poison)
		CASE(Void)
		CASE(Primitive)
		CASE(Aggregate)
		CASE(Texture2D)
		CASE(TextureCube)
		CASE(Texture2DArray)
		CASE(TextureCubeArray)
		CASE(Texture3D)
		CASE(TextureExternal)
		CASE(VirtualTexture)
		CASE(RuntimeVirtualTexture)
		CASE(ParameterCollection)
		CASE(SubstrateData)
		CASE(VTPageTableResult)
		default: UE_MIR_UNREACHABLE();
	}
	
	#undef CASE
}

bool ScalarKindIsArithmetic(EScalarKind Kind)
{
	return Kind != EScalarKind::Boolean;
}

bool ScalarKindIsAnyFloat(EScalarKind Kind)
{
	return Kind == EScalarKind::Float || Kind == EScalarKind::Double;
}

const TCHAR* ScalarKindToString(EScalarKind Kind)
{
	switch (Kind)
	{
		case EScalarKind::Boolean: return TEXT("bool");
		case EScalarKind::Integer: return TEXT("int");
		case EScalarKind::Float: return TEXT("MaterialFloat");
		case EScalarKind::Double: return TEXT("FLWCScalar");
		default: UE_MIR_UNREACHABLE();
	}
}

FType FType::FromShaderType(const UE::Shader::FType& InShaderType)
{
	check(!InShaderType.IsStruct());
	check(!InShaderType.IsObject());

	switch (InShaderType.ValueType)
	{
		case UE::Shader::EValueType::Void:
			return FType::MakeVoid();

		case UE::Shader::EValueType::Float1:
		case UE::Shader::EValueType::Float2:
		case UE::Shader::EValueType::Float3:
		case UE::Shader::EValueType::Float4:
			return FType::MakeFloatVector((int)InShaderType.ValueType - (int)UE::Shader::EValueType::Float1 + 1);

		case UE::Shader::EValueType::Int1:
		case UE::Shader::EValueType::Int2:
		case UE::Shader::EValueType::Int3:
		case UE::Shader::EValueType::Int4:
			return FType::MakeIntVector((int)InShaderType.ValueType - (int)UE::Shader::EValueType::Int1 + 1);

		case UE::Shader::EValueType::Bool1:
		case UE::Shader::EValueType::Bool2:
		case UE::Shader::EValueType::Bool3:
		case UE::Shader::EValueType::Bool4:
			return FType::MakeBoolVector((int)InShaderType.ValueType - (int)UE::Shader::EValueType::Bool1 + 1);

		case UE::Shader::EValueType::Double1:
		case UE::Shader::EValueType::Double2:
		case UE::Shader::EValueType::Double3:
		case UE::Shader::EValueType::Double4:
			return FType::MakeDoubleVector((int)InShaderType.ValueType - (int)UE::Shader::EValueType::Double1 + 1);

		default:
			UE_MIR_UNREACHABLE();
	}
}

FType FType::FromMaterialValueType(EMaterialValueType Type)
{
	switch (Type)
	{
		case MCT_VoidStatement: return FType::MakeVoid();
		case MCT_Float: return FType::MakeFloatScalar(); // MCT_Float represents any scalar or vector, but GetMaterialTypeFromInputType() maps FunctionInput_Scalar to MCT_Float instead of MCT_Float1, so interpret it as a scalar here
		case MCT_Float1: return FType::MakeFloatScalar();
		case MCT_Float2: return FType::MakeFloatVector(2);
		case MCT_Float3: return FType::MakeFloatVector(3);
		case MCT_Float4: return FType::MakeFloatVector(4);
		case MCT_TextureVirtual: return FType::MakeVirtualTexture();
		case MCT_Texture2D: return FType::MakeTexture2D();
		case MCT_TextureCube: return FType::MakeTextureCube();
		case MCT_Texture2DArray: return FType::MakeTexture2DArray();
		case MCT_TextureCubeArray: return FType::MakeTextureCubeArray();
		case MCT_VolumeTexture: return FType::MakeTexture3D();
		case MCT_TextureExternal: return FType::MakeTextureExternal();
		case MCT_TextureMeshPaint: return FType::MakeVirtualTexture();
		case MCT_ShadingModel: return FType::MakeIntScalar();
		case MCT_StaticBool: return FType::MakeBoolScalar();
		case MCT_UInt1: return FType::MakeIntScalar();
		case MCT_UInt2: return FType::MakeIntVector(2);
		case MCT_UInt3: return FType::MakeIntVector(3);
		case MCT_UInt4: return FType::MakeIntVector(4);
		case MCT_Bool: return FType::MakeBoolScalar();
		case MCT_LWCScalar: return FType::MakeDoubleScalar();
		case MCT_LWCVector2: return FType::MakeDoubleVector(2);
		case MCT_LWCVector3: return FType::MakeDoubleVector(3);
		case MCT_LWCVector4: return FType::MakeDoubleVector(4);
		case MCT_Float3x3: return FType::MakeFloat(3, 3);
		case MCT_Float4x4: return FType::MakeFloat(4, 4);
		case MCT_LWCMatrix: return FType::MakeDouble(4, 4);
		case MCT_MaterialAttributes: return FType::MakeAggregate(MaterialAttributesAggregate::Get());
		case MCT_Substrate: return FType::MakeSubstrateData();
		default: UE_MIR_UNREACHABLE();
	}
}

FType FType::FromMaterialParameterType(EMaterialParameterType Type)
{
	switch (Type)
	{
		case EMaterialParameterType::Scalar: return FType::MakeFloatScalar();
		case EMaterialParameterType::Vector: return FType::MakeFloatVector(4);
		case EMaterialParameterType::DoubleVector: return FType::MakeDoubleVector(4);
		case EMaterialParameterType::Texture: return FType::MakeTexture2D();
		case EMaterialParameterType::TextureCollection: UE_MIR_TODO();
		case EMaterialParameterType::ParameterCollection: UE_MIR_TODO();
		case EMaterialParameterType::Font: UE_MIR_TODO();
		case EMaterialParameterType::RuntimeVirtualTexture: return FType::MakeRuntimeVirtualTexture();
		case EMaterialParameterType::SparseVolumeTexture: UE_MIR_TODO();
		case EMaterialParameterType::StaticSwitch: return FType::MakeBoolScalar();
		default: UE_MIR_UNREACHABLE();
	}
}

FType FType::MakePoison()
{
	return {};
}

FType FType::MakeVoid()
{
	FType Type{};
	Type.Kind = ETypeKind::Void;
	return Type;
}

FType FType::MakePrimitive(EScalarKind InScalarKind, int NumRows, int NumColumns, bool bIsDoubleInverseMatrix)
{
	check(NumRows >= 1 && NumRows <= 4);
	check(NumColumns >= 1 && NumColumns <= 4);
	check(!bIsDoubleInverseMatrix || (InScalarKind == EScalarKind::Double && NumRows == 4 && NumColumns == 4));
	
	FType Type{};
	Type.Kind = ETypeKind::Primitive;
	Type.Primitive.ScalarKind = InScalarKind;
	Type.Primitive.NumRows = NumRows;
	Type.Primitive.NumColumns = NumColumns;
	Type.Primitive.bIsLWCInverseMatrix = bIsDoubleInverseMatrix;

	return Type;
}

FType FType::MakeAggregate(const UMaterialAggregate* InAggregate)
{
	FType Type{};
	Type.Kind = ETypeKind::Aggregate;
	Type.Aggregate = InAggregate;
	return Type;
}

FType FType::MakeParameterCollection()
{
	FType Type{};
	Type.Kind = ETypeKind::ParameterCollection;
	return Type;
}

FType FType::MakeTexture2D()
{
	FType Type{};
	Type.Kind = ETypeKind::Texture2D;
	return Type;
}

FType FType::MakeTextureCube()
{
	FType Type{};
	Type.Kind = ETypeKind::TextureCube;
	return Type;
}

FType FType::MakeTexture2DArray()
{
	FType Type{};
	Type.Kind = ETypeKind::Texture2DArray;
	return Type;
}

FType FType::MakeTextureCubeArray()
{
	FType Type{};
	Type.Kind = ETypeKind::TextureCubeArray;
	return Type;
}

FType FType::MakeTexture3D()
{
	FType Type{};
	Type.Kind = ETypeKind::Texture3D;
	return Type;
}

FType FType::MakeTextureExternal()
{
	FType Type{};
	Type.Kind = ETypeKind::TextureExternal;
	return Type;
}

FType FType::MakeVirtualTexture()
{
	FType Type{};
	Type.Kind = ETypeKind::VirtualTexture;
	return Type;
}

FType FType::MakeRuntimeVirtualTexture()
{
	FType Type{};
	Type.Kind = ETypeKind::RuntimeVirtualTexture;
	return Type;
}

FType FType::MakeSubstrateData()
{
	FType Type{};
	Type.Kind = ETypeKind::SubstrateData;
	return Type;
}

FType FType::MakeVTPageTableResult()
{
	FType Type{};
	Type.Kind = ETypeKind::VTPageTableResult;
	return Type;
}

FString FType::GetSpelling() const
{
	if (AsPrimitive())
	{
		FString Spelling;
		switch (Primitive.ScalarKind)
		{
			case EScalarKind::Boolean:Spelling.Append(TEXT("bool")); break;
			case EScalarKind::Integer: Spelling.Append(TEXT("int")); break;
			case EScalarKind::Float: Spelling.Append(TEXT("float")); break;
			case EScalarKind::Double:
				if (Primitive.IsScalar())
				{
					Spelling.Append(TEXT("FWSScalar"));
				}
				else if (Primitive.IsRowVector())
				{
					Spelling.Appendf(TEXT("FWSVector%d"), Primitive.NumColumns);
				}
				else if (Primitive.bIsLWCInverseMatrix)
				{
					Spelling.Append(TEXT("FWSInverseMatrix"));
				}
				else
				{
					Spelling.Append(TEXT("FWSMatrix"));
				}
				return Spelling;
			default: UE_MIR_UNREACHABLE();
		}
		if (Primitive.IsScalar())
		{
			// append nothing
		}
		else if (Primitive.IsRowVector())
		{
			Spelling.Appendf(TEXT("%d"), Primitive.NumColumns);
		}
		else
		{
			Spelling.Appendf(TEXT("%dx%d"), Primitive.NumRows, Primitive.NumColumns);
		}
		return Spelling;
	}
	else if (const UMaterialAggregate* MatAggregate = AsAggregate())
	{
		return MatAggregate->GetName();
	}

	return TypeKindToString(Kind);
}

FType FPrimitive::ToScalarKind(EScalarKind InScalarKind) const
{
	return FType::MakePrimitive(InScalarKind, NumRows, NumColumns);
}

FType FPrimitive::ToScalar() const
{
	return FType::MakeScalar(ScalarKind);
}

FType FPrimitive::ToVector(int InNumColumns) const
{
	return FType::MakeVector(ScalarKind, InNumColumns);
}

UE::Shader::EValueType FType::ToValueType() const
{
	using namespace UE::Shader;

	switch (Kind)
	{
		case ETypeKind::Primitive:
			if (Primitive.IsMatrix())
			{
				if (Primitive.NumRows == 4 && Primitive.NumColumns == 4)
				{
					if (Primitive.ScalarKind == EScalarKind::Float)
					{
						return EValueType::Float4x4;
					}
					else if (Primitive.ScalarKind == EScalarKind::Double)
					{
						return Primitive.bIsLWCInverseMatrix ? EValueType::DoubleInverse4x4 : EValueType::Double4x4;
					}
					else
					{
						return EValueType::Numeric4x4;
					}
				}

				return EValueType::Any;
			}

			check(Primitive.NumRows < 4 && Primitive.NumColumns <= 4);

			switch (Primitive.ScalarKind)
			{
				case EScalarKind::Boolean: return (EValueType)((int)EValueType::Bool1 + Primitive.NumColumns - 1);
				case EScalarKind::Integer: 	return (EValueType)((int)EValueType::Int1 + Primitive.NumColumns - 1);
				case EScalarKind::Float: return (EValueType)((int)EValueType::Float1 + Primitive.NumColumns - 1);
				case EScalarKind::Double:	return (EValueType)((int)EValueType::Double1 + Primitive.NumColumns - 1);
				default: UE_MIR_UNREACHABLE();
			}

		case ETypeKind::Aggregate:
			return EValueType::Struct;

		default:
			return EValueType::Object;
	}
}

bool FType::operator==(FType Other) const
{
	if (Kind != Other.Kind)
	{
		return false;
	}
	switch (Kind)
	{
		case ETypeKind::Primitive: return Primitive == Other.Primitive;
		case ETypeKind::Aggregate: return Aggregate == Other.Aggregate;
		default: return true;
	}
}

uint32 GetTypeHash(const FType& T)
{
	uint32 Hash = ::GetTypeHash((uint32)T.Kind);
	if (T.Kind == ETypeKind::Primitive)
	{
		static_assert(sizeof(FPrimitive) == sizeof(uint32));
		union
		{
			FPrimitive Primitive;
			uint32     Uint;
		} Test;
		Test.Primitive = T.Primitive;
		Hash = HashCombine(Hash,::GetTypeHash(Test.Uint));
	}
	else if (T.Kind == ETypeKind::Aggregate)
	{
		Hash = HashCombine(Hash, ::GetTypeHash(T.Aggregate));
	}
	return Hash;
}

} // namespace MIR

#endif // #if WITH_EDITOR
