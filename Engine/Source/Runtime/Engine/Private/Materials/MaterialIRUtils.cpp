// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialIRUtils.h"

#if WITH_EDITOR

namespace MIR
{

uint32 HashBytes(const void* InPtr, uint32 Size)
{
	if (!ensure((uintptr_t)InPtr % 4 == 0))
	{
		check(false);
		return 0;
	}

	uint32 Hash = 0;

	const uint32* WordPtr = (const uint32*)InPtr;
	uint32 NumWords = Size / 4;
	for (uint32 i = 0; i < NumWords; ++i)
	{
		Hash = HashCombineFast(Hash, WordPtr[i]);
	}

	const uint8* EndPtr = (const uint8*)(WordPtr + NumWords);

	uint32 End = 0;
	switch (Size % 4)
	{
		case 3: End |= uint32(EndPtr[2]) << 16; // fallthrough
		case 2: End |= uint32(EndPtr[1]) << 8;  // fallthrough
		case 1: End |= uint32(EndPtr[0]);        // fallthrough
			Hash = HashCombineFast(Hash, End);
		default: break;
	}

	return Hash;
}

} // namespace MIR

#endif // #if WITH_EDITOR
