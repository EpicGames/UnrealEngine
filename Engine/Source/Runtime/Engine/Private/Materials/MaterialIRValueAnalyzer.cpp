// Copyright Epic Games, Inc. All Rights Reserved.

#include "Runtime/Engine/Private/Materials/MaterialIRValueAnalyzer.h"

#if WITH_EDITOR

#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialExpressionDBufferTexture.h"
#include "Materials/MaterialExpressionUtils.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRDebug.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialInsights.h"
#include "Materials/Material.h"
#include "MaterialCachedData.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "ParameterCollection.h"
#include "Engine/Font.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "VT/RuntimeVirtualTexture.h"
#include "UObject/Package.h"
#include "Shader/Preshader2.h"

#include "ShaderCompiler.h"

static void AnalysisError_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FAnalysisError* AnalysisError)
{
	Analyzer.Module->AddError(AnalysisError->Expression, AnalysisError->Message.ToString());
}

static void PartialDerivative_AnalyzeInStage(FMaterialIRValueAnalyzer& Analyzer, MIR::FPartialDerivative* PartialDerivative, MIR::EStage Stage)
{
	if (Stage == MIR::Stage_Vertex)
	{
		Analyzer.Module->AddError(nullptr, FString::Printf(TEXT("Screen-space partial derivative DD%c cannot be computed in the vertex stage."), 'X' + (int32)PartialDerivative->Axis));
	}
}

static EMaterialShaderFrequency MapToMaterialShaderFrequencyOrAny(MIR::EStage Stage)
{
	switch (Stage)
	{
		case MIR::EStage::Stage_Vertex: return EMaterialShaderFrequency::Vertex;
		case MIR::EStage::Stage_Pixel: return EMaterialShaderFrequency::Pixel;
		case MIR::EStage::Stage_Compute: return EMaterialShaderFrequency::Compute;
	}
	return EMaterialShaderFrequency::Any;
}

// Maps texture EMaterialValueType values to EMaterialTextureParameterType.
static EMaterialTextureParameterType TextureMaterialValueTypeToParameterType(EMaterialValueType Type)
{
	switch (Type)
	{
		case MCT_Texture2D: return EMaterialTextureParameterType::Standard2D;
		case MCT_Texture2DArray: return EMaterialTextureParameterType::Array2D;
		case MCT_TextureCube: return EMaterialTextureParameterType::Cube;
		case MCT_TextureCubeArray: return EMaterialTextureParameterType::ArrayCube;
		case MCT_VolumeTexture: return EMaterialTextureParameterType::Volume;
		case MCT_TextureVirtual: return EMaterialTextureParameterType::Virtual;
		default: UE_MIR_UNREACHABLE();
	}
}

static void Extern_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FExtern* Extern)
{
	MIR::FExternAnalysisContext Context = {
		.MaterialInterface = Analyzer.MaterialInterface,
		.Module = Analyzer.Module,
		.Extern_Internal = Extern,
	};  
	Extern->Analyze(Context);
}

static void Extern_AnalyzeInStage(FMaterialIRValueAnalyzer& Analyzer, MIR::FExtern* Extern, MIR::EStage Stage)
{
	MIR::FExternAnalysisContext Context = {
		.MaterialInterface = Analyzer.MaterialInterface,
		.Module = Analyzer.Module,
		.Extern_Internal = Extern,
	};  
	Extern->AnalyzeInStage(Context, Stage);
}

static int32 AcquireVTStackIndex(FMaterialIRValueAnalyzer& Analyzer, MIR::FVTPageTableRead& VTPageTableRead, bool bGenerateFeedback, int32 TextureUniformIndex, int32 PreallocatedStackTextureIndex = INDEX_NONE)
{
	// Try to find matching VT stack entry, ignoring bGenerateFeedback to share stacks
	for (int32 VTStackIndex = 0; VTStackIndex < Analyzer.VTStacks.Num(); ++VTStackIndex)
	{
		FMaterialIRValueAnalyzer::FVTStackEntry& VTStack = Analyzer.VTStacks[VTStackIndex];

		// Don't reuse a full stack unless this texture is already a layer in it
		const FMaterialVirtualTextureStack& VTStackData = Analyzer.CompilationOutput->UniformExpressionSet.GetVTStack(VTStackIndex);
		const bool bStackIsFull = VTStackData.AreLayersFull() && VTStackData.FindLayer(TextureUniformIndex) == INDEX_NONE;

		if (!bStackIsFull &&
			VTStack.TexCoord == VTPageTableRead.TexCoord &&
			VTStack.AddressU == VTPageTableRead.AddressU &&
			VTStack.AddressV == VTPageTableRead.AddressV &&
			VTStack.MipValue == VTPageTableRead.MipValue &&
			VTStack.MipValueMode == VTPageTableRead.MipValueMode &&
			VTStack.bIsAdaptive == VTPageTableRead.bIsAdaptive &&
			VTStack.AspectRatio == VTPageTableRead.AspectRatio &&
			VTStack.PreallocatedStackTextureIndex == PreallocatedStackTextureIndex)
		{
			VTStack.bGenerateFeedback |= bGenerateFeedback;
			return VTStackIndex;
		}
	}

	// Add new VT stack entry
	FMaterialIRValueAnalyzer::FVTStackEntry& VTStack = Analyzer.VTStacks.AddDefaulted_GetRef();
	VTStack.TexCoord = VTPageTableRead.TexCoord;
	VTStack.bGenerateFeedback = bGenerateFeedback;
	VTStack.AddressU = VTPageTableRead.AddressU;
	VTStack.AddressV = VTPageTableRead.AddressV;
	VTStack.MipValue = VTPageTableRead.MipValue;
	VTStack.MipValueMode = VTPageTableRead.MipValueMode;
	VTStack.bIsAdaptive = VTPageTableRead.bIsAdaptive;
	VTStack.AspectRatio = VTPageTableRead.AspectRatio;
	VTStack.PreallocatedStackTextureIndex = PreallocatedStackTextureIndex;

	return Analyzer.CompilationOutput->UniformExpressionSet.AddVTStack(PreallocatedStackTextureIndex);
}

static void VTPageTableRead_AnalyzeInStage(FMaterialIRValueAnalyzer& Analyzer, MIR::FVTPageTableRead* VTPageTableRead, MIR::EStage Stage)
{
	// The VirtualTextureUniform can be either an RVT (URuntimeVirtualTexture) or a VT (UTexture2D)
	int32 TextureUniformIndex = INDEX_NONE;
	int32 VTLayerIndex = INDEX_NONE;
	uint16 VTPageTableIndex = 0;
	int32 PreallocatedStackTextureIndex = INDEX_NONE;

	if (const MIR::FVirtualTextureUniform* VTUniform = MIR::As<MIR::FVirtualTextureUniform>(VTPageTableRead->VirtualTextureUniform))
	{
		TextureUniformIndex = VTUniform->Analysis_UniformIndex;
		VTLayerIndex = VTUniform->VTLayerIndex;
		VTPageTableIndex = VTUniform->VTPageTableIndex;

		// RVTs have a pre-baked VT layer index (set by the asset), which means they use preallocated VT stacks.
		if (VTUniform->VTLayerIndex != INDEX_NONE)
		{
			PreallocatedStackTextureIndex = Analyzer.MaterialInterface->GetDefaultTextureIdx(VTUniform->VirtualTexture);
			check(PreallocatedStackTextureIndex != INDEX_NONE);
		}
	}
	else
	{
		const MIR::FTextureUniform* TexUniform = MIR::As<MIR::FTextureUniform>(VTPageTableRead->VirtualTextureUniform);
		checkf(TexUniform && TexUniform->Type.IsVirtualTexture(), TEXT("VTPageTableRead's VirtualTextureUniform must be FVirtualTextureUniform or a VirtualTexture FTextureUniform"));
		TextureUniformIndex = TexUniform->Analysis_UniformIndex;
	}

	check(TextureUniformIndex >= 0);

	// Only support GPU feedback from pixel shader
	const bool bGenerateFeedback = VTPageTableRead->bEnableFeedback && Stage == MIR::Stage_Pixel;

	const int32 VTStackIndex = AcquireVTStackIndex(Analyzer, *VTPageTableRead, bGenerateFeedback, TextureUniformIndex, PreallocatedStackTextureIndex);

	VTPageTableRead->VTStackIndex = VTStackIndex;

	// Check if VT layer is already known. Otherwise, acquire VT layer
	if (VTLayerIndex != INDEX_NONE)
	{
		// The layer index in the VT stack is already known, so fetch the page table from the texture object and assign it to the VT stack
		VTPageTableRead->VTPageTableIndex = VTPageTableIndex;
		Analyzer.CompilationOutput->UniformExpressionSet.SetVTLayer(VTStackIndex, VTLayerIndex, TextureUniformIndex);
	}
	else
	{
		int32 AcquiredVTLayerIndex = Analyzer.CompilationOutput->UniformExpressionSet.GetVTStack(VTStackIndex).FindLayer(TextureUniformIndex);
		if (AcquiredVTLayerIndex == INDEX_NONE)
		{
			AcquiredVTLayerIndex = Analyzer.CompilationOutput->UniformExpressionSet.AddVTLayer(VTStackIndex, TextureUniformIndex);
		}
		VTPageTableRead->VTPageTableIndex = AcquiredVTLayerIndex;
	}
}

static void Branch_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FBranch* Branch)
{
	Branch->TrueBlock = Analyzer.Module->AllocateArray<MIR::FBlock>(Analyzer.Module->GetNumEntryPoints());
	MIR::ZeroArray(Branch->TrueBlock);

	Branch->FalseBlock = Analyzer.Module->AllocateArray<MIR::FBlock>(Analyzer.Module->GetNumEntryPoints());
	MIR::ZeroArray(Branch->FalseBlock);
}

static const MIR::FConstant* GetConstantComponent(const MIR::FValue* Constant, int32 Index)
{
	if (const MIR::FConstant* Scalar = MIR::As<MIR::FConstant>(Constant))
	{
		check(Index == 0);
		return Scalar;
	}
	else
	{
		return MIR::As<MIR::FConstant>(MIR::As<MIR::FComposite>(Constant)->GetComponents()[Index]);
	}
}

static UE::Shader::FValue ConstantToShaderValue(const MIR::FValue* Constant)
{
	UE::Shader::FValue DefaultShaderValue;
	DefaultShaderValue.Type = Constant->Type.ToValueType();

	MIR::FPrimitive PrimitiveType = Constant->Type.GetPrimitive();
	for (int i = 0, NumComponents = PrimitiveType.NumComponents(); i < NumComponents; ++i)
	{
		switch (PrimitiveType.ScalarKind)
		{
			case MIR::EScalarKind::Boolean: DefaultShaderValue.Component.Add(GetConstantComponent(Constant, i)->Boolean); break;
			case MIR::EScalarKind::Integer: DefaultShaderValue.Component.Add((int)GetConstantComponent(Constant, i)->Integer); break;
			case MIR::EScalarKind::Float:   DefaultShaderValue.Component.Add(GetConstantComponent(Constant, i)->Float); break;
			case MIR::EScalarKind::Double:  DefaultShaderValue.Component.Add(GetConstantComponent(Constant, i)->Double); break;
		}
	}

	return DefaultShaderValue;
}

static void PrimitiveUniform_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FPrimitiveUniform* PrimitiveUniform)
{
	MIR::FPrimitive PrimitiveType = PrimitiveUniform->Type.GetPrimitive();
	FUniformExpressionSet& UniformExpressionSet = Analyzer.CompilationOutput->UniformExpressionSet;

	// Declare the default value on the Uniform Expression Set and retrieve its uniform buffer offset.
	UE::Shader::FValue DefaultShaderValue = ConstantToShaderValue(PrimitiveUniform->DefaultConstant);

	uint32 DefaultValueOffset = 0;
	if (!MIR::Find(Analyzer.UniformPrimitiveDefaultValueOffsets, DefaultShaderValue, DefaultValueOffset))
	{
		DefaultValueOffset = UniformExpressionSet.AddDefaultParameterValue(DefaultShaderValue);
		Analyzer.UniformPrimitiveDefaultValueOffsets.Add(DefaultShaderValue, DefaultValueOffset);
	}

	// Determine the Material Parameter Type from the primitive uniform's type.
	// TODO: By the way material parameters work we are confined to only a very limited set of types that we could potentially support.
	//       We should rework the parameter system to accept any primitive type: for example, we should split parameters into "primitive" parameters
	//       and all other "object" (e.g. texture) parameters. Then make primitive parameter be of any primitive type and allocate them onto the
	//       uniform buffer efficiently.
	EMaterialParameterType MaterialParameterType;
	if (PrimitiveUniform->Type == MIR::FType::MakeFloatScalar())
	{
		MaterialParameterType = EMaterialParameterType::Scalar;
	}
	else if (PrimitiveUniform->Type == MIR::FType::MakeFloatVector(4))
	{
		MaterialParameterType = EMaterialParameterType::Vector;
	}
	else if (PrimitiveUniform->Type == MIR::FType::MakeDoubleVector(4))
	{
		MaterialParameterType = EMaterialParameterType::DoubleVector;
	}
	else
	{
		Analyzer.Module->AddError(nullptr, FString::Printf(TEXT("Primitive uniforms can currently only be of these types: float, float4 or double4. Specified type is '%s'."), *PrimitiveUniform->Type.GetSpelling()));
		return;
	}

	// Register the numeric parameter on the Uniform Expression Set.
	// A few important remarks and findings: FindOrAddNumericParameter` API should be improved upon. It has several "leaks":
	//    - It uses MaterialParameterXXX which are front-end concepts. It should instead have its own, decoupled way to specify a uniform, and the material should keep a "binding table" between material parameters (frontend) and uniform values (backend).
	//      This binding table would be generated by the translator and stored in MaterialCompilationOutput for instance.
	//    - With the change above, the Material runtime would "push" values to the uniform buffer, rather than the latter "pulling" from the Material, in a sort of backwards direction. (Things should always go one direction, from higher level to lower level).
	//    - UE::Shader::FValue isn't great either. It encompasses ALL possible values, coupling together primitive and "objects" for no reason. FindOrAddNumericParameter should take a "PrimitiveValue" as the default as it makes no sense to specify
	//      a non primitive Value here, or, even better, take no default value at all, and delegate it to the Material runtime to provide some value, default or overriden by the user. Another example of "backwards" dependency.
	PrimitiveUniform->Analysis_UniformIndex = UniformExpressionSet.FindOrAddNumericParameter(MaterialParameterType, { FName(PrimitiveUniform->Name) }, DefaultValueOffset);

	// Allocate the space for the parameter on the uniform buffer.
	int32 PreshaderIndex = Analyzer.Preshader->AddValueInfo(PrimitiveUniform);
	 
	// Add the parameter evaluation to the uniform data.  Note that we don't know the used component count until the entire
	// tree has been processed, so the final component count is filled in during fixup.
	UniformExpressionSet.AddNumericParameterEvaluation(PrimitiveUniform->Analysis_UniformIndex, PreshaderIndex, MaterialParameterType == EMaterialParameterType::Scalar ? 1 : 4);
}

static int32 RegisterTextureUniform(FMaterialIRValueAnalyzer& Analyzer, FName Name, UObject* TextureObject, EMaterialValueType TextureValueType, int32 VTLayerIndex)
{
	check(VTLayerIndex <= UINT8_MAX);

	// TODO: TextureIndex should *not* rely on MaterialInterface->GetDefaultTextureIdx, as this relies on CachedExpressionData which
	//       should be removed entirely. Instead, we should investigate what this TextureIndex is, and instead determine it here directly.
	const FMaterialTextureParameterInfo TextureParameterInfo
	{
		.ParameterInfo            = { Name },
		.TextureIndex             = Analyzer.MaterialInterface->GetDefaultTextureIdx(TextureObject),
		.SamplerSource            = SSM_FromTextureAsset,
		.VirtualTextureLayerIndex = (uint8)VTLayerIndex,
	};

	check(TextureParameterInfo.TextureIndex != INDEX_NONE);

	const EMaterialTextureParameterType ParamType = TextureMaterialValueTypeToParameterType(TextureValueType);
	return Analyzer.CompilationOutput->UniformExpressionSet.FindOrAddTextureParameter(ParamType, TextureParameterInfo);
}

static int32 RegisterTextureExternalUniform(FMaterialIRValueAnalyzer& Analyzer, UTexture* TextureObject)
{
	// It should not be possible to have nullptr here as we fail before translation
	check(TextureObject);

	const FMaterialExternalTextureParameterInfo ExternalTextureInfo
	{
		.ExternalTextureGuid = TextureObject->GetExternalTextureGuid(),
		.SourceTextureIndex = Analyzer.MaterialInterface->GetDefaultTextureIdx(TextureObject),
	};

	return Analyzer.CompilationOutput->UniformExpressionSet.FindOrAddExternalTextureParameter(ExternalTextureInfo);
}

static void TextureUniform_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FTextureUniform* TextureUniform)
{
	// We need to handle downgrading virtual texture to texture 2D when virtual texturing is disabled in the project setting
	EMaterialValueType TextureValueType = TextureUniform->Texture->GetMaterialType();
	if (TextureValueType == MCT_TextureVirtual && !TextureUniform->Type.IsVirtualTexture())
	{
		TextureValueType = MCT_Texture2D;
	}
	if (TextureValueType == MCT_TextureExternal)
	{
		TextureUniform->Analysis_UniformIndex = RegisterTextureExternalUniform(Analyzer, TextureUniform->Texture);
	}
	else
	{
		TextureUniform->Analysis_UniformIndex = RegisterTextureUniform(Analyzer, FName(TextureUniform->Name), TextureUniform->Texture, TextureValueType, UINT8_MAX);
	}
}

static void VirtualTextureUniform_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FVirtualTextureUniform* VirtualTextureUniform)
{
	VirtualTextureUniform->Analysis_UniformIndex = RegisterTextureUniform(Analyzer, FName(VirtualTextureUniform->Name), VirtualTextureUniform->VirtualTexture, MCT_TextureVirtual, VirtualTextureUniform->VTLayerIndex);
}

static void PreshaderParameter_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FPreshaderParameter* Parameter)
{
	FUniformExpressionSet& UniformExpressionSet = Analyzer.CompilationOutput->UniformExpressionSet;

	// Get parameter name from source parameter
	FName SourceParameterName;
	if (MIR::FTextureUniform* TextureUniform = Parameter->SourceParameter->As<MIR::FTextureUniform>())
	{
		SourceParameterName = FName{ TextureUniform->Name };
	}
	else if (MIR::FVirtualTextureUniform* VirtualTextureUniform = Parameter->SourceParameter->As<MIR::FVirtualTextureUniform>())
	{
		SourceParameterName = FName{ VirtualTextureUniform->Name };
	}
	else
	{
		SourceParameterName = NAME_None;
	}

	// Make sure the parameter type is primitive
	MIR::FPrimitive PrimitiveType = Parameter->Type.GetPrimitive();

	// Only int, float and LWC parameters supported for now
	check(PrimitiveType.IsInteger() || PrimitiveType.IsAnyFloat());

	// Get Preshader2 type, and if it's an RVT double parameter, convert its type as LWC.  These are never read by
	// other non-uniforms, and so are directly stored as LWC by the op.
	EPreshader2Type Preshader2Type = FMaterialIRPreshader::ToPreshader2Type(PrimitiveType.ScalarKind);

	if (Parameter->Opcode == EPreshader2Opcode::RuntimeVirtualTextureUniform && Preshader2Type == EPreshader2Type::Double)
	{
		Preshader2Type = EPreshader2Type::LWC;
	}

	int32 PreshaderIndex = Analyzer.Preshader->AddValueInfo(Parameter);

	const FHashedMaterialParameterInfo HashedSourceParameterInfo{ SourceParameterName, EMaterialParameterAssociation::GlobalParameter, INDEX_NONE };

	// UniformIndex argument only applies for RuntimeVirtualTextureUniform, but it's harmless to pass it in regardless.
	UE::Preshader2::EmitTextureOp(Analyzer.Preshader->Data, Parameter->Opcode,
		Preshader2Type, (uint8)PrimitiveType.NumComponents(), (uint16)PreshaderIndex,
		HashedSourceParameterInfo, Parameter->TextureIndex, Parameter->Payload.UniformIndex);
}

static void SetMaterialOutput_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FSetMaterialOutput* SetMaterialOutput)
{
	// If this output is evaluated in the vertex stage, ensure its input is also valid there.
	// Inputs marked "PixelStageOnly" cannot be used when generating vertex-stage code
	if (SetMaterialOutput->Frequency == MIR::EMaterialOutputFrequency::PerVertex && SetMaterialOutput->Arg->HasSubgraphProperties(MIR::EGraphProperties::PixelStageOnly))
	{
		Analyzer.Module->AddError({}, FString::Printf(TEXT("Input to output '%s' is not available during the vertex stage."), *SetMaterialOutput->Name));
		return;
	}
	
	// Normal output cannot read the pixel normal
	if (SetMaterialOutput->Property == EMaterialProperty::MP_Normal && SetMaterialOutput->HasSubgraphProperties(MIR::EGraphProperties::ReadsPixelNormal))
	{
		Analyzer.Module->AddError(nullptr, TEXT("Cannot set material attribute %s to a value that depends on reading the pixel normal, as that would create a circular dependency."));
		return;
	}

	// Register the material output argument into the module. This "declares" the final output value of a material property.
	if (SetMaterialOutput->Property < MP_MaterialAttributes)
	{
		Analyzer.Module->SetPropertyValue(SetMaterialOutput->Property, SetMaterialOutput->Arg);
	}
}

static void MaterialParameterCollection_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FMaterialParameterCollection* MaterialParameterCollection)
{
	int32 CollectionIndex = Analyzer.Module->FindOrAddParameterCollection(MaterialParameterCollection->Collection);
		
	if (CollectionIndex == INDEX_NONE)
	{
		Analyzer.Module->AddError(nullptr, TEXT("Material references too many MaterialParameterCollections!  A material may only reference 2 different collections."));
	}

	MaterialParameterCollection->Analysis_CollectionIndex = CollectionIndex;
}

static void GetVertexInterpolator_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FGetVertexInterpolator* GetVertexInterpolator)
{
	// Record in the compilation output that the material is using vertex interpolators
	Analyzer.Module->GetCompilationOutput().bUsesVertexInterpolator = true;

	// Since this value is being analyzed it means it is actually used by the material to compute some output.
	// Push its paired set instruction to the list of vertex stage entry points to make it enabled and visible, so that it gets analyzed too.
	Analyzer.Module->GetEntryPoint(MIR::EStage::Stage_Vertex).Outputs.Push(GetVertexInterpolator->SetInstr);
}

static void SetVertexInterpolator_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FSetVertexInterpolator* SetVertexInterpolator)
{
	int32& NumInterpRegisters = Analyzer.Module->GetStatistics().NumUsedVertexInterpolatorFloatRegisters;

	// Allocate the appropriat number of slots (float scalars) used by this interpolator.
	SetVertexInterpolator->Analysis_BaseInterpolatorFloatRegister = NumInterpRegisters;
	NumInterpRegisters += SetVertexInterpolator->Arg->Type.GetPrimitive().NumComponents();

	// Validate that not too many interpolators are being used.
	// @massimo.tristano todo: expose a way to control the maximum number of interpolators.
	const int32 MaxNumVertexInterpolatorSlots = 8 * 2;
	if (NumInterpRegisters > MaxNumVertexInterpolatorSlots)
	{
		Analyzer.Module->AddError({}, FString::Printf(TEXT("Maximum number of vertex interpolator exceeded (%d)."), MaxNumVertexInterpolatorSlots));
		return;
	}
}

static void Call_Analyze(FMaterialIRValueAnalyzer& Analyzer, MIR::FCall* Call)
{
	if (Call->Function->UniqueId == 0)
	{
		const MIR::FFunctionHLSL* FunctionHLSL = static_cast<const MIR::FFunctionHLSL*>(Call->Function);
		Analyzer.Module->AddFunctionHLSL(FunctionHLSL);
		uint32 UniqueId = Analyzer.Module->GetFunctionHLSLs().Num();
		check(UniqueId <= UINT16_MAX);
		Call->Function->UniqueId = UniqueId;
	}
}

static void Call_AnalyzeInStage(FMaterialIRValueAnalyzer& Analyzer, MIR::FCall* Call, MIR::EStage Stage)
{
	Call->Function->Analysis_CallStageMask |= MIR::StageToMask(Stage);
}

static void TextureUniform_AnalyzeInStage(FMaterialIRValueAnalyzer& Analyzer, MIR::FTextureUniform* TextureUniform, MIR::EStage Stage)
{
	EMaterialTextureParameterType ParamType;
	switch (TextureUniform->Type.GetKind())
	{
		case MIR::ETypeKind::Texture2D:			ParamType = EMaterialTextureParameterType::Standard2D; break;
		case MIR::ETypeKind::TextureCube:		ParamType = EMaterialTextureParameterType::Cube;       break;
		case MIR::ETypeKind::Texture2DArray:	ParamType = EMaterialTextureParameterType::Array2D;    break;
		case MIR::ETypeKind::TextureCubeArray:	ParamType = EMaterialTextureParameterType::ArrayCube;  break;
		case MIR::ETypeKind::Texture3D:			ParamType = EMaterialTextureParameterType::Volume;     break;
		default:								return;		// VT / RVT handled via VT stacks
	}

	EShaderFrequency Frequency;
	switch (Stage)
	{
		case MIR::EStage::Stage_Vertex:		Frequency = SF_Vertex;	break;
		case MIR::EStage::Stage_Pixel:		Frequency = SF_Pixel;	break;
		case MIR::EStage::Stage_Compute:	Frequency = SF_Compute;	break;
		default:							checkNoEntry();			return;
	}

	Analyzer.CompilationOutput->UniformExpressionSet.AddTextureShaderFrequencyMask(
		ParamType,
		TextureUniform->Analysis_UniformIndex,
		static_cast<EShaderFrequencyMask>(1u << Frequency)
	);
}

static void RecurseMarkPreshaderTemp(MIR::FValue* Value)
{
	for (MIR::FValue* Use : Value->GetUses())
	{
		// Check if this value has already been marked as preshader (either Out or Temp).
		// If so, we can also assume its children have already been marked.
		if (Use &&
			!Use->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut) &&
			!Use->HasSubgraphProperties(MIR::EGraphProperties::PreshaderTemp) &&
			Use->HasSubgraphProperties(MIR::EGraphProperties::HasParameter))
		{
			Use->SetSubgraphProperties(MIR::EGraphProperties::PreshaderTemp);

			// TODO: the builder already performs analysis graph crawling. We should avoid doing further ad-hoc crawls 
			// and instead integrate this logic inside the existing behavior.
			RecurseMarkPreshaderTemp(Use);
		}
	}
}

// Marks the given value as PreshaderOut, if the value's properties allow it, and it's not already marked.
static void MarkPreshaderOut(FMaterialIRValueAnalyzer& Analyzer, MIR::FValue* Value)
{
	// Check if this is the first time we're marking the value as a preshader output.
	if (!Value->HasSubgraphProperties(MIR::EGraphProperties::Uniform | MIR::EGraphProperties::HasParameter) || Value->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut))
	{
		return;
	}

	// Both Uniform and HasParameter are set, mark it as PreshaderOut, and its children as PreshaderTemp if not already marked.
	Value->SetSubgraphProperties(MIR::EGraphProperties::PreshaderOut);
	Value->GraphProperties &= ~MIR::EGraphProperties::PreshaderTemp;

	// If this is an alias, mark the underlying value as PreshaderOut as well.  This may waste some space in the output buffer
	// if there is a mix of used and unused components, but this is assumed to be rare, and we want to err on the side of avoiding
	// the need to generate extra move opcodes, to save CPU perf.
	if (Value->Analysis_PreshaderOffset)
	{
		MIR::FValue* Alias = Analyzer.Preshader->ValueInfos[Value->Analysis_PreshaderOffset].Alias;
		if (Alias)
		{
			Alias->SetSubgraphProperties(MIR::EGraphProperties::PreshaderOut);
			Alias->GraphProperties &= ~MIR::EGraphProperties::PreshaderTemp;
		}
	}

	RecurseMarkPreshaderTemp(Value);

	// Composite is emitted here, as their emission is deferred, assuming composites will often be folded into operators or
	// dead stripped by the trivial tail swizzle optimization, unless there is a case where the final result actually needs
	// to be available on the GPU (the case we are encountering here).
	if (!Value->Analysis_PreshaderOffset)
	{
		if (MIR::FComposite* Composite = Value->As<MIR::FComposite>())
		{
			Analyzer.Preshader->EmitComposite(Composite);
		}
		else
		{
			// All other preshader compatible child expressions should already have been processed.
			UE_MIR_UNREACHABLE();
		}
	}
}

// Trivial Tail Swizzle optimization -- if this use is a swizzle (Composite of Subscripts all pointing to the same
// vector input value, or all identical values), we want to mark that underlying value as "PreshaderOut" rather than
// this one.  This prevents trivial tail swizzles from generating extra CPU preshader opcodes, given that swizzle on
// load is free.  For example, this is a common pattern that occurs when 3 components of a vector param are fetched:
//
//     _0 = Composite( Subscript(Param,0), Subscript(Param,1), Subscript(Param,2) );
//
// We're trying to avoid this generating an ".xyz" swizzle on the CPU.  For this example, the swizzle has the same
// order of components, but this optimization also applies if the components are reordered.  We want to apply this
// optimization here, to simplify downstream logic, which may involve multiple passes over the tree.  We can also
// accept constants as any of the parameters, as swizzling in a constant is also free.
static bool ApplyTrivialTailSwizzle(FMaterialIRValueAnalyzer& Analyzer, MIR::FValue* Use)
{
	if (!(Use->IsA(MIR::VK_Composite) && Use->Type.IsVector() && Use->GetUses().Num() > 0))
	{
		return false;
	}

	TConstArrayView<MIR::FValue*> Components = Use->As<MIR::FComposite>()->GetComponents();
	MIR::FSubscript* FirstSubscript = nullptr;
	MIR::FValue* FirstValue = nullptr;

	// Assume it's trivial until we find a case where it's not.
	bool bTrivialTailSwizzle = true;

	for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		// Constants can just be ignored
		if (Components[ComponentIndex]->IsA(MIR::VK_Constant))
		{
			continue;
		}

		if (Components[ComponentIndex]->IsA(MIR::VK_Subscript))
		{
			// If there's a mix of (non-constant) values and subscripts, it's not compatible with trivial tail swizzle!
			if (FirstValue)
			{
				bTrivialTailSwizzle = false;
				break;
			}

			MIR::FSubscript* CurrentSubscript = Components[ComponentIndex]->As<MIR::FSubscript>();
			if (!FirstSubscript)
			{
				FirstSubscript = CurrentSubscript;
			}
			else if (FirstSubscript->Arg != CurrentSubscript->Arg)
			{
				bTrivialTailSwizzle = false;
				break;
			}
		}
		else
		{
			// If there's a mix of (non-constant) values and subscripts, it's not compatible with trivial tail swizzle!
			if (FirstSubscript)
			{
				bTrivialTailSwizzle = false;
				break;
			}

			if (!FirstValue)
			{
				FirstValue = Components[ComponentIndex];
			}
			else if (FirstValue != Components[ComponentIndex])
			{
				bTrivialTailSwizzle = false;
				break;
			}
		}
	}

	if (bTrivialTailSwizzle)
	{
		if (FirstSubscript)
		{
			// Subscripts are always aliases of the underlying arg, so we need to mark that as used by non-uniform.
			MarkPreshaderOut(Analyzer, FirstSubscript->Arg);
		}
		else if (FirstValue)
		{
			MarkPreshaderOut(Analyzer, FirstValue);
		}

		Use->SetSubgraphProperties(MIR::EGraphProperties::TrivialTailSwizzle);
	}

	return bTrivialTailSwizzle;
}

// Handles logic that involves analyzing Uses of the given value.  Updates used component masks and sets PreshaderOut where needed.
static void AnalyzeUses(FMaterialIRValueAnalyzer& Analyzer, MIR::FValue* Value)
{
	for (MIR::FValue* Use : Value->GetUses())
	{
		if (!Use)
		{
			continue;
		}

		// Updated used component mask of uniform values.  Can't be done in UniformParameter_Analyze, because it needs
		// access to the parent Value to fetch its subscript index.
		// TODO: this code fights a little with the way the analyzer is designed. We already have a facility to analyze individual
		//       nodes in the graph (i.e. PrimitiveUniform_Analyze()).
		if (MIR::FPrimitiveUniform* UniformParameter = Use->As<MIR::FPrimitiveUniform>())
		{
			if (UniformParameter->Type.IsScalar())
			{
				UniformParameter->Analysis_ComponentMask |= (1 << 0);
			}
			else if (UniformParameter->Type.IsVector())
			{
				if (Value->IsA(MIR::VK_Subscript))
				{
					UniformParameter->Analysis_ComponentMask |= 1 << FMath::Clamp(Value->As<MIR::FSubscript>()->Index, 0, 3);
				}
				else
				{
					UniformParameter->Analysis_ComponentMask |= 0xf;
				}
			}
		}
		
		// If this graph node is not Uniform, attempt to mark any of its children as PreshaderOut where necessary.
		if (!(Value->GraphProperties & MIR::EGraphProperties::Uniform))
		{	
			if (!ApplyTrivialTailSwizzle(Analyzer, Use))
			{
				MarkPreshaderOut(Analyzer, Use);
			}
		}
	}
}


//////////////////////////////////////////////////////////////////////////////////////////
// FMaterialIRValueAnalyzer
//////////////////////////////////////////////////////////////////////////////////////////

void FMaterialIRValueAnalyzer::Setup(UMaterialInterface* InMaterialInterface, FMaterialIRModule* InModule, FMaterialIRPreshader* InPreshader, FMaterialInsights* InInsights)
{
	MaterialInterface = InMaterialInterface;
	Module = InModule;
	Insights = InInsights;
	Preshader = InPreshader;
	CompilationOutput = &InModule->GetCompilationOutput();
	UniformPrimitiveDefaultValueOffsets = {};
	EnvironmentDefines = {};
	VTStacks = {};
}

void FMaterialIRValueAnalyzer::Analyze(MIR::FValue* Value)
{
	// Flow the graph properties downstream from the value's uses into this value.
	for (MIR::FValue* Use : Value->GetUses())
	{
		if (Use)
		{
			// Note that we don't put this UnionMask of flags in the enum class, because it breaks Natvis flag display.
			const MIR::EGraphProperties UnionMask = MIR::EGraphProperties::PixelStageOnly | MIR::EGraphProperties::ReadsPixelNormal | MIR::EGraphProperties::HasParameter;

			Value->GraphProperties |= Use->GraphProperties & UnionMask;

			// Uniform property is a logical intersection (AND) rather than union (OR), but we also need to make sure we don't
			// clear any of the unioned properties, so always include those in the AND mask.
			Value->GraphProperties &= (Use->GraphProperties & MIR::EGraphProperties::Uniform) | UnionMask;
		}
	}

	// Analyze this value uses
	AnalyzeUses(*this, Value);

	#define EXPAND_CASE(ValueType) \
		case MIR::VK_##ValueType: \
		{ \
			ValueType##_Analyze(*this, static_cast<MIR::F##ValueType*>(Value)); \
			break; \
		}

	// IMPORTANT: Before adding a case here, read the FMaterialIRValueAnalyzer documentation.
	switch (Value->Kind)
	{
		EXPAND_CASE(AnalysisError);
		EXPAND_CASE(MaterialParameterCollection);
		EXPAND_CASE(PrimitiveUniform);
		EXPAND_CASE(TextureUniform);
		EXPAND_CASE(VirtualTextureUniform);
		EXPAND_CASE(Branch);
		EXPAND_CASE(PreshaderParameter);
		EXPAND_CASE(SetMaterialOutput);
		EXPAND_CASE(Extern);
		EXPAND_CASE(GetVertexInterpolator);
		EXPAND_CASE(SetVertexInterpolator);
		EXPAND_CASE(Call);
		default: break;
	}

	#undef EXPAND_CASE

	// After analyzing the value, check if this is a preshader value (Uniform with a Parameter in its subtree),
	// and if so, emit preshader code for it.  Note that the emitter only sets Uniform for operators that are
	// supported, so we don't have to worry about hitting this code for unsupported operators.
	if (Value->HasSubgraphProperties(MIR::EGraphProperties::Uniform | MIR::EGraphProperties::HasParameter))
	{
		Preshader->EmitValue(Value);
	}
}

void FMaterialIRValueAnalyzer::AnalyzeInStage(MIR::FValue* Value, MIR::EStage Stage)
{
	#define EXPAND_CASE(ValueType) \
	case MIR::VK_##ValueType: \
	{ \
		ValueType##_AnalyzeInStage(*this, static_cast<MIR::F##ValueType*>(Value), Stage); \
		break; \
	}

	// IMPORTANT: Before adding a case here, read the FMaterialIRValueAnalyzer documentation.
	switch (Value->Kind)
	{
		EXPAND_CASE(Extern);
		EXPAND_CASE(VTPageTableRead);
		EXPAND_CASE(PartialDerivative);
		EXPAND_CASE(Call);
		EXPAND_CASE(TextureUniform);
		default: break;
	}

	#undef EXPAND_CASE
}

#endif // #if WITH_EDITOR
