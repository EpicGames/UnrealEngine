// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRDebug.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIRInternal.h"
#include "Materials/MaterialInsights.h"
#include "Materials/MaterialIRPreshader.h"
#include "MaterialShared.h"
#include "Engine/Texture.h"

// It analyzes a value or instruction, performing semantic validation and side-effects execution.
//
// These analyze functions allow a value or instruction to perform non-trivial custom operations  
// and validation. These operations may have side effects, such as setting state in  
// CompilationOutput, allocating resources, etc. The reason these are done here (rather than at  
// FValue emission time) is that only values which are *actually* used (not pruned as unused or  
// optimized out) are analyzed. In other words, while implementing a value's analyze function, you  
// are guaranteed that the value is needed in the final material.
//
// Only values that require this kind of post-emission validation or that generate sidecar  
// resources should define Analyze functions. Otherwise, they can omit them.

// There are two types of analyze functions: Analyze() and AnalyzeInStage().
// - Analyze() is invoked once, regardless of which stage (vertex, pixel, or both) the value is  
//   scheduled for. It runs before any per-stage logic.
// - Some values require all or part of their analysis logic to run per stage. These should  
//   implement AnalyzeInStage() with that stage-specific logic.
//
// IMPORTANT: Only give a value AnalyzeInStage() if it needs per-stage analysis. If the logic is  
// stage-agnostic, it belongs in Analyze(). A value can implement both Analyze() and  
// AnalyzeInStage() if it needs general logic once and other logic per stage.
//
// # GraphProperties  
// Analyze functions can set and read GraphProperty flags. These are per-value bitflags that are  
// automatically propagated as the IR graph is analyzed. When a value receives its Analyze() call,  
// all its dependencies have already been analyzed, so it can freely inspect their GraphProperties.  
// For example, it can verify that no other value sets a specific graph property upstream.
struct FMaterialIRValueAnalyzer
{
	// Resets the analyzer to process a new translation run.
	void Setup(UMaterialInterface* InMaterialInterface, FMaterialIRModule* InModule, FMaterialIRPreshader* InPreshader, FMaterialInsights* InInsights = nullptr);
		
	// Performs stage-agnostic analysis on the given value.
	void Analyze(MIR::FValue* Value);

	// Performs stage-specific analysis on the given value, if the value kind requires some.
	void AnalyzeInStage(MIR::FValue* Value, MIR::EStage Stage);

	// The material being built.
	UMaterialInterface* MaterialInterface{};
	
	// The destination module that will contain the result of material translation.
	FMaterialIRModule* Module{};
	
	// Optional. Specifies the target MaterialInsights to populate, if provided.
	FMaterialInsights* Insights{};

	// Preshader intermediate representation
	FMaterialIRPreshader* Preshader{};

	// The destination compilation output whose state to populate, based on the values being analyzed.
	FMaterialCompilationOutput* CompilationOutput{};
		
	// Maps default values to their default-value offset, as used in UniformExpressionSet (e.g. UniformExpressionSet.AddDefaultParameterValue() and UniformExpressionSet.FindOrAddNumericParameter()).
	TMap<UE::Shader::FValue, uint32> UniformPrimitiveDefaultValueOffsets{};

	// List of enabled shader environment defines.
	TSet<FName> EnvironmentDefines;

	// Translator specific structure during generation of VT stacks. This is later converted to FMaterialVirtualTextureStack entries in FUniformExpressionSet.
	// The equivalent in the old translator is FHLSLMaterialTranslator::FMaterialVTStackEntry.
	struct FVTStackEntry
	{
		MIR::FValue* TexCoord{};
		bool bGenerateFeedback{};
		TextureAddress AddressU{ TA_MAX };
		TextureAddress AddressV{ TA_MAX };
		MIR::FValue* MipValue{};
		ETextureMipValueMode MipValueMode{ TMVM_None };
		bool bIsAdaptive{};
		float AspectRatio{ 1.0f };
		int32 PreallocatedStackTextureIndex{ INDEX_NONE };
	};

	// VT stack entries during analysis. This is later converted to FMaterialVirtualTextureStack entries in FUniformExpressionSet.
	TArray<FVTStackEntry> VTStacks;
};

#endif // #if WITH_EDITOR
