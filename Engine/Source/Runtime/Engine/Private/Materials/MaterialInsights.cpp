// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialInsights.h"

FMaterialInsights::FMaterialInsights()
{
}

void FMaterialInsights::Empty()
{
	ConnectionInsights.Empty();
	UniformParameterAllocationInsights.Empty();
	IRString.Empty();
	Legacy_ShaderStringParameters.Empty();
	LegacyHLSLCode.Empty();
	New_ShaderStringParameters.Empty();
	NewHLSLCode.Empty();
}

