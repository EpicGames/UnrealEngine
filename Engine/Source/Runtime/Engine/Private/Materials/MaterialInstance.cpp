// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialInstance.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "FinalPostProcessSettings.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Stats/StatsMisc.h"
#include "EngineModule.h"
#include "Engine/Font.h"
#include "Engine/Texture.h"
#include "Engine/TextureCollection.h"
#include "Materials/Material.h"
#include "UObject/Package.h"
#include "Materials/MaterialExpressionStaticBoolParameter.h"
#include "UObject/UObjectIterator.h"
#include "MeshUVChannelInfo.h"
#include "UObject/LinkerLoad.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PipelineStateCache.h"
#include "UnrealEngine.h"
#include "MaterialDomain.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionDoubleVectorParameter.h"
#include "Materials/MaterialExpressionTextureCollectionParameter.h"
#include "Materials/MaterialExpressionCollectionParameter.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Materials/MaterialExpressionFontSampleParameter.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSampleParameter.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialExpressionStaticComponentMaskParameter.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceUpdateParameterSet.h"
#include "Materials/MaterialInstanceSupport.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialSharedPrivate.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "Engine/ToonProfile.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ObjectCacheEventSink.h"
#include "Interfaces/ITargetPlatform.h"
#include "RenderUtils.h"
#include "ShaderCodeLibrary.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Misc/ScopedSlowTask.h"
#include "RendererInterface.h"
#include "ShaderPlatformQualitySettings.h"
#include "MaterialShaderQualitySettings.h"
#include "Stats/StatsTrace.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "ShaderCompiler.h"
#include "MaterialCachedData.h"
#include "ComponentRecreateRenderStateContext.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "VT/RuntimeVirtualTexture.h"
#include "LocalVertexFactory.h"
#include "PSOPrecacheMaterial.h"

#if WITH_EDITOR
#include "Cooker/CookDependency.h"
#include "Cooker/CookEvents.h"
#include "Editor.h"
#include "MaterialEditingLibrary.h"
#include "String/ParseTokens.h"
#endif

#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInstance)

DECLARE_CYCLE_STAT(TEXT("MaterialInstance CopyMatInstParams"), STAT_MaterialInstance_CopyMatInstParams, STATGROUP_Shaders);
DECLARE_CYCLE_STAT(TEXT("MaterialInstance Serialize"), STAT_MaterialInstance_Serialize, STATGROUP_Shaders);
DECLARE_CYCLE_STAT(TEXT("MaterialInstance CopyUniformParamsInternal"), STAT_MaterialInstance_CopyUniformParamsInternal, STATGROUP_Shaders);

// MaterialEditor flag that controls whether the material editor is in restrictive mode which entails:
//    - MaterialInstances that generate new shader permutations cannot have a cooked parent that is neither a base-material nor a material in the same module as the instance
//    - Force-disables the new experimental material translator
ENGINE_API bool bEnableRestrictiveMaterialMode = false;

const FMaterialInstanceCachedData FMaterialInstanceCachedData::EmptyData{};

void UMaterialInstance::StartCacheUniformExpressions() const
{
	bCachingUniformExpressions = true;
#if WITH_ODSC
	FODSCManager::RegisterMaterialInstance(this);
#endif
}

void UMaterialInstance::FinishCacheUniformExpressions() const
{
	bCachingUniformExpressions = false;
}

void FMaterialInstanceRenderProxy::StartCacheUniformExpressions() const
{
	Owner->StartCacheUniformExpressions();
}

void FMaterialInstanceRenderProxy::FinishCacheUniformExpressions() const
{
	Owner->FinishCacheUniformExpressions();
}

/**
 * Cache uniform expressions for the given material.
 * @param MaterialInstance - The material instance for which to cache uniform expressions.
 */
void CacheMaterialInstanceUniformExpressions(const UMaterialInstance* MaterialInstance, bool bRecreateUniformBuffer)
{
	if (MaterialInstance->GetInstanceRenderProxy())
	{
		MaterialInstance->GetInstanceRenderProxy()->CacheUniformExpressions_GameThread(bRecreateUniformBuffer);
	}
}

#if WITH_EDITOR
/**
 * Recaches uniform expressions for all material instances with a given parent.
 * WARNING: This function is a noop outside of the Editor!
 * @param ParentMaterial - The parent material to look for.
 */
void RecacheMaterialInstanceUniformExpressions(const UMaterialInterface* ParentMaterial, bool bRecreateUniformBuffer)
{
	if (GIsEditor && FApp::CanEverRender())
	{
		UE_LOGF(LogMaterial,Verbose,"Recaching MI Uniform Expressions for parent %ls", *ParentMaterial->GetFullName());
		TArray<FMICReentranceGuard> ReentranceGuards;
		for (TObjectIterator<UMaterialInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterialInstance* MaterialInstance = *It;
			do 
			{
				if (MaterialInstance->Parent == ParentMaterial)
				{
					UE_LOGF(LogMaterial,Verbose,"--> %ls", *MaterialInstance->GetFullName());
					CacheMaterialInstanceUniformExpressions(*It, bRecreateUniformBuffer);
					break;
				}
				new (ReentranceGuards) FMICReentranceGuard(MaterialInstance);
				MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
			} while (MaterialInstance && !MaterialInstance->GetReentrantFlag());
			ReentranceGuards.Reset();
		}
	}
}
#endif // #if WITH_EDITOR

FFontParameterValue::ValueType FFontParameterValue::GetValue(const FFontParameterValue& Parameter)
{
	ValueType Value = NULL;
	if (Parameter.FontValue && Parameter.FontValue->Textures.IsValidIndex(Parameter.FontPage))
	{
		// get the texture for the font page
		Value = Parameter.FontValue->Textures[Parameter.FontPage];
	}
	return Value;
}

FMaterialInstanceRenderProxy::FMaterialInstanceRenderProxy(UMaterialInstance* InOwner)
	: FMaterialRenderProxy(InOwner->GetName())
	, Parent(NULL)
	, Owner(InOwner)
	, GameThreadParent(NULL)
{
}

void FMaterialInstanceRenderProxy::GameThread_Destroy()
{
	FMaterialInstanceRenderProxy* RenderProxy = this;
	ENQUEUE_RENDER_COMMAND(FDestroyMaterialInstanceResourceCommand)(
		[RenderProxy](FRHICommandList& RHICmdList)
		{
			delete RenderProxy;
		});
}

#if 0
const FMaterial& FMaterialInstanceResource::GetMaterialWithFallback(ERHIFeatureLevel::Type InFeatureLevel, const FMaterialRenderProxy*& OutFallbackMaterialRenderProxy) const
{
	checkSlow(IsInParallelRenderingThread());

	if (Parent)
	{
		if (Owner->bHasStaticPermutationResource)
		{
			const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			FMaterialResource* StaticPermutationResource = FindMaterialResource(Owner->StaticPermutationMaterialResources, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), ActiveQualityLevel, true);
			if (StaticPermutationResource)
			{
				if (StaticPermutationResource->IsRenderingThreadShaderMapComplete())
				{
					// Verify that compilation has been finalized, the rendering thread shouldn't be touching it otherwise
					checkSlow(StaticPermutationResource->GetRenderingThreadShaderMap()->IsCompilationFinalized());
					// The shader map reference should have been NULL'ed if it did not compile successfully
					checkSlow(StaticPermutationResource->GetRenderingThreadShaderMap()->CompiledSuccessfully());
					return *StaticPermutationResource;
				}
				else
				{
					EMaterialDomain Domain = (EMaterialDomain)StaticPermutationResource->GetMaterialDomain();
					UMaterial* FallbackMaterial = UMaterial::GetDefaultMaterial(Domain);
					//there was an error, use the default material's resource
					OutFallbackMaterialRenderProxy = FallbackMaterial->GetRenderProxy();
					return OutFallbackMaterialRenderProxy->GetMaterialWithFallback(InFeatureLevel, OutFallbackMaterialRenderProxy);
				}
			}
		}
		else
		{
			//use the parent's material resource
			return Parent->GetRenderProxy()->GetMaterialWithFallback(InFeatureLevel, OutFallbackMaterialRenderProxy);
		}
	}

	// No Parent, or no StaticPermutationResource. This seems to happen if the parent is in the process of using the default material since it's being recompiled or failed to do so.
	UMaterial* FallbackMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	OutFallbackMaterialRenderProxy = FallbackMaterial->GetRenderProxy();
	return OutFallbackMaterialRenderProxy->GetMaterialWithFallback(InFeatureLevel, OutFallbackMaterialRenderProxy);
}
#endif // 0

const FMaterialRenderProxy* FMaterialInstanceRenderProxy::GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	if (Parent)
	{
		if (Owner->bHasStaticPermutationResource)
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			FMaterialResource* StaticPermutationResource = FindMaterialResource(Owner->StaticPermutationMaterialResources, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), ActiveQualityLevel, true);
			if (StaticPermutationResource)
			{
				EMaterialDomain Domain = (EMaterialDomain)StaticPermutationResource->GetMaterialDomain();
				UMaterial* FallbackMaterial = UMaterial::GetDefaultMaterial(Domain);
				//there was an error, use the default material's resource
				return FallbackMaterial->GetRenderProxy();
			}
		}
		else
		{
			//use the parent's material resource
			return Parent->GetRenderProxy()->GetFallback(InFeatureLevel);
		}
	}

	// No Parent, or no StaticPermutationResource. This seems to happen if the parent is in the process of using the default material since it's being recompiled or failed to do so.
	UMaterial* FallbackMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	return FallbackMaterial->GetRenderProxy();
}

const FMaterial* FMaterialInstanceRenderProxy::GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	checkSlow(IsInParallelRenderingThread());

	if (Parent)
	{
		if (Owner->bHasStaticPermutationResource)
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			const FMaterialResource* StaticPermutationResource = FindMaterialResource(Owner->StaticPermutationMaterialResources, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), ActiveQualityLevel, true);
			if (StaticPermutationResource && StaticPermutationResource->GetRenderingThreadShaderMap())
			{
				return StaticPermutationResource;
			}
		}
		else
		{
			const FMaterialRenderProxy* ParentProxy = Parent->GetRenderProxy();
			if (ParentProxy)
			{
				return ParentProxy->GetMaterialNoFallback(InFeatureLevel);
			}
		}
	}
	return nullptr;
}

UMaterialInterface* FMaterialInstanceRenderProxy::GetMaterialInterface() const
{
	return Owner;
}

bool FMaterialInstanceRenderProxy::GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
{
	checkSlow(IsInParallelRenderingThread());

	bool bResult = false;

	// Check for hard-coded parameters
	if (Type == EMaterialParameterType::Scalar && ParameterInfo.Name == SubsurfaceProfile::GetSubsurfaceProfileParameterName())
	{
		check(ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter);

		// When using Substrate with hidden conversion, this path can still be used. Since GetSubsurfaceProfileRT() won't be correct 
		// (it will retrieve the SSS profile override instead of the actual SSS Profile), we retrieve the SSS profile through the 
		// array version at index 0
		if (Substrate::IsSubstrateEnabled())
		{
			const USubsurfaceProfile* SSProfileOverrideRT = GetSubsurfaceProfileRT();
			const USubsurfaceProfile* SSProfileRT = NumSubsurfaceProfileRT() > 0 ? GetSubsurfaceProfileRT(0) : nullptr;
			// Set the root material Profile, or the profile overriden by any instances.
			OutValue = SubsurfaceProfile::GetSubsurfaceProfileId(SSProfileOverrideRT ? SSProfileOverrideRT : SSProfileRT);
			bResult = true;
		}
		else
		{
			const USubsurfaceProfile* MySubsurfaceProfileRT = GetSubsurfaceProfileRT();
			OutValue = SubsurfaceProfile::GetSubsurfaceProfileId(MySubsurfaceProfileRT);
			bResult = true;
		}
	}
	else if (Type == EMaterialParameterType::Scalar && NumSubsurfaceProfileRT() > 0)
	{
		const USubsurfaceProfile* SSProfileOverrideRT = GetSubsurfaceProfileRT();
		// Substrate general SubsurfaceProfile
		for (uint32 It = 0, Count = NumSubsurfaceProfileRT(); It < Count; ++It)
		{
			const USubsurfaceProfile* SSProfileRT = GetSubsurfaceProfileRT(It);
			if (ParameterInfo.Name == SubsurfaceProfile::CreateSubsurfaceProfileParameterName(SSProfileRT))
			{
				check(ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter);

				// Set the root material Profile, or the profile overriden by any instances.
				OutValue = SubsurfaceProfile::GetSubsurfaceProfileId(SSProfileOverrideRT ? SSProfileOverrideRT : SSProfileRT);
				bResult = true;
				break;
			}
		}
	}
	else if (Type == EMaterialParameterType::Scalar && NumSpecularProfileRT() > 0)
	{
		const USpecularProfile* SPOverrideRT = GetSpecularProfileOverrideRT();
		for (uint32 It=0,Count=NumSpecularProfileRT();It<Count;++It)
		{
			if (ParameterInfo.Name == SpecularProfile::GetSpecularProfileParameterName(GetSpecularProfileRT(It)))
			{
				check(ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter);
				OutValue = SpecularProfile::GetSpecularProfileId(SPOverrideRT ? SPOverrideRT : GetSpecularProfileRT(It));
				bResult = true;
				break;
			}
		}
	}
	else if (Type == EMaterialParameterType::Scalar && NumToonProfileRT() > 0)
	{
		const UToonProfile* SPOverrideRT = GetToonProfileOverrideRT();
		for (uint32 It=0,Count=NumToonProfileRT();It<Count;++It)
		{
			if (ParameterInfo.Name == ToonProfile::GetToonProfileParameterName(GetToonProfileRT(It)))
			{
				check(ParameterInfo.Association == EMaterialParameterAssociation::GlobalParameter);
				OutValue = ToonProfile::GetToonProfileId(SPOverrideRT ? SPOverrideRT : GetToonProfileRT(It));
				bResult = true;
				break;
			}
		}
	}
	if (!bResult)
	{
		// Check for instances overrides
		switch (Type)
		{
			case EMaterialParameterType::StaticSwitch: bResult = RenderThread_GetParameterValue<bool>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::Scalar: bResult = RenderThread_GetParameterValue<float>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::Vector: bResult = RenderThread_GetParameterValue<FLinearColor>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::DoubleVector: bResult = RenderThread_GetParameterValue<FVector4d>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::Texture: bResult = RenderThread_GetParameterValue<const UTexture*>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::TextureCollection: bResult = RenderThread_GetParameterValue<const UTextureCollection*>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::RuntimeVirtualTexture: bResult = RenderThread_GetParameterValue<const URuntimeVirtualTexture*>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::SparseVolumeTexture: bResult = RenderThread_GetParameterValue<const USparseVolumeTexture*>(ParameterInfo, OutValue); break;
			case EMaterialParameterType::ParameterCollection: bResult = RenderThread_GetParameterValue<const UMaterialParameterCollection*>(ParameterInfo, OutValue); break;
			default: ensure(false); break; // other parameter types are not expected on the render thread
		}
	}

	if (!bResult && Parent)
	{
		// Check parent
		FHashedMaterialParameterInfo ParentParameterInfo;
		if (ParameterInfo.RemapLayerIndex(ParentLayerIndexRemap, ParentParameterInfo))
		{
			bResult = Parent->GetRenderProxy()->GetParameterValue(Type, ParentParameterInfo, OutValue, Context);
		}
	}

	return bResult;
}

bool FMaterialInstanceRenderProxy::GetUserSceneTextureOverride(FName& InOutName) const
{
	checkSlow(IsInParallelRenderingThread());

	// Number of overrides possible is small (maximum 6, in most practical cases 1 or 2), and FName comparison cheap,
	// so the assumption is that an array search will be cheaper than the overhead of going through a hash lookup.
	// Plus an array takes half the space of THashedMaterialParameterMap, saving memory.
	for (const FUserSceneTextureOverride& Override : UserSceneTextureOverrides)
	{
		if (Override.Key == InOutName && Override.Value != NAME_None)
		{
			InOutName = Override.Value;
			return true;
		}
	}

	if (Parent)
	{
		return Parent->GetRenderProxy()->GetUserSceneTextureOverride(InOutName);
	}
	else
	{
		return false;
	}
}

EBlendableLocation FMaterialInstanceRenderProxy::GetBlendableLocation(const FMaterial* Base) const
{
	check(Base);
	checkSlow(IsInParallelRenderingThread());

	// Can't be overridden to BL_ReplacingTonemapper
	if (PostProcessBlendableOverrides.bOverrideBlendableLocation && PostProcessBlendableOverrides.BlendableLocationOverride != BL_ReplacingTonemapper)
	{
		// Can't be overridden from BL_ReplacingTonemapper 
		if ((EBlendableLocation)Base->GetBlendableLocation() == BL_ReplacingTonemapper)
		{
			return BL_ReplacingTonemapper;
		}

		return PostProcessBlendableOverrides.BlendableLocationOverride;
	}
	else if (Parent)
	{
		return Parent->GetRenderProxy()->GetBlendableLocation(Base);
	}
	else
	{
		return (EBlendableLocation)Base->GetBlendableLocation();
	}
}

int32 FMaterialInstanceRenderProxy::GetBlendablePriority(const FMaterial* Base) const
{
	check(Base);
	checkSlow(IsInParallelRenderingThread());

	if (PostProcessBlendableOverrides.bOverrideBlendablePriority)
	{
		return PostProcessBlendableOverrides.BlendablePriorityOverride;
	}
	else if (Parent)
	{
		return Parent->GetRenderProxy()->GetBlendablePriority(Base);
	}
	else
	{
		return Base->GetBlendablePriority();
	}
}

void UMaterialInstance::PropagateDataToMaterialProxy()
{
	if (RenderProxy)
	{
		UpdateMaterialRenderProxy(*RenderProxy);
	}
}

void FMaterialInstanceRenderProxy::GameThread_SetParent(UMaterialInterface* ParentMaterialInterface)
{
	// @todo loadtimes: this is no longer valid because of the ParallelFor calling AddPrimitive in UnrealEngine.cpp
	// check(IsInGameThread() || IsAsyncLoading());

	if (GameThreadParent != ParentMaterialInterface)
	{
		// Set the game thread accessible parent.
		UMaterialInterface* OldParent = GameThreadParent;
		GameThreadParent = ParentMaterialInterface;

		// Set the rendering thread's parent and instance pointers.
		check(ParentMaterialInterface != NULL);
		FMaterialInstanceRenderProxy* RenderProxy = this;
		ENQUEUE_RENDER_COMMAND(InitMaterialInstanceResource)(
			[RenderProxy, ParentMaterialInterface](FRHICommandListImmediate& RHICmdList)
			{
				RenderProxy->Parent = ParentMaterialInterface;
				RenderProxy->InvalidateUniformExpressionCache(true);
			});

		if (OldParent)
		{
			// make sure that the old parent sticks around until we've set the new parent on FMaterialInstanceResource
			OldParent->ParentRefFence.BeginFence();
		}
	}
}

void FMaterialInstanceRenderProxy::GameThread_UpdateCachedData(const FMaterialInstanceCachedData& CachedData)
{
	ENQUEUE_RENDER_COMMAND(MaterialInstanceResource_UpdateCachedData)(
		[RenderProxy = this, ParentLayerIndexRemap = CachedData.ParentLayerIndexRemap](FRHICommandListImmediate& RHICmdList) mutable
		{
			RenderProxy->ParentLayerIndexRemap = MoveTemp(ParentLayerIndexRemap);
		});
}

// Matches GetTypeHash for FMemoryImageMaterialParameterInfo
static uint32 GetTypeHashLegacy(const FHashedMaterialParameterInfoPacked& Value)
{
	return HashCombine(HashCombine(GetTypeHash(Value.Name), (int32)Value.Index), (uint32)Value.Association);
}

template <typename TInstanceType>
static bool SortMaterialInstanceParametersPredicate(
	const typename THashedMaterialParameterMap<TInstanceType>::TNamedParameter& Left,
	const typename THashedMaterialParameterMap<TInstanceType>::TNamedParameter& Right)
{
	// To keep the array sort the same as it has been historically, sort by the legacy type hash, not THashedMaterialParameterMap::TypeHash.
	// This only matters for duplicate items, where earlier duplicates take precedence over later ones.  Duplicates are exceedingly rare, and
	// possibly due to bugs, but we want to err on the side of preserving existing behavior.
	return GetTypeHashLegacy(Left.Info) < GetTypeHashLegacy(Right.Info);
}

void FMaterialInstanceRenderProxy::InitMIParameters(FMaterialInstanceParameterSet& ParameterSet)
{
	InvalidateUniformExpressionCache(false);

	// Sort the parameters.  Originally this was done so a binary lookup could be used.  We now have a hash table, but we're trying to preserve
	// the sort order logic to maintain consistent behavior where duplicate items occur.
	ParameterSet.ScalarParameters.Sort(SortMaterialInstanceParametersPredicate<float>);
	ParameterSet.VectorParameters.Sort(SortMaterialInstanceParametersPredicate<FLinearColor>);
	ParameterSet.DoubleVectorParameters.Sort(SortMaterialInstanceParametersPredicate<FVector4d>);
	ParameterSet.TextureParameters.Sort(SortMaterialInstanceParametersPredicate<const UTexture*>);
	ParameterSet.TextureCollectionParameters.Sort(SortMaterialInstanceParametersPredicate<const UTextureCollection*>);
	ParameterSet.RuntimeVirtualTextureParameters.Sort(SortMaterialInstanceParametersPredicate<const URuntimeVirtualTexture*>);
	ParameterSet.SparseVolumeTextureParameters.Sort(SortMaterialInstanceParametersPredicate<const USparseVolumeTexture*>);
	ParameterSet.ParameterCollectionParameters.Sort(SortMaterialInstanceParametersPredicate<const UMaterialParameterCollection*>);

	StaticSwitchParameterArray.Array = MoveTemp(ParameterSet.StaticSwitchParameters);
	ScalarParameterArray.Array = MoveTemp(ParameterSet.ScalarParameters);
	VectorParameterArray.Array = MoveTemp(ParameterSet.VectorParameters);
	DoubleVectorParameterArray.Array = MoveTemp(ParameterSet.DoubleVectorParameters);
	TextureParameterArray.Array = MoveTemp(ParameterSet.TextureParameters);
	TextureCollectionParameterArray.Array = MoveTemp(ParameterSet.TextureCollectionParameters);
	RuntimeVirtualTextureParameterArray.Array = MoveTemp(ParameterSet.RuntimeVirtualTextureParameters);
	SparseVolumeTextureParameterArray.Array = MoveTemp(ParameterSet.SparseVolumeTextureParameters);
	ParameterCollectionParameterArray.Array = MoveTemp(ParameterSet.ParameterCollectionParameters);
	UserSceneTextureOverrides = MoveTemp(ParameterSet.UserSceneTextureOverrides);
	PostProcessBlendableOverrides = ParameterSet.PostProcessBlendableOverrides;

	// Build hash tables.
	StaticSwitchParameterArray.HashAddAllItems();
	ScalarParameterArray.HashAddAllItems();
	VectorParameterArray.HashAddAllItems();
	DoubleVectorParameterArray.HashAddAllItems();
	TextureParameterArray.HashAddAllItems();
	TextureCollectionParameterArray.HashAddAllItems();
	RuntimeVirtualTextureParameterArray.HashAddAllItems();
	SparseVolumeTextureParameterArray.HashAddAllItems();
	ParameterCollectionParameterArray.HashAddAllItems();
}

void FMaterialInstanceRenderProxy::EvaluateParameterCollections(FRHICommandListBase& RHICmdList, FUniformExpressionCache& OutUniformExpressionCache, const FMaterialRenderContext& Context, FUniformExpressionCacheAsyncUpdater* Updater) const
{
	FMaterialRenderProxy::EvaluateParameterCollections(RHICmdList, OutUniformExpressionCache, Context, Updater);

	for (FGuid& CollectionStateId : OutUniformExpressionCache.ParameterCollections)
	{
		for (auto&& Entry : ParameterCollectionParameterArray.Array)
		{
			if (Entry.Value && Entry.Value->IsInstanceOf(CollectionStateId))
			{
				CollectionStateId = Entry.Value->StateId;
				break;
			}
		}
	}
}

bool UMaterialParameterCollection::IsInstanceOf(const FGuid& CollectionStateId) const
{
	for (const UMaterialParameterCollection* BaseCollection = this; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		if (BaseCollection->StateId == CollectionStateId)
		{
			return true;
		}
	}

	return false;
}

/**
* Updates a parameter on the material instance from the game thread.
*/
template <typename ParameterType>
void GameThread_UpdateMIParameter(const UMaterialInstance* Instance, const ParameterType& Parameter)
{
	if (FApp::CanEverRender())
	{
		const UMaterial* Material = Instance->GetMaterial_Concurrent();
		if (Material != nullptr)
		{
			EMaterialDomain Domain = Material->MaterialDomain;
			// check if this material has any relevance to path tracing
			if (Domain != MD_PostProcess && Domain != MD_UI && !Instance->GetUsageByFlag(MATUSAGE_EditorCompositing))
			{
				GetRendererModule().InvalidatePathTracedOutput(PathTracing::EInvalidateReason::UpdateMaterialParameter);
			}
		}
		FMaterialInstanceRenderProxy* RenderProxy = Instance->GetInstanceRenderProxy();
		const FMaterialParameterInfo& ParameterInfo = Parameter.ParameterInfo;
		typename ParameterType::ValueType Value = ParameterType::GetValue(Parameter);
		ENQUEUE_RENDER_COMMAND(SetMIParameterValue)(
			[RenderProxy, ParameterInfo, Value](FRHICommandListImmediate& RHICmdList)
			{
				RenderProxy->RenderThread_UpdateParameter(ParameterInfo, Value);
				RenderProxy->CacheUniformExpressions(RHICmdList, false);
			});
	}
}

#if WITH_EDITOR
template<typename ParameterType>
static void RemapLayerParameterIndicesArray(TArray<ParameterType>& Parameters, const TArray<int32>& RemapLayerIndices)
{
	int32 ParameterIndex = 0;
	while (ParameterIndex < Parameters.Num())
	{
		ParameterType& Parameter = Parameters[ParameterIndex];
		bool bRemovedParameter = false;
		if (Parameter.ParameterInfo.Association == LayerParameter)
		{
			const int32 NewIndex = RemapLayerIndices[Parameter.ParameterInfo.Index];
			if (NewIndex != INDEX_NONE)
			{
				Parameter.ParameterInfo.Index = NewIndex;
			}
			else
			{
				bRemovedParameter = true;
			}
		}
		else if (Parameter.ParameterInfo.Association == BlendParameter)
		{
			const int32 NewIndex = RemapLayerIndices[Parameter.ParameterInfo.Index + 1];
			if (NewIndex != INDEX_NONE)
			{
				Parameter.ParameterInfo.Index = NewIndex - 1;
			}
			else
			{
				bRemovedParameter = true;
			}
		}
		if (bRemovedParameter)
		{
			Parameters.RemoveAt(ParameterIndex);
		}
		else
		{
			++ParameterIndex;
		}
	}
}

template<typename ParameterType>
static void SwapLayerParameterIndicesArray(TArray<ParameterType>& Parameters, int32 OriginalIndex, int32 NewIndex)
{
	check(OriginalIndex > 0);
	check(NewIndex > 0);

	for (ParameterType& Parameter : Parameters)
	{
		if (Parameter.ParameterInfo.Association == LayerParameter)
		{
			if(Parameter.ParameterInfo.Index == OriginalIndex) Parameter.ParameterInfo.Index = NewIndex;
			else if (Parameter.ParameterInfo.Index == NewIndex) Parameter.ParameterInfo.Index = OriginalIndex;
		}
		else if (Parameter.ParameterInfo.Association == BlendParameter)
		{
			if (Parameter.ParameterInfo.Index == OriginalIndex - 1) Parameter.ParameterInfo.Index = NewIndex - 1;
			else if (Parameter.ParameterInfo.Index == NewIndex - 1) Parameter.ParameterInfo.Index = OriginalIndex - 1;
		}
	}
}

template<typename ParameterType>
static void RemoveLayerParameterIndicesArray(TArray<ParameterType>& Parameters, int32 RemoveIndex)
{
	int32 ParameterIndex = 0;
	while (ParameterIndex < Parameters.Num())
	{
		ParameterType& Parameter = Parameters[ParameterIndex];
		bool bRemovedParameter = false;
		if (Parameter.ParameterInfo.Association == LayerParameter)
		{
			const int32 Index = Parameter.ParameterInfo.Index;
			if (Index == RemoveIndex)
			{
				bRemovedParameter = true;
			}
			else if (Index > RemoveIndex)
			{
				Parameter.ParameterInfo.Index--;
			}
		}
		else if (Parameter.ParameterInfo.Association == BlendParameter)
		{
			const int32 Index = Parameter.ParameterInfo.Index + 1;
			if (Index == RemoveIndex)
			{
				bRemovedParameter = true;
			}
			else if (Index > RemoveIndex)
			{
				Parameter.ParameterInfo.Index--;
			}
		}
		if (bRemovedParameter)
		{
			Parameters.RemoveAt(ParameterIndex);
		}
		else
		{
			++ParameterIndex;
		}
	}
}

void UMaterialInstance::SwapLayerParameterIndices(int32 OriginalIndex, int32 NewIndex)
{
	if (OriginalIndex != NewIndex)
	{
		UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
		SwapLayerParameterIndicesArray(ScalarParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(VectorParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(DoubleVectorParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(TextureParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(TextureCollectionParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(ParameterCollectionParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(RuntimeVirtualTextureParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(SparseVolumeTextureParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(FontParameterValues, OriginalIndex, NewIndex);
		SwapLayerParameterIndicesArray(StaticParametersRuntime.StaticSwitchParameters, OriginalIndex, NewIndex);
		if (EditorOnly)
		{
			SwapLayerParameterIndicesArray(EditorOnly->StaticParameters.StaticComponentMaskParameters, OriginalIndex, NewIndex);
		}
	}
}

void UMaterialInstance::RemoveLayerParameterIndex(int32 Index)
{
	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	RemoveLayerParameterIndicesArray(ScalarParameterValues, Index);
	RemoveLayerParameterIndicesArray(VectorParameterValues, Index);
	RemoveLayerParameterIndicesArray(DoubleVectorParameterValues, Index);
	RemoveLayerParameterIndicesArray(TextureParameterValues, Index);
	RemoveLayerParameterIndicesArray(TextureCollectionParameterValues, Index);
	RemoveLayerParameterIndicesArray(ParameterCollectionParameterValues, Index);
	RemoveLayerParameterIndicesArray(RuntimeVirtualTextureParameterValues, Index);
	RemoveLayerParameterIndicesArray(SparseVolumeTextureParameterValues, Index);
	RemoveLayerParameterIndicesArray(FontParameterValues, Index);
	RemoveLayerParameterIndicesArray(StaticParametersRuntime.StaticSwitchParameters, Index);
	if (EditorOnly)
	{
		RemoveLayerParameterIndicesArray(EditorOnly->StaticParameters.StaticComponentMaskParameters, Index);
	}
}
#endif // WITH_EDITOR

bool UMaterialInstance::UpdateParameters()
{
	bool bDirty = false;

#if WITH_EDITOR
	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	if(IsTemplate(RF_ClassDefaultObject)==false && EditorOnly)
	{
		// Get a pointer to the parent material.
		UMaterial* ParentMaterial = NULL;
		UMaterialInstance* ParentInst = this;
		while(ParentInst && ParentInst->Parent)
		{
			if(ParentInst->Parent->IsA(UMaterial::StaticClass()))
			{
				ParentMaterial = Cast<UMaterial>(ParentInst->Parent);
				break;
			}
			else
			{
				ParentInst = Cast<UMaterialInstance>(ParentInst->Parent);
			}
		}

		if(ParentMaterial)
		{
			// Scalar parameters
			bDirty = UpdateParameterSet<FScalarParameterValue, UMaterialExpressionScalarParameter>(ScalarParameterValues, ParentMaterial) || bDirty;

			// Vector parameters	
			bDirty = UpdateParameterSet<FVectorParameterValue, UMaterialExpressionVectorParameter>(VectorParameterValues, ParentMaterial) || bDirty;

			// Vector parameters	
			bDirty = UpdateParameterSet<FDoubleVectorParameterValue, UMaterialExpressionDoubleVectorParameter>(DoubleVectorParameterValues, ParentMaterial) || bDirty;

			// Texture parameters
			bDirty = UpdateParameterSet<FTextureParameterValue, UMaterialExpressionTextureSampleParameter>(TextureParameterValues, ParentMaterial) || bDirty;

			// Texture Collection parameters
			bDirty = UpdateParameterSet<FTextureCollectionParameterValue, UMaterialExpressionTextureCollectionParameter>(TextureCollectionParameterValues, ParentMaterial) || bDirty;

			// Parameter Collection parameters
			bDirty = UpdateParameterSet<FParameterCollectionParameterValue, UMaterialExpressionCollectionParameter>(ParameterCollectionParameterValues, ParentMaterial) || bDirty;

			// Runtime Virtual Texture parameters
			bDirty = UpdateParameterSet<FRuntimeVirtualTextureParameterValue, UMaterialExpressionRuntimeVirtualTextureSampleParameter>(RuntimeVirtualTextureParameterValues, ParentMaterial) || bDirty;

			// Sparse Volume Texture parameters
			bDirty = UpdateParameterSet<FSparseVolumeTextureParameterValue, UMaterialExpressionSparseVolumeTextureSampleParameter>(SparseVolumeTextureParameterValues, ParentMaterial) || bDirty;

			// Font parameters
			bDirty = UpdateParameterSet<FFontParameterValue, UMaterialExpressionFontSampleParameter>(FontParameterValues, ParentMaterial) || bDirty;

			// Static switch parameters
			bDirty = UpdateParameterSet<FStaticSwitchParameter, UMaterialExpressionStaticBoolParameter>(StaticParametersRuntime.StaticSwitchParameters, ParentMaterial) || bDirty;

			// Static component mask parameters
			bDirty = UpdateParameterSet<FStaticComponentMaskParameter, UMaterialExpressionStaticComponentMaskParameter>(EditorOnly->StaticParameters.StaticComponentMaskParameters, ParentMaterial) || bDirty;
		}

		if (StaticParametersRuntime.bHasMaterialLayers && Parent)
		{
			FMaterialLayersFunctions ParentLayers;
			if (Parent->GetMaterialLayers(ParentLayers))
			{
				TArray<int32> RemapLayerIndices;
				if (FMaterialLayersFunctions::ResolveParent(ParentLayers,
					ParentLayers.EditorOnly,
					StaticParametersRuntime.MaterialLayers,
					EditorOnly->StaticParameters.MaterialLayers,
					RemapLayerIndices))
				{
					RemapLayerParameterIndicesArray(ScalarParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(VectorParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(DoubleVectorParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(TextureParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(TextureCollectionParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(ParameterCollectionParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(RuntimeVirtualTextureParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(SparseVolumeTextureParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(FontParameterValues, RemapLayerIndices);
					RemapLayerParameterIndicesArray(StaticParametersRuntime.StaticSwitchParameters, RemapLayerIndices);
					RemapLayerParameterIndicesArray(EditorOnly->StaticParameters.StaticComponentMaskParameters, RemapLayerIndices);
					bDirty = true;
				}
			}
		}

		if (bDirty)
		{
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
		}
	}
#endif // WITH_EDITOR

	return bDirty;
}

UMaterialInstance::UMaterialInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHasStaticPermutationResource = false;
	bLoadedCachedData = false;
#if WITH_EDITOR
	ReentrantFlag[0] = false;
	ReentrantFlag[1] = false;
#endif
	ShadingModels = MSM_Unlit;

	PhysMaterial = nullptr;
	for (TObjectPtr<UPhysicalMaterial>& PhysMat : PhysicalMaterialMap)
	{
		PhysMat = nullptr;
	}
}

void UMaterialInstance::PostInitProperties()	
{
	LLM_SCOPE(ELLMTag::MaterialInstance);
	Super::PostInitProperties();

	if(!HasAnyFlags(RF_ClassDefaultObject) && FApp::CanEverRenderOrProduceRenderData())
	{
		RenderProxy = new FMaterialInstanceRenderProxy(this);
		bResourceCreated = true;
	}
}

/**
 * Initializes MI parameters from the game thread.
 */
void GameThread_InitMIParameters(const UMaterialInstance& Instance)
{
	if (Instance.HasAnyFlags(RF_ClassDefaultObject) || !FApp::CanEverRender())
	{
		return;
	}

	FMaterialInstanceRenderProxy* RenderProxy = Instance.GetInstanceRenderProxy();
	FMaterialInstanceParameterSet ParameterSet;

	// Scalar parameters
	ParameterSet.ScalarParameters.Reserve(Instance.ScalarParameterValues.Num());
	for (const FScalarParameterValue& Parameter : Instance.ScalarParameterValues)
	{
		auto& ParamRef = ParameterSet.ScalarParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FScalarParameterValue::GetValue(Parameter);
	}

	// Vector parameters
	ParameterSet.VectorParameters.Reserve(Instance.VectorParameterValues.Num());
	for (const FVectorParameterValue& Parameter : Instance.VectorParameterValues)
	{
		auto& ParamRef = ParameterSet.VectorParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FVectorParameterValue::GetValue(Parameter);
	}

	// Double Vector parameters
	ParameterSet.DoubleVectorParameters.Reserve(Instance.DoubleVectorParameterValues.Num());
	for (const FDoubleVectorParameterValue& Parameter : Instance.DoubleVectorParameterValues)
	{
		auto& ParamRef = ParameterSet.DoubleVectorParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FDoubleVectorParameterValue::GetValue(Parameter);
	}

	// Texture + Fonts parameters
	ParameterSet.TextureParameters.Reserve(Instance.TextureParameterValues.Num() + Instance.FontParameterValues.Num());
	for (const FTextureParameterValue& Parameter : Instance.TextureParameterValues)
	{
		auto& ParamRef = ParameterSet.TextureParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FTextureParameterValue::GetValue(Parameter);
	}
	for (const FFontParameterValue& Parameter : Instance.FontParameterValues)
	{
		auto& ParamRef = ParameterSet.TextureParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FFontParameterValue::GetValue(Parameter);
	}

	ParameterSet.TextureCollectionParameters.Reserve(Instance.TextureCollectionParameterValues.Num());
	for (const FTextureCollectionParameterValue& Parameter : Instance.TextureCollectionParameterValues)
	{
		auto& ParamRef = ParameterSet.TextureCollectionParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FTextureCollectionParameterValue::GetValue(Parameter);
	}

	ParameterSet.ParameterCollectionParameters.Reserve(Instance.ParameterCollectionParameterValues.Num());
	for (const FParameterCollectionParameterValue& Parameter : Instance.ParameterCollectionParameterValues)
	{
		auto& ParamRef = ParameterSet.ParameterCollectionParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FParameterCollectionParameterValue::GetValue(Parameter);
	}

	// RuntimeVirtualTexture parameters
	ParameterSet.RuntimeVirtualTextureParameters.Reserve(Instance.RuntimeVirtualTextureParameterValues.Num());
	for (const FRuntimeVirtualTextureParameterValue& Parameter : Instance.RuntimeVirtualTextureParameterValues)
	{
		auto& ParamRef = ParameterSet.RuntimeVirtualTextureParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FRuntimeVirtualTextureParameterValue::GetValue(Parameter);
	}

	// SparseVolumeTexture parameters
	ParameterSet.SparseVolumeTextureParameters.Reserve(Instance.SparseVolumeTextureParameterValues.Num());
	for (const FSparseVolumeTextureParameterValue& Parameter : Instance.SparseVolumeTextureParameterValues)
	{
		auto& ParamRef = ParameterSet.SparseVolumeTextureParameters.AddDefaulted_GetRef();
		ParamRef.Info = Parameter.ParameterInfo;
		ParamRef.Value = FSparseVolumeTextureParameterValue::GetValue(Parameter);
	}

	FStaticParameterSet StaticParamSet = Instance.GetStaticParameters();
	ParameterSet.StaticSwitchParameters.Reserve(StaticParamSet.StaticSwitchParameters.Num());
	for(const FStaticSwitchParameter& Param : StaticParamSet.StaticSwitchParameters)
	{
		if(Param.IsOverride())
		{
			FMaterialParameterMetadata Result;
			Param.GetValue(Result);
			auto& ParamRef = ParameterSet.StaticSwitchParameters.AddDefaulted_GetRef();
			ParamRef.Info = FHashedMaterialParameterInfo(Param.ParameterInfo);
			ParamRef.Value = Result.Value.AsStaticSwitch();
		}
	}

	ParameterSet.UserSceneTextureOverrides = Instance.UserSceneTextureOverrides;
	ParameterSet.PostProcessBlendableOverrides.bOverrideBlendableLocation = Instance.bOverrideBlendableLocation;
	ParameterSet.PostProcessBlendableOverrides.bOverrideBlendablePriority = Instance.bOverrideBlendablePriority;
	ParameterSet.PostProcessBlendableOverrides.BlendableLocationOverride = Instance.BlendableLocationOverride;
	ParameterSet.PostProcessBlendableOverrides.BlendablePriorityOverride = Instance.BlendablePriorityOverride;

	ENQUEUE_RENDER_COMMAND(InitMIParameters)(
		[RenderProxy, Parameters = MoveTemp(ParameterSet)](FRHICommandListImmediate& RHICmdList) mutable
		{
			RenderProxy->InitMIParameters(Parameters);
		});
}

void UMaterialInstance::InitResources()
{	
	// Find the instance's parent.
	UMaterialInterface* SafeParent = NULL;
	if (Parent)
	{
		SafeParent = Parent;
	}

	// Don't use the instance's parent if it has a circular dependency on the instance.
	if (SafeParent && SafeParent->IsDependent_Concurrent(this))
	{
		SafeParent = NULL;
	}

	// Don't allow MIDs as parents for material instances.
	if (SafeParent && SafeParent->IsA(UMaterialInstanceDynamic::StaticClass()))
	{
		SafeParent = NULL;
	}

	// If the instance doesn't have a valid parent, use the default material as the parent.
	if (!SafeParent)
	{
		SafeParent = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	checkf(SafeParent, TEXT("Invalid parent on %s"), *GetFullName());

	// TODO - should merge all of render commands sent to initialize resource into a single command
	// Set the material instance's parent on its resources.
	if (RenderProxy != nullptr)
	{
		RenderProxy->GameThread_SetParent(SafeParent);
		RenderProxy->GameThread_UpdateCachedData(GetCachedInstanceData());
	}

	GameThread_InitMIParameters(*this);
	PropagateDataToMaterialProxy();

	CacheMaterialInstanceUniformExpressions(this);
}

UMaterialInstance::~UMaterialInstance()
{
}

const UMaterial* UMaterialInstance::GetMaterial() const
{
	check(IsInGameThread() || IsInParallelGameThread() || IsAsyncLoading());
	if(GetReentrantFlag())
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}

	FMICReentranceGuard	Guard(this);
	if(Parent)
	{
		return Parent->GetMaterial();
	}
	else
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}
}

const UMaterial* UMaterialInstance::GetMaterial_Concurrent(TMicRecursionGuard RecursionGuard) const
{
	if(!Parent || RecursionGuard.Contains(this))
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}

	RecursionGuard.Set(this);
	return Parent->GetMaterial_Concurrent(RecursionGuard);
}

UMaterial* UMaterialInstance::GetMaterial()
{
	if(GetReentrantFlag())
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}

	FMICReentranceGuard	Guard(this);
	if(Parent)
	{
		return Parent->GetMaterial();
	}
	else
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}
}

void UMaterialInstance::GetMaterialInheritanceChain(FMaterialInheritanceChain& OutChain) const
{
	if (!OutChain.MaterialInstances.Contains(this))
	{
		OutChain.MaterialInstances.Add(this);
		if (!OutChain.CachedExpressionData)
		{
			OutChain.CachedExpressionData = CachedExpressionData.Get();
		}

		if (Parent)
		{
			return Parent->GetMaterialInheritanceChain(OutChain);
		}
	}

	UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialInheritanceChain(OutChain);
}

const FMaterialCachedExpressionData& UMaterialInstance::GetCachedExpressionData(TMicRecursionGuard RecursionGuard) const
{
	const FMaterialCachedExpressionData* LocalData = CachedExpressionData.Get();
	if (LocalData)
	{
		return *LocalData;
	}

	if (Parent && !RecursionGuard.Contains(this))
	{
		RecursionGuard.Set(this);
		return Parent->GetCachedExpressionData(RecursionGuard);
	}

	return UMaterial::GetDefaultMaterial(MD_Surface)->GetCachedExpressionData();
}

bool UMaterialInstance::GetParameterOverrideValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutResult) const
{
	bool bResult = false;
	switch (Type)
	{
	case EMaterialParameterType::Scalar: bResult = GameThread_GetParameterValue(ScalarParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::Vector: bResult = GameThread_GetParameterValue(VectorParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::DoubleVector: bResult = GameThread_GetParameterValue(DoubleVectorParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::Texture: bResult = GameThread_GetParameterValue(TextureParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::TextureCollection: bResult = GameThread_GetParameterValue(TextureCollectionParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::ParameterCollection: bResult = GameThread_GetParameterValue(ParameterCollectionParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::RuntimeVirtualTexture: bResult = GameThread_GetParameterValue(RuntimeVirtualTextureParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::SparseVolumeTexture: bResult = GameThread_GetParameterValue(SparseVolumeTextureParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::Font: bResult = GameThread_GetParameterValue(FontParameterValues, ParameterInfo, OutResult); break;
	case EMaterialParameterType::StaticSwitch: bResult = GameThread_GetParameterValue(StaticParametersRuntime.StaticSwitchParameters, ParameterInfo, OutResult); break;
#if WITH_EDITORONLY_DATA
	case EMaterialParameterType::StaticComponentMask:
		bResult = GameThread_GetParameterValue(GetEditorOnlyData()->StaticParameters.StaticComponentMaskParameters, ParameterInfo, OutResult);
		break;
#endif // WITH_EDITORONLY_DATA
	default: checkNoEntry(); break;
	}
	return bResult;
}

bool UMaterialInstance::GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutResult, EMaterialGetParameterValueFlags Flags) const
{
	FMaterialInheritanceChain InstanceChain;
	GetMaterialInheritanceChain(InstanceChain);

	bool bResult = false;
	const bool bCheckInstanceOverrides = EnumHasAnyFlags(Flags, EMaterialGetParameterValueFlags::CheckInstanceOverrides);
	FMemoryImageMaterialParameterInfo CurrentParameterInfo = ParameterInfo;
	bool bHasValidParameter = true;

	// Check instance chain for overriden values
	int32 ParentIndex = 0;
	while (bHasValidParameter && ParentIndex < InstanceChain.MaterialInstances.Num())
	{
		const UMaterialInstance* Instance = InstanceChain.MaterialInstances[ParentIndex];

		// Don't check overrides for Index0, unless CheckInstanceOverrides is set
		if (ParentIndex > 0 || bCheckInstanceOverrides)
		{
			if (Instance->GetParameterOverrideValue(Type, CurrentParameterInfo, OutResult))
			{
#if WITH_EDITORONLY_DATA
				if (ParentIndex == 0)
				{
					// If value was set on this instance, set the override flag
					OutResult.bOverride = true;
				}
#endif
				bResult = true;
				break;
			}
		}

		bHasValidParameter = CurrentParameterInfo.RemapLayerIndex(MakeArrayView(Instance->GetCachedInstanceData().ParentLayerIndexRemap), CurrentParameterInfo);
		ParentIndex++;
	}

	// Look through the default values if we have not found an override
	if (!bResult && EnumHasAnyFlags(Flags, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		bResult = InstanceChain.GetCachedExpressionData().GetParameterValue(Type, ParameterInfo, OutResult);
	}

	return bResult;
}

bool UMaterialInstance::GetRefractionSettings(float& OutBiasValue) const
{
	bool bFoundAValue = false;

	FMaterialParameterInfo ParamInfo;
	if( GetLinkerUEVersion() >= VER_UE4_REFRACTION_BIAS_TO_REFRACTION_DEPTH_BIAS )
	{
		static FName NAME_RefractionDepthBias("RefractionDepthBias");
		ParamInfo.Name = NAME_RefractionDepthBias;
	}
	else
	{
		static FName NAME_RefractionBias("RefractionBias");
		ParamInfo.Name = NAME_RefractionBias;
	}

	const FScalarParameterValue* BiasParameterValue = GameThread_FindParameterByName(ScalarParameterValues, ParamInfo);
	if (BiasParameterValue)
	{
		OutBiasValue = BiasParameterValue->ParameterValue;
		return true;
	}
	else if(Parent)
	{
		return Parent->GetRefractionSettings(OutBiasValue);
	}
	else
	{
		return false;
	}
}

bool UMaterialInstance::GetUserSceneTextureOverride(FName& InOutName) const
{
	// Number of overrides possible is small (maximum 6, in most practical cases 1 or 2), and FName comparison cheap,
	// so the assumption is that an array search will be cheaper than the overhead of going through a hash lookup.
	// Plus an array takes half the space of THashedMaterialParameterMap, saving memory.
	for (const FUserSceneTextureOverride& Override : UserSceneTextureOverrides)
	{
		if (Override.Key == InOutName)
		{
			InOutName = Override.Value;
			return true;
		}
	}

	if (Parent)
	{
		return Parent->GetUserSceneTextureOverride(InOutName);
	}
	else
	{
		return false;
	}
}

EBlendableLocation UMaterialInstance::GetBlendableLocation(const UMaterial* Base) const
{
	check(Base);

	// Replacing Tonemapper can't be overridden from
	if (Base->BlendableLocation == BL_ReplacingTonemapper)
	{
		return BL_ReplacingTonemapper;
	}

	// Replacing Tonemapper can't be overridden to
	if (bOverrideBlendableLocation && BlendableLocationOverride != BL_ReplacingTonemapper)
	{
		return BlendableLocationOverride;
	}
	else if (Parent)
	{
		return Parent->GetBlendableLocation(Base);
	}
	else
	{
		return Base->BlendableLocation;
	}
}

int32 UMaterialInstance::GetBlendablePriority(const UMaterial* Base) const
{
	check(Base);
	if (bOverrideBlendablePriority)
	{
		return BlendablePriorityOverride;
	}
	else if (Parent)
	{
		return Parent->GetBlendablePriority(Base);
	}
	else
	{
		return Base->BlendablePriority;
	}
}

void UMaterialInstance::GetTextureExpressionValues(const FMaterialResource* MaterialResource, TArray<UTexture*>& OutTextures, TArray<TArray<int32>>* OutIndices, bool bIncludeTextureCollections, bool bAllowOverrides) const
{
	Super::GetTextureExpressionValues(MaterialResource, OutTextures, OutIndices, bIncludeTextureCollections, bAllowOverrides);

#if WITH_EDITOR
	for (int32 i = 0; i < TransientTextureParameterOverrides.Num(); ++i)
	{
		OutTextures.AddUnique(TransientTextureParameterOverrides[i].OverrideTexture);
		OutTextures.AddUnique(TransientTextureParameterOverrides[i].PreviousTexture);
	}
#endif
}

void UMaterialInstance::GetTextureCollectionExpressionValues(const FMaterialResource* MaterialResource, TArray<UTextureCollection*>& OutTextureCollections) const
{
	check(MaterialResource);
	const FUniformExpressionSet& UniformExpressions = MaterialResource->GetUniformExpressions();

	for (int32 TextureCollectionIndex = 0; TextureCollectionIndex < UniformExpressions.GetNumTextureCollections(); TextureCollectionIndex++)
	{
		UTextureCollection* TextureCollection = nullptr;
		UniformExpressions.GetGameThreadTextureCollectionValue(TextureCollectionIndex, this, *MaterialResource, TextureCollection);

		if (TextureCollection)
		{
			OutTextureCollections.AddUnique(TextureCollection);
		}
	}
}

void UMaterialInstance::GetUsedTextures(TArray<UTexture*>& OutTextures, TOptional<EMaterialQualityLevel::Type> InSpecificQualityLevel, TOptional<EShaderPlatform> InSpecificShaderPlatform) const
{
	OutTextures.Empty();

	// Do not care if we're running dedicated server
	if (!FPlatformProperties::IsServerOnly())
	{
		const FInt32Range QualityLevelRange = BuildQualityLevelRange(InSpecificQualityLevel);
		const FInt32Range ShaderPlatformRange = BuildShaderPlatformRange(InSpecificShaderPlatform);

		const UMaterial* BaseMaterial = GetMaterial();
		const UMaterialInstance* MaterialInstanceToUse = this;

		if (BaseMaterial && !BaseMaterial->IsDefaultMaterial())
		{
			// Walk up the material instance chain to the first parent that has static parameters
			while (MaterialInstanceToUse && !MaterialInstanceToUse->bHasStaticPermutationResource)
			{
				MaterialInstanceToUse = Cast<const UMaterialInstance>(MaterialInstanceToUse->Parent);
			}

			// Use the uniform expressions from the lowest material instance with static parameters in the chain, if one exists
			const UMaterialInterface* MaterialToUse = (MaterialInstanceToUse && MaterialInstanceToUse->bHasStaticPermutationResource) ? (const UMaterialInterface*)MaterialInstanceToUse : (const UMaterialInterface*)BaseMaterial;

			TArray<const FMaterialResource*, TInlineAllocator<4>> MatchedResources = GetMaterialResources(MaterialToUse, QualityLevelRange, ShaderPlatformRange);

			for (const FMaterialResource* CurrentResource : MatchedResources)
			{
				constexpr bool bIncludeTextureCollections = true;
				constexpr bool bAllowOverrides = true;
				GetTextureExpressionValues(CurrentResource, OutTextures, nullptr, bIncludeTextureCollections, bAllowOverrides);
			}
		}
		else
		{
			// If the material instance has no material, use the default material.
			UMaterial::GetDefaultMaterial(MD_Surface)->GetUsedTextures(OutTextures, InSpecificQualityLevel, InSpecificShaderPlatform);
		}
	}
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UMaterialInstance::LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const
{
	auto World = GetWorld();
	const EMaterialQualityLevel::Type QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	const ERHIFeatureLevel::Type FeatureLevel = World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel;

	Ar.Logf(TEXT("%sMaterialInstance: %s"), FCString::Tab(Indent), *GetName());

	if (FPlatformProperties::IsServerOnly())
	{
		Ar.Logf(TEXT("%sNo Textures: IsServerOnly"), FCString::Tab(Indent + 1));
	}
	else
	{
		const UMaterialInstance* MaterialInstanceToUse = nullptr;
		const UMaterial* MaterialToUse = nullptr;

		const UMaterialInterface* CurrentMaterialInterface = this;
		{
			TSet<const UMaterialInterface*> MaterialParents;

			// Walk up the parent chain to the materials to use.
			while (CurrentMaterialInterface && !MaterialParents.Contains(CurrentMaterialInterface))
			{
				MaterialParents.Add(CurrentMaterialInterface);

				const UMaterialInstance* CurrentMaterialInstance = Cast<const UMaterialInstance>(CurrentMaterialInterface);
				const UMaterial* CurrentMaterial = Cast<const UMaterial>(CurrentMaterialInterface);

				// The parent material is the first parent of this class.
				if (!MaterialToUse && CurrentMaterial)
				{
					MaterialToUse = CurrentMaterial;
				}

				if (!MaterialInstanceToUse && CurrentMaterialInstance && CurrentMaterialInstance->bHasStaticPermutationResource)
				{
					MaterialInstanceToUse = CurrentMaterialInstance;
				}

				CurrentMaterialInterface = CurrentMaterialInstance ? ToRawPtr(CurrentMaterialInstance->Parent) : nullptr;
			}
		}

		if (CurrentMaterialInterface)
		{
			Ar.Logf(TEXT("%sNo Textures : Cycling Parent Loop"), FCString::Tab(Indent + 1));
		}
		else if (MaterialInstanceToUse)
		{
			const FMaterialResource* MaterialResource = FindMaterialResource(MaterialInstanceToUse->StaticPermutationMaterialResources, GetFeatureLevelShaderPlatform_Checked(FeatureLevel), QualityLevel, true);
			if (MaterialResource)
			{
				if (MaterialResource->HasValidGameThreadShaderMap())
				{
					TArray<UTexture*> Textures;
					constexpr bool bIncludeTextureCollections = false;
					constexpr bool bAllowOverrides = true;
					GetTextureExpressionValues(MaterialResource, Textures, nullptr, bIncludeTextureCollections, bAllowOverrides);

					for (UTexture* Texture : Textures)
					{
						if (Texture)
						{
							Ar.Logf(TEXT("%s%s"), FCString::Tab(Indent + 1), *Texture->GetName());
						}
					}
				}
				else
				{
					Ar.Logf(TEXT("%sNo Textures : Invalid GameThread ShaderMap"), FCString::Tab(Indent + 1));
				}
			}
			else
			{
				Ar.Logf(TEXT("%sNo Textures : Invalid MaterialResource"), FCString::Tab(Indent + 1));
			}
		}
		else if (MaterialToUse)
		{
			MaterialToUse->LogMaterialsAndTextures(Ar, Indent + 1);
		}
		else
		{
			Ar.Logf(TEXT("%sNo Textures : No Material Found"), FCString::Tab(Indent + 1));
		}
	}
}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

// Deprecated in 5.7
void UMaterialInstance::ValidateTextureOverrides(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return ValidateTextureOverrides(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

void UMaterialInstance::ValidateTextureOverrides(EShaderPlatform InShaderPlatform) const
{
	if (!(IsInGameThread() || IsAsyncLoading() || UE::IsSavingPackage()))
	{
		// Fatal to call getmaterial in a non-game thread or async loading
		return;
	}

	const UMaterial* Material = GetMaterial_Concurrent();
	const FMaterialResource* CurrentResource = Material->GetMaterialResource(InShaderPlatform);
	const bool bShouldValidateVTUsage = UseVirtualTexturing(GMaxRHIShaderPlatform);

	if (!CurrentResource)
	{
		return;
	}

	FString MaterialName;
	GetName(MaterialName);

	for (uint32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		const EMaterialTextureParameterType ParameterType = (EMaterialTextureParameterType)TypeIndex;
		
		// SVT currently do not derive from UTexture and checking for GetMaterialType() validity is not necessary here because SVT are always MCT_SparseVolumeTexture.
		if (ParameterType == EMaterialTextureParameterType::SparseVolume)
		{
			continue;
		}

		for (const FMaterialTextureParameterInfo& TextureInfo : CurrentResource->GetUniformTextureExpressions(ParameterType))
		{
			UTexture* Texture = nullptr;
			TextureInfo.GetGameThreadTextureValue(this, *CurrentResource, Texture);
			if (Texture)
			{
				const EMaterialValueType TextureType = Texture->GetMaterialType();
				switch (ParameterType)
				{
				case EMaterialTextureParameterType::Standard2D:
					if (!(TextureType & (MCT_Texture2D | MCT_TextureExternal | MCT_TextureVirtual)))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" has invalid type, required 2D texture", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					else if (bShouldValidateVTUsage && (TextureType & MCT_TextureVirtual))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" requires non-virtual texture", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					break;
				case EMaterialTextureParameterType::Cube:
					if (!(TextureType & MCT_TextureCube))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" has invalid type, required Cube texture", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					break;
				case EMaterialTextureParameterType::Array2D:
					if (!(TextureType & MCT_Texture2DArray))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" has invalid type, required texture array", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					break;
				case EMaterialTextureParameterType::ArrayCube:
					if (!(TextureType & MCT_TextureCubeArray))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" has invalid type, required texture cube array", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					break;
				case EMaterialTextureParameterType::Volume:
					if (!(TextureType & MCT_VolumeTexture))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" has invalid type, required Volume texture", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					break;
				case EMaterialTextureParameterType::Virtual:
					if (!(TextureType & (MCT_Texture2D | MCT_TextureExternal | MCT_TextureVirtual)))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" has invalid type, required 2D texture", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					else if (bShouldValidateVTUsage && !(TextureType & MCT_TextureVirtual))
					{
						UE_LOGF(LogMaterial, Error, "MaterialInstance \"%ls\" parameter '%ls' assigned texture \"%ls\" requires virtual texture", *MaterialName, *TextureInfo.GetParameterName().ToString(), *Texture->GetName());
					}
					break;
				default:
					checkNoEntry();
					break;
				}
			}
		}
	}
}

void UMaterialInstance::GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform) const
{
	OutTextures.Empty();
	OutIndices.Empty();

	if (!FPlatformProperties::IsServerOnly())
	{
		const UMaterialInstance* MaterialInstanceToUse = this;
		// Walk up the material instance chain to the first parent that has static parameters
		while (MaterialInstanceToUse && !MaterialInstanceToUse->bHasStaticPermutationResource)
		{
			MaterialInstanceToUse = Cast<const UMaterialInstance>(MaterialInstanceToUse->Parent);
		}

		if (MaterialInstanceToUse && MaterialInstanceToUse->bHasStaticPermutationResource)
		{
			const FMaterialResource* CurrentResource = FindMaterialResource(MaterialInstanceToUse->StaticPermutationMaterialResources, InShaderPlatform, QualityLevel, true);
			if (CurrentResource)
			{
				constexpr bool bIncludeTextureCollections = false;
				constexpr bool bAllowOverrides = true;
				GetTextureExpressionValues(CurrentResource, OutTextures, &OutIndices, bIncludeTextureCollections, bAllowOverrides);
			}
		}
		else // Use the uniform expressions from the base material
		{ 
			const UMaterial* Material = GetMaterial();
			if (Material)
			{
				const FMaterialResource* MaterialResource = Material->GetMaterialResource(InShaderPlatform, QualityLevel);
				if( MaterialResource )
				{
					constexpr bool bIncludeTextureCollections = false;
					constexpr bool bAllowOverrides = true;
					GetTextureExpressionValues(MaterialResource, OutTextures, &OutIndices, bIncludeTextureCollections, bAllowOverrides);
				}
			}
			else // If the material instance has no material, use the default material.
			{
				UMaterial::GetDefaultMaterial(MD_Surface)->GetUsedTexturesAndIndices(OutTextures, OutIndices, QualityLevel, InShaderPlatform);
			}
		}
	}
}

void UMaterialInstance::OverrideTexture(const UTexture* InTextureToOverride, UTexture* OverrideTexture, ERHIFeatureLevel::Type InFeatureLevel)
{
#if WITH_EDITOR
	FMaterialResource* SourceMaterialResource = nullptr;
	if (bHasStaticPermutationResource)
	{
		SourceMaterialResource = GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
	}
	else
	{
		//@todo - this isn't handling chained MIC's correctly, where a parent in the chain has static parameters
		UMaterial* Material = GetMaterial();
		SourceMaterialResource = Material->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
	}

	if (SourceMaterialResource)
	{
		bool bShouldRecacheMaterialExpressions = false;
		for (int32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; TypeIndex++)
		{
			const TArrayView<const FMaterialTextureParameterInfo> Parameters = SourceMaterialResource->GetUniformTextureExpressions((EMaterialTextureParameterType)TypeIndex);
			// Iterate over each of the material's texture expressions.
			for (int32 i = 0; i < Parameters.Num(); ++i)
			{
				const FMaterialTextureParameterInfo& Parameter = Parameters[i];

				// Evaluate the expression in terms of this material instance.
				UTexture* Texture = NULL;
				Parameter.GetGameThreadTextureValue(this, *SourceMaterialResource, Texture);
				if (Texture != NULL && Texture == InTextureToOverride)
				{
					// Override this texture!
					SourceMaterialResource->TransientOverrides.SetTextureOverride((EMaterialTextureParameterType)TypeIndex, Parameter, OverrideTexture);
					bShouldRecacheMaterialExpressions = true;
				}
			}
		}

		if (bShouldRecacheMaterialExpressions)
		{
			RecacheUniformExpressions(false);
		}
	}

	// Override texture parameters as well
	OverrideTextureParameterValue(InTextureToOverride, OverrideTexture);
#endif // #if WITH_EDITOR
}

extern bool ShouldRecacheMaterialExpressions(const FMaterialResource* Resource, const FHashedMaterialParameterInfo& ParameterInfo);

void UMaterialInstance::SetNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value, EShaderPlatform InShaderPlatform)
{
#if WITH_EDITOR
	if (bHasStaticPermutationResource)
	{
		FMaterialResource* SourceMaterialResource = GetMaterialResource(InShaderPlatform);
		if (SourceMaterialResource)
		{
			SourceMaterialResource->TransientOverrides.SetNumericOverride(Type, ParameterInfo, Value);
			if (ShouldRecacheMaterialExpressions(SourceMaterialResource, ParameterInfo))
			{
				RecacheUniformExpressions(false);
			}
		}
	}
#endif // #if WITH_EDITOR
}

void UMaterialInstance::ClearNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, EShaderPlatform InShaderPlatform)
{
#if WITH_EDITOR
	if (bHasStaticPermutationResource)
	{
		FMaterialResource* SourceMaterialResource = GetMaterialResource(InShaderPlatform);
		if (SourceMaterialResource)
		{
			SourceMaterialResource->TransientOverrides.ClearNumericOverride(Type, ParameterInfo);
			if (ShouldRecacheMaterialExpressions(SourceMaterialResource, ParameterInfo))
			{
				RecacheUniformExpressions(false);
			}
		}
	}
#endif // #if WITH_EDITOR
}

uint32 UMaterialInstance::GetUsageFlags() const
{
	return UsageFlags;
}

bool UMaterialInstance::GetUsageByFlag(EMaterialUsage Usage) const
{
	return (UsageFlags & (1u << Usage)) != 0;
}

void UMaterialInstance::SetUsageByFlag(EMaterialUsage Usage, bool NewValue)
{
	BasePropertyOverrides.bOverride_UsageFlags |= 1u << Usage;

	if (NewValue)
	{
		BasePropertyOverrides.UsageFlags |= 1u << Usage;
		UsageFlags |= 1u << Usage;
	}
	else
	{
		BasePropertyOverrides.UsageFlags &= ~(1u << Usage);
		UsageFlags &= ~(1u << Usage);
	}
}

bool UMaterialInstance::SetMaterialUsage(EMaterialUsage Usage)
{
	UMaterial* Material = GetMaterial();
	if (Material == nullptr)
	{
		return false;
	}

	// Material usage is only relevant for materials that can be applied onto a mesh / use with different vertex factories.
	const EMaterialDomain MaterialDomain = Material->MaterialDomain;
	if (MaterialDomain != MD_Surface && MaterialDomain != MD_DeferredDecal && MaterialDomain != MD_Volume)
	{
		return false;
	}

	// Check that the material has been flagged for use with the given usage flag.
	if (!GetUsageByFlag(Usage))
	{
		// For materials which do not have their UsageFlags set the DefaultMaterial<type> should be used in game
		// Leaving this GIsEditor ensures that in game on PC will not look different than on the Consoles as we will not be compiling shaders on the fly
		// Using FApp::IsGame is not enough, during UWorld::SendAllEndOfFrameUpdates() it returns false even it PIE is running.
		// Ideally, FApp::IsGame should be revisited, but to avoid causing regressions, we also check IsPlayingSessionInEditor.
		bool bIsGame = FApp::IsGame();
#if WITH_EDITOR
		bIsGame = bIsGame || (GEditor && GEditor->IsPlayingSessionInEditor());
#endif
		if (GIsEditor && !bIsGame && Material->bAutomaticallySetUsageInEditor)
		{
			check(IsInGameThread());
			//Do not warn the user during automation testing
			if (!GIsAutomationTesting)
			{
				UE_LOGF(LogMaterial, Display, "Material %ls needed to set usage flag %ls", *GetPathName(), *GetUsageName(Usage));
			}

			// Open a material update context so this material can be modified safely.
			{
				FMaterialUpdateContext UpdateContext(
					// We need to sync with the rendering thread but don't reregister components
					// because SetMaterialUsage may be called during registration!
					FMaterialUpdateContext::EOptions::SyncWithRenderingThread
				);
				UpdateContext.AddMaterialInstance(this);

				// If the flag is missing in the editor, set it, and recompile shaders.
				SetUsageByFlag(Usage, true);

				// Compile and force the Id to be regenerated, since we changed the material in a way that changes compilation
				CacheResourceShadersForRendering(EMaterialShaderPrecompileMode::None);
			}

			// Mark the package dirty so that hopefully it will be saved with the new usage flag.
			MarkPackageDirty();

#if WITH_EDITOR
			// Rebuild any open material instance editors.
			UMaterialEditingLibrary::RebuildMaterialInstanceEditors(GetMaterial());
#endif
		}
		else
		{
			const uint32 UsageFlagBit = (1u << Usage);
			if ((UsageFlagWarnings & UsageFlagBit) == 0)
			{
				UE_LOGF(LogMaterial, Warning, "Material %ls missing usage flag %ls! Default Material will be used in game.", *GetPathName(), *GetUsageName(Usage));

				if (Material->bAutomaticallySetUsageInEditor)
				{
					UE_LOGF(LogMaterial, Warning, "     The material instance will recompile every editor launch until resaved.");
				}

				UsageFlagWarnings |= UsageFlagBit;
			}

			// Return failure if the flag is missing in game, since compiling shaders in game is not supported on some platforms.
			return false;
		}
	}
	return true;
}

void UMaterialInstance::GetDependencies(TSet<UMaterialInterface*>& Dependencies)
{
	if (GetReentrantFlag())
	{
		return;
	}

	Dependencies.Add(this);
	
	if (Parent)
	{
		FMICReentranceGuard	Guard(this);
		Parent->GetDependencies(Dependencies);
	}
}

bool UMaterialInstance::IsDependent(UMaterialInterface* TestDependency)
{
	if(TestDependency == this)
	{
		return true;
	}
	else if(Parent)
	{
		if(GetReentrantFlag())
		{
			return true;
		}

		FMICReentranceGuard	Guard(this);
		return Parent->IsDependent(TestDependency);
	}
	else
	{
		return false;
	}
}

bool UMaterialInstance::IsDependent_Concurrent(UMaterialInterface* TestDependency, TMicRecursionGuard RecursionGuard)
{
	if (TestDependency == this)
	{
		return true;
	}
	else if (Parent)
	{
		if (RecursionGuard.Contains(this))
		{
			return true;
		}

		RecursionGuard.Set(this);
		return Parent->IsDependent_Concurrent(TestDependency, RecursionGuard);
	}
	else
	{
		return false;
	}
}

// Deprecated in 5.7
void UMaterialInstanceDynamic::CopyScalarAndVectorParameters(const UMaterialInterface& SourceMaterialToCopyFrom, ERHIFeatureLevel::Type FeatureLevel)
{
	return CopyScalarAndVectorParameters(SourceMaterialToCopyFrom, GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
}

void UMaterialInstanceDynamic::CopyScalarAndVectorParameters(const UMaterialInterface& SourceMaterialToCopyFrom, EShaderPlatform ShaderPlatform)
{
	check(IsInGameThread());

	// We get the parameter list form the input material, this might be different from the base material
	// because static (bool) parameters can cause some parameters to be hidden
	FMaterialResource* MaterialResource = GetMaterialResource(ShaderPlatform);

	if (MaterialResource)
	{
		// first, clear out all the parameter values
		ClearParameterValuesInternal(EMaterialInstanceClearParameterFlag::Numeric);

		TArrayView<const FMaterialNumericParameterInfo> Array = MaterialResource->GetUniformNumericParameterExpressions();
		for (int32 i = 0, Count = Array.Num(); i < Count; ++i)
		{
			const FMaterialNumericParameterInfo& Parameter = Array[i];

			const UMaterialInterface* CheckMaterial = &SourceMaterialToCopyFrom;
			FMaterialParameterMetadata ParameterValue;
			bool bFoundValue = false;
			while (CheckMaterial)
			{
				const UMaterialInstance* CheckMaterialInstance = Cast<UMaterialInstance>(CheckMaterial);
				if (CheckMaterialInstance)
				{
					if (CheckMaterialInstance->GetParameterOverrideValue(Parameter.ParameterType, Parameter.ParameterInfo, ParameterValue))
					{
						bFoundValue = true;
						break;
					}
					CheckMaterial = CheckMaterialInstance->Parent;
				}
				else
				{
					break;
				}
			}

			if (!bFoundValue)
			{
				const UE::Shader::FValue DefaultValue = MaterialResource->GetUniformExpressions().GetDefaultParameterValue(Parameter.ParameterType, Parameter.DefaultValueOffset);
				ParameterValue.Value = FMaterialParameterValue(Parameter.ParameterType, DefaultValue);
			}

			AddParameterValueInternal(FMaterialParameterInfo(Parameter.ParameterInfo), ParameterValue);
		}

		// now, init the resources
		InitResources();
	}
}

void UMaterialInstanceDynamic::SetNaniteOverride(UMaterialInterface* InMaterial)
{
	NaniteOverrideMaterial.SetOverrideMaterial(InMaterial, true);
}

float UMaterialInstanceDynamic::GetOpacityMaskClipValue() const
{
	return Parent ? Parent->GetOpacityMaskClipValue() : 0.0f;
}

bool UMaterialInstanceDynamic::GetCastDynamicShadowAsMasked() const
{
	return Parent ? Parent->GetCastDynamicShadowAsMasked() : false;
}

EBlendMode UMaterialInstanceDynamic::GetBlendMode() const
{
	return Parent ? Parent->GetBlendMode() : BLEND_Opaque;
}

bool UMaterialInstanceDynamic::IsTwoSided() const
{
	return Parent ? Parent->IsTwoSided() : false;
}

bool UMaterialInstanceDynamic::IsThinSurface() const
{
	return Parent ? Parent->IsThinSurface() : false;
}

bool UMaterialInstanceDynamic::IsTranslucencyWritingVelocity() const
{
	return Parent ? Parent->IsTranslucencyWritingVelocity() : false;
}

bool UMaterialInstanceDynamic::IsDitheredLODTransition() const
{
	return Parent ? Parent->IsDitheredLODTransition() : false;
}

bool UMaterialInstanceDynamic::IsMasked() const
{
	return Parent ? Parent->IsMasked() : false;
}

uint32 UMaterialInstanceDynamic::GetUsageFlags() const
{
	return Parent ? Parent->GetUsageFlags() : 0;
}

bool UMaterialInstanceDynamic::GetUsageByFlag(EMaterialUsage Usage) const
{
	return Parent ? Parent->GetUsageByFlag(Usage) : false;
}

bool UMaterialInstanceDynamic::SetMaterialUsage(EMaterialUsage Usage)
{
	return Parent ? Parent->SetMaterialUsage(Usage) : false;
}

FDisplacementScaling UMaterialInstanceDynamic::GetDisplacementScaling() const
{
	return Parent ? Parent->GetDisplacementScaling() : FDisplacementScaling();
}

bool UMaterialInstanceDynamic::IsDisplacementFadeEnabled() const
{
	return Parent ? Parent->IsDisplacementFadeEnabled() : false;
}

FDisplacementFadeRange UMaterialInstanceDynamic::GetDisplacementFadeRange() const
{
	return Parent ? Parent->GetDisplacementFadeRange() : FDisplacementFadeRange();
}

float UMaterialInstanceDynamic::GetMaxWorldPositionOffsetDisplacement() const
{
	return Parent ? Parent->GetMaxWorldPositionOffsetDisplacement() : 0.0f;
}

bool UMaterialInstanceDynamic::HasPixelAnimation() const
{
	return Parent ? Parent->HasPixelAnimation() : false;
}

FMaterialShadingModelField UMaterialInstanceDynamic::GetShadingModels() const
{
	return Parent ? Parent->GetShadingModels() : MSM_DefaultLit;
}

bool UMaterialInstanceDynamic::IsShadingModelFromMaterialExpression() const
{
	return Parent ? Parent->IsShadingModelFromMaterialExpression() : false;
}

void UMaterialInstance::CopyMaterialInstanceParameters(UMaterialInterface* Source)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);
	SCOPE_CYCLE_COUNTER(STAT_MaterialInstance_CopyMatInstParams);

	if ((Source != nullptr) && (Source != this))
	{
		// First, clear out all the parameter values
		ClearParameterValuesInternal();

		//setup some arrays to use
		TArray<FMaterialParameterInfo> OutParameterInfo;
		TArray<FGuid> Guids;

		for (int32 ParameterTypeIndex = 0; ParameterTypeIndex < NumMaterialParameterTypes; ++ParameterTypeIndex)
		{
			const EMaterialParameterType ParameterType = (EMaterialParameterType)ParameterTypeIndex;
			if (!IsStaticMaterialParameter(ParameterType))
			{
				GetAllParameterInfoOfType(ParameterType, OutParameterInfo, Guids);
				ReserveParameterValuesInternal(ParameterType, OutParameterInfo.Num());
				for (const FMaterialParameterInfo& ParameterInfo : OutParameterInfo)
				{
					FMaterialParameterMetadata SourceValue;
					if (Source->GetParameterValue(ParameterType, ParameterInfo, SourceValue))
					{
						AddParameterValueInternal(ParameterInfo, SourceValue, EMaterialSetParameterValueFlags::SetCurveAtlas);
					}
				}
			}
		}
		
		// Now, init the resources
		InitResources();

#if WITH_EDITOR
		FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
	}
}

FMaterialResource* UMaterialInstance::GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel)
{
	if (bHasStaticPermutationResource)
	{
		if (QualityLevel == EMaterialQualityLevel::Num)
		{
			QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		}
		return FindMaterialResource(StaticPermutationMaterialResources, InShaderPlatform, QualityLevel, true);
	}

	//there was no static permutation resource
	return Parent ? Parent->GetMaterialResource(InShaderPlatform, QualityLevel) : nullptr;
}

bool UMaterialInstance::HasVertexInterpolator() const
{
	return Parent ? Parent->HasVertexInterpolator() : false;
}

bool UMaterialInstance::HasCustomizedUVs() const
{
	return Parent ? Parent->HasCustomizedUVs() : false;
}

bool UMaterialInstance::WritesToRuntimeVirtualTexture() const
{
	return Parent ? Parent->WritesToRuntimeVirtualTexture() : false;
}

bool UMaterialInstance::HasMeshPaintTexture() const
{
	return Parent ? Parent->HasMeshPaintTexture() : false;
}

bool UMaterialInstance::HasCustomPrimitiveData() const
{
	return Parent ? Parent->HasCustomPrimitiveData() : false;
}

const FMaterialResource* UMaterialInstance::GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel) const
{
	return const_cast<UMaterialInstance*>(this)->GetMaterialResource(InShaderPlatform, QualityLevel);
}

FMaterialRenderProxy* UMaterialInstance::GetRenderProxy() const
{
	return GetInstanceRenderProxy();
}

UPhysicalMaterial* UMaterialInstance::GetPhysicalMaterial() const
{
	if(GetReentrantFlag())
	{
		return UMaterial::GetDefaultMaterial(MD_Surface)->GetPhysicalMaterial();
	}

	FMICReentranceGuard	Guard(const_cast<UMaterialInstance*>(this));  // should not need this to determine loop

	if (bOverridePhysMaterial)
	{
		if (PhysMaterial)
		{
			return PhysMaterial;
		}
		else
		{
			check(GEngine->DefaultPhysMaterial != NULL);
			return GEngine->DefaultPhysMaterial;
		}
	}
	else if (Parent)
	{
		// If no physical material has been associated with this instance, simply use the parent's physical material.
		return Parent->GetPhysicalMaterial();
	}
	else
	{
		// no material specified and no parent, fall back to default physical material
		check( GEngine->DefaultPhysMaterial != NULL );
		return GEngine->DefaultPhysMaterial;
	}
}

UPhysicalMaterialMask* UMaterialInstance::GetPhysicalMaterialMask() const
{
	return nullptr;
}

UPhysicalMaterial* UMaterialInstance::GetPhysicalMaterialFromMap(int32 Index) const
{
	if (Index < 0 || Index >= EPhysicalMaterialMaskColor::MAX)
	{
		return nullptr;
	}
	return PhysicalMaterialMap[Index];
}

UMaterialInterface* UMaterialInstance::GetNaniteOverride(TMicRecursionGuard RecursionGuard) const
{
	if (NaniteOverrideMaterial.bEnableOverride)
	{
		return NaniteOverrideMaterial.GetOverrideMaterial();
	}
	else if (Parent && !RecursionGuard.Contains(this))
	{
		RecursionGuard.Set(this);
		return Parent->GetNaniteOverride(RecursionGuard);
	}
	else
	{
		return nullptr;
	}
}

#if WITH_EDITORONLY_DATA

void UMaterialInstance::SetStaticSwitchParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, bool Value)
{
	check(GIsEditor || IsRunningCommandlet());

	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	check(EditorOnly);

	for (FStaticSwitchParameter& StaticSwitches : StaticParametersRuntime.StaticSwitchParameters)
	{
		if (StaticSwitches.ParameterInfo == ParameterInfo)
		{
			StaticSwitches.bOverride = true;
			StaticSwitches.Value = Value;
			return;
		}
	}

	new(StaticParametersRuntime.StaticSwitchParameters) FStaticSwitchParameter(ParameterInfo, Value, true, FGuid());
}

void UMaterialInstance::SetStaticComponentMaskParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, FStaticComponentMaskValue Value)
{
	check(GIsEditor || IsRunningCommandlet());

	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	check(EditorOnly);

	for (FStaticComponentMaskParameter& Parameter : EditorOnly->StaticParameters.StaticComponentMaskParameters)
	{
		if (Parameter.ParameterInfo == ParameterInfo)
		{
			Parameter.bOverride = true;
			Parameter.R = Value.R;
			Parameter.G = Value.G;
			Parameter.B = Value.B;
			Parameter.A = Value.A;
			return;
		}
	}

	new(EditorOnly->StaticParameters.StaticComponentMaskParameters) FStaticComponentMaskParameter(ParameterInfo, Value.R, Value.G, Value.B, Value.A, true, FGuid());
}

void UMaterialInterface::GetStaticParameterValues(FStaticParameterSet& OutStaticParameters)
{
	check(IsInGameThread());

	if ((AllowCachingStaticParameterValuesCounter > 0) && CachedStaticParameterValues.IsSet())
	{
		OutStaticParameters = CachedStaticParameterValues.GetValue();
		return;
	}

	TMap<FMaterialParameterInfo, FMaterialParameterMetadata> ParameterValues;
	for (int32 ParameterTypeIndex = 0; ParameterTypeIndex < NumMaterialParameterTypes; ++ParameterTypeIndex)
	{
		const EMaterialParameterType ParameterType = (EMaterialParameterType)ParameterTypeIndex;
		if (IsStaticMaterialParameter(ParameterType))
		{
			ParameterValues.Reset();
			GetAllParametersOfType(ParameterType, ParameterValues);
			OutStaticParameters.AddParametersOfType(ParameterType, ParameterValues);
		}
	}

	if(UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(this))
	{
		if (UMaterialInstanceEditorOnlyData* EditorOnly = MaterialInstance->GetEditorOnlyData())
		{
			OutStaticParameters.EditorOnly.TerrainLayerWeightParameters = EditorOnly->StaticParameters.TerrainLayerWeightParameters;
		}
	}

	FMaterialLayersFunctions MaterialLayers;
	OutStaticParameters.bHasMaterialLayers = GetMaterialLayers(MaterialLayers);
	if (OutStaticParameters.bHasMaterialLayers)
	{
		OutStaticParameters.MaterialLayers = MoveTemp(MaterialLayers.GetRuntime());
		OutStaticParameters.EditorOnly.MaterialLayers = MoveTemp(MaterialLayers.EditorOnly);
	}

	OutStaticParameters.Validate();

	if (AllowCachingStaticParameterValuesCounter > 0)
	{
		CachedStaticParameterValues = OutStaticParameters;
	}
}
#endif // WITH_EDITORONLY_DATA

void UMaterialInstance::GetAllParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const
{
	FMaterialInheritanceChain InstanceChain;
	GetMaterialInheritanceChain(InstanceChain);

	OutParameters.Reset();
	InstanceChain.GetCachedExpressionData().GetAllParametersOfType(Type, OutParameters);

	TArray<int32, TInlineAllocator<16>> LayerIndexRemap;
	LayerIndexRemap.Empty(GetCachedInstanceData().ParentLayerIndexRemap.Num());
	for (int32 LayerIndex = 0; LayerIndex < GetCachedInstanceData().ParentLayerIndexRemap.Num(); ++LayerIndex)
	{
		LayerIndexRemap.Add(LayerIndex);
	}

	// We walk the inheritance hierarchy backwards to the root, so we keep track of overrides that are set, to avoid setting them again from less-derived instances
	// Alternately could walk the hierarchy starting from the root, but then we'd need an alternate way to track layer index remapping
	TSet<FMaterialParameterInfo, DefaultKeyFuncs<FMaterialParameterInfo>, TInlineSetAllocator<32>> OverridenParameters;

	for (int32 Index = 0; Index < InstanceChain.MaterialInstances.Num(); ++Index)
	{
		const UMaterialInstance* Instance = InstanceChain.MaterialInstances[Index];
		const bool bSetOverride = (Index == 0); // Only set the override flag for parameters overriden by the current material (always at slot0)
		switch (Type)
		{
		case EMaterialParameterType::Scalar: GameThread_ApplyParameterOverrides(Instance->ScalarParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::Vector: GameThread_ApplyParameterOverrides(Instance->VectorParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::DoubleVector: GameThread_ApplyParameterOverrides(Instance->DoubleVectorParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::Texture: GameThread_ApplyParameterOverrides(Instance->TextureParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::TextureCollection: GameThread_ApplyParameterOverrides(Instance->TextureCollectionParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::RuntimeVirtualTexture: GameThread_ApplyParameterOverrides(Instance->RuntimeVirtualTextureParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::SparseVolumeTexture: GameThread_ApplyParameterOverrides(Instance->SparseVolumeTextureParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::Font: GameThread_ApplyParameterOverrides(Instance->FontParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::StaticSwitch: GameThread_ApplyParameterOverrides(Instance->StaticParametersRuntime.StaticSwitchParameters, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
		case EMaterialParameterType::ParameterCollection: GameThread_ApplyParameterOverrides(Instance->ParameterCollectionParameterValues, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters); break;
#if WITH_EDITORONLY_DATA
		case EMaterialParameterType::StaticComponentMask:
			GameThread_ApplyParameterOverrides(Instance->GetEditorOnlyData()->StaticParameters.StaticComponentMaskParameters, LayerIndexRemap, bSetOverride, OverridenParameters, OutParameters);
			break;
#endif // WITH_EDITORONLY_DATA
		default: checkNoEntry();
		}

		if (Index + 1 < InstanceChain.MaterialInstances.Num())
		{
			const UMaterialInstance* ParentInstance = InstanceChain.MaterialInstances[Index + 1];
			RemapLayersForParent(LayerIndexRemap, ParentInstance->GetCachedInstanceData().ParentLayerIndexRemap.Num(), Instance->GetCachedInstanceData().ParentLayerIndexRemap);
		}
	}
}

#if WITH_EDITORONLY_DATA
bool UMaterialInstance::IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
{
	// Important that local function references are listed first so that traversing for a parameter
	// value we always hit the highest material in the hierarchy that can give us a valid value
	if (StaticParametersRuntime.bHasMaterialLayers)
	{
		for (UMaterialFunctionInterface* Layer : StaticParametersRuntime.MaterialLayers.Layers)
		{
			if (Layer)
			{
				if (!Layer->IterateDependentFunctions(Predicate))
				{
					return false;
				}
				if (!Predicate(Layer))
				{
					return false;
				}
			}
		}

		for (UMaterialFunctionInterface* Blend : StaticParametersRuntime.MaterialLayers.Blends)
		{
			if (Blend)
			{
				if (!Blend->IterateDependentFunctions(Predicate))
				{
					return false;
				}
				if (!Predicate(Blend))
				{
					return false;
				}
			}
		}
	}

	return Parent ? Parent->IterateDependentFunctions(Predicate) : true;
}

void UMaterialInstance::GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
{
	IterateDependentFunctions([&DependentFunctions](UMaterialFunctionInterface* MaterialFunction) -> bool
		{
			DependentFunctions.AddUnique(MaterialFunction);
			return true;
		});
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
void UMaterialInstance::ForceRecompileForRendering(EMaterialShaderPrecompileMode CompileMode)
{
	UpdateCachedData();
	CacheResourceShadersForRendering(CompileMode);
}
#endif // WITH_EDITOR

void UMaterialInstance::InitStaticPermutation(EMaterialShaderPrecompileMode PrecompileMode)
{
	UpdateOverridableBaseProperties();

#if WITH_EDITORONLY_DATA
	if (!GetPackage()->HasAnyPackageFlags(PKG_FilterEditorOnly))
	{
		bHasStaticPermutationResource = Parent && (HasStaticParameters() || HasOverridenBaseProperties());
		ValidateStaticPermutationAllowed();
	}
#else
	// Discard the MI's cooked shadermap - it was compiled against a parent that is no longer available, so its uniform buffer layout is incompatible with WorldGridMaterial's shaders.
	// Even though we already set BlendMode to BLEND_Opaque, GetMaterialResource() would still serve up the MIC's own StaticPermutationMaterialResources (compiled against the missing parent) 
	// while the render proxy uses WorldGridMaterial's uniform buffers.
	// This produces at draw-time a 'FOpaqueBasePassUniformParameters vs TranslucentBasePass' check failure because the two were compiled against different parents. 
	// Clearing the flag makes GetMaterialResource() delegate entirely to Parent (WorldGridMaterial), so shadermap and uniform buffers come from the same source and stay consistent.
	if (!Parent && bHasStaticPermutationResource)
	{
		UE_LOGF(LogMaterial, Error, "MaterialInstance [%ls] clearing bHasStaticPermutationResource to avoid shadermap/blendmode mismatch with default material.", *GetName());
		bHasStaticPermutationResource = false;
	}
#endif // WITH_EDITORONLY_DATA

	FMaterialResourceDeferredDeletionArray ResourcesToFree;

	if ( FApp::CanEverRender() ) 
	{
		// Cache shaders for the current platform to be used for rendering
		CacheResourceShadersForRendering(PrecompileMode, ResourcesToFree);
	}

	FMaterial::DeferredDeleteArray(ResourcesToFree);
}

EBlendMode ConvertLegacyBlendMode(EBlendMode InBlendMode, FMaterialShadingModelField InShadingModels);

static void SanitizeBlendMode(TEnumAsByte<EBlendMode>& InBlendMode)
{
	if (InBlendMode == BLEND_TranslucentColoredTransmittance)
	{
		InBlendMode = BLEND_Translucent;
	}
}

void UMaterialInstance::UpdateOverridableBaseProperties()
{
	//Parents base property overrides have to be cached by now.
	//This should be done on PostLoad()
	//Or via an FMaterialUpdateContext when editing.

	if (!Parent)
	{
		OpacityMaskClipValue = 0.0f;
		BlendMode = BLEND_Opaque;
		ShadingModels = MSM_DefaultLit;
		TwoSided = 0;
		bIsThinSurface = false;
		DitheredLODTransition = 0;
		bIsShadingModelFromMaterialExpression = 0;
		bOutputTranslucentVelocity = false;
		bHasPixelAnimation = false;
		bEnableTessellation = false;
		DisplacementScaling = FDisplacementScaling();
		bEnableDisplacementFade = false;
		DisplacementFadeRange = FDisplacementFadeRange();
		MaxWorldPositionOffsetDisplacement = 0.0f;
		bCompatibleWithLumenCardSharing = false;
		UsageFlags = 0;
		return;
	}

	if (BasePropertyOverrides.bOverride_OpacityMaskClipValue)
	{
		OpacityMaskClipValue = BasePropertyOverrides.OpacityMaskClipValue;
	}
	else
	{
		OpacityMaskClipValue = Parent->GetOpacityMaskClipValue();
		BasePropertyOverrides.OpacityMaskClipValue = OpacityMaskClipValue;
	}

	if ( BasePropertyOverrides.bOverride_CastDynamicShadowAsMasked )
	{
		bCastDynamicShadowAsMasked = BasePropertyOverrides.bCastDynamicShadowAsMasked;
	}
	else
	{
		bCastDynamicShadowAsMasked = Parent->GetCastDynamicShadowAsMasked();
		BasePropertyOverrides.bCastDynamicShadowAsMasked = bCastDynamicShadowAsMasked;
	}

	if(BasePropertyOverrides.bOverride_OutputTranslucentVelocity)
	{
		bOutputTranslucentVelocity = BasePropertyOverrides.bOutputTranslucentVelocity;
	}
	else
	{
		bOutputTranslucentVelocity = Parent->IsTranslucencyWritingVelocity();
		BasePropertyOverrides.bOutputTranslucentVelocity = bOutputTranslucentVelocity;
	}

	if (BasePropertyOverrides.bOverride_bHasPixelAnimation)
	{
		bHasPixelAnimation = BasePropertyOverrides.bHasPixelAnimation;
	}
	else
	{
		bHasPixelAnimation = Parent->HasPixelAnimation();
		BasePropertyOverrides.bHasPixelAnimation = bHasPixelAnimation;
	}

	if (BasePropertyOverrides.bOverride_bEnableTessellation)
	{
		bEnableTessellation = BasePropertyOverrides.bEnableTessellation;
	}
	else
	{
		bEnableTessellation = Parent->IsTessellationEnabled();
		BasePropertyOverrides.bEnableTessellation = bEnableTessellation;
	}

	if (BasePropertyOverrides.bOverride_ShadingModel)
	{
		if (BasePropertyOverrides.ShadingModel == MSM_FromMaterialExpression)
		{
			// Can't override using MSM_FromMaterialExpression, simply fall back to parent
			ShadingModels = Parent->GetShadingModels();
			bIsShadingModelFromMaterialExpression = Parent->IsShadingModelFromMaterialExpression();
		}
		else
		{
			// It's only possible to override using a single shading model
			ShadingModels = FMaterialShadingModelField(BasePropertyOverrides.ShadingModel);
			bIsShadingModelFromMaterialExpression = 0;
		}
	}
	else
	{
		ShadingModels = Parent->GetShadingModels();
		bIsShadingModelFromMaterialExpression = Parent->IsShadingModelFromMaterialExpression();

		if (bIsShadingModelFromMaterialExpression)
		{
			BasePropertyOverrides.ShadingModel = MSM_FromMaterialExpression; 
		}
		else
		{
			ensure(ShadingModels.CountShadingModels() == 1);
			BasePropertyOverrides.ShadingModel = ShadingModels.GetFirstShadingModel(); 
		}
	}

	if (Substrate::IsSubstrateEnabled())
	{
		BasePropertyOverrides.BlendMode = ConvertLegacyBlendMode(BasePropertyOverrides.BlendMode, ShadingModels);
		BlendMode = ConvertLegacyBlendMode(Parent->GetBlendMode(), ShadingModels);
	}
	else
	{
		SanitizeBlendMode(BlendMode);
		SanitizeBlendMode(BasePropertyOverrides.BlendMode);
	}

	if (BasePropertyOverrides.bOverride_BlendMode)
	{
		BlendMode = BasePropertyOverrides.BlendMode;
	}
	else
	{
		BlendMode = Parent->GetBlendMode();
		BasePropertyOverrides.BlendMode = BlendMode;
	}

#if !WITH_EDITOR
	// Filter out ShadingModels field to a current platform settings
	FilterOutPlatformShadingModels(GMaxRHIShaderPlatform, ShadingModels);
#endif

	if (BasePropertyOverrides.bOverride_TwoSided)
	{
		TwoSided = BasePropertyOverrides.TwoSided != 0;
	}
	else
	{
		TwoSided = Parent->IsTwoSided();
		BasePropertyOverrides.TwoSided = TwoSided;
	}

	if (BasePropertyOverrides.bOverride_bIsThinSurface)
	{
		bIsThinSurface = BasePropertyOverrides.bIsThinSurface != 0;
	}
	else
	{
		bIsThinSurface = Parent->IsThinSurface();
		BasePropertyOverrides.bIsThinSurface = bIsThinSurface;
	}

	if (BasePropertyOverrides.bOverride_DitheredLODTransition)
	{
		DitheredLODTransition = BasePropertyOverrides.DitheredLODTransition != 0;
	}
	else
	{
		DitheredLODTransition = Parent->IsDitheredLODTransition();
		BasePropertyOverrides.DitheredLODTransition = DitheredLODTransition;
	}

	if (BasePropertyOverrides.bOverride_DisplacementScaling)
	{
		DisplacementScaling = BasePropertyOverrides.DisplacementScaling;
	}
	else
	{
		DisplacementScaling = Parent->GetDisplacementScaling();
		BasePropertyOverrides.DisplacementScaling = DisplacementScaling;
	}

	if (BasePropertyOverrides.bOverride_bEnableDisplacementFade)
	{
		bEnableDisplacementFade = BasePropertyOverrides.bEnableDisplacementFade;
	}
	else
	{
		bEnableDisplacementFade = Parent->IsDisplacementFadeEnabled();
		BasePropertyOverrides.bEnableDisplacementFade = bEnableDisplacementFade;
	}

	if (BasePropertyOverrides.bOverride_DisplacementFadeRange)
	{
		DisplacementFadeRange = BasePropertyOverrides.DisplacementFadeRange;
	}
	else
	{
		DisplacementFadeRange = Parent->GetDisplacementFadeRange();
		BasePropertyOverrides.DisplacementFadeRange = DisplacementFadeRange;
	}

	if (BasePropertyOverrides.bOverride_MaxWorldPositionOffsetDisplacement)
	{
		MaxWorldPositionOffsetDisplacement = BasePropertyOverrides.MaxWorldPositionOffsetDisplacement;
	}
	else
	{
		MaxWorldPositionOffsetDisplacement = Parent->GetMaxWorldPositionOffsetDisplacement();
		BasePropertyOverrides.MaxWorldPositionOffsetDisplacement = MaxWorldPositionOffsetDisplacement;
	}

	if (BasePropertyOverrides.bOverride_CompatibleWithLumenCardSharing)
	{
		bCompatibleWithLumenCardSharing = BasePropertyOverrides.bCompatibleWithLumenCardSharing;
	}
	else
	{
		bCompatibleWithLumenCardSharing = Parent->IsCompatibleWithLumenCardSharing();
		BasePropertyOverrides.bCompatibleWithLumenCardSharing = bCompatibleWithLumenCardSharing;
	}

	const uint32 ParentUsageFlags = Parent->GetUsageFlags();
	UsageFlags = ParentUsageFlags;
	// Add flags that are overriden and set.
	UsageFlags |= BasePropertyOverrides.bOverride_UsageFlags & BasePropertyOverrides.UsageFlags;
	// Remove flags that are overriden and unset.
	UsageFlags &= ~(BasePropertyOverrides.bOverride_UsageFlags & ~BasePropertyOverrides.UsageFlags);
	// Non overridden flags default to parent value.
	BasePropertyOverrides.UsageFlags |= ~BasePropertyOverrides.bOverride_UsageFlags & ParentUsageFlags;
	// Reset the warning flags.
	UsageFlagWarnings = 0;
}

void UMaterialInstance::GetAllShaderMaps(TArray<FMaterialShaderMap*>& OutShaderMaps)
{
	for (FMaterialResource* CurrentResource : StaticPermutationMaterialResources)
	{
		FMaterialShaderMap* ShaderMap = CurrentResource->GetGameThreadShaderMap();
		OutShaderMaps.Add(ShaderMap);
	}
}

FMaterialResource* UMaterialInstance::AllocatePermutationResource()
{
	return new FMaterialResource();
}

void UMaterialInstance::CacheResourceShadersForRendering(EMaterialShaderPrecompileMode PrecompileMode, FMaterialResourceDeferredDeletionArray& OutResourcesToFree)
{
	check(IsInGameThread() || IsAsyncLoading());

	UpdateOverridableBaseProperties();

	if (bHasStaticPermutationResource && FApp::CanEverRender())
	{
		check(IsA(UMaterialInstanceConstant::StaticClass()));
		UMaterial* BaseMaterial = GetMaterial();

		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;

		TArray<FMaterialResource*> ResourcesToCache;
		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			// Only cache shaders for the quality level that will actually be used to render
			// In cooked build, there is no shader compilation but this is still needed to
			// register the loaded shadermap
			FMaterialResource* CurrentResource = FindOrCreateMaterialResource(StaticPermutationMaterialResources, BaseMaterial, this, ShaderPlatform, ActiveQualityLevel);
			check(CurrentResource);

			if (IsUsingNewHLSLGenerator())
			{
				// Release resources from unused qualities. For some reason, FindOrCreateMaterialResource checks material quality usage
				// but FindMaterialResource doesn't. The two functions can choose differently if unused quality resources aren't removed.
				// When that happens, stale material resources may be used for rendering and cause troubles
				TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num>> QualityLevelsUsed;
				GetQualityLevelUsage(QualityLevelsUsed, ShaderPlatform);

				for (int32 Index = 0; Index < StaticPermutationMaterialResources.Num(); ++Index)
				{
					FMaterialResource* MaterialResource = StaticPermutationMaterialResources[Index];
					if (MaterialResource != CurrentResource
						&& MaterialResource->GetFeatureLevel() == FeatureLevel
						&& MaterialResource->GetQualityLevel() != EMaterialQualityLevel::Num
						&& !QualityLevelsUsed[MaterialResource->GetQualityLevel()])
					{
						OutResourcesToFree.Add(MaterialResource);
						StaticPermutationMaterialResources.RemoveAtSwap(Index);
						--Index;
					}
				}
			}

			ResourcesToCache.Reset();
			ResourcesToCache.Add(CurrentResource);
			CacheShadersForResources(ShaderPlatform, ResourcesToCache, PrecompileMode);
		}
	}

	RecacheUniformExpressions(true);
	InitResources();
}

void UMaterialInstance::CacheResourceShadersForRendering(EMaterialShaderPrecompileMode PrecompileMode)
{
	FMaterialResourceDeferredDeletionArray ResourcesToFree;
	CacheResourceShadersForRendering(PrecompileMode, ResourcesToFree);
	FMaterial::DeferredDeleteArray(ResourcesToFree);
}

#if WITH_EDITOR
void UMaterialInstance::CacheResourceShadersForCooking(
	EShaderPlatform ShaderPlatform,
	TArray<FMaterialResourceForCooking>& OutCachedMaterialResources,
	EMaterialShaderPrecompileMode PrecompileMode,
	const ITargetPlatform* TargetPlatform,
	bool bBlocking
)
{
	if (bHasStaticPermutationResource)
	{
		UMaterial* BaseMaterial = GetMaterial();

		TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num> > QualityLevelsUsed;
		GetQualityLevelUsageForCooking(QualityLevelsUsed, ShaderPlatform);

		const UShaderPlatformQualitySettings* MaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(ShaderPlatform);
		bool bNeedDefaultQuality = false;

		ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);

		// only new resources need to have CacheShaders() called on them, whereas OutCachedMaterialResources
		// may already contain resources for another shader platform
		TArray<FMaterialResource*> NewResourcesToCache;
		for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; QualityLevelIndex++)
		{
			// Cache all quality levels actually used
			if (QualityLevelsUsed[QualityLevelIndex])
			{
				FMaterialResource* NewResource = AllocatePermutationResource();
				NewResource->SetMaterial(BaseMaterial, this, ShaderPlatform, (EMaterialQualityLevel::Type)QualityLevelIndex);
				NewResourcesToCache.Add(NewResource);
			}
			else
			{
				const FMaterialQualityOverrides& QualityOverrides = MaterialQualitySettings->GetQualityOverrides((EMaterialQualityLevel::Type)QualityLevelIndex);
				if (!QualityOverrides.bDiscardQualityDuringCook)
				{
					// don't have an explicit resource for this quality level, but still need to support it, so make sure we include a default quality resource
					bNeedDefaultQuality = true;
				}
			}
		}

		if (bNeedDefaultQuality)
		{
			FMaterialResource* NewResource = AllocatePermutationResource();
			NewResource->SetMaterial(BaseMaterial, this, ShaderPlatform);
			NewResourcesToCache.Add(NewResource);
		}

		// The editor needs to block if the caching call comes from cook on the fly, where the polling mechanisms are not active.
		// This is important so that the jobs finish and the CacheShadersCompletion() callback is triggered via FinishCacheShaders()!
		if (bBlocking)
		{
			CacheShadersForResources(ShaderPlatform, NewResourcesToCache, PrecompileMode, TargetPlatform);
		}
		else
		{
			// For cooking, we can call the begin function and it will be completed as part of the polling mechanism.
			BeginCacheShadersForResources(ShaderPlatform, NewResourcesToCache, PrecompileMode, TargetPlatform);
		}

		OutCachedMaterialResources.Reserve(NewResourcesToCache.Num() + OutCachedMaterialResources.Num());
		for (FMaterialResource* NewResource : NewResourcesToCache)
		{
			OutCachedMaterialResources.Add({ NewResource , ShaderPlatform });
		}
	}
}
#endif // WITH_EDITOR

namespace MaterialInstanceImpl
{
	void HandleCacheShadersForResourcesErrors(bool bSuccess, EShaderPlatform ShaderPlatform, UMaterialInstance* This, FMaterialResource* CurrentResource)
	{
		if (!bSuccess)
		{
			UMaterial* BaseMaterial = This->GetMaterial();

			FString ErrorString;

			ErrorString += FString::Printf(
				TEXT("Failed to compile Material Instance with Base %s for platform %s, Default Material will be used in game.\n"), 
				BaseMaterial ? *BaseMaterial->GetName() : TEXT("Null"), 
				*LegacyShaderPlatformToShaderFormat(ShaderPlatform).ToString()
				);

#if WITH_EDITOR
			const TArray<FString>& CompileErrors = CurrentResource->GetCompileErrors();
			for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
			{
				ErrorString += FString::Printf(TEXT("	%s\n"), *CompileErrors[ErrorIndex]);
			}
#endif // WITH_EDITOR

			UE_ASSET_LOG(LogMaterial, Warning, This, TEXT("%s"), *ErrorString);
		}
	}
}

void UMaterialInstance::CacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	UMaterial* BaseMaterial = GetMaterial();
#if WITH_EDITOR
	check(!HasAnyFlags(RF_NeedPostLoad));
	check(BaseMaterial!=nullptr && !BaseMaterial->HasAnyFlags(RF_NeedPostLoad));
	UpdateCachedData();
#endif

	for (int32 ResourceIndex = 0; ResourceIndex < ResourcesToCache.Num(); ResourceIndex++)
	{
		FMaterialResource* CurrentResource = ResourcesToCache[ResourceIndex];

		const bool bSuccess = CurrentResource->CacheShaders(PrecompileMode, TargetPlatform);

		MaterialInstanceImpl::HandleCacheShadersForResourcesErrors(bSuccess, ShaderPlatform, this, CurrentResource);
	}
}

#if WITH_EDITOR

void UMaterialInstance::BeginCacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	UMaterial* BaseMaterial = GetMaterial();
	check(!HasAnyFlags(RF_NeedPostLoad));
	check(BaseMaterial != nullptr && !BaseMaterial->HasAnyFlags(RF_NeedPostLoad));
	UpdateCachedData();

	for (int32 ResourceIndex = 0; ResourceIndex < ResourcesToCache.Num(); ResourceIndex++)
	{
		FMaterialResource* CurrentResource = ResourcesToCache[ResourceIndex];

		// Begin async cache shaders that will be polled and completed inside IsCompilationFinished as part of IsCachedCookedPlatformDataLoaded.
		CurrentResource->BeginCacheShaders(PrecompileMode, TargetPlatform,
			[WeakThis = MakeWeakObjectPtr(this), CurrentResource, ShaderPlatform](bool bSuccess)
			{
				if (UMaterialInstance* This = WeakThis.Get())
				{
					MaterialInstanceImpl::HandleCacheShadersForResourcesErrors(bSuccess, ShaderPlatform, This, CurrentResource);
				}
			}
		);
	}
}

#endif

void UMaterialInstance::CacheShaders(EMaterialShaderPrecompileMode CompileMode)
{
	InitStaticPermutation(CompileMode);
}

FGraphEventArray UMaterialInstance::PrecachePSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& InPreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs)
{
	FGraphEventArray GraphEvents;
	if (FApp::CanEverRender()  && (PipelineStateCache::IsPSOPrecachingEnabled() || IsPSOShaderPreloadingEnabled()))
	{
		// Make sure material is initialized.
		ConditionalPostLoad();

		if (bHasStaticPermutationResource)
		{			
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
			while (FeatureLevelsToCompile != 0)
			{
				const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
				FMaterialResource* StaticPermutationResource = FindMaterialResource(StaticPermutationMaterialResources, GetFeatureLevelShaderPlatform_Checked(FeatureLevel), ActiveQualityLevel, true/*bAllowDefaultMaterial*/);
				if (StaticPermutationResource)
				{
					GraphEvents.Append(StaticPermutationResource->CollectPSOs(VertexFactoryDataList, InPreCacheParams, Priority, OutMaterialPSORequestIDs));
				}
			}
		}
		else if (Parent)
		{
			GraphEvents = Parent->PrecachePSOs(VertexFactoryDataList, InPreCacheParams, Priority, OutMaterialPSORequestIDs);
		}
	}
	return GraphEvents;
}

#if WITH_EDITOR
void UMaterialInstance::CacheGivenTypesForCooking(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, EMaterialQualityLevel::Type QualityLevel, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*> PipelineTypes, const TArray<const FShaderType*>& ShaderTypes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialInstance::CacheGivenTypes);

	if (bHasStaticPermutationResource)
	{
		UMaterial* BaseMaterial = GetMaterial();

		if (QualityLevel == EMaterialQualityLevel::Num)
		{
			QualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
		}

		FMaterialResource* CurrentResource = FindOrCreateMaterialResource(StaticPermutationMaterialResources, BaseMaterial, this, ShaderPlatform, QualityLevel);
		check(CurrentResource);

		// Prepare the resource for compilation, but don't compile the completed shader map.
		const bool bSuccess = CurrentResource->CacheShaders(EMaterialShaderPrecompileMode::None);
		if (bSuccess)
		{
			CurrentResource->CacheGivenTypes(VFTypes, PipelineTypes, ShaderTypes);
		}
	}
}
#endif

bool UMaterialInstance::GetMaterialLayers(FMaterialLayersFunctions& OutLayers, TMicRecursionGuard RecursionGuard) const
{
	if (StaticParametersRuntime.bHasMaterialLayers)
	{
		OutLayers.GetRuntime() = StaticParametersRuntime.MaterialLayers;
#if WITH_EDITORONLY_DATA
		const UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();

		// cooked materials can strip out material layer information
		if (EditorOnly && EditorOnly->StaticParameters.MaterialLayers.LayerStates.Num() != 0)
		{
			OutLayers.EditorOnly = EditorOnly->StaticParameters.MaterialLayers;
			OutLayers.Validate();
		}
		else
		{
			return false;
		}
#endif // WITH_EDITORONLY_DATA
		return true;
	}

	if (Parent)
	{
		if (!RecursionGuard.Contains(this))
		{
			RecursionGuard.Set(this);
			if (Parent->GetMaterialLayers(OutLayers, RecursionGuard))
			{
#if WITH_EDITOR
				// If we got layers from our parent, mark them as linked to our parent
				OutLayers.LinkAllLayersToParent();
#endif // WITH_EDITOR
				return true;
			}
		}
	}
	return false;
}

bool UMaterialInstance::IsComplete() const
{
	bool bComplete = true;
	if (bHasStaticPermutationResource && FApp::CanEverRender())
	{
		check(IsA(UMaterialInstanceConstant::StaticClass()));

		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;

		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			FMaterialResource* CurrentResource = FindMaterialResource(StaticPermutationMaterialResources, ShaderPlatform, ActiveQualityLevel, true);
			if (CurrentResource && !CurrentResource->IsGameThreadShaderMapComplete())
			{
				bComplete = false;
				break;
			}
		}
	}
	return bComplete;
}

#if WITH_EDITOR
bool UMaterialInstance::IsCompiling() const
{
	bool bIsCompiling = false;
	if (bHasStaticPermutationResource && FApp::CanEverRender())
	{
		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;

		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			FMaterialResource* CurrentResource = FindMaterialResource(StaticPermutationMaterialResources, ShaderPlatform, ActiveQualityLevel, true);
			if (CurrentResource && !CurrentResource->IsCompilationFinished())
			{
				bIsCompiling = true;
				break;
			}
		}
	}
	return bIsCompiling;
}
#endif

#if WITH_EDITOR
bool UMaterialInstance::SetMaterialLayers(const FMaterialLayersFunctions& LayersValue)
{
	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	check(EditorOnly);

	bool bUpdatedLayers = false;
	if (!StaticParametersRuntime.bHasMaterialLayers ||
		StaticParametersRuntime.MaterialLayers != LayersValue.GetRuntime() ||
		EditorOnly->StaticParameters.MaterialLayers != LayersValue.EditorOnly)
	{
		bool bMatchesParentLayers = false;
		if (Parent)
		{
			FMaterialLayersFunctions ParentLayers;
			if (Parent->GetMaterialLayers(ParentLayers))
			{
				bMatchesParentLayers = LayersValue.MatchesParent(ParentLayers);
			}
		}

		if (bMatchesParentLayers)
		{
			bUpdatedLayers = StaticParametersRuntime.bHasMaterialLayers; // if we previously had layers, but are now clearing them to match parent
			StaticParametersRuntime.bHasMaterialLayers = false;
			StaticParametersRuntime.MaterialLayers.Empty();
			EditorOnly->StaticParameters.MaterialLayers.Empty();
		}
		else
		{
			bUpdatedLayers = true;
			StaticParametersRuntime.bHasMaterialLayers = true;
			StaticParametersRuntime.MaterialLayers = LayersValue.GetRuntime();
			EditorOnly->StaticParameters.MaterialLayers = LayersValue.EditorOnly;
		}
		FStaticParameterSet::Validate(StaticParametersRuntime, EditorOnly->StaticParameters);
	}
	return bUpdatedLayers;
}
#endif // WITH_EDITOR

template <typename ParameterType>
void TrimToOverriddenOnly(TArray<ParameterType>& Parameters)
{
	for (int32 ParameterIndex = Parameters.Num() - 1; ParameterIndex >= 0; ParameterIndex--)
	{
		if (!Parameters[ParameterIndex].bOverride)
		{
			Parameters.RemoveAt(ParameterIndex);
		}
	}
}

#if WITH_EDITOR

void UMaterialInstance::BeginCacheForCookedPlatformData( const ITargetPlatform *TargetPlatform )
{
	LLM_SCOPE(ELLMTag::Materials);
	TArray<FMaterialResourceForCooking> *CachedMaterialResourcesForPlatform = CachedMaterialResourcesForCooking.Find( TargetPlatform );

	if ( CachedMaterialResourcesForPlatform == NULL )
	{
		CachedMaterialResourcesForPlatform = &CachedMaterialResourcesForCooking.FindOrAdd( TargetPlatform );

		TArray<FName> DesiredShaderFormats;
		TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);

		GetCmdLineFilterShaderFormats(DesiredShaderFormats);

		// Cache shaders for each shader format, storing the results in CachedMaterialResourcesForCooking so they will be available during saving
		for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
		{
			const EShaderPlatform TargetShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);

			CacheResourceShadersForCooking(TargetShaderPlatform, *CachedMaterialResourcesForPlatform,
				EMaterialShaderPrecompileMode::Background, TargetPlatform);
		}
	}
}

bool UMaterialInstance::IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform ) 
{
	LLM_SCOPE(ELLMTag::Materials);

	if (UE_LOG_ACTIVE(LogMaterial, VeryVerbose))
	{
		TStringBuilder<2048> CookStateInfo;
		AppendCompileStateDebugInfo(CookStateInfo);
		UE_LOGF(LogMaterial, VeryVerbose, "MaterialInstance [%ls] Cook State:", *GetName());

		UE::String::ParseTokens(CookStateInfo.ToView(), TEXT('\n'),
			[](FStringView Line)
			{
				UE_LOGF(LogMaterial, VeryVerbose, "%.*ls", Line.Len(), Line.GetData());
			}, UE::String::EParseTokensOptions::SkipEmpty);
	}

	const TArray<FMaterialResourceForCooking>* CachedMaterialResourcesForPlatform = CachedMaterialResourcesForCooking.Find( TargetPlatform );
	if ( CachedMaterialResourcesForPlatform != NULL )
	{
		for (const FMaterialResourceForCooking& MaterialResource : *CachedMaterialResourcesForPlatform)
		{
			if (MaterialResource.Resource->IsCompilationFinished() == false)
			{
				return false;
			}
		}

		return true;
	}
	return false; // this happens if we haven't started caching (begincache hasn't been called yet)
}


void UMaterialInstance::ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	TArray<TRefCountPtr<FMaterialResource>> MaterialsToDelete;
	{
		TArray<FMaterialResourceForCooking> CachedMaterialResourcesForPlatform;
		CachedMaterialResourcesForCooking.RemoveAndCopyValue(TargetPlatform, CachedMaterialResourcesForPlatform);
		MaterialsToDelete.Reserve(CachedMaterialResourcesForPlatform.Num());
		for (FMaterialResourceForCooking& MaterialToDelete : CachedMaterialResourcesForPlatform)
		{
			MaterialsToDelete.Add(MoveTemp(MaterialToDelete.Resource));
		}
	}
	FMaterial::DeferredDeleteArray(MaterialsToDelete);
}


void UMaterialInstance::ClearAllCachedCookedPlatformData()
{
	TArray<TRefCountPtr<FMaterialResource>> MaterialsToDelete;
	for (TPair<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>& It : CachedMaterialResourcesForCooking)
	{
		MaterialsToDelete.Reserve(MaterialsToDelete.Num() + It.Value.Num());
		for (FMaterialResourceForCooking& MaterialToDelete : It.Value)
		{
			MaterialsToDelete.Add(MoveTemp(MaterialToDelete.Resource));
		}
	}
	CachedMaterialResourcesForCooking.Empty();
	FMaterial::DeferredDeleteArray(MaterialsToDelete);
}

void UMaterialInstance::AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const
{
	if (bHasStaticPermutationResource)
	{
		bool bAnyResources = false;
		if (!CachedMaterialResourcesForCooking.IsEmpty())
		{
			for (auto Iter = CachedMaterialResourcesForCooking.CreateConstIterator(); Iter; ++Iter)
			{
				for (const FMaterialResourceForCooking& CookResource : Iter.Value())
				{
					OutDebugInfo << "Resource for platform " << LexToString(CookResource.Platform) << ", IsCachingShaders=" << (CookResource.Resource->IsCachingShaders() ? "true" : "false") << "\n";
					bAnyResources = true;
					CookResource.Resource->AppendCompileStateDebugInfo(OutDebugInfo);
				}
			}
		}

		if (!bAnyResources)
		{
			OutDebugInfo << "No resources created\n";
		}
	}
	else
	{
		check(Parent);
		Parent->AppendCompileStateDebugInfo(OutDebugInfo);
	}
}

#endif

void UMaterialInstance::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);
	SCOPED_LOADTIMER(MaterialInstanceSerializeTime);
	SCOPE_CYCLE_COUNTER(STAT_MaterialInstance_Serialize);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
#if WITH_EDITOR
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
#endif

#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsCooking())
	{
		if (ShouldStripUnusedDefaultTextures())
		{
			UpdateReferencedDefaultTextures();
			constexpr bool bKeepProjectPrivateTextures = false;
			StripUnusedDefaultTextures(bKeepProjectPrivateTextures);
		}
		else
		{
			// All textures will be kept in Base materials
			ReferencedDefaultTextures.Empty();
		}
	}
	// Do not serialize the overrides
	ResetAllTextureParameterOverrides();
#endif

	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MaterialAttributeLayerParameters)
	{
		// Material attribute layers parameter refactor fix-up
		for (FScalarParameterValue& Parameter : ScalarParameterValues)
		{
			Parameter.ParameterInfo.Name = Parameter.ParameterName_DEPRECATED;
		}
		for (FVectorParameterValue& Parameter : VectorParameterValues)
		{
			Parameter.ParameterInfo.Name = Parameter.ParameterName_DEPRECATED;
		}
		for (FTextureParameterValue& Parameter : TextureParameterValues)
		{
			Parameter.ParameterInfo.Name = Parameter.ParameterName_DEPRECATED;
		}
		for (FFontParameterValue& Parameter : FontParameterValues)
		{
			Parameter.ParameterInfo.Name = Parameter.ParameterName_DEPRECATED;
		}
	}

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::MaterialLayerStacksAreNotParameters)
	{
		StaticParameters_DEPRECATED.UpdateLegacyMaterialLayersData();
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::TerrainLayerWeightsAreNotParameters)
	{
		StaticParameters_DEPRECATED.UpdateLegacyTerrainLayerWeightData();
	}

	if (Ar.IsLoading() && !StaticParameters_DEPRECATED.IsEmpty())
	{
		StaticParametersRuntime = MoveTemp(StaticParameters_DEPRECATED.GetRuntime());
		GetEditorOnlyData()->StaticParameters = MoveTemp(StaticParameters_DEPRECATED.EditorOnly);
		StaticParameters_DEPRECATED.Empty();
	}

#endif // WITH_EDITOR

	bool bAllowMissingCachedData = false;
	bool bSavedCachedData = false;
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::MaterialSavedCachedData)
	{
		// If we have editor data, up-to-date cached data can be regenerated on load
#if WITH_EDITORONLY_DATA
		// we want to save the cached data when cooking or duplicating the object in a cooked game
		const bool bWantToSaveCachedData = Ar.IsCooking();
#else
		// We want to copy the cached data when the material is duplicated either directly or during remote object migration
		const bool bDuplicatingObjectInACookedGame = FPlatformProperties::RequiresCookedData() && (Ar.HasAnyPortFlags(PPF_Duplicate | PPF_AvoidRemoteObjectMigration));
		const bool bWantToSaveCachedData = Ar.IsSaving() && bDuplicatingObjectInACookedGame;

		// Workaround for materials being created by annotation data at runtime not having CachedData
		if (bDuplicatingObjectInACookedGame)
		{
			bAllowMissingCachedData = true;
		}
#endif
		if (bWantToSaveCachedData)
		{
			if (CachedData)
			{
				bSavedCachedData = true;
			}
			else if (!bAllowMissingCachedData)
			{
				if (!Ar.IsObjectReferenceCollector())
				{
					// ClassDefault object is expected to be missing cached data, but in all other cases it should have been created when the material was loaded, in PostLoad
					checkf(HasAllFlags(RF_ClassDefaultObject), TEXT("Trying to save cooked material instance %s, missing CachedExpressionData"), *GetName());
				}
			}
		}

		Ar << bSavedCachedData;
	}
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && bSavedCachedData_DEPRECATED)
	{
		bSavedCachedData_DEPRECATED = false;
		bSavedCachedData = true;
	}
#endif // WITH_EDITORONLY_DATA

#if !WITH_EDITORONLY_DATA
	ensureMsgf(!Ar.IsLoading() || bSavedCachedData || bAllowMissingCachedData, TEXT("MaterialInstance %s must have saved cached data, if editor-only data is not present"), *GetName());
#endif

	if (bSavedCachedData)
	{
		if (Ar.IsLoading())
		{
			CachedData.Reset(new FMaterialInstanceCachedData());
			bLoadedCachedData = true;
		}
		check(CachedData);
		UScriptStruct* Struct = FMaterialInstanceCachedData::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)CachedData.Get(), Struct, nullptr);
	}

	// Only serialize the static permutation resource if one exists
	if (bHasStaticPermutationResource)
	{
		if (Ar.UEVer() >= VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS)
		{
#if WITH_EDITOR
			if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MaterialAttributeLayerParameters)
			{
				StaticParameters_DEPRECATED.SerializeLegacy(Ar);
				StaticParametersRuntime = MoveTemp(StaticParameters_DEPRECATED.GetRuntime());
				GetEditorOnlyData()->StaticParameters = MoveTemp(StaticParameters_DEPRECATED.EditorOnly);
			}
#endif

			UE::MaterialInterface::Private::SerializeInlineShaderMaps(
				Ar, LoadedMaterialResources
#if WITH_EDITOR
				, NAME_None
				, &CachedMaterialResourcesForCooking
#else
				, GetFName()
#endif
			);
		}
#if WITH_EDITOR
		else
		{
			const bool bLoadedByCookedMaterial = FPlatformProperties::RequiresCookedData() || GetPackage()->bIsCookedForEditor;

			FMaterialResource LegacyResource;
			LegacyResource.LegacySerialize(Ar);

			FMaterialShaderMapId LegacyId;
			LegacyId.Serialize(Ar, bLoadedByCookedMaterial);

			StaticParametersRuntime.StaticSwitchParameters = LegacyId.GetStaticSwitchParameters();
			TrimToOverriddenOnly(StaticParametersRuntime.StaticSwitchParameters);

			if (IsEditorOnlyDataValid())
			{
				GetEditorOnlyData()->StaticParameters.StaticComponentMaskParameters = LegacyId.GetStaticComponentMaskParameters();
				GetEditorOnlyData()->StaticParameters.TerrainLayerWeightParameters = LegacyId.GetTerrainLayerWeightParameters();
				TrimToOverriddenOnly(GetEditorOnlyData()->StaticParameters.StaticComponentMaskParameters);
			}
		}
#endif // WITH_EDITOR
	}

	if (Ar.UEVer() >= VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES )
	{
#if WITH_EDITORONLY_DATA
		if( Ar.UEVer() < VER_UE4_FIX_MATERIAL_PROPERTY_OVERRIDE_SERIALIZE )
		{
			// awful old native serialize of FMaterialInstanceBasePropertyOverrides UStruct
			Ar << bOverrideBaseProperties_DEPRECATED;
			bool bHasPropertyOverrides = false;
			Ar << bHasPropertyOverrides;
			if( bHasPropertyOverrides )
			{
				FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bOverride_OpacityMaskClipValue);
				Ar << BasePropertyOverrides.OpacityMaskClipValue;

				if( Ar.UEVer() >= VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES_PHASE_2 )
				{
					FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bOverride_BlendMode);
					Ar << BasePropertyOverrides.BlendMode;
					FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bOverride_ShadingModel);
					Ar << BasePropertyOverrides.ShadingModel;
					FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bOverride_TwoSided);
					FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.TwoSided);

					if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::MaterialInstanceBasePropertyOverridesThinSurface)
					{
						FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bOverride_bIsThinSurface);
						FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bIsThinSurface);
					}
					if( Ar.UEVer() >= VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES_DITHERED_LOD_TRANSITION )
					{
						FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.bOverride_DitheredLODTransition);
						FArchive_Serialize_BitfieldBool(Ar, BasePropertyOverrides.DitheredLODTransition);
					}
					// unrelated but closest change to bug
					if( Ar.UEVer() < VER_UE4_STATIC_SHADOW_DEPTH_MAPS )
					{
						// switched enum order
						switch( BasePropertyOverrides.ShadingModel )
						{
							case MSM_Unlit:			BasePropertyOverrides.ShadingModel = MSM_DefaultLit; break;
							case MSM_DefaultLit:	BasePropertyOverrides.ShadingModel = MSM_Unlit; break;
						}
					}
				}
			}
		}
#endif
	}
#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsCooking() && Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && FShaderLibraryCooker::NeedsShaderStableKeys(EShaderPlatform::SP_NumPlatforms))
	{
		SaveShaderStableKeys(Ar.CookingTarget());
	}
#endif

	if (Ar.IsSaving() && Ar.IsCooking())
	{
		ValidateTextureOverrides(GMaxRHIShaderPlatform);
	}

#if WITH_EDITOR
	// Undo textures removal for Multi Platform Cook
	if (Ar.IsSaving() && Ar.IsCooking() && ShouldStripUnusedDefaultTextures())
	{
		ReferencedDefaultTextures.Empty();
	}

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddPhysMatOverrideFlag)
	{
		// Set flag on materials which had Phys Mat already overriden
		bOverridePhysMaterial = bool(PhysMaterial);
	}
#endif
}

void UMaterialInstance::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	ValidateStaticPermutationAllowed();
#endif
}

void UMaterialInstance::PostLoad()
{
	LLM_SCOPE(ELLMTag::MaterialInstance);
	SCOPED_LOADTIMER(MaterialInstancePostLoad);

#if WITH_EDITORONLY_DATA // fixup serialization before everything else
	if (IsEditorOnlyDataValid() && !GetEditorOnlyData()->StaticParameters.StaticSwitchParameters_DEPRECATED.IsEmpty())
	{
		ensure(StaticParametersRuntime.StaticSwitchParameters.IsEmpty());
		StaticParametersRuntime.StaticSwitchParameters = MoveTemp(GetEditorOnlyData()->StaticParameters.StaticSwitchParameters_DEPRECATED);
	}
#endif

	Super::PostLoad();

#if WITH_EDITOR
	//recalculate any scalar params based on a curve position in an atlas in case the atlas changed
	for (FScalarParameterValue& ScalarParam : ScalarParameterValues)
	{
		if (ScalarParam.AtlasData.bIsUsedAsAtlasPosition)
		{
			UCurveLinearColorAtlas* Atlas = Cast<UCurveLinearColorAtlas>(ScalarParam.AtlasData.Atlas.Get());
			UCurveLinearColor* Curve = Cast<UCurveLinearColor>(ScalarParam.AtlasData.Curve.Get());
			if (Curve && Atlas)
			{
				Curve->ConditionalPostLoad();
				Atlas->ConditionalPostLoad();
				int32 Index = Atlas->GradientCurves.Find(Curve);
				if (Index != INDEX_NONE)
				{
					ScalarParam.ParameterValue = (float)Index;
				}
			}
		}
	}
#endif // WITH_EDITOR

	if (FApp::CanEverRender())
	{
		// Resources can be processed / registered now that we're back on the main thread
		ProcessSerializedInlineShaderMaps(this, LoadedMaterialResources, StaticPermutationMaterialResources);
	}
	else
	{
		// Discard all loaded material resources
		for (FMaterialResource& LoadedResource : LoadedMaterialResources)
		{
			LoadedResource.DiscardShaderMap();
		}
	}
	// Empty the list of loaded resources, we don't need it anymore
	LoadedMaterialResources.Empty();

	NaniteOverrideMaterial.FixupLegacySoftReference(this);

	AssertDefaultMaterialsPostLoaded();

	// Ensure that the instance's parent is PostLoaded before the instance.
	if (Parent)
	{
		Parent->ConditionalPostLoad();
	}

#if WITH_EDITOR
	ValidateStaticPermutationAllowed();
#endif

	// Add references to the expression object if we do not have one already, and fix up any names that were changed.
	UpdateParameters();

	// We have to make sure the resources are created for all used textures.
	for( int32 ValueIndex=0; ValueIndex<TextureParameterValues.Num(); ValueIndex++ )
	{
		// Make sure the texture is postloaded so the resource isn't null.
		UTexture* Texture = TextureParameterValues[ValueIndex].ParameterValue;
		if( Texture )
		{
			Texture->ConditionalPostLoad();
		}
	}

	// We have to make sure the resources are created for all used texture collections.
	for( int32 ValueIndex=0; ValueIndex<TextureCollectionParameterValues.Num(); ValueIndex++ )
	{
		// Make sure the texture is postloaded so the resource isn't null.
		UTextureCollection* TextureCollection = TextureCollectionParameterValues[ValueIndex].ParameterValue;
		if( TextureCollection)
		{
			TextureCollection->ConditionalPostLoad();
		}
	}

	// We have to make sure the resources are created for all used parameter collections.
	for (int32 ValueIndex = 0; ValueIndex < ParameterCollectionParameterValues.Num(); ValueIndex++)
	{
		// Make sure the collection is postloaded so the resource isn't null.
		UMaterialParameterCollection* Collection = ParameterCollectionParameterValues[ValueIndex].ParameterValue;
		if (Collection)
		{
			Collection->ConditionalPostLoad();
		}
	}

	// do the same for runtime virtual textures
	for (int32 ValueIndex = 0; ValueIndex < RuntimeVirtualTextureParameterValues.Num(); ValueIndex++)
	{
		// Make sure the texture is postloaded so the resource isn't null.
		URuntimeVirtualTexture* Value = RuntimeVirtualTextureParameterValues[ValueIndex].ParameterValue;
		if (Value)
		{
			Value->ConditionalPostLoad();
		}
	}

	// do the same for sparse virtual textures
	for (int32 ValueIndex = 0; ValueIndex < SparseVolumeTextureParameterValues.Num(); ValueIndex++)
	{
		// Make sure the texture is postloaded so the resource isn't null.
		USparseVolumeTexture* Value = SparseVolumeTextureParameterValues[ValueIndex].ParameterValue;
		if (Value)
		{
			Value->ConditionalPostLoad();
		}
	}

	// do the same for font textures
	for( int32 ValueIndex=0; ValueIndex < FontParameterValues.Num(); ValueIndex++ )
	{
		// Make sure the font is postloaded so the resource isn't null.
		UFont* Font = FontParameterValues[ValueIndex].FontValue;
		if( Font )
		{
			Font->ConditionalPostLoad();
		}
	}

	// And any material layers parameter's functions
	if (StaticParametersRuntime.bHasMaterialLayers)
	{
		for (UMaterialFunctionInterface* Dependency : StaticParametersRuntime.MaterialLayers.Layers)
		{
			if (Dependency)
			{
				Dependency->ConditionalPostLoad();
			}
		}

		for (UMaterialFunctionInterface* Dependency : StaticParametersRuntime.MaterialLayers.Blends)
		{
			if (Dependency)
			{
				Dependency->ConditionalPostLoad();
			}
		}
	}

	for (UObject* Texture : ReferencedDefaultTextures)
	{
		if (Texture)
		{
			Texture->ConditionalPostLoad();
		}
	}

#if !WITH_EDITOR
	// Filter out ShadingModels field to a current platform settings
	FilterOutPlatformShadingModels(GMaxRHIShaderPlatform, ShadingModels);
#endif

#if WITH_EDITOR
	UpdateCachedData();
#endif

	// called before we cache the uniform expression as a call to SubsurfaceProfileRT/SpecularProfileRT affects the data in there
	PropagateDataToMaterialProxy();

	STAT(double MaterialLoadTime = 0);
	{
		SCOPE_SECONDS_COUNTER(MaterialLoadTime);

		const bool bSkipCompilationOnPostLoad = IsMaterialMapDDCEnabled() == false;

		// Make sure static parameters are up to date and shaders are cached for the current platform
		if (bSkipCompilationOnPostLoad)
		{
			InitStaticPermutation(EMaterialShaderPrecompileMode::None);
		}
		else
		{
			InitStaticPermutation();
		}

#if WITH_EDITOR && 0 // the cooker will kick BeginCacheForCookedPlatformData on its own. If a commandlet is going to do rendering (e.g. rebuilding HLOD), InitStaticPermutation should have already created the necessary resources
		// enable caching in postload for derived data cache commandlet and cook by the book
		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		if (TPM && (TPM->RestrictFormatsToRuntimeOnly() == false) && AllowShaderCompiling()) 
		{
			TArray<ITargetPlatform*> Platforms = TPM->GetActiveTargetPlatforms();
			// Cache for all the shader formats that the cooking target requires
			for (int32 FormatIndex = 0; FormatIndex < Platforms.Num(); FormatIndex++)
			{
				BeginCacheForCookedPlatformData(Platforms[FormatIndex]);
			}
		}
#endif
	}

	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_MaterialLoading,(float)MaterialLoadTime);

	if (GIsEditor && GEngine != NULL && !IsTemplate() && Parent)
	{
		// Ensure that the ReferencedTextureGuids array is up to date.
		UpdateLightmassTextureTracking();
	}

	// Fixup for legacy instances which didn't recreate the lighting guid properly on duplication
	if (GetLinker() && GetLinker()->UEVer() < VER_UE4_BUMPED_MATERIAL_EXPORT_GUIDS)
	{
		extern TMap<FGuid, UMaterialInterface*> LightingGuidFixupMap;
		UMaterialInterface** ExistingMaterial = LightingGuidFixupMap.Find(GetLightingGuid());

		if (ExistingMaterial)
		{
			SetLightingGuid();
		}

		LightingGuidFixupMap.Add(GetLightingGuid(), this);
	}

	if (IsPSOShaderPreloadingEnabled())
	{
		// When dynamic preload shaders is enabled, we need to prelaod some material domains since there is no
		// code logic within the PSO precaching system.
		if (IsUIMaterial() || IsDeferredDecal() || IsPostProcessMaterial() || IsLightFunctionMaterial())
		{
			FGraphEventArray Unused;
			PreloadMaterialShaderMap(GetMaterialResource(GMaxRHIShaderPlatform), Unused);
		}
	}
	else if (IsDeferredDecal() || IsUIMaterial() || IsPostProcessMaterial() || IsLightFunctionMaterial())
	{		
		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
		VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));
		FPSOPrecacheParams PSOPrecacheParams;
		PrecachePSOs(VertexFactoryDataList, PSOPrecacheParams, EPSOPrecachePriority::High, MaterialPSOPrecacheRequestIDs);
	}
	//DumpDebugInfo(*GLog);

	CacheTexturesSamplingInfo();
}

#if WITH_EDITOR

bool UMaterialInstance::IsStaticPermutationAllowedForCandidateParent(UMaterialInterface* CandidateParent) const
{
	// Nothing to do if specified parent is null or if restrictive mode is disabled
	if (!CandidateParent || !bEnableRestrictiveMaterialMode)
	{
		return true;
	}

	// If parent is a material instance, we need to verify that itself allows for shader permutations to protect against
	// the situation where the user copies locally a material instance that wouldn't otherwise allow for permutations
	// and parents a new instance against the copy.
	UMaterialInstance* ParentAsInstance = Cast<UMaterialInstance>(CandidateParent);
	if (ParentAsInstance && !IsStaticPermutationAllowedForCandidateParent(ParentAsInstance->Parent))
	{
		UE_LOGF(LogMaterial, Warning,
			   "MaterialInstance '%ls' cannot allow static parameter permutations because its parent '%ls' ultimately derives from a restricted material.",
			   *this->GetName(),
			   *CandidateParent->GetName());
	}

	// Allow candidate material if it is included in base game.
	if (CandidateParent->bIncludedInBaseGame)
	{
		return true;
	}

	// Or if the candidate parent is a Material and it is flagged to be used as a special engine material
	UMaterial* ParentAsMaterial = Cast<UMaterial>(CandidateParent);
	if (ParentAsMaterial && ParentAsMaterial->bUsedAsSpecialEngineMaterial)
	{
		return true;
	}

	// Cache this material package
	UPackage* Package = GetPackage();

	if (Package->HasAnyPackageFlags(PKG_Cooked)								// If this material instance package is cooked
		|| Package == GetTransientPackage()									// Or if this material instance is transient
		|| !CandidateParent->GetPackage()->HasAnyPackageFlags(PKG_Cooked))	// Or if the candidate material is uncooked
	{
		return true;
	}
	
	// Specified material is not allowed to be this material instance parent
	return false;
}

void UMaterialInstance::ValidateStaticPermutationAllowed()
{
	const UMaterialInterface* PrevParent = Parent;

	// Update the flag that controls whether a static permutation is allowed for this material instance
	bDisallowStaticParameterPermutations = !IsStaticPermutationAllowedForCandidateParent(Parent);
	
	// Check that that either this material instance has no permutation or that it is allowed
	if (bHasStaticPermutationResource && bDisallowStaticParameterPermutations)
	{
		// We don't allow Material Instances to parent to cooked materials.
		UE_LOGF(LogMaterial, Warning, "Material instance '%ls' with cooked non-user non-base parent material '%ls' is not allowed to create new shader permutations. Setting parent to null.", *GetName(), *PrevParent->GetName());

		SetParentInternal(nullptr, true);
		bDisallowStaticParameterPermutations = false;
	}

	// If static switches are disabled, clear any latent existing switche
	if (bDisallowStaticParameterPermutations)
	{
		StaticParametersRuntime.Empty();
		bHasStaticPermutationResource = false;
	}
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UMaterialInstance::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMaterialInstanceEditorOnlyData::StaticClass()));
}
#endif

void UMaterialInstance::BeginDestroy()
{
#if WITH_ODSC
	FODSCManager::UnregisterMaterialInstance(this);
#endif

	TArray<TRefCountPtr<FMaterialResource>> ResourcesToDestroy;
	for (FMaterialResource* CurrentResource : StaticPermutationMaterialResources)
	{
		CurrentResource->SetOwnerBeginDestroyed();
		if (CurrentResource->PrepareDestroy_GameThread())
		{
			ResourcesToDestroy.Add(CurrentResource);
		}
	}

	Super::BeginDestroy();

	if (RenderProxy || ResourcesToDestroy.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(BeginDestroyCommand)(
		[ResourcesToDestroy = MoveTemp(ResourcesToDestroy), this](FRHICommandListImmediate& RHICmdList) mutable
		{
			if (RenderProxy)
			{
				RenderProxy->MarkForGarbageCollection();
				RenderProxy->ReleaseResource();
			}

			for (FMaterialResource* CurrentResource : ResourcesToDestroy)
			{
				CurrentResource->PrepareDestroy_RenderThread();
			}

			// Clear all references before assigning the atomic state below.
			ResourcesToDestroy.Empty();

			// And remove from deferred uniform expression cache queue if it's in that
			if (bCachingUniformExpressions)
			{
				RenderProxy->CancelCacheUniformExpressions();
				bCachingUniformExpressions = false;
			}

			// Clear flag set when Resource was created
			bResourceCreated = false;
		});
	}
}

bool UMaterialInstance::IsReadyForFinishDestroy()
{
	bool bIsReady = Super::IsReadyForFinishDestroy();

	return bIsReady && !bResourceCreated;
}

void UMaterialInstance::FinishDestroy()
{
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (RenderProxy)
		{
			RenderProxy->GameThread_Destroy();
			RenderProxy = nullptr;
		}
	}

	for (FMaterialResource* CurrentResource : StaticPermutationMaterialResources)
	{
		delete CurrentResource;
	}
	StaticPermutationMaterialResources.Empty();
#if WITH_EDITOR
	if (!GExitPurge)
	{
		ClearAllCachedCookedPlatformData();
	}
#endif
	CachedData.Reset();

	Super::FinishDestroy();
}

void UMaterialInstance::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UMaterialInstance* This = CastChecked<UMaterialInstance>(InThis);

	if (This->bHasStaticPermutationResource)
	{
		for (FMaterialResource* CurrentResource : This->StaticPermutationMaterialResources)
		{
			CurrentResource->AddReferencedObjects(Collector);
		}
	}

	Super::AddReferencedObjects(This, Collector);
}

bool UMaterialInstance::SetParentInternal(UMaterialInterface* NewParent, bool RecacheShaders)
{
	bool bSetParent = false;
	if (!Parent || Parent != NewParent)
	{
		// Check if the new parent is already an existing child
		UMaterialInstance* ParentAsMaterialInstance = Cast<UMaterialInstance>(NewParent);

		if (ParentAsMaterialInstance != nullptr && ParentAsMaterialInstance->IsChildOf(this))
		{
			UE_LOGF(LogMaterial, Warning, "%ls is not a valid parent for %ls as it is already a child of this material instance.",
				   *ParentAsMaterialInstance->GetFullName(),
				   *GetFullName());
		}
		else if (NewParent &&
				 !NewParent->IsA(UMaterial::StaticClass()) &&
				 !NewParent->IsA(UMaterialInstanceConstant::StaticClass()))
		{
			UE_LOGF(LogMaterial, Warning, "%ls is not a valid parent for %ls. Only Materials and MaterialInstanceConstants are valid parents for a material instance. Outer is %ls",
				*NewParent->GetFullName(),
				*GetFullName(),
				*GetNameSafe(GetOuter()));
		}
		else
		{
			Parent = NewParent;
			bSetParent = true;

#if WITH_EDITOR
			// Important to notify when the parent change for Material -> Material relationship update
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif

			if( Parent )
			{
				// It is possible to set a material's parent while post-loading. In
				// such a case it is also possible that the parent has not been
				// post-loaded, so call ConditionalPostLoad() just in case.
				Parent->ConditionalPostLoad();
			}
		}

		if (bSetParent && RecacheShaders)
		{
			// delete all the existing resources that may have previous parent as the owner
			if (StaticPermutationMaterialResources.Num() > 0)
			{
				FMaterialResourceDeferredDeletionArray ResourcesToFree;
				ResourcesToFree = MoveTemp(StaticPermutationMaterialResources);
				FMaterial::DeferredDeleteArray(ResourcesToFree);
				StaticPermutationMaterialResources.Reset();
			}
			InitStaticPermutation();
		}
		else
		{
			InitResources();
		}
	}

	if (bSetParent)
	{
		OnBaseMaterialSetEvent.Broadcast(this);
	}

	return bSetParent;
}

bool UMaterialInstance::SetVectorParameterByIndexInternal(int32 ParameterIndex, FLinearColor Value)
{
	FVectorParameterValue* ParameterValue = GameThread_FindParameterByIndex(VectorParameterValues, ParameterIndex);
	if (ParameterValue == nullptr)
	{
		return false;
	}

	if(ParameterValue->ParameterValue != Value)
	{
		ParameterValue->ParameterValue = Value;
		// Update the material instance data in the rendering thread.
		GameThread_UpdateMIParameter(this, *ParameterValue);
	}

	return true;
}

#if WITH_EDITORONLY_DATA
FMaterialInstanceParameterUpdateContext::FMaterialInstanceParameterUpdateContext(UMaterialInstance* InInstance, EMaterialInstanceClearParameterFlag InFlags)
	: Instance(InInstance)
	, bForceStaticPermutationUpdate(false)
{
	check(InInstance);
	EMaterialInstanceClearParameterFlag Flags = InFlags;
	if (EnumHasAnyFlags(Flags, EMaterialInstanceClearParameterFlag::Static))
	{
		// If we ask to clear static parameters, simply avoid copying them
		Flags &= ~EMaterialInstanceClearParameterFlag::Static;
	}
	else
	{
		InInstance->GetStaticParameterValues(StaticParameters);
	}

	BasePropertyOverrides = InInstance->BasePropertyOverrides;

	InInstance->ClearParameterValuesInternal(Flags);
}

FMaterialInstanceParameterUpdateContext::~FMaterialInstanceParameterUpdateContext()
{
	Instance->UpdateStaticPermutation(StaticParameters, BasePropertyOverrides, bForceStaticPermutationUpdate);
}

void FMaterialInstanceParameterUpdateContext::SetParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags)
{
	if (IsStaticMaterialParameter(Meta.Value.Type))
	{
		// Route static parameters to the static parameter set
		StaticParameters.SetParameterValue(ParameterInfo, Meta, Flags);
	}
	else
	{
		Instance->SetParameterValueInternal(ParameterInfo, Meta, Flags);
	}
}

void FMaterialInstanceParameterUpdateContext::SetForceStaticPermutationUpdate(bool bValue)
{
	bForceStaticPermutationUpdate = bValue;
}

void FMaterialInstanceParameterUpdateContext::SetBasePropertyOverrides(const FMaterialInstanceBasePropertyOverrides& InValue)
{
	BasePropertyOverrides = InValue;
}

void FMaterialInstanceParameterUpdateContext::SetMaterialLayers(const FMaterialLayersFunctions& InValue)
{
	StaticParameters.bHasMaterialLayers = true;
	StaticParameters.MaterialLayers = InValue.GetRuntime();
	StaticParameters.EditorOnly.MaterialLayers = InValue.EditorOnly;
	StaticParameters.Validate();
}
#endif // WITH_EDITORONLY_DATA

void UMaterialInstance::ReserveParameterValuesInternal(EMaterialParameterType Type, int32 Capacity)
{
	switch (Type)
	{
	case EMaterialParameterType::Scalar: ScalarParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::Vector: VectorParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::DoubleVector: DoubleVectorParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::Texture: TextureParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::TextureCollection: TextureCollectionParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::ParameterCollection: ParameterCollectionParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::Font: FontParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::RuntimeVirtualTexture: RuntimeVirtualTextureParameterValues.Reserve(Capacity); break;
	case EMaterialParameterType::SparseVolumeTexture: SparseVolumeTextureParameterValues.Reserve(Capacity); break;
	default: checkNoEntry();
	}
}

void UMaterialInstance::AddParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags)
{
	const bool bUseAtlas = EnumHasAnyFlags(Flags, EMaterialSetParameterValueFlags::SetCurveAtlas);
	const FMaterialParameterValue& Value = Meta.Value;
	FScalarParameterAtlasInstanceData AtlasData;
	switch (Value.Type)
	{
	case EMaterialParameterType::Scalar:
#if WITH_EDITORONLY_DATA
		if (bUseAtlas)
		{
			AtlasData.bIsUsedAsAtlasPosition = Meta.bUsedAsAtlasPosition;
			AtlasData.Atlas = Meta.ScalarAtlas;
			AtlasData.Curve = Meta.ScalarCurve;
		}
#endif // WITH_EDITORONLY_DATA
		ScalarParameterValues.Emplace(ParameterInfo, Value.AsScalar(), AtlasData);
		break;
	case EMaterialParameterType::Vector: VectorParameterValues.Emplace(ParameterInfo, Value.AsLinearColor()); break;
	case EMaterialParameterType::DoubleVector: DoubleVectorParameterValues.Emplace(ParameterInfo, Value.AsVector4d()); break;
	case EMaterialParameterType::Texture: TextureParameterValues.Emplace(ParameterInfo, Value.Texture); break;
	case EMaterialParameterType::TextureCollection: TextureCollectionParameterValues.Emplace(ParameterInfo, Value.TextureCollection); break;
	case EMaterialParameterType::ParameterCollection: ParameterCollectionParameterValues.Emplace(ParameterInfo, Value.ParameterCollection); break;
	case EMaterialParameterType::Font: FontParameterValues.Emplace(ParameterInfo, Value.Font.Value, Value.Font.Page); break;
	case EMaterialParameterType::RuntimeVirtualTexture: RuntimeVirtualTextureParameterValues.Emplace(ParameterInfo, Value.RuntimeVirtualTexture); break;
	case EMaterialParameterType::SparseVolumeTexture: SparseVolumeTextureParameterValues.Emplace(ParameterInfo, Value.SparseVolumeTexture); break;
	case EMaterialParameterType::StaticSwitch: break;
	default: checkNoEntry();
	}
}

void UMaterialInstance::SetParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags)
{
	const bool bUseAtlas = EnumHasAnyFlags(Flags, EMaterialSetParameterValueFlags::SetCurveAtlas);
	const FMaterialParameterValue& Value = Meta.Value;
	FScalarParameterAtlasInstanceData AtlasData;
	switch (Value.Type)
	{
	case EMaterialParameterType::Scalar:
#if WITH_EDITORONLY_DATA
		if (bUseAtlas)
		{
			AtlasData.bIsUsedAsAtlasPosition = Meta.bUsedAsAtlasPosition;
			AtlasData.Atlas = Meta.ScalarAtlas;
			AtlasData.Curve = Meta.ScalarCurve;
		}
#endif // WITH_EDITORONLY_DATA
		SetScalarParameterValueInternal(ParameterInfo, Value.AsScalar(), bUseAtlas, AtlasData);
		break;
	case EMaterialParameterType::Vector: SetVectorParameterValueInternal(ParameterInfo, Value.AsLinearColor()); break;
	case EMaterialParameterType::DoubleVector: SetDoubleVectorParameterValueInternal(ParameterInfo, Value.AsVector4d()); break;
	case EMaterialParameterType::Texture: SetTextureParameterValueInternal(ParameterInfo, Value.Texture); break;
	case EMaterialParameterType::TextureCollection: SetTextureCollectionParameterValueInternal(ParameterInfo, Value.TextureCollection); break;
	case EMaterialParameterType::Font: SetFontParameterValueInternal(ParameterInfo, Value.Font.Value, Value.Font.Page); break;
	case EMaterialParameterType::RuntimeVirtualTexture: SetRuntimeVirtualTextureParameterValueInternal(ParameterInfo, Value.RuntimeVirtualTexture); break;
	case EMaterialParameterType::SparseVolumeTexture: SetSparseVolumeTextureParameterValueInternal(ParameterInfo, Value.SparseVolumeTexture); break;
	case EMaterialParameterType::ParameterCollection: SetParameterCollectionParameterValueInternal(ParameterInfo, Value.ParameterCollection); break;
	default: checkNoEntry();
	}
}

void UMaterialInstance::SetVectorParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, FLinearColor Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	FVectorParameterValue* ParameterValue = GameThread_FindParameterByName(VectorParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if(!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(VectorParameterValues) FVectorParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		ParameterValue->ParameterValue = Value;
		// Update the material instance data in the rendering thread.
		GameThread_UpdateMIParameter(this, *ParameterValue);
	}
}

void UMaterialInstance::SetDoubleVectorParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, FVector4d Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	FDoubleVectorParameterValue* ParameterValue = GameThread_FindParameterByName(DoubleVectorParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if (!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(DoubleVectorParameterValues) FDoubleVectorParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		ParameterValue->ParameterValue = Value;
		// Update the material instance data in the rendering thread.
		GameThread_UpdateMIParameter(this, *ParameterValue);
	}
}

bool UMaterialInstance::SetScalarParameterByIndexInternal(int32 ParameterIndex, float Value)
{
	FScalarParameterValue* ParameterValue = GameThread_FindParameterByIndex(ScalarParameterValues, ParameterIndex);
	if (ParameterValue == nullptr)
	{
		return false;
	}
	
	if(ParameterValue->ParameterValue != Value)
	{
		ParameterValue->ParameterValue = Value;
		// Update the material instance data in the rendering thread.
		GameThread_UpdateMIParameter(this, *ParameterValue);
	}

	return true;
}

void UMaterialInstance::SetScalarParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, float Value, bool bUseAtlas, FScalarParameterAtlasInstanceData AtlasData)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	FScalarParameterValue* ParameterValue = GameThread_FindParameterByName(ScalarParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if(!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(ScalarParameterValues) FScalarParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	float ValueToSet = Value;
#if WITH_EDITORONLY_DATA
	if (bUseAtlas)
	{
		UCurveLinearColorAtlas* Atlas = Cast<UCurveLinearColorAtlas>(AtlasData.Atlas.Get());
		UCurveLinearColor* Curve = Cast<UCurveLinearColor>(AtlasData.Curve.Get());
		if (Atlas && Curve)
		{
			int32 Index = Atlas->GradientCurves.Find(Curve);
			if (Index != INDEX_NONE)
			{
				ValueToSet = (float)Index;
			}
		}
		ParameterValue->AtlasData = AtlasData;
	}
#endif // WITH_EDITORONLY_DATA

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != ValueToSet)
	{
		ParameterValue->ParameterValue = ValueToSet;
		// Update the material instance data in the rendering thread.
		GameThread_UpdateMIParameter(this, *ParameterValue);
	}
}

#if WITH_EDITOR
void UMaterialInstance::SetScalarParameterAtlasInternal(const FMaterialParameterInfo& ParameterInfo, FScalarParameterAtlasInstanceData AtlasData)
{
	FScalarParameterValue* ParameterValue = GameThread_FindParameterByName(ScalarParameterValues, ParameterInfo);

	if (ParameterValue)
	{
		ParameterValue->AtlasData = AtlasData;
		UCurveLinearColorAtlas* Atlas = Cast<UCurveLinearColorAtlas>(AtlasData.Atlas.Get());
		UCurveLinearColor* Curve = Cast<UCurveLinearColor>(AtlasData.Curve.Get());
		if (!Atlas || !Curve)
		{
			return;
		}
		int32 Index = Atlas->GradientCurves.Find(Curve);
		if (Index == INDEX_NONE)
		{
			return;
		}

		float NewValue = (float)Index;
		
		// Don't enqueue an update if it isn't needed
		if (ParameterValue->ParameterValue != NewValue)
		{
			ParameterValue->ParameterValue = NewValue;
			// Update the material instance data in the rendering thread.
			GameThread_UpdateMIParameter(this, *ParameterValue);
		}
	}
}
#endif

void UMaterialInstance::SetTextureParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, UTexture* Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	if (Value)
	{
		Value->ConditionalPostLoad();
	}

	FTextureParameterValue* ParameterValue = GameThread_FindParameterByName(TextureParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if(!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(TextureParameterValues) FTextureParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		// set as an ensure, because it is somehow possible to accidentally pass non-textures into here via blueprints...
		if (Value && ensureMsgf(Value->IsA(UTexture::StaticClass()), TEXT("Expecting a UTexture! Value='%s' class='%s'"), *Value->GetName(), *Value->GetClass()->GetName()))
		{
			ParameterValue->ParameterValue = Value;
			Value->AddToCluster(this, true);
			// Update the material instance data in the rendering thread.
			GameThread_UpdateMIParameter(this, *ParameterValue);

			// The set of textures (and their sampling scales) used by this instance has changed,
			// so the streaming-texture sampling cache must be rebuilt.
			CacheTexturesSamplingInfo();

#if WITH_EDITOR
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
		}
	}
}

void UMaterialInstance::SetTextureCollectionParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, UTextureCollection* Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	if (Value)
	{
		Value->ConditionalPostLoad();
	}

	FTextureCollectionParameterValue* ParameterValue = GameThread_FindParameterByName(TextureCollectionParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if(!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(TextureCollectionParameterValues) FTextureCollectionParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		// set as an ensure, because it is somehow possible to accidentally pass non-textures into here via blueprints...
		if (Value && ensureMsgf(Value->IsA(UTextureCollection::StaticClass()), TEXT("Expecting a UTextureCollection! Value='%s' class='%s'"), *Value->GetName(), *Value->GetClass()->GetName()))
		{
			ParameterValue->ParameterValue = Value;
			Value->AddToCluster(this, true);
			// Update the material instance data in the rendering thread.
			GameThread_UpdateMIParameter(this, *ParameterValue);

#if WITH_EDITOR
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
		}		
	}
}

void UMaterialInstance::SetParameterCollectionParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, UMaterialParameterCollection* Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	if (Value)
	{
		Value->ConditionalPostLoad();
	}

	FParameterCollectionParameterValue* ParameterValue = GameThread_FindParameterByName(ParameterCollectionParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if(!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(ParameterCollectionParameterValues) FParameterCollectionParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		// set as an ensure, because it is somehow possible to accidentally pass non-parameters into here via blueprints...
		if (Value && ensureMsgf(Value->IsA(UMaterialParameterCollection::StaticClass()), TEXT("Expecting a UMaterialParameterCollection! Value='%s' class='%s'"), *Value->GetName(), *Value->GetClass()->GetName()))
		{
			ParameterValue->ParameterValue = Value;
			Value->AddToCluster(this, true);
			// Update the material instance data in the rendering thread.
			GameThread_UpdateMIParameter(this, *ParameterValue);

#if WITH_EDITOR
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
		}		
	}
}

void UMaterialInstance::SetRuntimeVirtualTextureParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, URuntimeVirtualTexture* Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	if (Value)
	{
		Value->ConditionalPostLoad();
	}

	FRuntimeVirtualTextureParameterValue* ParameterValue = GameThread_FindParameterByName(RuntimeVirtualTextureParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if (!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(RuntimeVirtualTextureParameterValues) FRuntimeVirtualTextureParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		// set as an ensure, because it is somehow possible to accidentally pass non-textures into here via blueprints...
		if (Value && ensureMsgf(Value->IsA(URuntimeVirtualTexture::StaticClass()), TEXT("Expecting a URuntimeVirtualTexture! Value='%s' class='%s'"), *Value->GetName(), *Value->GetClass()->GetName()))
		{
			ParameterValue->ParameterValue = Value;
			Value->AddToCluster(this, true);
			// Update the material instance data in the rendering thread.
			GameThread_UpdateMIParameter(this, *ParameterValue);

#if WITH_EDITOR
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
		}
	}
}

void UMaterialInstance::SetSparseVolumeTextureParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, USparseVolumeTexture* Value)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	if (Value)
	{
		Value->ConditionalPostLoad();
	}

	FSparseVolumeTextureParameterValue* ParameterValue = GameThread_FindParameterByName(SparseVolumeTextureParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if (!ParameterValue)
	{
		// If there's no element for the named parameter in array yet, add one.
		ParameterValue = new(SparseVolumeTextureParameterValues) FSparseVolumeTextureParameterValue;
		ParameterValue->ParameterInfo = ParameterInfo;
		ParameterValue->ExpressionGUID.Invalidate();
		bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate || ParameterValue->ParameterValue != Value)
	{
		// set as an ensure, because it is somehow possible to accidentally pass non-textures into here via blueprints...
		if (Value && ensureMsgf(Value->IsA(USparseVolumeTexture::StaticClass()), TEXT("Expecting a USparseVolumeTexture! Value='%s' class='%s'"), *Value->GetName(), *Value->GetClass()->GetName()))
		{
			ParameterValue->ParameterValue = Value;
			// Update the material instance data in the rendering thread.
			GameThread_UpdateMIParameter(this, *ParameterValue);

#if WITH_EDITOR
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
		}
	}
}

void UMaterialInstance::SetFontParameterValueInternal(const FMaterialParameterInfo& ParameterInfo,class UFont* FontValue,int32 FontPage)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	if (FontValue)
	{
		FontValue->ConditionalPostLoad();
	}

	FFontParameterValue* ParameterValue = GameThread_FindParameterByName(FontParameterValues, ParameterInfo);

	bool bForceUpdate = false;
	if(!ParameterValue)
	{
			// If there's no element for the named parameter in array yet, add one.
			ParameterValue = new(FontParameterValues) FFontParameterValue;
			ParameterValue->ParameterInfo = ParameterInfo;
			ParameterValue->ExpressionGUID.Invalidate();
			bForceUpdate = true;
	}

	// Don't enqueue an update if it isn't needed
	if (bForceUpdate ||
		ParameterValue->FontValue != FontValue ||
		ParameterValue->FontPage != FontPage)
	{
		ParameterValue->FontValue = FontValue;
		ParameterValue->FontPage = FontPage;
		if (FontValue)
		{
			FontValue->AddToCluster(this, true);
		}
		// Update the material instance data in the rendering thread.
		GameThread_UpdateMIParameter(this, *ParameterValue);
	}
}

void UMaterialInstance::ClearParameterValuesInternal(EMaterialInstanceClearParameterFlag Flags)
{
	bool bUpdateResource = false;
	if (EnumHasAnyFlags(Flags, EMaterialInstanceClearParameterFlag::Numeric))
	{
		ScalarParameterValues.Empty();
		VectorParameterValues.Empty();
		DoubleVectorParameterValues.Empty();
		ParameterCollectionParameterValues.Empty();
		bUpdateResource = true;
	}

	if (EnumHasAnyFlags(Flags, EMaterialInstanceClearParameterFlag::Texture))
	{
#if WITH_EDITOR
		ResetAllTextureParameterOverrides();
#endif
		TextureParameterValues.Empty();
		TextureCollectionParameterValues.Empty();
		RuntimeVirtualTextureParameterValues.Empty();
		SparseVolumeTextureParameterValues.Empty();
		FontParameterValues.Empty();
		bUpdateResource = true;
	}

	if (EnumHasAnyFlags(Flags, EMaterialInstanceClearParameterFlag::Static))
	{
		StaticParametersRuntime.Empty();
#if WITH_EDITORONLY_DATA
		UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
		if (EditorOnly)
		{
			EditorOnly->StaticParameters.Empty();
		}
#endif // WITH_EDITORONLY_DATA
	}


	if (RenderProxy && bUpdateResource)
	{
		FMaterialInstanceRenderProxy* InRenderProxy = RenderProxy;
		ENQUEUE_RENDER_COMMAND(FClearMIParametersCommand)(
			[InRenderProxy](FRHICommandList& RHICmdList)
			{
				InRenderProxy->RenderThread_ClearParameters();
			});
	}

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif

	InitResources();
}

#if WITH_EDITOR
void UMaterialInstance::UpdateStaticPermutation(const FStaticParameterSet& NewParameters, FMaterialInstanceBasePropertyOverrides& NewBasePropertyOverrides, const bool bForceStaticPermutationUpdate /*= false*/, FMaterialUpdateContext* MaterialUpdateContext)
{
	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	FStaticParameterSet CompareParameters = NewParameters;

	TrimToOverriddenOnly(CompareParameters.StaticSwitchParameters);
	TrimToOverriddenOnly(CompareParameters.EditorOnly.StaticComponentMaskParameters);

	// Check to see if the material layers being assigned match values from the parent
	if (CompareParameters.bHasMaterialLayers && Parent)
	{
		FMaterialLayersFunctions ParentLayers;
		if (Parent->GetMaterialLayers(ParentLayers))
		{
			if (FMaterialLayersFunctions::MatchesParent(
				CompareParameters.MaterialLayers,
				CompareParameters.EditorOnly.MaterialLayers,
				ParentLayers,
				ParentLayers.EditorOnly))
			{
				CompareParameters.bHasMaterialLayers = false;
				CompareParameters.MaterialLayers.Empty();
			}
		}
	}

	const FStaticParameterSet CurrentParameters = GetStaticParameters();
	const bool bParamsHaveChanged = CurrentParameters != CompareParameters;
	const bool bBasePropertyOverridesHaveChanged = BasePropertyOverrides != NewBasePropertyOverrides;

	BasePropertyOverrides = NewBasePropertyOverrides;

	//Ensure our cached base property overrides are up to date.
	UpdateOverridableBaseProperties();

	const bool bHasBasePropertyOverrides = HasOverridenBaseProperties();

	const bool bWantsStaticPermutationResource = Parent && (!CompareParameters.IsEmpty() || bHasBasePropertyOverrides);
	const bool bRecacheShaders = (bHasStaticPermutationResource != bWantsStaticPermutationResource) || bParamsHaveChanged || (bBasePropertyOverridesHaveChanged && bWantsStaticPermutationResource) || bForceStaticPermutationUpdate;

	if (bRecacheShaders)
	{
		// This will flush the rendering thread which is necessary before changing bHasStaticPermutationResource, since the RT is reading from that directly
		FlushRenderingCommands();

		bHasStaticPermutationResource = bWantsStaticPermutationResource || bForceStaticPermutationUpdate;
		StaticParametersRuntime = CompareParameters.GetRuntime();
		EditorOnly->StaticParameters = CompareParameters.EditorOnly;

		UpdateCachedData();
		CacheResourceShadersForRendering(EMaterialShaderPrecompileMode::None);
		RecacheUniformExpressions(true);

		ValidateStaticPermutationAllowed();
	}

	// Not all base property overrides trigger a static permutation but changing them still requires render state recreation.
	const bool bAddToUpdateContext = bRecacheShaders || bBasePropertyOverridesHaveChanged;

	if (bAddToUpdateContext)
	{
		if (MaterialUpdateContext != nullptr)
		{
			MaterialUpdateContext->AddMaterialInstance(this);
		}
		else
		{
			// The update context will make sure any dependent MI's with static parameters get recompiled
			FMaterialUpdateContext LocalMaterialUpdateContext(FMaterialUpdateContext::EOptions::RecreateRenderStates);
			LocalMaterialUpdateContext.AddMaterialInstance(this);
		}
	}
}

void UMaterialInstance::SetPermutationParameters(const FStaticParameterSet& InParameters, FMaterialInstanceBasePropertyOverrides& InBasePropertyOverrides)
{
	UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	FStaticParameterSet NewParameters = InParameters;

	TrimToOverriddenOnly(NewParameters.StaticSwitchParameters);
	TrimToOverriddenOnly(NewParameters.EditorOnly.StaticComponentMaskParameters);
	
	BasePropertyOverrides = InBasePropertyOverrides;

	UpdateOverridableBaseProperties();

	StaticParametersRuntime = NewParameters.GetRuntime();
	EditorOnly->StaticParameters = NewParameters.EditorOnly;
}

TSoftObjectPtr<UObject> UMaterialInstance::GetEnumerationObject(int32 Index) const
{
	if (EnumerationObjects.IsValidIndex(Index) && !EnumerationObjects[Index].IsNull())
	{
		return EnumerationObjects[Index];
	}
	if (UMaterialInstance* ParentInstance = Cast<UMaterialInstance>(Parent))
	{
		return ParentInstance->GetEnumerationObject(Index);
	}
	return TSoftObjectPtr<UObject>();
}

void UMaterialInstance::GetReferencedTexturesAndOverrides(TSet<const UTexture*>& InOutTextures) const
{
	for (UObject* UsedObject : GetDefaultTextures())
	{
		if (const UTexture* UsedTexture = Cast<UTexture>(UsedObject))
		{
			InOutTextures.Add(UsedTexture);
		}
	}

	// Loop on all override parameters, since child MICs might not override some parameters of parent MICs.
	const UMaterialInstance* MaterialInstance = this;
	while (MaterialInstance)
	{
		for (const FTextureParameterValue& TextureParam : TextureParameterValues)
		{
			if (TextureParam.ParameterValue)
			{
				InOutTextures.Add(TextureParam.ParameterValue);
			}
		}
		MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
	}
}

void UMaterialInstance::UpdateCachedData()
{
	// Overridden for MIC/MID
#if WITH_EDITOR
	// Editor-only path. All callers (PostLoad, PostEditChange, ForceRecompileForRendering, static-parameter
	// updates) run on the game thread; CacheTexturesSamplingInfo asserts IsInGameThread() || IsInParallelGameThread()
	// so this call inherits that invariant.
	CacheTexturesSamplingInfo();
#endif
}

void UMaterialInstance::UpdateReferencedDefaultTextures()
{
	const FStaticParameterSet& LocalStaticParameters = GetStaticParameters();
	if (bHasStaticPermutationResource || LocalStaticParameters.bHasMaterialLayers)
	{
		const FMaterialCachedExpressionData& UsedCachedData = GetCachedExpressionData();
		ReferencedDefaultTextures = UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures;
	}
	else
	{
		ReferencedDefaultTextures.Empty();
	}
}

void UMaterialInstance::UpdateStaticPermutation(const FStaticParameterSet& NewParameters, FMaterialUpdateContext* MaterialUpdateContext)
{
	UpdateStaticPermutation(NewParameters, BasePropertyOverrides, false, MaterialUpdateContext);
}

void UMaterialInstance::UpdateStaticPermutation(FMaterialUpdateContext* MaterialUpdateContext)
{
	// Force the update, since we aren't technically changing anything
	UpdateStaticPermutation(GetStaticParameters(), BasePropertyOverrides, true, MaterialUpdateContext);
}

void UMaterialInstance::UpdateParameterNames()
{
	bool bDirty = UpdateParameters();

	// Atleast 1 parameter changed, initialize parameters
	if (bDirty)
	{
		InitResources();
	}
}

#endif // WITH_EDITOR

void UMaterialInstance::RecacheUniformExpressions(bool bRecreateUniformBuffer) const
{	
	CacheMaterialInstanceUniformExpressions(this, bRecreateUniformBuffer);
}

#if WITH_EDITOR
void UMaterialInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Ensure that the ReferencedTextureGuids array is up to date.
	if (GIsEditor)
	{
		UpdateLightmassTextureTracking();
	}

	if (PropertyChangedEvent.MemberProperty != nullptr && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterial, NaniteOverrideMaterial))
	{
		// Update primitives that might depend on the nanite override material.
		FGlobalComponentRecreateRenderStateContext RecreateComponentsRenderState;
	}

	if (PropertyChangedEvent.MemberProperty != nullptr && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMaterialInstance, Parent))
	{
		ValidateStaticPermutationAllowed();
	}

	// If BLEND_TranslucentColoredTransmittance is selected while Substrate is not enabled, force BLEND_Translucent blend mode
	if (!Substrate::IsSubstrateEnabled())
	{
		SanitizeBlendMode(BlendMode);
		SanitizeBlendMode(BasePropertyOverrides.BlendMode);
	}

	PropagateDataToMaterialProxy();

	InitResources();

	// Force UpdateStaticPermutation when change type is Redirected as this probably means a Material or MaterialInstance parent asset was deleted.
	const bool bForceStaticPermutationUpdate = IsA(UMaterialInstanceConstant::StaticClass()) && PropertyChangedEvent.ChangeType == EPropertyChangeType::Redirected;
	if (bForceStaticPermutationUpdate)
	{
		// This can run before UMaterial::PostEditChangeProperty has a chance to run, so explicitly call UpdateCachedExpressionData here
		UMaterial* BaseMaterial = GetMaterial();
		if (BaseMaterial && !BaseMaterial->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
		{
			BaseMaterial->UpdateCachedExpressionData();
		}
	}
	UpdateStaticPermutation(GetStaticParameters(), BasePropertyOverrides, bForceStaticPermutationUpdate);

	// To avoid unnecessary work, don't apply to child instances and RVT during interactive edits.
	if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) == 0)
	{
		RecacheMaterialInstanceUniformExpressions(this, false);

		if (GIsEditor)
		{
			// Brute force all flush virtual textures if this material writes to any runtime virtual texture.
			if (WritesToRuntimeVirtualTexture() || GetCachedExpressionData().bHasMaterialCacheOutput)
			{
				ENQUEUE_RENDER_COMMAND(FlushVTCommand)([](FRHICommandListImmediate& RHICmdList)
				{
					GetRendererModule().FlushVirtualTextureCache();
				});
			}
		}
	}

	UpdateCachedData();
}

void UMaterialInstance::PostEditUndo()
{
	Super::PostEditUndo();
}

#endif // WITH_EDITOR


bool UMaterialInstance::UpdateLightmassTextureTracking()
{
	bool bTexturesHaveChanged = false;
#if WITH_EDITOR
	TArray<UTexture*> UsedTextures;
	
	GetUsedTextures(UsedTextures);
	if (UsedTextures.Num() != ReferencedTextureGuids.Num())
	{
		bTexturesHaveChanged = true;
		// Just clear out all the guids and the code below will
		// fill them back in...
		ReferencedTextureGuids.Empty(UsedTextures.Num());
		ReferencedTextureGuids.AddZeroed(UsedTextures.Num());
	}
	
	for (int32 CheckIdx = 0; CheckIdx < UsedTextures.Num(); CheckIdx++)
	{
		UTexture* Texture = UsedTextures[CheckIdx];
		if (Texture)
		{
			if (ReferencedTextureGuids[CheckIdx] != Texture->GetLightingGuid())
			{
				ReferencedTextureGuids[CheckIdx] = Texture->GetLightingGuid();
				bTexturesHaveChanged = true;
			}
		}
		else
		{
			if (ReferencedTextureGuids[CheckIdx] != FGuid(0,0,0,0))
			{
				ReferencedTextureGuids[CheckIdx] = FGuid(0,0,0,0);
				bTexturesHaveChanged = true;
			}
		}
	}
#endif // WITH_EDITOR

	return bTexturesHaveChanged;
}


bool UMaterialInstance::GetCastShadowAsMasked() const
{
	if (LightmassSettings.bOverrideCastShadowAsMasked)
	{
		return LightmassSettings.bCastShadowAsMasked;
	}

	if (Parent)
	{
		return Parent->GetCastShadowAsMasked();
	}

	return false;
}

float UMaterialInstance::GetEmissiveBoost() const
{
	if (LightmassSettings.bOverrideEmissiveBoost)
	{
		return LightmassSettings.EmissiveBoost;
	}

	if (Parent)
	{
		return Parent->GetEmissiveBoost();
	}

	return 1.0f;
}

float UMaterialInstance::GetDiffuseBoost() const
{
	if (LightmassSettings.bOverrideDiffuseBoost)
	{
		return LightmassSettings.DiffuseBoost;
	}

	if (Parent)
	{
		return Parent->GetDiffuseBoost();
	}

	return 1.0f;
}

float UMaterialInstance::GetExportResolutionScale() const
{
	if (LightmassSettings.bOverrideExportResolutionScale)
	{
		return FMath::Clamp(LightmassSettings.ExportResolutionScale, .1f, 10.0f);
	}

	if (Parent)
	{
		return FMath::Clamp(Parent->GetExportResolutionScale(), .1f, 10.0f);
	}

	return 1.0f;
}

#if WITH_EDITOR
bool UMaterialInstance::GetGroupSortPriority(const FString& InGroupName, int32& OutSortPriority) const
{
	// @TODO: This needs to handle overridden functions, layers and blends
	const UMaterial* BaseMaterial = GetMaterial();
	if (BaseMaterial && BaseMaterial->GetGroupSortPriority(InGroupName, OutSortPriority))
	{
		return true;
	}

	return false;
}
bool UMaterialInstance::GetTexturesInPropertyChain(EMaterialProperty InProperty, TArray<UTexture*>& OutTextures,  
	TArray<FName>* OutTextureParamNames, struct FStaticParameterSet* InStaticParameterSet,
	ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality)
{
	if (Parent != NULL)
	{
		TArray<FName> LocalTextureParamNames;
		bool bResult = Parent->GetTexturesInPropertyChain(InProperty, OutTextures, &LocalTextureParamNames, InStaticParameterSet, InFeatureLevel, InQuality);
		if (LocalTextureParamNames.Num() > 0)
		{
			// Check textures set in parameters as well...
			for (int32 TPIdx = 0; TPIdx < LocalTextureParamNames.Num(); TPIdx++)
			{
				UTexture* ParamTexture = NULL;
				if (GetTextureParameterValue(LocalTextureParamNames[TPIdx], ParamTexture) == true)
				{
					if (ParamTexture != NULL)
					{
						OutTextures.AddUnique(ParamTexture);
					}
				}

				if (OutTextureParamNames != NULL)
				{
					OutTextureParamNames->AddUnique(LocalTextureParamNames[TPIdx]);
				}
			}
		}
		return bResult;
	}
	return false;
}
#endif // WITH_EDITOR

void UMaterialInstance::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (bHasStaticPermutationResource)
	{
		for (FMaterialResource* CurrentResource : StaticPermutationMaterialResources)
		{
			CurrentResource->GetResourceSizeEx(CumulativeResourceSize);
		}
	}

	if (RenderProxy)
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(FMaterialInstanceRenderProxy));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ScalarParameterValues.Num() * sizeof(THashedMaterialParameterMap<float>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(VectorParameterValues.Num() * sizeof(THashedMaterialParameterMap<FLinearColor>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(DoubleVectorParameterValues.Num() * sizeof(THashedMaterialParameterMap<FVector4d>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(TextureParameterValues.Num() * sizeof(THashedMaterialParameterMap<const UTexture*>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(RuntimeVirtualTextureParameterValues.Num() * sizeof(THashedMaterialParameterMap<const URuntimeVirtualTexture*>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SparseVolumeTextureParameterValues.Num() * sizeof(THashedMaterialParameterMap<const USparseVolumeTexture*>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FontParameterValues.Num() * sizeof(THashedMaterialParameterMap<const UTexture*>::TNamedParameter));
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(ParameterCollectionParameterValues.Num() * sizeof(THashedMaterialParameterMap<const UMaterialParameterCollection*>::TNamedParameter));

		// Record space for hash tables as well..
		if (ScalarParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(ScalarParameterValues.Num()) * sizeof(uint16));
		if (VectorParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(VectorParameterValues.Num()) * sizeof(uint16));
		if (DoubleVectorParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(DoubleVectorParameterValues.Num()) * sizeof(uint16));
		if (TextureParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(TextureParameterValues.Num()) * sizeof(uint16));
		if (RuntimeVirtualTextureParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(RuntimeVirtualTextureParameterValues.Num()) * sizeof(uint16));
		if (SparseVolumeTextureParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(SparseVolumeTextureParameterValues.Num()) * sizeof(uint16));
		if (FontParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(FontParameterValues.Num()) * sizeof(uint16));
		if (ParameterCollectionParameterValues.Num()) CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FDefaultSparseSetAllocator::GetNumberOfHashBuckets(ParameterCollectionParameterValues.Num()) * sizeof(uint16));
	}
}

FPostProcessMaterialNode* FindExistingBlendablePostProcessNode(const FFinalPostProcessSettings& Dest, const UMaterialInterface* Material, const UMaterial* Base)
{
	EBlendableLocation Location = Material->GetBlendableLocation(Base);
	int32 Priority = Material->GetBlendablePriority(Base);

	FBlendableEntry* Iterator = nullptr;

	for (FPostProcessMaterialNode* DataPtr = Dest.BlendableManager.IterateBlendables<FPostProcessMaterialNode>(Iterator); DataPtr; DataPtr = Dest.BlendableManager.IterateBlendables<FPostProcessMaterialNode>(Iterator))
	{
		// Only consider materials that are set as blendable
		if (DataPtr->GetIsBlendable() && DataPtr->GetLocation() == Location && DataPtr->GetPriority() == Priority && DataPtr->GetMaterialInterface()->GetMaterial() == Base)
		{
			return DataPtr;
		}
	}

	return nullptr;
}

void UMaterialInstance::AllMaterialsCacheResourceShadersForRendering(bool bUpdateProgressDialog, bool bCacheAllRemainingShaders)
{
#if WITH_EDITOR
	FScopedSlowTask SlowTask(100.f, NSLOCTEXT("Engine", "CacheMaterialInstanceShadersMessage", "Caching material instance shaders"), true);
	if (bUpdateProgressDialog)
	{
		SlowTask.Visibility = ESlowTaskVisibility::ForceVisible;
		SlowTask.MakeDialog();
	}
#endif // WITH_EDITOR

	TArray<UObject*> MaterialInstanceArray;
	GetObjectsOfClass(UMaterialInstance::StaticClass(), MaterialInstanceArray, true, RF_ClassDefaultObject, EInternalObjectFlags::None);
	float TaskIncrement = (float)100.0f / MaterialInstanceArray.Num();

	for (UObject* MaterialInstanceObj : MaterialInstanceArray)
	{
		UMaterialInstance* MaterialInstance = (UMaterialInstance*)MaterialInstanceObj;

		MaterialInstance->CacheResourceShadersForRendering(bCacheAllRemainingShaders ? EMaterialShaderPrecompileMode::Default : EMaterialShaderPrecompileMode::None);

#if WITH_EDITOR
		if (bUpdateProgressDialog)
		{
			SlowTask.EnterProgressFrame(TaskIncrement);
		}
#endif // WITH_EDITOR
	}
}


bool UMaterialInstance::IsChildOf(const UMaterialInterface* ParentMaterialInterface) const
{
	const UMaterialInterface* Material = this;

	while (Material != ParentMaterialInterface && Material != nullptr)
	{
		const UMaterialInstance* MaterialInstance = Cast<const UMaterialInstance>(Material);
		Material = (MaterialInstance != nullptr) ? ToRawPtr(MaterialInstance->Parent) : nullptr;
	}

	return (Material != nullptr);
}


/**
	Properties of the base material. Can now be overridden by instances.
*/

uint64 UMaterialInstance::GetBasePropertyOverridesHash() const
{
	check(IsInGameThread());

	const UMaterial* Mat = GetMaterial();
	check(Mat);

	FXxHash64Builder Hash;
	bool bHasOverrides = false;
	
	auto GetPropertyOverrideHash = [&](auto InstanceValue, auto MatValue, const FString &HashString)
	{
		bool bOverridden = false;
		if constexpr(std::is_floating_point_v<decltype(InstanceValue)>)
		{
			bOverridden = !FMath::IsNearlyEqual(InstanceValue, MatValue);
		}
		else
		{
			bOverridden = InstanceValue != MatValue;
		}
		
		if (bOverridden)
		{
			Hash.Update(*HashString, HashString.NumBytesWithoutNull());
			Hash.Update(reinterpret_cast<const uint8*>(&InstanceValue), sizeof(InstanceValue));
			bHasOverrides = true;
		}
	};

	GetPropertyOverrideHash(GetOpacityMaskClipValue(), Mat->GetOpacityMaskClipValue(), TEXT("bOverride_OpacityMaskClipValue"));
	GetPropertyOverrideHash(GetBlendMode(), Mat->GetBlendMode(), TEXT("bOverride_BlendMode"));
	GetPropertyOverrideHash(GetShadingModels(), Mat->GetShadingModels(), TEXT("bOverride_ShadingModel"));
	GetPropertyOverrideHash(IsTwoSided(), Mat->IsTwoSided(), TEXT("bOverride_TwoSided"));
	GetPropertyOverrideHash(IsThinSurface(), Mat->IsThinSurface(), TEXT("bOverride_bIsThinSurface"));
	GetPropertyOverrideHash(IsDitheredLODTransition(), Mat->IsDitheredLODTransition(), TEXT("bOverride_DitheredLODTransition"));
	GetPropertyOverrideHash(GetCastDynamicShadowAsMasked(), Mat->GetCastDynamicShadowAsMasked(), TEXT("bOverride_CastDynamicShadowAsMasked"));
	GetPropertyOverrideHash(IsTranslucencyWritingVelocity(), Mat->IsTranslucencyWritingVelocity(), TEXT("bOverride_OutputTranslucentVelocity"));
	GetPropertyOverrideHash(HasPixelAnimation(), Mat->HasPixelAnimation(), TEXT("bOverride_bHasPixelAnimation"));
	GetPropertyOverrideHash(IsTessellationEnabled(), Mat->IsTessellationEnabled(), TEXT("bOverride_bEnableTessellation"));
	GetPropertyOverrideHash(GetDisplacementScaling(), Mat->GetDisplacementScaling(), TEXT("bOverride_DisplacementScaling"));
	GetPropertyOverrideHash(IsDisplacementFadeEnabled(), Mat->IsDisplacementFadeEnabled(), TEXT("bOverride_bEnableDisplacementFade"));
	GetPropertyOverrideHash(GetDisplacementFadeRange(), Mat->GetDisplacementFadeRange(), TEXT("bOverride_DisplacementFadeRange"));
	GetPropertyOverrideHash(GetMaxWorldPositionOffsetDisplacement(), Mat->GetMaxWorldPositionOffsetDisplacement(), TEXT("bOverride_MaxWorldPositionOffsetDisplacement"));
	GetPropertyOverrideHash(IsCompatibleWithLumenCardSharing(), Mat->IsCompatibleWithLumenCardSharing(), TEXT("bOverride_CompatibleWithLumenCardSharing"));
	GetPropertyOverrideHash(GetUsageFlags(), Mat->GetUsageFlags(), TEXT("bOverride_UsageFlags"));

	if (bHasOverrides)
	{
		return Hash.Finalize().Hash;
	}
	else
	{
		return 0u;
	}
}

bool UMaterialInstance::HasOverridenBaseProperties() const
{
	const UMaterial* Material = GetMaterial_Concurrent();
	if (!Parent || !Material || Material->bUsedAsSpecialEngineMaterial)
	{
		return false;
	}

	// Note that we only return true for BaseProperties that need to trigger a static permutation.
	return
		GetBlendMode() != Parent->GetBlendMode() ||
		GetShadingModels() != Parent->GetShadingModels() ||
		IsTwoSided() != Parent->IsTwoSided() ||
		IsThinSurface() != Parent->IsThinSurface() ||
		IsDitheredLODTransition() != Parent->IsDitheredLODTransition() ||
		GetCastDynamicShadowAsMasked() != Parent->GetCastDynamicShadowAsMasked() ||
		IsTranslucencyWritingVelocity() != Parent->IsTranslucencyWritingVelocity() ||
		HasPixelAnimation() != Parent->HasPixelAnimation() ||
		IsTessellationEnabled() != Parent->IsTessellationEnabled() ||
		!FMath::IsNearlyEqual(GetOpacityMaskClipValue(), Parent->GetOpacityMaskClipValue()) ||
		GetUsageFlags() != Parent->GetUsageFlags();
}

#if WITH_EDITOR
FString UMaterialInstance::GetBasePropertyOverrideString() const
{
	FString BasePropString;
	if (HasOverridenBaseProperties())
	{
		// Only append information for the BaseProperties that cause a static permutation and are checked in HasOverridenBaseProperties().
		BasePropString.Appendf(TEXT("bOverride_OpacityMaskClipValue_%d, "), !FMath::IsNearlyEqual(GetOpacityMaskClipValue(), Parent->GetOpacityMaskClipValue()));
		BasePropString.Appendf(TEXT("bOverride_BlendMode_%d, "), (GetBlendMode() != Parent->GetBlendMode()));
		BasePropString.Appendf(TEXT("bOverride_ShadingModel_%d, "), (GetShadingModels() != Parent->GetShadingModels()));
		BasePropString.Appendf(TEXT("bOverride_TwoSided_%d, "), (IsTwoSided() != Parent->IsTwoSided()));
		BasePropString.Appendf(TEXT("bOverride_bIsThinSurface_%d, "), (IsThinSurface() != Parent->IsThinSurface()));
		BasePropString.Appendf(TEXT("bOverride_DitheredLODTransition_%d, "), (IsDitheredLODTransition() != Parent->IsDitheredLODTransition()));
		BasePropString.Appendf(TEXT("bOverride_CastDynamicShadowAsMasked_%d, "), (GetCastDynamicShadowAsMasked() != Parent->GetCastDynamicShadowAsMasked()));
		BasePropString.Appendf(TEXT("bOverride_OutputTranslucentVelocity_%d "), (IsTranslucencyWritingVelocity() != Parent->IsTranslucencyWritingVelocity()));
		BasePropString.Appendf(TEXT("bOverride_bHasPixelAnimation_%d "), (HasPixelAnimation() != Parent->HasPixelAnimation()));
		BasePropString.Appendf(TEXT("bOverride_bEnableTessellation_%d "), (IsTessellationEnabled() != Parent->IsTessellationEnabled()));
		BasePropString.Appendf(TEXT("bOverride_UsageFlags_%d "), (GetUsageFlags() != Parent->GetUsageFlags()));
	}
	return BasePropString;
}
#endif

float UMaterialInstance::GetOpacityMaskClipValue() const
{
	return OpacityMaskClipValue;
}

bool UMaterialInstance::GetCastDynamicShadowAsMasked() const
{
	return bCastDynamicShadowAsMasked;
}

EBlendMode UMaterialInstance::GetBlendMode() const
{
	return BlendMode;
}

FMaterialShadingModelField UMaterialInstance::GetShadingModels() const
{
	return ShadingModels;
}

bool UMaterialInstance::IsShadingModelFromMaterialExpression() const
{
	return bIsShadingModelFromMaterialExpression;
}

bool UMaterialInstance::IsTwoSided() const
{
	return TwoSided;
}

bool UMaterialInstance::IsThinSurface() const
{
	return bIsThinSurface;
}

bool UMaterialInstance::IsTranslucencyWritingVelocity() const
{
	return bOutputTranslucentVelocity && IsTranslucentBlendMode(GetBlendMode());
}

bool UMaterialInstance::IsTranslucencyVelocityFromDepth() const
{
	return Parent ? Parent->IsTranslucencyVelocityFromDepth() : false;
}

bool UMaterialInstance::IsDitheredLODTransition() const
{
	return DitheredLODTransition;
}

FDisplacementScaling UMaterialInstance::GetDisplacementScaling() const
{
	return DisplacementScaling;
}

bool UMaterialInstance::IsDisplacementFadeEnabled() const
{
	return bEnableDisplacementFade;
}

FDisplacementFadeRange UMaterialInstance::GetDisplacementFadeRange() const
{
	return DisplacementFadeRange;
}

float UMaterialInstance::GetMaxWorldPositionOffsetDisplacement() const
{
	return MaxWorldPositionOffsetDisplacement;
}

bool UMaterialInstance::ShouldAlwaysEvaluateWorldPositionOffset() const
{
	return Parent ? Parent->ShouldAlwaysEvaluateWorldPositionOffset() : false;
}

bool UMaterialInstance::IsDeferredDecal() const
{
	return Parent ? Parent->IsDeferredDecal() : false;
}

bool UMaterialInstance::IsUIMaterial() const
{
	return Parent ? Parent->IsUIMaterial() : false;
}

bool UMaterialInstance::IsPostProcessMaterial() const
{
	return Parent ? Parent->IsPostProcessMaterial() : false;
}

bool UMaterialInstance::IsLightFunctionMaterial() const
{
	return Parent ? Parent->IsLightFunctionMaterial() : false;
}

bool UMaterialInstance::HasPixelAnimation() const
{
	return bHasPixelAnimation;
}

bool UMaterialInstance::IsMasked() const
{
	return IsMaskedBlendMode(GetBlendMode()) || (IsTranslucentOnlyBlendMode(GetBlendMode()) && GetCastDynamicShadowAsMasked());
}

bool UMaterialInstance::IsCompatibleWithLumenCardSharing() const
{
	return bCompatibleWithLumenCardSharing;
}

USubsurfaceProfile* UMaterialInstance::GetSubsurfaceProfile_Internal() const
{
	checkSlow(IsInGameThread());
	if (bOverrideSubsurfaceProfile)
	{
		return SubsurfaceProfile;
	}

	// go up the chain if possible
	return Parent ? Parent->GetSubsurfaceProfile_Internal() : 0;
}

USubsurfaceProfile* UMaterialInstance::GetSubsurfaceProfileRoot_Internal() const
{
	// Return the subsurface profile form the root material to create named parameters.
	checkSlow(IsInGameThread());
	return Parent ? Parent->GetSubsurfaceProfileRoot_Internal() : 0;
}

uint32 UMaterialInstance::NumSubsurfaceProfileRoot_Internal() const
{
	// Return the subsurface profile count form the root material.
	checkSlow(IsInGameThread());
	return Parent ? Parent->NumSubsurfaceProfileRoot_Internal() : 0;
}

USubsurfaceProfile* UMaterialInstance::GetSubsurfaceProfileRoot_Internal(uint32 Index) const
{
	// Return the Subsurface profile from the root material.
	checkSlow(IsInGameThread());
	return Parent ? Parent->GetSubsurfaceProfileRoot_Internal(Index) : 0;
}

USubsurfaceProfile* UMaterialInstance::GetSubsurfaceProfileOverride_Internal() const
{
	// Return the possible override for all the instance, but root material always return null as no override since in this case the material Profile itself will be used.
	// The single overriden SSSProbile will overide all the Probile from the root material.
	checkSlow(IsInGameThread());
	if (bOverrideSubsurfaceProfile)
	{
		return SubsurfaceProfile;
	}
	return Parent ? Parent->GetSubsurfaceProfileOverride_Internal() : 0;
}

uint32 UMaterialInstance::NumSpecularProfile_Internal() const
{
	// Return the specular profile count form the root material.
	checkSlow(IsInGameThread());
	return Parent ? Parent->NumSpecularProfile_Internal() : 0;
}

USpecularProfile* UMaterialInstance::GetSpecularProfile_Internal(uint32 Index) const
{
	// Return the Specular profile from the root material.
	checkSlow(IsInGameThread());

	return Parent ? Parent->GetSpecularProfile_Internal(Index) : 0;
}

USpecularProfile* UMaterialInstance::GetSpecularProfileOverride_Internal() const
{
	// Recursively look for override among the ancestors
	USpecularProfile* Out = nullptr;
	if (bOverrideSpecularProfile && SpecularProfileOverride)
	{
		Out = SpecularProfileOverride;
	}
	else if (Parent)
	{
		Out = Parent->GetSpecularProfileOverride_Internal();
	}
	return Out;
}

uint32 UMaterialInstance::NumToonProfile_Internal() const
{
	// Return the toon profile count form the root material.
	checkSlow(IsInGameThread());
	return Parent ? Parent->NumToonProfile_Internal() : 0;
}

UToonProfile* UMaterialInstance::GetToonProfile_Internal(uint32 Index) const
{
	// Return the Toon profile from the root material.
	checkSlow(IsInGameThread());

	return Parent ? Parent->GetToonProfile_Internal(Index) : 0;
}

UToonProfile* UMaterialInstance::GetToonProfileOverride_Internal() const
{
	// Recursively look for override among the ancestors
	UToonProfile* Out = nullptr;
	if (bOverrideToonProfile && ToonProfileOverride)
	{
		Out = ToonProfileOverride;
	}
	else if (Parent)
	{
		Out = Parent->GetToonProfileOverride_Internal();
	}
	return Out;
}

bool UMaterialInstance::CastsRayTracedShadows() const
{
	//#dxr_todo: do per material instance override?
	return Parent ? Parent->CastsRayTracedShadows() : true;
}

bool UMaterialInstance::IsTessellationEnabled() const
{
	return bEnableTessellation;
}

bool UMaterialInstance::HasSubstrateRoughnessTracking() const
{
	return Parent ? Parent->HasSubstrateRoughnessTracking() : true;
}

/** Checks to see if an input property should be active, based on the state of the material */
bool UMaterialInstance::IsPropertyActive(EMaterialProperty InProperty) const
{
	const UMaterial* Material = GetMaterial();
	return Material ? Material->IsPropertyActiveInDerived(InProperty, this) : false;
}

bool UMaterialInstance::HasStaticParameters() const
{
	if (!StaticParametersRuntime.IsEmpty())
	{
		return true;
	}
#if WITH_EDITOR
	const UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	if (EditorOnly && !EditorOnly->StaticParameters.IsEmpty())
	{
		return true;
	}
#endif // WITH_EDITOR
	return false;
}

FStaticParameterSet UMaterialInstance::GetStaticParameters() const
{
	FStaticParameterSet Result;
	Result.GetRuntime() = StaticParametersRuntime;
#if WITH_EDITORONLY_DATA
	const UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	if (EditorOnly)
	{
		Result.EditorOnly = EditorOnly->StaticParameters;
	}
#endif // WITH_EDITORONLY_DATA
	return Result;
}

#if WITH_EDITOR
int32 UMaterialInstance::CompilePropertyEx( class FMaterialCompiler* Compiler, const FGuid& AttributeID )
{
	return Parent ? Parent->CompilePropertyEx(Compiler, AttributeID) : INDEX_NONE;
}

bool UMaterialInstance::TryGetUsedTexturesInCookCache(TArray<TObjectPtr<UObject>>& OutTextures)
{
	OutTextures.Empty();
	const UMaterial* BaseMaterial = GetMaterial_Concurrent();

	if (!BaseMaterial || BaseMaterial->IsDefaultMaterial())
	{
		return false;
	}

	const FMaterialCachedExpressionData& UsedCachedData = GetCachedExpressionData();
	TArray<const FMaterialResource*, TInlineAllocator<4>> MatchedResources = GetCookedCacheMaterialResources();

	for (const FMaterialResource* CurrentResource : MatchedResources)
	{
		const FUniformExpressionSet& CurrentSet = CurrentResource->GetUniformExpressions();
		for (int32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
		{
			for (int32 TextureIndex = 0; TextureIndex < CurrentSet.GetNumTextures((EMaterialTextureParameterType)TypeIndex); ++TextureIndex)
			{
				const FMaterialTextureParameterInfo& Parameter = CurrentSet.GetTextureParameter((EMaterialTextureParameterType)TypeIndex, TextureIndex);
				FMaterialParameterMetadata SourceValue;
				bool bIsOverriden = false;

				// Do not add overriden default textures
				for (int32 ParamTypeIdx = (int32)EMaterialParameterType::Texture; (ParamTypeIdx <= (int32)EMaterialParameterType::SparseVolumeTexture) && !bIsOverriden; ++ParamTypeIdx)
				{
					bIsOverriden |= GetParameterValue((EMaterialParameterType)ParamTypeIdx, Parameter.ParameterInfo, SourceValue, EMaterialGetParameterValueFlags::CheckInstanceOverrides);
				}

				if (!bIsOverriden)
				{
					OutTextures.AddUnique(UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures[Parameter.TextureIndex]);
				}
			}
		}
	}

	const bool bValidCompilationOutput = MatchedResources.Num() > 0;
	return bValidCompilationOutput;
}

const FStaticParameterSetEditorOnlyData& UMaterialInstance::GetEditorOnlyStaticParameters() const
{
	const UMaterialInstanceEditorOnlyData* EditorOnly = GetEditorOnlyData();
	return EditorOnly->StaticParameters;
}
#endif // WITH_EDITOR

void UMaterialInstance::GetLightingGuidChain(bool bIncludeTextures, TArray<FGuid>& OutGuids) const
{
#if WITH_EDITOR
	if (bIncludeTextures)
	{
		OutGuids.Append(ReferencedTextureGuids);
	}
	if (Parent)
	{
		Parent->GetLightingGuidChain(bIncludeTextures, OutGuids);
	}
	Super::GetLightingGuidChain(bIncludeTextures, OutGuids);
#endif
}

#if WITH_EDITOR
void UMaterialInstance::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);
	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		// @TODO : Remove any duplicate data from parent? Aims at improving change propagation (if controlled by parent)
		const ITargetPlatform* TargetPlatform = CookContext.GetTargetPlatform();
		check(TargetPlatform);
		TArray<FMaterialResourceForCooking>* Resources = CachedMaterialResourcesForCooking.Find(TargetPlatform);
		UE::MaterialInterface::Private::RecordMaterialDependenciesForCook(CookContext,
			Resources ? *Resources : TArray<FMaterialResourceForCooking>());

		UMaterialInterface* EffectiveParent = Parent ? Parent.Get() : UMaterial::GetDefaultMaterial(MD_Surface);
		if (EffectiveParent && EffectiveParent->GetPackage() != GetPackage())
		{
			CookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::TransitiveBuild(
				EffectiveParent->GetPackage()->GetFName()));
		}
	}
}
#endif


bool UMaterialInstance::CalculateTexturesSamplingInfo(UTexture* Texture, FTextureSamplingInfo& OutTextureSamplingInfo) const
{
	const bool bResultIsValid = Super::CalculateTexturesSamplingInfo(Texture, OutTextureSamplingInfo);
	if (!bResultIsValid && Parent)
	{
		return Parent->CalculateTexturesSamplingInfo(Texture, OutTextureSamplingInfo);
	}
	return bResultIsValid;
}

float UMaterialInstance::GetTextureDensity(FName TextureName, const struct FMeshUVChannelInfo& UVChannelData) const
{
	ensure(UVChannelData.bInitialized);

	const float Density = Super::GetTextureDensity(TextureName, UVChannelData);
	
	// If it is not handled by this instance, try the parent
	if (!Density && Parent)
	{
		return Parent->GetTextureDensity(TextureName, UVChannelData);
	}
	return Density;
}

bool UMaterialInstance::Equivalent(const UMaterialInstance* CompareTo) const
{
	if (Parent != CompareTo->Parent || 
		PhysMaterial != CompareTo->PhysMaterial ||
		bOverrideSubsurfaceProfile != CompareTo->bOverrideSubsurfaceProfile ||
		BasePropertyOverrides != CompareTo->BasePropertyOverrides ||
		NaniteOverrideMaterial.bEnableOverride != CompareTo->NaniteOverrideMaterial.bEnableOverride ||
		NaniteOverrideMaterial.GetOverrideMaterial() != CompareTo->NaniteOverrideMaterial.GetOverrideMaterial()
		)
	{
		return false;
	}

	if (!CompareValueArraysByExpressionGUID(TextureParameterValues, CompareTo->TextureParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(ScalarParameterValues, CompareTo->ScalarParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(VectorParameterValues, CompareTo->VectorParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(DoubleVectorParameterValues, CompareTo->DoubleVectorParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(RuntimeVirtualTextureParameterValues, CompareTo->RuntimeVirtualTextureParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(SparseVolumeTextureParameterValues, CompareTo->SparseVolumeTextureParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(FontParameterValues, CompareTo->FontParameterValues))
	{
		return false;
	}
	if (!CompareValueArraysByExpressionGUID(ParameterCollectionParameterValues, CompareTo->ParameterCollectionParameterValues))
	{
		return false;
	}
	const FStaticParameterSet LocalStaticParameters = GetStaticParameters();
	if (!LocalStaticParameters.Equivalent(CompareTo->GetStaticParameters()))
	{
		return false;
	}

	return true;
}

bool UMaterialInstance::IsRedundant() const
{
	if (!Parent)
	{
		return false;
	}
		 
	if (NaniteOverrideMaterial.bEnableOverride)
	{
		// Check if we resolve to a different material to our parent 
		TObjectPtr<UMaterialInterface> MyOverride = GetNaniteOverride();
		TObjectPtr<UMaterialInterface> ParentOverride = Parent->GetNaniteOverride();
		// Possible refinement: Could check if they are equivalent MIDs, or a redundant MID and its parent, but that would require loading them. 
		if (MyOverride != ParentOverride)
		{
			return false;
		}
	}

	if (HasStaticParameters())
	{
		return false;
	}
	if (GetPhysicalMaterial() != Parent->GetPhysicalMaterial())
	{
		return false;
	}
	for (int32 i = 0; i < EPhysicalMaterialMaskColor::MAX; ++i)
	{
		if (GetPhysicalMaterialFromMap(i) != Parent->GetPhysicalMaterialFromMap(i))
		{
			return false;
		}
	}
	if (GetSubsurfaceProfile_Internal() != Parent->GetSubsurfaceProfile_Internal())
	{
		return false;
	}
	if (Substrate::IsSubstrateEnabled() && NumSpecularProfile_Internal() > 0 && GetSpecularProfile_Internal(0) != Parent->GetSpecularProfile_Internal(0))
	{
		return false;
	}
	if (Substrate::IsSubstrateEnabled() && NumToonProfile_Internal() > 0 && GetToonProfile_Internal(0) != Parent->GetToonProfile_Internal(0))
	{
		return false;
	}
	// Assume that if any properties are overridden they are different to their parent
	if (
	   BasePropertyOverrides.bOverride_OpacityMaskClipValue 
	|| BasePropertyOverrides.bOverride_BlendMode
	|| BasePropertyOverrides.bOverride_ShadingModel
	|| BasePropertyOverrides.bOverride_DitheredLODTransition
	|| BasePropertyOverrides.bOverride_CastDynamicShadowAsMasked
	|| BasePropertyOverrides.bOverride_TwoSided 
	|| BasePropertyOverrides.bOverride_CompatibleWithLumenCardSharing
	|| BasePropertyOverrides.bOutputTranslucentVelocity
	|| BasePropertyOverrides.bHasPixelAnimation
	|| BasePropertyOverrides.bOverride_UsageFlags)
	{
		return false;
	}
	if (
		!TextureParameterValues.IsEmpty()
		|| !ScalarParameterValues.IsEmpty()
		|| !VectorParameterValues.IsEmpty()
		|| !DoubleVectorParameterValues.IsEmpty()
		|| !RuntimeVirtualTextureParameterValues.IsEmpty()
		|| !FontParameterValues.IsEmpty()
		|| !ParameterCollectionParameterValues.IsEmpty())
	{
		return false;
	}

	return true;
}

#if !UE_BUILD_SHIPPING

static void FindRedundantMICS(const TArray<FString>& Args)
{
	TArray<UObject*> MICs;
	GetObjectsOfClass(UMaterialInstance::StaticClass(), MICs);

	int32 NumRedundant = 0;
	for (int32 OuterIndex = 0; OuterIndex < MICs.Num(); OuterIndex++)
	{
		for (int32 InnerIndex = OuterIndex + 1; InnerIndex < MICs.Num(); InnerIndex++)
		{
			if (((UMaterialInstance*)MICs[OuterIndex])->Equivalent((UMaterialInstance*)MICs[InnerIndex]))
			{
				NumRedundant++;
				break;
			}
		}
	}
	UE_LOGF(LogConsoleResponse, Display, "----------------------------- %d UMaterialInstance's %d redundant ", MICs.Num(), NumRedundant);
}

static FAutoConsoleCommand FindRedundantMICSCmd(
	TEXT("FindRedundantMICS"),
	TEXT("Looks at all loaded MICs and looks for redundant ones."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FindRedundantMICS)
);

#endif

void UMaterialInstance::DumpDebugInfo(FOutputDevice& OutputDevice) const
{
	if (Parent)
	{
		if (bHasStaticPermutationResource)
		{
			for (FMaterialResource* CurrentResource : StaticPermutationMaterialResources)
			{
				CurrentResource->DumpDebugInfo(OutputDevice);
			}

#if WITH_EDITOR
			for (const TPair<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>& It : CachedMaterialResourcesForCooking)
			{
				for (const FMaterialResourceForCooking& CurrentResource : It.Value)
				{
					CurrentResource.Resource->DumpDebugInfo(OutputDevice);
				}
			}
#endif // WITH_EDITOR
		}
		else
		{
			UE_LOGF(LogConsoleResponse, Display, "    This MIC does not have static permulations, and is therefore is just a version of the parent.");
		}
	}
}

TObjectPtr<UObject> UMaterialInstance::GetDefaultTextureAtIdx(const int32 InIdx, TMicRecursionGuard RecursionGuard)
{
#if WITH_EDITORONLY_DATA
	const TArray<TObjectPtr<UObject>>& UsedTextures = GetDefaultTextures();
#else
	const TArray<TObjectPtr<UObject>>& UsedTextures = ReferencedDefaultTextures;
#endif

	// We also look through the Parents in case there's a texture that has been forced to 
	// remain there because it is Project Private (PKG_NotExternallyReferenceable)

	if (UsedTextures.IsValidIndex(InIdx) && UsedTextures[InIdx] != nullptr)
	{
		return UsedTextures[InIdx];
	}

	if (Parent && !RecursionGuard.Contains(this))
	{
		RecursionGuard.Set(this);
		return Parent->GetDefaultTextureAtIdx(InIdx, RecursionGuard);
	}

	return nullptr;
}

void UMaterialInstance::SaveShaderStableKeys(const class ITargetPlatform* TP)
{
#if WITH_EDITOR
	FStableShaderKeyAndValue SaveKeyVal;
	SaveKeyVal.ClassNameAndObjectPath.SetCompactFullNameFromObject(this);
	UMaterial* Base = GetMaterial();
	if (Base)
	{
		SaveKeyVal.MaterialDomain = FName(*MaterialDomainString(Base->MaterialDomain));
	}
	SaveShaderStableKeysInner(TP, SaveKeyVal);
#endif
}

void UMaterialInstance::SaveShaderStableKeysInner(const class ITargetPlatform* TP, const FStableShaderKeyAndValue& InSaveKeyVal)
{
#if WITH_EDITOR
	if (bHasStaticPermutationResource)
	{
		FStableShaderKeyAndValue SaveKeyVal(InSaveKeyVal);
		TArray<FMaterialResourceForCooking>* MatRes = CachedMaterialResourcesForCooking.Find(TP);
		if (MatRes)
		{
			for (FMaterialResourceForCooking& Mat : *MatRes)
			{
				if (Mat.Resource)
				{
					Mat.Resource->SaveShaderStableKeys(SaveKeyVal);
				}
			}
		}
	}
	else if (Parent)
	{
		Parent->SaveShaderStableKeysInner(TP, InSaveKeyVal);
	}
#endif
}

#if WITH_EDITOR
void UMaterialInstance::GetShaderTypes(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TArray<FDebugShaderTypeInfo>& OutShaderInfo)
{
	if (bHasStaticPermutationResource)
	{
		check(IsA(UMaterialInstanceConstant::StaticClass()));
		UMaterial* BaseMaterial = GetMaterial();

		uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
		const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;

		TArray<FMaterialResource*> ResourcesToCache;
		while (FeatureLevelsToCompile != 0)
		{
			const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
			const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

			// Only cache shaders for the quality level that will actually be used to render
			// In cooked build, there is no shader compilation but this is still needed to
			// register the loaded shadermap
			FMaterialResource* CurrentResource = FindOrCreateMaterialResource(StaticPermutationMaterialResources, BaseMaterial, this, Platform, ActiveQualityLevel);
			check(CurrentResource);
		}

		FPlatformTypeLayoutParameters LayoutParams;
		LayoutParams.InitializeForPlatform(TargetPlatform);

		for (FMaterialResource* CurrentResource : StaticPermutationMaterialResources)
		{
			CurrentResource->GetShaderTypes(LayoutParams, OutShaderInfo, FConservativeShaderParameters(CurrentResource));
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterialInstance::BeginAllowCachingStaticParameterValues()
{
	++AllowCachingStaticParameterValuesCounter;
}

void UMaterialInstance::EndAllowCachingStaticParameterValues()
{
	check(AllowCachingStaticParameterValuesCounter > 0);
	--AllowCachingStaticParameterValuesCounter;
	if (AllowCachingStaticParameterValuesCounter == 0)
	{
		CachedStaticParameterValues.Reset();
	}
}
#endif // WITH_EDITOR

template<typename ParameterType>
static void MergeParameterOverrides(TArray<ParameterType>& ParameterValues, const TArray<ParameterType>& SourceParameterValues)
{
	for (const ParameterType& SourceParameter : SourceParameterValues)
	{
		// If the parameter already exists, override it
		bool bExisting = false;
		for (ParameterType& ExistingParameter : ParameterValues)
		{
			if (ExistingParameter.ParameterInfo.Name == SourceParameter.ParameterInfo.Name)
			{
				ExistingParameter.ParameterValue = SourceParameter.ParameterValue;
				bExisting = true;
				break;
			}
		}

		// Instance has introduced a new parameter via static param set
		if (!bExisting)
		{
			ParameterValues.Add(SourceParameter);
		}
	}
}

void UMaterialInstance::CopyMaterialUniformParametersInternal(UMaterialInterface* Source)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);
	SCOPE_CYCLE_COUNTER(STAT_MaterialInstance_CopyUniformParamsInternal)

	if ((Source == nullptr) || (Source == this))
	{
		return;
	}

	ClearParameterValuesInternal();

	if (!FPlatformProperties::IsServerOnly())
	{
		// Build the chain as we don't know which level in the hierarchy will override which parameter
		TArray<UMaterialInterface*> Hierarchy;
		UMaterialInterface* NextSource = Source;
		while (NextSource)
		{
			Hierarchy.Add(NextSource);
			if (UMaterialInstance* AsInstance = Cast<UMaterialInstance>(NextSource))
			{
				NextSource = AsInstance->Parent;
			}
			else
			{
				NextSource = nullptr;
			}
		}

		// Walk chain from material base overriding discovered values. Worst case
		// here is a long instance chain with every value overridden on every level
		for (int Index = Hierarchy.Num() - 1; Index >= 0; --Index)
		{
			UMaterialInterface* Interface = Hierarchy[Index];

			// For instances override existing data
			if (UMaterialInstance* AsInstance = Cast<UMaterialInstance>(Interface))
			{
				MergeParameterOverrides(ScalarParameterValues, AsInstance->ScalarParameterValues);
				MergeParameterOverrides(VectorParameterValues, AsInstance->VectorParameterValues);
				MergeParameterOverrides(DoubleVectorParameterValues, AsInstance->DoubleVectorParameterValues);
				MergeParameterOverrides(TextureParameterValues, AsInstance->TextureParameterValues);
				MergeParameterOverrides(RuntimeVirtualTextureParameterValues, AsInstance->RuntimeVirtualTextureParameterValues);
				MergeParameterOverrides(SparseVolumeTextureParameterValues, AsInstance->SparseVolumeTextureParameterValues);
				MergeParameterOverrides(ParameterCollectionParameterValues, AsInstance->ParameterCollectionParameterValues);
				// No fonts?
			}
			else if (UMaterial* AsMaterial = Cast<UMaterial>(Interface))
			{
				// Material should be the base and only append new parameters
				checkSlow(ScalarParameterValues.Num() == 0);
				checkSlow(VectorParameterValues.Num() == 0);
				checkSlow(DoubleVectorParameterValues.Num() == 0);
				checkSlow(TextureParameterValues.Num() == 0);
				checkSlow(RuntimeVirtualTextureParameterValues.Num() == 0);
				checkSlow(SparseVolumeTextureParameterValues.Num() == 0);
				checkSlow(ParameterCollectionParameterValues.Num() == 0);

				const FMaterialResource* MaterialResource = nullptr;
				if (UWorld* World = AsMaterial->GetWorld())
				{
					MaterialResource = AsMaterial->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel()));
				}

				if (!MaterialResource)
				{
					MaterialResource = AsMaterial->GetMaterialResource(GMaxRHIShaderPlatform);
				}

				if (MaterialResource)
				{
					// Numeric
					for (const FMaterialNumericParameterInfo& Parameter : MaterialResource->GetUniformNumericParameterExpressions())
					{
						const UE::Shader::FValue DefaultValue = MaterialResource->GetUniformExpressions().GetDefaultParameterValue(Parameter.ParameterType, Parameter.DefaultValueOffset);
						const FMaterialParameterMetadata Meta(Parameter.ParameterType, DefaultValue);
						AddParameterValueInternal(Parameter.ParameterInfo.GetName(), Meta);
					}

					// Textures
					for (int32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; TypeIndex++)
					{
						for (const FMaterialTextureParameterInfo& Parameter : MaterialResource->GetUniformTextureExpressions((EMaterialTextureParameterType)TypeIndex))
						{
							if (!Parameter.ParameterInfo.Name.IsNone())
							{
								FTextureParameterValue* ParameterValue = new(TextureParameterValues) FTextureParameterValue;
								ParameterValue->ParameterInfo.Name = Parameter.ParameterInfo.GetName();
								Parameter.GetGameThreadTextureValue(AsMaterial, *MaterialResource, static_cast<UTexture *&>(ParameterValue->ParameterValue));
							}
						}
					}
				}
			}
		}

		InitResources();
	}

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
}

#if WITH_EDITOR
void UMaterialInstance::OverrideTextureParameterValue(const UTexture* InTextureToOverride, UTexture* OverrideTexture)
{
	int32 OverrideIndex = -1;

	// Find an existing texture parameter override if it exists
	// Iterate backwards to match ResetAllTextureParameterOverrides
	for (int32 i = TransientTextureParameterOverrides.Num() - 1; i >= 0; --i)
	{
		FTextureParameterOverride* CurrentOverride = &TransientTextureParameterOverrides[i];
		if (CurrentOverride->PreviousTexture == InTextureToOverride)
		{
			OverrideIndex = i;
			break;
		}
	}

	if (OverrideTexture == nullptr)
	{
		// Remove our entry from the overrides
		if (OverrideIndex != -1)
		{
			// Swap with previous
			OverrideTextureParameterValueInternal(TransientTextureParameterOverrides[OverrideIndex].OverrideTexture, TransientTextureParameterOverrides[OverrideIndex].PreviousTexture);
			TransientTextureParameterOverrides.RemoveAt(OverrideIndex);
		}
	}
	else
	{
		// Only cache if we actually have this texture as a parameter
		TObjectPtr<UTexture> OldTexture = OverrideTextureParameterValueInternal(InTextureToOverride, OverrideTexture);
		if (OverrideIndex == -1 && OldTexture != nullptr)
		{
			int32 NewArraySize = TransientTextureParameterOverrides.Add(FTextureParameterOverride(OldTexture, OverrideTexture));
			OverrideIndex = NewArraySize - 1;
		}
	}

	UMaterialInstance* ParentInstance = Parent.Get() ? Cast<UMaterialInstance>(Parent) : nullptr;
	if (ParentInstance)
	{
		uint32 OldParentParameterOverrideCount = ParentInstance->TransientTextureParameterOverrides.Num();
		ParentInstance->OverrideTextureParameterValue(InTextureToOverride, OverrideTexture);

		// Make sure to update ourself since the parent changed
		if (OldParentParameterOverrideCount != ParentInstance->TransientTextureParameterOverrides.Num())
		{
			FMaterialInstanceRenderProxy* LocalRenderProxy = RenderProxy;
			if (LocalRenderProxy)
			{
				ENQUEUE_RENDER_COMMAND(RefreshMIParameterValue)
					(
						[LocalRenderProxy](FRHICommandListImmediate& RHICmdList)
						{
							LocalRenderProxy->CacheUniformExpressions(RHICmdList, false);
						}
				);
			}
#if WITH_EDITOR
			FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
		}

	}
}

TObjectPtr<UTexture> UMaterialInstance::OverrideTextureParameterValueInternal(const UTexture* InTextureToOverride, UTexture* OverrideTexture)
{
	TObjectPtr<UTexture> OldTexture = nullptr;
	for (int32 i = 0; i < TextureParameterValues.Num(); ++i)
	{
		if (TextureParameterValues[i].ParameterValue == InTextureToOverride)
		{
			// Do not break early because there could be multiple references to the same texture
			TObjectPtr<class UTexture> OldParameterValue = TextureParameterValues[i].ParameterValue;
			SetTextureParameterValueInternal(TextureParameterValues[i].ParameterInfo, OverrideTexture);
			OldTexture = OldParameterValue;
		}
	}

	return OldTexture;
}

void UMaterialInstance::ResetAllTextureParameterOverrides()
{
	// Iterate backwards as textures are removed and the array is shifted
	for (int32 i = TransientTextureParameterOverrides.Num() - 1; i >= 0; --i)
	{
		OverrideTextureParameterValue(TransientTextureParameterOverrides[i].PreviousTexture, nullptr);
	}

	UMaterialInstance* ParentInstance = Parent.Get() ? Cast<UMaterialInstance>(Parent) : nullptr;
	if (ParentInstance)
	{
		ParentInstance->ResetAllTextureParameterOverrides();
	}
}
#endif

bool UMaterialInstance::GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue, bool bOveriddenOnly) const
{
	bool bResult = Super::GetTextureParameterValue(ParameterInfo, OutValue, bOveriddenOnly);
#if WITH_EDITOR
	// See if there is an override in place, if there is, replace it with the original
	if (bResult)
	{
		for (int32 i = 0; i < TransientTextureParameterOverrides.Num(); ++i)
		{
			if (TransientTextureParameterOverrides[i].OverrideTexture == OutValue)
			{
				OutValue = TransientTextureParameterOverrides[i].PreviousTexture.Get();
			}
		}
	}
#endif

	return bResult;
}

bool UMaterialInstance::GetTextureCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue, bool bOveriddenOnly) const
{
	bool bResult = Super::GetTextureCollectionParameterValue(ParameterInfo, OutValue, bOveriddenOnly);
	return bResult;
}
