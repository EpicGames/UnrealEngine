// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialInstanceConstant.cpp: MaterialInstanceConstant implementation.
=============================================================================*/

#include "Materials/MaterialInstanceConstant.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceSupport.h"
#include "MaterialCachedData.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInstanceConstant)

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#endif

UMaterialInstanceConstant::UMaterialInstanceConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PhysMaterialMask = nullptr;
}

void UMaterialInstanceConstant::FinishDestroy()
{
	Super::FinishDestroy();
}

void UMaterialInstanceConstant::PostLoad()
{
	LLM_SCOPE(ELLMTag::Materials);
	Super::PostLoad();
}

FLinearColor UMaterialInstanceConstant::K2_GetVectorParameterValue(FName ParameterName)
{
	FLinearColor Result(0,0,0);
	Super::GetVectorParameterValue(ParameterName, Result);
	return Result;
}

float UMaterialInstanceConstant::K2_GetScalarParameterValue(FName ParameterName)
{
	float Result = 0.f;
	Super::GetScalarParameterValue(ParameterName, Result);
	return Result;
}

UTexture* UMaterialInstanceConstant::K2_GetTextureParameterValue(FName ParameterName)
{
	UTexture* Result = NULL;
	Super::GetTextureParameterValue(ParameterName, Result);
	return Result;
}

UTextureCollection* UMaterialInstanceConstant::K2_GetTextureCollectionParameterValue(FName ParameterName)
{
	UTextureCollection* Result = NULL;
	Super::GetTextureCollectionParameterValue(ParameterName, Result);
	return Result;
}

UPhysicalMaterialMask* UMaterialInstanceConstant::GetPhysicalMaterialMask() const
{
	return PhysMaterialMask;
}

#if WITH_EDITOR
void UMaterialInstanceConstant::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	ParameterStateId = FGuid::NewGuid();
}

void UMaterialInstanceConstant::SetParentEditorOnly(UMaterialInterface* NewParent, bool RecacheShader)
{
	checkf(!Parent || GIsEditor || IsRunningCommandlet(), TEXT("SetParentEditorOnly() may only be used to initialize (not change) the parent outside of the editor, GIsEditor=%d, IsRunningCommandlet()=%d"),
		GIsEditor ? 1 : 0,
		IsRunningCommandlet() ? 1 : 0);

	if (SetParentInternal(NewParent, RecacheShader))
	{
		ValidateStaticPermutationAllowed();
		UpdateCachedData();
	}
}

void UMaterialInstanceConstant::CopyMaterialUniformParametersEditorOnly(UMaterialInterface* Source, bool bIncludeStaticParams)
{
	CopyMaterialUniformParametersInternal(Source);

	if (bIncludeStaticParams && (Source != nullptr) && (Source != this))
	{
		if (UMaterialInstance* SourceMatInst = Cast<UMaterialInstance>(Source))
		{
			FStaticParameterSet SourceParamSet;
			SourceMatInst->GetStaticParameterValues(SourceParamSet);

			FStaticParameterSet MyParamSet;
			GetStaticParameterValues(MyParamSet);

			MyParamSet.StaticSwitchParameters = SourceParamSet.StaticSwitchParameters;

			UpdateStaticPermutation(MyParamSet);

			InitResources();
		}
	}
}

void UMaterialInstanceConstant::SetVectorParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, FLinearColor Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetVectorParameterValueInternal(ParameterInfo,Value);
}

void UMaterialInstanceConstant::SetDoubleVectorParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, FVector4d Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetDoubleVectorParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceConstant::SetScalarParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, float Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetScalarParameterValueInternal(ParameterInfo,Value);
}

void UMaterialInstanceConstant::SetScalarParameterAtlasEditorOnly(const FMaterialParameterInfo& ParameterInfo, FScalarParameterAtlasInstanceData AtlasData)
{
	check(GIsEditor || IsRunningCommandlet());
	SetScalarParameterAtlasInternal(ParameterInfo, AtlasData);
}

void UMaterialInstanceConstant::SetTextureParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, UTexture* Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetTextureParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceConstant::SetTextureCollectionParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, UTextureCollection* Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetTextureCollectionParameterValueInternal(ParameterInfo,Value);
}

void UMaterialInstanceConstant::SetParameterCollectionParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, UMaterialParameterCollection* Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetParameterCollectionParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceConstant::SetRuntimeVirtualTextureParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, URuntimeVirtualTexture* Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetRuntimeVirtualTextureParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceConstant::SetSparseVolumeTextureParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, USparseVolumeTexture* Value)
{
	check(GIsEditor || IsRunningCommandlet());
	SetSparseVolumeTextureParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceConstant::SetFontParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo,class UFont* FontValue,int32 FontPage)
{
	check(GIsEditor || IsRunningCommandlet());
	SetFontParameterValueInternal(ParameterInfo, FontValue, FontPage);
}

void UMaterialInstanceConstant::ClearParameterValuesEditorOnly()
{
	check(GIsEditor || IsRunningCommandlet());
	ClearParameterValuesInternal();
}

void UMaterialInstanceConstant::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag("HasStaticPermutationResource", bHasStaticPermutationResource ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
}

#if WITH_EDITOR
void FMaterialInstanceCachedData::InitializeForConstant(const FMaterialLayersFunctions* Layers, const FMaterialLayersFunctions* ParentLayers)
{
	const int32 NumLayers = Layers ? Layers->Layers.Num() : 0;
	ParentLayerIndexRemap.Empty(NumLayers);
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		int32 ParentLayerIndex = INDEX_NONE;
		if (ParentLayers && Layers->EditorOnly.LayerLinkStates[LayerIndex] == EMaterialLayerLinkState::LinkedToParent)
		{
			const FGuid& LayerGuid = Layers->EditorOnly.LayerGuids[LayerIndex];
			ParentLayerIndex = ParentLayers->EditorOnly.LayerGuids.Find(LayerGuid);
		}
		ParentLayerIndexRemap.Add(ParentLayerIndex);
	}
}
#endif // WITH_EDITOR

void UMaterialInstanceConstant::UpdateCachedData()
{
	// Always use base material textures in Editor, unless we have Layers
	ReferencedDefaultTextures.Empty();

	// Don't need to rebuild cached data if it was serialized
	if (!bLoadedCachedData)
	{
		if (!CachedData)
		{
			CachedData.Reset(new FMaterialInstanceCachedData());
		}

		FMaterialLayersFunctions Layers;
		const bool bHasLayers = GetMaterialLayers(Layers);

		FMaterialLayersFunctions ParentLayers;
		const bool bParentHasLayers = Parent && Parent->GetMaterialLayers(ParentLayers);
		CachedData->InitializeForConstant(bHasLayers ? &Layers : nullptr, bParentHasLayers ? &ParentLayers : nullptr);
		if (RenderProxy)
		{
			RenderProxy->GameThread_UpdateCachedData(*CachedData);
		}
	}

	if (!bLoadedCachedExpressionData)
	{
		const bool bUsingNewHLSLGenerator = IsUsingNewHLSLGenerator();
		TUniquePtr<FMaterialCachedExpressionData> LocalCachedExpressionData;

		// If we have overridden material layers, need to create a local cached expression data
		// Otherwise we can leave it as null, and use cached data from our parent
		const FStaticParameterSet& LocalStaticParameters = GetStaticParameters();
		if (LocalStaticParameters.bHasMaterialLayers)
		{
			UMaterial* BaseMaterial = GetMaterial();

			FMaterialLayersFunctions MaterialLayers;
			LocalStaticParameters.GetMaterialLayers(MaterialLayers);

			LocalCachedExpressionData.Reset(new FMaterialCachedExpressionData());
			LocalCachedExpressionData->AnalyzeMaterial(*BaseMaterial, &MaterialLayers);

			ReferencedDefaultTextures = LocalCachedExpressionData->EditorOnlyData->EditorReferencedDefaultTextures;
		}

		CachedExpressionData = MoveTemp(LocalCachedExpressionData);
		if (CachedExpressionData)
		{
			EditorOnlyData->CachedExpressionData = CachedExpressionData->EditorOnlyData;
		}

#if WITH_EDITOR
		FObjectCacheEventSink::NotifyReferencedTextureChanged_Concurrent(this);
#endif
	}

	CacheTexturesSamplingInfo();
}

void UMaterialInstanceConstant::SetNaniteOverrideMaterial(bool bInEnableOverride, UMaterialInterface* InOverrideMaterial)
{
	NaniteOverrideMaterial.bEnableOverride = bInEnableOverride;
	NaniteOverrideMaterial.OverrideMaterialEditor = InOverrideMaterial;
}

// Walk up the parent chain as long as they are all UMaterialInstanceConstant.  Gather the CRC32 of all parent ParameterStateId values.
static uint32 GetMICParameterStateIdCRCRecursive(const UMaterialInstanceConstant* MIC)
{
	uint32 CRC = 0;
	if (MIC && !MIC->GetReentrantFlag())
	{
		FMICReentranceGuard Guard(MIC);
		UMaterialInstanceConstant* Next = Cast<UMaterialInstanceConstant>(MIC->Parent);
		CRC = GetMICParameterStateIdCRCRecursive(Next);

		CRC = FCrc::TypeCrc32(MIC->ParameterStateId, CRC);
	}

	return CRC;
}

uint32 UMaterialInstanceConstant::ComputeAllStateCRC() const
{
	uint32 CRC = Super::ComputeAllStateCRC();

	const UMaterial* BaseMaterial = GetMaterial();
	if (BaseMaterial)
	{
		CRC = FCrc::TypeCrc32(BaseMaterial->StateId, CRC);
	}

	CRC = FCrc::TypeCrc32(GetMICParameterStateIdCRCRecursive(this), CRC);

	return CRC;
}

#endif // #if WITH_EDITOR
