// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MaterialInstanceDynamic.cpp: MaterialInstanceDynamic implementation.
==============================================================================*/

#include "Materials/MaterialInstanceDynamic.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "UObject/Package.h"
#include "Materials/MaterialInstanceSupport.h"
#include "Engine/Texture.h"
#include "Engine/TextureCollection.h"
#include "Misc/RuntimeErrors.h"
#include "Misc/ScopeRWLock.h"
#include "ObjectCacheEventSink.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInstanceDynamic)

UMaterialInstanceDynamic::UMaterialInstanceDynamic(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void FMaterialInstanceCachedData::InitializeForDynamic(const FMaterialLayersFunctions* ParentLayers)
{
	const int32 NumLayers = ParentLayers ? ParentLayers->Layers.Num() : 0;
	ParentLayerIndexRemap.Empty(NumLayers);
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		ParentLayerIndexRemap.Add(LayerIndex);
	}
}

void UMaterialInstanceDynamic::UpdateCachedLayerData()
{
	FMaterialLayersFunctions ParentLayers;
	bool bParentHasLayers = false;
	if (Parent)
	{
		bParentHasLayers = Parent->GetMaterialLayers(ParentLayers);
	}

	if (!CachedData)
	{
		CachedData.Reset(new FMaterialInstanceCachedData());
	}
	CachedData->InitializeForDynamic(bParentHasLayers ? &ParentLayers : nullptr);

	if (RenderProxy)
	{
		RenderProxy->GameThread_UpdateCachedData(*CachedData);
	}
}

void UMaterialInstanceDynamic::UpdateCachedDataDynamic()
{
	UpdateCachedLayerData();
	CacheTexturesSamplingInfo();

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyReferencedTextureChanged_Concurrent(this);
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UMaterialInstanceDynamic::UpdateCachedData()
{
	UpdateCachedDataDynamic();
}
#endif // WITH_EDITOR

void UMaterialInstanceDynamic::InitializeMID(class UMaterialInterface* ParentMaterial)
{
	SetParentInternal(ParentMaterial, false);
	UpdateCachedLayerData();

	if (CanCacheTexturesSamplingInfo())
	{
		/* If there is a parent material we can inherit the cached textures sampling info.
		 * Since MID can modify the textures or sampling information we make a deep copy.
		 *
		 * KNOWN LIMITATION: the deep copy is a snapshot of the parent's cache at MID-init time.
		 * If the parent UMaterial's cache is rebuilt later (e.g. editor-time texture changes that
		 * fire UMaterial::PostEditChangeProperty -> CacheTexturesSamplingInfo), MIDs created before
		 * that point retain the stale snapshot. There is no observer pattern propagating
		 * parent-cache invalidation to dependent MIDs.
		 *
		 * Acceptable today because:
		 *   - At runtime in cooked content, parent textures don't change after PostLoad.
		 *   - In editor sessions, MIDs are typically transient and short-lived; recreating the MID
		 *     picks up the new parent cache.
		 * Long-lived MIDs that need to track parent edits must call CacheTexturesSamplingInfo()
		 * manually after the parent edit. */
		if (const TSharedPtr<const FMaterialCachedTexturesSamplingInfo> ParentCachedTexturesSamplingInfo = ParentMaterial ? ParentMaterial->GetCachedTexturesSamplingInfo() : TSharedPtr<const FMaterialCachedTexturesSamplingInfo>{})
		{
			UE::TWriteScopeLock<UMaterialInterface::FCacheRWLock> WriteLock(CachedTexturesSamplingInfoLock);
			CachedTexturesSamplingInfo = MakeShared<FMaterialCachedTexturesSamplingInfo>(*ParentCachedTexturesSamplingInfo);
		}
		else
		{
			CacheTexturesSamplingInfo();
		}
	}

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyReferencedTextureChanged_Concurrent(this);
#endif // WITH_EDITOR
}

UMaterialInstanceDynamic* UMaterialInstanceDynamic::Create(UMaterialInterface* ParentMaterial, UObject* InOuter, FName Name)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	UPackage* TransientPackage = GetTransientPackage();

	UObject* Outer = InOuter ? InOuter : TransientPackage;

	if (Name == NAME_None && ParentMaterial)
	{
		Name = MakeUniqueObjectName(InOuter, UMaterialInstanceDynamic::StaticClass(), FName(FString("MID_") + ParentMaterial->GetName()));
	}

	if (Name != NAME_None)
	{
		UMaterialInstanceDynamic* ExistingMID = FindObjectFast<UMaterialInstanceDynamic>(Outer, *Name.ToString(), EFindObjectFlags::ExactClass);
		if (ExistingMID)
		{
			bool bRenamed = false;

#if WITH_EDITOR
			// try to rename to enhance construction script determinism:
			AActor* OwningActor = Cast<AActor>(Outer);
			if (!OwningActor)
			{
				OwningActor = Outer->GetTypedOuter<AActor>();
			}

			if (OwningActor && OwningActor->IsRunningUserConstructionScript())
			{
				bRenamed = true;
				// a collision, we're going to move this existing mid to the transient package and claim the name 
				// for ourself:
				ExistingMID->Rename(
					nullptr,
					TransientPackage,
					REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional
				);
			}
#endif

			if (!bRenamed)
			{
				// If a MID is made with the same name and outer as another, it will overwrite it. To avoid this we will change the name when there is a collision.
				Name = MakeUniqueObjectName(Outer, UMaterialInstanceDynamic::StaticClass(), Name);
			}
		}
	}
	UMaterialInstanceDynamic* MID = NewObject<UMaterialInstanceDynamic>(Outer, Name);
	MID->InitializeMID(ParentMaterial);
	return MID;
}

void UMaterialInstanceDynamic::SetVectorParameterValue(FName ParameterName, FLinearColor Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetVectorParameterValueInternal(ParameterInfo,Value);
}

void UMaterialInstanceDynamic::SetDoubleVectorParameterValue(FName ParameterName, FVector4 Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetDoubleVectorParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceDynamic::SetVectorParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, FLinearColor Value)
{
	SetVectorParameterValueInternal(ParameterInfo, Value);
}

FLinearColor UMaterialInstanceDynamic::K2_GetVectorParameterValue(FName ParameterName)
{
	FLinearColor Result(0,0,0);
	FMaterialParameterInfo ParameterInfo(ParameterName);
	Super::GetVectorParameterValue(ParameterInfo, Result);
	return Result;
}

FLinearColor UMaterialInstanceDynamic::K2_GetVectorParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo)
{
	FLinearColor Result(0, 0, 0);
	Super::GetVectorParameterValue(ParameterInfo, Result);
	return Result;
}

void UMaterialInstanceDynamic::SetScalarParameterValue(FName ParameterName, float Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetScalarParameterValueInternal(ParameterInfo,Value);
}

void UMaterialInstanceDynamic::SetScalarParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, float Value)
{
	SetScalarParameterValueInternal(ParameterInfo, Value);
}

bool UMaterialInstanceDynamic::InitializeScalarParameterAndGetIndex(const FName& ParameterName, float Value, int32& OutParameterIndex)
{
	OutParameterIndex = INDEX_NONE;

	FMaterialParameterInfo ParameterInfo(ParameterName); // @TODO: This will only work for non-layered parameters
	SetScalarParameterValueInternal(ParameterInfo, Value);

	OutParameterIndex = GameThread_FindParameterIndexByName(ScalarParameterValues, ParameterInfo);

	return (OutParameterIndex != INDEX_NONE);
}

bool UMaterialInstanceDynamic::SetScalarParameterByIndex(int32 ParameterIndex, float Value)
{
	return SetScalarParameterByIndexInternal(ParameterIndex, Value);
}

bool UMaterialInstanceDynamic::InitializeVectorParameterAndGetIndex(const FName& ParameterName, const FLinearColor& Value, int32& OutParameterIndex)
{
	OutParameterIndex = INDEX_NONE;

	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetVectorParameterValueInternal(ParameterInfo, Value);

	OutParameterIndex = GameThread_FindParameterIndexByName(VectorParameterValues, ParameterInfo);

	return (OutParameterIndex != INDEX_NONE);
}

bool UMaterialInstanceDynamic::SetVectorParameterByIndex(int32 ParameterIndex, const FLinearColor& Value)
{
	return SetVectorParameterByIndexInternal(ParameterIndex, Value);
}

float UMaterialInstanceDynamic::K2_GetScalarParameterValue(FName ParameterName)
{
	float Result = 0.f;
	FMaterialParameterInfo ParameterInfo(ParameterName);
	Super::GetScalarParameterValue(ParameterInfo, Result);
	return Result;
}

float UMaterialInstanceDynamic::K2_GetScalarParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo)
{
	float Result = 0.f;
	Super::GetScalarParameterValue(ParameterInfo, Result);
	return Result;
}

void UMaterialInstanceDynamic::SetTextureParameterValue(FName ParameterName, UTexture* Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetTextureParameterValueByInfo(ParameterInfo, Value);
}

void UMaterialInstanceDynamic::SetTextureParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, UTexture* Value)
{
	// Save the texture renaming as it will be useful to remap the texture streaming data.
	UTexture* RenamedTexture = NULL;
	Super::GetTextureParameterValue(ParameterInfo, RenamedTexture);

	if (Value && RenamedTexture && Value->GetFName() != RenamedTexture->GetFName())
	{
		RenamedTextures.FindOrAdd(Value->GetFName()).AddUnique(RenamedTexture->GetFName());
	}

	SetTextureParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceDynamic::SetRuntimeVirtualTextureParameterValue(FName ParameterName, class URuntimeVirtualTexture* Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetRuntimeVirtualTextureParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceDynamic::SetRuntimeVirtualTextureParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture* Value)
{
	SetRuntimeVirtualTextureParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceDynamic::SetSparseVolumeTextureParameterValue(FName ParameterName, class USparseVolumeTexture* Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetSparseVolumeTextureParameterValueInternal(ParameterInfo, Value);
}

UTexture* UMaterialInstanceDynamic::K2_GetTextureParameterValue(FName ParameterName)
{
	UTexture* Result = NULL;
	FMaterialParameterInfo ParameterInfo(ParameterName);
	Super::GetTextureParameterValue(ParameterInfo, Result);
	return Result;
}

UTexture* UMaterialInstanceDynamic::K2_GetTextureParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo)
{
	UTexture* Result = NULL;
	Super::GetTextureParameterValue(ParameterInfo, Result);
	return Result;
}

void UMaterialInstanceDynamic::SetTextureCollectionParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, UTextureCollection* Value)
{
	SetTextureCollectionParameterValueInternal(ParameterInfo, Value);
}

void UMaterialInstanceDynamic::SetTextureCollectionParameterValue(FName ParameterName, UTextureCollection* Value)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	SetTextureCollectionParameterValueByInfo(ParameterInfo, Value);
}

UTextureCollection* UMaterialInstanceDynamic::K2_GetTextureCollectionParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo)
{
	UTextureCollection* Result = NULL;
	Super::GetTextureCollectionParameterValue(ParameterInfo, Result);
	return Result;
}

UTextureCollection* UMaterialInstanceDynamic::K2_GetTextureCollectionParameterValue(FName ParameterName)
{
	FMaterialParameterInfo ParameterInfo(ParameterName);
	return K2_GetTextureCollectionParameterValueByInfo(ParameterInfo);
}

void UMaterialInstanceDynamic::SetFontParameterValue(const FMaterialParameterInfo& ParameterInfo,class UFont* FontValue,int32 FontPage)
{
	SetFontParameterValueInternal(ParameterInfo,FontValue,FontPage);
}

void UMaterialInstanceDynamic::ClearParameterValues()
{
	ClearParameterValuesInternal();
}


// could be optimized but surely faster than GetAllVectorParameterNames()
void GameThread_FindAllScalarParameterNames(UMaterialInstance* MaterialInstance, TArray<FName>& InOutNames)
{
	while(MaterialInstance)
	{
		for(int32 i = 0, Num = MaterialInstance->ScalarParameterValues.Num(); i < Num; ++i)
		{
			InOutNames.AddUnique(MaterialInstance->ScalarParameterValues[i].ParameterInfo.Name);
		}

		MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
	}
}

// could be optimized but surely faster than GetAllVectorParameterNames()
void GameThread_FindAllVectorParameterNames(UMaterialInstance* MaterialInstance, TArray<FName>& InOutNames)
{
	while(MaterialInstance)
	{
		for(int32 i = 0, Num = MaterialInstance->VectorParameterValues.Num(); i < Num; ++i)
		{
			InOutNames.AddUnique(MaterialInstance->VectorParameterValues[i].ParameterInfo.Name);
		}

		MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
	}
}

// Finds a parameter by name from the game thread, traversing the chain up to the BaseMaterial.
FScalarParameterValue* GameThread_GetScalarParameterValue(UMaterialInstance* MaterialInstance, FName Name)
{
	UMaterialInterface* It = 0;
	FMaterialParameterInfo ParameterInfo(Name); // @TODO: This will only work for non-layered parameters

	while(MaterialInstance)
	{
		if(FScalarParameterValue* Ret = GameThread_FindParameterByName(MaterialInstance->ScalarParameterValues, ParameterInfo))
		{
			return Ret;
		}

		It = MaterialInstance->Parent;
		MaterialInstance = Cast<UMaterialInstance>(It);
	}

	return 0;
}

// Finds a parameter by name from the game thread, traversing the chain up to the BaseMaterial.
FVectorParameterValue* GameThread_GetVectorParameterValue(UMaterialInstance* MaterialInstance, FName Name)
{
	UMaterialInterface* It = 0;
	FMaterialParameterInfo ParameterInfo(Name); // @TODO: This will only work for non-layered parameters

	while(MaterialInstance)
	{
		if(FVectorParameterValue* Ret = GameThread_FindParameterByName(MaterialInstance->VectorParameterValues, ParameterInfo))
		{
			return Ret;
		}

		It = MaterialInstance->Parent;
		MaterialInstance = Cast<UMaterialInstance>(It);
	}

	return 0;
}

void UMaterialInstanceDynamic::K2_InterpolateMaterialInstanceParams(UMaterialInstance* SourceA, UMaterialInstance* SourceB, float Alpha)
{
	if(SourceA && SourceB)
	{
		UMaterial* BaseA = SourceA->GetBaseMaterial();
		UMaterial* BaseB = SourceB->GetBaseMaterial();

		if(BaseA == BaseB)
		{
			// todo: can be optimized, at least we can reserve
			TArray<FName> Names;

			GameThread_FindAllScalarParameterNames(SourceA, Names);
			GameThread_FindAllScalarParameterNames(SourceB, Names);

			// Interpolate the scalar parameters common to both materials
			for(int32 Idx = 0, Count = Names.Num(); Idx < Count; ++Idx)
			{
				FName Name = Names[Idx];

				auto ParamValueA = GameThread_GetScalarParameterValue(SourceA, Name);
				auto ParamValueB = GameThread_GetScalarParameterValue(SourceB, Name);

				if(ParamValueA || ParamValueB)
				{
					auto Default = 0.0f;

					if(!ParamValueA || !ParamValueB)
					{
						BaseA->GetScalarParameterValue(Name, Default);
					}

					auto ValueA = ParamValueA ? ParamValueA->ParameterValue : Default;
					auto ValueB = ParamValueB ? ParamValueB->ParameterValue : Default;

					SetScalarParameterValue(Name, FMath::Lerp(ValueA, ValueB, Alpha));
				}
			}

			// reused array to minimize further allocations
			Names.Empty();
			GameThread_FindAllVectorParameterNames(SourceA, Names);
			GameThread_FindAllVectorParameterNames(SourceB, Names);

			// Interpolate the vector parameters common to both
			for(int32 Idx = 0, Count = Names.Num(); Idx < Count; ++Idx)
			{
				FName Name = Names[Idx];

				auto ParamValueA = GameThread_GetVectorParameterValue(SourceA, Name);
				auto ParamValueB = GameThread_GetVectorParameterValue(SourceB, Name);

				if(ParamValueA || ParamValueB)
				{
					auto Default = FLinearColor(EForceInit::ForceInit);

					if(!ParamValueA || !ParamValueB)
					{
						BaseA->GetVectorParameterValue(Name, Default);
					}

					auto ValueA = ParamValueA ? ParamValueA->ParameterValue : Default;
					auto ValueB = ParamValueB ? ParamValueB->ParameterValue : Default;

					SetVectorParameterValue(Name, FMath::Lerp(ValueA, ValueB, Alpha));
				}
			}
		}
		else
		{
			// to find bad usage of this method
			// Maybe we can log a content error instead
			// ensure(BaseA == BaseB);
		}
	}
}

void UMaterialInstanceDynamic::K2_CopyMaterialInstanceParameters(UMaterialInterface* Source, bool bQuickParametersOnly /*= false*/)
{
	if (bQuickParametersOnly)
	{
		CopyMaterialUniformParameters(Source);	
	}
	else
	{
		CopyMaterialInstanceParameters(Source);
	}
}

void UMaterialInstanceDynamic::CopyMaterialUniformParameters(UMaterialInterface* Source)
{
	CopyMaterialUniformParametersInternal(Source);
}

void UMaterialInstanceDynamic::CopyInterpParameters(UMaterialInstance* Source)
{
	// we might expose as blueprint function so we have the input a pointer instead of a reference
	if(Source)
	{
		// copy the array and update the renderer data structures

		for (auto& it : Source->ScalarParameterValues)
		{
			SetScalarParameterValue(it.ParameterInfo.Name, it.ParameterValue);
		}

		for (auto& it : Source->VectorParameterValues)
		{
			SetVectorParameterValue(it.ParameterInfo.Name, it.ParameterValue);
		}

		for (auto& it : Source->DoubleVectorParameterValues)
		{
			SetDoubleVectorParameterValue(it.ParameterInfo.Name, it.ParameterValue);
		}

		for (auto& it : Source->TextureParameterValues)
		{
			SetTextureParameterValue(it.ParameterInfo.Name, it.ParameterValue);
		}

		for (auto& it : Source->FontParameterValues)
		{
			SetFontParameterValue(it.ParameterInfo.Name, it.FontValue, it.FontPage);
		}
	}
}

void UMaterialInstanceDynamic::CopyParameterOverrides(UMaterialInstance* MaterialInstance)
{
	LLM_SCOPE(ELLMTag::MaterialInstance);

	ClearParameterValues();
	if (ensureAsRuntimeWarning(MaterialInstance != nullptr))
	{
		VectorParameterValues = MaterialInstance->VectorParameterValues;
		DoubleVectorParameterValues = MaterialInstance->DoubleVectorParameterValues;
		ScalarParameterValues = MaterialInstance->ScalarParameterValues;
		TextureParameterValues = MaterialInstance->TextureParameterValues;
		FontParameterValues = MaterialInstance->FontParameterValues;
	}

#if WITH_EDITOR
	FObjectCacheEventSink::NotifyReferencedTextureChanged_Concurrent(this);
#endif

	InitResources();
}

float UMaterialInstanceDynamic::GetTextureDensity(FName TextureName, const struct FMeshUVChannelInfo& UVChannelData) const
{
	float Density = Super::GetTextureDensity(TextureName, UVChannelData);

	// Also try any renames. Note that even though it could be renamed, the texture could still be used by the parent.
	const TArray<FName>* Renames = RenamedTextures.Find(TextureName);
	if (Renames)
	{
		for (FName Rename : *Renames)
		{
			Density = FMath::Max<float>(Density, Super::GetTextureDensity(Rename, UVChannelData));
		}
	}
	return Density;
}

