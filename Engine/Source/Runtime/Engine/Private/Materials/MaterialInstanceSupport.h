// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialInstance.h: MaterialInstance definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderingThread.h"
#include "MaterialShared.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialRenderProxy.h"
#include "HAL/LowLevelMemTracker.h"
#include <limits>

class UTexture;
struct FMaterialInstanceCachedData;

/**
 * Cache uniform expressions for the given material instance.
 * @param MaterialInstance - The material instance for which to cache uniform expressions.
 */
void CacheMaterialInstanceUniformExpressions(const UMaterialInstance* MaterialInstance, bool bRecreateUniformBuffer = false);

/**
 * Recaches uniform expressions for all material instances with a given parent.
 * WARNING: This function is a noop outside of the Editor!
 * @param ParentMaterial - The parent material to look for.
 */
void RecacheMaterialInstanceUniformExpressions(const UMaterialInterface* ParentMaterial, bool bRecreateUniformBuffer);

/** Protects the members of a UMaterialInstanceConstant from re-entrance. */
class FMICReentranceGuard
{
public:
#if !WITH_EDITOR
	FMICReentranceGuard(const UMaterialInstance* InMaterial) {}
#else
	FMICReentranceGuard(const UMaterialInstance* InMaterial)
	{
		if (!IsInParallelGameThread())
		{
			bIsInGameThread = IsInGameThread();
			Material = const_cast<UMaterialInstance*>(InMaterial);

			if (Material->GetReentrantFlag(bIsInGameThread) == true)
			{
				UE_LOGF(LogMaterial, Warning, "InMaterial: %ls GameThread: %d RenderThread: %d", *InMaterial->GetFullName(), IsInGameThread(), IsInRenderingThread());
				check(!Material->GetReentrantFlag(bIsInGameThread));
			}
			Material->SetReentrantFlag(true, bIsInGameThread);
		}
	}

	~FMICReentranceGuard()
	{
		Material->SetReentrantFlag(false, bIsInGameThread);
	}

private:
	bool bIsInGameThread;
	UMaterialInstance* Material;
#endif // WITH_EDITOR
};

/**
 * Structure that packs FHashedMaterialParameterInfo plus a "HashNext" value into 16 bytes to save memory.  Allows key/value
 * pairs of 16-byte aligned items like vectors to fit in 32 bytes, rather 48, saving significant space on padding.  This
 * introduces limitations that Index cannot exceed 32767, and a given material can't have more than 16384 named parameters
 * of a given type.  Both of those numbers are at least a couple orders of magnitude larger than seen in practice.
 */
struct FHashedMaterialParameterInfoPacked
{
	FScriptName	Name;
	int16		Index = 0;
	uint16		HashNext : 14;
	uint16		Association : 2;

	FHashedMaterialParameterInfoPacked()
	{
		HashNext = 0;
		Association = 0;
	}

	FORCEINLINE FHashedMaterialParameterInfoPacked(const struct FHashedMaterialParameterInfoPacked& Rhs)
		: Name(Rhs.Name), Index(Rhs.Index), HashNext(0), Association(Rhs.Association)
	{
	}

	FHashedMaterialParameterInfoPacked& operator=(const FHashedMaterialParameterInfo& InInfo)
	{
		check((InInfo.Index >= std::numeric_limits<int16>::min()) && (InInfo.Index <= std::numeric_limits<int16>::max()));
		check(InInfo.Association <= 3);

		Name = InInfo.Name;
		Index = (int16)InInfo.Index;
		Association = InInfo.Association;
		return *this;
	}

	/** HashNext ignored in equality comparisons */
	bool operator==(const FHashedMaterialParameterInfoPacked& Other) const
	{
		return Name == Other.Name && Index == Other.Index && Association == Other.Association;
	}

	bool operator==(const FHashedMaterialParameterInfo& Other) const
	{
		return Name == Other.Name && Index == Other.Index && Association == Other.Association;
	}
};

template <typename ValueType>
class THashedMaterialParameterMap
{
public:

	/** Material instances store pairs of names and values in arrays to look up parameters at run time. */
	struct TNamedParameter
	{
		FHashedMaterialParameterInfoPacked Info;
		ValueType Value;
	};

	TArray<TNamedParameter> Array;
	TArray<uint16> HashTable;				// Indices are plus one, so zero represents invalid index

	template<typename ParameterInfo>
	static uint32 TypeHash(const ParameterInfo& Info)
	{
		// We expect the Name to be the only part of the key to vary in 99.995% of keys.  If this changes, we could make
		// the hash take into account more fields than just Name, at some cost in perf generating the hash.  It's important
		// to mix the key bits, because GetTypeHash(FScriptName) just returns an index, which is not good for a hash table.
		int32 Key = GetTypeHash(Info.Name);
		Key += ~(Key << 15);
		Key ^= (Key >> 10);
		Key += (Key << 3);
		Key ^= (Key >> 6);
		Key += ~(Key << 11);
		Key ^= (Key >> 16);
		return (uint32)Key;
	}

	// Add an item from the Array to the hash table.  Hash table must already be allocated.
	void HashAddOneItem(int32 AddedIndex)
	{
		// Clear next index -- we will be adding the item at the end of the bucket it ends up in.
		Array[AddedIndex].Info.HashNext = 0;

		check(!HashTable.IsEmpty());
		uint32 HashIndex = THashedMaterialParameterMap::TypeHash(Array[AddedIndex].Info) & (HashTable.Num() - 1);
		int32 ItemIndex = HashTable[HashIndex];

		if (!ItemIndex)
		{
			// Add to root table.  Indices are plus one, so zero represents invalid index.
			HashTable[HashIndex] = AddedIndex + 1;
		}
		else
		{
			// Items in table are incremented by one, get the actual item index
			ItemIndex--;

			// Find the last item in the chain
			for (int32 NextItem = Array[ItemIndex].Info.HashNext; NextItem; NextItem = Array[NextItem].Info.HashNext)
			{
				ItemIndex = NextItem;
			}
			Array[ItemIndex].Info.HashNext = (uint16)AddedIndex;
		}
	}

	// Add all items to hash table.  Can be used to rehash the full array.
	void HashAddAllItems(int32 NumHashBuckets = 0)
	{
		// If zero is passed in (or anything invalid), choose a default size
		if (NumHashBuckets <= 0)
		{
			NumHashBuckets = FDefaultSparseSetAllocator::GetNumberOfHashBuckets(Array.Num());
		}
		if (Array.IsEmpty())
		{
			HashTable.Empty();
		}
		else
		{
			HashTable.SetNumUninitialized(NumHashBuckets);
			FMemory::Memset(HashTable.GetData(), 0, HashTable.Num() * HashTable.GetTypeSize());
			for (int32 Index = 0; Index < Array.Num(); Index++)
			{
				HashAddOneItem(Index);
			}
		}
	}

	void Empty()
	{
		Array.Empty();
		HashTable.Empty();
	}
};

/**
* The resource used to render a UMaterialInstance.
*/
class FMaterialInstanceRenderProxy: public FMaterialRenderProxy
{
public:
	/** Initialization constructor. */
	FMaterialInstanceRenderProxy(UMaterialInstance* InOwner);

	/**
	 * Called from the game thread to destroy the material instance on the rendering thread.
	 */
	void GameThread_Destroy();

	// FRenderResource interface.
	virtual FString GetFriendlyName() const override { return Owner->GetName(); }

	// FMaterialRenderProxy interface.
	/** Get the FMaterial that should be used for rendering, but might not be in a valid state to actually use.  Can return NULL. */
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type FeatureLevel) const override;
	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override;
	virtual UMaterialInterface* GetMaterialInterface() const override;
	
	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override;
	virtual bool GetUserSceneTextureOverride(FName& InOutValue) const override;
	virtual EBlendableLocation GetBlendableLocation(const FMaterial* Base) const override;
	virtual int32 GetBlendablePriority(const FMaterial* Base) const override;

	void GameThread_SetParent(UMaterialInterface* ParentMaterialInterface);

	void GameThread_UpdateCachedData(const FMaterialInstanceCachedData& CachedData);

	void InitMIParameters(struct FMaterialInstanceParameterSet& ParameterSet);

	/**
	 * Clears all parameters set on this material instance.
	 */
	void RenderThread_ClearParameters()
	{
		InvalidateUniformExpressionCache(false);
		VectorParameterArray.Empty();
		DoubleVectorParameterArray.Empty();
		ScalarParameterArray.Empty();
		TextureParameterArray.Empty();
		TextureCollectionParameterArray.Empty();
		RuntimeVirtualTextureParameterArray.Empty();
		SparseVolumeTextureParameterArray.Empty();
		ParameterCollectionParameterArray.Empty();
		UserSceneTextureOverrides.Empty();
	}

	/**
	 * Updates a named parameter on the render thread.
	 */
	template <typename ValueType>
	void RenderThread_UpdateParameter(const FHashedMaterialParameterInfo& ParameterInfo, const ValueType& Value )
	{
		LLM_SCOPE(ELLMTag::MaterialInstance);

		InvalidateUniformExpressionCache(false);
		THashedMaterialParameterMap<ValueType>& ValueArray = GetValueArray<ValueType>();
		int32 Index = RenderThread_FindParameterByNameInternal<ValueType>(ParameterInfo);

		if (Index != INDEX_NONE)
		{
			ValueArray.Array[Index].Value = Value;
		}
		else
		{
			typename THashedMaterialParameterMap<ValueType>::TNamedParameter NewParameter;
			NewParameter.Info = ParameterInfo;
			NewParameter.Value = Value;
			ValueArray.Array.Add(NewParameter);
			
			// FHashedMaterialParameterInfoPacked::HashNext is limited to 14 bits
			check(ValueArray.Array.Num() <= (1<<14));

			const int32 NumHashBuckets = FDefaultSparseSetAllocator::GetNumberOfHashBuckets(ValueArray.Array.Num());
			if (NumHashBuckets == ValueArray.HashTable.Num())
			{
				ValueArray.HashAddOneItem(ValueArray.Array.Num() - 1);
			}
			else
			{
				ValueArray.HashAddAllItems(NumHashBuckets);
			}
		}
	}

	/**
	 * Retrieves a parameter by name.
	 */
	template <typename ValueType>
	bool RenderThread_GetParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue) const
	{
		const THashedMaterialParameterMap<ValueType>& ValueArray = GetValueArray<ValueType>();
		int Index = RenderThread_FindParameterByNameInternal<ValueType>(ParameterInfo);

		if (Index != INDEX_NONE && IsValidParameterValue(ValueArray.Array[Index].Value))
		{
			OutValue = ValueArray.Array[Index].Value;
			return true;
		}
		return false;
	}

private:
	virtual void EvaluateParameterCollections(FRHICommandListBase& RHICmdList, FUniformExpressionCache& OutUniformExpressionCache, const FMaterialRenderContext& Context, FUniformExpressionCacheAsyncUpdater* Updater) const override;

	/**
	 * Retrieves the array of values for a given type.
	 */
	template <typename ValueType> THashedMaterialParameterMap<ValueType>& GetValueArray() { return ScalarParameterArray; }
	template <typename ValueType> const THashedMaterialParameterMap<ValueType>& GetValueArray() const { return ScalarParameterArray; }

	static bool IsValidParameterValue(float) { return true; }
	static bool IsValidParameterValue(const FLinearColor&) { return true; }
	static bool IsValidParameterValue(const FVector4d&) { return true; }
	static bool IsValidParameterValue(const UTexture* Value) { return Value != nullptr; }
	static bool IsValidParameterValue(const UTextureCollection* Value) { return Value != nullptr; }
	static bool IsValidParameterValue(const UMaterialParameterCollection* Value) { return Value != nullptr; }
	static bool IsValidParameterValue(const URuntimeVirtualTexture* Value) { return Value != nullptr; }
	static bool IsValidParameterValue(const USparseVolumeTexture* Value) { return Value != nullptr; }
	static bool IsValidParameterValue(FName Value) { return true; }

	virtual void StartCacheUniformExpressions() const override;
	virtual void FinishCacheUniformExpressions() const override;

	// Validation logic that exhaustively searches the array, to compare with what the hash table logic finds
	#define HASHED_MATERIAL_PARAMETER_MAP_VALIDATE 0

	template <typename ValueType>
	int32 RenderThread_FindParameterByNameInternal(const FHashedMaterialParameterInfo& ParameterInfo) const
	{
		const THashedMaterialParameterMap<ValueType>& ValueArray = GetValueArray<ValueType>();

#if HASHED_MATERIAL_PARAMETER_MAP_VALIDATE
		int32 ValidateItemIndex = INDEX_NONE;
		for (int32 Index = 0; Index < ValueArray.Array.Num(); Index++)
		{
			if (ValueArray.Array[Index].Info == ParameterInfo)
			{
				ValidateItemIndex = Index;
				break;
			}
		}
#endif  // HASHED_MATERIAL_PARAMETER_MAP_VALIDATE

		uint32 HashTableSize = ValueArray.HashTable.Num();
		if (HashTableSize)
		{
			uint32 HashValue = THashedMaterialParameterMap<ValueType>::TypeHash(ParameterInfo);
			uint32 HashIndex = HashValue & (HashTableSize - 1);
			int32 ItemIndex = (int32)ValueArray.HashTable[HashIndex];

			if (ItemIndex)
			{
				// Items in table are incremented by one, get the actual item index
				ItemIndex--;
				do
				{
					if (ValueArray.Array[ItemIndex].Info == ParameterInfo)
					{
#if HASHED_MATERIAL_PARAMETER_MAP_VALIDATE
						check(ValidateItemIndex == ItemIndex);
#endif
						return ItemIndex;
					}

					// Additional items in a chain must have a non-zero index, because items in the array are linked to buckets
					// in order, so the zeroeth item in the array must be first in its bucket, and zero can't be a valid next index.
					ItemIndex = (int32)ValueArray.Array[ItemIndex].Info.HashNext;

				} while (ItemIndex);
			}
		}
		else
		{
			// No hash table -- item array should also be empty
			check(ValueArray.Array.IsEmpty());
		}

#if HASHED_MATERIAL_PARAMETER_MAP_VALIDATE
		check(ValidateItemIndex == INDEX_NONE);
#endif
		return INDEX_NONE;
	}
	
	#undef HASHED_MATERIAL_PARAMETER_MAP_VALIDATE

	/** The parent of the material instance. */
	UMaterialInterface* Parent;

	/** The UMaterialInstance which owns this resource. */
	UMaterialInstance* Owner;

	/** The game thread accessible parent of the material instance. */
	UMaterialInterface* GameThreadParent;
	
	/** StaticSwitch parameters to select material permutation **/
	THashedMaterialParameterMap<bool> StaticSwitchParameterArray;
	/** Vector parameters for this material instance. */
	THashedMaterialParameterMap<FLinearColor> VectorParameterArray;
	/** DoubleVector parameters for this material instance. */
	THashedMaterialParameterMap<FVector4d> DoubleVectorParameterArray;
	/** Scalar parameters for this material instance. */
	THashedMaterialParameterMap<float> ScalarParameterArray;
	/** Texture parameters for this material instance. */
	THashedMaterialParameterMap<const UTexture*> TextureParameterArray;
	/** TextureCollection parameters for this material instance. */
	THashedMaterialParameterMap<const UTextureCollection*> TextureCollectionParameterArray;
	/** Runtime Virtual Texture parameters for this material instance. */
	THashedMaterialParameterMap<const URuntimeVirtualTexture*> RuntimeVirtualTextureParameterArray;
	/** Sparse Volume Texture parameters for this material instance. */
	THashedMaterialParameterMap<const USparseVolumeTexture*> SparseVolumeTextureParameterArray;
	/** Parameter collection parameters for this material instance. */
	THashedMaterialParameterMap<const UMaterialParameterCollection*> ParameterCollectionParameterArray;
	/** Remap layer indices for parent */
	TArray<int32> ParentLayerIndexRemap;
	/** User Scene Texture overrides for this material instance. */
	TArray<FUserSceneTextureOverride> UserSceneTextureOverrides;
	/** Post Process overrides for this material instance. */
	FPostProcessBlendableOverrides PostProcessBlendableOverrides;
};

template <> FORCEINLINE THashedMaterialParameterMap<bool>& FMaterialInstanceRenderProxy::GetValueArray() { return StaticSwitchParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<float>& FMaterialInstanceRenderProxy::GetValueArray() { return ScalarParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<FLinearColor>& FMaterialInstanceRenderProxy::GetValueArray() { return VectorParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<FVector4d>& FMaterialInstanceRenderProxy::GetValueArray() { return DoubleVectorParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<const UTexture*>& FMaterialInstanceRenderProxy::GetValueArray() { return TextureParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<const UTextureCollection*>& FMaterialInstanceRenderProxy::GetValueArray() { return TextureCollectionParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<const URuntimeVirtualTexture*>& FMaterialInstanceRenderProxy::GetValueArray() { return RuntimeVirtualTextureParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<const USparseVolumeTexture*>& FMaterialInstanceRenderProxy::GetValueArray() { return SparseVolumeTextureParameterArray; }
template <> FORCEINLINE THashedMaterialParameterMap<const UMaterialParameterCollection*>& FMaterialInstanceRenderProxy::GetValueArray() { return ParameterCollectionParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<bool>& FMaterialInstanceRenderProxy::GetValueArray() const { return StaticSwitchParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<float>& FMaterialInstanceRenderProxy::GetValueArray() const { return ScalarParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<FLinearColor>& FMaterialInstanceRenderProxy::GetValueArray() const { return VectorParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<FVector4d>& FMaterialInstanceRenderProxy::GetValueArray() const { return DoubleVectorParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<const UTexture*>& FMaterialInstanceRenderProxy::GetValueArray() const { return TextureParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<const UTextureCollection*>& FMaterialInstanceRenderProxy::GetValueArray() const { return TextureCollectionParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<const URuntimeVirtualTexture*>& FMaterialInstanceRenderProxy::GetValueArray() const { return RuntimeVirtualTextureParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<const USparseVolumeTexture*>& FMaterialInstanceRenderProxy::GetValueArray() const { return SparseVolumeTextureParameterArray; }
template <> FORCEINLINE const THashedMaterialParameterMap<const UMaterialParameterCollection*>& FMaterialInstanceRenderProxy::GetValueArray() const { return ParameterCollectionParameterArray; }

struct FMaterialInstanceParameterSet
{
	TArray<THashedMaterialParameterMap<bool>::TNamedParameter>									StaticSwitchParameters;
	TArray<THashedMaterialParameterMap<float>::TNamedParameter>									ScalarParameters;
	TArray<THashedMaterialParameterMap<FLinearColor>::TNamedParameter>							VectorParameters;
	TArray<THashedMaterialParameterMap<FVector4d>::TNamedParameter>								DoubleVectorParameters;
	TArray<THashedMaterialParameterMap<const UTexture*>::TNamedParameter>						TextureParameters;
	TArray<THashedMaterialParameterMap<const UTextureCollection*>::TNamedParameter>				TextureCollectionParameters;
	TArray<THashedMaterialParameterMap<const URuntimeVirtualTexture*>::TNamedParameter>			RuntimeVirtualTextureParameters;
	TArray<THashedMaterialParameterMap<const USparseVolumeTexture*>::TNamedParameter>			SparseVolumeTextureParameters;
	TArray<THashedMaterialParameterMap<const UMaterialParameterCollection*>::TNamedParameter>	ParameterCollectionParameters;
	TArray<FUserSceneTextureOverride>															UserSceneTextureOverrides;
	FPostProcessBlendableOverrides																PostProcessBlendableOverrides;
};
	
/** Finds a parameter by name from the game thread. */
template <typename ParameterType>
ParameterType* GameThread_FindParameterByName(TArray<ParameterType>& Parameters, const FHashedMaterialParameterInfo& ParameterInfo)
{
	for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ParameterIndex++)
	{
		ParameterType* Parameter = &Parameters[ParameterIndex];
		if (Parameter->ParameterInfo == ParameterInfo)
		{
			return Parameter;
		}
	}
	return NULL;
}

template <typename ParameterType>
const ParameterType* GameThread_FindParameterByName(const TArray<ParameterType>& Parameters, const FHashedMaterialParameterInfo& ParameterInfo)
{
	for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ParameterIndex++)
	{
		const ParameterType* Parameter = &Parameters[ParameterIndex];
		if (Parameter->ParameterInfo == ParameterInfo)
		{
			return Parameter;
		}
	}
	return NULL;
}

template <typename ParameterType>
int32 GameThread_FindParameterIndexByName(const TArray<ParameterType>& Parameters, const FHashedMaterialParameterInfo& ParameterInfo)
{
	for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ++ParameterIndex)
	{
		const ParameterType* Parameter = &Parameters[ParameterIndex];
		if (Parameter->ParameterInfo == ParameterInfo)
		{
			return ParameterIndex;
		}
	}

	return INDEX_NONE;
}

/** Finds a parameter by index from the game thread. */
template <typename ParameterType>
ParameterType* GameThread_FindParameterByIndex(TArray<ParameterType>& Parameters, int32 Index)
{
	if (!Parameters.IsValidIndex(Index))
	{
		return nullptr;
	}

	return &Parameters[Index];
}

template <typename ParameterType>
const ParameterType* GameThread_FindParameterByIndex(const TArray<ParameterType>& Parameters, int32 Index)
{
	if (!Parameters.IsValidIndex(Index))
	{
		return nullptr;
	}

	return &Parameters[Index];
}

template <typename ParameterType>
inline bool GameThread_GetParameterValue(const TArray<ParameterType>& Parameters, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutResult)
{
	for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ParameterIndex++)
	{
		const ParameterType* Parameter = &Parameters[ParameterIndex];
		if (Parameter->IsOverride() && Parameter->IsValid() && Parameter->ParameterInfo == ParameterInfo)
		{
			Parameter->GetValue(OutResult);
			return true;
		}
	}
	return false;
}

template <typename ParameterType, typename OverridenParametersType>
inline void GameThread_ApplyParameterOverrides(
	const TArray<ParameterType>& Parameters,
	TArrayView<const int32> LayerIndexRemap,
	bool bSetOverride,
	OverridenParametersType& OverridenParameters, // TSet<FMaterialParameterInfo, ...>
	TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters,
	bool bAddIfNotFound = false)
{
	for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ParameterIndex++)
	{
		const ParameterType* Parameter = &Parameters[ParameterIndex];
		if (Parameter->IsOverride())
		{
			FMaterialParameterInfo ParameterInfo;
			if (Parameter->ParameterInfo.RemapLayerIndex(LayerIndexRemap, ParameterInfo))
			{
				bool bPreviouslyOverriden = false;
				OverridenParameters.Add(ParameterInfo, &bPreviouslyOverriden);
				if (!bPreviouslyOverriden)
				{
					FMaterialParameterMetadata* Result = bAddIfNotFound ? &OutParameters.FindOrAdd(ParameterInfo) : OutParameters.Find(ParameterInfo);
					if (Result)
					{
						Parameter->GetValue(*Result);
#if WITH_EDITORONLY_DATA
						if (bSetOverride)
						{
							Result->bOverride = true;
						}
#endif // WITH_EDITORONLY_DATA
					}
				}
			}
		}
	}
}

template<typename TArrayType>
inline void RemapLayersForParent(TArrayType& LayerIndexRemap, int32 NumParentLayers, TArrayView<const int32> ParentLayerIndexRemap)
{
	TArrayType NewLayerIndexRemap;
	NewLayerIndexRemap.Init(INDEX_NONE, NumParentLayers);

	check(LayerIndexRemap.Num() == ParentLayerIndexRemap.Num());
	for (int32 i = 0; i < ParentLayerIndexRemap.Num(); ++i)
	{
		const int32 ParentLayerIndex = ParentLayerIndexRemap[i];
		if (ParentLayerIndex != INDEX_NONE)
		{
			NewLayerIndexRemap[ParentLayerIndex] = LayerIndexRemap[i];
		}
	}
	LayerIndexRemap = MoveTemp(NewLayerIndexRemap);
}
