// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialInterface.cpp: UMaterialInterface implementation.
=============================================================================*/

#include "Materials/MaterialInterface.h"
#include "Misc/ScopeRWLock.h"

#include "MeshUVChannelInfo.h"
#include "RenderingThread.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PrimitiveViewRelevance.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialSharedPrivate.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/AssetUserData.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "ObjectCacheEventSink.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "Engine/NeuralProfile.h"
#include "Engine/ToonProfile.h"
#include "Interfaces/ITargetPlatform.h"
#include "Components/PrimitiveComponent.h"
#include "ContentStreaming.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "MeshBatch.h"
#include "Engine/Scene.h"
#include "RenderUtils.h"
#include "TextureCompiler.h"
#include "MaterialDomain.h"
#include "MaterialShaderQualitySettings.h"
#include "Materials/MaterialRenderProxy.h"
#include "ProfilingDebugging/CookStats.h"
#include "ShaderPlatformQualitySettings.h"
#include "ObjectCacheContext.h"
#include "MaterialCachedData.h"
#include "Components/DecalComponent.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/Package.h"
#include "ShaderCompiler.h"
#include "Materials/MaterialInsights.h"
#include "UnrealEngine.h"
#include "UObject/ObjectEditorOptionalSupport.h"
#include "UObject/ICookInfo.h"
#include "Async/UniqueLock.h"
#include "Engine/TextureCollection.h"

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#include "Misc/PackageName.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialInterface)

DECLARE_CYCLE_STAT(TEXT("MaterialInterface StripUnusedDefaultTextures"), STAT_MaterialInterface_StripUnusedDefaultTextures, STATGROUP_Shaders);

#define LOCTEXT_NAMESPACE "MaterialInterface"

/** 
 * This is used to deprecate data that has been built with older versions.
 * To regenerate the data, commands like "BUILDMATERIALTEXTURESTREAMINGDATA" can be used in the editor.
 * Ideally the data would be stored the DDC instead of the asset, but this is not yet  possible because it requires the GPU.
 */
#define MATERIAL_TEXTURE_STREAMING_DATA_VERSION 1

ENGINE_API extern bool bEnableRestrictiveMaterialMode;

//////////////////////////////////////////////////////////////////////////

UEnum* UMaterialInterface::SamplerTypeEnum = nullptr;

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
static bool GShouldStripUnusedDefaultTextures = true;
static FAutoConsoleVariableRef CVarStripUnusedDefaultTextures(
	TEXT("r.Material.StripUnusedDefaultTextures"),
	GShouldStripUnusedDefaultTextures,
	TEXT("If enabled, unused Default textures will be discarded at cook time.\n"),
	ECVF_ReadOnly);
#endif

bool IsHairStrandsGeometrySupported(const EShaderPlatform Platform)
{
	check(Platform != SP_NumPlatforms);

	return FDataDrivenShaderPlatformInfo::GetSupportsHairStrandGeometry(Platform)
		&& IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
}

bool IsCompatibleWithHairStrands(const FMaterial* Material, const ERHIFeatureLevel::Type FeatureLevel)
{
	return
		ERHIFeatureLevel::SM5 <= FeatureLevel &&
		Material && Material->IsUsedWithHairStrands() && 
		IsOpaqueOrMaskedBlendMode(*Material);
}

bool IsCompatibleWithHairStrands(EShaderPlatform Platform, const FMaterialShaderParameters& Parameters)
{
	return
		IsHairStrandsGeometrySupported(Platform) &&
		Parameters.bIsUsedWithHairStrands &&
		IsOpaqueOrMaskedBlendMode(Parameters);
}

EMaterialQualityLevel::Type GetCurrentMaterialQualityLevelChecked()
{
	checkf(!IsRunningCookCommandlet(), TEXT("Calling GetCurrentMaterialQualityLevelChecked while inside a cook commandlet is not correct, as it will return the cookers MaterialQualityLevel and not the MaterialQualityLevel you are cooking for"));
	check(GetCachedScalabilityCVars().bInitialized);
	return GetCachedScalabilityCVars().MaterialQualityLevel;
}

static EMaterialGetParameterValueFlags MakeParameterValueFlags(bool bOveriddenOnly)
{
	EMaterialGetParameterValueFlags Result = EMaterialGetParameterValueFlags::CheckInstanceOverrides;
	if (!bOveriddenOnly)
	{
		Result |= EMaterialGetParameterValueFlags::CheckNonOverrides;
	}
	return Result;
}

#if WITH_EDITORONLY_DATA

// Deletes any invalid EditorOnlyData object with specified name that isn't the one assigned to specified interface.
// Returns whether the MI's EditorOnlyData was renamed.
static bool SafeRenameEditorOnlyData(UMaterialInterface* MaterialInterface, const FString& EditorOnlyDataName, bool bLogMessageIfRenamed)
{
	UObject* EditorOnlyCurrent = MaterialInterface->GetEditorOnlyData();
	UObject* EditorOnlyWithIncorrectName = StaticFindObject(/*Class=*/ nullptr, MaterialInterface, *EditorOnlyDataName, EFindObjectFlags::ExactClass);
	if (EditorOnlyWithIncorrectName && EditorOnlyWithIncorrectName != EditorOnlyCurrent)
	{
		if (EditorOnlyWithIncorrectName->IsRooted())
		{
			UE_LOGF(LogMaterial, Warning, "MaterialInterface '%ls' editor only data ('%ls') could not be marked for garbage bacause it was rooted. Please resave this material.", *MaterialInterface->GetName(), *EditorOnlyWithIncorrectName->GetName());
			return false;
		}
		FName UniqueDummyName = MakeUniqueObjectName(GetTransientPackage(), UMaterialInterfaceEditorOnlyData::StaticClass());
		EditorOnlyWithIncorrectName->Rename(*UniqueDummyName.ToString(), GetTransientPackage(), REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		EditorOnlyWithIncorrectName->MarkAsGarbage();
	}

	if (EditorOnlyCurrent && EditorOnlyCurrent->GetName() != EditorOnlyDataName)
	{
		if (bLogMessageIfRenamed)
		{
			UE_LOGF(LogMaterial, Display, "MaterialInterface %ls has a incorrectly name EditorOnlyData '%ls'. This may cause issues when running the editor on cooked data. Trying to rename it to the correct name '%ls'.", *MaterialInterface->GetName(), *EditorOnlyCurrent->GetName(), *EditorOnlyDataName);
		}

		return EditorOnlyCurrent->Rename(*EditorOnlyDataName, nullptr, REN_NonTransactional | REN_DontCreateRedirectors);
	}

	return true;
}

#endif

//////////////////////////////////////////////////////////////////////////

/** Copies the material's relevance flags to a primitive's view relevance flags. */
void FMaterialRelevance::SetPrimitiveViewRelevance(FPrimitiveViewRelevance& OutViewRelevance) const
{
	OutViewRelevance.Raw = Raw;
}

//////////////////////////////////////////////////////////////////////////

#if WITH_EDITORONLY_DATA

namespace UE::MaterialInterface
{
	FString GetEditorOnlyDataName(const TCHAR* InMaterialName)
	{
		return FString::Printf(TEXT("%sEditorOnlyData"), InMaterialName);
	}
}

#endif // WITH_EDITORONLY_DATA

UMaterialInterface::UMaterialInterface() = default;

UMaterialInterface::UMaterialInterface(FVTableHelper& Helper)
	: Super(Helper)
{
}

UMaterialInterface::~UMaterialInterface()
{
}

UMaterialInterface::UMaterialInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		MaterialDomainString(MD_Surface); // find the enum for this now before we start saving

		InitDefaultMaterials();

		if (SamplerTypeEnum == nullptr)
		{
			SamplerTypeEnum = StaticEnum<EMaterialSamplerType>();
			check(SamplerTypeEnum);
		}

		SetLightingGuid();
	}
}

void UMaterialInterface::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

#if WITH_EDITOR
	if (FArchiveCookContext* CookContext = Ar.GetCookContext())
	{
		// Mark whether this material is part of the base game or of a DLC shipped with the base game.
		// This provides information on whether it is safe to be used as a parent safely in child modules.
		bIncludedInBaseGame = Ar.GetCookContext()->GetCookingDLC() == UE::Cook::ECookingDLC::No;

		// Make sure we are not cooking materials with the new translator flag enabled as it is still experimental.
		const UMaterial* Material = GetMaterial_Concurrent();
		if (Material && Material->IsUsingNewHLSLGenerator())
		{
			UE_LOGF(LogMaterial, Display, "Material '%ls' is marked to use the new material translator. Please refrain from submitting materials using the new translator as it is still experimental.", *GetName());
		}
	}
#endif

	Super::Serialize(Ar);

	bool bSavedCachedExpressionData = false;
	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::MaterialInterfaceSavedCachedData)
	{
		if ((Ar.IsCooking() || (FPlatformProperties::RequiresCookedData() && Ar.IsSaving() && (Ar.GetPortFlags() & PPF_Duplicate))) && (bool)CachedExpressionData)
		{
			bSavedCachedExpressionData = true;
		}

		Ar << bSavedCachedExpressionData;
	}

	if (bSavedCachedExpressionData)
	{
		if (Ar.IsLoading())
		{
			CachedExpressionData.Reset(new FMaterialCachedExpressionData());
			bLoadedCachedExpressionData = true;
#if WITH_EDITORONLY_DATA
			EditorOnlyData->CachedExpressionData = CachedExpressionData->EditorOnlyData;
#endif
		}
		else
		{
#if WITH_EDITOR
			CachedExpressionData->Validate(*this);
#endif
		}

		check(CachedExpressionData);
		UScriptStruct* Struct = FMaterialCachedExpressionData::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)CachedExpressionData.Get(), Struct, nullptr);

#if WITH_EDITOR
		FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(this);
#endif
	}
    // we don't consider bLoadedCachedExpressionData here because in editor we call UpdateCachedExpressionData which can
    // create CachedExpressionData if it wasn't serialized
	else if (Ar.IsObjectReferenceCollector() && CachedExpressionData)
	{
 		UScriptStruct* Struct = FMaterialCachedExpressionData::StaticStruct();
 		Struct->SerializeTaggedProperties(Ar, (uint8*)CachedExpressionData.Get(), Struct, nullptr);
	}
}

void UMaterialInterface::PostLoad()
{
	Super::PostLoad();
	
	PostLoadDefaultMaterials();

#if WITH_EDITORONLY_DATA
	if (TextureStreamingDataVersion != MATERIAL_TEXTURE_STREAMING_DATA_VERSION)
	{
		TextureStreamingData.Empty();
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
void UMaterialInterface::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMaterialInterfaceEditorOnlyData::StaticClass()));
}
#endif

const FMaterialCachedExpressionData& UMaterialInterface::GetCachedExpressionData(TMicRecursionGuard) const
{
	const FMaterialCachedExpressionData* LocalData = CachedExpressionData.Get();
	return LocalData ? *LocalData : FMaterialCachedExpressionData::EmptyData;
}

#if WITH_EDITORONLY_DATA
int32 UMaterialInterface::GetDefaultTextureIdx(UObject* InTexture) const
{
	const FMaterialCachedExpressionData& UsedCachedData = GetCachedExpressionData();
	return UsedCachedData.GetDefaultTextureIdx(InTexture);
}

bool UMaterialInterface::ContainsDefaultTexture(UObject* InTexture) const
{
	return GetDefaultTextureIdx(InTexture) != INDEX_NONE;
}

const TArray<TObjectPtr<UObject>>& UMaterialInterface::GetDefaultTextures() const
{
	const FMaterialCachedExpressionData& UsedCachedData = GetCachedExpressionData();

	// Can happen inside a Material with bLoadedCachedExpressionData and no EditorOnlyData
	if (UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures.Num() == 0)
	{
		return ReferencedDefaultTextures;
	}

	return UsedCachedData.EditorOnlyData->EditorReferencedDefaultTextures;
}

#endif // WITH_EDITORONLY_DATA

bool UMaterialInterface::IsUsingNewHLSLGenerator() const
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.Material.Translator.EnableNew"));
	if (bEnableRestrictiveMaterialMode || !CVar->GetValueOnAnyThread())
	{
		return false;
	}
	const UMaterial* BaseMaterial = GetMaterial_Concurrent();
	return BaseMaterial ? BaseMaterial->bEnableNewHLSLGenerator : false;
}

bool UMaterialInterface::IsUsingNewTranslatorPrototype() const
{
	return IsUsingNewHLSLGenerator();
}

const FSubstrateCompilationConfig& UMaterialInterface::GetSubstrateCompilationConfig() const
{
	const UMaterial* BaseMaterial = GetMaterial_Concurrent();
	static FSubstrateCompilationConfig DefaultFSubstrateCompilationConfig = FSubstrateCompilationConfig();
	return BaseMaterial ? BaseMaterial->SubstrateCompilationConfig : DefaultFSubstrateCompilationConfig;
}

ENGINE_API void UMaterialInterface::SetSubstrateCompilationConfig(FSubstrateCompilationConfig& SubstrateCompilationConfig)
{
	UMaterial* BaseMaterial = GetMaterial();
	if (BaseMaterial)
	{
		BaseMaterial->SubstrateCompilationConfig = SubstrateCompilationConfig;
	}
}

void UMaterialInterface::GetQualityLevelUsage(TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num> >& OutQualityLevelsUsed, EShaderPlatform ShaderPlatform, bool bCooking)
{
	OutQualityLevelsUsed = GetCachedExpressionData().QualityLevelsUsed;
	if (OutQualityLevelsUsed.Num() == 0)
	{
		OutQualityLevelsUsed.AddDefaulted(EMaterialQualityLevel::Num);
	}
	if (ShaderPlatform != SP_NumPlatforms)
	{
		const UShaderPlatformQualitySettings* MaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(ShaderPlatform);
		for (int32 Quality = 0; Quality < EMaterialQualityLevel::Num; ++Quality)
		{
			const FMaterialQualityOverrides& QualityOverrides = MaterialQualitySettings->GetQualityOverrides((EMaterialQualityLevel::Type)Quality);
			if (bCooking && QualityOverrides.bDiscardQualityDuringCook)
			{
				OutQualityLevelsUsed[Quality] = false;
			}
			else if (QualityOverrides.bEnableOverride &&
				QualityOverrides.HasAnyOverridesSet() &&
				QualityOverrides.CanOverride(ShaderPlatform))
			{
				OutQualityLevelsUsed[Quality] = true;
			}
		}
	}

#if WITH_EDITOR
	// if we specified -CacheMaterialQuality= on the cmd line we should only cook that quality.
	static const int32 MaterialQualityLevel = GetCmdLineMaterialQualityToCache();
	if (MaterialQualityLevel != INDEX_NONE)
	{
		// the format is only valid if it is a desired format for this platform.
		if (OutQualityLevelsUsed.IsValidIndex(MaterialQualityLevel))
		{
			// only cache the format specified on the command line.
			OutQualityLevelsUsed.SetNumZeroed(OutQualityLevelsUsed.Num());
			OutQualityLevelsUsed[MaterialQualityLevel] = true;
		}
	}
#endif
}

TArrayView<const TObjectPtr<UObject>> UMaterialInterface::GetReferencedTextures() const
{
#if WITH_EDITORONLY_DATA
	return GetDefaultTextures();
#else
	return ReferencedDefaultTextures;
#endif
}

TConstArrayView<TObjectPtr<UTextureCollection>> UMaterialInterface::GetReferencedTextureCollections() const
{
	return GetCachedExpressionData().ReferencedTextureCollections;
}

#if WITH_EDITOR
void UMaterialInterface::GetReferencedTexturesAndOverrides(TSet<const UTexture*>& InOutTextures) const
{
	const TArray<TObjectPtr<UObject>> Textures = GetDefaultTextures();
	for (UObject* UsedObject : Textures)
	{
		if (const UTexture* UsedTexture = Cast<UTexture>(UsedObject))
		{
			InOutTextures.Add(UsedTexture);
		}
	}
}
#endif // WITH_EDITOR

// Deprecated in 5.7
void UMaterialInterface::GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel) const
{
	GetUsedTexturesAndIndices(OutTextures, OutIndices, QualityLevel, GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
}

void UMaterialInterface::GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform) const
{
	GetUsedTextures(OutTextures, QualityLevel, InShaderPlatform);
	OutIndices.AddDefaulted(OutTextures.Num());
}

void UMaterialInterface::GetTextureExpressionValues(const FMaterialResource* MaterialResource, TArray<UTexture*>& OutTextures, TArray<TArray<int32>>* OutIndices, bool bIncludeTextureCollections, bool bAllowOverrides) const
{
	check(MaterialResource);
	const FUniformExpressionSet& UniformExpressions = MaterialResource->GetUniformExpressions();

	// Pre-allocate array
	if (OutIndices) 
	{
		uint32 NumTextures = 0u;
		for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
		{
			NumTextures += UniformExpressions.GetNumTextures((EMaterialTextureParameterType)TypeIndex);
		}
		OutIndices->Empty(NumTextures);
	}

	for (int32 TypeIndex = 0; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		// Iterate over each of the material's texture expressions.
		for (int32 TextureIndex = 0; TextureIndex < UniformExpressions.GetNumTextures((EMaterialTextureParameterType)TypeIndex); TextureIndex++)
		{
			// Evaluate the expression in terms of this material instance.
			UTexture* Texture = nullptr;
			UniformExpressions.GetGameThreadTextureValue((EMaterialTextureParameterType)TypeIndex, TextureIndex, this, *MaterialResource, Texture, bAllowOverrides);

			if (Texture)
			{
				const int32 InsertIndex = OutTextures.AddUnique(Texture);

				if (OutIndices)
				{
					if (InsertIndex >= OutIndices->Num())
					{
						OutIndices->AddDefaulted(InsertIndex - OutIndices->Num() + 1);
					}

					const FMaterialTextureParameterInfo& Parameter = UniformExpressions.GetTextureParameter((EMaterialTextureParameterType)TypeIndex, TextureIndex);
					(*OutIndices)[InsertIndex].Add(Parameter.TextureIndex);
				}
			}
		}
	}

	if (bIncludeTextureCollections)
	{
		checkf(OutIndices == nullptr, TEXT("Texture Collections don't work with Texture Indices."));

		for (int32 TextureCollectionIndex = 0; TextureCollectionIndex < UniformExpressions.GetNumTextureCollections(); TextureCollectionIndex++)
		{
			UTextureCollection* TextureCollection = nullptr;
			UniformExpressions.GetGameThreadTextureCollectionValue(TextureCollectionIndex, this, *MaterialResource, TextureCollection);

			if (TextureCollection)
			{
				for (UTexture* Texture : TextureCollection->Textures)
				{
					if (Texture)
					{
						OutTextures.AddUnique(Texture);
					}
				}
			}
		}
	}
}

TArray<const FMaterialResource*, TInlineAllocator<4>> UMaterialInterface::GetMaterialResources(const UMaterialInterface* InMaterial, const FInt32Range InQualityLevelRange, const FInt32Range InShaderPlatformRange)
{
	TArray<const FMaterialResource*, TInlineAllocator<4>> MatchedResources;
	// Parse all relevant quality and feature levels.
	for (int32 QualityLevelIndex = InQualityLevelRange.GetLowerBoundValue(); QualityLevelIndex <= InQualityLevelRange.GetUpperBoundValue(); ++QualityLevelIndex)
	{
		for (int32 ShaderPlatformIndex = InShaderPlatformRange.GetLowerBoundValue(); ShaderPlatformIndex <= InShaderPlatformRange.GetUpperBoundValue(); ++ShaderPlatformIndex)
		{
			if (!FDataDrivenShaderPlatformInfo::IsValid((EShaderPlatform)ShaderPlatformIndex))
			{
				continue;
			}

			const FMaterialResource* CurrentResource = InMaterial->GetMaterialResource((EShaderPlatform)ShaderPlatformIndex, (EMaterialQualityLevel::Type)QualityLevelIndex);
			if (CurrentResource)
			{
				MatchedResources.AddUnique(CurrentResource);
			}
		}
	}

	return MatchedResources;
}

#if WITH_EDITORONLY_DATA
bool UMaterialInterface::GetStaticSwitchParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, FGuid& OutExpressionGuid, bool bOveriddenOnly /*= false*/) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::StaticSwitch, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutExpressionGuid = Result.ExpressionGuid;
		OutValue = Result.Value.AsStaticSwitch();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetStaticComponentMaskParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& R, bool& G, bool& B, bool& A, FGuid& OutExpressionGuid, bool bOveriddenOnly /*= false*/) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::StaticSwitch, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutExpressionGuid = Result.ExpressionGuid;
		R = Result.Value.Bool[0];
		G = Result.Value.Bool[1];
		B = Result.Value.Bool[2];
		A = Result.Value.Bool[3];
		return true;
	}
	return false;
}
#endif // WITH_EDITORONLY_DATA

// Deprecated in 5.7
bool UMaterialInterface::IsUsingWorldPositionOffset_Concurrent(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return IsUsingWorldPositionOffset_Concurrent(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

bool UMaterialInterface::IsUsingWorldPositionOffset_Concurrent(EShaderPlatform InShaderPlatform) const
{
#if !WITH_EDITOR
	if (const FCachedMaterialRelevance* Cached = CachedMaterialRelevance.TryGet())
	{
		return Cached->Relevance.bUsesWorldPositionOffset;
	}
#endif
	if (const UMaterial* Material = GetMaterial_Concurrent())
	{
		const FMaterialResource* MaterialResource = GetMaterialResource(InShaderPlatform);
		if (!MaterialResource || !MaterialResource->GetMaterial())
		{
			MaterialResource = Material->GetMaterialResource(InShaderPlatform);
		}
		if (MaterialResource)
		{
			return MaterialResource->MaterialUsesWorldPositionOffset_GameThread();
		}
	}
	return false;
}

bool UMaterialInterface::IsUsingMotionVectorWorldOffset_Concurrent(EShaderPlatform InShaderPlatform) const
{
#if !WITH_EDITOR
	if (const FCachedMaterialRelevance* Cached = CachedMaterialRelevance.TryGet())
	{
		return Cached->bUsesMotionVectorWorldOffset;
	}
#endif
	if (const UMaterial* Material = GetMaterial_Concurrent())
	{
		const FMaterialResource* MaterialResource = GetMaterialResource(InShaderPlatform);
		if (!MaterialResource || !MaterialResource->GetMaterial())
		{
			MaterialResource = Material->GetMaterialResource(InShaderPlatform);
		}
		if (MaterialResource)
		{
			return MaterialResource->MaterialUsesMotionVectorWorldOffset_GameThread();
		}
	}
	return false;
}

bool UMaterialInterface::IsUsingTemporalResponsiveness_Concurrent(EShaderPlatform InShaderPlatform) const
{
#if !WITH_EDITOR
	if (const FCachedMaterialRelevance* Cached = CachedMaterialRelevance.TryGet())
	{
		return Cached->bUsesTemporalResponsiveness;
	}
#endif
	if (const UMaterial* Material = GetMaterial_Concurrent())
	{
		const FMaterialResource* MaterialResource = GetMaterialResource(InShaderPlatform);
		if (!MaterialResource || !MaterialResource->GetMaterial())
		{
			MaterialResource = Material->GetMaterialResource(InShaderPlatform);
		}
		if (MaterialResource)
		{
			return MaterialResource->MaterialUsesTemporalResponsiveness_GameThread();
		}
	}
	return false;
}


// Deprecated 5.7
FMaterialRelevance UMaterialInterface::GetRelevance_Internal(const UMaterial* Material, ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetRelevance_Internal(Material, GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UMaterialInterface::GetRelevance_Internal(const UMaterial* Material, EShaderPlatform InShaderPlatform) const
{
	if (!Material)
	{
		return FMaterialRelevance();
	}

	const FMaterialResource* MaterialResource = GetMaterialResource(InShaderPlatform);

	// If material is invalid e.g. unparented instance, fallback to the passed in material
	bool bIsValidMaterialResource = (MaterialResource != nullptr) && (MaterialResource->GetMaterial() != nullptr);
	if (!bIsValidMaterialResource && (Material != nullptr))
	{
		MaterialResource = Material->GetMaterialResource(InShaderPlatform);
	}

	if (MaterialResource == nullptr)
	{
		return FMaterialRelevance();
	}

	const bool bIsMobile = IsMobilePlatform(InShaderPlatform);
	const bool bUsesSingleLayerWaterMaterial = MaterialResource->GetShadingModels().HasShadingModel(MSM_SingleLayerWater);
	const bool bIsSinglePassWaterTranslucent = bIsMobile && bUsesSingleLayerWaterMaterial && IsUsingSingleLayerWaterFallback(InShaderPlatform);

	// Note that even though XX_GameThread() api is called, this function can be called on non game thread via 
	// GetRelevance_Concurrent()
	static const TArray<EMaterialShadingModel> AnisoShadingModels = { MSM_DefaultLit, MSM_ClearCoat };
	bool bUsesAnisotropy = MaterialResource->GetShadingModels().HasAnyShadingModel(AnisoShadingModels) && 
		MaterialResource->MaterialUsesAnisotropy_GameThread();

	const EBlendMode BlendMode = (EBlendMode)GetBlendMode();
	const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode) || bIsSinglePassWaterTranslucent; // We want meshes with water materials to be scheduled for translucent pass on mobile.

	EMaterialDomain Domain = (EMaterialDomain)MaterialResource->GetMaterialDomain();
	bool bDecal = (Domain == MD_DeferredDecal);

	// Determine the material's view relevance.
	FMaterialRelevance MaterialRelevance;

	MaterialRelevance.ShadingModelMask = GetShadingModels().GetShadingModelField();

	if ((uint16(MaterialResource->MaterialGetSubstrateMaterialBsdfFeatures_GameThread()) & uint16(ESubstrateBsdfFeature::Toon)) > 0)
	{
		// Add the Substrate shading model to the ShadingModelMask so that we detect on mobile that is it not only DefaultLit material in the scene.
		// So that the base pass write the correct stencil values for the toon shading model to be schedule during the "all shading models" pass.
		MaterialRelevance.ShadingModelMask |= EMaterialShadingModel::MSM_Strata;
	}

	MaterialRelevance.CustomDepthStencilUsageMask = MaterialResource->GetCustomDepthStencilUsageMask_GameThread();
	MaterialRelevance.bDecal = bDecal;

	// Check whether the material can be drawn in the separate translucency pass as per FMaterialResource::IsTranslucencyAfterDOFEnabled and IsMobileSeparateTranslucencyEnabled
	EMaterialTranslucencyPass TranslucencyPass = MTP_BeforeDOF;
	const bool bSupportsSeparateTranslucency = Material->MaterialDomain != MD_UI && Material->MaterialDomain != MD_DeferredDecal;
	if (bIsTranslucent && bSupportsSeparateTranslucency)
	{
		if (bIsMobile)
		{
			if (Material->bEnableMobileSeparateTranslucency)
			{
				TranslucencyPass = MTP_AfterDOF;
			}
		}
		else
		{
			TranslucencyPass = Material->TranslucencyPass;
		}
	}			

	// If dual blending is supported, and we are rendering post-DOF translucency, then we also need to render a second pass to the modulation buffer.
	// The modulation buffer can also be used for regular modulation shaders after DoF.
	const bool bMaterialSeparateModulation = MaterialResource->IsDualBlendingEnabled() || IsModulateBlendMode(BlendMode);

	// Encode Substrate BSDF into a mask where each bit correspond to a number of BSDF (1-8)
	const uint8 SubstrateBSDFCount = FMath::Max(MaterialResource->MaterialGetSubstrateClosureCount_GameThread(), uint8(1u));
	const uint8 SubstrateBSDFCountMask = 1u << uint8(FMath::Min(SubstrateBSDFCount - 1, 8));
	const uint8 SubstrateUintPerPixel = FMath::Max(MaterialResource->MaterialGetSubstrateUintPerPixel_GameThread(), uint8(1u));

	MaterialRelevance.bOpaque = !bIsTranslucent;
	MaterialRelevance.bMasked = IsMasked();
	MaterialRelevance.bDistortion = MaterialResource->IsDistorted();
	MaterialRelevance.bHairStrands = IsCompatibleWithHairStrands(MaterialResource, FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(InShaderPlatform));
	MaterialRelevance.bTwoSided = MaterialResource->IsTwoSided();
	MaterialRelevance.bSeparateTranslucency = bIsTranslucent && (TranslucencyPass == MTP_AfterDOF);
	MaterialRelevance.bTranslucencyModulate = bMaterialSeparateModulation;
	MaterialRelevance.bPostMotionBlurTranslucency = (TranslucencyPass == MTP_AfterMotionBlur);
	MaterialRelevance.bNormalTranslucency = bIsTranslucent && (TranslucencyPass == MTP_BeforeDOF);
	MaterialRelevance.bDisableDepthTest = bIsTranslucent && Material->bDisableDepthTest;
	MaterialRelevance.bUsesSceneColorCopy = bIsTranslucent && MaterialResource->RequiresSceneColorCopy_GameThread();
	MaterialRelevance.bOutputsTranslucentVelocity = MaterialResource->IsTranslucencyWritingVelocity();
	MaterialRelevance.bUsesGlobalDistanceField = MaterialResource->UsesGlobalDistanceField_GameThread();
	MaterialRelevance.bUsesWorldPositionOffset = MaterialResource->MaterialUsesWorldPositionOffset_GameThread();
	MaterialRelevance.bUsesDisplacement = MaterialResource->MaterialUsesDisplacement_GameThread();
	MaterialRelevance.bUsesPixelDepthOffset = MaterialResource->MaterialUsesPixelDepthOffset_GameThread();
	ETranslucencyLightingMode TranslucencyLightingMode = MaterialResource->GetTranslucencyLightingMode();
	MaterialRelevance.bTranslucentSurfaceLighting = bIsTranslucent && (TranslucencyLightingMode == TLM_SurfacePerPixelLighting || TranslucencyLightingMode == TLM_Surface);
	MaterialRelevance.bUsesSceneDepth = MaterialResource->MaterialUsesSceneDepthLookup_GameThread();
	MaterialRelevance.bHasVolumeMaterialDomain = MaterialResource->IsVolumetricPrimitive();
	MaterialRelevance.bUsesDistanceCullFade = MaterialResource->MaterialUsesDistanceCullFade_GameThread();
	MaterialRelevance.bUsesSkyMaterial = Material->bIsSky;
	MaterialRelevance.bUsesSingleLayerWaterMaterial = (bUsesSingleLayerWaterMaterial && !bIsSinglePassWaterTranslucent);
	MaterialRelevance.bUsesAnisotropy = bUsesAnisotropy;
	MaterialRelevance.bUsesCustomizedUVs = MaterialResource->GetNumCustomizedUVs() > 0;
	MaterialRelevance.bUsesVertexInterpolator = MaterialResource->MaterialUsesVertexInterpolator_GameThread();
	MaterialRelevance.SubstrateClosureCountMask = SubstrateBSDFCountMask;
	MaterialRelevance.SubstrateUintPerPixel = SubstrateUintPerPixel;
	MaterialRelevance.SubstrateTileTypeMask = GetSubstrateTileTypeAsUint8(MaterialResource->MaterialGetSubstrateTileType_GameThread());
	MaterialRelevance.bIsLightFunctionAtlasCompatible = MaterialResource->MaterialIsLightFunctionAtlasCompatible_GameThread();
	MaterialRelevance.bSamplesMaterialCache = MaterialResource->SamplesMaterialCache();
	MaterialRelevance.bHasMaterialCacheOutput = MaterialResource->HasMaterialCacheOutput();
	MaterialRelevance.bUsesFirstPersonInterpolation = MaterialResource->HasFirstPersonOutput();

	return MaterialRelevance;
}

FMaterialParameterInfo UMaterialInterface::GetParameterInfo(EMaterialParameterAssociation Association, FName ParameterName, UMaterialFunctionInterface* LayerFunction) const
{
	int32 Index = INDEX_NONE;
	if (Association != GlobalParameter)
	{
		if (LayerFunction)
		{
			FMaterialLayersFunctions MaterialLayers;
			if (GetMaterialLayers(MaterialLayers))
			{
				if (Association == BlendParameter) Index = MaterialLayers.Blends.Find(LayerFunction);
				else if (Association == LayerParameter) Index = MaterialLayers.Layers.Find(LayerFunction);
			}
		}
		if (Index == INDEX_NONE)
		{
			return FMaterialParameterInfo();
		}
	}

	return FMaterialParameterInfo(ParameterName, Association, Index);
}

// Deprecated in 5.7
FMaterialRelevance UMaterialInterface::GetRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UMaterialInterface::GetRelevance(EShaderPlatform InShaderPlatform) const
{
#if !WITH_EDITOR
	if (const FCachedMaterialRelevance* Cached = CachedMaterialRelevance.TryGet())
	{
		return Cached->Relevance;
	}
#endif
	// Find the interface's concrete material.
	const UMaterial* Material = GetMaterial();
	return GetRelevance_Internal(Material, InShaderPlatform);
}

// Deprecated in 5.7
FMaterialRelevance UMaterialInterface::GetRelevance_Concurrent(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetRelevance_Concurrent(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UMaterialInterface::GetRelevance_Concurrent(EShaderPlatform InShaderPlatform) const
{
#if !WITH_EDITOR
	if (const FCachedMaterialRelevance* Cached = CachedMaterialRelevance.TryGet())
	{
		return Cached->Relevance;
	}
#endif
	// Find the interface's concrete material.
	const UMaterial* Material = GetMaterial_Concurrent();
	return GetRelevance_Internal(Material, InShaderPlatform);
}

#if !WITH_EDITOR
void UMaterialInterface::WarmRelevanceCache(EShaderPlatform InShaderPlatform)
{
	if (!CachedMaterialRelevance.TryGet())
	{
		check(IsInGameThread());
		FCachedMaterialRelevance Cache;
		Cache.Relevance = GetRelevance_Concurrent(InShaderPlatform);
		Cache.bUsesMotionVectorWorldOffset = IsUsingMotionVectorWorldOffset_Concurrent(InShaderPlatform);
		Cache.bUsesTemporalResponsiveness = IsUsingTemporalResponsiveness_Concurrent(InShaderPlatform);
		CachedMaterialRelevance.Set(Cache);
	}
}
#endif // !WITH_EDITOR

int32 UMaterialInterface::GetWidth() const
{
	return ME_PREV_THUMBNAIL_SZ+(ME_STD_BORDER*2);
}

int32 UMaterialInterface::GetHeight() const
{
	return ME_PREV_THUMBNAIL_SZ+ME_CAPTION_HEIGHT+(ME_STD_BORDER*2);
}


void UMaterialInterface::SetForceMipLevelsToBeResident( bool OverrideForceMiplevelsToBeResident, bool bForceMiplevelsToBeResidentValue, float ForceDuration, int32 CinematicTextureGroups, bool bFastResponse )
{
	TArray<UTexture*> Textures;
	
	GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked());
	
#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation(Textures);
#endif

	for ( int32 TextureIndex=0; TextureIndex < Textures.Num(); ++TextureIndex )
	{
		UTexture2D* Texture = Cast<UTexture2D>(Textures[TextureIndex]);
		if ( Texture )
		{
			Texture->SetForceMipLevelsToBeResident( ForceDuration, CinematicTextureGroups );
			if (OverrideForceMiplevelsToBeResident)
			{
				Texture->bForceMiplevelsToBeResident = bForceMiplevelsToBeResidentValue;
			}

			if (bFastResponse && (ForceDuration > 0.f || Texture->bForceMiplevelsToBeResident))
			{
				static IConsoleVariable* CVarAllowFastForceResident = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Streaming.AllowFastForceResident"));

				Texture->bIgnoreStreamingMipBias = CVarAllowFastForceResident && CVarAllowFastForceResident->GetInt();
				if (Texture->IsStreamable())
				{
					IStreamingManager::Get().GetRenderAssetStreamingManager().FastForceFullyResident(Texture);
				}
			}
		}
	}
}

void UMaterialInterface::RecacheAllMaterialUniformExpressions(bool bRecreateUniformBuffer)
{
	// For each interface, recache its uniform parameters
	for (TObjectIterator<UMaterialInterface> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		It->RecacheUniformExpressions(bRecreateUniformBuffer);
	}
}

void UMaterialInterface::SubmitRemainingJobsForWorld(UWorld* World, EMaterialShaderPrecompileMode CompileMode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialInterface::SubmitRemainingJobsForWorld);

	TSet<UMaterialInterface*> MaterialsToCache;
	FObjectCacheContextScope ObjectCacheScope;

	for (IPrimitiveComponent* PrimitiveComponentInterface : ObjectCacheScope.GetContext().GetPrimitiveComponents())
	{
		if (World && PrimitiveComponentInterface->GetWorld() != World)
		{
			continue;
		}

		if (PrimitiveComponentInterface->IsRenderStateCreated())
		{
			TObjectCacheIterator<UMaterialInterface> UsedMaterials = ObjectCacheScope.GetContext().GetUsedMaterials(PrimitiveComponentInterface);
			for (UMaterialInterface* MaterialInterface : UsedMaterials)
			{
				if (MaterialInterface && !MaterialInterface->IsComplete())
				{
					MaterialsToCache.Add(MaterialInterface);
				}
			}
		}
	}

	// Add UI and PP Materials.
	for (TObjectIterator<UMaterialInterface> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UMaterial* Material = It->GetMaterial();
		if (Material &&
			(Material->IsUIMaterial() || Material->IsPostProcessMaterial()) &&
			!Material->IsComplete())
		{
			MaterialsToCache.Add(*It);
		}
	}

	// Add Decal Component Materials.
	for (TObjectIterator<UDecalComponent> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It)
		{
			TArray<UMaterialInterface*> OutMaterials;
			It->GetUsedMaterials(OutMaterials);

			for (UMaterialInterface* MaterialInterface : OutMaterials)
			{
				if (MaterialInterface && !MaterialInterface->IsComplete())
				{
					MaterialsToCache.Add(MaterialInterface);
				}
			}
		}
	}

	if (MaterialsToCache.Num())
	{
		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::SyncWithRenderingThread);
		for (UMaterialInterface* Material : MaterialsToCache)
		{
			// This is needed because CacheShaders blindly recreates uniform buffers
			// which can only be done if the draw command is going to be re-cached.
			UpdateContext.AddMaterialInterface(Material);
			Material->CacheShaders(CompileMode);
		}
	}
}

bool UMaterialInterface::IsReadyForFinishDestroy()
{
	bool bIsReady = Super::IsReadyForFinishDestroy();
	bIsReady = bIsReady && ParentRefFence.IsFenceComplete(); 
	return bIsReady;
}

void UMaterialInterface::BeginDestroy()
{
	ParentRefFence.BeginFence();
	Super::BeginDestroy();

#if WITH_EDITOR
	// The object cache needs to be notified when we're getting destroyed
	FObjectCacheEventSink::NotifyMaterialDestroyed_Concurrent(this);
#endif
}

void UMaterialInterface::FinishDestroy()
{
	CachedExpressionData.Reset();
	Super::FinishDestroy();
}

void UMaterialInterface::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UMaterialInterface* This = CastChecked<UMaterialInterface>(InThis);
	if (This->CachedExpressionData)
	{
		This->CachedExpressionData->AddReferencedObjects(Collector);
	}

	{
		// Snapshot the TSharedPtr under the read lock and release the lock before invoking the
		// collector. The TSharedPtr keeps the cache alive past lock release, and writers
		// (CacheTexturesSamplingInfo) don't have to wait on the per-texture collector callbacks.
		TSharedPtr<FMaterialCachedTexturesSamplingInfo> CachedSnapshot;
		{
			UE::TReadScopeLock<FCacheRWLock> ReadLock(This->CachedTexturesSamplingInfoLock);
			CachedSnapshot = This->CachedTexturesSamplingInfo;
		}
		if (CachedSnapshot)
		{
			CachedSnapshot->AddReferencedObjects(Collector);
		}
	}

	Super::AddReferencedObjects(This, Collector);
}

void FMaterialCachedTexturesSamplingInfo::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FTextureSamplingInfo& Info : TextureSamplingInfos)
	{
		Collector.AddReferencedObject(Info.Texture);
	}
}

void FMaterialCachedTexturesSamplingInfo::GetStreamingRenderAssetPrimitiveInfo(
	const FBoxSphereBounds& ComponentBounds, const float ComponentScaling, const FMeshUVChannelInfo& InUVChannelData,
	const EMaterialQualityLevel::Type& MaterialQualityLevel, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures) const
{
	const TArray<uint16>& Mapping = ShaderPlatformMapping.QualityMapping[MaterialQualityLevel];

	for (const uint16& SamplingInfoIndex : Mapping)
	{
		check(TextureSamplingInfos.IsValidIndex(SamplingInfoIndex));
		const FTextureSamplingInfo& TextureSamplingInfo = TextureSamplingInfos[SamplingInfoIndex];
		check(TextureSamplingInfo.Texture);

		const float TextureDensity = [&]
		{
			const float MaterialDensity = TextureSamplingInfo.GetTextureDensity(InUVChannelData);
			// fallback based on FStreamingTextureLevelContext::ProcessMaterial
			if (!MaterialDensity)
			{
				return InUVChannelData.LocalUVDensities[0];
			}
			return MaterialDensity;
		}();

		FStreamingRenderAssetPrimitiveInfo& Info = OutStreamingTextures.AddDefaulted_GetRef();
		Info.RenderAsset = TextureSamplingInfo.Texture;
		Info.TexelFactor = TextureDensity * ComponentScaling;
		Info.PackedRelativeBox = PackedRelativeBox_Identity;
		UnpackRelativeBox(ComponentBounds, Info.PackedRelativeBox, Info.Bounds);

		// checkSlow rather than ensure: this runs per-texture-per-primitive on every SSAM registration.
		// The invariant is that TextureDensity (from the cache or UV fallback) and ComponentScaling are finite;
		// non-finite values imply an upstream bug worth catching in Debug builds, not worth the per-call cost
		// in Development/Test/Shipping.
		checkSlow(FMath::IsFinite(Info.TexelFactor));
	}
}

void UMaterialInterface::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	// use the non-templated CreateEditorOnlyData because we need to use the virtual to get the class of the EOData
	// additionally, pass in an overridden name because we have existing EOData in the wild that must load correctly
	UObject* EOData = UE::EditorOptional::CreateEditorOptionalObject(this, GetEditorOnlyDataClass(), *UE::MaterialInterface::GetEditorOnlyDataName(*GetName()));
	EditorOnlyData = CastChecked<UMaterialInterfaceEditorOnlyData>(EOData);
#endif
	Super::PostInitProperties();
}

void UMaterialInterface::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	SetLightingGuid();

#if WITH_EDITORONLY_DATA
	// If this MI references its editor only data, make sure it has the correct name, or the editor
	// won't run properly when running on cooked data.
	FString EditorOnlyDataName = UE::MaterialInterface::GetEditorOnlyDataName(*GetName());
	SafeRenameEditorOnlyData(this, EditorOnlyDataName, false);
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void UMaterialInterface::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// flush the lighting guid on all changes
	SetLightingGuid();

	LightmassSettings.EmissiveBoost = FMath::Max(LightmassSettings.EmissiveBoost, 0.0f);
	LightmassSettings.DiffuseBoost = FMath::Max(LightmassSettings.DiffuseBoost, 0.0f);
	LightmassSettings.ExportResolutionScale = FMath::Clamp(LightmassSettings.ExportResolutionScale, 0.0f, 16.0f);

	for (UAssetUserData* Datum : AssetUserData)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

namespace MaterialPrivate
{
	static const FName NAME_HasSceneColor("HasSceneColor");
	static const FName NAME_HasPerInstanceRandom("HasPerInstanceRandom");
	static const FName NAME_HasPerInstanceCustomData("HasPerInstanceCustomData");
	static const FName NAME_HasVertexInterpolator("HasVertexInterpolator");
}

void UMaterialInterface::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Add(
		MaterialPrivate::NAME_HasSceneColor,
		FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("HasSceneColor", "Has Scene Color"))
	);

	OutMetadata.Add(
		MaterialPrivate::NAME_HasPerInstanceRandom,
		FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("HasPerInstanceRandom", "Has Per Instance Random"))
	);

	OutMetadata.Add(
		MaterialPrivate::NAME_HasPerInstanceCustomData,
		FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("HasPerInstanceCustomData", "Has Per Instance Custom Data"))
	);

	OutMetadata.Add(
		MaterialPrivate::NAME_HasVertexInterpolator,
		FAssetRegistryTagMetadata()
		.SetDisplayName(LOCTEXT("HasVertexInterpolator", "Has Vertex Interpolator"))
	);
}

void UMaterialInterface::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	{
		const FMaterialCachedExpressionData& CachedData = GetCachedExpressionData();
		Context.AddTag(FAssetRegistryTag(MaterialPrivate::NAME_HasSceneColor, CachedData.bHasSceneColor ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(FAssetRegistryTag(MaterialPrivate::NAME_HasPerInstanceRandom, CachedData.bHasPerInstanceRandom ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(FAssetRegistryTag(MaterialPrivate::NAME_HasPerInstanceCustomData, CachedData.bHasPerInstanceCustomData ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(FAssetRegistryTag(MaterialPrivate::NAME_HasVertexInterpolator, CachedData.bHasVertexInterpolator ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	}

	Super::GetAssetRegistryTags(Context);
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UMaterialInterface::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	Super::AppendToClassSchema(Context);

	UE::MaterialInterface::Private::HashMaterialStaticClassDependenciesForCook(Context);

	const bool bStripUnusedDefaultTextures = ShouldStripUnusedDefaultTextures();
	const uint8 Byte = bStripUnusedDefaultTextures ? 1 : 0;
	Context.Update(&Byte, sizeof(Byte));
}
#endif

void UMaterialInterface::GetLightingGuidChain(bool bIncludeTextures, TArray<FGuid>& OutGuids) const
{
	const FMaterialCachedExpressionData& CachedData = GetCachedExpressionData();
	CachedData.AppendReferencedFunctionIdsTo(OutGuids);
	CachedData.AppendReferencedParameterCollectionIdsTo(OutGuids);

#if WITH_EDITORONLY_DATA
	OutGuids.Add(LightingGuid);
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
uint32 UMaterialInterface::ComputeAllStateCRC() const
{
	uint32 CRC = 0xffffffff;

	const FMaterialCachedExpressionData& CachedData = GetCachedExpressionData();

	// use the precalculated CRC for the function info state ids (faster, as there can be thousands of these)
	CRC = FCrc::TypeCrc32(CachedData.FunctionInfosStateCRC, CRC);

	// mix in the parameter collection info state ids
	for (const FMaterialParameterCollectionInfo& CollectionInfo : CachedData.ParameterCollectionInfos)
	{
		CRC = FCrc::TypeCrc32(CollectionInfo.StateId, CRC);
	}

	return CRC;
}
#endif // WITH_EDITOR

bool UMaterialInterface::GetVectorParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Vector, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.AsLinearColor();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetDoubleVectorParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, FVector4d& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::DoubleVector, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.AsVector4d();
		return true;
	}
	return false;
}

#if WITH_EDITOR
bool UMaterialInterface::IsVectorParameterUsedAsChannelMask(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Vector, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutValue = Result.bUsedAsChannelMask;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetVectorParameterChannelNames(const FHashedMaterialParameterInfo& ParameterInfo, FParameterChannelNames& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Vector, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutValue = Result.ChannelNames;
		return true;
	}
	return false;
}

bool UMaterialInterface::IsDoubleVectorParameterUsedAsChannelMask(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::DoubleVector, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutValue = Result.bUsedAsChannelMask;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetDoubleVectorParameterChannelNames(const FHashedMaterialParameterInfo& ParameterInfo, FParameterChannelNames& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::DoubleVector, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutValue = Result.ChannelNames;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetScalarParameterSliderMinMax(const FHashedMaterialParameterInfo& ParameterInfo, float& OutSliderMin, float& OutSliderMax) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Scalar, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutSliderMin = Result.ScalarMin;
		OutSliderMax = Result.ScalarMax;
		return true;
	}
	return false;
}
#endif // WITH_EDITOR

bool UMaterialInterface::GetScalarParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Scalar, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.AsScalar();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue, EMaterialGetParameterValueFlags Flags) const
{
	return false;
}

#if WITH_EDITOR
bool UMaterialInterface::IsScalarParameterUsedAsAtlasPosition(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, TSoftObjectPtr<class UCurveLinearColor>& Curve, TSoftObjectPtr<class UCurveLinearColorAtlas>& Atlas) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Scalar, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutValue = Result.bUsedAsAtlasPosition;
		Curve = Result.ScalarCurve;
		Atlas = Result.ScalarAtlas;
		return true;
	}
	return false;
}
#endif // WITH_EDITOR

bool UMaterialInterface::GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, UTexture*& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Texture, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.Texture;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetTextureCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, UTextureCollection*& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::TextureCollection, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.TextureCollection;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetParameterCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, UMaterialParameterCollection*& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::ParameterCollection, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.ParameterCollection;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetRuntimeVirtualTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, URuntimeVirtualTexture*& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::RuntimeVirtualTexture, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.RuntimeVirtualTexture;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetSparseVolumeTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, USparseVolumeTexture*& OutValue, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::SparseVolumeTexture, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutValue = Result.Value.SparseVolumeTexture;
		return true;
	}
	return false;
}

#if WITH_EDITOR
bool UMaterialInterface::GetTextureParameterChannelNames(const FHashedMaterialParameterInfo& ParameterInfo, FParameterChannelNames& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Texture, ParameterInfo, Result, EMaterialGetParameterValueFlags::CheckNonOverrides))
	{
		OutValue = Result.ChannelNames;
		return true;
	}
	return false;
}
#endif

bool UMaterialInterface::GetFontParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UFont*& OutFontValue, int32& OutFontPage, bool bOveriddenOnly) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterValue(EMaterialParameterType::Font, ParameterInfo, Result, MakeParameterValueFlags(bOveriddenOnly)))
	{
		OutFontValue = Result.Value.Font.Value;
		OutFontPage = Result.Value.Font.Page;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetParameterDefaultValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue) const
{
	return GetParameterValue(Type, ParameterInfo, OutValue, EMaterialGetParameterValueFlags::CheckNonOverrides);
}


bool UMaterialInterface::GetScalarParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::Scalar, ParameterInfo, Result))
	{
		OutValue = Result.Value.AsScalar();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetVectorParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::Vector, ParameterInfo, Result))
	{
		OutValue = Result.Value.AsLinearColor();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetDoubleVectorParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, FVector4& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::DoubleVector, ParameterInfo, Result))
	{
		OutValue = Result.Value.AsVector4d();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetTextureParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::Texture, ParameterInfo, Result))
	{
		OutValue = Result.Value.Texture;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetTextureCollectionParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::TextureCollection, ParameterInfo, Result))
	{
		OutValue = Result.Value.TextureCollection;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetParameterCollectionParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection*& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::ParameterCollection, ParameterInfo, Result))
	{
		OutValue = Result.Value.ParameterCollection;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetRuntimeVirtualTextureParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture*& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::RuntimeVirtualTexture, ParameterInfo, Result))
	{
		OutValue = Result.Value.RuntimeVirtualTexture;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetSparseVolumeTextureParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture*& OutValue) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::SparseVolumeTexture, ParameterInfo, Result))
	{
		OutValue = Result.Value.SparseVolumeTexture;
		return true;
	}
	return false;
}

bool UMaterialInterface::GetFontParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UFont*& OutFontValue, int32& OutFontPage) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::Font, ParameterInfo, Result))
	{
		OutFontValue = Result.Value.Font.Value;
		OutFontPage = Result.Value.Font.Page;
		return true;
	}
	return false;
}

#if WITH_EDITOR
bool UMaterialInterface::GetStaticSwitchParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, FGuid& OutExpressionGuid) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::StaticSwitch, ParameterInfo, Result))
	{
		OutExpressionGuid = Result.ExpressionGuid;
		OutValue = Result.Value.AsStaticSwitch();
		return true;
	}
	return false;
}

bool UMaterialInterface::GetStaticComponentMaskParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutR, bool& OutG, bool& OutB, bool& OutA, FGuid& OutExpressionGuid) const
{
	FMaterialParameterMetadata Result;
	if (GetParameterDefaultValue(EMaterialParameterType::StaticComponentMask, ParameterInfo, Result))
	{
		OutExpressionGuid = Result.ExpressionGuid;
		OutR = Result.Value.Bool[0];
		OutG = Result.Value.Bool[1];
		OutB = Result.Value.Bool[2];
		OutA = Result.Value.Bool[3];
		return true;
	}
	return false;
}

#endif // WITH_EDITOR

void UMaterialInterface::GetAllParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const
{
	OutParameters.Reset();
	GetCachedExpressionData().GetAllParametersOfType(Type, OutParameters);
}

void UMaterialInterface::GetAllParameterInfoOfType(EMaterialParameterType Type, TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	OutParameterInfo.Reset();
	OutParameterIds.Reset();
	GetCachedExpressionData().GetAllParameterInfoOfType(Type, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllScalarParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::Scalar, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllVectorParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::Vector, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllDoubleVectorParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::DoubleVector, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllTextureParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::Texture, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllTextureCollectionParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::TextureCollection, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllParameterCollectionParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::ParameterCollection, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllRuntimeVirtualTextureParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::RuntimeVirtualTexture, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllSparseVolumeTextureParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::SparseVolumeTexture, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllFontParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::Font, OutParameterInfo, OutParameterIds);
}

#if WITH_EDITORONLY_DATA
void UMaterialInterface::GetAllStaticSwitchParameterInfo(TArray<FMaterialParameterInfo>&OutParameterInfo, TArray<FGuid>&OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::StaticSwitch, OutParameterInfo, OutParameterIds);
}

void UMaterialInterface::GetAllStaticComponentMaskParameterInfo(TArray<FMaterialParameterInfo>&OutParameterInfo, TArray<FGuid>&OutParameterIds) const
{
	GetAllParameterInfoOfType(EMaterialParameterType::StaticComponentMask, OutParameterInfo, OutParameterIds);
}
#endif // WITH_EDITOR

bool UMaterialInterface::GetRefractionSettings(float& OutBiasValue) const
{
	return false;
}

bool UMaterialInterface::GetUserSceneTextureOverride(FName& InOutName) const
{
	return false;
}

FName UMaterialInterface::GetUserSceneTextureOutput(const UMaterial* Base) const
{
	FName Result = NAME_None;

	// Replacing tonemapper can't override output.
	if (Base->BlendableLocation != BL_ReplacingTonemapper)
	{
		// UserSceneTexture output overrides are stored under key "NAME_None".  We store them in the override lookup to save space
		// in the base structure, by avoiding a separate field just for the output override.
		if (!GetUserSceneTextureOverride(Result) && Base)
		{
			// If no override was found, get the result from the base material
			Result = FName(Base->UserSceneTexture);
		}
	}
	return Result;

}

// Certain implementations support overrides to the base material setting, default behavior is to just return the value from the base material
EBlendableLocation UMaterialInterface::GetBlendableLocation(const UMaterial* Base) const
{
	check(Base);
	return Base->BlendableLocation;
}
int32 UMaterialInterface::GetBlendablePriority(const UMaterial* Base) const
{
	check(Base);
	return Base->BlendablePriority;
}


#if WITH_EDITOR
bool UMaterialInterface::GetParameterDesc(const FHashedMaterialParameterInfo& ParameterInfo, FString& OutDesc) const
{
	for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
	{
		FMaterialParameterMetadata Meta;
		if (GetParameterValue((EMaterialParameterType)TypeIndex, ParameterInfo, Meta, EMaterialGetParameterValueFlags::CheckNonOverrides))
		{
			OutDesc = Meta.Description;
			return true;
		}
	}
	return false;
}

bool UMaterialInterface::GetGroupName(const FHashedMaterialParameterInfo& ParameterInfo, FName& OutDesc) const
{
	for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
	{
		FMaterialParameterMetadata Meta;
		if (GetParameterValue((EMaterialParameterType)TypeIndex, ParameterInfo, Meta, EMaterialGetParameterValueFlags::CheckNonOverrides))
		{
			OutDesc = Meta.Group;
			return true;
		}
	}
	return false;
}

bool UMaterialInterface::GetParameterSortPriority(const FHashedMaterialParameterInfo& ParameterInfo, int32& OutSortPriority) const
{
	for (int32 TypeIndex = 0; TypeIndex < NumMaterialParameterTypes; ++TypeIndex)
	{
		FMaterialParameterMetadata Meta;
		if (GetParameterValue((EMaterialParameterType)TypeIndex, ParameterInfo, Meta, EMaterialGetParameterValueFlags::CheckNonOverrides))
		{
			OutSortPriority = Meta.SortPriority;
			return true;
		}
	}
	return false;
}
#endif // WITH_EDITOR

UMaterial* UMaterialInterface::GetBaseMaterial()
{
	return GetMaterial();
}

void UMaterialInterface::OnAssignedAsOverride(const UObject* Owner)
{

}

void UMaterialInterface::OnRemovedAsOverride(const UObject* Owner)
{

}

bool DoesMaterialUseTexture(const UMaterialInterface* Material,const UTexture* CheckTexture)
{
	//Do not care if we're running dedicated server
	if (FPlatformProperties::IsServerOnly())
	{
		return false;
	}

	TArray<UTexture*> Textures;
	Material->GetUsedTextures(Textures);
	for (int32 i = 0; i < Textures.Num(); i++)
	{
		if (Textures[i] == CheckTexture)
		{
			return true;
		}
	}
	return false;
}

float UMaterialInterface::GetOpacityMaskClipValue() const
{
	return 0.0f;
}

EBlendMode UMaterialInterface::GetBlendMode() const
{
	return BLEND_Opaque;
}

bool UMaterialInterface::IsTwoSided() const
{
	return false;
}

bool UMaterialInterface::IsThinSurface() const
{
	return false;
}

bool UMaterialInterface::IsDitheredLODTransition() const
{
	return false;
}

bool UMaterialInterface::IsTranslucencyWritingCustomDepth() const
{
	return false;
}

bool UMaterialInterface::IsTranslucencyWritingVelocity() const
{
	return false;
}

bool UMaterialInterface::IsTranslucencyVelocityFromDepth() const
{
	return false;
}

bool UMaterialInterface::IsTranslucencyWritingFrontLayerTransparency() const
{
	return false;
}

bool UMaterialInterface::IsMasked() const
{
	return false;
}

FDisplacementScaling UMaterialInterface::GetDisplacementScaling() const
{
	return FDisplacementScaling();
}

bool UMaterialInterface::IsDisplacementFadeEnabled() const
{
	return false;
}

FDisplacementFadeRange UMaterialInterface::GetDisplacementFadeRange() const
{
	return FDisplacementFadeRange();
}

float UMaterialInterface::GetMaxWorldPositionOffsetDisplacement() const
{
	return 0.0f;
}

bool UMaterialInterface::ShouldAlwaysEvaluateWorldPositionOffset() const
{
	return false;
}

bool UMaterialInterface::HasPixelAnimation() const
{
	return false;
}

bool UMaterialInterface::IsDeferredDecal() const
{
	return false;
}

bool UMaterialInterface::IsUIMaterial() const
{
	return false;
}

bool UMaterialInterface::IsPostProcessMaterial() const
{
	return false;
}

bool UMaterialInterface::IsLightFunctionMaterial() const
{
	return false;
}

bool UMaterialInterface::GetCastDynamicShadowAsMasked() const
{
	return false;
}

bool UMaterialInterface::HasVertexInterpolator() const
{
	return false;
}

bool UMaterialInterface::HasCustomizedUVs() const
{
	return false;
}

bool UMaterialInterface::WritesToRuntimeVirtualTexture() const
{
	return false;
}

bool UMaterialInterface::HasMeshPaintTexture() const
{
	return false;
}

bool UMaterialInterface::HasCustomPrimitiveData() const
{
	return false;
}

FMaterialShadingModelField UMaterialInterface::GetShadingModels() const
{
	return MSM_DefaultLit;
}

bool UMaterialInterface::IsShadingModelFromMaterialExpression() const
{
	return false;
}

USubsurfaceProfile* UMaterialInterface::GetSubsurfaceProfile_Internal() const
{
	return nullptr;
}

USubsurfaceProfile* UMaterialInterface::GetSubsurfaceProfileRoot_Internal() const
{
	return nullptr;
}

USubsurfaceProfile* UMaterialInterface::GetSubsurfaceProfileRoot_Internal(uint32 Index) const
{
	return nullptr;
}

USubsurfaceProfile* UMaterialInterface::GetSubsurfaceProfileOverride_Internal() const
{
	return nullptr;
}

USpecularProfile* UMaterialInterface::GetSpecularProfile_Internal(uint32 Index) const
{
	return nullptr;
}

USpecularProfile* UMaterialInterface::GetSpecularProfileOverride_Internal() const
{
	return nullptr;
}

UToonProfile* UMaterialInterface::GetToonProfile_Internal(uint32 Index) const
{
	return nullptr;
}

UToonProfile* UMaterialInterface::GetToonProfileOverride_Internal() const
{
	return nullptr;
}

UNeuralProfile* UMaterialInterface::GetNeuralProfile_Internal() const
{
	return nullptr;
}

uint32 UMaterialInterface::NumSubsurfaceProfileRoot_Internal() const
{
	return 0u;
}

uint32 UMaterialInterface::NumSpecularProfile_Internal() const
{
	return 0u;
}

uint32 UMaterialInterface::NumToonProfile_Internal() const
{
	return 0u;
}

bool UMaterialInterface::CastsRayTracedShadows() const
{
	return true;
}

bool UMaterialInterface::IsTessellationEnabled() const
{
	return false;
}

bool UMaterialInterface::HasSubstrateRoughnessTracking() const
{
	return true;
}

bool UMaterialInterface::IsCompatibleWithLumenCardSharing() const
{
	return false;
}

void UMaterialInterface::SetFeatureLevelToCompile(ERHIFeatureLevel::Type FeatureLevel, bool bShouldCompile)
{
	uint32 FeatureLevelBit = (1 << FeatureLevel);
	if (bShouldCompile)
	{
		FeatureLevelsToForceCompile |= FeatureLevelBit;
	}
	else
	{
		FeatureLevelsToForceCompile &= (~FeatureLevelBit);
	}
}

uint32 UMaterialInterface::FeatureLevelsForAllMaterials = 0;

void UMaterialInterface::SetGlobalRequiredFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bShouldCompile)
{
	uint32 FeatureLevelBit = (1 << FeatureLevel);
	if (bShouldCompile)
	{
		FeatureLevelsForAllMaterials |= FeatureLevelBit;
	}
	else
	{
		FeatureLevelsForAllMaterials &= (~FeatureLevelBit);
	}
}


uint32 UMaterialInterface::GetFeatureLevelsToCompileForRendering() const
{
	return FeatureLevelsToForceCompile | GetFeatureLevelsToCompileForAllMaterials();
}


void UMaterialInterface::UpdateMaterialRenderProxy(FMaterialRenderProxy& Proxy)
{
	// no 0 pointer
	check(&Proxy);

	FMaterialShadingModelField MaterialShadingModels = GetShadingModels();

	// For better performance we only update SubsurfaceProfileRT if the feature is used and substrate is not enabled
	// When Substrate is enabled, this is ONLY used as an override for Subsurface Profile on material instance (override all Subsurface Profiles at once for now)
	if (UseSubsurfaceProfile(MaterialShadingModels) && !Substrate::IsSubstrateEnabled())
	{
		USubsurfaceProfile* LocalSubsurfaceProfile = GetSubsurfaceProfile_Internal();
		
		FSubsurfaceProfileStruct Settings;
		if (LocalSubsurfaceProfile)
		{
			Settings = LocalSubsurfaceProfile->Settings;
		}

		FMaterialRenderProxy* InProxy = &Proxy;
		ENQUEUE_RENDER_COMMAND(UpdateMaterialRenderProxySubsurface)(
		[Settings, LocalSubsurfaceProfile, InProxy](FRHICommandListImmediate& RHICmdList)
		{
			if (LocalSubsurfaceProfile)
			{
				const uint32 AllocationId = GSubsurfaceProfileTextureObject.AddOrUpdateProfile(Settings, LocalSubsurfaceProfile);
				check(AllocationId >= 0 && AllocationId < MAX_SUBSURFACE_PROFILE_COUNT);
			}
			InProxy->SetSubsurfaceProfileRT(LocalSubsurfaceProfile/*, ParameterName */); // how to have a unique identifier?
		});
	}

	UMaterial* Material = GetMaterial();
	if (Material && Material->IsPostProcessMaterial())
	{
		struct FEntry
		{
			UNeuralProfile* Profile = nullptr;
			FNeuralProfileStruct Setting;
			FGuid Guid;
		};
		
		UNeuralProfile* LocalNeuralProfile = GetNeuralProfile_Internal();
		
		if (LocalNeuralProfile)
		{
			FEntry Entry;
			Entry.Profile = LocalNeuralProfile;
			Entry.Setting = LocalNeuralProfile->Settings;
			Entry.Guid = LocalNeuralProfile->Guid;

			FMaterialRenderProxy* InProxy = &Proxy;
			ENQUEUE_RENDER_COMMAND(UpdateMaterialRenderProxyNNEModelData)(
				[LocalNeuralProfile, InProxy, Entry, Material](FRHICommandListImmediate& RHICmdList)
				{
					
					const uint32 AllocationId = NeuralProfile::AddOrUpdateProfile(Entry.Profile, Entry.Guid, Entry.Setting);
					check(AllocationId >= 0 && AllocationId < MAX_NEURAL_PROFILE_COUNT);

					Material->NeuralProfileId = AllocationId;
					
					InProxy->SetNeuralProfileRT(LocalNeuralProfile);
				});
		}
	}

	if (Substrate::IsSubstrateEnabled())
	{
		FMaterialRenderProxy* InProxy = &Proxy;

		// Specular profiles
		{
			struct FEntry
			{
				USpecularProfile* Profile = nullptr;
				FSpecularProfileStruct Settings;
				const FTextureReference* Texture = nullptr;
				FGuid Guid;
			};
			TArray<FEntry> Entries;
			for (int32 It = 0, Count = NumSpecularProfile_Internal(); It<Count; ++It)
			{
				FEntry& Entry = Entries.AddDefaulted_GetRef();
				Entry.Profile = GetSpecularProfile_Internal(It);
				if (Entry.Profile)
				{
					Entry.Settings 	= Entry.Profile->Settings;
					Entry.Guid 		= Entry.Profile->Guid;	
					if (!Entry.Settings.IsProcedural())
					{
						Entry.Texture = &Entry.Profile->Settings.Texture->TextureReference;
					}
				}
			}

			USpecularProfile* SpecularProfileOverride = GetSpecularProfileOverride_Internal();
			if (Entries.Num() > 0 || SpecularProfileOverride)
			{
				ENQUEUE_RENDER_COMMAND(UpdateMaterialRenderProxySpecular)(
				[InProxy, Entries, SpecularProfileOverride](FRHICommandListImmediate& RHICmdList)
				{
					for (const FEntry& Entry : Entries)
					{
						if (Entry.Profile)
						{
							const uint32 AllocationId = SpecularProfile::AddOrUpdateProfile(Entry.Profile, Entry.Guid, Entry.Settings, Entry.Texture);
							check(AllocationId >= 0 && AllocationId < MAX_SPECULAR_PROFILE_COUNT);
						}
						InProxy->AddSpecularProfileRT(Entry.Profile);
					}

					if (SpecularProfileOverride)
					{
						const FTextureReference* SPOverrideTexture = nullptr;
						if (!SpecularProfileOverride->Settings.IsProcedural() && SpecularProfileOverride->Settings.Texture)
						{
							SPOverrideTexture = &SpecularProfileOverride->Settings.Texture->TextureReference;
						}

						SpecularProfile::AddOrUpdateProfile(SpecularProfileOverride, SpecularProfileOverride->Guid, SpecularProfileOverride->Settings, SPOverrideTexture);
					}

					// Set the specular profile override using the default SpecularProfile pointer.
					InProxy->SetSpecularProfileOverrideRT(SpecularProfileOverride);
				});
			}
		}

		// Subsurface profiles
		{
			struct FEntry
			{
				USubsurfaceProfile* Profile = nullptr;
				FSubsurfaceProfileStruct Settings;
			};
			TArray<FEntry> Entries;
			for (int32 It = 0, Count = NumSubsurfaceProfileRoot_Internal(); It < Count; ++It)
			{
				FEntry& Entry = Entries.AddDefaulted_GetRef();
				Entry.Profile = GetSubsurfaceProfileRoot_Internal(It);
				if (Entry.Profile)
				{
					Entry.Settings = Entry.Profile->Settings;
				}
			}
			// For legacy materials not converted to Substrate, the SSS profiles array is not populated. 
			// SSS is then retrieve from the legacy path
			if (Entries.IsEmpty() && GetSubsurfaceProfile_Internal() != nullptr)
			{
				FEntry& Entry = Entries.AddDefaulted_GetRef();
				Entry.Profile = GetSubsurfaceProfile_Internal();
				if (Entry.Profile)
				{
					Entry.Settings = Entry.Profile->Settings;
				}
			}

			USubsurfaceProfile* SubsurfaceProfileOverride = GetSubsurfaceProfileOverride_Internal();
			FSubsurfaceProfileStruct SubsurfaceProfileOverrideSettings;
			if (SubsurfaceProfileOverride)
			{
				SubsurfaceProfileOverrideSettings = SubsurfaceProfileOverride->Settings;
			}

			ENQUEUE_RENDER_COMMAND(UpdateMaterialRenderProxySubsurfaceProfiles)(
			[InProxy, Entries, SubsurfaceProfileOverride, SubsurfaceProfileOverrideSettings](FRHICommandListImmediate& RHICmdList)
			{
				InProxy->ClearSubsurfaceProfileRT(); // Clear to not add duplicates when updating from the editor.
				for (const FEntry& Entry : Entries)
				{
					if (Entry.Profile)
					{
						const uint32 AllocationId = GSubsurfaceProfileTextureObject.AddOrUpdateProfile(Entry.Settings, Entry.Profile);
						check(AllocationId >= 0 && AllocationId < MAX_SUBSURFACE_PROFILE_COUNT);
					}
					InProxy->AddSubsurfaceProfileRT(Entry.Profile);
				}

				if (SubsurfaceProfileOverride)
				{
					GSubsurfaceProfileTextureObject.AddOrUpdateProfile(SubsurfaceProfileOverrideSettings, SubsurfaceProfileOverride);
				}

				// Set the subsurface profile override using the default SubsurfaceProfile pointer.
				InProxy->SetSubsurfaceProfileRT(SubsurfaceProfileOverride);
			});
		}

		// Toon profiles
		{
			struct FEntry
			{
				UToonProfile* Profile = nullptr;
				FToonProfileStruct Settings;
				FGuid Guid;
			};
			TArray<FEntry> Entries;
			for (int32 It = 0, Count = NumToonProfile_Internal(); It < Count; ++It)
			{
				FEntry& Entry = Entries.AddDefaulted_GetRef();
				Entry.Profile = GetToonProfile_Internal(It);
				if (Entry.Profile)
				{
					Entry.Settings = Entry.Profile->Settings;
					Entry.Guid = Entry.Profile->Guid;
				}
			}

			UToonProfile* ToonProfileOverride = GetToonProfileOverride_Internal();
			if (Entries.Num() > 0 || ToonProfileOverride)
			{
				ENQUEUE_RENDER_COMMAND(UpdateMaterialRenderProxyToon)(
					[InProxy, Entries, ToonProfileOverride](FRHICommandListImmediate& RHICmdList)
					{
						for (const FEntry& Entry : Entries)
						{
							if (Entry.Profile)
							{
								const uint32 AllocationId = ToonProfile::AddOrUpdateProfile(Entry.Profile, Entry.Guid, Entry.Settings);
								check(AllocationId >= 0 && AllocationId < MAX_TOON_PROFILE_COUNT);
							}
							InProxy->AddToonProfileRT(Entry.Profile);
						}

						if (ToonProfileOverride)
						{
							ToonProfile::AddOrUpdateProfile(ToonProfileOverride, ToonProfileOverride->Guid, ToonProfileOverride->Settings);
						}

						// Set the Toon profile override using the default ToonProfile pointer.
						InProxy->SetToonProfileOverrideRT(ToonProfileOverride);
					});
			}
		}
	}
}

#if WITH_EDITOR
bool UMaterialInterface::ShouldStripUnusedDefaultTextures()
{
	return GShouldStripUnusedDefaultTextures;
}

TArray<const FMaterialResource*, TInlineAllocator<4>> UMaterialInterface::GetCookedCacheMaterialResources() const
{
	TArray<const FMaterialResource*, TInlineAllocator<4>> CookedResources;
	for (const auto& KVP : CachedMaterialResourcesForCooking)
	{
		const TArray<FMaterialResourceForCooking>& PlatformResources = KVP.Value;

		for (const FMaterialResourceForCooking& ResForCooking : PlatformResources)
		{
			if (ResForCooking.Resource)
			{
				CookedResources.Add(ResForCooking.Resource);
			}
		}
	}

	return CookedResources;
}

void UMaterialInterface::StripUnusedDefaultTextures(const bool bInKeepProjectPrivateTextures)
{
	SCOPE_CYCLE_COUNTER(STAT_MaterialInterface_StripUnusedDefaultTextures);

	if (!ShouldStripUnusedDefaultTextures())
	{
		static bool bWarnedOnce = false;
		if (!bWarnedOnce)
		{
			UE_LOGF(LogMaterial, Warning, "Calling StripUnusedDefaultTextures even though ShouldStripUnusedDefaultTextures returns false.");
			bWarnedOnce = true;
		}

		return;
	}

	TArray<TObjectPtr<UObject>> UsedTextures;
	const bool bValidUsedTextures = TryGetUsedTexturesInCookCache(UsedTextures);

	if (!bValidUsedTextures)
	{
		if (!bInKeepProjectPrivateTextures)
		{
			// Textures will be kept in Parent material
			ReferencedDefaultTextures.Empty();
		}

		return;
	}

	for (int32 Idx = 0; Idx < ReferencedDefaultTextures.Num(); ++Idx)
	{
		const TObjectPtr<UObject> CurrentTexture = ReferencedDefaultTextures[Idx];

		if (!CurrentTexture)
		{
			continue;
		}

		// Check for project private textures (PKC_NotExternallyReferenceable)
		const UPackage* TexturePackage = CurrentTexture->GetPackage();
		const UPackage* OwnPackage = GetPackage();

		if (!TexturePackage || !OwnPackage)
		{
			continue;
		}

		const bool bUsedTexture = UsedTextures.Contains(CurrentTexture);
		bool bRemoveTexture = !bUsedTexture;
		const bool bIsPackagePrivate = TexturePackage->GetAssetAccessSpecifier() == EAssetAccessSpecifier::Private;

		if (!bInKeepProjectPrivateTextures)
		{
			// We are in a Material Instance, check for private package and mount points
			if (bIsPackagePrivate)
			{
				const FName OwnMountPointName = FPackageName::GetPackageMountPoint(FNameBuilder(OwnPackage->GetFName()).ToView());
				const FName TextureMountPointName = FPackageName::GetPackageMountPoint(FNameBuilder(TexturePackage->GetFName()).ToView());

				if (!OwnMountPointName.IsNone() && !TextureMountPointName.IsNone())
				{
					if (OwnMountPointName != TextureMountPointName)
					{
						// Private package, different mount points -> remove texture
						bRemoveTexture = true;
					}
				}
				else
				{
					// Private package and None mount points -> remove texture
					bRemoveTexture = true;
				}
			}
		}
		else
		{
			// We are in a Material, we only remove if the package is not private
			bRemoveTexture &= !bIsPackagePrivate;
		}

		if (bRemoveTexture)
		{
			ReferencedDefaultTextures[Idx] = nullptr;
		}
	}
}
#endif // WITH_EDITOR

bool FMaterialTextureInfo::IsValid(bool bCheckTextureIndex) const
{ 
#if WITH_EDITORONLY_DATA
	if (bCheckTextureIndex && (TextureIndex < 0 || TextureIndex >= TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL))
	{
		return false;
	}
#endif
	return TextureName != NAME_None && SamplingScale > UE_SMALL_NUMBER && UVChannelIndex >= 0 && UVChannelIndex < TEXSTREAM_MAX_NUM_UVCHANNELS;
}

void UMaterialInterface::SortTextureStreamingData(bool bForceSort, bool bFinalSort)
{
#if WITH_EDITOR
	UE::TUniqueLock Lock(TextureStreamingDataMutex);

	// In cook that was already done in the save.
	if (!bTextureStreamingDataSorted || bForceSort)
	{
		TSet<const UTexture*> UsedTextures;
		if (bFinalSort)
		{
			TSet<const UTexture*> UnfilteredUsedTextures;
			GetReferencedTexturesAndOverrides(UnfilteredUsedTextures);

			// Sort some of the conditions that could make the texture unstreamable, to make the data leaner.
			// Note that because we are cooking, UStreamableRenderAsset::bIsStreamable is not reliable here.
			for (const UTexture* UnfilteredTexture : UnfilteredUsedTextures)
			{
				if (UnfilteredTexture && UnfilteredTexture->IsPossibleToStream() )
				{
					UsedTextures.Add(UnfilteredTexture);
				}
			}
		}

		for (int32 Index = 0; Index < TextureStreamingData.Num(); ++Index)
		{
			FMaterialTextureInfo& TextureData = TextureStreamingData[Index];
			UTexture* Texture = Cast<UTexture>(TextureData.TextureReference.ResolveObject());

			// Also, when cooking, only keep textures that are directly referenced by this material to prevent non-deterministic cooking.
			// This would happen if a texture reference resolves to a texture not used anymore by this material. The resolved object could then be valid or not.
			if (Texture && (!bFinalSort || UsedTextures.Contains(Texture)))
			{
				TextureData.TextureName = Texture->GetFName();
			}
			else if (bFinalSort) // In the final sort we remove null names as they will never match.
			{
				TextureStreamingData.RemoveAtSwap(Index);
				--Index;
			}
			else
			{
				TextureData.TextureName = NAME_None;
			}
		}

		// Sort by name to be compatible with FindTextureStreamingDataIndexRange
		// Use stable sort to preserve order and avoid non-determinism from multi-platform cooks.
		TextureStreamingData.StableSort([](const FMaterialTextureInfo& Lhs, const FMaterialTextureInfo& Rhs) 
		{ 
			// Sort by register indices when the name are the same, as when initially added in the streaming data.
			if (Lhs.TextureName == Rhs.TextureName)
			{
				return Lhs.TextureIndex < Rhs.TextureIndex;

			}
			return Lhs.TextureName.LexicalLess(Rhs.TextureName); 
		});
		bTextureStreamingDataSorted = true;
	}
#endif // WITH_EDITOR
}

extern 	TAutoConsoleVariable<int32> CVarStreamingUseMaterialData;

bool UMaterialInterface::FindTextureStreamingDataIndexRange(FName TextureName, int32& LowerIndex, int32& HigherIndex) const
{
#if WITH_EDITOR
	UE::TUniqueLock Lock(TextureStreamingDataMutex);
#endif

#if WITH_EDITORONLY_DATA
	// Because of redirectors (when textures are renammed), the texture names might be invalid and we need to udpate the data at every load.
	// Normally we would do that in the post load, but since the process needs to resolve the SoftObjectPaths, this is forbidden at that place.
	// As a workaround, we do it on demand. Note that this is not required in cooked build as it is done in the presave.
	const_cast<UMaterialInterface*>(this)->SortTextureStreamingData(false, false);
#endif

	if (CVarStreamingUseMaterialData.GetValueOnGameThread() == 0 || CVarStreamingUseNewMetrics.GetValueOnGameThread() == 0)
	{
		return false;
	}

	const int32 NumTextureSteamingData = TextureStreamingData.Num();
	for (int32 Index = 0; Index < NumTextureSteamingData; ++Index)
	{
		if (TextureStreamingData[Index].TextureName == TextureName)
		{
			// Find the range of entries for this texture. 
			// This is possible because the same texture could be bound to several register and also be used with different sampling UV.
			LowerIndex = Index;
			while (Index + 1 < NumTextureSteamingData && TextureStreamingData[Index + 1].TextureName == TextureName)
			{
				++Index;
			}
			HigherIndex = Index;
			return true;
		}
	}
	return false;
}

void UMaterialInterface::SetTextureStreamingData(const TArray<FMaterialTextureInfo>& InTextureStreamingData)
{
#if WITH_EDITOR
	UE::TUniqueLock Lock(TextureStreamingDataMutex);
#endif

	TextureStreamingData = InTextureStreamingData;
#if WITH_EDITORONLY_DATA
	bTextureStreamingDataSorted = false;
	TextureStreamingDataVersion = InTextureStreamingData.Num() ? MATERIAL_TEXTURE_STREAMING_DATA_VERSION : 0;
	TextureStreamingDataMissingEntries.Empty();
#endif
	SortTextureStreamingData(true, false);
}

float FTextureSamplingInfo::GetTextureDensity(const FMeshUVChannelInfo& UVChannelData) const
{
	float TextureDensity = 0.0f;
	for (int32 Index = 0; Index < MAX_TEXCOORDS; ++Index)
	{
		const float SamplingScale = ChannelMinSamplingScale[Index];
		TextureDensity = FMath::Max<float>(UVChannelData.LocalUVDensities[Index] / SamplingScale, TextureDensity);
	}
	return FMath::IsNearlyZero(TextureDensity) ? 0.0f : TextureDensity;
}

float UMaterialInterface::GetTextureDensity(FName TextureName, const FMeshUVChannelInfo& UVChannelData) const
{
#if WITH_EDITOR
	UE::TUniqueLock Lock(TextureStreamingDataMutex);
#endif

	ensure(UVChannelData.bInitialized);

	int32 LowerIndex = INDEX_NONE;
	int32 HigherIndex = INDEX_NONE;
	if (FindTextureStreamingDataIndexRange(TextureName, LowerIndex, HigherIndex))
	{
		// Compute the max, at least one entry will be valid. 
		float MaxDensity = 0;
		for (int32 Index = LowerIndex; Index <= HigherIndex; ++Index)
		{
			const FMaterialTextureInfo& MatchingData = TextureStreamingData[Index];
			ensure(MatchingData.IsValid() && MatchingData.TextureName == TextureName);
			MaxDensity = FMath::Max<float>(UVChannelData.LocalUVDensities[MatchingData.UVChannelIndex] / MatchingData.SamplingScale, MaxDensity);
		}
		return MaxDensity;
	}

	// Otherwise return 0 to indicate the data is not found.
	return 0;
}

uint32 UMaterialInterface::GetFeatureLevelsToCompileForAllMaterials()
{
	return FeatureLevelsForAllMaterials | (1 << GMaxRHIFeatureLevel);
}

bool UMaterialInterface::UseAnyStreamingTexture() const
{
	TArray<UTexture*> Textures;
	GetUsedTextures(Textures);

	for (UTexture* Texture : Textures)
	{
		if (Texture && Texture->IsStreamable())
		{
			return true;
		}
	}
	return false;
}

void UMaterialInterface::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
	const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
	if (TargetPlatform && TargetPlatform->RequiresCookedData())
	{
		SortTextureStreamingData(true, true);
	}

#if WITH_EDITORONLY_DATA
	// Make sure the EditorOnlyData is named correctly. Some MI assets have a differently named editor only data that can cause problems in the editor running on cooked data.
	FString EditorOnlyDataName = UE::MaterialInterface::GetEditorOnlyDataName(*GetName());
	if (!ObjectSaveContext.IsProceduralSave())
	{
		SafeRenameEditorOnlyData(this, EditorOnlyDataName, true);
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	if (ObjectSaveContext.IsCooking())
	{
		GShaderCompilerStats->IncrementMaterialCook();
	}
#endif // WITH_EDITOR
}

void UMaterialInterface::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UMaterialInterface::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void UMaterialInterface::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

void UMaterialInterface::EnsureIsComplete()
{
	// TODO:
	// The commented out code is roughly what we want to do to make the shaders ready. However this currently breaks for 
	// material instances, where IsComplete isn't checking for readiness of the parent, and seems to not work even when
	// CacheShaders is called on the parent. For now, to avoid issues with user blueprints failing on their first
	// draw with on demand shader compilation, we do the more direct requests below, which seem to be more reliable.
	
	//if (!IsComplete())
	//{
	//	FScopedSlowTask SlowTask(0.0f, LOCTEXT("CacheShaders", "Caching Shaders..."));
	//	SlowTask.MakeDialog();
	//	CacheShaders(EMaterialShaderPrecompileMode::Synchronous);
	//}

#if WITH_EDITOR
	uint32 FeatureLevelsToCompile = GetFeatureLevelsToCompileForRendering();
	while (FeatureLevelsToCompile != 0)
	{
		const ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevelsToCompile);
		FMaterialResource* MaterialResource = GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
		if (MaterialResource && !MaterialResource->IsGameThreadShaderMapComplete())
		{
			MaterialResource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::ForceLocal);
			MaterialResource->FinishCompilation(); // this is WITH_EDITOR only
		}
	}
#endif
}

void UMaterialInterface::FilterOutPlatformShadingModels(const EShaderPlatform InPlatform, FMaterialShadingModelField& ShadingModels)
{
	const FStaticShaderPlatform Platform(InPlatform);
	if (ShadingModels.CountShadingModels() > 1 && !AllowPerPixelShadingModels(Platform))
	{
		ShadingModels = FMaterialShadingModelField(MSM_DefaultLit);
	}

	uint32 ShadingModelsMask = GetPlatformShadingModelsMask(Platform);
	if (ShadingModelsMask != 0xFFFFFFFF)
	{
		uint16 FilteredShadingModels = (ShadingModels.GetShadingModelField() & ShadingModelsMask);
		ShadingModels.SetShadingModelField(FilteredShadingModels);
		if (!ShadingModels.IsValid())
		{
			ShadingModels = FMaterialShadingModelField(MSM_DefaultLit);
		}
	}
}

FInt32Range UMaterialInterface::BuildQualityLevelRange(TOptional<EMaterialQualityLevel::Type>  InQualityLevel)
{
	FInt32Range QualityLevelRange(0, EMaterialQualityLevel::Num - 1);

	if (InQualityLevel.IsSet())
	{
		EMaterialQualityLevel::Type QualityLevel = InQualityLevel.GetValue();
		ensure(QualityLevel != EMaterialQualityLevel::Num);

		if (QualityLevel == EMaterialQualityLevel::Num)
		{
			QualityLevel = GetCurrentMaterialQualityLevelChecked();
		}
		QualityLevelRange = FInt32Range(QualityLevel, QualityLevel);
	}

	return QualityLevelRange;
}

FInt32Range UMaterialInterface::BuildShaderPlatformRange(TOptional<EShaderPlatform> InShaderPlatform)
{
#if WITH_EDITOR
	FInt32Range ShaderPlatformRange(0, EShaderPlatform::SP_NumPlatforms - 1);

	if (InShaderPlatform.IsSet())
	{
		EShaderPlatform ShaderPlatform = InShaderPlatform.GetValue();
		ensure(ShaderPlatform != EShaderPlatform::SP_NumPlatforms);

		if (ShaderPlatform == EShaderPlatform::SP_NumPlatforms)
		{
			ShaderPlatform = GetMaxShaderPlatformChecked();
		}
		ShaderPlatformRange = FInt32Range(ShaderPlatform, ShaderPlatform);
	}
#else
	FInt32Range ShaderPlatformRange(GMaxRHIShaderPlatform, GMaxRHIShaderPlatform);
#endif

	return ShaderPlatformRange;
}

#if WITH_EDITORONLY_DATA


const UClass* UMaterialInterface::GetEditorOnlyDataClass() const
{
	return UMaterialInterfaceEditorOnlyData::StaticClass();
}

#endif // WITH_EDITORONLY_DATA

bool UMaterialInterface::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	bool bRenamed = Super::Rename(NewName, NewOuter, Flags);
#if WITH_EDITORONLY_DATA
	// if we have EditorOnlyData, also rename it if we are changing the material's name
	if (bRenamed && NewName && EditorOnlyData)
	{
		FString EditorOnlyDataName = UE::MaterialInterface::GetEditorOnlyDataName(NewName);
		bRenamed = SafeRenameEditorOnlyData(this, EditorOnlyDataName, false);
	}
#endif
	return bRenamed;
}

namespace UE::Private
{
	bool GEnableTexturesSamplingStreamingCache = false;
	static FAutoConsoleVariableRef CVarEnableTexturesSamplingStreamingCache(
		TEXT("r.Streaming.EnableTexturesSamplingStreamingCache"),
		GEnableTexturesSamplingStreamingCache,
		TEXT("Enables caching of material texture sampling. Used by FSimpleStreamableAssetManager's primary gather path (FStaticMeshSceneProxy / Nanite::FSceneProxy GetStreamableRenderAssetInfo) and by texture streaming in a cooked build. The cache is automatically enabled when FSimpleStreamableAssetManager is enabled.\n")
		TEXT("Read-only - must be set before the first UMaterialInterface is constructed, because the per-material lock guarding the cache is only allocated when this is enabled."),
		ECVF_ReadOnly);
}

bool UMaterialInterface::IsTexturesSamplingCacheGloballyEnabled()
{
	/* During cooking we do not need a cache, and we cannot relay on GetMaxShaderPlatformChecked
	 * It should be possible to cache the data during the cooking, but we have to do it for TargetPlatform */
	if (IsRunningCookCommandlet())
	{
		return false;
	}
	return UE::Private::GEnableTexturesSamplingStreamingCache || FSimpleStreamableAssetManager::IsEnabled();
}

bool UMaterialInterface::CanCacheTexturesSamplingInfo() const
{
	return IsTexturesSamplingCacheGloballyEnabled();
}

TSharedPtr<FMaterialCachedTexturesSamplingInfo> UMaterialInterface::ComputeCachedTexturesSamplingInfo() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialInterface::ComputeCachedTexturesSamplingInfo);

	check(IsInGameThread() || IsInParallelGameThread());

	TSharedPtr<FMaterialCachedTexturesSamplingInfo> Cache = MakeShared<FMaterialCachedTexturesSamplingInfo>();

	const EShaderPlatform ShaderPlatform = GetMaxShaderPlatformChecked();
	FMaterialCachedTexturesSamplingInfo::FShaderPlatformMapping& ShaderPlatformMapping = Cache->ShaderPlatformMapping;

	// TextureToInfoIndex dedups FTextureSamplingInfo entries across quality levels. CalculateTexturesSamplingInfo
	// is quality-level-independent so the same UTexture* always yields the same Info; keying dedup on the
	// texture pointer avoids the O(n^2) TArray::AddUnique walk for materials with many textures.
	TMap<const UTexture*, int32> TextureToInfoIndex;

	for (EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Low; QualityLevel < EMaterialQualityLevel::Num; QualityLevel = static_cast<EMaterialQualityLevel::Type>(QualityLevel + 1))
	{
		TArray<uint16>& Mapping = ShaderPlatformMapping.QualityMapping[QualityLevel];
		TArray<UTexture*> Textures;
		GetUsedTextures(Textures, QualityLevel, ShaderPlatform);

		for (UTexture* Texture : Textures)
		{
			/* We will only cache the sampling information for Textures that support streaming
			 * to avoid timing issue instead of relying on dynamic IsStreamable that uses registration status of asset
			 * we will rely on RenderResourceSupportsStreaming that is based on asset properties
			 */
			if (!Texture->RenderResourceSupportsStreaming())
			{
				continue;
			}

			int32 Index = INDEX_NONE;
			if (const int32* ExistingIndex = TextureToInfoIndex.Find(Texture))
			{
				Index = *ExistingIndex;
			}
			else
			{
				// If the texture is referenced but not sampled it will not have an entry
				FTextureSamplingInfo Info{Texture};
				if (!CalculateTexturesSamplingInfo(Texture, Info))
				{
					continue;
				}
				Index = Cache->TextureSamplingInfos.Add(MoveTemp(Info));
				check(Index < TNumericLimits<uint16>::Max());
				TextureToInfoIndex.Add(Texture, Index);
			}

			Mapping.AddUnique(Index);
		}
	}
	return Cache;
}

void UMaterialInterface::CacheTexturesSamplingInfo()
{
	check(IsInGameThread() || IsInParallelGameThread());

	if (!CanCacheTexturesSamplingInfo())
	{
		return;
	}

	// Compute outside the lock; holding the write lock across GetUsedTextures would stall every reader.
	TSharedPtr<FMaterialCachedTexturesSamplingInfo> NewCache = ComputeCachedTexturesSamplingInfo();

	UE::TWriteScopeLock<FCacheRWLock> WriteLock(CachedTexturesSamplingInfoLock);
	CachedTexturesSamplingInfo = MoveTemp(NewCache);
}

TSharedPtr<const FMaterialCachedTexturesSamplingInfo> UMaterialInterface::GetCachedTexturesSamplingInfo() const
{
	// Cache disabled via cvar or in a cook commandlet. Callers tolerate null -- typically by omitting the
	// material from their OutStreamableRenderAssets report, gated with a once-only ensure at the callsite.
	if (!CanCacheTexturesSamplingInfo())
	{
		return nullptr;
	}

	// Fast path: cache already populated. Copy the shared pointer out under the read lock and return.
	{
		UE::TReadScopeLock<FCacheRWLock> ReadLock(CachedTexturesSamplingInfoLock);
		if (CachedTexturesSamplingInfo)
		{
			return CachedTexturesSamplingInfo;
		}
	}

	// Slow path: populate on demand. Populating walks GetUsedTextures + CalculateTexturesSamplingInfo,
	// which are safe on GT/parallel-GT but not on a generic worker. A worker that arrives here before
	// PostLoad / UpdateCachedData has proactively populated the cache falls through to nullptr and
	// the caller omits this material from its streaming report.
	if (!(IsInGameThread() || IsInParallelGameThread()))
	{
		return nullptr;
	}

	TSharedPtr<FMaterialCachedTexturesSamplingInfo> NewCache = ComputeCachedTexturesSamplingInfo();

	// Double-check-locked publish. A parallel-GT caller may have populated while we computed; in that
	// case keep the existing cache and discard the one we just built.
	UE::TWriteScopeLock<FCacheRWLock> WriteLock(CachedTexturesSamplingInfoLock);
	if (!CachedTexturesSamplingInfo)
	{
		CachedTexturesSamplingInfo = MoveTemp(NewCache);
	}
	return CachedTexturesSamplingInfo;
}

bool UMaterialInterface::CalculateTexturesSamplingInfo(UTexture* Texture, FTextureSamplingInfo& OutTextureSamplingInfo) const
{
	OutTextureSamplingInfo = FTextureSamplingInfo(Texture);
	int32 LowerIndex = INDEX_NONE;
	int32 UpperIndex = INDEX_NONE;

	bool bIsSampled = false;

	if (FindTextureStreamingDataIndexRange(Texture->GetFName(), LowerIndex, UpperIndex))
	{
		for (int32 Index = LowerIndex; Index <= UpperIndex; ++Index)
		{
			const FMaterialTextureInfo& Info = TextureStreamingData[Index];

			// Reject non-finite or non-positive sampling scales at source. NaN propagating into
			// ChannelMinSamplingScale would make FTextureSamplingInfo::operator==(default) report
			// distinct for every copy (NaN != NaN bitwise). A zero or near-zero scale would survive
			// IsFinite but produce divide-by-zero (LocalUVDensities[i] / 0.0f -> +Inf) inside
			// FTextureSamplingInfo::GetTextureDensity, tainting Info.TexelFactor with Inf and
			// failing the checkSlow in FMaterialCachedTexturesSamplingInfo::GetStreamingRenderAssetPrimitiveInfo.
			// Predicate matches FMaterialTextureInfo::IsValid()'s SamplingScale > UE_SMALL_NUMBER.
			if (!FMath::IsFinite(Info.SamplingScale) || Info.SamplingScale <= UE_SMALL_NUMBER)
			{
				continue;
			}

			OutTextureSamplingInfo.ChannelMinSamplingScale[Info.UVChannelIndex] = FMath::Min<float>(Info.SamplingScale, OutTextureSamplingInfo.ChannelMinSamplingScale[Info.UVChannelIndex]);
			bIsSampled = true;
		}
	}
	return bIsSampled;
}

UMaterialInterfaceEditorOnlyData::UMaterialInterfaceEditorOnlyData()
{
}

UMaterialInterfaceEditorOnlyData::UMaterialInterfaceEditorOnlyData(FVTableHelper& Helper)
	: Super(Helper)
{
} 

UMaterialInterfaceEditorOnlyData::~UMaterialInterfaceEditorOnlyData()
{
}

void UMaterialInterfaceEditorOnlyData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	bool bSavedCachedExpressionData = false;
	if ((Ar.IsCooking() || (FPlatformProperties::RequiresCookedData() && Ar.IsSaving() && (Ar.GetPortFlags() & PPF_Duplicate))) && (bool)CachedExpressionData)
	{
		bSavedCachedExpressionData = true;
	}

	Ar << bSavedCachedExpressionData;

	if (bSavedCachedExpressionData)
	{
		// if we do not have our CachedExpressionData set at this point, 
		// it means this object's name doesn't match the default created object name and we need to fix our pointer into the material interface
		if (CachedExpressionData == nullptr)
		{
#if WITH_EDITORONLY_DATA
			UMaterialInterface* Owner = Cast<UMaterialInterface>(GetOuter());
			Owner->EditorOnlyData = this;
			CachedExpressionData = Owner->CachedExpressionData->EditorOnlyData;
#endif
		}

		check(CachedExpressionData);
		UScriptStruct* Struct = FMaterialCachedExpressionEditorOnlyData::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)CachedExpressionData.Get(), Struct, nullptr);
		bLoadedCachedExpressionData = true;
	}
}

#if WITH_EDITOR
void UMaterialInterface::SetMarkTextureAsEditorStreamingPool(bool bInMarkAsEditorStreamingPool)
{
	TArray<UTexture*> Textures;
	GetUsedTextures(Textures);

	for (UTexture* Texture : Textures)
	{
		Texture->bMarkAsEditorStreamingPool = bInMarkAsEditorStreamingPool;
	}
}
#endif

FString UMaterialInterface::GetUsageName(EMaterialUsage Usage)
{
	FString UsageName;
	switch (Usage)
	{
	case MATUSAGE_SkeletalMesh: UsageName = TEXT("SkeletalMesh"); break;
	case MATUSAGE_ParticleSprites: UsageName = TEXT("ParticleSprites"); break;
	case MATUSAGE_BeamTrails: UsageName = TEXT("BeamTrails"); break;
	case MATUSAGE_MeshParticles: UsageName = TEXT("MeshParticles"); break;
	case MATUSAGE_NiagaraSprites: UsageName = TEXT("NiagaraSprites"); break;
	case MATUSAGE_NiagaraRibbons: UsageName = TEXT("NiagaraRibbons"); break;
	case MATUSAGE_NiagaraMeshParticles: UsageName = TEXT("NiagaraMeshParticles"); break;
	case MATUSAGE_StaticLighting: UsageName = TEXT("StaticLighting"); break;
	case MATUSAGE_MorphTargets: UsageName = TEXT("MorphTargets"); break;
	case MATUSAGE_SplineMesh: UsageName = TEXT("SplineMeshes"); break;
	case MATUSAGE_InstancedStaticMeshes: UsageName = TEXT("InstancedStaticMeshes"); break;
	case MATUSAGE_GeometryCollections: UsageName = TEXT("GeometryCollections"); break;
	case MATUSAGE_Clothing: UsageName = TEXT("Clothing"); break;
	case MATUSAGE_GeometryCache: UsageName = TEXT("GeometryCache"); break;
	case MATUSAGE_Water: UsageName = TEXT("Water"); break;
	case MATUSAGE_HairStrands: UsageName = TEXT("HairStrands"); break;
	case MATUSAGE_LidarPointCloud: UsageName = TEXT("LidarPointCloud"); break;
	case MATUSAGE_VirtualHeightfieldMesh: UsageName = TEXT("VirtualHeightfieldMesh"); break;
	case MATUSAGE_Nanite: UsageName = TEXT("Nanite"); break;
	case MATUSAGE_Voxels: UsageName = TEXT("Voxels"); break;
	case MATUSAGE_Curves: UsageName = TEXT("Curves"); break;
	case MATUSAGE_VolumetricCloud: UsageName = TEXT("VolumetricCloud"); break;
	case MATUSAGE_HeterogeneousVolumes: UsageName = TEXT("HeterogeneousVolumes"); break;
	case MATUSAGE_StaticMesh: UsageName = TEXT("StaticMesh"); break;
	case MATUSAGE_EditorCompositing: UsageName = TEXT("EditorCompositing"); break;
	case MATUSAGE_NeuralNetworks: UsageName = TEXT("NeuralNetworks"); break;
	case MATUSAGE_MeshDeformer: UsageName = TEXT("MeshDeformer"); break;
	case MATUSAGE_InstancedSkinnedMesh: UsageName = TEXT("InstancedSkinnedMesh"); break;
	default: UE_LOGF(LogMaterial, Fatal, "Unknown material usage: %u", (int32)Usage);
	};
	return UsageName;
}

bool UMaterialInterface::CheckMaterialUsage(EMaterialUsage Usage)
{
	check(IsInGameThread());
	return SetMaterialUsage(Usage);
}

bool UMaterialInterface::CheckMaterialUsage_Concurrent(EMaterialUsage Usage) const
{
	const uint32 UsageFlagBit = (1u << Usage);

	bool bUsageSetSuccessfully = false;
	if (NeedsSetMaterialUsage_Concurrent(bUsageSetSuccessfully, Usage))
	{
		if (IsInGameThread())
		{
			bUsageSetSuccessfully = const_cast<UMaterialInterface*>(this)->CheckMaterialUsage(Usage);
		}
		else
		{
			const uint32 PrevCacheMissMask = std::atomic_fetch_or(&UsageFlagCacheMiss, UsageFlagBit);
			if ((PrevCacheMissMask & UsageFlagBit) == 0u)
			{
				struct FCallSMU
				{
					UMaterialInterface* Material;
					EMaterialUsage Usage;

					FCallSMU(UMaterialInterface* InMaterial, EMaterialUsage InUsage)
						: Material(InMaterial)
						, Usage(InUsage)
					{
					}

					void Task()
					{
						Material->CheckMaterialUsage(Usage);
					}
				};
				UE_LOGF(LogMaterial, Log, "Had to pass SMU back to game thread. Please fix material usage flag %ls on %ls", *GetUsageName(Usage), *GetPathNameSafe(this));

				TSharedRef<FCallSMU, ESPMode::ThreadSafe> CallSMU = MakeShareable(new FCallSMU(const_cast<UMaterialInterface*>(this), Usage));

				DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.CheckMaterialUsage"),
					STAT_FSimpleDelegateGraphTask_CheckMaterialUsage,
					STATGROUP_TaskGraphTasks);

				FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
					FSimpleDelegateGraphTask::FDelegate::CreateThreadSafeSP(CallSMU, &FCallSMU::Task),
					GET_STATID(STAT_FSimpleDelegateGraphTask_CheckMaterialUsage), NULL, ENamedThreads::GameThread_Local
				);
			}
		}
	}
	else if ((UsageFlagCacheMiss.load(std::memory_order_relaxed) & UsageFlagBit) != 0u)
	{
		std::atomic_fetch_and(&UsageFlagCacheMiss, ~UsageFlagBit);
	}

	return bUsageSetSuccessfully;
}

bool UMaterialInterface::NeedsSetMaterialUsage_Concurrent(bool& bOutHasUsage, EMaterialUsage Usage) const
{
	bOutHasUsage = false;

	// Material usage is only relevant for materials that can be applied onto a mesh / use with different vertex factories.
	UMaterial const* BaseMaterial = GetMaterial_Concurrent();
	if (BaseMaterial == nullptr)
	{
		return false;
	}

	const EMaterialDomain MaterialDomain = BaseMaterial->MaterialDomain;
	if (MaterialDomain != MD_Surface && MaterialDomain != MD_DeferredDecal && MaterialDomain != MD_Volume && MaterialDomain != MD_UI)
	{
		return false;
	}

	// Check that the material has been flagged for use with the given usage flag.
	if (!GetUsageByFlag(Usage) && !BaseMaterial->bUsedAsSpecialEngineMaterial)
	{
		const uint32 UsageFlagBit = (1u << Usage);
		if ((UsageFlagWarnings & UsageFlagBit) == 0)
		{
			// This will be overwritten later by SetMaterialUsage, since we are saying that it needs to be called with the return value
			return true;
		}
		else
		{
			// We have already warned about this, so we aren't going to warn or compile or set anything this time
			return false;
		}
	}
	
	bOutHasUsage = true;
	return false;
}

#undef LOCTEXT_NAMESPACE 
