// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialKeyGeneratorContext.h"

#if WITH_EDITOR
#include "RHIStrings.h"
#endif

#if WITH_EDITOR

FMaterialKeyGeneratorContext::FMaterialKeyGeneratorContext(
	TUniqueFunction<void(const void* Data, uint64 Size)>&& InResultFunc, EShaderPlatform InShaderPlatform)
	: KeyGen(MoveTemp(InResultFunc))
	, ShaderFormat(InShaderPlatform < EShaderPlatform::SP_NumPlatforms ? 
		LegacyShaderPlatformToShaderFormat(InShaderPlatform) : NAME_None)
	, ShaderPlatform(InShaderPlatform)
	, Mode(EMode::Emitting)
{
}

FMaterialKeyGeneratorContext::FMaterialKeyGeneratorContext(FString& InResultString, EShaderPlatform InShaderPlatform)
	: KeyGen(InResultString)
	, ShaderFormat(InShaderPlatform < EShaderPlatform::SP_NumPlatforms ? 
		LegacyShaderPlatformToShaderFormat(InShaderPlatform) : NAME_None)
	, ShaderPlatform(InShaderPlatform)
	, Mode(EMode::Emitting)
{
	InResultString.Reserve(16384);
}

FMaterialKeyGeneratorContext::FMaterialKeyGeneratorContext(FCbWriter& InWriter, EShaderPlatform InShaderPlatform)
	: Writer(&InWriter)
	, ShaderFormat(InShaderPlatform < EShaderPlatform::SP_NumPlatforms ? 
		LegacyShaderPlatformToShaderFormat(InShaderPlatform) : NAME_None)
	, ShaderPlatform(InShaderPlatform)
	, Mode(EMode::Saving)
{
}

FMaterialKeyGeneratorContext::FMaterialKeyGeneratorContext(FCbObjectView LoadRoot, EShaderPlatform InShaderPlatform)
	: ObjectStack()
	, ShaderFormat(InShaderPlatform < EShaderPlatform::SP_NumPlatforms ? 
		LegacyShaderPlatformToShaderFormat(InShaderPlatform) : NAME_None)
	, ShaderPlatform(InShaderPlatform)
	, Mode(EMode::Loading)
{
	ObjectStack.Add(LoadRoot);
}

FMaterialKeyGeneratorContext::~FMaterialKeyGeneratorContext()
{
	switch (Mode)
	{
	case EMode::Emitting:
		KeyGen.~FShaderKeyGenerator();
		break;
	case EMode::Saving:
		break;
	case EMode::Loading:
		ObjectStack.~TArray();
		break;
	default:
		checkNoEntry();
		break;
	}
}

FCbObjectView FMaterialKeyGeneratorContext::GetCurrentObject()
{
	// Caller only calls when type is EMode::Loading
	return ObjectStack.Last();
}

void FMaterialKeyGeneratorContext::RecordObjectStart(FUtf8StringView Name)
{
	switch (Mode)
	{
	case EMode::Emitting:
		break;
	case EMode::Saving:
		*Writer << Name;
		Writer->BeginObject();
		break;
	case EMode::Loading:
	{
		FCbFieldView SubObjectField = GetCurrentObject()[Name];
		FCbObjectView SubObject = SubObjectField.AsObjectView();
		if (SubObjectField.HasError())
		{
			bHasLoadError = true;
		}
		ObjectStack.Add(SubObject);
		break;
	}
	default:
		checkNoEntry();
		break;
	}
}

void FMaterialKeyGeneratorContext::RecordObjectEnd()
{
	switch (Mode)
	{
	case EMode::Emitting:
		break;
	case EMode::Saving:
		Writer->EndObject();
		break;
	case EMode::Loading:
		check(ObjectStack.Num() >= 2);
		ObjectStack.Pop(EAllowShrinking::No);
		break;
	default:
		checkNoEntry();
		break;
	}
}

#endif