// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialParameterCollection.h"
#include "ShaderParameterMetadata.h" // IWYU pragma: keep

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialParameterCollection)
