// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialRenderProxy.h"
#include "MaterialCachedData.h"
#include "Engine/Texture.h"
#include "EngineModule.h"
#include "ExternalTexture.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "RenderGraphBuilder.h"
#include "MaterialCache/IMaterialCacheTagProvider.h"
#include "Engine/TextureCollection.h"
#include "Engine/VirtualTextureCollection.h"
#include "VT/RuntimeVirtualTexture.h"

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif

int32 GDeferUniformExpressionCaching = 1;
FAutoConsoleVariableRef CVarDeferUniformExpressionCaching(
	TEXT("r.DeferUniformExpressionCaching"),
	GDeferUniformExpressionCaching,
	TEXT("Whether to defer caching of uniform expressions until a rendering command needs them up to date.  Deferring updates is more efficient because multiple SetVectorParameterValue calls in a frame will only result in one update."),
	ECVF_RenderThreadSafe
);

int32 GUniformExpressionCacheAsyncUpdates = 1;
FAutoConsoleVariableRef CVarUniformExpressionCacheAsyncUpdates(
	TEXT("r.UniformExpressionCacheAsyncUpdates"),
	GUniformExpressionCacheAsyncUpdates,
	TEXT("Whether to allow async updates of uniform expression caches."),
	ECVF_RenderThreadSafe);

FUniformExpressionCache::~FUniformExpressionCache()
{
	ResetAllocatedVTs();
	UniformBuffer.SafeRelease();
	UniformBufferVS.SafeRelease();
}

class FUniformExpressionCacheAsyncUpdateTask
{
public:
	void Begin()
	{
		ReferenceCount++;
	}

	void End()
	{
		check(ReferenceCount > 0);
		if (--ReferenceCount == 0)
		{
			Wait();
			Task = {};
		}
	}

	bool IsEnabled() const
	{
		return ReferenceCount > 0 && GUniformExpressionCacheAsyncUpdates > 0 && !GRHICommandList.Bypass();
	}

	void SetTask(const UE::Tasks::FTask& InTask)
	{
		check(IsEnabled());
		Task = InTask;
	}

	const UE::Tasks::FTask& GetTask() { return Task; }

	void Wait()
	{
		Task.Wait();
	}

private:
	UE::Tasks::FTask Task;
	int32 ReferenceCount = 0;

} GUniformExpressionCacheAsyncUpdateTask;

FUniformExpressionCacheAsyncUpdateScope::FUniformExpressionCacheAsyncUpdateScope()
{
	ENQUEUE_RENDER_COMMAND(BeginAsyncUniformExpressionCacheUpdates)(
		[](FRHICommandList&)
		{
			GUniformExpressionCacheAsyncUpdateTask.Begin();
		});
}

FUniformExpressionCacheAsyncUpdateScope::~FUniformExpressionCacheAsyncUpdateScope()
{
	ENQUEUE_RENDER_COMMAND(EndAsyncUniformExpressionCacheUpdates)(
		[](FRHICommandList&)
		{
			GUniformExpressionCacheAsyncUpdateTask.End();
		});
}

void FUniformExpressionCacheAsyncUpdateScope::WaitForTask()
{
	GUniformExpressionCacheAsyncUpdateTask.Wait();
}

class FUniformExpressionCacheAsyncUpdater
{
public:
	void Add(FUniformExpressionCache* UniformExpressionCache, const FUniformExpressionSet* UniformExpressionSet, const FRHIUniformBufferLayout* UniformBufferLayout, const FRHIUniformBufferLayout* InUniformBufferLayoutVS, const FMaterialRenderContext& Context)
	{
		Items.Emplace(UniformExpressionCache, UniformExpressionSet, UniformBufferLayout, InUniformBufferLayoutVS, Context);
	}

	void Update(FRHICommandListBase& RHICmdList)
	{
		check(!RHICmdList.IsImmediate());

		if (Items.IsEmpty())
		{
			RHICmdList.FinishRecording();
			return;
		}

		UE::Tasks::FTask Task = UE::Tasks::Launch(
			UE_SOURCE_LOCATION,
			[Items = MoveTemp(Items), &RHICmdList]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FUniformExpressionCacheAsyncUpdater::Update);
				FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

				{
					FRHICommandListScopedFence ScopedFence(RHICmdList);
					FMemMark Mark(FMemStack::Get());

					for (const FItem& Item : Items)
					{
						uint32 TempBufferSize = Item.UniformBufferLayout->ConstantBufferSize + Item.UniformExpressionSet->GetTemporaryBufferSize() * (4 * sizeof(float));
						uint8* TempBuffer = FMemStack::Get().PushBytes(TempBufferSize, SHADER_PARAMETER_STRUCT_ALIGNMENT);

						FMaterialRenderContext Context(Item.MaterialRenderProxy, *Item.Material, Item.bShowSelection);

						Item.UniformExpressionSet->FillUniformBuffer(Context, Item.AllocatedVTs, Item.UniformBufferLayout, TempBuffer, TempBufferSize);
				
						if (Item.UniformBufferVS)
						{
							check(Item.UniformBufferVS != Item.UniformBuffer);
							check(Item.UniformBufferLayoutVS);
							check(Item.UniformsVS);
							uint8* UniformsVSTempBuffer = FMemStack::Get().PushBytes(Item.UniformBufferLayoutVS->ConstantBufferSize, SHADER_PARAMETER_STRUCT_ALIGNMENT);
							FUniformExpressionSet::CompactUniformBuffer(UniformsVSTempBuffer, Item.UniformBufferLayoutVS->ConstantBufferSize, TempBuffer, Item.UniformBufferLayout->ConstantBufferSize, *Item.UniformsVS);
							RHICmdList.UpdateUniformBuffer(Item.UniformBufferVS, UniformsVSTempBuffer);
						}

						RHICmdList.UpdateUniformBuffer(Item.UniformBuffer, TempBuffer);
					}
				}

				RHICmdList.FinishRecording();

			}, GUniformExpressionCacheAsyncUpdateTask.GetTask());

		GUniformExpressionCacheAsyncUpdateTask.SetTask(Task);
	}

private:
	struct FItem
	{
		FItem() = default;
		FItem(FUniformExpressionCache* InUniformExpressionCache, const FUniformExpressionSet* InUniformExpressionSet, const FRHIUniformBufferLayout* InUniformBufferLayout, const FRHIUniformBufferLayout* InUniformBufferLayoutVS, const FMaterialRenderContext& Context)
			: UniformBuffer(InUniformExpressionCache->UniformBuffer)
			, UniformBufferVS(InUniformExpressionCache->UniformBufferVS)
			, AllocatedVTs(InUniformExpressionCache->AllocatedVTs)
			, UniformExpressionSet(InUniformExpressionSet)
			, UniformBufferLayout(InUniformBufferLayout)
			, UniformBufferLayoutVS(InUniformBufferLayoutVS)
			, MaterialRenderProxy(Context.MaterialRenderProxy)
			, Material(&Context.Material)
			, UniformsVS( InUniformExpressionSet->GetCompactUniformsVS())
			, bShowSelection(Context.bShowSelection)
		{
			check((UniformBufferVS == nullptr) || (UniformBufferVS != UniformBuffer));
		}

		TRefCountPtr<FRHIUniformBuffer> UniformBuffer;
		TRefCountPtr<FRHIUniformBuffer> UniformBufferVS;
		TArray<IAllocatedVirtualTexture*, FConcurrentLinearArrayAllocator> AllocatedVTs;
		const FUniformExpressionSet* UniformExpressionSet = nullptr;
		const FRHIUniformBufferLayout* UniformBufferLayout = nullptr;
		const FRHIUniformBufferLayout* UniformBufferLayoutVS = nullptr;
		const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
		const FMaterial* Material = nullptr;
		const FCompactUniformExpressionSet* UniformsVS = nullptr;
		bool bShowSelection = false;
	};

	TArray<FItem, FConcurrentLinearArrayAllocator> Items;
};

void FUniformExpressionCache::ResetAllocatedVTs()
{
	for (int32 i = 0; i < OwnedAllocatedVTs.Num(); ++i)
	{
		GetRendererModule().DestroyVirtualTexture(OwnedAllocatedVTs[i]);
	}
	AllocatedVTs.Reset();
	OwnedAllocatedVTs.Reset();
}


bool FMaterialRenderProxy::GetVectorValue(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor* OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::Vector, ParameterInfo, Value, Context))
	{
		*OutValue = Value.AsLinearColor();
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetScalarValue(const FHashedMaterialParameterInfo& ParameterInfo, float* OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::Scalar, ParameterInfo, Value, Context))
	{
		*OutValue = Value.AsScalar();
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetTextureValue(const FHashedMaterialParameterInfo& ParameterInfo, const UTexture** OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::Texture, ParameterInfo, Value, Context))
	{
		*OutValue = Value.Texture;
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetTextureValue(const FHashedMaterialParameterInfo& ParameterInfo, const URuntimeVirtualTexture** OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::RuntimeVirtualTexture, ParameterInfo, Value, Context))
	{
		*OutValue = Value.RuntimeVirtualTexture;
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetTextureValue(const FHashedMaterialParameterInfo& ParameterInfo, const USparseVolumeTexture** OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::SparseVolumeTexture, ParameterInfo, Value, Context))
	{
		*OutValue = Value.SparseVolumeTexture;
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetTextureCollectionValue(const FHashedMaterialParameterInfo& ParameterInfo, const UTextureCollection** OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::TextureCollection, ParameterInfo, Value, Context))
	{
		*OutValue = Value.TextureCollection;
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetParameterCollectionValue(const FHashedMaterialParameterInfo& ParameterInfo, const UMaterialParameterCollection** OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(EMaterialParameterType::ParameterCollection, ParameterInfo, Value, Context))
	{
		*OutValue = Value.ParameterCollection;
		return true;
	}
	return false;
}

bool FMaterialRenderProxy::GetParameterShaderValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, UE::Shader::FValue& OutValue, const FMaterialRenderContext& Context) const
{
	FMaterialParameterValue Value;
	if (GetParameterValue(Type, ParameterInfo, Value, Context))
	{
		OutValue = Value.AsShaderValue();
		return true;
	}
	return false;
}

FName FMaterialRenderProxy::GetUserSceneTextureOutput(const FMaterial* Base) const
{
	FName Result = NAME_None;

	// Replacing tonemapper can't override output.
	if (Base->GetBlendableLocation() != BL_ReplacingTonemapper)
	{
		// UserSceneTexture output overrides are stored under key "NAME_None".  We store them in the override lookup to save space
		// in the base structure, by avoiding a separate field just for the output override.
		if (!GetUserSceneTextureOverride(Result) && Base)
		{
			// If no override was found, get the result from the base material
			Result = FName(Base->GetRenderingThreadShaderMap()->GetUserSceneTextureOutput());
		}
	}
	return Result;
}

EBlendableLocation FMaterialRenderProxy::GetBlendableLocation(const FMaterial* Base) const
{
	check(Base);
	return (EBlendableLocation)Base->GetBlendableLocation();
}

int32 FMaterialRenderProxy::GetBlendablePriority(const FMaterial* Base) const
{
	check(Base);
	return Base->GetBlendablePriority();
}

static void InvalidateMaterialRenderProxyExpressionCache(FMaterialRenderProxy* MaterialProxy)
{
	MaterialProxy->InvalidateUniformExpressionCache(false);
	
	UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel)
	{
		MaterialProxy->UpdateUniformExpressionCacheIfNeeded(InFeatureLevel);
	});
}

static void OnVirtualTextureDestroyedCB(const FVirtualTextureProducerHandle&, void* Baton)
{
	InvalidateMaterialRenderProxyExpressionCache(static_cast<FMaterialRenderProxy*>(Baton));
}

static void OnTagChangedCB(void* Baton)
{
	InvalidateMaterialRenderProxyExpressionCache(static_cast<FMaterialRenderProxy*>(Baton));
}

IAllocatedVirtualTexture* FMaterialRenderProxy::GetPreallocatedVTStack(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const FMaterialVirtualTextureStack& VTStack) const
{
	check(VTStack.IsPreallocatedStack());

	const URuntimeVirtualTexture* Texture;
	VTStack.GetTextureValue(Context, UniformExpressionSet, Texture);

	if (Texture == nullptr)
	{
		return nullptr;
	}

	if (!bMarkedTransient)
	{
		GetRendererModule().AddVirtualTextureProducerDestroyedCallback(Texture->GetProducerHandle(), &OnVirtualTextureDestroyedCB, const_cast<FMaterialRenderProxy*>(this));
		bHasVirtualTextureCallbacks = true;
	}

	return Texture->GetAllocatedVirtualTexture();
}

IAllocatedVirtualTexture* FMaterialRenderProxy::AllocateVTStack(FRHICommandListBase& RHICmdList, const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const FMaterialVirtualTextureStack& VTStack) const
{
	check(!VTStack.IsPreallocatedStack());
	const uint32 NumLayers = VTStack.GetNumLayers();
	if (NumLayers == 0u)
	{
		return nullptr;
	}

	const UTexture* LayerTextures[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { nullptr };
	VTStack.GetTextureValues(Context, UniformExpressionSet, LayerTextures);

	const UMaterialInterface* MaterialInterface = GetMaterialInterface();

	FAllocatedVTDescription VTDesc;
	if (MaterialInterface)
	{
		VTDesc.Name = MaterialInterface->GetFName();
	}
	VTDesc.Dimensions = 2;
	VTDesc.NumTextureLayers = NumLayers;
	bool bFoundValidLayer = false;
	for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
	{
		const UTexture* Texture = LayerTextures[LayerIndex];
		if (!Texture)
		{
			continue;
		}

		// GetResource() is safe to call from the render thread.
		const FTextureResource* TextureResource = Texture->GetResource();
		if (TextureResource)
		{
			const FVirtualTexture2DResource* VirtualTextureResourceForLayer = TextureResource->GetVirtualTexture2DResource();

			if (VirtualTextureResourceForLayer == nullptr)
			{
				// The placeholder used during async texture compilation is expected to be of the wrong type since
				// no VT infos are available until later in the compilation process. This will be resolved
				// once the final texture resource is available.
#if WITH_EDITOR
				if (!TextureResource->IsProxy())
#endif
				{
					UE_LOGF(LogMaterial, Warning, "Material '%ls' expects texture '%ls' to be Virtual",
						*GetFriendlyName(), *Texture->GetName());
				}
				continue;
			}
			else
			{
				// All tile sizes need to match
				check(!bFoundValidLayer || VTDesc.TileSize == VirtualTextureResourceForLayer->GetTileSize());
				check(!bFoundValidLayer || VTDesc.TileBorderSize == VirtualTextureResourceForLayer->GetBorderSize());

				const FVirtualTextureProducerHandle& ProducerHandle = VirtualTextureResourceForLayer->GetProducerHandle();
				if (ProducerHandle.IsValid())
				{
					VTDesc.TileSize = VirtualTextureResourceForLayer->GetTileSize();
					VTDesc.TileBorderSize = VirtualTextureResourceForLayer->GetBorderSize();
					VTDesc.ProducerHandle[LayerIndex] = ProducerHandle;
					VTDesc.ProducerLayerIndex[LayerIndex] = 0u;
					bFoundValidLayer = true;

					if (!bMarkedTransient)
					{
						GetRendererModule().AddVirtualTextureProducerDestroyedCallback(ProducerHandle, &OnVirtualTextureDestroyedCB, const_cast<FMaterialRenderProxy*>(this));
						bHasVirtualTextureCallbacks = true;
					}
				}
			}
		}
	}

	if (bFoundValidLayer)
	{
		return GetRendererModule().AllocateVirtualTexture(RHICmdList, VTDesc);
	}
	return nullptr;
}


void FMaterialRenderProxy::EvaluateUniformExpressions(FRHICommandListBase& RHICmdList, FUniformExpressionCache& OutUniformExpressionCache, const FMaterialRenderContext& Context, FUniformExpressionCacheAsyncUpdater* Updater) const
{
	SCOPE_CYCLE_COUNTER(STAT_CacheUniformExpressions);

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialRenderProxy::EvaluateUniformExpressions);

	// Retrieve the material's uniform expression set.
	FMaterialShaderMap* ShaderMap = Context.Material.GetRenderingThreadShaderMap();
	const FUniformExpressionSet& UniformExpressionSet = ShaderMap->GetUniformExpressionSet();

	// Initialize this to null, and set it to its final value (ShaderMap) at the end of the function.  The goal is to increase the timing window where
	// we can detect thread safety bugs where the uniform expression cache is accessed while the expressions are being evaluated.  Bugs will typically
	// manifest as an assert in FMaterialShader::GetShaderBindings, called from various mesh draw command generation logic.
	OutUniformExpressionCache.CachedUniformExpressionShaderMap = nullptr;

	OutUniformExpressionCache.ResetAllocatedVTs();
	OutUniformExpressionCache.AllocatedVTs.Empty(UniformExpressionSet.VTStacks.Num());
	OutUniformExpressionCache.OwnedAllocatedVTs.Empty(UniformExpressionSet.VTStacks.Num());

	if (bHasVirtualTextureCallbacks)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		bHasVirtualTextureCallbacks = false;
	}

	for (int32 i = 0; i < UniformExpressionSet.VTStacks.Num(); ++i)
	{
		const FMaterialVirtualTextureStack& VTStack = UniformExpressionSet.VTStacks[i];
		IAllocatedVirtualTexture* AllocatedVT = nullptr;
		if (VTStack.IsPreallocatedStack())
		{
			AllocatedVT = GetPreallocatedVTStack(Context, UniformExpressionSet, VTStack);
		}
		else
		{
			AllocatedVT = AllocateVTStack(RHICmdList, Context, UniformExpressionSet, VTStack);
			if (AllocatedVT != nullptr)
			{
				OutUniformExpressionCache.OwnedAllocatedVTs.Add(AllocatedVT);
			}
		}
		OutUniformExpressionCache.AllocatedVTs.Add(AllocatedVT);
	}

	for (int32 TagIndex = 0; TagIndex < UniformExpressionSet.MaterialCacheTagStacks.Num(); ++TagIndex)
	{
		const FMaterialCacheTagStack& Stack = UniformExpressionSet.MaterialCacheTagStacks[TagIndex];

		// Subscribe to tag changes which require rebinding
		IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
		TagProvider->AddTagSceneInvalidationCallback(Stack.TagGuid, &OnTagChangedCB, const_cast<FMaterialRenderProxy*>(this));
		bHasMaterialCacheCallbacks = true;
	}

	for (int32 VTCIndex = 0; VTCIndex < UniformExpressionSet.UniformTextureCollectionParameters.Num(); ++VTCIndex)
	{
		const FMaterialTextureCollectionParameterInfo& Parameter = UniformExpressionSet.UniformTextureCollectionParameters[VTCIndex];
		if (!Parameter.bIsVirtualCollection)
		{
			continue;
		}

		const UTextureCollection* Texture = nullptr;
		UniformExpressionSet.GetTextureCollectionValue(VTCIndex, Context, Context.Material, Texture);

		// May not be bound, but must be a VTC
		if (!Texture || !ensure(Texture->IsVirtualCollection()))
		{
			continue;
		}

		// May not have a valid resource
		FVirtualTextureCollectionResource* Resource = static_cast<FVirtualTextureCollectionResource*>(Texture->GetResource());
		if (!Resource || !ensure(!Resource->bIsBindless))
		{
			continue;
		}

		// Subscribe to VT destruction
		GetRendererModule().AddVirtualTextureProducerDestroyedCallback(Resource->GetProducerHandle(), &OnVirtualTextureDestroyedCB, const_cast<FMaterialRenderProxy*>(this));
		bHasVirtualTextureCallbacks = true;
	}

	const FRHIUniformBufferLayout* UniformBufferLayout = ShaderMap->GetUniformBufferLayout();
	const FRHIUniformBufferLayout* UniformBufferLayoutVS = ShaderMap->GetUniformBufferLayoutVS();

	auto ValidateLayout = [](FUniformBufferRHIRef& UniformBuffer, const FRHIUniformBufferLayout* UniformBufferLayout)
	{
		if (IsValidRef(UniformBuffer))
		{
			if (!UniformBuffer->IsValid())
			{
				UE_LOGF(LogMaterial, Fatal, "The Uniformbuffer needs to be valid if it has been set");
			}

			/**
				* The actual pointer may not match because there are cases(in the editor, during the shader compilation) when material's shader map gets updated without proxy's cache
				* getting invalidated, but the layout contents must match.
				*/
#if WITH_EDITOR
			/**
				* If we are in the editor, this is likely occuring due to a mismatch of buffer invalidations and the type of shaders that need processing, so we can safely reset the
				* Buffer layout to avoid any engine crashes. However we still want this check in project builds for safety when doing QA passes.
				*/
			if (UniformBuffer->GetLayoutPtr() != UniformBufferLayout || *UniformBuffer->GetLayoutPtr() != *UniformBufferLayout)
			{
				UniformBuffer = nullptr;
			}
#else
			check(UniformBuffer->GetLayoutPtr() == UniformBufferLayout || *UniformBuffer->GetLayoutPtr() == *UniformBufferLayout);
#endif
		}	
	};

	ValidateLayout(OutUniformExpressionCache.UniformBuffer, UniformBufferLayout);
	ValidateLayout(OutUniformExpressionCache.UniformBufferVS, UniformBufferLayoutVS);

	const FCompactUniformExpressionSet* CompactUniformsVS = UniformExpressionSet.GetCompactUniformsVS();
	bool bShouldHaveDedicatedMaterialUB = (CompactUniformsVS != nullptr && !CompactUniformsVS->CompactionSegments.IsEmpty());

	if (Updater)
	{
		if (!IsValidRef(OutUniformExpressionCache.UniformBuffer))
		{
			OutUniformExpressionCache.UniformBuffer = RHICreateUniformBuffer(nullptr, UniformBufferLayout, UniformBuffer_MultiFrame);
		}

		if (bShouldHaveDedicatedMaterialUB)
		{
			if (!IsValidRef(OutUniformExpressionCache.UniformBufferVS))
			{
				OutUniformExpressionCache.UniformBufferVS = RHICreateUniformBuffer(nullptr, UniformBufferLayoutVS, UniformBuffer_MultiFrame);
			}
		}
		else
		{
			OutUniformExpressionCache.UniformBufferVS = nullptr;
		}

		Updater->Add(&OutUniformExpressionCache, &UniformExpressionSet, UniformBufferLayout, UniformBufferLayoutVS, Context);
	}
	else
	{
		FMemMark Mark(FMemStack::Get());
		uint32 TempBufferSize = UniformBufferLayout->ConstantBufferSize + UniformExpressionSet.GetTemporaryBufferSize() * (4 * sizeof(float));
		uint8* TempBuffer = FMemStack::Get().PushBytes(TempBufferSize, SHADER_PARAMETER_STRUCT_ALIGNMENT);
		UniformExpressionSet.FillUniformBuffer(Context, OutUniformExpressionCache, UniformBufferLayout, TempBuffer, TempBufferSize);

		auto CreateOrUpdateUniformBuffer = [](FRHICommandListBase& RHICmdList, FUniformBufferRHIRef& UniformBuffer, const uint8* TempBuffer, const FRHIUniformBufferLayout* UniformBufferLayout)
		{
			if (IsValidRef(UniformBuffer))
			{
				RHICmdList.UpdateUniformBuffer(UniformBuffer, TempBuffer);
			}
			else
			{
				UniformBuffer = RHICreateUniformBuffer(TempBuffer, UniformBufferLayout, UniformBuffer_MultiFrame);
			}

		};

		CreateOrUpdateUniformBuffer(RHICmdList, OutUniformExpressionCache.UniformBuffer, TempBuffer, UniformBufferLayout);

		if (bShouldHaveDedicatedMaterialUB)
		{
			uint8* UniformsVSTempBuffer = FMemStack::Get().PushBytes(UniformBufferLayoutVS->ConstantBufferSize, SHADER_PARAMETER_STRUCT_ALIGNMENT);
			FUniformExpressionSet::CompactUniformBuffer(UniformsVSTempBuffer, UniformBufferLayoutVS->ConstantBufferSize, TempBuffer, UniformBufferLayout->ConstantBufferSize, *CompactUniformsVS);
				
			CreateOrUpdateUniformBuffer(RHICmdList, OutUniformExpressionCache.UniformBufferVS, UniformsVSTempBuffer, UniformBufferLayoutVS);
		}
		else
		{
			OutUniformExpressionCache.UniformBufferVS = nullptr;
		}
	}

	EvaluateParameterCollections(RHICmdList, OutUniformExpressionCache, Context, Updater);

	// Deliberately set this last, see comment above where it's initialized to nullptr
	OutUniformExpressionCache.CachedUniformExpressionShaderMap = ShaderMap;

	++UniformExpressionCacheSerialNumber;
}

void FMaterialRenderProxy::EvaluateParameterCollections(FRHICommandListBase& RHICmdList, FUniformExpressionCache& OutUniformExpressionCache, const FMaterialRenderContext& Context, FUniformExpressionCacheAsyncUpdater* Updater) const
{
	// Retrieve the material's uniform expression set.
	FMaterialShaderMap* ShaderMap = Context.Material.GetRenderingThreadShaderMap();
	const FUniformExpressionSet& UniformExpressionSet = ShaderMap->GetUniformExpressionSet();

	OutUniformExpressionCache.ParameterCollections = UniformExpressionSet.ParameterCollections;
}

void FMaterialRenderProxy::CacheUniformExpressions(FRHICommandListBase& RHICmdList, bool bRecreateUniformBuffer)
{
	// Register the render proxy's as a render resource so it can receive notifications to free the uniform buffer.
	InitResource(RHICmdList);

	bool bUsingNewLoader = FPlatformProperties::RequiresCookedData();

	check((bUsingNewLoader && GIsInitialLoad) || // The EDL at boot time maybe not load the default materials first; we need to intialize materials before the default materials are done
		UMaterial::GetDefaultMaterial(MD_Surface));


	if (IsMarkedForGarbageCollection())
	{
		UE_LOGF(LogMaterial, Fatal, "Cannot queue the Expression Cache for Material %ls when it is about to be deleted", *MaterialName);
	}
	StartCacheUniformExpressions();

	DeferredUniformExpressionCacheRequestsMutex.Lock();
	DeferredUniformExpressionCacheRequests.Add(this);
	DeferredUniformExpressionCacheRequestsMutex.Unlock();

	InvalidateUniformExpressionCache(bRecreateUniformBuffer);

	if (!GDeferUniformExpressionCaching)
	{
		FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions(RHICmdList);
	}
}

void FMaterialRenderProxy::CacheUniformExpressions_GameThread(bool bRecreateUniformBuffer)
{
	if (FApp::CanEverRender())
	{
		UE_LOGF(LogMaterial, VeryVerbose, "Caching uniform expressions for material: %ls", *GetFriendlyName());

		FMaterialRenderProxy* RenderProxy = this;
		ENQUEUE_RENDER_COMMAND(FCacheUniformExpressionsCommand)(
			[RenderProxy, bRecreateUniformBuffer](FRHICommandListImmediate& RHICmdList)
		{
			RenderProxy->CacheUniformExpressions(RHICmdList, bRecreateUniformBuffer);
		});
	}
}

void FMaterialRenderProxy::InvalidateUniformExpressionCache(bool bRecreateUniformBuffer)
{
	GUniformExpressionCacheAsyncUpdateTask.Wait();

	// Async RDG tasks can call FMaterialShader::SetParameters which touch the material uniform expression cache.
	FRDGBuilder::WaitForAsyncExecuteTask();

#if WITH_EDITOR
	FStaticLightingSystemInterface::OnMaterialInvalidated.Broadcast(this);
#endif

	UE::TScopeLock Lock(Mutex);

	if (bHasVirtualTextureCallbacks)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		bHasVirtualTextureCallbacks = false;
	}

	if (bHasMaterialCacheCallbacks)
	{
		IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
		TagProvider->RemoveTagSceneInvalidationCallbacks(this);
		bHasMaterialCacheCallbacks = false;
	}

	++UniformExpressionCacheSerialNumber;
	for (int32 i = 0; i < ERHIFeatureLevel::Num; ++i)
	{
		UniformExpressionCache[i].CachedUniformExpressionShaderMap = nullptr;
		UniformExpressionCache[i].ResetAllocatedVTs();

		if (bRecreateUniformBuffer)
		{
			// This is required if the FMaterial is being recompiled (the uniform buffer layout will change).
			// This should only be done if the calling code is using FMaterialUpdateContext to recreate the rendering state of primitives using this material, 
			// Since cached mesh commands also cache uniform buffer pointers.
			UniformExpressionCache[i].UniformBuffer = nullptr;
			UniformExpressionCache[i].UniformBufferVS = nullptr;
		}
	}
}

const FMaterial* FMaterialRenderProxy::UpdateUniformExpressionCacheIfNeeded(FRHICommandListBase& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel) const
{
	// Don't cache uniform expressions if an entirely different FMaterialRenderProxy is going to be used for rendering
	const FMaterial* Material = GetMaterialNoFallback(InFeatureLevel);
	// Note: We would actually need to compare the FMaterialShaderMapId of both shader maps but a simple pointer compare also works
	// because shader maps are currently swapped out whenever they are modified.
	UE::TScopeLock Lock(Mutex);
	if (Material && Material->GetRenderingThreadShaderMap() != UniformExpressionCache[InFeatureLevel].CachedUniformExpressionShaderMap)
	{
		// Async RDG tasks can call FMaterialShader::SetParameters which touch the material uniform expression cache.
		FRDGBuilder::WaitForAsyncExecuteTask();
		FMaterialRenderContext MaterialRenderContext(this, *Material, nullptr);
		MaterialRenderContext.bShowSelection = GIsEditor;
		EvaluateUniformExpressions(RHICmdList, UniformExpressionCache[InFeatureLevel], MaterialRenderContext, nullptr);
	}

	return Material;
}

void FMaterialRenderProxy::UpdateUniformExpressionCacheIfNeeded(ERHIFeatureLevel::Type InFeatureLevel) const
{
	UpdateUniformExpressionCacheIfNeeded(FRHICommandListImmediate::Get(), InFeatureLevel);
}

FMaterialRenderProxy::FMaterialRenderProxy(FString InMaterialName)
	: SubsurfaceProfileRT(nullptr)
	, MaterialName(MoveTemp(InMaterialName))
	, MarkedForGarbageCollection(0)
	, DeletedFlag(0)
	, bMarkedTransient(false)
	, bHasVirtualTextureCallbacks(false)
	, bHasMaterialCacheCallbacks(false)
{
}

FMaterialRenderProxy::~FMaterialRenderProxy()
{
	// We only wait on deletions happening on the render thread. Async deletions can happen during scene render shutdown and those are waited on explicitly.
	if (IsInRenderingThread())
	{
		GUniformExpressionCacheAsyncUpdateTask.Wait();
	}

	if (IsInitialized())
	{
		ReleaseResource();
	}

	if (bHasVirtualTextureCallbacks)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		bHasVirtualTextureCallbacks = false;
	}

	if (bHasMaterialCacheCallbacks)
	{
		IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
		TagProvider->RemoveTagSceneInvalidationCallbacks(this);
		bHasMaterialCacheCallbacks = false;
	}

	DeletedFlag = -1;
}

void FMaterialRenderProxy::InitRHI(FRHICommandListBase& RHICmdList)
{
#if WITH_EDITOR
	// MaterialRenderProxyMap is only used by shader compiling
	if (!FPlatformProperties::RequiresCookedData())
	{
		FScopeLock Locker(&MaterialRenderProxyMapLock);
		FMaterialRenderProxy::MaterialRenderProxyMap.Add(this);
	}
#endif // WITH_EDITOR
}

void FMaterialRenderProxy::CancelCacheUniformExpressions()
{
	DeferredUniformExpressionCacheRequestsMutex.Lock();
	DeferredUniformExpressionCacheRequests.Remove(this);
	DeferredUniformExpressionCacheRequestsMutex.Unlock();
}

void FMaterialRenderProxy::ReleaseRHI()
{
#if WITH_EDITOR
	if (!FPlatformProperties::RequiresCookedData())
	{
		FScopeLock Locker(&MaterialRenderProxyMapLock);
		FMaterialRenderProxy::MaterialRenderProxyMap.Remove(this);
	}
#endif // WITH_EDITOR

	DeferredUniformExpressionCacheRequestsMutex.Lock();
	bool bRemoved = DeferredUniformExpressionCacheRequests.Remove(this) != 0;
	DeferredUniformExpressionCacheRequestsMutex.Unlock();

	if (bRemoved)
	{
		// Notify that we're finished with this inflight cache request, because the object is being released
		FinishCacheUniformExpressions();
	}

	InvalidateUniformExpressionCache(true);

	FExternalTextureRegistry::Get().RemoveMaterialRenderProxyReference(this);
}

void FMaterialRenderProxy::ReleaseResource()
{
	ReleaseResourceFlag = -1;
	FRenderResource::ReleaseResource();
	if (bHasVirtualTextureCallbacks)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		bHasVirtualTextureCallbacks = false;
	}

	if (bHasMaterialCacheCallbacks)
	{
		IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
		TagProvider->RemoveTagSceneInvalidationCallbacks(this);
		bHasMaterialCacheCallbacks = false;
	}
}


const FMaterial& FMaterialRenderProxy::GetMaterialWithFallback(ERHIFeatureLevel::Type InFeatureLevel, const FMaterialRenderProxy*& OutFallbackMaterialRenderProxy) const
{
	const FMaterial* BaseMaterial = GetMaterialNoFallback(InFeatureLevel);
	const FMaterial* Material = BaseMaterial;
	if (!Material || !Material->IsRenderingThreadShaderMapComplete())
	{
		const FMaterialRenderProxy* FallbackMaterialProxy = this;
		do
		{
			FallbackMaterialProxy = FallbackMaterialProxy->GetFallback(InFeatureLevel);
			check(FallbackMaterialProxy);
			Material = FallbackMaterialProxy->GetMaterialNoFallback(InFeatureLevel);
		} while (!Material || !Material->IsRenderingThreadShaderMapComplete());
		OutFallbackMaterialRenderProxy = FallbackMaterialProxy;

#if WITH_EDITOR
		if (BaseMaterial)
		{
			BaseMaterial->SubmitCompileJobs_RenderThread(EShaderCompileJobPriority::Normal);
		}
#endif // WITH_EDITOR
	}
	return *Material;
}

const FMaterial& FMaterialRenderProxy::GetIncompleteMaterialWithFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	const FMaterial* Material = GetMaterialNoFallback(InFeatureLevel);
	if (!Material)
	{
		const FMaterialRenderProxy* FallbackMaterialProxy = this;
		do
		{
			FallbackMaterialProxy = FallbackMaterialProxy->GetFallback(InFeatureLevel);
			check(FallbackMaterialProxy);
			Material = FallbackMaterialProxy->GetMaterialNoFallback(InFeatureLevel);
		} while (!Material);
	}
	return *Material;
}

void FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions(FRHICommandListBase& RHICmdList, UE::Tasks::FTask* TaskIfAsync)
{
	LLM_SCOPE(ELLMTag::Materials);

	const bool bAllowAsyncUpdate = RHICmdList.IsImmediate() && GUniformExpressionCacheAsyncUpdateTask.IsEnabled();

	FRHICommandListBase* RHICmdListTask = &RHICmdList;

	// Create an async command list when immediate command list is supplied and async update is allowed.
	if (bAllowAsyncUpdate)
	{
		RHICmdListTask = new FRHICommandList(FRHIGPUMask::All());
		RHICmdListTask->SwitchPipeline(ERHIPipeline::Graphics);
	}
	else
	{
		GUniformExpressionCacheAsyncUpdateTask.Wait();
	}

	DeferredUniformExpressionCacheRequestsMutex.Lock();
	TSet<FMaterialRenderProxy*> UniformExpressions = MoveTemp(DeferredUniformExpressionCacheRequests);
	DeferredUniformExpressionCacheRequestsMutex.Unlock();

	auto EvaluateUniformExpressionsLambda = [RHICmdListTask, bAllowAsyncUpdate, UniformExpressions = MoveTemp(UniformExpressions)]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Material_UpdateDeferredCachedUniformExpressions);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateDeferredCachedUniformExpressions);

		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		FUniformExpressionCacheAsyncUpdater Updater;
		FUniformExpressionCacheAsyncUpdater* UpdaterIfEnabled = bAllowAsyncUpdate ? &Updater : nullptr;

		for (TSet<FMaterialRenderProxy*>::TConstIterator It(UniformExpressions); It; ++It)
		{
			FMaterialRenderProxy* MaterialProxy = *It;
			if (MaterialProxy->IsDeleted())
			{
				UE_LOGF(LogMaterial, Fatal, "FMaterialRenderProxy deleted and GC mark was: %i", MaterialProxy->IsMarkedForGarbageCollection());
			}

			UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel)
			{
				// Don't bother caching if we'll be falling back to a different FMaterialRenderProxy for rendering anyway
				const FMaterial* Material = MaterialProxy->GetMaterialNoFallback(InFeatureLevel);
				if (Material && Material->GetRenderingThreadShaderMap())
				{
					FMaterialRenderContext MaterialRenderContext(MaterialProxy, *Material, nullptr);
					MaterialRenderContext.bShowSelection = GIsEditor;
					MaterialProxy->EvaluateUniformExpressions(*RHICmdListTask, MaterialProxy->UniformExpressionCache[(int32)InFeatureLevel], MaterialRenderContext, UpdaterIfEnabled);
				}
			});

			MaterialProxy->FinishCacheUniformExpressions();
		}

		if (UpdaterIfEnabled)
		{
			UpdaterIfEnabled->Update(*RHICmdListTask);
		}
	};

	if (TaskIfAsync && bAllowAsyncUpdate)
	{
		*TaskIfAsync = UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(EvaluateUniformExpressionsLambda));
	}
	else
	{
		EvaluateUniformExpressionsLambda();
	}

	if (bAllowAsyncUpdate)
	{
		FRHICommandListImmediate::Get(RHICmdList).QueueAsyncCommandListSubmit(RHICmdListTask);
	}
}

void FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions()
{
	UpdateDeferredCachedUniformExpressions(FRHICommandListImmediate::Get());
}

bool FMaterialRenderProxy::HasDeferredUniformExpressionCacheRequests()
{
	UE::TScopeLock Lock(DeferredUniformExpressionCacheRequestsMutex);
	return DeferredUniformExpressionCacheRequests.Num() > 0;
}

#if WITH_EDITOR
TSet<FMaterialRenderProxy*> FMaterialRenderProxy::MaterialRenderProxyMap;
FCriticalSection FMaterialRenderProxy::MaterialRenderProxyMapLock;
#endif // WITH_EDITOR
TSet<FMaterialRenderProxy*> FMaterialRenderProxy::DeferredUniformExpressionCacheRequests;
UE::FMutex FMaterialRenderProxy::DeferredUniformExpressionCacheRequestsMutex;

/*-----------------------------------------------------------------------------
	FColoredMaterialRenderProxy
-----------------------------------------------------------------------------*/

FColoredMaterialRenderProxy::FColoredMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InColor, FName InColorParamName)
	: FMaterialRenderProxy(InParent ? InParent->GetMaterialName() : TEXT("FColoredMaterialRenderProxy"))
	, Parent(InParent)
	, Color(InColor)
	, ColorParamName(InColorParamName)
{
}

const FMaterial* FColoredMaterialRenderProxy::GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return Parent->GetMaterialNoFallback(InFeatureLevel);
}

const FMaterialRenderProxy* FColoredMaterialRenderProxy::GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return Parent->GetFallback(InFeatureLevel);
}

bool FColoredMaterialRenderProxy::GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
{
	if (Type == EMaterialParameterType::Vector && ParameterInfo.Name == ColorParamName)
	{
		OutValue = Color;
		return true;
	}
	else
	{
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
}

/*-----------------------------------------------------------------------------
	FColoredTexturedMaterialRenderProxy
-----------------------------------------------------------------------------*/

FColoredTexturedMaterialRenderProxy::FColoredTexturedMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InColor, FName InColorParamName, const UTexture* InTexture, FName InTextureParamName)
	: FColoredMaterialRenderProxy(InParent, InColor, InColorParamName)
	, Texture(InTexture)
	, TextureParamName(InTextureParamName)
{
}

bool FColoredTexturedMaterialRenderProxy::GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
{
	if (Type == EMaterialParameterType::Texture && ParameterInfo.Name == TextureParamName)
	{
		OutValue = Texture;
		return true;
	}
	else
	{
		if (Type == EMaterialParameterType::Scalar && ParameterInfo.Name == UVChannelParamName)
		{
			OutValue = UVChannel;
			return true;
		}
		else
		{
			// Call base class to make sure we override the color parameter if needed
			return FColoredMaterialRenderProxy::GetParameterValue(Type, ParameterInfo, OutValue, Context);
		}
	}
}

/*-----------------------------------------------------------------------------
	FOverrideSelectionColorMaterialRenderProxy
-----------------------------------------------------------------------------*/

FOverrideSelectionColorMaterialRenderProxy::FOverrideSelectionColorMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InSelectionColor)
	: FMaterialRenderProxy(InParent ? InParent->GetMaterialName() : TEXT("FOverrideSelectionColorMaterialRenderProxy"))
	, Parent(InParent)
	, SelectionColor(FLinearColor(InSelectionColor.R, InSelectionColor.G, InSelectionColor.B, 1))
{
}

const FMaterial* FOverrideSelectionColorMaterialRenderProxy::GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return Parent->GetMaterialNoFallback(InFeatureLevel);
}

const FMaterialRenderProxy* FOverrideSelectionColorMaterialRenderProxy::GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return Parent->GetFallback(InFeatureLevel);
}

bool FOverrideSelectionColorMaterialRenderProxy::GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
{
	if (Type == EMaterialParameterType::Vector && ParameterInfo.Name == FName(NAME_SelectionColor))
	{
		OutValue = SelectionColor;
		return true;
	}
	else
	{
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
}

/*-----------------------------------------------------------------------------
	FLightingDensityMaterialRenderProxy
-----------------------------------------------------------------------------*/

FLightingDensityMaterialRenderProxy::FLightingDensityMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InColor, const FVector2D& InLightmapResolution)
	: FColoredMaterialRenderProxy(InParent, InColor)
	, LightmapResolution(InLightmapResolution)
{
}

static FName NAME_LightmapRes = FName("LightmapRes");

bool FLightingDensityMaterialRenderProxy::GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
{
	if (Type == EMaterialParameterType::Vector && ParameterInfo.Name == NAME_LightmapRes)
	{
		OutValue = FLinearColor(LightmapResolution.X, LightmapResolution.Y, 0.0f, 0.0f);
		return true;
	}
	return FColoredMaterialRenderProxy::GetParameterValue(Type, ParameterInfo, OutValue, Context);
}

