// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialShader.h"

#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "RenderUtils.h"
#include "Stats/StatsMisc.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialInterface.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "MaterialDomain.h"
#include "MaterialShaderMapLayout.h"
#include "SceneInterface.h"
#include "ShaderCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ProfilingDebugging/CookStats.h"
#include "Stats/StatsTrace.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/EditorObjectVersion.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Misc/PathViews.h"
#include "SceneTexturesConfig.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "ShaderCodeLibrary.h"
#include "ShaderSerialization.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Materials/Material.h"
#include "HLSLMaterialTranslator.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheValidation.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Hash/Blake3.h"

#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#endif

#if WITH_EDITOR
#include "Algo/Sort.h"
#include "Containers/AnsiString.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookDependencyContext.h"
#include "Cooker/CookEvents.h"
#include "Interfaces/ITargetPlatform.h"
#include "MaterialCachedData.h"
#include "MaterialKeyGeneratorContext.h"
#include "Materials/MaterialSharedPrivate.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryReader.h"
#endif

CSV_DECLARE_CATEGORY_MODULE_EXTERN(RENDERCORE_API, Shaders);

int32 GMaterialExcludeNonPipelinedShaders = 1;
static FAutoConsoleVariableRef CVarMaterialExcludeNonPipelinedShaders(
	TEXT("r.Material.ExcludeNonPipelinedShaders"),
	GMaterialExcludeNonPipelinedShaders,
	TEXT("if != 0, standalone shaders that are also part of FShaderPipeline will not be compiled (default)."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<FString> CVarMaterialShaderMapDump(
	TEXT("r.Material.ShaderMapDump"),
	"",
	TEXT("Outputs a textual dump of all shader maps found for the given named material (specified by path).\n")
	TEXT("Note that this will include any instances of said material created by a MaterialInstance.\n")
	TEXT("Files (.txt extension) will be dumped to Saved\\MaterialShaderMaps named with the DDC key hash.\n"),
	ECVF_ReadOnly);

#if WITH_EDITOR
static TAutoConsoleVariable<FString> CVarShaderCompilerDebugDDCKeyAsset(
	TEXT("r.ShaderCompiler.DebugDDCKeyAsset"),
	FString(),
	TEXT("if set, assets containing this name will print their full DDC key hash information when requested"),
	ECVF_Default
);
#endif // WITH_EDITOR

#if ENABLE_COOK_STATS
namespace MaterialShaderCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("MaterialShader.Usage"), TEXT(""));
	});
}
#endif

//
// Globals
//
FCriticalSection FMaterialShaderMap::GIdToMaterialShaderMapCS;
TMap<FMaterialShaderMapId,FMaterialShaderMap*> FMaterialShaderMap::GIdToMaterialShaderMap[SP_NumPlatforms];
#if ALLOW_SHADERMAP_DEBUG_DATA
TArray<FMaterialShaderMap*> FMaterialShaderMap::AllMaterialShaderMaps;
FCriticalSection FMaterialShaderMap::AllMaterialShaderMapsGuard;
#endif
// defined in the same module (Material.cpp)
bool PoolSpecialMaterialsCompileJobs();

/** 
 * Tracks material resources and their shader maps that are being compiled.
 * Uses a TRefCountPtr as this will be the only reference to a shader map while it is being compiled.
 */
//TMap<TRefCountPtr<FMaterialShaderMap>, TArray<FMaterial*> > FMaterialShaderMap::ShaderMapsBeingCompiled;


/** Converts an EMaterialShadingModel to a string description. */
FString GetShadingModelString(EMaterialShadingModel ShadingModel)
{
	FString ShadingModelName;
	switch(ShadingModel)
	{
		case MSM_Unlit:				ShadingModelName = TEXT("MSM_Unlit"); break;
		case MSM_DefaultLit:		ShadingModelName = TEXT("MSM_DefaultLit"); break;
		case MSM_Subsurface:		ShadingModelName = TEXT("MSM_Subsurface"); break;
		case MSM_PreintegratedSkin:	ShadingModelName = TEXT("MSM_PreintegratedSkin"); break;
		case MSM_ClearCoat:			ShadingModelName = TEXT("MSM_ClearCoat"); break;
		case MSM_SubsurfaceProfile:	ShadingModelName = TEXT("MSM_SubsurfaceProfile"); break;
		case MSM_TwoSidedFoliage:	ShadingModelName = TEXT("MSM_TwoSidedFoliage"); break;
		case MSM_Hair:				ShadingModelName = TEXT("MSM_Hair"); break;
		case MSM_Cloth:				ShadingModelName = TEXT("MSM_Cloth"); break;
		case MSM_Eye:				ShadingModelName = TEXT("MSM_Eye"); break;
		case MSM_SingleLayerWater:	ShadingModelName = TEXT("MSM_SingleLayerWater"); break;
		case MSM_ThinTranslucent:	ShadingModelName = TEXT("MSM_ThinTranslucent"); break;
		default: ShadingModelName = TEXT("Unknown"); break;
	}
	return ShadingModelName;
}

/** Helpers class to identify and remove shader types that are going to be used by the pipelines. 
    Standalone shaders of that type should no longer be required, but removing them from the shadermap layout is a bigger endeavour */
class FPipelinedShaderFilter
{
	TSet<const FShaderType*>	PipelinedShaderTypes;
	bool						bAnyTypesExcluded = false;

public:

	FPipelinedShaderFilter(EShaderPlatform ShaderPlatform, TArray<FShaderPipelineType*> Pipelines)
	{
		if (ExcludeNonPipelinedShaderTypes(ShaderPlatform))
		{
			for (FShaderPipelineType* Pipeline : Pipelines)
			{
				if (Pipeline->ShouldOptimizeUnusedOutputs(ShaderPlatform))
				{
					PipelinedShaderTypes.Append(Pipeline->GetStages());
				}
			}

			bAnyTypesExcluded = PipelinedShaderTypes.Num() != 0;
		}
	}

	inline bool IsPipelinedType(const FShaderType* Type) const
	{
		return bAnyTypesExcluded ? PipelinedShaderTypes.Contains(Type) : false;
	}
};

/** Converts an FMaterialShadingModelField to a string description containing all the shading models present, delimited by "|" */
FString GetShadingModelFieldString(FMaterialShadingModelField ShadingModels, const FShadingModelToStringDelegate& Delegate, const FString& Delimiter)
{
	FString ShadingModelsName;
	uint32 TempShadingModels = (uint32)ShadingModels.GetShadingModelField();

	while (TempShadingModels)
	{
		uint32 BitIndex = FMath::CountTrailingZeros(TempShadingModels); // Find index of first set bit
		TempShadingModels &= ~(1 << BitIndex); // Flip first set bit to 0
		ShadingModelsName += Delegate.Execute((EMaterialShadingModel)BitIndex); // Add the name of the shading model corresponding to that bit

		// If there are more bits left, add a pipe limiter to the string 
		if (TempShadingModels)
		{
			ShadingModelsName.Append(Delimiter);
		}
	}

	return ShadingModelsName;
}

/** Converts an FMaterialShadingModelField to a string description containing all the shading models present, delimited by "|" */
FString GetShadingModelFieldString(FMaterialShadingModelField ShadingModels)
{
	return GetShadingModelFieldString(ShadingModels, FShadingModelToStringDelegate::CreateStatic(&GetShadingModelString), TEXT("|"));
}

/** Converts an EBlendMode to a string description. */
FString GetBlendModeString(EBlendMode BlendMode)
{
	return FString(UMaterial::GetBlendModeString(BlendMode));
}

#if WITH_EDITOR
/** Creates a string key for the derived data cache given a shader map id. */
FString GetMaterialShaderMapKeyString(const FMaterialShaderMapId& ShaderMapId,
	const FConservativeShaderParameters& ShaderParameters, EShaderPlatform Platform,
	bool bIncludeKeyStringShaderDependencies)
{
	FString ShaderMapKeyString;
	ShaderMapKeyString.Reserve(1024 * 1024 * 2);
	FMaterialKeyGeneratorContext Context(ShaderMapKeyString, Platform);
	Context.SetFlags(EMaterialKeyInclude::ShaderDependencies, bIncludeKeyStringShaderDependencies);
	RecordOrEmitMaterialShaderMapKey(Context, ShaderMapId, ShaderParameters);
	ShaderMapKeyString.Shrink();

	return ShaderMapKeyString;
}

void RecordOrEmitMaterialShaderMapKey(FMaterialKeyGeneratorContext& Context,
	const FMaterialShaderMapId& ShaderMapId, const FConservativeShaderParameters& ShaderParameters)
{
	check(!Context.IsLoading());
	RecordOrEmitMaterialShaderMapKey(Context, const_cast<FMaterialShaderMapId&>(ShaderMapId),
		const_cast<FConservativeShaderParameters&>(ShaderParameters));
}

/**
 * Creates a ShaderKey for a given a shader map id, or saves and loads the fields of the ShaderMapId
 * for creating the ShaderKey later.
 */
void RecordOrEmitMaterialShaderMapKey(FMaterialKeyGeneratorContext& Context,
	FMaterialShaderMapId& ShaderMapId, FConservativeShaderParameters& ShaderParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RecordOrEmitMaterialShaderMapKey);

	FName Format = Context.GetShaderFormat();
	EShaderPlatform Platform = Context.GetShaderPlatform();

	// Emitting and recording are mutually exclusive and we rely on that
	check(!(Context.IsRecording() && Context.IsEmitting()));
	
	if (Context.IsRecording())
	{
		Context.RecordObjectStart("ShaderMapId");
		ShaderMapId.RecordAndEmit(Context);
		Context.RecordObjectEnd();

		Context.RecordObjectStart("ShaderParameters");
		ShaderParameters.Record(Context);
		Context.RecordObjectEnd();
	}
	else
	{
		FShaderKeyGenerator& KeyGen = Context.GetKeyGenIfEmittingChecked();

		KeyGen.AppendDebugText(TEXTVIEW("MATSM_"));
		KeyGen.Append(GetMaterialShaderMapDDCGuid());
		KeyGen.AppendSeparator();

		KeyGen.Append(Format);
		KeyGen.AppendSeparator();
		KeyGen.Append(GetTargetPlatformManagerRef().ShaderFormatVersion(Format));
		KeyGen.AppendSeparator();

		if (Context.HasAllFlags(EMaterialKeyInclude::Globals))
		{
			ShaderMapAppendKey(Platform, KeyGen);
		}
		ShaderMapId.RecordAndEmit(Context);

		FMemoryHasherBlake3 EnvHasher;

		// Hash any modifications applied to the compilation environment for each shadertype that will be compiled.
		// When the shadermap ID is constructed, individual shadertype dependencies are added for any pipelines, hence
		// we don't need to have equivalent hashing code for the pipeline dependencies.
		for (const FShaderTypeDependency& TypeDep : ShaderMapId.ShaderTypeDependencies)
		{
			const FShaderType* ShaderType = FindShaderTypeByName(TypeDep.ShaderTypeName);
			if (const FMaterialShaderType* MatShaderType = ShaderType->GetMaterialShaderType())
			{
				if (MatShaderType->ShouldCompilePermutation(Platform, ShaderParameters, TypeDep.PermutationId, ShaderMapId.GetPermutationFlags()))
				{
					FShaderCompilerEnvironment EnvModifications(EnvHasher);
					MatShaderType->SetupCompileEnvironment(Platform, ShaderParameters, TypeDep.PermutationId, ShaderMapId.GetPermutationFlags(), EnvModifications);
					EnvModifications.SerializeEverythingButFiles(EnvHasher);
				}
			}
			else if (const FMeshMaterialShaderType* MeshMatShaderType = ShaderType->GetMeshMaterialShaderType())
			{
				for (const FVertexFactoryTypeDependency& VFDep : ShaderMapId.VertexFactoryTypeDependencies)
				{
					const FVertexFactoryType* VFType = FindVertexFactoryType(VFDep.VertexFactoryTypeName);
					const bool bVFShouldCache = FMeshMaterialShaderType::ShouldCompileVertexFactoryPermutation(Platform, ShaderParameters, VFType, ShaderType, ShaderMapId.GetPermutationFlags());
					const bool bShaderShouldCache = MeshMatShaderType->ShouldCompilePermutation(Platform, ShaderParameters, VFType, TypeDep.PermutationId, ShaderMapId.GetPermutationFlags());
					if (bVFShouldCache && bShaderShouldCache)
					{
						FShaderCompilerEnvironment EnvModifications(EnvHasher);
						VFType->ModifyCompilationEnvironment(FVertexFactoryShaderPermutationParameters(Platform, ShaderParameters, VFType, ShaderType, ShaderMapId.GetPermutationFlags()), EnvModifications);
						MeshMatShaderType->SetupCompileEnvironment(Platform, ShaderParameters, VFType, TypeDep.PermutationId, ShaderMapId.GetPermutationFlags(), EnvModifications);
						EnvModifications.SerializeEverythingButFiles(EnvHasher);
					}
				}
			}
		}
		KeyGen.AppendDebugText(TEXTVIEW("_EMH_"));
		KeyGen.Append(EnvHasher.Finalize());
		KeyGen.AppendSeparator();

		if (Context.HasAllFlags(EMaterialKeyInclude::Globals))
		{
			FMaterialAttributeDefinitionMap::AppendDDCKey(KeyGen);
		}
		FShaderCompileUtilities::AppendGBufferDDCKey(Platform, KeyGen);
		FHLSLMaterialTranslator::AppendVersion(KeyGen, Platform);
	}
}

static UE::DerivedData::FCacheKey GetMaterialShaderMapKey(const FStringView MaterialShaderMapKey)
{
	static UE::DerivedData::FCacheBucket Bucket(ANSITEXTVIEW("MaterialShaderMap"), TEXTVIEW("MaterialShader"));
	return {Bucket, FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(MaterialShaderMapKey)))};
}

static UE::FSharedString GetMaterialShaderMapName(const FStringView MaterialPath, const FMaterialShaderMapId& ShaderMapId, const EShaderPlatform Platform)
{
	FName FeatureLevelName;
	GetFeatureLevelName(ShaderMapId.FeatureLevel, FeatureLevelName);
	return UE::FSharedString(WriteToString<256>(MaterialPath,
		TEXTVIEW(" ["), FDataDrivenShaderPlatformInfo::GetName(Platform),
		TEXTVIEW(", "), FeatureLevelName,
		TEXTVIEW(", "), LexToString(ShaderMapId.QualityLevel),
		TEXTVIEW("]")));
}


FCbWriter& operator<<(FCbWriter& Writer, const EShaderPermutationFlags& PermutationFlags)
{
	Writer << static_cast<uint32>(PermutationFlags);
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, EShaderPermutationFlags& OutPermutationFlags)
{
	OutPermutationFlags = EShaderPermutationFlags::None;
	uint32 IntValue = Field.AsUInt32();
	if (Field.HasError())
	{
		return false;
	}
	OutPermutationFlags = static_cast<EShaderPermutationFlags>(IntValue);
	return true;
}

#endif // WITH_EDITOR

#if WITH_EDITOR

namespace UE::MaterialInterface::Private
{

void FRecordedMaterialResourceForCooking::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << ShaderPlatform;
	Writer << ExpressionIncludes;
	Writer.BeginObject();
	FMaterialKeyGeneratorContext KeyGenContext(Writer, ShaderPlatform);
	KeyGenContext.RemoveFlags(EMaterialKeyInclude::Globals | EMaterialKeyInclude::UObjectData);
	RecordOrEmitMaterialShaderMapKey(KeyGenContext, ShaderMapId, ShaderParameters);
	Writer.EndObject();
	Writer.EndArray();
}

bool FRecordedMaterialResourceForCooking::TryLoad(FCbFieldView Field)
{
	*this = FRecordedMaterialResourceForCooking();

	FCbFieldViewIterator ElementIter = Field.CreateViewIterator();
	if (!LoadFromCompactBinary(ElementIter++, ShaderPlatform))
	{
		return false;
	}
	if (!LoadFromCompactBinary(ElementIter++, ExpressionIncludes))
	{
		return false;
	}
	FCbObjectView ObjectView = ElementIter.AsObjectView();
	if ((ElementIter++).HasError())
	{
		return false;
	}
	FMaterialKeyGeneratorContext KeyGenContext(ObjectView, ShaderPlatform);
	KeyGenContext.RemoveFlags(EMaterialKeyInclude::Globals | EMaterialKeyInclude::UObjectData);
	RecordOrEmitMaterialShaderMapKey(KeyGenContext, ShaderMapId, ShaderParameters);
	if (KeyGenContext.HasLoadError())
	{
		return false;
	}
	return true;
}

} // namespace UE::MaterialInterface::Private

bool LoadFromCompactBinary(FCbFieldView Field,
	UE::MaterialInterface::Private::FRecordedMaterialResourceForCooking& OutValue)
{
	return OutValue.TryLoad(Field);
}

namespace UE::MaterialInterface::Private
{

// DDC Key for Material structs saved in incremental cook artifacts, bump when changing related classes which
// can be loaded/saved to CompactBinary
#define MATERIAL_CLASS_INCREMENTALCOOK_VER TEXT("961F0483-145A-4C7A-8DC3-60C382D81DDE")

TMap<EShaderPlatform, FBlake3Hash> GShaderMapPlatformHashes;
FRWLock GShaderMapPlatformHashesLock;

FBlake3Hash GetShaderPlatformIncrementalCookHash(EShaderPlatform ShaderPlatform)
{
	{
		FReadScopeLock ScopeLock(GShaderMapPlatformHashesLock);
		const FBlake3Hash* ExistingHash = GShaderMapPlatformHashes.Find(ShaderPlatform);
		if (ExistingHash)
		{
			return *ExistingHash;
		}
	}

	FBlake3 Hasher;
	FShaderKeyGenerator KeyGen([&Hasher](const void* Data, uint64 Size)
		{
			Hasher.Update(Data, Size);
		});
	ShaderMapAppendKey(ShaderPlatform, KeyGen);
	FBlake3Hash Hash = Hasher.Finalize();

	FWriteScopeLock ScopeLock(GShaderMapPlatformHashesLock);
	GShaderMapPlatformHashes.FindOrAdd(ShaderPlatform, Hash);
	return Hash;
}

void HashMaterialStaticClassDependenciesForCook(FAppendToClassSchemaContext& Context)
{
	Context.Update(&GetMaterialShaderMapDDCGuid(), sizeof(GetMaterialShaderMapDDCGuid()));

	FShaderKeyGenerator KeyGen([&Context](const void* Data, uint64 Size) { Context.Update(Data, Size); });
	FMaterialAttributeDefinitionMap::AppendDDCKey(KeyGen);

	// Record the data used by all FPlatformTypeLayoutParameters by appending a default-constructed one.
	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.Append(KeyGen);

	bool bSupportCookedEditorConfigValue = false;
	GConfig->GetBool(TEXT("CookedEditorSettings"), TEXT("bSupportCookedEditor"), bSupportCookedEditorConfigValue, GGameIni);
	uint8 Byte = bSupportCookedEditorConfigValue ? 1 : 0;
	Context.Update(&Byte, sizeof(Byte));

	for (EMaterialQualityLevel::Type QualityLevel = static_cast<EMaterialQualityLevel::Type>(0);
		QualityLevel < EMaterialQualityLevel::Type::Num;
		QualityLevel = static_cast<EMaterialQualityLevel::Type>(static_cast<uint32>(QualityLevel) + 1))
	{
		TStringBuilder<32> QualityLevelName(InPlace, GetMaterialQualityLevelFName(QualityLevel));
		Context.Update(*QualityLevelName, QualityLevelName.Len() * sizeof((*QualityLevelName)[0]));
	}
	for (ERHIFeatureLevel::Type FeatureLevel = static_cast<ERHIFeatureLevel::Type>(0);
		FeatureLevel < ERHIFeatureLevel::Type::Num;
		FeatureLevel = static_cast<ERHIFeatureLevel::Type>(static_cast<uint32>(FeatureLevel) + 1))
	{
		FName FeatureLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);
		TStringBuilder<32> FeatureLevelNameStr(InPlace, FeatureLevelName);
		Context.Update(*FeatureLevelNameStr, FeatureLevelNameStr.Len() * sizeof((*FeatureLevelNameStr)[0]));
	}

	FStringView VersionString(MATERIAL_CLASS_INCREMENTALCOOK_VER);
	Context.Update(VersionString.GetData(), VersionString.Len() * sizeof(VersionString[0]));
}

constexpr int32 HashMaterialDependenciesForCookArgsVersion = 1;
void HashMaterialDependenciesForCook(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context)
{
	int32 ArgsVersion = -1;
	bool bValid = false;

	TArray<FRecordedMaterialResourceForCooking> CookResources;
	FCbFieldViewIterator ArgField(Args);
	ArgsVersion = (ArgField++).AsInt32();
	if (ArgsVersion == HashMaterialDependenciesForCookArgsVersion)
	{
		bValid = LoadFromCompactBinary(ArgField++, CookResources);
	}
	if (!bValid)
	{
		Context.LogError(FString::Printf(TEXT("Unsupported arguments version %d."), ArgsVersion));
		return;
	}

	for (FRecordedMaterialResourceForCooking& Resource : CookResources)
	{
		FMaterialKeyGeneratorContext KeyGenContext([&Context](const void* Data, uint64 Size)
			{
				Context.Update(Data, Size);
			}, Resource.ShaderPlatform);
		KeyGenContext.RemoveFlags(EMaterialKeyInclude::Globals | EMaterialKeyInclude::UObjectData);
		RecordOrEmitMaterialShaderMapKey(KeyGenContext, Resource.ShaderMapId, Resource.ShaderParameters);
		for (const FString& FileName : Resource.ExpressionIncludes)
		{
			const FSHAHash* Hash = TryGetShaderFileHash(*FileName, Resource.ShaderPlatform);
			if (!Hash)
			{
				Context.LogError(FString::Printf(TEXT("Could not find file %s for hashing."), *FileName));
				continue;
			}
			Context.Update(&Hash->Hash, sizeof(Hash->Hash));
		}
		const FMaterialShaderMapLayout& Layout = AcquireMaterialShaderMapLayout(Resource.ShaderPlatform,
			GetShaderPermutationFlags(Resource.ShaderMapId.LayoutParams), Resource.ShaderParameters);
		Context.Update(&Layout.ShaderCompileHash.Hash, sizeof(Layout.ShaderCompileHash.Hash));

		FBlake3Hash ShaderMapPlatformHash = GetShaderPlatformIncrementalCookHash(Resource.ShaderPlatform);
		Context.Update(&ShaderMapPlatformHash.GetBytes(), sizeof(ShaderMapPlatformHash.GetBytes()));
	}
}

UE_COOK_DEPENDENCY_FUNCTION(HashMaterialDependenciesForCook, UE::MaterialInterface::Private::HashMaterialDependenciesForCook);

void RecordMaterialDependenciesForCook(UE::Cook::FCookEventContext CookContext,
	TConstArrayView<FMaterialResourceForCooking> Resources)
{
	const ITargetPlatform* TargetPlatform = CookContext.GetTargetPlatform();
	check(TargetPlatform); // Caller should call only when cooking

	TArray<FRecordedMaterialResourceForCooking> RecordedResources;
	RecordedResources.Reserve(Resources.Num());

	// Sort the array of resources for determinism
	TArray<FMaterialResourceForCooking> SortedResources(Resources);
	Algo::Sort(SortedResources, [](const FMaterialResourceForCooking& A, const FMaterialResourceForCooking& B)
		{
			return static_cast<uint32>(A.Platform) < static_cast<uint32>(B.Platform);
		});

	// Calculate the ShaderMapId for each resource and save it into an array of FRecordedMaterialResourceForCooking
	// that is suitable for persistent saving as parameters of HashMaterialDependenciesForCook.
	for (const FMaterialResourceForCooking& ResourceForCooking : SortedResources)
	{
		const FMaterialResource& Resource = *ResourceForCooking.Resource;

		FRecordedMaterialResourceForCooking& RecordedResource = RecordedResources.Emplace_GetRef();
		RecordedResource.ShaderPlatform = ResourceForCooking.Platform;
		RecordedResource.ShaderParameters = FConservativeShaderParameters(&Resource);

		EMaterialKeyInclude IncludeFlags = EMaterialKeyInclude::All & ~EMaterialKeyInclude::Globals;
		Resource.BuildShaderMapId(RecordedResource.ShaderMapId, TargetPlatform,
			IncludeFlags);

		TSet<FString>& ExpressionIncludes = Resource.GetCachedExpressionData().EditorOnlyData->ExpressionIncludeFilePaths;
		// ExpressionIncludeFilePaths is already deterministically sorted, so no need to sort it here.
		RecordedResource.ExpressionIncludes.Reserve(ExpressionIncludes.Num());
		for (const FString& FilePath : ExpressionIncludes)
		{
			RecordedResource.ExpressionIncludes.Add(FilePath);
		}
	}

	// Store the FRecordedMaterialResourceForCooking in compact binary and pass them to the cooker to save in 
	// CookContext's list of dependencies for our package.
	FCbWriter Writer;
	Writer << HashMaterialDependenciesForCookArgsVersion;
	Writer << RecordedResources;

	CookContext.AddLoadBuildDependency(
		UE::Cook::FCookDependency::Function(
			UE_COOK_DEPENDENCY_FUNCTION_CALL(HashMaterialDependenciesForCook), Writer.Save()));
}

} // namespace UE::MaterialInterface::Private

namespace UE::MaterialTranslatorUtils
{

const TCHAR* GetVTAddressMode(TextureAddress Address)
{
	switch (Address)
	{
		case TA_Wrap: return TEXT("VTADDRESSMODE_WRAP");
		case TA_Clamp: return TEXT("VTADDRESSMODE_CLAMP");
		case TA_Mirror: return TEXT("VTADDRESSMODE_MIRROR");
		default: checkNoEntry(); return nullptr;
	}
}

void GetTextureAddressForSamplerSource(const UTexture* InTexture, ESamplerSourceMode InSamplerSource, TextureAddress& OutStaticAddressX, TextureAddress& OutStaticAddressY, TextureAddress& OutStaticAddressZ)
{
	// StaticAddress mode at time of compile
	// This may not be 100% correct, if SamplerSource is set to 'SSM_FromTextureAsset', as texture parameter may change the address mode in a derived instance
	OutStaticAddressX = TA_Wrap;
	OutStaticAddressY = TA_Wrap;
	OutStaticAddressZ = TA_Wrap;

	if (InTexture && InTexture->Source.GetNumBlocks() > 1)
	{
		// UDIM (multi-block) texture are forced to use wrap address mode
		// This is important for supporting VT stacks made from UDIMs with differing number of blocks, as this requires wrapping vAddress for certain layers
		OutStaticAddressX = TA_Wrap;
		OutStaticAddressY = TA_Wrap;
		OutStaticAddressZ = TA_Wrap;
	}
	else
	{
		switch (InSamplerSource)
		{
			case SSM_FromTextureAsset:
				if (InTexture)
				{
					OutStaticAddressX = InTexture->GetTextureAddressX();
					OutStaticAddressY = InTexture->GetTextureAddressY();
					OutStaticAddressZ = InTexture->GetTextureAddressZ();
				}
				break;
			case SSM_Wrap_WorldGroupSettings:
				OutStaticAddressX = TA_Wrap;
				OutStaticAddressY = TA_Wrap;
				OutStaticAddressZ = TA_Wrap;
				break;
			case SSM_Clamp_WorldGroupSettings:
			case SSM_TerrainWeightmapGroupSettings:
				OutStaticAddressX = TA_Clamp;
				OutStaticAddressY = TA_Clamp;
				OutStaticAddressZ = TA_Clamp;
				break;
			default:
				checkNoEntry();
				break;
			}
	}
}

bool IsLWCEnabled()
{
	static IConsoleVariable* CVarLWCIsEnabled = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MaterialEditor.LWCEnabled"));
	const bool bLWCIsEnabled = CVarLWCIsEnabled ? CVarLWCIsEnabled->GetBool() : true;
	return bLWCIsEnabled;
}

int32 GetLWCTruncateMode()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MaterialEditor.LWCTruncateMode"));
	const int32 LWCTruncateValue = CVar ? CVar->GetValueOnAnyThread() : 0;
	return LWCTruncateValue;
}

EMaterialCommonBasis GetMaterialCommonBasis(EMaterialVectorCoordTransformSource X)
{
	static const EMaterialCommonBasis ConversionTable[] = {
		MCB_Tangent,					// TRANSFORMSOURCE_Tangent
		MCB_Local,						// TRANSFORMSOURCE_Local
		MCB_World,						// TRANSFORMSOURCE_World
		MCB_View,						// TRANSFORMSOURCE_View
		MCB_Camera,						// TRANSFORMSOURCE_Camera
		MCB_MeshParticle,				// TRANSFORMSOURCE_Particle
		MCB_Instance,					// TRANSFORMSOURCE_Instance
	};
	static_assert(
		EMaterialVectorCoordTransformSource::TRANSFORMSOURCE_MAX == UE_ARRAY_COUNT(ConversionTable),
		"Mapping of enum EMaterialVectorCoordTransformSource to EMaterialCommonBasis is out of sync; ConversionTable[] array needs to be updated!");
	return ConversionTable[X];
}

EMaterialCommonBasis GetMaterialCommonBasis(EMaterialVectorCoordTransform X)
{
	static const EMaterialCommonBasis ConversionTable[] = {
		MCB_Tangent,					// TRANSFORM_Tangent
		MCB_Local,						// TRANSFORM_Local
		MCB_World,						// TRANSFORM_World
		MCB_View,						// TRANSFORM_View
		MCB_Camera,						// TRANSFORM_Camera
		MCB_MeshParticle,				// TRANSFORM_Particle
		MCB_Instance,					// TRANSFORM_Instance
	};
	static_assert(
		EMaterialVectorCoordTransform::TRANSFORM_MAX == UE_ARRAY_COUNT(ConversionTable),
		"Mapping of enum EMaterialVectorCoordTransform to EMaterialCommonBasis is out of sync; ConversionTable[] array needs to be updated!");
	return ConversionTable[X];
}

EMaterialCommonBasis GetMaterialCommonBasis(EMaterialPositionTransformSource X)
{
	static const EMaterialCommonBasis ConversionTable[] = {
		MCB_Local,						// TRANSFORMPOSSOURCE_Local
		MCB_World,						// TRANSFORMPOSSOURCE_World
		MCB_PeriodicWorld,				// TRANSFORMPOSSOURCE_PeriodicWorld
		MCB_TranslatedWorld,			// TRANSFORMPOSSOURCE_TranslatedWorld
		MCB_FirstPerson,				// TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld
		MCB_View,						// TRANSFORMPOSSOURCE_View
		MCB_Camera,						// TRANSFORMPOSSOURCE_Camera
		MCB_MeshParticle,				// TRANSFORMPOSSOURCE_Particle
		MCB_Instance,					// TRANSFORMPOSSOURCE_Instance
	};
	static_assert(
		EMaterialPositionTransformSource::TRANSFORMPOSSOURCE_MAX == UE_ARRAY_COUNT(ConversionTable),
		"Mapping of enum EMaterialPositionTransformSource to EMaterialCommonBasis is out of sync; ConversionTable[] array needs to be updated!");
	return ConversionTable[X];
}

bool IsFloatPowerOfTwo(float InValue)
{
	return ((*reinterpret_cast<int*>(&InValue)) & 0x007FFFFF) == 0; // zero mantisse
}

} // namespace UE::MaterialTranslatorUtils

#endif // WITH_EDITOR

/** Called for every material shader to update the appropriate stats. */
void UpdateMaterialShaderCompilingStats(const FMaterial* Material)
{
	INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumTotalMaterialShaders,1);

	switch(Material->GetBlendMode())
	{
		case BLEND_Opaque: INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumOpaqueMaterialShaders,1); break;
		case BLEND_Masked: INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumMaskedMaterialShaders,1); break;
		default: INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumTransparentMaterialShaders,1); break;
	}

	FMaterialShadingModelField ShadingModels = Material->GetShadingModels();
	
	if (ShadingModels.HasOnlyShadingModel(MSM_Unlit))
	{
		INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumUnlitMaterialShaders, 1);
	}
	else if (ShadingModels.HasAnyShadingModel({ MSM_DefaultLit, MSM_Subsurface, MSM_PreintegratedSkin, MSM_ClearCoat, MSM_Cloth, MSM_SubsurfaceProfile, MSM_TwoSidedFoliage, MSM_SingleLayerWater, MSM_ThinTranslucent }))
	{
		INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumLitMaterialShaders, 1);
	}


	if (Material->IsSpecialEngineMaterial())
	{
		INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumSpecialMaterialShaders,1);
	}
	if (Material->IsUsedWithParticleSystem())
	{
		INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumParticleMaterialShaders,1);
	}
	if (Material->IsUsedWithSkeletalMesh())
	{
		INC_DWORD_STAT_BY(STAT_ShaderCompiling_NumSkinnedMaterialShaders,1);
	}
}

void FStaticParameterBase::AppendKeyString(FString& KeyString) const
{
	FShaderKeyGenerator KeyGen(KeyString);
	Append(KeyGen);
}

void FStaticParameterBase::Append(FShaderKeyGenerator& KeyGen) const
{
	ParameterInfo.Append(KeyGen);
	KeyGen.AppendBoolInt(bOverride);
	KeyGen.Append(ExpressionGUID);
}

void FStaticSwitchParameter::AppendKeyString(FString& KeyString) const
{
	FShaderKeyGenerator KeyGen(KeyString);
	Append(KeyGen);
}

void FStaticSwitchParameter::Append(FShaderKeyGenerator& KeyGen) const
{
	FStaticParameterBase::Append(KeyGen);
	KeyGen.Append(Value);
}

void FStaticComponentMaskParameter::AppendKeyString(FString& KeyString) const
{
	FShaderKeyGenerator KeyGen(KeyString);
	Append(KeyGen);
}

void FStaticComponentMaskParameter::Append(FShaderKeyGenerator& KeyGen) const
{
	FStaticParameterBase::Append(KeyGen);
	KeyGen.AppendBoolInt(R);
	KeyGen.AppendBoolInt(G);
	KeyGen.AppendBoolInt(B);
	KeyGen.AppendBoolInt(A);
}

void FStaticTerrainLayerWeightParameter::AppendKeyString(FString& KeyString) const
{
	FShaderKeyGenerator KeyGen(KeyString);
	Append(KeyGen);
}

void FStaticTerrainLayerWeightParameter::Append(FShaderKeyGenerator& KeyGen) const
{
	KeyGen.Append(LayerName);
	KeyGen.Append(WeightmapIndex);
}

FStaticParameterSet::FStaticParameterSet(const FStaticParameterSet& InValue) = default;

FStaticParameterSet& FStaticParameterSet::operator=(const FStaticParameterSet& InValue)
{
	StaticSwitchParameters = InValue.StaticSwitchParameters;
#if WITH_EDITORONLY_DATA
	EditorOnly.StaticComponentMaskParameters = InValue.EditorOnly.StaticComponentMaskParameters;
	EditorOnly.TerrainLayerWeightParameters = InValue.EditorOnly.TerrainLayerWeightParameters;
#endif // WITH_EDITORONLY_DATA
	MaterialLayers = InValue.MaterialLayers;
	bHasMaterialLayers = InValue.bHasMaterialLayers;
	if (bHasMaterialLayers)
	{
		MaterialLayers = InValue.MaterialLayers;
#if WITH_EDITORONLY_DATA
		EditorOnly.MaterialLayers = InValue.EditorOnly.MaterialLayers;
#endif
	}
	return *this;
}

void FStaticParameterSet::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	if (StaticSwitchParameters_DEPRECATED.Num() > 0)
	{
		EditorOnly.StaticSwitchParameters_DEPRECATED = MoveTemp(StaticSwitchParameters_DEPRECATED);
	}
	if (EditorOnly.StaticSwitchParameters_DEPRECATED.Num() > 0)
	{
		StaticSwitchParameters = MoveTemp(EditorOnly.StaticSwitchParameters_DEPRECATED);
	}
	if (StaticComponentMaskParameters_DEPRECATED.Num() > 0)
	{
		EditorOnly.StaticComponentMaskParameters = MoveTemp(StaticComponentMaskParameters_DEPRECATED);
	}
	if (TerrainLayerWeightParameters_DEPRECATED.Num() > 0)
	{
		EditorOnly.TerrainLayerWeightParameters = MoveTemp(TerrainLayerWeightParameters_DEPRECATED);
	}
	// If we serialized a legacy 'MaterialLayers' property from a FMaterialLayersFunctions property, capture the editor-only portion here
	if (MaterialLayers.LegacySerializedEditorOnlyData)
	{
		EditorOnly.MaterialLayers = MoveTemp(*MaterialLayers.LegacySerializedEditorOnlyData);
		MaterialLayers.LegacySerializedEditorOnlyData.Reset();
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void FStaticParameterSet::SerializeLegacy(FArchive& Ar)
{
	// Old UMaterialInstances may use this path to serialize their 'StaticParameters' (newer assets will use automatic tagged serialization)
	// Even older UMaterialInstances may serialize FMaterialShaderMapId, which will potentially use this path as well (newer FMaterialShaderMapId do not serialize FStaticParameterSet directly)
	// In both cases, the data will be loaded from uasset, so backwards compatibility is required
	// New assets should *not* use this path, so this doesn't need to handle future version changes

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	Ar << EditorOnly.StaticSwitchParameters_DEPRECATED;
	Ar << EditorOnly.StaticComponentMaskParameters;
	Ar << EditorOnly.TerrainLayerWeightParameters;

	if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::MaterialLayersParameterSerializationRefactor)
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::MaterialLayerStacksAreNotParameters)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Ar << MaterialLayersParameters_DEPRECATED;
			if (MaterialLayersParameters_DEPRECATED.Num() > 0)
			{
				bHasMaterialLayers = true;
				MaterialLayers = MoveTemp(MaterialLayersParameters_DEPRECATED[0].Value.GetRuntime());
				EditorOnly.MaterialLayers = MoveTemp(MaterialLayersParameters_DEPRECATED[0].Value.EditorOnly);
				MaterialLayersParameters_DEPRECATED.Empty();
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void FStaticParameterSet::UpdateLegacyTerrainLayerWeightData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	int32 ParameterIndex = 0;
	while (ParameterIndex < EditorOnly.TerrainLayerWeightParameters.Num())
	{
		FStaticTerrainLayerWeightParameter& TerrainParameter = EditorOnly.TerrainLayerWeightParameters[ParameterIndex];
		if (TerrainParameter.bOverride_DEPRECATED)
		{
			TerrainParameter.LayerName = TerrainParameter.ParameterInfo_DEPRECATED.Name;
			ParameterIndex++;
		}
		else
		{
			// Remove any parameters that didn't have bOverride set
			EditorOnly.TerrainLayerWeightParameters.RemoveAt(ParameterIndex);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FStaticParameterSet::UpdateLegacyMaterialLayersData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (MaterialLayersParameters_DEPRECATED.Num() > 0)
	{
		bHasMaterialLayers = true;
		MaterialLayers = MoveTemp(MaterialLayersParameters_DEPRECATED[0].Value.GetRuntime());
		EditorOnly.MaterialLayers = MoveTemp(MaterialLayersParameters_DEPRECATED[0].Value.EditorOnly);
		MaterialLayersParameters_DEPRECATED.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FStaticParameterSet::GetMaterialLayers(FMaterialLayersFunctions& OutMaterialLayers) const
{
	if (bHasMaterialLayers)
	{
		OutMaterialLayers.GetRuntime() = MaterialLayers;
		OutMaterialLayers.EditorOnly = EditorOnly.MaterialLayers;
		return true;
	}
	return false;
}

void FStaticParameterSet::Validate(const FStaticParameterSetRuntimeData& Runtime, const FStaticParameterSetEditorOnlyData& EditorOnly)
{
	FMaterialLayersFunctions::Validate(Runtime.MaterialLayers, EditorOnly.MaterialLayers);
}

#endif // WITH_EDITOR

bool FStaticParameterSet::operator==(const FStaticParameterSet& ReferenceSet) const
{
	if (bHasMaterialLayers != ReferenceSet.bHasMaterialLayers)
	{
		return false;
	}

	if(StaticSwitchParameters.Num() != ReferenceSet.StaticSwitchParameters.Num())
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (EditorOnly.StaticComponentMaskParameters.Num() != ReferenceSet.EditorOnly.StaticComponentMaskParameters.Num()
		|| EditorOnly.TerrainLayerWeightParameters.Num() != ReferenceSet.EditorOnly.TerrainLayerWeightParameters.Num())
	{
		return false;
	}

	if (EditorOnly.StaticComponentMaskParameters != ReferenceSet.EditorOnly.StaticComponentMaskParameters)
	{
		return false;
	}

	if (EditorOnly.TerrainLayerWeightParameters != ReferenceSet.EditorOnly.TerrainLayerWeightParameters)
	{
		return false;
	}
#endif // WITH_EDITORONLY_DATA

	if (StaticSwitchParameters != ReferenceSet.StaticSwitchParameters)
	{
		return false;
	}

	if (bHasMaterialLayers)
	{
		if (MaterialLayers != ReferenceSet.MaterialLayers)
		{
			return false;
		}
#if WITH_EDITORONLY_DATA
		if (EditorOnly.MaterialLayers != ReferenceSet.EditorOnly.MaterialLayers)
		{
			return false;
		}
#endif // WITH_EDITORONLY_DATA
	}

	return true;
}

void FStaticParameterSet::SortForEquivalent()
{
	StaticSwitchParameters.Sort([](const FStaticSwitchParameter& A, const FStaticSwitchParameter& B) { return B.ExpressionGUID < A.ExpressionGUID; });
#if WITH_EDITORONLY_DATA
	EditorOnly.StaticComponentMaskParameters.Sort([](const FStaticComponentMaskParameter& A, const FStaticComponentMaskParameter& B) { return B.ExpressionGUID < A.ExpressionGUID; });
	EditorOnly.TerrainLayerWeightParameters.Sort([](const FStaticTerrainLayerWeightParameter& A, const FStaticTerrainLayerWeightParameter& B) { return B.LayerName.LexicalLess(A.LayerName); });
#endif // WITH_EDITORONLY_DATA
}

bool FStaticParameterSet::Equivalent(const FStaticParameterSet& ReferenceSet) const
{
	if (bHasMaterialLayers != ReferenceSet.bHasMaterialLayers)
	{
		return false;
	}

	if(StaticSwitchParameters.Num() != ReferenceSet.StaticSwitchParameters.Num())
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (EditorOnly.StaticComponentMaskParameters.Num() != ReferenceSet.EditorOnly.StaticComponentMaskParameters.Num()
		|| EditorOnly.TerrainLayerWeightParameters.Num() != ReferenceSet.EditorOnly.TerrainLayerWeightParameters.Num())
	{
		return false;
	}
#endif // WITH_EDITORONLY_DATA

	// this is not ideal, but it is easy to code up
	FStaticParameterSet Temp1 = *this;
	FStaticParameterSet Temp2 = ReferenceSet;
	Temp1.SortForEquivalent();
	Temp2.SortForEquivalent();
	return (Temp1 == Temp2);
}

#if WITH_EDITORONLY_DATA
void FStaticParameterSet::SetParameterValue(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags)
{
	const FMaterialParameterValue& Value = Meta.Value;
	switch (Value.Type)
	{
	case EMaterialParameterType::StaticSwitch: SetStaticSwitchParameterValue(ParameterInfo, Meta.ExpressionGuid, Value.AsStaticSwitch()); break;
	case EMaterialParameterType::StaticComponentMask: SetStaticComponentMaskParameterValue(ParameterInfo, Meta.ExpressionGuid, Value.Bool[0], Value.Bool[1], Value.Bool[2], Value.Bool[3]); break;
	default: checkNoEntry();
	}
}

void FStaticParameterSet::AddParametersOfType(EMaterialParameterType Type, const TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& Values)
{
	switch (Type)
	{
	case EMaterialParameterType::StaticSwitch:
		StaticSwitchParameters.Empty(Values.Num());
		for (const auto& It : Values)
		{
			const FMaterialParameterMetadata& Meta = It.Value;
			check(Meta.Value.Type == Type);
			if(!Meta.bDynamicSwitchParameter)
			{
				StaticSwitchParameters.Emplace(It.Key, Meta.Value.AsStaticSwitch(), Meta.bOverride, Meta.ExpressionGuid);
			}
		}
		break;
	case EMaterialParameterType::StaticComponentMask:
		EditorOnly.StaticComponentMaskParameters.Empty(Values.Num());
		for (const auto& It : Values)
		{
			const FMaterialParameterMetadata& Meta = It.Value;
			check(Meta.Value.Type == Type);
			EditorOnly.StaticComponentMaskParameters.Emplace(It.Key,
				Meta.Value.Bool[0],
				Meta.Value.Bool[1],
				Meta.Value.Bool[2],
				Meta.Value.Bool[3],
				Meta.bOverride,
				Meta.ExpressionGuid);
		}
		break;
	default: checkNoEntry();
	}
}

void FStaticParameterSet::SetStaticSwitchParameterValue(const FMaterialParameterInfo& ParameterInfo, const FGuid& ExpressionGuid, bool Value)
{
	for (FStaticSwitchParameter& Parameter : StaticSwitchParameters)
	{
		if (Parameter.ParameterInfo == ParameterInfo)
		{
			Parameter.bOverride = true;
			Parameter.Value = Value;
			return;
		}
	}

	new(StaticSwitchParameters) FStaticSwitchParameter(ParameterInfo, Value, true, ExpressionGuid);
}

void FStaticParameterSet::SetStaticComponentMaskParameterValue(const FMaterialParameterInfo& ParameterInfo, const FGuid& ExpressionGuid, bool R, bool G, bool B, bool A)
{
	for (FStaticComponentMaskParameter& Parameter : EditorOnly.StaticComponentMaskParameters)
	{
		if (Parameter.ParameterInfo == ParameterInfo)
		{
			Parameter.bOverride = true;
			Parameter.R = R;
			Parameter.G = G;
			Parameter.B = B;
			Parameter.A = A;
			return;
		}
	}

	new(EditorOnly.StaticComponentMaskParameters) FStaticComponentMaskParameter(ParameterInfo, R, G, B, A, true, ExpressionGuid);
}
#endif // WITH_EDITORONLY_DATA



#if WITH_EDITOR
FString FSubstrateCompilationConfig::GetShaderMapKeyString() const
{
	FString SubStrateCompStr;
	FShaderKeyGenerator KeyGen(SubStrateCompStr);
	Append(KeyGen);
	return SubStrateCompStr;
}
void FSubstrateCompilationConfig::Append(FShaderKeyGenerator& KeyGen) const
{
	if (bFullSimplify)
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SBSTRFS"));
	}
	if (BytesPerPixelOverride >= 0)
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SBSTRBS"));
	}
	if (ClosuresPerPixelOverride >= 0)
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SBSTRCS"));
	}
}

void FSubstrateCompilationConfig::UpdateHash(FXxHash64Builder& Hasher) const
{
	Hasher.Update((const uint8*)(&bFullSimplify), sizeof(bFullSimplify));
	Hasher.Update((const uint8*)(&BytesPerPixelOverride), sizeof(BytesPerPixelOverride));
	Hasher.Update((const uint8*)(&ClosuresPerPixelOverride), sizeof(ClosuresPerPixelOverride));
}

void FSubstrateCompilationConfig::Serialize(FArchive& Ar)
{
	Ar << bFullSimplify;
	Ar << BytesPerPixelOverride;
	Ar << ClosuresPerPixelOverride;
}
#endif

void FMaterialShaderMapId::Serialize(FArchive& Ar, bool bLoadingCooked)
{
	SCOPED_LOADTIMER(FMaterialShaderMapId_Serialize);

	// Note: FMaterialShaderMapId is saved both in packages (legacy UMaterialInstance) and the DDC (FMaterialShaderMap)
	// Backwards compatibility only works with FMaterialShaderMapId's stored in packages.  
	// Only serialized in legacy packages if UEVer() < VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS
	// You must bump MATERIALSHADERMAP_DERIVEDDATA_VER as well if changing the serialization of FMaterialShaderMapId.
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	const bool bIsLegacyPackage = Ar.UEVer() < VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS;

	// Ensure saved content is correct
	check(!Ar.IsSaving() || IsContentValid());

#if WITH_EDITOR
	const bool bIsSavingCooked = Ar.IsSaving() && Ar.IsCooking();
	bIsCookedId = bLoadingCooked;

	if (!bIsSavingCooked && !bLoadingCooked)
	{
		uint32 UsageInt = Usage;
		Ar << UsageInt;
		Usage = (EMaterialShaderMapUsage::Type)UsageInt;

		if (Usage == EMaterialShaderMapUsage::MaterialExportCustomOutput)
		{
			Ar << UsageCustomOutput;
		}

		Ar << BaseMaterialId;
	}
#endif

	if (!bIsLegacyPackage)
	{
		static_assert(sizeof(QualityLevel) == 1, "If you change the size of QualityLevel, you must adjust this serialization code and bump MATERIALSHADERMAP_DERIVEDDATA_VER");
		Ar << (uint8&)QualityLevel;
		Ar << (int32&)FeatureLevel;
	}
	else
	{
		uint8 LegacyQualityLevel;
		Ar << LegacyQualityLevel;
	}

#if WITH_EDITOR
	if (!bIsSavingCooked && !bLoadingCooked)
	{
		if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MaterialShaderMapIdSerialization)
		{
			// Serialize using old path
			FStaticParameterSet ParameterSet;
			ParameterSet.SerializeLegacy(Ar);
			UpdateFromParameterSet(ParameterSet);
		}
		else
		{
			Ar << StaticSwitchParameters;
			Ar << StaticComponentMaskParameters;
			Ar << TerrainLayerWeightParameters;
			if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::MaterialLayerStacksAreNotParameters)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				TArray<FStaticMaterialLayersParameter::ID> MaterialLayersParameterIDs;
				Ar << MaterialLayersParameterIDs;
				if (MaterialLayersParameterIDs.Num() > 0)
				{
					MaterialLayersId = MoveTemp(MaterialLayersParameterIDs[0].Functions);
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			else
			{
				Ar << MaterialLayersId;
			}
		}

		Ar << ReferencedFunctions;

		if (Ar.UEVer() >= VER_UE4_COLLECTIONS_IN_SHADERMAPID)
		{
			Ar << ReferencedParameterCollections;
		}

		if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::AddedMaterialSharedInputs &&
			Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::RemovedMaterialSharedInputCollection)
		{
			TArray<FGuid> Deprecated;
			Ar << Deprecated;
		}

		Ar << ShaderTypeDependencies;
		if (!bIsLegacyPackage)
		{
			Ar << ShaderPipelineTypeDependencies;
		}
		Ar << VertexFactoryTypeDependencies;

		if (!bIsLegacyPackage)
		{
			Ar << TextureReferencesHash;
		}
		else
		{
			FSHAHash LegacyHash;
			Ar << LegacyHash;
			TextureReferencesHash = 0u;
		}

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::AddedMaterialExpressionIncludesHash)
		{
			Ar << ExpressionIncludesHash;
		}
		else
		{
			ExpressionIncludesHash = 0u;
		}

		if (Ar.UEVer() >= VER_UE4_MATERIAL_INSTANCE_BASE_PROPERTY_OVERRIDES)
		{
			Ar << BasePropertyOverridesHash;
		}
		else
		{
			BasePropertyOverridesHash = 0u;
		}

		if (!bIsLegacyPackage)
		{
			Ar << bUsingNewHLSLGenerator;
			Ar << PreferredShaderCodeFormat;
			Ar << ExternalCodeReferencesHash;
			Ar << BaseMaterialPathHash;
		}
		else
		{
			bUsingNewHLSLGenerator = false;
			PreferredShaderCodeFormat = NAME_None;
			ExternalCodeReferencesHash = 0u;
			BaseMaterialPathHash = 0u;
		}

		// SUBSTRATE_TODO We do not need to serialize FSubstrateCompilationConfig for now since this is only used when debugging in the editor.
		// However we might want to do that when compilation config will change between raster and path tracing for instance.
		// So currently, the shader map DDC key string won't be changing, but if the user toggles simplification on via the Material Editor it will cache a new map. 
		// In other words we only cache the simplified shader map version of the material in the editor (and not during cooks).
	}
	else
	{
		if (bIsSavingCooked)
		{
			// Saving cooked data, this should be valid
			CookedShaderMapIdHash = GetMaterialHash();
			checkf(CookedShaderMapIdHash != 0u, TEXT("Tried to save an invalid shadermap id hash during cook"));
		}
		
		Ar << CookedShaderMapIdHash;
	}
#else
	// Cooked so can assume this is valid
	Ar << CookedShaderMapIdHash;
	checkf(CookedShaderMapIdHash != 0u, TEXT("Loaded an invalid cooked shadermap id hash"));
#endif // WITH_EDITOR

	if (!bIsLegacyPackage)
	{
		Ar << LayoutParams;
	}
	else
	{
		LayoutParams.InitializeForCurrent();
	}

	// Ensure loaded content is correct
	check(!Ar.IsLoading() || IsContentValid());
}

#if WITH_EDITOR
/** Hashes the material-specific part of this shader map Id. */
uint64 FMaterialShaderMapId::GetMaterialHash(bool bWithStaticParameters) const
{
	check(IsContentValid());
	FXxHash64Builder HashState;

	HashState.Update((const uint8*)&Usage, sizeof(Usage));
	if (Usage == EMaterialShaderMapUsage::MaterialExportCustomOutput)
	{
		HashState.Update(*UsageCustomOutput, UsageCustomOutput.NumBytesWithoutNull());
	}

	HashState.Update((const uint8*)&BaseMaterialId, sizeof(BaseMaterialId));

	FString QualityLevelString;
	GetMaterialQualityLevelName(QualityLevel, QualityLevelString);
	HashState.Update(*QualityLevelString, QualityLevelString.NumBytesWithoutNull());

	HashState.Update((const uint8*)&FeatureLevel, sizeof(FeatureLevel));


	//Hash the static parameters
	if (bWithStaticParameters)
	{
		for (const FStaticSwitchParameter& StaticSwitchParameter : StaticSwitchParameters)
		{
			StaticSwitchParameter.UpdateHash(HashState);
		}
	}
	for (const FStaticComponentMaskParameter& StaticComponentMaskParameter : StaticComponentMaskParameters)
	{
		StaticComponentMaskParameter.UpdateHash(HashState);
	}
	for (const FStaticTerrainLayerWeightParameter& StaticTerrainLayerWeightParameter : TerrainLayerWeightParameters)
	{
		StaticTerrainLayerWeightParameter.UpdateHash(HashState);
	}
	if (MaterialLayersId)
	{
		MaterialLayersId->UpdateHash(HashState);
	}

	for (int32 FunctionIndex = 0; FunctionIndex < ReferencedFunctions.Num(); FunctionIndex++)
	{
		HashState.Update((const uint8*)&ReferencedFunctions[FunctionIndex], sizeof(ReferencedFunctions[FunctionIndex]));
	}

	for (int32 CollectionIndex = 0; CollectionIndex < ReferencedParameterCollections.Num(); CollectionIndex++)
	{
		HashState.Update((const uint8*)&ReferencedParameterCollections[CollectionIndex], sizeof(ReferencedParameterCollections[CollectionIndex]));
	}

	for (int32 VertexFactoryIndex = 0; VertexFactoryIndex < VertexFactoryTypeDependencies.Num(); VertexFactoryIndex++)
	{
		HashState.Update((const uint8*)&VertexFactoryTypeDependencies[VertexFactoryIndex].VFSourceHash, sizeof(VertexFactoryTypeDependencies[VertexFactoryIndex].VFSourceHash));
	}

	HashState.Update((const uint8*)&TextureReferencesHash, sizeof(TextureReferencesHash));

	HashState.Update((const uint8*)&ExpressionIncludesHash, sizeof(ExpressionIncludesHash));

	HashState.Update((const uint8*)&BasePropertyOverridesHash, sizeof(BasePropertyOverridesHash));

	HashState.Update((const uint8*)&bUsingNewHLSLGenerator, sizeof(bUsingNewHLSLGenerator));

	if (PreferredShaderCodeFormat != NAME_None)
	{
		const FString PreferredShaderCodeFormatString = PreferredShaderCodeFormat.ToString();
		HashState.Update((const uint8*)PreferredShaderCodeFormatString.GetCharArray().GetData(), PreferredShaderCodeFormatString.GetCharArray().NumBytes());
	}

	HashState.Update((const uint8*)&ExternalCodeReferencesHash, sizeof(ExternalCodeReferencesHash));

	SubstrateCompilationConfig.UpdateHash(HashState);

	return HashState.Finalize().Hash;
}
#endif // WITH_EDITOR

/** 
* Tests this set against another for equality.
* 
* @param ReferenceSet	The set to compare against
* @return				true if the sets are equal
*/
bool FMaterialShaderMapId::Equals(const FMaterialShaderMapId& ReferenceSet, bool bWithStaticParameters) const
{
	// Ensure data is in valid state for comparison
	check(IsContentValid() && ReferenceSet.IsContentValid());

#if WITH_EDITOR
	if (IsCookedId() != ReferenceSet.IsCookedId())
	{
		return false;
	}

	if (bUsingNewHLSLGenerator != ReferenceSet.bUsingNewHLSLGenerator)
	{
		return false;
	}

	if (PreferredShaderCodeFormat != ReferenceSet.PreferredShaderCodeFormat)
	{
		return false;
	}

	if (SubstrateCompilationConfig != ReferenceSet.SubstrateCompilationConfig)
	{
		return false;
	}

	if (!IsCookedId())
	{
		if (BaseMaterialId != ReferenceSet.BaseMaterialId)
		{
			return false;
		}
	}
	else
#endif
	{
		if (CookedShaderMapIdHash != ReferenceSet.CookedShaderMapIdHash)
		{
			return false;
		}
	}

#if WITH_EDITOR
	if (Usage != ReferenceSet.Usage
		|| UsageCustomOutput != ReferenceSet.UsageCustomOutput)
	{
		return false;
	}
#endif

	if (QualityLevel != ReferenceSet.QualityLevel
		|| FeatureLevel != ReferenceSet.FeatureLevel)
	{
		return false;
	}

	if (LayoutParams != ReferenceSet.LayoutParams)
	{
		return false;
	}

#if WITH_EDITOR
	if (!IsCookedId())
	{
		if ((bWithStaticParameters && StaticSwitchParameters.Num() != ReferenceSet.StaticSwitchParameters.Num())
			|| StaticComponentMaskParameters.Num() != ReferenceSet.StaticComponentMaskParameters.Num()
			|| TerrainLayerWeightParameters.Num() != ReferenceSet.TerrainLayerWeightParameters.Num()
			|| ReferencedFunctions.Num() != ReferenceSet.ReferencedFunctions.Num()
			|| ReferencedParameterCollections.Num() != ReferenceSet.ReferencedParameterCollections.Num()
			|| ShaderTypeDependencies.Num() != ReferenceSet.ShaderTypeDependencies.Num()
			|| ShaderPipelineTypeDependencies.Num() != ReferenceSet.ShaderPipelineTypeDependencies.Num()
			|| VertexFactoryTypeDependencies.Num() != ReferenceSet.VertexFactoryTypeDependencies.Num())
		{
			return false;
		}

		if ((bWithStaticParameters && StaticSwitchParameters != ReferenceSet.StaticSwitchParameters)
			|| StaticComponentMaskParameters != ReferenceSet.StaticComponentMaskParameters
			|| TerrainLayerWeightParameters != ReferenceSet.TerrainLayerWeightParameters
			|| MaterialLayersId != ReferenceSet.MaterialLayersId)
		{
			return false;
		}

		for (int32 RefFunctionIndex = 0; RefFunctionIndex < ReferenceSet.ReferencedFunctions.Num(); RefFunctionIndex++)
		{
			const FGuid& ReferenceGuid = ReferenceSet.ReferencedFunctions[RefFunctionIndex];

			if (ReferencedFunctions[RefFunctionIndex] != ReferenceGuid)
			{
				return false;
			}
		}

		for (int32 RefCollectionIndex = 0; RefCollectionIndex < ReferenceSet.ReferencedParameterCollections.Num(); RefCollectionIndex++)
		{
			const FGuid& ReferenceGuid = ReferenceSet.ReferencedParameterCollections[RefCollectionIndex];

			if (ReferencedParameterCollections[RefCollectionIndex] != ReferenceGuid)
			{
				return false;
			}
		}

		for (int32 ShaderIndex = 0; ShaderIndex < ShaderTypeDependencies.Num(); ShaderIndex++)
		{
			const FShaderTypeDependency& ShaderTypeDependency = ShaderTypeDependencies[ShaderIndex];

			if (ShaderTypeDependency != ReferenceSet.ShaderTypeDependencies[ShaderIndex])
			{
				return false;
			}
		}

		for (int32 ShaderPipelineIndex = 0; ShaderPipelineIndex < ShaderPipelineTypeDependencies.Num(); ShaderPipelineIndex++)
		{
			const FShaderPipelineTypeDependency& ShaderPipelineTypeDependency = ShaderPipelineTypeDependencies[ShaderPipelineIndex];

			if (ShaderPipelineTypeDependency != ReferenceSet.ShaderPipelineTypeDependencies[ShaderPipelineIndex])
			{
				return false;
			}
		}

		for (int32 VFIndex = 0; VFIndex < VertexFactoryTypeDependencies.Num(); VFIndex++)
		{
			const FVertexFactoryTypeDependency& VFDependency = VertexFactoryTypeDependencies[VFIndex];

			if (VFDependency != ReferenceSet.VertexFactoryTypeDependencies[VFIndex])
			{
				return false;
			}
		}

		if (TextureReferencesHash != ReferenceSet.TextureReferencesHash)
		{
			return false;
		}

		if (ExpressionIncludesHash != ReferenceSet.ExpressionIncludesHash)
		{
			return false;
		}

		if (BasePropertyOverridesHash != ReferenceSet.BasePropertyOverridesHash)
		{
			return false;
		}

		if (ExternalCodeReferencesHash != ReferenceSet.ExternalCodeReferencesHash)
		{
			return false;
		}

		if (BaseMaterialPathHash != ReferenceSet.BaseMaterialPathHash)
		{
			return false;
		}
	}
#endif // WITH_EDITOR

	return true;
}

/** Ensure content is valid - for example overrides are set deterministically for serialization and sorting */
bool FMaterialShaderMapId::IsContentValid() const
{
#if WITH_EDITOR
	if (!LayoutParams.IsInitialized())
	{
		return false;
	}

	//We expect overrides to be set to false
	for (const FStaticSwitchParameter& StaticSwitchParameter : StaticSwitchParameters)
	{
		if (StaticSwitchParameter.bOverride != false)
		{
			return false;
		}
	}
	for (const FStaticComponentMaskParameter& StaticComponentMaskParameter : StaticComponentMaskParameters)
	{
		if (StaticComponentMaskParameter.bOverride != false)
		{
			return false;
		}
	}
#endif // WITH_EDITOR
	return true;
}


#if WITH_EDITOR
void FMaterialShaderMapId::UpdateFromParameterSet(const FStaticParameterSet& StaticParameters)
{
	struct FStaticParameterCompare
	{
		bool operator()(const FStaticParameterBase& Lhs, const FStaticParameterBase& Rhs) const
		{
			if (Lhs.ParameterInfo.Association != Rhs.ParameterInfo.Association) return Lhs.ParameterInfo.Association < Rhs.ParameterInfo.Association;
			else if (Lhs.ParameterInfo.Index != Rhs.ParameterInfo.Index) return Lhs.ParameterInfo.Index < Rhs.ParameterInfo.Index;
			else return Lhs.ParameterInfo.Name.LexicalLess(Rhs.ParameterInfo.Name);
		}
	};

	struct FStaticTerrainLayerWeightParameterCompare
	{
		bool operator()(const FStaticTerrainLayerWeightParameter& Lhs, const FStaticTerrainLayerWeightParameter& Rhs) const
		{
			return Lhs.LayerName.LexicalLess(Rhs.LayerName);
		}
	};

	StaticSwitchParameters = StaticParameters.StaticSwitchParameters;
	StaticComponentMaskParameters = StaticParameters.EditorOnly.StaticComponentMaskParameters;
	TerrainLayerWeightParameters = StaticParameters.EditorOnly.TerrainLayerWeightParameters;
	if (StaticParameters.bHasMaterialLayers)
	{
		MaterialLayersId = StaticParameters.MaterialLayers.GetID(StaticParameters.EditorOnly.MaterialLayers);
	}

	// Sort the arrays by parameter name, ensure the ID is not influenced by the order
	StaticSwitchParameters.Sort(FStaticParameterCompare());
	StaticComponentMaskParameters.Sort(FStaticParameterCompare());
	TerrainLayerWeightParameters.Sort(FStaticTerrainLayerWeightParameterCompare());

	//since bOverrides aren't used to check id matches, make sure they're consistently set to false in the static parameter set as part of the id.
	//this ensures deterministic cook results, rather than allowing bOverride to be set in the shader map's copy of the id based on the first id used.
	for (FStaticSwitchParameter& StaticSwitchParameter : StaticSwitchParameters)
	{
		StaticSwitchParameter.bOverride = false;
	}
	for (FStaticComponentMaskParameter& StaticComponentMaskParameter : StaticComponentMaskParameters)
	{
		StaticComponentMaskParameter.bOverride = false;
	}
}	

uint32 GetTypeHash(FPlatformTypeLayoutParameters Params) { return HashCombine(Params.Flags, Params.MaxFieldAlignment); }

void FMaterialShaderMapId::AppendStaticParameters(FShaderKeyGenerator& KeyGen) const
{
	for (const FStaticSwitchParameter& StaticSwitchParameter : StaticSwitchParameters)
	{
		StaticSwitchParameter.Append(KeyGen);
	}
	for (const FStaticComponentMaskParameter& StaticComponentMaskParameter : StaticComponentMaskParameters)
	{
		StaticComponentMaskParameter.Append(KeyGen);
	}
	for (const FStaticTerrainLayerWeightParameter& StaticTerrainLayerWeightParameter : TerrainLayerWeightParameters)
	{
		StaticTerrainLayerWeightParameter.Append(KeyGen);
	}
}

void FMaterialShaderMapId::AppendKeyString(FString& KeyString, bool bIncludeSourceAndMaterialState, bool bIncludeKeyStringShaderDependencies) const
{
	FMaterialKeyGeneratorContext Context(KeyString, EShaderPlatform::SP_NumPlatforms);
	Context.SetFlags(EMaterialKeyInclude::SourceAndMaterialState, bIncludeSourceAndMaterialState);
	Context.SetFlags(EMaterialKeyInclude::ShaderDependencies, bIncludeKeyStringShaderDependencies);
	const_cast<FMaterialShaderMapId&>(*this).RecordAndEmit(Context);
}

extern bool GAllowSkipAlphaComputationForMaterialBlend;

void FMaterialShaderMapId::RecordAndEmit(FMaterialKeyGeneratorContext& Context)
{
	check(Context.IsLoading() || IsContentValid());
	if (Context.HasAllFlags(EMaterialKeyInclude::SourceAndMaterialState | EMaterialKeyInclude::UObjectData))
	{
		Context.RecordAndEmit("BaseMaterialId", BaseMaterialId);
		Context.EmitSeparator();
	}

	if (Context.HasAllFlags(EMaterialKeyInclude::UObjectData))
	{
		Context.Record("QualityLevel", QualityLevel);
		Context.Emit(GetMaterialQualityLevelFName(QualityLevel));
		Context.EmitSeparator();

		Context.Record("FeatureLevel", FeatureLevel);
		FName FeatureLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);
		Context.Emit(FeatureLevelName);
		Context.EmitSeparator();
	}

	Context.RecordAndEmit("LayoutParams", LayoutParams);
	Context.RecordAndEmit("BaseMaterialPathHash", BaseMaterialPathHash);

	// Recording of some fields necessary for IncludeUObjectData is not yet implemented; we only support emitting them
	check(!Context.HasAllFlags(EMaterialKeyInclude::UObjectData) || !Context.IsRecording());
	if (Context.HasAllFlags(EMaterialKeyInclude::UObjectData) && Context.IsEmitting())
	{
		FShaderKeyGenerator& KeyGen = Context.GetKeyGenIfEmittingChecked();
		AppendStaticParameters(KeyGen);

		if (MaterialLayersId)
		{
			MaterialLayersId->Append(KeyGen);
		}

		KeyGen.AppendSeparator();
		KeyGen.Append(static_cast<int32>(Usage));
		KeyGen.AppendSeparator();

		if (Usage == EMaterialShaderMapUsage::MaterialExportCustomOutput)
		{
			KeyGen.Append(UsageCustomOutput);
			KeyGen.AppendSeparator();
		}

		if (Context.HasAllFlags(EMaterialKeyInclude::SourceAndMaterialState))
		{
			// Add any referenced functions to the key so that we will recompile when they are changed
			::Append(KeyGen, ReferencedFunctions);
		}
	}

	if (Context.HasAllFlags(EMaterialKeyInclude::Globals))
	{
		const FSHAHash LayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FMaterialShaderMapContent>(), LayoutParams);
		Context.EmitSeparator();
		Context.Emit(LayoutHash);
		Context.EmitSeparator();
	}

	Context.EmitSeparator();

	if (Context.HasAllFlags(EMaterialKeyInclude::SourceAndMaterialState | EMaterialKeyInclude::UObjectData))
	{
		Context.RecordAndEmit("ReferencedParameterCollections", ReferencedParameterCollections);
	}

	// Add the inputs for any shaders that are stored inline in the shader map
	// Callers can use the Dependencies arrays, so record them all even if not HasAllFlags(ShaderDependencies)
	Context.Record("ShaderTypeDependencies", ShaderTypeDependencies);
	Context.Record("ShaderPipelineTypeDependencies", ShaderPipelineTypeDependencies);
	Context.Record("VertexFactoryTypeDependencies", VertexFactoryTypeDependencies);
	Context.PostLoad([this, ShaderPlatform=Context.GetShaderPlatform()]()
		{
			for (FShaderTypeDependency& Value : ShaderTypeDependencies)
			{
				Value.RefreshCachedSourceHash(ShaderPlatform);
			}
			for (FShaderPipelineTypeDependency& Value : ShaderPipelineTypeDependencies)
			{
				Value.RefreshCachedSourceHash(ShaderPlatform);
			}
			for (FVertexFactoryTypeDependency& Value : VertexFactoryTypeDependencies)
			{
				Value.RefreshCachedSourceHash(ShaderPlatform);
			}
		});
	if (Context.HasAllFlags(EMaterialKeyInclude::ShaderDependencies) && Context.IsEmitting())
	{
		FShaderKeyGenerator& KeyGen = Context.GetKeyGenIfEmittingChecked();
		AppendShaderDependencies(
			KeyGen,
			MakeArrayView(ShaderTypeDependencies),
			MakeArrayView(ShaderPipelineTypeDependencies),
			MakeArrayView(VertexFactoryTypeDependencies),
			LayoutParams,
			Context.HasAllFlags(EMaterialKeyInclude::SourceAndMaterialState));
	}

	// Recording of these hashes is not useful, because the hashes do not include the list of files that contributed
	// to the hash.
	check(!Context.HasAllFlags(EMaterialKeyInclude::UObjectData) || !Context.IsRecording());
	if (Context.HasAllFlags(EMaterialKeyInclude::UObjectData))
	{
		Context.RecordAndEmit("TextureReferencesHash", TextureReferencesHash);
		Context.RecordAndEmit("ExpressionIncludesHash", ExpressionIncludesHash);
		Context.RecordAndEmit("BasePropertyOverridesHash", BasePropertyOverridesHash);
		Context.RecordAndEmit("ExternalCodeReferencesHash", ExternalCodeReferencesHash);
	}

	Context.RecordAndEmit("bUsingNewHLSLGenerator", bUsingNewHLSLGenerator);

	if (PreferredShaderCodeFormat != NAME_None)
	{
		Context.RecordAndEmit("PreferredShaderCodeFormat", PreferredShaderCodeFormat);
	}

	// Recording of some fields necessary for IncludeUObjectData is not yet implemented; we only support emitting them
	check(!Context.HasAllFlags(EMaterialKeyInclude::UObjectData) || !Context.IsRecording());
	if (Context.HasAllFlags(EMaterialKeyInclude::UObjectData) && Context.IsEmitting())
	{
		FShaderKeyGenerator& KeyGen = Context.GetKeyGenIfEmittingChecked();
		SubstrateCompilationConfig.Append(KeyGen);
	}

	// Changing these CVar should trigger a re-translate operation
	{
		uint32 MiscFlags = 0;
		if (UseMaterialVSUniformBuffer())
		{
			MiscFlags |= 1 << 0;
		}

		if (GAllowSkipAlphaComputationForMaterialBlend)
		{
			MiscFlags |= 1 << 1;
		}

		Context.Emit(MiscFlags);
	}
}

void FMaterialShaderMapId::SetShaderDependencies(const TArray<FShaderType*>& ShaderTypes, const TArray<const FShaderPipelineType*>& ShaderPipelineTypes, const TArray<FVertexFactoryType*>& VFTypes, EShaderPlatform ShaderPlatform)
{
	if (!FPlatformProperties::RequiresCookedData() && AllowShaderCompiling())
	{
		for (const FShaderType* ShaderType : ShaderTypes)
		{
			FShaderTypeDependency Dependency;
			Dependency.ShaderTypeName = ShaderType->GetHashedName();
			Dependency.SourceHash = ShaderType->GetSourceHash(ShaderPlatform);
			for (int32 Id = 0; Id < ShaderType->GetPermutationCount(); ++Id)
			{
				Dependency.PermutationId = Id;
				ShaderTypeDependencies.Add(Dependency);
			}
		}

		for (const FVertexFactoryType* VFType : VFTypes)
		{
			FVertexFactoryTypeDependency Dependency;
			Dependency.VertexFactoryTypeName = VFType->GetHashedName();
			Dependency.VFSourceHash = VFType->GetSourceHash(ShaderPlatform);
			VertexFactoryTypeDependencies.Add(Dependency);
		}

		for (const FShaderPipelineType* Pipeline : ShaderPipelineTypes)
		{
			FShaderPipelineTypeDependency Dependency;
			Dependency.ShaderPipelineTypeName = Pipeline->GetHashedName();
			Dependency.StagesSourceHash = Pipeline->GetSourceHash(ShaderPlatform);
			ShaderPipelineTypeDependencies.Add(Dependency);
		}
	}
}

static void PrepareMaterialShaderCompileJob(EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	const FMaterial* Material,
	const FMaterialCompilationOutput& CompilationOutput,
	FSharedShaderCompilerEnvironment* MaterialEnvironment,
	const FShaderPipelineType* ShaderPipeline,
	const FString& DebugGroupName,
	const FString& DebugDescription,
	const FString& DebugExtension,
	FShaderCompileJob* NewJob)
{
	const FShaderCompileJobKey& Key = NewJob->Key;
	const FMaterialShaderType* ShaderType = Key.ShaderType->AsMaterialShaderType();

	NewJob->bBypassCache = Material->IsPreview() || !Material->IsPersistent();
	NewJob->Input.SharedEnvironment = MaterialEnvironment;
	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;

	UE_LOGF(LogShaders, Verbose, "			%ls", ShaderType->GetName());

	//update material shader stats
	UpdateMaterialShaderCompilingStats(Material);

	Material->SetupExtraCompilationSettings(NewJob->Input.ExtraSettings);

	// Allow the shader type to modify the compile environment.
	ShaderType->SetupCompileEnvironment(Platform, FCompilationShaderParameters(Material, CompilationOutput), Key.PermutationId, PermutationFlags, ShaderEnvironment);

	// Compile the shader environment passed in with the shader type's source code.
	::GlobalBeginCompileShader(
		DebugGroupName,
		nullptr,
		ShaderType,
		ShaderPipeline,
		Key.PermutationId,
		ShaderType->GetShaderFilename(),
		ShaderType->GetFunctionName(),
		FShaderTarget(ShaderType->GetFrequency(), Platform),
		NewJob->Input,
		true,
		DebugDescription,
		DebugExtension
	);
}

/**
 * Enqueues a compilation for a new shader of this type.
 * @param Material - The material to link the shader with.
 */
void FMaterialShaderType::BeginCompileShader(
	EShaderCompileJobPriority Priority,
	uint32 ShaderMapJobId,
	int32 PermutationId,
	const FMaterial* Material,
	const FMaterialCompilationOutput& CompilationOutput,
	FSharedShaderCompilerEnvironment* MaterialEnvironment,
	EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	const FString& DebugGroupName,
	const TCHAR* DebugDescription,
	const TCHAR* DebugExtension
	) const
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(ShaderMapJobId, FShaderCompileJobKey(this, Platform, nullptr, PermutationId), Priority);
	if (NewJob)
	{
		PrepareMaterialShaderCompileJob(Platform, PermutationFlags, Material, CompilationOutput, MaterialEnvironment, nullptr, DebugGroupName, DebugDescription, DebugExtension, NewJob);
		NewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
	}
}

void FMaterialShaderType::BeginCompileShaderPipeline(
	EShaderCompileJobPriority Priority,
	uint32 ShaderMapJobId,
	EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	const FMaterial* Material,
	const FMaterialCompilationOutput& CompilationOutput,
	FSharedShaderCompilerEnvironment* MaterialEnvironment,
	const FShaderPipelineType* ShaderPipeline,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	const FString& DebugGroupName,
	const TCHAR* DebugDescription,
	const TCHAR* DebugExtension)
{
	check(ShaderPipeline);
	UE_LOGF(LogShaders, Verbose, "	Pipeline: %ls", ShaderPipeline->GetName());

	// Add all the jobs as individual first, then add the dependencies into a pipeline job
	auto* NewPipelineJob = GShaderCompilingManager->PreparePipelineCompileJob(ShaderMapJobId, FShaderPipelineCompileJobKey(ShaderPipeline, Platform, nullptr, kUniqueShaderPermutationId), Priority);
	if (NewPipelineJob)
	{
		for (FShaderCompileJob* StageJob : NewPipelineJob->StageJobs)
		{
			PrepareMaterialShaderCompileJob(Platform, PermutationFlags, Material, CompilationOutput, MaterialEnvironment, ShaderPipeline, DebugGroupName, DebugDescription, DebugExtension, StageJob);
		}
		NewJobs.Add(FShaderCommonCompileJobPtr(NewPipelineJob));
	}
}

/**
 * Either creates a new instance of this type or returns an equivalent existing shader.
 * @param Material - The material to link the shader with.
 * @param CurrentJob - Compile job that was enqueued by BeginCompileShader.
 */
FShader* FMaterialShaderType::FinishCompileShader(
	const FUniformExpressionSet& UniformExpressionSet,
	uint64 MaterialShaderMapHash,
	const FShaderCompileJob& CurrentJob,
	const FShaderPipelineType* ShaderPipelineType,
	const FString& InDebugDescription
	) const
{
	check(CurrentJob.bSucceeded);

	if (ShaderPipelineType && !ShaderPipelineType->ShouldOptimizeUnusedOutputs(CurrentJob.Input.Target.GetPlatform()))
	{
		// If sharing shaders in this pipeline, remove it from the type/id so it uses the one in the shared shadermap list
		ShaderPipelineType = nullptr;
	}

	FShader* Shader = ConstructCompiled(CompiledShaderInitializerType(this, CurrentJob.Key.PermutationId, CurrentJob.Output, UniformExpressionSet, MaterialShaderMapHash, ShaderPipelineType, nullptr, InDebugDescription));
	CurrentJob.Output.ParameterMap.VerifyBindingsAreComplete(GetName(), CurrentJob.Output.Target, CurrentJob.Key.VFType);

	return Shader;
}
#endif // WITH_EDITOR

bool FMaterialShaderType::ShouldCompilePermutation(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, int32 PermutationId, EShaderPermutationFlags Flags) const
{
	return FShaderType::ShouldCompilePermutation(FMaterialShaderPermutationParameters(Platform, MaterialParameters, PermutationId, Flags));
}

bool FMaterialShaderType::ShouldCompilePipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, EShaderPermutationFlags Flags)
{
	const FMaterialShaderPermutationParameters Parameters(Platform, MaterialParameters, kUniqueShaderPermutationId, Flags);
	for (const FShaderType* ShaderType : ShaderPipelineType->GetStages())
	{
		checkSlow(ShaderType->GetMaterialShaderType());
		if (!ShaderType->ShouldCompilePermutation(Parameters))
		{
			return false;
		}
	}
	return true;
}

#if WITH_EDITOR
void FMaterialShaderType::SetupCompileEnvironment(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, int32 PermutationId, EShaderPermutationFlags PermutationFlags, FShaderCompilerEnvironment& Environment) const
{
	// Allow the shader type to modify its compile environment.
	ModifyCompilationEnvironment(FMaterialShaderPermutationParameters(Platform, MaterialParameters, PermutationId, PermutationFlags), Environment);
}
#endif // WITH_EDITOR

/**
* Finds the shader map for a material.
* @param StaticParameterSet - The static parameter set identifying the shader map
* @param Platform - The platform to lookup for
* @return NULL if no cached shader map was found.
*/
TRefCountPtr<FMaterialShaderMap> FMaterialShaderMap::FindId(const FMaterialShaderMapId& ShaderMapId, EShaderPlatform InPlatform)
{
	FScopeLock ScopeLock(&GIdToMaterialShaderMapCS);
	check(ShaderMapId.IsValid());
	TRefCountPtr<FMaterialShaderMap> Result = GIdToMaterialShaderMap[InPlatform].FindRef(ShaderMapId);
	check(Result == nullptr || (!Result->bDeletedThroughDeferredCleanup && Result->bRegistered) );
	return Result;
}

#if WITH_EDITOR
void FMaterialShaderMap::GetAllOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes)
{
#if ALLOW_SHADERMAP_DEBUG_DATA
	FScopeLock AllMatSMAccess(&AllMaterialShaderMapsGuard);
	for (const FMaterialShaderMap* ShaderMap : AllMaterialShaderMaps)
	{
		ShaderMap->GetOutdatedTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
	}
#endif
}

TRACE_DECLARE_INT_COUNTER(Shaders_FMaterialShaderMapDDCRequests, TEXT("Shaders/FMaterialShaderMap/DDCRequests"));
TRACE_DECLARE_INT_COUNTER(Shaders_FMaterialShaderMapDDCHits, TEXT("Shaders/FMaterialShaderMap/DDCHits"));
TRACE_DECLARE_MEMORY_COUNTER(Shaders_FMaterialShaderMapDDCBytesReceived, TEXT("Shaders/FMaterialShaderMap/DDCBytesReceived"));
TRACE_DECLARE_MEMORY_COUNTER(Shaders_FMaterialShaderMapDDCBytesSent, TEXT("Shaders/FMaterialShaderMap/DDCBytesSent"));

void FMaterialShaderMap::LoadFromDerivedDataCache(const FMaterial* Material, const FMaterialShaderMapId& ShaderMapId, EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, TRefCountPtr<FMaterialShaderMap>& InOutShaderMap, FString& OutDDCKeyDesc)
{
	InOutShaderMap = BeginLoadFromDerivedDataCache(Material, ShaderMapId, InPlatform, TargetPlatform, InOutShaderMap, OutDDCKeyDesc)->Get();
}

struct FMaterialShaderMapAsyncLoadContext : public FMaterialShaderMap::FAsyncLoadContext
{
	FString								DataKey;
	FShaderCacheLoadContext				LoadContext;
	EShaderPlatform						Platform;
	const FMaterial*					Material = nullptr;
	UE::DerivedData::FRequestOwner		RequestOwner{ UE::DerivedData::EPriority::Normal };
	TRefCountPtr<FMaterialShaderMap>	ShaderMap;

	bool IsReady() const override
	{
		return RequestOwner.Poll();
	}

	TRefCountPtr<FMaterialShaderMap> Get() override
	{
		// Make sure the async work is complete
		if (!RequestOwner.Poll())
		{
			COOK_STAT(auto Timer = MaterialShaderCookStats::UsageStats.TimeAsyncWait());
			COOK_STAT(Timer.TrackCyclesOnly());
			RequestOwner.Wait();
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::FinishLoadFromDerivedDataCache);
		COOK_STAT(auto Timer = MaterialShaderCookStats::UsageStats.TimeSyncWork());

		if (LoadContext.HasData())
		{
			TRACE_COUNTER_INCREMENT(Shaders_FMaterialShaderMapDDCHits);

			int64 BytesReceived = LoadContext.GetSerializedSize();
			TRACE_COUNTER_ADD(Shaders_FMaterialShaderMapDDCBytesReceived, BytesReceived);
			COOK_STAT(Timer.AddHit(BytesReceived));

			ShaderMap = new FMaterialShaderMap();
			// Deserialize from the cached data
			ShaderMap->Serialize(LoadContext);

			check(Material != nullptr);

			const FString InDataKey = GetMaterialShaderMapKeyString(ShaderMap->GetShaderMapId(), FConservativeShaderParameters(Material), Platform, true);

			if (InDataKey != DataKey)
			{
				UE_LOGF(LogMaterial, Warning, "Shader map key recomputed from DDC data: %ls", *InDataKey);
				UE_LOGF(LogMaterial, Warning, "Shader map key from request: %ls", *DataKey);
				UE_LOGF(LogMaterial, Warning, "Cached data size %llu", BytesReceived);
				checkf(false, TEXT("DDC key constructed from deserialized shadermap does not match request key!"));
			}

			// Register in the global map
			ShaderMap->Register(Platform);

			GShaderCompilerStats->AddDDCHit(1);

			UE::DerivedData::FCacheKey DdcKey = GetMaterialShaderMapKey(DataKey);
			UE_LOGF(LogMaterial, Verbose, "Loaded shaders for %ls from DDC (key hash: %ls)", *Material->GetAssetName(), *LexToString(DdcKey.Hash));
		}
		else
		{
			TRACE_COUNTER_INCREMENT(Shaders_FMaterialShaderMapDDCRequests);
			// We should be build the data later, and we can track that the resource was built there when we push it to the DDC.
			COOK_STAT(Timer.TrackCyclesOnly());

			GShaderCompilerStats->AddDDCMiss(1);
		}

		return ShaderMap;
	}
};

TSharedRef<FMaterialShaderMap::FAsyncLoadContext> FMaterialShaderMap::BeginLoadFromDerivedDataCache(const FMaterial* Material, const FMaterialShaderMapId& ShaderMapId, EShaderPlatform InPlatform, const ITargetPlatform* TargetPlatform, TRefCountPtr<FMaterialShaderMap>& InShaderMap, FString& OutDDCKeyDesc)
{
	using namespace UE::DerivedData;

	TSharedRef<FMaterialShaderMapAsyncLoadContext> Result = MakeShared<FMaterialShaderMapAsyncLoadContext>();

	if (InShaderMap != nullptr)
	{
		check(InShaderMap->GetShaderPlatform() == InPlatform);
		Result->ShaderMap = InShaderMap;
	}
	else
	{
		// Shader map was not found in memory, try to load it from the DDC
		STAT(double MaterialDDCTime = 0);
		{
			SCOPE_SECONDS_COUNTER(MaterialDDCTime);
			COOK_STAT(auto Timer = MaterialShaderCookStats::UsageStats.TimeSyncWork());
			COOK_STAT(Timer.TrackCyclesOnly());
			Result->DataKey = GetMaterialShaderMapKeyString(ShaderMapId, FConservativeShaderParameters(Material), InPlatform);
			FCacheKey CacheKey = GetMaterialShaderMapKey(Result->DataKey);
			OutDDCKeyDesc = LexToString(CacheKey.Hash);

			FString DebugDDCKeyAsset = CVarShaderCompilerDebugDDCKeyAsset.GetValueOnAnyThread();
			if (UNLIKELY(ShouldDumpShaderDDCKeys()) ||
				(Material->IsDefaultMaterial() && Material->GetMaterialDomain() == EMaterialDomain::MD_Surface) ||
				UNLIKELY(!DebugDDCKeyAsset.IsEmpty() && Material->GetAssetName().Contains(DebugDDCKeyAsset)))
			{	
				DumpShaderDDCKeyToFile(InPlatform, ShaderMapId.LayoutParams.WithEditorOnly(), *Material->GetDebugGroupName(), Result->DataKey);
			}

			bool bCheckCache = true;

			// If NoShaderDDC then don't check for a material the first time we encounter it to simulate
			// a cold DDC
			static bool bNoShaderDDC =
				FParse::Param(FCommandLine::Get(), TEXT("noshaderddc")) || 
				FParse::Param(FCommandLine::Get(), TEXT("nomaterialshaderddc"));

			if (bNoShaderDDC)
			{
				static TSet<uint32> SeenKeys;

				const uint32 KeyHash = GetTypeHash(Result->DataKey);

				if (!SeenKeys.Contains(KeyHash))
				{
					bCheckCache = false;
					SeenKeys.Add(KeyHash);
				}
			}

			// Do not check the DDC if the material isn't persistent, because
			//   - this results in a lot of DDC requests when editing materials which are almost always going to return nothing.
			//   - since the get call is synchronous, this can cause a hitch if there's network latency
			if (Material->IsPersistent())
			{
				FCacheGetRequest Request;
				Request.Name = GetMaterialShaderMapName(Material->GetFullPath(), ShaderMapId, InPlatform);
				Request.Key = CacheKey;
				Result->Material = Material;
				Result->Platform = InPlatform;

				GetCache().Get({ Request }, Result->RequestOwner, [Result, bCheckCache](FCacheGetResponse&& Response)
					{
						if (bCheckCache)
						{
							if (Response.Status == EStatus::Ok)
							{
								Result->LoadContext.ReadFromRecord(Response.Record, /* bIsPersisent */ true);
							}

							// This callback might hold the last reference to Result, which owns RequestOwner, so
							// we must not cancel in the Owner's destructor; Canceling in a callback will deadlock.
							Result->RequestOwner.KeepAlive();
						}
					});
			}
		}
		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_DDCLoading,(float)MaterialDDCTime);
	}

	return Result;
}

void FMaterialShaderMap::SaveToDerivedDataCache(const FConservativeShaderParameters& ShaderParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::SaveToDerivedDataCache);
	COOK_STAT(auto Timer = MaterialShaderCookStats::UsageStats.TimeSyncWork());

	FShaderCacheSaveContext Ctx;
	Serialize(Ctx);

	const FString DataKey = GetMaterialShaderMapKeyString(ShaderMapId, ShaderParameters, GetShaderPlatform());

	using namespace UE::DerivedData;
	FCacheKey Key = GetMaterialShaderMapKey(DataKey);

	UE::FSharedString RequestName = GetMaterialShaderMapName(GetMaterialPath(), ShaderMapId, GetShaderPlatform());
	UE_LOGF(LogMaterial, Verbose, "Saved shaders for %ls to DDC (key hash: %ls)", *RequestName, *LexToString(Key.Hash));
	UE_LOGF(LogMaterial, VeryVerbose, "Full DDC data key for %ls: %ls", *RequestName, *DataKey);

	if (!CVarMaterialShaderMapDump->GetString().IsEmpty() && CVarMaterialShaderMapDump->GetString() == GetMaterialPath())
	{
		TStringBuilder<256> Path;
		FPathViews::Append(Path, FPaths::ProjectSavedDir(), TEXT("MaterialShaderMaps"), TEXT(""));
		Path << Key.Hash << ".txt";
		FArchive* DumpAr = IFileManager::Get().CreateFileWriter(*Path, FILEWRITE_Silent);
		if (DumpAr)
		{
			FTCHARToUTF8 Converter(*ToString());
			DumpAr->Serialize(const_cast<UTF8CHAR*>(reinterpret_cast<const UTF8CHAR*>(Converter.Get())), Converter.Length());
		}
	}

	FRequestOwner AsyncOwner(EPriority::Normal);
	FRequestBarrier AsyncBarrier(AsyncOwner);
	GetCache().Put({ { RequestName, Ctx.BuildCacheRecord(Key) } }, AsyncOwner);

	uint64 SerializedSize = Ctx.GetSerializedSize();
	TRACE_COUNTER_ADD(Shaders_FMaterialShaderMapDDCBytesSent, SerializedSize);
	COOK_STAT(Timer.AddMiss(SerializedSize));

	AsyncOwner.KeepAlive();
}
#endif // WITH_EDITOR

void FMaterialShaderMap::SaveForRemoteRecompile(FArchive& Ar, const TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap> > >& CompiledShaderMaps)
{
	// now we serialize a map (for each material), but without inline the resources, since they are above
	int32 MapSize = CompiledShaderMaps.Num();
	Ar << MapSize;

	for (TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap> > >::TConstIterator It(CompiledShaderMaps); It; ++It)
	{
		const TArray<TRefCountPtr<FMaterialShaderMap> >& ShaderMapArray = It.Value();

		FString MaterialName = It.Key();
		Ar << MaterialName;

		int32 NumShaderMaps = ShaderMapArray.Num();
		Ar << NumShaderMaps;

		for (int32 Index = 0; Index < ShaderMapArray.Num(); Index++)
		{
			FMaterialShaderMap* ShaderMap = ShaderMapArray[Index];
			if (ShaderMap)
			{
				uint8 bIsValid = 1;
				Ar << bIsValid;
				FShaderSerializeContext Ctx(Ar);
				ShaderMap->Serialize(Ctx);
			}
			else
			{
				uint8 bIsValid = 0;
				Ar << bIsValid;
			}
		}
	}
}

void FMaterialShaderMap::LoadForRemoteRecompile(FArchive& Ar, EShaderPlatform ShaderPlatform, TArray<UMaterialInterface*>& OutLoadedMaterials)
{
	TMap<FMaterialShaderMapId, TRefCountPtr<FMaterialShaderMap>> LoadedShaderMapsDictionary;

	struct FMaterialShaderMapData
	{
		FString MaterialName;
		TArray<FMaterialShaderMapId> LoadedShaderMapsIds;
	};

	TArray<FMaterialShaderMapData> MaterialShaderMapDataArray;

	int32 MapSize;
	Ar << MapSize;

	MaterialShaderMapDataArray.Reserve(MapSize);
	for (int32 MaterialIndex = 0; MaterialIndex < MapSize; MaterialIndex++)
	{
		FMaterialShaderMapData& MaterialShaderMapData = MaterialShaderMapDataArray.Emplace_GetRef();
		Ar << MaterialShaderMapData.MaterialName;

		int32 ActorSeparator = MaterialShaderMapData.MaterialName.Find(TEXT(":::"));
		if (ActorSeparator != INDEX_NONE)
		{
			MaterialShaderMapData.MaterialName.LeftInline(ActorSeparator);
		}

		int32 NumShaderMaps = 0;
		Ar << NumShaderMaps;

		MaterialShaderMapData.LoadedShaderMapsIds.Reserve(NumShaderMaps);
		for (int32 ShaderMapIndex = 0; ShaderMapIndex < NumShaderMaps; ShaderMapIndex++)
		{
			uint8 bIsValid = 0;
			Ar << bIsValid;

			if (bIsValid)
			{
				TRefCountPtr<FMaterialShaderMap> ShaderMap = new FMaterialShaderMap();

				// serialize the id and the material shader map
				FShaderSerializeContext Ctx(Ar);
				ShaderMap->Serialize(Ctx);

				LoadedShaderMapsDictionary.Add(ShaderMap->GetShaderMapId(), ShaderMap);
				MaterialShaderMapData.LoadedShaderMapsIds.Add(ShaderMap->GetShaderMapId());
			}
		}
	}

	for (auto IterLoadedShaderMaps = LoadedShaderMapsDictionary.CreateIterator(); IterLoadedShaderMaps; ++IterLoadedShaderMaps)
	{
		// Register in the global map
		IterLoadedShaderMaps.Value()->RegisterForODSC(ShaderPlatform);
	}

	check(MaterialShaderMapDataArray.Num() == MapSize);
	for (int32 MaterialIndex = 0; MaterialIndex < MapSize; MaterialIndex++)
	{
		TArray<TRefCountPtr<FMaterialShaderMap>> LoadedShaderMaps;
		const FMaterialShaderMapData& MaterialShaderMapData = MaterialShaderMapDataArray[MaterialIndex];
		const FString MaterialName = MaterialShaderMapData.MaterialName;


		LoadedShaderMaps.Reserve(MaterialShaderMapData.LoadedShaderMapsIds.Num());
		for (const FMaterialShaderMapId& MaterialShaderMapId : MaterialShaderMapData.LoadedShaderMapsIds)
		{
			LoadedShaderMaps.Add(LoadedShaderMapsDictionary.FindChecked(MaterialShaderMapId));
		}

#if WITH_ODSC
		FODSCManager::RegisterMaterialShaderMaps(MaterialName, LoadedShaderMaps);
#endif

		UMaterialInterface* MatchingMaterial = FindObject<UMaterialInterface>(nullptr, *MaterialName);
		if (!MatchingMaterial)
		{
			continue;
		}
		OutLoadedMaterials.Add(MatchingMaterial);

		// Assign in two passes: first pass for shader maps with unspecified quality levels,
		// Second pass for shader maps with a specific quality level
		for (int32 PassIndex = 0; PassIndex < 2; PassIndex++)
		{
			for (int32 ShaderMapIndex = 0; ShaderMapIndex < LoadedShaderMaps.Num(); ShaderMapIndex++)
			{
				FMaterialShaderMap* LoadedShaderMap = LoadedShaderMaps[ShaderMapIndex];
	
				if (LoadedShaderMap->GetShaderPlatform() == ShaderPlatform 
					&& LoadedShaderMap->GetShaderMapId().FeatureLevel == GetMaxSupportedFeatureLevel(ShaderPlatform))
				{
					EMaterialQualityLevel::Type LoadedQualityLevel = LoadedShaderMap->GetShaderMapId().QualityLevel;

					for (int32 QualityLevelIndex = 0; QualityLevelIndex < EMaterialQualityLevel::Num; QualityLevelIndex++)
					{
						// First pass: assign shader maps with unspecified quality levels to all material resources
						if ((PassIndex == 0 && LoadedQualityLevel == EMaterialQualityLevel::Num)
							// Second pass: assign shader maps with a specified quality level to only the appropriate material resource
							|| (PassIndex == 1 && QualityLevelIndex == LoadedQualityLevel))
						{
							FMaterialResource* MaterialResource = MatchingMaterial->GetMaterialResource(ShaderPlatform, (EMaterialQualityLevel::Type)QualityLevelIndex);
							MaterialResource->SetGameThreadShaderMap(LoadedShaderMap);
						}
					}
				}
			}
		}
	}
}

void FMaterialShaderMapContent::Finalize(const FShaderMapResourceCode* Code)
{
	FShaderHashBuilder Hasher;
	FShaderMapContent::Finalize(Code);
	UpdateHash(Hasher);

	for (FMeshMaterialShaderMap* MeshShaderMap : OrderedMeshShaderMaps)
	{
		MeshShaderMap->Finalize(Code);
		MeshShaderMap->UpdateHash(Hasher);
	}

	ShaderContentHash = Hasher.Finalize();
}

#if WITH_EDITOR

static TMap<uint32, TRefCountPtr<FMaterialShaderMap>>& GetCompilingShaderMapLookup()
{
	static TMap<uint32, TRefCountPtr<FMaterialShaderMap>> Map;
	return Map;
}

static FRWLock GCompilingShaderMapLock;

FMaterialShaderMap* FMaterialShaderMap::FindCompilingShaderMap(uint32 CompilingId)
{
	FReadScopeLock Locker(GCompilingShaderMapLock);
	return GetCompilingShaderMapLookup().FindRef(CompilingId);
}

uint32 FMaterialShaderMap::AcquireCompilingId(const TRefCountPtr<FSharedShaderCompilerEnvironment>& InMaterialEnvironment)
{
	check(IsInGameThread());
	if (CompilingId == 0u)
	{
		PendingCompilerEnvironment = InMaterialEnvironment;
		FWriteScopeLock Locker(GCompilingShaderMapLock);
		CompilingId = FShaderCommonCompileJob::GetNextJobId();
		GetCompilingShaderMapLookup().Add(CompilingId, this);
	}

	check(PendingCompilerEnvironment == InMaterialEnvironment);

	return CompilingId;
}

void FMaterialShaderMap::ReleaseCompilingId()
{
	check(IsInGameThread());
	if (CompilingId != 0u)
	{
		FWriteScopeLock Locker(GCompilingShaderMapLock);
		check(CompilingMaterialDependencies.Num() == 0);
		check(CompilingMaterialNumExternalDependencies == 0);
#if ALLOW_SHADERMAP_DEBUG_DATA
		if (IsRunningCookCommandlet())
		{
			InFlightJobs.Empty();
		}
#endif
		verify(GetCompilingShaderMapLookup().Remove(CompilingId) == 1);
		CompilingId = 0;
	}

	PendingCompilerEnvironment.SafeRelease();
}

void FMaterialShaderMap::AddCompilingDependency(FMaterial* Material)
{
	CompilingMaterialDependencies.AddUnique(Material);
	// if any of our dependencies is persistent, we're persistent
	bIsPersistent |= Material->IsPersistent();
}

void FMaterialShaderMap::RemoveCompilingDependency(FMaterial* Material)
{
	check(CompilingId != 0u);
	const int32 NumRemoved = CompilingMaterialDependencies.RemoveSingle(Material);
	check(NumRemoved == 1);
	checkSlow(!CompilingMaterialDependencies.Contains(Material));
	CheckReleaseCompilingId();
}

void FMaterialShaderMap::CheckReleaseCompilingId()
{
	if (CompilingMaterialDependencies.Num() == 0 && CompilingMaterialNumExternalDependencies==0)
	{
		const TArray<int32> CompilingIdsToCancel = { (int32)CompilingId };
		GShaderCompilingManager->CancelCompilation(GetFriendlyName(), CompilingIdsToCancel);

		ReleaseCompilingId();
	}
}

int32 FMaterialShaderMap::SubmitCompileJobs(uint32 CompilingShaderMapId,
	const FMaterial* Material,
	const TRefCountPtr<FSharedShaderCompilerEnvironment>& MaterialEnvironment,
	EShaderCompileJobPriority InPriority) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::SubmitCompileJobs);

	check(CompilingShaderMapId != 0u);
	check(MaterialEnvironment);

	TArray<FShaderCommonCompileJobPtr> CompileJobs;

	uint32 NumShaders = 0;
	uint32 NumVertexFactories = 0;

	const EShaderPlatform ShaderPlatform = GetShaderPlatform();
	const EShaderPermutationFlags LocalPermutationFlags = ShaderMapId.GetPermutationFlags();

	const FMaterialCompilationOutput& CompilationOutput = GetCompilationOutput();
	const FCompilationShaderParameters MaterialParameters(Material, CompilationOutput);

	const FMaterialShaderMapLayout& Layout = AcquireMaterialShaderMapLayout(ShaderPlatform, LocalPermutationFlags, MaterialParameters);

#if ALLOW_SHADERMAP_DEBUG_DATA
	FString DebugExtensionStr;
	FString DebugDescriptionStr;

	DebugExtensionStr = FString::Printf(TEXT("_%08x%08x"), ShaderMapId.BaseMaterialId.A, ShaderMapId.BaseMaterialId.B);
	DebugDescriptionStr = GetDebugDescription();

	const TCHAR* DebugExtension = *DebugExtensionStr;
	const TCHAR* DebugDescription = *DebugDescriptionStr;

#else
	const TCHAR* DebugExtension = nullptr;
	const TCHAR* DebugDescription = nullptr;
#endif

	// Iterate over all vertex factory types.
	for (const FMeshMaterialShaderMapLayout& MeshLayout : Layout.MeshShaderMaps)
	{
		const FMeshMaterialShaderMap* MeshShaderMap = GetMeshShaderMap(MeshLayout.VertexFactoryType);

		uint32 NumShadersPerVF = 0;
		TSet<FString> ShaderTypeNames;

		// Do not submit jobs for the shader types that are included in some pipeline stages if that pipeline is optimzing unused outputs.
		// In such a case, these shaders should not be used runtime anymore
		FPipelinedShaderFilter PipelinedShaderFilter(ShaderPlatform, MeshLayout.ShaderPipelines);

		// Iterate over all mesh material shader types.
		TMap<TShaderTypePermutation<const FShaderType>, FShaderCompileJob*> SharedShaderJobs;
		for (const FShaderLayoutEntry& Shader : MeshLayout.Shaders)
		{
			FMeshMaterialShaderType* ShaderType = static_cast<FMeshMaterialShaderType*>(Shader.ShaderType);
			if (!Material->ShouldCache(ShaderType, MeshLayout.VertexFactoryType))
			{
				continue;
			}

			// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
			checkf(ShaderMapId.ContainsVertexFactoryType(MeshLayout.VertexFactoryType), TEXT("Material shader map %s from material %s is missing expected vertex factory type %s. Parameters Differences:\n%s"),
				GetFriendlyName(), *Material->GetAssetPath().ToString(), MeshLayout.VertexFactoryType->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));
			checkf(ShaderMapId.ContainsShaderType(ShaderType, kUniqueShaderPermutationId), TEXT("Material shader map %s from material %s is missing expected shader type %s. Parameters Differences:\n%s"),
				GetFriendlyName(), *Material->GetAssetPath().ToString(), ShaderType->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));

			NumShadersPerVF++;
			// only compile the shader if we don't already have it and it is not a pipelined one
			if (!PipelinedShaderFilter.IsPipelinedType(ShaderType) && (!MeshShaderMap || !MeshShaderMap->HasShader(ShaderType, Shader.PermutationId)))
			{
				// Compile this mesh material shader for this material and vertex factory type.
				ShaderType->BeginCompileShader(InPriority,
					CompilingShaderMapId,
					Shader.PermutationId,
					ShaderPlatform,
					LocalPermutationFlags,
					Material,
					CompilationOutput,
					MaterialEnvironment,
					MeshLayout.VertexFactoryType,
					CompileJobs,
					Material->GetDebugGroupName(),
					DebugDescription,
					DebugExtension
				);
			}
		}

		// Now the pipeline jobs; if it's a shareable pipeline, do not add duplicate jobs
		for (FShaderPipelineType* Pipeline : MeshLayout.ShaderPipelines)
		{
			if (!Material->ShouldCachePipeline(Pipeline, MeshLayout.VertexFactoryType))
			{
				continue;
			}

			auto& StageTypes = Pipeline->GetStages();

			// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
			checkf(ShaderMapId.ContainsShaderPipelineType(Pipeline), TEXT("Material shader map %s from material %s missing expected pipeline type %s. Parameters Differences:\n%s"),
				GetFriendlyName(), *Material->GetAssetPath().ToString(), Pipeline->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));

			checkf(ShaderMapId.ContainsVertexFactoryType(MeshLayout.VertexFactoryType), TEXT("Material shader map %s from material %s missing expected Vertex Factory type %s. Parameters Differences:\n%s"),
				GetFriendlyName(), *Material->GetAssetPath().ToString(), MeshLayout.VertexFactoryType->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));

			if (Pipeline->ShouldOptimizeUnusedOutputs(ShaderPlatform))
			{
				NumShadersPerVF += StageTypes.Num();

				for (auto* ShaderType : StageTypes)
				{
					// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
					checkf(ShaderMapId.ContainsShaderType(ShaderType, kUniqueShaderPermutationId), TEXT("Material shader map %s from material %s missing expected shader type %s. Parameters Differences:\n%s"),
						GetFriendlyName(), *Material->GetAssetPath().ToString(), ShaderType->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));
				}


				// Make a pipeline job with all the stages
				FMeshMaterialShaderType::BeginCompileShaderPipeline(InPriority,
					CompilingShaderMapId,
					kUniqueShaderPermutationId,
					ShaderPlatform,
					LocalPermutationFlags,
					Material,
					CompilationOutput,
					MaterialEnvironment,
					MeshLayout.VertexFactoryType,
					Pipeline,
					CompileJobs,
					Material->GetDebugGroupName(),
					DebugDescription,
					DebugExtension);
			}
			else
			{
				// If sharing shaders amongst pipelines, add this pipeline as a dependency of an existing job
				for (const FShaderType* ShaderType : StageTypes)
				{
					TShaderTypePermutation<const FShaderType> ShaderTypePermutation(ShaderType, kUniqueShaderPermutationId);
					FShaderCompileJob** Job = SharedShaderJobs.Find(ShaderTypePermutation);
					checkf(Job, TEXT("Couldn't find existing shared job for mesh shader %s on pipeline %s!"), ShaderType->GetName(), Pipeline->GetName());
					auto* SingleJob = (*Job)->GetSingleShaderJob();
					auto& PipelinesToShare = SingleJob->SharingPipelines.FindOrAdd(MeshLayout.VertexFactoryType);
					check(!PipelinesToShare.Contains(Pipeline));
					PipelinesToShare.Add(Pipeline);
				}
			}
		}

		NumShaders += NumShadersPerVF;
		if (NumShadersPerVF > 0)
		{
			UE_LOGF(LogShaders, Verbose, "			%ls - %u shaders", MeshLayout.VertexFactoryType->GetName(), NumShadersPerVF);
			NumVertexFactories++;
		}
	}

	// Do not submit jobs for the shader types that are included in some pipeline stages if that pipeline is optimzing unused outputs.
	// In such a case, these shaders should not be used runtime anymore
	FPipelinedShaderFilter PipelinedShaderFilter(ShaderPlatform, Layout.ShaderPipelines);

	// Iterate over all material shader types.
	TMap<TShaderTypePermutation<const FShaderType>, FShaderCompileJob*> SharedShaderJobs;
	for (const FShaderLayoutEntry& Shader : Layout.Shaders)
	{
		FMaterialShaderType* ShaderType = static_cast<FMaterialShaderType*>(Shader.ShaderType);
		if (!Material->ShouldCache(ShaderType, nullptr))
		{
			continue;
		}

		// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
		checkf(ShaderMapId.ContainsShaderType(ShaderType, kUniqueShaderPermutationId), TEXT("Material shader map %s from Material %s missing expected shader type %s. Parameters Differences:\n%s"),
			GetFriendlyName(), *Material->GetAssetPath().ToString(), ShaderType->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));

		// Compile this material shader for this material.
		TArray<FString> ShaderErrors;

		// Only compile the shader if we don't already have it
		if (!PipelinedShaderFilter.IsPipelinedType(ShaderType) && !GetContent()->HasShader(ShaderType, Shader.PermutationId))
		{
			ShaderType->BeginCompileShader(InPriority,
				CompilingShaderMapId,
				Shader.PermutationId,
				Material,
				CompilationOutput,
				MaterialEnvironment,
				ShaderPlatform,
				LocalPermutationFlags,
				CompileJobs,
				Material->GetDebugGroupName(),
				DebugDescription,
				DebugExtension
			);
		}
		NumShaders++;
	}

	if (RHISupportsShaderPipelines(ShaderPlatform))
	{
		for (FShaderPipelineType* Pipeline : Layout.ShaderPipelines)
		{
			if (!Material->ShouldCachePipeline(Pipeline, nullptr))
			{
				continue;
			}

			auto& StageTypes = Pipeline->GetStages();

			// Verify that the shader map Id contains inputs for any shaders that will be put into this shader map
			checkf(ShaderMapId.ContainsShaderPipelineType(Pipeline), TEXT("Material shader map %s from material %s missing expected pipeline type %s. Parameters Differences:\n%s"),
				GetFriendlyName(), *Material->GetAssetPath().ToString(), Pipeline->GetName(), *DescribeCompilationShaderParametersDifferences(Material, MaterialParameters));

			if (Pipeline->ShouldOptimizeUnusedOutputs(ShaderPlatform))
			{
				NumShaders += StageTypes.Num();
				FMaterialShaderType::BeginCompileShaderPipeline(InPriority,
					CompilingShaderMapId,
					ShaderPlatform,
					LocalPermutationFlags,
					Material,
					CompilationOutput,
					MaterialEnvironment,
					Pipeline,
					CompileJobs,
					Material->GetDebugGroupName(),
					DebugDescription,
					DebugExtension);
			}
			else
			{
				// If sharing shaders amongst pipelines, add this pipeline as a dependency of an existing job
				for (const FShaderType* ShaderType : StageTypes)
				{
					TShaderTypePermutation<const FShaderType> ShaderTypePermutation(ShaderType, kUniqueShaderPermutationId);
					FShaderCompileJob** Job = SharedShaderJobs.Find(ShaderTypePermutation);
					checkf(Job, TEXT("Couldn't find existing shared job for material shader %s on pipeline %s!"), ShaderType->GetName(), Pipeline->GetName());
					auto* SingleJob = (*Job)->GetSingleShaderJob();
					auto& PipelinesToShare = SingleJob->SharingPipelines.FindOrAdd(nullptr);
					check(!PipelinesToShare.Contains(Pipeline));
					PipelinesToShare.Add(Pipeline);
				}
			}
		}
	}

	UE_LOGF(LogShaders, Verbose, "		%u Shaders among %u VertexFactories", NumShaders, NumVertexFactories);

#if ALLOW_SHADERMAP_DEBUG_DATA
	if (IsRunningCookCommandlet())
	{
		for (const FShaderCommonCompileJobPtr& Job : CompileJobs)
		{
			TStringBuilder<2048> JobName;
			Job->AppendDebugName(JobName);
			InFlightJobs.Add(Job->Hash, { JobName.ToString(), Job->JobStatusPtr });
		}
	}
#endif

	GShaderCompilingManager->SubmitJobs(CompileJobs, Material->GetBaseMaterialPathName(), GetDebugDescription());

	return CompileJobs.Num();
}

/**
* Compiles the shaders for a material and caches them in this shader map.
* @param Material - The material to compile shaders for.
* @param InShaderMapId - the set of static parameters to compile for
* @param Platform - The platform to compile to
*/
void FMaterialShaderMap::Compile(
	FMaterial* Material,
	const FMaterialShaderMapId& InShaderMapId,
	const TRefCountPtr<FSharedShaderCompilerEnvironment>& MaterialEnvironment,
	const FMaterialCompilationOutput& InMaterialCompilationOutput,
	EShaderPlatform InPlatform,
	EMaterialShaderPrecompileMode PrecompileMode)
{
	if (FPlatformProperties::RequiresCookedData())
	{
		UE_LOGF(LogMaterial, Fatal, "Trying to compile %ls at run-time, which is not supported on consoles!", *Material->GetFriendlyName());
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::Compile);
	check(!Material->bContainsInlineShaders);

	// Assign a unique identifier so that shaders from this shader map can be associated with it after a deferred compile
	AcquireCompilingId(MaterialEnvironment); 

	FMaterialShaderMapContent* NewContent = new FMaterialShaderMapContent(InPlatform);
	NewContent->MaterialCompilationOutput = InMaterialCompilationOutput;
	AssignContent(NewContent);

	ShaderMapId = InShaderMapId;
	bIsPersistent = Material->IsPersistent();

	// Fill in UserSceneTextureOutput -- only supported for post process domain, and not supported for BL_ReplacingTonemapper
	const UMaterial* MaterialObject = Material->GetMaterialInterface()->GetMaterial();
	if (MaterialObject && MaterialObject->MaterialDomain == EMaterialDomain::MD_PostProcess && MaterialObject->BlendableLocation != EBlendableLocation::BL_ReplacingTonemapper)
	{
		NewContent->UserSceneTextureOutput = FScriptName(MaterialObject->UserSceneTexture);
		NewContent->UserTextureDivisorX = MaterialObject->UserTextureDivisor.X;
		NewContent->UserTextureDivisorY = MaterialObject->UserTextureDivisor.Y;
		NewContent->ResolutionRelativeToInput = FScriptName(MaterialObject->ResolutionRelativeToInput);
	}

#if ALLOW_SHADERMAP_DEBUG_DATA
	// Store the material name for debugging purposes.
	// Note: Material instances with static parameters will have the same FriendlyName for their shader maps!
	NewContent->FriendlyName = Material->GetFriendlyName();
	NewContent->MaterialPath = Material->GetAssetPath().ToString();

	// Log debug information about the material being compiled.
	const FString MaterialUsage = Material->GetMaterialUsageDescription();
	FString WorkingDebugDescription = FString::Printf(
		TEXT("Compiling %s: Platform=%s, Usage=%s"),
		*NewContent->FriendlyName,
		*FDataDrivenShaderPlatformInfo::GetName(InPlatform).ToString(),
		*MaterialUsage
	);
	for (int32 StaticSwitchIndex = 0; StaticSwitchIndex < ShaderMapId.GetStaticSwitchParameters().Num(); ++StaticSwitchIndex)
	{
		const FStaticSwitchParameter& StaticSwitchParameter = ShaderMapId.GetStaticSwitchParameters()[StaticSwitchIndex];
		WorkingDebugDescription += FString::Printf(
			TEXT(", StaticSwitch'%s'=%s"),
			*StaticSwitchParameter.ParameterInfo.ToString(),
			StaticSwitchParameter.Value ? TEXT("True") : TEXT("False")
		);
	}
	for (int32 StaticMaskIndex = 0; StaticMaskIndex < ShaderMapId.GetStaticComponentMaskParameters().Num(); ++StaticMaskIndex)
	{
		const FStaticComponentMaskParameter& StaticComponentMaskParameter = ShaderMapId.GetStaticComponentMaskParameters()[StaticMaskIndex];
		WorkingDebugDescription += FString::Printf(
			TEXT(", StaticMask'%s'=%s%s%s%s"),
			*StaticComponentMaskParameter.ParameterInfo.ToString(),
			StaticComponentMaskParameter.R ? TEXT("R") : TEXT(""),
			StaticComponentMaskParameter.G ? TEXT("G") : TEXT(""),
			StaticComponentMaskParameter.B ? TEXT("B") : TEXT(""),
			StaticComponentMaskParameter.A ? TEXT("A") : TEXT("")
		);
	}
	for (int32 StaticLayerIndex = 0; StaticLayerIndex < ShaderMapId.GetTerrainLayerWeightParameters().Num(); ++StaticLayerIndex)
	{
		const FStaticTerrainLayerWeightParameter& StaticTerrainLayerWeightParameter = ShaderMapId.GetTerrainLayerWeightParameters()[StaticLayerIndex];
		WorkingDebugDescription += FString::Printf(
			TEXT(", StaticTerrainLayer'%s'=%s"),
			*StaticTerrainLayerWeightParameter.LayerName.ToString(),
			*FString::Printf(TEXT("Weightmap%u"), StaticTerrainLayerWeightParameter.WeightmapIndex)
		);
	}

	if (ShaderMapId.GetMaterialLayersId())
	{
		const FMaterialLayersFunctions::ID& MaterialLayersId = *ShaderMapId.GetMaterialLayersId();
		WorkingDebugDescription += TEXT("Layers:");
		bool StartWithComma = false;
		for (const auto &Layer : MaterialLayersId.LayerIDs)
		{
			WorkingDebugDescription += (StartWithComma ? TEXT(", ") : TEXT("")) + Layer.ToString();
			StartWithComma = true;
		}
		WorkingDebugDescription += TEXT(", Blends:");
		StartWithComma = false;
		for (const auto &Blend : MaterialLayersId.BlendIDs)
		{
			WorkingDebugDescription += (StartWithComma ? TEXT(", ") : TEXT("")) + Blend.ToString();
			StartWithComma = true;
		}
		WorkingDebugDescription += TEXT(", LayerStates:");
		StartWithComma = false;
		for (bool State : MaterialLayersId.LayerStates)
		{
			WorkingDebugDescription += (StartWithComma ? TEXT(", ") : TEXT(""));
			WorkingDebugDescription += (State ? TEXT("1") : TEXT("0"));
			StartWithComma = true;
		}
	}

	// If we aren't actually compiling shaders don't print the debug message that we are compiling shaders.
	if (PrecompileMode != EMaterialShaderPrecompileMode::None)
	{
		UE_LOGF(LogShaders, Verbose, "	%ls", *WorkingDebugDescription);
	}
	NewContent->DebugDescription = *WorkingDebugDescription;
#else
	FString DebugExtension;
#endif 

	Material->SetCompilingShaderMap(this);

	// Register this shader map in the global map with the material's ID.
	Register(InPlatform);

	// Mark the shader map as not having been finalized with ProcessCompilationResults
	bCompilationFinalized = false;

	// Mark as not having been compiled
	bCompiledSuccessfully = false;

	if (PrecompileMode != EMaterialShaderPrecompileMode::None)
	{
		EShaderCompileJobPriority CompilePriority = EShaderCompileJobPriority::Low;
		if (PrecompileMode == EMaterialShaderPrecompileMode::Synchronous)
		{
			CompilePriority = EShaderCompileJobPriority::High;
		}
		else if (!Material->IsPersistent())
		{
			// Note: using Material->IsPersistent() to detect whether this is a preview material which should have higher priority over background compiling
			CompilePriority = EShaderCompileJobPriority::Normal;
		}

		// Material can filter out all our shader types, essentially preventing the compilation from happening, which can make the shadermap stuck in
		// "always being compiled" mode. If we find out that we submitted 0 jobs, consider compilation finished.
		// Note that the shader map can be shared between FMaterials. Here we assume that they all will filter the shader types the same
		// (and have no good way to check this).
		if (SubmitCompileJobs(CompilingId, Material, MaterialEnvironment, CompilePriority) == 0)
		{
			RemoveCompilingDependency(Material);
			if (CompilingId == 0u)
			{
				bCompilationFinalized = true;
				bCompiledSuccessfully = true;
				// create resource code even if it's empty (needed during the serialization and possibly other places).
				GetResourceCode();

#if WITH_EDITOR
				if (bIsPersistent)
				{
					SaveToDerivedDataCache(FConservativeShaderParameters(Material));
				}
#endif
			}
		}
	}

	// Compile the shaders for this shader map now if the material is not deferring and deferred compiles are not enabled globally
	// If we're early in the startup we can save some time by compiling all special/default materials asynchronously, even if normally they are synchronous
	if (CompilingId != 0u && PrecompileMode == EMaterialShaderPrecompileMode::Synchronous && !PoolSpecialMaterialsCompileJobs())
	{
		TArray<int32> CurrentShaderMapId;
		CurrentShaderMapId.Add(CompilingId);
		GShaderCompilingManager->FinishCompilation(
			GetFriendlyName(),
			CurrentShaderMapId);
	}
}

static FHashedName GetPreprocessedSourceKey(const FVertexFactoryType* VertexFactoryType, const FShaderType* ShaderType, int32 PermutationId)
{
	if (VertexFactoryType)
	{
		return FHashedName(FString::Printf(TEXT("%s/%s/%d"), VertexFactoryType->GetName(), ShaderType->GetName(), PermutationId));
	}
	else
	{
		return FHashedName(FString::Printf(TEXT("%s/%d"), ShaderType->GetName(), PermutationId));
	}
}

FShader* FMaterialShaderMap::ProcessCompilationResultsForSingleJob(FShaderCompileJob* SingleJob, const FShaderPipelineType* ShaderPipeline, uint64 MaterialShaderMapHash)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::ProcessCompilationResultsForSingleJob);

	check(SingleJob);
	const FShaderCompileJob& CurrentJob = *SingleJob;
	check(CurrentJob.Id == CompilingId);
	checkf(CurrentJob.Input.Target.Platform == GetShaderPlatform(), TEXT("Job %s platform %s does not match shader map platform %s"), 
		*CurrentJob.Input.ShaderName,
		*LexToString(CurrentJob.Input.Target.GetPlatform()),
		*LexToString(GetShaderPlatform()));

	GetResourceCode()->AddShaderCompilerOutput(CurrentJob.Output, CurrentJob.Key, SingleJob->Input.GenerateDebugInfo());

#if ALLOW_SHADERMAP_DEBUG_DATA
	CompileTime += SingleJob->Output.CompileTime;
#endif
	FShader* Shader = nullptr;
	if (CurrentJob.Key.VFType)
	{
		const FVertexFactoryType* VertexFactoryType = CurrentJob.Key.VFType;
		check(VertexFactoryType->IsUsedWithMaterials());
		FMeshMaterialShaderMap* MeshShaderMap = AcquireMeshShaderMap(VertexFactoryType);

		check(MeshShaderMap);
		const FMeshMaterialShaderType* MeshMaterialShaderType = CurrentJob.Key.ShaderType->GetMeshMaterialShaderType();
		check(MeshMaterialShaderType);
		Shader = MeshMaterialShaderType->FinishCompileShader(GetContent()->MaterialCompilationOutput.UniformExpressionSet, MaterialShaderMapHash, CurrentJob, ShaderPipeline, GetFriendlyName());
		check(Shader);
		if (!ShaderPipeline)
		{
			//check(!MeshShaderMap->HasShader(MeshMaterialShaderType, CurrentJob.PermutationId));
			Shader = MeshShaderMap->FindOrAddShader(MeshMaterialShaderType->GetHashedName(), CurrentJob.Key.PermutationId, Shader);
		}
	}
	else
	{
		const FMaterialShaderType* MaterialShaderType = CurrentJob.Key.ShaderType->GetMaterialShaderType();
		check(MaterialShaderType);
		Shader = MaterialShaderType->FinishCompileShader(GetContent()->MaterialCompilationOutput.UniformExpressionSet, MaterialShaderMapHash, CurrentJob, ShaderPipeline, GetFriendlyName());

		check(Shader);
		if (!ShaderPipeline)
		{
			//check(!GetContent()->HasShader(MaterialShaderType, CurrentJob.PermutationId));
			Shader = GetMutableContent()->FindOrAddShader(MaterialShaderType->GetHashedName(), CurrentJob.Key.PermutationId, Shader);
		}
	}

	// add shader source
	{
		// Keep the preprocessed source list sorted by a name constructed from VF/ShaderType/PermutationId and deduplicate entries
		const FHashedName Key = GetPreprocessedSourceKey(CurrentJob.Key.VFType, CurrentJob.Key.ShaderType, CurrentJob.Key.PermutationId);

		const int32 Index = Algo::LowerBoundBy(GetMutableContent()->ShaderProcessedSource, Key, [](const FMaterialProcessedSource& Value) { return Value.Name; });
		if (Index >= GetMutableContent()->ShaderProcessedSource.Num() || GetMutableContent()->ShaderProcessedSource[Index].Name != Key)
		{
			GetMutableContent()->ShaderProcessedSource.EmplaceAt(Index, Key, CurrentJob.GetFinalSourceView().GetData());
		}
	}

	return Shader;
}

void FMaterialShaderMap::AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const
{
#if ALLOW_SHADERMAP_DEBUG_DATA
	check(IsRunningCookCommandlet()); 
	for (const TPair<uint32, TPair<FString, TSharedPtr<FShaderCompileJobStatus>>>& StatusPair : InFlightJobs)
	{
		TSharedPtr<FShaderCompileJobStatus> Status = StatusPair.Value.Value;
		OutDebugInfo << "Waiting on job [" << StatusPair.Value.Key << "]";
		if (!Status->GetInputHash().IsZero())
		{
			OutDebugInfo << " (input hash " << LexToString(Status->GetInputHash()) << ")";
		}
		OutDebugInfo << "; current status is: " << LexToString(Status->GetStatus()) << ", completion method is: " << LexToString(Status->GetCompletionMethod()) << "\n";
	}
#endif
}

void FMaterialShaderMap::ProcessCompilationResults(const TArray<FShaderCommonCompileJobPtr>& InCompilationResults, int32& InOutJobIndex, float& TimeBudget)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::ProcessCompilationResults);

	check(!bCompilationFinalized);

	double StartTime = FPlatformTime::Seconds();

	uint64 MaterialShaderMapHash = ShaderMapId.GetMaterialHash();

	do
	{
		auto& BaseJob = InCompilationResults[InOutJobIndex++];
#if ALLOW_SHADERMAP_DEBUG_DATA
		if (IsRunningCookCommandlet())
		{
			InFlightJobs.Remove(BaseJob->Hash);
		}
#endif

		FShaderCompileJob* SingleJob = BaseJob->GetSingleShaderJob();
		if (SingleJob)
		{
			FShader* Shader = ProcessCompilationResultsForSingleJob(SingleJob, nullptr, MaterialShaderMapHash);
			for (const auto& Pair : SingleJob->SharingPipelines)
			{
				const FVertexFactoryType* VFType = Pair.Key;
				FShaderMapContent* ShaderMapForPipeline = GetMutableContent();
				if (VFType)
				{
					ShaderMapForPipeline = AcquireMeshShaderMap(VFType->GetHashedName());
				}

				for (const FShaderPipelineType* PipelineType : Pair.Value)
				{
					FShaderPipeline* Pipeline = ShaderMapForPipeline->GetShaderPipeline(PipelineType);
					if (!Pipeline)
					{
						Pipeline = new FShaderPipeline(PipelineType);
						ShaderMapForPipeline->AddShaderPipeline(Pipeline);
					}
					Pipeline->AddShader(Shader, SingleJob->Key.PermutationId);
				}
			}
		}
		else
		{
			auto* PipelineJob = BaseJob->GetShaderPipelineJob();
			check(PipelineJob);

			const FShaderPipelineCompileJob& CurrentJob = *PipelineJob;
			check(CurrentJob.Id == CompilingId);

			const FVertexFactoryType* VertexFactoryType = CurrentJob.StageJobs[0]->GetSingleShaderJob()->Key.VFType;
			FShaderPipeline* ShaderPipeline = new FShaderPipeline(CurrentJob.Key.ShaderPipeline);
			if (VertexFactoryType)
			{
				check(VertexFactoryType->IsUsedWithMaterials());
				FMeshMaterialShaderMap* MeshShaderMap = AcquireMeshShaderMap(VertexFactoryType);
				check(MeshShaderMap);
				ShaderPipeline = MeshShaderMap->FindOrAddShaderPipeline(ShaderPipeline);
			}
			else
			{
				ShaderPipeline = GetMutableContent()->FindOrAddShaderPipeline(ShaderPipeline);
			}

			for (int32 Index = 0; Index < CurrentJob.StageJobs.Num(); ++Index)
			{
				SingleJob = CurrentJob.StageJobs[Index]->GetSingleShaderJob();
				FShader* Shader = ProcessCompilationResultsForSingleJob(SingleJob, PipelineJob->Key.ShaderPipeline, MaterialShaderMapHash);
				Shader = ShaderPipeline->FindOrAddShader(Shader, SingleJob->Key.PermutationId);
				check(VertexFactoryType == CurrentJob.StageJobs[Index]->GetSingleShaderJob()->Key.VFType);
			}
			ShaderPipeline->Validate(CurrentJob.Key.ShaderPipeline);
		}

		double NewStartTime = FPlatformTime::Seconds();
		TimeBudget -= NewStartTime - StartTime;
		StartTime = NewStartTime;
	}
	while ((TimeBudget > 0.0f) && InOutJobIndex < InCompilationResults.Num());
}

#endif // WITH_EDITOR

uint64 GetMaterialShaderParametersHash(const FMaterialShaderParameters& MaterialParameters, uint64 Seed)
{
	if (MaterialParameters.MaterialShaderTags.Num())
	{
		uint32 ShaderTagsHash = GetArrayHash<FName>(MaterialParameters.MaterialShaderTags.GetData(), MaterialParameters.MaterialShaderTags.Num());
		Seed = Seed ^ (uint64)ShaderTagsHash;
	}

	return CityHash64WithSeed((char*)&static_cast<const FMaterialShaderParametersBase&>(MaterialParameters), sizeof(FMaterialShaderParametersBase), Seed);
}

class FMaterialShaderMapLayoutCache
{
public:
	static FMaterialShaderMapLayoutCache& Get()
	{
		static FMaterialShaderMapLayoutCache Instance;
		return Instance;
	}

	const FMaterialShaderMapLayout& AcquireLayout(EShaderPlatform Platform, EShaderPermutationFlags Flags, const FMaterialShaderParameters& MaterialParameters)
	{
		const uint64 ParameterHash = GetMaterialShaderParametersHash(MaterialParameters, (uint64)Platform | ((uint64)Flags << 32));

		FMaterialShaderMapLayout* Layout = FindOrCreateLayout(Platform, Flags, ParameterHash, MaterialParameters);

		return *Layout;
	}

private:
	FMaterialShaderMapLayout* FindOrCreateLayout(const EShaderPlatform Platform, const EShaderPermutationFlags Flags, const uint64 ParameterHash, const FMaterialShaderParameters& MaterialParameters)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMapLayoutCache::FindOrCreateLayout);

		FMaterialShaderMapLayout* Layout = nullptr;
		{
			FReadScopeLock Locker(LayoutLock);
			Layout = FindLayout(ParameterHash);
		}

		if (!Layout)
		{
			FWriteScopeLock Locker(LayoutLock);
			// Need to check for existing index again once we've taken the write-lock
			// TODO maybe better to create the layout without any lock, then discard if another thread created the same layout
			Layout = FindLayout(ParameterHash);
			if (!Layout)
			{
				Layout = new FMaterialShaderMapLayout();
				CreateLayout(*Layout, Platform, Flags, MaterialParameters);
				Layout->Platform = Platform;

				const int32 Index = ShaderMapLayouts.Add(TUniquePtr<FMaterialShaderMapLayout>(Layout));
				MaterialParameterHashes.Add(ParameterHash);
				ShaderMapHashTable.Add(ParameterHash, Index);

				check(MaterialParameterHashes.Num() == ShaderMapLayouts.Num());
			}
		}

		check(Layout);
		return Layout;
	}

	FMaterialShaderMapLayout* FindLayout(uint64 ParameterHash)
	{
		for (int32 Index = ShaderMapHashTable.First(ParameterHash); ShaderMapHashTable.IsValid(Index); Index = ShaderMapHashTable.Next(Index))
		{
			if (MaterialParameterHashes[Index] == ParameterHash)
			{
				return ShaderMapLayouts[Index].Get();
			}
		}
		return nullptr;
	}

	void CreateLayout(FMaterialShaderMapLayout& Layout, EShaderPlatform Platform, EShaderPermutationFlags Flags, const FMaterialShaderParameters& MaterialParameters)
	{
		SCOPED_LOADTIMER(FMaterialShaderMapLayoutCache_CreateLayout);

		const TArray<FShaderType*>& SortedMaterialShaderTypes = FShaderType::GetSortedTypes(FShaderType::EShaderTypeForDynamicCast::Material);
		const TArray<FShaderType*>& SortedMeshMaterialShaderTypes = FShaderType::GetSortedTypes(FShaderType::EShaderTypeForDynamicCast::MeshMaterial);
		const TArray<FShaderPipelineType*>& SortedMaterialPipelineTypes = FShaderPipelineType::GetSortedTypes(FShaderType::EShaderTypeForDynamicCast::Material);
		const TArray<FShaderPipelineType*>& SortedMeshMaterialPipelineTypes = FShaderPipelineType::GetSortedTypes(FShaderType::EShaderTypeForDynamicCast::MeshMaterial);

		FShaderHashBuilder Hasher;
#if WITH_EDITOR
		// GetSourceHash is only available if AllowShaderCompiling (it asserts otherwise).
		// We only need the hash of sources in the editor, to support incremental cooking's need to know
		// when to recook the Material.
		check(AllowShaderCompiling());
		FShaderHashBuilder ShaderCompileHasher;
#endif

		for (FShaderType* BaseShaderType : SortedMaterialShaderTypes)
		{
			// Find this shader type in the material's shader map.
			FMaterialShaderType* ShaderType = static_cast<FMaterialShaderType*>(BaseShaderType);
			const int32 PermutationCount = ShaderType->GetPermutationCount();
			for (int32 PermutationId = 0; PermutationId < PermutationCount; ++PermutationId)
			{
				if (ShaderType->ShouldCompilePermutation(Platform, MaterialParameters, PermutationId, Flags))
				{
					Layout.Shaders.Add(FShaderLayoutEntry(ShaderType, PermutationId));

					const uint64 TypeNameHash = ShaderType->GetHashedName().GetHash();
					Hasher.Update((uint8*)&TypeNameHash, sizeof(TypeNameHash));
					Hasher.Update((uint8*)&PermutationId, sizeof(PermutationId));
#if WITH_EDITOR
					const FSHAHash& Hash = ShaderType->GetSourceHash(Platform);
					ShaderCompileHasher.Update((uint8*)&Hash, sizeof(Hash));
#endif
				}
			}
		}

		if (RHISupportsShaderPipelines(Platform))
		{
			// Iterate over all pipeline types
			for (FShaderPipelineType* ShaderPipelineType : SortedMaterialPipelineTypes)
			{
				if (FMaterialShaderType::ShouldCompilePipeline(ShaderPipelineType, Platform, MaterialParameters, Flags))
				{
					Layout.ShaderPipelines.Add(ShaderPipelineType);

					const uint64 TypeNameHash = ShaderPipelineType->GetHashedName().GetHash();
					Hasher.Update((uint8*)&TypeNameHash, sizeof(TypeNameHash));
#if WITH_EDITOR
					const FSHAHash& Hash = ShaderPipelineType->GetSourceHash(Platform);
					ShaderCompileHasher.Update((uint8*)&Hash, sizeof(Hash));
#endif
				}
			}
		}

		for (FVertexFactoryType* VertexFactoryType : FVertexFactoryType::GetSortedMaterialTypes())
		{
			FMeshMaterialShaderMapLayout* MeshLayout = nullptr;
			for (FShaderType* BaseShaderType : SortedMeshMaterialShaderTypes)
			{
				FMeshMaterialShaderType* ShaderType = static_cast<FMeshMaterialShaderType*>(BaseShaderType);

				if (!FMeshMaterialShaderType::ShouldCompileVertexFactoryPermutation(Platform, MaterialParameters, VertexFactoryType, ShaderType, Flags))
				{
					continue;
				}

#if WITH_EDITOR
				const FSHAHash& VertexFactorySourceHash = VertexFactoryType->GetSourceHash(Platform);
				ShaderCompileHasher.Update((uint8*)&VertexFactorySourceHash, sizeof(VertexFactorySourceHash));
#endif

				const int32 PermutationCount = ShaderType->GetPermutationCount();
				for (int32 PermutationId = 0; PermutationId < PermutationCount; ++PermutationId)
				{
					if (ShaderType->ShouldCompilePermutation(Platform, MaterialParameters, VertexFactoryType, PermutationId, Flags))
					{
						if (!MeshLayout)
						{
							MeshLayout = new(Layout.MeshShaderMaps) FMeshMaterialShaderMapLayout(VertexFactoryType);
						}
						MeshLayout->Shaders.Add(FShaderLayoutEntry(ShaderType, PermutationId));

						const uint64 TypeNameHash = ShaderType->GetHashedName().GetHash();
						Hasher.Update((uint8*)&TypeNameHash, sizeof(TypeNameHash));
						Hasher.Update((uint8*)&PermutationId, sizeof(PermutationId));
#if WITH_EDITOR
						const FSHAHash& Hash = ShaderType->GetSourceHash(Platform);
						ShaderCompileHasher.Update((uint8*)&Hash, sizeof(Hash));
#endif
					}
				}
			}

			if (RHISupportsShaderPipelines(Platform))
			{
				for (FShaderPipelineType* ShaderPipelineType : SortedMeshMaterialPipelineTypes)
				{
					if (!FMeshMaterialShaderType::ShouldCompileVertexFactoryPipeline(ShaderPipelineType, Platform, MaterialParameters, VertexFactoryType, Flags))
					{
						continue;
					}

					if (FMeshMaterialShaderType::ShouldCompilePipeline(ShaderPipelineType, Platform, MaterialParameters, VertexFactoryType, Flags))
					{
						// Now check the completeness of the shader map
						if (!MeshLayout)
						{
							MeshLayout = new(Layout.MeshShaderMaps) FMeshMaterialShaderMapLayout(VertexFactoryType);
						}
						MeshLayout->ShaderPipelines.Add(ShaderPipelineType);

						const uint64 TypeNameHash = ShaderPipelineType->GetHashedName().GetHash();
						Hasher.Update((uint8*)&TypeNameHash, sizeof(TypeNameHash));
#if WITH_EDITOR
						const FSHAHash& Hash = ShaderPipelineType->GetSourceHash(Platform);
						ShaderCompileHasher.Update((uint8*)&Hash, sizeof(Hash));
#endif
					}
				}
			}
		}

		Layout.ShaderMapHash = Hasher.Finalize();
#if WITH_EDITOR
		ShaderCompileHasher.Update((uint8*)&Layout.ShaderMapHash, sizeof(Layout.ShaderMapHash));
		Layout.ShaderCompileHash = ShaderCompileHasher.Finalize();
#endif
	}

	TArray<TUniquePtr<FMaterialShaderMapLayout>> ShaderMapLayouts;
	TArray<uint64> MaterialParameterHashes;
	FHashTable ShaderMapHashTable;
	FRWLock LayoutLock;
};

const FMaterialShaderMapLayout& AcquireMaterialShaderMapLayout(EShaderPlatform Platform, EShaderPermutationFlags Flags, const FMaterialShaderParameters& MaterialParameters)
{
	return FMaterialShaderMapLayoutCache::Get().AcquireLayout(Platform, Flags, MaterialParameters);
}

bool FMaterialShaderMap::IsComplete(const FMaterial* Material, bool bSilent)
{
	SCOPED_LOADTIMER(FMaterialShaderMap_IsComplete);

	const FMaterialShaderMapContent* LocalContent = GetContent();
	const EShaderPlatform Platform = LocalContent->GetShaderPlatform();
	const FMaterialCompilationOutput& CompilationOutput = GetCompilationOutput();
	const FCompilationShaderParameters MaterialParameters(Material, CompilationOutput);


	// IsODSCOnly shaders aren't in shader map so don't include when gathering for IsComplete().
	EShaderPermutationFlags Flags = ShaderMapId.GetPermutationFlags() & ~EShaderPermutationFlags::IsODSCOnly;
	const FMaterialShaderMapLayout& Layout = AcquireMaterialShaderMapLayout(Platform, Flags, MaterialParameters);
	if (Layout.ShaderMapHash == LocalContent->ShaderContentHash)
	{
		return true;
	}

	// If our hash doesn't match the cached layout hash, shader map may still be complete
	// This can happen if FMaterial::ShouldCache is set to return false for any shaders that are included in the cached layout

	{
		// exclude shaders that are going to be uniquely used by the pipelines
		FPipelinedShaderFilter PipelinedShaderFilter(Platform, Layout.ShaderPipelines);
		for (const FShaderLayoutEntry& Shader : Layout.Shaders)
		{
			if (!LocalContent->HasShader(Shader.ShaderType, Shader.PermutationId))
			{
				if (!PipelinedShaderFilter.IsPipelinedType(Shader.ShaderType) && Material->ShouldCache(Shader.ShaderType, nullptr))
				{
					if (!bSilent)
					{
						UE_LOGF(LogMaterial, Warning, "Incomplete material %ls, missing FMaterialShader (%ls, %d).", *Material->GetFriendlyName(), Shader.ShaderType->GetName(), Shader.PermutationId);
					}
					return false;
				}
			}
		}
	}

	for (const FShaderPipelineType* Pipeline : Layout.ShaderPipelines)
	{
		if (!LocalContent->HasShaderPipeline(Pipeline))
		{
			if (Material->ShouldCachePipeline(Pipeline, nullptr))
			{
				if (!bSilent)
				{
					UE_LOGF(LogMaterial, Warning, "Incomplete material %ls, missing pipeline %ls.", *Material->GetFriendlyName(), Pipeline->GetName());
				}
				return false;
			}
		}
	}

	for (const FMeshMaterialShaderMapLayout& MeshLayout : Layout.MeshShaderMaps)
	{
		FPipelinedShaderFilter PipelinedShaderFilter(Platform, MeshLayout.ShaderPipelines);
		const FMeshMaterialShaderMap* MeshShaderMap = LocalContent->GetMeshShaderMap(MeshLayout.VertexFactoryType->GetHashedName());

		for (const FShaderLayoutEntry& Shader : MeshLayout.Shaders)
		{
			if (Material->ShouldCache(Shader.ShaderType, MeshLayout.VertexFactoryType))
			{
				if (!PipelinedShaderFilter.IsPipelinedType(Shader.ShaderType) && (!MeshShaderMap || !MeshShaderMap->HasShader(Shader.ShaderType, Shader.PermutationId)))
				{
					if (!bSilent)
					{
						if (!MeshShaderMap)
						{
							UE_LOGF(LogMaterial, Warning, "Incomplete material %ls, missing Vertex Factory %ls.", *Material->GetFriendlyName(), MeshLayout.VertexFactoryType->GetName());
						}
						else
						{
							UE_LOGF(LogMaterial, Warning, "Incomplete material %ls, missing (%ls, %d) from %ls.", *Material->GetFriendlyName(), Shader.ShaderType->GetName(), Shader.PermutationId, MeshLayout.VertexFactoryType->GetName());
						}
					}
					return false;
				}
			}
		}

		for (const FShaderPipelineType* Pipeline : MeshLayout.ShaderPipelines)
		{
			if (!MeshShaderMap || !MeshShaderMap->HasShaderPipeline(Pipeline))
			{
				if (Material->ShouldCachePipeline(Pipeline, MeshLayout.VertexFactoryType))
				{
					if (!bSilent)
					{
						if (!MeshShaderMap)
						{
							UE_LOGF(LogMaterial, Warning, "Incomplete material %ls, missing Vertex Factory %ls.", *Material->GetFriendlyName(), MeshLayout.VertexFactoryType->GetName());
						}
						else
						{
							UE_LOGF(LogMaterial, Warning, "Incomplete material %ls, missing pipeline %ls from %ls.", *Material->GetFriendlyName(), Pipeline->GetName(), MeshLayout.VertexFactoryType->GetName());
						}
					}
					return false;
				}
			}
		}
	}

	// Was missing some shaders from the initial layout, but all of those shaders were explicitly exluced by our FMaterial::ShouldCache implementation
	return true;
}

FPSOPrecacheDataArray FMaterialShaderMap::CollectPSOPrecacheData(const FMaterialPSOPrecacheParams& PrecacheParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::CollectPSOPrecacheData);

	// Shouldn't get here if the type doesn't support precaching
	check(PrecacheParams.VertexFactoryData.VertexFactoryType == nullptr || PrecacheParams.VertexFactoryData.VertexFactoryType->SupportsPSOPrecaching());

#if PSO_PRECACHING_VALIDATE
	ConditionalBreakOnPSOPrecacheMaterial(*PrecacheParams.Material, INDEX_NONE);
#endif // PSO_PRECACHING_VALIDATE

	// Has data for this VF type
	const FMaterialShaderMapContent* LocalContent = GetContent();
	if (LocalContent == nullptr)
	{
		return FPSOPrecacheDataArray();
	}

	// Check if all shader chunks for this shader map are resolved (i.e. IoStore chunks are installed).
	// If not, bail out early to avoid a fatal error in GetResultOrDie when creating shaders from unresolved chunks.
	// The higher-level rendering system will use the shader map once it becomes resolved, so we only risk a hitch.
	if (!ensureMsgf(GetResource() && GetResource()->IsResolved(), TEXT("Shadermap resource '%s' is not resolved during PSO precaching"), PrecacheParams.Material ? *PrecacheParams.Material->GetAssetName() : TEXT("<Unknown>")))
	{
		return FPSOPrecacheDataArray();
	}

	bool bHasShaderMapForVFType = PrecacheParams.Material->IsUIMaterial() || (PrecacheParams.VertexFactoryData.VertexFactoryType && LocalContent->GetMeshShaderMap(PrecacheParams.VertexFactoryData.VertexFactoryType->GetHashedName()));
	if (!bHasShaderMapForVFType)
	{
		return FPSOPrecacheDataArray();
	}

	// Only feature level is currently set as init settings - rest is default
	// (multiview & alpha channel not taken into account here)
	FSceneTexturesConfigInitSettings SceneTexturesConfigInitSettings;
	SceneTexturesConfigInitSettings.FeatureLevel = PrecacheParams.FeatureLevel;

	FSceneTexturesConfig SceneTexturesConfig;
	SceneTexturesConfig.Init(SceneTexturesConfigInitSettings);

	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(PrecacheParams.FeatureLevel);

	FPSOPrecacheDataArray PSOInitializers;
	PSOInitializers.Reserve(32);
	
	for (int32 Index = 0; Index < FPSOCollectorCreateManager::GetPSOCollectorCount(ShadingPath); ++Index)
	{
		PSOCollectorCreateFunction CreateFunction = FPSOCollectorCreateManager::GetCreateFunction(ShadingPath, Index);
		if (CreateFunction)
		{
			IPSOCollector* PSOCollector = CreateFunction(PrecacheParams.FeatureLevel);
			if (PSOCollector != nullptr)
			{
#if PSO_PRECACHING_VALIDATE
				ConditionalBreakOnPSOPrecacheMaterial(*PrecacheParams.Material, Index);
#endif // PSO_PRECACHING_VALIDATE

				PSOCollector->CollectPSOInitializers(SceneTexturesConfig, *PrecacheParams.Material, PrecacheParams.VertexFactoryData, PrecacheParams.PrecachePSOParams, PSOInitializers);
				delete PSOCollector;
			}
		}
	}

	return PSOInitializers;
}




#if WITH_EDITOR
const FMemoryImageString* FMaterialShaderMap::GetShaderSource(const FVertexFactoryType* VertexFactoryType, const FShaderType* ShaderType, int32 PermutationId) const
{
	FHashedName Key = GetPreprocessedSourceKey(VertexFactoryType, ShaderType, PermutationId);
	for (const FMaterialProcessedSource& Source : GetContent()->ShaderProcessedSource)
	{
		if (Source.Name == Key)
		{
			return &Source.Source;
		}
	}

	return nullptr;
}
#endif // WITH_EDITOR

#if !UE_BUILD_SHIPPING
void FMaterialShaderMap::DumpShaderList(FShaderListReport& Out) const
{
	Super::DumpShaderList(Out);
	
	for (FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
	{
		if (MeshShaderMap)
		{
			MeshShaderMap->DumpShaderList(*this, Out);
		}
	}
}
#endif // !UE_BUILD_SHIPPING

void FMaterialShaderMap::GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
{
	uint64 MaterialShaderMapHash;
#if WITH_EDITOR
	MaterialShaderMapHash = ShaderMapId.IsCookedId() ? ShaderMapId.CookedShaderMapIdHash : ShaderMapId.GetMaterialHash();
#else
	MaterialShaderMapHash = ShaderMapId.CookedShaderMapIdHash;
#endif

	GetContent()->GetShaderList(*this, MaterialShaderMapHash, OutShaders);
	for (FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
	{
		if (MeshShaderMap)
		{
			MeshShaderMap->GetShaderList(*this, MaterialShaderMapHash, OutShaders);
		}
	}
}

void FMaterialShaderMap::GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, OutShaders);
	for (FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
	{
		if (MeshShaderMap)
		{
			MeshShaderMap->GetShaderList(*this, OutShaders);
		}
	}
}

void FMaterialShaderMap::GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const
{
	GetContent()->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
	for(FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
	{
		if (MeshShaderMap)
		{
			MeshShaderMap->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
		}
	}
}

uint32 FMaterialShaderMap::GetShaderNum() const
{
	return GetContent()->GetNumShaders();
}

/**
 * Registers a material shader map in the global map so it can be used by materials.
 */
void FMaterialShaderMap::Register(EShaderPlatform InShaderPlatform)
{
	// Lazy initializer to bind OnSharedShaderMapResourceExplicitRelease to ShaderMapResourceExplicitRelease
	static struct FMaterialShaderMapInnerLazyInitializer
	{
		FMaterialShaderMapInnerLazyInitializer()
		{
			OnSharedShaderMapResourceExplicitRelease.BindStatic(&FMaterialShaderMap::ShaderMapResourceExplicitRelease);
		}
	} MaterialShaderMapInnerLazyInitializer;

	if (CreateShadersOnLoad() && GMaxRHIShaderPlatform == InShaderPlatform)
	{
		FShaderMapResource* ShaderResource = GetResource();
		if (ShaderResource)
		{
			ShaderResource->BeginCreateAllShaders();
		}
	}

	if (!bRegistered)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShaderMaps);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<int>* CsvStatNumShaderMaps = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShaderMaps"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatNumShaderMaps->Add(1);
#endif
	}

	{
		FScopeLock ScopeLock(&GIdToMaterialShaderMapCS);

		FMaterialShaderMap *CachedMap = GIdToMaterialShaderMap[GetShaderPlatform()].FindRef(ShaderMapId);

		// Only add new item if there's not already one in the map.
		// Items can possibly already be in the map because the GIdToMaterialShaderMapCS is not being locked between search & register lookups and new shader might be compiled
		if (CachedMap == nullptr)
		{
			GIdToMaterialShaderMap[GetShaderPlatform()].Add(ShaderMapId,this);
			bRegistered = true;
		}
		else
		{
			// Sanity check - We did not register so either bRegistered is false or this item is already in the map
			check((bRegistered == false && CachedMap != this) || (bRegistered == true && CachedMap == this));
		}
	}
}

void FMaterialShaderMap::RegisterForODSC(EShaderPlatform InShaderPlatform)
{
	Register(InShaderPlatform);

	{
		FScopeLock ScopeLock(&GIdToMaterialShaderMapCS);

		FMaterialShaderMap* CachedMap = GIdToMaterialShaderMap[GetShaderPlatform()].FindRef(ShaderMapId);
		if (CachedMap == nullptr)
		{
			GIdToMaterialShaderMap[GetShaderPlatform()].Add(ShaderMapId, this);
			bRegistered = true;
		}
		else
		{	
			if (CachedMap != this)
			{
				// deregister the existing map.
				int RemoveIndex = GIdToMaterialShaderMap[GetShaderPlatform()].Remove(ShaderMapId);
				check(RemoveIndex != INDEX_NONE);
				CachedMap->bRegistered = false;

				// register ourselves.
				GIdToMaterialShaderMap[GetShaderPlatform()].Add(ShaderMapId, this);
				bRegistered = true;
			}
		}

#if WITH_ODSC
		bIsFromODSC = true;
#endif
	}
}

void FMaterialShaderMap::AddRef()
{
	FScopeLock ScopeLock(&GIdToMaterialShaderMapCS);
	check(!bDeletedThroughDeferredCleanup);
	++NumRefs;
}

void FMaterialShaderMap::Release()
{
	{
		FScopeLock ScopeLock(&GIdToMaterialShaderMapCS);

		check(NumRefs.load() > 0);
		if (--NumRefs == 0)
		{
			if (bRegistered)
			{
				bRegistered = false;
				DEC_DWORD_STAT(STAT_Shaders_NumShaderMaps);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
				TCsvPersistentCustomStat<int>* CsvStatNumShaderMaps = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShaderMaps"), CSV_CATEGORY_INDEX(Shaders));
				CsvStatNumShaderMaps->Sub(1);
#endif

				FMaterialShaderMap *CachedMap = GIdToMaterialShaderMap[GetShaderPlatform()].FindRef(ShaderMapId);

				// Map is marked as registered therefore we do expect it to be in the cache
				// If this does not happen there's a bug in the code causing ShaderMapID to be the same for two different objects.
				check(CachedMap == this);
				
				if (CachedMap == this)
				{
					GIdToMaterialShaderMap[GetShaderPlatform()].Remove(ShaderMapId);
				}
			}
			else
			{
				//sanity check - the map has not been registered and therefore should not appear in the cache
				check(GetShaderPlatform()>= EShaderPlatform::SP_NumPlatforms || GIdToMaterialShaderMap[GetShaderPlatform()].FindRef(ShaderMapId) != this);
			}

#if WITH_EDITOR
			FinalizedClone.SafeRelease();
#endif // WITH_EDITOR
			check(!bDeletedThroughDeferredCleanup);
			bDeletedThroughDeferredCleanup = true;
		}
	}
	if (bDeletedThroughDeferredCleanup)
	{
		BeginCleanup(this);
	}
}

FMaterialShaderMap::FMaterialShaderMap() :
	NumRefs(0),
	bDeletedThroughDeferredCleanup(false),
	bRegistered(false),
	bCompilationFinalized(true),
	bCompiledSuccessfully(true),
	bIsPersistent(false)
#if WITH_ODSC
	, bIsFromODSC(false)
#endif
{
	checkSlow(IsInGameThread() || IsAsyncLoading());
#if ALLOW_SHADERMAP_DEBUG_DATA
	CompileTime = 0.f;
	{
		FScopeLock AllMatSMAccess(&AllMaterialShaderMapsGuard);
		AllMaterialShaderMaps.Add(this);
	}
#endif
}

FMaterialShaderMap::~FMaterialShaderMap()
{ 
	checkSlow(IsInGameThread() || IsAsyncLoading());
	check(bDeletedThroughDeferredCleanup);
	check(!bRegistered);
#if ALLOW_SHADERMAP_DEBUG_DATA
	if(GShaderCompilerStats != 0 && GetContent())
	{
		FString Path = GetMaterialPath();
		if (Path.IsEmpty())
		{
			Path = GetFriendlyName();
		}
		GShaderCompilerStats->RegisterCookedShaders(GetShaderNum(), CompileTime, GetShaderPlatform(), Path, GetDebugDescription());
	}
	{
		FScopeLock AllMatSMAccess(&AllMaterialShaderMapsGuard);
		AllMaterialShaderMaps.RemoveSwap(this);
	}
#endif

	// This is an unsavory hack to repair STAT_Shaders_NumShadersLoaded not being calculated right in the superclass because the Content class isn't allowed to have virtual functions atm.
	// A better way is tracked in UE-127112
#if STATS || CSV_PROFILER_STATS
	if (GetContent())
	{
		// Account for the extra shaders we missed because Content's GetNumShaders() isn't virtual and add them here. Note: Niagara needs the same
		uint32 BaseClassShaders = FShaderMapBase::GetContent()->GetNumShaders();
		uint32 TotalShadersIncludingBaseClass = GetContent()->GetNumShaders();
		uint32 OwnShaders = TotalShadersIncludingBaseClass - BaseClassShaders;
		DEC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, OwnShaders);

#if CSV_PROFILER_STATS 
		static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatNumShadersLoaded->Sub(OwnShaders);
#endif // CSV_PROFILER_STATS
	}
#endif // STATS || CSV_PROFILER_STATS
}

#if WITH_EDITOR
FMaterialShaderMap* FMaterialShaderMap::AcquireFinalizedClone()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialShaderMap::AcquireFinalizedClone);

	checkSlow(IsInGameThread());

	const FMaterialShaderMapContent* LocalContent = GetContent();
	const FShaderMapResourceCode* LocalCode = GetResourceCode();

	checkf(LocalContent, TEXT("Can't clone shader map %s"), GetFriendlyName());
	checkf(LocalCode, TEXT("Can't clone shader map %s"), GetFriendlyName());

	if (GetFrozenContentSize() == 0u)
	{
		// If content isn't frozen yet, make sure to finalize it before making clone
		const_cast<FMaterialShaderMapContent*>(LocalContent)->Finalize(LocalCode);
	}

	LocalContent->Validate(*this);

	if (FinalizedClone && FinalizedClone->GetShaderContentHash() == GetShaderContentHash())
	{
		// Re-used existing clone if it's still valid
		return FinalizedClone;
	}

	FMaterialShaderMap* Clone = new FMaterialShaderMap();
	Clone->ShaderMapId = ShaderMapId;
	Clone->bCompilationFinalized = bCompilationFinalized;
	Clone->bCompiledSuccessfully = bCompiledSuccessfully;
	Clone->bIsPersistent = bIsPersistent;
	Clone->AssignCopy(*this);
	Clone->AssociateWithAssets(GetAssociatedAssets());

	FinalizedClone = Clone;
	return Clone;
}

FMaterialShaderMap* FMaterialShaderMap::GetFinalizedClone() const
{
	return FinalizedClone;
}
#endif // WITH_EDITOR

bool FMaterialShaderMap::Serialize(FShaderSerializeContext& Ctx)
{
	SCOPED_LOADTIMER(FMaterialShaderMap_Serialize);
	// Note: This is saved to the DDC, not into packages (except when cooked)
	// Backwards compatibility therefore will not work based on the version of Ar
	// Instead, just bump MATERIALSHADERMAP_DERIVEDDATA_VER
	FArchive& Ar = Ctx.GetMainArchive();
	ShaderMapId.Serialize(Ar, Ctx.bLoadingCooked);
	bool bSerialized = Super::Serialize(Ctx);
#if STATS || CSV_PROFILER_STATS
	// This is an unsavory hack to repair STAT_Shaders_NumShadersLoaded not being calculated right in the superclass because the Content class isn't allowed to have virtual functions atm.
	// A better way is tracked in UE-127112
	if (bSerialized && Ar.IsLoading())
	{
		// Account for the extra shaders we missed because Content's GetNumShaders() isn't virtual and add them here. Note: Niagara needs the same
		uint32 BaseClassShaders = FShaderMapBase::GetContent()->GetNumShaders();
		uint32 TotalShadersIncludingBaseClass = GetContent()->GetNumShaders();
		uint32 OwnShaders = TotalShadersIncludingBaseClass - BaseClassShaders;
		INC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, OwnShaders);

#if CSV_PROFILER_STATS
		static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatNumShadersLoaded->Add(OwnShaders);
#endif // CSV_PROFILER_STATS
	}
#endif // STATS || CSV_PROFILER_STATS
	return bSerialized;
}

#if WITH_EDITOR
uint32 FMaterialShaderMap::GetMaxTextureSamplers() const
{
	uint32 MaxTextureSamplers = GetContent()->GetMaxTextureSamplersShaderMap(*this);

	for (int32 Index = 0;Index < GetContent()->OrderedMeshShaderMaps.Num();Index++)
	{
		const FMeshMaterialShaderMap* MeshShaderMap = GetContent()->OrderedMeshShaderMaps[Index];
		if (MeshShaderMap)
		{
			MaxTextureSamplers = FMath::Max(MaxTextureSamplers, MeshShaderMap->GetMaxTextureSamplersShaderMap(*this));
		}
	}

	return MaxTextureSamplers;
}
#endif // WITH_EDITOR

FMaterialShaderMapContent::~FMaterialShaderMapContent()
{
	int a = 0;
}

const FMeshMaterialShaderMap* FMaterialShaderMapContent::GetMeshShaderMap(const FHashedName& VertexFactoryTypeName) const
{
	const int32 Index = Algo::BinarySearchBy(OrderedMeshShaderMaps, VertexFactoryTypeName, FProjectMeshShaderMapToKey());
	if (Index != INDEX_NONE)
	{
		FMeshMaterialShaderMap* MeshShaderMap = OrderedMeshShaderMaps[Index].Get();
		checkSlow(MeshShaderMap->GetVertexFactoryTypeName() == VertexFactoryTypeName);
		return MeshShaderMap;
	}
	return nullptr;
}

FMeshMaterialShaderMap* FMaterialShaderMapContent::AcquireMeshShaderMap(const FHashedName& VertexFactoryTypeName)
{
	FMeshMaterialShaderMap* ShaderMap = const_cast<FMeshMaterialShaderMap*>(GetMeshShaderMap(VertexFactoryTypeName));
	if (!ShaderMap)
	{
		ShaderMap = new FMeshMaterialShaderMap(GetShaderPlatform(), VertexFactoryTypeName);
		AddMeshShaderMap(VertexFactoryTypeName, ShaderMap);
	}
	return ShaderMap;
}

void FMaterialShaderMapContent::AddMeshShaderMap(const FHashedName& VertexFactoryTypeName, FMeshMaterialShaderMap* MeshShaderMap)
{
	check(VertexFactoryTypeName == MeshShaderMap->GetVertexFactoryTypeName());
	checkSlow(GetMeshShaderMap(VertexFactoryTypeName) == nullptr);
	const int32 Index = Algo::LowerBoundBy(OrderedMeshShaderMaps, VertexFactoryTypeName, FProjectMeshShaderMapToKey());
	OrderedMeshShaderMaps.Insert(MeshShaderMap, Index);
}

void FMaterialShaderMapContent::RemoveMeshShaderMap(const FHashedName& VertexFactoryTypeName)
{
	const int32 Index = Algo::BinarySearchBy(OrderedMeshShaderMaps, VertexFactoryTypeName, FProjectMeshShaderMapToKey());
	if (Index != INDEX_NONE)
	{
		FMeshMaterialShaderMap* MeshShaderMap = OrderedMeshShaderMaps[Index].Get();
		delete MeshShaderMap;
		OrderedMeshShaderMaps.RemoveAt(Index);
	}
}

#if WITH_EDITOR
void FMaterialShaderMap::GetEstimatedLWCFuncUsageComplexity(uint32& LWCComplexityVS, uint32& LWCComplexityPS, uint32& LWCComplexityCS) const
{
	auto GetLWCComplexity = [](const uint16 (&Usages)[(int)ELWCFunctionKind::Max]) {
		int Complexity = 0;
		for (int Idx = 0; Idx < (int)ELWCFunctionKind::Max; Idx++)
		{
			int CurUsages = Usages[Idx];
			int Cost;
			switch ((ELWCFunctionKind)Idx)
			{
				// Somewhat arbitrary, but broadly assigns cost classes compared to working in a local/translated space instead.
				case ELWCFunctionKind::Constructor:
				case ELWCFunctionKind::Promote:
					Cost = 0;
					break;
				case ELWCFunctionKind::Demote:
					Cost = 10;
					break;
				case ELWCFunctionKind::Add:
				case ELWCFunctionKind::Subtract:
					Cost = 20;
					break;
				case ELWCFunctionKind::MultiplyVectorVector:
				case ELWCFunctionKind::MultiplyVectorMatrix:
				case ELWCFunctionKind::MultiplyMatrixMatrix:
				case ELWCFunctionKind::Divide:
					Cost = 30;
					break;
				default:
					Cost = 40;
			}
			Complexity += Cost * CurUsages;
		}
		return Complexity;
	};
	LWCComplexityVS = GetLWCComplexity(GetContent()->MaterialCompilationOutput.EstimatedLWCFuncUsagesVS);
	LWCComplexityPS = GetLWCComplexity(GetContent()->MaterialCompilationOutput.EstimatedLWCFuncUsagesPS);
	LWCComplexityCS = GetLWCComplexity(GetContent()->MaterialCompilationOutput.EstimatedLWCFuncUsagesCS);
}
#endif

void FMaterialShaderMap::DumpDebugInfo(FOutputDevice& OutputDevice) const
{
	// Turn off as it makes diffing hard
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	OutputDevice.Logf(TEXT("Frequency, Target, VFType, ShaderType, SourceHash, VFSourceHash, OutputHash, IsShaderPipeline"));

	{
		TMap<FShaderId, TShaderRef<FShader>> ShadersL;
		GetShaderList(ShadersL);
		for (auto& KeyValue : ShadersL)
		{
			FShader* Shader = KeyValue.Value.GetShader();
			const FVertexFactoryType* VertexFactoryType = Shader->GetVertexFactoryType(GetPointerTable());
			OutputDevice.Logf(TEXT("%s, %s, %s, %s, %s, %s, %s, %s"),
				GetShaderFrequencyString(Shader->GetFrequency()),
				*LegacyShaderPlatformToShaderFormat(GetShaderPlatform()).ToString(),
				VertexFactoryType ? VertexFactoryType->GetName() : TEXT("null"),
				Shader->GetType(GetPointerTable())->GetName(),
				*Shader->GetHash().ToString(),
				*Shader->GetVertexFactoryHash().ToString(),
				*Shader->GetOutputHash().ToString(),
				TEXT("false")
				);
		}
	}

	{
		TArray<FShaderPipelineRef> ShaderPipelinesL;
		GetShaderPipelineList(ShaderPipelinesL);
		for (FShaderPipelineRef Value : ShaderPipelinesL)
		{
			FShaderPipeline* ShaderPipeline = Value.GetPipeline();
			TArray<TShaderRef<FShader>> ShadersL = ShaderPipeline->GetShaders(*this);
			for (TShaderRef<FShader>& Shader : ShadersL)
			{
				const FVertexFactoryType* VertexFactoryType = Shader->GetVertexFactoryType(GetPointerTable());
				OutputDevice.Logf(TEXT("%s, %s, %s, %s, %s, %s, %s, %s"),
					GetShaderFrequencyString(Shader->GetFrequency()),
					*LegacyShaderPlatformToShaderFormat(GetShaderPlatform()).ToString(),
					VertexFactoryType ? VertexFactoryType->GetName() : TEXT("null"),
					Shader->GetType(GetPointerTable())->GetName(),
					*Shader->GetHash().ToString(),
					*Shader->GetVertexFactoryHash().ToString(),
					*Shader->GetOutputHash().ToString(),
					TEXT("true")
					);
			}
		}
	}
}

#if WITH_EDITOR
void FMaterialShaderMap::InitializeForODSC(EShaderPlatform TargetShaderPlatform, const FMaterialCompilationOutput& NewCompilationOutput)
{
	// Empty Content
	FMaterialShaderMapContent* NewContent = new FMaterialShaderMapContent(TargetShaderPlatform);
	NewContent->MaterialCompilationOutput = NewCompilationOutput;
	AssignContent(NewContent);

	// Empty Code
	GetResourceCode();
}
#endif // WITH_EDITOR

void FMaterialShaderMap::PostFinalizeContent()
{
	UniformBufferLayout.SafeRelease();
	UniformBufferLayoutVS.SafeRelease();

	if (GetContent())
	{
		UniformBufferLayout = RHICreateUniformBufferLayout(GetUniformExpressionSet().GetUniformBufferLayoutInitializer());
		const FCompactUniformExpressionSet* CompactUniformsVS = GetUniformExpressionSet().GetCompactUniformsVS();
		if (CompactUniformsVS)
		{
			UniformBufferLayoutVS = RHICreateUniformBufferLayout(CompactUniformsVS->UniformBufferLayoutInitializer);
		}
	}
}

#if WITH_EDITOR
void FMaterialShaderMap::GetOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const
{
	FShaderMapBase::GetOutdatedTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);

	for (FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
	{
		if (MeshShaderMap)
		{
			MeshShaderMap->GetOutdatedTypes(*this, OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
		}
	}
}

void FMaterialShaderMap::SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, const FStableShaderKeyAndValue& SaveKeyVal)
{
	FShaderMapBase::SaveShaderStableKeys(TargetShaderPlatform, SaveKeyVal);
	for (FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
	{
		if (MeshShaderMap)
		{
			MeshShaderMap->SaveShaderStableKeys(*this, TargetShaderPlatform, SaveKeyVal);
		}
	}
}
#endif // WITH_EDITOR

void FMaterialShaderMap::ShaderMapResourceExplicitRelease(const FShaderMapResource* ShaderMapResource)
{
	EShaderPlatform ShaderPlatform = ShaderMapResource->GetPlatform();

	// visit cached Material shader map and remove ones that have been released (possibly due to GC leak) to avoid use after free in RT
	FScopeLock ScopeLock(&GIdToMaterialShaderMapCS);
	for (auto MaterialMapIterator = GIdToMaterialShaderMap[ShaderPlatform].CreateIterator(); MaterialMapIterator; ++MaterialMapIterator)
	{
		if (MaterialMapIterator->Value->GetResource() == ShaderMapResource)
		{
			// The FMaterialShaderMap uses bRegistered to track whether it has been added to the global map.
			// If we remove it through explicit release we must unregister or an assert will fire from FMaterialShaderMap::Release()
			MaterialMapIterator->Value->bRegistered = false;
			MaterialMapIterator.RemoveCurrent();
		}
	}
}

/**
 * Dump material stats for a given platform.
 * 
 * @param	Platform	Platform to dump stats for.
 */
void DumpMaterialStats(EShaderPlatform Platform)
{
#if ALLOW_DEBUG_FILES && ALLOW_SHADERMAP_DEBUG_DATA
	FDiagnosticTableViewer MaterialViewer(*FDiagnosticTableViewer::GetUniqueTemporaryFilePath(TEXT("MaterialStats")));

	//#todo-rco: Pipelines

	// Mapping from friendly material name to shaders associated with it.
	TMultiMap<FString, TShaderRef<FShader>> MaterialToShaderMap;
	TMultiMap<FString, FShaderPipeline*> MaterialToShaderPipelineMap;

	struct FMaterialInfo
	{
		uint32 CodeSize;
		bool bUseForRendering;
	};

	// Set of material names.
	TMap<FString, FMaterialInfo> MaterialStats;

	// Look at in-memory shader use.
	FScopeLock AllMatSMAccess(&FMaterialShaderMap::AllMaterialShaderMapsGuard);
	for (int32 ShaderMapIndex = 0; ShaderMapIndex < FMaterialShaderMap::AllMaterialShaderMaps.Num(); ShaderMapIndex++)
	{
		FMaterialShaderMap* MaterialShaderMap = FMaterialShaderMap::AllMaterialShaderMaps[ShaderMapIndex];
		TMap<FShaderId, TShaderRef<FShader>> Shaders;
		TArray<FShaderPipelineRef> ShaderPipelines;
		MaterialShaderMap->GetShaderList(Shaders);
		MaterialShaderMap->GetShaderPipelineList(ShaderPipelines);
		FShaderMapResource* Resource = MaterialShaderMap->GetResource();

		FMaterialInfo MI;
		MI.CodeSize = 0;

		// Add friendly name to list of materials.
		FString FriendlyName = MaterialShaderMap->GetFriendlyName();

#if !WITH_EDITORONLY_DATA
		if (FriendlyName.IsEmpty() && Resource != nullptr)
		{
			FriendlyName = Resource->GetOwnerName().ToString();
			MI.CodeSize = MaterialShaderMap->GetFrozenContentSize() + Resource->GetSizeBytes();
			MI.bUseForRendering = Resource->ContainsAtLeastOneRHIShaderCreated();
		}
#endif

		MaterialStats.FindOrAdd(FriendlyName, MI);

		// Add shaders to mapping per friendly name as there might be multiple
		for (auto& KeyValue : Shaders)
		{
			MaterialToShaderMap.AddUnique(FriendlyName, KeyValue.Value);
		}

		for (const FShaderPipelineRef& Pipeline : ShaderPipelines)
		{
			for (const TShaderRef<FShader>& Shader : Pipeline.GetShaders())
			{
				MaterialToShaderMap.AddUnique(FriendlyName, Shader);
			}
			MaterialToShaderPipelineMap.AddUnique(FriendlyName, Pipeline.GetPipeline());
		}
	}

	// Write a row of headings for the table's columns.
	MaterialViewer.AddColumn(TEXT("Name"));
	MaterialViewer.AddColumn(TEXT("Shaders"));
	MaterialViewer.AddColumn(TEXT("Code Size"));
	MaterialViewer.AddColumn(TEXT("Used For Rendering"));
	MaterialViewer.AddColumn(TEXT("Pipelines"));
	MaterialViewer.CycleRow();

	// Iterate over all materials, gathering shader stats.
	int32 TotalCodeSize = 0;
	int32 TotalShaderCount = 0;
	int32 TotalShaderPipelineCount = 0;
	int32 TotalUsedShaderMapCount = 0;

	for (TMap<FString, FMaterialInfo>::TIterator It(MaterialStats); It; ++It)
	{
		// Retrieve list of shaders in map.
		TArray<TShaderRef<FShader>> Shaders;
		MaterialToShaderMap.MultiFind(It.Key(), Shaders);
		TArray<FShaderPipeline*> ShaderPipelines;
		MaterialToShaderPipelineMap.MultiFind(It.Key(), ShaderPipelines);

		// Iterate over shaders and gather stats.
		FMaterialInfo& MI = It.Value();

#if WITH_EDITORONLY_DATA
		if (MI.CodeSize == 0)
		{
			for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ShaderIndex++)
			{
				const TShaderRef<FShader>& Shader = Shaders[ShaderIndex];
				MI.CodeSize += Shader->GetCodeSize();
			}
		}
#endif

		TotalCodeSize += MI.CodeSize;
		TotalShaderCount += Shaders.Num();
		TotalShaderPipelineCount += ShaderPipelines.Num();
		TotalUsedShaderMapCount += MI.bUseForRendering ? 1 : 0;

		// Dump stats
		MaterialViewer.AddColumn(*It.Key());
		MaterialViewer.AddColumn(TEXT("%u"), Shaders.Num());
		MaterialViewer.AddColumn(TEXT("%u"), MI.CodeSize);
		MaterialViewer.AddColumn(TEXT("%u"), MI.bUseForRendering);
		MaterialViewer.AddColumn(TEXT("%u"), ShaderPipelines.Num());
		MaterialViewer.CycleRow();
	}

	// Add a total row.
	MaterialViewer.CycleRow();
	MaterialViewer.AddColumn(TEXT("Total"));
	MaterialViewer.AddColumn(TEXT("%u"), TotalShaderCount);
	MaterialViewer.AddColumn(TEXT("%u"), TotalCodeSize);
	MaterialViewer.AddColumn(TEXT("%u"), TotalUsedShaderMapCount);
	MaterialViewer.AddColumn(TEXT("%u"), TotalShaderPipelineCount);
	MaterialViewer.CycleRow();
#endif // ALLOW_DEBUG_FILES && ALLOW_SHADERMAP_DEBUG_DATA
}

