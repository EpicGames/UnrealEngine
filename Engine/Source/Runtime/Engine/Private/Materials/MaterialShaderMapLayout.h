// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/ShaderHash.h"
#include "RHIDefinitions.h"

class FShaderType;
class FShaderPipelineType;
class FVertexFactoryType;
struct FMaterialShaderParameters;
enum class EShaderPermutationFlags : uint32;
enum EShaderPlatform : uint16;

class FShaderLayoutEntry
{
public:
	FShaderLayoutEntry() : ShaderType(nullptr), PermutationId(0) {}
	FShaderLayoutEntry(FShaderType* InShaderType, int32 InPermutationId) : ShaderType(InShaderType), PermutationId(InPermutationId) {}

	FShaderType* ShaderType;
	int32 PermutationId;
};

class FShaderMapLayout
{
public:
	EShaderPlatform Platform;
	TArray<FShaderLayoutEntry> Shaders;
	TArray<FShaderPipelineType*> ShaderPipelines;
};

class FMeshMaterialShaderMapLayout : public FShaderMapLayout
{
public:
	explicit FMeshMaterialShaderMapLayout(FVertexFactoryType* InVertexFactoryType) : VertexFactoryType(InVertexFactoryType) {}

	FVertexFactoryType* VertexFactoryType;
};

class FMaterialShaderMapLayout : public FShaderMapLayout
{
public:
	TArray<FMeshMaterialShaderMapLayout> MeshShaderMaps;
	FShaderHash ShaderMapHash;
#if WITH_EDITOR
	/** Only available if AllowShaderCompiling(). Includes all of ShaderMapHash, plus hash of the shader source files. */
	FShaderHash ShaderCompileHash;
#endif
};

const FMaterialShaderMapLayout& AcquireMaterialShaderMapLayout(EShaderPlatform Platform, EShaderPermutationFlags Flags, const FMaterialShaderParameters& MaterialParameters);
