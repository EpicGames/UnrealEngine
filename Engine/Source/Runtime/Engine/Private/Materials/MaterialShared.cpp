// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialShared.cpp: Shared material implementation.
=============================================================================*/

#include "MaterialShared.h"
#include "MaterialSharedPrivate.h"
#include "Misc/DelayedAutoRegister.h"
#include "Stats/StatsMisc.h"
#include "Stats/StatsTrace.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Materials/Material.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialRenderProxy.h"
#include "Materials/MaterialShaderMapLayout.h"
#include "ComponentReregisterContext.h"
#include "MaterialDomain.h"
#include "Materials/MaterialExpressionBreakMaterialAttributes.h"
#include "Materials/MaterialExpressionRerouteBase.h"
#include "ShaderCompiler.h"
#include "ShaderSerialization.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "RendererInterface.h"
#include "Materials/HLSLMaterialTranslator.h"
#include "ComponentRecreateRenderStateContext.h"
#include "EngineModule.h"
#include "Engine/Texture2D.h"
#include "Engine/Font.h"
#include "SceneView.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "PSOPrecacheMaterial.h"
#include "ShaderPlatformQualitySettings.h"
#include "MaterialShaderQualitySettings.h"
#include "Engine/RendererSettings.h"
#include "ShaderCodeLibrary.h"
#include "HAL/FileManager.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "UObject/StrongObjectPtr.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/ConfigCacheIni.h"
#include "MaterialCachedData.h"
#include "VT/RuntimeVirtualTexture.h"
#if WITH_EDITOR
#include "MaterialKeyGeneratorContext.h"
#include "Misc/AsciiSet.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Shader/PreshaderEvaluate.h"
#include "UObject/ICookInfo.h"
#endif
#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#endif
#include "ProfilingDebugging/CountersTrace.h"
#include "RenderCore.h"
#include "SubstrateDefinitions.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ProfilingDebugging/CookStats.h"
#include "Engine/NeuralProfile.h"

#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRModuleBuilder.h"
#include "Materials/MaterialIRToHLSLTranslator.h"
#include "Materials/MaterialSourceTemplate.h"
#include "Materials/MaterialInsights.h"
#include "Materials/MaterialExpressionCustom.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialShared)

#define LOCTEXT_NAMESPACE "MaterialShared"

DEFINE_LOG_CATEGORY(LogMaterial);

ENGINE_API extern bool bEnableRestrictiveMaterialMode;

#if ENABLE_COOK_STATS
namespace MaterialSharedCookStats
{
	static double FinishCacheShadersSec = 0.0;

	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			AddStat(TEXT("Material"), FCookStatsManager::CreateKeyValueArray(
				TEXT("FinishCacheShadersSec"), FinishCacheShadersSec
			));
		});
}
#endif

#if WITH_EDITOR

static TAutoConsoleVariable<bool> CVarMaterialEdPreshaderDumpToHLSL(
	TEXT("r.MaterialEditor.PreshaderDumpToHLSL"),
	true,
	TEXT("Controls whether to append preshader expressions and parameter reference counts to the HLSL source window (as comments at the end of the code)."),
	ECVF_RenderThreadSafe);

#endif

static TAutoConsoleVariable<bool> CVarUsingUseNewMaterialTranslatorPrototype(
	TEXT("r.Material.Translator.EnableNew"),
	false,
	TEXT("Controls whether to enable the new material translator prototype (WIP) ."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaterialUseShaderCompilationParameters(
	TEXT("r.Material.UseShaderCompilationParameters"),
	1,
	TEXT("Controls whether the parameters used for determining which shaders to compile are based on translation output or conservative material data."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

bool IsUsingNewMaterialTranslatorPrototype()
{
	return CVarUsingUseNewMaterialTranslatorPrototype.GetValueOnAnyThread();
}

IMPLEMENT_TYPE_LAYOUT(FHashedMaterialParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FUniformExpressionSet);

void FUniformExpressionSet::CountTextureCollections(uint32& BindlessCollections, uint32& VirtualCollections) const
{
	VirtualCollections = 0;
	
	for (const FMaterialTextureCollectionParameterInfo& UniformTextureCollectionParameter : UniformTextureCollectionParameters)
	{
		VirtualCollections += UniformTextureCollectionParameter.bIsVirtualCollection;
	}

	BindlessCollections = UniformTextureCollectionParameters.Num() - VirtualCollections;
}

IMPLEMENT_TYPE_LAYOUT(FMaterialCompilationOutput);
IMPLEMENT_TYPE_LAYOUT(FMeshMaterialShaderMap);
IMPLEMENT_TYPE_LAYOUT(FMaterialProcessedSource);
IMPLEMENT_TYPE_LAYOUT(FMaterialShaderMapContent);
IMPLEMENT_TYPE_LAYOUT(FMaterialUniformParameterEvaluation);
IMPLEMENT_TYPE_LAYOUT(FMaterialUniformPreshaderHeader);
IMPLEMENT_TYPE_LAYOUT(FMaterialNumericParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FMaterialTextureParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FMaterialTextureCollectionParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FMaterialExternalTextureParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FMaterialVirtualTextureStack);
IMPLEMENT_TYPE_LAYOUT(FMaterialCacheTagStack);
IMPLEMENT_TYPE_LAYOUT(FCompactionMetaDataSegment);
IMPLEMENT_TYPE_LAYOUT(FCompactUniformExpressionSet);

struct FAllowCachingStaticParameterValues
{
	FAllowCachingStaticParameterValues(FMaterial& InMaterial)
#if WITH_EDITOR
		: Material(InMaterial)
#endif // WITH_EDITOR
	{
#if WITH_EDITOR
		Material.BeginAllowCachingStaticParameterValues();
#endif // WITH_EDITOR
	};

#if WITH_EDITOR
	~FAllowCachingStaticParameterValues()
	{
		Material.EndAllowCachingStaticParameterValues();
	}

private:
	FMaterial& Material;
#endif // WITH_EDITOR
};

static FAutoConsoleCommand GFlushMaterialUniforms(
	TEXT("r.FlushMaterialUniforms"),
	TEXT(""),
	FConsoleCommandDelegate::CreateStatic(
		[]()
{
	for (TObjectIterator<UMaterialInterface> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UMaterialInterface* Material = *It;
		FMaterialRenderProxy* MaterialProxy = Material->GetRenderProxy();
		if (MaterialProxy)
		{
			MaterialProxy->CacheUniformExpressions_GameThread(false);
		}
	}
})
);

#if WITH_EDITOR
class FMaterialDumpDebugInfoExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec_Editor(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("material dumpdebuginfo")))
		{
			FString RequestedMaterialName(FParse::Token(Cmd, 0));

			if (RequestedMaterialName.Len() > 0)
			{
				for (TObjectIterator<UMaterialInterface> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
				{
					UMaterialInterface* Material = *It;
					if (Material && Material->GetName() == RequestedMaterialName)
					{
						Material->DumpDebugInfo(Ar);
						break;
					}
				}
				return true;
			}
		}
		return false;
	}
};
static FMaterialDumpDebugInfoExecHelper GMaterialDumpDebugInfoExecHelper;
#endif

// defined in the same module (Material.cpp)
bool PoolSpecialMaterialsCompileJobs();

// UE_DEPRECATED(5.7, "Please use AllowDitheredLODTransition with a EShaderPlatform argument")
bool AllowDitheredLODTransition(ERHIFeatureLevel::Type FeatureLevel)
{
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	return AllowDitheredLODTransition(ShaderPlatform);
}

bool AllowDitheredLODTransition(EShaderPlatform ShaderPlatform)
{
	// On mobile support for 'Dithered LOD Transition' has to be explicitly enabled in projects settings
	if (IsMobilePlatform(ShaderPlatform) && !FReadOnlyCVARCache::MobileAllowDitheredLODTransition(ShaderPlatform))
	{
		return false;
	}
	return true;
}

FName MaterialQualityLevelNames[] = 
{
	FName("Low"),
	FName("High"),
	FName("Medium"),
	FName("Epic"),
	FName("Num")
};

static_assert(UE_ARRAY_COUNT(MaterialQualityLevelNames) == EMaterialQualityLevel::Num + 1, "Missing entry from material quality level names.");

void GetMaterialQualityLevelName(EMaterialQualityLevel::Type InQualityLevel, FString& OutName)
{
	check(InQualityLevel < UE_ARRAY_COUNT(MaterialQualityLevelNames));
	MaterialQualityLevelNames[(int32)InQualityLevel].ToString(OutName);
}

FName GetMaterialQualityLevelFName(EMaterialQualityLevel::Type InQualityLevel)
{
	check(InQualityLevel < UE_ARRAY_COUNT(MaterialQualityLevelNames));
	return MaterialQualityLevelNames[(int32)InQualityLevel];
}

#if WITH_EDITOR

/**
* What shader format should we explicitly cook for?
* @returns shader format name or NAME_None if the switch was not specified.
*
* @note: -CacheShaderFormat=
*/
FName GetCmdLineShaderFormatToCache()
{
	FString ShaderFormat;
	FParse::Value(FCommandLine::Get(), TEXT("-CacheShaderFormat="), ShaderFormat);
	return ShaderFormat.Len() ? FName(ShaderFormat) : NAME_None;
}

void GetCmdLineFilterShaderFormats(TArray<FName>& InOutShderFormats)
{
	// if we specified -CacheShaderFormat= on the cmd line we should only cook that format.
	static const FName CommandLineShaderFormat = GetCmdLineShaderFormatToCache();
	if (CommandLineShaderFormat != NAME_None)
	{
		// the format is only valid if it is a desired format for this platform.
		if (InOutShderFormats.Contains(CommandLineShaderFormat))
		{
			// only cache the format specified on the command line.
			InOutShderFormats.Reset(1);
			InOutShderFormats.Add(CommandLineShaderFormat);
		}
	}
}

int32 GetCmdLineMaterialQualityToCache()
{
	int32 MaterialQuality = INDEX_NONE;
	FParse::Value(FCommandLine::Get(), TEXT("-CacheMaterialQuality="), MaterialQuality);
	return MaterialQuality;
}
#endif


#if WITH_EDITOR
static FDelayedAutoRegisterHelper GBlockedMaterialDebugDelegateRegister(EDelayedRegisterRunPhase::StartOfEnginePreInit, []
	{
		UE::Cook::FDelegates::PackageBlocked.AddLambda([](const UObject* Obj, FStringBuilderBase& OutDebugInfo)
			{
				const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Obj);
				if (MaterialInterface)
				{
					MaterialInterface->AppendCompileStateDebugInfo(OutDebugInfo);
				}
			});
	});
#endif

int32 FMaterialCompiler::Errorf(const TCHAR* Format,...)
{
	TCHAR	ErrorText[2048];
	GET_TYPED_VARARGS( TCHAR, ErrorText, UE_ARRAY_COUNT(ErrorText), UE_ARRAY_COUNT(ErrorText)-1, Format, Format );
	return Error(ErrorText);
}

int32 FMaterialCompiler::ScalarParameter(FName ParameterName, float DefaultValue)
{
	return NumericParameter(EMaterialParameterType::Scalar, ParameterName, DefaultValue);
}

int32 FMaterialCompiler::VectorParameter(FName ParameterName, const FLinearColor& DefaultValue)
{
	return NumericParameter(EMaterialParameterType::Vector, ParameterName, DefaultValue);
}

UE_IMPLEMENT_STRUCT("/Script/Engine", ExpressionInput);
UE_IMPLEMENT_STRUCT("/Script/Engine", ColorMaterialInput);
UE_IMPLEMENT_STRUCT("/Script/Engine", ScalarMaterialInput);
UE_IMPLEMENT_STRUCT("/Script/Engine", VectorMaterialInput);
UE_IMPLEMENT_STRUCT("/Script/Engine", Vector2MaterialInput);
UE_IMPLEMENT_STRUCT("/Script/Engine", MaterialAttributesInput);

#if WITH_EDITOR

struct FConnectionMask
{
	bool Mask;
	bool MaskR;
	bool MaskG;
	bool MaskB;
	bool MaskA;
};

/** Helper function that returns the most restrictive components mask between specified input and its connected output. */
static FConnectionMask GetConnectionMask(const FExpressionInput* Input)
{
	FConnectionMask CM = {
		(bool)Input->Mask,
		(bool)Input->MaskR,
		(bool)Input->MaskG,
		(bool)Input->MaskB,
		(bool)Input->MaskA,
	};


	if (!Input->Mask && Input->Expression->GetOutputs().IsValidIndex(Input->OutputIndex))
	{
		FExpressionOutput& Output = Input->Expression->GetOutputs()[Input->OutputIndex];
		CM.Mask = (bool)Output.Mask;
		CM.MaskR = (bool)Output.MaskR;
		CM.MaskG = (bool)Output.MaskG;
		CM.MaskB = (bool)Output.MaskB;
		CM.MaskA = (bool)Output.MaskA;
	}

	return CM;
}

int32 FExpressionInput::Compile(class FMaterialCompiler* Compiler)
{
	if (!Expression)
	{
		return INDEX_NONE;
	}

	Expression->ValidateState();
	int32 ExpressionResult = Compiler->CallExpression(FMaterialExpressionKey(Expression, OutputIndex, Compiler->GetMaterialAttribute(), Compiler->IsCurrentlyCompilingForPreviousFrame()),Compiler);
	
	// Early out if compiling expression failed
	if (ExpressionResult == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	// Use the most restrictive components mask between this input and connected output.
	// We do this to make sure that an out-of-date expression input (that most likely caches the mask from its
	// connected output) gets the correct mask if the output mask has changed (for instance because now it
	// specifies a mask where it didn't before when the material was saved)
	FConnectionMask CM = GetConnectionMask(this);
	if (CM.Mask)
	{
		ExpressionResult = Compiler->ComponentMask(ExpressionResult, CM.MaskR, CM.MaskG, CM.MaskB, CM.MaskA);
	}

	return ExpressionResult;
}

void FExpressionInput::Connect( int32 InOutputIndex, class UMaterialExpression* InExpression )
{
	InExpression->ConnectExpression(this, InOutputIndex);
}

FExpressionInput FExpressionInput::GetTracedInput() const
{
	if (Expression != nullptr && Expression->IsA(UMaterialExpressionRerouteBase::StaticClass()))
	{
		UMaterialExpressionRerouteBase* Reroute = CastChecked<UMaterialExpressionRerouteBase>(Expression);
		return Reroute->TraceInputsToRealInput();
	}
	return *this;
}

FExpressionOutput* FExpressionInput::GetConnectedOutput() const
{
	if (IsConnected())
	{
		TArray<FExpressionOutput>& ExpressionOutputs = Expression->GetOutputs();
		return ExpressionOutputs.IsValidIndex(OutputIndex) ? &ExpressionOutputs[OutputIndex] : nullptr;
	}
	return nullptr;
}

#endif // WITH_EDITOR

/** Native serialize for FMaterialExpression struct */
static bool SerializeExpressionInput(FArchive& Ar, FExpressionInput& Input)
{
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.CustomVer(FCoreObjectVersion::GUID) < FCoreObjectVersion::MaterialInputNativeSerialize)
	{
		return false;
	}

	Ar << Input.Expression;

	Ar << Input.OutputIndex;
	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PinsStoreFName)
	{
		Ar << Input.InputName;
	}
	else
	{
		FString InputNameStr;
		Ar << InputNameStr;
		Input.InputName = *InputNameStr;
	}

	Ar << Input.Mask;
	Ar << Input.MaskR;
	Ar << Input.MaskG;
	Ar << Input.MaskB;
	Ar << Input.MaskA;

	return true;
}

template <typename InputType>
static bool SerializeMaterialInput(FArchive& Ar, FMaterialInput<InputType>& Input)
{
	if (SerializeExpressionInput(Ar, Input))
	{
		bool bUseConstantValue = Input.UseConstant;
		Ar << bUseConstantValue;
		Input.UseConstant = bUseConstantValue;
		Ar << Input.Constant;
		return true;
	}
	else
	{
		return false;
	}
}

bool FExpressionInput::Serialize(FArchive& Ar)
{
	return SerializeExpressionInput(Ar, *this);
}

bool FColorMaterialInput::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MaterialInputUsesLinearColor)
	{
		FMaterialInput<FColor> OldValue;
		if (SerializeMaterialInput<FColor>(Ar, OldValue))
		{
			this->Expression = OldValue.Expression;
			this->OutputIndex = OldValue.OutputIndex;
			this->InputName = OldValue.InputName;
			this->Mask = OldValue.Mask;
			this->MaskR = OldValue.MaskR;
			this->MaskG = OldValue.MaskG;
			this->MaskB = OldValue.MaskB;
			this->MaskA = OldValue.MaskA;
			this->UseConstant = OldValue.UseConstant;
			this->Constant = OldValue.Constant.ReinterpretAsLinear();
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		return SerializeMaterialInput<FLinearColor>(Ar, *this);	
	}
}

bool FScalarMaterialInput::Serialize(FArchive& Ar)
{
	return SerializeMaterialInput<float>(Ar, *this);
}

bool FShadingModelMaterialInput::Serialize(FArchive& Ar)
{
	return SerializeMaterialInput<uint32>(Ar, *this);
}

bool FSubstrateMaterialInput::Serialize(FArchive& Ar)
{
	return SerializeMaterialInput<uint32>(Ar, *this);
}


bool FVectorMaterialInput::Serialize(FArchive& Ar)
{
	return SerializeMaterialInput<FVector3f>(Ar, *this);
}

bool FVector2MaterialInput::Serialize(FArchive& Ar)
{
	return SerializeMaterialInput<FVector2f>(Ar, *this);
}

bool FMaterialAttributesInput::Serialize(FArchive& Ar)
{
	return SerializeExpressionInput(Ar, *this);
}

void FColorMaterialInput::DefaultValueChanged(const FString& DefaultValue)
{
#if WITH_EDITOR
	Constant.InitFromString(DefaultValue);
	UseConstant = true;
#endif
}

FString FColorMaterialInput::GetDefaultValue() const
{
	FString DefaultValue;
#if WITH_EDITOR
	DefaultValue = Constant.ToString();
#endif
	return DefaultValue;
}

void FScalarMaterialInput::DefaultValueChanged(const FString& DefaultValue)
{
#if WITH_EDITOR
	Constant = FCString::Atof(*DefaultValue);
	UseConstant = true;
#endif
}

FString FScalarMaterialInput::GetDefaultValue() const
{
	FString DefaultValue;
#if WITH_EDITOR
	DefaultValue = FString::SanitizeFloat(Constant);
#endif
	return DefaultValue;
}

void FVector2MaterialInput::DefaultValueChanged(const FString& DefaultValue)
{
#if WITH_EDITOR
	FVector2f Value;
	Value.InitFromString(DefaultValue);
	Constant = Value;
	UseConstant = true;
#endif
}

FString FVector2MaterialInput::GetDefaultValue() const
{
	FString DefaultValue;
#if WITH_EDITOR
	DefaultValue = FString("(X=") + FString::SanitizeFloat(Constant.X) + FString(",Y=") + FString::SanitizeFloat(Constant.Y) + FString(")");
#endif
	return DefaultValue;
}

void FVectorMaterialInput::DefaultValueChanged(const FString& DefaultValue)
{
#if WITH_EDITOR
	//Parse string to split its contents separated by ','
	TArray<FString> Elements;
	DefaultValue.ParseIntoArray(Elements, TEXT(","), true);
	check(Elements.Num() == 3);
	Constant.X = FCString::Atof(*Elements[0]);
	Constant.Y = FCString::Atof(*Elements[1]);
	Constant.Z = FCString::Atof(*Elements[2]);
	UseConstant = true;
#endif
}

FString FVectorMaterialInput::GetDefaultValue() const
{
	FString DefaultValue;
#if WITH_EDITOR
	DefaultValue = FString::SanitizeFloat(Constant.X) + FString(",") + FString::SanitizeFloat(Constant.Y) + FString(",") + FString::SanitizeFloat(Constant.Z);
#endif
	return DefaultValue;
}

#if WITH_EDITOR

// As present in UMaterialGraphNode_Root::UpdateInputUseConstant, which fixes up Materials on modification.
// This ensures Materials which have not been fixed up don't cause issues.
static inline bool IsValidPropertyForInlineConst(const EMaterialProperty InProp)
{
	return InProp != MP_WorldPositionOffset && InProp != MP_Anisotropy && InProp != MP_PixelDepthOffset && InProp != MP_Displacement;
}

int32 FColorMaterialInput::CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	const bool bValidPropertyForInlineConstant = IsValidPropertyForInlineConst(Property);

	if (UseConstant && bValidPropertyForInlineConstant)
	{
		return Compiler->Constant3(Constant.R, Constant.G, Constant.B);
	}
	else if (Expression)
	{
		int32 ResultIndex = FExpressionInput::Compile(Compiler);
		if (ResultIndex != INDEX_NONE)
		{
			return ResultIndex;
		}
	}

	return Compiler->ForceCast(FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property), MCT_Float3);
}



int32 FScalarMaterialInput::CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	const bool bValidPropertyForInlineConstant = IsValidPropertyForInlineConst(Property);

	if (UseConstant && bValidPropertyForInlineConstant)
	{
		return Compiler->Constant(Constant);
	}
	else if (Expression)
	{
		int32 ResultIndex = FExpressionInput::Compile(Compiler);
		if (ResultIndex != INDEX_NONE)
		{
			return ResultIndex;
		}
	}
	
	return Compiler->ForceCast(FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property), MCT_Float1);
}

int32 FShadingModelMaterialInput::CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	if (Expression)
	{
		int32 ResultIndex = FExpressionInput::Compile(Compiler);
		if (ResultIndex != INDEX_NONE)
		{
			return ResultIndex;
		}
	}

	return Compiler->ForceCast(FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property), MCT_ShadingModel, MFCF_ExactMatch);
}

int32 FSubstrateMaterialInput::CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	if (Expression)
	{
		int32 ResultIndex = FExpressionInput::Compile(Compiler);
		if (ResultIndex != INDEX_NONE)
		{
			return ResultIndex;
		}
	}

	return Compiler->ForceCast(FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property), MCT_Substrate);
}


int32 FVectorMaterialInput::CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	const bool bValidPropertyForInlineConstant = IsValidPropertyForInlineConst(Property);

	if (UseConstant && bValidPropertyForInlineConstant)
	{
		return Compiler->Constant3(Constant.X, Constant.Y, Constant.Z);
	}
	else if(Expression)
	{
		int32 ResultIndex = FExpressionInput::Compile(Compiler);
		if (ResultIndex != INDEX_NONE)
		{
			return ResultIndex;
		}
	}
	return Compiler->ForceCast(FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property), MCT_Float3);
}

int32 FVector2MaterialInput::CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property)
{
	const bool bValidPropertyForInlineConstant = IsValidPropertyForInlineConst(Property);

	if (UseConstant && bValidPropertyForInlineConstant)
	{
		return Compiler->Constant2(Constant.X, Constant.Y);
	}
	else if (Expression)
	{
		int32 ResultIndex = FExpressionInput::Compile(Compiler);
		if (ResultIndex != INDEX_NONE)
		{
			return ResultIndex;
		}
	}

	return Compiler->ForceCast(FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property), MCT_Float2);
}

int32 FMaterialAttributesInput::CompileWithDefault(class FMaterialCompiler* Compiler, const FGuid& AttributeID)
{
	int32 Ret = INDEX_NONE;
	if(Expression)
	{
		FScopedMaterialCompilerAttribute ScopedMaterialCompilerAttribute(Compiler, AttributeID);
		Ret = FExpressionInput::Compile(Compiler);

		if (Ret != INDEX_NONE && !Expression->IsResultMaterialAttributes(OutputIndex))
		{
			Compiler->Error(TEXT("Cannot connect a non MaterialAttributes node to a MaterialAttributes pin."));
		}
	}

	EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(AttributeID);
	SetConnectedProperty(Property, Ret != INDEX_NONE);

	if( Ret == INDEX_NONE )
	{
		Ret = FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, AttributeID);
	}

	return Ret;
}
#endif  // WITH_EDITOR

void FMaterial::BuildShaderMapIdOverride(const FBuildShaderMapIdArgs& Args) const
{ 
	FMaterialShaderMapId& OutId = *Args.OutId;
	if (bLoadedCookedShaderMapId)
	{
		if (GameThreadShaderMap && (IsInGameThread() || IsInAsyncLoadingThread()))
		{
			OutId = GameThreadShaderMap->GetShaderMapId();
		}
		else if (RenderingThreadShaderMap && IsInParallelRenderingThread())
		{
			OutId = RenderingThreadShaderMap->GetShaderMapId();
		}
		else
		{
			UE_LOGF(LogMaterial, Fatal, "Tried to access cooked shader map ID from unknown thread");
		}
	}
	else
	{
#if WITH_EDITOR
		OutId.LayoutParams.InitializeForPlatform(Args.TargetPlatform);

		TArray<FShaderType*> ShaderTypes;
		TArray<FVertexFactoryType*> VFTypes;
		TArray<const FShaderPipelineType*> ShaderPipelineTypes;

		if (EnumHasAnyFlags(Args.IncludeFlags, EMaterialKeyInclude::Globals))
		{
			GetDependentShaderAndVFTypes(OutId.LayoutParams, ShaderTypes, ShaderPipelineTypes, VFTypes);
		}

		OutId.BaseMaterialPathHash = FXxHash64::HashBuffer(Args.BaseMaterialPath.GetCharArray().GetData(), Args.BaseMaterialPath.NumBytesWithoutNull()).Hash;
		OutId.Usage = GetShaderMapUsage();
		OutId.bUsingNewHLSLGenerator = IsUsingNewHLSLGenerator();
		OutId.PreferredShaderCodeFormat = FDataDrivenShaderPlatformInfo::GetPreferredShaderCodeFormat(Args.Platform);
		OutId.BaseMaterialId = GetMaterialId();
		OutId.QualityLevel = GetQualityLevel();
		OutId.FeatureLevel = GetFeatureLevel();
		OutId.SetShaderDependencies(ShaderTypes, ShaderPipelineTypes, VFTypes, Args.Platform);
		OutId.TextureReferencesHash = GetReferencedTexturesHash();
		OutId.ExpressionIncludesHash = GetExpressionIncludesHash();
		OutId.ExternalCodeReferencesHash = GetExternalCodeReferencesHash();

		OutId.SubstrateCompilationConfig = GetSubstrateCompilationConfig();

#else
		OutId.QualityLevel = GetQualityLevel();
		OutId.FeatureLevel = GetFeatureLevel();

		if (Args.TargetPlatform != nullptr)
		{
			UE_LOGF(LogMaterial, Error, "FMaterial::GetShaderMapId: TargetPlatform is not null, but a cooked executable cannot target platforms other than its own.");
		}
		OutId.LayoutParams.InitializeForCurrent();

		UE_LOGF(LogMaterial, Error, "Tried to access an uncooked shader map ID in a cooked application");
#endif
	}
}

#if WITH_EDITORONLY_DATA
// Deprecated 5.7
void FMaterial::GetStaticParameterSet(EShaderPlatform Platform, FStaticParameterSet& OutSet) const
{
	GetStaticParameterSet(OutSet);
}

void FMaterial::GetStaticParameterSet(FStaticParameterSet& OutSet) const
{
	// Clear the set in default implementation
	OutSet = FStaticParameterSet();
}
#endif // WITH_EDITORONLY_DATA

ERefractionMode FMaterial::GetRefractionMode() const 
{ 
	return RM_None;
}

const FMaterialCachedExpressionData& FMaterial::GetCachedExpressionData() const
{
	const UMaterialInterface* MaterialInterface = GetMaterialInterface();
	return MaterialInterface ? MaterialInterface->GetCachedExpressionData() : FMaterialCachedExpressionData::EmptyData;
}

bool FMaterial::IsRequiredComplete() const
{ 
	return IsDefaultMaterial() || IsSpecialEngineMaterial();
}

#if WITH_EDITOR
void FMaterial::AddShaderMapIDsWithUnfinishedCompilation(TArray<int32>& ShaderMapIds)
{
	if (GameThreadCompilingShaderMapId != 0u && GShaderCompilingManager->IsCompilingShaderMap(GameThreadCompilingShaderMapId))
	{
		ShaderMapIds.Add(GameThreadCompilingShaderMapId);
	}
}

bool FMaterial::IsCompilationFinished() const
{
	if (CacheShadersPending.IsValid() && !CacheShadersPending->IsReady())
	{
		return false;
	}

	FinishCacheShaders();

	if (GameThreadCompilingShaderMapId != 0u)
	{
		return !GShaderCompilingManager->IsCompilingShaderMap(GameThreadCompilingShaderMapId);
	}
	return true;
}

void FMaterial::CancelCompilation()
{
	if (CacheShadersPending.IsValid())
	{
		CacheShadersPending.Reset();
	}

	if (CacheShadersCompletion)
	{
		CacheShadersCompletion.Reset();
	}

	TArray<int32> ShaderMapIdsToCancel;
	AddShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToCancel);

	if (ShaderMapIdsToCancel.Num() > 0)
	{
		// Cancel all compile jobs for these shader maps.
		GShaderCompilingManager->CancelCompilation(*GetFriendlyName(), ShaderMapIdsToCancel);
	}
}

void FMaterial::FinishCompilation()
{
	FinishCacheShaders();

	TArray<int32> ShaderMapIdsToFinish;
	AddShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);

	if (ShaderMapIdsToFinish.Num() > 0)
	{
		// Block until the shader maps that we will save have finished being compiled
		GShaderCompilingManager->FinishCompilation(*GetFriendlyName(), ShaderMapIdsToFinish);
	}
}

void FMaterial::FinishCompilation(const TCHAR* MaterialName, const TArray<FMaterial*>& MaterialsToCompile)
{
	for(const FMaterial* Material: MaterialsToCompile)
	{
		Material->FinishCacheShaders();
	}

	TArray<int32> ShaderMapIdsToFinish;
	for(FMaterial* Material: MaterialsToCompile)
	{
		Material->AddShaderMapIDsWithUnfinishedCompilation(ShaderMapIdsToFinish);
	}

	if (ShaderMapIdsToFinish.Num() > 0)
	{
		// Block until the shader maps that we will save have finished being compiled
		GShaderCompilingManager->FinishCompilation(MaterialName, ShaderMapIdsToFinish);
	}
}

bool FMaterial::IsUsingNewHLSLGenerator() const
{
	const UMaterialInterface* MaterialInterface = GetMaterialInterface();
	return MaterialInterface ? MaterialInterface->IsUsingNewHLSLGenerator() : false;
}

const FSubstrateCompilationConfig& FMaterial::GetSubstrateCompilationConfig() const
{
	const UMaterialInterface* MaterialInterface = GetMaterialInterface();
	static FSubstrateCompilationConfig DefaultFSubstrateCompilationConfig = FSubstrateCompilationConfig();
	return MaterialInterface ? MaterialInterface->GetSubstrateCompilationConfig() : DefaultFSubstrateCompilationConfig;
}

void FMaterial::SetSubstrateCompilationConfig(FSubstrateCompilationConfig& SubstrateCompilationConfig)
{
	UMaterialInterface* MaterialInterface = GetMaterialInterface();
	if (MaterialInterface)
	{
		MaterialInterface->SetSubstrateCompilationConfig(SubstrateCompilationConfig);
	}
}

#endif // WITH_EDITOR

bool FMaterial::HasValidGameThreadShaderMap() const
{
	if(!GameThreadShaderMap || !GameThreadShaderMap->IsCompilationFinalized())
	{
		return false;
	}
	return true;
}

const FMaterialShaderMap* FMaterial::GetShaderMapToUse() const 
{ 
	const FMaterialShaderMap* ShaderMapToUse = NULL;

	if (!IsInAnyRenderingThread())
	{
		// If we are accessing uniform texture expressions on the game thread, use results from a shader map whose compile is in flight that matches this material
		// This allows querying what textures a material uses even when it is being asynchronously compiled
		ShaderMapToUse = GameThreadShaderMap;

#if WITH_EDITOR
		if (!ShaderMapToUse && GameThreadCompilingShaderMapId != 0u)
		{
			ShaderMapToUse = FMaterialShaderMap::FindCompilingShaderMap(GameThreadCompilingShaderMapId);
		}
#endif // WITH_EDITOR

		checkf(!ShaderMapToUse || ShaderMapToUse->GetNumRefs() > 0, TEXT("NumRefs %i, GameThreadShaderMap 0x%08" PTRINT_x_FMT), ShaderMapToUse->GetNumRefs(), GetGameThreadShaderMap());
	}
	else 
	{
		ShaderMapToUse = GetRenderingThreadShaderMap();
	}

	return ShaderMapToUse;
}

const FUniformExpressionSet& FMaterial::GetUniformExpressions() const
{ 
	const FMaterialShaderMap* ShaderMapToUse = GetShaderMapToUse();
	if (ShaderMapToUse)
	{
		return ShaderMapToUse->GetUniformExpressionSet();
	}

	static const FUniformExpressionSet EmptyExpressions;
	return EmptyExpressions;
}

TArrayView<const FMaterialTextureParameterInfo> FMaterial::GetUniformTextureExpressions(EMaterialTextureParameterType Type) const
{
	return GetUniformExpressions().UniformTextureParameters[(uint32)Type];
}

TConstArrayView<FMaterialTextureCollectionParameterInfo> FMaterial::GetUniformTextureCollectionExpressions() const
{
	return GetUniformExpressions().UniformTextureCollectionParameters;
}

TArrayView<const FMaterialNumericParameterInfo> FMaterial::GetUniformNumericParameterExpressions() const
{ 
	return GetUniformExpressions().UniformNumericParameters;
}

bool FMaterial::RequiresSceneColorCopy_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->RequiresSceneColorCopy() : false; 
}

bool FMaterial::RequiresSceneColorCopy_RenderThread() const
{
	check(IsInParallelRenderingThread());
	if (RenderingThreadShaderMap)
	{
		return RenderingThreadShaderMap->RequiresSceneColorCopy();
	}
	return false;
}

bool FMaterial::NeedsSceneTextures() const 
{
	check(IsInParallelRenderingThread());

	if (RenderingThreadShaderMap)
	{
		return RenderingThreadShaderMap->NeedsSceneTextures();
	}
	
	return false;
}

bool FMaterial::NeedsGBuffer() const
{
	check(IsInParallelRenderingThread());

	if ((IsOpenGLPlatform(GMaxRHIShaderPlatform) || FDataDrivenShaderPlatformInfo::GetOverrideFMaterial_NeedsGBufferEnabled(GMaxRHIShaderPlatform)) // @todo: TTP #341211 
		&& !IsMobilePlatform(GMaxRHIShaderPlatform)) 
	{
		return true;
	}

	if (RenderingThreadShaderMap)
	{
		return RenderingThreadShaderMap->NeedsGBuffer();
	}

	return false;
}


bool FMaterial::UsesEyeAdaptation() const 
{
	check(IsInParallelRenderingThread());

	if (RenderingThreadShaderMap)
	{
		return RenderingThreadShaderMap->UsesEyeAdaptation();
	}

	return false;
}

bool FMaterial::UsesGlobalDistanceField_GameThread() const 
{ 
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesGlobalDistanceField() : false; 
}

bool FMaterial::MaterialUsesWorldPositionOffset_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesWorldPositionOffset() : false;
}

bool FMaterial::MaterialUsesWorldPositionOffset_GameThread() const
{ 
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesWorldPositionOffset() : false; 
}

bool FMaterial::MaterialUsesDisplacement_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesDisplacement() : false;
}

bool FMaterial::MaterialUsesDisplacement_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesDisplacement() : false;
}

bool FMaterial::MaterialModifiesMeshPosition_RenderThread() const
{ 
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->ModifiesMeshPosition() : false;
}

bool FMaterial::MaterialModifiesMeshPosition_GameThread() const
{
	FMaterialShaderMap* ShaderMap = GameThreadShaderMap.GetReference();
	return ShaderMap ? ShaderMap->ModifiesMeshPosition() : false;
}

bool FMaterial::MaterialMayModifyMeshPosition() const
{
	// Conservative estimate based on expressions present in graph.
	// This function is only intended to be used for DDC Key/Iterative Cooking
	return HasVertexPositionOffsetConnected() || HasPixelDepthOffsetConnected() || HasDisplacementConnected() || HasFirstPersonOutput();
}

bool FMaterial::MaterialUsesPixelDepthOffset_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesPixelDepthOffset() : false;
}

bool FMaterial::MaterialUsesPixelDepthOffset_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesPixelDepthOffset() : false;
}

bool FMaterial::MaterialUsesTemporalResponsiveness_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesTemporalResponsiveness() : false;
}

bool FMaterial::MaterialUsesTemporalResponsiveness_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesTemporalResponsiveness() : false;
}

bool FMaterial::MaterialUsesMotionVectorWorldOffset_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesMotionVectorWorldOffset() : false;
}

bool FMaterial::MaterialUsesMotionVectorWorldOffset_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesMotionVectorWorldOffset() : false;
}

bool FMaterial::MaterialUsesDistanceCullFade_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->UsesDistanceCullFade() : false;
}

bool FMaterial::MaterialUsesSceneDepthLookup_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesSceneDepthLookup() : false;
}

bool FMaterial::MaterialUsesSceneDepthLookup_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->UsesSceneDepthLookup() : false;
}

uint8 FMaterial::GetCustomDepthStencilUsageMask_GameThread() const
{
	uint8 CustomDepthStencilUsageMask = 0;
	if (GameThreadShaderMap.GetReference())
	{
		CustomDepthStencilUsageMask |= GameThreadShaderMap->UsesSceneTexture(PPI_CustomDepth) ? 1 : 0;
		CustomDepthStencilUsageMask |= GameThreadShaderMap->UsesSceneTexture(PPI_CustomStencil) ? 1 << 1 : 0;
	}
	return CustomDepthStencilUsageMask;
}

uint8 FMaterial::GetRuntimeVirtualTextureOutputAttibuteMask_GameThread() const
{
	return GameThreadShaderMap.GetReference() ? GameThreadShaderMap->GetRuntimeVirtualTextureOutputAttributeMask() : 0;
}

uint8 FMaterial::GetRuntimeVirtualTextureOutputAttibuteMask_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->GetRuntimeVirtualTextureOutputAttributeMask() : 0;
}

bool FMaterial::MaterialUsesAnisotropy_GameThread() const
{
	return GameThreadShaderMap ? (GameThreadShaderMap->UsesAnisotropy() || EnumHasAnyFlags(GameThreadShaderMap->GetSubstrateMaterialBsdfFeatures(), ESubstrateBsdfFeature::Anisotropy)) : false;
}

bool FMaterial::MaterialUsesAnisotropy_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? (RenderingThreadShaderMap->UsesAnisotropy() || EnumHasAnyFlags(RenderingThreadShaderMap->GetSubstrateMaterialBsdfFeatures(), ESubstrateBsdfFeature::Anisotropy)) : false;
}

bool FMaterial::MaterialUsesVertexInterpolator_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->UsesVertexInterpolator() : false;
}

bool FMaterial::MaterialUsesVertexInterpolator_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->UsesVertexInterpolator() : false;
}

bool FMaterial::MaterialIsLightFunctionAtlasCompatible_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->IsLightFunctionAtlasCompatible() : false;
}

bool FMaterial::MaterialIsLightFunctionAtlasCompatible_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->IsLightFunctionAtlasCompatible() : false;
}

uint8 FMaterial::MaterialGetSubstrateMaterialType_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->GetSubstrateMaterialType() : false;
}

uint8 FMaterial::MaterialGetSubstrateMaterialType_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->GetSubstrateMaterialType() : false;
}

uint8 FMaterial::MaterialGetSubstrateClosureCount_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->GetSubstrateClosureCount() : false;
}

uint8 FMaterial::MaterialGetSubstrateClosureCount_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->GetSubstrateClosureCount() : false;
}

uint8 FMaterial::MaterialGetSubstrateUintPerPixel_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->GetSubstrateUintPerPixel() : false;
}

uint8 FMaterial::MaterialGetSubstrateUintPerPixel_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->GetSubstrateUintPerPixel() : false;
}

ESubstrateTileType FMaterial::MaterialGetSubstrateTileType_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->GetSubstrateTileType() : ESubstrateTileType::ECount;
}

ESubstrateTileType FMaterial::MaterialGetSubstrateTileType_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->GetSubstrateTileType() : ESubstrateTileType::ECount;
}

ESubstrateBsdfFeature FMaterial::MaterialGetSubstrateMaterialBsdfFeatures_GameThread() const
{
	return GameThreadShaderMap ? GameThreadShaderMap->GetSubstrateMaterialBsdfFeatures() : ESubstrateBsdfFeature::None;
}

ESubstrateBsdfFeature FMaterial::MaterialGetSubstrateMaterialBsdfFeatures_RenderThread() const
{
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap ? RenderingThreadShaderMap->GetSubstrateMaterialBsdfFeatures() : ESubstrateBsdfFeature::None;
}

void FMaterial::SetGameThreadShaderMap(FMaterialShaderMap* InMaterialShaderMap)
{
	checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());

	bool bAssumeShaderMapIsComplete = false;
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAssumeShaderMapIsComplete = FPlatformProperties::RequiresCookedData();
#endif
	const bool bIsComplete = bAssumeShaderMapIsComplete || (InMaterialShaderMap ? InMaterialShaderMap->IsComplete(this, true) : false);
	GameThreadShaderMap = InMaterialShaderMap;
	if (LIKELY(GameThreadShaderMap))
	{
		GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
	}
	bGameThreadShaderMapIsComplete.store(bIsComplete, std::memory_order_relaxed);

	TRefCountPtr<FMaterial> Material = this;
	TRefCountPtr<FMaterialShaderMap> ShaderMap = InMaterialShaderMap;
	ENQUEUE_RENDER_COMMAND(SetGameThreadShaderMap)([Material = MoveTemp(Material), ShaderMap = MoveTemp(ShaderMap), bIsComplete](FRHICommandListImmediate& RHICmdList) mutable
	{
		Material->RenderingThreadShaderMap = MoveTemp(ShaderMap);
		Material->bRenderingThreadShaderMapIsComplete = bIsComplete;
	});
}

void FMaterial::UpdateInlineShaderMapIsComplete()
{
	checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
	check(bContainsInlineShaders);
	// We expect inline shader maps to be complete, so we want to log missing shaders here
	const bool bSilent = false;

	bool bAssumeShaderMapIsComplete = false;
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	bAssumeShaderMapIsComplete = FPlatformProperties::RequiresCookedData();
#endif
	const bool bIsComplete = bAssumeShaderMapIsComplete || GameThreadShaderMap->IsComplete(this, bSilent);

	bGameThreadShaderMapIsComplete.store(bIsComplete, std::memory_order_relaxed);
	TRefCountPtr<FMaterial> Material = this;
	ENQUEUE_RENDER_COMMAND(UpdateGameThreadShaderMapIsComplete)([Material = MoveTemp(Material), bIsComplete](FRHICommandListImmediate& RHICmdList) mutable
	{
		Material->bRenderingThreadShaderMapIsComplete.store(bIsComplete, std::memory_order_relaxed);
	});
}

void FMaterial::SetInlineShaderMap(FMaterialShaderMap* InMaterialShaderMap)
{
	checkSlow(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
	check(InMaterialShaderMap);

	GameThreadShaderMap = InMaterialShaderMap;
	GameThreadShaderMap->GetResource()->SetOwnerName(GetOwnerFName());
	bContainsInlineShaders = true;
	bLoadedCookedShaderMapId = true;

	// SetInlineShaderMap is called during PostLoad(), before given UMaterial(Instance) is fully initialized
	// Can't check for completeness yet
	bGameThreadShaderMapIsComplete.store(false, std::memory_order_relaxed);
	GameThreadShaderMapSubmittedPriority = EShaderCompileJobPriority::None;

	TRefCountPtr<FMaterial> Material = this;
	TRefCountPtr<FMaterialShaderMap> ShaderMap = InMaterialShaderMap;
	ENQUEUE_RENDER_COMMAND(SetInlineShaderMap)([Material = MoveTemp(Material), ShaderMap = MoveTemp(ShaderMap)](FRHICommandListImmediate& RHICmdList) mutable
	{
		Material->RenderingThreadShaderMap = MoveTemp(ShaderMap);
		Material->bRenderingThreadShaderMapIsComplete = false;
		Material->RenderingThreadShaderMapSubmittedPriority = -1;
	});
}

#if WITH_EDITOR
void FMaterial::SetCompilingShaderMap(FMaterialShaderMap* InMaterialShaderMap)
{
	checkSlow(IsInGameThread());
	const uint32 CompilingShaderMapId = InMaterialShaderMap->GetCompilingId();
	if (CompilingShaderMapId != GameThreadCompilingShaderMapId)
	{
		ReleaseGameThreadCompilingShaderMap();

		GameThreadCompilingShaderMapId = CompilingShaderMapId;
		check(GameThreadCompilingShaderMapId != 0u);
		InMaterialShaderMap->AddCompilingDependency(this);

		GameThreadPendingCompilerEnvironment = InMaterialShaderMap->GetPendingCompilerEnvironment();
		GameThreadShaderMapSubmittedPriority = EShaderCompileJobPriority::None;

		TRefCountPtr<FMaterial> Material = this;
		TRefCountPtr<FSharedShaderCompilerEnvironment> PendingCompilerEnvironment = InMaterialShaderMap->GetPendingCompilerEnvironment();
		ENQUEUE_RENDER_COMMAND(SetCompilingShaderMap)([Material = MoveTemp(Material), CompilingShaderMapId, PendingCompilerEnvironment = MoveTemp(PendingCompilerEnvironment)](FRHICommandListImmediate& RHICmdList) mutable
		{
			Material->RenderingThreadCompilingShaderMapId = CompilingShaderMapId;
			Material->RenderingThreadPendingCompilerEnvironment = MoveTemp(PendingCompilerEnvironment);
			Material->RenderingThreadShaderMapSubmittedPriority = -1;
		});
	}
}

bool FMaterial::ReleaseGameThreadCompilingShaderMap()
{
	bool bReleased = false;
	if (GameThreadCompilingShaderMapId != 0u)
	{
		FMaterialShaderMap* PrevShaderMap = FMaterialShaderMap::FindCompilingShaderMap(GameThreadCompilingShaderMapId);
		if (PrevShaderMap)
		{
			PrevShaderMap->RemoveCompilingDependency(this);
		}
		GameThreadCompilingShaderMapId = 0u;
		bReleased = true;
	}
	return bReleased;
}
#endif // WITH_EDITOR

void FMaterial::ReleaseRenderThreadCompilingShaderMap()
{
	checkSlow(IsInGameThread());

	TRefCountPtr<FMaterial> Material = this;
	ENQUEUE_RENDER_COMMAND(DeferredDestroyMaterial)([Material = MoveTemp(Material)](FRHICommandListImmediate& RHICmdList) mutable
	{
		Material->PrepareDestroy_RenderThread();
	});
}

FMaterialShaderMap* FMaterial::GetRenderingThreadShaderMap() const 
{ 
	check(IsInParallelRenderingThread());
	return RenderingThreadShaderMap; 
}

void FMaterial::SetRenderingThreadShaderMap(TRefCountPtr<FMaterialShaderMap>& InMaterialShaderMap)
{
	check(IsInRenderingThread());
	RenderingThreadShaderMap = MoveTemp(InMaterialShaderMap);
	bRenderingThreadShaderMapIsComplete = RenderingThreadShaderMap ? RenderingThreadShaderMap->IsComplete(this, true) : false;
	// if SM isn't complete, it is perhaps a partial update incorporating results from the already submitted compile jobs.
	// Only reset the priority if the SM is complete, as otherwise we risk resubmitting the same jobs over and over again 
	// as FMaterialRenderProxy::GetMaterialWithFallback will queue job submissions any time it sees an incomplete SM.
	if (bRenderingThreadShaderMapIsComplete)
	{
		RenderingThreadShaderMapSubmittedPriority = -1;
	}
}

void FMaterial::AddReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITOR
	Collector.AddStableReferenceArray(&ErrorExpressions);
#endif
}

struct FLegacyTextureLookup
{
	void Serialize(FArchive& Ar)
	{
		Ar << TexCoordIndex;
		Ar << TextureIndex;
		Ar << UScale;
		Ar << VScale;
	}

	int32 TexCoordIndex;
	int32 TextureIndex;	

	float UScale;
	float VScale;
};

FArchive& operator<<(FArchive& Ar, FLegacyTextureLookup& Ref)
{
	Ref.Serialize( Ar );
	return Ar;
}

void FMaterial::LegacySerialize(FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS)
	{
		TArray<FString> LegacyStrings;
		Ar << LegacyStrings;

		TMap<UMaterialExpression*,int32> LegacyMap;
		Ar << LegacyMap;
		int32 LegacyInt;
		Ar << LegacyInt;

		FeatureLevel = ERHIFeatureLevel::SM4_REMOVED;
		QualityLevel = EMaterialQualityLevel::High;

#if !WITH_EDITOR
		FGuid Id_DEPRECATED;
		UE_LOGF(LogMaterial, Error, "Attempted to serialize legacy material data at runtime, this content should be re-saved and re-cooked");
#endif	
		Ar << Id_DEPRECATED;

		TArray<UTexture*> LegacyTextures;
		Ar << LegacyTextures;

		bool bTemp2;
		Ar << bTemp2;

		bool bTemp;
		Ar << bTemp;

		TArray<FLegacyTextureLookup> LegacyLookups;
		Ar << LegacyLookups;

		uint32 DummyDroppedFallbackComponents = 0;
		Ar << DummyDroppedFallbackComponents;
	}

	SerializeInlineShaderMap(Ar);
}

void FMaterial::SerializeInlineShaderMap(FArchive& Ar, const FName& SerializingAsset)
{
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogShaders, Fatal, "This platform requires cooked packages, and shaders were not cooked into this material %ls.", *GetFriendlyName());
	}

	if (bCooked)
	{
		if (Ar.IsCooking())
		{
#if WITH_EDITOR
			FinishCompilation();

			bool bValid = GameThreadShaderMap != nullptr && GameThreadShaderMap->CompiledSuccessfully() && (GameThreadShaderMap->GetShaderNum() > 0);
			
			Ar << bValid;

			if (bValid)
			{
				FShaderSerializeContext Ctx{ Ar };
				GameThreadShaderMap->Serialize(Ctx);
			}
			else
			{
				bool bShouldWarn = true;
				FString InvalidShadermapReason;
				
				if (!GameThreadShaderMap)
				{
					InvalidShadermapReason += TEXT("Shadermap pointer is null.");
				}
				else
				{
					if (!GameThreadShaderMap->CompiledSuccessfully())
					{
						InvalidShadermapReason += TEXT("Shadermap exists but wasn't compiled successfully (yet?)");
					}
					else
					{
						if (HasAnyUsage())
						{
							InvalidShadermapReason += TEXT("Shadermap exists, has usage flags but has no shaders");
						}
						else
						{
							UE_LOGF(LogMaterial, Log, "Material resource (in %ls hierarchy) has no usage flags and will have an empty ShaderMap.",
								*GetFriendlyName());
							bShouldWarn = false;
						}
					}

				}

				if (bShouldWarn)
				{
					UE_LOGF(LogMaterial, Warning, "Cooking a material resource (in %ls hierarchy) that doesn't have a valid ShaderMap! %ls",
						*GetFriendlyName(), *InvalidShadermapReason);
				}
			}
#else
			UE_LOGF(LogMaterial, Fatal, "Internal error: cooking outside the editor is not possible.");
			// unreachable
#endif
		}
		else
		{
			bool bValid = false;
			Ar << bValid;

			if (bValid)
			{
				TRefCountPtr<FMaterialShaderMap> LoadedShaderMap = new FMaterialShaderMap();
				FShaderSerializeContext Ctx(Ar);
				Ctx.bLoadingCooked = bCooked && Ar.IsLoading();
				Ctx.SerializingAsset = SerializingAsset;
				if (LoadedShaderMap->Serialize(Ctx))
				{
					GameThreadShaderMap = MoveTemp(LoadedShaderMap);
#if WITH_EDITOR
					GameThreadShaderMap->AssociateWithAsset(GetAssetPath());
#endif
				}
			}
			else
			{
				UE_LOGF(LogMaterial, Error, "Loading a material resource %ls with an invalid ShaderMap!", *GetFriendlyName());
			}
		}
	}
}

void FMaterial::RegisterInlineShaderMap(bool bLoadingCooked)
{
	if (GameThreadShaderMap)
	{
		// Toss the loaded shader data if this is a server only instance
		//@todo - don't cook it in the first place
		if (FApp::CanEverRender())
		{
			RenderingThreadShaderMap = GameThreadShaderMap;
			bRenderingThreadShaderMapIsComplete = GameThreadShaderMap->IsValidForRendering();
		}
	}
}

FName FMaterial::GetOwnerFName() const
{
	UMaterialInterface* Owner = GetMaterialInterface();
	return Owner ? Owner->GetOutermost()->GetFName() : NAME_None;
}

void FMaterialResource::LegacySerialize(FArchive& Ar)
{
	FMaterial::LegacySerialize(Ar);

	if (Ar.UEVer() < VER_UE4_PURGED_FMATERIAL_COMPILE_OUTPUTS)
	{
		int32 BlendModeOverrideValueTemp = 0;
		Ar << BlendModeOverrideValueTemp;
		bool bDummyBool = false;
		Ar << bDummyBool;
		Ar << bDummyBool;
	}
}

TObjectPtr<UObject> FMaterialResource::GetMatDefaultTextureAtIdx(const int32 InIdx) const
{
	if (MaterialInstance)
	{
		TObjectPtr<UObject> Texture =  MaterialInstance->GetDefaultTextureAtIdx(InIdx);
		if (Texture)
		{
			return Texture;
		}
	}

	if (Material)
	{
		return Material->GetDefaultTextureAtIdx(InIdx);
	}

	return UMaterial::GetDefaultMaterial(MD_Surface)->GetDefaultTextureAtIdx(InIdx);
}

#if WITH_EDITOR
int32 FMaterialResource::GetMatDefaultTextureIdx(TObjectPtr<UObject> InTexture) const
{
	if (MaterialInstance)
	{
		const int32 Idx = MaterialInstance->GetDefaultTextureIdx(InTexture);
		if (Idx != INDEX_NONE)
		{
			return Idx;
		}
	}

	if (Material)
	{
		return Material->GetDefaultTextureIdx(InTexture);
	}

	return UMaterial::GetDefaultMaterial(MD_Surface)->GetDefaultTextureIdx(InTexture);
}

const TArray<TObjectPtr<UObject>>& FMaterialResource::GetMatDefaultTextures() const
{
	if (MaterialInstance)
	{
		const TArray<TObjectPtr<UObject>>& Textures = MaterialInstance->GetDefaultTextures();
		if (Textures.Num() > 0)
		{
			return Textures;
		}
	}

	if (Material)
	{
		return Material->GetDefaultTextures();
	}

	return UMaterial::GetDefaultMaterial(MD_Surface)->GetDefaultTextures();
}
#endif // WITH_EDITOR

TArrayView<const TObjectPtr<UObject>> FMaterialResource::GetReferencedTextures() const
{
	if (MaterialInstance)
	{
		const TArrayView<const TObjectPtr<UObject>> Textures = MaterialInstance->GetReferencedTextures();
		if (Textures.Num())
		{
			return Textures;
		}
	}
	
	if (Material)
	{
		return Material->GetReferencedTextures();
	}

	return UMaterial::GetDefaultMaterial(MD_Surface)->GetReferencedTextures();
}

TConstArrayView<TObjectPtr<UTextureCollection>> FMaterialResource::GetReferencedTextureCollections() const
{
	if (MaterialInstance)
	{
		TConstArrayView<TObjectPtr<UTextureCollection>> TextureCollections = MaterialInstance->GetReferencedTextureCollections();
		if (TextureCollections.Num())
		{
			return TextureCollections;
		}
	}

	if (Material)
	{
		return Material->GetReferencedTextureCollections();
	}

	return UMaterial::GetDefaultMaterial(MD_Surface)->GetReferencedTextureCollections();
}

void FMaterialResource::FeedbackMaterialLayersInstancedGraphFromCompilation(const FMaterialLayersFunctions* InLayers)
{
	if (InLayers)
	{
		MaterialLayersFunctions = *(InLayers);
	}
}

const FMaterialLayersFunctions* FMaterialResource::GetMaterialLayers() const
{
	return &MaterialLayersFunctions;
}

void FMaterialResource::AddReferencedObjects(FReferenceCollector& Collector)
{
	FMaterial::AddReferencedObjects(Collector);

	Collector.AddStableReference(&Material);
	Collector.AddStableReference(&MaterialInstance);
}

bool FMaterialResource::GetAllowDevelopmentShaderCompile()const
{
	return Material->bAllowDevelopmentShaderCompile;
}

void FMaterial::ReleaseShaderMap()
{
	UE_CLOGF(IsOwnerBeginDestroyed(), LogMaterial, Error, "ReleaseShaderMap called on FMaterial %ls, owner is BeginDestroyed", *GetDebugName());

	if (GameThreadShaderMap)
	{
		GameThreadShaderMap = nullptr;
		
		TRefCountPtr<FMaterial> Material = this;
		ENQUEUE_RENDER_COMMAND(ReleaseShaderMap)(
		[Material = MoveTemp(Material)](FRHICommandList& RHICmdList)
		{
			Material->RenderingThreadShaderMap = nullptr;
			Material->bRenderingThreadShaderMapIsComplete = false;
		});
	}
}

void FMaterial::DiscardShaderMap()
{
	check(RenderingThreadShaderMap == nullptr);
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap = nullptr;
	}
}

EMaterialDomain FMaterialResource::GetMaterialDomain() const { return Material->MaterialDomain; }
bool FMaterialResource::IsTangentSpaceNormal() const { return Material->bTangentSpaceNormal; }
bool FMaterialResource::ShouldGenerateSphericalParticleNormals() const { return Material->bGenerateSphericalParticleNormals; }
bool FMaterialResource::ShouldDisableDepthTest() const { return Material->bDisableDepthTest; }
bool FMaterialResource::ShouldWriteOnlyAlpha() const { return Material->bWriteOnlyAlpha; }
bool FMaterialResource::ShouldEnableResponsiveAA() const { return Material->bEnableResponsiveAA; }
bool FMaterialResource::ShouldDoSSR() const { return Material->bScreenSpaceReflections; }
bool FMaterialResource::ShouldDoContactShadows() const { return Material->bContactShadows; }
bool FMaterialResource::HasPixelAnimation() const { return (MaterialInstance ? MaterialInstance->HasPixelAnimation() : Material->HasPixelAnimation()) && GetMaterialDomain() == MD_Surface && IsOpaqueOrMaskedBlendMode(GetBlendMode()); }
bool FMaterialResource::UsesTemporalResponsiveness() const { return GetCachedExpressionData().bUsesTemporalResponsiveness;}
bool FMaterialResource::UsesMotionVectorWorldOffset() const { return GetCachedExpressionData().bUsesMotionVectorWorldOffset;}
bool FMaterialResource::IsWireframe() const { return Material->Wireframe; }
bool FMaterialResource::IsUIMaterial() const { return Material->MaterialDomain == MD_UI; }
bool FMaterialResource::IsPostProcessMaterial() const { return Material->MaterialDomain == MD_PostProcess; }
bool FMaterialResource::IsLightFunction() const { return Material->MaterialDomain == MD_LightFunction; }
bool FMaterialResource::IsDeferredDecal() const { return Material->MaterialDomain == MD_DeferredDecal; }
bool FMaterialResource::IsVolumetricPrimitive() const { return Material->MaterialDomain == MD_Volume; }
bool FMaterialResource::IsSpecialEngineMaterial() const { return Material->bUsedAsSpecialEngineMaterial; }
bool FMaterialResource::HasVertexPositionOffsetConnected() const { return GetCachedExpressionData().IsPropertyConnected(MP_WorldPositionOffset); }
bool FMaterialResource::HasPixelDepthOffsetConnected() const { return GetCachedExpressionData().IsPropertyConnected(MP_PixelDepthOffset); }
EMaterialShadingRate FMaterialResource::GetShadingRate() const { return Material->ShadingRate; }
bool FMaterialResource::IsVariableRateShadingAllowed() const { return Material->bAllowVariableRateShading; }
FString FMaterialResource::GetBaseMaterialPathName() const { return Material->GetPathName(); }
FString FMaterialResource::GetDebugName() const
{
	if (MaterialInstance)
	{
		return FString::Printf(TEXT("%s (MI:%s)"), *GetBaseMaterialPathName(), *MaterialInstance->GetPathName());
	}

	return GetBaseMaterialPathName();
}

bool FMaterialResource::IsUsedWithEditorCompositing() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_EditorCompositing) : Material->GetUsageByFlag(MATUSAGE_EditorCompositing);
}

bool FMaterialResource::IsUsedWithSkeletalMesh() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_SkeletalMesh) : Material->GetUsageByFlag(MATUSAGE_SkeletalMesh);
}

bool FMaterialResource::IsUsedWithGeometryCache() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_GeometryCache) : Material->GetUsageByFlag(MATUSAGE_GeometryCache);
}

bool FMaterialResource::IsUsedWithWater() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_Water) : Material->GetUsageByFlag(MATUSAGE_Water);
}

bool FMaterialResource::IsUsedWithHairStrands() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_HairStrands) : Material->GetUsageByFlag(MATUSAGE_HairStrands);
}

bool FMaterialResource::IsUsedWithLidarPointCloud() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_LidarPointCloud) : Material->GetUsageByFlag(MATUSAGE_LidarPointCloud);
}

bool FMaterialResource::IsUsedWithVirtualHeightfieldMesh() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_VirtualHeightfieldMesh) : Material->GetUsageByFlag(MATUSAGE_VirtualHeightfieldMesh);
}

bool FMaterialResource::IsUsedWithNeuralNetworks() const
{
	if (!Material->IsPostProcessMaterial())
	{
		return false;
	}
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_NeuralNetworks) : Material->GetUsageByFlag(MATUSAGE_NeuralNetworks);
}

bool FMaterialResource::IsUsedWithMeshDeformer() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_MeshDeformer) : Material->GetUsageByFlag(MATUSAGE_MeshDeformer);
}

bool FMaterialResource::IsUsedWithInstancedSkinnedMesh() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_InstancedSkinnedMesh) : Material->GetUsageByFlag(MATUSAGE_InstancedSkinnedMesh);
}

bool FMaterialResource::IsUsedWithLandscape() const
{
	return false;
}

bool FMaterialResource::IsUsedWithParticleSystem() const
{
	return MaterialInstance 
		? MaterialInstance->GetUsageByFlag(MATUSAGE_ParticleSprites) || MaterialInstance->GetUsageByFlag(MATUSAGE_BeamTrails)
		: Material->GetUsageByFlag(MATUSAGE_ParticleSprites) || Material->GetUsageByFlag(MATUSAGE_BeamTrails);
}

bool FMaterialResource::IsUsedWithParticleSprites() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_ParticleSprites) : Material->GetUsageByFlag(MATUSAGE_ParticleSprites);
}

bool FMaterialResource::IsUsedWithBeamTrails() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_BeamTrails) : Material->GetUsageByFlag(MATUSAGE_BeamTrails);
}

bool FMaterialResource::IsUsedWithMeshParticles() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_MeshParticles) : Material->GetUsageByFlag(MATUSAGE_MeshParticles);
}

bool FMaterialResource::IsUsedWithNiagaraSprites() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_NiagaraSprites) : Material->GetUsageByFlag(MATUSAGE_NiagaraSprites);
}

bool FMaterialResource::IsUsedWithNiagaraRibbons() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_NiagaraRibbons) : Material->GetUsageByFlag(MATUSAGE_NiagaraRibbons);
}

bool FMaterialResource::IsUsedWithNiagaraMeshParticles() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_NiagaraMeshParticles) : Material->GetUsageByFlag(MATUSAGE_NiagaraMeshParticles);
}

bool FMaterialResource::IsUsedWithStaticLighting() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_StaticLighting) : Material->GetUsageByFlag(MATUSAGE_StaticLighting);
}

bool FMaterialResource::IsUsedWithMorphTargets() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_MorphTargets) : Material->GetUsageByFlag(MATUSAGE_MorphTargets);
}

bool FMaterialResource::IsUsedWithSplineMeshes() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_SplineMesh) : Material->GetUsageByFlag(MATUSAGE_SplineMesh);
}

bool FMaterialResource::IsUsedWithInstancedStaticMeshes() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_InstancedStaticMeshes) : Material->GetUsageByFlag(MATUSAGE_InstancedStaticMeshes);
}

bool FMaterialResource::IsUsedWithGeometryCollections() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_GeometryCollections) : Material->GetUsageByFlag(MATUSAGE_GeometryCollections);
}

bool FMaterialResource::IsUsedWithAPEXCloth() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_Clothing) : Material->GetUsageByFlag(MATUSAGE_Clothing);
}

bool FMaterialResource::IsUsedWithNanite() const
{
	const bool bUsedWithNanite = MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_Nanite) : Material->GetUsageByFlag(MATUSAGE_Nanite);
	if (bUsedWithNanite)
	{
		return true;
	}

	static const auto NaniteForceEnableMeshesCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.ForceEnableMeshes"));
	static const bool bNaniteForceEnableMeshes = NaniteForceEnableMeshesCvar && NaniteForceEnableMeshesCvar->GetValueOnAnyThread() != 0;

	if (bNaniteForceEnableMeshes)
	{
		const bool bIsInGameThread = (IsInGameThread() || IsInParallelGameThread());
		const FMaterialShaderMap* ShaderMap = bIsInGameThread ? GetGameThreadShaderMap() : GetRenderingThreadShaderMap();

		bool bIsCookedMaterial = (ShaderMap && ShaderMap->GetShaderMapId().IsCookedId());
		if (bIsCookedMaterial)
		{
			return Material->GetUsageByFlag(MATUSAGE_Nanite);
		}

		return true;
	}

	return false;
}

bool FMaterialResource::IsUsedWithVoxels() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_Voxels) : Material->GetUsageByFlag(MATUSAGE_Voxels);
}

bool FMaterialResource::IsUsedWithCurves() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_Curves) : Material->GetUsageByFlag(MATUSAGE_Curves);
}

bool FMaterialResource::IsUsedWithVolumetricCloud() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_VolumetricCloud) : Material->GetUsageByFlag(MATUSAGE_VolumetricCloud);
}

bool FMaterialResource::IsUsedWithHeterogeneousVolumes() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_HeterogeneousVolumes) : Material->GetUsageByFlag(MATUSAGE_HeterogeneousVolumes);
}

bool FMaterialResource::IsUsedWithStaticMesh() const
{
	return MaterialInstance ? MaterialInstance->GetUsageByFlag(MATUSAGE_StaticMesh) : Material->GetUsageByFlag(MATUSAGE_StaticMesh);
}

bool FMaterialResource::HasAnyUsage() const
{
	return MaterialInstance ? (MaterialInstance->GetUsageFlags() != 0) : (Material->GetUsageFlags() != 0);
}

bool FMaterialResource::SamplesMaterialCache() const
{
	return Material->GetCachedExpressionData().bSamplesMaterialCache;
}

bool FMaterialResource::HasMaterialCacheOutput() const
{
	return Material->GetCachedExpressionData().bHasMaterialCacheOutput;
}

bool FMaterialResource::IsTranslucencyAfterDOFEnabled() const 
{ 
	return Material->TranslucencyPass == MTP_AfterDOF
		&& !IsUIMaterial()
		&& !IsDeferredDecal();
}

bool FMaterialResource::IsTranslucencyAfterMotionBlurEnabled() const 
{ 
	return Material->TranslucencyPass == MTP_AfterMotionBlur
		&& !IsUIMaterial()
		&& !IsDeferredDecal();
}

bool FMaterialResource::IsDualBlendingEnabled() const
{
	bool bMaterialRequestsDualSourceBlending = Material->ShadingModel == MSM_ThinTranslucent;
	if (IsSubstrateMaterial())
	{
		bMaterialRequestsDualSourceBlending = GetBlendMode() == EBlendMode::BLEND_TranslucentColoredTransmittance;
	}
	const bool bIsPlatformSupported = RHISupportsDualSourceBlending(GetShaderPlatform()) || IsMobilePlatform(GetShaderPlatform()); // Mobile renderer has runtime fallbacks
	return bMaterialRequestsDualSourceBlending && bIsPlatformSupported;
}

bool FMaterialResource::IsMobileSeparateTranslucencyEnabled() const
{
	return Material->IsMobileSeparateTranslucencyEnabled();
}

bool FMaterialResource::IsFullyRough() const
{
	return Material->bFullyRough;
}

bool FMaterialResource::GetForceCompatibleWithLightFunctionAtlas() const
{
	return Material->bForceCompatibleWithLightFunctionAtlas;
}

bool FMaterialResource::UseNormalCurvatureToRoughness() const
{
	return Material->bNormalCurvatureToRoughness;
}

EMaterialFloatPrecisionMode FMaterialResource::GetMaterialFloatPrecisionMode() const
{
	return Material->FloatPrecisionMode;
}

bool FMaterialResource::IsUsingAlphaToCoverage() const
{
	return Material->bUseAlphaToCoverage && Material->MaterialDomain == EMaterialDomain::MD_Surface && IsMaskedBlendMode(*Material) && !WritesEveryPixel();
}

bool FMaterialResource::IsUsingPreintegratedGFForSimpleIBL() const
{
	return Material->bForwardRenderUsePreintegratedGFForSimpleIBL;
}

bool FMaterialResource::IsUsingHQForwardReflections() const
{
	return Material->bUseHQForwardReflections;
}

bool FMaterialResource::GetForwardBlendsSkyLightCubemaps() const
{
	return Material->bForwardBlendsSkyLightCubemaps;
}

bool FMaterialResource::IsUsingPlanarForwardReflections() const
{
	return Material->bUsePlanarForwardReflections;
}

bool FMaterialResource::IsNonmetal() const
{
	return (!Material->IsPropertyConnected(MP_Metallic) && !Material->IsPropertyConnected(MP_Specular));
}

bool FMaterialResource::UseLmDirectionality() const
{
	return Material->bUseLightmapDirectionality;
}

/**
 * Should shaders compiled for this material be saved to disk?
 */
bool FMaterialResource::IsPersistent() const { return true; }

FGuid FMaterialResource::GetMaterialId() const
{
	// It's possible for Material to become null due to AddReferencedObjects
	return Material ? Material->StateId : FGuid();
}

ETranslucencyLightingMode FMaterialResource::GetTranslucencyLightingMode() const { return (ETranslucencyLightingMode)Material->TranslucencyLightingMode; }

float FMaterialResource::GetOpacityMaskClipValue() const 
{
	return MaterialInstance ? MaterialInstance->GetOpacityMaskClipValue() : Material->GetOpacityMaskClipValue();
}

bool FMaterialResource::GetCastDynamicShadowAsMasked() const
{
	return MaterialInstance ? MaterialInstance->GetCastDynamicShadowAsMasked() : Material->GetCastDynamicShadowAsMasked();
}

EBlendMode FMaterialResource::GetBlendMode() const 
{
	return MaterialInstance ? MaterialInstance->GetBlendMode() : Material->GetBlendMode();
}

ERefractionMode FMaterialResource::GetRefractionMode() const
{
	return Material->RefractionMethod;
}

bool FMaterialResource::GetRootNodeOverridesDefaultRefraction() const
{
	return false;
}

FMaterialShadingModelField FMaterialResource::GetShadingModels() const 
{
	return MaterialInstance ? MaterialInstance->GetShadingModels() : Material->GetShadingModels();
}

bool FMaterialResource::IsShadingModelFromMaterialExpression() const 
{
	return MaterialInstance ? MaterialInstance->IsShadingModelFromMaterialExpression() : Material->IsShadingModelFromMaterialExpression(); 
}

bool FMaterialResource::IsTwoSided() const 
{
	return MaterialInstance ? MaterialInstance->IsTwoSided() : Material->IsTwoSided();
}

bool FMaterialResource::IsThinSurface() const
{
	return MaterialInstance ? MaterialInstance->IsThinSurface() : Material->IsThinSurface();
}

bool FMaterialResource::IsDitheredLODTransition() const 
{
	if (!AllowDitheredLODTransition(GetShaderPlatform()))
	{
		return false;
	}

	return MaterialInstance ? MaterialInstance->IsDitheredLODTransition() : Material->IsDitheredLODTransition();
}

bool FMaterialResource::IsTranslucencyWritingCustomDepth() const
{
	// We cannot call UMaterial::IsTranslucencyWritingCustomDepth because we need to check the instance potentially overriden blend mode.
	return Material->AllowTranslucentCustomDepthWrites != 0 && IsTranslucentBlendMode(GetBlendMode());
}

bool FMaterialResource::IsTranslucencyWritingVelocity() const
{
	return MaterialInstance ? MaterialInstance->IsTranslucencyWritingVelocity() : Material->IsTranslucencyWritingVelocity();
}

bool FMaterialResource::IsTranslucencyVelocityFromDepth() const
{
	return MaterialInstance ? MaterialInstance->IsTranslucencyVelocityFromDepth() : Material->IsTranslucencyVelocityFromDepth();
}

bool FMaterialResource::IsTranslucencyWritingFrontLayerTransparency() const
{
	// We cannot call UMaterial::IsTranslucencyWritingFrontLayerTransparency because we need to check the instance potentially overriden blend mode.
	return IsTranslucentBlendMode(GetBlendMode())
		&& (Material->TranslucencyLightingMode == TLM_Surface || Material->TranslucencyLightingMode == TLM_SurfacePerPixelLighting)
		&& Material->bAllowFrontLayerTranslucency;
}

bool FMaterialResource::IsMasked() const 
{
	return MaterialInstance ? MaterialInstance->IsMasked() : Material->IsMasked();
}

bool FMaterialResource::IsDitherMasked() const 
{
	return Material->DitherOpacityMask && IsMasked();
}

bool FMaterialResource::AllowNegativeEmissiveColor() const 
{
	return Material->bAllowNegativeEmissiveColor;
}

bool FMaterialResource::IsDistorted() const { return Material->bUsesDistortion && IsTranslucentBlendMode(GetBlendMode()); }
ERefractionCoverageMode FMaterialResource::GetRefractionCoverageMode() const { return Material->RefractionCoverageMode; }
EPixelDepthOffsetMode FMaterialResource::GetPixelDepthOffsetMode() const { return Material->PixelDepthOffsetMode; }
float FMaterialResource::GetTranslucencyDirectionalLightingIntensity() const { return Material->TranslucencyDirectionalLightingIntensity; }
float FMaterialResource::GetTranslucentShadowDensityScale() const { return Material->TranslucentShadowDensityScale; }
float FMaterialResource::GetTranslucentSelfShadowDensityScale() const { return Material->TranslucentSelfShadowDensityScale; }
float FMaterialResource::GetTranslucentSelfShadowSecondDensityScale() const { return Material->TranslucentSelfShadowSecondDensityScale; }
float FMaterialResource::GetTranslucentSelfShadowSecondOpacity() const { return Material->TranslucentSelfShadowSecondOpacity; }
float FMaterialResource::GetTranslucentBackscatteringExponent() const { return Material->TranslucentBackscatteringExponent; }
FLinearColor FMaterialResource::GetTranslucentMultipleScatteringExtinction() const { return Material->TranslucentMultipleScatteringExtinction; }
float FMaterialResource::GetTranslucentShadowStartOffset() const { return Material->TranslucentShadowStartOffset; }
float FMaterialResource::GetRefractionDepthBiasValue() const { return Material->RefractionDepthBias; }
bool FMaterialResource::ShouldApplyFogging() const { return Material->bUseTranslucencyVertexFog; }
bool FMaterialResource::ShouldApplyCloudFogging() const { return Material->bApplyCloudFogging; }
bool FMaterialResource::ShouldAlwaysEvaluateWorldPositionOffset() const { return Material->bAlwaysEvaluateWorldPositionOffset; }
bool FMaterialResource::IsSky() const { return Material->bIsSky; }
bool FMaterialResource::AllowTranslucentLocalLightShadow() const { return Material->bAllowTranslucentLocalLightShadow; }
float FMaterialResource::GetTranslucentLocalLightShadowQuality() const { return Material->TranslucentLocalLightShadowQuality; }
float FMaterialResource::GetTranslucentDirectionalLightShadowQuality() const { return Material->TranslucentDirectionalLightShadowQuality; }
bool FMaterialResource::ComputeFogPerPixel() const {return Material->bComputeFogPerPixel;}
FString FMaterialResource::GetFriendlyName() const { return GetNameSafe(Material); } //avoid using the material instance name here, we want materials that share a shadermap to also share a friendly name.
FString FMaterialResource::GetAssetName() const { return MaterialInstance ? GetNameSafe(MaterialInstance) : GetNameSafe(Material); }

FDisplacementScaling FMaterialResource::GetDisplacementScaling() const
{
	return GetMaterialInterface()->GetDisplacementScaling();
}

bool FMaterialResource::IsDisplacementFadeEnabled() const
{
	return GetMaterialInterface()->IsDisplacementFadeEnabled();
}

FDisplacementFadeRange FMaterialResource::GetDisplacementFadeRange() const
{
	return GetMaterialInterface()->GetDisplacementFadeRange();
}

uint32 FMaterialResource::GetMaterialDecalResponse() const
{
	return Material->GetMaterialDecalResponse();
}

bool FMaterialResource::HasBaseColorConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_BaseColor);
}

bool FMaterialResource::HasNormalConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_Normal);
}

bool FMaterialResource::HasRoughnessConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_Roughness);
}

bool FMaterialResource::HasSpecularConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_Specular);
}

bool FMaterialResource::HasMetallicConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_Metallic);
}

bool FMaterialResource::HasEmissiveColorConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_EmissiveColor);
}

bool FMaterialResource::HasAnisotropyConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_Anisotropy);
}

bool FMaterialResource::HasAmbientOcclusionConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_AmbientOcclusion);
}

bool FMaterialResource::HasDisplacementConnected() const
{
	return GetCachedExpressionData().IsPropertyConnected(MP_Displacement);
}

bool FMaterialResource::IsSubstrateMaterial() const
{
	const bool bSubstrateEnabled						= Substrate::IsSubstrateEnabled();
	const bool bSubstrateHiddenMaterialAssetConversion	= Substrate::IsHiddenMaterialAssetConversionEnabled();
	const bool bIsSubstrateBlendableGBufferEnabled		= Substrate::IsSubstrateBlendableGBufferEnabled(GetShaderPlatform());

	// Material not using Substrate code must be reported as non-substrate because the translucent blend mode needs to be different. Must match bSubstrateMaterialUsesLegacyMaterialCompilation from HLSLMaterialTranslator.
	const bool bIsSubstrateMaterialCompiledAsLegacy		= bSubstrateEnabled && bSubstrateHiddenMaterialAssetConversion && bIsSubstrateBlendableGBufferEnabled && !HasMaterialPropertyConnected(MP_FrontMaterial);

	return bSubstrateEnabled && !bIsSubstrateMaterialCompiledAsLegacy;
}

bool FMaterialResource::HasMaterialPropertyConnected(EMaterialProperty In) const
{
	// SUBSTRATE_TODO: temporary validation until we have converted all domains
	const bool bIsSubstrateSupportedDomain = 
		Material->MaterialDomain == MD_PostProcess || 
		Material->MaterialDomain == MD_LightFunction ||
		Material->MaterialDomain == MD_DeferredDecal || 
		Material->MaterialDomain == MD_Surface ||
		Material->MaterialDomain == MD_Volume ||
		Material->MaterialDomain == MD_UI;

	if (Substrate::IsSubstrateEnabled() && bIsSubstrateSupportedDomain)
	{
		if (In == MP_AmbientOcclusion)
		{
			// AO is specified on the root node so use the regular accessor.
			return Material->HasAmbientOcclusionConnected();
		}
		// Substrate material traversal is cached as this is an expensive operation
		return Material->IsPropertyConnected(In);
	}
	else
	{
		switch (In)
		{
		case MP_EmissiveColor: 		return Material->HasEmissiveColorConnected();
		case MP_Opacity: 			return Material->HasEmissiveColorConnected();
		case MP_BaseColor: 			return Material->HasBaseColorConnected();
		case MP_Normal: 			return Material->HasNormalConnected();
		case MP_Roughness: 			return Material->HasRoughnessConnected();
		case MP_Specular: 			return Material->HasSpecularConnected();
		case MP_Metallic: 			return Material->HasMetallicConnected();
		case MP_Anisotropy: 		return Material->HasAnisotropyConnected();
		case MP_AmbientOcclusion: 	return Material->HasAmbientOcclusionConnected();
		}
	}
	return false;
}

bool FMaterialResource::RequiresSynchronousCompilation() const
{
	return Material->IsDefaultMaterial();
}

bool FMaterialResource::IsDefaultMaterial() const
{
	return Material->IsDefaultMaterial();
}

int32 FMaterialResource::GetNumCustomizedUVs() const
{
	return Material->NumCustomizedUVs;
}

int32 FMaterialResource::GetNumMaterialCacheTags() const
{
	return GetUniformExpressions().GetMaterialCacheTagStacks().Num();
}

int32 FMaterialResource::GetBlendableLocation() const
{
	return Material->BlendableLocation;
}

int32 FMaterialResource::GetBlendablePriority() const
{
	return Material->BlendablePriority;
}

bool FMaterialResource::GetBlendableOutputAlpha() const
{
	return Material->IsPostProcessMaterialOutputingAlpha();
}

bool FMaterialResource::GetDisablePreExposureScale() const
{
	return GetMaterialDomain() == MD_PostProcess && Material->bDisablePreExposureScale;
}

bool FMaterialResource::IsStencilTestEnabled() const
{
	return GetMaterialDomain() == MD_PostProcess && Material->bEnableStencilTest;
}

uint32 FMaterialResource::GetStencilRefValue() const
{
	return GetMaterialDomain() == MD_PostProcess ? Material->StencilRefValue : 0;
}

int32 FMaterialResource::GetNeuralProfileId() const
{ 
	return GetMaterialDomain() == MD_PostProcess ? Material->NeuralProfileId : INDEX_NONE;
}

bool FMaterialResource::HasSubstrateRoughnessTracking() const
{ 
	return Material->HasSubstrateRoughnessTracking();
}

uint32 FMaterialResource::GetStencilCompare() const
{
	return GetMaterialDomain() == MD_PostProcess ? uint32(Material->StencilCompare.GetValue()) : 0;
}

bool FMaterialResource::HasPerInstanceCustomData() const
{
	return GetCachedExpressionData().bHasPerInstanceCustomData;
}

bool FMaterialResource::HasPerInstanceRandom() const
{
	return GetCachedExpressionData().bHasPerInstanceRandom;
}

bool FMaterialResource::HasVertexInterpolator() const
{
	return GetCachedExpressionData().bHasVertexInterpolator;
}

bool FMaterialResource::HasRuntimeVirtualTextureOutput() const
{
	return GetCachedExpressionData().bHasRuntimeVirtualTextureOutput;
}

bool FMaterialResource::HasFirstPersonOutput() const
{
	return GetCachedExpressionData().bHasFirstPersonOutput;
}

bool FMaterialResource::CastsRayTracedShadows() const
{
	return Material->bCastRayTracedShadows;
}

bool FMaterialResource::IsTessellationEnabled() const
{
	return GetMaterialInterface()->IsTessellationEnabled();
}

bool FMaterialResource::HasRenderTracePhysicalMaterialOutputs() const
{
	return Material->GetRenderTracePhysicalMaterialOutputs().Num() > 0;
}

uint16 FMaterialResource::GetPreshaderGap() const
{
	return Material->PreshaderGap;
}

UMaterialInterface* FMaterialResource::GetMaterialInterface() const 
{ 
	return MaterialInstance ? (UMaterialInterface*)MaterialInstance : (UMaterialInterface*)Material;
}

#if WITH_EDITOR
void FMaterialResource::GetShaderTags(TArray<FName>& OutShaderTags) const
{
	const FMaterialCachedExpressionData& CachedExpressionData = GetMaterialInterface()->GetCachedExpressionData();
	OutShaderTags.Append(CachedExpressionData.EditorOnlyData->ShaderTags);
}

EMaterialTranslateValidationFlags FMaterialResource::GetMaterialTranslateValidationFlags() const
{
	return Material->GetMaterialTranslateValidationFlags();
}

void FMaterialResource::NotifyCompilationFinished()
{
	UMaterial::NotifyCompilationFinished(MaterialInstance ? (UMaterialInterface*)MaterialInstance : (UMaterialInterface*)Material);
}

FName FMaterialResource::GetAssetPath() const
{
	FName OutermostName;
	if (MaterialInstance)
	{
		OutermostName = MaterialInstance->GetOutermost()->GetFName();
	}
	else if (Material)
	{
		OutermostName = Material->GetOutermost()->GetFName();
	}
	else
	{
		// neither is known
		return NAME_None;
	}

	return OutermostName;
}

bool FMaterialResource::IsUsingNewHLSLGenerator() const
{
	if (Material)
	{
		return Material->IsUsingNewHLSLGenerator();
	}
	return false;
}

bool FMaterialResource::CheckInValidStateForCompilation(FMaterialCompiler* Compiler) const
{
	return Material && Material->CheckInValidStateForCompilation(Compiler);
}

void FMaterial::AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const
{
	check(IsInGameThread());

	if (CacheShadersPending && !CacheShadersPending->IsReady())
	{
		OutDebugInfo << "Pending async DDC load\n";
	}
	else if (GetGameThreadCompilingShaderMapId() != 0u)
	{
		FMaterialShaderMap* CompilingShaderMap = FMaterialShaderMap::FindCompilingShaderMap(GetGameThreadCompilingShaderMapId());
		if (CompilingShaderMap)
		{
			CompilingShaderMap->AppendCompileStateDebugInfo(OutDebugInfo);
		}
	}
	else
	{
		FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap();
		OutDebugInfo << "Compilation not executing; shadermap is " << (!ShaderMap ? "null" : (ShaderMap->IsComplete(this, true) ? "complete" : "incomplete")) << "\n";
	}
}

#endif

FString FMaterialResource::GetFullPath() const
{
	if (MaterialInstance)
	{
		return MaterialInstance->GetPathName();
	}
	if (Material)
	{
		return Material->GetPathName();
	}

	return FString();
}

void FMaterialResource::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	TSet<const FMaterialShaderMap*> UniqueShaderMaps;
	UniqueShaderMaps.Add(GetGameThreadShaderMap());

	for (TSet<const FMaterialShaderMap*>::TConstIterator It(UniqueShaderMaps); It; ++It)
	{
		const FMaterialShaderMap* MaterialShaderMap = *It;
		if (MaterialShaderMap)
		{
			CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MaterialShaderMap->GetFrozenContentSize());

			const FShaderMapResource* Resource = MaterialShaderMap->GetResource();
			if (Resource)
			{
				CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Resource->GetSizeBytes());
			}
		}
	}
}

#if UE_CHECK_FMATERIAL_LIFETIME
uint32 FMaterial::AddRef() const
{
	const int32 Refs = NumDebugRefs.Increment();
	UE_CLOG(Refs <= 0, LogMaterial, Fatal, TEXT("FMaterial::AddRef, Invalid NumDebugRefs %d"), Refs);
	UE_CLOGF(Refs > 5000, LogMaterial, Warning, "FMaterial::AddRef, Suspicious NumDebugRefs %d", Refs);
	return uint32(Refs);
}

uint32 FMaterial::Release() const
{
	const int32 Refs = NumDebugRefs.Decrement();
	UE_CLOG(Refs < 0, LogMaterial, Fatal, TEXT("FMaterial::Release, Invalid NumDebugRefs %d"), Refs);
	UE_CLOGF(Refs > 5000, LogMaterial, Warning, "FMaterial::Release, Suspicious NumDebugRefs %d", Refs);
	return uint32(Refs);
}
#endif // UE_CHECK_FMATERIAL_LIFETIME

bool FMaterial::PrepareDestroy_GameThread()
{
	check(IsInGameThread());
	
	// Make local copy to make sure lock is held at short as possible
	TArray<FMaterialPSOPrecacheRequestID> TmpPrecachedPSORequestIDs;	
	{
		FScopeLock ScopeLock(&PrecachedPSORequestIDsCS);
		TmpPrecachedPSORequestIDs = MoveTemp(PrecachedPSORequestIDs);
		PrecachedPSORequestIDs.Empty();
	}
	ReleasePSOPrecacheData(TmpPrecachedPSORequestIDs);

#if WITH_EDITOR
	const bool bReleasedCompilingId = ReleaseGameThreadCompilingShaderMap();

	if (GIsEditor)
	{
		const FSetElementId FoundId = EditorLoadedMaterialResources.FindId(this);
		if (FoundId.IsValidId())
		{
			// Remove the material from EditorLoadedMaterialResources if found
			EditorLoadedMaterialResources.Remove(FoundId);
		}
	}

	return bReleasedCompilingId;
#else
	return false;
#endif
}

void FMaterial::PrepareDestroy_RenderThread()
{
	check(IsInRenderingThread());

#if WITH_EDITOR
	RenderingThreadCompilingShaderMapId = 0u;
	RenderingThreadPendingCompilerEnvironment.SafeRelease();
#endif
}

void FMaterial::DeferredDelete(FMaterial* InMaterial)
{
	if (InMaterial)
	{
		if (InMaterial->PrepareDestroy_GameThread())
		{
			TRefCountPtr<FMaterial> Material(InMaterial);
			ENQUEUE_RENDER_COMMAND(DeferredDestroyMaterial)([Material = MoveTemp(Material)](FRHICommandListImmediate& RHICmdList) mutable
			{
				FMaterial* MaterialToDelete = Material.GetReference();
				MaterialToDelete->PrepareDestroy_RenderThread();
				Material.SafeRelease();
				delete MaterialToDelete;
			});
		}
		else
		{
			delete InMaterial;
		}
	}
}

void FMaterial::DeleteMaterialsOnRenderThread(TArray<TRefCountPtr<FMaterial>>& MaterialsRenderThread)
{
	if (MaterialsRenderThread.Num() > 0)
	{
		ENQUEUE_RENDER_COMMAND(DeferredDestroyMaterialArray)([MaterialsRenderThread = MoveTemp(MaterialsRenderThread)](FRHICommandListImmediate& RHICmdList) mutable
		{
			for (TRefCountPtr<FMaterial>& Material : MaterialsRenderThread)
			{
				FMaterial* MaterialToDestroy = Material.GetReference();
				MaterialToDestroy->PrepareDestroy_RenderThread();
				Material.SafeRelease();
				delete MaterialToDestroy;
			}
		});
	}
}

/**
 * Destructor
 */
FMaterial::~FMaterial()
{
#if WITH_ODSC
	FODSCManager::UnregisterMaterialName(this);
#endif

#if WITH_EDITOR
	check(GameThreadCompilingShaderMapId == 0u);
	check(RenderingThreadCompilingShaderMapId == 0u);
	check(!RenderingThreadPendingCompilerEnvironment.IsValid());
#endif // WITH_EDITOR

#if UE_CHECK_FMATERIAL_LIFETIME
	const uint32 NumRemainingRefs = GetRefCount();
	UE_CLOGF(NumRemainingRefs > 0u, LogMaterial, Fatal, "%ls Leaked %d refs", *GetDebugName(), NumRemainingRefs);
#endif // UE_CHECK_FMATERIAL_LIFETIME

#if WITH_EDITOR
	checkf(!EditorLoadedMaterialResources.Contains(this), TEXT("FMaterial is still in EditorLoadedMaterialResources when destroyed, should use FMaterial::DeferredDestroy to remove"));
#endif // WITH_EDITOR
}

#if WITH_EDITOR

/** Populates OutEnvironment with defines needed to compile shaders for this material. */
void FMaterial::SetupMaterialEnvironment(
	TConstArrayView<FShaderParametersMetadata*> InUniformBufferStructs,
	const FUniformExpressionSet& InUniformExpressionSet,
	FShaderCompilerEnvironment& OutEnvironment
	) const
{
	// Add the material uniform buffer definition.
	for (const FShaderParametersMetadata* InUniformBufferStruct : InUniformBufferStructs)
	{
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(InUniformBufferStruct->GetShaderVariableName(), *InUniformBufferStruct, GetShaderPlatform(), OutEnvironment);
	}

	if (GetMaterialDomain() == MD_Surface)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_SupportsMinimalBindless);
	}

	// Mark as using external texture if uniform expression contains external texture
	if (InUniformExpressionSet.UniformExternalTextureParameters.Num() > 0)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_UsesExternalTexture);
	}

	if (!Substrate::IsSubstrateEnabled())
	{
		switch(GetBlendMode())
		{
		case BLEND_Opaque:
		case BLEND_Masked:
		{
			// Only set MATERIALBLENDING_MASKED if the material is truly masked
			//@todo - this may cause mismatches with what the shader compiles and what the renderer thinks the shader needs
			// For example IsTranslucentBlendMode doesn't check IsMasked
			if(!WritesEveryPixel())
			{
				SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_MASKED, 1);
			}
			else
			{
				SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_SOLID, 1);
			}
			break;
		}
		case BLEND_AlphaComposite:
		{
			// Blend mode will reuse MATERIALBLENDING_TRANSLUCENT
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_ALPHACOMPOSITE, true);
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			break;
		}
		case BLEND_AlphaHoldout:
		{
			// Blend mode will reuse MATERIALBLENDING_TRANSLUCENT
			SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_ALPHAHOLDOUT, 1);
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			break;
		}
		case BLEND_TranslucentColoredTransmittance:
		case BLEND_Translucent:
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			break;

		case BLEND_Additive: SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_ADDITIVE, true); break;
		case BLEND_Modulate: SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_MODULATE, true); break;

		default: 
			UE_LOGF(LogMaterial, Warning, "Unknown material blend mode: %u  Setting to BLEND_Opaque",(int32)GetBlendMode());
			SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_SOLID, 1);
		}
	}
	else
	{
		switch (GetBlendMode())
		{
		case BLEND_Opaque:
		case BLEND_Masked:
		{
			// Only set MATERIALBLENDING_MASKED if the material is truly masked
			//@todo - this may cause mismatches with what the shader compiles and what the renderer thinks the shader needs
			// For example IsTranslucentBlendMode doesn't check IsMasked
			if (!WritesEveryPixel())
			{
				SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_MASKED, 1);
			}
			else
			{
				SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_SOLID, 1);
			}
			break;
		}
		case BLEND_Additive:
		{
			SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_ADDITIVE, 1);
			SET_SHADER_DEFINE(OutEnvironment, SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE, 1);
			break;
		}
		case BLEND_AlphaComposite:
		{
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_ALPHACOMPOSITE, true);
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			SET_SHADER_DEFINE(OutEnvironment, SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE, 1);
			break;
		}
		case BLEND_TranslucentGreyTransmittance:
		{
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			SET_SHADER_DEFINE(OutEnvironment, SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE, 1);
			break;
		}
		case BLEND_TranslucentColoredTransmittance:
		{
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			SET_SHADER_DEFINE(OutEnvironment, SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE, 1);
			break;
		}
		case BLEND_ColoredTransmittanceOnly:
		{
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_MODULATE, true);
			SET_SHADER_DEFINE(OutEnvironment, SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY, 1);
			break;
		}
		case BLEND_AlphaHoldout:
		{
			SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(OutEnvironment, MATERIALBLENDING_TRANSLUCENT, true);
			SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_ALPHAHOLDOUT, 1);
			SET_SHADER_DEFINE(OutEnvironment, SUBSTRATE_BLENDING_ALPHAHOLDOUT, 1);
			break;
		}
		default:
			UE_LOGF(LogMaterial, Error, "%ls: Unkown Substrate material blend mode could not be converted to Starta. (Asset: %ls) Setting to BLEND_Opaque", *GetFriendlyName(), *GetAssetName());
			SET_SHADER_DEFINE(OutEnvironment, MATERIALBLENDING_SOLID, 1);
		}
	}

	{
		EMaterialDecalResponse MaterialDecalResponse = (EMaterialDecalResponse)GetMaterialDecalResponse();

		// bit 0:color/1:normal/2:roughness to enable/disable parts of the DBuffer decal effect
		int32 MaterialDecalResponseMask = 0;

		switch(MaterialDecalResponse)
		{
			case MDR_None:					MaterialDecalResponseMask = 0; break;
			case MDR_ColorNormalRoughness:	MaterialDecalResponseMask = 1 + 2 + 4; break;
			case MDR_Color:					MaterialDecalResponseMask = 1; break;
			case MDR_ColorNormal:			MaterialDecalResponseMask = 1 + 2; break;
			case MDR_ColorRoughness:		MaterialDecalResponseMask = 1 + 4; break;
			case MDR_Normal:				MaterialDecalResponseMask = 2; break;
			case MDR_NormalRoughness:		MaterialDecalResponseMask = 2 + 4; break;
			case MDR_Roughness:				MaterialDecalResponseMask = 4; break;
			default:
				check(0);
		}

		SET_SHADER_DEFINE(OutEnvironment, MATERIALDECALRESPONSEMASK, MaterialDecalResponseMask);
	}

	switch(GetRefractionMode())
	{
	case RM_IndexOfRefraction:			SET_SHADER_DEFINE(OutEnvironment, REFRACTION_USE_INDEX_OF_REFRACTION,			1); break;
	case RM_PixelNormalOffset:			SET_SHADER_DEFINE(OutEnvironment, REFRACTION_USE_PIXEL_NORMAL_OFFSET,			1); break;
	case RM_2DOffset:					SET_SHADER_DEFINE(OutEnvironment, REFRACTION_USE_2D_OFFSET,						1); break;
	case RM_None:						SET_SHADER_DEFINE(OutEnvironment, REFRACTION_USE_NONE,							1); break;
	case RM_IndexOfRefractionFromF0:	SET_SHADER_DEFINE(OutEnvironment, REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0,	1); break;
	default: 
		UE_LOGF(LogMaterial, Warning, "Unknown material refraction mode: %u  Setting to RM_IndexOfRefraction",(int32)GetRefractionMode());
		SET_SHADER_DEFINE(OutEnvironment, REFRACTION_USE_INDEX_OF_REFRACTION, 1);
	}

	SET_SHADER_DEFINE(OutEnvironment, USE_DITHERED_LOD_TRANSITION_FROM_MATERIAL, IsDitheredLODTransition());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_TWOSIDED, IsTwoSided());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_ISTHINSURFACE, IsThinSurface());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_TANGENTSPACENORMAL, IsTangentSpaceNormal());
	SET_SHADER_DEFINE(OutEnvironment, GENERATE_SPHERICAL_PARTICLE_NORMALS,ShouldGenerateSphericalParticleNormals());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_USES_SCENE_COLOR_COPY, RequiresSceneColorCopy_GameThread());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_USE_PREINTEGRATED_GF, IsUsingPreintegratedGFForSimpleIBL());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_HQ_FORWARD_REFLECTION_CAPTURES, IsUsingHQForwardReflections());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_FORWARD_BLENDS_SKYLIGHT_CUBEMAPS, GetForwardBlendsSkyLightCubemaps());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_PLANAR_FORWARD_REFLECTIONS, IsUsingPlanarForwardReflections());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_NONMETAL, IsNonmetal());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_USE_LM_DIRECTIONALITY, UseLmDirectionality());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_SSR, ShouldDoSSR() && IsTranslucentBlendMode(GetBlendMode()));
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_CONTACT_SHADOWS, ShouldDoContactShadows() && IsTranslucentBlendMode(GetBlendMode()));
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DITHER_OPACITY_MASK, IsDitherMasked());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS, UseNormalCurvatureToRoughness() ? 1 : 0);
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR, AllowNegativeEmissiveColor());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_OUTPUT_OPACITY_AS_ALPHA, GetBlendableOutputAlpha());
	SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY, GetCastDynamicShadowAsMasked());
	SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENT_WRITING_VELOCITY, IsTranslucencyWritingVelocity());
	SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_VELOCITY_FROM_DEPTH, IsTranslucencyWritingVelocity() && IsTranslucencyVelocityFromDepth());
	SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENT_WRITING_FRONT_LAYER_TRANSPARENCY, IsTranslucencyWritingFrontLayerTransparency());
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_USE_ALPHA_TO_COVERAGE, IsUsingAlphaToCoverage() ? 1 : 0);
	SET_SHADER_DEFINE(OutEnvironment, MATERIAL_TRANSLUCENT_PASS_AFTERMOTIONBLUR, IsTranslucencyAfterMotionBlurEnabled() ? 1 : 0);

	bool bFullPrecisionInMaterial = false;
	bool bFullPrecisionInPS = false;

	GetOutputPrecision(GetMaterialFloatPrecisionMode(), bFullPrecisionInPS, bFullPrecisionInMaterial);

	if (bFullPrecisionInMaterial)
	{
		SET_SHADER_DEFINE(OutEnvironment, FORCE_MATERIAL_FLOAT_FULL_PRECISION, 1);
	}

	OutEnvironment.FullPrecisionInPS |= bFullPrecisionInPS;

	switch(GetMaterialDomain())
	{
		case MD_Surface:				SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_SURFACE,			1); break;
		case MD_DeferredDecal:			SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_DEFERREDDECAL,	1); break;
		case MD_LightFunction:			SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_LIGHTFUNCTION,	1); break;
		case MD_Volume:					SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_VOLUME,			1); break;
		case MD_PostProcess:			SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_POSTPROCESS,		1); break;
		case MD_UI:						SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_UI,				1); break;
		default:
			UE_LOGF(LogMaterial, Warning, "Unknown material domain: %u  Setting to MD_Surface",(int32)GetMaterialDomain());
			SET_SHADER_DEFINE(OutEnvironment, MATERIAL_DOMAIN_SURFACE, 1);
	};

	if (IsTranslucentBlendMode(GetBlendMode()))
	{
		switch(GetTranslucencyLightingMode())
		{
		case TLM_VolumetricNonDirectional: SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL, 1); break;
		case TLM_VolumetricDirectional: SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL, 1); break;
		case TLM_VolumetricPerVertexNonDirectional: SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL, 1); break;
		case TLM_VolumetricPerVertexDirectional: SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL, 1); break;
		case TLM_Surface: SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME, 1); break;
		case TLM_SurfacePerPixelLighting: SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING, 1); break;

		default: 
			UE_LOGF(LogMaterial, Warning, "Unknown lighting mode: %u",(int32)GetTranslucencyLightingMode());
			SET_SHADER_DEFINE(OutEnvironment, TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL, 1); break;
		};
	}

	if( IsUsedWithEditorCompositing() )
	{
		SET_SHADER_DEFINE(OutEnvironment, EDITOR_PRIMITIVE_MATERIAL, 1);
	}

	SET_SHADER_DEFINE(OutEnvironment, USE_STENCIL_LOD_DITHER_DEFAULT, IsStencilForLODDitherEnabled(GetShaderPlatform()) != 0 ? 1 : 0);

	{
		switch (GetMaterialDomain())
		{
			case MD_Surface:		SET_SHADER_DEFINE(OutEnvironment, MATERIALDOMAIN_SURFACE, 1); break;
			case MD_DeferredDecal:	SET_SHADER_DEFINE(OutEnvironment, MATERIALDOMAIN_DEFERREDDECAL, 1); break;
			case MD_LightFunction:	SET_SHADER_DEFINE(OutEnvironment, MATERIALDOMAIN_LIGHTFUNCTION, 1); break;
			case MD_PostProcess:	SET_SHADER_DEFINE(OutEnvironment, MATERIALDOMAIN_POSTPROCESS, 1); break;
			case MD_UI:				SET_SHADER_DEFINE(OutEnvironment, MATERIALDOMAIN_UI, 1); break;
		}
	}
}
#endif // WITH_EDITOR

/**
 * Caches the material shaders for this material with no static parameters on the given platform.
 * This is used by material resources of UMaterials.
 */
bool FMaterial::CacheShaders(EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	return CacheShaders(PrecompileMode, TargetPlatform);
}

bool FMaterial::CacheShaders(EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	FAllowCachingStaticParameterValues AllowCachingStaticParameterValues(*this);
	FMaterialShaderMapId NoStaticParametersId;
	BuildShaderMapId(NoStaticParametersId, TargetPlatform);
	return CacheShaders(NoStaticParametersId, PrecompileMode, TargetPlatform);
}

/**
 * Caches the material shaders for the given static parameter set and platform.
 * This is used by material resources of UMaterialInstances.
 */
#if WITH_EDITOR
void FMaterial::BeginCacheShaders(const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback)
{
	BeginCacheShaders(ShaderMapId, PrecompileMode, TargetPlatform, MoveTemp(CompletionCallback));
}
#else
bool FMaterial::CacheShaders(const FMaterialShaderMapId & ShaderMapId, EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform * TargetPlatform)
{
	return CacheShaders(ShaderMapId, PrecompileMode, TargetPlatform);
}
#endif

#if WITH_EDITOR
void FMaterial::BeginCacheShaders(const FMaterialShaderMapId& ShaderMapId,  EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform, TUniqueFunction<void (bool bSuccess)>&& CompletionCallback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterial::BeginCacheShaders);
#else
bool FMaterial::CacheShaders(const FMaterialShaderMapId& ShaderMapId, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterial::CacheShaders);
#endif
	UE_CLOGF(!ShaderMapId.IsValid(), LogMaterial, Warning, "Invalid shader map ID caching shaders for '%ls', will use default material.", *GetFriendlyName());
#if WITH_EDITOR
	DebugGroupName = GetUniqueAssetName(ShaderMapId) / LexToString(GetQualityLevel());

	FString DDCKeyHash;

	// Just make sure that we don't already have a pending cache going on.
	FinishCacheShaders();
	
	bool bBeginFoundCompiling = false;
#endif // WITH_EDITOR

	// If we loaded this material with inline shaders, use what was loaded (GameThreadShaderMap) instead of looking in the DDC
	if (bContainsInlineShaders)
	{
		TRefCountPtr<FMaterialShaderMap> ExistingShaderMap = nullptr;
		
		if (GameThreadShaderMap)
		{
			// Note: in the case of an inlined shader map, the shadermap Id will not be valid because we stripped some editor-only data needed to create it
			// Get the shadermap Id from the shadermap that was inlined into the package, if it exists
			ExistingShaderMap = FMaterialShaderMap::FindId(GameThreadShaderMap->GetShaderMapId(), GetShaderPlatform());
		}

		// Re-use an identical shader map in memory if possible, removing the reference to the inlined shader map
		if (ExistingShaderMap)
		{
			SetGameThreadShaderMap(ExistingShaderMap);
		}
		else if (GameThreadShaderMap)
		{
			// We are going to use the inlined shader map, register it so it can be re-used by other materials
			UpdateInlineShaderMapIsComplete();
			GameThreadShaderMap->Register(GetShaderPlatform());
		}
	}
	else
	{
#if WITH_EDITOR
		if (AllowShaderCompiling())
		{
			TRefCountPtr<FMaterialShaderMap> ShaderMap = FMaterialShaderMap::FindId(ShaderMapId, GetShaderPlatform());
			if (ShaderMap)
			{
				// another material has registered this shader map and its compilation is in-progress
				// we only need to ensure it contains all shaders required for this material
				if (ShaderMap->GetCompilingId() != 0u)
				{
					SetCompilingShaderMap(ShaderMap);
					ShaderMap = ShaderMap->GetFinalizedClone();
					bBeginFoundCompiling = true;
				}
			}

			// If we are loading individual shaders from the shader job cache don't attempt to load full maps.
			const bool bSkipCompilationOnPostLoad = IsMaterialMapDDCEnabled() == false;

			// Attempt to load from the derived data cache if we are uncooked and don't have any shadermap.
			// If we have an incomplete shadermap, continue with it to prevent creation of duplicate shadermaps for the same ShaderMapId
			if (!ShaderMap && !FPlatformProperties::RequiresCookedData())
			{
				if (bSkipCompilationOnPostLoad == false || IsRequiredComplete())
				{
					TRefCountPtr<FMaterialShaderMap> LoadedShaderMap;
					CacheShadersPending = FMaterialShaderMap::BeginLoadFromDerivedDataCache(this, ShaderMapId, GetShaderPlatform(), TargetPlatform, LoadedShaderMap, DDCKeyHash);
				}
			}

			check(!ShaderMap || ShaderMap->GetFrozenContentSize() > 0u);
			SetGameThreadShaderMap(ShaderMap);
		}
#endif // WITH_EDITOR
	}

	// In editor, we split the function in half with the remaining to be called as part of the 
	// FinishCacheShaders once the DDC call initiated in BeginLoadFromDerivedDataCache above has finished.
	// For client builds, this is executed in place without any lambda.
#if WITH_EDITOR
	CacheShadersCompletion = [this, ShaderMapId, DDCKeyHash, bBeginFoundCompiling, PrecompileMode, TargetPlatform, CompletionCallback = MoveTemp(CompletionCallback)]() {

	ON_SCOPE_EXIT{ CacheShadersCompletion.Reset(); };
	
	bool bFoundCompiling = bBeginFoundCompiling;
	if (!GameThreadShaderMap)
	{
		TRefCountPtr<FMaterialShaderMap> ShaderMap;
		if (CacheShadersPending.IsValid()) // we started a load above, check the result
		{
			ShaderMap = CacheShadersPending->Get();
			CacheShadersPending.Reset();
		}
		
		if (!ShaderMap) // if we still don't have a shader map it wasn't initially in the inprocess cache and also was not in the DDC
		{
			// we need to check again if another material has created, registered and began compilation on the shader map we need
			// since our previous call to FMaterialShaderMap::FindId in BeginCacheShaders
			// this can occur if multiple materials referencing the same shadermap get a BeginCacheShaders call in the same tick
			ShaderMap = FMaterialShaderMap::FindId(ShaderMapId, GetShaderPlatform());
			if (ShaderMap)
			{
				// as above, it's possible (and in this case likely) that if we found a shader map that its compilation is already
				// in progress, triggered by another material being processed in this tick. similarly we need to check that it contains
				// all shaders required for this material (and queue compilation for any that are missing).
				if (ShaderMap->GetCompilingId() != 0u)
				{
					SetCompilingShaderMap(ShaderMap);
					ShaderMap = ShaderMap->GetFinalizedClone();
					bFoundCompiling = true;
				}
			}
		}
		check(!ShaderMap || ShaderMap->GetFrozenContentSize() > 0u);
		SetGameThreadShaderMap(ShaderMap);
	}

	// some of the above paths did not mark the shader map as associated with an asset, do so
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap->AssociateWithAsset(GetAssetPath());
	}
#endif

	UMaterialInterface* MaterialInterface = GetMaterialInterface();
	const bool bMaterialInstance = MaterialInterface && MaterialInterface->IsA(UMaterialInstance::StaticClass());
	const bool bRequiredComplete = !bMaterialInstance && IsRequiredComplete();
	

	bool bShaderMapValid = (bool)GameThreadShaderMap;
#if WITH_EDITOR
	if (bShaderMapValid && bRequiredComplete && !bFoundCompiling)
#else
	if (bShaderMapValid && bRequiredComplete)
#endif
	{
		// Special engine materials (default materials) are required to be complete
		// We can bypass this check in the case where we found a "required complete" shader map whose compilation is in progress;
		// we will check if it's complete below and queue any jobs necessary if not (we only need to log warnings if we found a map without
		// compilation in progress that is incomplete).
		bool bAssumeShaderMapIsComplete = false;
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
		bAssumeShaderMapIsComplete = FPlatformProperties::RequiresCookedData();
#endif
		bShaderMapValid = bAssumeShaderMapIsComplete || GameThreadShaderMap->IsComplete(this, false);
	}

	if (!bShaderMapValid)
	{
		// if we can't compile shaders, fall into the requires cooked path
		if (bContainsInlineShaders || FPlatformProperties::RequiresCookedData() || !AllowShaderCompiling())
		{
			if (bRequiredComplete)
			{
				UMaterialInterface* Interface = GetMaterialInterface();
				FString Instance;
				if (Interface)
				{
					Instance = Interface->GetPathName();
				}

				//assert if the default material's shader map was not found, since it will cause problems later
				UE_LOGF(LogMaterial, Fatal,"Failed to find shader map for default material %ls(%ls)! Please make sure cooking was successful (%ls inline shaders, %ls GTSM)",
					*GetFriendlyName(),
					*Instance,
					bContainsInlineShaders ? TEXT("Contains") : TEXT("No"),
					GameThreadShaderMap ? TEXT("has") : TEXT("null")
				);
			}
			else
			{
				UE_LOGF(LogMaterial, Log, "Can't compile %ls with cooked content, will use default material instead", *GetFriendlyName());
			}

			// Reset the shader map so the default material will be used.
			SetGameThreadShaderMap(nullptr);
		}
		else
		{
			const bool bSkipCompilationForODSC = !RequiresSynchronousCompilation() && (GShaderCompilingManager->IsShaderCompilationSkipped() || (IsMaterialMapDDCEnabled() == false));
			// If we aren't actually generating shadermaps, don't print the debug message that we are generating shadermaps.
			if (!bSkipCompilationForODSC)
			{
				const TCHAR* ShaderMapCondition;
				if (GameThreadShaderMap)
				{
					ShaderMapCondition = TEXT("Incomplete");
				}
				else
				{
					ShaderMapCondition = TEXT("Missing");
				}
#if WITH_EDITOR
				FString ShaderPlatformName = FGenericDataDrivenShaderPlatformInfo::GetName(GetShaderPlatform()).ToString();
				UE_LOGF(LogMaterial, Display, "%ls cached shadermap for %ls in %ls, %ls, %ls, %ls (DDC key hash: %ls), compiling. %ls",
					ShaderMapCondition,
					*GetFullPath(),
					*ShaderPlatformName,
					*LexToString(ShaderMapId.QualityLevel),
					*LexToString(ShaderMapId.FeatureLevel),
					ShaderMapId.LayoutParams.WithEditorOnly() ? TEXT("Editor") : TEXT("Game"),
					*DDCKeyHash,
					IsSpecialEngineMaterial() ? TEXT("Is special engine material.") : TEXT("")
				);
#else
				UE_LOGF(LogMaterial, Display, "%ls cached shader map for material %ls, compiling. %ls",
					ShaderMapCondition,
					*GetAssetName(),
					IsSpecialEngineMaterial() ? TEXT("Is special engine material.") : TEXT("")
				);
#endif
			}

#if WITH_EDITORONLY_DATA
			FStaticParameterSet StaticParameterSet;
			GetStaticParameterSet(StaticParameterSet);

			// If there's no cached shader map for this material, compile a new one.
			// This is just kicking off the async compile, GameThreadShaderMap will not be complete yet
			bShaderMapValid = BeginCompileShaderMap(ShaderMapId, StaticParameterSet, PrecompileMode, TargetPlatform);
#endif // WITH_EDITORONLY_DATA

			if (!bShaderMapValid)
			{
				// If it failed to compile the material, reset the shader map so the material isn't used.
				SetGameThreadShaderMap(nullptr);

#if WITH_EDITOR
				if (IsDefaultMaterial())
				{
					for (int32 ErrorIndex = 0; ErrorIndex < CompileErrors.Num(); ErrorIndex++)
					{
						// Always log material errors in an unsuppressed category
						UE_LOGF(LogMaterial, Warning, "	%ls", *CompileErrors[ErrorIndex]);
					}

					// Assert if the default material could not be compiled, since there will be nothing for other failed materials to fall back on.
					if (AreShaderErrorsFatal())
					{
						UE_LOGF(LogMaterial, Fatal, "Failed to compile default material %ls!", *GetFriendlyName());
					}
					else
					{
						UE_LOGF(LogMaterial, Error, "Failed to compile default material %ls!", *GetFriendlyName());
					}
				}
#endif // WITH_EDITOR
			}
		}
	}
	else
	{
#if WITH_EDITOR
		// We have a shader map, the shader map is incomplete, and we've been asked to compile.
		if (AllowShaderCompiling() && 
			!IsGameThreadShaderMapComplete() && 
			(PrecompileMode != EMaterialShaderPrecompileMode::None))
		{
			// Submit the remaining shaders in the map for compilation.
			SubmitCompileJobs_GameThread(EShaderCompileJobPriority::High);
		}
		else
		{
			// Clear outdated compile errors as we're not calling Translate on this path
			CompileErrors.Empty();
			ErrorExpressions.Empty();
		}
#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	if (CompletionCallback)
	{
		CompletionCallback(bShaderMapValid);
	}
#endif
	return bShaderMapValid;

#if WITH_EDITOR
}; // Close the lambda
#endif
}

#if WITH_EDITOR
// Deprecated in 5.7
FString FMaterial::GetUniqueAssetName(EShaderPlatform Platform, const FMaterialShaderMapId& ShaderMapId) const
{
	return GetUniqueAssetName(ShaderMapId);
}

FString FMaterial::GetUniqueAssetName(const FMaterialShaderMapId& ShaderMapId) const
{
	FXxHash64Builder Hasher;
	// Construct a hash representing the material shadermap ID key, but excluding source code hashes and material function/parameter 
	// collection guids, such that this hash remains stable when edits to this data are applied (including source version bumps)
	FMaterialKeyGeneratorContext KeyGenCtx([&Hasher](const void* Data, uint64 Size) { Hasher.Update(Data, Size); }, GetShaderPlatform());
	KeyGenCtx.RemoveFlags(EMaterialKeyInclude::SourceAndMaterialState | EMaterialKeyInclude::Globals | EMaterialKeyInclude::ShaderDependencies);
	const_cast<FMaterialShaderMapId&>(ShaderMapId).RecordAndEmit(KeyGenCtx);
	// Hash the base material path as well to differentiate materials with the same name and different paths.
	// note we explicitly _do not_ use the path of the asset itself as if the asset is a material instance we 
	// want it to properly deduplicate against other instances which might end up pointing to the same shadermap.
	FString BaseMaterialPath = GetBaseMaterialPathName();
	Hasher.Update(BaseMaterialPath.GetCharArray().GetData(), BaseMaterialPath.Len() * sizeof(TCHAR));
	return FString::Printf(TEXT("%s_%llx"), *GetFriendlyName(), Hasher.Finalize().Hash);
}
#endif // WITH_EDITOR

FCriticalSection FMaterial::PrecachedPSORequestIDsCS;

// Deprecated 5.7
FGraphEventArray FMaterial::CollectPSOs(ERHIFeatureLevel::Type InFeatureLevel, const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs)
{
	return CollectPSOs(VertexFactoryDataList, PreCacheParams, Priority, OutMaterialPSORequestIDs);
}

FGraphEventArray FMaterial::CollectPSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterial::CollectPSOs);
	
	FGraphEventArray GraphEvents;
	if (GameThreadShaderMap == nullptr)
	{
		return GraphEvents;
	}

	for (const FPSOPrecacheVertexFactoryData& VFData : VertexFactoryDataList)
	{
		if (!VFData.VertexFactoryType->SupportsPSOPrecaching())
		{
			continue;
		}

		FMaterialPSOPrecacheParams Params;
		Params.FeatureLevel = FeatureLevel;
		Params.Material = this;
		Params.VertexFactoryData = VFData;
		Params.PrecachePSOParams = PreCacheParams;

		FMaterialPSOPrecacheRequestID RequestID = PrecacheMaterialPSOs(Params, Priority, GraphEvents);
		if (RequestID != INDEX_NONE)
		{
			OutMaterialPSORequestIDs.AddUnique(RequestID);

			// Verified in game thread above
			FScopeLock ScopeLock(&PrecachedPSORequestIDsCS);
			PrecachedPSORequestIDs.AddUnique(RequestID);
		}
	}
	return GraphEvents;
}

TArray<FMaterialPSOPrecacheRequestID> FMaterial::GetMaterialPSOPrecacheRequestIDs() const
{
	TArray<FMaterialPSOPrecacheRequestID> TmpPrecachedPSORequestIDs;
	{
		FScopeLock ScopeLock(&PrecachedPSORequestIDsCS);
		TmpPrecachedPSORequestIDs = PrecachedPSORequestIDs;
	}
	return TmpPrecachedPSORequestIDs;
}

void FMaterial::ClearPrecachedPSORequestIDs()
{
	FScopeLock ScopeLock(&PrecachedPSORequestIDsCS);
	PrecachedPSORequestIDs.Empty();
}

#if WITH_EDITOR

// Deprecated in 5.7
void FMaterial::BeginCacheShaders(EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback)
{
	BeginCacheShaders(PrecompileMode, TargetPlatform, MoveTemp(CompletionCallback));
}

void FMaterial::BeginCacheShaders(EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback)
{
	FAllowCachingStaticParameterValues AllowCachingStaticParameterValues(*this);
	FMaterialShaderMapId NoStaticParametersId;
	BuildShaderMapId(NoStaticParametersId, TargetPlatform);
	return BeginCacheShaders(NoStaticParametersId, PrecompileMode, TargetPlatform, MoveTemp(CompletionCallback));
}

bool FMaterial::IsCachingShaders() const
{
	return CacheShadersCompletion || CacheShadersPending.IsValid();
}

bool FMaterial::FinishCacheShaders() const
{
	COOK_STAT(FScopedDurationTimer BlockingTimer(MaterialSharedCookStats::FinishCacheShadersSec));

	if (CacheShadersCompletion)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FinishCacheShaders);

		return CacheShadersCompletion();
	}

	return false;
}

// Deprecated in 5.7
bool FMaterial::CacheShaders(const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	return CacheShaders(ShaderMapId, PrecompileMode, TargetPlatform);
}

bool FMaterial::CacheShaders(const FMaterialShaderMapId& ShaderMapId, EMaterialShaderPrecompileMode PrecompileMode, const ITargetPlatform* TargetPlatform)
{
	BeginCacheShaders(ShaderMapId, PrecompileMode, TargetPlatform);

	return FinishCacheShaders();
}

// Deprecated in 5.7
void FMaterial::CacheGivenTypes(EShaderPlatform Platform, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*>& PipelineTypes, const TArray<const FShaderType*>& ShaderTypes)
{
	CacheGivenTypes(VFTypes, PipelineTypes, ShaderTypes);
}

void FMaterial::CacheGivenTypes(const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*>& PipelineTypes, const TArray<const FShaderType*>& ShaderTypes)
{
	if (CompileErrors.Num())
	{
		UE_LOGF(LogMaterial, Warning, "Material failed to compile.");
		for (const FString& CompileError : CompileErrors)
		{
			UE_LOGF(LogMaterial, Warning, "%ls", *CompileError);
		}

		return;
	}

	if (bGameThreadShaderMapIsComplete.load(std::memory_order_relaxed))
	{
		UE_LOGF(LogMaterial, Verbose, "Cache given types for a material resource %ls with a complete ShaderMap", *GetFriendlyName());
		return;
	}

	if (GameThreadShaderMap)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterial::CacheGivenTypes);
		check(IsInGameThread());
		checkf(ShaderTypes.Num() == VFTypes.Num(), TEXT("The size of the shader type array and vertex factory type array must match."));
		checkf(PipelineTypes.Num() == ShaderTypes.Num(), TEXT("The size of the pipeline type array and shader type array must match.  Pass in null entries if pipelines are not used."));
		checkf(GetGameThreadCompilingShaderMapId() != 0, TEXT("Material is not prepared to compile yet.  Please call CacheShaders first."));

		TArray<FShaderCommonCompileJobPtr> CompileJobs;
		for (int i = 0; i < VFTypes.Num(); ++i)
		{
			const FVertexFactoryType* VFType = VFTypes[i];
			const FShaderPipelineType* PipelineType = PipelineTypes[i];
			const FShaderType* ShaderType = ShaderTypes[i];

			if (PipelineType)
			{
				FMeshMaterialShaderType::BeginCompileShaderPipeline(
					EShaderCompileJobPriority::ForceLocal,
					GetGameThreadCompilingShaderMapId(),
					0,
					GetShaderPlatform(),
					GameThreadShaderMap->GetPermutationFlags(),
					this,
					GameThreadShaderMap->GetCompilationOutput(),
					GameThreadPendingCompilerEnvironment,
					VFType,
					PipelineType,
					CompileJobs,
					GetDebugGroupName(),
					nullptr,
					nullptr);
			}
			else if (ShaderType->GetTypeForDynamicCast() == FShaderType::EShaderTypeForDynamicCast::Material)
			{
				const EShaderPermutationFlags ShaderPermutation = GameThreadShaderMap->GetPermutationFlags();
				const FMaterialCompilationOutput& CompilationOutput = GameThreadShaderMap->GetCompilationOutput();

				const uint32 CompilingShaderMapId = GetGameThreadCompilingShaderMapId();
				const FMaterialShaderType* MaterialShaderType = ShaderType->GetMaterialShaderType();

				for (int32 PermutationId = 0; PermutationId < ShaderType->GetPermutationCount(); ++PermutationId)
				{
					const bool bShaderShouldCompile = MaterialShaderType->ShouldCompilePermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), PermutationId, ShaderPermutation);
					if (!bShaderShouldCompile)
					{
						continue;
					}

					MaterialShaderType->BeginCompileShader(
						EShaderCompileJobPriority::ForceLocal,
						CompilingShaderMapId,
						PermutationId,
						this,
						GameThreadShaderMap->GetCompilationOutput(),
						GameThreadPendingCompilerEnvironment,
						GetShaderPlatform(),
						ShaderPermutation,
						CompileJobs,
						DebugGroupName,
						nullptr,
						nullptr);
				}
			}
			else if (ShaderType->GetTypeForDynamicCast() == FShaderType::EShaderTypeForDynamicCast::MeshMaterial)
			{
				const EShaderPermutationFlags ShaderPermutation = GameThreadShaderMap->GetPermutationFlags();
				const FMaterialCompilationOutput& CompilationOutput = GameThreadShaderMap->GetCompilationOutput();

				const FMeshMaterialShaderType* MeshMaterialShaderType = ShaderType->GetMeshMaterialShaderType();

				for (int32 PermutationId = 0; PermutationId < ShaderType->GetPermutationCount(); ++PermutationId)
				{
					const bool bShaderShouldCompile = MeshMaterialShaderType->ShouldCompilePermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), VFType, PermutationId, ShaderPermutation);
					if (!bShaderShouldCompile)
					{
						continue;
					}

					ShaderType->AsMeshMaterialShaderType()->BeginCompileShader(
						EShaderCompileJobPriority::ForceLocal,
						GetGameThreadCompilingShaderMapId(),
						PermutationId,
						GetShaderPlatform(),
						ShaderPermutation,
						this,
						GameThreadShaderMap->GetCompilationOutput(),
						GameThreadPendingCompilerEnvironment,
						VFType,
						CompileJobs,
						DebugGroupName,
						nullptr,
						nullptr);
				}
			}
		}

		GShaderCompilingManager->SubmitJobs(CompileJobs, GetBaseMaterialPathName(), GameThreadShaderMap->GetDebugDescription());
	}
}

bool FMaterial::Translate_Legacy(const FMaterialShaderMapId& ShaderMapId,
	const FStaticParameterSet& InStaticParameters,
	const ITargetPlatform* InTargetPlatform,
	FMaterialCompilationOutput& OutCompilationOutput,
	TRefCountPtr<FSharedShaderCompilerEnvironment>& OutMaterialEnvironment)
{
	FString MaterialTranslationDDCKeyString = GetMaterialShaderMapKeyString(ShaderMapId, FConservativeShaderParameters(this), GetShaderPlatform(), false);
	FHLSLMaterialTranslator MaterialTranslator(this, OutCompilationOutput, InStaticParameters, GetShaderPlatform(), GetQualityLevel(), ShaderMapId.FeatureLevel, InTargetPlatform, &ShaderMapId.SubstrateCompilationConfig, MaterialTranslationDDCKeyString);
	EHLSLMaterialTranslatorResult Result = MaterialTranslator.Translate(false);

	// If the DDC result was invalid we need to invoke translation again turning the DDC off.
	if (Result == EHLSLMaterialTranslatorResult::RetryWithoutDDC)
	{
		// FHLSLMaterialTranslator is designed as single use. After a call to Translate() no other calls are allowed.
		// Destruct the current instance and create a new one before translating the material again, forcing the translator
		// to translate the material instead of accessing the DDC cache.
		OutCompilationOutput = {};
		MaterialTranslator.~FHLSLMaterialTranslator();
		new (&MaterialTranslator) FHLSLMaterialTranslator(this, OutCompilationOutput, InStaticParameters, GetShaderPlatform(), GetQualityLevel(), ShaderMapId.FeatureLevel, InTargetPlatform, &ShaderMapId.SubstrateCompilationConfig, MaterialTranslationDDCKeyString);
		Result = MaterialTranslator.Translate(true);
	}

	if (Result != EHLSLMaterialTranslatorResult::Success)
	{
		return false;
	}

	// Create a shader compiler environment for the material that will be shared by all jobs from this material
	OutMaterialEnvironment = new FSharedShaderCompilerEnvironment();
	OutMaterialEnvironment->TargetPlatform = InTargetPlatform;
	MaterialTranslator.GetMaterialEnvironment(GetShaderPlatform(), *OutMaterialEnvironment);

	// Add generated HLSL shader code to virtual include map to be included by the respective base shader (e.g. BasePassPixelShader.usf)
	FString MaterialShaderCode = MaterialTranslator.GetMaterialShaderCode();
	OutMaterialEnvironment->IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/Material.ush"), MoveTemp(MaterialShaderCode));
	
	return true;
}

static void EmitDebugInfoComment(const FMaterialInsights& Insights, FString& ShaderCode)
{
	ShaderCode.Append("\n/* INSIGHTS\n");
	ShaderCode.Append("Uniform Buffer Content:\n");

	if (Insights.UniformParameterAllocationInsights.IsEmpty()) {
		ShaderCode.Append(TEXT("\tNo uniform parameters used by the material\n"));
	}
	else {
		for (auto& ParamInsight : Insights.UniformParameterAllocationInsights)
		{
			ShaderCode.Appendf(TEXT(" - UniformBuffer[%d]."), ParamInsight.BufferSlotIndex);
			for (uint32 i = 0; i < ParamInsight.ComponentsCount; ++i)
			{
				check(ParamInsight.BufferSlotOffset + i < 4);
				ShaderCode.AppendChar(TEXT("xyzw")[ParamInsight.BufferSlotOffset + i]);
			}
			ShaderCode.Appendf(TEXT(" = %s."), *ParamInsight.ParameterName.ToString());
			for (uint32 i = 0; i < ParamInsight.ComponentsCount; ++i)
			{
				check(i < 4);
				ShaderCode.AppendChar(TEXT("xyzw")[i]);
			}
			ShaderCode.AppendChar('\n');
		}
	}

	ShaderCode.Append("*/\n");
}

bool FMaterial::Translate_New(const FMaterialShaderMapId& InShaderMapId,
	const FStaticParameterSet& InStaticParameters,
	const ITargetPlatform* InTargetPlatform,
	FMaterialCompilationOutput& OutCompilationOutput,
	TRefCountPtr<FSharedShaderCompilerEnvironment>& OutMaterialEnvironment)
{
	UE_LOGF(LogMaterial, Display, "Translating material '%ls' with the new experimental material translator.", *GetMaterialInterface()->GetFullName());

	// Clear existing Material Errors.
	CompileErrors.Empty();
	ErrorExpressions.Empty();

	FMaterialIRModule Module;

	// Build the material
	FMaterialIRModuleBuilder Builder = {
		.MaterialInterface = GetMaterialInterface(),
		.ShaderPlatform = GetShaderPlatform(),
		.TargetPlatform = InTargetPlatform,
		.FeatureLevel = InShaderMapId.FeatureLevel,
		.QualityLevel = InShaderMapId.QualityLevel,
		.BlendMode = GetBlendMode(),
		.StaticParameters = &InStaticParameters,
		.TargetInsights = GetMaterialInterface()->MaterialInsight.Get(),
		.PreviewExpression = GetMaterialGraphNodePreviewExpression()
	};

	if (!Builder.Build(&Module))
	{
		for (const FMaterialIRModule::FError& Error : Module.GetErrors())
		{
			ErrorExpressions.Push(Error.Expression);
			CompileErrors.Push(Error.Message);
		}

		return false;
	}
	
	// Copy over the compilation output
	OutCompilationOutput = Module.GetCompilationOutput();
	OutMaterialEnvironment = new FSharedShaderCompilerEnvironment();

	// Translate the material IR module to HLSL template string parameters and material environment
	TMap<FString, FString> ShaderStringParameters;

	FMaterialIRToHLSLTranslation Translation{
		.Module = &Module,
		.Material = this,
		.StaticParameters = &InStaticParameters,
		.TargetPlatform = InTargetPlatform,
	};

	Translation.Run(ShaderStringParameters, *OutMaterialEnvironment);

	// Interpolate HLSL parameters with the material shader template to produce the final shader source
	int32 LineNumber;
	FStringTemplateResolver Resolver = FMaterialSourceTemplate::Get().BeginResolve(GetShaderPlatform(), &LineNumber);
	ShaderStringParameters.Add({TEXT("line_number"), FString::Printf(TEXT("%u"), LineNumber)});
	Resolver.SetParameterMap(&ShaderStringParameters);

	// Interpolate the final material shader source string
	FString MaterialShaderCode = Resolver.Finalize();

	// Emit uniform data debug information to the end of the generated shader
	EmitDebugInfoComment(*GetMaterialInterface()->MaterialInsight, MaterialShaderCode);

	GetMaterialInterface()->MaterialInsight->New_ShaderStringParameters = ShaderStringParameters;

	OutMaterialEnvironment->IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/Material.ush"), MoveTemp(MaterialShaderCode));
	
	return true;
}

bool FMaterial::Translate(const FMaterialShaderMapId& InShaderMapId,
	const FStaticParameterSet& InStaticParameters,
	const ITargetPlatform* InTargetPlatform,
	FMaterialCompilationOutput& OutCompilationOutput,
	TRefCountPtr<FSharedShaderCompilerEnvironment>& OutMaterialEnvironment)
{
	// Reset the material insights instance if this is a valid material interface.
	if (UMaterialInterface* Interface = GetMaterialInterface())
	{
		Interface->MaterialInsight.Reset(new FMaterialInsights);
	}

	// Use the new translator if the shader map uses it.
	if (InShaderMapId.bUsingNewHLSLGenerator && !bEnableRestrictiveMaterialMode)
	{
		return Translate_New(InShaderMapId, InStaticParameters, InTargetPlatform, OutCompilationOutput, OutMaterialEnvironment);
	}
	else
	{
		return Translate_Legacy(InShaderMapId, InStaticParameters, InTargetPlatform, OutCompilationOutput, OutMaterialEnvironment);
	}
}

/**
* Compiles this material for Platform
*
* @param ShaderMapId - the set of static parameters to compile
* @param Platform - the platform to compile for
* @param StaticParameterSet - static parameters
* @return - true if compile succeeded or was not necessary (shader map for ShaderMapId was found and was complete)
*/
bool FMaterial::BeginCompileShaderMap(
	const FMaterialShaderMapId& ShaderMapId, 
	const FStaticParameterSet &StaticParameterSet,
	EMaterialShaderPrecompileMode PrecompileMode,
	const ITargetPlatform* TargetPlatform)
{
	bool bSuccess = false;

	STAT(double MaterialCompileTime = 0);

	TRefCountPtr<FMaterialShaderMap> NewShaderMap = new FMaterialShaderMap();

	SCOPE_SECONDS_COUNTER(MaterialCompileTime);

	NewShaderMap->AssociateWithAsset(GetAssetPath());

	// Generate the material shader code.
	FMaterialCompilationOutput NewCompilationOutput;
	TRefCountPtr<FSharedShaderCompilerEnvironment> MaterialEnvironment;
	bSuccess = Translate(ShaderMapId, StaticParameterSet, TargetPlatform, NewCompilationOutput, MaterialEnvironment);

	if(bSuccess)
	{
		FShaderCompileUtilities::GenerateBrdfHeaders((EShaderPlatform)GetShaderPlatform());
		FShaderCompileUtilities::ApplyDerivedDefines(*MaterialEnvironment, nullptr, (EShaderPlatform)GetShaderPlatform());

		{
			// Get this material's override floating point precision setting
			EMaterialFloatPrecisionMode MaterialFloatPrecisionMode = EMaterialFloatPrecisionMode::MFPM_Default;
			const UMaterialInterface* MaterialInterface = GetMaterialInterface();
			if (MaterialInterface)
			{
				MaterialFloatPrecisionMode = MaterialInterface->GetMaterial()->FloatPrecisionMode;
			}

			// Sets the floating point precision to be used for the PreshaderBuffer uniform. This is controlled via this material's floating point override set in the material editor
			EShaderPrecisionModifier::Type ParameterPrecision = (IsMobilePlatform((EShaderPlatform)GetShaderPlatform()) && MaterialFloatPrecisionMode == MFPM_Half_WithShaderParameters) ? EShaderPrecisionModifier::Half : EShaderPrecisionModifier::Float;
			TArray<FShaderParametersMetadata*> UniformBufferStructs = NewCompilationOutput.UniformExpressionSet.CreateBufferStructs(ParameterPrecision);

			SetupMaterialEnvironment(UniformBufferStructs, NewCompilationOutput.UniformExpressionSet, *MaterialEnvironment);
			for (FShaderParametersMetadata* UniformBufferStruct : UniformBufferStructs)
			{
				delete UniformBufferStruct;
			}
		}

		// we can ignore requests for synch compilation if we are compiling for a different platform than we're running, or we're a commandlet that doesn't render (e.g. cooker)
		const bool bCanIgnoreSynchronousRequirement = (TargetPlatform && !TargetPlatform->IsRunningPlatform()) || (IsRunningCommandlet() && !IsAllowCommandletRendering());
		const bool bSkipCompilationForODSC = !RequiresSynchronousCompilation() && GShaderCompilingManager->IsShaderCompilationSkipped();
		if (bSkipCompilationForODSC)
		{
			// Force compilation off.
			PrecompileMode = EMaterialShaderPrecompileMode::None;
		}
		else if (!bCanIgnoreSynchronousRequirement && RequiresSynchronousCompilation())
		{
			// Force sync compilation by material
			PrecompileMode = EMaterialShaderPrecompileMode::Synchronous;
		}
		else if (!GShaderCompilingManager->AllowAsynchronousShaderCompiling() && PrecompileMode != EMaterialShaderPrecompileMode::None)
		{
			// No support for background async compile
			PrecompileMode = EMaterialShaderPrecompileMode::Synchronous;
		}
		// Compile the shaders for the material.
		NewShaderMap->Compile(this, ShaderMapId, MaterialEnvironment, NewCompilationOutput, GetShaderPlatform(), PrecompileMode);

		// early in the startup we can save some time by compiling all special/default materials asynchronously, even if normally they are synchronous
		if (PrecompileMode == EMaterialShaderPrecompileMode::Synchronous && !PoolSpecialMaterialsCompileJobs())
		{
			// If this is a synchronous compile, assign the compile result to the output
			check(NewShaderMap->GetCompilingId() == 0u);
			if (NewShaderMap->CompiledSuccessfully())
			{
				NewShaderMap->FinalizeContent();
				SetGameThreadShaderMap(NewShaderMap);
			}
			else
			{
				SetGameThreadShaderMap(nullptr);
			}
		}
		else if (PrecompileMode == EMaterialShaderPrecompileMode::None && bSkipCompilationForODSC)
		{
			// We didn't perform a compile so do ODSC specific cleanup here.
			ReleaseGameThreadCompilingShaderMap();
			ReleaseRenderThreadCompilingShaderMap();

			NewShaderMap->ReleaseCompilingId();
			check(NewShaderMap->GetCompilingId() == 0u);

			// Tell the map it was successful even though we didn't compile shaders into.
			// This ensures the map will be saved and cooked out.
			NewShaderMap->SetCompiledSuccessfully(true);

			// We didn't compile any shaders but still assign the result
			NewShaderMap->FinalizeContent();
			SetGameThreadShaderMap(NewShaderMap);
		}
		else
		{
			SetGameThreadShaderMap(NewShaderMap->AcquireFinalizedClone());
		}
	}

	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_MaterialCompiling,(float)MaterialCompileTime);
	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_MaterialShaders,(float)MaterialCompileTime);

	return bSuccess;
}

#endif // WITH_EDITOR

/**
 * Should the shader for this material with the given platform, shader type and vertex 
 * factory type combination be compiled
 *
 * @param ShaderType	Which shader is being compiled
 * @param VertexFactory	Which vertex factory is being compiled (can be NULL)
 *
 * @return true if the shader should be compiled
 */
// Deprecated 5.7
bool FMaterial::ShouldCache(EShaderPlatform Platform, const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const
{
	return ShouldCache(ShaderType, VertexFactoryType);
}

bool FMaterial::ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const
{
	return true;
}

// Deprecated in 5.7
bool FMaterial::ShouldCachePipeline(EShaderPlatform Platform, const FShaderPipelineType* PipelineType, const FVertexFactoryType* VertexFactoryType) const
{
	return ShouldCachePipeline(PipelineType, VertexFactoryType);
}

bool FMaterial::ShouldCachePipeline(const FShaderPipelineType* PipelineType, const FVertexFactoryType* VertexFactoryType) const
{
	for (const FShaderType* ShaderType : PipelineType->GetStages())
	{
		if (!ShouldCache(ShaderType, VertexFactoryType))
		{
			return false;
		}
	}

	// Only include the pipeline if all shaders should be cached
	return true;
}

/**
 * Finds the shader matching the template type and the passed in vertex factory, asserts if not found.
 */
TShaderRef<FShader> FMaterial::GetShader(FMeshMaterialShaderType* ShaderType, FVertexFactoryType* VertexFactoryType, int32 PermutationId, bool bFatalIfMissing) const
{
#if WITH_EDITOR && DO_CHECK
	// Attempt to get some more info for a rare crash (UE-35937)
	FMaterialShaderMap* GameThreadShaderMapPtr = GameThreadShaderMap;
	checkf( RenderingThreadShaderMap, TEXT("RenderingThreadShaderMap was NULL (GameThreadShaderMap is %p). This may relate to bug UE-35937"), GameThreadShaderMapPtr );
#endif
	const FMeshMaterialShaderMap* MeshShaderMap = RenderingThreadShaderMap->GetMeshShaderMap(VertexFactoryType);
	FShader* Shader = MeshShaderMap ? MeshShaderMap->GetShader(ShaderType, PermutationId) : nullptr;
	if (!Shader)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterial::GetShader);

		if (bFatalIfMissing)
		{
			auto noinline_lambda = [&](...) FORCENOINLINE
			{
				// we don't care about thread safety because we are about to crash 
				const auto CachedGameThreadShaderMap = GameThreadShaderMap;
				const auto CachedGameMeshShaderMap = CachedGameThreadShaderMap ? CachedGameThreadShaderMap->GetMeshShaderMap(VertexFactoryType) : nullptr;
				bool bShaderWasFoundInGameShaderMap = CachedGameMeshShaderMap && CachedGameMeshShaderMap->GetShader(ShaderType, PermutationId) != nullptr;
				const FMaterialCompilationOutput& CompilationOutput = RenderingThreadShaderMap->GetCompilationOutput();

				// Get the ShouldCache results that determine whether the shader should be compiled
				auto ShaderPermutation = RenderingThreadShaderMap->GetPermutationFlags();
				bool bMaterialShouldCache = ShouldCache(ShaderType, VertexFactoryType);
				bool bVFShouldCache = FMeshMaterialShaderType::ShouldCompileVertexFactoryPermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), VertexFactoryType, ShaderType, ShaderPermutation);
				bool bShaderShouldCache = ShaderType->ShouldCompilePermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), VertexFactoryType, PermutationId, ShaderPermutation);
				FString MaterialUsage = GetMaterialUsageDescription();

				int BreakPoint = 0;

				// Assert with detailed information if the shader wasn't found for rendering.  
				// This is usually the result of an incorrect ShouldCache function.
				UE_LOGF(LogMaterial, Error,
					"Couldn't find Shader (%ls, %d) for Material Resource %ls!\n" "		RenderMeshShaderMap %d, RenderThreadShaderMap %d\n" "		GameMeshShaderMap %d, GameThreadShaderMap %d, bShaderWasFoundInGameShaderMap %d\n" "		With VF=%ls, Platform=%ls\n" "		ShouldCache: Mat=%u, VF=%u, Shader=%u \n" "		MaterialUsageDesc: %ls",
					ShaderType->GetName(), PermutationId, *GetFriendlyName(),
					MeshShaderMap != nullptr, RenderingThreadShaderMap != nullptr,
					CachedGameMeshShaderMap != nullptr, CachedGameThreadShaderMap != nullptr, bShaderWasFoundInGameShaderMap,
					VertexFactoryType->GetName(), *LegacyShaderPlatformToShaderFormat(GetShaderPlatform()).ToString(),
					bMaterialShouldCache, bVFShouldCache, bShaderShouldCache,
					*MaterialUsage
				);

				if (MeshShaderMap)
				{
					TMap<FHashedName, TShaderRef<FShader>> List;
					MeshShaderMap->GetShaderList(*RenderingThreadShaderMap, List);

					for (const auto& ShaderPair : List)
					{
						FString TypeName = ShaderPair.Value.GetType()->GetName();
						UE_LOGF(LogMaterial, Error, "ShaderType found in MaterialMap: %ls", *TypeName);
					}
				}

				UE_LOGF(LogMaterial, Fatal, "Fatal Error Material not found");
			};
			noinline_lambda();
		}

		return TShaderRef<FShader>();
	}

	return TShaderRef<FShader>(Shader, *RenderingThreadShaderMap);
}

void FMaterial::GetOutputPrecision(EMaterialFloatPrecisionMode FloatPrecisionMode, bool& FullPrecisionInPS, bool& FullPrecisionInMaterial)
{
	static const IConsoleVariable* CVarFloatPrecisionMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.FloatPrecisionMode"));

	if (FloatPrecisionMode != EMaterialFloatPrecisionMode::MFPM_Default)
	{
		FullPrecisionInMaterial = FloatPrecisionMode == EMaterialFloatPrecisionMode::MFPM_Full_MaterialExpressionOnly;
		FullPrecisionInPS = FloatPrecisionMode == EMaterialFloatPrecisionMode::MFPM_Full;
	}
	else if (CVarFloatPrecisionMode)
	{
		int MobilePrecisionMode = FMath::Clamp(CVarFloatPrecisionMode->GetInt(), (int32_t)EMobileFloatPrecisionMode::Half, (int32_t)EMobileFloatPrecisionMode::Full);

		FullPrecisionInMaterial = MobilePrecisionMode == EMobileFloatPrecisionMode::Full_MaterialExpressionOnly;
		FullPrecisionInPS = MobilePrecisionMode == EMobileFloatPrecisionMode::Full;
	}
}

TRACE_DECLARE_ATOMIC_INT_COUNTER(Shaders_OnDemandShaderRequests, TEXT("Shaders/OnDemandShaderRequests"));
bool FMaterial::TryGetShaders(const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType, FMaterialShaders& OutShaders) const
{
	// TRACE_CPUPROFILER_EVENT_SCOPE(FMaterial::TryGetShaders); // <-- disabled by default due to verbosity (hundreds of calls per frame)


	const bool bIsInGameThread = (IsInGameThread() || IsInParallelGameThread());
	const FMaterialShaderMap* ShaderMap = bIsInGameThread ? GameThreadShaderMap : RenderingThreadShaderMap;
	const bool bShaderMapComplete = bIsInGameThread ? IsGameThreadShaderMapComplete() : IsRenderingThreadShaderMapComplete();

	if (ShaderMap == nullptr)
	{
		return false;
	}

#if WITH_ODSC
	const bool bIsODSCActive = FODSCManager::IsODSCActive();
	bool bShouldForceRecompile = bIsODSCActive && FODSCManager::ShouldForceRecompile(ShaderMap, this);
	const bool bUseDefaultMaterialOnRecompile = FODSCManager::UseDefaultMaterialOnRecompile();
#else
	constexpr bool bIsODSCActive = false;
	constexpr bool bShouldForceRecompile = false;
	constexpr bool bUseDefaultMaterialOnRecompile = false;
#endif

	OutShaders.ShaderMap = ShaderMap;
	const EShaderPermutationFlags PermutationFlags = ShaderMap->GetPermutationFlags();
	const FShaderMapContent* ShaderMapContent = InVertexFactoryType
		? static_cast<const FShaderMapContent*>(ShaderMap->GetMeshShaderMap(InVertexFactoryType))
		: static_cast<const FShaderMapContent*>(ShaderMap->GetContent());

	const FMaterialCompilationOutput& CompilationOutput = ShaderMap->GetCompilationOutput();

	TArray<FShaderCommonCompileJobPtr> CompileJobs;
	bool bMissingShader = false;

	auto ShouldCacheShaderType = [this, &PermutationFlags, &CompilationOutput](const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType, const int32 PermutationId) -> bool {
		// Check to see if the FMaterial should cache these types.
		if (!ShouldCache(ShaderType, VertexFactoryType))
		{
			return false;
		}
		// if we are just a MaterialShaderType (not associated with a mesh)
		if (const FMaterialShaderType* MaterialShader = ShaderType->GetMaterialShaderType())
		{
			return MaterialShader->ShouldCompilePermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), PermutationId, PermutationFlags);
		}

		// if we are a MeshMaterialShader
		if (const FMeshMaterialShaderType* MeshMaterialShader = ShaderType->GetMeshMaterialShaderType())
		{
			const bool bVFShouldCache = FMeshMaterialShaderType::ShouldCompileVertexFactoryPermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), VertexFactoryType, ShaderType, PermutationFlags);
			const bool bShaderShouldCache = MeshMaterialShader->ShouldCompilePermutation(GetShaderPlatform(), FCompilationShaderParameters(this, CompilationOutput), VertexFactoryType, PermutationId, PermutationFlags);
			return bVFShouldCache && bShaderShouldCache;
		}
		
		return false;
	};

	if (InTypes.PipelineType && RHISupportsShaderPipelines(GetShaderPlatform()) && UseShaderPipelines(GetShaderPlatform()))
	{
		FShaderPipeline* Pipeline = ShaderMapContent ? ShaderMapContent->GetShaderPipeline(InTypes.PipelineType) : nullptr;
		if (Pipeline)
		{
			OutShaders.Pipeline = Pipeline;
			for (int32 FrequencyIndex = 0; FrequencyIndex < SF_NumGraphicsFrequencies; ++FrequencyIndex)
			{
				const FShaderType* ShaderType = InTypes.ShaderType[FrequencyIndex];
				FShader* Shader = Pipeline->GetShader((EShaderFrequency)FrequencyIndex);
				if (Shader)
				{
					check(Shader->GetType(ShaderMap->GetPointerTable()) == ShaderType);
					OutShaders.Shaders[FrequencyIndex] = Shader;
				}
				else
				{
					check(!ShaderType);
				}
			}
		}

		bool bRequestNewCompilation = (Pipeline == nullptr) || bShouldForceRecompile;
#if WITH_ODSC
		TArray<FShaderId> RequestShaderIds;
		TArray<FString> ShaderStageNamesToCompile;
        bool bODSCRequestAlreadySent = false;
		if (bIsODSCActive && (Pipeline == nullptr || bShouldForceRecompile))
		{
			for (auto* ShaderType : InTypes.PipelineType->GetStages())
			{
				ShaderStageNamesToCompile.Add(ShaderType->GetName());
				RequestShaderIds.Add(FShaderId(ShaderType, ShaderMap->GetShaderMapId().CookedShaderMapIdHash, InTypes.PipelineType->GetHashedName(), InVertexFactoryType, kUniqueShaderPermutationId, GetShaderPlatform()));
			}

			bODSCRequestAlreadySent = GODSCManager->CheckIfRequestAlreadySent(RequestShaderIds, this);
		}

		if (bODSCRequestAlreadySent)
		{
			bRequestNewCompilation = false;
		    bMissingShader |= ((Pipeline == nullptr) || bUseDefaultMaterialOnRecompile);
		}
#endif

        // we don't do 'else' here because when bShouldForceRecompile is true, we still want to use the current 
        // pipeline until we have a new one ready. The ODSC server might fail to find the right shader, and this might
        // skew results when doing some A/B comparisons		
		if (bRequestNewCompilation)
		{
			if (InTypes.PipelineType->ShouldOptimizeUnusedOutputs(GetShaderPlatform()))
			{
    		    bMissingShader |= ((Pipeline == nullptr) || bUseDefaultMaterialOnRecompile);

#if WITH_EDITOR || WITH_ODSC
				for (const FShaderType* ShaderType : InTypes.PipelineType->GetStages())
				{
					const int32 PermutationId = InTypes.PermutationId[ShaderType->GetFrequency()];

					if (!ShouldCacheShaderType(ShaderType, InVertexFactoryType, PermutationId))
					{
						return false;
					}
				}
#endif // WITH_EDITOR || WITH_ODSC

#if WITH_ODSC
				if (FPlatformProperties::RequiresCookedData() && !bODSCRequestAlreadySent)
				{
					if (bIsODSCActive)
					{
						const FString VFTypeName(InVertexFactoryType ? InVertexFactoryType->GetName() : TEXT(""));
						const FString PipelineName(InTypes.PipelineType->GetName());
						GODSCManager->AddThreadedShaderPipelineRequest(GetShaderPlatform(), GetFeatureLevel(), GetQualityLevel(), this, VFTypeName, PipelineName,
						                                               ShaderStageNamesToCompile, kUniqueShaderPermutationId, RequestShaderIds);
					}
				}
				else
#endif
				{
#if WITH_EDITOR
					const uint32 CompilingShaderMapId = bIsInGameThread ? GameThreadCompilingShaderMapId : RenderingThreadCompilingShaderMapId;
					if (CompilingShaderMapId != 0u)
					{
						if (!bShaderMapComplete)
						{
							if (InVertexFactoryType)
							{
								FMeshMaterialShaderType::BeginCompileShaderPipeline(
									EShaderCompileJobPriority::ForceLocal, 
									CompilingShaderMapId, 
									kUniqueShaderPermutationId, 
									ShaderPlatform, 
									PermutationFlags, 
									this,
									ShaderMap->GetCompilationOutput(),
									RenderingThreadPendingCompilerEnvironment, 
									InVertexFactoryType, 
									InTypes.PipelineType, 
									CompileJobs,
									GetDebugGroupName(),
									nullptr, 
									nullptr);
							}
							else
							{
								FMaterialShaderType::BeginCompileShaderPipeline(
									EShaderCompileJobPriority::ForceLocal, 
									CompilingShaderMapId, 
									ShaderPlatform, 
									PermutationFlags, 
									this,
									ShaderMap->GetCompilationOutput(),
									RenderingThreadPendingCompilerEnvironment, 
									InTypes.PipelineType, 
									CompileJobs,
									GetDebugGroupName(),
									nullptr, 
									nullptr);
							}
						}
					}
#endif // WITH_EDITOR
				}
			}
		}
	}
	else
	{
		for (int32 FrequencyIndex = 0; FrequencyIndex < SF_NumFrequencies; ++FrequencyIndex)
		{
			const FShaderType* ShaderType = InTypes.ShaderType[FrequencyIndex];
			if (ShaderType)
			{
				const int32 PermutationId = InTypes.PermutationId[FrequencyIndex];
				FShader* Shader = ShaderMapContent ? ShaderMapContent->GetShader(ShaderType, PermutationId) : nullptr;
				if (Shader)
				{
					OutShaders.Shaders[FrequencyIndex] = Shader;
				}
		        // we don't do 'else' here because when bShouldForceRecompile is true, we still want to use the current 
		        // shader until we have a new one ready. The ODSC server might fail to find the right shader, and this might
		        // skew results when doing some A/B comparisons		

				bool bRequestNewCompilation = (Shader == nullptr) || bShouldForceRecompile;
#if WITH_ODSC
				TArray<FShaderId> RequestShaderIds;
                bool bODSCRequestAlreadySent = false;
				if (bIsODSCActive && (Shader == nullptr || bShouldForceRecompile))
				{
					RequestShaderIds.Add(FShaderId(ShaderType, ShaderMap->GetShaderMapId().CookedShaderMapIdHash, FHashedName(), InVertexFactoryType, PermutationId, GetShaderPlatform()));

					bODSCRequestAlreadySent = GODSCManager->CheckIfRequestAlreadySent(RequestShaderIds, this);
				}

				if (bODSCRequestAlreadySent)
				{
    				bMissingShader |= ((Shader == nullptr) || bUseDefaultMaterialOnRecompile);
					bRequestNewCompilation = false;
				}
#endif
				
				if (bRequestNewCompilation)
				{
    				bMissingShader |= ((Shader == nullptr) || bUseDefaultMaterialOnRecompile);
#if WITH_EDITOR || WITH_ODSC
					if (!ShouldCacheShaderType(ShaderType, InVertexFactoryType, PermutationId))
					{
						continue;
					}
#endif // WITH_EDITOR || WITH_ODSC

#if WITH_ODSC
					if (FPlatformProperties::RequiresCookedData() && !bODSCRequestAlreadySent)
					{
						if (bIsODSCActive)
						{
							const FString VFTypeName(InVertexFactoryType ? InVertexFactoryType->GetName() : TEXT(""));
							const FString PipelineName;
							TArray<FString> ShaderStageNamesToCompile;
							ShaderStageNamesToCompile.Add(ShaderType->GetName());

							GODSCManager->AddThreadedShaderPipelineRequest(GetShaderPlatform(), GetFeatureLevel(), GetQualityLevel(), this, VFTypeName, PipelineName,
							                                               ShaderStageNamesToCompile, PermutationId, RequestShaderIds);
						}
					}
					else
#endif
					{
#if WITH_EDITOR
						const uint32 CompilingShaderMapId = bIsInGameThread ? GameThreadCompilingShaderMapId : RenderingThreadCompilingShaderMapId;
						if (CompilingShaderMapId != 0u)
						{
							if (!bShaderMapComplete)
							{
								if (InVertexFactoryType)
								{
									ShaderType->AsMeshMaterialShaderType()->BeginCompileShader(
										EShaderCompileJobPriority::ForceLocal,
										CompilingShaderMapId,
										PermutationId,
										ShaderPlatform,
										PermutationFlags,
										this,
										ShaderMap->GetCompilationOutput(),
										RenderingThreadPendingCompilerEnvironment,
										InVertexFactoryType,
										CompileJobs,
										GetDebugGroupName(),
										nullptr,
										nullptr);
								}
								else
								{
									ShaderType->AsMaterialShaderType()->BeginCompileShader(
										EShaderCompileJobPriority::ForceLocal,
										CompilingShaderMapId,
										PermutationId,
										this,
										ShaderMap->GetCompilationOutput(),
										RenderingThreadPendingCompilerEnvironment,
										ShaderPlatform,
										PermutationFlags,
										CompileJobs,
										GetDebugGroupName(),
										nullptr,
										nullptr);
								}
							}
						}
#endif // WITH_EDITOR
					}
				}
			}
		}
	}

	if (CompileJobs.Num() > 0)
	{
		TRACE_COUNTER_ADD(Shaders_OnDemandShaderRequests, CompileJobs.Num());
		GShaderCompilingManager->SubmitJobs(CompileJobs, GetBaseMaterialPathName(), ShaderMap->GetDebugDescription());
	}

	return !bMissingShader;
}

bool FMaterial::HasShaders(const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType) const
{
	FMaterialShaders UnusedShaders;
	return TryGetShaders(InTypes, InVertexFactoryType, UnusedShaders);
}

// Deprecated 5.7
bool FMaterial::ShouldCacheShaders(const EShaderPlatform InShaderPlatform, const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType) const
{
	return ShouldCacheShaders(InTypes, InVertexFactoryType);
}

bool FMaterial::ShouldCacheShaders(const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType) const
{
	for (int32 FrequencyIndex = 0; FrequencyIndex < SF_NumGraphicsFrequencies; ++FrequencyIndex)
	{
		const FShaderType* ShaderType = InTypes.ShaderType[FrequencyIndex];
		if (ShaderType && !ShouldCache(ShaderType, InVertexFactoryType))
		{
			return false;
		}
	}
	return true;
}

FShaderPipelineRef FMaterial::GetShaderPipeline(class FShaderPipelineType* ShaderPipelineType, FVertexFactoryType* VertexFactoryType, bool bFatalIfNotFound) const
{
	const FMeshMaterialShaderMap* MeshShaderMap = RenderingThreadShaderMap->GetMeshShaderMap(VertexFactoryType);
	FShaderPipeline* ShaderPipeline = MeshShaderMap ? MeshShaderMap->GetShaderPipeline(ShaderPipelineType) : nullptr;
	if (!ShaderPipeline)
	{
		if (bFatalIfNotFound)
		{
			auto noinline_lambda = [&](...) FORCENOINLINE
			{
				// Get the ShouldCache results that determine whether the shader should be compiled
				auto ShaderPermutation = RenderingThreadShaderMap->GetPermutationFlags();
				FString MaterialUsage = GetMaterialUsageDescription();

				const FMaterialCompilationOutput& CompilationOutput = RenderingThreadShaderMap->GetCompilationOutput();

				UE_LOGF(LogMaterial, Error,
					"Couldn't find ShaderPipeline %ls for Material Resource %ls!", ShaderPipelineType->GetName(), *GetFriendlyName());

				for (auto* ShaderType : ShaderPipelineType->GetStages())
				{
					FShader* Shader = MeshShaderMap ? MeshShaderMap->GetShader((FShaderType*)ShaderType) : RenderingThreadShaderMap->GetShader((FShaderType*)ShaderType).GetShader();
					const FCompilationShaderParameters ShaderParams(this, CompilationOutput);

					if (!Shader)
					{
						UE_LOGF(LogMaterial, Error, "Missing %ls shader %ls!", GetShaderFrequencyString(ShaderType->GetFrequency(), false), ShaderType->GetName());
					}
					else if (ShaderType->GetMeshMaterialShaderType())
					{
						bool bMaterialShouldCache = ShouldCache(ShaderType->GetMeshMaterialShaderType(), VertexFactoryType);
						bool bVFShouldCache = FMeshMaterialShaderType::ShouldCompileVertexFactoryPermutation(GetShaderPlatform(), ShaderParams, VertexFactoryType, ShaderType, ShaderPermutation);
						bool bShaderShouldCache = ShaderType->GetMeshMaterialShaderType()->ShouldCompilePermutation(GetShaderPlatform(), ShaderParams, VertexFactoryType, kUniqueShaderPermutationId, ShaderPermutation);

						UE_LOGF(LogMaterial, Error, "%ls %ls ShouldCache: Mat=%u, VF=%u, Shader=%u",
							GetShaderFrequencyString(ShaderType->GetFrequency(), false), ShaderType->GetName(), bMaterialShouldCache, bVFShouldCache, bShaderShouldCache);
					}
					else if (ShaderType->GetMaterialShaderType())
					{
						bool bMaterialShouldCache = ShouldCache(ShaderType->GetMaterialShaderType(), VertexFactoryType);
						bool bShaderShouldCache = ShaderType->GetMaterialShaderType()->ShouldCompilePermutation(GetShaderPlatform(), ShaderParams, kUniqueShaderPermutationId, ShaderPermutation);

						UE_LOGF(LogMaterial, Error, "%ls %ls ShouldCache: Mat=%u, NO VF, Shader=%u",
							GetShaderFrequencyString(ShaderType->GetFrequency(), false), ShaderType->GetName(), bMaterialShouldCache, bShaderShouldCache);
					}
				}

				int BreakPoint = 0;

				// Assert with detailed information if the shader wasn't found for rendering.  
				// This is usually the result of an incorrect ShouldCache function.
				UE_LOGF(LogMaterial, Fatal,
					"		With VF=%ls, Platform=%ls\n" "		MaterialUsageDesc: %ls",
					VertexFactoryType->GetName(), *LegacyShaderPlatformToShaderFormat(GetShaderPlatform()).ToString(),
					*MaterialUsage
					);
			};
			noinline_lambda();
		}

		return FShaderPipelineRef();
	}

	return FShaderPipelineRef(ShaderPipeline, *RenderingThreadShaderMap);
}

#if WITH_EDITOR
TSet<FMaterial*> FMaterial::EditorLoadedMaterialResources;
#endif // WITH_EDITOR

/*-----------------------------------------------------------------------------
	FMaterialRenderContext
-----------------------------------------------------------------------------*/

/** 
 * Constructor
 */
FMaterialRenderContext::FMaterialRenderContext(
	const FMaterialRenderProxy* InMaterialRenderProxy,
	const FMaterial& InMaterial,
	const FSceneView* InView)
		: MaterialRenderProxy(InMaterialRenderProxy)
		, Material(InMaterial)
{
	bShowSelection = GIsEditor && InView && InView->Family->EngineShowFlags.Selection;
}

/*-----------------------------------------------------------------------------
	FMaterialVirtualTextureStack
-----------------------------------------------------------------------------*/

FMaterialVirtualTextureStack::FMaterialVirtualTextureStack()
	: NumLayers(0u)
	, PreallocatedStackTextureIndex(INDEX_NONE)
#if WITH_EDITORONLY_DATA
	, ShaderFrequencyMask(0)
#endif
{
	for (uint32 i = 0u; i < VIRTUALTEXTURE_SPACE_MAXLAYERS; ++i)
	{
		LayerUniformExpressionIndices[i] = INDEX_NONE;
	}
}

FMaterialVirtualTextureStack::FMaterialVirtualTextureStack(int32 InPreallocatedStackTextureIndex)
	: NumLayers(0u)
	, PreallocatedStackTextureIndex(InPreallocatedStackTextureIndex)
#if WITH_EDITORONLY_DATA
	, ShaderFrequencyMask(0)
#endif
{
	for (uint32 i = 0u; i < VIRTUALTEXTURE_SPACE_MAXLAYERS; ++i)
	{
		LayerUniformExpressionIndices[i] = INDEX_NONE;
	}
}

uint32 FMaterialVirtualTextureStack::AddLayer()
{
	const uint32 LayerIndex = NumLayers++;
	return LayerIndex;
}

uint32 FMaterialVirtualTextureStack::SetLayer(int32 LayerIndex, int32 UniformExpressionIndex)
{
	check(UniformExpressionIndex >= 0);
	check(LayerIndex >= 0 && LayerIndex < VIRTUALTEXTURE_SPACE_MAXLAYERS);
	LayerUniformExpressionIndices[LayerIndex] = UniformExpressionIndex;
	NumLayers = FMath::Max<uint32>(LayerIndex + 1, NumLayers);
	return LayerIndex;
}

int32 FMaterialVirtualTextureStack::FindLayer(int32 UniformExpressionIndex) const
{
	for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
	{
		if (LayerUniformExpressionIndices[LayerIndex] == UniformExpressionIndex)
		{
			return LayerIndex;
		}
	}
	return -1;
}

void FMaterialVirtualTextureStack::GetTextureValues(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, UTexture const** OutValues) const
{
	FMemory::Memzero(OutValues, sizeof(UTexture*) * VIRTUALTEXTURE_SPACE_MAXLAYERS);
	
	for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
	{
		const int32 ParameterIndex = LayerUniformExpressionIndices[LayerIndex];
		if (ParameterIndex != INDEX_NONE)
		{
			const UTexture* Texture = nullptr;
			UniformExpressionSet.GetTextureValue(EMaterialTextureParameterType::Virtual, ParameterIndex, Context, Context.Material, Texture);
			OutValues[LayerIndex] = Texture;
		}
	}
}

void FMaterialVirtualTextureStack::GetTextureValue(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const URuntimeVirtualTexture*& OutValue) const
{
	OutValue = nullptr;
	for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
	{
		const int32 ParameterIndex = LayerUniformExpressionIndices[LayerIndex];
		if (ParameterIndex != INDEX_NONE)
		{
			const URuntimeVirtualTexture* Texture = nullptr;
			UniformExpressionSet.GetTextureValue(ParameterIndex, Context, Context.Material, Texture);
			OutValue = Texture;
			break;
		}
	}
}

void FMaterialVirtualTextureStack::Serialize(FArchive& Ar)
{
	uint32 SerializedNumLayers = NumLayers;
	Ar << SerializedNumLayers;
	NumLayers = FMath::Min(SerializedNumLayers, uint32(VIRTUALTEXTURE_SPACE_MAXLAYERS));

	for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
	{
		Ar << LayerUniformExpressionIndices[LayerIndex];
	}

	for (uint32 LayerIndex = NumLayers; LayerIndex < SerializedNumLayers; ++LayerIndex)
	{
		int32 DummyIndex = INDEX_NONE;
		Ar << DummyIndex;
	}

	Ar << PreallocatedStackTextureIndex;
}

#if WITH_EDITOR
void FMaterial::SubmitCompileJobs_GameThread(EShaderCompileJobPriority Priority)
{
	check(IsInGameThread());

	if (GameThreadCompilingShaderMapId != 0u && GameThreadShaderMap)
	{
		const EShaderCompileJobPriority SubmittedPriority = GameThreadShaderMapSubmittedPriority;

		// To avoid as much useless work as possible, we make sure to submit our compile jobs only once per priority upgrade.
		if (GameThreadShaderMapSubmittedPriority == EShaderCompileJobPriority::None || Priority > SubmittedPriority)
		{
			check(GameThreadPendingCompilerEnvironment.IsValid());

			GameThreadShaderMapSubmittedPriority = Priority;
			GameThreadShaderMap->SubmitCompileJobs(GameThreadCompilingShaderMapId, this, GameThreadPendingCompilerEnvironment, Priority);
		}
	}
}

void FMaterial::SubmitCompileJobs_RenderThread(EShaderCompileJobPriority Priority) const
{
	check(IsInParallelRenderingThread());
	if (RenderingThreadCompilingShaderMapId != 0u && RenderingThreadShaderMap)
	{
		// std::atomic don't support enum class, so we have to make sure our cast assumptions are respected.
		static_assert((int8)EShaderCompileJobPriority::None == -1 && EShaderCompileJobPriority::Low < EShaderCompileJobPriority::ForceLocal, "Revise EShaderCompileJobPriority cast assumptions");
		EShaderCompileJobPriority SubmittedPriority = (EShaderCompileJobPriority)RenderingThreadShaderMapSubmittedPriority.load(std::memory_order_relaxed);

		// To avoid as much useless work as possible, we make sure to submit our compile jobs only once per priority upgrade.
		if (SubmittedPriority == EShaderCompileJobPriority::None || Priority > SubmittedPriority)
		{
			RenderingThreadShaderMapSubmittedPriority = (int8)Priority;
			RenderingThreadShaderMap->SubmitCompileJobs(RenderingThreadCompilingShaderMapId, this, RenderingThreadPendingCompilerEnvironment, Priority);
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
/** Returns the number of samplers used in this material, or -1 if the material does not have a valid shader map (compile error or still compiling). */
int32 FMaterialResource::GetSamplerUsage() const
{
	if (GetGameThreadShaderMap())
	{
		return GetGameThreadShaderMap()->GetMaxTextureSamplers();
	}

	return -1;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void FMaterialResource::GetUserInterpolatorUsage(uint32& NumUsedUVScalars, uint32& NumUsedCustomInterpolatorScalars) const
{
	NumUsedUVScalars = NumUsedCustomInterpolatorScalars = 0;

	if (const FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap())
	{
		NumUsedUVScalars = ShaderMap->GetNumUsedUVScalars();
		NumUsedCustomInterpolatorScalars = ShaderMap->GetNumUsedCustomInterpolatorScalars();
	}
}

void FMaterialResource::GetEstimatedNumTextureSamples(uint32& VSSamples, uint32& PSSamples) const
{
	VSSamples = PSSamples = 0;
	if (const FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap())
	{
		ShaderMap->GetEstimatedNumTextureSamples(VSSamples, PSSamples);
	}
}

uint32 FMaterialResource::GetEstimatedNumVirtualTextureLookups() const
{
	if (const FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap())
	{
		return ShaderMap->GetEstimatedNumVirtualTextureLookups();
	}
	return 0;
}

void FMaterialResource::GetEstimatedLWCFuncUsages(FLWCUsagesArray& UsagesVS, FLWCUsagesArray& UsagesPS, FLWCUsagesArray& UsagesCS) const
{
	if (const FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap())
	{
		ShaderMap->GetEstimatedLWCFuncUsages(UsagesVS, UsagesPS, UsagesCS);
	}
}

#endif // WITH_EDITOR

uint32 FMaterialResource::GetNumVirtualTextureStacks() const
{
	if (const FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap())
	{
		return ShaderMap->GetNumVirtualTextureStacks();
	}
	return 0;
}


FString FMaterialResource::GetMaterialUsageDescription() const
{
	check(Material);
	FString BaseDescription = FString::Printf(
		TEXT("LightingModel=%s, BlendMode=%s, "),
		*GetShadingModelFieldString(GetShadingModels()), *GetBlendModeString(GetBlendMode()));

	// this changed from ",SpecialEngine, TwoSided" to ",SpecialEngine=1, TwoSided=1, TSNormal=0, ..." to be more readable
	BaseDescription += FString::Printf(
		TEXT("SpecialEngine=%d, TwoSided=%d, TSNormal=%d, Masked=%d, Distorted=%d, WritesEveryPixel=%d, ModifiesMeshPosition=%d")
		TEXT(", Usage={"),
		(int32)IsSpecialEngineMaterial(), (int32)IsTwoSided(), (int32)IsTangentSpaceNormal(), (int32)IsMasked(), (int32)IsDistorted(), (int32)WritesEveryPixel(), (int32)MaterialMayModifyMeshPosition()
		);

	bool bFirst = true;
	for (int32 MaterialUsageIndex = 0; MaterialUsageIndex < MATUSAGE_MAX; MaterialUsageIndex++)
	{
		if (Material->GetUsageByFlag((EMaterialUsage)MaterialUsageIndex))
		{
			if (!bFirst)
			{
				BaseDescription += FString(",");
			}
			BaseDescription += UMaterialInterface::GetUsageName((EMaterialUsage)MaterialUsageIndex);
			bFirst = false;
		}
	}
	BaseDescription += FString("}");

	return BaseDescription;
}

static void AddSortedShader(TArray<FShaderType*>& Shaders, FShaderType* Shader)
{
	const int32 SortedIndex = Algo::LowerBoundBy(Shaders, Shader->GetHashedName(), [](const FShaderType* InShaderType) { return InShaderType->GetHashedName(); });
	if (SortedIndex >= Shaders.Num() || Shaders[SortedIndex] != Shader)
	{
		Shaders.Insert(Shader, SortedIndex);
	}
}

static void AddSortedShaderPipeline(TArray<const FShaderPipelineType*>& Pipelines, const FShaderPipelineType* Pipeline)
{
	const int32 SortedIndex = Algo::LowerBoundBy(Pipelines, Pipeline->GetHashedName(), [](const FShaderPipelineType* InPiplelineType) { return InPiplelineType->GetHashedName(); });
	if (SortedIndex >= Pipelines.Num() || Pipelines[SortedIndex] != Pipeline)
	{
		Pipelines.Insert(Pipeline, SortedIndex);
	}
}

// Deprecated in 5.7
void FMaterial::GetDependentShaderAndVFTypes(EShaderPlatform Platform, const FPlatformTypeLayoutParameters& LayoutParams, TArray<FShaderType*>& OutShaderTypes, TArray<const FShaderPipelineType*>& OutShaderPipelineTypes, TArray<FVertexFactoryType*>& OutVFTypes) const
{
	GetDependentShaderAndVFTypes(LayoutParams, OutShaderTypes, OutShaderPipelineTypes, OutVFTypes);
}

void FMaterial::GetDependentShaderAndVFTypes(const FPlatformTypeLayoutParameters& LayoutParams, TArray<FShaderType*>& OutShaderTypes, TArray<const FShaderPipelineType*>& OutShaderPipelineTypes, TArray<FVertexFactoryType*>& OutVFTypes) const
{
	const FConservativeShaderParameters MaterialParameters(this);
	const FMaterialShaderMapLayout& Layout = AcquireMaterialShaderMapLayout(GetShaderPlatform(), GetShaderPermutationFlags(LayoutParams), MaterialParameters);

	for (const FShaderLayoutEntry& Shader : Layout.Shaders)
	{
		if (ShouldCache(Shader.ShaderType, nullptr))
		{
			AddSortedShader(OutShaderTypes, Shader.ShaderType);
		}
	}

	for (const FShaderPipelineType* Pipeline : Layout.ShaderPipelines)
	{
		if (ShouldCachePipeline(Pipeline, nullptr))
		{
			AddSortedShaderPipeline(OutShaderPipelineTypes, Pipeline);
			for (const FShaderType* Type : Pipeline->GetStages())
			{
				AddSortedShader(OutShaderTypes, (FShaderType*)Type);
			}
		}
	}

	for (const FMeshMaterialShaderMapLayout& MeshLayout : Layout.MeshShaderMaps)
	{
		bool bIncludeVertexFactory = false;
		for (const FShaderLayoutEntry& Shader : MeshLayout.Shaders)
		{
			if (ShouldCache(Shader.ShaderType, MeshLayout.VertexFactoryType))
			{
				bIncludeVertexFactory = true;
				AddSortedShader(OutShaderTypes, Shader.ShaderType);
			}
		}

		for (const FShaderPipelineType* Pipeline : MeshLayout.ShaderPipelines)
		{
			if (ShouldCachePipeline(Pipeline, MeshLayout.VertexFactoryType))
			{
				bIncludeVertexFactory = true;
				AddSortedShaderPipeline(OutShaderPipelineTypes, Pipeline);
				for (const FShaderType* Type : Pipeline->GetStages())
				{
					AddSortedShader(OutShaderTypes, (FShaderType*)Type);
				}
			}
		}

		if (bIncludeVertexFactory)
		{
			// Vertex factories are already sorted
			OutVFTypes.Add(MeshLayout.VertexFactoryType);
		}
	}
}

#if WITH_EDITOR
uint64 FMaterial::GetReferencedTexturesHash() const
{
	FXxHash64Builder HashState;

	const TArray<TObjectPtr<UObject>>& ReferencedTextures = GetMatDefaultTextures();
	// Hash the names of the uniform expression textures to capture changes in their order or values resulting from material compiler code changes
	for (int32 TextureIndex = 0; TextureIndex < ReferencedTextures.Num(); TextureIndex++)
	{
		FString TextureName;

		if (ReferencedTextures[TextureIndex])
		{
			TextureName = ReferencedTextures[TextureIndex]->GetName();
		}

		HashState.Update(*TextureName, TextureName.NumBytesWithoutNull());
	}

	UMaterialShaderQualitySettings* MaterialShaderQualitySettings = UMaterialShaderQualitySettings::Get();
	if(MaterialShaderQualitySettings->HasPlatformQualitySettings(GetShaderPlatform(), QualityLevel))
	{
		MaterialShaderQualitySettings->GetShaderPlatformQualitySettings(GetShaderPlatform())->AppendToHashState(QualityLevel, HashState);
	}

	return HashState.Finalize().Hash;
}

uint64 FMaterial::GetExpressionIncludesHash() const
{
	FXxHash64Builder HashState;

	for (const FString& ExpressionIncludeFilePath : GetCachedExpressionData().EditorOnlyData->ExpressionIncludeFilePaths)
	{
		checkf(!ExpressionIncludeFilePath.IsEmpty(), TEXT("Expression include path is empty but it should have been previously validated."));

		const FSHAHash& FileHash = GetShaderFileHash(*ExpressionIncludeFilePath, GetShaderPlatform());
		HashState.Update(FileHash.Hash, UE_ARRAY_COUNT(FileHash.Hash));
	}

	return HashState.Finalize().Hash;
}

uint64 FMaterial::GetExternalCodeReferencesHash() const
{
	return GetCachedExpressionData().GetExternalCodeReferencesHash();
}
#endif // WITH_EDITOR

bool FMaterial::GetMaterialExpressionSource(FString& OutSource)
{
#if WITH_EDITORONLY_DATA
	const ITargetPlatform* TargetPlatform = nullptr;

	FMaterialShaderMapId ShaderMapId;
	BuildShaderMapId(ShaderMapId, TargetPlatform);
	FStaticParameterSet StaticParameterSet;
	GetStaticParameterSet(StaticParameterSet);

	FMaterialCompilationOutput NewCompilationOutput;
	TRefCountPtr<FSharedShaderCompilerEnvironment> MaterialEnvironment;
	const bool bSuccess = Translate(ShaderMapId, StaticParameterSet, TargetPlatform, NewCompilationOutput, MaterialEnvironment);

	if (bSuccess)
	{
		FString* Source = MaterialEnvironment->IncludeVirtualPathToContentsMap.Find(TEXT("/Engine/Generated/Material.ush"));
		if (Source)
		{
			OutSource = MoveTemp(*Source);

			// If we've succesfully translated using the new translator, let's compile w/ the old translator so we have source to compare against.
			if (ShaderMapId.bUsingNewHLSLGenerator)
			{
				ShaderMapId.bUsingNewHLSLGenerator = false;

				// Translate_Legacy will fill out FMaterialInsights::Legacy_ShaderStringParameters
				FMaterialCompilationOutput LegacyNewCompilationOutput;
				TRefCountPtr<FSharedShaderCompilerEnvironment> LegacyMaterialEnvironment;
				Translate_Legacy(ShaderMapId, StaticParameterSet, TargetPlatform, LegacyNewCompilationOutput, LegacyMaterialEnvironment);
			}

			if (CVarMaterialEdPreshaderDumpToHLSL.GetValueOnGameThread())
			{
				OutSource.AppendChar('\n');

				TMap<FString, uint32> ParameterReferences;
				FMaterialRenderContext MaterialContext(nullptr, *this, nullptr);
				UE::Shader::FPreshaderDataContext PreshaderContextBase(NewCompilationOutput.UniformExpressionSet.UniformPreshaderData);
				for (int32 PreshaderIndex = 0; PreshaderIndex < NewCompilationOutput.UniformExpressionSet.UniformPreshaders.Num(); PreshaderIndex++)
				{
					const FMaterialUniformPreshaderHeader& PreshaderHeader = NewCompilationOutput.UniformExpressionSet.UniformPreshaders[PreshaderIndex];

					UE::Shader::FPreshaderDataContext PreshaderContext(PreshaderContextBase, PreshaderHeader.OpcodeOffset, PreshaderHeader.OpcodeSize);
					FString PreshaderDebug = PreshaderGenerateDebugString(NewCompilationOutput.UniformExpressionSet, MaterialContext, PreshaderContext, &ParameterReferences);

					// If this is a numeric field, add a swizzle for it
					const TCHAR* SwizzleSuffix = TEXT("");
					if (((uint8)PreshaderHeader.Type >= (uint8)UE::Shader::EValueType::Float1 && (uint8)PreshaderHeader.Type <= (uint8)UE::Shader::EValueType::Float4) ||
						((uint8)PreshaderHeader.Type >= (uint8)UE::Shader::EValueType::Int1 && (uint8)PreshaderHeader.Type <= (uint8)UE::Shader::EValueType::Int4) ||
						((uint8)PreshaderHeader.Type >= (uint8)UE::Shader::EValueType::Bool1 && (uint8)PreshaderHeader.Type <= (uint8)UE::Shader::EValueType::Bool4))
					{
						UE::Shader::FType ShaderType(PreshaderHeader.Type);

						// First axis is offset, second axis is number of components (minus one)
						static const TCHAR* SwizzleTable[4][4] =
						{
							{ TEXT(".x"), TEXT(".xy"), TEXT(".xyz"), TEXT(".xyzw") },
							{ TEXT(".y"), TEXT(".yz"), TEXT(".yzw"), TEXT(".yzw?") },
							{ TEXT(".z"), TEXT(".zw"), TEXT(".zw?"), TEXT(".zw??") },
							{ TEXT(".w"), TEXT(".w?"), TEXT(".w??"), TEXT(".w???") },
						};
						SwizzleSuffix = SwizzleTable[PreshaderHeader.BufferOffset % 4][ShaderType.GetNumComponents() - 1];
					}

					OutSource.Appendf(TEXT("// PreshaderBuffer[%d]%s = %s\n"), PreshaderHeader.BufferOffset / 4, SwizzleSuffix, *PreshaderDebug);
				}

				// Sort parameter references by frequency
				TArray<FSetElementId> ParameterReferencesSort;
				ParameterReferencesSort.Reserve(ParameterReferences.Num());
				for (auto ParameterReferenceIt = ParameterReferences.CreateConstIterator(); ParameterReferenceIt; ++ParameterReferenceIt)
				{
					ParameterReferencesSort.Add(ParameterReferenceIt.GetId());
				}
				Algo::Sort(ParameterReferencesSort, [&ParameterReferences](const FSetElementId& A, const FSetElementId& B)
					{
						const auto& ParameterReferenceA = ParameterReferences.Get(A);
						const auto& ParameterReferenceB = ParameterReferences.Get(B);
						if (ParameterReferenceA.Value != ParameterReferenceB.Value)
						{
							// Descending count
							return ParameterReferenceA.Value > ParameterReferenceB.Value;
						}
						return ParameterReferenceA.Key < ParameterReferenceB.Key;
					});

				// Print parameter references
				OutSource.Append("\n// Preshader parameter reference counts:\n");

				for (int32 ParameterReferenceIndex = 0; ParameterReferenceIndex < ParameterReferencesSort.Num(); ++ParameterReferenceIndex)
				{
					const auto& ParameterReference = ParameterReferences.Get(ParameterReferencesSort[ParameterReferenceIndex]);
					OutSource.Appendf(TEXT("// Param[\"%s\"] = %d\n"), *ParameterReference.Key, ParameterReference.Value);
				}
			}
			return true;
		}
	}
	return false;
#else
	UE_LOGF(LogMaterial, Fatal,"Not supported.");
	return false;
#endif
}

void FMaterial::GetPreshaderStats(uint32& TotalParameters, uint32& TotalOps) const
{
	TotalParameters = 0;
	TotalOps = 0;

#if WITH_EDITORONLY_DATA
	const FMaterialShaderMap* ShaderMap = GetGameThreadShaderMap();
	if (ShaderMap)
	{
		const FUniformExpressionSet& UniformExpressionSet = ShaderMap->GetUniformExpressionSet();
		FMaterialRenderContext MaterialContext(nullptr, *this, nullptr);
		UE::Shader::FPreshaderDataContext PreshaderContextBase(UniformExpressionSet.UniformPreshaderData);
		for (const FMaterialUniformPreshaderHeader& PreshaderHeader : UniformExpressionSet.UniformPreshaders)
		{
			UE::Shader::FPreshaderDataContext PreshaderContext(PreshaderContextBase, PreshaderHeader.OpcodeOffset, PreshaderHeader.OpcodeSize);
			PreshaderComputeDebugStats(UniformExpressionSet, MaterialContext, PreshaderContext, TotalParameters, TotalOps);
		}
	}
#else
	UE_LOGF(LogMaterial, Fatal,"GetPreshaderStats is only supported for WITH_EDITOR builds.");
#endif
}

bool FMaterial::WritesEveryPixel(bool bShadowPass) const
{
	const bool bVFTypeSupportsNullPixelShader = !IsUsedWithInstancedStaticMeshes();
	return WritesEveryPixel(bShadowPass, bVFTypeSupportsNullPixelShader, false);
}

bool FMaterial::WritesEveryPixel(bool bShadowPass, bool bVFTypeSupportsNullPixelShader) const
{
	return WritesEveryPixel(bShadowPass, bVFTypeSupportsNullPixelShader, false);
}

bool FMaterial::WritesEveryPixel(bool bShadowPass, bool bVFTypeSupportsNullPixelShader, bool bOpacityMaskIsOpaque) const
{
	bool bLocalStencilDitheredLOD = IsStencilForLODDitherEnabled(GetShaderPlatform());
	return (!IsMasked() || bOpacityMaskIsOpaque)
		// Render dithered material as masked if a stencil prepass is not used (UE-50064, UE-49537)
		&& !((bShadowPass || !bLocalStencilDitheredLOD) && IsDitheredLODTransition())
		&& !IsWireframe()
		// If the VF type requires a PS, return false so a PS will be picked
		&& !(bLocalStencilDitheredLOD && IsDitheredLODTransition() && !bVFTypeSupportsNullPixelShader)
		&& !IsStencilTestEnabled();
}

#if WITH_EDITOR
/** Recompiles any materials in the EditorLoadedMaterialResources list if they are not complete. */
void FMaterial::UpdateEditorLoadedMaterialResources(EShaderPlatform InShaderPlatform)
{
	for (TSet<FMaterial*>::TIterator It(EditorLoadedMaterialResources); It; ++It)
	{
		FMaterial* CurrentMaterial = *It;
		if (!CurrentMaterial->GetGameThreadShaderMap() || !CurrentMaterial->GetGameThreadShaderMap()->IsComplete(CurrentMaterial, true))
		{
			CurrentMaterial->CacheShaders();
		}
	}
}
#endif // WITH_EDITOR

void FMaterial::DumpDebugInfo(FOutputDevice& OutputDevice)
{
	if (GameThreadShaderMap)
	{
		GameThreadShaderMap->DumpDebugInfo(OutputDevice);
	}
}

void FMaterial::SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, FStableShaderKeyAndValue& SaveKeyVal)
{
	SaveShaderStableKeys(SaveKeyVal);
}

void FMaterial::SaveShaderStableKeys(FStableShaderKeyAndValue& SaveKeyVal)
{
#if WITH_EDITOR
	if (GameThreadShaderMap)
	{
		FString FeatureLevelName;
		GetFeatureLevelName(FeatureLevel, FeatureLevelName);
		SaveKeyVal.FeatureLevel = FName(*FeatureLevelName);

		FString QualityLevelString;
		GetMaterialQualityLevelName(QualityLevel, QualityLevelString);
		SaveKeyVal.QualityLevel = FName(*QualityLevelString);

		GameThreadShaderMap->SaveShaderStableKeys(GetShaderPlatform(), SaveKeyVal);
	}
#endif
}

#if WITH_EDITOR
void FMaterial::GetShaderTypesForLayout(EShaderPlatform Platform, const FShaderMapLayout& Layout, FVertexFactoryType* VertexFactory, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const
{
	GetShaderTypesForLayout(Layout, VertexFactory, OutShaderInfo);
}

void FMaterial::GetShaderTypesForLayout(const FShaderMapLayout& Layout, FVertexFactoryType* VertexFactory, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const
{
	FDebugShaderTypeInfo DebugInfo;
	DebugInfo.VFType = VertexFactory;

	for (const FShaderLayoutEntry& Shader : Layout.Shaders)
	{
		if (ShouldCache(Shader.ShaderType, VertexFactory))
		{
			DebugInfo.ShaderTypes.Add(Shader.ShaderType);
		}
	}

	for (const FShaderPipelineType* Pipeline : Layout.ShaderPipelines)
	{
		if (ShouldCachePipeline(Pipeline, VertexFactory))
		{
			FDebugShaderPipelineInfo PipelineInfo;
			PipelineInfo.Pipeline = Pipeline;

			for (const FShaderType* Type : Pipeline->GetStages())
			{
				PipelineInfo.ShaderTypes.Add((FShaderType*)Type);
			}

			DebugInfo.Pipelines.Add(PipelineInfo);
		}
	}

	OutShaderInfo.Add(DebugInfo);
}

// Deprecated in 5.7
void FMaterial::GetShaderTypes(EShaderPlatform Platform, const FPlatformTypeLayoutParameters& LayoutParams, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const
{
	GetShaderTypes(LayoutParams, OutShaderInfo, FConservativeShaderParameters(this));
}

void FMaterial::GetShaderTypes(const FPlatformTypeLayoutParameters& LayoutParams, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const
{
	const FConservativeShaderParameters MaterialParameters(this);
	GetShaderTypes(LayoutParams, OutShaderInfo, MaterialParameters);
}

void FMaterial::GetShaderTypes(const FPlatformTypeLayoutParameters& LayoutParams, TArray<FDebugShaderTypeInfo>& OutShaderInfo, const FMaterialShaderParameters& InParameters) const
{
	const FMaterialShaderMapLayout& Layout = AcquireMaterialShaderMapLayout(GetShaderPlatform(), GetShaderPermutationFlags(LayoutParams), InParameters);
	GetShaderTypesForLayout(Layout, nullptr, OutShaderInfo);

	for (const FMeshMaterialShaderMapLayout& MeshLayout : Layout.MeshShaderMaps)
	{
		GetShaderTypesForLayout(MeshLayout, MeshLayout.VertexFactoryType, OutShaderInfo);
	}
}

#endif

FMaterialUpdateContext::FMaterialUpdateContext(uint32 Options, EShaderPlatform InShaderPlatform)
{
	bool bReregisterComponents = (Options & EOptions::ReregisterComponents) != 0;
	bool bRecreateRenderStates = ((Options & EOptions::RecreateRenderStates) != 0) && FApp::CanEverRender();

	bSyncWithRenderingThread = (Options & EOptions::SyncWithRenderingThread) != 0;
	bSkipRecacheStaticPermutations = (Options & EOptions::SkipRecacheStaticPermutations) != 0;
	bSkipUpdateStaticDrawLists = (Options & EOptions::SkipUpdateStaticDrawLists) != 0;

	if (bReregisterComponents)
	{
		ComponentReregisterContext = MakeUnique<FGlobalComponentReregisterContext>();
	}
	else if (bRecreateRenderStates)
	{
		ComponentRecreateRenderStateContext = MakeUnique<FGlobalComponentRecreateRenderStateContext>();
	}
	if (bSyncWithRenderingThread)
	{
		FlushRenderingCommands();
	}
	ShaderPlatform = InShaderPlatform;
}

void FMaterialUpdateContext::AddMaterial(UMaterial* Material)
{
	UpdatedMaterials.Add(Material);
	UpdatedMaterialInterfaces.Add(Material);
}

void FMaterialUpdateContext::AddMaterialInstance(UMaterialInstance* Instance)
{
	UpdatedMaterials.Add(Instance->GetMaterial());
	UpdatedMaterialInterfaces.Add(Instance);
}

void FMaterialUpdateContext::AddMaterialInterface(UMaterialInterface* Interface)
{
	UpdatedMaterials.Add(Interface->GetMaterial());
	UpdatedMaterialInterfaces.Add(Interface);
}

FMaterialUpdateContext::~FMaterialUpdateContext()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialUpdateContext::~FMaterialUpdateContext);

	double StartTime = FPlatformTime::Seconds();
	bool bProcess = false;

	// if the shader platform that was processed is not the currently rendering shader platform, 
	// there's no reason to update all of the runtime components
	UMaterialInterface::IterateOverActiveFeatureLevels([&](ERHIFeatureLevel::Type InFeatureLevel)
	{
		if (ShaderPlatform == GShaderPlatformForFeatureLevel[InFeatureLevel])
		{
			bProcess = true;
		}
	});

	if (!bProcess)
	{
		return;
	}

	UE::RenderCommandPipe::FSyncScope SyncScope;

	// Flush rendering commands even though we already did so in the constructor.
	// Anything may have happened since the constructor has run. The flush is
	// done once here to avoid calling it once per static permutation we update.
	if (bSyncWithRenderingThread)
	{
		FlushRenderingCommands();
	}

	TArray<const FMaterial*> MaterialResourcesToUpdate;
	TArray<UMaterialInstance*> InstancesToUpdate;

	const bool bUpdateStaticDrawLists = !bSkipUpdateStaticDrawLists && !ComponentReregisterContext && !ComponentRecreateRenderStateContext && FApp::CanEverRender();

	// If static draw lists must be updated, gather material resources from all updated materials.
	if (bUpdateStaticDrawLists)
	{
		for (TSet<UMaterial*>::TConstIterator It(UpdatedMaterials); It; ++It)
		{
			UMaterial* Material = *It;
			MaterialResourcesToUpdate.Append(Material->MaterialResources);
		}
	}

	// Go through all loaded material instances and recompile their static permutation resources if needed
	// This is necessary since the parent UMaterial stores information about how it should be rendered, (eg bUsesDistortion)
	// but the child can have its own shader map which may not contain all the shaders that the parent's settings indicate that it should.
	for (TObjectIterator<UMaterialInstance> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UMaterialInstance* CurrentMaterialInstance = *It;
		UMaterial* BaseMaterial = CurrentMaterialInstance->GetMaterial();

		if (UpdatedMaterials.Contains(BaseMaterial))
		{
			// Check to see if this instance is dependent on any of the material interfaces we directly updated.
			for (auto InterfaceIt = UpdatedMaterialInterfaces.CreateConstIterator(); InterfaceIt; ++InterfaceIt)
			{
				if (CurrentMaterialInstance->IsDependent(*InterfaceIt))
				{
					InstancesToUpdate.Add(CurrentMaterialInstance);
					break;
				}
			}
		}
	}

	// Material instances that use this base material must have their uniform expressions recached 
	// However, some material instances that use this base material may also depend on another MI with static parameters
	// So we must traverse upwards and ensure all parent instances that need updating are recached first.
	int32 NumInstancesWithStaticPermutations = 0;

	TFunction<void(UMaterialInstance* MI)> UpdateInstance = [&](UMaterialInstance* MI)
	{
		if (MI->Parent && InstancesToUpdate.Contains(MI->Parent))
		{
			if (UMaterialInstance* ParentInst = Cast<UMaterialInstance>(MI->Parent))
			{
				UpdateInstance(ParentInst);
			}
		}

#if WITH_EDITOR
		MI->UpdateCachedData();
#endif
		MI->RecacheUniformExpressions(true);
		if (!bSkipRecacheStaticPermutations)
		{
			MI->InitStaticPermutation(EMaterialShaderPrecompileMode::None);//bHasStaticPermutation can change.
		}
		if (MI->bHasStaticPermutationResource)
		{
			NumInstancesWithStaticPermutations++;
			// Collect FMaterial's that have been recompiled
			if (bUpdateStaticDrawLists)
			{
				MaterialResourcesToUpdate.Append(MI->StaticPermutationMaterialResources);
			}
		}
		InstancesToUpdate.Remove(MI);
	};

	while (InstancesToUpdate.Num() > 0)
	{
		UpdateInstance(InstancesToUpdate.Last());
	}
	
	for (FSceneInterface* Scene : GetRendererModule().GetAllocatedScenes())
	{
		ENQUEUE_RENDER_COMMAND(ReloadNaniteFixedFunctionBins)(
		[Scene](FRHICommandListImmediate& RHICmdList) mutable
		{
			Scene->ReloadNaniteFixedFunctionBins();
		});
	}

	if (bUpdateStaticDrawLists)
	{
		// Update static draw lists affected by any FMaterials that were recompiled
		// This is only needed if we aren't reregistering components which is not always
		// safe, e.g. while a component is being registered.
		GetRendererModule().UpdateStaticDrawListsForMaterials(MaterialResourcesToUpdate);
	}
	else if (ComponentReregisterContext)
	{
		ComponentReregisterContext.Reset();
	}
	else if (ComponentRecreateRenderStateContext)
	{
		ComponentRecreateRenderStateContext.Reset();
	}

	double EndTime = FPlatformTime::Seconds();

	if (UpdatedMaterials.Num() > 0)
	{
		UE_LOGF(LogMaterial, Verbose,
			   "%.2f seconds spent updating %d materials, %d interfaces, %d instances, %d with static permutations.",
			   (float)(EndTime - StartTime),
			   UpdatedMaterials.Num(),
			   UpdatedMaterialInterfaces.Num(),
			   InstancesToUpdate.Num(),
			   NumInstancesWithStaticPermutations
			);
	}
}

bool UMaterialInterface::IsPropertyActive(EMaterialProperty InProperty)const
{
	//TODO: Disable properties in instances based on the currently set overrides and other material settings?
	//For now just allow all properties in instances. 
	//This had to be refactored into the instance as some override properties alter the properties that are active.
	return false;
}

#if WITH_EDITOR

int32 UMaterialInterface::CompilePropertyEx(FMaterialCompiler* Compiler, const FGuid& AttributeID )
{
	return INDEX_NONE;
}

int32 UMaterialInterface::CompileProperty(FMaterialCompiler* Compiler, EMaterialProperty Property, uint32 ForceCastFlags)
{
	int32 Result = INDEX_NONE;

	if (IsPropertyActive(Property))
	{
		Result = CompilePropertyEx(Compiler, FMaterialAttributeDefinitionMap::GetID(Property));
	}
	else
	{
		Result = FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property);
	}

	if (Result == INDEX_NONE && Property == MP_FrontMaterial && Substrate::IsSubstrateEnabled())
	{
		Result = Compiler->SubstrateCreateAndRegisterNullMaterial();
	}

	if (Result != INDEX_NONE)
	{
		// Cast is always required to go between float and LWC
		const EMaterialValueType ResultType = Compiler->GetParameterType(Result);
		const EMaterialValueType PropertyType = FMaterialAttributeDefinitionMap::GetValueType(Property);
		if ((ForceCastFlags & MFCF_ForceCast) || IsLWCType(ResultType) != IsLWCType(PropertyType))
		{
			Result = Compiler->ForceCast(Result, PropertyType, ForceCastFlags);
		}
	}

	return Result;
}
#endif // WITH_EDITOR

void UMaterialInterface::AnalyzeMaterialProperty(EMaterialProperty InProperty, int32& OutNumTextureCoordinates, bool& bOutRequiresVertexData)
{
#if WITH_EDITORONLY_DATA
	FMaterialAnalysisResult Result;
	AnalyzeMaterialPropertyEx(InProperty, Result);

	OutNumTextureCoordinates = Result.TextureCoordinates.FindLast(true) + 1;
	bOutRequiresVertexData = Result.bRequiresVertexData;
#endif
}

void UMaterialInterface::AnalyzeMaterialPropertyEx(EMaterialProperty InProperty, FMaterialAnalysisResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	AnalyzeMaterialCompilationInCallback([InProperty, this](FMaterialCompiler* Compiler)
	{
		Compiler->SetMaterialProperty(InProperty);
		this->CompileProperty(Compiler, InProperty);
	}, OutResult);
#endif
}

void UMaterialInterface::AnalyzeMaterialCustomOutput(UMaterialExpressionCustomOutput* InCustomOutput, int32 InOutputIndex, FMaterialAnalysisResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	AnalyzeMaterialCompilationInCallback([InCustomOutput, InOutputIndex](FMaterialCompiler* Compiler)
	{
		Compiler->SetMaterialProperty(MP_MAX, InCustomOutput->GetShaderFrequency(InOutputIndex));
		InCustomOutput->Compile(Compiler, InOutputIndex);
	}, OutResult);
#endif
}

void UMaterialInterface::AnalyzeMaterialCompilationInCallback(TFunctionRef<void (FMaterialCompiler*)> InCompilationCallback, FMaterialAnalysisResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	// FHLSLMaterialTranslator collects all required information during translation, but these data are protected. Needs to
	// derive own class from it to get access to these data.
	class FMaterialAnalyzer : public FHLSLMaterialTranslator
	{
	public:
		FMaterialAnalyzer(FMaterial* InMaterial, FMaterialCompilationOutput& InMaterialCompilationOutput, const FStaticParameterSet& StaticParameters, EShaderPlatform InPlatform, EMaterialQualityLevel::Type InQualityLevel, ERHIFeatureLevel::Type InFeatureLevel)
			: FHLSLMaterialTranslator(InMaterial, InMaterialCompilationOutput, StaticParameters, InPlatform, InQualityLevel, InFeatureLevel)
		{}

		using FHLSLMaterialTranslator::AllocatedUserTexCoords;
		using FHLSLMaterialTranslator::ShadingModelsFromCompilation;
		using FHLSLMaterialTranslator::bUsesVertexColor;
		using FHLSLMaterialTranslator::bUsesTransformVector;
		using FHLSLMaterialTranslator::bNeedsWorldPositionExcludingShaderOffsets;
		using FHLSLMaterialTranslator::bUsesAOMaterialMask;
		using FHLSLMaterialTranslator::bUsesLightmapUVs;
		using FHLSLMaterialTranslator::bUsesVertexPosition;
	};

	FMaterialCompilationOutput TempOutput;
	FMaterialResource* MaterialResource = GetMaterialResource(GMaxRHIShaderPlatform);
	if (MaterialResource == nullptr)
	{
		ForceRecompileForRendering(); // Make sure material has a resource to avoid crash
		MaterialResource = GetMaterialResource(GMaxRHIShaderPlatform);
	}

	FMaterialShaderMapId ShaderMapID;
	MaterialResource->BuildShaderMapId(ShaderMapID, nullptr);
	FStaticParameterSet StaticParamSet;
	MaterialResource->GetStaticParameterSet(StaticParamSet);
	FMaterialAnalyzer MaterialTranslator(MaterialResource, TempOutput, StaticParamSet, GMaxRHIShaderPlatform, MaterialResource->GetQualityLevel(), GMaxRHIFeatureLevel);

	InCompilationCallback(&MaterialTranslator);

	// Request data from translator
	OutResult.TextureCoordinates = MaterialTranslator.AllocatedUserTexCoords;
	OutResult.ShadingModels = MaterialTranslator.ShadingModelsFromCompilation;
	OutResult.bRequiresVertexData =
		MaterialTranslator.bUsesVertexColor ||
		MaterialTranslator.bUsesTransformVector ||
		MaterialTranslator.bNeedsWorldPositionExcludingShaderOffsets ||
		MaterialTranslator.bUsesAOMaterialMask ||
		MaterialTranslator.bUsesLightmapUVs ||
		MaterialTranslator.bUsesVertexPosition;
	OutResult.bRequiresWorldPosition = MaterialTranslator.bNeedsWorldPositionExcludingShaderOffsets;
	OutResult.EstimatedNumTextureSamplesVS = TempOutput.EstimatedNumTextureSamplesVS;
	OutResult.EstimatedNumTextureSamplesPS = TempOutput.EstimatedNumTextureSamplesPS;
#endif
}

void UMaterialInterface::AnalyzeMaterialTranslationOutput(FMaterialResource* MaterialResource, EShaderPlatform ShaderPlatform, FMaterialAnalysisResult& OutResult)
{
	UMaterialInterface::AnalyzeMaterialTranslationOutput(MaterialResource, ShaderPlatform, false, OutResult);
}

void UMaterialInterface::AnalyzeMaterialTranslationOutput(FMaterialResource* MaterialResource, EShaderPlatform ShaderPlatform, bool ValidationMode, FMaterialAnalysisResult& OutResult)
{
#if WITH_EDITORONLY_DATA
	FStaticParameterSet StaticParamSet;
	MaterialResource->GetStaticParameterSet(StaticParamSet);

	FMaterialCompilationOutput TempOutput;
	FHLSLMaterialTranslator MaterialTranslator(MaterialResource, TempOutput, StaticParamSet, ShaderPlatform, MaterialResource->GetQualityLevel(), MaterialResource->GetFeatureLevel());
	MaterialTranslator.EnableValidationMode(ValidationMode);

	const EHLSLMaterialTranslatorResult TranslationResult = MaterialTranslator.Translate(true);

	OutResult.bTranslationSuccess = TranslationResult == EHLSLMaterialTranslatorResult::Success;
	OutResult.EstimatedNumTextureSamplesVS = TempOutput.EstimatedNumTextureSamplesVS;
	OutResult.EstimatedNumTextureSamplesPS = TempOutput.EstimatedNumTextureSamplesPS;
#else
	OutResult = {};
#endif
}

#if WITH_EDITOR
bool UMaterialInterface::IsTextureReferencedByProperty(EMaterialProperty InProperty, const UTexture* InTexture)
{
	class FFindTextureVisitor : public IMaterialExpressionVisitor
	{
	public:
		explicit FFindTextureVisitor(const UTexture* InTexture) : Texture(InTexture), FoundTexture(false) {}

		virtual EMaterialExpressionVisitResult Visit(UMaterialExpression* InExpression) override
		{
			const UMaterialExpression::ReferencedTextureArray ReferencedTextures = InExpression->GetReferencedTextures();
			if (ReferencedTextures.Contains( Texture ))
			{
				FoundTexture = true;
				return MVR_STOP;
			}
			return MVR_CONTINUE;
		}

		const UTexture* Texture;
		bool FoundTexture;
	};

	FMaterialResource* MaterialResource = GetMaterialResource(GMaxRHIShaderPlatform);
	if (!MaterialResource)
	{
		return false;
	}

	FMaterialCompilationOutput TempOutput;
	FMaterialShaderMapId ShaderMapID;
	MaterialResource->BuildShaderMapId(ShaderMapID, nullptr);
	FStaticParameterSet StaticParamSet;
	MaterialResource->GetStaticParameterSet(StaticParamSet);
	FHLSLMaterialTranslator MaterialTranslator(MaterialResource, TempOutput, StaticParamSet, GMaxRHIShaderPlatform, MaterialResource->GetQualityLevel(), GMaxRHIFeatureLevel);

	FFindTextureVisitor Visitor(InTexture);
	MaterialTranslator.VisitExpressionsForProperty(InProperty, Visitor);
	return Visitor.FoundTexture;
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
//Reorder the output index for any FExpressionInput connected to a UMaterialExpressionBreakMaterialAttributes.
//If the order of pins in the material results or the make/break attributes nodes changes 
//then the OutputIndex stored in any FExpressionInput coming from UMaterialExpressionBreakMaterialAttributes will be wrong and needs reordering.
void DoMaterialAttributeReorder(FExpressionInput* Input, const FPackageFileVersion& UEVer, int32 RenderObjVer, int32 UE5MainVer)
{
	if( Input && Input->Expression && Input->Expression->IsA(UMaterialExpressionBreakMaterialAttributes::StaticClass()) )
	{
		if(UEVer < VER_UE4_MATERIAL_ATTRIBUTES_REORDERING )
		{
			switch(Input->OutputIndex)
			{
			case 4: Input->OutputIndex = 7; break;
			case 5: Input->OutputIndex = 4; break;
			case 6: Input->OutputIndex = 5; break;
			case 7: Input->OutputIndex = 6; break;
			}
		}
		
		if(UEVer < VER_UE4_FIX_REFRACTION_INPUT_MASKING && Input->OutputIndex == 13 )
		{
			Input->Mask = 1;
			Input->MaskR = 1;
			Input->MaskG = 1;
			Input->MaskB = 1;
			Input->MaskA = 0;
		}

		// closest version to the clear coat change
		if(UEVer < VER_UE4_ADD_ROOTCOMPONENT_TO_FOLIAGEACTOR && Input->OutputIndex >= 12 )
		{
			Input->OutputIndex += 2;
		}

		if (RenderObjVer < FRenderingObjectVersion::AnisotropicMaterial)
		{
			int32 OutputIdx = Input->OutputIndex;

			if (OutputIdx >= 4)
			{
				++Input->OutputIndex;
			}
			
			if (OutputIdx >= 8)
			{
				++Input->OutputIndex;
			}
		}

		if (UE5MainVer < FUE5MainStreamObjectVersion::RemovingTessellationParameters)
		{
			// Removing MP_WorldDisplacement (11) and MP_TessellationMultiplier (12)
			if (Input->OutputIndex == 11 || Input->OutputIndex == 12)
			{
				Input->Expression = nullptr;
			}
			else if (Input->OutputIndex >= 13)
			{
				Input->OutputIndex -= 2;
			}
		}
	}
}
#endif // WITH_EDITORONLY_DATA
//////////////////////////////////////////////////////////////////////////

FMaterialInstanceBasePropertyOverrides::FMaterialInstanceBasePropertyOverrides()
	:bOverride_OpacityMaskClipValue(false)
	,bOverride_BlendMode(false)
	,bOverride_ShadingModel(false)
	,bOverride_DitheredLODTransition(false)
	,bOverride_CastDynamicShadowAsMasked(false)
	,bOverride_TwoSided(false)
	,bOverride_bIsThinSurface(false)
	,bOverride_OutputTranslucentVelocity(false)
	,bOverride_bHasPixelAnimation(false)
	,bOverride_bEnableTessellation(false)
	,bOverride_DisplacementScaling(false)
	,bOverride_bEnableDisplacementFade(false)
	,bOverride_DisplacementFadeRange(false)
	,bOverride_MaxWorldPositionOffsetDisplacement(false)
	,bOverride_CompatibleWithLumenCardSharing(false)
	,bOverride_UsageFlags(0)
	,TwoSided(0)
	,bIsThinSurface(false)
	,DitheredLODTransition(0)
	,bCastDynamicShadowAsMasked(false)
	,bOutputTranslucentVelocity(false)
	,bHasPixelAnimation(false)
	,bEnableTessellation(false)
	,bEnableDisplacementFade(false)
	,bCompatibleWithLumenCardSharing(false)
	,BlendMode(BLEND_Opaque)
	,ShadingModel(MSM_DefaultLit)
	,OpacityMaskClipValue(.333333f)
	,DisplacementScaling()
	,DisplacementFadeRange()
	,MaxWorldPositionOffsetDisplacement(0.0f)
	,UsageFlags(0)
{
}

bool FMaterialInstanceBasePropertyOverrides::operator==(const FMaterialInstanceBasePropertyOverrides& Other)const
{
	return	bOverride_OpacityMaskClipValue == Other.bOverride_OpacityMaskClipValue &&
		bOverride_BlendMode == Other.bOverride_BlendMode &&
		bOverride_ShadingModel == Other.bOverride_ShadingModel &&
		bOverride_DitheredLODTransition == Other.bOverride_DitheredLODTransition &&
		bOverride_CastDynamicShadowAsMasked == Other.bOverride_CastDynamicShadowAsMasked &&
		bOverride_TwoSided == Other.bOverride_TwoSided &&
		bOverride_bIsThinSurface == Other.bOverride_bIsThinSurface &&
		bOverride_OutputTranslucentVelocity == Other.bOverride_OutputTranslucentVelocity &&
		bOverride_bHasPixelAnimation == Other.bOverride_bHasPixelAnimation &&
		bOverride_bEnableTessellation == Other.bOverride_bEnableTessellation &&
		bOverride_DisplacementScaling == Other.bOverride_DisplacementScaling &&
		bOverride_bEnableDisplacementFade == Other.bOverride_bEnableDisplacementFade &&
		bOverride_DisplacementFadeRange == Other.bOverride_DisplacementFadeRange &&
		bOverride_MaxWorldPositionOffsetDisplacement == Other.bOverride_MaxWorldPositionOffsetDisplacement &&
		bOverride_CompatibleWithLumenCardSharing == Other.bOverride_CompatibleWithLumenCardSharing &&
		bOverride_UsageFlags == Other.bOverride_UsageFlags &&
		OpacityMaskClipValue == Other.OpacityMaskClipValue &&
		BlendMode == Other.BlendMode &&
		TwoSided == Other.TwoSided &&
		ShadingModel == Other.ShadingModel &&
		bIsThinSurface == Other.bIsThinSurface &&
		DitheredLODTransition == Other.DitheredLODTransition &&
		bCastDynamicShadowAsMasked == Other.bCastDynamicShadowAsMasked &&
		bOutputTranslucentVelocity == Other.bOutputTranslucentVelocity &&
		bHasPixelAnimation == Other.bHasPixelAnimation &&
		bEnableTessellation == Other.bEnableTessellation &&
		DisplacementScaling == Other.DisplacementScaling &&
		bEnableDisplacementFade == Other.bEnableDisplacementFade &&
		DisplacementFadeRange == Other.DisplacementFadeRange &&
		bCompatibleWithLumenCardSharing == Other.bCompatibleWithLumenCardSharing &&
		MaxWorldPositionOffsetDisplacement == Other.MaxWorldPositionOffsetDisplacement &&
		UsageFlags == Other.UsageFlags;
}

bool FMaterialInstanceBasePropertyOverrides::operator!=(const FMaterialInstanceBasePropertyOverrides& Other)const
{
	return !(*this == Other);
}

//////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
bool FMaterialShaderMapId::ContainsShaderType(const FShaderType* ShaderType, int32 PermutationId) const
{
	for (int32 TypeIndex = 0; TypeIndex < ShaderTypeDependencies.Num(); TypeIndex++)
	{
		if (ShaderTypeDependencies[TypeIndex].ShaderTypeName == ShaderType->GetHashedName() &&
			ShaderTypeDependencies[TypeIndex].PermutationId == PermutationId)
		{
			return true;
		}
	}

	return false;
}

bool FMaterialShaderMapId::ContainsShaderPipelineType(const FShaderPipelineType* ShaderPipelineType) const
{
	for (int32 TypeIndex = 0; TypeIndex < ShaderPipelineTypeDependencies.Num(); TypeIndex++)
	{
		if (ShaderPipelineTypeDependencies[TypeIndex].ShaderPipelineTypeName == ShaderPipelineType->GetHashedName())
		{
			return true;
		}
	}

	return false;
}

bool FMaterialShaderMapId::ContainsVertexFactoryType(const FVertexFactoryType* VFType) const
{
	for (int32 TypeIndex = 0; TypeIndex < VertexFactoryTypeDependencies.Num(); TypeIndex++)
	{
		if (VertexFactoryTypeDependencies[TypeIndex].VertexFactoryTypeName == VFType->GetHashedName())
		{
			return true;
		}
	}

	return false;
}
#endif // WITH_EDITOR
//////////////////////////////////////////////////////////////////////////

FMaterialResourceMemoryWriter::FMaterialResourceMemoryWriter(FArchive& Ar) :
	FMemoryWriter(Bytes, Ar.IsPersistent(), false, TEXT("FShaderMapMemoryWriter")),
	ParentAr(&Ar)
{
	check(Ar.IsSaving());
	this->SetByteSwapping(Ar.IsByteSwapping());
	this->SetSavePackageData(Ar.GetSavePackageData());
}

FMaterialResourceMemoryWriter::~FMaterialResourceMemoryWriter()
{
	SerializeToParentArchive();
}

FArchive& FMaterialResourceMemoryWriter::operator<<(class FName& Name)
{
	const int32* Idx = Name2Indices.Find(Name.GetDisplayIndex());
	int32 NewIdx;
	if (Idx)
	{
		NewIdx = *Idx;
	}
	else
	{
		NewIdx = Name2Indices.Num();
		Name2Indices.Add(Name.GetDisplayIndex(), NewIdx);
	}
	int32 InstNum = Name.GetNumber();
	static_assert(sizeof(decltype(DeclVal<FName>().GetNumber())) == sizeof(int32), "FName serialization in FMaterialResourceMemoryWriter requires changing, InstNum is no longer 32-bit");
	*this << NewIdx << InstNum;
	return *this;
}

void FMaterialResourceMemoryWriter::SerializeToParentArchive()
{
	FArchive& Ar = *ParentAr;
	check(Ar.IsSaving() && this->IsByteSwapping() == Ar.IsByteSwapping());

	// Make a array of unique names used by the shader map
	TArray<FNameEntryId> DisplayIndices;
	auto NumNames = Name2Indices.Num();
	DisplayIndices.Empty(NumNames);
	DisplayIndices.AddDefaulted(NumNames);
	for (const auto& Pair : Name2Indices)
	{
		DisplayIndices[Pair.Value] = Pair.Key;
	}

	Ar << NumNames;
	for (FNameEntryId DisplayIdx : DisplayIndices)
	{
		FName::GetEntry(DisplayIdx)->Write(Ar);
	}
	
	Ar << Locs;
	auto NumBytes = Bytes.Num();
	Ar << NumBytes;
	Ar.Serialize(&Bytes[0], NumBytes);
}

FMaterialResourceProxyReader::FMaterialResourceProxyReader(
	FArchive& Ar,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel) :
	FArchiveProxy(Ar),
	OffsetToEnd(-1)
{
	check(InnerArchive.IsLoading());
	Initialize(FeatureLevel, QualityLevel, FeatureLevel != ERHIFeatureLevel::Num);
}

FMaterialResourceProxyReader::FMaterialResourceProxyReader(
	const TCHAR* Filename,
	uint32 NameMapOffset,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel) :
	TUniquePtr(IFileManager::Get().CreateFileReader(Filename, FILEREAD_NoFail)), // Create and store the FileArchive
	FArchiveProxy(*Get()), // Now link it to the archive proxy
	OffsetToEnd(-1)
{
	InnerArchive.Seek(NameMapOffset);
	Initialize(FeatureLevel, QualityLevel);
}

FMaterialResourceProxyReader::~FMaterialResourceProxyReader()
{
	if (OffsetToEnd != -1)
	{
		InnerArchive.Seek(OffsetToEnd);
	}
}

FArchive& FMaterialResourceProxyReader::operator<<(class FName& Name)
{
	int32 NameIdx;
	int32 InstNum;
	static_assert(sizeof(decltype(DeclVal<FName>().GetNumber())) == sizeof(int32), "FName serialization in FMaterialResourceProxyReader requires changing, InstNum is no longer 32-bit");
	InnerArchive << NameIdx << InstNum;
	if (NameIdx >= 0 && NameIdx < Names.Num())
	{
		Name = FName(Names[NameIdx], InstNum);
	}
	else
	{
		UE_LOGF(LogMaterial, Fatal, "FMaterialResourceProxyReader: deserialized an invalid FName, NameIdx=%d, Names.Num()=%d (Offset=%lld, InnerArchive.Tell()=%lld, OffsetToFirstResource=%lld)", 
			NameIdx, Names.Num(), Tell(), InnerArchive.Tell(), OffsetToFirstResource);
	}
	return *this;
}

void FMaterialResourceProxyReader::Initialize(
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel,
	bool bSeekToEnd)
{
	SCOPED_LOADTIMER(FMaterialResourceProxyReader_Initialize);

	decltype(Names.Num()) NumNames;
	InnerArchive << NumNames;
	Names.Empty(NumNames);
	for (int32 Idx = 0; Idx < NumNames; ++Idx)
	{
		FNameEntrySerialized Entry(ENAME_LinkerConstructor);
		InnerArchive << Entry;
		Names.Add(Entry);
	}

	TArray<FMaterialResourceLocOnDisk> Locs;
	InnerArchive << Locs;
	check(Locs[0].Offset == 0);
	decltype(DeclVal<TArray<uint8>>().Num()) NumBytes;
	InnerArchive << NumBytes;

	OffsetToFirstResource = InnerArchive.Tell();

	if (bSeekToEnd)
	{
		OffsetToEnd = OffsetToFirstResource + NumBytes;
	}
}

typedef TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>> FMaterialsToUpdateMap;

void SetShaderMapsOnMaterialResources_RenderThread(FRHICommandListImmediate& RHICmdList, FMaterialsToUpdateMap& MaterialsToUpdate)
{
	SCOPE_CYCLE_COUNTER(STAT_Scene_SetShaderMapsOnMaterialResources_RT);

#if WITH_EDITOR
	bool bUpdateFeatureLevel[ERHIFeatureLevel::Num] = { false };

	// Async RDG tasks can call FMaterialShader::SetParameters which touch the material uniform expression cache.
	FRDGBuilder::WaitForAsyncExecuteTask();

	for (auto& It : MaterialsToUpdate)
	{
		FMaterial* Material = It.Key;
		Material->SetRenderingThreadShaderMap(It.Value);
		//check(!ShaderMap || ShaderMap->IsValidForRendering());
		bUpdateFeatureLevel[Material->GetFeatureLevel()] = true;
	}

	// Iterate through all loaded material render proxies and recache their uniform expressions if needed
	// This search does not scale well, but is only used when uploading async shader compile results
	for (int32 FeatureLevelIndex = 0; FeatureLevelIndex < UE_ARRAY_COUNT(bUpdateFeatureLevel); ++FeatureLevelIndex)
	{
		if (bUpdateFeatureLevel[FeatureLevelIndex])
		{
			const ERHIFeatureLevel::Type MaterialFeatureLevel = (ERHIFeatureLevel::Type) FeatureLevelIndex;

			FScopeLock Locker(&FMaterialRenderProxy::GetMaterialRenderProxyMapLock());
			for (FMaterialRenderProxy* MaterialProxy : FMaterialRenderProxy::GetMaterialRenderProxyMap())
			{
				const FMaterial* Material = MaterialProxy->GetMaterialNoFallback(MaterialFeatureLevel);

				// Using ContainsByHash so we can pass a raw-ptr to TMap method that wants a TRefCountPtr
				if (Material && Material->GetRenderingThreadShaderMap() && MaterialsToUpdate.ContainsByHash(GetTypeHash(Material), Material))
				{
					MaterialProxy->CacheUniformExpressions(RHICmdList, true);
				}
			}
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

void FMaterial::SetShaderMapsOnMaterialResources(const TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>>& MaterialsToUpdate)
{
	for (const auto& It : MaterialsToUpdate)
	{
		FMaterial* Material = It.Key;
		const TRefCountPtr<FMaterialShaderMap>& ShaderMap = It.Value;
		Material->GameThreadShaderMap = ShaderMap;
		if (LIKELY(Material->GameThreadShaderMap))
		{
			Material->GameThreadShaderMap->GetResource()->SetOwnerName(Material->GetOwnerFName());
			Material->bGameThreadShaderMapIsComplete.store(ShaderMap->IsComplete(Material, true), std::memory_order_relaxed);
		}
		else
		{
			Material->bGameThreadShaderMapIsComplete.store(false, std::memory_order_relaxed);
		}
	}

	ENQUEUE_RENDER_COMMAND(FSetShaderMapOnMaterialResources)(
	[InMaterialsToUpdate = MaterialsToUpdate](FRHICommandListImmediate& RHICmdList) mutable
	{
		SetShaderMapsOnMaterialResources_RenderThread(RHICmdList, InMaterialsToUpdate);
	});

	// The parent material of some material in GetMaterialRenderProxyMap may be destroyed before SetShaderMapsOnMaterialResources_RenderThread
	// executes. Iterate over all materials in the proxy map and bump their parent fences so that their IsReadyForFinishDestroy() will return false
	// until the fence executes, which is after the RT task above completes.
	// This guarantees that the parents of all materials in GetMaterialRenderProxyMap are safe from deletion while the RT task executes.
	FScopeLock Locker(&FMaterialRenderProxy::GetMaterialRenderProxyMapLock());
	for (FMaterialRenderProxy* Proxy : FMaterialRenderProxy::GetMaterialRenderProxyMap())
	{
		UMaterialInstance* MI = Cast<UMaterialInstance>(Proxy->GetMaterialInterface());
		while (MI)
		{
			if (MI->Parent)
			{
				MI->Parent->ParentRefFence.BeginFence();
			}
			MI = Cast<UMaterialInstance>(MI->Parent);
		}
	}
}

#endif // WITH_EDITOR

FMaterialParameterValue::FMaterialParameterValue(EMaterialParameterType InType, const UE::Shader::FValue& InValue)
{
	switch (InType)
	{
	case EMaterialParameterType::Scalar: *this = InValue.AsFloatScalar(); break;
	case EMaterialParameterType::Vector: *this = InValue.AsLinearColor(); break;
	case EMaterialParameterType::DoubleVector: *this = InValue.AsVector4d(); break;
	case EMaterialParameterType::StaticSwitch: *this = InValue.AsBoolScalar(); break;
	case EMaterialParameterType::StaticComponentMask:
	{
		const UE::Shader::FBoolValue BoolValue = InValue.AsBool();
		*this = FMaterialParameterValue(BoolValue[0], BoolValue[1], BoolValue[2], BoolValue[3]);
		break;
	}
	default: ensure(false); Type = EMaterialParameterType::None; break;
	}
}

UE::Shader::FValue FMaterialParameterValue::AsShaderValue() const
{
	switch (Type)
	{
	case EMaterialParameterType::Scalar: return Float[0];
	case EMaterialParameterType::Vector: return UE::Shader::FValue(Float[0], Float[1], Float[2], Float[3]);
	case EMaterialParameterType::DoubleVector: return UE::Shader::FValue(Double[0], Double[1], Double[2], Double[3]);
	case EMaterialParameterType::StaticSwitch: return Bool[0];
	case EMaterialParameterType::StaticComponentMask: return UE::Shader::FValue(Bool[0], Bool[1], Bool[2], Bool[3]);
	case EMaterialParameterType::Texture:
	case EMaterialParameterType::TextureCollection:
	case EMaterialParameterType::ParameterCollection:
	case EMaterialParameterType::Font:
	case EMaterialParameterType::RuntimeVirtualTexture:
	case EMaterialParameterType::SparseVolumeTexture:
		// Non-numeric types, can't represent as shader values
		return UE::Shader::FValue();
	default:
		checkNoEntry();
		return UE::Shader::FValue();
	}
}

UObject* FMaterialParameterValue::AsTextureObject() const
{
	UObject* Result = nullptr;
	switch (Type)
	{
	case EMaterialParameterType::Texture: Result = Texture; break;
	case EMaterialParameterType::RuntimeVirtualTexture: Result = RuntimeVirtualTexture; break;
	case EMaterialParameterType::SparseVolumeTexture: Result = SparseVolumeTexture; break;
	case EMaterialParameterType::Font:
		if (Font.Value && Font.Value->Textures.IsValidIndex(Font.Page))
		{
			Result = Font.Value->Textures[Font.Page];
		}
		break;
	default:
		break;
	}
	return Result;
}

UE::Shader::FType GetShaderValueType(EMaterialParameterType Type)
{
	switch (Type)
	{
	case EMaterialParameterType::Scalar: return UE::Shader::EValueType::Float1;
	case EMaterialParameterType::Vector: return UE::Shader::EValueType::Float4;
	case EMaterialParameterType::DoubleVector: return UE::Shader::EValueType::Double4;
	case EMaterialParameterType::StaticSwitch: return UE::Shader::EValueType::Bool1;
	case EMaterialParameterType::StaticComponentMask: return UE::Shader::EValueType::Bool4;
	case EMaterialParameterType::Texture:
	case EMaterialParameterType::TextureCollection:
	case EMaterialParameterType::ParameterCollection:
	case EMaterialParameterType::RuntimeVirtualTexture:
	case EMaterialParameterType::Font:
		return FMaterialTextureValue::GetTypeName();
	case EMaterialParameterType::SparseVolumeTexture:
		return UE::Shader::EValueType::Void; // TODO
	default:
		checkNoEntry();
		return UE::Shader::EValueType::Void;
	}
}

template<typename TParameter>
static bool RemapParameterLayerIndex(TArrayView<const int32> IndexRemap, const TParameter& ParameterInfo, TParameter& OutResult)
{
	int32 NewIndex = INDEX_NONE;
	switch (ParameterInfo.Association)
	{
	case GlobalParameter:
		// No remapping for global parameters
		OutResult = ParameterInfo;
		return true;
	case LayerParameter:
		if (IndexRemap.IsValidIndex(ParameterInfo.Index))
		{
			NewIndex = IndexRemap[ParameterInfo.Index];
			if (NewIndex != INDEX_NONE)
			{
				OutResult = ParameterInfo;
				OutResult.Index = NewIndex;
				return true;
			}
		}
		break;
	case BlendParameter:
		if (IndexRemap.IsValidIndex(ParameterInfo.Index + 1))
		{
			// Indices for blend parameters are offset by 1
			NewIndex = IndexRemap[ParameterInfo.Index + 1];
			if (NewIndex != INDEX_NONE)
			{
				check(NewIndex > 0);
				OutResult = ParameterInfo;
				OutResult.Index = NewIndex - 1;
				return true;
			}
		}
		break;
	default:
		checkNoEntry();
		break;
	}
	return false;
}

void FMaterialParameterInfo::AppendString(FString& Out) const
{
	FShaderKeyGenerator KeyGen(Out);
	Append(KeyGen);
}

void FMaterialParameterInfo::Append(FShaderKeyGenerator& KeyGen) const
{
	KeyGen.Append(Name);
	KeyGen.Append(Association);
	KeyGen.Append(Index);
}

bool FMaterialParameterInfo::RemapLayerIndex(TArrayView<const int32> IndexRemap, FMaterialParameterInfo& OutResult) const
{
	return RemapParameterLayerIndex(IndexRemap, *this, OutResult);
}

bool FMemoryImageMaterialParameterInfo::RemapLayerIndex(TArrayView<const int32> IndexRemap, FMemoryImageMaterialParameterInfo& OutResult) const
{
	return RemapParameterLayerIndex(IndexRemap, *this, OutResult);
}

static_assert(TIsTrivial<FMaterialShaderParametersBase>::Value, "FMaterialShaderParametersBase - Must be trivial for serialization and hashing");

FMaterialShaderParametersBase::FMaterialShaderParametersBase(const FMaterial* InMaterial)
{
	// zero-initialize is required even when InMaterial!=null so that all bytes in sizeof(*this) are initialized
	// and we get consistent hashes.
	FMemory::Memzero(*this);
	if (!InMaterial)
	{
		return;
	}

	MaterialDomain = InMaterial->GetMaterialDomain();
	ShadingModels = InMaterial->GetShadingModels();
	BlendMode = InMaterial->GetBlendMode();
	FeatureLevel = InMaterial->GetFeatureLevel();
	QualityLevel = InMaterial->GetQualityLevel();
	PreshaderGap = InMaterial->GetPreshaderGap();
	BlendableLocation = InMaterial->GetBlendableLocation();
	NumCustomizedUVs = InMaterial->GetNumCustomizedUVs();
	NumMaterialCacheTags = InMaterial->GetNumMaterialCacheTags();
	StencilCompare = InMaterial->GetStencilCompare();
	bIsDefaultMaterial = InMaterial->IsDefaultMaterial();
	bIsSpecialEngineMaterial = InMaterial->IsSpecialEngineMaterial();
	bIsMasked = InMaterial->IsMasked();
	bIsDitherMasked = InMaterial->IsDitherMasked();
	bIsTwoSided = InMaterial->IsTwoSided();
	bIsThinSurface = InMaterial->IsThinSurface();
	bIsDistorted = InMaterial->IsDistorted();
	bShouldCastDynamicShadows = InMaterial->ShouldCastDynamicShadows();

	if (Substrate::IsSubstrateEnabled())
	{
		bIsSubstrateMaterial = InMaterial->IsSubstrateMaterial();
		bHasDiffuseAlbedoConnected  = InMaterial->HasMaterialPropertyConnected(MP_DiffuseColor);
		bHasF0Connected = InMaterial->HasMaterialPropertyConnected(MP_SpecularColor);
		bHasBaseColorConnected = InMaterial->HasMaterialPropertyConnected(MP_BaseColor);
		bHasNormalConnected = InMaterial->HasMaterialPropertyConnected(MP_Normal);
		bHasRoughnessConnected = InMaterial->HasMaterialPropertyConnected(MP_Roughness);
		bHasSpecularConnected = InMaterial->HasMaterialPropertyConnected(MP_Specular);
		bHasMetallicConnected = InMaterial->HasMaterialPropertyConnected(MP_Metallic);
		bHasEmissiveColorConnected = InMaterial->HasMaterialPropertyConnected(MP_EmissiveColor);
		bHasAmbientOcclusionConnected = InMaterial->HasMaterialPropertyConnected(MP_AmbientOcclusion);
	}
	else
	{
		bHasBaseColorConnected = InMaterial->HasBaseColorConnected();
		bHasNormalConnected = InMaterial->HasNormalConnected();
		bHasRoughnessConnected = InMaterial->HasRoughnessConnected();
		bHasSpecularConnected = InMaterial->HasSpecularConnected();
		bHasMetallicConnected = InMaterial->HasMetallicConnected();
		bHasEmissiveColorConnected = InMaterial->HasEmissiveColorConnected();
		bHasAmbientOcclusionConnected = InMaterial->HasAmbientOcclusionConnected();
	}

	bIsTessellationEnabled = InMaterial->IsTessellationEnabled();
	bIsUsedWithStaticLighting = InMaterial->IsUsedWithStaticLighting();
	bIsUsedWithParticleSprites = InMaterial->IsUsedWithParticleSprites();
	bIsUsedWithMeshParticles = InMaterial->IsUsedWithMeshParticles();
	bIsUsedWithNiagaraSprites = InMaterial->IsUsedWithNiagaraSprites();
	bIsUsedWithNiagaraMeshParticles = InMaterial->IsUsedWithNiagaraMeshParticles();
	bIsUsedWithNiagaraRibbons = InMaterial->IsUsedWithNiagaraRibbons();
	bIsUsedWithLandscape = InMaterial->IsUsedWithLandscape();
	bIsUsedWithBeamTrails = InMaterial->IsUsedWithBeamTrails();
	bIsUsedWithSplineMeshes = InMaterial->IsUsedWithSplineMeshes();
	bIsUsedWithSkeletalMesh = InMaterial->IsUsedWithSkeletalMesh();
	bIsUsedWithMorphTargets = InMaterial->IsUsedWithMorphTargets();
	bIsUsedWithAPEXCloth = InMaterial->IsUsedWithAPEXCloth();
	bIsUsedWithGeometryCache = InMaterial->IsUsedWithGeometryCache();
	bIsUsedWithGeometryCollections = InMaterial->IsUsedWithGeometryCollections();
	bIsUsedWithHairStrands = InMaterial->IsUsedWithHairStrands();
	bIsUsedWithWater = InMaterial->IsUsedWithWater();
	bIsTranslucencyWritingVelocity = InMaterial->IsTranslucencyWritingVelocity();
	bIsTranslucencyWritingCustomDepth = InMaterial->IsTranslucencyWritingCustomDepth();
	bIsDitheredLODTransition = InMaterial->IsDitheredLODTransition();
	bIsUsedWithInstancedStaticMeshes = InMaterial->IsUsedWithInstancedStaticMeshes();
	bHasPerInstanceRandom = InMaterial->HasPerInstanceRandom();
	bHasRuntimeVirtualTextureOutput = InMaterial->HasRuntimeVirtualTextureOutput();
	bIsUsedWithLidarPointCloud = InMaterial->IsUsedWithLidarPointCloud();
	bIsUsedWithVirtualHeightfieldMesh = InMaterial->IsUsedWithVirtualHeightfieldMesh();
	bIsUsedWithNeuralNetworks = InMaterial->IsUsedWithNeuralNetworks();
	bIsUsedWithNanite = InMaterial->IsUsedWithNanite();
	bIsUsedWithVoxels = InMaterial->IsUsedWithVoxels();
	bIsUsedWithCurves = InMaterial->IsUsedWithCurves();
	bIsStencilTestEnabled = InMaterial->IsStencilTestEnabled();
	bIsTranslucencySurface = InMaterial->GetTranslucencyLightingMode() == ETranslucencyLightingMode::TLM_Surface || InMaterial->GetTranslucencyLightingMode() == ETranslucencyLightingMode::TLM_SurfacePerPixelLighting;
	bShouldDisableDepthTest = InMaterial->ShouldDisableDepthTest();
	bHasRenderTracePhysicalMaterialOutput = InMaterial->HasRenderTracePhysicalMaterialOutputs();
	bIsUsedWithVolumetricCloud = InMaterial->IsUsedWithVolumetricCloud();
	bIsUsedWithHeterogeneousVolumes = InMaterial->IsUsedWithHeterogeneousVolumes();
	bIsMobileSeparateTranslucencyEnabled = InMaterial->IsMobileSeparateTranslucencyEnabled();
	bAlwaysEvaluateWorldPositionOffset = InMaterial->ShouldAlwaysEvaluateWorldPositionOffset();
	bDisablePreExposureScale = InMaterial->GetDisablePreExposureScale();
	bAllowVariableRateShading = InMaterial->IsVariableRateShadingAllowed();
	bIsUsedWithStaticMesh = InMaterial->IsUsedWithStaticMesh();
	bIsTranslucentSurfaceLighting = InMaterial->GetTranslucencyLightingMode() == ETranslucencyLightingMode::TLM_SurfacePerPixelLighting; // get lighting mode for mobile translucent receive CSM 
	bHasFirstPersonInterpolation = InMaterial->HasFirstPersonOutput();
	bSamplesMaterialCache = InMaterial->SamplesMaterialCache();
	bHasMaterialCacheOutput = InMaterial->HasMaterialCacheOutput();
	bIsUsedWithMeshDeformer = InMaterial->IsUsedWithMeshDeformer();
	bIsUsedWithInstancedSkinnedMesh = InMaterial->IsUsedWithInstancedSkinnedMesh();
}

FMaterialShaderParameters::FMaterialShaderParameters(const FMaterial* InMaterial)
	: FMaterialShaderParametersBase(InMaterial)
{
#if WITH_EDITOR
	if (InMaterial)
	{
		InMaterial->GetShaderTags(MaterialShaderTags);
	}
#endif

}

void FMaterialShaderParameters::InitializeShaderLayoutParamsFromMaterial(const FMaterial* InMaterial)
{
	if (Substrate::IsSubstrateEnabled())
	{
		bHasAnisotropyConnected = InMaterial->HasMaterialPropertyConnected(MP_Anisotropy);
	}
	else
	{
		bHasAnisotropyConnected = InMaterial->HasAnisotropyConnected();
	}

	bHasVertexPositionOffsetConnected = InMaterial->HasVertexPositionOffsetConnected();
	bHasPixelDepthOffsetConnected = InMaterial->HasPixelDepthOffsetConnected();
	bHasDisplacementConnected = InMaterial->HasDisplacementConnected();
	bMaterialMayModifyMeshPosition = InMaterial->MaterialMayModifyMeshPosition();
	bHasPerInstanceCustomData = InMaterial->HasPerInstanceCustomData();
	bHasVertexInterpolator = InMaterial->HasVertexInterpolator();
	bUsesTemporalResponsiveness = InMaterial->UsesTemporalResponsiveness();
	bUsesMotionVectorWorldOffset = InMaterial->UsesMotionVectorWorldOffset();
	bWritesEveryPixel = InMaterial->WritesEveryPixel(/*bShadowPass*/false);
	bWritesEveryPixelShadowPass = InMaterial->WritesEveryPixel(/*bShadowPass*/true);
}

void FMaterialShaderParameters::InitializeShaderLayoutParamsFromCompOutput(const FMaterial* InMaterial, const FMaterialCompilationOutput& InCompilationOutput)
{
	bHasAnisotropyConnected = InCompilationOutput.bUsesAnisotropy;
	bHasVertexPositionOffsetConnected = InCompilationOutput.bUsesWorldPositionOffset;
	bHasPixelDepthOffsetConnected = InCompilationOutput.bUsesPixelDepthOffset;
	bHasDisplacementConnected = InCompilationOutput.bUsesDisplacement;
	bMaterialMayModifyMeshPosition = InCompilationOutput.bModifiesMeshPosition;
	bHasPerInstanceCustomData = InCompilationOutput.bUsesPerInstanceCustomData;
	bHasVertexInterpolator = InCompilationOutput.bUsesVertexInterpolator;
	bUsesTemporalResponsiveness = InCompilationOutput.bUsesTemporalResponsiveness;
	bUsesMotionVectorWorldOffset = InCompilationOutput.bUsesMotionVectorWorldOffset;
	
	// Consider whether opacity mask is actually used when calculating WritesEveryPixel().
	// Treat the case where a custom HLSL expression is discarding pixels as equivalent to the use of opacity mask here.
	const bool bVFTypeSupportsNullPixelShader = !InMaterial->IsUsedWithInstancedStaticMeshes();
	const bool bUsesOpacityMaskOrDiscard = InCompilationOutput.bUsesOpacityMask || InCompilationOutput.bUsesCustomHLSLPixelDiscard || (bShouldCastDynamicShadows && InCompilationOutput.bUsesOpacity);
	bWritesEveryPixel = InMaterial->WritesEveryPixel(/*bShadowPass*/false, bVFTypeSupportsNullPixelShader, !bUsesOpacityMaskOrDiscard);
	bWritesEveryPixelShadowPass = InMaterial->WritesEveryPixel(/*bShadowPass*/true, bVFTypeSupportsNullPixelShader, !bUsesOpacityMaskOrDiscard);
}

FCompilationShaderParameters::FCompilationShaderParameters(const FMaterial* InMaterial, const FMaterialCompilationOutput& InCompilationOutput)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: FMaterialShaderParameters(InMaterial)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	static const int32 bUseShaderCompilationParameters = CVarMaterialUseShaderCompilationParameters.GetValueOnAnyThread();
	if (bUseShaderCompilationParameters)
	{
		InitializeShaderLayoutParamsFromCompOutput(InMaterial, InCompilationOutput);
	}
	else
	{
		InitializeShaderLayoutParamsFromMaterial(InMaterial);
	}
}

FConservativeShaderParameters::FConservativeShaderParameters()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: FMaterialShaderParameters() {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FConservativeShaderParameters::FConservativeShaderParameters(const FMaterial* InMaterial)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: FMaterialShaderParameters(InMaterial)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	InitializeShaderLayoutParamsFromMaterial(InMaterial);
}


#if WITH_EDITOR
void FMaterialShaderParametersBase::Save(FCbWriter& Writer) const
{
	Writer.AddBinary(FMemoryView(this, sizeof(*this)));
}

bool FMaterialShaderParametersBase::TryLoad(FCbFieldView Field)
{
	*this = FMaterialShaderParametersBase();

	FMemoryView MemoryView = Field.AsBinaryView();
	if (Field.HasError() || MemoryView.GetSize() != sizeof(*this))
	{
		return false;
	}
	FMemory::Memcpy(this, MemoryView.GetData(), sizeof(*this));
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, FMaterialShaderParametersBase& OutValue)
{
	return OutValue.TryLoad(Field);
}

void FConservativeShaderParameters::Record(FMaterialKeyGeneratorContext& Context)
{
	Context.Record("MaterialShaderParametersBase", static_cast<FMaterialShaderParametersBase&>(*this));
	Context.Record("MaterialShaderTags", MaterialShaderTags);
}

int32 FMaterialCompilationOutput::FindOrAddUserSceneTexture(FName UserSceneTexture)
{
	FScriptName UserSceneTextureScript(UserSceneTexture);
	int32 FoundIndex = UserSceneTextureInputs.Find(UserSceneTextureScript);
	if (FoundIndex != INDEX_NONE)
	{
		return FoundIndex + PPI_UserSceneTexture0;
	}

	if (UserSceneTextureInputs.Num() < kPostProcessMaterialInputCountMax)
	{
		FoundIndex = UserSceneTextureInputs.Num();
		UserSceneTextureInputs.Add(UserSceneTextureScript);
		return FoundIndex + PPI_UserSceneTexture0;
	}
	return INDEX_NONE;
}

int32 FMaterialCompilationOutput::FindUserSceneTexture(FName UserSceneTexture) const
{
	FScriptName UserSceneTextureScript(UserSceneTexture);
	int32 FoundIndex = UserSceneTextureInputs.Find(UserSceneTextureScript);
	if (FoundIndex != INDEX_NONE)
	{
		return FoundIndex + PPI_UserSceneTexture0;
	}
	return INDEX_NONE;
}

int32 FMaterialCompilationOutput::GetNumPostProcessInputsUsed() const
{
	// Check how many post process inputs are used in the mask (SceneTexture nodes explicitly referencing PostProcessInput0-6)
	int32 NumPostProcessInputs = 0;
	for (int32 InputIndex = 0; InputIndex < kPostProcessMaterialInputCountMax; InputIndex++)
	{
		if (IsSceneTextureUsed((ESceneTextureId)(PPI_PostProcessInput0 + InputIndex)))
		{
			NumPostProcessInputs++;
		}
	}

	// Add any UserSceneTexture inputs, which take up any remaining unused input slots
	return NumPostProcessInputs + UserSceneTextureInputs.Num();
}

namespace UE::MaterialTranslatorUtils
{

FString GenerateUserSceneTextureRemapHLSLDefines(const FMaterialCompilationOutput& CompilationOutput)
{
	if (CompilationOutput.UserSceneTextureInputs.IsEmpty())
	{
		return TEXT("");
	}

	FString UserSceneTextureEnumRemap;
	FString UserSceneTextureNameRemap;
	int32 PostProcessIndex = 0;
	for (int32 UserIndex = 0; UserIndex < CompilationOutput.UserSceneTextureInputs.Num();)
	{
		if (PostProcessIndex >= kPostProcessMaterialInputCountMax)
		{
			// If we run out of slots, go ahead and generate something that will still compile -- this should be detected and fail earlier.
			UserSceneTextureEnumRemap.Appendf(TEXT("#define PPI_UserSceneTexture%d -1\n"), UserIndex);
			UserIndex++;
		}
		else if (CompilationOutput.IsSceneTextureUsed((ESceneTextureId)(PPI_PostProcessInput0 + PostProcessIndex)))
		{
			// This slot is used by a SceneTexture node somewhere else in the material, skip over it
			PostProcessIndex++;
		}
		else
		{
			UserSceneTextureEnumRemap.Appendf(TEXT("#define PPI_UserSceneTexture%d PPI_PostProcessInput%d\n"), UserIndex, PostProcessIndex);

			// If the name is a token, add another define including the name.  This allows custom HLSL to easily reference a named input,
			// without worrying about the remapping of the name.
			constexpr FAsciiSet TokenStartChars = "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ_";
			constexpr FAsciiSet TokenContinueChars = "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789_";

			FString UserInputName = CompilationOutput.UserSceneTextureInputs[UserIndex].ToString();
			if (TokenStartChars.Test(UserInputName[0]) && FAsciiSet::HasOnly(UserInputName, TokenContinueChars))
			{
				UserSceneTextureNameRemap.Appendf(TEXT("#define PPIUser_%s PPI_PostProcessInput%d\n"), *UserInputName, PostProcessIndex);
			}

			UserIndex++;
			PostProcessIndex++;
		}
	}

	return UserSceneTextureEnumRemap + TEXT("\n") + UserSceneTextureNameRemap;
}

FString SceneTextureIdToHLSLString(ESceneTextureId TexId)
{
	// PPI_UserSceneTexture0-6 are emitted as symbols rather than numbers, so they can be remapped later using defines generated by the function above.
	if (TexId >= PPI_UserSceneTexture0 && TexId <= PPI_UserSceneTexture6)
	{
		return FString::Printf(TEXT("PPI_UserSceneTexture%d"), TexId - PPI_UserSceneTexture0);
	}
	else
	{
		return FString::Printf(TEXT("%d"), TexId);
	}
}

EWorldPositionIncludedOffsets GetWorldPositionTypeWithOrigin(EPositionOrigin PositionOrigin, bool IncludeOffsets)
{
	switch (PositionOrigin)
	{
	case EPositionOrigin::Absolute: return IncludeOffsets ? WPT_Default : WPT_ExcludeAllShaderOffsets;
	case EPositionOrigin::CameraRelative: return IncludeOffsets ? WPT_CameraRelative : WPT_CameraRelativeNoOffsets;
	default: checkNoEntry();
	}
	return WPT_Default;
}

FName GetWorldPositionInputName(EPositionOrigin PositionOrigin)
{
	switch (PositionOrigin)
	{
	case EPositionOrigin::CameraRelative: return TEXT("Translated World Position");
	case EPositionOrigin::Absolute: return TEXT("World Position");
	default: checkNoEntry();
	}
	return FName();
}

// FMaterialResource::IsTranslucencyAfterDOFEnabled
static bool IsTranslucencyAfterDOFEnabled(UMaterial* Material)
{
	return Material->TranslucencyPass == MTP_AfterDOF
		&& !Material->IsUIMaterial()
		&& !Material->IsDeferredDecal();
}

// FMaterialResource::IsTranslucencyAfterMotionBlurEnabled
static bool IsTranslucencyAfterMotionBlurEnabled(UMaterial* Material)
{
	return Material->TranslucencyPass == MTP_AfterMotionBlur
		&& !Material->IsUIMaterial()
		&& !Material->IsDeferredDecal();
}

EMaterialValueType GetTexturePropertyValueType(EMaterialValueType TextureType)
{
	return (TextureType == MCT_VolumeTexture || TextureType == MCT_Texture2DArray || TextureType == MCT_SparseVolumeTexture) ? MCT_Float3 : MCT_Float2;
}

void FinalCompileValidation(
	UMaterial* Material,
	const FMaterialCompilationOutput& MaterialCompilationOutput,
	FMaterialShadingModelField MaterialShadingModels,
	EBlendMode BlendMode,
	bool bHasFrontMaterialExpr,
	EShaderPlatform Platform,
	TArray<FString>& OutErrors)
{
	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	const bool bSubstrateHiddenConversionEnabled = Substrate::IsHiddenMaterialAssetConversionEnabled();
	const bool bSubstrateFrontMaterialPluggedIn = bHasFrontMaterialExpr;
	const EMaterialDomain Domain = Material->MaterialDomain;

	// If Substrate is enabled or if this is a Substrate material cooked/used in non-Substrate mode, 
	// we disable this warning as Substrate supports 'colored transmittance only' mode (i.e, modulate).
	if (!bSubstrateEnabled && !bHasFrontMaterialExpr && IsModulateBlendMode(BlendMode) && MaterialShadingModels.IsLit() && !Material->IsDeferredDecal())
	{
		OutErrors.Add(TEXT("Dynamically lit translucency is not supported for BLEND_Modulate materials."));
	}

	if (Domain == MD_Surface)
	{
		if (IsModulateBlendMode(BlendMode) && IsTranslucencyAfterDOFEnabled(Material) && !(RHISupportsDualSourceBlending(Platform) || IsMobilePlatform(Platform)))
		{
			OutErrors.Add(TEXT("Translucency after DOF with BLEND_Modulate is only allowed on mobile platforms or desktop platforms that support dual-blending. Consider using BLEND_Translucent with black emissive"));
		}
	}

	if (MaterialCompilationOutput.RequiresSceneColorCopy())
	{
		if (Domain != MD_Surface)
		{
			OutErrors.Add(TEXT("Only 'surface' material domain can use the scene color node."));
		}
		else if (!IsTranslucentBlendMode(BlendMode))
		{
			OutErrors.Add(TEXT("Only translucent materials can use the scene color node."));
		}
	}

	if (IsAlphaHoldoutBlendMode(BlendMode) && !Material->IsPostProcessMaterial())
	{
		if (!MaterialShadingModels.IsUnlit())
		{
			OutErrors.Add(TEXT("Alpha Holdout blend mode must use unlit shading model."));
		}

		const int32 FnMainVersion = Material->GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);

		// Note: As a retroactive fix for content failing to cook, we use the closest fn main version change to constrain the error only to recently created materials.
		if (FnMainVersion > FFortniteMainBranchObjectVersion::LandscapeTargetLayersInLandscapeActor)
		{
			if (IsTranslucencyAfterDOFEnabled(Material) || IsTranslucencyAfterMotionBlurEnabled(Material))
			{
				// We must write into the alpha channel of the SceneColor now for later translucents rendered after DOF to be able to be affected by the alpha.
				OutErrors.Add(TEXT("Alpha Holdout blend mode must use BeforeDOF under Translucency / Advanced / Translucency Pass."));
			}
		}
	}

	if (Domain == MD_Volume && BlendMode != BLEND_Additive)
	{
		OutErrors.Add(TEXT("Volume materials must use an Additive blend mode."));
	}
	if (Domain == MD_Volume && Material->GetUsageByFlag(MATUSAGE_SkeletalMesh))
	{
		OutErrors.Add(TEXT("Volume materials are not compatible with skinned meshes: they are voxelised as boxes anyway. Please disable UsedWithSkeletalMesh on the material."));
	}

	if (Domain == MD_LightFunction && !IsOpaqueBlendMode(BlendMode))
	{
		OutErrors.Add(TEXT("Light function materials must be opaque."));
	}

	if (Domain == MD_LightFunction && MaterialShadingModels.IsLit())
	{
		OutErrors.Add(TEXT("Light function materials must use unlit."));
	}

	if (Domain == MD_PostProcess && MaterialShadingModels.IsLit())
	{
		OutErrors.Add(TEXT("Post process materials must use unlit."));
	}

	if (Material->bAllowNegativeEmissiveColor && MaterialShadingModels.IsLit())
	{
		OutErrors.Add(TEXT("Only unlit materials can output negative emissive color."));
	}

	if (Material->bIsSky && (!MaterialShadingModels.IsUnlit() || !(IsOpaqueOrMaskedBlendMode(BlendMode))))
	{
		OutErrors.Add(TEXT("Sky materials must be opaque or masked, and unlit. They are expected to completely replace the background."));
	}

	if (MaterialShadingModels.HasShadingModel(MSM_SingleLayerWater))
	{
		if (!IsOpaqueOrMaskedBlendMode(BlendMode))
		{
			OutErrors.Add(TEXT("SingleLayerWater materials must be opaque or masked."));
		}
		if (!MaterialShadingModels.HasOnlyShadingModel(MSM_SingleLayerWater))
		{
			OutErrors.Add(TEXT("SingleLayerWater materials cannot be combined with other shading models.")); // Simply untested for now
		}

		if (!Material->HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionSingleLayerWaterMaterialOutput>()
			&& (!bSubstrateEnabled || (bSubstrateHiddenConversionEnabled && !bSubstrateFrontMaterialPluggedIn))
			)
		{
			OutErrors.Add(TEXT("SingleLayerWater materials requires the use of SingleLayerWaterMaterial output node."));
		}
	}

	if (MaterialShadingModels.HasShadingModel(MSM_ThinTranslucent))
	{
		if (!IsTranslucentOnlyBlendMode(BlendMode))
		{
			OutErrors.Add(TEXT("ThinTranslucent materials must be translucent."));
		}

		const ETranslucencyLightingMode TranslucencyLightingMode = Material->TranslucencyLightingMode;

		if (TranslucencyLightingMode != ETranslucencyLightingMode::TLM_SurfacePerPixelLighting)
		{
			OutErrors.Add(TEXT("ThinTranslucent materials must use Surface Per Pixel Lighting (Translucency->LightingMode=Surface ForwardShading).\n"));
		}
		if (!MaterialShadingModels.HasOnlyShadingModel(MSM_ThinTranslucent))
		{
			OutErrors.Add(TEXT("ThinTranslucent materials cannot be combined with other shading models.")); // Simply untested for now
		}
		if (!Material->HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionThinTranslucentMaterialOutput>())
		{
			OutErrors.Add(TEXT("ThinTranslucent materials requires the use of ThinTranslucentMaterial output node."));
		}
		if (IsTranslucencyAfterMotionBlurEnabled(Material))
		{
			// We don't currently have a separate translucency modulation pass for After Motion Blur
			OutErrors.Add(TEXT("ThinTranslucent materials are not currently supported in the \"After Motion Blur\" translucency pass."));
		}
	}

	if (Material->HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionAbsorptionMediumMaterialOutput>())
	{
		// User placed an AbsorptionMedium node, make sure default lit is being used
		if (!MaterialShadingModels.HasShadingModel(MSM_DefaultLit))
		{
			OutErrors.Add(TEXT("AbsorptionMedium custom output requires Default Lit shading model."));
		}
		if (Material->RefractionMethod != RM_IndexOfRefraction && Material->RefractionMethod != RM_IndexOfRefractionFromF0)
		{
			OutErrors.Add(TEXT("AbsorptionMedium custom output requires \"Index of Refraction\" as the \"Refraction Mode\"."));
		}
	}

	if (Domain == MD_DeferredDecal && !(IsTranslucentOnlyBlendMode(BlendMode) || BlendMode == BLEND_AlphaComposite || IsModulateBlendMode(BlendMode)))
	{
		// We could make the change for the user but it would be confusing when going to DeferredDecal and back
		// or we would have to pay a performance cost to make the change more transparently.
		// The change saves performance as with translucency we don't need to test for MeshDecals in all opaque rendering passes
		const TCHAR* AllowedBlendModes = bSubstrateEnabled ? 
			TEXT("TranslucentGreyTransmittance, TranslucentColorTransmittance, AlphaComposite (Premultiplied Alpha), or ColoredTransmittanceOnly.") :
			TEXT("Translucent, AlphaComposite(Premultiplied Alpha), or Modulate.");

		OutErrors.Add(FString::Printf(TEXT("Material using the DeferredDecal domain can only use the Blend Modes %s"), AllowedBlendModes));
	}

	int32 NumPostProcessInputs = MaterialCompilationOutput.GetNumPostProcessInputsUsed();
	if (NumPostProcessInputs > kPostProcessMaterialInputCountMax)
	{
		OutErrors.Add(FString::Printf(TEXT("Maximum Scene Texture post process inputs exceeded (%d > %d), between SceneTexture nodes with PostProcessInputs or UserSceneTexture nodes."), NumPostProcessInputs, kPostProcessMaterialInputCountMax));
	}

	if (IsModulateBlendMode(BlendMode) && IsTranslucencyAfterMotionBlurEnabled(Material))
	{
		// We don't currently have a separate translucency modulation pass for After Motion Blur
		OutErrors.Add(TEXT("Blend Mode \"Modulate\" materials are not currently supported in the \"After Motion Blur\" translucency pass."));
	}
}

enum class AsciiFlags
{
	TerminatorOrSlash = (1 << 0),	// Null terminator OR slash
	Whitespace = (1 << 1),			// Includes other special characters below 32 (in addition to tab / newline)
	Other = (1 << 2),				// Anything else not one of the other types
	SymbolStart = (1 << 3),			// Letters plus underscore (anything that can start a symbol)
	Digit = (1 << 4),
	Dot = (1 << 5),
	Quote = (1 << 6),
	Hash = (1 << 7),
};

static uint8 AsciiFlagTable[256] =
{
	1,2,2,2,2,2,2,2, 2,2,2,2,2,2,2,2, 2,2,2,2,2,2,2,2, 2,2,2,2,2,2,2,2,		// Treat all special characters as whitespace

	2,4,64,128,4,4,4,4,			// 34 == Quote  35 == Hash
	4,4,4,4,4,4,32,1,			// 46 == Dot    47 == Slash
	16,16,16,16,16,16,16,16,	// Digits 0-7
	16,16,4,4,4,4,4,4,			// Digits 8-9

	4,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,4,4,4,4,8,		// Upper case letters,  95 == Underscore
	4,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,4,4,4,4,4,		// Lower case letters

	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,		// Treat all non-ASCII characters as Other
	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,
	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,
	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,
};

const TCHAR* FindHlslIdentifierInCode(const TCHAR*& Code, const TCHAR* Identifier, TArray<FStringView>& OutCompoundTokens)
{
	FStringView RootIdentifier;
	OutCompoundTokens.Empty();

	#define FETCH_ASCII_FLAG_TABLE(CHARACTER) AsciiFlagTable[(uint8)FMath::Min(CHARACTER, (TCHAR)UINT8_MAX)]

	const TCHAR* SearchChar = Code;
	uint8 SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar);
	bool bInCompoundIdentifier = false;

	// Scanning loop
	while (1)
	{
		static constexpr uint8 AsciiFlagsSymbol = (uint8)AsciiFlags::SymbolStart | (uint8)AsciiFlags::Digit;
		static constexpr uint8 AsciiFlagsStartNumberOrDirective = (uint8)AsciiFlags::Digit | (uint8)AsciiFlags::Dot | (uint8)AsciiFlags::Hash;
		static constexpr uint8 AsciiFlagsEndNumberOrDirective = (uint8)AsciiFlags::Whitespace | (uint8)AsciiFlags::Other | (uint8)AsciiFlags::Quote | (uint8)AsciiFlags::TerminatorOrSlash;

		if (SearchCharFlag & (uint8)AsciiFlags::Whitespace)
		{
			// Always skip over whitespace
			SearchChar++;
			SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar);
		}
		else if (SearchChar[0] == '/' && (SearchChar[1] == '/' || SearchChar[1] == '*'))
		{
			// Always skip over comments
			if (SearchChar[1] == '/')
			{
				SearchChar += 2;
				while (*SearchChar && *SearchChar != '\r' && *SearchChar != '\n')
				{
					SearchChar++;
				}

				// Could be end of string, if not, skip over newline
				if (*SearchChar)
				{
					SearchChar++;
				}
			}
			else
			{
				SearchChar += 2;
				while (*SearchChar && (SearchChar[0] != '*' || SearchChar[1] != '/'))
				{
					SearchChar++;
				}

				// Could be end of string, if not, skip over comment terminator
				if (*SearchChar)
				{
					SearchChar += 2;
				}
			}
			SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar);
		}
		else if (!RootIdentifier.IsEmpty() && (SearchCharFlag & ((uint8)AsciiFlags::Dot)))
		{
			// Dot after an identifier starts a compound identifier
			bInCompoundIdentifier = true;
			SearchChar++;
			SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar);
		}
		else if (!RootIdentifier.IsEmpty() && !bInCompoundIdentifier)
		{
			// Any other character besides whitespace, comment, or dot terminates an identifier.  Check if it matches what we are searching for.
			if (RootIdentifier == Identifier)
			{
				Code = SearchChar;
				return RootIdentifier.GetData();
			}

			// Clear the state
			RootIdentifier = FStringView();
			OutCompoundTokens.Empty();

			// Don't advance character -- restart loop, no longer tracking an identifier
		}
		else if (SearchCharFlag & (uint8)AsciiFlags::SymbolStart)
		{
			// Start of an identifier, determine the bounds of the identifier
			const TCHAR* IdentifierStart = SearchChar;

			SearchChar++;
			while ((SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar)) & AsciiFlagsSymbol)
			{
				SearchChar++;
			}

			FStringView IdentifierView(IdentifierStart, SearchChar - IdentifierStart);

			// Add identifier as a compound identifier or the root identifier
			if (bInCompoundIdentifier)
			{
				OutCompoundTokens.Add(IdentifierView);
				bInCompoundIdentifier = false;
			}
			else
			{
				check(RootIdentifier.IsEmpty());
				RootIdentifier = IdentifierView;
			}
		}
		else if (!RootIdentifier.IsEmpty())
		{
			// If we reach this point, we are in an invalid syntax compound identifier, where an identifier token and dot are not followed
			// by another identifier.  Clear the state and restart parsing.
			bInCompoundIdentifier = false;
			RootIdentifier = FStringView();
			OutCompoundTokens.Empty();

			// Don't advance character -- restart loop, no longer tracking an identifier.
		}
		else if ((SearchCharFlag & (uint8)AsciiFlags::Other) || (*SearchChar == '/'))
		{
			// Not in an identifier or comment -- skip over various characters with no special handling
			SearchChar++;
			SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar);
		}
		else if (SearchCharFlag & AsciiFlagsStartNumberOrDirective)
		{
			// Number or directive, skip to Whitespace, Other, or Quote (numbers may contain letters or #, i.e. "1.#INF" for infinity, or "e" for an exponent)
			SearchChar++;
			while (!((SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar)) & AsciiFlagsEndNumberOrDirective))
			{
				SearchChar++;
			}
		}
		else if (SearchCharFlag & (uint8)AsciiFlags::Quote)
		{
			// Quote, skip to next Quote (or maybe end of string if text is malformed), ignoring the quote if it's escaped
			SearchChar++;
			while (*SearchChar && (*SearchChar != '\"' || *(SearchChar - 1) == '\\'))
			{
				SearchChar++;
			}

			// Could be end of string or the quote -- skip over the quote if not the null terminator
			if (*SearchChar)
			{
				SearchChar++;
			}
			SearchCharFlag = FETCH_ASCII_FLAG_TABLE(*SearchChar);
		}
		// Must be null terminator -- we've tested all other possibilities
		else
		{
			// End of string
			Code = SearchChar;
			OutCompoundTokens.Empty();
			return nullptr;
		}
	}

	#undef FETCH_ASCII_FLAG_TABLE
}

// Fixup for SceneTexture and UserSceneTexture inputs to custom HLSL.  Returns a new string, rather than modifying a Code FString
// in place, to allow the code to be shared with the HLSL tree code path (MaterialExpressionHLSL.cpp), which uses TStringBuilder.
// An empty string is returned if no fixup was required (the common case), avoiding reallocation.  OutSceneTextureInfo tracks
// which inputs are SceneTexture or UserSceneTexture (left empty if no scene textures).  A -1 indicates a scene texture input not
// directly used as a symbol in the custom HLSL, allowing the fetch to be dead stripped, while 1 is an actually used input.
//
// The goal is to allow the scene texture ID to be automatically generated in the code, without the user needing to manually
// insert the correct identifier based on settings in the connected node.  Without this fixup, the user needs to change the
// custom HLSL any time the input node changes, which is tedious and error prone.  For SceneTexture, if they (for example) change
// SceneColor to Depth, they need to change PPI_SceneColor to PPI_Depth.  For UserSceneTexture, if the name is changed from Input
// to NewInput, they need to change PPIUser_Input to PPIUser_NewInput.  Similar manual changes are also required if a completely
// different SceneTexture node is linked.  Besides substituting the ID, minimalist shorthand ".Fetch" function call style syntax
// is provided for offset fetches (accepts either float2 or two floats).  The original syntax of referencing "Input" without the
// ".ID" or ".Fetch" suffixes still works for fetching a non-offset sample.
//
//		Input.ID -> Replace text with ID enum of connected input (PPI_* or PPIUser_*)
//			Example usage:	SceneTextureFetch(Input.ID, float2(0,0))
//							SceneTextureFetch(/*Input.ID=*/ PPI_SceneColor, float2(0,0))
//
//		Input.Fetch(*) -> Replace with SceneTextureFetchFunc(Parameters, ID, *)
//			Example usage:	Input.Fetch(1,-1)
//							SceneTextureFetchFunc(Parameters, /*Input=*/ PPI_SceneColor, 1,-1)
//
// Note that this text replacement happens before preprocessor macro substitution, and so it's not possible to use the shorthand
// syntax with the user specified "InputName" as a macro argument, but you can pass "InputName.ID", for example:
//
//		// Shorthand syntax will not work, because substitution of "InputName" as the "Input" macro argument happens later in preprocess
//		#define WEIGHTED_FETCH_SCENE_TEXTURE(Input, SampleIndex) \
//			(Input.Fetch(Offsets[SampleIndex], 0) * Weights[SampleIndex])
//
//		Color += WEIGHTED_FETCH_SCENE_TEXTURE(InputName, SampleIndex);
//
//		// But the syntax that accepts a scene texture ID works fine
//		#define WEIGHTED_FETCH_SCENE_TEXTURE(InputID, SampleIndex) \
//			(SceneTextureFetch(InputID, float2(Offsets[SampleIndex], 0)) * Weights[SampleIndex])
//
//		Color += WEIGHTED_FETCH_SCENE_TEXTURE(InputName.ID, SampleIndex);
//
//		// This also works fine, because InputName is inside the macro, and not a macro argument
//		#define WEIGHTED_FETCH_SCENE_TEXTURE(SampleIndex) \
//			(InputName.Fetch(Offsets[SampleIndex], 0) * Weights[SampleIndex])
//
FString CustomExpressionSceneTextureInputFixup(const UMaterialExpressionCustom* Custom, const TCHAR* Code, TArray<int8>& OutSceneTextureInfo)
{
	// Allocated when required fixup is first encountered
	FString ModifiedCode;

	for (int32 InputIndex = 0; InputIndex < Custom->Inputs.Num(); InputIndex++)
	{
		const FCustomInput& Input = Custom->Inputs[InputIndex];

		if (Input.InputName.IsNone())
		{
			continue;
		}

		FExpressionInput ExpressionInput = Input.Input.GetTracedInput();
		const UMaterialExpression* Expression = ExpressionInput.Expression;
		if (!Expression || ExpressionInput.OutputIndex != 0 || !(Expression->IsA<UMaterialExpressionSceneTexture>() || Expression->IsA<UMaterialExpressionUserSceneTexture>()))
		{
			continue;
		}

		FString InputNameString = Input.InputName.ToString();
		FString InputIDString;

		if (const UMaterialExpressionSceneTexture* SceneTextureExpression = Cast<UMaterialExpressionSceneTexture>(Expression))
		{
			InputIDString = StaticEnum<ESceneTextureId>()->GetNameStringByValue(static_cast<int>(SceneTextureExpression->SceneTextureId));
		}
		else
		{
			const UMaterialExpressionUserSceneTexture* UserSceneTextureExpression = Cast<UMaterialExpressionUserSceneTexture>(Expression);
			InputIDString = TEXT("PPIUser_") + UserSceneTextureExpression->UserSceneTexture.ToString();
		}

		// We've encountered a scene texture, allocate our info array
		if (!OutSceneTextureInfo.Num())
		{
			OutSceneTextureInfo.SetNumZeroed(Custom->Inputs.Num());
		}

		// Mark with -1 to indicate that it's a scene texture input, but might not be directly used in the source.
		OutSceneTextureInfo[InputIndex] = -1;

		int32 CodeOffset = 0;
		while (1)
		{
			TArray<FStringView> CompoundTokens;
			const TCHAR* CodeStart = ModifiedCode.IsEmpty() ? Code : *ModifiedCode;
			const TCHAR* CodeString = &CodeStart[CodeOffset];
			const TCHAR* FoundInputName = FindHlslIdentifierInCode(CodeString, *InputNameString, CompoundTokens);

			if (!FoundInputName)
			{
				break;
			}

			CodeOffset = FoundInputName - CodeStart;

			int32 OriginalCodeOffset = CodeOffset;
			if (CompoundTokens.Num() == 1)
			{
				// Check if this is using the "Fetch" function call shorthand syntax.  If so, replace it with the corresponding fetch function.
				if (CompoundTokens[0] == TEXT("Fetch") && *CodeString == '(')
				{
					FString ReplacementText = FString::Printf(TEXT("SceneTextureFetchFunc(Parameters, /*%s=*/ %s, "), *InputNameString, *InputIDString);

					if (ModifiedCode.IsEmpty())
					{
						ModifiedCode.Reserve(FCString::Strlen(Code) + ReplacementText.Len());
						ModifiedCode = Code;
					}

					// Add one character to amount removed, as we are overwriting the open parentheses as well
					ModifiedCode.RemoveAt(CodeOffset, CodeString - FoundInputName + 1, EAllowShrinking::No);
					ModifiedCode.InsertAt(CodeOffset, ReplacementText);

					CodeOffset += ReplacementText.Len();
				}
				else if (CompoundTokens[0] == TEXT("ID"))
				{
					// Replace "Input.ID" identifier with ID (including comment for readability if there are compile errors)
					FString ReplacementText = FString::Printf(TEXT("/*%s.ID=*/ %s"), *InputNameString, *InputIDString);

					if (ModifiedCode.IsEmpty())
					{
						ModifiedCode.Reserve(FCString::Strlen(Code) + ReplacementText.Len());
						ModifiedCode = Code;
					}

					ModifiedCode.RemoveAt(CodeOffset, CompoundTokens[0].GetData() + CompoundTokens[0].Len() - FoundInputName, EAllowShrinking::No);
					ModifiedCode.InsertAt(CodeOffset, ReplacementText);

					CodeOffset += ReplacementText.Len();
				}
			}

			// If we didn't insert and skip over replacement code, skip over the token that was found (CodeString will have
			// been set past the end of the token by the function "FindHlslIdentifierInCode").
			if (CodeOffset == OriginalCodeOffset)
			{
				CodeOffset = CodeString - CodeStart;

				// Scene Texture input pin symbol was directly used by the custom HLSL, mark it with a 1, as the input pin expression needs to be compiled in
				OutSceneTextureInfo[InputIndex] = 1;
			}
		}
	}

	return ModifiedCode;
}

bool IsDevelopmentFeatureEnabled(const FName& FeatureName, EShaderPlatform Platform, UMaterial* Material)
{
	if (FeatureName == NAME_SelectionColor)
	{
		// This is an editor-only feature (see FDefaultMaterialInstance::GetVectorValue).

		// Determine if we're sure the editor will never run using the target shader platform.
		// The list below may not be comprehensive enough, but it definitely includes platforms which won't use selection color for sure.
		const bool bEditorMayUseTargetShaderPlatform = IsPCPlatform(Platform);
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.CompileShadersForDevelopment"));
		const bool bCompileShadersForDevelopment = (CVar && CVar->GetValueOnAnyThread() != 0);

		return
			// Does the material explicitly forbid development features?
			Material->bAllowDevelopmentShaderCompile
			// Can the editor run using the current shader platform?
			&& bEditorMayUseTargetShaderPlatform
			// Are shader development features globally disabled?
			&& bCompileShadersForDevelopment;
	}

	return true;
}

}  // namespace UE::MaterialTranslatorUtils

FString DescribeCompilationShaderParametersDifferences(const FMaterial* InMaterial, const FCompilationShaderParameters& InCompilationParams)
{
	const FConservativeShaderParameters ConservativeParams(InMaterial);
	return DescribeShaderParameterDifferences(ConservativeParams, InCompilationParams, TEXT("Conservative Parameters"), TEXT("Compilation Parameters"));
}

FString DescribeShaderParameterDifferences(
	const FMaterialShaderParametersBase& A,
	const FMaterialShaderParametersBase& B,
	const TCHAR* LabelA,
	const TCHAR* LabelB)
{
	TStringBuilder<2048> Result;
	int32 DiffCount = 0;

#define COMPARE_BOOL(Field) \
    if (A.Field != B.Field) \
    { \
        Result.Appendf(TEXT("  DIFF [%s]: %s=%s vs %s=%s\n"), TEXT(#Field), LabelA, A.Field ? TEXT("true") : TEXT("false"), LabelB, B.Field ? TEXT("true") : TEXT("false")); \
        ++DiffCount; \
    }

#define COMPARE_INT(Field) \
    if (A.Field != B.Field) \
    { \
        Result.Appendf(TEXT("  DIFF [%s]: %s=%d vs %s=%d\n"), TEXT(#Field), LabelA, (int32)A.Field, LabelB, (int32)B.Field); \
        ++DiffCount; \
    }

	// --- Scalar / Enum fields ---
	COMPARE_INT(MaterialDomain);
	COMPARE_INT(BlendMode);
	COMPARE_INT(FeatureLevel);
	COMPARE_INT(QualityLevel);
	COMPARE_INT(PreshaderGap);
	COMPARE_INT(BlendableLocation);
	COMPARE_INT(NumCustomizedUVs);
	COMPARE_INT(StencilCompare);
	COMPARE_INT(NumMaterialCacheTags);

	if (A.ShadingModels != B.ShadingModels)
	{
		Result.Appendf(TEXT("  DIFF [ShadingModels]: %s=0x%X vs %s=0x%X\n"),
			LabelA, A.ShadingModels.GetShadingModelField(),
			LabelB, B.ShadingModels.GetShadingModelField());
		++DiffCount;
	}

	// --- 16-bit packed flags ---
	COMPARE_BOOL(bIsUsedWithStaticMesh);
	COMPARE_BOOL(bIsTranslucentSurfaceLighting);
	COMPARE_BOOL(bHasFirstPersonInterpolation);
	COMPARE_BOOL(bIsSubstrateMaterial);
	COMPARE_BOOL(bUsesTemporalResponsiveness);
	COMPARE_BOOL(bUsesMotionVectorWorldOffset);
	COMPARE_BOOL(bSamplesMaterialCache);
	COMPARE_BOOL(bHasMaterialCacheOutput);
	COMPARE_BOOL(bIsUsedWithMeshDeformer);
	COMPARE_BOOL(bIsUsedWithInstancedSkinnedMesh);

	// --- 64-bit packed flags: General ---
	COMPARE_BOOL(bIsDefaultMaterial);
	COMPARE_BOOL(bIsSpecialEngineMaterial);
	COMPARE_BOOL(bIsMasked);
	COMPARE_BOOL(bIsDitherMasked);
	COMPARE_BOOL(bIsTwoSided);
	COMPARE_BOOL(bIsThinSurface);
	COMPARE_BOOL(bIsDistorted);
	COMPARE_BOOL(bShouldCastDynamicShadows);
	COMPARE_BOOL(bWritesEveryPixel);
	COMPARE_BOOL(bWritesEveryPixelShadowPass);

	// --- Property connected flags ---
	COMPARE_BOOL(bHasDiffuseAlbedoConnected);
	COMPARE_BOOL(bHasF0Connected);
	COMPARE_BOOL(bHasBaseColorConnected);
	COMPARE_BOOL(bHasNormalConnected);
	COMPARE_BOOL(bHasRoughnessConnected);
	COMPARE_BOOL(bHasSpecularConnected);
	COMPARE_BOOL(bHasMetallicConnected);
	COMPARE_BOOL(bHasEmissiveColorConnected);
	COMPARE_BOOL(bHasAmbientOcclusionConnected);
	COMPARE_BOOL(bHasAnisotropyConnected);

	// --- Mesh modification flags ---
	COMPARE_BOOL(bIsTessellationEnabled);
	COMPARE_BOOL(bHasDisplacementConnected);
	COMPARE_BOOL(bHasVertexPositionOffsetConnected);
	COMPARE_BOOL(bHasPixelDepthOffsetConnected);
	COMPARE_BOOL(bMaterialMayModifyMeshPosition);

	// --- Usage flags (bIsUsedWith*) ---
	COMPARE_BOOL(bIsUsedWithStaticLighting);
	COMPARE_BOOL(bIsUsedWithParticleSprites);
	COMPARE_BOOL(bIsUsedWithMeshParticles);
	COMPARE_BOOL(bIsUsedWithNiagaraSprites);
	COMPARE_BOOL(bIsUsedWithNiagaraMeshParticles);
	COMPARE_BOOL(bIsUsedWithNiagaraRibbons);
	COMPARE_BOOL(bIsUsedWithLandscape);
	COMPARE_BOOL(bIsUsedWithBeamTrails);
	COMPARE_BOOL(bIsUsedWithSplineMeshes);
	COMPARE_BOOL(bIsUsedWithSkeletalMesh);
	COMPARE_BOOL(bIsUsedWithMorphTargets);
	COMPARE_BOOL(bIsUsedWithAPEXCloth);
	COMPARE_BOOL(bIsUsedWithGeometryCache);
	COMPARE_BOOL(bIsUsedWithGeometryCollections);
	COMPARE_BOOL(bIsUsedWithHairStrands);
	COMPARE_BOOL(bIsUsedWithWater);
	COMPARE_BOOL(bIsUsedWithInstancedStaticMeshes);
	COMPARE_BOOL(bIsUsedWithLidarPointCloud);
	COMPARE_BOOL(bIsUsedWithVirtualHeightfieldMesh);
	COMPARE_BOOL(bIsUsedWithNeuralNetworks);
	COMPARE_BOOL(bIsUsedWithNanite);
	COMPARE_BOOL(bIsUsedWithVoxels);
	COMPARE_BOOL(bIsUsedWithCurves);
	COMPARE_BOOL(bIsUsedWithVolumetricCloud);
	COMPARE_BOOL(bIsUsedWithHeterogeneousVolumes);

	// --- Misc flags ---
	COMPARE_BOOL(bIsTranslucencyWritingVelocity);
	COMPARE_BOOL(bIsTranslucencyWritingCustomDepth);
	COMPARE_BOOL(bIsDitheredLODTransition);
	COMPARE_BOOL(bHasPerInstanceCustomData);
	COMPARE_BOOL(bHasPerInstanceRandom);
	COMPARE_BOOL(bHasVertexInterpolator);
	COMPARE_BOOL(bHasRuntimeVirtualTextureOutput);
	COMPARE_BOOL(bIsStencilTestEnabled);
	COMPARE_BOOL(bIsTranslucencySurface);
	COMPARE_BOOL(bShouldDisableDepthTest);
	COMPARE_BOOL(bHasRenderTracePhysicalMaterialOutput);
	COMPARE_BOOL(bIsMobileSeparateTranslucencyEnabled);
	COMPARE_BOOL(bAlwaysEvaluateWorldPositionOffset);
	COMPARE_BOOL(bDisablePreExposureScale);
	COMPARE_BOOL(bAllowVariableRateShading);

#undef COMPARE_BOOL
#undef COMPARE_INT

	if (DiffCount == 0)
	{
		// Make sure no additional properties have been added which have been missed here.
		if (FMemory::Memcmp(&A, &B, sizeof(FMaterialShaderParametersBase)) == 0)
		{
			Result.Appendf(TEXT("FMaterialShaderParametersBase: %s and %s are identical."), LabelA, LabelB);
		}
		else
		{
			Result.Appendf(TEXT("FMaterialShaderParametersBase: %s and %s are different but properties are missing from DescribeShaderParameterDifferences ."), LabelA, LabelB);
		}
	}
	else
	{
		Result.Appendf(TEXT("FMaterialShaderParametersBase: %d difference(s) between %s and %s."), DiffCount, LabelA, LabelB);
	}

	return FString(Result.ToView());
}

const TArray<TObjectPtr<UObject>>& FEditorMaterialResourceBase::GetMatDefaultTextures() const
{
	return ReferencedTextures;
}

int32 FEditorMaterialResourceBase::GetMatDefaultTextureIdx(TObjectPtr<UObject> InTexture) const
{
	return ReferencedTextures.Find(InTexture);
}

TObjectPtr<UObject> FEditorMaterialResourceBase::GetMatDefaultTextureAtIdx(const int32 InIdx) const
{
	if (!ensureAlwaysMsgf(ReferencedTextures.IsValidIndex(InIdx), TEXT("Invalid texture Idx %d requested from Array with %d Elements."), InIdx, ReferencedTextures.Num()))
	{
		return nullptr;
	}

	return ReferencedTextures[InIdx];
}

TArrayView<const TObjectPtr<UObject>> FEditorMaterialResourceBase::GetReferencedTextures() const
{
	return ReferencedTextures;
}

TConstArrayView<TObjectPtr<UTextureCollection>> FEditorMaterialResourceBase::GetReferencedTextureCollections() const
{
	return MakeArrayView(ReferencedTextureCollections);
}

// Helper class for getting translation output from a Material/Material Instance.
class FDebugShaderInfoMaterialResource : public FMaterialResource
{
public:
	FDebugShaderInfoMaterialResource(UMaterial* InMat, UMaterialInstance* InMatInstance, const ERHIFeatureLevel::Type FeatureLevel, const EShaderPlatform ShaderPlatform,
		const EMaterialQualityLevel::Type QualityLevel)
		: FMaterialResource()
	{
		SetMaterial(InMat);
		SetMaterialInstance(InMatInstance);

		SetQualityLevelProperties(ShaderPlatform, QualityLevel);
	}

	virtual ~FDebugShaderInfoMaterialResource() = default;

	bool TryBuildCompilationOutput(FMaterialCompilationOutput& OutCompilationOutput)
	{
		FStaticParameterSet UsedStaticParams;

		if (MaterialInstance)
		{
			MaterialInstance->GetStaticParameterValues(UsedStaticParams);
		}
		else if (Material)
		{
			Material->GetStaticParameterValues(UsedStaticParams);
		}
		else
		{
			UE_LOGF(LogMaterial, Error, "FDebugShaderInfoMaterialResource created without Material or MaterialInstance");
			return false;
		}

		FMaterialShaderMapId ShaderMapId;
		BuildShaderMapId(ShaderMapId, nullptr);

		// TODO: Handle DDC Key creation once new translator supports it
		if (ShaderMapId.bUsingNewHLSLGenerator && !bEnableRestrictiveMaterialMode)
		{
			FMaterialIRModule Module;

			FMaterialIRModuleBuilder Builder = {
				.MaterialInterface = GetMaterialInterface(),
				.ShaderPlatform = GetShaderPlatform(),
				.TargetPlatform = nullptr,
				.FeatureLevel = ShaderMapId.FeatureLevel,
				.QualityLevel = GetQualityLevel(),
				.BlendMode = GetBlendMode(),
				.StaticParameters = &UsedStaticParams,
			};

			if (!Builder.Build(&Module))
			{
				return false;
			}

			OutCompilationOutput = Module.GetCompilationOutput();

			return true;
		}
		else
		{
			FString MaterialTranslationDDCKeyString = GetMaterialShaderMapKeyString(ShaderMapId, FConservativeShaderParameters(this), GetShaderPlatform(), false);
			FHLSLMaterialTranslator MaterialTranslator(this, OutCompilationOutput, UsedStaticParams, GetShaderPlatform(), GetQualityLevel(), ShaderMapId.FeatureLevel, nullptr, &ShaderMapId.SubstrateCompilationConfig, MaterialTranslationDDCKeyString);
			EHLSLMaterialTranslatorResult Result = MaterialTranslator.Translate(false);

			return Result == EHLSLMaterialTranslatorResult::Success;
		}
	}

};

TArray<FDebugShaderTypeInfo> GetShadersUsingCompilationParameters_GameThread(UMaterialInterface* InMaterial, const EShaderPlatform InShaderPlatform)
{
	UMaterialInstance* Inst = Cast<UMaterialInstance>(InMaterial);
	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(nullptr);

	FMaterialResource* Resource = InMaterial->GetMaterialResource(InShaderPlatform);

	if (Resource)
	{
		if (Resource->IsGameThreadShaderMapComplete())
		{
			const FMaterialShaderMap* ShaderMap = Resource->GetGameThreadShaderMap();
			const FMaterialCompilationOutput& CompOutput = ShaderMap->GetCompilationOutput();

			TArray<FDebugShaderTypeInfo> ShaderTypeInfos;
			Resource->GetShaderTypes(LayoutParams, ShaderTypeInfos, FCompilationShaderParameters(Resource, CompOutput));

			return ShaderTypeInfos;
		}

	}
	UMaterial* BaseMaterial = InMaterial->GetMaterial();
	FDebugShaderInfoMaterialResource ShaderInfoResource(BaseMaterial, Inst, GMaxRHIFeatureLevel, InShaderPlatform, EMaterialQualityLevel::Num);

	FMaterialCompilationOutput CompOutput;
	const bool bValidOutput = ShaderInfoResource.TryBuildCompilationOutput(CompOutput);

	TArray<FDebugShaderTypeInfo> ShaderTypeInfos;
	if (bValidOutput)
	{
		ShaderInfoResource.GetShaderTypes(LayoutParams, ShaderTypeInfos, FCompilationShaderParameters(&ShaderInfoResource, CompOutput));
	}
	else
	{
		UE_LOGF(LogMaterial, Display, ("Failed to translate material resource (in %ls hierarchy) for Compilation Output. Returning Conservative Shader Layout"),
			*ShaderInfoResource.GetFriendlyName());

		ShaderInfoResource.GetShaderTypes(LayoutParams, ShaderTypeInfos, FConservativeShaderParameters(&ShaderInfoResource));
	}

	return ShaderTypeInfos;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
