// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Engine/EngineTypes.h"
#include "Engine/TextureDefines.h"
#include "MaterialShared.h"
#include "Materials/MaterialExpressionTransform.h"
#include "Materials/MaterialExpressionTransformPosition.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "RHIFeatureLevel.h"
#include "RHIShaderPlatform.h"
#include "SceneTypes.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"

namespace UE::Cook { class FCookEventContext; }
struct FAppendToClassSchemaContext;
struct FMaterialResourceForCooking;
class UMaterialExpressionCustom;

namespace UE::MaterialInterface::Private
{

/** Record the instance-specific cook dependencies (config, hlsl) from the resources for a UMaterialInterface. */
void RecordMaterialDependenciesForCook(UE::Cook::FCookEventContext CookContext,
	TConstArrayView<FMaterialResourceForCooking> Resources);

/** Record the cook dependencies that apply to every UMaterialInterface. */
void HashMaterialStaticClassDependenciesForCook(FAppendToClassSchemaContext& Context);

/**
 * A copy of the fields saved from an FMaterialResourceForCooking that includes only the data necessary for
 * calculating non-UObject-based material dependencies. This struct is marshalled to CompactBinary and stored in cook
 * metadata and read at the beginning of incremental cooks to check whether the material's dependencies have changed.
 */
struct FRecordedMaterialResourceForCooking
{
	FMaterialShaderMapId ShaderMapId;
	FConservativeShaderParameters ShaderParameters;
	EShaderPlatform ShaderPlatform;
	/**
	 * We have to handle ExpressionIncludes separately rather than using ShaderMapId.ExpressionIncludesHash,
	 * because we need to record the filename of each expression include to recalculate it.
	 */
	TArray<FString> ExpressionIncludes;

	void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
};

} // namespace UE::MaterialInterface::Private

// Utility functions shared between the old and new material translator.
namespace UE::MaterialTranslatorUtils
{
	// Converts a TextureAddress enum to its corresponding string representation.
	const TCHAR* GetVTAddressMode(TextureAddress Address);

	// Determines the texture addressing mode for a texture based on its sampler source.
	void GetTextureAddressForSamplerSource(const UTexture* InTexture, ESamplerSourceMode InSamplerSource, TextureAddress& OutStaticAddressX, TextureAddress& OutStaticAddressY, TextureAddress& OutStaticAddressZ);

	// Returns true if the LWCs are enabled in the engine.
	bool IsLWCEnabled();

	// Returns the current LWC truncation mode.
	int32 GetLWCTruncateMode();

	// Maps a vector transform source to a common basis (e.g., world, local, tangent).
	EMaterialCommonBasis GetMaterialCommonBasis(EMaterialVectorCoordTransformSource X);

	// Maps a vector transform operation to a common basis.
	EMaterialCommonBasis GetMaterialCommonBasis(EMaterialVectorCoordTransform X);

	// Maps a position transform source to a common basis.
	EMaterialCommonBasis GetMaterialCommonBasis(EMaterialPositionTransformSource X);

	// Returns true if the given float is a power of two.
	bool IsFloatPowerOfTwo(float InValue);

	// Generates HLSL defines to remap PPI_UserSceneTexture0-6 tokens generated by the HLSL emitter/translator to PPI_PostProcessInput0-6,
	// using unoccupied slots (UsedSceneTextures bits not set) in the order they are encountered.
	FString GenerateUserSceneTextureRemapHLSLDefines(const FMaterialCompilationOutput& CompilationOutput);

	// Converts a scene texture identifier to its corresponding HLSL string representation.
	FString SceneTextureIdToHLSLString(ESceneTextureId TexId);

	// Returns the value type of a texture property with the specified texture value type, e.g. MCT_Float3 for Volume textures.
	EMaterialValueType GetTexturePropertyValueType(EMaterialValueType TextureType);

	// Maps the specified position origin to its corresponding EWorldPositionIncludedOffsets value with an include offsets option.
	EWorldPositionIncludedOffsets GetWorldPositionTypeWithOrigin(EPositionOrigin PositionOrigin, bool IncludeOffsets = true);

	// Returns a human readable name (en-US) for the specified position origin enumeration value.
	FName GetWorldPositionInputName(EPositionOrigin PositionOrigin);

	// Final compile validation logic, run at the end of translation to HLSL.
	void FinalCompileValidation(
		UMaterial* Material,
		const FMaterialCompilationOutput& MaterialCompilationOutput,
		FMaterialShadingModelField MaterialShadingModels,
		EBlendMode BlendMode,
		bool bHasFrontMaterialExpr,
		EShaderPlatform Platform,
		TArray<FString>& OutErrors);

	// Utility function to search for the given HLSL Identifier in a block of Code.  Takes into account comments and quoted text
	// that should be ignored when searching for identifiers.  Returns the start of the identifier, and the input pointer "Code"
	// will be advanced to the first non-whitespace, non-comment character after the end of the identifier.  Or null will be
	// returned if the identifier isn't found.  OutCompoundTokens contain additional tokens if it's part of a compound identifier
	// separated by periods (for "Identifier.x", OutCompoundTokens would include "x").
	const TCHAR* FindHlslIdentifierInCode(const TCHAR*& Code, const TCHAR* Identifier, TArray<FStringView>& OutCompoundTokens);

	// Fixup for SceneTexture and UserSceneTexture inputs to custom HLSL. See implementation for more info.
	FString CustomExpressionSceneTextureInputFixup(const UMaterialExpressionCustom* Custom, const TCHAR* Code, TArray<int8>& OutSceneTextureInfo);
	
	// Check for whether development feature with the given name should be enabled (currently just used for editor Selection Color).
	bool IsDevelopmentFeatureEnabled(const FName& FeatureName, EShaderPlatform Platform, UMaterial* Material);


} // namespace UE::MaterialTranslatorUtils

/** Builds a conservative-vs-compilation shader parameter diff string for the given material. */
FString DescribeCompilationShaderParametersDifferences(const FMaterial* InMaterial, const FCompilationShaderParameters& InCompilationParams);

/** Returns a field-by-field diff of two FMaterialShaderParametersBase structs. */
FString DescribeShaderParameterDifferences(const FMaterialShaderParametersBase& A,
		const FMaterialShaderParametersBase& B, const TCHAR* LabelA = TEXT("A"), const TCHAR* LabelB = TEXT("B"));

///////////////////////////////////////////////////////
// CompactBinary interface for FRecordedMaterialResourceForCooking
///////////////////////////////////////////////////////

inline FCbWriter& operator<<(FCbWriter& Writer,
	const UE::MaterialInterface::Private::FRecordedMaterialResourceForCooking& Value)
{
	Value.Save(Writer);
	return Writer;
}
bool LoadFromCompactBinary(FCbFieldView Field,
	UE::MaterialInterface::Private::FRecordedMaterialResourceForCooking& OutValue);

///////////////////////////////////////////////////////
// Implementation details for UE_DEFINEINLINE_COMPACTBINARY_ENUM_INT
///////////////////////////////////////////////////////

template <typename IntType>
inline IntType GetViewFieldAsInteger(FCbFieldView FieldView, IntType DefaultValue)
{
	static_assert(sizeof(IntType) != 1, "Not implemented");
}
template <>
inline uint8 GetViewFieldAsInteger(FCbFieldView FieldView, uint8 DefaultValue)
{
	return FieldView.AsUInt8(DefaultValue);
}
template <>
inline uint16 GetViewFieldAsInteger(FCbFieldView FieldView, uint16 DefaultValue)
{
	return FieldView.AsUInt16(DefaultValue);
}

/**
 * Declare and Define inline functions for LoadFromCompactBinary and operator<< for an enum, by converting
 * the enum to an integer.
 */
#define UE_DEFINEINLINE_COMPACTBINARY_ENUM_INT(EnumType, IntType, NumValues, InvalidValue)				\
	inline bool LoadFromCompactBinary(FCbFieldView Field, EnumType& OutValue)							\
	{																									\
		IntType IntValue = GetViewFieldAsInteger<IntType>(Field, static_cast<IntType>(InvalidValue));	\
		if (Field.HasError() || IntValue >= static_cast<IntType>(NumValues))							\
		{																								\
			OutValue = static_cast<EnumType>(InvalidValue);												\
			return false;																				\
		}																								\
		OutValue = static_cast<EnumType>(IntValue);														\
		return true;																					\
	}																									\
	inline FCbWriter& operator<<(FCbWriter& Writer, EnumType Value)										\
	{																									\
		Writer << static_cast<IntType>(Value);															\
		return Writer;																					\
	}

///////////////////////////////////////////////////////
// CompactBinary functions for enums used by FRecordedMaterialResourceForCooking
///////////////////////////////////////////////////////

UE_DEFINEINLINE_COMPACTBINARY_ENUM_INT(EShaderPlatform, uint16, SP_NumPlatforms, SP_NumPlatforms);
UE_DEFINEINLINE_COMPACTBINARY_ENUM_INT(EMaterialQualityLevel::Type, uint8,
	EMaterialQualityLevel::Num, EMaterialQualityLevel::Low);
UE_DEFINEINLINE_COMPACTBINARY_ENUM_INT(ERHIFeatureLevel::Type, uint8,
	ERHIFeatureLevel::Num, ERHIFeatureLevel::ES3_1);


#endif // WITH_EDITOR
