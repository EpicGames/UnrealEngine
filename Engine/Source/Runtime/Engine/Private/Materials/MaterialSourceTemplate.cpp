// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MaterialSourceTemplate.h"

#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#include "MaterialShared.h"
#include "Misc/ScopeRWLock.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

FMaterialSourceTemplate& FMaterialSourceTemplate::Get()
{
	static FMaterialSourceTemplate Instance;
	return Instance;
}

FMaterialSourceTemplate::FMaterialSourceTemplate()
{
	for (int32 ShaderPlatformIndex = 0; ShaderPlatformIndex < SP_NumPlatforms; ++ShaderPlatformIndex)
	{
		MaterialTemplateLineNumbers[ShaderPlatformIndex] = INDEX_NONE;
		bFileWatchInvalidation[ShaderPlatformIndex] = false;
	}

	FString FileWatchDirectory(FPaths::EngineDir() + TEXT("Shaders/Private"));

	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
	DirectoryWatcherModule.Get()->RegisterDirectoryChangedCallback_Handle(
		FileWatchDirectory,
		IDirectoryWatcher::FDirectoryChanged::CreateLambda([this](const TArray<struct FFileChangeData>& Changes)
		{
			for (FFileChangeData const& Change : Changes)
			{
				if ((Change.Action == FFileChangeData::FCA_Modified || Change.Action == FFileChangeData::FCA_Added) && Change.Filename.EndsWith(TEXT("MaterialTemplate.ush")))
				{
					for (int32 ShaderPlatformIndex = 0; ShaderPlatformIndex < SP_NumPlatforms; ++ShaderPlatformIndex)
					{
						FWriteScopeLock Lock{ RWLocks[ShaderPlatformIndex] };
						// Only set invalidation flag for this shader platform if we have the template loaded.
						bFileWatchInvalidation[ShaderPlatformIndex] = !Templates[ShaderPlatformIndex].GetTemplateString().IsEmpty();
					}
				}
			}
		}),
		FileWatchHandle);
}

FStringTemplateResolver FMaterialSourceTemplate::BeginResolve(EShaderPlatform ShaderPlatform, int32* MaterialTemplateLineNumber)
{
	Preload(ShaderPlatform);

	if (MaterialTemplateLineNumber)
	{
		*MaterialTemplateLineNumber = MaterialTemplateLineNumbers[ShaderPlatform];
	}

	return { Templates[ShaderPlatform], 50 * 1024 };
}

const FStringTemplate& FMaterialSourceTemplate::GetTemplate(EShaderPlatform ShaderPlatform)
{
	Preload(ShaderPlatform);
	return Templates[ShaderPlatform];
}

const FString& FMaterialSourceTemplate::GetTemplateHashString(EShaderPlatform ShaderPlatform)
{
	Preload(ShaderPlatform);
	return TemplateHashString[ShaderPlatform];
}

bool FMaterialSourceTemplate::Preload(EShaderPlatform ShaderPlatform)
{
	// Is the material source template already loaded?
	{
		FReadScopeLock Lock{ RWLocks[ShaderPlatform] };
		if (!Templates[ShaderPlatform].GetTemplateString().IsEmpty() && !bFileWatchInvalidation[ShaderPlatform])
		{
			return true;
		}
	}

	// Material source template not yet loaded. Acquire a write lock an try again.
	FWriteScopeLock Lock{ RWLocks[ShaderPlatform] };
	if (!Templates[ShaderPlatform].GetTemplateString().IsEmpty() && !bFileWatchInvalidation[ShaderPlatform])
	{
		return true;
	}

	static const TCHAR* VirtualFilePath = TEXT("/Engine/Private/MaterialTemplate.ush");
	
	if (bFileWatchInvalidation[ShaderPlatform])
	{
		InvalidateShaderFileCacheEntry(VirtualFilePath, ShaderPlatform);
		bFileWatchInvalidation[ShaderPlatform] = false;
	}

	FString MaterialTemplateString;
	LoadShaderSourceFileChecked(VirtualFilePath, ShaderPlatform, MaterialTemplateString);

	// Normalize line endings -- preprocessor does this later if necessary, but that can run faster if it's already done, and doing it here
	// means it only happens once when the template gets loaded, rather than for every Material shader.
	MaterialTemplateString.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);

	// Find the string index of the '#line' statement in MaterialTemplate.usf
	const int32 LineIndex = MaterialTemplateString.Find(TEXT("#line"), ESearchCase::CaseSensitive);
	check(LineIndex != INDEX_NONE);

	// Count line endings before the '#line' statement
	int32 TemplateLineNumber = 1;
	int32 StartPosition = LineIndex + 1;
	const TCHAR* Cur = *MaterialTemplateString;
	const TCHAR* End = Cur + LineIndex;
	while (Cur != End)
	{
		if (*Cur++ == TEXT('\n'))
		{
			TemplateLineNumber++;
		}
	}

	// Save the material template line numbers for this shader platform
	MaterialTemplateLineNumbers[ShaderPlatform] = TemplateLineNumber;

	// Load the material string template
	FStringTemplate::FErrorInfo ErrorInfo;
	if (!Templates[ShaderPlatform].Load(MoveTemp(MaterialTemplateString), ErrorInfo))
	{
		UE_LOGF(LogMaterial, Error, "Error in MaterialTemplate.ush source template at line %d offset %d: %ls", ErrorInfo.Line, ErrorInfo.Offset, ErrorInfo.Message.GetData());
		return false;
	}

	// Calculate a template hash based on the parameter contained (regardless of order)
	FSHA1 TemplateHash = {};
	TArray<FStringView> Parameters;
	Templates[ShaderPlatform].GetParameters(Parameters);
	Parameters.Sort();
	for (const FStringView& Param : Parameters)
	{
		TemplateHash.UpdateWithString(Param.GetData(), Param.Len());
	}

	TemplateHashString[ShaderPlatform] = LexToString(TemplateHash.Finalize());

	return true;
}

#endif // WITH_EDITOR
