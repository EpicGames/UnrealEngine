// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/StringTemplate.h"
#include "HAL/CriticalSection.h"
#include "RHI.h"

#if WITH_EDITOR

/** Cache of material templates to be used for material source generation. */
struct FMaterialSourceTemplate
{
	/** Returns the singleton instance */
	static FMaterialSourceTemplate& Get();

	FMaterialSourceTemplate();

	/** Creates and returns a string template resolver for the specified shader platform.
	 * The appropriate template is loaded for the first time if it wasn't loaded before.*/
	FStringTemplateResolver BeginResolve(EShaderPlatform ShaderPlatform, int32* MaterialTemplateLineNumber = nullptr);

	/* Returns the string template for specified shader platform.  */
	const FStringTemplate& GetTemplate(EShaderPlatform ShaderPlatform);

	/** Return the hash of the material template for specified shader platform 
	  * The information hashed is the string contained within "$TemplateVersion{...}" and the name of the parameters
	  * contained in the template string (insensitive to ordering).
	  */
	const FString& GetTemplateHashString(EShaderPlatform ShaderPlatform);

	/** Loads a material source template if still unloaded. */
	bool Preload(EShaderPlatform ShaderPlatform);

	/** Cached templates per shader platform */
	FStringTemplate Templates[SP_NumPlatforms];

	/** Hash of the template parameters and TemplateVersion parameter extracted from file */
	FString TemplateHashString[SP_NumPlatforms];

	/** Cached material template line numbers per shader platform */
	int32 MaterialTemplateLineNumbers[SP_NumPlatforms];

	/** Per shader platform flags that are set true if the template is invalidated by a file change. */
	bool bFileWatchInvalidation[SP_NumPlatforms];

	/** Handle for file watcher. */
	FDelegateHandle FileWatchHandle;

	/** RW locks for thread safe access */
	FRWLock RWLocks[SP_NumPlatforms];
};

#endif // WITH_EDITOR
