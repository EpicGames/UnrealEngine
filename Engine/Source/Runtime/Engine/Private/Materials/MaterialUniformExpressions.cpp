// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialShared.cpp: Shared material implementation.
=============================================================================*/

#include "Materials/MaterialUniformExpressions.h"

#include "EngineModule.h"
#include "Engine/Texture.h"
#include "Engine/TextureCollection.h"
#include "SceneManagement.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialRenderProxy.h"
#include "RHIStaticStates.h"
#include "Shader/PreshaderEvaluate.h"
#include "Shader/Preshader2.h"
#include "ExternalTexture.h"

#include "GlobalRenderResources.h"
#include "Shader/PreshaderTypes.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "VT/RuntimeVirtualTexture.h"
#include "TextureResource.h"
#include "MaterialCache/MaterialCacheAttribute.h"
#include "MaterialCache/MaterialCacheTagProvider.h"
#include "MaterialCache/MaterialCacheTagSceneData.h"
#include "Engine/VirtualTextureCollection.h"

void WriteMaterialUniformAccess(UE::Shader::EValueComponentType ComponentType, uint32 NumComponents, uint32 UniformOffset, const TCHAR* MaterialNamePrefix, FStringBuilderBase& OutResult)
{
	static const TCHAR IndexToMask[] = TEXT("xyzw");
	uint32 RegisterIndex = UniformOffset / 4;
	uint32 RegisterOffset = UniformOffset % 4;
	uint32 NumComponentsToWrite = NumComponents;
	bool bConstructor = false;

	check(ComponentType == UE::Shader::EValueComponentType::Float); // TODO - other types

	while (NumComponentsToWrite > 0u)
	{
		const uint32 NumComponentsInRegister = FMath::Min(NumComponentsToWrite, 4u - RegisterOffset);
		if (NumComponentsInRegister < NumComponents && !bConstructor)
		{
			// Uniform will be split across multiple registers, so add the constructor to concat them together
			OutResult.Appendf(TEXT("float%d("), NumComponents);
			bConstructor = true;
		}

		OutResult.Appendf(TEXT("%s.PreshaderBuffer[%u]"), MaterialNamePrefix, RegisterIndex);
		// Can skip writing mask if we're taking all 4 components from the register
		if (NumComponentsInRegister < 4u)
		{
			OutResult.AppendChar(TCHAR('.'));
			for (uint32 i = 0u; i < NumComponentsInRegister; ++i)
			{
				OutResult.AppendChar(IndexToMask[RegisterOffset + i]);
			}
		}
		NumComponentsToWrite -= NumComponentsInRegister;
		RegisterIndex++;
		RegisterOffset = 0u;
		if (NumComponentsToWrite > 0u)
		{
			OutResult.Append(TEXT(", "));
		}
	}
	if (bConstructor)
	{
		OutResult.Append(TEXT(")"));
	}
}

TLinkedList<FMaterialUniformExpressionType*>*& FMaterialUniformExpressionType::GetTypeList()
{
	static TLinkedList<FMaterialUniformExpressionType*>* TypeList = NULL;
	return TypeList;
}

TMap<FName,FMaterialUniformExpressionType*>& FMaterialUniformExpressionType::GetTypeMap()
{
	static TMap<FName,FMaterialUniformExpressionType*> TypeMap;

	// Move types from the type list to the type map.
	TLinkedList<FMaterialUniformExpressionType*>* TypeListLink = GetTypeList();
	while(TypeListLink)
	{
		TLinkedList<FMaterialUniformExpressionType*>* NextLink = TypeListLink->Next();
		FMaterialUniformExpressionType* Type = **TypeListLink;

		TypeMap.Add(FName(Type->Name),Type);
		TypeListLink->Unlink();
		delete TypeListLink;

		TypeListLink = NextLink;
	}

	return TypeMap;
}

FGuid FMaterialRenderContext::GetExternalTextureGuid(const FGuid& ExternalTextureGuid, const FName& ParameterName, int32 SourceTextureIndex) const
{
	FGuid GuidToLookup;
	if (ExternalTextureGuid.IsValid())
	{
		// Use the compile-time GUID if it is set
		GuidToLookup = ExternalTextureGuid;
	}
	else
	{
		const UTexture* TextureParameterObject = nullptr;
		if (!ParameterName.IsNone() && MaterialRenderProxy && MaterialRenderProxy->GetTextureValue(ParameterName, &TextureParameterObject, *this) && TextureParameterObject)
		{
			GuidToLookup = TextureParameterObject->GetExternalTextureGuid();
		}
		else
		{
			// Otherwise attempt to use the texture index in the material, if it's valid
			const UTexture* TextureObject = SourceTextureIndex != INDEX_NONE ? GetIndexedTexture<UTexture>(Material, SourceTextureIndex) : nullptr;
			if (TextureObject)
			{
				GuidToLookup = TextureObject->GetExternalTextureGuid();
			}
		}
	}
	return GuidToLookup;
}

void FMaterialRenderContext::GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureIndex, const UTexture*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialRenderProxy || !MaterialRenderProxy->GetTextureValue(ParameterInfo, &OutValue, *this))
	{
		OutValue = GetIndexedTexture<UTexture>(Material, TextureIndex);
	}
}

void FMaterialRenderContext::GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureIndex, const URuntimeVirtualTexture*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialRenderProxy || !MaterialRenderProxy->GetTextureValue(ParameterInfo, &OutValue, *this))
	{
		OutValue = GetIndexedTexture<URuntimeVirtualTexture>(Material, TextureIndex);
	}
}

void FMaterialRenderContext::GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureIndex, const USparseVolumeTexture*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialRenderProxy || !MaterialRenderProxy->GetTextureValue(ParameterInfo, &OutValue, *this))
	{
		OutValue = GetIndexedTexture<USparseVolumeTexture>(Material, TextureIndex);
	}
}

void FMaterialRenderContext::GetTextureCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureCollectionIndex, const UTextureCollection*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialRenderProxy || !MaterialRenderProxy->GetTextureCollectionValue(ParameterInfo, &OutValue, *this))
	{
		OutValue = GetIndexedTextureCollection(Material, TextureCollectionIndex);
	}
}

FMaterialUniformExpressionType::FMaterialUniformExpressionType(const TCHAR* InName)
	: Name(InName)
{
	// Put the type in the type list until the name subsystem/type map are initialized.
	(new TLinkedList<FMaterialUniformExpressionType*>(this))->LinkHead(GetTypeList());
}

void FMaterialUniformExpression::WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const
{
	UE_LOGF(LogMaterial, Warning, "Missing WriteNumberOpcodes impl for %ls", GetType()->GetName());
	OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::ConstantZero).Write(UE::Shader::EValueType::Float1);
}

void FUniformParameterOverrides::SetNumericOverride(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value)
{
	const FNumericParameterKey Key{ ParameterInfo, Type };
	const UE::Shader::EValueType ShaderType = GetShaderValueType(Type);
	check(ShaderType == Value.GetType());
	NumericOverrides.FindOrAdd(Key) = Value;
}

void FUniformParameterOverrides::ClearNumericOverride(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo)
{
	const FNumericParameterKey Key{ ParameterInfo, Type };
	NumericOverrides.Remove(Key);
}

bool FUniformParameterOverrides::GetNumericOverride(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, UE::Shader::FValue& OutValue) const
{
	const FNumericParameterKey Key{ ParameterInfo, Type };
	const UE::Shader::FValue* Result = NumericOverrides.Find(Key);
	if (Result)
	{
		OutValue = *Result;
		return true;
	}
	return false;
}

void FUniformParameterOverrides::SetTextureOverride(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& ParameterInfo, UTexture* Texture)
{
	check(IsInGameThread());
	const uint32 TypeIndex = (uint32)Type;
	const FTextureParameterKey Key{ ParameterInfo.ParameterInfo, ParameterInfo.TextureIndex };

	if (Texture)
	{
		GameThreadTextureOverides[TypeIndex].FindOrAdd(Key) = Texture;
	}
	else
	{
		GameThreadTextureOverides[TypeIndex].Remove(Key);
	}

	FUniformParameterOverrides* Self = this;
	ENQUEUE_RENDER_COMMAND(SetTextureOverrideCommand)(
		[Self, TypeIndex, Key, Texture](FRHICommandListImmediate& RHICmdList)
	{
		if (Texture)
		{
			Self->RenderThreadTextureOverrides[TypeIndex].FindOrAdd(Key) = Texture;
		}
		else
		{
			Self->RenderThreadTextureOverrides[TypeIndex].Remove(Key);
		}
	});
}

UTexture* FUniformParameterOverrides::GetTextureOverride_GameThread(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& ParameterInfo) const
{
	check(IsInGameThread() || IsInParallelGameThread());
	const uint32 TypeIndex = (uint32)Type;
	const FTextureParameterKey Key{ ParameterInfo.ParameterInfo, ParameterInfo.TextureIndex };
	UTexture* const* Result = GameThreadTextureOverides[TypeIndex].Find(Key);
	return Result ? *Result : nullptr;
}

UTexture* FUniformParameterOverrides::GetTextureOverride_RenderThread(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& ParameterInfo) const
{
	check(IsInParallelRenderingThread());
	const uint32 TypeIndex = (uint32)Type;
	const FTextureParameterKey Key{ ParameterInfo.ParameterInfo, ParameterInfo.TextureIndex };
	UTexture* const* Result = RenderThreadTextureOverrides[TypeIndex].Find(Key);
	return Result ? *Result : nullptr;
}

bool FUniformExpressionSet::IsEmpty() const
{
	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		if (UniformTextureParameters[TypeIndex].Num() != 0)
		{
			return false;
		}
	}

	return UniformNumericParameters.Num() == 0
		&& UniformPreshaders.Num() == 0
		&& UniformExternalTextureParameters.Num() == 0
		&& UniformTextureCollectionParameters.Num() == 0
		&& VTStacks.Num() == 0
		&& MaterialCacheTagStacks.Num() == 0
		&& ParameterCollections.Num() == 0;
}

bool FUniformExpressionSet::operator==(const FUniformExpressionSet& ReferenceSet) const
{
	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		if (UniformTextureParameters[TypeIndex].Num() != ReferenceSet.UniformTextureParameters[TypeIndex].Num())
		{
			return false;
		}
	}

	if (UniformNumericParameters.Num() != ReferenceSet.UniformNumericParameters.Num()
		|| UniformPreshaders.Num() != ReferenceSet.UniformPreshaders.Num()
		|| UniformExternalTextureParameters.Num() != ReferenceSet.UniformExternalTextureParameters.Num()
		|| UniformTextureCollectionParameters.Num() != ReferenceSet.UniformTextureCollectionParameters.Num()
		|| VTStacks.Num() != ReferenceSet.VTStacks.Num()
		|| MaterialCacheTagStacks.Num() != ReferenceSet.MaterialCacheTagStacks.Num()
		|| ParameterCollections.Num() != ReferenceSet.ParameterCollections.Num())
	{
		return false;
	}

	for (int32 i = 0; i < UniformNumericParameters.Num(); i++)
	{
		if (UniformNumericParameters[i] != ReferenceSet.UniformNumericParameters[i])
		{
			return false;
		}
	}

	for (int32 i = 0; i < UniformPreshaders.Num(); i++)
	{
		if (UniformPreshaders[i] != ReferenceSet.UniformPreshaders[i])
		{
			return false;
		}
	}

	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		for (int32 i = 0; i < UniformTextureParameters[TypeIndex].Num(); i++)
		{
			if (UniformTextureParameters[TypeIndex][i] != ReferenceSet.UniformTextureParameters[TypeIndex][i])
			{
				return false;
			}
		}
	}

	for (int32 i = 0; i < UniformExternalTextureParameters.Num(); i++)
	{
		if (UniformExternalTextureParameters[i] != ReferenceSet.UniformExternalTextureParameters[i])
		{
			return false;
		}
	}


	for (int32 i = 0; i < UniformTextureCollectionParameters.Num(); i++)
	{
		if (UniformTextureCollectionParameters[i] != ReferenceSet.UniformTextureCollectionParameters[i])
		{
			return false;
		}
	}

	for (int32 i = 0; i < VTStacks.Num(); i++)
	{
		if (VTStacks[i] != ReferenceSet.VTStacks[i])
		{
			return false;
		}
	}

	for (int32 i = 0; i < MaterialCacheTagStacks.Num(); i++)
	{
		if (MaterialCacheTagStacks[i] != ReferenceSet.MaterialCacheTagStacks[i])
		{
			return false;
		}
	}

	for (int32 i = 0; i < ParameterCollections.Num(); i++)
	{
		if (ParameterCollections[i] != ReferenceSet.ParameterCollections[i])
		{
			return false;
		}
	}

	if (UniformPreshaderData != ReferenceSet.UniformPreshaderData)
	{
		return false;
	}

	return true;
}

FString FUniformExpressionSet::GetSummaryString() const
{
	return FString::Printf(TEXT("(%u preshaders, %u 2d tex, %u cube tex, %u 2darray tex, %u cubearray tex, %u 3d tex, %u virtual tex, %u sparse volume tex, %u external tex, %u tex collections, %u VT stacks, %u material cache tag stacks, %u collections)"),
		UniformPreshaders.Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::Standard2D].Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::Cube].Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::Array2D].Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::ArrayCube].Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::Volume].Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::Virtual].Num(),
		UniformTextureParameters[(uint32)EMaterialTextureParameterType::SparseVolume].Num(),
		UniformExternalTextureParameters.Num(),
		UniformTextureCollectionParameters.Num(),
		VTStacks.Num(),
		MaterialCacheTagStacks.Num(),
		ParameterCollections.Num()
		);
}

void FUniformExpressionSet::SetParameterCollections(TConstArrayView<const UMaterialParameterCollection*> InCollections)
{
	ParameterCollections.Empty(InCollections.Num());

	for (int32 CollectionIndex = 0; CollectionIndex < InCollections.Num(); CollectionIndex++)
	{
		ParameterCollections.Add(InCollections[CollectionIndex]->StateId);
	}
}

#if WITH_EDITOR
uint32 CountTextureCollectionParameterInfo(const TMemoryImageArray<FMaterialTextureCollectionParameterInfo>& ParameterCollection, EShaderFrequencyMask ShaderFrequencyMask)
{
	uint32 NumElements = 0;
	for (const FMaterialTextureCollectionParameterInfo& ParameterCollectionMember : ParameterCollection)
	{
		if (!ParameterCollectionMember.bIsVirtualCollection)
		{
			continue;
		}

		if (!!(ParameterCollectionMember.ShaderFrequencyMask & ShaderFrequencyMask))
		{
			++NumElements;
		}
	}

	return NumElements;
}

template<class TMemberType>
uint32 CountIndividualMembersSimple(const TMemoryImageArray<TMemberType>& ParameterCollection, EShaderFrequencyMask ShaderFrequencyMask)
{
	uint32 NumElements = 0;
	for (const TMemberType& ParameterCollectionMember : ParameterCollection)
	{
		if (!!(ParameterCollectionMember.ShaderFrequencyMask & ShaderFrequencyMask))
		{
			++NumElements;
		}
	}

	return NumElements;
}

void AddCopySegment(TArray<FCompactionMetaDataSegment>& CopySegments, 
	const FShaderParametersMetadata::FMember& SrcMember,
	const FShaderParametersMetadata::FMember& DestMember,
	uint32 DestMemberSize)
{
	FCompactionMetaDataSegment PerFrequencyUniformMetadata;
	PerFrequencyUniformMetadata.SrcOffset = SrcMember.GetOffset();
	PerFrequencyUniformMetadata.DestOffset = DestMember.GetOffset();
	PerFrequencyUniformMetadata.SizeToCopy = DestMemberSize;
	CopySegments.Add(PerFrequencyUniformMetadata);
}

static FShaderParametersMetadata* CreateCompactShaderParametersMetadata(FCompactUniformExpressionSet* CompactUniformExpressions,
	const TCHAR* UniformBufferLayoutInitializerName,
	const TCHAR* LayoutName, const TCHAR* StructTypeName, const TCHAR* ShaderVariableName,
	const TArray<FShaderParametersMetadata::FMember>& MembersVS, uint32 StructSize, const TArray<FCompactionMetaDataSegment>& CopySegments)
{
	if (CompactUniformExpressions == nullptr)
	{
		return nullptr;
	}

	CompactUniformExpressions->UniformBufferLayoutInitializer = FRHIUniformBufferLayoutInitializer(UniformBufferLayoutInitializerName);

	FShaderParametersMetadata::EUsageFlags UsageFlags = FShaderParametersMetadata::EUsageFlags::None;
	EnumAddFlags(UsageFlags, FShaderParametersMetadata::EUsageFlags::BindlessAccessible);

	FShaderParametersMetadata* NewUniformBufferStruct = new FShaderParametersMetadata(
		FShaderParametersMetadata::EUseCase::DataDrivenUniformBuffer,
		EUniformBufferBindingFlags::Shader,
		LayoutName,
		StructTypeName,
		ShaderVariableName,
		nullptr,
		UE_LOG_SOURCE_FILE(__FILE__),
		__LINE__,
		StructSize,
		MembersVS,
		false,
		&CompactUniformExpressions->UniformBufferLayoutInitializer,
		UsageFlags);

	// Sort by SrcOffset because the destination buffer is compact
	Algo::SortBy(CopySegments, &FCompactionMetaDataSegment::SrcOffset);

	// CopySegments contains the mapping for every member between the canonical UB and the compact one
	// Try to merge copy sections that are contiguous in the canonical UB
	FCompactionMetaDataSegment* FinalCompactionMetaData = &CompactUniformExpressions->CompactionSegments.AddZeroed_GetRef();
	for (const FCompactionMetaDataSegment& CopySegment : CopySegments)
	{
		if (FinalCompactionMetaData->SizeToCopy != 0)
		{
			uint64 ExpectedSrcOffset = FinalCompactionMetaData->SrcOffset + FinalCompactionMetaData->SizeToCopy;
			// Check if there's a hole between the current aggregated segment, and the new property
			if (ExpectedSrcOffset == CopySegment.SrcOffset)
			{
				// Since everything is contiguous, we can add the current property to the aggregated segment
				// Make sure that the destination offsets align. Holes are only supposed to exist in the canonical UB
				check(CopySegment.DestOffset == FinalCompactionMetaData->DestOffset + FinalCompactionMetaData->SizeToCopy);
				FinalCompactionMetaData->SizeToCopy += CopySegment.SizeToCopy;
			}
			else
			{
				// If we have a hole, start a new segment that we will fill below
				FinalCompactionMetaData = &CompactUniformExpressions->CompactionSegments.AddZeroed_GetRef();
			}
		}

		// new segment initialized from the current property's data. Subsequent members will try to merge with it
		if (FinalCompactionMetaData->SizeToCopy == 0)
		{
			FinalCompactionMetaData->SizeToCopy = CopySegment.SizeToCopy;
			FinalCompactionMetaData->SrcOffset = CopySegment.SrcOffset;
			FinalCompactionMetaData->DestOffset = CopySegment.DestOffset;
		}
	}

	return NewUniformBufferStruct;

}

static bool IsNumericMember(const FShaderParametersMetadata::FMember& Member)
{
	switch (Member.GetBaseType())
	{
		case UBMT_INT32:
		case UBMT_UINT32:
		case UBMT_FLOAT32:
			return true;
		default:
			return false;
	}

}

TArray<FShaderParametersMetadata*> FUniformExpressionSet::CreateBufferStructs(EShaderPrecisionModifier::Type ParameterPrecision)
{
	// Make sure FUniformExpressionSet::CreateDebugLayout() is in sync
	TArray<FShaderParametersMetadata::FMember> Members;
	uint32 NextMemberOffset = 0;

	TArray<FShaderParametersMetadata::FMember> MembersVS;
	uint32 NextMemberOffsetVS = 0;
	
	TArray<FCompactionMetaDataSegment> CopySegments;

	EShaderFrequencyMask ShaderFrequencyMaskVS = 1;

	bool bAllowNumericsInCompactVS = true;

	FCompactUniformExpressionSet* CompactUniformsVS = GetCompactUniformsVS();
	bool bAddVSMembers = (CompactUniformsVS != nullptr);

	auto ExecuteAddToMembersVS =
	[&Members, &MembersVS, &bAllowNumericsInCompactVS, &NextMemberOffsetVS, &NextMemberOffset, &CopySegments, bAddVSMembers]
		(uint32 DestNumMembers, uint32 DestCPUSize)
	{
		if (!bAddVSMembers)
		{
			return;
		}

		const FShaderParametersMetadata::FMember& SrcMember = Members.Last();

		if (!bAllowNumericsInCompactVS)
		{
			check(!IsNumericMember(SrcMember));
		}

		new(MembersVS) FShaderParametersMetadata::FMember(SrcMember.GetName(), SrcMember.GetShaderType(), SrcMember.GetFileLine(), NextMemberOffsetVS, SrcMember.GetBaseType(), SrcMember.GetPrecision(),  SrcMember.GetNumRows(), SrcMember.GetNumColumns(), DestNumMembers, SrcMember.GetStructMetadata());
		NextMemberOffsetVS += DestCPUSize;
			
		uint32 SrcSize = NextMemberOffset - SrcMember.GetOffset();
		check(DestCPUSize <= SrcSize);
		AddCopySegment(CopySegments, SrcMember, MembersVS.Last(), DestCPUSize);	
	};

	// As soon as one element of the array has the VS frequency, add the whole array: This makes indexing consistent between the 2 UBs, and HLSL generation easier
	// Preshaders are the exception, since they do change quite a lot and impact VS often. The rest is much rarer
	auto TryAddToMembersVS =
	[&Members, &MembersVS, &NextMemberOffsetVS, &NextMemberOffset, ShaderFrequencyMaskVS, &ExecuteAddToMembersVS]
		<class TMemberType>
		(const TMemoryImageArray<TMemberType>& Container)
	{
		uint32 NumMembers = CountIndividualMembersSimple(Container, ShaderFrequencyMaskVS);
		if (NumMembers > 0)
		{
			uint32 SrcSize = NextMemberOffset - Members.Last().GetOffset();
			ExecuteAddToMembersVS(Members.Last().GetNumElements(), SrcSize);
		}
	};

	if (!UniformTextureCollectionParameters.IsEmpty())
	{
		uint32 BindlessCollectionCount = 0;
		uint32 VirtualCollectionCount  = 0;
		CountTextureCollections(BindlessCollectionCount, VirtualCollectionCount);

		// Bindless collections do not make use of uniforms
		if (VirtualCollectionCount)
		{
			static_assert(sizeof(FUintVector4) * UE::HLSL::IndirectVirtualTextureUniformDWord4Count == sizeof(UE::HLSL::FIndirectVirtualTextureUniform), "Unexpected size");
			new(Members) FShaderParametersMetadata::FMember(TEXT("TextureCollectionPackedUniforms"), TEXT("FIndirectVirtualTextureUniform"), __LINE__, NextMemberOffset, UBMT_UINT32, EShaderPrecisionModifier::Float, 1, 4, UE::HLSL::IndirectVirtualTextureUniformDWord4Count * VirtualCollectionCount, NULL);
			NextMemberOffset += sizeof(UE::HLSL::FIndirectVirtualTextureUniform) * VirtualCollectionCount;

			uint32 NumMembers = CountTextureCollectionParameterInfo(UniformTextureCollectionParameters, ShaderFrequencyMaskVS);
			if (NumMembers > 0)
			{
				uint32 SrcSize = NextMemberOffset - Members.Last().GetOffset();
				ExecuteAddToMembersVS(Members.Last().GetNumElements(), SrcSize);
			}
		}
	}
	
	if (VTStacks.Num())
	{
		// 2x uint4 per VTStack
		new(Members) FShaderParametersMetadata::FMember(TEXT("VTPackedPageTableUniform"),TEXT(""),__LINE__,NextMemberOffset, UBMT_UINT32, EShaderPrecisionModifier::Float, 1, 4, VTStacks.Num() * 2, NULL);
		NextMemberOffset += VTStacks.Num() * sizeof(FUintVector4) * 2;
		TryAddToMembersVS(VTStacks);
	}

	if (MaterialCacheTagStacks.Num())
	{
		new(Members) FShaderParametersMetadata::FMember(TEXT("MaterialCacheTagPageTableUniform"),TEXT(""), __LINE__, NextMemberOffset, UBMT_UINT32, EShaderPrecisionModifier::Float, 1, 4, MaterialCacheTagStacks.Num(), NULL);
		NextMemberOffset += MaterialCacheTagStacks.Num() * sizeof(FUintVector4);
		TryAddToMembersVS(MaterialCacheTagStacks);
	}

	const int32 NumVirtualTextures = UniformTextureParameters[(uint32)EMaterialTextureParameterType::Virtual].Num();
	if (NumVirtualTextures > 0)
	{
		// 1x uint4 per Virtual Texture
		new(Members) FShaderParametersMetadata::FMember(TEXT("VTPackedUniform"), TEXT(""),__LINE__,NextMemberOffset, UBMT_UINT32, EShaderPrecisionModifier::Float, 1, 4, NumVirtualTextures, NULL);
		NextMemberOffset += NumVirtualTextures * sizeof(FUintVector4);
		TryAddToMembersVS(UniformTextureParameters[(uint32)EMaterialTextureParameterType::Virtual]);
	}

	const int32 NumSparseVolumeTextures = UniformTextureParameters[(uint32)EMaterialTextureParameterType::SparseVolume].Num();
	if (NumSparseVolumeTextures > 0)
	{
		// 2x uint4 per SVT
		new(Members) FShaderParametersMetadata::FMember(TEXT("SVTPackedUniform"), TEXT(""), __LINE__, NextMemberOffset, UBMT_UINT32, EShaderPrecisionModifier::Float, 1, 4, NumSparseVolumeTextures * 2, NULL);
		NextMemberOffset += NumSparseVolumeTextures * sizeof(FUintVector4) * 2;
		TryAddToMembersVS(UniformTextureParameters[(uint32)EMaterialTextureParameterType::SparseVolume]);
	}

	if (UniformPreshaderBufferSize > 0u)
	{	
		// ParameterPrecision below will either be Half or Float based on a user's selection on this material's override floating point option in Material Editor
		// "PreshaderBuffer" uses half precision when the option "Use Half-precision also on shader parameters" is selected
		// Other options will make the uniform "PreshaderBuffer" use full float precision
		// Using the material fp override on a per material case gives users fine grained control to choose between taking the cvar r.Mobile.FloatPrecisionMode value with "Default" or by explicitly choosing the level of half-precision they want 
		new(Members) FShaderParametersMetadata::FMember(TEXT("PreshaderBuffer"), TEXT(""), __LINE__, NextMemberOffset, UBMT_FLOAT32, ParameterPrecision, 1, 4, UniformPreshaderBufferSize, NULL);
		
		NextMemberOffset += UniformPreshaderBufferSize * sizeof(FVector4f);
		if (CompactUniformsVS && CompactUniformsVS->UniformPreshaderBufferSize > 0)
		{
			ExecuteAddToMembersVS(CompactUniformsVS->UniformPreshaderBufferSize, CompactUniformsVS->UniformPreshaderBufferSize * sizeof(FVector4f));
		}
	}

	bAllowNumericsInCompactVS = false;

	check((NextMemberOffset % (2 * SHADER_PARAMETER_POINTER_ALIGNMENT)) == 0);
	check((NextMemberOffsetVS % (2 * SHADER_PARAMETER_POINTER_ALIGNMENT)) == 0);

	const uint32 MaxUniformExpressions = 128;

	static FString Texture2DNames[MaxUniformExpressions];
	static FString Texture2DSamplerNames[MaxUniformExpressions];
	static FString TextureCubeNames[MaxUniformExpressions];
	static FString TextureCubeSamplerNames[MaxUniformExpressions];
	static FString Texture2DArrayNames[MaxUniformExpressions];
	static FString Texture2DArraySamplerNames[MaxUniformExpressions];
	static FString TextureCubeArrayNames[MaxUniformExpressions];
	static FString TextureCubeArraySamplerNames[MaxUniformExpressions];
	static FString VolumeTextureNames[MaxUniformExpressions];
	static FString VolumeTextureSamplerNames[MaxUniformExpressions];
	static FString ExternalTextureNames[MaxUniformExpressions];
	static FString TextureCollectionNames[MaxUniformExpressions];
	static FString TextureCollectionPhysicalNames[MaxUniformExpressions];
	static FString TextureCollectionSamplerNames[MaxUniformExpressions];
	static FString TextureCollectionPageTableNames[MaxUniformExpressions];
	static FString MediaTextureSamplerNames[MaxUniformExpressions];
	static FString VirtualTexturePageTableNames0[MaxUniformExpressions];
	static FString VirtualTexturePageTableNames1[MaxUniformExpressions];
	static FString VirtualTexturePageTableIndirectionNames[MaxUniformExpressions];
	static FString VirtualTexturePhysicalNames[MaxUniformExpressions];
	static FString VirtualTexturePhysicalSamplerNames[MaxUniformExpressions];
	static FString SparseVolumeTexturePageTableNames[MaxUniformExpressions];
	static FString SparseVolumeTexturePhysicalANames[MaxUniformExpressions];
	static FString SparseVolumeTexturePhysicalBNames[MaxUniformExpressions];
	static FString SparseVolumeTexturePhysicalSamplerNames[MaxUniformExpressions];
	
	static FString MaterialCacheTagBuffer[MaxUniformExpressions];
	static FString MaterialCacheTagPageTableNames0[MaxUniformExpressions];
	static FString MaterialCacheTagPhysicalNames[MaxUniformExpressions];
	static FString MaterialCacheTagPhysicalSamplerNames[MaxUniformExpressions];
	
	static bool bInitializedTextureNames = false;
	if (!bInitializedTextureNames)
	{
		bInitializedTextureNames = true;
		for (int32 i = 0; i < MaxUniformExpressions; ++i)
		{
			Texture2DNames[i] = FString::Printf(TEXT("Texture2D_%d"), i);
			Texture2DSamplerNames[i] = FString::Printf(TEXT("Texture2D_%dSampler"), i);
			TextureCubeNames[i] = FString::Printf(TEXT("TextureCube_%d"), i);
			TextureCubeSamplerNames[i] = FString::Printf(TEXT("TextureCube_%dSampler"), i);
			Texture2DArrayNames[i] = FString::Printf(TEXT("Texture2DArray_%d"), i);
			Texture2DArraySamplerNames[i] = FString::Printf(TEXT("Texture2DArray_%dSampler"), i);
			TextureCubeArrayNames[i] = FString::Printf(TEXT("TextureCubeArray_%d"), i);
			TextureCubeArraySamplerNames[i] = FString::Printf(TEXT("TextureCubeArray_%dSampler"), i);
			VolumeTextureNames[i] = FString::Printf(TEXT("VolumeTexture_%d"), i);
			VolumeTextureSamplerNames[i] = FString::Printf(TEXT("VolumeTexture_%dSampler"), i);
			ExternalTextureNames[i] = FString::Printf(TEXT("ExternalTexture_%d"), i);
			TextureCollectionNames[i] = FString::Printf(TEXT("TextureCollection_%d"), i);
			TextureCollectionPhysicalNames[i] = FString::Printf(TEXT("TextureCollectionPhysical_%d"), i);
			TextureCollectionSamplerNames[i] = FString::Printf(TEXT("TextureCollectionSampler_%d"), i);
			TextureCollectionPageTableNames[i] = FString::Printf(TEXT("TextureCollectionPageTable_%d"), i);
			MediaTextureSamplerNames[i] = FString::Printf(TEXT("ExternalTexture_%dSampler"), i);
			VirtualTexturePageTableNames0[i] = FString::Printf(TEXT("VirtualTexturePageTable0_%d"), i);
			VirtualTexturePageTableNames1[i] = FString::Printf(TEXT("VirtualTexturePageTable1_%d"), i);
			VirtualTexturePageTableIndirectionNames[i] = FString::Printf(TEXT("VirtualTexturePageTableIndirection_%d"), i);
			VirtualTexturePhysicalNames[i] = FString::Printf(TEXT("VirtualTexturePhysical_%d"), i);
			VirtualTexturePhysicalSamplerNames[i] = FString::Printf(TEXT("VirtualTexturePhysical_%dSampler"), i);
			SparseVolumeTexturePageTableNames[i] = FString::Printf(TEXT("SparseVolumeTexturePageTable_%d"), i);
			SparseVolumeTexturePhysicalANames[i] = FString::Printf(TEXT("SparseVolumeTexturePhysicalA_%d"), i);
			SparseVolumeTexturePhysicalBNames[i] = FString::Printf(TEXT("SparseVolumeTexturePhysicalB_%d"), i);
			SparseVolumeTexturePhysicalSamplerNames[i] = FString::Printf(TEXT("SparseVolumeTexturePhysical_%dSampler"), i);
			
			MaterialCacheTagBuffer[i] = FString::Printf(TEXT("MaterialCacheTagBuffer_%d"), i);
			MaterialCacheTagPageTableNames0[i] = FString::Printf(TEXT("MaterialCacheTagPageTable0_%d"), i);
			MaterialCacheTagPhysicalNames[i] = FString::Printf(TEXT("MaterialCacheTagPhysical_%d"), i);
			MaterialCacheTagPhysicalSamplerNames[i] = FString::Printf(TEXT("MaterialCacheTagPhysical_%dSampler"), i);
		}
	}

	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		check(UniformTextureParameters[TypeIndex].Num() <= MaxUniformExpressions);
	}
	check(VTStacks.Num() <= MaxUniformExpressions);
	check(MaterialCacheTagStacks.Num() <= MaxUniformExpressions);

	auto AddMemberIfNeeded = [&Members, &MembersVS, &NextMemberOffsetVS, &NextMemberOffset, ShaderFrequencyMaskVS, &ExecuteAddToMembersVS]
		(EShaderFrequencyMask ShaderFrequencyMask)
	{
		if (!!(ShaderFrequencyMask & ShaderFrequencyMaskVS))
		{
			check(Members.Last().GetNumElements() == 0);
			ExecuteAddToMembersVS(Members.Last().GetNumElements(), NextMemberOffset - Members.Last().GetOffset());
		}

	};

	auto AddTextureMember = [this, &Members, &MembersVS, &NextMemberOffsetVS, ShaderFrequencyMaskVS, &AddMemberIfNeeded]
		(int32 i, EMaterialTextureParameterType MaterialTextureParameterType)
	{
		const FMaterialTextureParameterInfo& MaterialTextureParameterInfo = UniformTextureParameters[(uint32)MaterialTextureParameterType][i];
		AddMemberIfNeeded(MaterialTextureParameterInfo.ShaderFrequencyMask);
	};

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::Standard2D].Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*Texture2DNames[i],TEXT("Texture2D"),__LINE__,NextMemberOffset,UBMT_TEXTURE,EShaderPrecisionModifier::Float,1,1,0,NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Standard2D);
		new(Members) FShaderParametersMetadata::FMember(*Texture2DSamplerNames[i],TEXT("SamplerState"),__LINE__,NextMemberOffset,UBMT_SAMPLER,EShaderPrecisionModifier::Float,1,1,0,NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Standard2D);
	}

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::Cube].Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*TextureCubeNames[i],TEXT("TextureCube"),__LINE__,NextMemberOffset,UBMT_TEXTURE,EShaderPrecisionModifier::Float,1,1,0,NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Cube);
		new(Members) FShaderParametersMetadata::FMember(*TextureCubeSamplerNames[i],TEXT("SamplerState"),__LINE__,NextMemberOffset,UBMT_SAMPLER,EShaderPrecisionModifier::Float,1,1,0,NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Cube);
	}

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::Array2D].Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*Texture2DArrayNames[i], TEXT("Texture2DArray"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Array2D);
		new(Members) FShaderParametersMetadata::FMember(*Texture2DArraySamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Array2D);
	}

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::ArrayCube].Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*TextureCubeArrayNames[i], TEXT("TextureCubeArray"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::ArrayCube);
		new(Members) FShaderParametersMetadata::FMember(*TextureCubeArraySamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::ArrayCube);
	}

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::Volume].Num(); ++i)
	{
		const FMaterialTextureParameterInfo& Parameter = UniformTextureParameters[(uint32)EMaterialTextureParameterType::Volume][i];
		
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*VolumeTextureNames[i], TEXT("Texture3D"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Volume);
		new(Members) FShaderParametersMetadata::FMember(*VolumeTextureSamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Volume);
	}

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::SparseVolume].Num(); ++i)
	{
		const FMaterialTextureParameterInfo& Parameter = UniformTextureParameters[(uint32)EMaterialTextureParameterType::SparseVolume][i];

		// Page Table
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*SparseVolumeTexturePageTableNames[i], TEXT("Texture3D<uint>"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::SparseVolume);
		
		// Physical A
		check((NextMemberOffset% SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*SparseVolumeTexturePhysicalANames[i], TEXT("Texture3D"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::SparseVolume);

		// Physical B
		check((NextMemberOffset% SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*SparseVolumeTexturePhysicalBNames[i], TEXT("Texture3D"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::SparseVolume);

		// Sampler
		check((NextMemberOffset% SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*SparseVolumeTexturePhysicalSamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::SparseVolume);
	}

	for (int32 i = 0; i < UniformExternalTextureParameters.Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*ExternalTextureNames[i], TEXT("TextureExternal"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddMemberIfNeeded(UniformExternalTextureParameters[i].ShaderFrequencyMask);
		
		new(Members) FShaderParametersMetadata::FMember(*MediaTextureSamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddMemberIfNeeded(UniformExternalTextureParameters[i].ShaderFrequencyMask);
	}

	for (int32 i = 0; i < UniformTextureCollectionParameters.Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);

		// Resource collections are RHI constructs, texture collections are software virtual tables
		if (!UniformTextureCollectionParameters[i].bIsVirtualCollection)
		{
			new(Members) FShaderParametersMetadata::FMember(*TextureCollectionNames[i], TEXT("FResourceCollection"), __LINE__, NextMemberOffset, UBMT_RESOURCE_COLLECTION, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(UniformTextureCollectionParameters[i].ShaderFrequencyMask);
		}
		else
		{
			new(Members) FShaderParametersMetadata::FMember(*TextureCollectionNames[i], TEXT("FIndirectVirtualTexture"), __LINE__, NextMemberOffset, UBMT_SRV, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(UniformTextureCollectionParameters[i].ShaderFrequencyMask);
			
			new(Members) FShaderParametersMetadata::FMember(*TextureCollectionPageTableNames[i], TEXT("Texture2D<uint4>"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(UniformTextureCollectionParameters[i].ShaderFrequencyMask);
		
			new(Members) FShaderParametersMetadata::FMember(*TextureCollectionPhysicalNames[i], TEXT("Texture2D"), __LINE__, NextMemberOffset, UBMT_SRV, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(UniformTextureCollectionParameters[i].ShaderFrequencyMask);
	
			new(Members) FShaderParametersMetadata::FMember(*TextureCollectionSamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(UniformTextureCollectionParameters[i].ShaderFrequencyMask);
		}
	}

	for (int32 i = 0; i < VTStacks.Num(); ++i)
	{
		const FMaterialVirtualTextureStack& Stack = VTStacks[i];
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		new(Members) FShaderParametersMetadata::FMember(*VirtualTexturePageTableNames0[i], TEXT("Texture2D<uint4>"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddMemberIfNeeded(Stack.ShaderFrequencyMask);
		if (Stack.GetNumLayers() > 4u)
		{
			new(Members) FShaderParametersMetadata::FMember(*VirtualTexturePageTableNames1[i], TEXT("Texture2D<uint4>"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(Stack.ShaderFrequencyMask);
		}
		new(Members) FShaderParametersMetadata::FMember(*VirtualTexturePageTableIndirectionNames[i], TEXT("Texture2D<uint>"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddMemberIfNeeded(Stack.ShaderFrequencyMask);
	}

	for (int32 TagIndex = 0; TagIndex < MaterialCacheTagStacks.Num(); ++TagIndex)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);

		new(Members) FShaderParametersMetadata::FMember(*MaterialCacheTagBuffer[TagIndex], TEXT("StructuredBuffer<FMaterialCacheTagEntry>"), __LINE__, NextMemberOffset, UBMT_SRV, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddMemberIfNeeded(MaterialCacheTagStacks[TagIndex].ShaderFrequencyMask);
		
		new(Members) FShaderParametersMetadata::FMember(*MaterialCacheTagPageTableNames0[TagIndex], TEXT("Texture2D<uint4>"), __LINE__, NextMemberOffset, UBMT_TEXTURE, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddMemberIfNeeded(MaterialCacheTagStacks[TagIndex].ShaderFrequencyMask);

		for (int32 LayerIndex = 0; LayerIndex < MaterialCacheMaxRuntimeLayers; LayerIndex++)
		{
			new(Members) FShaderParametersMetadata::FMember(*MaterialCacheTagPhysicalNames[TagIndex * MaterialCacheMaxRuntimeLayers + LayerIndex], TEXT("Texture2D"), __LINE__, NextMemberOffset, UBMT_SRV, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
			NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
			AddMemberIfNeeded(MaterialCacheTagStacks[TagIndex].ShaderFrequencyMask);
		}
	}

	for (int32 i = 0; i < UniformTextureParameters[(uint32)EMaterialTextureParameterType::Virtual].Num(); ++i)
	{
		check((NextMemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
		check((NextMemberOffsetVS % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);

		// VT physical textures are bound as SRV, allows aliasing the same underlying texture with both sRGB/non-sRGB views
		new(Members) FShaderParametersMetadata::FMember(*VirtualTexturePhysicalNames[i], TEXT("Texture2D"), __LINE__, NextMemberOffset, UBMT_SRV, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Virtual);
		new(Members) FShaderParametersMetadata::FMember(*VirtualTexturePhysicalSamplerNames[i], TEXT("SamplerState"), __LINE__, NextMemberOffset, UBMT_SAMPLER, EShaderPrecisionModifier::Float, 1, 1, 0, NULL);
		NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
		AddTextureMember(i, EMaterialTextureParameterType::Virtual);
	}

	// Check if we need any texture at the VS level, and include the default samplers if so
	bool bRequiresWorldGroupSettingsSamplers = false;
	for (const FShaderParametersMetadata::FMember& MemberVS : MembersVS)
	{
		if (!IsNumericMember(MemberVS))
		{
			bRequiresWorldGroupSettingsSamplers = true;
			break;
		}
	}

	EShaderFrequencyMask ShaderFrequencyMaskWorldGroupSettings = bRequiresWorldGroupSettingsSamplers ? ShaderFrequencyMaskVS : 0;

	new(Members) FShaderParametersMetadata::FMember(TEXT("Wrap_WorldGroupSettings"),TEXT("SamplerState"), __LINE__, NextMemberOffset,UBMT_SAMPLER,EShaderPrecisionModifier::Float,1,1,0,NULL);
	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
	AddMemberIfNeeded(ShaderFrequencyMaskWorldGroupSettings);

	new(Members) FShaderParametersMetadata::FMember(TEXT("Clamp_WorldGroupSettings"),TEXT("SamplerState"), __LINE__, NextMemberOffset,UBMT_SAMPLER,EShaderPrecisionModifier::Float,1,1,0,NULL);
	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
	AddMemberIfNeeded(ShaderFrequencyMaskWorldGroupSettings);

	const uint32 StructSize = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	const uint32 StructSizeVS = Align(NextMemberOffsetVS, SHADER_PARAMETER_STRUCT_ALIGNMENT);

	UniformBufferLayoutInitializer = FRHIUniformBufferLayoutInitializer(TEXT("Material"));

	FShaderParametersMetadata::EUsageFlags UsageFlags = FShaderParametersMetadata::EUsageFlags::None;
	EnumAddFlags(UsageFlags, FShaderParametersMetadata::EUsageFlags::BindlessAccessible);

	FShaderParametersMetadata* UniformBufferStruct = new FShaderParametersMetadata(
		FShaderParametersMetadata::EUseCase::DataDrivenUniformBuffer,
		EUniformBufferBindingFlags::Shader,
		TEXT("Material"),
		TEXT("MaterialUniforms"),
		TEXT("Material"),
		nullptr,
		UE_LOG_SOURCE_FILE(__FILE__),
		__LINE__,
		StructSize,
		Members,
		false,
		&UniformBufferLayoutInitializer,
		UsageFlags
	);

	TArray<FShaderParametersMetadata*> AllUniformBufferStructs;
	AllUniformBufferStructs.Add(UniformBufferStruct);

	if (StructSizeVS > 0)
	{
		check(CompactUniformsVS != nullptr)
		check(!MembersVS.IsEmpty());
		check(!CopySegments.IsEmpty());

		FShaderParametersMetadata* ShaderParametersMetadataVS = CreateCompactShaderParametersMetadata(CompactUniformsVS, TEXT("MaterialVS"), TEXT("MaterialVS"), TEXT("MaterialUniformsVS"), TEXT("MaterialVS"), MembersVS, StructSizeVS, CopySegments);

		if (ShaderParametersMetadataVS)
		{
			check(CompactUniformsVS->UniformBufferLayoutInitializer.ConstantBufferSize <= UniformBufferLayoutInitializer.ConstantBufferSize);
			AllUniformBufferStructs.Add(ShaderParametersMetadataVS);
		}
	}
	else
	{
		check(MembersVS.IsEmpty());
		check(CopySegments.IsEmpty());
		CompactUniformsVSOptional.Empty();
	}

	return AllUniformBufferStructs;
}
#endif

FUniformExpressionSet::FVTPackedStackAndLayerIndex FUniformExpressionSet::GetVTStackAndLayerIndex(int32 UniformExpressionIndex) const
{
	for (int32 VTStackIndex = 0; VTStackIndex < VTStacks.Num(); ++VTStackIndex)
	{
		const FMaterialVirtualTextureStack& VTStack = VTStacks[VTStackIndex];
		const int32 LayerIndex = VTStack.FindLayer(UniformExpressionIndex);
		if (LayerIndex >= 0)
		{
			return FVTPackedStackAndLayerIndex(VTStackIndex, LayerIndex);
		}
	}

	checkNoEntry();
	return FVTPackedStackAndLayerIndex(0xffff, 0xffff);
}

void FMaterialUniformExpression::GetNumberValue(const struct FMaterialRenderContext& Context, FLinearColor& OutValue) const
{
	using namespace UE::Shader;

	FPreshaderData PreshaderData;
	WriteNumberOpcodes(PreshaderData);
	FPreshaderStack Stack;
	const FPreshaderValue Value = PreshaderData.Evaluate(nullptr, Context, Stack);
	OutValue = Value.AsShaderValue().AsLinearColor();
}

int32 FUniformExpressionSet::FindOrAddTextureParameter(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& Info)
{
	for (int32 i = 0; i < UniformTextureParameters[(int32)Type].Num(); ++i)
	{
		if (UniformTextureParameters[(int32)Type][i] == Info)
		{
			return i;
		}
	}

	return UniformTextureParameters[(int32)Type].Add(Info);
}

int32 FUniformExpressionSet::FindOrAddTextureCollectionParameter(const FMaterialTextureCollectionParameterInfo& Info)
{
	for (int32 Index = 0; Index < UniformTextureCollectionParameters.Num(); Index++)
	{
		if (UniformTextureCollectionParameters[Index] == Info)
		{
			return Index;
		}
	}

	return UniformTextureCollectionParameters.Add(Info);
}

int32 FUniformExpressionSet::FindOrAddExternalTextureParameter(const FMaterialExternalTextureParameterInfo& Info)
{
	for (int32 i = 0; i < UniformExternalTextureParameters.Num(); ++i)
	{
		if (UniformExternalTextureParameters[i] == Info)
		{
			return i;
		}
	}

	return UniformExternalTextureParameters.Add(Info);
}

int32 FUniformExpressionSet::FindOrAddNumericParameter(EMaterialParameterType Type, const FMaterialParameterInfo& ParameterInfo, uint32 DefaultValueOffset)
{
	for (int32 i = 0; i < UniformNumericParameters.Num(); ++i)
	{
		const FMaterialNumericParameterInfo& Parameter = UniformNumericParameters[i];
		if (Parameter.ParameterType == Type && Parameter.DefaultValueOffset == DefaultValueOffset && Parameter.ParameterInfo == ParameterInfo)
		{
			return i;
		}
	}

	const int32 Index = UniformNumericParameters.Num();
	FMaterialNumericParameterInfo& Parameter = UniformNumericParameters.AddDefaulted_GetRef();
	Parameter.ParameterType = Type;
	Parameter.ParameterInfo = ParameterInfo;
	Parameter.DefaultValueOffset = DefaultValueOffset;
	return Index;
}

uint32 FUniformExpressionSet::AddDefaultParameterValue(const UE::Shader::FValue& Value)
{
	const uint32 Offset = DefaultValues.Num();
	UE::Shader::FMemoryImageValue DefaultValueMemory = Value.AsMemoryImage();
	DefaultValues.Append(DefaultValueMemory.Bytes, DefaultValueMemory.Size);
	return Offset;
}

void FUniformExpressionSet::AddNumericParameterEvaluation(uint32 ParameterIndex, uint32 BufferOffset, uint32 ComponentCount)
{
	check(ParameterIndex <= UINT16_MAX);
	check(BufferOffset <= UINT16_MAX);
	check(ComponentCount <= 4);
	UniformParameterEvaluations.Add({
		.ParameterIndex = (uint16)ParameterIndex,
		.BufferOffset = (uint16)BufferOffset,
		.ComponentCount = (uint16)ComponentCount
	});
}

void FUniformExpressionSet::FixupNumericParameterEvaluations(TConstArrayView<uint16> Remap, TConstArrayView<uint8> UniformComponentCounts)
{
	for (FMaterialUniformParameterEvaluation& Evaluation : UniformParameterEvaluations)
	{
		// Override component count if provided
		uint16 ComponentCount = UniformComponentCounts.Num() ? UniformComponentCounts[Evaluation.BufferOffset] : 0;
		if (ComponentCount)
		{
			Evaluation.ComponentCount = ComponentCount;
		}

		// Then remap BufferOffset
		Evaluation.BufferOffset = Remap[Evaluation.BufferOffset];
	}
}

uint32 FUniformExpressionSet::AllocateFromUniformBuffer(uint32 NumOfFloat4s)
{
	uint32 Temp = UniformPreshaderBufferSize;
	UniformPreshaderBufferSize += NumOfFloat4s;
	return Temp;
}

int32 FUniformExpressionSet::AddVTStack(int32 InPreallocatedStackTextureIndex)
{
	return VTStacks.Add(FMaterialVirtualTextureStack(InPreallocatedStackTextureIndex));
}

int32 FUniformExpressionSet::AddVTLayer(int32 StackIndex, int32 TextureIndex)
{
	const int32 VTLayerIndex = VTStacks[StackIndex].AddLayer();
	VTStacks[StackIndex].SetLayer(VTLayerIndex, TextureIndex);
	return VTLayerIndex;
}

void FUniformExpressionSet::SetVTLayer(int32 StackIndex, int32 VTLayerIndex, int32 TextureIndex)
{
	VTStacks[StackIndex].SetLayer(VTLayerIndex, TextureIndex);
}

void FUniformExpressionSet::WriteUniformPreshaderEntry(uint32 BufferOffset, UE::Shader::EValueType EntryType, const TFunction<void(UE::Shader::FPreshaderData&)>& EntryGenerator)
{
	FMaterialUniformPreshaderHeader& PreshaderHeader = UniformPreshaders.AddDefaulted_GetRef();

	PreshaderHeader.OpcodeOffset = UniformPreshaderData.Num();
	{
		EntryGenerator(UniformPreshaderData);
	}
	PreshaderHeader.OpcodeSize = UniformPreshaderData.Num() - PreshaderHeader.OpcodeOffset;

	PreshaderHeader.BufferOffset = BufferOffset;
	PreshaderHeader.Type = EntryType;
}

void FUniformExpressionSet::GetGameThreadTextureValue(EMaterialTextureParameterType Type, int32 Index, const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTexture*& OutValue, bool bAllowOverride) const
{
	check(IsInGameThread() || IsInParallelGameThread());
	OutValue = NULL;
	const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(Type, Index);
#if WITH_EDITOR
	if (bAllowOverride)
	{
		UTexture* OverrideTexture = Material.TransientOverrides.GetTextureOverride_GameThread(Type, Parameter);
		if (OverrideTexture)
		{
			OutValue = OverrideTexture;
			return;
		}
	}
#endif // WITH_EDITOR
	Parameter.GetGameThreadTextureValue(MaterialInterface, Material, OutValue);
}

void FUniformExpressionSet::GetTextureValue(EMaterialTextureParameterType Type, int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const UTexture*& OutValue) const
{
	check(IsInParallelRenderingThread());
	const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(Type, Index);
#if WITH_EDITOR
	{
		UTexture* OverrideTexture = Material.TransientOverrides.GetTextureOverride_RenderThread(Type, Parameter);
		if (OverrideTexture)
		{
			OutValue = OverrideTexture;
			return;
		}
	}
#endif // WITH_EDITOR
	Context.GetTextureParameterValue(Parameter.ParameterInfo, Parameter.TextureIndex , OutValue);
}

void FUniformExpressionSet::GetTextureValue(int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const URuntimeVirtualTexture*& OutValue) const
{
	check(IsInParallelRenderingThread());
	const int32 VirtualTexturesNum = GetNumTextures(EMaterialTextureParameterType::Virtual);
	if (ensure(Index < VirtualTexturesNum))
	{
		const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Virtual, Index);
		Context.GetTextureParameterValue(Parameter.ParameterInfo, Parameter.TextureIndex, OutValue);
	}
	else
	{
		OutValue = nullptr;
	}
}

void FUniformExpressionSet::GetTextureValue(int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const USparseVolumeTexture*& OutValue) const
{
	check(IsInParallelRenderingThread());
	const int32 VolumeTexturesNum = GetNumTextures(EMaterialTextureParameterType::SparseVolume);
	if (ensure(Index < VolumeTexturesNum))
	{
		const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::SparseVolume, Index);
		Context.GetTextureParameterValue(Parameter.ParameterInfo, Parameter.TextureIndex, OutValue);
	}
	else
	{
		OutValue = nullptr;
	}
}

void FUniformExpressionSet::GetTextureCollectionValue(int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const UTextureCollection*& OutValue) const
{
	check(IsInParallelRenderingThread());
	if (ensure(Index < GetNumTextureCollections()))
	{
		const FMaterialTextureCollectionParameterInfo& Parameter = GetTextureCollectionParameter(Index);
		Context.GetTextureCollectionParameterValue(Parameter.ParameterInfo, Parameter.TextureCollectionIndex, OutValue);
	}
	else
	{
		OutValue = nullptr;
	}
}

void FUniformExpressionSet::GetGameThreadTextureCollectionValue(int32 Index, const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTextureCollection*& OutValue) const
{
	check(IsInGameThread() || IsInParallelGameThread());
	OutValue = nullptr;
	const FMaterialTextureCollectionParameterInfo& Parameter = GetTextureCollectionParameter(Index);
	Parameter.GetGameThreadTextureCollectionValue(MaterialInterface, Material, OutValue);
}

void FUniformExpressionSet::FillUniformBuffer(const FMaterialRenderContext& MaterialRenderContext, const FUniformExpressionCache& UniformExpressionCache, const FRHIUniformBufferLayout* UniformBufferLayout, uint8* TempBuffer, int TempBufferSize) const
{
	FillUniformBuffer(MaterialRenderContext, UniformExpressionCache.AllocatedVTs, UniformBufferLayout, TempBuffer, TempBufferSize);
}

void FUniformExpressionSet::CompactUniformBuffer(uint8* DestBuffer, int32 DestBufferSize, const uint8* SrcBuffer, int32 SrcBufferSize, const FCompactUniformExpressionSet& CompactUniforms)
{
	FMemoryView         SrcView = MakeMemoryView(SrcBuffer, SrcBufferSize);
	FMutableMemoryView  DestView = MakeMemoryView(DestBuffer, DestBufferSize);

	for (const FCompactionMetaDataSegment& CompactionSegment : CompactUniforms.CompactionSegments)
	{
		FMemoryView        SrcSegment = SrcView.Mid(CompactionSegment.SrcOffset,  CompactionSegment.SizeToCopy);
		FMutableMemoryView DestSegment = DestView.Mid(CompactionSegment.DestOffset, CompactionSegment.SizeToCopy);

		// Make sure that we won't stomp memory if compaction metadata is corrupted. TMemoryView::Mid adjusts the size based on the parent's capacity
		if ( !ensureAlways(SrcSegment.GetSize() == CompactionSegment.SizeToCopy) ||
			 !ensureAlways(DestSegment.GetSize() == CompactionSegment.SizeToCopy))
		{
			continue;
		}

		DestSegment.CopyFrom(SrcSegment);
	}
}

static void CopyValueToUniformBuffer(const UE::Shader::FValue& Value, float* PreshaderBuffer, uint32 BufferOffset)
{
	using namespace UE::Shader;
	FValueTypeDescription UniformTypeDesc = GetValueTypeDescription(Value.GetType());

	if ( UniformTypeDesc.ComponentType == EValueComponentType::Float)
	{
		FFloatValue FloatValue = Value.AsFloat();
		float* DestAddress = PreshaderBuffer + BufferOffset;
		for (int32 i = 0; i < Value.GetNumComponents(); ++i)
		{
			*DestAddress++ = FloatValue[i];
		}
	}
	else if (UniformTypeDesc.ComponentType == EValueComponentType::Int)
	{
		FIntValue IntValue = Value.AsInt();
		int32* DestAddress = (int32*)PreshaderBuffer + BufferOffset;
		for (int32 i = 0; i < Value.GetNumComponents(); ++i)
		{
			*DestAddress++ = IntValue[i];
		}
	}
	else if (UniformTypeDesc.ComponentType == EValueComponentType::Bool)
	{
		FBoolValue BoolValue = Value.AsBool();
		uint32 Mask = 0u;
		for (int32 i = 0; i < Value.GetNumComponents(); ++i)
		{
			if (BoolValue[i])
			{
				Mask |= (1u << i);
			}
		}

		uint32 BufferOffsetAdjusted = BufferOffset / 32u;
		uint32 BufferBitOffset = BufferOffset % 32u;
		check(BufferBitOffset + Value.GetNumComponents ()<= 32u);

		uint32* DestAddress = (uint32*)PreshaderBuffer + BufferOffsetAdjusted;
		if (BufferBitOffset == 0u)
		{
			// First update to a group of bits needs to initialize memory
			*DestAddress = Mask;
		}
		else
		{
			// Combine with any previous bits
			*DestAddress |= (Mask << BufferBitOffset);
		}
	}
	else if (UniformTypeDesc.ComponentType == EValueComponentType::Double)
	{
		FDoubleValue DoubleValue = Value.AsDouble();

		float ValueHigh[4] = {};
		float ValueLow[4] = {};
		for (int32 i = 0; i < Value.GetNumComponents(); ++i)
		{
			const FDFScalar ScalarValue(DoubleValue[i]);
			ValueHigh[i] = ScalarValue.High;
			ValueLow[i] = ScalarValue.Low;
		}

		float* DestAddress = PreshaderBuffer + BufferOffset;
		for (int32 i = 0; i < Value.GetNumComponents(); ++i)
		{
			*DestAddress++ = ValueHigh[i];
		}
		for (int32 i = 0; i < Value.GetNumComponents(); ++i)
		{
			*DestAddress++ = ValueLow[i];
		}
	}
	else
	{
		ensure(false);
	}
}

void FUniformExpressionSet::FillUniformBuffer(const FMaterialRenderContext& MaterialRenderContext, TConstArrayView<IAllocatedVirtualTexture*> AllocatedVTs, const FRHIUniformBufferLayout* UniformBufferLayout, uint8* TempBuffer, int TempBufferSize) const
{
	using namespace UE::Shader;
	check(IsInParallelRenderingThread());

	if (UniformBufferLayout->ConstantBufferSize > 0)
	{
		// stat disabled by default due to low-value/high-frequency
		//QUICK_SCOPE_CYCLE_COUNTER(STAT_FUniformExpressionSet_FillUniformBuffer);

		void* BufferCursor = TempBuffer;
		check(BufferCursor <= TempBuffer + TempBufferSize);
		
		if (!UniformTextureCollectionParameters.IsEmpty())
		{
			// Bindless collections do not make use of uniforms
			for (const FMaterialTextureCollectionParameterInfo& ParameterInfo : UniformTextureCollectionParameters)
			{
				if (!ParameterInfo.bIsVirtualCollection)
				{
					continue;
				}
				
				const UTextureCollection* TextureCollection = nullptr;
				ParameterInfo.GetTextureCollection(MaterialRenderContext, TextureCollection);
		
				auto* VTPackedPageTableUniform = static_cast<UE::HLSL::FIndirectVirtualTextureUniform*>(BufferCursor);
				BufferCursor = VTPackedPageTableUniform + 1;

				if (FVirtualTextureCollectionResource* Resource = static_cast<FVirtualTextureCollectionResource*>(TextureCollection->GetResource()))
				{
					*VTPackedPageTableUniform = Resource->GetVirtualPackedUniform();
				}
				else
				{
					*VTPackedPageTableUniform = UE::HLSL::FIndirectVirtualTextureUniform{};
				}
			}
		}
		
		// Dump virtual texture per page table uniform data
		check(AllocatedVTs.Num() == VTStacks.Num());
		for ( int32 VTStackIndex = 0; VTStackIndex < VTStacks.Num(); ++VTStackIndex)
		{
			const IAllocatedVirtualTexture* AllocatedVT = AllocatedVTs[VTStackIndex];
			FUintVector4* VTPackedPageTableUniform = (FUintVector4*)BufferCursor;
			if (AllocatedVT)
			{
				AllocatedVT->GetPackedPageTableUniform(VTPackedPageTableUniform);
			}
			else
			{
				VTPackedPageTableUniform[0] = FUintVector4(ForceInitToZero);
				VTPackedPageTableUniform[1] = FUintVector4(ForceInitToZero);
			}
			BufferCursor = VTPackedPageTableUniform + 2;
		}
		
		for (int32 TagIndex = 0; TagIndex < MaterialCacheTagStacks.Num(); ++TagIndex)
		{
			const FMaterialCacheTagStack& Tag = MaterialCacheTagStacks[TagIndex];

			// Get the uniform data associated with this tag
			IMaterialCacheTagProvider*   TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
			FMaterialCacheTagUniformData Data        = TagProvider->GetUniformData(Tag.TagGuid);
			
			FUintVector4* VTPackedPageTableUniform = (FUintVector4*)BufferCursor;
			*VTPackedPageTableUniform = Data.PackedTableUniform;
			BufferCursor = VTPackedPageTableUniform + 1;
		}
		
		// Dump virtual texture per physical texture uniform data
		for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Virtual); ++ExpressionIndex)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Virtual, ExpressionIndex);

			FUintVector4* VTPackedUniform = (FUintVector4*)BufferCursor;
			BufferCursor = VTPackedUniform + 1;

			bool bFoundTexture = false;

			// Check for streaming virtual texture
			if (!bFoundTexture)
			{
				const UTexture* Texture = nullptr;
				GetTextureValue(EMaterialTextureParameterType::Virtual, ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Texture);
				if (Texture != nullptr)
				{
					const FVTPackedStackAndLayerIndex StackAndLayerIndex = GetVTStackAndLayerIndex(ExpressionIndex);
					const IAllocatedVirtualTexture* AllocatedVT = AllocatedVTs[StackAndLayerIndex.StackIndex];
					if (AllocatedVT)
					{
						AllocatedVT->GetPackedUniform(VTPackedUniform, StackAndLayerIndex.LayerIndex);
					}
					bFoundTexture = true;
				}
			}
			
			// Now check for runtime virtual texture
			if (!bFoundTexture)
			{
				const URuntimeVirtualTexture* Texture = nullptr;
				GetTextureValue(ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Texture);
				if (Texture != nullptr)
				{
					IAllocatedVirtualTexture const* AllocatedVT = Texture->GetAllocatedVirtualTexture();
					if (AllocatedVT)
					{
						AllocatedVT->GetPackedUniform(VTPackedUniform, Parameter.VirtualTextureLayerIndex);
					}
					else
					{
						*VTPackedUniform = FUintVector4(0, 0, 0, 0);
					}
				}
			}
		}

		// Dump per SparseVolumeTexture uniform data
		for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::SparseVolume); ++ExpressionIndex)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::SparseVolume, ExpressionIndex);

			FUintVector4* SVTPackedUniform = (FUintVector4*)BufferCursor;
			BufferCursor = SVTPackedUniform + 2;

			const USparseVolumeTexture* SVTexture = nullptr;
			GetTextureValue(ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, SVTexture);
			const UE::SVT::FTextureRenderResources* RenderResources = SVTexture != nullptr ? SVTexture->GetTextureRenderResources() : nullptr;
			if (RenderResources)
			{
				RenderResources->GetPackedUniforms(SVTPackedUniform[0], SVTPackedUniform[1]);
			}
			else
			{
				SVTPackedUniform[0] = FUintVector4();
				SVTPackedUniform[1] = FUintVector4();
			}
		}

		// Perform direct parameter evaluation
		for (const FMaterialUniformParameterEvaluation& Evaluation : UniformParameterEvaluations) {
			// Get the numeric parameter information
			const FMaterialNumericParameterInfo& Parameter = GetNumericParameter(Evaluation.ParameterIndex);
		
			// Resolve the parameter value from the material-chain
			// Note: it is really not ideal to perform this evaluation here, as this is somewhat static information.
			//		 Ideally it should be the translator that performs this parameter resolution at translation time
			//		 and exports the necessary data to *efficiently* access the actual parameter here.
			bool bFoundParameter = false;
			UE::Shader::FValue ParameterShaderValue;

			// First allow proxy the chance to override parameter
			if (MaterialRenderContext.MaterialRenderProxy 
				&& MaterialRenderContext.MaterialRenderProxy->GetParameterShaderValue(Parameter.ParameterType, Parameter.ParameterInfo, ParameterShaderValue, MaterialRenderContext))
			{
				bFoundParameter = true;
			}

			// Then allow the editor to override the parameter value
			#if WITH_EDITOR
			if (!bFoundParameter
				&& MaterialRenderContext.Material.TransientOverrides.GetNumericOverride(Parameter.ParameterType, Parameter.ParameterInfo, ParameterShaderValue))
			{
				bFoundParameter = true;
			}
			#endif // WITH_EDITOR

			if (!bFoundParameter) {
				ParameterShaderValue = GetDefaultParameterValue(Parameter.ParameterType, Parameter.DefaultValueOffset);
			}

			// Copy the value to the target position, after modifying the type and value to the desired component count.
			ParameterShaderValue.Type.ValueType = MakeValueType(ParameterShaderValue.Type.ValueType, Evaluation.ComponentCount);
			ParameterShaderValue.Component.SetNum(Evaluation.ComponentCount);

			CopyValueToUniformBuffer(ParameterShaderValue, (float*)BufferCursor, Evaluation.BufferOffset);
		}

		// Dump preshader results into buffer.
		float* const PreshaderBuffer = (float*)BufferCursor;
		if (UniformPreshaderData.bPreshader2)
		{
			// Preshader2 executes a single bytecode stream to write all outputs to the buffer.  UniformPreshaderBufferSize is a count of float4.
			FPreshaderDataContext Preshader2Context(UniformPreshaderData, TArrayView<float>(PreshaderBuffer, (UniformPreshaderBufferSize + UniformPreshaderData.Preshader2TemporarySize)*4));
			UE::Preshader2::Evaluate(Preshader2Context, &MaterialRenderContext);
		}
		else
		{
			FPreshaderStack PreshaderStack;
			FPreshaderDataContext PreshaderBaseContext(UniformPreshaderData);
			for (const FMaterialUniformPreshaderHeader& Preshader : UniformPreshaders)
			{
				FPreshaderDataContext PreshaderContext(PreshaderBaseContext, Preshader.OpcodeOffset, Preshader.OpcodeSize);
				const FPreshaderValue Result = EvaluatePreshader(this, MaterialRenderContext, PreshaderStack, PreshaderContext);

				// Fast path for non-structure float1 to float4 fields, which represents the vast majority of cases
				EValueType FieldType = Preshader.Type;
				EValueType ResultType = Result.Type.ValueType;

				// This min-max logic assumes that Float1 through Float4 are sequential in the EValueType enum, so we can do just two comparisons
				// to detect if both Result and the destination field are float types.
				static_assert((int32)EValueType::Float2 == (int32)EValueType::Float1 + 1);
				static_assert((int32)EValueType::Float3 == (int32)EValueType::Float2 + 1);
				static_assert((int32)EValueType::Float4 == (int32)EValueType::Float3 + 1);

				int32 MinValueType = FMath::Min((int32)FieldType, (int32)ResultType);
				int32 MaxValueType = FMath::Max((int32)FieldType, (int32)ResultType);

				if (MinValueType >= (int32)EValueType::Float1 && MaxValueType <= (int32)EValueType::Float4)
				{
					float* DestAddress = PreshaderBuffer + Preshader.BufferOffset;

					// Copy components that exist in both result and destination buffer
					int32 NumCopyComponents = MinValueType - ((int32)EValueType::Float1 - 1);
					for (int32 i = 0; i < NumCopyComponents; ++i)
					{
						*DestAddress++ = Result.Component[i].Float;
					}

					// Zero out any additional components that exist in the destination buffer
					int32 NumDestComponents = (int32)FieldType - ((int32)EValueType::Float1 - 1);
					for (int32 i = NumCopyComponents; i < NumDestComponents; ++i)
					{
						*DestAddress++ = 0.0f;
					}
				}
				else
				{
					const FValueTypeDescription UniformTypeDesc = GetValueTypeDescription(Preshader.Type);
					const int32 NumFieldComponents = UniformTypeDesc.NumComponents;

					const int32 NumResultComponents = FMath::Min<int32>(NumFieldComponents, Result.Component.Num());
					check(NumResultComponents > 0);

					// The type generated by the preshader might not match the expected type
					FValue FieldValue(Result.Type.GetComponentType(0), NumResultComponents);
					for (int32 i = 0; i < NumResultComponents; ++i)
					{
						FieldValue.Component[i] = Result.Component[i];
					}

					CopyValueToUniformBuffer(FieldValue, PreshaderBuffer, Preshader.BufferOffset);
				}
			}
		}

		// Offsets the cursor to next first resource.
		BufferCursor = PreshaderBuffer + UniformPreshaderBufferSize * 4;
		check(BufferCursor <= TempBuffer + TempBufferSize);

#if DO_CHECK
		{
			uint32 NumPageTableTextures = 0u;
			uint32 NumPageTableIndirectionTextures = 0u;
			for (int i = 0; i < VTStacks.Num(); ++i)
			{
				NumPageTableTextures += VTStacks[i].GetNumLayers() > 4u ? 2: 1;
				NumPageTableIndirectionTextures++;
			}
	
			uint32 BindlessCollectionCount = 0;
			uint32 VirtualCollectionCount  = 0;
			CountTextureCollections(BindlessCollectionCount, VirtualCollectionCount);
		
			check(UniformBufferLayout->Resources.Num() == 
				UniformTextureParameters[(uint32)EMaterialTextureParameterType::Standard2D].Num() * 2
				+ UniformTextureParameters[(uint32)EMaterialTextureParameterType::Cube].Num() * 2
				+ UniformTextureParameters[(uint32)EMaterialTextureParameterType::Array2D].Num() * 2
				+ UniformTextureParameters[(uint32)EMaterialTextureParameterType::ArrayCube].Num() * 2
				+ UniformTextureParameters[(uint32)EMaterialTextureParameterType::Volume].Num() * 2
				+ UniformTextureParameters[(uint32)EMaterialTextureParameterType::SparseVolume].Num() * 4
				+ UniformExternalTextureParameters.Num() * 2
				+ BindlessCollectionCount * 1
				+ VirtualCollectionCount * 4
				+ UniformTextureParameters[(uint32)EMaterialTextureParameterType::Virtual].Num() * 2
				+ MaterialCacheTagStacks.Num() * (MaterialCacheMaxRuntimeLayers + 2)
				+ NumPageTableTextures
				+ NumPageTableIndirectionTextures
				+ 2);
		}
#endif // DO_CHECK

		// Cache 2D texture uniform expressions.
		for(int32 ExpressionIndex = 0;ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Standard2D);ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Standard2D, ExpressionIndex);

			const UTexture* Value = nullptr;
			GetTextureValue(EMaterialTextureParameterType::Standard2D, ExpressionIndex, MaterialRenderContext,MaterialRenderContext.Material,Value);
			if (Value)
			{
				// Pre-application validity checks (explicit ensures to avoid needless string allocation)
				//const FMaterialUniformExpressionTextureParameter* TextureParameter = (Uniform2DTextureExpressions[ExpressionIndex]->GetType() == &FMaterialUniformExpressionTextureParameter::StaticType) ?
				//	&static_cast<const FMaterialUniformExpressionTextureParameter&>(*Uniform2DTextureExpressions[ExpressionIndex]) : nullptr;

				// gmartin: Trying to locate UE-23902
				if (!Value->IsValidLowLevel())
				{
					ensureMsgf(false, TEXT("Texture not valid! UE-23902! Parameter (%s)"), *Parameter.ParameterInfo.Name.ToString());
				}

				// Trying to track down a dangling pointer bug.
				checkf(
					Value->IsA<UTexture>(),
					TEXT("Expecting a UTexture! Name(%s), Type(%s), TextureParameter(%s), Expression(%d), Material(%s)"),
					*Value->GetName(), *Value->GetClass()->GetName(),
					*Parameter.ParameterInfo.Name.ToString(),
					ExpressionIndex,
					*MaterialRenderContext.Material.GetFriendlyName());

				// Do not allow external textures to be applied to normal texture samplers
				if (Value->GetMaterialType() == MCT_TextureExternal)
				{
					FText MessageText = FText::Format(
						NSLOCTEXT("MaterialExpressions", "IncompatibleExternalTexture", " applied to a non-external Texture2D sampler. This may work by chance on some platforms but is not portable. Please change sampler type to 'External'. Parameter '{0}' (slot {1}) in material '{2}'"),
						FText::FromName(Parameter.ParameterInfo.GetName()),
						ExpressionIndex,
						FText::FromString(*MaterialRenderContext.Material.GetFriendlyName()));

					GLog->Logf(ELogVerbosity::Warning, TEXT("%s"), *MessageText.ToString());
				}
			}

			void** ResourceTableTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);
			check(BufferCursor <= TempBuffer + TempBufferSize);

			// ExternalTexture is allowed here, with warning above
			// VirtualTexture is allowed here, as these may be demoted to regular textures on platforms that don't have VT support
			const uint32 ValidTextureTypes = MCT_Texture2D | MCT_TextureVirtual | MCT_TextureExternal;

			bool bValueValid = false;

			// TextureReference.TextureReferenceRHI is cleared from a render command issued by UTexture::BeginDestroy
			// It's possible for this command to trigger before a given material is cleaned up and removed from deferred update list
			// Technically I don't think it's necessary to check 'Resource' for nullptr here, as if TextureReferenceRHI has been initialized, that should be enough
			// Going to leave the check for now though, to hopefully avoid any unexpected problems
			if (Value && Value->GetResource() && Value->TextureReference.TextureReferenceRHI && (Value->GetMaterialType() & ValidTextureTypes) != 0u)
			{
				const FSamplerStateRHIRef* SamplerSource = &Value->GetResource()->SamplerStateRHI;

				const ESamplerSourceMode SourceMode = Parameter.SamplerSource;
				if (SourceMode == SSM_Wrap_WorldGroupSettings)
				{
					SamplerSource = &Wrap_WorldGroupSettings->SamplerStateRHI;
				}
				else if (SourceMode == SSM_Clamp_WorldGroupSettings)
				{
					SamplerSource = &Clamp_WorldGroupSettings->SamplerStateRHI;
				}
				// NOTE: for SSM_TerrainWeightmapGroupSettings, the generated code always tries to use the per-view sampler, but we can fallback to the texture-associated sampler if necessary

				if (*SamplerSource)
				{
					*ResourceTableTexturePtr = Value->TextureReference.TextureReferenceRHI;
					*ResourceTableSamplerPtr = *SamplerSource;
					bValueValid = true;
				}
				else
				{
					ensureMsgf(false,
						TEXT("Texture %s of class %s had invalid sampler source. Material %s with texture expression in slot %i. Sampler source mode %d. Resource initialized: %d"),
						*Value->GetName(), *Value->GetClass()->GetName(),
						*MaterialRenderContext.Material.GetFriendlyName(), ExpressionIndex, SourceMode,
						Value->GetResource()->IsInitialized());
				}
			}

			if (!bValueValid)
			{
				check(GWhiteTexture->TextureRHI);
				*ResourceTableTexturePtr = GWhiteTexture->TextureRHI;
				check(GWhiteTexture->SamplerStateRHI);
				*ResourceTableSamplerPtr = GWhiteTexture->SamplerStateRHI;
			}
		}

		// Cache cube texture uniform expressions.
		for(int32 ExpressionIndex = 0;ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Cube); ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Cube, ExpressionIndex);

			const UTexture* Value = nullptr;
			GetTextureValue(EMaterialTextureParameterType::Cube, ExpressionIndex, MaterialRenderContext,MaterialRenderContext.Material,Value);

			void** ResourceTableTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);
			check(BufferCursor <= TempBuffer + TempBufferSize);

			if(Value && Value->GetResource() && Value->TextureReference.TextureReferenceRHI && (Value->GetMaterialType() & MCT_TextureCube) != 0u)
			{
				*ResourceTableTexturePtr = Value->TextureReference.TextureReferenceRHI;
				const FSamplerStateRHIRef* SamplerSource = &Value->GetResource()->SamplerStateRHI;

				const ESamplerSourceMode SourceMode = Parameter.SamplerSource;
				if (SourceMode == SSM_Wrap_WorldGroupSettings)
				{
					SamplerSource = &Wrap_WorldGroupSettings->SamplerStateRHI;
				}
				else if (SourceMode == SSM_Clamp_WorldGroupSettings)
				{
					SamplerSource = &Clamp_WorldGroupSettings->SamplerStateRHI;
				}
				check(SourceMode != SSM_TerrainWeightmapGroupSettings); // not allowed for cubemaps

				check(*SamplerSource);
				*ResourceTableSamplerPtr = *SamplerSource;
			}
			else
			{
				check(GWhiteTextureCube->TextureRHI);
				*ResourceTableTexturePtr = GWhiteTextureCube->TextureRHI;
				check(GWhiteTextureCube->SamplerStateRHI);
				*ResourceTableSamplerPtr = GWhiteTextureCube->SamplerStateRHI;
			}
		}

		// Cache 2d array texture uniform expressions.
		for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Array2D); ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Array2D, ExpressionIndex);

			const UTexture* Value;
			GetTextureValue(EMaterialTextureParameterType::Array2D, ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Value);

			void** ResourceTableTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);

			if (Value && Value->GetResource() && Value->TextureReference.TextureReferenceRHI && (Value->GetMaterialType() & MCT_Texture2DArray) != 0u)
			{
				*ResourceTableTexturePtr = Value->TextureReference.TextureReferenceRHI;
				const FSamplerStateRHIRef* SamplerSource = &Value->GetResource()->SamplerStateRHI;
				ESamplerSourceMode SourceMode = Parameter.SamplerSource;
				if (SourceMode == SSM_Wrap_WorldGroupSettings)
				{
					SamplerSource = &Wrap_WorldGroupSettings->SamplerStateRHI;
				}
				else if (SourceMode == SSM_Clamp_WorldGroupSettings)
				{
					SamplerSource = &Clamp_WorldGroupSettings->SamplerStateRHI;
				}
				check(SourceMode != SSM_TerrainWeightmapGroupSettings); // not allowed for texture arrays

				check(*SamplerSource);
				*ResourceTableSamplerPtr = *SamplerSource;
			}
			else
			{
				check(GBlackArrayTexture->TextureRHI);
				*ResourceTableTexturePtr = GBlackArrayTexture->TextureRHI;
				check(GBlackArrayTexture->SamplerStateRHI);
				*ResourceTableSamplerPtr = GBlackArrayTexture->SamplerStateRHI;
			}
		}

		// Cache cube array texture uniform expressions.
		for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::ArrayCube); ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::ArrayCube, ExpressionIndex);

			const UTexture* Value;
			GetTextureValue(EMaterialTextureParameterType::ArrayCube, ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Value);

			void** ResourceTableTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);

			if (Value && Value->GetResource() && Value->TextureReference.TextureReferenceRHI && (Value->GetMaterialType() & MCT_TextureCubeArray) != 0u)
			{
				*ResourceTableTexturePtr = Value->TextureReference.TextureReferenceRHI;
				const FSamplerStateRHIRef* SamplerSource = &Value->GetResource()->SamplerStateRHI;
				ESamplerSourceMode SourceMode = Parameter.SamplerSource;
				if (SourceMode == SSM_Wrap_WorldGroupSettings)
				{
					SamplerSource = &Wrap_WorldGroupSettings->SamplerStateRHI;
				}
				else if (SourceMode == SSM_Clamp_WorldGroupSettings)
				{
					SamplerSource = &Clamp_WorldGroupSettings->SamplerStateRHI;
				}
				check(SourceMode != SSM_TerrainWeightmapGroupSettings); // not allowed for texture cube arrays

				check(*SamplerSource);
				*ResourceTableSamplerPtr = *SamplerSource;
			}
			else
			{
				check(GBlackArrayTexture->TextureRHI);
				*ResourceTableTexturePtr = GBlackCubeArrayTexture->TextureRHI;
				check(GBlackArrayTexture->SamplerStateRHI);
				*ResourceTableSamplerPtr = GBlackCubeArrayTexture->SamplerStateRHI;
			}
		}

		// Cache volume texture uniform expressions.
		for (int32 ExpressionIndex = 0;ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Volume);ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Volume, ExpressionIndex);

			void** ResourceTableTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);
			check(BufferCursor <= TempBuffer + TempBufferSize);

			const UTexture* Value = nullptr;
			GetTextureValue(EMaterialTextureParameterType::Volume, ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Value);

			if (Value && Value->GetResource() && Value->TextureReference.TextureReferenceRHI && (Value->GetMaterialType() & MCT_VolumeTexture) != 0u)
			{
				*ResourceTableTexturePtr = Value->TextureReference.TextureReferenceRHI;
				const FSamplerStateRHIRef* SamplerSource = &Value->GetResource()->SamplerStateRHI;

				const ESamplerSourceMode SourceMode = Parameter.SamplerSource;
				if (SourceMode == SSM_Wrap_WorldGroupSettings)
				{
					SamplerSource = &Wrap_WorldGroupSettings->SamplerStateRHI;
				}
				else if (SourceMode == SSM_Clamp_WorldGroupSettings)
				{
					SamplerSource = &Clamp_WorldGroupSettings->SamplerStateRHI;
				}

				check(*SamplerSource);
				*ResourceTableSamplerPtr = *SamplerSource;
			}
			else
			{
				check(GBlackVolumeTexture->TextureRHI);
				*ResourceTableTexturePtr = GBlackVolumeTexture->TextureRHI;
				check(GBlackVolumeTexture->SamplerStateRHI);
				*ResourceTableSamplerPtr = GBlackVolumeTexture->SamplerStateRHI;
			}

		}

		// Cache sparse volume texture uniform expressions.
		for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::SparseVolume); ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::SparseVolume, ExpressionIndex);

			void** ResourceTableTexturePageTablePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableTexturePhysicalAPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableTexturePhysicalBPtr = (void**)((uint8*)BufferCursor + 2 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTablePhysicalSamplerPtr = (void**)((uint8*)BufferCursor + 3 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 4);
			check(BufferCursor <= TempBuffer + TempBufferSize);

			check(GBlackVolumeTexture->TextureRHI);
			*ResourceTableTexturePageTablePtr = GBlackUintVolumeTexture->TextureRHI;
			*ResourceTableTexturePhysicalAPtr = GBlackVolumeTexture->TextureRHI;
			*ResourceTableTexturePhysicalBPtr = GBlackVolumeTexture->TextureRHI;
			check(GBlackVolumeTexture->SamplerStateRHI);
			*ResourceTablePhysicalSamplerPtr = GBlackVolumeTexture->SamplerStateRHI;
			
			const USparseVolumeTexture* SVTexture = nullptr;
			GetTextureValue(ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, SVTexture);
			if (SVTexture != nullptr)
			{
				const UE::SVT::FTextureRenderResources* RenderResources = SVTexture->GetTextureRenderResources();
				if (RenderResources)
				{
					FRHITexture* PageTableTexture = RenderResources->GetPageTableTexture();
					FRHITexture* TileDataATexture = RenderResources->GetPhysicalTileDataATexture();
					FRHITexture* TileDataBTexture = RenderResources->GetPhysicalTileDataBTexture();

					// It's possible for RenderResources to be valid, but PageTableTexture still null, so we need to have a fallback (a black uint volume texture)
					*ResourceTableTexturePageTablePtr = PageTableTexture ? PageTableTexture : *ResourceTableTexturePageTablePtr;
					*ResourceTableTexturePhysicalAPtr = TileDataATexture ? TileDataATexture : *ResourceTableTexturePhysicalAPtr;
					*ResourceTableTexturePhysicalBPtr = TileDataBTexture ? TileDataBTexture : *ResourceTableTexturePhysicalBPtr;
					*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				}
			}
		}

		// Cache external texture uniform expressions.
		uint32 ImmutableSamplerIndex = 0;
		for (int32 ExpressionIndex = 0; ExpressionIndex < UniformExternalTextureParameters.Num(); ExpressionIndex++)
		{
			FTextureRHIRef TextureRHI;
			FSamplerStateRHIRef SamplerStateRHI;

			void** ResourceTableTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTableSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);
			check(BufferCursor <= TempBuffer + TempBufferSize);

			if (UniformExternalTextureParameters[ExpressionIndex].GetExternalTexture(MaterialRenderContext, TextureRHI, SamplerStateRHI))
			{
				*ResourceTableTexturePtr = TextureRHI;
				*ResourceTableSamplerPtr = SamplerStateRHI;
			}
			else
			{
				check(GWhiteTexture->TextureRHI);
				*ResourceTableTexturePtr = GWhiteTexture->TextureRHI;
				check(GWhiteTexture->SamplerStateRHI);
				*ResourceTableSamplerPtr = GWhiteTexture->SamplerStateRHI;
			}
		}

		if (!UniformTextureCollectionParameters.IsEmpty())
		{
			for (const FMaterialTextureCollectionParameterInfo& ParameterInfo : UniformTextureCollectionParameters)
			{
				const UTextureCollection* TextureCollection = nullptr;
				ParameterInfo.GetTextureCollection(MaterialRenderContext, TextureCollection);

				FTextureCollectionResource* Resource = TextureCollection ? TextureCollection->GetResource() : nullptr;

				if (!ParameterInfo.bIsVirtualCollection)
				{
					void** ResourceTableBufferPtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
					
					if (FBindlessTextureCollectionResource* BindlessResource = static_cast<FBindlessTextureCollectionResource*>(Resource))
					{
						*ResourceTableBufferPtr = BindlessResource->GetResourceCollectionRHI();
					}
					else
					{
						*ResourceTableBufferPtr = GEmptyResourceCollection.ResourceCollection;
					}
		
					BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT);
					check(BufferCursor <= TempBuffer + TempBufferSize);
				}
				else
				{
					void** ResourceTableBufferPtr          = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
					void** ResourceTablePageTablePtr       = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
					void** ResourceTablePhysicalTexturePtr = (void**)((uint8*)BufferCursor + 2 * SHADER_PARAMETER_POINTER_ALIGNMENT);
					void** ResourceTablePhysicalSamplerPtr = (void**)((uint8*)BufferCursor + 3 * SHADER_PARAMETER_POINTER_ALIGNMENT);
					
					*ResourceTableBufferPtr = nullptr;
					*ResourceTablePageTablePtr = nullptr;
					*ResourceTablePhysicalTexturePtr = nullptr;
					
					if (FVirtualTextureCollectionResource* VirtualResource = static_cast<FVirtualTextureCollectionResource*>(Resource))
					{
						*ResourceTableBufferPtr          = VirtualResource->GetVirtualCollectionRHI();
						*ResourceTablePageTablePtr       = VirtualResource->GetVirtualPageTable();
						*ResourceTablePhysicalTexturePtr = VirtualResource->GetVirtualPhysicalTextureSRV();
					}
					
					if (!*ResourceTableBufferPtr || !*ResourceTablePageTablePtr || !*ResourceTablePhysicalTexturePtr)
					{
						*ResourceTableBufferPtr          = GEmptyVertexBufferUInt4WithUAV->ShaderResourceViewRHI;
						*ResourceTablePageTablePtr       = GBlackUintTexture->TextureRHI.GetReference();
						*ResourceTablePhysicalTexturePtr = GBlackTexture->TextureRHI.GetReference();
					}

					*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_AnisotropicPoint, AM_Clamp, AM_Clamp, AM_Clamp, 0, 8>::GetRHI();

					BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 4);
					check(BufferCursor <= TempBuffer + TempBufferSize);
				}
			}
		}

		// Cache virtual texture page table uniform expressions.
		for (int32 VTStackIndex = 0; VTStackIndex < VTStacks.Num(); ++VTStackIndex)
		{
			void** ResourceTablePageTexture0Ptr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + SHADER_PARAMETER_POINTER_ALIGNMENT;

			void** ResourceTablePageTexture1Ptr = nullptr;
			if (VTStacks[VTStackIndex].GetNumLayers() > 4u)
			{
				ResourceTablePageTexture1Ptr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
				BufferCursor = ((uint8*)BufferCursor) + SHADER_PARAMETER_POINTER_ALIGNMENT;
			}

			void** ResourceTablePageIndirectionBuffer = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + SHADER_PARAMETER_POINTER_ALIGNMENT;

			const IAllocatedVirtualTexture* AllocatedVT = AllocatedVTs[VTStackIndex];
			if (AllocatedVT != nullptr)
			{
				FRHITexture* PageTable0RHI = AllocatedVT->GetPageTableTexture(0u);
				ensure(PageTable0RHI);
				*ResourceTablePageTexture0Ptr = PageTable0RHI;

				if (ResourceTablePageTexture1Ptr != nullptr) //-V1051
				{
					FRHITexture* PageTable1RHI = AllocatedVT->GetPageTableTexture(1u);
					ensure(PageTable1RHI);
					*ResourceTablePageTexture1Ptr = PageTable1RHI;
				}

				FRHITexture* PageTableIndirectionRHI = AllocatedVT->GetPageTableIndirectionTexture();
				ensure(PageTableIndirectionRHI);
				*ResourceTablePageIndirectionBuffer = PageTableIndirectionRHI;
			}
			else
			{
				// Don't have valid resources to bind for this VT, so make sure something is bound
				*ResourceTablePageTexture0Ptr = GBlackUintTexture->TextureRHI;
				if (ResourceTablePageTexture1Ptr != nullptr) //-V1051
				{
					*ResourceTablePageTexture1Ptr = GBlackUintTexture->TextureRHI;
				}
				*ResourceTablePageIndirectionBuffer = GBlackUintTexture->TextureRHI;
			}
		}

		for (int32 TagIndex = 0; TagIndex < MaterialCacheTagStacks.Num(); ++TagIndex)
		{
			const FMaterialCacheTagStack& Stack = MaterialCacheTagStacks[TagIndex];

			// Get the bindings associated with this tag
			IMaterialCacheTagProvider*   TagProvider = GetRendererModule().GetMaterialCacheTagProvider();
			FMaterialCacheTagBindingData Data        = TagProvider->GetBindingData(Stack.TagGuid);
			
			*(void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT) = Data.TagBufferSRV;
			check(Data.TagBufferSRV);
			
			*(void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT) = Data.PageTableSRV;
			check(Data.PageTableSRV);
			
			for (uint32 LayerIndex = 0; LayerIndex < MaterialCacheMaxRuntimeLayers; LayerIndex++)
			{
				FRHIShaderResourceView* SRV = Data.PhysicalTextureSRVs.IsValidIndex(LayerIndex) ? Data.PhysicalTextureSRVs[LayerIndex] : static_cast<FRHIShaderResourceView*>(GBlackTextureWithSRV->ShaderResourceViewRHI);
				*(void**)((uint8*)BufferCursor + (2 + LayerIndex) * SHADER_PARAMETER_POINTER_ALIGNMENT) = SRV;
				check(SRV);
			}
			
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * (MaterialCacheMaxRuntimeLayers + 2));
		}

		// Cache virtual texture physical uniform expressions.
		for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Virtual); ExpressionIndex++)
		{
			const FMaterialTextureParameterInfo& Parameter = GetTextureParameter(EMaterialTextureParameterType::Virtual, ExpressionIndex);

			FTextureRHIRef TexturePhysicalRHI;
			FSamplerStateRHIRef PhysicalSamplerStateRHI;

			bool bValidResources = false;
			void** ResourceTablePhysicalTexturePtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			void** ResourceTablePhysicalSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);

			// Check for streaming virtual texture
			if (!bValidResources)
			{
				const UTexture* Texture = nullptr;
				GetTextureValue(EMaterialTextureParameterType::Virtual, ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Texture);
				if (Texture && Texture->GetResource())
				{
					const FVTPackedStackAndLayerIndex StackAndLayerIndex = GetVTStackAndLayerIndex(ExpressionIndex);
					FVirtualTexture2DResource* VTResource = (FVirtualTexture2DResource*)Texture->GetResource();
					check(VTResource);

					const IAllocatedVirtualTexture* AllocatedVT = AllocatedVTs[StackAndLayerIndex.StackIndex];
					if (AllocatedVT != nullptr)
					{
						FRHIShaderResourceView* PhysicalViewRHI = AllocatedVT->GetPhysicalTextureSRV(StackAndLayerIndex.LayerIndex, VTResource->bSRGB);
						if (PhysicalViewRHI)
						{
							*ResourceTablePhysicalTexturePtr = PhysicalViewRHI;
							*ResourceTablePhysicalSamplerPtr = VTResource->SamplerStateRHI;
							bValidResources = true;
						}
					}
				}
			}
			static const auto CVarVTAnisotropic = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.AnisotropicFiltering"));
			const bool VTAnisotropic = CVarVTAnisotropic && CVarVTAnisotropic->GetValueOnAnyThread() != 0;
			static const auto CVarVTMaxAnisotropic = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.MaxAnisotropy"));
			const int32 VTMaxAnisotropic = (VTAnisotropic && CVarVTMaxAnisotropic) ? CVarVTMaxAnisotropic->GetValueOnAnyThread(): 1;
			// Now check for runtime virtual texture
			if (!bValidResources)
			{
				const URuntimeVirtualTexture* Texture = nullptr;
				GetTextureValue(ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Texture);
				if (Texture != nullptr)
				{
					IAllocatedVirtualTexture const* AllocatedVT = Texture->GetAllocatedVirtualTexture();
					if (AllocatedVT != nullptr)
					{
						const int32 LayerIndex = Parameter.VirtualTextureLayerIndex;
						FRHIShaderResourceView* PhysicalViewRHI = AllocatedVT->GetPhysicalTextureSRV(LayerIndex, Texture->IsLayerSRGB(LayerIndex));
						if (PhysicalViewRHI != nullptr)
						{
							*ResourceTablePhysicalTexturePtr = PhysicalViewRHI;
							if(VTMaxAnisotropic >= 8)
							{
								*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_AnisotropicPoint, AM_Clamp, AM_Clamp, AM_Clamp, 0, 8>::GetRHI();
							}
							else if(VTMaxAnisotropic >= 4)
							{
								*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_AnisotropicPoint, AM_Clamp, AM_Clamp, AM_Clamp, 0, 4>::GetRHI();
							}
							else if (VTMaxAnisotropic >= 2)
							{
								*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_AnisotropicPoint, AM_Clamp, AM_Clamp, AM_Clamp, 0, 2>::GetRHI();
							}
							else
							{
								*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_AnisotropicPoint, AM_Clamp, AM_Clamp, AM_Clamp, 0, 1>::GetRHI();
							}
							bValidResources = true;
						}
					}
				}
			}
			// Don't have valid resources to bind for this VT, so make sure something is bound
			if (!bValidResources)
			{
				*ResourceTablePhysicalTexturePtr = GBlackTextureWithSRV->ShaderResourceViewRHI;
				*ResourceTablePhysicalSamplerPtr = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 8>::GetRHI();
			}
		}

		{
			void** Wrap_WorldGroupSettingsSamplerPtr = (void**)((uint8*)BufferCursor + 0 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			check(Wrap_WorldGroupSettings->SamplerStateRHI);
			*Wrap_WorldGroupSettingsSamplerPtr = Wrap_WorldGroupSettings->SamplerStateRHI;

			void** Clamp_WorldGroupSettingsSamplerPtr = (void**)((uint8*)BufferCursor + 1 * SHADER_PARAMETER_POINTER_ALIGNMENT);
			check(Clamp_WorldGroupSettings->SamplerStateRHI);
			*Clamp_WorldGroupSettingsSamplerPtr = Clamp_WorldGroupSettings->SamplerStateRHI;

			BufferCursor = ((uint8*)BufferCursor) + (SHADER_PARAMETER_POINTER_ALIGNMENT * 2);
			check(BufferCursor <= TempBuffer + TempBufferSize);
		}
	}
}

uint32 FUniformExpressionSet::GetReferencedTexture2DRHIHash(const FMaterialRenderContext& MaterialRenderContext) const
{
	uint32 BaseHash = 0;

	for (int32 ExpressionIndex = 0; ExpressionIndex < GetNumTextures(EMaterialTextureParameterType::Standard2D); ExpressionIndex++)
	{
		const UTexture* Value;
		GetTextureValue(EMaterialTextureParameterType::Standard2D, ExpressionIndex, MaterialRenderContext, MaterialRenderContext.Material, Value);

		const uint32 ValidTextureTypes = MCT_Texture2D | MCT_TextureVirtual | MCT_TextureExternal;

		FRHITexture* TexturePtr = nullptr;
		if (Value && Value->GetResource() && Value->TextureReference.TextureReferenceRHI && (Value->GetMaterialType() & ValidTextureTypes) != 0u)
		{
			TexturePtr = Value->TextureReference.TextureReferenceRHI->GetReferencedTexture();
		}
		BaseHash = PointerHash(TexturePtr, BaseHash);
	}

	return BaseHash;
}

FMaterialUniformExpressionTexture::FMaterialUniformExpressionTexture() :
	TextureIndex(INDEX_NONE),
	TextureLayerIndex(INDEX_NONE),
	MaterialCacheTagIndex(INDEX_NONE),
	PageTableLayerIndex(INDEX_NONE),
#if WITH_EDITORONLY_DATA
	SamplerType(SAMPLERTYPE_Color),
#endif
	SamplerSource(SSM_FromTextureAsset),
	bVirtualTexture(false)
{}

FMaterialUniformExpressionTexture::FMaterialUniformExpressionTexture(int32 InTextureIndex, EMaterialSamplerType InSamplerType, ESamplerSourceMode InSamplerSource, bool InVirtualTexture) :
	TextureIndex(InTextureIndex),
	TextureLayerIndex(INDEX_NONE),
	MaterialCacheTagIndex(INDEX_NONE),
	PageTableLayerIndex(INDEX_NONE),
#if WITH_EDITORONLY_DATA
	SamplerType(InSamplerType),
#endif
	SamplerSource(InSamplerSource),
	bVirtualTexture(InVirtualTexture)
{
}

// This constructor is called for setting up VirtualTextures
FMaterialUniformExpressionTexture::FMaterialUniformExpressionTexture(int32 InTextureIndex, int16 InTextureLayerIndex, int16 InPageTableLayerIndex, EMaterialSamplerType InSamplerType)
	: TextureIndex(InTextureIndex)
	, TextureLayerIndex(InTextureLayerIndex)
	, MaterialCacheTagIndex(INDEX_NONE)
	, PageTableLayerIndex(InPageTableLayerIndex)
#if WITH_EDITORONLY_DATA
	, SamplerType(InSamplerType)
#endif
	, SamplerSource(SSM_Wrap_WorldGroupSettings)
	, bVirtualTexture(true)
{
}

// This constructor is called for setting up SparseVolumeTextures
FMaterialUniformExpressionTexture::FMaterialUniformExpressionTexture(int32 InTextureIndex, EMaterialSamplerType InSamplerType)
	: TextureIndex(InTextureIndex)
	, TextureLayerIndex(INDEX_NONE)
	, MaterialCacheTagIndex(INDEX_NONE)
	, PageTableLayerIndex(INDEX_NONE)
#if WITH_EDITORONLY_DATA
	, SamplerType(InSamplerType)
#endif
	, SamplerSource(SSM_Wrap_WorldGroupSettings)
	, bVirtualTexture(false)
{
}

void FMaterialUniformExpressionTexture::GetTextureParameterInfo(FMaterialTextureParameterInfo& OutParameter) const
{
	OutParameter.TextureIndex = TextureIndex;
	OutParameter.SamplerSource = SamplerSource;
	OutParameter.VirtualTextureLayerIndex = TextureLayerIndex;	// VirtualTextureLayerIndex will be 255 if PageTableLayerIndex==INDEX_NONE.
#if WITH_EDITORONLY_DATA
	OutParameter.ShaderFrequencyMask = 0;
#endif
}

bool FMaterialUniformExpressionTexture::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}
	FMaterialUniformExpressionTexture* OtherTextureExpression = (FMaterialUniformExpressionTexture*)OtherExpression;

	return TextureIndex == OtherTextureExpression->TextureIndex && 
		TextureLayerIndex == OtherTextureExpression->TextureLayerIndex &&
		PageTableLayerIndex == OtherTextureExpression->PageTableLayerIndex &&
		MaterialCacheTagIndex == OtherTextureExpression->MaterialCacheTagIndex &&
		bVirtualTexture == OtherTextureExpression->bVirtualTexture;
}

// FMaterialUniformExpressionTextureCollection

FMaterialUniformExpressionTextureCollection::FMaterialUniformExpressionTextureCollection(int32 InTextureCollectionIndex, int32 InTextureCollectionTypePrefixIndex, bool bIsVirtualCollection)
	: TextureCollectionIndex(InTextureCollectionIndex)
	, TextureCollectionTypePrefixIndex(InTextureCollectionTypePrefixIndex)
    , bIsVirtualCollection(bIsVirtualCollection)
{
}

bool FMaterialUniformExpressionTextureCollection::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}

	const FMaterialUniformExpressionTextureCollection* OtherTextureCollectionExpression = static_cast<const FMaterialUniformExpressionTextureCollection*>(OtherExpression);
	return TextureCollectionIndex == OtherTextureCollectionExpression->TextureCollectionIndex;
}

FMaterialUniformExpressionTextureCollectionParameter* FMaterialUniformExpressionTextureCollection::GetTextureCollectionParameterUniformExpression()
{
	return nullptr;
}

void FMaterialUniformExpressionTextureCollection::GetTextureCollectionParameterInfo(FMaterialTextureCollectionParameterInfo& OutParameter) const
{
	OutParameter.TextureCollectionIndex = this->TextureCollectionIndex;
	OutParameter.bIsVirtualCollection = this->bIsVirtualCollection;
#if WITH_EDITORONLY_DATA
	OutParameter.ShaderFrequencyMask = 0;
#endif
}

// FMaterialUniformExpressionTextureCollectionParameter

FMaterialUniformExpressionTextureCollectionParameter::FMaterialUniformExpressionTextureCollectionParameter(const FMaterialParameterInfo& InParameterInfo, int32 InTextureCollectionIndex, int32 InTextureCollectionTypePrefixIndex, bool bIsVirtualCollection)
	: Super(InTextureCollectionIndex, InTextureCollectionTypePrefixIndex, bIsVirtualCollection)
	, ParameterInfo(InParameterInfo)
{
}

bool FMaterialUniformExpressionTextureCollectionParameter::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}

	const FMaterialUniformExpressionTextureCollectionParameter* OtherParameter = static_cast<const FMaterialUniformExpressionTextureCollectionParameter*>(OtherExpression);
	return ParameterInfo == OtherParameter->ParameterInfo && Super::IsIdentical(OtherParameter);
}

bool FMaterialUniformExpressionTextureCollectionParameter::IsConstant() const
{
	return false;
}

FMaterialUniformExpressionTextureCollectionParameter* FMaterialUniformExpressionTextureCollectionParameter::GetTextureCollectionParameterUniformExpression()
{
	return this;
}

void FMaterialUniformExpressionTextureCollectionParameter::GetTextureCollectionParameterInfo(FMaterialTextureCollectionParameterInfo& OutParameter) const
{
	Super::GetTextureCollectionParameterInfo(OutParameter);
	OutParameter.ParameterInfo = ParameterInfo;
}

// FMaterialUniformExpressionExternalTextureBase

FMaterialUniformExpressionExternalTextureBase::FMaterialUniformExpressionExternalTextureBase(int32 InSourceTextureIndex)
	: SourceTextureIndex(InSourceTextureIndex)
{}

FMaterialUniformExpressionExternalTextureBase::FMaterialUniformExpressionExternalTextureBase(const FGuid& InGuid)
	: SourceTextureIndex(INDEX_NONE)
	, ExternalTextureGuid(InGuid)
{
}

bool FMaterialUniformExpressionExternalTextureBase::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}

	const auto* Other = static_cast<const FMaterialUniformExpressionExternalTextureBase*>(OtherExpression);
	return SourceTextureIndex == Other->SourceTextureIndex && ExternalTextureGuid == Other->ExternalTextureGuid;
}

FGuid FMaterialUniformExpressionExternalTextureBase::ResolveExternalTextureGUID(const FMaterialRenderContext& Context, TOptional<FName> ParameterName) const
{
	return Context.GetExternalTextureGuid(ExternalTextureGuid, ParameterName.IsSet() ? ParameterName.GetValue() : FName(), SourceTextureIndex);
}

void FMaterialUniformExpressionExternalTexture::GetExternalTextureParameterInfo(FMaterialExternalTextureParameterInfo& OutParameter) const
{
	OutParameter.ExternalTextureGuid = ExternalTextureGuid;
	OutParameter.SourceTextureIndex = SourceTextureIndex;
#if WITH_EDITORONLY_DATA
	OutParameter.ShaderFrequencyMask = 0;
#endif
}

FMaterialUniformExpressionExternalTextureParameter::FMaterialUniformExpressionExternalTextureParameter()
{}

FMaterialUniformExpressionExternalTextureParameter::FMaterialUniformExpressionExternalTextureParameter(FName InParameterName, int32 InTextureIndex)
	: Super(InTextureIndex)
	, ParameterName(InParameterName)
{}

void FMaterialUniformExpressionExternalTextureParameter::GetExternalTextureParameterInfo(FMaterialExternalTextureParameterInfo& OutParameter) const
{
	Super::GetExternalTextureParameterInfo(OutParameter);
	OutParameter.ParameterName = NameToScriptName(ParameterName);
#if WITH_EDITORONLY_DATA
	OutParameter.ShaderFrequencyMask = 0;
#endif
}

bool FMaterialUniformExpressionExternalTextureParameter::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}

	auto* Other = static_cast<const FMaterialUniformExpressionExternalTextureParameter*>(OtherExpression);
	return ParameterName == Other->ParameterName && Super::IsIdentical(OtherExpression);
}

void FMaterialTextureParameterInfo::GetGameThreadTextureValue(const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTexture*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialInterface->GetTextureParameterValue(ParameterInfo, OutValue))
	{
		OutValue = GetIndexedTexture<UTexture>(Material, TextureIndex);
	}
}

void FMaterialTextureParameterInfo::GetGameThreadTextureValue(const UMaterialInterface* MaterialInterface, const FMaterial& Material, USparseVolumeTexture*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialInterface->GetSparseVolumeTextureParameterValue(ParameterInfo, OutValue))
	{
		OutValue = GetIndexedTexture<USparseVolumeTexture>(Material, TextureIndex);
	}
}

void FMaterialTextureCollectionParameterInfo::GetTextureCollection(const FMaterialRenderContext& Context, const UTextureCollection*& OutTextureCollection) const
{
	Context.GetTextureCollectionParameterValue(ParameterInfo, TextureCollectionIndex, OutTextureCollection);
}

void FMaterialTextureCollectionParameterInfo::GetGameThreadTextureCollectionValue(const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTextureCollection*& OutValue) const
{
	if (ParameterInfo.Name.IsNone() || !MaterialInterface->GetTextureCollectionParameterValue(ParameterInfo, OutValue))
	{
		OutValue = GetIndexedTextureCollection(Material, TextureCollectionIndex);
	}
}

bool FMaterialExternalTextureParameterInfo::GetExternalTexture(const FMaterialRenderContext& Context, FTextureRHIRef& OutTextureRHI, FSamplerStateRHIRef& OutSamplerStateRHI) const
{
	check(IsInParallelRenderingThread());
	const FGuid GuidToLookup = Context.GetExternalTextureGuid(ExternalTextureGuid, ScriptNameToName(ParameterName), SourceTextureIndex);
	return FExternalTextureRegistry::Get().GetExternalTexture(Context.MaterialRenderProxy, GuidToLookup, OutTextureRHI, OutSamplerStateRHI);
}

bool FMaterialUniformExpressionExternalTextureCoordinateScaleRotation::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType() || !Super::IsIdentical(OtherExpression))
	{
		return false;
	}

	const auto* Other = static_cast<const FMaterialUniformExpressionExternalTextureCoordinateScaleRotation*>(OtherExpression);
	return ParameterName == Other->ParameterName;
}

void FMaterialUniformExpressionExternalTextureCoordinateScaleRotation::WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const
{
	const FScriptName Name = ParameterName.IsSet() ? NameToScriptName(ParameterName.GetValue()) : FScriptName();
	OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::ExternalTextureCoordinateScaleRotation).Write(Name).Write(ExternalTextureGuid).Write<int32>(SourceTextureIndex);
}

bool FMaterialUniformExpressionExternalTextureCoordinateOffset::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType() || !Super::IsIdentical(OtherExpression))
	{
		return false;
	}

	const auto* Other = static_cast<const FMaterialUniformExpressionExternalTextureCoordinateOffset*>(OtherExpression);
	return ParameterName == Other->ParameterName;
}

void FMaterialUniformExpressionExternalTextureCoordinateOffset::WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const
{
	const FScriptName Name = ParameterName.IsSet() ? NameToScriptName(ParameterName.GetValue()) : FScriptName();
	OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::ExternalTextureCoordinateOffset).Write(Name).Write(ExternalTextureGuid).Write<int32>(SourceTextureIndex);
}

FMaterialUniformExpressionRuntimeVirtualTextureUniform::FMaterialUniformExpressionRuntimeVirtualTextureUniform()
	: bParameter(false)
	, TextureIndex(INDEX_NONE)
	, VectorIndex(INDEX_NONE)
{
}

FMaterialUniformExpressionRuntimeVirtualTextureUniform::FMaterialUniformExpressionRuntimeVirtualTextureUniform(int32 InTextureIndex, int32 InVectorIndex)
	: bParameter(false)
	, TextureIndex(InTextureIndex)
	, VectorIndex(InVectorIndex)
{
}

FMaterialUniformExpressionRuntimeVirtualTextureUniform::FMaterialUniformExpressionRuntimeVirtualTextureUniform(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, int32 InVectorIndex)
	: bParameter(true)
	, ParameterInfo(InParameterInfo)
	, TextureIndex(InTextureIndex)
	, VectorIndex(InVectorIndex)
{
}

bool FMaterialUniformExpressionRuntimeVirtualTextureUniform::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}

	const auto* Other = static_cast<const FMaterialUniformExpressionRuntimeVirtualTextureUniform*>(OtherExpression);
	return ParameterInfo == Other->ParameterInfo && TextureIndex == Other->TextureIndex && VectorIndex == Other->VectorIndex;
}

void FMaterialUniformExpressionRuntimeVirtualTextureUniform::WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const
{
	const FHashedMaterialParameterInfo WriteParameterInfo = bParameter ? ParameterInfo : FHashedMaterialParameterInfo();
	OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::RuntimeVirtualTextureUniform).Write(WriteParameterInfo).Write((int32)TextureIndex).Write((int32)VectorIndex);
}

FMaterialUniformExpressionSparseVolumeTextureUniform::FMaterialUniformExpressionSparseVolumeTextureUniform()
	: bParameter(false)
	, TextureIndex(INDEX_NONE)
	, VectorIndex(INDEX_NONE)
{
}

FMaterialUniformExpressionSparseVolumeTextureUniform::FMaterialUniformExpressionSparseVolumeTextureUniform(int32 InTextureIndex, int32 InVectorIndex)
	: bParameter(false)
	, TextureIndex(InTextureIndex)
	, VectorIndex(InVectorIndex)
{
}

FMaterialUniformExpressionSparseVolumeTextureUniform::FMaterialUniformExpressionSparseVolumeTextureUniform(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, int32 InVectorIndex)
	: bParameter(true)
	, ParameterInfo(InParameterInfo)
	, TextureIndex(InTextureIndex)
	, VectorIndex(InVectorIndex)
{
}

bool FMaterialUniformExpressionSparseVolumeTextureUniform::IsIdentical(const FMaterialUniformExpression* OtherExpression) const
{
	if (GetType() != OtherExpression->GetType())
	{
		return false;
	}

	const auto* Other = static_cast<const FMaterialUniformExpressionSparseVolumeTextureUniform*>(OtherExpression);
	return ParameterInfo == Other->ParameterInfo && TextureIndex == Other->TextureIndex && VectorIndex == Other->VectorIndex;
}

void FMaterialUniformExpressionSparseVolumeTextureUniform::WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const
{
	const FHashedMaterialParameterInfo WriteParameterInfo = bParameter ? ParameterInfo : FHashedMaterialParameterInfo();
	OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::SparseVolumeTextureUniform).Write(WriteParameterInfo).Write((int32)TextureIndex).Write((int32)VectorIndex);
}

/**
 * Deprecated FMaterialUniformExpressionRuntimeVirtualTextureParameter in favor of FMaterialUniformExpressionRuntimeVirtualTextureUniform
 * Keep around until we no longer need to support serialization of 4.23 data
 */
class FMaterialUniformExpressionRuntimeVirtualTextureParameter_DEPRECATED : public FMaterialUniformExpressionRuntimeVirtualTextureUniform
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRuntimeVirtualTextureParameter_DEPRECATED);
};

IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTexture);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureCollection);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureCollectionParameter);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionConstant);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionGenericConstant);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionNumericParameter);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionStaticBoolParameter);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureParameter);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureBase);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTexture);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureParameter);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureCoordinateScaleRotation);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureCoordinateOffset);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRuntimeVirtualTextureUniform);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSparseVolumeTextureUniform);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFlipBookTextureParameter);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSine);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSquareRoot);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRcp);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLength);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionNormalize);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExponential);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExponential2);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLogarithm);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLogarithm2);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLogarithm10);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFoldedMath);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionPeriodic);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionAppendVector);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionMin);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionMax);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionClamp);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSaturate);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionComponentSwizzle);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFloor);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionCeil);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFrac);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFmod);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionModulo);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionAbs);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureProperty);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTrigMath);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRound);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTruncate);
IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSign);
