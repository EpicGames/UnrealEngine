// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================
	UniformExpressions.h: Uniform expression definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialLayersFunctions.h"

/**
 * Represents a subclass of FMaterialUniformExpression.
 */
class FMaterialUniformExpressionType
{
public:
	/**
	 * @return The global uniform expression type list.  The list is used to temporarily store the types until
	 *			the name subsystem has been initialized.
	 */
	static TLinkedList<FMaterialUniformExpressionType*>*& GetTypeList();

	/**
	 * Should not be called until the name subsystem has been initialized.
	 * @return The global uniform expression type map.
	 */
	static TMap<FName, FMaterialUniformExpressionType*>& GetTypeMap();

	/**
	 * Minimal initialization constructor.
	 */
	FMaterialUniformExpressionType(const TCHAR* InName);

	const TCHAR* GetName() const { return Name; }

private:
	const TCHAR* Name;
};

#define DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(Name) \
	public: \
	static FMaterialUniformExpressionType StaticType; \
	virtual FMaterialUniformExpressionType* GetType() const { return &StaticType; }

#define IMPLEMENT_MATERIALUNIFORMEXPRESSION_TYPE(Name) \
	FMaterialUniformExpressionType Name::StaticType(TEXT(#Name));

/**
 * Represents an expression which only varies with uniform inputs.
 */
class FMaterialUniformExpression : public FRefCountedObject
{
public:
	virtual ~FMaterialUniformExpression() = default;

	virtual FMaterialUniformExpressionType* GetType() const = 0;
	virtual class FMaterialUniformExpressionTexture* GetTextureUniformExpression() { return nullptr; }
	virtual class FMaterialUniformExpressionExternalTexture* GetExternalTextureUniformExpression() { return nullptr; }
	virtual class FMaterialUniformExpressionTextureCollection* GetTextureCollectionUniformExpression() { return nullptr; }
	virtual bool IsConstant() const { return false; }
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const { return false; }

	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const;

	virtual void GetNumberValue(const struct FMaterialRenderContext& Context, FLinearColor& OutValue) const;

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const { return TArrayView<const FMaterialUniformExpression*>(); }

	/** Offset of this uniform, within the shader's uniform buffer array */
	int32 UniformOffset = INDEX_NONE;

	/** Index of this uniform in the material translator's list of unique expressions */
	int32 UniformIndex = INDEX_NONE;

	EShaderFrequencyMask ShaderFrequencyMask = 0;
};

/**
 * A texture expression.
 */
class FMaterialUniformExpressionTexture : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTexture);

public:
	FMaterialUniformExpressionTexture();
	FMaterialUniformExpressionTexture(int32 InTextureIndex, EMaterialSamplerType InSamplerType, ESamplerSourceMode InSamplerSource, bool InVirtualTexture);
	FMaterialUniformExpressionTexture(int32 InTextureIndex, int16 InTextureLayerIndex, int16 InPageTableLayerIndex, EMaterialSamplerType InSamplerType);
	FMaterialUniformExpressionTexture(int32 InTextureIndex, EMaterialSamplerType InSamplerType);
	
	//~ Begin FMaterialUniformExpression Interface.
	virtual class FMaterialUniformExpressionTexture* GetTextureUniformExpression() { return this; }
	virtual class FMaterialUniformExpressionTextureParameter* GetTextureParameterUniformExpression() { return nullptr; }
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const;
	//~ End FMaterialUniformExpression Interface.

	/** Gets texture index which is the index in the full set of referenced textures for this material. */
	int32 GetTextureIndex() const { return TextureIndex; }
	/** Gets the texture layer index in the virtual texture stack if this is fixed. If we don't have a fixed layer then we will allocate during compilation (and not store here). */
	int32 GetTextureLayerIndex() const { return TextureLayerIndex; }
	int32 GetMaterialCacheTagIndex() const { return MaterialCacheTagIndex; }
	/** Gets the page table channel index in the virtual texture stack if this is fixed. If we don't have a fixed layer then we will allocate during compilation (and not store here). */
	int32 GetPageTableLayerIndex() const { return PageTableLayerIndex; }
	void SetMaterialCacheTagIndex(uint32 InMaterialCacheTagIndex) { MaterialCacheTagIndex = InMaterialCacheTagIndex; }
	
#if WITH_EDITORONLY_DATA
	/** Get the sampling/decoding logic to compile in the shader for this texture. */
	EMaterialSamplerType GetSamplerType() const { return SamplerType; }
#endif

	/** Get the sampler state object to use (globally shared or from texture asset).  */
	ESamplerSourceMode GetSamplerSource() const { return SamplerSource; }

	virtual void GetTextureParameterInfo(FMaterialTextureParameterInfo& OutParameter) const;

protected:
	/** Index into FMaterial::GetReferencedTextures */
	int32 TextureIndex;
	/** Texture layer index in virtual texture stack if preallocated */
	int16 TextureLayerIndex;
	/** Material cache tag index if this is a material cache binding */
	int16 MaterialCacheTagIndex;
	/** Page table layer index in virtual texture stack if preallocated */
	int16 PageTableLayerIndex;
#if WITH_EDITORONLY_DATA
	/** Sampler logic for this expression */
	EMaterialSamplerType SamplerType;
#endif
	/** Sampler state object source for this expression */
	ESamplerSourceMode SamplerSource;
	/** Virtual texture flag used only for unique serialization */
	bool bVirtualTexture;
};

class FMaterialUniformExpressionTextureCollection : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureCollection);

public:
	FMaterialUniformExpressionTextureCollection(int32 InTextureCollectionIndex, int32 InTextureCollectionTypePrefixIndex, bool bIsVirtualCollection);

	//~ Begin FMaterialUniformExpression Interface.
	virtual class FMaterialUniformExpressionTextureCollection* GetTextureCollectionUniformExpression() final { return this; }
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	//~ End FMaterialUniformExpression Interface.

	virtual class FMaterialUniformExpressionTextureCollectionParameter* GetTextureCollectionParameterUniformExpression();
	virtual void GetTextureCollectionParameterInfo(FMaterialTextureCollectionParameterInfo& OutParameter) const;

	int32 GetTextureCollectionIndex() const
	{
		return TextureCollectionIndex;
	}

	void SetTextureCollectionTypePrefixIndex(uint32 InPrefix)
	{
		TextureCollectionTypePrefixIndex = InPrefix;
	}

	int32 GetTextureCollectionTypePrefixIndex() const
	{
		return TextureCollectionTypePrefixIndex;
	}

	bool IsVirtualCollection() const
	{
		return bIsVirtualCollection;
	}

protected:
	int32 TextureCollectionIndex;
	int32 TextureCollectionTypePrefixIndex;
	bool bIsVirtualCollection;
};

class FMaterialUniformExpressionTextureCollectionParameter : public FMaterialUniformExpressionTextureCollection
{
	using Super = FMaterialUniformExpressionTextureCollection;
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureCollectionParameter);

public:
	FMaterialUniformExpressionTextureCollectionParameter(const FMaterialParameterInfo& InParameterInfo, int32 InTextureCollectionIndex, int32 InTextureCollectionTypePrefixIndex, bool bIsVirtualCollection);

	//~ Begin FMaterialUniformExpression Interface.
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	virtual bool IsConstant() const override;
	//~ End FMaterialUniformExpression Interface.

	//~ Begin FMaterialUniformExpressionTextureCollection Interface.
	virtual FMaterialUniformExpressionTextureCollectionParameter* GetTextureCollectionParameterUniformExpression() final;
	virtual void GetTextureCollectionParameterInfo(FMaterialTextureCollectionParameterInfo& OutParameter) const override;
	//~ End FMaterialUniformExpressionTextureCollection Interface.

	FName GetParameterName() const
	{
		return ParameterInfo.GetName();
	}

	const FHashedMaterialParameterInfo& GetParameterInfo() const
	{
		return ParameterInfo;
	}

private:
	FHashedMaterialParameterInfo ParameterInfo;
};

class FMaterialUniformExpressionExternalTextureBase : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureBase);
public:

	FMaterialUniformExpressionExternalTextureBase(int32 InSourceTextureIndex = INDEX_NONE);
	FMaterialUniformExpressionExternalTextureBase(const FGuid& InExternalTextureGuid);

	virtual bool IsConstant() const override { return false; }
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;

	int32 GetSourceTextureIndex() const { return SourceTextureIndex; }

protected:

	/** Resolve the guid that relates to texture information inside FExternalTexture */
	FGuid ResolveExternalTextureGUID(const FMaterialRenderContext& Context, TOptional<FName> ParameterName = TOptional<FName>()) const;

	/** Index of the texture in the material that should be used to retrieve the external texture GUID at runtime (or INDEX_NONE) */
	int32 SourceTextureIndex;
	/** Optional external texture GUID defined at compile time */
	FGuid ExternalTextureGuid;
};

/**
* An external texture expression.
*/
class FMaterialUniformExpressionExternalTexture : public FMaterialUniformExpressionExternalTextureBase
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTexture);
public:

	FMaterialUniformExpressionExternalTexture(int32 InSourceTextureIndex = INDEX_NONE) : FMaterialUniformExpressionExternalTextureBase(InSourceTextureIndex) {}
	FMaterialUniformExpressionExternalTexture(const FGuid& InGuid) : FMaterialUniformExpressionExternalTextureBase(InGuid) {}

	// FMaterialUniformExpression interface.
	virtual FMaterialUniformExpressionExternalTexture* GetExternalTextureUniformExpression() override { return this; }
	virtual class FMaterialUniformExpressionExternalTextureParameter* GetExternalTextureParameterUniformExpression() { return nullptr; }

	virtual void GetExternalTextureParameterInfo(FMaterialExternalTextureParameterInfo& OutParameter) const;
};

/**
 */
class FMaterialUniformExpressionConstant: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionConstant);
public:
	FMaterialUniformExpressionConstant() {}
	FMaterialUniformExpressionConstant(const FLinearColor& InValue,uint8 InValueType):
		Value(InValue),
		ValueType(InValueType)
	{}

	// FMaterialUniformExpression interface.
	virtual bool IsConstant() const
	{
		return true;
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionConstant* OtherConstant = (FMaterialUniformExpressionConstant*)OtherExpression;
		return OtherConstant->ValueType == ValueType && OtherConstant->Value == Value;
	}

	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		UE::Shader::FValue ConstantValue(Value);
		switch (ValueType)
		{
		case MCT_Float:
		case MCT_Float1: ConstantValue.Type = UE::Shader::EValueType::Float1; break;
		case MCT_Float2: ConstantValue.Type = UE::Shader::EValueType::Float2; break;
		case MCT_Float3: ConstantValue.Type = UE::Shader::EValueType::Float3; break;
		case MCT_Float4: ConstantValue.Type = UE::Shader::EValueType::Float4; break;
		default: checkNoEntry(); break;
		}
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Constant);
		OutData.Write(ConstantValue);
	}

	virtual void GetNumberValue(const FMaterialRenderContext& Context, FLinearColor& OutValue) const override
	{
		OutValue = Value;
	}

private:
	FLinearColor Value;
	uint8 ValueType;
};

/**
 */
class FMaterialUniformExpressionGenericConstant : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionGenericConstant);
public:
	FMaterialUniformExpressionGenericConstant() {}
	FMaterialUniformExpressionGenericConstant(const UE::Shader::FValue& InValue) :
		Value(InValue)
	{}

	// FMaterialUniformExpression interface.
	virtual bool IsConstant() const
	{
		return true;
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionGenericConstant* OtherConstant = (FMaterialUniformExpressionGenericConstant*)OtherExpression;
		return OtherConstant->Value == Value;
	}

	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Constant);
		OutData.Write(Value);
	}

	virtual void GetNumberValue(const FMaterialRenderContext& Context, FLinearColor& OutValue) const override
	{
		OutValue = Value.AsLinearColor();
	}

private:
	UE::Shader::FValue Value;
};

/**
 */
class FMaterialUniformExpressionNumericParameter : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionNumericParameter);
public:

	FMaterialUniformExpressionNumericParameter() {}
	FMaterialUniformExpressionNumericParameter(const FMaterialParameterInfo& InParameterInfo, int32 InParameterIndex)
		: ParameterInfo(InParameterInfo)
		, ParameterIndex(InParameterIndex)
	{
		check(InParameterIndex >= 0 && InParameterIndex <= 0xffff);
	}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Parameter);
		OutData.Write((uint16)ParameterIndex);
	}

	virtual bool IsConstant() const
	{
		return false;
	}

	const FHashedMaterialParameterInfo& GetParameterInfo() const
	{
		return ParameterInfo;
	}

	FName GetParameterName() const
	{
		return ParameterInfo.GetName();
	}

	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionNumericParameter* OtherParameter = (FMaterialUniformExpressionNumericParameter*)OtherExpression;
		return ParameterInfo == OtherParameter->ParameterInfo && ParameterIndex == OtherParameter->ParameterIndex;
	}
private:
	FHashedMaterialParameterInfo ParameterInfo;
	int32 ParameterIndex;
};

/**
 */
class FMaterialUniformExpressionStaticBoolParameter : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionStaticBoolParameter);
public:

	FMaterialUniformExpressionStaticBoolParameter() {}
	FMaterialUniformExpressionStaticBoolParameter(const FMaterialParameterInfo& InParameterInfo, uint32 InParameterIndex)
		: ParameterIndex(InParameterIndex)
		, ParameterInfo(InParameterInfo)
	{
		check(InParameterIndex >= 0 && InParameterIndex <= 0xffff);
	}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Parameter);
		OutData.Write((uint16)ParameterIndex);
	}

	virtual bool IsConstant() const
	{
		return false;
	}

	const FHashedMaterialParameterInfo& GetParameterInfo() const
	{
		return ParameterInfo;
	}

	FName GetParameterName() const
	{
		return ParameterInfo.GetName();
	}

	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionStaticBoolParameter* OtherParameter = (FMaterialUniformExpressionStaticBoolParameter*)OtherExpression;
		return ParameterInfo == OtherParameter->ParameterInfo && ParameterIndex == OtherParameter->ParameterIndex;
	}
private:
	uint32 ParameterIndex;
	FHashedMaterialParameterInfo ParameterInfo;
};

/** @return The texture that was associated with the given index when the given material had its uniform expressions/HLSL code generated. */
template<typename TextureType>
static TextureType* GetIndexedTexture(const FMaterial& Material, int32 TextureIndex)
{
	UObject* IndexedTexture = Material.GetMatDefaultTextureAtIdx(TextureIndex);

	// Can return nullptr if TextureType doesn't match type of indexed texture
	return Cast<TextureType>(IndexedTexture);
}

inline UTextureCollection* GetIndexedTextureCollection(const FMaterial& Material, int32 TextureCollectionIndex)
{
	UTextureCollection* IndexedTextureCollection = nullptr;
	const TArrayView<const TObjectPtr<UTextureCollection>> ReferencedTextureCollections = Material.GetReferencedTextureCollections();
	if (ReferencedTextureCollections.IsValidIndex(TextureCollectionIndex))
	{
		IndexedTextureCollection = ReferencedTextureCollections[TextureCollectionIndex];
	}

	if (IndexedTextureCollection == nullptr)
	{
		static bool bWarnedOnce = false;
		if (!bWarnedOnce)
		{
			UE_LOGF(LogMaterial, Warning, "%ls: Requesting an invalid TextureIndex! (%u / %u)", *Material.GetFriendlyName(), TextureCollectionIndex, ReferencedTextureCollections.Num());
			bWarnedOnce = true;
		}
	}

	return IndexedTextureCollection;
}

/**
 * A texture parameter expression.
 */
class FMaterialUniformExpressionTextureParameter: public FMaterialUniformExpressionTexture
{
	typedef FMaterialUniformExpressionTexture Super;
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureParameter);
public:

	FMaterialUniformExpressionTextureParameter() {}

	FMaterialUniformExpressionTextureParameter(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, EMaterialSamplerType InSamplerType, ESamplerSourceMode InSourceMode, bool InVirtualTexture)
		: Super(InTextureIndex, InSamplerType, InSourceMode, InVirtualTexture)
		, ParameterInfo(InParameterInfo)
	{}

	FMaterialUniformExpressionTextureParameter(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, int32 InTextureLayerIndex, int32 InPageTableLayerIndex, EMaterialSamplerType InSamplerType)
		: Super(InTextureIndex, InTextureLayerIndex, InPageTableLayerIndex, InSamplerType)
		, ParameterInfo(InParameterInfo)
	{}

	FMaterialUniformExpressionTextureParameter(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, EMaterialSamplerType InSamplerType)
		: Super(InTextureIndex, InSamplerType)
		, ParameterInfo(InParameterInfo)
	{}

	// FMaterialUniformExpression interface.
	virtual class FMaterialUniformExpressionTextureParameter* GetTextureParameterUniformExpression() override { return this; }

	virtual void GetTextureParameterInfo(FMaterialTextureParameterInfo& OutParameter) const override
	{
		Super::GetTextureParameterInfo(OutParameter);
		OutParameter.ParameterInfo = ParameterInfo;
	}

	virtual bool IsConstant() const
	{
		return false;
	}

	FName GetParameterName() const
	{
		return ParameterInfo.GetName();
	}

	const FHashedMaterialParameterInfo& GetParameterInfo() const
	{
		return ParameterInfo;
	}

	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionTextureParameter* OtherParameter = (FMaterialUniformExpressionTextureParameter*)OtherExpression;
		return ParameterInfo == OtherParameter->ParameterInfo && Super::IsIdentical(OtherParameter);
	}

private:
	FHashedMaterialParameterInfo ParameterInfo;
	int32 ParameterIndex;
};

/**
 * A flipbook texture parameter expression.
 */
class FMaterialUniformExpressionFlipBookTextureParameter : public FMaterialUniformExpressionTexture
{
	typedef FMaterialUniformExpressionTexture Super;
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFlipBookTextureParameter);
public:

	FMaterialUniformExpressionFlipBookTextureParameter() {}

	virtual bool IsConstant() const
	{
		return false;
	}
};


/**
 * An external texture parameter expression.
 */
class FMaterialUniformExpressionExternalTextureParameter: public FMaterialUniformExpressionExternalTexture
{
	typedef FMaterialUniformExpressionExternalTexture Super;
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureParameter);
public:

	FMaterialUniformExpressionExternalTextureParameter();
	FMaterialUniformExpressionExternalTextureParameter(FName InParameterName, int32 InTextureIndex);

	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	virtual void GetExternalTextureParameterInfo(FMaterialExternalTextureParameterInfo& OutParameter) const override;
	virtual FMaterialUniformExpressionExternalTextureParameter* GetExternalTextureParameterUniformExpression() override { return this; }

	FName GetParameterName() const
	{
		return ParameterName;
	}

private:
	FName ParameterName;
};

/**
 */
class FMaterialUniformExpressionSine: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSine);
public:

	FMaterialUniformExpressionSine() {}
	FMaterialUniformExpressionSine(FMaterialUniformExpression* InX,bool bInIsCosine):
		X(InX),
		bIsCosine(bInIsCosine)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(bIsCosine ? UE::Shader::EPreshaderOpcode::Cos : UE::Shader::EPreshaderOpcode::Sin);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionSine* OtherSine = (FMaterialUniformExpressionSine*)OtherExpression;
		return X->IsIdentical(OtherSine->X) && bIsCosine == OtherSine->bIsCosine;
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
	bool bIsCosine;
};

/**
 */
enum ETrigMathOperation
{
	TMO_Sin,
	TMO_Cos,
	TMO_Tan,
	TMO_Asin,
	TMO_Acos,
	TMO_Atan,
	TMO_Atan2
};

/**
 */
class FMaterialUniformExpressionTrigMath: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTrigMath);
public:

	FMaterialUniformExpressionTrigMath() {}
	FMaterialUniformExpressionTrigMath(FMaterialUniformExpression* InX, ETrigMathOperation InOp):
		X(InX),
		Y(InX),
		Op(InOp)
	{}

	FMaterialUniformExpressionTrigMath(FMaterialUniformExpression* InX, FMaterialUniformExpression* InY, ETrigMathOperation InOp):
		X(InX),
		Y(InY),
		Op(InOp)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		if (Op == TMO_Atan2)
		{
			Y->WriteNumberOpcodes(OutData);
		}
		switch (Op)
		{
		case TMO_Sin: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Sin); break;
		case TMO_Cos: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Cos); break;
		case TMO_Tan: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Tan); break;
		case TMO_Asin: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Asin); break;
		case TMO_Acos: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Acos); break;
		case TMO_Atan: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Atan); break;
		case TMO_Atan2: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Atan2); break;
		default: checkNoEntry(); break;
		}
	}

	virtual bool IsConstant() const
	{
		return X->IsConstant() && Y->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionTrigMath* OtherTrig = (FMaterialUniformExpressionTrigMath*)OtherExpression;
		return X->IsIdentical(OtherTrig->X) && Y->IsIdentical(OtherTrig->Y) && Op == OtherTrig->Op;
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
	TRefCountPtr<FMaterialUniformExpression> Y;
	uint8 Op;
};

/**
 */
class FMaterialUniformExpressionSquareRoot: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSquareRoot);
public:

	FMaterialUniformExpressionSquareRoot() {}
	FMaterialUniformExpressionSquareRoot(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Sqrt);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionSquareRoot* OtherSqrt = (FMaterialUniformExpressionSquareRoot*)OtherExpression;
		return X->IsIdentical(OtherSqrt->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionRcp : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRcp);
public:

	FMaterialUniformExpressionRcp() {}
	FMaterialUniformExpressionRcp(FMaterialUniformExpression* InX) :
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Rcp);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionRcp* OtherRcp = (FMaterialUniformExpressionRcp*)OtherExpression;
		return X->IsIdentical(OtherRcp->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionLength: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLength);
public:

	FMaterialUniformExpressionLength() : ValueType(MCT_Float) {}
	FMaterialUniformExpressionLength(FMaterialUniformExpression* InX, uint32 InValueType = MCT_Float):
		X(InX),
		ValueType(InValueType)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Length);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionLength* OtherSqrt = (FMaterialUniformExpressionLength*)OtherExpression;
		return X->IsIdentical(OtherSqrt->X) && ValueType == OtherSqrt->ValueType;
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
	uint32 ValueType;
};

/**
 */
class FMaterialUniformExpressionNormalize : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionNormalize);
public:

	FMaterialUniformExpressionNormalize() {}
	FMaterialUniformExpressionNormalize(FMaterialUniformExpression* InX) : X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Normalize);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionNormalize* OtherSqrt = (FMaterialUniformExpressionNormalize*)OtherExpression;
		return X->IsIdentical(OtherSqrt->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionExponential : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExponential);
public:

	FMaterialUniformExpressionExponential()
	{}
	FMaterialUniformExpressionExponential(FMaterialUniformExpression* InX) :
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Exp);
	}
	bool IsConstant() const override
	{
		return X->IsConstant();
	}
	bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override
	{
		if(GetType() != OtherExpression->GetType())
		{
			return false;
		}

		const FMaterialUniformExpressionExponential* OtherExp = static_cast<const FMaterialUniformExpressionExponential*>(OtherExpression);
		return X->IsIdentical(OtherExp->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionExponential2 : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExponential2);
public:

	FMaterialUniformExpressionExponential2()
	{}
	FMaterialUniformExpressionExponential2(FMaterialUniformExpression* InX) :
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Exp2);
	}
	bool IsConstant() const override
	{
		return X->IsConstant();
	}
	bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override
	{
		if(GetType() != OtherExpression->GetType())
		{
			return false;
		}

		const FMaterialUniformExpressionExponential2* OtherExp2= static_cast<const FMaterialUniformExpressionExponential2*>(OtherExpression);
		return X->IsIdentical(OtherExp2->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionLogarithm : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLogarithm);
public:

	FMaterialUniformExpressionLogarithm()
	{}
	FMaterialUniformExpressionLogarithm(FMaterialUniformExpression* InX) :
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Log);
	}
	bool IsConstant() const override
	{
		return X->IsConstant();
	}
	bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override
	{
		if(GetType() != OtherExpression->GetType())
		{
			return false;
		}

		const FMaterialUniformExpressionLogarithm* OtherLog = static_cast<const FMaterialUniformExpressionLogarithm*>(OtherExpression);
		return X->IsIdentical(OtherLog->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionLogarithm2: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLogarithm2);
public:

	FMaterialUniformExpressionLogarithm2() {}
	FMaterialUniformExpressionLogarithm2(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Log2);
	}
	bool IsConstant() const override
	{
		return X->IsConstant();
	}
	bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}

		auto OtherLog = static_cast<const FMaterialUniformExpressionLogarithm2 *>(OtherExpression);
		return X->IsIdentical(OtherLog->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionLogarithm10: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionLogarithm10);
public:

	FMaterialUniformExpressionLogarithm10() {}
	FMaterialUniformExpressionLogarithm10(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Log10);
	}
	bool IsConstant() const override
	{
		return X->IsConstant();
	}
	bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}

		auto OtherLog = static_cast<const FMaterialUniformExpressionLogarithm10*>(OtherExpression);
		return X->IsIdentical(OtherLog->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
enum EFoldedMathOperation
{
	FMO_Add,
	FMO_Sub,
	FMO_Mul,
	FMO_Div,
	FMO_Dot,
	FMO_Cross
};

class FMaterialUniformExpressionFoldedMath: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFoldedMath);
public:

	FMaterialUniformExpressionFoldedMath() : ValueType(MCT_Float) {}
	FMaterialUniformExpressionFoldedMath(FMaterialUniformExpression* InA,FMaterialUniformExpression* InB,uint8 InOp, uint32 InValueType = MCT_Float):
		A(InA),
		B(InB),
		ValueType(InValueType),
		Op(InOp)	
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		A->WriteNumberOpcodes(OutData);
		B->WriteNumberOpcodes(OutData);

		switch (Op)
		{
		case FMO_Add: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Add); break;
		case FMO_Sub: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Sub); break;
		case FMO_Mul: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Mul); break;
		case FMO_Div: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Div); break;
		case FMO_Dot: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Dot); break;
		case FMO_Cross: OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Cross); break;
		default: checkNoEntry(); break;
		}
	}

	virtual bool IsConstant() const
	{
		return A->IsConstant() && B->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionFoldedMath* OtherMath = (FMaterialUniformExpressionFoldedMath*)OtherExpression;
		return A->IsIdentical(OtherMath->A) && B->IsIdentical(OtherMath->B) && Op == OtherMath->Op && ValueType == OtherMath->ValueType;
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&A, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> A;
	TRefCountPtr<FMaterialUniformExpression> B;
	uint32 ValueType;
	uint8 Op;
};

/**
 * A hint that only the fractional part of this expession's value matters.
 */
class FMaterialUniformExpressionPeriodic: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionPeriodic);
public:

	FMaterialUniformExpressionPeriodic() {}
	FMaterialUniformExpressionPeriodic(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Fractional);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionPeriodic* OtherPeriodic = (FMaterialUniformExpressionPeriodic*)OtherExpression;
		return X->IsIdentical(OtherPeriodic->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionAppendVector: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionAppendVector);
public:

	FMaterialUniformExpressionAppendVector() {}
	FMaterialUniformExpressionAppendVector(FMaterialUniformExpression* InA,FMaterialUniformExpression* InB,uint32 InNumComponentsA):
		A(InA),
		B(InB),
		NumComponentsA(InNumComponentsA)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		A->WriteNumberOpcodes(OutData);
		B->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::AppendVector);// .Write((uint8)NumComponentsA);
	}
	virtual bool IsConstant() const
	{
		return A->IsConstant() && B->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionAppendVector* OtherAppend = (FMaterialUniformExpressionAppendVector*)OtherExpression;
		return A->IsIdentical(OtherAppend->A) && B->IsIdentical(OtherAppend->B) && NumComponentsA == OtherAppend->NumComponentsA;
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&A, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> A;
	TRefCountPtr<FMaterialUniformExpression> B;
	uint32 NumComponentsA;
};

/**
 */
class FMaterialUniformExpressionMin: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionMin);
public:

	FMaterialUniformExpressionMin() {}
	FMaterialUniformExpressionMin(FMaterialUniformExpression* InA,FMaterialUniformExpression* InB):
		A(InA),
		B(InB)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		A->WriteNumberOpcodes(OutData);
		B->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Min);
	}
	virtual bool IsConstant() const
	{
		return A->IsConstant() && B->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionMin* OtherMin = (FMaterialUniformExpressionMin*)OtherExpression;
		return A->IsIdentical(OtherMin->A) && B->IsIdentical(OtherMin->B);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&A, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> A;
	TRefCountPtr<FMaterialUniformExpression> B;
};

/**
 */
class FMaterialUniformExpressionMax: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionMax);
public:

	FMaterialUniformExpressionMax() {}
	FMaterialUniformExpressionMax(FMaterialUniformExpression* InA,FMaterialUniformExpression* InB):
		A(InA),
		B(InB)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		A->WriteNumberOpcodes(OutData);
		B->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Max);
	}
	virtual bool IsConstant() const
	{
		return A->IsConstant() && B->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionMax* OtherMax = (FMaterialUniformExpressionMax*)OtherExpression;
		return A->IsIdentical(OtherMax->A) && B->IsIdentical(OtherMax->B);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&A, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> A;
	TRefCountPtr<FMaterialUniformExpression> B;
};

/**
 */
class FMaterialUniformExpressionClamp: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionClamp);
public:

	FMaterialUniformExpressionClamp() {}
	FMaterialUniformExpressionClamp(FMaterialUniformExpression* InInput,FMaterialUniformExpression* InMin,FMaterialUniformExpression* InMax):
		Input(InInput),
		Min(InMin),
		Max(InMax)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		Input->WriteNumberOpcodes(OutData);
		Min->WriteNumberOpcodes(OutData);
		Max->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Clamp);
	}
	virtual bool IsConstant() const
	{
		return Input->IsConstant() && Min->IsConstant() && Max->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionClamp* OtherClamp = (FMaterialUniformExpressionClamp*)OtherExpression;
		return Input->IsIdentical(OtherClamp->Input) && Min->IsIdentical(OtherClamp->Min) && Max->IsIdentical(OtherClamp->Max);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&Input, 3);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> Input;
	TRefCountPtr<FMaterialUniformExpression> Min;
	TRefCountPtr<FMaterialUniformExpression> Max;
};

/**
 */
class FMaterialUniformExpressionSaturate: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSaturate);
public:

	FMaterialUniformExpressionSaturate() {}
	FMaterialUniformExpressionSaturate(FMaterialUniformExpression* InInput):
		Input(InInput)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		Input->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Saturate);
	}
	virtual bool IsConstant() const
	{
		return Input->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionSaturate* OtherClamp = (FMaterialUniformExpressionSaturate*)OtherExpression;
		return Input->IsIdentical(OtherClamp->Input);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&Input, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> Input;
};

class FMaterialUniformExpressionComponentSwizzle : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionComponentSwizzle);
public:

	FMaterialUniformExpressionComponentSwizzle() {}
	FMaterialUniformExpressionComponentSwizzle(FMaterialUniformExpression* InX, int8 InR, int8 InG, int8 InB, int8 InA) :
		X(InX),
		IndexR(InR),
		IndexG(InG),
		IndexB(InB),
		IndexA(InA)
	{
		NumElements = 0;
		if (InA >= 0)
		{
			check(InA <= 3);
			++NumElements;
			check(InB >= 0);
		}

		if (InB >= 0)
		{
			check(InB <= 3);
			++NumElements;
			check(InG >= 0);
		}

		if (InG >= 0)
		{
			check(InG <= 3);
			++NumElements;
		}

		// At least one proper index
		check(InR >= 0 && InR <= 3);
		++NumElements;
	}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::ComponentSwizzle).Write((uint8)NumElements).Write((uint8)IndexR).Write((uint8)IndexG).Write((uint8)IndexB).Write((uint8)IndexA);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		auto* OtherSwizzle = (FMaterialUniformExpressionComponentSwizzle*)OtherExpression;
		return X->IsIdentical(OtherSwizzle->X) &&
			NumElements == OtherSwizzle->NumElements &&
			IndexR == OtherSwizzle->IndexR &&
			IndexG == OtherSwizzle->IndexG &&
			IndexB == OtherSwizzle->IndexB &&
			IndexA == OtherSwizzle->IndexA;
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
	int8 IndexR;
	int8 IndexG;
	int8 IndexB;
	int8 IndexA;
	int8 NumElements;
};

/**
 */
class FMaterialUniformExpressionFloor: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFloor);
public:

	FMaterialUniformExpressionFloor() {}
	FMaterialUniformExpressionFloor(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Floor);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionFloor* OtherFloor = (FMaterialUniformExpressionFloor*)OtherExpression;
		return X->IsIdentical(OtherFloor->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionCeil: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionCeil);
public:

	FMaterialUniformExpressionCeil() {}
	FMaterialUniformExpressionCeil(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Ceil);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionCeil* OtherCeil = (FMaterialUniformExpressionCeil*)OtherExpression;
		return X->IsIdentical(OtherCeil->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionRound: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRound);
public:

	FMaterialUniformExpressionRound() {}
	FMaterialUniformExpressionRound(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Round);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionRound* OtherRound = (FMaterialUniformExpressionRound*)OtherExpression;
		return X->IsIdentical(OtherRound->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionTruncate: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTruncate);
public:

	FMaterialUniformExpressionTruncate() {}
	FMaterialUniformExpressionTruncate(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Trunc);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionTruncate* OtherTrunc = (FMaterialUniformExpressionTruncate*)OtherExpression;
		return X->IsIdentical(OtherTrunc->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionSign: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSign);
public:

	FMaterialUniformExpressionSign() {}
	FMaterialUniformExpressionSign(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Sign);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionSign* OtherSign = (FMaterialUniformExpressionSign*)OtherExpression;
		return X->IsIdentical(OtherSign->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionFrac: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFrac);
public:

	FMaterialUniformExpressionFrac() {}
	FMaterialUniformExpressionFrac(FMaterialUniformExpression* InX):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Frac);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionFrac* OtherFrac = (FMaterialUniformExpressionFrac*)OtherExpression;
		return X->IsIdentical(OtherFrac->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionFmod : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionFmod);
public:

	FMaterialUniformExpressionFmod() {}
	FMaterialUniformExpressionFmod(FMaterialUniformExpression* InA,FMaterialUniformExpression* InB):
		A(InA),
		B(InB)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		A->WriteNumberOpcodes(OutData);
		B->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Fmod);
	}
	virtual bool IsConstant() const
	{
		return A->IsConstant() && B->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionFmod* OtherMax = (FMaterialUniformExpressionFmod*)OtherExpression;
		return A->IsIdentical(OtherMax->A) && B->IsIdentical(OtherMax->B);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&A, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> A;
	TRefCountPtr<FMaterialUniformExpression> B;
};

/**
 */
class FMaterialUniformExpressionModulo : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionModulo);
public:

	FMaterialUniformExpressionModulo() {}
	FMaterialUniformExpressionModulo(FMaterialUniformExpression* InA,FMaterialUniformExpression* InB):
		A(InA),
		B(InB)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		A->WriteNumberOpcodes(OutData);
		B->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Modulo);
	}
	virtual bool IsConstant() const
	{
		return A->IsConstant() && B->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		const FMaterialUniformExpressionModulo* OtherMax = (const FMaterialUniformExpressionModulo*)OtherExpression;
		return A->IsIdentical(OtherMax->A) && B->IsIdentical(OtherMax->B);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&A, 2);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> A;
	TRefCountPtr<FMaterialUniformExpression> B;
};

/**
 * Absolute value evaluator for a given input expression
 */
class FMaterialUniformExpressionAbs: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionAbs);
public:

	FMaterialUniformExpressionAbs() {}
	FMaterialUniformExpressionAbs( FMaterialUniformExpression* InX ):
		X(InX)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		X->WriteNumberOpcodes(OutData);
		OutData.WriteOpcode(UE::Shader::EPreshaderOpcode::Abs);
	}
	virtual bool IsConstant() const
	{
		return X->IsConstant();
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}
		FMaterialUniformExpressionAbs* OtherAbs = (FMaterialUniformExpressionAbs*)OtherExpression;
		return X->IsIdentical(OtherAbs->X);
	}

	virtual TArrayView<const FMaterialUniformExpression*> GetChildren() const override
	{
		return TArrayView<const FMaterialUniformExpression*>((const FMaterialUniformExpression**)&X, 1);
	}

private:
	TRefCountPtr<FMaterialUniformExpression> X;
};

/**
 */
class FMaterialUniformExpressionTextureProperty: public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionTextureProperty);
public:
	
	FMaterialUniformExpressionTextureProperty() {}
	FMaterialUniformExpressionTextureProperty(FMaterialUniformExpressionTexture* InTextureExpression, EMaterialExposedTextureProperty InTextureProperty)
		: TextureExpression(InTextureExpression)
		, TextureProperty(InTextureProperty)
	{}

	// FMaterialUniformExpression interface.
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override
	{
		FMaterialTextureParameterInfo TextureParameter;
		TextureExpression->GetTextureParameterInfo(TextureParameter);

		UE::Shader::EPreshaderOpcode Op = UE::Shader::EPreshaderOpcode::Nop;
		switch (TextureProperty)
		{
		case TMTM_TextureSize: Op = UE::Shader::EPreshaderOpcode::TextureSize; break;
		case TMTM_TexelSize: Op = UE::Shader::EPreshaderOpcode::TexelSize; break;
		default: checkNoEntry(); break;
		}
		OutData.WriteOpcode(Op).Write(TextureParameter.ParameterInfo).Write((int32)TextureParameter.TextureIndex);
	}
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override
	{
		if (GetType() != OtherExpression->GetType())
		{
			return false;
		}

		auto OtherTexturePropertyExpression = (const FMaterialUniformExpressionTextureProperty*)OtherExpression;
		
		if (TextureProperty != OtherTexturePropertyExpression->TextureProperty)
		{
			return false;
		}

		return TextureExpression->IsIdentical(OtherTexturePropertyExpression->TextureExpression);
	}
	
private:
	TRefCountPtr<FMaterialUniformExpressionTexture> TextureExpression;
	int8 TextureProperty;
};


/**
 * A uniform expression to lookup the UV coordinate rotation and scale for an external texture
 */
class FMaterialUniformExpressionExternalTextureCoordinateScaleRotation : public FMaterialUniformExpressionExternalTextureBase
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureCoordinateScaleRotation);
public:

	FMaterialUniformExpressionExternalTextureCoordinateScaleRotation(){}
	FMaterialUniformExpressionExternalTextureCoordinateScaleRotation(const FGuid& InGuid) : FMaterialUniformExpressionExternalTextureBase(InGuid) {}
	FMaterialUniformExpressionExternalTextureCoordinateScaleRotation(int32 InSourceTextureIndex, TOptional<FName> InParameterName) : FMaterialUniformExpressionExternalTextureBase(InSourceTextureIndex), ParameterName(InParameterName) {}

	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override;

protected:
	typedef FMaterialUniformExpressionExternalTextureBase Super;

	/** Optional texture parameter name */
	TOptional<FName> ParameterName;
};

/**
 * A uniform expression to lookup the UV coordinate offset for an external texture
 */
class FMaterialUniformExpressionExternalTextureCoordinateOffset : public FMaterialUniformExpressionExternalTextureBase
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionExternalTextureCoordinateOffset);
public:

	FMaterialUniformExpressionExternalTextureCoordinateOffset(){}
	FMaterialUniformExpressionExternalTextureCoordinateOffset(const FGuid& InGuid) : FMaterialUniformExpressionExternalTextureBase(InGuid) {}
	FMaterialUniformExpressionExternalTextureCoordinateOffset(int32 InSourceTextureIndex, TOptional<FName> InParameterName) : FMaterialUniformExpressionExternalTextureBase(InSourceTextureIndex), ParameterName(InParameterName) {}

	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override;

protected:
	typedef FMaterialUniformExpressionExternalTextureBase Super;

	/** Optional texture parameter name */
	TOptional<FName> ParameterName;
};

/**
 * A uniform expression to retrieve one of the vector uniform parameters stored in a URuntimeVirtualTexture
 */
class FMaterialUniformExpressionRuntimeVirtualTextureUniform : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionRuntimeVirtualTextureUniform);

public:
	FMaterialUniformExpressionRuntimeVirtualTextureUniform();
	/** Construct with the index of the texture reference and the vector index that we want to retrieve. */
	FMaterialUniformExpressionRuntimeVirtualTextureUniform(int32 InTextureIndex, int32 InVectorIndex);
	/** Construct with a URuntimeVirtualTexture parameter and the vector index that we want to retrieve. */
	FMaterialUniformExpressionRuntimeVirtualTextureUniform(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, int32 InVectorIndex);

	//~ Begin FMaterialUniformExpression Interface.
	virtual bool IsConstant() const override { return false; }
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override;
	//~ End FMaterialUniformExpression Interface.

protected:
	/** Is this expression using a material instance parameter. */
	bool bParameter;
	/** Contains the parameter info used if bParameter is true. */
	FHashedMaterialParameterInfo ParameterInfo;
	/** Index of the associated URuntimeVirtualTexture in the material texture references used if bParameter is false. */
	int32 TextureIndex;
	/** Index of the uniform vector to fetch from the URuntimeVirtualTexture. */
	int32 VectorIndex;
};

/**
 * A uniform expression to retrieve one of the vector uniform parameters stored in a USparseVolumeTexture
 */
class FMaterialUniformExpressionSparseVolumeTextureUniform : public FMaterialUniformExpression
{
	DECLARE_MATERIALUNIFORMEXPRESSION_TYPE(FMaterialUniformExpressionSparseVolumeTextureUniform);

public:
	FMaterialUniformExpressionSparseVolumeTextureUniform();
	/** Construct with the index of the texture reference and the vector index that we want to retrieve. */
	FMaterialUniformExpressionSparseVolumeTextureUniform(int32 InTextureIndex, int32 InVectorIndex);
	/** Construct with a URuntimeVirtualTexture parameter and the vector index that we want to retrieve. */
	FMaterialUniformExpressionSparseVolumeTextureUniform(const FMaterialParameterInfo& InParameterInfo, int32 InTextureIndex, int32 InVectorIndex);

	//~ Begin FMaterialUniformExpression Interface.
	virtual bool IsConstant() const override { return false; }
	virtual bool IsIdentical(const FMaterialUniformExpression* OtherExpression) const override;
	virtual void WriteNumberOpcodes(UE::Shader::FPreshaderData& OutData) const override;
	//~ End FMaterialUniformExpression Interface.

protected:
	/** Is this expression using a material instance parameter. */
	bool bParameter;
	/** Contains the parameter info used if bParameter is true. */
	FHashedMaterialParameterInfo ParameterInfo;
	/** Index of the associated URuntimeVirtualTexture in the material texture references used if bParameter is false. */
	int32 TextureIndex;
	/** Index of the uniform vector to fetch from the URuntimeVirtualTexture. */
	int32 VectorIndex;
};
