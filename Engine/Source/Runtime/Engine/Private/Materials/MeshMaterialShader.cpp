// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshMaterialShader.cpp: Mesh material shader implementation.
=============================================================================*/

#include "MeshMaterialShader.h"
#include "ShaderCompiler.h"
#include "ShaderDiagnostics.h"

#if WITH_EDITOR

static void PrepareMeshMaterialShaderCompileJob(EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	const FMaterial* Material,
	const FMaterialCompilationOutput& CompilationOutput,
	FSharedShaderCompilerEnvironment* MaterialEnvironment,
	const FShaderPipelineType* ShaderPipeline,
	const FString& DebugGroupName,
	const TCHAR* DebugDescription,
	const TCHAR* DebugExtension,
	FShaderCompileJob* NewJob)
{
	const FShaderCompileJobKey& Key = NewJob->Key;
	const FMeshMaterialShaderType* ShaderType = Key.ShaderType->AsMeshMaterialShaderType();
	const FVertexFactoryType* VertexFactoryType = Key.VFType;

	NewJob->Input.SharedEnvironment = MaterialEnvironment;
	FShaderCompilerEnvironment& ShaderEnvironment = NewJob->Input.Environment;
	ShaderEnvironment.TargetPlatform = MaterialEnvironment->TargetPlatform;
	NewJob->bIsDefaultMaterial = Material->IsDefaultMaterial();
	NewJob->bIsGlobalShader = false;
	NewJob->bBypassCache = Material->IsPreview() || !Material->IsPersistent();

	if (IsShaderDevelopmentModeEnabled())
	{
		NewJob->bErrorsAreLikelyToBeCode = true;
	}

	const FCompilationShaderParameters MaterialParameters(Material, CompilationOutput);

	// apply the vertex factory changes to the compile environment
	check(VertexFactoryType);
	VertexFactoryType->ModifyCompilationEnvironment(FVertexFactoryShaderPermutationParameters(Platform, MaterialParameters, VertexFactoryType, ShaderType, PermutationFlags), ShaderEnvironment);

	Material->SetupExtraCompilationSettings(NewJob->Input.ExtraSettings);

	//update material shader stats
	UpdateMaterialShaderCompilingStats(Material);

	UE_LOGF(LogShaders, Verbose, "			%ls", ShaderType->GetName());

	// Allow the shader type to modify the compile environment.
	ShaderType->SetupCompileEnvironment(Platform, MaterialParameters, VertexFactoryType, Key.PermutationId, PermutationFlags, ShaderEnvironment);

	bool bAllowDevelopmentShaderCompile = Material->GetAllowDevelopmentShaderCompile();

	// Compile the shader environment passed in with the shader type's source code.
	::GlobalBeginCompileShader(
		DebugGroupName,
		VertexFactoryType,
		ShaderType,
		ShaderPipeline,
		Key.PermutationId,
		ShaderType->GetShaderFilename(),
		ShaderType->GetFunctionName(),
		FShaderTarget(ShaderType->GetFrequency(), Platform),
		NewJob->Input,
		bAllowDevelopmentShaderCompile,
		DebugDescription,
		DebugExtension
	);
}


/**
 * Enqueues a compilation for a new shader of this type.
 * @param Platform - The platform to compile for.
 * @param Material - The material to link the shader with.
 * @param VertexFactoryType - The vertex factory to compile with.
 */
void FMeshMaterialShaderType::BeginCompileShader(
	EShaderCompileJobPriority Priority,
	uint32 ShaderMapJobId,
	int32 PermutationId,
	EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	const FMaterial* Material,
	const FMaterialCompilationOutput& CompilationOutput,
	FSharedShaderCompilerEnvironment* MaterialEnvironment,
	const FVertexFactoryType* VertexFactoryType,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	const FString& DebugGroupName,
	const TCHAR* DebugDescription,
	const TCHAR* DebugExtension) const
{
	FShaderCompileJob* NewJob = GShaderCompilingManager->PrepareShaderCompileJob(ShaderMapJobId, FShaderCompileJobKey(this, Platform, VertexFactoryType, PermutationId), Priority);
	if (NewJob)
	{
		PrepareMeshMaterialShaderCompileJob(Platform, PermutationFlags, Material, CompilationOutput, MaterialEnvironment, nullptr, DebugGroupName, DebugDescription, DebugExtension, NewJob);
		NewJobs.Add(FShaderCommonCompileJobPtr(NewJob));
	}
}

void FMeshMaterialShaderType::BeginCompileShaderPipeline(
	EShaderCompileJobPriority Priority,
	uint32 ShaderMapJobId,
	int32 PermutationId,
	EShaderPlatform Platform,
	EShaderPermutationFlags PermutationFlags,
	const FMaterial* Material,
	const FMaterialCompilationOutput& CompilationOutput,
	FSharedShaderCompilerEnvironment* MaterialEnvironment,
	const FVertexFactoryType* VertexFactoryType,
	const FShaderPipelineType* ShaderPipeline,
	TArray<FShaderCommonCompileJobPtr>& NewJobs,
	const FString& DebugGroupName,
	const TCHAR* DebugDescription,
	const TCHAR* DebugExtension)
{
	check(ShaderPipeline);
	UE_LOGF(LogShaders, Verbose, "	Pipeline: %ls", ShaderPipeline->GetName());

	// Add all the jobs as individual first, then add the dependencies into a pipeline job
	auto* NewPipelineJob = GShaderCompilingManager->PreparePipelineCompileJob(ShaderMapJobId, FShaderPipelineCompileJobKey(ShaderPipeline, Platform, VertexFactoryType, PermutationId), Priority);
	if (NewPipelineJob)
	{
		for (FShaderCompileJob* StageJob : NewPipelineJob->StageJobs)
		{
			PrepareMeshMaterialShaderCompileJob(Platform, PermutationFlags, Material, CompilationOutput, MaterialEnvironment, ShaderPipeline, DebugGroupName, DebugDescription, DebugExtension, StageJob);
		}
		NewJobs.Add(FShaderCommonCompileJobPtr(NewPipelineJob));
	}
}


static inline FString GetJobName(const FShaderCompileJob* SingleJob, const FShaderPipelineType* ShaderPipelineType, const FString& InDebugDescription)
{
	FString String = SingleJob->Input.GenerateShaderName();
	if (ShaderPipelineType)
	{
		String += FString::Printf(TEXT(" Pipeline '%s'"), ShaderPipelineType->GetName());
	}
	if (SingleJob->Key.VFType)
	{
		String += FString::Printf(TEXT(" VF '%s'"), SingleJob->Key.VFType->GetName());
	}
	String += FString::Printf(TEXT(" Type '%s'"), SingleJob->Key.ShaderType->GetName());
	String += FString::Printf(TEXT(" '%s' Entry '%s' Permutation %i %s"), *SingleJob->Input.VirtualSourceFilePath, *SingleJob->Input.EntryPointName, SingleJob->Key.PermutationId, *InDebugDescription);
	return String;
}

FShader* FMeshMaterialShaderType::FinishCompileShader( 
	const FUniformExpressionSet& UniformExpressionSet, 
	uint64 MaterialShaderMapHash,
	const FShaderCompileJob& CurrentJob,
	const FShaderPipelineType* ShaderPipelineType,
	const FString& InDebugDescription) const
{
	checkf(CurrentJob.bSucceeded, TEXT("Failed MeshMaterialType compilation job: %s"), *GetJobName(&CurrentJob, ShaderPipelineType, InDebugDescription));
	checkf(CurrentJob.Key.VFType, TEXT("No VF on MeshMaterialType compilation job: %s"), *GetJobName(&CurrentJob, ShaderPipelineType, InDebugDescription));

	if (ShaderPipelineType && !ShaderPipelineType->ShouldOptimizeUnusedOutputs(CurrentJob.Input.Target.GetPlatform()))
	{
		// If sharing shaders in this pipeline, remove it from the type/id so it uses the one in the shared shadermap list
		ShaderPipelineType = nullptr;
	}

	FShader* Shader = ConstructCompiled(CompiledShaderInitializerType(this, CurrentJob.Key.PermutationId, CurrentJob.Output, UniformExpressionSet, MaterialShaderMapHash, InDebugDescription, ShaderPipelineType, CurrentJob.Key.VFType));
	CurrentJob.Output.ParameterMap.VerifyBindingsAreComplete(GetName(), CurrentJob.Output.Target, CurrentJob.Key.VFType);

	return Shader;
}

#endif // WITH_EDITOR

bool FMeshMaterialShaderType::ShouldCompilePermutation(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, int32 PermutationId, EShaderPermutationFlags Flags) const
{
	return FShaderType::ShouldCompilePermutation(FMeshMaterialShaderPermutationParameters(Platform, MaterialParameters, VertexFactoryType, PermutationId, Flags));
}

bool FMeshMaterialShaderType::ShouldCompileVertexFactoryPermutation(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, const FShaderType* ShaderType, EShaderPermutationFlags Flags)
{
	return VertexFactoryType->ShouldCache(FVertexFactoryShaderPermutationParameters(Platform, MaterialParameters, VertexFactoryType, ShaderType, Flags));
}

bool FMeshMaterialShaderType::ShouldCompilePipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, EShaderPermutationFlags Flags)
{
	const FMeshMaterialShaderPermutationParameters Parameters(Platform, MaterialParameters, VertexFactoryType, kUniqueShaderPermutationId, Flags);
	for (const FShaderType* ShaderType : ShaderPipelineType->GetStages())
	{
		checkSlow(ShaderType->GetMeshMaterialShaderType());
		if (!ShaderType->ShouldCompilePermutation(Parameters))
		{
			return false;
		}
	}
	return true;
}

bool FMeshMaterialShaderType::ShouldCompileVertexFactoryPipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, EShaderPermutationFlags Flags)
{
	for (const FShaderType* ShaderType : ShaderPipelineType->GetStages())
	{
		checkSlow(ShaderType->GetMeshMaterialShaderType());

		if (!VertexFactoryType->ShouldCache(FVertexFactoryShaderPermutationParameters(Platform, MaterialParameters, VertexFactoryType, ShaderType, Flags)))
		{
			return false;
		}
	}

	return true;
}

#if WITH_EDITOR
void FMeshMaterialShaderType::SetupCompileEnvironment(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, int32 PermutationId, EShaderPermutationFlags Flags, FShaderCompilerEnvironment& Environment) const
{
	// Allow the shader type to modify its compile environment.
	FShaderType::ModifyCompilationEnvironment(FMeshMaterialShaderPermutationParameters(Platform, MaterialParameters, VertexFactoryType, PermutationId, Flags), Environment);
}

#endif // WITH_EDITOR
