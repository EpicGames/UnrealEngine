// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterCollection.h"
#include "UObject/UObjectIterator.h"
#include "RenderingThread.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "MaterialShared.h"
#include "MaterialCachedData.h"
#include "Materials/Material.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Materials/MaterialParameterCollection.h"
#include "Misc/App.h"
#include "RenderGraphBuilder.h"
#include "Templates/Tuple.h"

int32 GDeferUpdateRenderStates = 1;
FAutoConsoleVariableRef CVarDeferUpdateRenderStates(
	TEXT("r.DeferUpdateRenderStates"),
	GDeferUpdateRenderStates,
	TEXT("Whether to defer updating the render states of material parameter collections when a parameter is changed until a rendering command needs them up to date.  Deferring updates is more efficient because multiple SetVectorParameterValue and SetScalarParameterValue calls in a frame will only result in one update."),
	ECVF_RenderThreadSafe
);

int32 GMaterialParameterCollectionMaxVectorStorage = 1280;
FAutoConsoleVariableRef CVarMaterialParameterCollectionMaxVectorStorage(
	TEXT("r.MPC.MaxVectorStorage"),
	GMaterialParameterCollectionMaxVectorStorage,
	TEXT("The maximum number of vectors allowed in a parameter collection without generating a warning."),
	ECVF_RenderThreadSafe
);

TMultiMap<FGuid, FMaterialParameterCollectionInstanceResource*> GDefaultMaterialParameterCollectionInstances;

UMaterialParameterCollection::UMaterialParameterCollection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ReleasedByRT(true)
{
	DefaultResource = nullptr;
}

void UMaterialParameterCollection::PostInitProperties()
{
	Super::PostInitProperties();

	if (LIKELY(!HasAnyFlags(RF_ClassDefaultObject) && FApp::CanEverRender()))
	{
		DefaultResource = new FMaterialParameterCollectionInstanceResource();
	}
}

void UMaterialParameterCollection::PostLoad()
{
	Super::PostLoad();

	if (Base)
	{
		// Determine if the structure of the Base has changed.
		if (BaseStateId != Base->StateId)
		{
			UpdateOverrides(Base);
		}
	}
	else
	{
		// Determine if there used to be a Base.
		if (BaseStateId != FGuid())
		{
			BaseStateId = FGuid();
			StateId = FGuid::NewGuid();
			ScalarParameterBaseOverrides.Empty();
			VectorParameterBaseOverrides.Empty();
		}
	}
	
	if (!StateId.IsValid())
	{
		StateId = FGuid::NewGuid();
	}

	CreateBufferStruct();
	SetupWorldParameterCollectionInstances();
	UpdateDefaultResource(true);
}

void UMaterialParameterCollection::SetupWorldParameterCollectionInstances()
{
	for (TObjectIterator<UWorld> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		UWorld* CurrentWorld = *It;
		ULevel* Level = CurrentWorld->PersistentLevel;
		const bool bIsWorldPartitionRuntimeCell = Level && Level->IsWorldPartitionRuntimeCell();
		if (!bIsWorldPartitionRuntimeCell)
		{
			CurrentWorld->AddParameterCollectionInstance(this, true);
		}
	}
}

void UMaterialParameterCollection::BeginDestroy()
{
	if (DefaultResource)
	{
		ReleasedByRT = false;

		FMaterialParameterCollectionInstanceResource* Resource = DefaultResource;
		FGuid Id = StateId;
		FThreadSafeBool* Released = &ReleasedByRT;
		ENQUEUE_RENDER_COMMAND(RemoveDefaultResourceCommand)(
			[Resource, Id, Released](FRHICommandListImmediate& RHICmdList)
			{
				// Async RDG tasks can call FMaterialShader::SetParameters which touch material parameter collections.
				FRDGBuilder::WaitForAsyncExecuteTask();
				GDefaultMaterialParameterCollectionInstances.RemoveSingle(Id, Resource);
				*Released = true;
			}
		);
	}

	Super::BeginDestroy();
}

bool UMaterialParameterCollection::IsReadyForFinishDestroy()
{
	bool bIsReady = Super::IsReadyForFinishDestroy();
	return bIsReady && ReleasedByRT;
}

void UMaterialParameterCollection::FinishDestroy()
{
	if (DefaultResource)
	{
		DefaultResource->GameThread_Destroy();
		DefaultResource = nullptr;
	}

	Super::FinishDestroy();
}

#if WITH_EDITOR

bool UMaterialParameterCollection::SetScalarParameterDefaultValueByInfo(FCollectionScalarParameter ScalarParameter)
{
	// if the input parameter exists, pass the name and value down to SetScalarParameterDefaultValue
	// since we want to preserve the Guid of the parameter that's already on the asset
	return SetScalarParameterDefaultValue(ScalarParameter.ParameterName, ScalarParameter.DefaultValue);
}

bool UMaterialParameterCollection::SetScalarParameterDefaultValue(FName ParameterName, const float Value)
{
	for (UMaterialParameterCollection* BaseCollection = this; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		// Search each collection's scalar parameters for the named parameter.
		for (int32 ParameterIndex = 0; ParameterIndex < BaseCollection->ScalarParameters.Num(); ++ParameterIndex)
		{
			FCollectionScalarParameter& CollectionParameter = BaseCollection->ScalarParameters[ParameterIndex];
			if (CollectionParameter.ParameterName != ParameterName)
			{
				continue;
			}

			if (BaseCollection == this)
			{
				// If this collections own the parameter,
				// change the value in the array itself to maintain its GUID.
				CollectionParameter.DefaultValue = Value;
			}
			else
			{
				// Otherwise, set an override value for the parameter in this collection.
				float& OverrideValue = ScalarParameterBaseOverrides.FindOrAdd(CollectionParameter.Id);
				OverrideValue = Value;
			}

			return true;
		}
	}

	return false;
}

bool UMaterialParameterCollection::SetVectorParameterDefaultValueByInfo(FCollectionVectorParameter VectorParameter)
{
	// if the input parameter exists, pass the name and value down to SetVectorParameterDefaultValue
	// since we want to preserve the Guid of the parameter that's already on the asset
	return SetVectorParameterDefaultValue(VectorParameter.ParameterName, VectorParameter.DefaultValue);
}

bool UMaterialParameterCollection::SetVectorParameterDefaultValue(FName ParameterName, const FLinearColor& Value)
{
	for (UMaterialParameterCollection* BaseCollection = this; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		// Search each collection's vector parameters for the named parameter.
		for (int32 ParameterIndex = 0; ParameterIndex < BaseCollection->VectorParameters.Num(); ++ParameterIndex)
		{
			FCollectionVectorParameter& CollectionParameter = BaseCollection->VectorParameters[ParameterIndex];
			if (CollectionParameter.ParameterName != ParameterName)
			{
				continue;
			}

			if (BaseCollection == this)
			{
				// If this collections own the parameter,
				// change the value in the array itself to maintain its GUID.
				CollectionParameter.DefaultValue = Value;
			}
			else
			{
				// Otherwise, set an override value for the parameter in this collection.
				FLinearColor& OverrideValue = VectorParameterBaseOverrides.FindOrAdd(CollectionParameter.Id);
				OverrideValue = Value;
			}

			return true;
		}
	}

	return false;
}

int32 PreviousTotalVectorStorage = 0;

void UMaterialParameterCollection::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	PreviousTotalVectorStorage = GetTotalVectorStorage();
}

void UMaterialParameterCollection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (Base && (!BaseStateId.IsValid() || BaseStateId != Base->StateId))
	{
		BaseStateId = Base->StateId;

		// Verify that a new Base does not create a circular dependency.
		for (UMaterialParameterCollection* BaseCollection = Base; BaseCollection; BaseCollection = BaseCollection->Base)
		{
			if (BaseCollection == this)
			{
				Base = nullptr;
				break;
			}
		}
	}

	if (!Base)
	{
		// Clear Base-dependent state.
		BaseStateId = FGuid();
		ScalarParameterBaseOverrides.Empty();
		VectorParameterBaseOverrides.Empty();
	}

	SanitizeParameters(&UMaterialParameterCollection::ScalarParameters);
	SanitizeParameters(&UMaterialParameterCollection::VectorParameters);

	// If the storage total has changed, an element has been added or removed, and we need to update the uniform buffer layout,
	// Which also requires recompiling any referencing materials
	int32 TotalVectorStorage = GetTotalVectorStorage();
	if (TotalVectorStorage != PreviousTotalVectorStorage)
	{
		// Generate a new Id so that unloaded materials that reference this collection will update correctly on load
		// Now that we changed the guid, we must recompile all materials which reference this collection
		StateId = FGuid::NewGuid();

		// Update the uniform buffer layout
		CreateBufferStruct();

		// If this collection is the base of another collection, the other collection must be updated.
		TArray<UMaterialParameterCollection*> DerivedCollections;
		for (TObjectIterator<UMaterialParameterCollection> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterialParameterCollection* Collection = *It;
			if (Collection != this && Collection->UpdateOverrides(this))
			{
				Collection->CreateBufferStruct();
				DerivedCollections.Add(Collection);
			}
		}

		// Create a material update context so we can safely update materials using this parameter collection.
		{
			FMaterialUpdateContext UpdateContext;

			// Go through all materials in memory and recompile them if they use this material parameter collection
			for (TObjectIterator<UMaterial> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
			{
				UMaterial* CurrentMaterial = *It;

				bool bRecompile = false;

				// Preview materials often use expressions for rendering that are not in their Expressions array, 
				// And therefore their MaterialParameterCollectionInfos are not up to date.
				if (CurrentMaterial->bIsPreviewMaterial || CurrentMaterial->bIsFunctionPreviewMaterial)
				{
					bRecompile = true;
				}
				else
				{
					for (int32 CollectionInfoIndex = 0; CollectionInfoIndex < CurrentMaterial->GetCachedExpressionData().ParameterCollectionInfos.Num(); ++CollectionInfoIndex)
					{
						// If this collection is referenced by a material, or is a base of a collection referenced by a material, the material must be recompiled.
						UMaterialParameterCollection* Collection = CurrentMaterial->GetCachedExpressionData().ParameterCollectionInfos[CollectionInfoIndex].ParameterCollection;
						while (Collection != this && Collection)
						{
							Collection = Collection->Base;
						}

						if (Collection)
						{
							bRecompile = true;
							break;
						}
					}
				}

				if (bRecompile)
				{
					UpdateContext.AddMaterial(CurrentMaterial);

					// Propagate the change to this material
					CurrentMaterial->PreEditChange(nullptr);
					CurrentMaterial->PostEditChange();
					CurrentMaterial->MarkPackageDirty();
				}
			}

			// Recreate all uniform buffers based on this collection
			for (TObjectIterator<UWorld> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
			{
				UWorld* CurrentWorld = *It;
				CurrentWorld->UpdateParameterCollectionInstances(true, true);
			}

			UpdateDefaultResource(true);

			// Updated the default resource of any derived collections that are loaded.
			for (UMaterialParameterCollection* DerivedCollection : DerivedCollections)
			{
				DerivedCollection->UpdateDefaultResource(true);
			}
		}
	}
	else
	{
		// If this collection is the base of another collection, the other collection must be updated.
		for (TObjectIterator<UMaterialParameterCollection> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UMaterialParameterCollection* Collection = *It;
			if (Collection != this)
			{
				Collection->UpdateOverrides(this);
			}
		}

		// We didn't need to recreate the uniform buffer, just update its contents
		for (TObjectIterator<UWorld> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UWorld* CurrentWorld = *It;
			CurrentWorld->UpdateParameterCollectionInstances(true, false);
		}

		UpdateDefaultResource(false);
	}

#if WITH_EDITOR
	CollectionChangedDelegate.Broadcast();
#endif

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FName UMaterialParameterCollection::GetBaseParameterCollectionMemberName()
{
	return GET_MEMBER_NAME_CHECKED(UMaterialParameterCollection, Base);
}

#endif // WITH_EDITOR

TTuple<FString, int32> SplitParameterName(FName ParameterName)
{
	FString NameString;
	ParameterName.ToString(NameString);

	int32 Number = 0;
	int32 NumberStartIndex = NameString.FindLastCharByPredicate([](TCHAR Letter){ return !FChar::IsDigit(Letter); }) + 1;
	if (NumberStartIndex < NameString.Len())
	{
		ensure(TCString<TCHAR>::IsNumeric(&NameString[NumberStartIndex]));
		TTypeFromString<int32>::FromString(Number, &NameString[NumberStartIndex]);
		NameString.LeftInline(NumberStartIndex);
	}

	if (NameString.IsEmpty())
	{
		NameString = TEXT("Param");
	}
	
	return { MoveTemp(NameString), Number };
}

template<typename FCollectionParameterType>
void UMaterialParameterCollection::SanitizeParameters(TArray<FCollectionParameterType> UMaterialParameterCollection::* CollectionParameters)
{
	TSet<FGuid> ActiveParameterIds;
	TMap<FString, TSet<int32>> ActiveParameterNames;

	// Collect active parameter ids and names from any base collections.
	// SanitizeParameters should be called when base collections have already been sanitized.
	for (const UMaterialParameterCollection* BaseCollection = Base; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		// Add each parameter in each base.
		for (int32 BaseParameterIndex = 0; BaseParameterIndex < (BaseCollection->*CollectionParameters).Num(); ++BaseParameterIndex)
		{
			const FCollectionParameterType& BaseCollectionParameter = (BaseCollection->*CollectionParameters)[BaseParameterIndex];

			// Update the set of active ids.
			ActiveParameterIds.Add(BaseCollectionParameter.Id);

			// Split the parameter name into a name and number.
			TTuple<FString, int32> ParameterNameAndNumber = SplitParameterName(BaseCollectionParameter.ParameterName);

			// Update the map of active parameter numbers.
			ActiveParameterNames.FindOrAdd(get<0>(ParameterNameAndNumber)).Add(get<1>(ParameterNameAndNumber));
		}
	}

	// Sanitize each parameter in the collection.
	for (int32 ParameterIndex = 0; ParameterIndex < (this->*CollectionParameters).Num(); ++ParameterIndex)
	{
		FCollectionParameterType& CollectionParameter = (this->*CollectionParameters)[ParameterIndex];

		// Ensure the parameter has a unique id.
		while (ActiveParameterIds.Contains(CollectionParameter.Id))
		{
			FPlatformMisc::CreateGuid(CollectionParameter.Id);
		}
		ActiveParameterIds.Add(CollectionParameter.Id);

		// Split the parameter name into a name and number.
		TTuple<FString, int32> ParameterNameAndNumber = SplitParameterName(CollectionParameter.ParameterName);

		// Find the next available parameter number for the parameter name.
		int32 ParameterNumber = get<1>(ParameterNameAndNumber);
		TSet<int32>& ActiveParameterNumbers = ActiveParameterNames.FindOrAdd(get<0>(ParameterNameAndNumber));
		while (ActiveParameterNumbers.Contains(ParameterNumber))
		{
			++ParameterNumber;
		}
		ActiveParameterNumbers.Add(ParameterNumber);

		// If the parameter number has changed, update the parameter name.
		if (ParameterNumber != get<1>(ParameterNameAndNumber))
		{
			CollectionParameter.ParameterName = FName(*FString::Printf(TEXT("%s%u"), *get<0>(ParameterNameAndNumber), ParameterNumber));
		}
	}
}

bool UMaterialParameterCollection::UpdateOverrides(UMaterialParameterCollection* BaseCollection)
{
	if (!Base || (Base != BaseCollection && !Base->UpdateOverrides(BaseCollection)))
	{
		// Indicate that this collection is not based on the BaseCollection.
		return false;
	}

	// Ensure the collection's parameter names account for the base.
	SanitizeParameters(&UMaterialParameterCollection::ScalarParameters);
	SanitizeParameters(&UMaterialParameterCollection::VectorParameters);

	// If the BaseStateId is up-to-date, the base overrides should be as well.
	if (BaseStateId == Base->StateId)
	{
		return true;
	}

	BaseStateId = Base->StateId;

	// Generate a new Id so that unloaded materials that reference this collection will update correctly on load.
	// All materials that reference this collection must be recompiled.
	StateId = FGuid::NewGuid();

	// Remove missing scalar parameter overrides.
	TMap<FGuid, float> ScalarParameterBaseOverridesCache;
	Swap(ScalarParameterBaseOverridesCache, ScalarParameterBaseOverrides);
	ScalarParameterBaseOverrides.Empty(ScalarParameterBaseOverridesCache.Num());
	for (auto& Entry : ScalarParameterBaseOverridesCache)
	{
		if (Base->GetParameterName(Entry.Key) != NAME_None)
		{
			ScalarParameterBaseOverrides.Add(Entry);
		}
	}

	// Remove missing vector parameter overrides.
	TMap<FGuid, FLinearColor> VectorParameterBaseOverridesCache;
	Swap(VectorParameterBaseOverridesCache, VectorParameterBaseOverrides);
	VectorParameterBaseOverrides.Empty(VectorParameterBaseOverridesCache.Num());
	for (auto& Entry : VectorParameterBaseOverridesCache)
	{
		if (Base->GetParameterName(Entry.Key) != NAME_None)
		{
			VectorParameterBaseOverrides.Add(Entry);
		}
	}

	// Indicate that this collection is based on the BaseCollection.
	return true;
}

int32 UMaterialParameterCollection::GetScalarParameterIndexByName(FName ParameterName) const
{
	// loop over all the available scalar parameters and look for a name match
	for (int32 ParameterIndex = 0; ParameterIndex < ScalarParameters.Num(); ParameterIndex++)
	{
		if(ScalarParameters[ParameterIndex].ParameterName == ParameterName)
		{
			return ParameterIndex;
		}
	}
	// if not found, return -1
	return -1;
}

int32 UMaterialParameterCollection::GetVectorParameterIndexByName(FName ParameterName) const
{
	// loop over all the available vector parameters and look for a name match
	for (int32 ParameterIndex = 0; ParameterIndex < VectorParameters.Num(); ParameterIndex++)
	{
		if(VectorParameters[ParameterIndex].ParameterName == ParameterName)
		{
			return ParameterIndex;
		}
	}
	// if not found, return -1
	return -1;
}

TArray<FName> UMaterialParameterCollection::GetScalarParameterNames() const
{
	TArray<FName> Names;
	GetParameterNames(Names, false);
	return Names;
}

TArray<FName> UMaterialParameterCollection::GetVectorParameterNames() const
{
	TArray<FName> Names;
	GetParameterNames(Names, true);
	return Names;
}

float UMaterialParameterCollection::GetScalarParameterDefaultValue(FName ParameterName, bool& bParameterFound) const
{
	for (const UMaterialParameterCollection* BaseCollection = this; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		// Search each base collection's scalar parameters for the named parameter.
		for (int32 ParameterIndex = 0; ParameterIndex < BaseCollection->ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& CollectionParameter = BaseCollection->ScalarParameters[ParameterIndex];
			if (CollectionParameter.ParameterName == ParameterName)
			{
				bParameterFound = true;

				// Search each derived collection's overrides for the parameter's id.
				for (const UMaterialParameterCollection* OverrideCollection = this; OverrideCollection != BaseCollection; OverrideCollection = OverrideCollection->Base)
				{
					if (const float* OverrideValue = OverrideCollection->ScalarParameterBaseOverrides.Find(CollectionParameter.Id))
					{
						// Use the overridden value.
						return *OverrideValue;
					}
				}

				// Use the default value.
				return CollectionParameter.DefaultValue;
			}
		}
	}

	bParameterFound = false;
	return 0.f;
}

FLinearColor UMaterialParameterCollection::GetVectorParameterDefaultValue(FName ParameterName, bool& bParameterFound) const
{
	for (const UMaterialParameterCollection* BaseCollection = this; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		// Search each base collection's vector parameters for the named parameter.
		for (int32 ParameterIndex = 0; ParameterIndex < BaseCollection->VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& CollectionParameter = BaseCollection->VectorParameters[ParameterIndex];
			if (CollectionParameter.ParameterName == ParameterName)
			{
				bParameterFound = true;

				// Search each derived collection's overrides for the parameter's id.
				for (const UMaterialParameterCollection* OverrideCollection = this; OverrideCollection != BaseCollection; OverrideCollection = OverrideCollection->Base)
				{
					if (const FLinearColor* OverrideValue = OverrideCollection->VectorParameterBaseOverrides.Find(CollectionParameter.Id))
					{
						// Use the overridden value.
						return *OverrideValue;
					}
				}

				// Use the default value.
				return CollectionParameter.DefaultValue;
			}
		}
	}

	bParameterFound = false;
	return FLinearColor::Black;
}

UMaterialParameterCollection* UMaterialParameterCollection::GetBaseParameterCollection() const
{
	return Base;
}

FName UMaterialParameterCollection::GetParameterName(const FGuid& Id) const
{
	for (const UMaterialParameterCollection* Collection = this; Collection; Collection = Collection->Base)
	{
		// Search each base collection's scalar parameters for the parameter's id.
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& CollectionParameter = Collection->ScalarParameters[ParameterIndex];
			if (CollectionParameter.Id == Id)
			{
				return CollectionParameter.ParameterName;
			}
		}

		// Search each base collection's vector parameters for the parameter's id.
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& CollectionParameter = Collection->VectorParameters[ParameterIndex];
			if (CollectionParameter.Id == Id)
			{
				return CollectionParameter.ParameterName;
			}
		}
	}

	return NAME_None;
}

FGuid UMaterialParameterCollection::GetParameterId(FName ParameterName) const
{
	for (const UMaterialParameterCollection* Collection = this; Collection; Collection = Collection->Base)
	{
		// Search each base collection's scalar parameters for the named parameter.
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& CollectionParameter = Collection->ScalarParameters[ParameterIndex];
			if (CollectionParameter.ParameterName == ParameterName)
			{
				return CollectionParameter.Id;
			}
		}

		// Search each base collection's vector parameters for the named parameter.
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& CollectionParameter = Collection->VectorParameters[ParameterIndex];
			if (CollectionParameter.ParameterName == ParameterName)
			{
				return CollectionParameter.Id;
			}
		}
	}

	return FGuid();
}

void UMaterialParameterCollection::GetParameterIndex(const FGuid& Id, int32& OutIndex, int32& OutComponentIndex) const
{
	OutIndex = -1;
	OutComponentIndex = -1;

	int32 ScalarParameterBase = GetTotalVectorStorage();
	for (const UMaterialParameterCollection* Collection = this; Collection; Collection = Collection->Base)
	{
		// Find this collection's scalar and vector parameter offsets into the vector storage.
		int32 VectorParameterBase = ScalarParameterBase - Collection->VectorParameters.Num();
		ScalarParameterBase = VectorParameterBase - FMath::DivideAndRoundUp(Collection->ScalarParameters.Num(), 4);

		// Search the scalar parameters for the parameter id.
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& CollectionParameter = Collection->ScalarParameters[ParameterIndex];
			if (CollectionParameter.Id == Id)
			{
				// Scalar parameters are packed into float4's, so derive the indices accordingly.
				OutIndex = ScalarParameterBase + ParameterIndex / 4;
				OutComponentIndex = ParameterIndex % 4;
				return;
			}
		}

		// Search the vector parameters for the parameter id.
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& CollectionParameter = Collection->VectorParameters[ParameterIndex];
			if (CollectionParameter.Id == Id)
			{
				// Vector parameters don't use the component index.
				OutIndex = VectorParameterBase + ParameterIndex;
				return;
			}
		}
	}
}

int32 UMaterialParameterCollection::GetTotalVectorStorage() const
{
	// Sum the vector-aligned storage required for each collection's scalar and vector parameters.
	int32 NumVectors = FMath::DivideAndRoundUp(ScalarParameters.Num(), 4) + VectorParameters.Num();
	for (const UMaterialParameterCollection* Collection = Base; Collection; Collection = Collection->Base)
	{
		NumVectors += FMath::DivideAndRoundUp(Collection->ScalarParameters.Num(), 4) + Collection->VectorParameters.Num();
	}
	UE_CLOGF(NumVectors > GMaterialParameterCollectionMaxVectorStorage, LogMaterial, Warning,
		"'%ls' requires more than the maximum configured number of vectors of storage (%u) specified by the 'r.MPC.MaxVectorStorage' cvar.",
		*GetPathName(), GMaterialParameterCollectionMaxVectorStorage);
	return NumVectors;
}

void UMaterialParameterCollection::GetParameterNames(TArray<FName>& OutParameterNames, bool bVectorParameters) const
{
	if (Base)
	{
		// Add base collection parameter names.
		Base->GetParameterNames(OutParameterNames, bVectorParameters);
	}

	if (bVectorParameters)
	{
		// Add each vector parameter name.
		for (int32 ParameterIndex = 0; ParameterIndex < VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& Parameter = VectorParameters[ParameterIndex];
			OutParameterNames.Add(Parameter.ParameterName);
		}
	}
	else
	{
		// Add each scalar parameter name.
		for (int32 ParameterIndex = 0; ParameterIndex < ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& Parameter = ScalarParameters[ParameterIndex];
			OutParameterNames.Add(Parameter.ParameterName);
		}
	}
}

const FCollectionScalarParameter* UMaterialParameterCollection::GetScalarParameterByName(FName ParameterName) const
{
	// Search each collection's scalar parameters for the named parameter.
	for (const UMaterialParameterCollection* Collection = this; Collection; Collection = Collection->Base)
	{
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& CollectionParameter = Collection->ScalarParameters[ParameterIndex];
			if (CollectionParameter.ParameterName == ParameterName)
			{
				return &CollectionParameter;
			}
		}
	}

	return nullptr;
}

const FCollectionVectorParameter* UMaterialParameterCollection::GetVectorParameterByName(FName ParameterName) const
{
	// Search each collection's vector parameters for the named parameter.
	for (const UMaterialParameterCollection* Collection = this; Collection; Collection = Collection->Base)
	{
		for (int32 ParameterIndex = 0; ParameterIndex < Collection->VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& CollectionParameter = Collection->VectorParameters[ParameterIndex];
			if (CollectionParameter.ParameterName == ParameterName)
			{
				return &CollectionParameter;
			}
		}
	}

	return nullptr;
}

void UMaterialParameterCollection::CreateBufferStruct()
{	
	if (UNLIKELY(!FApp::CanEverRenderOrProduceRenderData()))
	{
		return;
	}

	TArray<FShaderParametersMetadata::FMember> Members;
	uint32 NextMemberOffset = 0;
	
	const uint32 NumVectors = GetTotalVectorStorage();
	new(Members) FShaderParametersMetadata::FMember(TEXT("Vectors"),TEXT(""),__LINE__,NextMemberOffset,UBMT_FLOAT32,EShaderPrecisionModifier::Half,1,4,NumVectors, nullptr);
	const uint32 VectorArraySize = NumVectors * sizeof(FVector4f);
	NextMemberOffset += VectorArraySize;
	const uint32 StructSize = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);

	// If Collections ever get non-numeric resources (eg Textures), OutEnvironment.ResourceTableMap has a map by name
	// and the N ParameterCollection Uniform Buffers ALL are named "MaterialCollection" with different hashes!
	// (and the hlsl cbuffers are named MaterialCollection0, etc, so the names don't match the layout)
	UniformBufferStruct = MakeUnique<FShaderParametersMetadata>(
		FShaderParametersMetadata::EUseCase::DataDrivenUniformBuffer,
		EUniformBufferBindingFlags::Shader,
		TEXT("MaterialCollection"),
		TEXT("MaterialCollection"),
		TEXT("MaterialCollection"),
		nullptr,
		__FILE__,
		__LINE__,
		StructSize,
		Members
		);
}

void UMaterialParameterCollection::GetDefaultParameterData(TArray<FVector4f>& ParameterData) const
{
	GetParameterData(ParameterData, nullptr, nullptr);
}

void UMaterialParameterCollection::GetParameterData(TArray<FVector4f>& ParameterData, const TMap<FName, float>* ScalarParameterInstanceOverrides, const TMap<FName, FLinearColor>* VectorParameterInstanceOverrides) const
{
	// The memory layout created here must match the index assignment in UMaterialParameterCollection::GetParameterIndex

	// Allocate the vector storage.
	ParameterData.SetNum(GetTotalVectorStorage());

	int32 ScalarParameterBase = ParameterData.Num();
	for (const UMaterialParameterCollection* BaseCollection = this; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		// Find this collection's scalar and vector parameter offsets into the vector storage.
		int32 VectorParameterBase = ScalarParameterBase - BaseCollection->VectorParameters.Num();
		ScalarParameterBase = VectorParameterBase - FMath::DivideAndRoundUp(BaseCollection->ScalarParameters.Num(), 4);

		// Set the value of each scalar parameter in this collection.
		for (int32 ParameterIndex = 0; ParameterIndex < BaseCollection->ScalarParameters.Num(); ++ParameterIndex)
		{
			const FCollectionScalarParameter& CollectionParameter = BaseCollection->ScalarParameters[ParameterIndex];
			FVector4f& CurrentVector = ParameterData[ScalarParameterBase + ParameterIndex / 4];

			// Pack into the appropriate component of this packed vector
			float& CurrentValue = CurrentVector[ParameterIndex % 4];

			// Determine if scalar instance parameters are available.
			if (ScalarParameterInstanceOverrides)
			{
				// Search the instance's scalar parameters for the named parameter.
				if (const float* OverrideValue = ScalarParameterInstanceOverrides->Find(CollectionParameter.ParameterName))
				{
					// Use the instance's parameter value.
					CurrentValue = *OverrideValue;
					continue;
				}
			}

			// Speculatively set the parameter value to the default.
			CurrentValue = CollectionParameter.DefaultValue;

			// Search the derived collections' scalar parameter overrides for the parameter id.
			for (const UMaterialParameterCollection* OverrideCollection = this; OverrideCollection != BaseCollection; OverrideCollection = OverrideCollection->Base)
			{
				if (const float* OverrideValue = OverrideCollection->ScalarParameterBaseOverrides.Find(CollectionParameter.Id))
				{
					// Use the overridden parameter value.
					CurrentValue = *OverrideValue;
					break;
				}
			}
		}

		// Set the value of each vector parameter in this collection.
		for (int32 ParameterIndex = 0; ParameterIndex < BaseCollection->VectorParameters.Num(); ++ParameterIndex)
		{
			const FCollectionVectorParameter& CollectionParameter = BaseCollection->VectorParameters[ParameterIndex];

			FVector4f& CurrentValue = ParameterData[VectorParameterBase + ParameterIndex];

			// Determine if vector instance parameters are available.
			if (VectorParameterInstanceOverrides)
			{
				// Search the instance's vector parameters for the named parameter.
				if (const FLinearColor* OverrideValue = VectorParameterInstanceOverrides->Find(CollectionParameter.ParameterName))
				{
					// Use the instance's parameter value.
					CurrentValue = *OverrideValue;
					continue;
				}
			}

			// Speculatively set the parameter value to the default.
			CurrentValue = CollectionParameter.DefaultValue;

			// Search the derived collections' vector parameter overrides for the parameter id.
			for (const UMaterialParameterCollection* OverrideCollection = this; OverrideCollection != BaseCollection; OverrideCollection = OverrideCollection->Base)
			{
				if (const FLinearColor* OverrideValue = OverrideCollection->VectorParameterBaseOverrides.Find(CollectionParameter.Id))
				{
					// Use the overridden parameter value.
					CurrentValue = *OverrideValue;
					break;
				}
			}
		}
	}
}

void UMaterialParameterCollection::UpdateDefaultResource(bool bRecreateUniformBuffer)
{
	if (UNLIKELY(!FApp::CanEverRender()))
	{
		return;
	}

	// Propagate the new values to the rendering thread
	TArray<FVector4f> ParameterData;
	GetDefaultParameterData(ParameterData);
	DefaultResource->GameThread_UpdateContents(StateId, ParameterData, GetFName(), bRecreateUniformBuffer);

	FGuid Id = StateId;
	FMaterialParameterCollectionInstanceResource* Resource = DefaultResource;
	ENQUEUE_RENDER_COMMAND(UpdateDefaultResourceCommand)(
		[Id, Resource](FRHICommandListImmediate& RHICmdList)
		{
			// Async RDG tasks can call FMaterialShader::SetParameters which touch material parameter collections.
			FRDGBuilder::WaitForAsyncExecuteTask();
			GDefaultMaterialParameterCollectionInstances.Add(Id, Resource);
		}
	);
}

UMaterialParameterCollectionInstance::UMaterialParameterCollectionInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Resource = nullptr;
	bNeedsRenderStateUpdate = false;
}

void UMaterialParameterCollectionInstance::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject) && FApp::CanEverRender())
	{
		Resource = new FMaterialParameterCollectionInstanceResource();
	}
}

void UMaterialParameterCollectionInstance::SetCollection(UMaterialParameterCollection* InCollection, UWorld* InWorld)
{
	Collection = InCollection;
	World = InWorld;
}

bool UMaterialParameterCollectionInstance::SetScalarParameterValue(FName ParameterName, float ParameterValue)
{
	if (!World.IsValid())
	{
		return false;
	}

	check(Collection.IsValid());

	if (Collection->GetScalarParameterByName(ParameterName))
	{
		float* ExistingValue = ScalarParameterValues.Find(ParameterName);
		bool bUpdateUniformBuffer = false;

		if (ExistingValue && *ExistingValue != ParameterValue)
		{
			// Update the existing instance override if the new value is different
			bUpdateUniformBuffer = true;
			*ExistingValue = ParameterValue;
		}
		else if (!ExistingValue)
		{
			// Add a new instance override
			bUpdateUniformBuffer = true;
			ScalarParameterValues.Add(ParameterName, ParameterValue);
		}

		if (bUpdateUniformBuffer)
		{
			UpdateRenderState(false);
			ScalarParameterUpdatedDelegate.Broadcast(ScalarParameterUpdate(ParameterName, ParameterValue));
		}

		return true;
	}

	return false;
}

bool UMaterialParameterCollectionInstance::SetVectorParameterValue(FName ParameterName, const FLinearColor& ParameterValue)
{
	if (!World.IsValid())
	{
		return false;
	}

	check(Collection.IsValid());

	if (Collection->GetVectorParameterByName(ParameterName))
	{
		FLinearColor* ExistingValue = VectorParameterValues.Find(ParameterName);
		bool bUpdateUniformBuffer = false;

		if (ExistingValue && *ExistingValue != ParameterValue)
		{
			// Update the existing instance override if the new value is different
			bUpdateUniformBuffer = true;
			*ExistingValue = ParameterValue;
		}
		else if (!ExistingValue)
		{
			// Add a new instance override
			bUpdateUniformBuffer = true;
			VectorParameterValues.Add(ParameterName, ParameterValue);
		}

		if (bUpdateUniformBuffer)
		{
			UpdateRenderState(false);
			VectorParameterUpdatedDelegate.Broadcast(VectorParameterUpdate(ParameterName, ParameterValue));
		}

		return true;
	}

	return false;
}

bool UMaterialParameterCollectionInstance::GetScalarParameterValue(FName ParameterName, float& OutParameterValue) const
{
	if (const FCollectionScalarParameter* Parameter = Collection->GetScalarParameterByName(ParameterName))
	{
		return GetScalarParameterValue(*Parameter, OutParameterValue);
	}

	return false;
}

bool UMaterialParameterCollectionInstance::GetVectorParameterValue(FName ParameterName, FLinearColor& OutParameterValue) const
{
	if (const FCollectionVectorParameter* Parameter = Collection->GetVectorParameterByName(ParameterName))
	{
		return GetVectorParameterValue(*Parameter, OutParameterValue);
	}

	return false;
}

bool UMaterialParameterCollectionInstance::GetScalarParameterValue(const FCollectionScalarParameter& Parameter, float& OutParameterValue) const
{
	const float* InstanceValue = ScalarParameterValues.Find(Parameter.ParameterName);
	OutParameterValue = InstanceValue != nullptr ? *InstanceValue : Parameter.DefaultValue;
	return true;
}

bool UMaterialParameterCollectionInstance::GetVectorParameterValue(const FCollectionVectorParameter& Parameter, FLinearColor& OutParameterValue) const
{
	const FLinearColor* InstanceValue = VectorParameterValues.Find(Parameter.ParameterName);
	OutParameterValue = InstanceValue != nullptr ? *InstanceValue : Parameter.DefaultValue;
	return true;
}

void UMaterialParameterCollectionInstance::UpdateRenderState(bool bRecreateUniformBuffer)
{
	// Don't need material parameters on the server
	if (!World.IsValid() || World->GetNetMode() == NM_DedicatedServer)
	{
		return;
	}

	bNeedsRenderStateUpdate = true;
	World->SetMaterialParameterCollectionInstanceNeedsUpdate();

	if (!GDeferUpdateRenderStates || bRecreateUniformBuffer)
	{
		DeferredUpdateRenderState(bRecreateUniformBuffer);
	}
}

void UMaterialParameterCollectionInstance::DeferredUpdateRenderState(bool bRecreateUniformBuffer)
{
	checkf(bNeedsRenderStateUpdate || !bRecreateUniformBuffer, TEXT("DeferredUpdateRenderState was told to recreate the uniform buffer, but there's nothing to update"));

	if (bNeedsRenderStateUpdate && World.IsValid())
	{
		// Propagate the new values to the rendering thread
		TArray<FVector4f> ParameterData;
		GetParameterData(ParameterData);
		Resource->GameThread_UpdateContents(Collection.IsValid() ? Collection->StateId : FGuid(), ParameterData, GetFName(), bRecreateUniformBuffer);
	}

	bNeedsRenderStateUpdate = false;
}

void UMaterialParameterCollectionInstance::ForceReturnToDefaultValues()
{
	ScalarParameterValues.Empty();
	VectorParameterValues.Empty();
	bool bNeedsUniformBuffer = !(Resource && Resource->GetUniformBuffer() && Resource->GetUniformBuffer()->IsValid());
	UpdateRenderState(bNeedsUniformBuffer);
}

void UMaterialParameterCollectionInstance::GetParameterData(TArray<FVector4f>& ParameterData) const
{
	if (const UMaterialParameterCollection* BaseCollection = Collection.Get())
	{
		BaseCollection->GetParameterData(ParameterData, &ScalarParameterValues, &VectorParameterValues);
	}
}

void UMaterialParameterCollectionInstance::FinishDestroy()
{
	if (Resource)
	{
		Resource->GameThread_Destroy();
		Resource = nullptr;
	}

	Super::FinishDestroy();
}

void FMaterialParameterCollectionInstanceResource::GameThread_UpdateContents(const FGuid& InGuid, const TArray<FVector4f>& Data, const FName& InOwnerName, bool bRecreateUniformBuffer)
{
	if (UNLIKELY(!FApp::CanEverRender()))
	{
		return;
	}

	FMaterialParameterCollectionInstanceResource* Resource = this;
	ENQUEUE_RENDER_COMMAND(UpdateCollectionCommand)(
		[InGuid, Data, InOwnerName, Resource, bRecreateUniformBuffer](FRHICommandListImmediate& RHICmdList)
		{
			if (bRecreateUniformBuffer)
			{
				// Async RDG tasks can call FMaterialShader::SetParameters which touch material parameter collections.
				FRDGBuilder::WaitForAsyncExecuteTask();
			}
			Resource->UpdateContents(InGuid, Data, InOwnerName, bRecreateUniformBuffer);
		}
	);
}

void FMaterialParameterCollectionInstanceResource::GameThread_Destroy()
{
	FMaterialParameterCollectionInstanceResource* Resource = this;
	ENQUEUE_RENDER_COMMAND(DestroyCollectionCommand)(
		[Resource](FRHICommandListImmediate& RHICmdList)
		{
			// Async RDG tasks can call FMaterialShader::SetParameters which touch material parameter collections.
			FRDGBuilder::WaitForAsyncExecuteTask();
			Resource->UniformBuffer.SafeRelease();

			// FRHIUniformBuffer instances take raw pointers to the layout struct.
			// Delete the resource instance (and its layout) on the RHI thread to avoid deleting the layout
			// whilst the RHI is using it, and also avoid having to completely flush the RHI thread.
			RHICmdList.EnqueueLambda([Resource](FRHICommandListImmediate&)
			{
				delete Resource;
			});
		}
	);
}

FMaterialParameterCollectionInstanceResource::FMaterialParameterCollectionInstanceResource() = default;

FMaterialParameterCollectionInstanceResource::~FMaterialParameterCollectionInstanceResource()
{
	check(!UniformBuffer.IsValid());
}

void FMaterialParameterCollectionInstanceResource::UpdateContents(const FGuid& InId, const TArray<FVector4f>& Data, const FName& InOwnerName, bool bRecreateUniformBuffer)
{
	Id = InId;
	OwnerName = InOwnerName;

	if (InId != FGuid() && Data.Num() > 0)
	{
		const uint32 NewSize = Data.GetTypeSize() * Data.Num();
		check(UniformBufferLayout == nullptr || UniformBufferLayout->Resources.Num() == 0);

		if (!bRecreateUniformBuffer && IsValidRef(UniformBuffer))
		{
			check(NewSize == UniformBufferLayout->ConstantBufferSize);
			check(UniformBuffer->GetLayoutPtr() == UniformBufferLayout);
			FRHICommandListImmediate::Get().UpdateUniformBuffer(UniformBuffer, Data.GetData());
		}
		else
		{
			FRHIUniformBufferLayoutInitializer UniformBufferLayoutInitializer(TEXT("MaterialParameterCollectionInstanceResource"));
			UniformBufferLayoutInitializer.ConstantBufferSize = NewSize;
			UniformBufferLayoutInitializer.ComputeHash();

			UniformBufferLayout = RHICreateUniformBufferLayout(UniformBufferLayoutInitializer);

			UniformBuffer = RHICreateUniformBuffer(Data.GetData(), UniformBufferLayout, UniformBuffer_MultiFrame);
		}
	}
}
