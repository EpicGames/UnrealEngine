// Copyright Epic Games, Inc. All Rights Reserved.

#include "StringTemplate.h"

static FStringView Substring(const FString& String, int Begin, int Length)
{
	return { GetData(String) + Begin, Length};
}

FStringTemplate::FStringTemplate()
{
}

bool FStringTemplate::Load(FString TemplateStringArg, FErrorInfo& OutErrorinfo)
{
	TArray<FChunk> NewChunks;
	int32 ChunkBegin = 0;
	int32 ChunkLength = 0;
	int32 Offset = 0;
	int32 Line = 1;
	TCHAR Char = 0;
	NumNamedParameters = 0;

	OutErrorinfo = {};

	// Local function that pushes the lookahead character to the current chunk
	// and reads the next in the string. If at string end, Char is set to 0.
	auto Next = [&]() -> TCHAR
	{
		++Offset;
		++ChunkLength;
		if (Offset < TemplateStringArg.Len())
		{
			Char = TemplateStringArg[Offset];
		}
		else
		{
			Char = 0;
		}
		return Char;
	};

	// Checks that the lookahead characters is `C` and if so, pushes it to the current
	// chunk and reads the next character. Otherwise it returns false.
	auto Accept = [&](TCHAR C) -> bool
	{
		if (Char == C)
		{
			Next();
			return true;
		}
		return false;
	};

	// Pushes the current chunk substring to the list of chunks and resets the
	// current chunk substring coordinates.
	auto PushChunk = [&](bool bIsParameter)
	{
		NewChunks.Add({ Substring(TemplateStringArg, ChunkBegin, ChunkLength), bIsParameter });
		ChunkBegin = Offset;
		ChunkLength = 0;
	};

	// Read the first character
	Char = TemplateStringArg.IsEmpty() ? 0 : TemplateStringArg[0];

	// Scan the entire string...
	while (Char != 0)
	{
		// Record token line and offset in the error info
		OutErrorinfo.Line = Line;
		OutErrorinfo.Offset = Offset;

		// Check whether this is a parameter
		if (Accept('%'))
		{
			if (Accept('s')) // e.g. %s
			{
				ChunkLength -= 2; // take out the "%s"

				// Push the chunk of text read so far
				if (ChunkLength > 0)
				{
					PushChunk(false);
				}

				// Push an nameless parameter chunk
				NewChunks.Add({ TEXT(""), true});
			}
			else if (Accept('{')) // e.g. ${my_parameter}
			{
				ChunkLength -= 2; // take out the "%{"

				// Push the chunk of text read so far
				if (ChunkLength > 0)
				{
					PushChunk(false);
				}

				// Parse the parameter name
				while (Char != 0 && Char != '}')
				{
					Next();
				}

				// Expect the closing parameter name bracket
				if (!Accept('}'))
				{
					OutErrorinfo.Message = TEXT("missing '}' in named parameter");
					return false;
				}

				ChunkLength -= 1; // take out the "}"

				// Push the parameter name as the next chunk
				PushChunk(true);
				NumNamedParameters += 1;
			}
		}
		else
		{
			Line += (Char == '\n');

			// Not a parameter ahead, push the character to the current chunk
			Next();
		}
	}

	// If there's a trailing chunk of text, push the last chunk
	if (ChunkLength > 0)
	{
		PushChunk(false);
	}

	// Parsing complete and successful
	TemplateString = MoveTemp(TemplateStringArg);
	Chunks = MoveTemp(NewChunks);

	return true;

}

void FStringTemplate::GetParameters(TArray<FStringView>& OutParams) const
{
	OutParams.Reserve(NumNamedParameters);
	for (const FChunk& Chunk : Chunks)
	{
		if (Chunk.bIsParameter && Chunk.Text.Len() > 0)
		{
			OutParams.Push(Chunk.Text);
		}
	}
}

FStringTemplateResolver::FStringTemplateResolver(const FStringTemplate& Template, uint32 ResolvedStringSizeHint)
: Template{ Template }
{
	ResolvedString.Reserve(ResolvedStringSizeHint);
}

void FStringTemplateResolver::Advance(FStringView NextNamelessParameterValue)
{
	// Process all chunks until a nameless parameter is encountered (inclusively)
	for (; ChunkIndex < Template.Chunks.Num(); ++ChunkIndex)
	{
		const FStringTemplate::FChunk& Chunk = Template.Chunks[ChunkIndex];
		if (Chunk.bIsParameter)
		{
			if (Chunk.Text.Len() > 0)
			{
				// This is a named parameter, resolve it.
				const FString* ParameterValue = NamedParameters ? NamedParameters->FindByHash(GetTypeHash(Chunk.Text), Chunk.Text) : nullptr;
				if (ParameterValue)
				{
					ResolvedString += *ParameterValue;
				}
			}
			else
			{
				// This is a nameless parameter
				ResolvedString += NextNamelessParameterValue;
				++ChunkIndex;

				// Quit here, the rest of the chunks will be resolved by further Advance() calls and by Finalize()
				return;
			}
		}
		else
		{
			// Push the static chunk of text
			ResolvedString += Chunk.Text;
		}
	}
}

FString FStringTemplateResolver::Finalize(uint32 NewResolvedStringSizeHint)
{
	// Finish processing the remaining chunks, using an empty string for the remaining nameless parameters.
	while (ChunkIndex < Template.Chunks.Num())
	{
		Advance(TEXT(""));
	}

	ChunkIndex = 0;
	return MoveTemp(ResolvedString);
}
