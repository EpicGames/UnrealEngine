// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Containers/StringView.h"
#include "Containers/Array.h"
#include "Containers/Map.h"

/**
 * A string template is a utility class used to generate parameterized strings.
 * To use it, initialize the template by loading a template string with the
 * constructor or manually calling Load() after construction.
 * The Load() function will parse the template string and split it into chunks,
 * where each chunk is either a static span of text or a parameter. After a
 * template has been loaded, resolve the full string using an instance of
 * FStringTemplateResolver.
 * 
 * Parameters can be specifed *named* and *nameless* depending on your needs.
 * A named parameter can be specified with the syntax `%{parameter_name}` while
 * a nameless parameter with `%s` (without backquotes). A string can include both
 * type of parameters. Keep in mind that nameless parameters will be resolved in
 * the order they appear in the template string.
 * 
 * Here is an example of a valid template string:
 *
 *		The last president of the USA is %{name_of_president}.
 *		He was born in %s.
 * 
 * Finally, use may force the emission of the '%' character using the "%%" escape sequence.
 */
class FStringTemplate
{
public:
	/**
	 * Wraps info about where an error in the template string is.
	 */
	struct FErrorInfo
	{
		/** A user friendly textual description of the error */
		FStringView Message;

		/** Which line contains the error */
		int32 Line;

		/** At what offset in characters from the start */
		int32 Offset;
	};

public:
	/**
	 * Constructs an empty string template.
	 */
	FStringTemplate();

	/**
	 * Loads the specified template string for future string resolutions with Resolve().
	 * Returns: whether parsing the string was completed successfully.
	 * Note: `TemplateString` is passed by value as it will be stored within this instance.
	 *        Consider using MoveTemp() to pass ownership of the template string if you
	 *        no longer need it.
	 */
	bool Load(FString TemplateString, FErrorInfo& Errorinfo);

	/**
	 * Returns the template string.
	 */
	const FString& GetTemplateString() const { return TemplateString; }

	/**
	 * Returns the number of named parameters in this template string, useful to reserve
	 * enough capacity in your parameters map.
	 */
	int32 GetNumNamedParameters() const { return NumNamedParameters; }

	/**
	 * Returns the array of named parameters contained in this template.
	 */
	void GetParameters(TArray<FStringView>& OutParams) const;

private:
	/** Information about a template string chunk (a static text or a parameter) */
	struct FChunk
	{
		/** Static text or name of the parameter */
		FStringView Text;

		/** Whether this is a parameter */
		bool bIsParameter;
	};

	/** The loaded and parsed template string */
	FString TemplateString;

	/** Chunks making out the loaded template string */
	TArray<FChunk> Chunks;

	/** The number of named parameters in this template string */
	int32 NumNamedParameters = 0;

	friend class FStringTemplateResolver;
};

/**
 * It provides the mechanism to resolve a FStringTemplate instance into a string.
 * 
 * To resolve a template into a string, instantiate this class then set all the
 * named parameters with SetParameterMap() _before_ calling Advance() or Finalize().
 * 
 * Once all named parameters have been set (or if you wish to set no named parameters),
 * call Advance() repeatedly, passing in the value of the next nameless parameter. Call
 * Advance() for each nameless parameter you wish to replace.
 * 
 * When you have no more nameless parameters to use or if you have none at all, call
 * Finalize() to complete resolving the string which will be returned by it.
 */
class FStringTemplateResolver
{
public:
	/**
	 * Constructs the resolver using specified `Template`.
	 * You can optionally provide a value for `ResolvedStringSizeHint` to be used to reserve
	 * space in the resolved string.
	 */
	FStringTemplateResolver(const FStringTemplate& Template, uint32 ResolvedStringSizeHint = 0);

	/**
	 * Returns the template this resolver is using.
	 */
	const FStringTemplate& GetTemplate() const { return Template; }

	/**
	 * Sets the map of named parameters.
	 * Note: make sure all named parameters are set before calling Advance() or Finalize() otherwise
	 *       you may get an incorrect resolved string.
	 */
	void SetParameterMap(const TMap<FString, FString>* Parameters)
	{
		NamedParameters = Parameters;
	}

	/** 
	 * Generates the next section of the string until a nameless parameter is encountered for which
	 * `NextNamelessParameterValue` is used as the replaced value. Call Advance() for each nameless
	 * parameter you need to push.
	 */
	void Advance(FStringView NextNamelessParameterValue);

	/**
	 * Completes generating the string and returns it.
	 * Note: Any subsequent and unresolved nameless parameter will be replaced to the empty string.
	 */
	FString Finalize(uint32 ResolvedStringSizeHint = 0);
	
private:
	/** The template to use */
	const FStringTemplate& Template;

	/** Index of the next chunk to process */
	int32 ChunkIndex = 0;

	/** Map of named parameters key-values */
	const TMap<FString, FString>* NamedParameters = nullptr;

	/** The resolved string being generated */
	FString ResolvedString;
};
