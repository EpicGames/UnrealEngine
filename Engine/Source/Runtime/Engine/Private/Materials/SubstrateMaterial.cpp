// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubstrateMaterial.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialInterface.h"
#include "SubstrateDefinitions.h"
#include "RenderUtils.h"

FString GetSubstrateBSDFName(uint8 BSDFType)
{
	switch (BSDFType)
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:
		return TEXT("SLAB");
	case SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD:
		return TEXT("VOLUMETRICFOGCLOUD");
	case SUBSTRATE_BSDF_TYPE_UNLIT:
		return TEXT("UNLIT");
	case SUBSTRATE_BSDF_TYPE_HAIR:
		return TEXT("HAIR");
	case SUBSTRATE_BSDF_TYPE_EYE:
		return TEXT("EYE");
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		return TEXT("SINGLELAYERWATER");
	case SUBSTRATE_BSDF_TYPE_TOON:
		return TEXT("TOON");
	}
	check(false);
	return "";
}

FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoCreateNullSharedLocalBasis()
{
	return FSubstrateRegisteredSharedLocalBasis();
}

FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoCreateSharedLocalBasis(FMaterialCompiler* Compiler, int32 NormalCodeChunk, int32 TangentCodeChunk)
{
	if (TangentCodeChunk == INDEX_NONE)
	{
		return Compiler->SubstrateCompilationInfoRegisterSharedLocalBasis(NormalCodeChunk);
	}
	return Compiler->SubstrateCompilationInfoRegisterSharedLocalBasis(NormalCodeChunk, TangentCodeChunk);
}

namespace SubstrateMaterialPrivate
{
	inline bool IsSubstrateEnabled()
	{
		static bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
		return bSubstrateEnabled;
	}
}

#define IsGenericBlendMode_Conversion0(Name, LegacyCondition) \
	bool Is##Name##BlendMode(EBlendMode BlendMode)											{ return LegacyCondition; } \
	bool Is##Name##BlendMode(const FMaterial& In)											{ return Is##Name##BlendMode(In.GetBlendMode()); } \
	bool Is##Name##BlendMode(const UMaterialInterface& In)									{ return Is##Name##BlendMode(In.GetBlendMode()); } \
	bool Is##Name##BlendMode(const FMaterialShaderParameters& In)							{ return Is##Name##BlendMode(In.BlendMode); }

#define IsGenericBlendMode_Conversion1(Name, LegacyCondition, SubstrateCondition) \
	bool Is##Name##BlendMode(EBlendMode BlendMode)											{ return SubstrateMaterialPrivate::IsSubstrateEnabled() ? (SubstrateCondition) : (LegacyCondition); } \
	bool Is##Name##BlendMode(const FMaterial& In)											{ return Is##Name##BlendMode(In.GetBlendMode()); } \
	bool Is##Name##BlendMode(const UMaterialInterface& In)									{ return Is##Name##BlendMode(In.GetBlendMode()); } \
	bool Is##Name##BlendMode(const FMaterialShaderParameters& In)							{ return Is##Name##BlendMode(In.BlendMode); }

IsGenericBlendMode_Conversion0(Opaque,			BlendMode == BLEND_Opaque)																												// Opaque blend mode
IsGenericBlendMode_Conversion0(Masked,			BlendMode == BLEND_Masked)																												// Masked blend mode
IsGenericBlendMode_Conversion0(OpaqueOrMasked,	BlendMode == BLEND_Opaque || BlendMode == BLEND_Masked)																					// Opaque or Masked blend mode
IsGenericBlendMode_Conversion0(Translucent,		BlendMode != BLEND_Opaque && BlendMode != BLEND_Masked)																					// General translucency (i.e., blend mode is something else than Opaque/Masked)
IsGenericBlendMode_Conversion1(TranslucentOnly, BlendMode == BLEND_Translucent, BlendMode == BLEND_TranslucentColoredTransmittance || BlendMode == BLEND_TranslucentGreyTransmittance) 	// Explicit translucency blend mode
IsGenericBlendMode_Conversion0(AlphaHoldout,	BlendMode == BLEND_AlphaHoldout)																										// AlphaHoldout blend mode
IsGenericBlendMode_Conversion1(Modulate,		BlendMode == BLEND_Modulate, BlendMode == BLEND_ColoredTransmittanceOnly)																// Modulate blend mode
IsGenericBlendMode_Conversion0(Additive,		BlendMode == BLEND_Additive)																											// Additive blend mode
IsGenericBlendMode_Conversion0(AlphaComposite,	BlendMode == BLEND_AlphaComposite)																										// AlphaComposite blend mode