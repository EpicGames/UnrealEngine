// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "MaterialCompiler.h"
#include "SubstrateDefinitions.h"

class FMaterialCompiler;

FString GetSubstrateBSDFName(uint8 BSDFType);

FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoCreateNullSharedLocalBasis();
FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoCreateSharedLocalBasis(FMaterialCompiler* Compiler, int32 NormalCodeChunk, int32 TangentCodeChunk = INDEX_NONE);

struct FSubstrateMaterialComplexity
{
	bool bIsComplexSpecial;
	bool bIsSingle;
	bool bIsSimple;
	// If all the above are false, complexity is Complex

	void Reset()
	{
		bIsComplexSpecial = false;
		bIsSingle = false;
		bIsSimple = false;
	}

	uint8 SubstrateMaterialType() const
	{
		return bIsSimple ? SUBSTRATE_MATERIAL_TYPE_SIMPLE : (bIsSingle ? SUBSTRATE_MATERIAL_TYPE_SINGLE : (bIsComplexSpecial ? SUBSTRATE_MATERIAL_TYPE_COMPLEX_SPECIAL : SUBSTRATE_MATERIAL_TYPE_COMPLEX));
	}

	bool IsSimple() const			{ return SubstrateMaterialType() == SUBSTRATE_MATERIAL_TYPE_SIMPLE; }
	bool IsSingle() const			{ return SubstrateMaterialType() == SUBSTRATE_MATERIAL_TYPE_SINGLE; }
	bool IsComplex() const			{ return SubstrateMaterialType() == SUBSTRATE_MATERIAL_TYPE_COMPLEX; }
	bool IsComplexSpecial() const 	{ return SubstrateMaterialType() == SUBSTRATE_MATERIAL_TYPE_COMPLEX_SPECIAL; }

	static FString ToString(uint8 InMaterialType, bool bUpperCase = false)
	{
		switch (InMaterialType)
		{
			case SUBSTRATE_MATERIAL_TYPE_SIMPLE: 			return bUpperCase ? TEXT("SIMPLE")  		: TEXT("Simple");
			case SUBSTRATE_MATERIAL_TYPE_SINGLE: 			return bUpperCase ? TEXT("SINGLE")  		: TEXT("Single");
			case SUBSTRATE_MATERIAL_TYPE_COMPLEX: 			return bUpperCase ? TEXT("COMPLEX") 		: TEXT("Complex");
			case SUBSTRATE_MATERIAL_TYPE_COMPLEX_SPECIAL: 	return bUpperCase ? TEXT("COMPLEX SPECIAL") : TEXT("Complex Special");
			default: 										return bUpperCase ? TEXT("UNKOWN") 			: TEXT("Unknown");
		}
	}
};