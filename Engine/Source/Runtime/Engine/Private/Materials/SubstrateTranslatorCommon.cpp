// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SubstrateTranslatorCommon.h"
#include "HLSLMaterialTranslator.h"
#include "Materials/MaterialIREmitter.h"
#include "MaterialShared.h"
#include "Materials/MaterialExpressionClearCoatNormalCustomOutput.h"
#include "Materials/MaterialExpressionTangentOutput.h"
#include "Materials/MaterialExpressionSubstrate.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "MaterialDomain.h"
#include "Engine/EngineTypes.h"
#include "Materials/MaterialExpressionStaticSwitchParameter.h"

#include <functional>

namespace Substrate
{


FString GetParametersSharedLocalBasesName(ESubstrateCompilationContext CompilationContextIndex)
{
	switch (CompilationContextIndex)
	{
	case ESubstrateCompilationContext::SCC_Default:
		return TEXT("SharedLocalBases");
	case ESubstrateCompilationContext::SCC_FullySimplified:
		return TEXT("SharedLocalBasesFullySimplified");
	}
	check(false);
	return TEXT("ERROR");
}

FString GetParametersSubstrateTreeName(ESubstrateCompilationContext CompilationContextIndex)
{
	switch (CompilationContextIndex)
	{
	case ESubstrateCompilationContext::SCC_Default:
		return TEXT("SubstrateTree");
	case ESubstrateCompilationContext::SCC_FullySimplified:
		return TEXT("SubstrateTreeFullySimplified");
	}
	check(false);
	return TEXT("ERROR");
}


FString GetSubstrateSharedLocalBasisIndexMacroInner(const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis, uint32 Mode)
{
#if WITH_EDITOR
	return FString::Printf(TEXT("SHAREDLOCALBASIS_INDEX_%u_%u"), SharedLocalBasis.GraphSharedLocalBasisIndex, Mode);
#else
	return TEXT("");
#endif
}

const TCHAR* GetSubstrateOperatorStr(int32 OperatorType)
{
	switch (OperatorType)
	{
	case SUBSTRATE_OPERATOR_WEIGHT:
	{
		return TEXT("WEIGHT    ");
	}
	case SUBSTRATE_OPERATOR_VERTICAL:
	{
		return TEXT("VERTICAL  ");
	}
	case SUBSTRATE_OPERATOR_HORIZONTAL:
	{
		return TEXT("HORIZONTAL");
	}
	case SUBSTRATE_OPERATOR_ADD:
	{
		return TEXT("ADD       ");
	}
	case SUBSTRATE_OPERATOR_SELECT:
	{
		return TEXT("SELECT    ");
	}
	case SUBSTRATE_OPERATOR_BSDF:
	{
		return TEXT("BSDF      ");
	}
	case SUBSTRATE_OPERATOR_BSDF_LEGACY:
	{
		return TEXT("BSDFLEGACY");
	}
	}
	return TEXT("UNKNOWN   ");
};



/**
 * This function is temporary. It's only been duplicated for validation purposes and it will be removed in a few days when the latest changes settle without problems.
 */
void FSubstrateCompilationContext::SubstrateEvaluateSharedLocalBases(
	uint8& OutRequestedSharedLocalBasesCount,
	FSubstrateDefines* OutSubstrateDefines,
	MIR::FEmitter* Em)
{
	/*
	* The final output code/workflow for shared tangent basis should look like
	*
	* #define SHAREDLOCALBASIS_INDEX_0 0		// default, unused
	* #define SHAREDLOCALBASIS_INDEX_1 0		// default, unused
	* #define SHAREDLOCALBASIS_INDEX_2 0
	*
	* FSubstrateData BSDF0 = GetSubstrateSlabBSDF(... SHAREDLOCALBASIS_0, NormalCode0 ...)
	* FSubstrateData BSDF1 = GetSubstrateSlabBSDF(... SHAREDLOCALBASIS_1, NormalCode1 ...)
	*
	* float3 NormalCode2 = lerp(NormalCode0, NormalCode1, mix)
	* FSubstrateData BSDF2 = SubstrateHorizontalMixingParameterBlending(BSDF0, BSDF1, mix, NormalCode2, SHAREDLOCALBASIS_INDEX_2, SharedLocalBases.Types) // will internally create NormalCode2
	*
	* tParameters.SharedLocalBases.Normals[SHAREDLOCALBASIS_INDEX_2] = NormalCode2;
	* #if MATERIAL_TANGENTSPACENORMAL
	* Parameters.SharedLocalBases.Normals[SHAREDLOCALBASIS_INDEX_2] *= Parameters.TwoSidedSign;
	* #endif
	*/

	FSubstrateRegisteredSharedLocalBasis UsedSharedLocalBasesInfo[SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS];
	FinalUsedSharedLocalBasesCount = 0;
	OutRequestedSharedLocalBasesCount = 0;
	const FString ParameterSharedLocalBasesName = GetParametersSharedLocalBasesName(CompilationContextIndex);

	if (CompilationContextIndex == SCC_FullySimplified)
	{
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\t#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1\n"));
	}

	for (int32 OpIt = 0; OpIt < SubstrateMaterialExpressionRegisteredOperators.Num(); ++OpIt)
	{
		const FSubstrateOperator& BSDFOperator = SubstrateMaterialExpressionRegisteredOperators[OpIt];
		if (BSDFOperator.BSDFIndex == INDEX_NONE || BSDFOperator.IsDiscarded())
		{
			continue;	// not a BSDF or if discarded (i.e. not the root of a parameter blending subtree), then there is no local basis to register
		}

		if (BSDFOperator.BSDFRegisteredSharedLocalBasis.NormalCodeChunk == INDEX_NONE && BSDFOperator.BSDFRegisteredSharedLocalBasis.TangentCodeChunk == INDEX_NONE)
		{
			continue;	// We skip null normal on certain BSDF, for instance unlit.
		}
		const FSubstrateSharedLocalBasesInfo& SubstrateSharedLocalBasesInfo = SubstrateCompilationInfoGetMatchingSharedLocalBasisInfo(BSDFOperator.BSDFRegisteredSharedLocalBasis);

		// First, we check that the normal/tangent has not already written out (avoid 2 BSDFs sharing the same normal to note generate the same code twice)
		bool bAlreadyProcessed = false;
		for (uint8 i = 0; i < FinalUsedSharedLocalBasesCount; ++i)
		{
			if (UsedSharedLocalBasesInfo[i].NormalCodeChunkHash == SubstrateSharedLocalBasesInfo.SharedData.NormalCodeChunkHash &&
				(UsedSharedLocalBasesInfo[i].TangentCodeChunkHash == SubstrateSharedLocalBasesInfo.SharedData.TangentCodeChunkHash || BSDFOperator.BSDFRegisteredSharedLocalBasis.TangentCodeChunk == INDEX_NONE))
			{
				bAlreadyProcessed = true;
				break;
			}
		}
		if (bAlreadyProcessed)
		{
			continue;
		}

		++OutRequestedSharedLocalBasesCount;
		if (FinalUsedSharedLocalBasesCount >= SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS)
		{
			continue;
		}

		const uint8 FinalSharedLocalBasisIndex = FinalUsedSharedLocalBasesCount++;
		UsedSharedLocalBasesInfo[FinalSharedLocalBasisIndex] = SubstrateSharedLocalBasesInfo.SharedData;

		FString SubstrateSharedNormalParamName;
		FString SubstrateSharedTangentParamName;
		FString SubstrateSharedNormalParamCode;
		FString SubstrateSharedTangentParamCode;
		if (Em)
		{
			SubstrateSharedNormalParamName = FString(TEXT("SubstrateNormal"));

			SubstrateSharedNormalParamName += ParameterSharedLocalBasesName;
			SubstrateSharedNormalParamName.AppendInt(FinalSharedLocalBasisIndex);
			Em->SetCustomOutputs(SubstrateSharedNormalParamName, { &SubstrateSharedLocalBasesInfo.NormalArg, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
			SubstrateSharedNormalParamCode = FString::Printf(TEXT("Parameters.%s0"), *SubstrateSharedNormalParamName);

			if (SubstrateSharedLocalBasesInfo.TangentArg.IsValid())
			{
				SubstrateSharedTangentParamName = FString(TEXT("SubstrateTangent"));

				SubstrateSharedTangentParamName += ParameterSharedLocalBasesName;
				SubstrateSharedTangentParamName.AppendInt(FinalSharedLocalBasisIndex);
				Em->SetCustomOutputs(SubstrateSharedTangentParamName, { &SubstrateSharedLocalBasesInfo.TangentArg, 1 }, MIR::EMaterialOutputFrequency::PerPixel);
				SubstrateSharedTangentParamCode = FString::Printf(TEXT("Parameters.%s0"), *SubstrateSharedTangentParamName);
			}

		}

		// Write out normals
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\tParameters.%s.Normals[%u] = %s;\n"), *ParameterSharedLocalBasesName, FinalSharedLocalBasisIndex, Em ? *SubstrateSharedNormalParamCode : *SubstrateSharedLocalBasesInfo.NormalCode);
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\t#if MATERIAL_TANGENTSPACENORMAL\n"));
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\tParameters.%s.Normals[%u] *= Parameters.TwoSidedSign;\n"), *ParameterSharedLocalBasesName, FinalSharedLocalBasisIndex);
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\t#endif\n"));

		// Write out tangents
		if ((!Em && SubstrateSharedLocalBasesInfo.SharedData.TangentCodeChunk != INDEX_NONE) || (Em && SubstrateSharedLocalBasesInfo.TangentArg.IsValid()))
		{
			SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\tParameters.%s.Tangents[%u] = %s;\n"), *ParameterSharedLocalBasesName, FinalSharedLocalBasisIndex, Em ? *SubstrateSharedTangentParamCode : *SubstrateSharedLocalBasesInfo.TangentCode);
		}
		else
		{
			SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\tParameters.%s.Tangents[%u] = Parameters.TangentToWorld[0];\n"), *ParameterSharedLocalBasesName, FinalSharedLocalBasisIndex);
		}
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\t#if MATERIAL_TANGENTSPACENORMAL\n"));
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\tParameters.%s.Tangents[%u] *= Parameters.TwoSidedSign;\n"), *ParameterSharedLocalBasesName, FinalSharedLocalBasisIndex);
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\t#endif\n"));
	}

	SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\tParameters.%s.Count = %u;\n"), *ParameterSharedLocalBasesName, FinalUsedSharedLocalBasesCount);

	if (CompilationContextIndex == SCC_FullySimplified)
	{
		SubstratePixelNormalInitializerValues += FString::Printf(TEXT("\t#endif // SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL\n"));
	}

	if (OutSubstrateDefines)
	{
		// Now write out all the macros, them mapping from the BSDF to the effective position/index in the shared local basis array they should write to.
		OutSubstrateDefines->Reserve(CodeChunkToSubstrateSharedLocalBasis.Num());
		for (TMultiMap<uint64, FSubstrateSharedLocalBasesInfo>::TConstIterator It(CodeChunkToSubstrateSharedLocalBasis); It; ++It)
		{
			// The default linear output index will be 0 by default, and different if in fact the shared local basis points to one that is effectively in used in the array of shared local bases.
			uint8 LinearIndex = 0;
			for (uint8 i = 0; i < FinalUsedSharedLocalBasesCount; ++i)
			{
				if (UsedSharedLocalBasesInfo[i].NormalCodeChunkHash == It->Value.SharedData.NormalCodeChunkHash &&
					(UsedSharedLocalBasesInfo[i].TangentCodeChunkHash == It->Value.SharedData.TangentCodeChunkHash || It->Value.SharedData.TangentCodeChunk == INDEX_NONE))
				{
					LinearIndex = i;
					break;
				}
			}

			OutSubstrateDefines->Emplace(*GetSubstrateSharedLocalBasisIndexMacroInner(It->Value.SharedData, CompilationContextIndex), LinearIndex);
		}
	}
}


FSubstrateCompilationContext::FSubstrateCompilationContext()
{
	Initialise();
}


FSubstrateCompilationContext::FSubstrateCompilationContext(ESubstrateCompilationContext InCompilationContext)
{
	Initialise();
	CompilationContextIndex = InCompilationContext;
}

void FSubstrateCompilationContext::Initialise()
{
	CompilationContextIndex = ESubstrateCompilationContext::SCC_MAX;

	NextFreeSubstrateShaderNormalIndex = 0;
	FinalUsedSharedLocalBasesCount = 0;
	SubstrateMaterialRootOperator = nullptr;
	SubstrateMaterialExpressionRegisteredOperators.Reserve(SUBSTRATE_MAX_COMPILER_REGISTERED_OPERATOR_COUNT);
	SubstrateMaterialExpressionToOperatorIndex.Reserve(SUBSTRATE_MAX_COMPILER_REGISTERED_OPERATOR_COUNT);
	SubstrateMaterialEffectiveClosureCount = 0;
	SubstrateMaterialRequestedSizeByte = 0;
	SubstrateMaterialComplexity.Reset();
	bSubstrateMaterialIsUnlitNode = false;
	bSubstrateWritesEmissive = false;
	bSubstrateWritesAmbientOcclusion = false;
	bSubstrateTreeOutOfStackDepthOccurred = false;

	// Default value used as the root of the tree for the first path (when a node parent==nullptr).
	SubstrateNodeIdentifierStack.Push(FGuid(0x7AEE, 0xBAD, 0xDEAD, 0xBEEF));

	SubstrateThicknessIndexToExpressionInput.SetNum(0);
	SubstrateThicknessStack.SetNum(0);
}


bool FSubstrateCompilationContext::SubstrateGenerateDerivedMaterialOperatorData(FSubstrateTranslatorData& SubstrateTranslatorData)
{
	if (SubstrateMaterialExpressionRegisteredOperators.IsEmpty())
	{
		SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Could not find any Substrate operators or BSDFs in Material %s (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
		return false;
	}

	//
	// Evaluate the one and only root node.
	// And make sure each and every path of the Substrate tree have valid children and path.
	//
	int32 RootIndex = INDEX_NONE;
	for (auto& It : SubstrateMaterialExpressionRegisteredOperators)
	{
		if (It.ParentIndex == INDEX_NONE)
		{
			check(RootIndex == INDEX_NONE);	// There can only be one
			RootIndex = It.Index;
		}

		if (It.OperatorType == SUBSTRATE_OPERATOR_BSDF)
		{
			// Gather information about data written by BSDF
			bSubstrateWritesEmissive |= It.bBSDFWritesEmissive > 0;
			bSubstrateWritesAmbientOcclusion |= It.bBSDFWritesAmbientOcclusion > 0;
		}

		if (It.IsDiscarded())
		{
			continue; // ignore discarded operations in sub tree using parameter blending
		}

		bool bMustHaveLeftChild = false;
		bool bMustHaveRightChild = false;
		switch (It.OperatorType)
		{
			// Operators without any child
		case SUBSTRATE_OPERATOR_BSDF:
		{
		}
		break;

		// Operators with two children
		case SUBSTRATE_OPERATOR_HORIZONTAL:
		case SUBSTRATE_OPERATOR_VERTICAL:
		case SUBSTRATE_OPERATOR_ADD:
		case SUBSTRATE_OPERATOR_SELECT:
		{
			bMustHaveLeftChild = true;
			bMustHaveRightChild = true;
		}
		break;

		// Operators with a single child
		case SUBSTRATE_OPERATOR_WEIGHT:
		{
			bMustHaveLeftChild = true;
		}
		break;
		}

		if (bMustHaveLeftChild && It.LeftIndex == INDEX_NONE)
		{
			SubstrateTranslatorData.Errorf(FString::Printf(TEXT("A Substrate Operator %s node is missing its first input from material %s (asset: %s).\r\n"), GetSubstrateOperatorStr(It.OperatorType), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
			return false;
		}
		if (bMustHaveRightChild && It.RightIndex == INDEX_NONE)
		{
			SubstrateTranslatorData.Errorf(FString::Printf(TEXT("A Substrate Operator %s node is missing its second input from material %s (asset: %s).\r\n"), GetSubstrateOperatorStr(It.OperatorType), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
			return false;
		}
	}
	if (RootIndex != INDEX_NONE)
	{
		SubstrateMaterialRootOperator = &SubstrateMaterialExpressionRegisteredOperators[RootIndex];
	}
	if (!SubstrateMaterialRootOperator)
	{
		SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Cannot find the root of the Substrate Tree for Material %s (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
		return false;
	}

	//
	// Validate the tree operators with respect to how BSDF and operator can flow.
	// Also evaluate a bunch of data required.
	//
	enum EMaterialTypeValidation : uint32
	{
		MTV_BSDF_None = 0x000,
		MTV_BSDF_Unlit = 0x001,
		MTV_BSDF_Slab = 0x002,
		MTV_BSDF_Hair = 0x004,
		MTV_BSDF_Eye = 0x008,
		MTV_BSDF_SLW = 0x010,
		MTV_BSDF_Toon = 0x020,
		MTV_BSDF_VFogCloud = 0x040,
		MTV_UI = 0x080,
		MTV_Decal = 0x100,
		MTV_PostProcess = 0x200,
		MTV_LightFunction = 0x400,

		MTV_SelectOp_ValidMultipleBSDFs = (MTV_BSDF_Slab | MTV_BSDF_Hair | MTV_BSDF_Eye | MTV_BSDF_Toon),	// Unlit cannot be in there yet due to the specific define for it (SUBSTRATE_OPTIMIZED_UNLIT).
		MTV_WeightOp_AllowedList = (MTV_BSDF_Unlit | MTV_BSDF_Slab | MTV_BSDF_Toon),
		MTV_SelectOp_AllowedList = (MTV_SelectOp_ValidMultipleBSDFs),										// This only works because parameter blending is enforced so there will always be a single BSDF of a single type only. And hair or toon BSDFs will never pass through other operator allowed list.
		MTV_HorizoOp_AllowedList = MTV_BSDF_Slab,
		MTV_VerticOp_AllowedList = MTV_BSDF_Slab,
		MTV_DecalOp_AllowedList = MTV_BSDF_Slab,
		MTV_AddOp_AllowedList = MTV_BSDF_Slab
	};
	struct FSubstrateTreeValidation
	{
		uint32 MaterialType = MTV_BSDF_None;
		bool bValid = true;
	};
	FSubstrateTreeValidation SubstrateTreeValidation;
	{

		auto MergeSubstrateTreeValidationParameterBlended = [](const FSubstrateTreeValidation& A, const FSubstrateTreeValidation& B)
		{
			FSubstrateTreeValidation Result;
			Result.MaterialType = A.MaterialType | B.MaterialType;
			Result.bValid = A.bValid && B.bValid;
			return Result;
		};
		auto MergeSubstrateTreeValidation = [](const FSubstrateTreeValidation& A, const FSubstrateTreeValidation& B)
		{
			FSubstrateTreeValidation Result;
			Result.MaterialType = A.MaterialType | B.MaterialType;
			Result.bValid = A.bValid && B.bValid;
			return Result;
		};

		auto ValidateAllowedList = [&](FSubstrateOperator& CurrentOperator, FSubstrateTreeValidation SubstrateTreeValidation)
		{
			if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_BSDF)
			{
				return true;
			}

			FString OperatorName;
			uint32 AllowedListMask = 0;
			if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_WEIGHT && CurrentOperator.SubUsage == SUBSTRATE_OPERATOR_SUBUSAGE_DECAL)
			{
				OperatorName = TEXT("ConvertToDecal");
				AllowedListMask = MTV_DecalOp_AllowedList;
			}
			else if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_WEIGHT)
			{
				OperatorName = TEXT("Weight");
				AllowedListMask = MTV_WeightOp_AllowedList;
			}
			else if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_HORIZONTAL)
			{
				OperatorName = TEXT("Horizontal");
				AllowedListMask = MTV_HorizoOp_AllowedList;
			}
			else if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_VERTICAL)
			{
				OperatorName = TEXT("Vertical");
				AllowedListMask = MTV_VerticOp_AllowedList;
			}
			else if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_ADD)
			{
				OperatorName = TEXT("Add");
				AllowedListMask = MTV_AddOp_AllowedList;
			}
			else if (CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_SELECT)
			{
				OperatorName = TEXT("Select");
				AllowedListMask = MTV_SelectOp_AllowedList;
			}
			else
			{
				SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: unhandled operator type from ValidateAllowedList. (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName()));
				return false;
			}

			uint32 FoundNotAllowedTypeMask = SubstrateTreeValidation.MaterialType & (~AllowedListMask);

			if (FoundNotAllowedTypeMask != 0)
			{
				FString NotAllowedTypeNames;
				NotAllowedTypeNames += "{";
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_Unlit))
				{
					NotAllowedTypeNames += TEXT("Unlit ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_Slab))
				{
					NotAllowedTypeNames += TEXT("Slab ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_Hair))
				{
					NotAllowedTypeNames += TEXT("Hair ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_Eye))
				{
					NotAllowedTypeNames += TEXT("Eye ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_SLW))
				{
					NotAllowedTypeNames += TEXT("SLW ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_Toon))
				{
					NotAllowedTypeNames += TEXT("Toon ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_BSDF_VFogCloud))
				{
					NotAllowedTypeNames += TEXT("VFogCloud ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_UI))
				{
					NotAllowedTypeNames += TEXT("UI ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_Decal))
				{
					NotAllowedTypeNames += TEXT("Decal ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_PostProcess))
				{
					NotAllowedTypeNames += TEXT("PostProcess ");
				}
				if (!!(FoundNotAllowedTypeMask & MTV_LightFunction))
				{
					NotAllowedTypeNames += TEXT("LightFunction ");
				}
				NotAllowedTypeNames += "}";

				SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: Operator %s cannot be used with the following data flowing through %s. (asset: %s).\r\n"), *OperatorName, *NotAllowedTypeNames, *SubstrateTranslatorData.GetAssetName()));
				return false;
			}
			return true;
		};

		std::function<FSubstrateTreeValidation(FSubstrateOperator&)> WalkOperators = [&](FSubstrateOperator& CurrentOperator) -> FSubstrateTreeValidation
		{
			FSubstrateTreeValidation Out;
			switch (CurrentOperator.OperatorType)
			{
			case SUBSTRATE_OPERATOR_VERTICAL:
			case SUBSTRATE_OPERATOR_HORIZONTAL:
			case SUBSTRATE_OPERATOR_ADD:
			{
				FSubstrateTreeValidation A = WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
				FSubstrateTreeValidation B = WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex]);
				Out = MergeSubstrateTreeValidation(A, B);
				break;
			}
			case SUBSTRATE_OPERATOR_SELECT:
			{
				if (CurrentOperator.bNodeRequestParameterBlending)
				{
					FSubstrateTreeValidation A = WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
					FSubstrateTreeValidation B = WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex]);
					Out = MergeSubstrateTreeValidation(A, B);
				}
				else
				{
					SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: Select node can only be parameter blended as of today. (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName()));
				}
				break;
			}
			case SUBSTRATE_OPERATOR_WEIGHT:
			{
				Out = WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
				break;
			}
			case SUBSTRATE_OPERATOR_BSDF:
			{
				if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_UNLIT)
				{
					if (CurrentOperator.SubUsage == SUBSTRATE_OPERATOR_SUBUSAGE_NONE)
					{
						Out.MaterialType = MTV_BSDF_Unlit;
					}
					else if (CurrentOperator.SubUsage == SUBSTRATE_OPERATOR_SUBUSAGE_UI)
					{
						Out.MaterialType = MTV_UI;
					}
					else if (CurrentOperator.SubUsage == SUBSTRATE_OPERATOR_SUBUSAGE_LIGHTFUNCTION)
					{
						Out.MaterialType = MTV_LightFunction;
					}
					else if (CurrentOperator.SubUsage == SUBSTRATE_OPERATOR_SUBUSAGE_POSTPROCESS)
					{
						Out.MaterialType = MTV_PostProcess;
					}
					else
					{
						SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: unhandled Unlit BSDF sub type from WalkParameterBlendedSubTree. (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName()));
					}
				}
				else if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_SLAB)
				{
					Out.MaterialType = MTV_BSDF_Slab;
				}
				else if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD)
				{
					Out.MaterialType = MTV_BSDF_VFogCloud;
				}
				else if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_HAIR)
				{
					Out.MaterialType = MTV_BSDF_Hair;
				}
				else if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER)
				{
					Out.MaterialType = MTV_BSDF_SLW;
				}
				else if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_TOON)
				{
					Out.MaterialType = MTV_BSDF_Toon;
				}
				else if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_EYE)
				{
					Out.MaterialType = MTV_BSDF_Eye;
				}
				else
				{
					SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: unhandled BSDF type from WalkParameterBlendedSubTree. (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName()));
				}
				break;
			}
			}
			Out.bValid = ValidateAllowedList(CurrentOperator, Out);
			return Out;
		};

		SubstrateTreeValidation = WalkOperators(*SubstrateMaterialRootOperator);
		if (!SubstrateTreeValidation.bValid)
		{
			SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: invalid Substrate node graph. (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName()));
			return false;
		}
	}

	// Evaluate if simplification is needed.
	SubstrateSimplificationStatus.bRunFullSimplification = SubstrateTranslatorData.SubstrateCompilationConfig.bFullSimplify || CompilationContextIndex == ESubstrateCompilationContext::SCC_FullySimplified;
	if (!SubstrateSimplificationStatus.bRunFullSimplification)
	{
		//
		// Generate LayerDepth value for all operators/bsdfs for progressive material simplification and sort them.
		//
		int VOpTopBranchCountTaken = 0;
		int VOpBottomBranchCountTaken = 0;
		std::function<void(FSubstrateOperator&)> WalkOperatorsForSimplification = [&](FSubstrateOperator& CurrentOperator) -> void
		{
			CurrentOperator.LayerDepth = VOpBottomBranchCountTaken;
			switch (CurrentOperator.OperatorType)
			{
			case SUBSTRATE_OPERATOR_VERTICAL:
			{
				VOpTopBranchCountTaken++;
				WalkOperatorsForSimplification(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
				VOpTopBranchCountTaken--;
				VOpBottomBranchCountTaken++;
				WalkOperatorsForSimplification(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex]);
				VOpBottomBranchCountTaken--;
				break;
			}
			case SUBSTRATE_OPERATOR_HORIZONTAL:
			case SUBSTRATE_OPERATOR_ADD:
			case SUBSTRATE_OPERATOR_SELECT:
			{
				WalkOperatorsForSimplification(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
				WalkOperatorsForSimplification(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex]);
				break;
			}
			case SUBSTRATE_OPERATOR_WEIGHT:
			{
				WalkOperatorsForSimplification(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
				break;
			}
			case SUBSTRATE_OPERATOR_BSDF:
			{
				break;
			}
			}

			// Add operators to simplify with a priority.
			switch (CurrentOperator.OperatorType)
			{
			case SUBSTRATE_OPERATOR_VERTICAL:
			case SUBSTRATE_OPERATOR_HORIZONTAL:
			case SUBSTRATE_OPERATOR_ADD:
			{
				FSubstrateSimplificationStatus::FOperatorToSimplify OperatorToSimplify;
				OperatorToSimplify.Data.Index = CurrentOperator.Index;
				OperatorToSimplify.Data.Depth = CurrentOperator.LayerDepth;
				SubstrateSimplificationStatus.OperatorSimplificationOrder.Push(OperatorToSimplify);
				break;
			}
			}
		};
		WalkOperatorsForSimplification(*SubstrateMaterialRootOperator);

		SubstrateSimplificationStatus.OperatorSimplificationOrder.Sort();	// sort according to depth
	}

	EShaderPlatform ShaderPlatform = SubstrateTranslatorData.GetShaderPlatform();
	const uint32 SubstrateBytePerPixel = SubstrateTranslatorData.SubstrateCompilationConfig.BytesPerPixelOverride > 0 ? SubstrateTranslatorData.SubstrateCompilationConfig.BytesPerPixelOverride : Substrate::GetBytePerPixel(ShaderPlatform);

	const uint32 SubstrateClosurePerPixel = SubstrateTranslatorData.SubstrateCompilationConfig.ClosuresPerPixelOverride > 0 ? SubstrateTranslatorData.SubstrateCompilationConfig.ClosuresPerPixelOverride : Substrate::GetClosurePerPixel(ShaderPlatform, SubstrateTranslatorData.GetCookTargetedMaterialQualityLevel());

	bool bFirstLoop = true;
	bool bRequestMaterialDetailsSet = false;
	do
	{
		if (!bFirstLoop && !SubstrateSimplificationStatus.bFullSimplificationStepHasBeenRun && !SubstrateSimplificationStatus.bMaterialFitsInMemoryBudget && SubstrateSimplificationStatus.OperatorSimplificationOrder.Num() > 0)
		{
			// Mark the deepest operator for parameter blending
			FSubstrateSimplificationStatus::FOperatorToSimplify& OperatorToSimplify = SubstrateSimplificationStatus.OperatorSimplificationOrder.Top();
			SubstrateMaterialExpressionRegisteredOperators[OperatorToSimplify.Data.Index].bNodeRequestParameterBlending = true;
			SubstrateSimplificationStatus.OperatorSimplificationOrder.Pop(EAllowShrinking::No);

			// Mark that this is similar to have run full simplification
			SubstrateSimplificationStatus.bFullSimplificationStepHasBeenRun |= SubstrateSimplificationStatus.OperatorSimplificationOrder.Num() == 0;
		}
		else if (!bFirstLoop && SubstrateSimplificationStatus.bFullSimplificationStepHasBeenRun && !SubstrateSimplificationStatus.bMaterialFitsInMemoryBudget && SubstrateSimplificationStatus.OperatorSimplificationOrder.Num() == 0 && !SubstrateSimplificationStatus.bSlabSimplificationStepHasBeenRun)
		{
			for (auto& It : SubstrateMaterialExpressionRegisteredOperators)
			{
				// Disable all optional features for now to fit.
				// SUBSTRATE_TODO we will need to refine that to account for platforms supporting SSS for instance.
				It.BSDFFeatures = ESubstrateBsdfFeature::None;
			}
			SubstrateSimplificationStatus.bSlabSimplificationStepHasBeenRun = true;
		}
		else if (bFirstLoop)
		{
			// Fall through. This is needed for material with a single BSDF and no operator to simplify.
			bFirstLoop = false;
		}
		else
		{
			SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Unkown Substrate material simplification status error for %s (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
			return false;
		}

		// Reset some data
		SubstrateMaterialEffectiveClosureCount = 0;

		//
		// Parse the tree and mark nodes that are the root of a subtree using parameter blending, while other nodes in that tree are forced to use parameter blending.
		// Allocate BSDFIndex at the same time.
		// Check BSDF that should be unique.
		//
		bool bHasUnlit = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_Unlit);
		bool bHasVFogCloud = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_VFogCloud);
		bool bHasHair = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_Hair);
		bool bHasEye = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_Eye);
		bool bHasSLW = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_SLW);
		bool bHasToon = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_Toon);
		bool bHasSlab = !!(SubstrateTreeValidation.MaterialType & MTV_BSDF_Slab);
		bool bHasUI = !!(SubstrateTreeValidation.MaterialType & MTV_UI);
		bool bHasDecal = !!(SubstrateTreeValidation.MaterialType & MTV_Decal);
		bool bHasPostProcess = !!(SubstrateTreeValidation.MaterialType & MTV_PostProcess);
		bool bHasLightFunction = !!(SubstrateTreeValidation.MaterialType & MTV_LightFunction);
		{
			int VOpTopBranchCountTaken = 0;
			int VOpBottomBranchCountTaken = 0;
			bool bSubstrateUsesVerticalLayering = false;

			std::function<void(FSubstrateOperator&, bool)> WalkOperators = [&](FSubstrateOperator& CurrentOperator, bool bInsideParameterBlendingSubTree) -> void
			{
				const bool bCurrentOpRequestParameterBlending = CurrentOperator.bNodeRequestParameterBlending || SubstrateSimplificationStatus.bRunFullSimplification;
				const bool bRootOfParameterBlendingSubTree = bCurrentOpRequestParameterBlending && !bInsideParameterBlendingSubTree;
				const bool bUseParameterBlending = bCurrentOpRequestParameterBlending || bInsideParameterBlendingSubTree;

				if (CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_SLAB																													// Any operators
					|| CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_UNLIT																												// Weight operator
					|| CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_HAIR || CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_EYE || CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_TOON)	// Select operator
				{
					// Update the parameter blending data for BSDFs supporting operators.
					// We also need to do this for UNLIT since it supports Coverage operator.
					CurrentOperator.bUseParameterBlending = bUseParameterBlending;
					CurrentOperator.bRootOfParameterBlendingSubTree = bRootOfParameterBlendingSubTree;
				}

				switch (CurrentOperator.OperatorType)
				{
				case SUBSTRATE_OPERATOR_VERTICAL:
				{
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex], bUseParameterBlending);
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex], bUseParameterBlending);
					break;
				}
				case SUBSTRATE_OPERATOR_HORIZONTAL:
				case SUBSTRATE_OPERATOR_ADD:
				case SUBSTRATE_OPERATOR_SELECT:
				{
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex], bUseParameterBlending);
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex], bUseParameterBlending);
					break;
				}
				case SUBSTRATE_OPERATOR_WEIGHT:
				{
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex], bUseParameterBlending);
					break;
				}
				case SUBSTRATE_OPERATOR_BSDF:
				{
					if (!bInsideParameterBlendingSubTree)
					{
						CurrentOperator.BSDFIndex = SubstrateMaterialEffectiveClosureCount++;
					}

					bHasUnlit |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_UNLIT;
					bHasVFogCloud |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD;
					bHasHair |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_HAIR;
					bHasEye |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_EYE;
					bHasSLW |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER;
					bHasToon |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_TOON;
					bHasSlab |= CurrentOperator.BSDFType == SUBSTRATE_BSDF_TYPE_SLAB;
					break;
				}
				}

				// We mark the top of a parameter blending tree as a BSDF now to allocate a slot for it that can then be used next for non-parameter blending operations.
				// Intermediate parameter blending BSDF and operation will be done inline and stored in FSubstrateData.
				if (CurrentOperator.OperatorType != SUBSTRATE_OPERATOR_BSDF && bRootOfParameterBlendingSubTree)
				{
					CurrentOperator.OperatorType = SUBSTRATE_OPERATOR_BSDF;
					CurrentOperator.BSDFIndex = SubstrateMaterialEffectiveClosureCount++;
					// We do not reset LeftIndex and RightIndex because those are needed to recover local tangent basis information needed with parameter blending.
				}

				// When at least one vertical operator exists that is not parameter blending, we can enabled writing to opaque rough refraction buffer.
				bSubstrateUsesVerticalLayering = bSubstrateUsesVerticalLayering || (!CurrentOperator.bUseParameterBlending && CurrentOperator.OperatorType == SUBSTRATE_OPERATOR_VERTICAL);
			};

			WalkOperators(*SubstrateMaterialRootOperator, false);

			if (CompilationContextIndex == ESubstrateCompilationContext::SCC_Default)
			{
				// Only write those data for the default material
				const bool bIsOpaqueOrMasked = IsOpaqueOrMaskedBlendMode(*SubstrateTranslatorData.GetMaterialInterface());
				SubstrateTranslatorData.bSubstrateMaterialOutputOpaqueRoughRefractions = bSubstrateUsesVerticalLayering && bIsOpaqueOrMasked;
			}
			bSubstrateMaterialIsUnlitNode = bHasUnlit;
		}

		//
		// Make sure all the types have valid children operator indices according to the parameter blending setup for this simplification status.
		//
		for (const auto& It : SubstrateMaterialExpressionRegisteredOperators)
		{
			if (It.IsDiscarded())
			{
				continue; // ignore discarded operations in sub tree using parameter blending
			}

			check(It.Index != INDEX_NONE);

			switch (It.OperatorType)
			{
				// Operators without any child
			case SUBSTRATE_OPERATOR_BSDF:
			{
				if (!It.bUseParameterBlending)
				{
					// When using parameter blending, we need to keep indices to be able to recover local basis information for normal blending.
					check(It.LeftIndex == INDEX_NONE && It.RightIndex == INDEX_NONE && It.BSDFIndex != INDEX_NONE);
				}
				break;
			}

			// Operators with two children
			case SUBSTRATE_OPERATOR_HORIZONTAL:
			case SUBSTRATE_OPERATOR_VERTICAL:
			case SUBSTRATE_OPERATOR_ADD:
			case SUBSTRATE_OPERATOR_SELECT:
			{
				check(It.RightIndex != INDEX_NONE);
			}
			// Fallthrough

			// Operators with a single child
			case SUBSTRATE_OPERATOR_WEIGHT:
			{
				check(It.LeftIndex != INDEX_NONE);
			}
			// Fallthrough
			}
		}

		//
		// Compute the maximum depth from the BSDF node for each operator
		//
		{
			std::function<void(FSubstrateOperator&)>  WalkOperatorsToRoot = [&](FSubstrateOperator& CurrentOperator) -> void
			{
				switch (CurrentOperator.OperatorType)
				{
				case SUBSTRATE_OPERATOR_WEIGHT:
				{
					CurrentOperator.MaxDistanceFromLeaves = SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex].MaxDistanceFromLeaves + 1;
					break;
				}
				case SUBSTRATE_OPERATOR_VERTICAL:
				case SUBSTRATE_OPERATOR_HORIZONTAL:
				case SUBSTRATE_OPERATOR_ADD:
				case SUBSTRATE_OPERATOR_SELECT:
				{
					CurrentOperator.MaxDistanceFromLeaves = FMath::Max(
						SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex].MaxDistanceFromLeaves,
						SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex].MaxDistanceFromLeaves) + 1;
					break;
				}
				case SUBSTRATE_OPERATOR_BSDF:
				{
					CurrentOperator.MaxDistanceFromLeaves = 0;
					break;
				}
				}

				if (CurrentOperator.ParentIndex != INDEX_NONE)
				{
					WalkOperatorsToRoot(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.ParentIndex]);
				}
			};

			for (auto& It : SubstrateMaterialExpressionRegisteredOperators)
			{
				if (It.IsDiscarded())
				{
					continue; // ignore discarded operations in sub tree using parameter blending
				}

				if (It.OperatorType == SUBSTRATE_OPERATOR_BSDF)
				{
					// Recursively parse all nodes from BSDF to the root node and update the necessary properties.
					WalkOperatorsToRoot(It);
				}
			}
		}

		//
		// Compute IsTop or IsBottom layer using a depth first tree visit while counting vertical right and left branches taken.
		// When a BSDF is encountered, and it is the root of a parameter blend subtree, we continue to parse, setting the same information as the root.
		//
		{
			int VOpTopBranchCountTaken = 0;
			int VOpBottomBranchCountTaken = 0;

			std::function<void(FSubstrateOperator&, const FSubstrateOperator&)> PopulateParameterBlendedSubtree = [&](FSubstrateOperator& CurrentOperator, const FSubstrateOperator& RootOperator) -> void
			{
				switch (CurrentOperator.OperatorType)
				{
				case SUBSTRATE_OPERATOR_VERTICAL:
				case SUBSTRATE_OPERATOR_HORIZONTAL:
				case SUBSTRATE_OPERATOR_ADD:
				case SUBSTRATE_OPERATOR_SELECT:
				{
					PopulateParameterBlendedSubtree(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex], RootOperator);
					PopulateParameterBlendedSubtree(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex], RootOperator);
					break;
				}
				case SUBSTRATE_OPERATOR_WEIGHT:
				{
					PopulateParameterBlendedSubtree(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex], RootOperator);
					break;
				}
				case SUBSTRATE_OPERATOR_BSDF:
				{
					CurrentOperator.LayerDepth = RootOperator.LayerDepth;
					CurrentOperator.bIsTop = RootOperator.bIsTop;
					CurrentOperator.bIsBottom = RootOperator.bIsBottom;
					break;
				}
				}
			};

			std::function<void(FSubstrateOperator&)> WalkOperators = [&](FSubstrateOperator& CurrentOperator) -> void
			{
				switch (CurrentOperator.OperatorType)
				{
				case SUBSTRATE_OPERATOR_VERTICAL:
				{
					VOpTopBranchCountTaken++;
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
					VOpTopBranchCountTaken--;
					VOpBottomBranchCountTaken++;
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex]);
					VOpBottomBranchCountTaken--;
					break;
				}
				case SUBSTRATE_OPERATOR_HORIZONTAL:
				case SUBSTRATE_OPERATOR_ADD:
				case SUBSTRATE_OPERATOR_SELECT:
				{
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex]);
					break;
				}
				case SUBSTRATE_OPERATOR_WEIGHT:
				{
					WalkOperators(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex]);
					break;
				}
				case SUBSTRATE_OPERATOR_BSDF:
				{
					const int32 VopCount = VOpTopBranchCountTaken + VOpBottomBranchCountTaken;
					CurrentOperator.LayerDepth = VOpBottomBranchCountTaken;
					CurrentOperator.bIsTop = VopCount == 0 || VOpTopBranchCountTaken == VopCount;
					CurrentOperator.bIsBottom = VopCount == 0 || VOpBottomBranchCountTaken == VopCount;
					if (CurrentOperator.bRootOfParameterBlendingSubTree)
					{
						// Make sure to also set up those values onto the BSDF within parameter blend sub trees.
						if (CurrentOperator.LeftIndex != INDEX_NONE)
						{
							PopulateParameterBlendedSubtree(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.LeftIndex], CurrentOperator);
						}
						if (CurrentOperator.RightIndex != INDEX_NONE)
						{
							PopulateParameterBlendedSubtree(SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.RightIndex], CurrentOperator);
						}
					}
					break;
				}
				}
			};

			WalkOperators(*SubstrateMaterialRootOperator);
		}

		//
		// Compute the per pixel byte count required by the materials.
		//
		{
			// Compute the shared local basis count only
			// But we cannot use SubstrateEvaluateSharedLocalBases here, because the material has not been compiled yet so all the bases would just default to the same.
			// SUBSTRATE_TODO: can we do that material generation in two passes? 
			//		1- A first one to evaluate the normal/tangent code
			//		2- Operators are processed and simplification computed based on memory budget
			//		3- Material is finally compiled for with operator updated to fit in memory budget.
			uint8 UsedSharedLocalBasesCount = SubstrateMaterialEffectiveClosureCount;

			const uint32 UintByteSize = sizeof(uint32);
			SubstrateMaterialRequestedSizeByte = 0;

			// 1. Evaluate simple/single BSDF
			SubstrateMaterialComplexity.bIsSimple = SubstrateMaterialEffectiveClosureCount == 1;
			SubstrateMaterialComplexity.bIsSingle = SubstrateMaterialEffectiveClosureCount == 1;
			SubstrateMaterialComplexity.bIsComplexSpecial = false;
			ESubstrateBsdfFeature SubstrateMaterialBsdfFeatures = ESubstrateBsdfFeature::None;
			bool bIsFastWaterPath = false;
			bool bCustomEncoding = false;
			bool bMayHaveCoverageLessThan1 = false;
			for (const auto& It : SubstrateMaterialExpressionRegisteredOperators)
			{
				if (It.IsDiscarded() && It.OperatorType == SUBSTRATE_OPERATOR_WEIGHT)
				{
					// If a BSDF modified by a weight operator, its weight will be < 1.0f, and it won't be a "single" material anymore.
					// This is also the case if the operator is "discarded" due to parameter blending, because the resulting BSDF might not have a coverage of 1 in the end.
					// For instance with a BSDF => Weight => Horizonal node with parameter blending is encountered and weight is less than 1.
					bMayHaveCoverageLessThan1 |= true;
				}

				if (It.IsDiscarded())
				{
					continue; // ignore discarded operations in sub tree using parameter blending
				}

				switch (It.OperatorType)
				{
				case SUBSTRATE_OPERATOR_BSDF:
				{
					// From the compiler side, we can only assume the top layer has gray scale luminance weight.
					const bool bMayHaveColoredWeight = !It.bIsTop;

					// Aggregate all BSDFs features used by the material
					SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature(It.BSDFFeatures);

					switch (It.BSDFType)
					{
					case SUBSTRATE_BSDF_TYPE_SLAB:
					{
						bool bIsComplexSpecial = false;
						if (It.Has(ESubstrateBsdfFeature::ComplexSpecialMask))
						{
							// We need to check each features if really enabled for a platofrms
							bIsComplexSpecial = It.Has(ESubstrateBsdfFeature::Glint) && Substrate::IsGlintEnabled(ShaderPlatform);
						}

						SubstrateMaterialComplexity.bIsSimple = SubstrateMaterialComplexity.bIsSimple && !bMayHaveColoredWeight && !It.Has(ESubstrateBsdfFeature::ComplexMask) && !bIsComplexSpecial && !It.Has(ESubstrateBsdfFeature::SingleMask);
						SubstrateMaterialComplexity.bIsSingle = SubstrateMaterialComplexity.bIsSingle && !bMayHaveColoredWeight && !It.Has(ESubstrateBsdfFeature::ComplexMask) && !bIsComplexSpecial;
						SubstrateMaterialComplexity.bIsComplexSpecial |= bIsComplexSpecial;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_HAIR:
					{
						SubstrateMaterialComplexity.bIsSimple = false;
						SubstrateMaterialComplexity.bIsSingle = false;
						bCustomEncoding = true;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_EYE:
					{
						SubstrateMaterialComplexity.bIsSimple = false;
						SubstrateMaterialComplexity.bIsSingle = false;
						bCustomEncoding = true;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
					{
						SubstrateMaterialComplexity.bIsSimple = false;
						SubstrateMaterialComplexity.bIsSingle = false;
						bIsFastWaterPath = true;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_TOON:
					{
						// Use the complex path for now.
						SubstrateMaterialComplexity.bIsSimple = false;
						SubstrateMaterialComplexity.bIsSingle = false;
						break;
					}
					}
					break;
				}
				case SUBSTRATE_OPERATOR_WEIGHT:
				{
					// If a BSDF modified by a weight operator, its weight will be < 1.0f, and it won't be a single material anymore
					SubstrateMaterialComplexity.bIsSimple = false;
					SubstrateMaterialComplexity.bIsSingle = false;
					break;
				}
				}
			}
			SubstrateMaterialComplexity.bIsSingle = SubstrateMaterialComplexity.bIsSingle && !SubstrateMaterialComplexity.bIsSimple;

			if (bMayHaveCoverageLessThan1)
			{
				// Material with coverage<1 can only be complex. It will be made simple or single at export time if possible.
				SubstrateMaterialComplexity.bIsSimple = false;
				SubstrateMaterialComplexity.bIsSingle = false;
			}

			// 2. Header

			if (!SubstrateMaterialComplexity.bIsSimple && !SubstrateMaterialComplexity.bIsSingle && !bCustomEncoding && !bIsFastWaterPath) // header written later, 
			{
				// Packed Header
				SubstrateMaterialRequestedSizeByte += UintByteSize;

				// Shared local bases between BSDFs
				SubstrateMaterialRequestedSizeByte += UsedSharedLocalBasesCount * SUBSTRATE_PACKED_SHAREDLOCALBASIS_STRIDE_BYTES;
			}
			// Note:
			//  - We do not need to account for the Top Normal texture when evaluating the material byte count for the optimization algorithm.
			//  - This is because we only need to optimize for the Substrate uint material buffer.

			// 2. Process the list of BSDFs for worst case memory usage and count operators.
			static_assert(SUBSTRATE_MAX_CLOSURE_COUNT_FOR_CLOSUREOFFSET == (32u / SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT));
			static_assert(SUBSTRATE_MAX_CLOSURE_COUNT <= (1u << SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT));
			const uint32 ClosureMaxByteCountForOffset = uint32(1u << SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT) * sizeof(uint32);
			uint32 OperatorCount = 0;
			SubstrateMaterialClosureCount = 0;
			for (auto& It : SubstrateMaterialExpressionRegisteredOperators)
			{
				if (It.IsDiscarded())
				{
					continue; // ignore discarded operations in sub tree using parameter blending
				}

				// Operators are weight, vertical layering, etc. 
				// Be aware that BSDFs also count as Operators when they are promoted from parameter blending!
				OperatorCount++;

				const uint32 PreSubstrateMaterialRequestedSizeByte = SubstrateMaterialRequestedSizeByte;
				switch (It.OperatorType)
				{
				case SUBSTRATE_OPERATOR_BSDF:
				{
					// we have encountered a new BSDF which directly link to a single closure evaluation
					SubstrateMaterialClosureCount++;

					// From the compiler side, we can only assume the top layer has gray scale luminance weight.
					const bool bMayHaveColoredWeight = !It.bIsTop;

					if (SubstrateMaterialComplexity.bIsSimple)
					{
						// Header
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						// Disney material
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						break; // Stop here
					}
					else if (SubstrateMaterialComplexity.bIsSingle)
					{
						// Header
						SubstrateMaterialRequestedSizeByte += UintByteSize;
					}
					else if (bCustomEncoding)
					{
						// Header
						SubstrateMaterialRequestedSizeByte += UintByteSize;
					}
					else if (bIsFastWaterPath)
					{
						// Header + Data
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						// Data
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						break; // Stop here
					}
					else if (bMayHaveColoredWeight)
					{
						// BSDF state
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						// Color weight
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						// Light transmittance weight
						SubstrateMaterialRequestedSizeByte += UintByteSize;
					}
					else
					{
						// BSDF state with gray scale weight
						SubstrateMaterialRequestedSizeByte += UintByteSize;
					}

					switch (It.BSDFType)
					{
					case SUBSTRATE_BSDF_TYPE_SLAB:
					{
						// Compute values closer to the reality for HasSSS and IsSimpleVolume, now that we know that we know the topology of the material.
						const bool bIsSimpleVolume = !It.bIsBottom && It.Has(ESubstrateBsdfFeature::MFPPluggedIn);
						const bool bHasSSS = It.bIsBottom && It.Has(ESubstrateBsdfFeature::SSS);

						SubstrateMaterialRequestedSizeByte += UintByteSize;
						SubstrateMaterialRequestedSizeByte += UintByteSize;

						// When using blendable GBuffer, do not account for features's byte requests. as downcasting is done SubstrateExport(). 
						// Otherwise this would cause material to not compile as we do not demote feature during simplification.
						if (Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform))
						{
							break;
						}

						if (It.Has(ESubstrateBsdfFeature::EdgeColor | ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						if (bHasSSS || bIsSimpleVolume)
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						if (It.Has(ESubstrateBsdfFeature::Fuzz))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						if (It.Has(ESubstrateBsdfFeature::Glint) && Substrate::IsGlintEnabled(ShaderPlatform))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
							SubstrateMaterialRequestedSizeByte += UintByteSize;
							SubstrateMaterialRequestedSizeByte += UintByteSize;
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						if (It.Has(ESubstrateBsdfFeature::SpecularProfile))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						if (It.Has(ESubstrateBsdfFeature::Eye))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						if (It.Has(ESubstrateBsdfFeature::Hair))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}
						break;
					}
					case SUBSTRATE_BSDF_TYPE_HAIR:
					{
						// Custom encoding
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_EYE:
					{
						// Custom encoding
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
					{
						SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Substrate error: single layer water should go through through a dedicated fast path in %s (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
						return false;
					}
					case SUBSTRATE_BSDF_TYPE_TOON:
					{
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						SubstrateMaterialRequestedSizeByte += UintByteSize;
						// We cannot add more uint here to stay compatible with legacy gbuffer when Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform).
						// Possible work around: disable some toon shading feature that cannot be stored in the GBuffer, or increase Custom GBufferD size to 64 bits.

						if (!Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform)
							&& It.Has(ESubstrateBsdfFeature::Anisotropy))
						{
							SubstrateMaterialRequestedSizeByte += UintByteSize;
						}

						break;
					}
					case SUBSTRATE_BSDF_TYPE_UNLIT:
					{
						// Never stored, it goes directly into the scene as emitted luminance.
						break;
					}
					default:
					{
						SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Unkownd BSDF type encountered in %s (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
						return false;
					}
					}
					break;
				} // case SUBSTRATE_OPERATOR_BSDF
				} // switch (It.OperatorType)
			}

			if (OperatorCount > SUBSTRATE_MAX_OPERATOR_COUNT)
			{
				// Why do we have an operator limit: due to the size of the array of Operator in FSubstrateTre and the way the Substrate tree is exported for advanced debug purpose. Parameter blending can help working around that. 
				SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Material %s have too many Substrate Operators (asset: %s): %d / %d. Please note that BSDFs also count as an operator. Use parameter blending to workaround that limitation.\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath(), OperatorCount, SUBSTRATE_MAX_OPERATOR_COUNT));
				return false;
			}

			SubstrateSimplificationStatus.bMaterialFitsInMemoryBudget = (SubstrateMaterialRequestedSizeByte <= SubstrateBytePerPixel) && (SubstrateMaterialClosureCount <= SubstrateClosurePerPixel);
			if (!SubstrateSimplificationStatus.bMaterialFitsInMemoryBudget && SubstrateSimplificationStatus.bFullSimplificationStepHasBeenRun && SubstrateSimplificationStatus.bSlabSimplificationStepHasBeenRun)
			{
				// If we have already run the full simplification but the material still does not fit in memory, we must fail the material compilation.
				SubstrateTranslatorData.Errorf(FString::Printf(TEXT("Material %s could not be simplified to fit in Substrate per pixel (asset: %s).\r\n"), *SubstrateTranslatorData.GetAssetName(), *SubstrateTranslatorData.GetAssetPath()));
				return false;
			}
			if (!bRequestMaterialDetailsSet)
			{
				// Record the original requested byte size before simplification, only for the first pass.
				SubstrateSimplificationStatus.OriginalRequestedByteSize = SubstrateMaterialRequestedSizeByte;
				SubstrateSimplificationStatus.OriginalRequestedClosureCount = SubstrateMaterialClosureCount;
				bRequestMaterialDetailsSet = true;
			}
			SubstrateSimplificationStatus.bFullSimplificationStepHasBeenRun |= SubstrateSimplificationStatus.bRunFullSimplification;

			const uint32 RequestedSizeInUint = FMath::DivideAndRoundUp(SubstrateMaterialRequestedSizeByte, 4u);
			check(RequestedSizeInUint < 256u);

			if (CompilationContextIndex == ESubstrateCompilationContext::SCC_Default)
			{
				// Only write those data for the default material
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.SubstrateMaterialType = SubstrateMaterialComplexity.SubstrateMaterialType();
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.SubstrateClosureCount = SubstrateMaterialEffectiveClosureCount;
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.SubstrateUintPerPixel = uint8(FMath::Clamp(RequestedSizeInUint, 0u, 0xFF));
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures = SubstrateMaterialBsdfFeatures;

#if WITH_EDITOR
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.SharedLocalBasesCount = 0; // FinalUsedSharedLocalBasesCount is not valid yet
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.RequestedBytePerPixel = SubstrateSimplificationStatus.OriginalRequestedByteSize;
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.PlatformBytePerPixel = SubstrateBytePerPixel;

				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.RequestedClosurePerPixel = SubstrateSimplificationStatus.OriginalRequestedClosureCount;
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.PlatformClosurePixel = SubstrateClosurePerPixel;

				UMaterialInterface* MaterialInterface = SubstrateTranslatorData.GetMaterialInterface();
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.bIsThin = MaterialInterface && MaterialInterface->IsThinSurface() ? 1 : 0;

				uint32 BSDFTypeCount = FMath::CountBits(SubstrateTreeValidation.MaterialType & MTV_SelectOp_ValidMultipleBSDFs);

				// The order of ifs here is important.
				if (MaterialInterface && MaterialInterface->IsLightFunctionMaterial())
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_LIGHTFUNCTION;
				}
				else if (MaterialInterface && MaterialInterface->IsPostProcessMaterial())
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_POSTPROCESS;
				}
				else if (MaterialInterface && MaterialInterface->IsUIMaterial())
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_UI;
				}
				else if (MaterialInterface && MaterialInterface->IsDeferredDecal())
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_DECAL;
				}
				else if (SubstrateMaterialEffectiveClosureCount > 1 && (SubstrateTreeValidation.MaterialType == (SubstrateTreeValidation.MaterialType & MTV_BSDF_Slab)))
				{
					// Only slabs, not hair nor toon.
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_MULTIPLESLABS;
				}
				else if (SubstrateMaterialEffectiveClosureCount > 1) // Only slabs can have multiple closures because the select nodde will always parameter blend slabs into a single one
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_MULTIPLESLABS;
				}
				else if (BSDFTypeCount > 1)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_MULTIPLEBSDFS;
				}
				else if (bHasUnlit)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_UNLIT;
				}
				else if (bHasVFogCloud)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_VOLUMETRICFOGCLOUD;
				}
				else if (bHasHair)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_HAIR;
				}
				else if (bHasEye)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_EYE;
				}
				else if (bHasSLW)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_SINGLELAYERWATER;
				}
				else if (bHasToon)
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_TOON;
				}
				else
				{
					SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.MaterialType = SUBSTRATE_MATERIAL_TYPE_SINGLESLAB;
				}

				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.bMaterialOutOfBudgetHasBeenSimplified |= !SubstrateSimplificationStatus.bMaterialFitsInMemoryBudget;

				if (OperatorCount <= SUBSTRATE_COMPILATION_OUTPUT_MAX_OPERATOR)
				{
					int32 OperatorIndex = 0;
					for (auto& It : SubstrateMaterialExpressionRegisteredOperators)
					{
						SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.Operators[OperatorIndex] = SubstrateMaterialExpressionRegisteredOperators[OperatorIndex];
						OperatorIndex++;
					}
				}
				SubstrateTranslatorData.GetMaterialCompilationOutput().SubstrateMaterialCompilationOutput.RootOperatorIndex = RootIndex;
#endif // EDITOR_ONLY
			}
		}
	} while (!SubstrateSimplificationStatus.bMaterialFitsInMemoryBudget);

	return true; // Success
}

FSubstrateSharedLocalBasesInfo FSubstrateCompilationContext::SubstrateCompilationInfoGetMatchingSharedLocalBasisInfo(const FSubstrateRegisteredSharedLocalBasis& SearchedSharedLocalBasis)
{
	check(NextFreeSubstrateShaderNormalIndex < 255);	// Out of shared local basis slots

	// Find a basis which matches both the Normal & the Tangent code chunks
	TArray<const FSubstrateSharedLocalBasesInfo*> NormalInfos;
	CodeChunkToSubstrateSharedLocalBasis.MultiFindPointer(SearchedSharedLocalBasis.NormalCodeChunkHash, NormalInfos);

	// We first try to find a perfect match for normal and tangent from all the registered element.
	for (const FSubstrateSharedLocalBasesInfo* NormalInfo : NormalInfos)
	{
		if (SearchedSharedLocalBasis.TangentCodeChunk == INDEX_NONE ||											// We selected the first available normal if there is no tangent specified on the material.
			SearchedSharedLocalBasis.TangentCodeChunkHash == NormalInfo->SharedData.TangentCodeChunkHash)		// Otherwise we select the normal+tangent that exactly matches the request.
		{
			return *NormalInfo;
		}
	}

	check(0);	// When the compiler is querying, this is to get a result to generate code from a fully processed graph. No result means a bug happened during graph processing.
	return FSubstrateSharedLocalBasesInfo();
}


FSubstrateTranslatorData::FSubstrateTranslatorData()
: bSubstrateEnabled(false)
, bMaterialIsSubstrate(false)
, bMaterialUsesRootNodeToSubstrateHiddenConversion(false)
, bProjectSubstrateHiddenMaterialAssetConversionEnabled(false)
, bSubstrateMaterialUsesLegacyMaterialCompilation(false)
, bSubstrateWritesEmissive(false)
, bSubstrateWritesAmbientOcclusion(false)
, bSubstrateUsesConversionFromLegacy(false)
, bSubstrateMaterialOutputOpaqueRoughRefractions(false)
, SubstrateCompilationConfig()
, MaterialCompilationOutput(nullptr)
, CompilerLegacy(nullptr)
, CompilerNew(nullptr)
{
}

FSubstrateTranslatorData::~FSubstrateTranslatorData()
{

}


void FSubstrateTranslatorData::Initialize(UMaterialInterface* InMaterialInterface, FMaterialCompilationOutput& InMaterialCompilationOutput, FMaterialCompiler* InCompilerLegacy, MIR::FEmitter* InCompilerNew)
{
	check(!!InCompilerLegacy != !!InCompilerNew); // XOR

	FrontMaterialExpr = nullptr;

	MaterialCompilationOutput = &InMaterialCompilationOutput;
	CompilerLegacy = InCompilerLegacy;
	CompilerNew = InCompilerNew;

	if (CompilerLegacy)
	{
		CookTargetedMaterialQualityLevel = CompilerLegacy->GetQualityLevel();
	}
	else if (CompilerNew)
	{
		CookTargetedMaterialQualityLevel = CompilerNew->GetQualityLevel();
	}
	else
	{
		check(false);
	}

	MaterialInterface = InMaterialInterface;

	if (MaterialInterface)
	{
		AssetName = MaterialInterface->GetName();
		AssetPath = MaterialInterface->GetPathName();
	}
}

void FSubstrateTranslatorData::Errorf(const FString& Format)
{
	if (CompilerLegacy)
	{
		CompilerLegacy->Errorf(*Format);
	}
	else
	{
		CompilerNew->Error(Format);
	}
}

EShaderPlatform FSubstrateTranslatorData::GetShaderPlatform()
{
	if (CompilerLegacy)
	{
		return CompilerLegacy->GetShaderPlatform();
	}
	return CompilerNew->GetShaderPlatform();
}

FMaterialShadingModelField FSubstrateTranslatorData::GetMaterialShadingModels()
{
	return MaterialShadingModels;
}

void FSubstrateTranslatorData::ProcessSubstrateTreeAndTopology(
	struct FMaterialShadingModelField& InMaterialShadingModels, 
	UMaterialExpression* ExpressionToPreview,
	bool& bDoClearFunctionStack)
{
	EShaderPlatform Platform = GetShaderPlatform();
	bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	bProjectSubstrateHiddenMaterialAssetConversionEnabled = Substrate::IsHiddenMaterialAssetConversionEnabled();
	MaterialShadingModels = InMaterialShadingModels;
	FrontMaterialExpr = nullptr;
	int32 FrontMaterialOutputIndex = INDEX_NONE;

	bSubstrateWritesEmissive = false;
	bSubstrateWritesAmbientOcclusion = false;

	if (ExpressionToPreview)
	{
		// If this is a compilation used for a preview of a node from am aterial graph, then we need ot use that node as the root of the Substrate tree.
		// Then the resulting SubstrateData will be converted to a color using SubstrateCompilePreview translator function when compiling the emissive color channel the node has been short circuited into.
		const uint32 ExpressionPreviewOutputIndex = 0;
		if (ExpressionToPreview && ExpressionToPreview->IsResultSubstrateMaterial(ExpressionPreviewOutputIndex))
		{
			FrontMaterialExpr = ExpressionToPreview;
			FrontMaterialOutputIndex = ExpressionPreviewOutputIndex;
		}
		else
		{
			FrontMaterialExpr = nullptr;	// This is not a Substrate input that is connected there so we cannot create a Substrate tree.
		}
	}
	else
	{
		FSubstrateMaterialInput* FrontMaterialInput = MaterialInterface ? &MaterialInterface->GetMaterial()->GetEditorOnlyData()->FrontMaterial : nullptr;
		FrontMaterialExpr = FrontMaterialInput ? FrontMaterialInput->GetTracedInput().Expression : nullptr;
		FrontMaterialOutputIndex = FrontMaterialInput ? FrontMaterialInput->OutputIndex : INDEX_NONE;
	}

	bMaterialUsesRootNodeToSubstrateHiddenConversion = bSubstrateEnabled && bProjectSubstrateHiddenMaterialAssetConversionEnabled && !FrontMaterialExpr;

	// If bSubstrateMaterialUsesLegacyMaterialCompilation logic is changed, then FMaterialResource::IsSubstrateMaterial must be updated.
	bSubstrateMaterialUsesLegacyMaterialCompilation = bMaterialUsesRootNodeToSubstrateHiddenConversion && Substrate::IsSubstrateBlendableGBufferEnabled(Platform);
	if (bSubstrateMaterialUsesLegacyMaterialCompilation)
	{
		// In this case we use the legacy material shader: faster to compile and less instructions (no substrate conversion)
		bSubstrateEnabled = false;
		bProjectSubstrateHiddenMaterialAssetConversionEnabled = false;
		bMaterialUsesRootNodeToSubstrateHiddenConversion = false;

		// Still set some Substrate material compilation output that are needed at runtime.
		MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateClosureCount = 1;
		MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateUintPerPixel = 0;
		MaterialCompilationOutput->SubstrateMaterialCompilationOutput.bMaterialUsesLegacyGBufferDataPassThrough = 1;
		if (MaterialShadingModels.HasOnlyShadingModel(MSM_DefaultLit))
		{
			MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_SIMPLE;
		}
		else
		{
			if (MaterialShadingModels.HasShadingModel(MSM_Cloth))
			{
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_SINGLE;
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature::Fuzz;
			}
			if (MaterialShadingModels.HasShadingModel(MSM_ClearCoat))
			{
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_SINGLE;
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat;
			}
			if (MaterialShadingModels.HasAnyShadingModel({ MSM_Subsurface,MSM_SubsurfaceProfile,MSM_TwoSidedFoliage,MSM_Eye,MSM_PreintegratedSkin }))
			{
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_SINGLE;
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature::SSS;
			}
			if (MaterialShadingModels.HasShadingModel(MSM_Eye))
			{
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_COMPLEX;
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature::Eye;
			}
			if (MaterialShadingModels.HasShadingModel(MSM_Hair))
			{
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialType = SUBSTRATE_MATERIAL_TYPE_COMPLEX;
				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures |= ESubstrateBsdfFeature::Hair;
			}
		}
	}

	if (bMaterialUsesRootNodeToSubstrateHiddenConversion)
	{
		UMaterial* BaseMaterial = MaterialInterface ? MaterialInterface->GetMaterial() : nullptr;
		UMaterialEditorOnlyData* EditorOnlyData = BaseMaterial ? BaseMaterial->GetEditorOnlyData() : nullptr;

		{
			// Gather input for custom output
			UMaterialExpressionThinTranslucentMaterialOutput* ThinTranslucentOutput = nullptr;
			UMaterialExpressionSingleLayerWaterMaterialOutput* SingleLayerWaterOutput = nullptr;
			UMaterialExpressionClearCoatNormalCustomOutput* ClearCoatBottomNormalOutput = nullptr;
			UMaterialExpressionTangentOutput* TangentOutput = nullptr;
			TArray<class UMaterialExpressionCustomOutput*> CustomOutputExpressions;
			if (BaseMaterial)
			{
				BaseMaterial->GetAllCustomOutputExpressions(CustomOutputExpressions);
			}

			for (UMaterialExpressionCustomOutput* Expression : CustomOutputExpressions)
			{
				// Gather custom output for thin translucency
				if (ThinTranslucentOutput == nullptr && Cast<UMaterialExpressionThinTranslucentMaterialOutput>(Expression))
				{
					ThinTranslucentOutput = Cast<UMaterialExpressionThinTranslucentMaterialOutput>(Expression);
					check(ThinTranslucentOutput);

					ThinTranslucentTransmittanceColor = &ThinTranslucentOutput->TransmittanceColor;
					ThinTranslucentSurfaceCoverage    = &ThinTranslucentOutput->SurfaceCoverage;
				}

				// Gather custom output for single layer water
				if (SingleLayerWaterOutput == nullptr && Cast<UMaterialExpressionSingleLayerWaterMaterialOutput>(Expression))
				{
					SingleLayerWaterOutput = Cast<UMaterialExpressionSingleLayerWaterMaterialOutput>(Expression);
					check(SingleLayerWaterOutput);

					WaterScatteringCoefficients = &SingleLayerWaterOutput->ScatteringCoefficients;
					WaterAbsorptionCoefficients = &SingleLayerWaterOutput->AbsorptionCoefficients;
					WaterPhaseG = &SingleLayerWaterOutput->PhaseG;
					ColorScaleBehindWater = &SingleLayerWaterOutput->ColorScaleBehindWater;
				}

				// Gather custom output for clear coat
				if (ClearCoatBottomNormalOutput == nullptr && Cast<UMaterialExpressionClearCoatNormalCustomOutput>(Expression))
				{
					ClearCoatBottomNormalOutput = Cast<UMaterialExpressionClearCoatNormalCustomOutput>(Expression);
					check(ClearCoatBottomNormalOutput);
					ClearCoatNormal = &ClearCoatBottomNormalOutput->Input;
				}

				// Gather custom output for tangent (unused atm)
				if (TangentOutput == nullptr && Cast<UMaterialExpressionTangentOutput>(Expression))
				{
					TangentOutput = Cast<UMaterialExpressionTangentOutput>(Expression);
					check(TangentOutput);
					CustomTangent = &TangentOutput->Input;
				}

				if (ThinTranslucentOutput && SingleLayerWaterOutput && ClearCoatBottomNormalOutput && TangentOutput)
				{
					break;
				}
			}
		}

		for (uint32 SubstrateCompilationContextIndex = 0; SubstrateCompilationContextIndex < ESubstrateCompilationContext::SCC_MAX; ++SubstrateCompilationContextIndex)
		{
			// This is needed because SubstrateGenerateMaterialTopologyTree will call some compiler context though material expressions.
			CurrentSubstrateCompilationContext = ESubstrateCompilationContext(SubstrateCompilationContextIndex);

			FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[SubstrateCompilationContextIndex];
			SubstrateCtx = FSubstrateCompilationContext(ESubstrateCompilationContext(SubstrateCompilationContextIndex));

			if (EditorOnlyData)
			{
				FGuid RootNodeConversionGUID = FGuid(0, 1, 2, 3); // This dummy GUID is the root node.
				bSubstrateWritesEmissive |= EditorOnlyData->EmissiveColor.IsConnected();
				bSubstrateWritesAmbientOcclusion |= EditorOnlyData->AmbientOcclusion.IsConnected();

				FExpressionInput* SurfaceThickness = nullptr;

				SubstrateThicknessStackPush(nullptr, SurfaceThickness);

				if (BaseMaterial && BaseMaterial->bUseMaterialAttributes)
				{
					UMaterialExpressionSubstrateConvertMaterialAttributes::SubstrateGenerateMaterialTopologyTreeCommon(
						*this, RootNodeConversionGUID, nullptr /*Parent*/, 0,
						BaseMaterial ? FMaterialAttributeDefinitionMap::GetConnectedMaterialAttributesBitmask(BaseMaterial->GetExpressions()) : 0,
						EditorOnlyData->ShadingModelFromMaterialExpression.IsConnected(),
						EditorOnlyData->MaterialAttributes.IsConnected(MP_EmissiveColor));
				}
				else if (BaseMaterial &&  (BaseMaterial->MaterialDomain == MD_Surface || BaseMaterial->MaterialDomain == MD_RuntimeVirtualTexture || BaseMaterial->MaterialDomain == MD_DeferredDecal) && EditorOnlyData)
				{
					FSubstrateOperator* Op = UMaterialExpressionSubstrateShadingModels::SubstrateGenerateMaterialTopologyTreeCommon(
						*this, RootNodeConversionGUID, nullptr /*Parent*/, 0,
						EditorOnlyData->EmissiveColor,
						EditorOnlyData->Anisotropy,
						*ClearCoatNormal,
						*CustomTangent,
						EditorOnlyData->ShadingModelFromMaterialExpression);

					if (BaseMaterial && BaseMaterial->MaterialDomain == MD_DeferredDecal)
					{
						// We notify that this is for decal using this usage flag.
						// Otherwise, we do not need to do anything else, the ConvertToDecal node is only here to enforce parameter blending.
						// This in order to get a signle BSDF as input to convert as a decal.
						Op->SubUsage |= SUBSTRATE_OPERATOR_SUBUSAGE_DECAL;
					}
				}
				else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_Volume && EditorOnlyData)
				{
					UMaterialExpressionSubstrateVolumetricFogCloudBSDF::SubstrateGenerateMaterialTopologyTreeCommon(
						*this, RootNodeConversionGUID, nullptr /*Parent*/, 0,
						EditorOnlyData->EmissiveColor,
						EditorOnlyData->AmbientOcclusion);
				}
				else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_LightFunction && EditorOnlyData)
				{
					UMaterialExpressionSubstrateLightFunction::SubstrateGenerateMaterialTopologyTreeCommon(
						*this, RootNodeConversionGUID, nullptr /*Parent*/, 0);
				}
				else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_PostProcess && EditorOnlyData)
				{
					UMaterialExpressionSubstratePostProcess::SubstrateGenerateMaterialTopologyTreeCommon(
						*this, RootNodeConversionGUID, nullptr /*Parent*/, 0);
				}
				else if (BaseMaterial && BaseMaterial->MaterialDomain == MD_UI && EditorOnlyData)
				{
					UMaterialExpressionSubstrateUI::SubstrateGenerateMaterialTopologyTreeCommon(
						*this, RootNodeConversionGUID, nullptr /*Parent*/, 0);
				}

				SubstrateThicknessStackPop();

				check(SubstrateCtx.SubstrateThicknessStack.Num() == 0);

				if (SubstrateCtx.bSubstrateTreeOutOfStackDepthOccurred)
				{
					Errorf(FString::Printf(TEXT(" %s [%s]: Substrate - Cyclic graph detected when we only support acyclic graph."), *GetAssetName(), *GetAssetPath()));
					return;
				}
				if (!SubstrateCtx.SubstrateGenerateDerivedMaterialOperatorData(*this))
				{
					Errorf(FString::Printf(TEXT("Substrate material errors encountered.")));
					return;
				}
			}

		}
		CurrentSubstrateCompilationContext = ESubstrateCompilationContext::SCC_Default;
	}
	else if (bSubstrateEnabled && FrontMaterialExpr)
	{
		for (uint32 SubstrateCompilationContextIndex = 0; SubstrateCompilationContextIndex < ESubstrateCompilationContext::SCC_MAX; ++SubstrateCompilationContextIndex)
		{
			// This is needed because SubstrateGenerateMaterialTopologyTree will call some compiler context though material expressions.
			CurrentSubstrateCompilationContext = ESubstrateCompilationContext(SubstrateCompilationContextIndex);

			FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[SubstrateCompilationContextIndex];
			SubstrateCtx = FSubstrateCompilationContext(ESubstrateCompilationContext(SubstrateCompilationContextIndex));

			UMaterial* Material = MaterialInterface ? MaterialInterface->GetMaterial() : nullptr;
			FScalarMaterialInput* SurfaceThickness = Material && Material->IsThinSurface() ? &Material->GetEditorOnlyData()->SurfaceThickness : nullptr;

			SubstrateThicknessStackPush(nullptr, SurfaceThickness);

			FrontMaterialExpr->SubstrateGenerateMaterialTopologyTree(*this, nullptr, FrontMaterialOutputIndex);
			SubstrateThicknessStackPop();

			check(SubstrateCtx.SubstrateThicknessStack.Num() == 0);

			if (SubstrateCtx.bSubstrateTreeOutOfStackDepthOccurred)
			{
				Errorf(FString::Printf(TEXT(" %s [%s]: Substrate - Cyclic graph detected when we only support acyclic graph."), *GetAssetName(), *GetAssetPath()));
				return;
			}
			if (!SubstrateCtx.SubstrateGenerateDerivedMaterialOperatorData(*this))
			{
				Errorf(FString::Printf(TEXT("Substrate material errors encountered.")));
				return;
			}

			bSubstrateWritesEmissive |= SubstrateCtx.bSubstrateWritesEmissive;
			bSubstrateWritesAmbientOcclusion |= SubstrateCtx.bSubstrateWritesAmbientOcclusion;
		}
		CurrentSubstrateCompilationContext = ESubstrateCompilationContext::SCC_Default;

		bDoClearFunctionStack = true;
	}
}

bool FSubstrateTranslatorData::GenerateSubstrateTreeHLSLCode(bool bSubstrateFrontMaterialProvided)
{
	if (bSubstrateEnabled)
	{
		for (uint32 SubstrateCompilationContextIndex = 0; SubstrateCompilationContextIndex < ESubstrateCompilationContext::SCC_MAX; ++SubstrateCompilationContextIndex)
		{
			FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[SubstrateCompilationContextIndex];

			FString TreeFunctionPostFix = "ERROR";
			switch (SubstrateCompilationContextIndex)
			{
			case ESubstrateCompilationContext::SCC_Default:
				TreeFunctionPostFix = TEXT("");
				break;
			case ESubstrateCompilationContext::SCC_FullySimplified:
				TreeFunctionPostFix = TEXT("_FullySimplified");
				break;
			default:
				check(false);
			}

			bool bSubstrateFrontMaterialIsValid = bSubstrateFrontMaterialProvided;
			if (bSubstrateFrontMaterialIsValid)
			{
				// The material can be null when some entries are automatically generated, for instance in the material layer blending system
				bSubstrateFrontMaterialIsValid &= SubstrateCtx.SubstrateMaterialEffectiveClosureCount > 0;

				UMaterial* Material = MaterialInterface ? MaterialInterface->GetMaterial() : nullptr;
				if (!bSubstrateFrontMaterialIsValid && Material && Material->bUseMaterialAttributes)
				{
					Errorf(FString::Printf(TEXT(" %s [%s]: Substrate - Material has no BSDF: this can happen when the SubstrateConvertMaterialAttributes node is not used and when material attributes are enabled."), *GetAssetName(), *GetAssetPath()));
					return false;
				}
			}

			if (bMaterialUsesRootNodeToSubstrateHiddenConversion)
			{
				// Hardcoded substrate tree node visit when using hidden conversion.
				bMaterialIsSubstrate = true;

				if (SubstrateCompilationContextIndex == ESubstrateCompilationContext::SCC_Default)
				{
					MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialDescription = "";
					SubstrateTreeHLSL += "// Substrate: HiddenMaterialAssetConversion\n";

					// Adde default Substrate functions
					SubstrateTreeHLSL += "#if TEMPLATE_USES_SUBSTRATE\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit(float3 V) {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance(FSubstrateIntegrationSettings Settings, float3 V)\n"
										"{\n"
										"	#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION\n"
										"						SubstrateTree.UpdateSingleBSDFOperatorCoverageTransmittance(this, 0, Settings, V);\n"
										"	#else\n"
										"						UpdateSingleBSDFOperatorCoverageTransmittance(SubstrateTree, this, 0, Settings, V);\n"
										"	#endif\n"
										"}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance() {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit() {}\n";
					SubstrateTreeHLSL += "#endif // TEMPLATE_USES_SUBSTRATE\n";
				}
				else
				{
					SubstrateTreeHLSL += "#if TEMPLATE_USES_SUBSTRATE\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified(float3 V) {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance_FullySimplified(FSubstrateIntegrationSettings Settings, float3 V)\n"
										"{\n"
										"	#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION\n"
										"						SubstrateTree.UpdateSingleBSDFOperatorCoverageTransmittance(this, 0, Settings, V);\n"
										"	#else\n"
										"						UpdateSingleBSDFOperatorCoverageTransmittance(SubstrateTree, this, 0, Settings, V);\n"
										"	#endif\n"
										"}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance_FullySimplified() {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified() {}\n";
					SubstrateTreeHLSL += "#endif // TEMPLATE_USES_SUBSTRATE\n";
					}
			}
			else if (bSubstrateFrontMaterialIsValid)
			{
				bMaterialIsSubstrate = true;

				if (SubstrateCtx.SubstrateMaterialRootOperator)
				{
					// Now implement the functions needed to process the material topology

					// Pre-Update the slab BSDF with operators (like thin film coating, which can alter F0/F90)
					{
						// Update the coverage/transmittance of each node in the graph
						check(SubstrateCtx.SubstrateMaterialRootOperator);
						int32 RootMaximumDistanceToLeaves = SubstrateCtx.SubstrateMaterialRootOperator->MaxDistanceFromLeaves;

						SubstrateTreeHLSL += FString::Printf(TEXT("void  FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit%s(float3 V)\n"), *TreeFunctionPostFix);
						SubstrateTreeHLSL += "{\n";
						for (uint32 ClosureIndex = 0; ClosureIndex < SubstrateCtx.SubstrateMaterialEffectiveClosureCount; ++ClosureIndex)
						{
							SubstrateTreeHLSL += "\t{\n";
							for (auto& It : SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators)
							{
								if (!It.IsDiscarded() && It.BSDFIndex == ClosureIndex)
								{
									// Walk up the graph to the root node and apply weight factors
									std::function<void(const FSubstrateOperator&, int32)> WalkOperatorsUp = [&](const FSubstrateOperator& CurrentOperator, int32 PreviousOperatorIndex) -> void
										{
											switch (CurrentOperator.OperatorType)
											{
											case SUBSTRATE_OPERATOR_WEIGHT:
											{
												break; // NOP
											}
											case SUBSTRATE_OPERATOR_HORIZONTAL:
											{
												break; // NOP
											}
											case SUBSTRATE_OPERATOR_VERTICAL:
											{
												// example SubstrateTreeHLSL += FString::Printf(TEXT("\t PreUpdateAllBSDFWithBottomUpOperatorVisit_Vertical(this, SubstrateTree, SubstrateTree.BSDFs[%d], V, %d /*Op index*/, %d /*PreviousIsInputA*/);\n"), BSDFIndex, CurrentOperator.Index, CurrentOperator.LeftIndex == PreviousOperatorIndex ? 1 : 0);
												break; // NOP
											}
											case SUBSTRATE_OPERATOR_ADD:
											{
												break; // NOP
											}
											case SUBSTRATE_OPERATOR_SELECT:
											{
												break; // NOP
											}
											default:
											case SUBSTRATE_OPERATOR_BSDF:
											{
												check(false);
											}
											}

											if (CurrentOperator.ParentIndex != INDEX_NONE)
											{
												WalkOperatorsUp(SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.ParentIndex], CurrentOperator.Index);
											}
										};

									const int32 BSDFOperatorIndex = It.Index;
									const FSubstrateOperator& BSDFOperator = SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[BSDFOperatorIndex];

									// Start visiting node up from the BSDF leaf only if it has a parent.
									if (BSDFOperator.ParentIndex != INDEX_NONE)
									{
										WalkOperatorsUp(SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[BSDFOperator.ParentIndex], BSDFOperator.Index);
									}
								}
							}
							SubstrateTreeHLSL += "\t}\n";
						}
						SubstrateTreeHLSL += "}\n";
					}

					// Update the coverage/transmittance of each leaves (==BSDFs) of the Substrate tree.
					{
						SubstrateTreeHLSL += FString::Printf(TEXT("void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance%s(FSubstrateIntegrationSettings Settings, float3 V)\n"), *TreeFunctionPostFix);
						SubstrateTreeHLSL += "{\n";
						for (uint32 ClosureIndex = 0; ClosureIndex < SubstrateCtx.SubstrateMaterialEffectiveClosureCount; ++ClosureIndex)
						{
							SubstrateTreeHLSL += FString::Printf(TEXT("\t #if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION\n"));
							SubstrateTreeHLSL += FString::Printf(TEXT("\t SubstrateTree.UpdateSingleBSDFOperatorCoverageTransmittance(this, %d, Settings, V);\n"), ClosureIndex);
							SubstrateTreeHLSL += FString::Printf(TEXT("\t #else\n"));
							SubstrateTreeHLSL += FString::Printf(TEXT("\t UpdateSingleBSDFOperatorCoverageTransmittance(SubstrateTree, this, %d, Settings, V);\n"), ClosureIndex);
							SubstrateTreeHLSL += FString::Printf(TEXT("\t #endif\n"));
						}
						SubstrateTreeHLSL += "}\n";
					}

					// Propagate up the coverage/transmittance of each node in the Substrate tree.
					// For that we visit all the operator according to their distance from the Substrate tree leaves, from small to large.
					{
						check(SubstrateCtx.SubstrateMaterialRootOperator);
						int32 RootMaximumDistanceToLeaves = SubstrateCtx.SubstrateMaterialRootOperator->MaxDistanceFromLeaves;

						SubstrateTreeHLSL += FString::Printf(TEXT("void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance%s()\n"), *TreeFunctionPostFix);
						SubstrateTreeHLSL += "{\n";
						for (int32 DistanceToLeaves = 1; DistanceToLeaves <= RootMaximumDistanceToLeaves; ++DistanceToLeaves)
						{
							SubstrateTreeHLSL += FString::Printf(TEXT("\t// MaxDistanceFromLeaves = %d \n"), DistanceToLeaves);
							for (auto& It : SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators)
							{
								if (!It.IsDiscarded() && It.MaxDistanceFromLeaves == DistanceToLeaves)
								{
									SubstrateTreeHLSL += FString::Printf(TEXT("\t SubstrateTree.UpdateSingleOperatorCoverageTransmittance(%d /*operator index*/);\n"), It.Index);
								}
							}
						}
						SubstrateTreeHLSL += "}\n";
					}

					// Update the luminance weight of each BSDF according to the operators it has to traverse bottom-up to the Substrate tree root node.
					{
						// Update the coverage/transmittance of each node in the graph
						check(SubstrateCtx.SubstrateMaterialRootOperator);
						int32 RootMaximumDistanceToLeaves = SubstrateCtx.SubstrateMaterialRootOperator->MaxDistanceFromLeaves;

						SubstrateTreeHLSL += FString::Printf(TEXT("void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit%s()\n"), *TreeFunctionPostFix);
						SubstrateTreeHLSL += "{\n";
						for (uint32 ClosureIndex = 0; ClosureIndex < SubstrateCtx.SubstrateMaterialEffectiveClosureCount; ++ClosureIndex)
						{
							for (auto& It : SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators)
							{
								if (!It.IsDiscarded() && It.BSDFIndex == ClosureIndex)
								{
									// Walk up the graph to the root node and apply weight factors
									std::function<void(const FSubstrateOperator&, int32)> WalkOperatorsUp = [&](const FSubstrateOperator& CurrentOperator, int32 PreviousOperatorIndex) -> void
										{
											switch (CurrentOperator.OperatorType)
											{
											case SUBSTRATE_OPERATOR_WEIGHT:
											{
												SubstrateTreeHLSL += FString::Printf(TEXT("\t SubstrateTree.UpdateAllBSDFWithBottomUpOperatorVisit_Weight(%d /*BSDFIndex*/, %d /*Op index*/, %d /*PreviousIsInputA*/);\n"), ClosureIndex, CurrentOperator.Index, 1);
												break;
											}
											case SUBSTRATE_OPERATOR_HORIZONTAL:
											{
												SubstrateTreeHLSL += FString::Printf(TEXT("\t SubstrateTree.UpdateAllBSDFWithBottomUpOperatorVisit_Horizontal(%d /*BSDFIndex*/, %d /*Op index*/, %d /*PreviousIsInputA*/);\n"), ClosureIndex, CurrentOperator.Index, CurrentOperator.LeftIndex == PreviousOperatorIndex ? 1 : 0);
												break;
											}
											case SUBSTRATE_OPERATOR_VERTICAL:
											{
												SubstrateTreeHLSL += FString::Printf(TEXT("\t SubstrateTree.UpdateAllBSDFWithBottomUpOperatorVisit_Vertical(%d /*BSDFIndex*/, %d /*Op index*/, %d /*PreviousIsInputA*/);\n"), ClosureIndex, CurrentOperator.Index, CurrentOperator.LeftIndex == PreviousOperatorIndex ? 1 : 0);
												break;
											}
											case SUBSTRATE_OPERATOR_ADD:
											case SUBSTRATE_OPERATOR_SELECT:
											{
												break; // NOP
											}
											default:
											case SUBSTRATE_OPERATOR_BSDF:
											{
												check(false);
											}
											}

											if (CurrentOperator.ParentIndex != INDEX_NONE)
											{
												WalkOperatorsUp(SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[CurrentOperator.ParentIndex], CurrentOperator.Index);
											}
										};

									const int32 BSDFOperatorIndex = It.Index;
									const FSubstrateOperator& BSDFOperator = SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[BSDFOperatorIndex];

									// Start visiting node up from the BSDF leaf only if it has a parent.
									if (BSDFOperator.ParentIndex != INDEX_NONE)
									{
										WalkOperatorsUp(SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[BSDFOperator.ParentIndex], BSDFOperator.Index);
									}
								}
							}
						}
						SubstrateTreeHLSL += "}\n";
					}
				}

				// Check if normal/tangent basis are valid
				{
					SubstrateCtx.FinalUsedSharedLocalBasesCount = 0;
					uint8 RequestedSharedLocalBasesCount = 0;
					SubstrateCtx.SubstrateEvaluateSharedLocalBases(RequestedSharedLocalBasesCount, nullptr, nullptr);// CompilerNew);
					if (RequestedSharedLocalBasesCount > SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS)
					{
						Errorf(FString::Printf(TEXT(" %s [%s]: Substrate - Material has more unique normal/tangent basis than the allowed limit %d/%d."), *GetAssetName(), *GetAssetPath(), RequestedSharedLocalBasesCount, SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS));
					}
				}
			}
			else
			{
				if (SubstrateCompilationContextIndex == ESubstrateCompilationContext::SCC_Default)
				{
					MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialDescription = "";
					SubstrateTreeHLSL += "// No Substrate material provided\n";

					// Adde default Substrate functions
					SubstrateTreeHLSL += "#if TEMPLATE_USES_SUBSTRATE\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit(float3 V) {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance(FSubstrateIntegrationSettings Settings, float3 V) {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance() {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit() {}\n";
					SubstrateTreeHLSL += "#endif\n";
				}
				else
				{
					SubstrateTreeHLSL += "#if TEMPLATE_USES_SUBSTRATE\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified(float3 V) {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance_FullySimplified(FSubstrateIntegrationSettings Settings, float3 V) {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance_FullySimplified() {}\n";
					SubstrateTreeHLSL += "void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified() {}\n";
					SubstrateTreeHLSL += "#endif\n";
				}
			}
		}
	}
	return true;
}


void FSubstrateTranslatorData::GeneratePixelMemberDeclarationHLSLCode(FString& OutHLSLCode, const FString& FrontMaterialSubstrateDataHLSLType)
{
	OutHLSLCode += FString::Printf(TEXT("\t#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1\n"));
	OutHLSLCode += FString::Printf(TEXT("\t%s FullySimplifiedFrontMaterial;\n"), *FrontMaterialSubstrateDataHLSLType);
	OutHLSLCode += FString::Printf(TEXT("\t#endif\n"));
}

void FSubstrateTranslatorData::GeneratePixelInputInitializerHLSLCode(FString& OutHLSLCode)
{
	if (CompilerNew)
	{
		OutHLSLCode += FString::Printf(TEXT("\tPixelMaterialInputs.FrontMaterial = Parameters.SubstrateFrontMaterial0;\n"));
		OutHLSLCode += FString::Printf(TEXT("\t#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1\n"));
		OutHLSLCode += FString::Printf(TEXT("\tPixelMaterialInputs.FullySimplifiedFrontMaterial = Parameters.SubstrateFrontMaterialFullySimplified0;\n"));
		OutHLSLCode += FString::Printf(TEXT("\t#endif\n"));
	}
	else if (CompilerLegacy)
	{
		if (!FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunks.IsEmpty())
		{
			OutHLSLCode += FString::Printf(TEXT("\t#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1\n"));
			OutHLSLCode += FString::Printf(TEXT("\tPixelMaterialInputs.FullySimplifiedFrontMaterial = %s;\n"), *FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunks);
			OutHLSLCode += FString::Printf(TEXT("\t#endif\n"));
		}
	}
	else
	{
		check(false);
	}
}

void FSubstrateTranslatorData::GeneratePixelNormalInitializerHLSLCode(FString& OutHLSLCode)
{
	for (uint32 SubstrateCompilationContextIndex = 0; SubstrateCompilationContextIndex < ESubstrateCompilationContext::SCC_MAX; ++SubstrateCompilationContextIndex)
	{
		FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[SubstrateCompilationContextIndex];
		if (SubstrateCtx.CodeChunkToSubstrateSharedLocalBasis.Num() > 0)
		{
			OutHLSLCode += SubstrateCtx.SubstratePixelNormalInitializerValues;
		}
	}
	if (CompilerNew)
	{
		if (bMaterialUsesRootNodeToSubstrateHiddenConversion)
		{
			// See bMaterialUsesRootNodeToSubstrateHiddenConversion from MaterialIRModuleBuilder as to why we do that.
			OutHLSLCode += FString::Printf(TEXT("\t#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1\n"));
			OutHLSLCode += FString::Printf(TEXT("\tParameters.SharedLocalBasesFullySimplified = Parameters.SharedLocalBases;\n"));
			OutHLSLCode += FString::Printf(TEXT("\tParameters.SubstrateTreeFullySimplified = Parameters.SubstrateTree;\n"));
			OutHLSLCode += FString::Printf(TEXT("\t#endif\n"));
		}
	}
}

void FSubstrateTranslatorData::GenerateMaterialCompilationOutputAndDefines(
	bool bOpacityPropertyIsUsed,
	int32 MaterialExportType, int32 MaterialExportContext, int32 MaterialExportLegacyBlendMode,
	FString& DescriptionStringCommentForDebug,
	FSubstrateDefines* OutSubstrateDefines
)
{
	bSubstratePremultipliedAlphaOpacityOverridden = bMaterialIsSubstrate && bOpacityPropertyIsUsed;

	UMaterial* Material = MaterialInterface ? MaterialInterface->GetMaterial() : nullptr;
	bDistortionAccountForCoverage = bSubstrateEnabled && !bMaterialUsesRootNodeToSubstrateHiddenConversion && Material && Material->RefractionCoverageMode == RCM_CoverageAccountedFor;

	if (bMaterialIsSubstrate)
	{
		SubstrateMaterialExportType = MaterialExportType;
		SubstrateMaterialExportContext = MaterialExportContext;
		SubstrateMaterialExportLegacyBlendMode = MaterialExportLegacyBlendMode;

		// Unlit cannot be combined with other BSDF so we can simply pick the default strata context
		bSubstrateOptimizedUnlit = SubstrateCompilationContext[ESubstrateCompilationContext::SCC_Default].bSubstrateMaterialIsUnlitNode;

		bSubstrateRoughnessTracking = Material && Material->HasSubstrateRoughnessTracking();

		{
			// For now, the fully simplified mode is used for Lumen or anything else supported inlined evaluation. The export is only valid for the default case.
			// STRATA_TODO: generate an export for the different context (need to generate two export functions: the default one and the FullSimplification one)
			FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[ESubstrateCompilationContext::SCC_Default];
			bSubstrateSinglePath = SubstrateCtx.SubstrateMaterialComplexity.IsSingle();
			bSubstrateFastPath = SubstrateCtx.SubstrateMaterialComplexity.IsSimple();
			SubstrateClampedClosureCount = SubstrateCtx.SubstrateMaterialEffectiveClosureCount;
			bSubstrateComplexSpecialPath = SubstrateCtx.SubstrateMaterialComplexity.IsComplexSpecial();
		}

		bool EyeIrisNormalPluggedIn = false;
		bool EyeIrisTangentPluggedIn = false;

		FString SubstrateMaterialDescription;
		for (uint32 SubstrateCompilationContextIndex = 0; SubstrateCompilationContextIndex < ESubstrateCompilationContext::SCC_MAX; ++SubstrateCompilationContextIndex)
		{
			ESubstrateBsdfFeature MaterialBsdfFeatures = ESubstrateBsdfFeature::None;

			FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[SubstrateCompilationContextIndex];
			const FSubstrateSimplificationStatus& SubstrateSimplificationStatus = SubstrateCtx.SubstrateSimplificationStatus;

			check(SubstrateCtx.SubstrateMaterialRootOperator);
			uint32 RootMaximumDistanceToLeaves = SubstrateCtx.SubstrateMaterialRootOperator->MaxDistanceFromLeaves;

			// Compute the shared local basis count and generate the hlsl shader code for it.
			SubstrateCtx.SubstratePixelNormalInitializerValues = FString::Printf(TEXT("\n\n\n\t// Substrate normal and tangent\n"));
			SubstrateCtx.FinalUsedSharedLocalBasesCount = 0;
			uint8 RequestedSharedLocalBasesCount = 0;
			SubstrateCtx.SubstrateEvaluateSharedLocalBases(RequestedSharedLocalBasesCount, OutSubstrateDefines, nullptr);

#if WITH_EDITOR
			// Now write some feedback to the user, but only produce debug string if in editor
			{
				// Output some debug info as comment in code and in the material stat window
				const uint32 SubstrateBytePerPixel_Platform = Substrate::GetBytePerPixel(GetShaderPlatform());
				const uint32 SubstrateClosurePerPixel_Platform = Substrate::GetClosurePerPixel(GetShaderPlatform(), GetCookTargetedMaterialQualityLevel());
				FString SubstrateMaterialContextDescription;

				auto GetSubstrateCompilationContextName = [](uint32 Index)
				{
					switch (Index)
					{
					case ESubstrateCompilationContext::SCC_Default:
						return TEXT("Default");
					case ESubstrateCompilationContext::SCC_FullySimplified:
						return TEXT("FullySimplified");
					default:
						check(false);
					}
					return TEXT("ERROR");
				};
				FString SubstrateCompilationContextName = GetSubstrateCompilationContextName(SubstrateCompilationContextIndex);
				SubstrateMaterialContextDescription += FString::Printf(TEXT("\n\n----- SUBSTRATE - %s -----\n"), *SubstrateCompilationContextName);
				SubstrateMaterialContextDescription += FString::Printf(TEXT("SubstrateCompilationInfo -\n"));
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Byte Per Pixel Budget                           %u\n"), SubstrateBytePerPixel_Platform);
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Closure Per Pixel Budget                        %u\n"), SubstrateClosurePerPixel_Platform);
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Requested Byte Size before simplification       %u (%d UINT32)\n"), SubstrateSimplificationStatus.OriginalRequestedByteSize, SubstrateSimplificationStatus.OriginalRequestedByteSize / 4);
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Requested Byte Size after simplification        %u (%d UINT32)\n"), SubstrateCtx.SubstrateMaterialRequestedSizeByte, SubstrateCtx.SubstrateMaterialRequestedSizeByte / 4);
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Requested Closure Count before simplification   %u\n"), SubstrateSimplificationStatus.OriginalRequestedClosureCount);
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Requested Closure Count after simplification    %u\n"), SubstrateCtx.SubstrateMaterialClosureCount);
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - Material complexity                             %s\n"), *FSubstrateMaterialComplexity::ToString(SubstrateCtx.SubstrateMaterialComplexity.SubstrateMaterialType(), true /* Upper case */));
				SubstrateMaterialContextDescription += FString::Printf(TEXT(" - BSDF Count                                      %i\n"), SubstrateCtx.SubstrateMaterialEffectiveClosureCount); // REMOVE?
				if (RequestedSharedLocalBasesCount > SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS)
				{
					SubstrateMaterialDescription += FString::Printf(TEXT(" - SharedLocalBasesCount                      %i (Requested:%i)\n"), SubstrateCtx.FinalUsedSharedLocalBasesCount, RequestedSharedLocalBasesCount);
				}
				else
				{
					SubstrateMaterialDescription += FString::Printf(TEXT(" - SharedLocalBasesCount                      %i\n"), SubstrateCtx.FinalUsedSharedLocalBasesCount);
				}


				for (int32 OpIt = 0; OpIt < SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators.Num(); ++OpIt)
				{
					FSubstrateOperator& BSDFOperator = SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[OpIt];

					EyeIrisNormalPluggedIn  = EyeIrisNormalPluggedIn  || BSDFOperator.Has(ESubstrateBsdfFeature::EyeIrisNormalPluggedIn);
					EyeIrisTangentPluggedIn = EyeIrisTangentPluggedIn || BSDFOperator.Has(ESubstrateBsdfFeature::EyeIrisTangentPluggedIn);

					if (BSDFOperator.BSDFIndex == INDEX_NONE || BSDFOperator.IsDiscarded())
					{
						continue;	// not a BSDF or if discarded (i.e. not the root of a parameter blending subtree), then there is no local basis to register
					}
					if (BSDFOperator.BSDFRegisteredSharedLocalBasis.NormalCodeChunk == INDEX_NONE && BSDFOperator.BSDFRegisteredSharedLocalBasis.TangentCodeChunk == INDEX_NONE)
					{
						continue;	// We skip null normal on certain BSDF, for instance unlit.
					}
					const FSubstrateSharedLocalBasesInfo& SubstrateSharedLocalBasesInfo = SubstrateCtx.SubstrateCompilationInfoGetMatchingSharedLocalBasisInfo(BSDFOperator.BSDFRegisteredSharedLocalBasis);

					SubstrateMaterialContextDescription += FString::Printf(TEXT("     - %s - SharedLocalBasisIndexMacro = %s \n"), *GetSubstrateBSDFName(BSDFOperator.BSDFType), *Substrate::GetSubstrateSharedLocalBasisIndexMacroInner(BSDFOperator.BSDFRegisteredSharedLocalBasis, SubstrateCompilationContextIndex));
				}

				SubstrateMaterialContextDescription += FString::Printf(TEXT("----------- SUBSTRATE TREE - %s -----------\n"), *SubstrateCompilationContextName);
				SubstrateMaterialContextDescription += FString::Printf(TEXT("Graph maximum distance to leaves %u\n"), RootMaximumDistanceToLeaves);
				// Debug print operators according to depth from root.
				{

					for (int32 DistanceToLeaves = RootMaximumDistanceToLeaves; DistanceToLeaves >= 0; --DistanceToLeaves)
					{
						SubstrateMaterialContextDescription += FString::Printf(TEXT("----- DistanceFromLeaves = %d -----\n"), DistanceToLeaves);
						for (auto& It : SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators)
						{
							if (!It.IsDiscarded() && It.MaxDistanceFromLeaves == DistanceToLeaves)
							{
								SubstrateMaterialContextDescription += FString::Printf(TEXT("\tIdx=%d Op=%s ParentIdx=%d LeftIndex=%d RightIndex=%d BSDFIdx=%d LayerDepth=%d IsTop=%d IsBot=%d BSDFType=%s SSS=%d MFP=%d F90=%d Rough2=%d Fuzz=%d Aniso=%d Glint=%d SpecularProfile=%d Eye=%d Hair=%d Toon=%d\n GUID=%s\n"),
									It.Index, GetSubstrateOperatorStr(It.OperatorType), It.ParentIndex, It.LeftIndex, It.RightIndex, It.BSDFIndex, It.LayerDepth, It.bIsTop, It.bIsBottom,
									*GetSubstrateBSDFName(It.BSDFType), 
									It.Has(ESubstrateBsdfFeature::SSS), 
									It.Has(ESubstrateBsdfFeature::MFPPluggedIn), 
									It.Has(ESubstrateBsdfFeature::EdgeColor), 
									It.Has(ESubstrateBsdfFeature::SecondRoughnessOrSimpleClearCoat), 
									It.Has(ESubstrateBsdfFeature::Fuzz), 
									It.Has(ESubstrateBsdfFeature::Anisotropy), 
									It.Has(ESubstrateBsdfFeature::Glint), 
									It.Has(ESubstrateBsdfFeature::SpecularProfile), 
									It.Has(ESubstrateBsdfFeature::Eye), 
									It.Has(ESubstrateBsdfFeature::Hair),
									It.Has(ESubstrateBsdfFeature::Toon),
									*It.MaterialExpressionGuid.ToString());

								// Aggregate all the features used within the material
								MaterialBsdfFeatures |= ESubstrateBsdfFeature(It.BSDFFeatures);
							}
						}
					}
				}

				
				DescriptionStringCommentForDebug += TEXT("/*");
				DescriptionStringCommentForDebug += SubstrateMaterialContextDescription;
				DescriptionStringCommentForDebug += TEXT("*/");

				SubstrateMaterialDescription += SubstrateMaterialContextDescription;

				MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures = MaterialBsdfFeatures;

				if (SubstrateCompilationContextIndex == ESubstrateCompilationContext::SCC_Default)
				{
					MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SharedLocalBasesCount = SubstrateCtx.FinalUsedSharedLocalBasesCount;
				}
			}
#endif // WITH_EDITOR
		}
		MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialDescription = SubstrateMaterialDescription;

		bSubstrateLegacyIrisNormal = EyeIrisNormalPluggedIn;
		bSubstrateLegacyIrisTangent = EyeIrisTangentPluggedIn;
		SubstrateMaterialBsdfFeatures = MaterialCompilationOutput->SubstrateMaterialCompilationOutput.SubstrateMaterialBsdfFeatures;
	}

	// Setup the remaining defines
	if (OutSubstrateDefines)
	{
		OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("MATERIAL_IS_SUBSTRATE"),									bMaterialIsSubstrate ? 1 : 0));
		OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("MATERIAL_IS_SUBSTRATEHIDDENCONVERSION"),					bMaterialUsesRootNodeToSubstrateHiddenConversion ? 1 : 0));
		OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_PREMULTIPLIED_ALPHA_OPACITY_OVERRIDEN"),			bSubstratePremultipliedAlphaOpacityOverridden ? 1 : 0));
		OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("DISTORTION_ACCOUNT_FOR_COVERAGE"),							bDistortionAccountForCoverage ? 1 : 0));
		if(bMaterialIsSubstrate)
		{
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_USES_CONVERSION_FROM_LEGACY"),				bSubstrateUsesConversionFromLegacy ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_OPTIMIZED_UNLIT"),							bSubstrateOptimizedUnlit ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_MATERIAL_OUTPUT_OPAQUE_ROUGH_REFRACTIONS"),	bSubstrateMaterialOutputOpaqueRoughRefractions ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_MATERIAL_EXPORT_TYPE"),						SubstrateMaterialExportType));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_MATERIAL_EXPORT_CONTEXT"),					SubstrateMaterialExportContext));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODE"),			SubstrateMaterialExportLegacyBlendMode));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_MATERIAL_ROUGHNESS_TRACKING"),				bSubstrateRoughnessTracking ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_LEGACY_IRIS_NORMAL"),						bSubstrateLegacyIrisNormal ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_LEGACY_IRIS_TANGENT"),						bSubstrateLegacyIrisTangent ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_SINGLEPATH"),								bSubstrateSinglePath ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_FASTPATH"),									bSubstrateFastPath ? 1 : 0));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_CLAMPED_CLOSURE_COUNT"),						(int32)SubstrateClampedClosureCount));
			OutSubstrateDefines->Add(TPair<FString, int32>(TEXT("SUBSTRATE_COMPLEXSPECIALPATH"),						bSubstrateComplexSpecialPath ? 1 : 0));
			}
	}
}

int32 FSubstrateTranslatorData::SubstrateThicknessStackGetThicknessIndex()
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	return SubstrateCtx.SubstrateThicknessStack.Top();
}


FGuid FSubstrateTranslatorData::SubstrateTreeStackGetPathUniqueId()
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	return SubstrateCtx.SubstrateNodeIdentifierStack.Top();
}

FGuid FSubstrateTranslatorData::SubstrateTreeStackGetParentPathUniqueId()
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	if (SubstrateCtx.SubstrateNodeIdentifierStack.Num() < 2)
	{
		// return some default when Substrate tree stack unique guid cannot be found
		FGuid NullParent;
		return NullParent;
	}
	return SubstrateCtx.SubstrateNodeIdentifierStack.Last(1);
}

bool FSubstrateTranslatorData::GetSubstrateTreeOutOfStackDepthOccurred()
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	return SubstrateCtx.bSubstrateTreeOutOfStackDepthOccurred;
}

FSubstrateOperator& FSubstrateTranslatorData::SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending)
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];

	if (OperatorType == SUBSTRATE_OPERATOR_BSDF_LEGACY)
	{
		// We register the fact that a legacy material conversion is used and register a simple BSDF
		bSubstrateUsesConversionFromLegacy = true;
		OperatorType = SUBSTRATE_OPERATOR_BSDF;
	}

	static FSubstrateOperator DefaultOperatorOnError = FSubstrateOperator();

	if (SubstrateCtx.SubstrateMaterialExpressionToOperatorIndex.Find(SubstrateExpressionGuid))
	{
		// It is not possible to register/use a Substrate BSDF multiple times with this same exact graph path. (that would break the Substrate tree code generation)
		Errorf(FString::Printf(TEXT("Material %s: It is not possible to uses a Substrate BSDF (or any ouput of type SubstrateData) multiple times within a Substrate material topology with the same graph path GUID (asset: %s).\r\n"), *GetAssetName(), *GetAssetPath()));
		return DefaultOperatorOnError;
	}

	const uint32 NewOperatorIndex = SubstrateCtx.SubstrateMaterialExpressionToOperatorIndex.Num();
	if (NewOperatorIndex >= SUBSTRATE_MAX_COMPILER_REGISTERED_OPERATOR_COUNT)
	{
		Errorf(FString::Printf(TEXT("Material %s have too many Substrate Operators: the compiler is failing (asset: %s).\r\n"), *GetAssetName(), *GetAssetPath()));
		return DefaultOperatorOnError;
	}

	int32* ParentOperatorIndex = SubstrateCtx.SubstrateMaterialExpressionToOperatorIndex.Find(SubstrateParentExpressionGuid);
	if (Parent != nullptr && ParentOperatorIndex == nullptr)
	{
		Errorf(FString::Printf(TEXT("Material %s tries to register unknown operator parents (asset: %s).\r\n"), *GetAssetName(), *GetAssetPath()));
		return DefaultOperatorOnError;
	}

	SubstrateCtx.SubstrateMaterialExpressionToOperatorIndex.Add(SubstrateExpressionGuid, NewOperatorIndex);

	FSubstrateOperator& NewOperator = SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators.AddDefaulted_GetRef();

	NewOperator.OperatorType = OperatorType;
	NewOperator.bNodeRequestParameterBlending = bUseParameterBlending;
	NewOperator.Index = NewOperatorIndex;
	NewOperator.ParentIndex = ParentOperatorIndex != nullptr ? *ParentOperatorIndex : INDEX_NONE;
	NewOperator.LeftIndex = INDEX_NONE;
	NewOperator.RightIndex = INDEX_NONE;

	NewOperator.BSDFIndex = INDEX_NONE;	// Allocated later to be able to account for inline
	NewOperator.SubUsage = SUBSTRATE_OPERATOR_SUBUSAGE_NONE;

	NewOperator.MaxDistanceFromLeaves = 0;
	NewOperator.bIsBottom = false;
	NewOperator.bIsTop = false;
	NewOperator.MaterialExpressionGuid = ChildMaterialExpressionGuid;

	return NewOperator;
}

FSubstrateOperator& FSubstrateTranslatorData::SubstrateGetDefaultBSDFOperator(FGuid MaterialExpressionGuid, UMaterialExpression* Parent)
{
	FSubstrateRegisteredSharedLocalBasis NewRegisteredSharedLocalBasis;
	if (CompilerLegacy)
	{
		int32 NormalCodeChunk = CompilerLegacy->VertexNormal();
		const int32 NullTangentCodeChunk = INDEX_NONE;
		NewRegisteredSharedLocalBasis = SubstrateCompilationInfoCreateSharedLocalBasis(CompilerLegacy, NormalCodeChunk, NullTangentCodeChunk);
	}
	else
	{
		check(false); // Missing NewRegisteredSharedLocalBasis for CompilerNew
	}

	FSubstrateOperator& SlabOperator = SubstrateCompilationRegisterOperator(SUBSTRATE_OPERATOR_BSDF, SubstrateTreeStackGetPathUniqueId(), MaterialExpressionGuid, Parent, SubstrateTreeStackGetParentPathUniqueId());
	SlabOperator.BSDFRegisteredSharedLocalBasis = NewRegisteredSharedLocalBasis;
	SlabOperator.BSDFType = SUBSTRATE_BSDF_TYPE_SLAB;
	SlabOperator.ThicknessIndex = SubstrateThicknessStackGetThicknessIndex();

	return SlabOperator;
}

FSubstrateOperator& FSubstrateTranslatorData::SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid)
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	auto* OperatorIndex = SubstrateCtx.SubstrateMaterialExpressionToOperatorIndex.Find(SubstrateExpressionGuid);
	if (!(OperatorIndex && *OperatorIndex >= 0 && *OperatorIndex < SUBSTRATE_MAX_COMPILER_REGISTERED_OPERATOR_COUNT))
	{
		static FSubstrateOperator DefaultOperatorOnError = FSubstrateOperator();
		return DefaultOperatorOnError;
	};
	return SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[*OperatorIndex];
}

FSubstrateOperator* FSubstrateTranslatorData::SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex)
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	int32 OperatorCount = SubstrateCtx.SubstrateMaterialExpressionToOperatorIndex.Num();
	if (OperatorIndex < 0 || OperatorIndex >= OperatorCount)
	{
		Errorf(FString::Printf(TEXT("SubstrateCompilationGetOperatorFromIndex - OperatorIndex out of range %s (asset: %s).\r\n"), *GetAssetName(), *GetAssetPath()));
		return nullptr;
	};
	return &SubstrateCtx.SubstrateMaterialExpressionRegisteredOperators[OperatorIndex];
}

int32 FSubstrateTranslatorData::SubstrateThicknessStackPush(UMaterialExpression* Expression, FScalarMaterialInput* Input)
{
	// If the input is actually not a constant, reroute it to the Expression overload to that the expression can be traced.
	if (Input && !Input->UseConstant)
	{
		return SubstrateThicknessStackPush(Expression, (FExpressionInput*)Input);
	}

	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	int32 Index = SubstrateCtx.SubstrateThicknessIndexToExpressionInput.Num();
	FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression& Entry = SubstrateCtx.SubstrateThicknessIndexToExpressionInput.AddDefaulted_GetRef();
	Entry.MaterialInput = Input;
	SubstrateCtx.SubstrateThicknessStack.Push(Index);
	return Index;
}

int32 FSubstrateTranslatorData::SubstrateThicknessStackPush(UMaterialExpression* Expression, FExpressionInput* Input)
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	int32 Index = SubstrateCtx.SubstrateThicknessIndexToExpressionInput.Num();
	FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression& Entry = SubstrateCtx.SubstrateThicknessIndexToExpressionInput.AddDefaulted_GetRef();
	Entry.ExpressionInput = Input;
	SubstrateCtx.SubstrateThicknessStack.Push(Index);
	return Index;
}

void FSubstrateTranslatorData::SubstrateThicknessStackPop()
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	check(SubstrateCtx.SubstrateThicknessStack.Num() >= 1);
	SubstrateCtx.SubstrateThicknessStack.Pop();
}

FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression* FSubstrateTranslatorData::SubstrateThicknessStackGetExpression(int32 Index)
{
	FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression* ThicknessExpression = nullptr;
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	if (Index == INDEX_NONE || Index >= SubstrateCtx.SubstrateThicknessIndexToExpressionInput.Num())
	{
		Errorf(*FString::Printf(TEXT(" SubstrateThichkness: %i could not be found)"), Index));
	}
	else
	{
		ThicknessExpression = &SubstrateCtx.SubstrateThicknessIndexToExpressionInput[Index];
	}
	return ThicknessExpression;
}

FGuid FSubstrateTranslatorData::SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex)
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];

	// Create an md5 hash for the parent, its input pin index and current node to represent the path.
	uint32 IntputHashBuffer[9];
	FGuid PreviousNodeGuid = SubstrateTreeStackGetPathUniqueId();
	FGuid NodeGuid = Expression->MaterialExpressionGuid;
	IntputHashBuffer[0] = PreviousNodeGuid.A;
	IntputHashBuffer[1] = PreviousNodeGuid.B;
	IntputHashBuffer[2] = PreviousNodeGuid.C;
	IntputHashBuffer[3] = PreviousNodeGuid.D;
	IntputHashBuffer[4] = InputIndex;
	IntputHashBuffer[5] = NodeGuid.A;
	IntputHashBuffer[6] = NodeGuid.B;
	IntputHashBuffer[7] = NodeGuid.C;
	IntputHashBuffer[8] = NodeGuid.D;

	uint32 OutputHashBuffer[]{ 0, 0, 0, 0 };
	FMD5 IdentifierStringHash;
	IdentifierStringHash.Update((uint8*)IntputHashBuffer, sizeof(IntputHashBuffer));
	IdentifierStringHash.Final((uint8*)&OutputHashBuffer);

	SubstrateCtx.SubstrateNodeIdentifierStack.Push(FGuid(OutputHashBuffer[0], OutputHashBuffer[1], OutputHashBuffer[2], OutputHashBuffer[3]));

	SubstrateCtx.bSubstrateTreeOutOfStackDepthOccurred = SubstrateCtx.bSubstrateTreeOutOfStackDepthOccurred || (SubstrateCtx.SubstrateNodeIdentifierStack.Num() > SUBSTRATE_TREE_MAX_DEPTH);


	return SubstrateCtx.SubstrateNodeIdentifierStack.Top();
}

void FSubstrateTranslatorData::SubstrateTreeStackPop()
{
	FSubstrateCompilationContext& SubstrateCtx = SubstrateCompilationContext[CurrentSubstrateCompilationContext];
	check(SubstrateCtx.SubstrateNodeIdentifierStack.Num() >= 2);// 2 because there must always be the root remaining.
	SubstrateCtx.SubstrateNodeIdentifierStack.Pop();
}

class FMaterialCompiler* FSubstrateTranslatorData::GetCompilerLegacy()
{
	return CompilerLegacy;
}

class MIR::FEmitter* FSubstrateTranslatorData::GetCompilerNew()
{
	return CompilerNew;
}

void FSubstrateTranslatorData::PushFunction(class FMaterialFunctionCompileState* FunctionState)
{
	if (CompilerLegacy)
	{
		CompilerLegacy->PushFunction(FunctionState);
	}
}

void FSubstrateTranslatorData::PopFunction()
{
	if (CompilerLegacy)
	{
		CompilerLegacy->PopFunction();
	}
}

} // namespace Substrate

#endif // WITH_EDITOR
