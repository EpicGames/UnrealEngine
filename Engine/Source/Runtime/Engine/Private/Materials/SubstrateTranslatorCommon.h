// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "SubstrateMaterial.h"
#include "MaterialShared.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialIREmitter.h"
#include "Containers/Array.h"


struct FEnvironmentDefines;
class FMaterialCompiler;
class UMaterialExpression;

namespace MIR
{
	class FEmitter;
	struct FValueRef;
};

// This limitation is required so that the operator array is never reallocated, invalidating references to it while parsing the Substrate tree within SubstrateGenerateMaterialTopologyTree for instance.
#define SUBSTRATE_MAX_COMPILER_REGISTERED_OPERATOR_COUNT 128

namespace Substrate
{

struct FSubstrateTranslatorData;

enum ESubstrateCompilationContext : uint8
{
	SCC_Default = 0u,
	SCC_FullySimplified = 1u,
	SCC_MAX = 2u
};


FString GetParametersSharedLocalBasesName(ESubstrateCompilationContext CompilationContextIndex);
FString GetParametersSubstrateTreeName(ESubstrateCompilationContext CompilationContextIndex);
FString GetSubstrateSharedLocalBasisIndexMacroInner(const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis, uint32 Mode);
const TCHAR* GetSubstrateOperatorStr(int32 OperatorType);


/** Describe the simplification status.Once the material has been compiled, it can be used to understand if and how it has been simplified. */
struct FSubstrateSimplificationStatus
{
	bool bMaterialFitsInMemoryBudget = false;	// Track whether or not the material fits.

	uint32 OriginalRequestedByteSize = 0;
	uint32 OriginalRequestedClosureCount = 0;
	bool bRunFullSimplification = false;	// Simple implementation for now: if the material does not fit, we simply everything.
	bool bFullSimplificationStepHasBeenRun = false;

	bool bSlabSimplificationStepHasBeenRun = false;

	struct FOperatorToSimplify
	{
		union
		{
			uint32 PackedData;
			struct
			{
				uint32 Index : 16; // Index of the operator
				uint32 Depth : 16; // Depth of the operator
			} Data;
		};

		FORCEINLINE bool operator!=(FOperatorToSimplify B) const
		{
			return PackedData != B.PackedData;
		}

		FORCEINLINE bool operator<(FOperatorToSimplify B) const
		{
			return PackedData < B.PackedData;
		}
	};
	TArray<FOperatorToSimplify> OperatorSimplificationOrder;
};


/** Represent a shared local basis description with its associated code. */
struct FSubstrateSharedLocalBasesInfo
{
	FSubstrateRegisteredSharedLocalBasis SharedData;
	FString NormalCode;
	FString TangentCode;
	MIR::FValueRef NormalArg;
	MIR::FValueRef TangentArg;
};


typedef TArray<TPair<FString, int32>> FSubstrateDefines;


/** Compilation context can be different for fully simplified or not Substrate tree. */
struct FSubstrateCompilationContext
{
	FSubstrateCompilationContext();

	FSubstrateCompilationContext(ESubstrateCompilationContext InCompilationContext);

	ESubstrateCompilationContext CompilationContextIndex;

	/** The code initializing the array of shared local bases. */
	FString SubstratePixelNormalInitializerValues;
	/** The next free index that can be used to represent a unique macros pointing to the position in the array of shared local bases written to memory once the shader is executed. */
	uint8 NextFreeSubstrateShaderNormalIndex;
	/** The effective final shared local bases count used by the final shader. */
	uint8 FinalUsedSharedLocalBasesCount;
	/** Tracks shared local bases used by Substrate materials, mapping a normal code chunk hash to a SharedMaterialInfo.
		* A normal code chunk hash can point to multiple shared info in case it is paired with different tangents. */
	TMultiMap<uint64, FSubstrateSharedLocalBasesInfo> CodeChunkToSubstrateSharedLocalBasis;

	TMap<FGuid, int32> SubstrateMaterialExpressionToOperatorIndex;
	TArray<FSubstrateOperator> SubstrateMaterialExpressionRegisteredOperators;
	FSubstrateOperator* SubstrateMaterialRootOperator;
	uint32 SubstrateMaterialEffectiveClosureCount; // Also acts as requested 
	uint32 SubstrateMaterialRequestedSizeByte;
	uint32 SubstrateMaterialClosureCount;
	FSubstrateMaterialComplexity SubstrateMaterialComplexity;
	bool bSubstrateMaterialIsUnlitNode;

	bool bSubstrateWritesEmissive;
	bool bSubstrateWritesAmbientOcclusion;

	/** Stack of unique id for each node of the Substrate tree
	* This is transient and updated on the fly in the exact same way when parsing node for
	*  1- SubstrateGenerateMaterialTopologyTree: generating a picture of the Substrate material tree for code generation and simplifications.
	*  2- CompilePropertyAndSetMaterialProperty(MP_ShadingModel): compiling the code with some features enabled/disabled and accounting for tree simplification decided beforehand.
	* It is not valid to use that information outside of those functions.
	*/
	TArray<FGuid> SubstrateNodeIdentifierStack;

	/**
		* This can be used to know if the Substrate tree we are trying to build is too deep and we should stop the compilation.
		* True means that we have likely encountered node re-entry leading to cyclic graph we cannot handle and compile internally: we must fail the compilation.
		*/
	bool bSubstrateTreeOutOfStackDepthOccurred;

	/** Stack of thickness input used for propagating thickness information from root node and vertical operation
	* This is transient and updated when calling SubstrateGenerateMaterialTopologyTree. The information is then stored into the FStratOperator
	*/
	TArray<int32> SubstrateThicknessStack;
	TArray<FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression> SubstrateThicknessIndexToExpressionInput;

	FSubstrateSimplificationStatus SubstrateSimplificationStatus;

	bool SubstrateGenerateDerivedMaterialOperatorData(FSubstrateTranslatorData& TranslatorData);

	void SubstrateEvaluateSharedLocalBases(uint8& RequestedSharedLocalBasesCount, FSubstrateDefines* OutSubstrateDefines, MIR::FEmitter* Em = nullptr);

	FSubstrateSharedLocalBasesInfo SubstrateCompilationInfoGetMatchingSharedLocalBasisInfo(const FSubstrateRegisteredSharedLocalBasis& SearchedSharedLocalBasis);

private:
	void Initialise();
};

/**
* This data is used to share common data and code between the legacy and new translator.
* 
* When it comes to the new profiler, Substrate process is as follows:
* 1- First initialize() the data from Step_Initialize (MaterialIRModuleBuilder.cpp)
* 2- From Step_BuildMaterialExpressionsToIRGraph
*		a- Generate the Substrate tree representing the topology ProcessSubstrateTreeAndTopology.
*		  This functions also calls SubstrateGenerateDerivedMaterialOperatorData which verifies the validitiy of the tree, 
*		  simplifies it if needed and generate the fully simplified Substrate tree.
*		b- The generated FrontMaterial pixel input is stored in a custom output called SubstrateFrontMaterial.
*		b- We then removed all the Substrate node from the Context.BuiltExpressions for all the contexts. 
*		  We want to generate IR for them but this time for the fully simplified material with the substrate context SCC_FullySimplified.
*		  The generated FrontMaterial pixel input is stored in a custom output called SubstrateFrontMaterialFullySimplified.
* 3- From Step_EmitSetMaterialPropertyInstructions, this is where we assigne the FrontMaterial IR using BuildCommon function when hidden conversion is enabled.
*	The result is then assigned to SubstrateFrontMaterial and SubstrateFrontMaterialFullySimplified custom output, similarly to Step2 above.
*	This is alsowhere we generate the HLSL backend code we need for several substrate element such as front material & local shared bases assignement, 
*	substrate tree functions, same defines as for the legacy translator, etc.
* 4- And from there we are DONE.
* 
* Notes:
*	- Substrate in the new translator heavily relied on named customOutput as temporary storage for intermediate data such as
*	  FSubstrateData for front material, the fully simplified version as well as all the shared local bases.
*	- We do not call SubstrateTreeStackPush and Pop since the way the Build() function on the nodes is not done using depth first but using a stack per context.
*	  So it is not possible to build a valid SubstrateTree stack unique GUID using those functions. Instead it is generated for each Substrate node from from BuildTopMaterialExpression.
* 
**/
struct FSubstrateTranslatorData : public FSubstrateTranslatorDataInterface
{
	/** True whether Substrate is enabled */
	uint32 bSubstrateEnabled : 1;

	/** True if the material is detected as a Substrate material at compile time.
	 * This is decoupled from runtime FMaterialResource::IsSubstrateMaterial but practically fine since this is only temporary until Substrate is the main shading system. Only really used at runtime for translucency dual source blending.
	 */
	uint32 bMaterialIsSubstrate : 1;

	/** True if Substrate hidden conversion from the root node is used by this material. (enabled for the project, FrontMaterial is not plugged in). */
	uint32 bMaterialUsesRootNodeToSubstrateHiddenConversion : 1;

	/** True if Substrate hidden conversion from the root node is enabled for the project.*/
	uint32 bProjectSubstrateHiddenMaterialAssetConversionEnabled : 1;

	uint32 bSubstrateMaterialUsesLegacyMaterialCompilation : 1;

	Substrate::FSubstrateCompilationContext SubstrateCompilationContext[Substrate::ESubstrateCompilationContext::SCC_MAX];

	Substrate::ESubstrateCompilationContext CurrentSubstrateCompilationContext = Substrate::ESubstrateCompilationContext::SCC_Default;
	int32 FullySimplifiedSubstrateFrontMaterialCodeChunk = INDEX_NONE;
	FString FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunkDefinitions;
	FString FullySimplifiedSubstrateFrontMaterialTranslatedCodeChunks;

	FString SubstrateTreeHLSL;

	bool bSubstrateWritesEmissive;
	bool bSubstrateWritesAmbientOcclusion;

	bool bSubstrateUsesConversionFromLegacy;
	bool bSubstrateMaterialOutputOpaqueRoughRefractions;

	// Some more data needed for defines
	bool bSubstratePremultipliedAlphaOpacityOverridden = false;
	int32 SubstrateMaterialExportType = 0;
	int32 SubstrateMaterialExportContext = 0;
	int32 SubstrateMaterialExportLegacyBlendMode = 0;
	bool bSubstrateOptimizedUnlit = false;
	bool bSubstrateSinglePath = false;
	bool bSubstrateFastPath = false;
	uint32 SubstrateClampedClosureCount = 0;
	bool SubstrateIsComplexSpecialPath = false;
	bool bSubstrateComplexSpecialPath = false;
	bool bSubstrateLegacyIrisNormal = false;
	bool bSubstrateLegacyIrisTangent = false;
	bool bSubstrateRoughnessTracking = false;
	bool bDistortionAccountForCoverage = false;
	ESubstrateBsdfFeature SubstrateMaterialBsdfFeatures = ESubstrateBsdfFeature::None;

	/** Substrate material compilation and simplification configuration. */
	FSubstrateCompilationConfig SubstrateCompilationConfig;

	UMaterialExpression* FrontMaterialExpr;

	FSubstrateTranslatorData();
	virtual ~FSubstrateTranslatorData();

	void Initialize(UMaterialInterface* MaterialInterface, FMaterialCompilationOutput& MaterialCompilationOutput, FMaterialCompiler* CompilerLegacy = nullptr, MIR::FEmitter* CompilerNew = nullptr);

	EShaderPlatform GetShaderPlatform();

	UMaterialInterface* GetMaterialInterface() { return MaterialInterface; }
	FMaterialCompilationOutput& GetMaterialCompilationOutput() { return *MaterialCompilationOutput; }

	void ProcessSubstrateTreeAndTopology(
		struct FMaterialShadingModelField& MaterialShadingModels,
		UMaterialExpression* ExpressionToPreview,
		bool& bDoClearFunctionStack);

	bool GenerateSubstrateTreeHLSLCode(bool bSubstrateFrontMaterialProvided);

	void GeneratePixelMemberDeclarationHLSLCode(FString& OutHLSLCode, const FString& FrontMaterialSubstrateDataHLSLType);
	void GeneratePixelInputInitializerHLSLCode(FString& OutHLSLCode);
	void GeneratePixelNormalInitializerHLSLCode(FString& OutHLSLCode);

	void GenerateMaterialCompilationOutputAndDefines(
		bool bOpacityPropertyIsUsed,
		int32 MaterialExportType, int32 MaterialExportContext, int32 MaterialExportLegacyBlendMode,
		FString& DescriptionStringCommentForDebug,
		FSubstrateDefines* OutSubstrateDefines
	);

	FExpressionInput* GetThinTranslucentTransmittanceColor()	{ return ThinTranslucentTransmittanceColor; }
	FExpressionInput* GetThinTranslucentSurfaceCoverage()		{ return ThinTranslucentSurfaceCoverage; }
	FExpressionInput* GetWaterScatteringCoefficients()			{ return WaterScatteringCoefficients; }
	FExpressionInput* GetWaterAbsorptionCoefficients()			{ return WaterAbsorptionCoefficients; }
	FExpressionInput* GetWaterPhaseG()							{ return WaterPhaseG; }
	FExpressionInput* GetColorScaleBehindWater()				{ return ColorScaleBehindWater; }
	FExpressionInput* GetClearCoatNormal()						{ return ClearCoatNormal; }
	FExpressionInput* GetCustomTangent()						{ return CustomTangent; }

	uint32 GetFreeSharedLocalBasesIndex()						{ return SharedLocalBasesFreeIndex++; }

	virtual FMaterialShadingModelField GetMaterialShadingModels() override;

	// Common fonctions for all translators, that were specific to the legacy translator before.
	virtual void Errorf(const FString& Format) override;

	virtual FGuid SubstrateTreeStackGetPathUniqueId() override;
	virtual FGuid SubstrateTreeStackGetParentPathUniqueId() override;
	virtual bool GetSubstrateTreeOutOfStackDepthOccurred() override;

	virtual int32 SubstrateThicknessStackGetThicknessIndex() override;
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FScalarMaterialInput* Input) override;
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FExpressionInput* Input) override;
	virtual void SubstrateThicknessStackPop() override;
	virtual FSubstrateTranslatorDataInterface::FSubstrateThicknessExpression* SubstrateThicknessStackGetExpression(int32 Index) override;

	virtual FGuid SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex) override;
	virtual void SubstrateTreeStackPop() override;

	virtual FSubstrateOperator& SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending = false) override;
	virtual FSubstrateOperator& SubstrateGetDefaultBSDFOperator(FGuid MaterialExpressionGuid, UMaterialExpression* Parent) override;
	virtual FSubstrateOperator& SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid) override;
	virtual FSubstrateOperator* SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex) override;

	virtual class FMaterialCompiler* GetCompilerLegacy() override;
	virtual class MIR::FEmitter* GetCompilerNew() override;

	virtual void PushFunction(class FMaterialFunctionCompileState* FunctionState);
	virtual void PopFunction();

	const FString& GetAssetName() { return AssetName; }
	const FString& GetAssetPath() { return AssetPath; }

	const EMaterialQualityLevel::Type GetCookTargetedMaterialQualityLevel() { return CookTargetedMaterialQualityLevel; }

private:

	UMaterialInterface*				MaterialInterface;
	FMaterialCompilationOutput*		MaterialCompilationOutput;
	FMaterialCompiler*				CompilerLegacy;
	MIR::FEmitter*					CompilerNew;

	EMaterialQualityLevel::Type		CookTargetedMaterialQualityLevel; // the cook targeted material quality level

	FString							AssetName;
	FString							AssetPath;

	FMaterialShadingModelField		MaterialShadingModels;

	FExpressionInput				NullInput;
	FExpressionInput*				ThinTranslucentTransmittanceColor = &NullInput;
	FExpressionInput*				ThinTranslucentSurfaceCoverage = &NullInput;
	FExpressionInput*				WaterScatteringCoefficients = &NullInput;
	FExpressionInput*				WaterAbsorptionCoefficients = &NullInput;
	FExpressionInput*				WaterPhaseG = &NullInput;
	FExpressionInput*				ColorScaleBehindWater = &NullInput;
	FExpressionInput*				ClearCoatNormal = &NullInput;
	FExpressionInput*				CustomTangent = &NullInput;

	uint32							SharedLocalBasesFreeIndex = 0;
};


}; // namespace Substrate

#endif // WITH_EDITOR
