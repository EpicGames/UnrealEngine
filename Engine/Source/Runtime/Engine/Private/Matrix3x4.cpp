// Copyright Epic Games, Inc. All Rights Reserved.

#include "Matrix3x4.h"

void TransposeTransforms(FMatrix3x4* DstTransforms, const FMatrix44f* SrcTransforms, int64 Count)
{
#if PLATFORM_ALWAYS_HAS_AVX_2
	const __m256i Deinterleave = _mm256_setr_epi32(0, 4, 1, 5, 2, 6, 3, 7);
	int64 TransformIndex = 0;

	for (; TransformIndex + 2 <= Count; TransformIndex += 2)
	{
		const FMatrix44f& SrcTransformA = SrcTransforms[TransformIndex + 0];
		const FMatrix44f& SrcTransformB = SrcTransforms[TransformIndex + 1];
		FMatrix3x4& DstTransformA = DstTransforms[TransformIndex + 0];
		FMatrix3x4& DstTransformB = DstTransforms[TransformIndex + 1];

		__m256 InRowA01 = _mm256_loadu_ps(&(SrcTransformA.M[0][0])); // A00,A01,A02,A03 | A10,A11,A12,A13
		__m256 InRowA23 = _mm256_loadu_ps(&(SrcTransformA.M[2][0])); // A20,A21,A22,A23 | A30,A31,A32,A33
		__m256 InRowB01 = _mm256_loadu_ps(&(SrcTransformB.M[0][0])); // B00,B01,B23,B23 | B10,B11,B12,B13
		__m256 InRowB23 = _mm256_loadu_ps(&(SrcTransformB.M[2][0])); // B20,B21,B22,B23 | B30,B31,B32,B33

		// First transpose pass (interleave with row 2 away)
		__m256 TempA0 = _mm256_unpacklo_ps(InRowA01, InRowA23); // A00,A20,A01,A21 | A10,A30,A11,A31
		__m256 TempA1 = _mm256_unpackhi_ps(InRowA01, InRowA23); // A02,A22,A03,A23 | A12,A32,A13,A33
		__m256 TempB0 = _mm256_unpacklo_ps(InRowB01, InRowB23); // B00,B20,B01,B21 | B10,B30,B11,B31
		__m256 TempB1 = _mm256_unpackhi_ps(InRowB01, InRowB23); // B02,B22,B03,B23 | B12,B32,B13,B33

		// Second transpose pass needs to cross 128b boundaries
		__m256 FinalA0 = _mm256_permutevar8x32_ps(TempA0, Deinterleave); // A00,A10,A20,A30 | A01,A11,A21,A31
		__m256 FinalA1 = _mm256_permutevar8x32_ps(TempA1, Deinterleave); // A02,A12,A22,A32 | A03,A13,A23,A33
		__m256 FinalB0 = _mm256_permutevar8x32_ps(TempB0, Deinterleave); // B00,B10,B20,B30 | B01,B11,B21,B31
		__m256 FinalB1 = _mm256_permutevar8x32_ps(TempB1, Deinterleave); // B02,B12,B22,B32 | B03,B13,B23,B33

		// Store the results
		// It's cheaper to throw in the 128-bit stores than it is to try to shuffle things so everything is aligned
		_mm256_storeu_ps(&(DstTransformA.M[0][0]), FinalA0);
		_mm_storeu_ps(&(DstTransformA.M[2][0]), _mm256_castps256_ps128(FinalA1));
		_mm256_storeu_ps(&(DstTransformB.M[0][0]), FinalB0);
		_mm_storeu_ps(&(DstTransformB.M[2][0]), _mm256_castps256_ps128(FinalB1));
	}

	// Take care of final transform if odd count
	if (TransformIndex < Count)
	{
		TransposeTransform(DstTransforms[TransformIndex], SrcTransforms[TransformIndex]);
	}
#else
	for (int64 TransformIndex = 0; TransformIndex < Count; ++TransformIndex)
	{
		const FMatrix44f& SrcTransform = SrcTransforms[TransformIndex];
		FMatrix3x4& DstTransform = DstTransforms[TransformIndex];
		TransposeTransform(DstTransform, SrcTransform);
	}
#endif
}