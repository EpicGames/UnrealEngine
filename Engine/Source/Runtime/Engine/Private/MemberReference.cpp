// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/MemberReference.h"
#include "EngineLogs.h"
#include "Misc/ConfigCacheIni.h"
#include "Stats/Stats.h"
#include "UObject/CoreRedirects.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MemberReference)

#if WITH_EDITOR
#include "Kismet2/BlueprintEditorUtils.h"
#endif

//////////////////////////////////////////////////////////////////////////
// FMemberReference

void FMemberReference::SetExternalMember(FName InMemberName, TSubclassOf<class UObject> InMemberParentClass)
{
	MemberName = InMemberName;
#if WITH_EDITOR
	MemberParent = (InMemberParentClass != nullptr) ? InMemberParentClass->GetAuthoritativeClass() : nullptr;
#else
	MemberParent = *InMemberParentClass;
#endif
	MemberScope.Empty();
	bSelfContext = false;
	bWasDeprecated = false;
}

void FMemberReference::SetExternalMember(FName InMemberName, TSubclassOf<class UObject> InMemberParentClass, const FGuid& InMemberGuid)
{
	SetExternalMember(InMemberName, InMemberParentClass);
	MemberGuid = InMemberGuid;
}

void FMemberReference::SetGlobalField(FName InFieldName, UPackage* InParentPackage)
{
	MemberName = InFieldName;
	MemberParent = InParentPackage;
	MemberScope.Empty();
	bSelfContext = false;
	bWasDeprecated = false;
}

void FMemberReference::SetExternalDelegateMember(FName InMemberName)
{
	SetExternalMember(InMemberName, nullptr);
}

void FMemberReference::SetSelfMember(FName InMemberName)
{
	MemberName = InMemberName;
	MemberParent = nullptr;
	MemberScope.Empty();
	bSelfContext = true;
	bWasDeprecated = false;
}

void FMemberReference::SetSelfMember(FName InMemberName, const FGuid& InMemberGuid)
{
	SetSelfMember(InMemberName);
	MemberGuid = InMemberGuid;
}

void FMemberReference::SetDirect(const FName InMemberName, const FGuid InMemberGuid, TSubclassOf<class UObject> InMemberParentClass, bool bIsConsideredSelfContext)
{
	MemberName = InMemberName;
	MemberGuid = InMemberGuid;
	bSelfContext = bIsConsideredSelfContext;
	bWasDeprecated = false;
	MemberParent = InMemberParentClass;
	MemberScope.Empty();
}

#if WITH_EDITOR
void FMemberReference::SetGivenSelfScope(const FName InMemberName, const FGuid InMemberGuid, TSubclassOf<class UObject> InMemberParentClass, TSubclassOf<class UObject> SelfScope) const
{
	MemberName = InMemberName;
	MemberGuid = InMemberGuid;
	MemberParent = (InMemberParentClass != nullptr) ? InMemberParentClass->GetAuthoritativeClass() : nullptr;
	MemberScope.Empty();

	// SelfScope should always be valid, but if it's not ensure and move on, the node will be treated as if it's not self scoped.
	ensure(SelfScope);
	bSelfContext = SelfScope && ((SelfScope->IsChildOf(InMemberParentClass)) || (SelfScope->ClassGeneratedBy == InMemberParentClass->ClassGeneratedBy));
	bWasDeprecated = false;

	if (bSelfContext)
	{
		MemberParent = nullptr;
	}
}
#endif

void FMemberReference::SetLocalMember(FName InMemberName, UStruct* InScope, const FGuid InMemberGuid)
{
	SetLocalMember(InMemberName, InScope->GetName(), InMemberGuid);
}

void FMemberReference::SetLocalMember(FName InMemberName, FString InScopeName, const FGuid InMemberGuid)
{
	MemberName = InMemberName;
	MemberScope = InScopeName;
	MemberGuid = InMemberGuid;
	bSelfContext = false;
}

void FMemberReference::InvalidateScope()
{
	if( IsSelfContext() )
	{
		MemberParent = nullptr;
	}
	else if(IsLocalScope())
	{
		MemberScope.Empty();

		// Make it into a member reference since we are clearing the local context
		bSelfContext = true;
	}
}

bool FMemberReference::IsSparseClassData(const UClass* OwningClass) const
{
	bool bIsSparseClassData = false;
	UScriptStruct* SparseClassDataStruct = OwningClass ? OwningClass->GetSparseClassDataStruct() : nullptr;
	if (SparseClassDataStruct)
	{
		FProperty* VariableProperty = FindFProperty<FProperty>(SparseClassDataStruct, GetMemberName());
		bIsSparseClassData = VariableProperty != nullptr;
	}

	return bIsSparseClassData;
}

#if WITH_EDITOR

FString FMemberReference::GetReferenceSearchString(UClass* InFieldOwner) const
{
	if (!IsLocalScope())
	{
		if (InFieldOwner)
		{
			if (MemberGuid.IsValid())
			{
				return FString::Printf(TEXT("Nodes(VariableReference(MemberName=+\"%s\" && MemberGuid(A=%i && B=%i && C=%i && D=%i)) || Name=\"(%s)\") || Pins(Binding=\"%s\") || Binding=\"%s\""), *MemberName.ToString(), MemberGuid.A, MemberGuid.B, MemberGuid.C, MemberGuid.D, *MemberName.ToString(), *MemberName.ToString(), *MemberName.ToString());
			}
			else
			{
				FString ExportMemberParentName;
				ExportMemberParentName = InFieldOwner->GetClass()->GetName();
				ExportMemberParentName.AppendChar('\'');
				ExportMemberParentName += InFieldOwner->GetAuthoritativeClass()->GetPathName();
				ExportMemberParentName.AppendChar('\'');

				return FString::Printf(TEXT("Nodes(VariableReference(MemberName=+\"%s\" && (MemberParent=\"%s\" || bSelfContext=true) ) || Name=\"(%s)\") || Pins(Binding=\"%s\") || Binding=\"%s\""), *MemberName.ToString(), *ExportMemberParentName, *MemberName.ToString(), *MemberName.ToString(), *MemberName.ToString());
			}
		}
		else if (MemberGuid.IsValid())
		{
			return FString::Printf(TEXT("Nodes(VariableReference(MemberName=+\"%s\" && MemberGuid(A=%i && B=%i && C=%i && D=%i)) || Name=\"(%s)\") || Pins(Binding=\"%s\") || Binding=\"%s\""), *MemberName.ToString(), MemberGuid.A, MemberGuid.B, MemberGuid.C, MemberGuid.D, *MemberName.ToString(), *MemberName.ToString(), *MemberName.ToString());
		}
		else
		{
			return FString::Printf(TEXT("Nodes(VariableReference(MemberName=+\"%s\") || Name=\"(%s)\") || Pins(Binding=\"%s\") || Binding=\"%s\""), *MemberName.ToString(), *MemberName.ToString(), *MemberName.ToString(), *MemberName.ToString());
		}
	}
	else
	{
		return FString::Printf(TEXT("Nodes(VariableReference((MemberName=+\"%s\" && MemberScope=+\"%s\"))) || Binding=\"%s\""), *MemberName.ToString(), *GetMemberScopeName(), *MemberName.ToString());
	}
}

FName FMemberReference::RefreshLocalVariableName(UClass* InSelfScope) const
{
	TArray<UBlueprint*> Blueprints;
	UBlueprint::GetBlueprintHierarchyFromClass(InSelfScope, Blueprints);

	FName RenamedMemberName = NAME_None;
	for (int32 BPIndex = 0; BPIndex < Blueprints.Num(); ++BPIndex)
	{
		RenamedMemberName = FBlueprintEditorUtils::FindLocalVariableNameByGuid(Blueprints[BPIndex], MemberGuid);
		if (RenamedMemberName != NAME_None)
		{
			MemberName = RenamedMemberName;
			break;
		}
	}
	return RenamedMemberName;
}

bool FMemberReference::bFieldRedirectMapInitialized = false;

void FMemberReference::InitFieldRedirectMap()
{
	// Soft deprecated, replaced by FCoreRedirects but will read the old ini format for the foreseeable future
	if (!bFieldRedirectMapInitialized)
	{
		if (GConfig)
		{
			TArray<FCoreRedirect> NewRedirects;

			const FConfigSection* PackageRedirects = GConfig->GetSection( TEXT("/Script/Engine.Engine"), false, GEngineIni );
			for (FConfigSection::TConstIterator It(*PackageRedirects); It; ++It)
			{
				if (It.Key() == TEXT("K2FieldRedirects"))
				{
					FString OldFieldPathString;
					FString NewFieldPathString;

					FParse::Value( *It.Value().GetValue(), TEXT("OldFieldName="), OldFieldPathString );
					FParse::Value( *It.Value().GetValue(), TEXT("NewFieldName="), NewFieldPathString );

					// Add both a Property and Function redirect, as we don't know which it's trying to be with the old syntax
					FCoreRedirect* PropertyRedirect = new (NewRedirects) FCoreRedirect(ECoreRedirectFlags::Type_Property, OldFieldPathString, NewFieldPathString);
					FCoreRedirect* FunctionRedirect = new (NewRedirects) FCoreRedirect(ECoreRedirectFlags::Type_Function, OldFieldPathString, NewFieldPathString);
				}			
				if (It.Key() == TEXT("K2ParamRedirects"))
				{
					// Ignore NodeName/Title as it's not useful
					FName OldParam = NAME_None;
					FName NewParam = NAME_None;

					FString OldParamValues;
					FString NewParamValues;
					FString CustomValueMapping;

					FParse::Value( *It.Value().GetValue(), TEXT("OldParamName="), OldParam );
					FParse::Value( *It.Value().GetValue(), TEXT("NewParamName="), NewParam );
					FParse::Value( *It.Value().GetValue(), TEXT("OldParamValues="), OldParamValues );
					FParse::Value( *It.Value().GetValue(), TEXT("NewParamValues="), NewParamValues );
					FParse::Value( *It.Value().GetValue(), TEXT("CustomValueMapping="), CustomValueMapping );

					TArray<FString> OldParamValuesList;
					TArray<FString> NewParamValuesList;
					OldParamValues.ParseIntoArray(OldParamValuesList, TEXT(";"), false);
					NewParamValues.ParseIntoArray(NewParamValuesList, TEXT(";"), false);

					if (OldParamValuesList.Num() != NewParamValuesList.Num())
					{
						UE_LOGF(LogBlueprint, Warning, "Unequal lengths for old and new param values for  param redirect '%ls' to '%ls'.", *(OldParam.ToString()), *(NewParam.ToString()));
					}

					if (CustomValueMapping.Len() > 0 && (OldParamValuesList.Num() > 0 || NewParamValuesList.Num() > 0))
					{
						UE_LOGF(LogBlueprint, Warning, "Both Custom and Automatic param value remapping specified for param redirect '%ls' to '%ls'.  Only Custom will be applied.", *(OldParam.ToString()), *(NewParam.ToString()));
					}

					FCoreRedirect* Redirect = new (NewRedirects) FCoreRedirect(ECoreRedirectFlags::Type_Property, OldParam.ToString(), NewParam.ToString());

					for (int32 i = FMath::Min(OldParamValuesList.Num(), NewParamValuesList.Num()) - 1; i >= 0; --i)
					{
						int32 CurSize = Redirect->ValueChanges.Num();
						Redirect->ValueChanges.Add(OldParamValuesList[i], NewParamValuesList[i]);
						if (CurSize == Redirect->ValueChanges.Num())
						{
							UE_LOGF(LogBlueprint, Warning, "Duplicate old param value '%ls' for param redirect '%ls' to '%ls'.", *(OldParamValuesList[i]), *(OldParam.ToString()), *(NewParam.ToString()));
						}
					}
				}			
			}

			FCoreRedirects::AddRedirectList(NewRedirects, TEXT("InitFieldRedirectMap"));
			bFieldRedirectMapInitialized = true;
		}
	}
}

UClass* FMemberReference::GetClassToUse(UClass* InClass, bool bUseUpToDateClass)
{
	if(bUseUpToDateClass)
	{
		return FBlueprintEditorUtils::GetMostUpToDateClass(InClass);
	}
	else
	{
		return InClass;
	}
}

#endif
template<class TFieldType>
TFieldType* ResolveUFunctionImpl(FName MemberName)
{
	return nullptr;
}

template<>
inline UFunction* ResolveUFunctionImpl(FName MemberName)
{
	UFunction* ReturnField = nullptr;
	FString const StringName = MemberName.ToString();
	for (TObjectIterator<UPackage> PackageIt; PackageIt && (ReturnField == nullptr); ++PackageIt)
	{
		if (PackageIt->HasAnyPackageFlags(PKG_CompiledIn) == false)
		{
			continue;
		}

		// NOTE: this could return the wrong field (if there are 
		//       two like-named delegates defined in separate packages)
		ReturnField = FindObject<UFunction>(*PackageIt, *StringName);
	}
	return ReturnField;
}

template<class TFieldType>
TFieldType* ResolveUField(FFieldClass* InClass, UPackage* TargetPackage, FName MemberName)
{
	return nullptr;
}
template<class TFieldType>
TFieldType* ResolveUField(UClass* InClass, UPackage* TargetPackage, FName MemberName) 
{
	return FindObject<TFieldType>(TargetPackage, *MemberName.ToString());;
}

template<class TFieldType>
UObject* GetFieldOuter(TFieldType Field)
{
	return Field->GetOuter();
}

template<>
inline UObject* GetFieldOuter(FField* Field)
{
	return Field->GetOwner<UObject>();
}

template<class TFieldType, class TFieldTypeClass>
TFieldType* FMemberReference::ResolveMemberImpl(UClass* SelfScope, TFieldTypeClass* FieldClass, const bool bAlwaysFollowRedirects, const bool bIsUFunctionOrMulticastDelegate) const
{
	TFieldType* ReturnField = nullptr;

	if (bSelfContext && SelfScope == nullptr)
	{
		UE_LOGF(LogBlueprint, Warning, "FMemberReference::ResolveMember (%ls) bSelfContext == true, but no scope supplied!", *MemberName.ToString());
	}

	// Check if the member reference is function scoped
	if (IsLocalScope())
	{
		UStruct* MemberScopeStruct = FindUField<UStruct>(SelfScope, *MemberScope);

		// Find in target scope
		ReturnField = FindUFieldOrFProperty<TFieldType>(MemberScopeStruct, MemberName, EFieldIterationFlags::IncludeAll);

#if WITH_EDITOR
		if (ReturnField == nullptr)
		{
			// If the property was not found, refresh the local variable name and try again
			const FName RenamedMemberName = RefreshLocalVariableName(SelfScope);
			if (RenamedMemberName != NAME_None)
			{
				ReturnField = FindUFieldOrFProperty<TFieldType>(MemberScopeStruct, MemberName, EFieldIterationFlags::IncludeAll);
			}
		}
#endif
	}
	else
	{
#if WITH_EDITOR
		const bool bCanFollowRedirects = !UE::IsSavingPackage();
#else
		const bool bCanFollowRedirects = bAlwaysFollowRedirects;
#endif

		// Look for remapped member
		UClass* TargetScope = GetScope(SelfScope);
		if (bCanFollowRedirects && TargetScope)
		{
			// bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect is required to avoid invalid redirects. Usecase
			// showing why: Consider a BPGC based on Pawn, and it overrides the OnLand event. But FindRemappedField
			// returns Character::OnLanded because of a redirector on Pawn:
			//    "+K2FieldRedirects=(OldFieldName="Pawn.OnLanded",NewFieldName="Character.OnLanded")"
			// Character.OnLanded is not a valid field for the BPGC that inherits from Pawn.
			constexpr bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect = true;
			ReturnField = static_cast<TFieldType*>(FindRemappedField(FieldClass, TargetScope, MemberName,
				bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect));
		}

		if (ReturnField != nullptr)
		{
			// Fix up this struct, we found a redirect
			MemberName = ReturnField->GetFName();
			MemberParent = Cast<UClass>(GetFieldOuter(static_cast<typename TFieldType::BaseFieldClass*>(ReturnField)));

			MemberGuid.Invalidate();
			UBlueprint::GetGuidFromClassByFieldName<TFieldType>(TargetScope, MemberName, MemberGuid);

			if (UClass* ParentAsClass = GetMemberParentClass())
			{
#if WITH_EDITOR
				ParentAsClass = ParentAsClass->GetAuthoritativeClass();
#endif
				MemberParent = ParentAsClass;

				// Re-evaluate self-ness against the redirect if we were given a valid SelfScope
				// For functions and multicast delegates we don't want to go from not-self to self as the target pin type should remain consistent
				if (SelfScope != nullptr && 
					(bSelfContext || !bIsUFunctionOrMulticastDelegate))
				{
#if WITH_EDITOR
					bSelfContext = SelfScope->IsChildOf(ParentAsClass) || SelfScope->ClassGeneratedBy == ParentAsClass->ClassGeneratedBy;
#else
					bSelfContext = SelfScope->IsChildOf(ParentAsClass);
#endif

					if (bSelfContext)
					{
						MemberParent = nullptr;
					}
				}
			}
		}
		else if (TargetScope != nullptr)
		{
#if WITH_EDITOR
			bool bUseUpToDateClass = SelfScope && SelfScope->GetAuthoritativeClass() != SelfScope;
			TargetScope = GetClassToUse(TargetScope, bUseUpToDateClass);
			if (TargetScope)
#endif
			{
				auto FindSparseClassDataField = [this, TargetScope]() -> TFieldType*
				{
					if (UScriptStruct* SparseClassDataStruct = TargetScope->GetSparseClassDataStruct())
					{
						return FindUFieldOrFProperty<TFieldType>(SparseClassDataStruct, MemberName, EFieldIterationFlags::IncludeAll);
					}
					return nullptr;
				};

				// Check the target scope first
				if (TFieldType* TargetField = FindUFieldOrFProperty<TFieldType>(TargetScope, MemberName, EFieldIterationFlags::IncludeAll))
				{
					if (FProperty* TargetProperty = FFieldVariant(TargetField).Get<FProperty>();
						TargetProperty && TargetProperty->HasAllPropertyFlags(CPF_Deprecated))
					{
						// If this property is deprecated, check to see if the sparse data has a property that 
						// we should use instead (eg, when migrating data from an object into the sparse data)
						ReturnField = FindSparseClassDataField();
					}
					if (!ReturnField)
					{
						ReturnField = TargetField;
					}
				}

				// Check the sparse data for the field
				if (!ReturnField)
				{
					ReturnField = FindSparseClassDataField();
				}
			}


#if !WITH_EDITOR
			if (bAlwaysFollowRedirects)
#endif
			{
				// If the reference variable is valid we need to make sure that our GUID matches
				if (ReturnField != nullptr)
				{
					UBlueprint::GetGuidFromClassByFieldName<TFieldType>(TargetScope, MemberName, MemberGuid);
				}
				// If we have a GUID find the reference variable and make sure the name is up to date and find the field again
				// For now only variable references will have valid GUIDs.  Will have to deal with finding other names subsequently
				else if (MemberGuid.IsValid())
				{
					const FName RenamedMemberName = UBlueprint::GetFieldNameFromClassByGuid<TFieldType>(TargetScope, MemberGuid);
					if (RenamedMemberName != NAME_None)
					{
						MemberName = RenamedMemberName;
						ReturnField = FindUFieldOrFProperty<TFieldType>(TargetScope, MemberName, EFieldIterationFlags::IncludeAll);
					}
				}
			}
		}
		else if (UPackage* TargetPackage = GetMemberParentPackage())
		{
			ReturnField = ResolveUField<TFieldType>(FieldClass, TargetPackage, MemberName);
		}
		// For backwards compatibility: as of CL 2412156, delegate signatures 
		// could have had a null MemberParentClass (for those natively 
		// declared outside of a class), we used to rely on the following 
		// FindObject<>; however this was not reliable (hence the addition 
		// of GetMemberParentPackage(), etc.)
		else if (MemberName.ToString().EndsWith(HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX))
		{
			ReturnField = ResolveUFunctionImpl<TFieldType>(MemberName);
			if (ReturnField != nullptr)
			{
				UE_LOGF(LogBlueprint, Display, "Generic delegate signature ref (%ls). Explicitly setting it to: '%ls'. Make sure this is correct (there could be multiple native delegate types with this name).", *MemberName.ToString(), *ReturnField->GetPathName());
				MemberParent = ReturnField->GetOutermost();
			}
		}
	}
	// Check to see if the member has been deprecated
	if (FProperty* Property = FFieldVariant(ReturnField).Get<FProperty>())
	{
#if WITH_EDITORONLY_DATA
		// Initially this originated from python bindings, but this is useful to check for blueprints too, so that they can be upgraded.
		static const FName NAME_DeprecatedProperty = "DeprecatedProperty";
		bWasDeprecated = Property->HasAnyPropertyFlags(CPF_Deprecated) || Property->HasMetaData(NAME_DeprecatedProperty);
#else
		bWasDeprecated = Property->HasAnyPropertyFlags(CPF_Deprecated);
#endif // WITH_EDITORONLY_DATA
	}

	return ReturnField;
}

FProperty* FMemberReference::ResolveMemberProperty(UClass* SelfScope, const bool bAlwaysFollowRedirects, FFieldClass* FieldClass) const
{
	return ResolveMemberImpl<FProperty, FFieldClass>(SelfScope, FieldClass, bAlwaysFollowRedirects, FieldClass == FMulticastDelegateProperty::StaticClass());
}

UFunction* FMemberReference::ResolveMemberFunction(UClass* SelfScope, const bool bAlwaysFollowRedirects) const
{
	const bool bIsAUFunction = true;
	return ResolveMemberImpl<UFunction, UClass>(SelfScope, UFunction::StaticClass(), bAlwaysFollowRedirects, bIsAUFunction);
}

template <typename TFieldType>
TFieldType* FindRemappedFieldImpl(FName FieldClassOutermostName, FName FieldClassName, UClass* InitialScope,
	FName InitialName, bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FMemberReference::FindRemappedField"), STAT_LinkerLoad_FindRemappedField, STATGROUP_LoadTimeVerbose);

#if WITH_EDITOR
	FMemberReference::InitFieldRedirectMap();
#endif

	// In the case of a bifurcation of a variable (e.g. moved from a parent into certain children), verify that we don't also define the variable in the current scope first
	if (FindUFieldOrFProperty<TFieldType>(InitialScope, InitialName))
	{
		return nullptr;
	}

	// Step up the class chain to check if us or any of our parents specify a redirect
	UClass* TestRemapClass = InitialScope;
	while (TestRemapClass != nullptr)
	{
		UClass* SearchClass = TestRemapClass;

		FName NewFieldName;

		FCoreRedirectObjectName OldRedirectName = FCoreRedirectObjectName(InitialName, TestRemapClass->GetFName(), TestRemapClass->GetOutermost()->GetFName());
		FCoreRedirectObjectName NewRedirectName = FCoreRedirects::GetRedirectedName(FCoreRedirects::GetFlagsForTypeName(FieldClassOutermostName, FieldClassName), OldRedirectName);

		if (NewRedirectName != OldRedirectName)
		{
			NewFieldName = NewRedirectName.ObjectName;

			if (OldRedirectName.OuterName != NewRedirectName.OuterName)
			{
				// Try remapping class, this only works if class is in memory
				FString ClassName = NewRedirectName.OuterName.ToString();

				if ( !NewRedirectName.PackageName.IsNone() )
				{
					// Use package if it's there
					ClassName = FString::Printf(TEXT("%s.%s"), *NewRedirectName.PackageName.ToString(), *NewRedirectName.OuterName.ToString());
					SearchClass = (UClass*)StaticFindObject(UClass::StaticClass(), nullptr, *ClassName);
				}
				else
				{
					SearchClass = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Fatal, TEXT("FindRemappedFieldImpl"));
				}				

				if (!SearchClass)
				{
					UE_LOGF(LogBlueprint, Log, "UK2Node:  Unable to find updated field name for '%ls' on unknown class '%ls'.", *InitialName.ToString(), *ClassName);
					return nullptr;
				}
			}
		}

		if (NewFieldName != NAME_None)
		{
			// Find the actual field specified by the redirector, so we can return it and update the node that uses it
			TFieldType* NewField = FindUFieldOrFProperty<TFieldType>(SearchClass, NewFieldName);
			if (NewField != nullptr)
			{
				if (bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect &&
					TestRemapClass != InitialScope && !InitialScope->IsChildOf(SearchClass))
				{
					UE_LOGF(LogBlueprint, Log, "UK2Node:  Unable to update field. Remapped field '%ls' in not owned by given scope. Scope: '%ls', Owner: '%ls'.", *InitialName.ToString(), *InitialScope->GetName(), *NewFieldName.ToString());
				}
				else
				{
					UE_LOGF(LogBlueprint, Log, "UK2Node:  Fixed up old field '%ls' to new name '%ls' on class '%ls'.", *InitialName.ToString(), *NewFieldName.ToString(), *SearchClass->GetName());
					return NewField;
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Log, "UK2Node:  Unable to find updated field name for '%ls' on class '%ls'.", *InitialName.ToString(), *SearchClass->GetName());
			}

			return nullptr;
		}

		TestRemapClass = TestRemapClass->GetSuperClass();
	}

	return nullptr;
}

UField* FMemberReference::FindRemappedField(UClass* FieldClass, UClass* InitialScope, FName InitialName,
	bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect)
{	
	return FindRemappedFieldImpl<UField>(FieldClass->GetOutermost()->GetFName(), FieldClass->GetFName(), InitialScope,
		InitialName, bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect);
}

FField* FMemberReference::FindRemappedField(FFieldClass* FieldClass, UClass* InitialScope, FName InitialName,
	bool bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect)
{
	return FindRemappedFieldImpl<FField>(GLongCoreUObjectPackageName, FieldClass->GetFName(), InitialScope, InitialName,
		bInitialScopeMustBeOwnerOfFieldForParentScopeRedirect);
}
