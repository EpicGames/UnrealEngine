// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshBatch.h"
#include "MaterialShared.h"
#include "Materials/MaterialRenderProxy.h"

bool FMeshBatch::IsTranslucent(ERHIFeatureLevel::Type InFeatureLevel) const
{
	// Note: blend mode does not depend on the feature level we are actually rendering in.
	return IsTranslucentBlendMode(MaterialRenderProxy->GetIncompleteMaterialWithFallback(InFeatureLevel));
}

bool FMeshBatch::IsDecal(ERHIFeatureLevel::Type InFeatureLevel) const
{
	// Note: does not depend on the feature level we are actually rendering in.
	const FMaterial& Mat = MaterialRenderProxy->GetIncompleteMaterialWithFallback(InFeatureLevel);
	return Mat.IsDeferredDecal();
}

bool FMeshBatch::IsDualBlend(ERHIFeatureLevel::Type InFeatureLevel) const
{
	const FMaterial& Mat = MaterialRenderProxy->GetIncompleteMaterialWithFallback(InFeatureLevel);
	return Mat.IsDualBlendingEnabled();
}

bool FMeshBatch::UseForHairStrands(ERHIFeatureLevel::Type InFeatureLevel) const
{
	if (InFeatureLevel != ERHIFeatureLevel::SM5)
	{
		return false;
	}

	const FMaterial& Mat = MaterialRenderProxy->GetIncompleteMaterialWithFallback(InFeatureLevel);
	return IsCompatibleWithHairStrands(&Mat, InFeatureLevel);
}

bool FMeshBatch::IsMasked(ERHIFeatureLevel::Type InFeatureLevel) const
{
	// Note: blend mode does not depend on the feature level we are actually rendering in.
	return MaterialRenderProxy->GetIncompleteMaterialWithFallback(InFeatureLevel).IsMasked();
}
