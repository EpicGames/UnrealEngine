// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshBudgetProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBudgetProjectSettings)

#if WITH_EDITOR
#include "Engine/StaticMesh.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "StaticMeshResources.h"

#define LOCTEXT_NAMESPACE "MeshBudget"

DEFINE_LOG_CATEGORY(LogMeshBudget);

void FMeshBudgetProjectSettingsUtils::SetLodGroupForStaticMesh(UStaticMesh* StaticMesh)
{
	//Log only once the project settings error
	static bool bProjectSettingsValidated = false;

	const UMeshBudgetProjectSettings* MeshBudgetProjectSettings = GetDefault<UMeshBudgetProjectSettings>();
	if (!MeshBudgetProjectSettings || !MeshBudgetProjectSettings->bEnableStaticMeshBudget)
	{
		return;
	}

	const ITargetPlatform* Platform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	check(Platform);
	TArray<FName> LodGroupNames;
	Platform->GetStaticMeshLODSettings().GetLODGroupNames(LodGroupNames);

	if (StaticMesh->GetLODGroup() != NAME_None)
	{
		if (LodGroupNames.Contains(StaticMesh->GetLODGroup()))
		{
			//We already have a valid LODGroup name
			return;
		}
	}

	//We must assign a budget LOD group, we will assign the maximum budget that fit the extent of the mesh
	const double AssetExtent = StaticMesh->GetBoundingBox().GetExtent().Size();
	FName ValidBudgetName = NAME_None;
	double MaxMinimumExtentAssign = -1.0;
	for (const FStaticMeshBudgetInfo& MeshBudgetInfo : MeshBudgetProjectSettings->StaticMeshBudgetInfos)
	{
		if (!LodGroupNames.Contains(MeshBudgetInfo.LodGroupName))
		{
			if (!bProjectSettingsValidated)
			{
				UE_LOGF(LogMeshBudget, Warning, "MeshBudgetProjectSettings MeshBudgetInfo [%ls] LOD group name do not exist add it to the ini file. Skipping this budget... ", *MeshBudgetInfo.LodGroupName.ToString());
			}
			//Skip invalid budget lod group name
			continue;
		}

		if (MaxMinimumExtentAssign < MeshBudgetInfo.MinimumExtent && AssetExtent >= MeshBudgetInfo.MinimumExtent)
		{
			ValidBudgetName = MeshBudgetInfo.LodGroupName;
			MaxMinimumExtentAssign = MeshBudgetInfo.MinimumExtent;
		}
	}

	if (ValidBudgetName != NAME_None)
	{
		StaticMesh->SetLODGroup(ValidBudgetName, false, false);
	}
	else if (!bProjectSettingsValidated)
	{
		UE_LOGF(LogMeshBudget, Error, "MeshBudgetProjectSettings Cannot assign any lod group to static mesh, the budget data is not configure properly in the project settings. ");
	}

	bProjectSettingsValidated = true;
}

#undef LOCTEXT_NAMESPACE

#endif
