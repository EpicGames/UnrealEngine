// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshCardRepresentation.cpp
=============================================================================*/

#include "MeshCardRepresentation.h"
#include "MeshCardBuild.h"
#include "Engine/StaticMeshSourceData.h"
#include "StaticMeshResources.h"
#include "EngineLogs.h"
#include "ProfilingDebugging/CookStats.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "Misc/AutomationTest.h"
#include "DistanceFieldAtlas.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "ObjectCacheContext.h"
#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"

#if WITH_EDITOR
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheKey.h"
#include "StaticMeshCompiler.h"
#include "MeshUtilities.h"
#endif

#define LOCTEXT_NAMESPACE "MeshCardRepresentation"

#if WITH_EDITORONLY_DATA
#include "IMeshBuilderModule.h"
#endif

struct FSignedDistanceFieldBuildPartData;

#if ENABLE_COOK_STATS
namespace CardRepresentationCookStats
{
	FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("CardRepresentation.Usage"), TEXT(""));
	});
}
#endif

static TAutoConsoleVariable<int32> CVarCardRepresentation(
	TEXT("r.MeshCardRepresentation"),
	1,
	TEXT(""),
	ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarCardRepresentationMinDensity(
	TEXT("r.MeshCardRepresentation.MinDensity"),
	0.2f,
	TEXT("How much of filled area needs to be there to spawn a card, [0;1] range."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarCardRepresentationNormalTreshold(
	TEXT("r.MeshCardRepresentation.NormalTreshold"),
	0.25f,
	TEXT("Normal treshold when surface elements should be clustered together."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarCardRepresentationDebug(
	TEXT("r.MeshCardRepresentation.Debug"),
	0,
	TEXT("Enable mesh cards debugging. Skips DDCs and appends extra debug data."),
	ECVF_Cheat);

static TAutoConsoleVariable<int32> CVarCardRepresentationDebugSurfelDirection(
	TEXT("r.MeshCardRepresentation.Debug.SurfelDirection"),
	-1,
	TEXT("Generate cards for only surfels pointing in a specific direction."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarMeshCardRepresentationEstimate(
	TEXT("r.CookMemory.MeshCardRepresentationEstimate"), true,
	TEXT("Enable memory estimates for mesh card representation asset cooks"),
	ECVF_ReadOnly
);

FAsyncCardRepresentationTask* CreateCardRepresentationTask(
	const FString& InDDCKey,
	UStaticMesh* Mesh,
	UStaticMesh* GenerateSource,
	FDistanceFieldVolumeData* DistanceFieldData,
	int32 MaxLumenMeshCards,
	bool bGenerateDistanceFieldAsIfTwoSided,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData,
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData,
	FCardRepresentationData& OutRenderData);

float MeshCardRepresentation::GetMinDensity()
{
	return FMath::Clamp(CVarCardRepresentationMinDensity.GetValueOnAnyThread(), 0.0f, 1.0f);
}

float MeshCardRepresentation::GetNormalTreshold()
{
	return FMath::Clamp(CVarCardRepresentationNormalTreshold.GetValueOnAnyThread(), 0.0f, 1.0f);
}

bool MeshCardRepresentation::IsDebugMode()
{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	return CVarCardRepresentationDebug.GetValueOnAnyThread() != 0;
#else
	return false;
#endif
}

int32 MeshCardRepresentation::GetDebugSurfelDirection()
{
	return FMath::Clamp(CVarCardRepresentationDebugSurfelDirection.GetValueOnAnyThread(), -1, 5);
}

FVector3f MeshCardRepresentation::GetAxisAlignedDirection(uint32 AxisAlignedDirectionIndex)
{
	const uint32 AxisIndex = AxisAlignedDirectionIndex / 2;

	FVector3f Direction(0.0f, 0.0f, 0.0f);
	Direction[AxisIndex] = AxisAlignedDirectionIndex & 1 ? 1.0f : -1.0f;
	return Direction;
}

int32 MeshCardRepresentation::GetAxisAlignedDirectionIndex(const FVector3f& Direction)
{
	const FVector3f DirectionAbs = Direction.GetAbs();

	if (DirectionAbs == FVector3f::XAxisVector)
	{
		return Direction.X > 0.0f ? 1 : 0;
	}
	else if (DirectionAbs == FVector3f::YAxisVector)
	{
		return Direction.Y > 0.0f ? 3 : 2;
	}
	else if (DirectionAbs == FVector3f::ZAxisVector)
	{
		return Direction.Z > 0.0f ? 5 : 4;
	}
	else
	{
		return INDEX_NONE;
	}
}

void MeshCardRepresentation::SetCardsFromBounds(FMeshCardsBuildData& CardData, ELumenCardDilationMode DilationMode, bool bCardCoversHalfBounds)
{
	CardData.CardBuildData.SetNum(6);

	const FVector3f MeshCenter(CardData.Bounds.GetCenter());
	const FVector3f MeshExtent(CardData.Bounds.GetExtent());
	const float CardZOffset = 5.f;

	for (uint32 DirectionIndex = 0; DirectionIndex < 6u; ++DirectionIndex)
	{
		FLumenCardOBBf OBB;

		OBB.AxisZ = GetAxisAlignedDirection(DirectionIndex);
		OBB.AxisZ.FindBestAxisVectors(OBB.AxisX, OBB.AxisY);
		OBB.AxisX = FVector3f::CrossProduct(OBB.AxisZ, OBB.AxisY);
		OBB.AxisX.Normalize();

		// Project 3D mesh extent onto the specific axes of this CardOBB (this just reshuffles them but the combinatorics are messy).
		OBB.Extent.X = FMath::Abs(OBB.AxisX.Dot(MeshExtent));
		OBB.Extent.Y = FMath::Abs(OBB.AxisY.Dot(MeshExtent));
		OBB.Extent.Z = FMath::Abs(OBB.AxisZ.Dot(MeshExtent));

		if (bCardCoversHalfBounds)
		{
			// Shrink and shift the card box to cover half the mesh bounds along Z and then bump it forward a bit
			OBB.Extent.Z = OBB.Extent.Z * 0.5f + CardZOffset;
			OBB.Origin = MeshCenter + OBB.Extent.Z * OBB.AxisZ;
		}
		else
		{
			OBB.Extent.Z += CardZOffset;
			OBB.Origin = MeshCenter + CardZOffset * OBB.AxisZ;
		}

		CardData.CardBuildData[DirectionIndex].OBB = OBB;
		CardData.CardBuildData[DirectionIndex].AxisAlignedDirectionIndex = DirectionIndex;
		CardData.CardBuildData[DirectionIndex].DilationMode = DilationMode;
	}
}

FCardRepresentationAsyncQueue* GCardRepresentationAsyncQueue = NULL;

#if WITH_EDITORONLY_DATA

// DDC key for card representation data, must be changed when modifying the generation code or data format
#define CARDREPRESENTATION_DERIVEDDATA_VER TEXT("BE3395EF-F023-4F1D-9827-1DD3C88E16DC")

void AppendMeshCardConstantsToClassSchema(FAppendToClassSchemaContext& Context)
{
	const float MinDensity = MeshCardRepresentation::GetMinDensity();
	const float NormalTreshold = MeshCardRepresentation::GetNormalTreshold();
	uint8 DebugMode = MeshCardRepresentation::IsDebugMode() ? 1 : 0;
	FStringView VersionString(CARDREPRESENTATION_DERIVEDDATA_VER);

	Context.Update(&MinDensity, sizeof(MinDensity));
	Context.Update(&NormalTreshold, sizeof(NormalTreshold));
	Context.Update(&DebugMode, sizeof(DebugMode));
	Context.Update(VersionString.GetData(), VersionString.Len() * sizeof(VersionString[0]));
}

FString BuildCardRepresentationDerivedDataKey(const FString& InMeshKey, int32 MaxLumenMeshCards)
{
	const float MinDensity = MeshCardRepresentation::GetMinDensity();
	const float NormalTreshold = MeshCardRepresentation::GetNormalTreshold();
	const bool bDebugMode = MeshCardRepresentation::IsDebugMode();

	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacyCARD"), TEXTVIEW("CardRepresentation"));
	return FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("CARD"),
		*FString::Printf(TEXT("%s_%s%s%.3f_%.3f_%d"), *InMeshKey, CARDREPRESENTATION_DERIVEDDATA_VER, bDebugMode ? TEXT("_DEBUG_") : TEXT(""),
			MinDensity, NormalTreshold, MaxLumenMeshCards),
		TEXT(""));
}

extern void BuildSignedDistanceFieldBuildSectionData(
	const FMeshSectionInfoMap& SectionInfoMap,
	uint32 LODIndex,
	const TArray<FStaticMaterial>& StaticMaterials,
	const TArray<int32>& MaterialRemap,
	const FStaticMeshRenderData& RenderData,
	TArray<FSignedDistanceFieldBuildSectionData>& OutData);

FAsyncCardRepresentationTask* BeginCacheMeshCardRepresentationInternal(
	UStaticMesh* StaticMeshAsset,
	FStaticMeshRenderData& RenderData,
	const FString& DistanceFieldKey,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData,
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData,
	FDistanceFieldVolumeData* DistanceFieldData = nullptr)
{
	static const auto CVarCards = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MeshCardRepresentation"));

	FAsyncCardRepresentationTask* NewTask = nullptr;

	if (CVarCards->GetValueOnAnyThread() != 0)
	{
		if (RenderData.LODResources.IsValidIndex(0))
		{
			if (!RenderData.LODResources[0].CardRepresentationData)
			{
				RenderData.LODResources[0].CardRepresentationData = new FCardRepresentationData();
			}

			// Only generate distance fields and card representations for the base render data, not platform render data.
			if (&RenderData == StaticMeshAsset->GetRenderData())
			{
				const FStaticMeshSourceModel& SourceModel = StaticMeshAsset->GetSourceModel(0);
				const FMeshBuildSettings& BuildSettings = SourceModel.BuildSettings;
				FString Key = BuildCardRepresentationDerivedDataKey(DistanceFieldKey, BuildSettings.MaxLumenMeshCards);
				
				// Version the key against changes in the source model contents.
				if (SourceModel.GetMeshDescriptionBulkData())
				{
					Key += SourceModel.GetMeshDescriptionBulkData()->GetUnversionedIdString();
				}

				UStaticMesh* MeshToGenerateFrom = StaticMeshAsset;

				NewTask = CreateCardRepresentationTask(
					Key,
					StaticMeshAsset,
					MeshToGenerateFrom,
					DistanceFieldData,
					BuildSettings.MaxLumenMeshCards,
					BuildSettings.bGenerateDistanceFieldAsIfTwoSided,
					MoveTemp(OptionalSourceMeshData),
					MoveTemp(OptionalPartData),
					*RenderData.LODResources[0].CardRepresentationData);
			}
			else
			{
				// Copy the base render data to platform render data.  If it hasn't been generated or loaded from DDC yet, it will be empty,
				// and the async queue will later copy the actual data when complete.
				FCardRepresentationData* BaseCardRepresentationData = StaticMeshAsset->GetRenderData()->LODResources[0].CardRepresentationData;
				if (BaseCardRepresentationData)
				{
					*RenderData.LODResources[0].CardRepresentationData = *BaseCardRepresentationData;
				}
			}
		}
	}

	return NewTask;
}

FAsyncCardRepresentationTask* CreateCardRepresentationTask(
	const FString& InDDCKey,
	UStaticMesh* Mesh,
	UStaticMesh* GenerateSource,
	FDistanceFieldVolumeData* DistanceFieldData,
	int32 MaxLumenMeshCards,
	bool bGenerateDistanceFieldAsIfTwoSided,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData,
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData,
	FCardRepresentationData& OutRenderData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationData::CacheDerivedData);

	TArray<uint8> DerivedData;

	COOK_STAT(auto Timer = CardRepresentationCookStats::UsageStats.TimeSyncWork());

	if (!MeshCardRepresentation::IsDebugMode() && GetDerivedDataCacheRef().GetSynchronous(*InDDCKey, DerivedData, Mesh->GetPathName()))
	{
		COOK_STAT(Timer.AddHit(DerivedData.Num()));
		FMemoryReader Ar(DerivedData, /*bIsPersistent=*/ true);
		Ar << OutRenderData;

		if (Ar.IsError())
		{
			UE_LOGF(LogStaticMesh, Error, "Error while deserializing Mesh Card derived data for %ls from DDC (key %ls)", *Mesh->GetPathName(), *InDDCKey);
		}
	}
	else
	{
		check(Mesh && GenerateSource);

		// Always use GetRunningTargetPlatform() here (the cooker's platform, typically Windows Editor), as the data is intended to be platform independent.
		// Besides that, TargetPlatform is only used to fetch an IMeshBuilderModule interface, and in practice the BuildMeshVertexPositions function on
		// that interface doesn't vary per platform (all platforms internally use FStaticMeshBuilder::BuildMeshVertexPositions).
		IMeshBuilderModule& MeshBuilderModule = IMeshBuilderModule::GetForPlatform(GetTargetPlatformManagerRef().GetRunningTargetPlatform());

		// We don't actually build the resource until later, so only track the cycles used here.
		COOK_STAT(Timer.TrackCyclesOnly());
		FAsyncCardRepresentationTask* NewTask = new FAsyncCardRepresentationTask;
		NewTask->DDCKey = InDDCKey;
		NewTask->StaticMesh = Mesh;
		NewTask->GenerateSource = GenerateSource;
		NewTask->DistanceFieldData = DistanceFieldData;
		NewTask->GeneratedCardRepresentation = new FCardRepresentationData();
		NewTask->MaxLumenMeshCards = MaxLumenMeshCards;
		NewTask->bGenerateDistanceFieldAsIfTwoSided = bGenerateDistanceFieldAsIfTwoSided;

		const uint32 LODIndex = 0;

		BuildSignedDistanceFieldBuildSectionData(Mesh->GetSectionInfoMap(), LODIndex, Mesh->GetStaticMaterials(), {}, *Mesh->GetRenderData(), NewTask->SectionData);

		if (!OptionalPartData.IsEmpty())
		{
			NewTask->PartData = MoveTemp(OptionalPartData);
		}
		else if (Mesh->IsNaniteAssembly())
		{
			const TArray<TObjectPtr<UStaticMesh>>& PartReferences = Mesh->GetCachedNaniteAssemblyReferences();
			const int32 NumParts = PartReferences.Num();

			TArray<FSignedDistanceFieldBuildPartData> PartDatas;
			PartDatas.SetNum(NumParts);

			for (int32 PartIndex = 0; PartIndex < NumParts; ++PartIndex)
			{
				const TObjectPtr<UStaticMesh>& Ref = PartReferences[PartIndex];

				if (Ref)
				{
					FSignedDistanceFieldBuildPartData& PartData = PartDatas[PartIndex];

					const FMeshSectionInfoMap& SectionInfoMap = Ref->GetSectionInfoMap();
					const TArray<FStaticMaterial>& StaticMaterials = Mesh->GetStaticMaterials();
					const TArray<int32>& MaterialRemap = Mesh->GetNaniteSettings().NaniteAssemblyData.Parts[PartIndex].MaterialRemap;

					BuildSignedDistanceFieldBuildSectionData(SectionInfoMap, LODIndex, StaticMaterials, MaterialRemap, *Mesh->GetRenderData(), PartData.SectionData);

					FSourceMeshDataForDerivedDataTask& SourceMeshData = PartData.SourceMeshData;

					if (!MeshBuilderModule.BuildMeshVertexPositions(Ref, SourceMeshData.TriangleIndices, SourceMeshData.VertexPositions, SourceMeshData.Sections))
					{
						UE_LOGF(LogStaticMesh, Error, "Failed to build static mesh. See previous line(s) for details.");
					}
				}
				else
				{
					UE_LOGF(LogStaticMesh, Warning, "Nanite Assembly doesn't have a valid part reference. See previous line(s) for details.");
				}
			}

			NewTask->PartData = PartDatas;
		}

		// Nanite overrides source static mesh with a coarse representation. Need to load original data before we build the mesh SDF.
		if (OptionalSourceMeshData.IsValid())
		{
			NewTask->SourceMeshData = MoveTemp(OptionalSourceMeshData);
		}
		else if (Mesh->IsNaniteEnabled())
		{
			if (!MeshBuilderModule.BuildMeshVertexPositions(Mesh, NewTask->SourceMeshData.TriangleIndices, NewTask->SourceMeshData.VertexPositions, NewTask->SourceMeshData.Sections))
			{
				if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
				{
					delete NewTask;
					return nullptr;
				}

				UE_LOGF(LogStaticMesh, Error, "Failed to build static mesh. See previous line(s) for details.");
			}
		}

		return NewTask;
	}

	return nullptr;
}

void BeginCacheMeshCardRepresentation(
	UStaticMesh* StaticMeshAsset,
	FStaticMeshRenderData& RenderData,
	const FString& DistanceFieldKey,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData,
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData)
{
	FAsyncCardRepresentationTask* NewTask = BeginCacheMeshCardRepresentationInternal(StaticMeshAsset, RenderData, DistanceFieldKey, MoveTemp(OptionalSourceMeshData), MoveTemp(OptionalPartData));

	if (NewTask)
	{
		GCardRepresentationAsyncQueue->AddTask(NewTask);
	}
}

void FCardRepresentationData::CacheDerivedData(
	const FString& InDDCKey,
	UStaticMesh* Mesh,
	UStaticMesh* GenerateSource,
	int32 MaxLumenMeshCards,
	bool bGenerateDistanceFieldAsIfTwoSided,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData,
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData)
{
	FAsyncCardRepresentationTask* NewTask = CreateCardRepresentationTask(InDDCKey, Mesh, GenerateSource, nullptr, MaxLumenMeshCards, bGenerateDistanceFieldAsIfTwoSided, MoveTemp(OptionalSourceMeshData), MoveTemp(OptionalPartData), *this);

	if (NewTask)
	{
		GCardRepresentationAsyncQueue->AddTask(NewTask);
	}
}

#endif

void FMeshCardsBuildData::FitToNewBounds(const FBox& NewBounds)
{
	const FVector OldOrigin = Bounds.GetCenter();
	const FVector OldExtent = Bounds.GetExtent();
	const FVector NewOrigin = NewBounds.GetCenter();
	// Make sure the new bounds isn't empty. Matches the behavior in BuildMeshCards()
	const FVector NewExtent = FVector::Max(NewBounds.GetExtent() + 1.0, FVector::OneVector);

	const FVector3f OldToNewScale(NewExtent / OldExtent);

	for (FLumenCardBuildData& Card : CardBuildData)
	{
		const FVector3f CardSpaceScale = Card.OBB.RotateLocalToCard(OldToNewScale).GetAbs();
		
		Card.OBB.Extent *= CardSpaceScale;
		Card.OBB.Origin = (Card.OBB.Origin - FVector3f(OldOrigin)) * OldToNewScale + FVector3f(NewOrigin);
	}

	Bounds = FBox(NewOrigin - NewExtent, NewOrigin + NewExtent);
}

int32 GUseAsyncCardRepresentationBuildQueue = 1;
static FAutoConsoleVariableRef CVarCardRepresentationAsyncBuildQueue(
	TEXT("r.MeshCardRepresentation.Async"),
	GUseAsyncCardRepresentationBuildQueue,
	TEXT("."),
	ECVF_Default | ECVF_ReadOnly
	);

FCardRepresentationAsyncQueue::FCardRepresentationAsyncQueue()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
#if WITH_EDITOR
	MeshUtilities = NULL;

	const int32 MaxConcurrency = -1;
	// In Editor, we allow faster compilation by letting the asset compiler's scheduler organize work.
	FQueuedThreadPool* InnerThreadPool = FAssetCompilingManager::Get().GetThreadPool();
#else
	const int32 MaxConcurrency = 1;
	FQueuedThreadPool* InnerThreadPool = GThreadPool;
#endif

	if (InnerThreadPool != nullptr)
	{
		ThreadPool = MakeUnique<FQueuedThreadPoolWrapper>(InnerThreadPool, MaxConcurrency, [](EQueuedWorkPriority) { return EQueuedWorkPriority::Lowest; });
	}

	FAssetCompilingManager::Get().RegisterManager(this);

	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FCardRepresentationAsyncQueue::OnPostReachabilityAnalysis);
}

FCardRepresentationAsyncQueue::~FCardRepresentationAsyncQueue()
{
	FAssetCompilingManager::Get().UnregisterManager(this);
	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostReachabilityAnalysisHandle);
}

void FCardRepresentationAsyncQueue::OnPostReachabilityAnalysis()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::CancelUnreachableMeshes);

	CancelAndDeleteTaskByPredicate([this](FAsyncCardRepresentationTask* Task) { return IsTaskInvalid(Task); });
}
void FAsyncCardRepresentationTaskWorker::DoWork()
{
	GCardRepresentationAsyncQueue->Build(&Task);
}

FName FCardRepresentationAsyncQueue::GetStaticAssetTypeName()
{
	return TEXT("UE-MeshCard");
}

FName FCardRepresentationAsyncQueue::GetAssetTypeName() const 
{
	return GetStaticAssetTypeName();
}

FTextFormat FCardRepresentationAsyncQueue::GetAssetNameFormat() const
{
	return LOCTEXT("MeshCardNameFormat", "{0}|plural(one=Mesh Card,other=Mesh Cards)");
}

TArrayView<FName> FCardRepresentationAsyncQueue::GetDependentTypeNames() const 
{
	static FName DependentTypeNames[] = { FDistanceFieldAsyncQueue::GetStaticAssetTypeName() };
	return TArrayView<FName>(DependentTypeNames);
}

int32 FCardRepresentationAsyncQueue::GetNumRemainingAssets() const
{
	return GetNumOutstandingTasks();
}

void FCardRepresentationAsyncQueue::FinishAllCompilation()
{
	BlockUntilAllBuildsComplete();
}

bool FCardRepresentationAsyncQueue::IsTaskInvalid(FAsyncCardRepresentationTask* Task) const
{
	return (Task->StaticMesh && Task->StaticMesh->IsUnreachable()) || (Task->GenerateSource && Task->GenerateSource->IsUnreachable());
}

void FCardRepresentationAsyncQueue::CancelAndDeleteTaskByPredicate(TFunctionRef<bool (FAsyncCardRepresentationTask*)> InShouldCancelPredicate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::CancelAndDeleteTaskByPredicate);

	FScopeLock Lock(&CriticalSection);

	if (ReferencedTasks.Num() || PendingTasks.Num() || CompletedTasks.Num())
	{
		TSet<FAsyncCardRepresentationTask*> Removed;

		auto RemoveByPredicate =
			[&InShouldCancelPredicate, &Removed](TSet<FAsyncCardRepresentationTask*>& Tasks)
		{
			for (auto It = Tasks.CreateIterator(); It; ++It)
			{
				FAsyncCardRepresentationTask* Task = *It;
				if (InShouldCancelPredicate(Task))
				{
					Removed.Add(Task);
					It.RemoveCurrent();
				}
			}
		};

		RemoveByPredicate(PendingTasks);
		RemoveByPredicate(ReferencedTasks);
		RemoveByPredicate(CompletedTasks);

		Lock.Unlock();

		CancelAndDeleteTask(Removed);
	}
}

void FCardRepresentationAsyncQueue::CancelAndDeleteTask(const TSet<FAsyncCardRepresentationTask*>& Tasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::CancelAndDeleteTask);

	// Do all the cancellation first to make sure none of these tasks
	// get scheduled as we're waiting for completion.
	for (FAsyncCardRepresentationTask* Task : Tasks)
	{
		if (Task->AsyncTask)
		{
			Task->AsyncTask->Cancel();
		}
	}

	for (FAsyncCardRepresentationTask* Task : Tasks)
	{
		if (Task->AsyncTask)
		{
			Task->AsyncTask->EnsureCompletion();
			Task->AsyncTask.Reset();
		}
	}

	for (FAsyncCardRepresentationTask* Task : Tasks)
	{
		if (Task->GeneratedCardRepresentation != nullptr)
		{
			// Rendering thread may still be referencing the old one, use the deferred cleanup interface to delete it next frame when it is safe
			BeginCleanup(Task->GeneratedCardRepresentation);
		}

#if DO_GUARD_SLOW
		{
			FScopeLock Lock(&CriticalSection);
			check(!PendingTasks.Contains(Task));
			check(!ReferencedTasks.Contains(Task));
			check(!CompletedTasks.Contains(Task));
		}
#endif

		delete Task;
	}
}

void FCardRepresentationAsyncQueue::StartBackgroundTask(FAsyncCardRepresentationTask* Task)
{
	check(Task->AsyncTask == nullptr);
	
	int64 RequiredMemory = -1;

#if WITH_EDITOR
	if (CVarMeshCardRepresentationEstimate.GetValueOnAnyThread()) {
		if (!MeshUtilities)
		{
			FPlatformAtomics::InterlockedCompareExchangePointer((void**)&MeshUtilities, &FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities")), nullptr);
		}
		RequiredMemory = MeshUtilities->MemoryForGenerateMeshCardRepresentation(Task->SourceMeshData.GetNumIndices());
	}	
#endif

	Task->AsyncTask = MakeUnique<FAsyncTask<FAsyncCardRepresentationTaskWorker>>(*Task);
	Task->AsyncTask->StartBackgroundTask(ThreadPool.Get(), EQueuedWorkPriority::Lowest, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("Card"));
}

void FCardRepresentationAsyncQueue::RunImmediateTask(FAsyncCardRepresentationTask* Task)
{
#if WITH_EDITOR
	if (!MeshUtilities)
	{
		FPlatformAtomics::InterlockedCompareExchangePointer((void**)&MeshUtilities, &FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities")), nullptr);
	}

	Build(Task);

	// Array protection when called from multiple threads
	FScopeLock Lock(&CriticalSection);
	ReferencedTasks.Add(Task);
	CompletedTasks.Add(Task);
#endif
}

void FCardRepresentationAsyncQueue::ProcessPendingTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::ProcessPendingTasks);

	FScopeLock Lock(&CriticalSection);

	for (auto It = PendingTasks.CreateIterator(); It; ++It)
	{
		FAsyncCardRepresentationTask* Task = *It;
		if (Task->GenerateSource == nullptr || 
			Task->GenerateSource->IsCompiling() == false)
		{
			StartBackgroundTask(Task);
			It.RemoveCurrent();
		}
	}
}

void FCardRepresentationAsyncQueue::ProcessPendingTasksForMesh(const UStaticMesh* InStaticMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::ProcessPendingTasksForMesh);

	FScopeLock Lock(&CriticalSection);

	for (auto It = PendingTasks.CreateIterator(); It; ++It)
	{
		FAsyncCardRepresentationTask* Task = *It;
		if ((Task->StaticMesh == InStaticMesh || Task->GenerateSource == InStaticMesh) &&
			(Task->GenerateSource == nullptr || Task->GenerateSource->IsCompiling() == false))
		{
			StartBackgroundTask(Task);
			It.RemoveCurrent();
		}
	}
}

void FCardRepresentationAsyncQueue::AddTask(FAsyncCardRepresentationTask* Task)
{
#if WITH_EDITOR
	// This could happen during the cancellation of async static mesh build
	// Simply delete the task if the static mesh are being garbage collected
	if (IsTaskInvalid(Task))
	{
		CancelAndDeleteTask({ Task });
		return;
	}

	if (!MeshUtilities)
	{
		FPlatformAtomics::InterlockedCompareExchangePointer((void**)&MeshUtilities, &FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities")), nullptr);
	}

	const bool bUseAsyncBuild = GUseAsyncCardRepresentationBuildQueue || !IsInGameThread();
	const bool bIsCompiling = Task->GenerateSource->IsCompiling();
	{
		// Array protection when called from multiple threads
		FScopeLock Lock(&CriticalSection);
		check(!CompletedTasks.Contains(Task)); // reusing same pointer for a new task that is marked completed but has been canceled...
		ReferencedTasks.Add(Task);

		// The Source Mesh's RenderData is not ready yet, postpone the build
		if (bIsCompiling)
		{
			PendingTasks.Add(Task);
		}
		else if (bUseAsyncBuild)
		{
			// Make sure the Task is launched while we hold the lock to avoid
			// race with cancellation
			StartBackgroundTask(Task);
		}
	}

	if (!bIsCompiling && !bUseAsyncBuild)
	{
		Build(Task);
	}
#else
	UE_LOGF(LogStaticMesh,Fatal,"Tried to build a card representation without editor support (this should have been done during cooking)");
#endif
}

void FCardRepresentationAsyncQueue::CancelBuild(UStaticMesh* InStaticMesh)
{
	CancelBuilds({ InStaticMesh });
}

void FCardRepresentationAsyncQueue::CancelBuilds(const TSet<UStaticMesh*>& InStaticMeshes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::CancelBuilds);

	CancelAndDeleteTaskByPredicate(
		[&InStaticMeshes](FAsyncCardRepresentationTask* Task) 
		{
			return InStaticMeshes.Contains(Task->GenerateSource) || InStaticMeshes.Contains(Task->StaticMesh);
		}
	);
}

void FCardRepresentationAsyncQueue::CancelAllOutstandingBuilds()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::CancelAllOutstandingBuilds);

	TSet<FAsyncCardRepresentationTask*> OutstandingTasks;
	{
		FScopeLock Lock(&CriticalSection);
		PendingTasks.Reset();
		OutstandingTasks = MoveTemp(ReferencedTasks);
	}

	CancelAndDeleteTask(OutstandingTasks);
}

void FCardRepresentationAsyncQueue::RescheduleBackgroundTask(FAsyncCardRepresentationTask* InTask, EQueuedWorkPriority InPriority)
{
	if (InTask->AsyncTask && InTask->AsyncTask->GetPriority() != InPriority)
	{
		InTask->AsyncTask->Reschedule(GThreadPool, InPriority);
	}
}

void FCardRepresentationAsyncQueue::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::FinishCompilationForObjects);

	TSet<UStaticMesh*> StaticMeshes;
	for (UObject* Object : InObjects)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			StaticMeshes.Add(StaticMesh);
		}
		else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Object))
		{
			if (StaticMeshComponent->GetStaticMesh())
			{
				StaticMeshes.Add(StaticMeshComponent->GetStaticMesh());
			}
		}
	}

	for (UStaticMesh* StaticMesh : StaticMeshes)
	{
		BlockUntilBuildComplete(StaticMesh, false);
	}
}

void FCardRepresentationAsyncQueue::MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects)
{
	TSet<UStaticMesh*> StaticMeshes;
	for (UObject* Object : InObjects)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			StaticMeshes.Add(StaticMesh);
		}
	}

	if (StaticMeshes.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::MarkCompilationAsCanceled);

		CancelAndDeleteTaskByPredicate(
		    [&StaticMeshes](FAsyncCardRepresentationTask* Task)
		    {
			    // Mark canceled is only a hint and should be non-blocking so only remove or cancel tasks that are either not started or already finished.
			    if (Task->AsyncTask == nullptr || Task->AsyncTask->IsDone())
			    {
				    return StaticMeshes.Contains(Task->GenerateSource) || StaticMeshes.Contains(Task->StaticMesh);
			    }

				return false;
		    }
		);
	}
}

int32 FCardRepresentationAsyncQueue::GetNumOutstandingTasks(const UStaticMesh* InStaticMesh) const
{
	FScopeLock Lock(&CriticalSection);
	if (!ReferencedTasks.Num())
	{
		return 0;
	}

	int32 Count = 0;
	for (auto It = ReferencedTasks.CreateConstIterator(); It; ++It)
	{
		const FAsyncCardRepresentationTask* Task = *It;
		if (InStaticMesh == Task->GenerateSource || InStaticMesh == Task->StaticMesh)
		{
			++Count;
		}
	}

	return Count;
}

void FCardRepresentationAsyncQueue::BlockUntilBuildComplete(UStaticMesh* InStaticMesh, bool bWarnIfBlocked)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::BlockUntilBuildComplete);

	// We will track the wait time here, but only the cycles used.
	// This function is called whether or not an async task is pending, 
	// so we have to look elsewhere to properly count how many resources have actually finished building.
	COOK_STAT(auto Timer = CardRepresentationCookStats::UsageStats.TimeAsyncWait());
	COOK_STAT(Timer.TrackCyclesOnly());
	bool bReferenced = false;
	bool bHadToBlock = false;
	double StartTime = 0;

#if WITH_EDITOR
	FStaticMeshCompilingManager::Get().FinishCompilation({ InStaticMesh });
	if (GDistanceFieldAsyncQueue)
	{
		GDistanceFieldAsyncQueue->BlockUntilBuildComplete(InStaticMesh, bWarnIfBlocked);
	}	
#endif

	TSet<UStaticMesh*> RequiredFinishCompilation;
	do 
	{
		ProcessAsyncTasksForMesh(InStaticMesh);

		bReferenced = false;

		{
			FScopeLock Lock(&CriticalSection);
			for (FAsyncCardRepresentationTask* Task : ReferencedTasks)
			{
				if (Task->StaticMesh == InStaticMesh ||
					Task->GenerateSource == InStaticMesh)
				{
					bReferenced = true;

					// If the task we are waiting on depends on other static meshes
					// we need to force finish them too.
#if WITH_EDITOR
					
					if (Task->GenerateSource != nullptr &&
						Task->GenerateSource->IsCompiling())
					{
						RequiredFinishCompilation.Add(Task->GenerateSource);
					}

					if (Task->StaticMesh != nullptr &&
						Task->StaticMesh->IsCompiling())
					{
						RequiredFinishCompilation.Add(Task->StaticMesh);
					}
#endif

					RescheduleBackgroundTask(Task, EQueuedWorkPriority::Blocking);
				}
			}
		}

#if WITH_EDITOR
		// Call the finish compilation outside of the critical section since those compilations
		// might need to register new distance field tasks which also uses the critical section.
		if (RequiredFinishCompilation.Num())
		{
			FStaticMeshCompilingManager::Get().FinishCompilation(RequiredFinishCompilation.Array());
			if (GDistanceFieldAsyncQueue)
			{
				for (UStaticMesh* StaticMesh : RequiredFinishCompilation)
				{
					GDistanceFieldAsyncQueue->BlockUntilBuildComplete(StaticMesh, bWarnIfBlocked);
				}
			}
		}
#endif

		if (bReferenced)
		{
			if (!bHadToBlock)
			{
				StartTime = FPlatformTime::Seconds();
			}

			bHadToBlock = true;
			FPlatformProcess::Sleep(.01f);
		}
	} 
	while (bReferenced);

	if (bHadToBlock &&
		bWarnIfBlocked
#if WITH_EDITOR
		&& !FAutomationTestFramework::Get().GetCurrentTest() // HACK - Don't output this warning during automation test
#endif
		)
	{
		UE_LOGF(LogStaticMesh, Display, "Main thread blocked for %.3fs for async card representation build of %ls to complete!  This can happen if the mesh is rebuilt excessively.",
			(float)(FPlatformTime::Seconds() - StartTime), 
			*InStaticMesh->GetName());
	}
}

void FCardRepresentationAsyncQueue::BlockUntilAllBuildsComplete()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::BlockUntilAllBuildsComplete)
	while (true)
	{
#if WITH_EDITOR
		FStaticMeshCompilingManager::Get().FinishAllCompilation();
		if (GDistanceFieldAsyncQueue)
		{
			GDistanceFieldAsyncQueue->BlockUntilAllBuildsComplete();
		}
#endif
		// Reschedule as highest prio since we're explicitly waiting on them
		{
			FScopeLock Lock(&CriticalSection);
			for (FAsyncCardRepresentationTask* Task : ReferencedTasks)
			{
				RescheduleBackgroundTask(Task, EQueuedWorkPriority::Blocking);
			}
		}

		ProcessAsyncTasks();

		if (GetNumOutstandingTasks() <= 0)
		{
			break;
		}

		FPlatformProcess::Sleep(.01f);
	} 
}

void FCardRepresentationAsyncQueue::Build(FAsyncCardRepresentationTask* Task)
{
#if WITH_EDITOR
	COOK_STAT(auto Timer = CardRepresentationCookStats::UsageStats.TimeSyncWork());
	COOK_STAT(Timer.TrackCyclesOnly());
	
	// Editor 'force delete' can null any UObject pointers which are seen by reference collecting (eg UProperty or serialized)
	if (Task->StaticMesh && Task->GenerateSource)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::Build);

		const FStaticMeshLODResources& LODModel = Task->GenerateSource->GetRenderData()->LODResources[0];

		FMeshDataForDerivedDataTask MeshData;
		MeshData.SourceMeshData = &Task->SourceMeshData;
		MeshData.LODModel = &LODModel;
		MeshData.SectionData = Task->SectionData;
		MeshData.Bounds = (FBoxSphereBounds3f)Task->GenerateSource->GetRenderData()->Bounds;
		MeshData.Parts = Task->PartData;
		if (!MeshData.Parts.IsEmpty())
		{
			MeshData.Nodes = Task->GenerateSource->GetNaniteSettings().NaniteAssemblyData.Nodes;
		}

		Task->bSuccess = MeshUtilities->GenerateCardRepresentationData(
			Task->StaticMesh->GetName(),
			MeshData,
			Task->DistanceFieldData ? Task->DistanceFieldData : Task->GenerateSource->GetRenderData()->LODResources[0].DistanceFieldData,
			Task->MaxLumenMeshCards,
			Task->bGenerateDistanceFieldAsIfTwoSided,
			*Task->GeneratedCardRepresentation);
	}

	{
		FScopeLock Lock(&CriticalSection);
		// Avoid adding to the completed list if the task has been canceled
		if (ReferencedTasks.Contains(Task))
		{
			CompletedTasks.Add(Task);
		}
	}

#endif
}

void FCardRepresentationAsyncQueue::ProcessAsyncTasks(bool bLimitExecutionTime)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::ProcessAsyncTasks);

	ProcessPendingTasks();

	FObjectCacheContextScope ObjectCacheScope; // Declared here for caching optimization purposes
	const double MaxProcessingTime = 0.016f;
	double StartTime = FPlatformTime::Seconds();
	bool bMadeProgress = false;
	while (!bLimitExecutionTime || (FPlatformTime::Seconds() - StartTime) < MaxProcessingTime)
	{
		FAsyncCardRepresentationTask* Task = nullptr;
		{
			FScopeLock Lock(&CriticalSection);
			if (CompletedTasks.Num() > 0)
			{
				// Pop any element from the set
				auto Iterator = CompletedTasks.CreateIterator();
				Task = *Iterator;
				Iterator.RemoveCurrent();

				verify(ReferencedTasks.Remove(Task));
			}
		}

		if (Task == nullptr)
		{
			break;
		}
		bMadeProgress = true;

		FinalizeAndDeleteTask(Task, ObjectCacheScope);
	}

	if (bMadeProgress)
	{
		Notification->Update(GetNumRemainingAssets());
	}
#endif
}

void FCardRepresentationAsyncQueue::ProcessAsyncTasksForMesh(const UStaticMesh* InStaticMesh, bool bLimitExecutionTime)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FCardRepresentationAsyncQueue::ProcessAsyncTasksForMesh);

	ProcessPendingTasksForMesh(InStaticMesh);

	FObjectCacheContextScope ObjectCacheScope; // Declared here for mem/caching optimization purposes
	const double MaxProcessingTime = 0.016f;
	double StartTime = FPlatformTime::Seconds();
	bool bMadeProgress = false;
	while (!bLimitExecutionTime || (FPlatformTime::Seconds() - StartTime) < MaxProcessingTime)
	{
		FAsyncCardRepresentationTask* Task = nullptr;
		{
			// Try and find a task that matches the static mesh
			FScopeLock Lock(&CriticalSection);
			for (auto CompletedTaskIt = CompletedTasks.CreateIterator(); CompletedTaskIt; ++CompletedTaskIt)
			{
				FAsyncCardRepresentationTask* TaskToCheck = *CompletedTaskIt;

				if (TaskToCheck != nullptr && (TaskToCheck->StaticMesh == InStaticMesh || TaskToCheck->GenerateSource == InStaticMesh))
				{
					Task = TaskToCheck;
					CompletedTaskIt.RemoveCurrent();
					verify(ReferencedTasks.Remove(Task));
					break;
				}
			}
		}

		if (Task == nullptr)
		{
			break;
		}
		bMadeProgress = true;

		FinalizeAndDeleteTask(Task, ObjectCacheScope);
	}

	if (bMadeProgress)
	{
		Notification->Update(GetNumRemainingAssets());
	}
#endif
}

#if WITH_EDITOR
void FCardRepresentationAsyncQueue::FinalizeAndDeleteTask(FAsyncCardRepresentationTask* InTask, FObjectCacheContextScope& InObjectCacheScope)
{
	check(InTask);
	check(IsInGameThread()); // Timer wont work properly outside of gamethread.

	if (!InTask->bSuccess)
	{
		CancelAndDeleteTask(TSet<FAsyncCardRepresentationTask*>({ InTask }));
		return;
	}

	// We want to count each resource built from a DDC miss, so count each iteration of the loop separately.
	COOK_STAT(auto Timer = CardRepresentationCookStats::UsageStats.TimeSyncWork());

	if (InTask->AsyncTask)
	{
		InTask->AsyncTask->EnsureCompletion();
		InTask->AsyncTask.Reset();
	}

	// Editor 'force delete' can null any UObject pointers which are seen by reference collecting (eg UProperty or serialized)
	if (InTask->StaticMesh && InTask->bSuccess)
	{
		check(!InTask->StaticMesh->IsCompiling());
		
		FStaticMeshRenderData* RenderData = InTask->StaticMesh->GetRenderData();
		FCardRepresentationData* OldCardData = RenderData->LODResources[0].CardRepresentationData;

		// Assign the new data, this is safe because the render threads makes a copy of the pointer at scene proxy creation time.
		RenderData->LODResources[0].CardRepresentationData = InTask->GeneratedCardRepresentation;

		// Any already created render state needs to be dirtied
		if (RenderData->IsInitialized())
		{
			for (IStaticMeshComponent* Component : InObjectCacheScope.GetContext().GetStaticMeshComponents(InTask->StaticMesh))
			{
				IPrimitiveComponent* PrimitiveComponent = Component->GetPrimitiveComponentInterface();

				if (PrimitiveComponent->IsRegistered() && PrimitiveComponent->IsRenderStateCreated())
				{
					PrimitiveComponent->MarkRenderStateDirty();
				}
			}
		}

		// Rendering thread may still be referencing the old one, use the deferred cleanup interface to delete it next frame when it is safe
		BeginCleanup(OldCardData);

		// Need also to update platform render data if it's being cached
		FStaticMeshRenderData* PlatformRenderData = RenderData->NextCachedRenderData.Get();
		while (PlatformRenderData)
		{
			if (PlatformRenderData->LODResources[0].CardRepresentationData)
			{
				*PlatformRenderData->LODResources[0].CardRepresentationData = *InTask->GeneratedCardRepresentation;
			}
			PlatformRenderData = PlatformRenderData->NextCachedRenderData.Get();
		}

		if (!MeshCardRepresentation::IsDebugMode())
		{
			TArray<uint8> DerivedData;
			// Save built data to DDC
			FMemoryWriter Ar(DerivedData, /*bIsPersistent=*/ true);
			Ar << *(InTask->StaticMesh->GetRenderData()->LODResources[0].CardRepresentationData);
			GetDerivedDataCacheRef().Put(*InTask->DDCKey, DerivedData, InTask->StaticMesh->GetPathName());
			COOK_STAT(Timer.AddMiss(DerivedData.Num()));
		}
	}

	delete InTask;
}
#endif

void FCardRepresentationAsyncQueue::Shutdown()
{
	CancelAllOutstandingBuilds();

	UE_LOGF(LogStaticMesh, Log, "Abandoning remaining async card representation tasks for shutdown");
	ThreadPool.Reset();
}

#undef LOCTEXT_NAMESPACE
