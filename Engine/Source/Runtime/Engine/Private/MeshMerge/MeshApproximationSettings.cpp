// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMerge/MeshApproximationSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshApproximationSettings)

bool FMeshApproximationSettings::operator==(const FMeshApproximationSettings& Other) const
{
	return OutputType == Other.OutputType
		&& ApproximationAccuracy == Other.ApproximationAccuracy
		&& ClampVoxelDimension == Other.ClampVoxelDimension
		&& bAttemptAutoThickening == Other.bAttemptAutoThickening
		&& TargetMinThicknessMultiplier == Other.TargetMinThicknessMultiplier
		&& BaseCapping == Other.BaseCapping
		&& WindingThreshold == Other.WindingThreshold
		&& bFillGaps == Other.bFillGaps
		&& GapDistance == Other.GapDistance
		&& OcclusionMethod == Other.OcclusionMethod
		&& SimplifyMethod == Other.SimplifyMethod
		&& TargetTriCount == Other.TargetTriCount
		&& TrianglesPerM == Other.TrianglesPerM
		&& GeometricDeviation == Other.GeometricDeviation
		&& bGenerateNaniteEnabledMesh == Other.bGenerateNaniteEnabledMesh
		&& NaniteFallbackTarget == Other.NaniteFallbackTarget
		&& NaniteFallbackPercentTriangles == Other.NaniteFallbackPercentTriangles
		&& NaniteFallbackRelativeError == Other.NaniteFallbackRelativeError
		&& bSupportRayTracing == Other.bSupportRayTracing
		&& bAllowDistanceField == Other.bAllowDistanceField
		&& MultiSamplingAA == Other.MultiSamplingAA
		&& RenderCaptureResolution == Other.RenderCaptureResolution
		&& MaterialSettings == Other.MaterialSettings
		&& CaptureFieldOfView == Other.CaptureFieldOfView
		&& NearPlaneDist == Other.NearPlaneDist
		&& bPrintDebugMessages == Other.bPrintDebugMessages
		&& bEmitFullDebugMesh == Other.bEmitFullDebugMesh;
}

bool FMeshApproximationSettings::operator!=(const FMeshApproximationSettings& Other) const
{
	return !(*this == Other);
}

#if WITH_EDITORONLY_DATA
void FMeshApproximationSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		FMeshApproximationSettings DefaultObject;
		if (NaniteProxyTrianglePercent_DEPRECATED != DefaultObject.NaniteProxyTrianglePercent_DEPRECATED)
		{
			NaniteFallbackTarget = ENaniteFallbackTarget::Auto;
			NaniteFallbackPercentTriangles = NaniteProxyTrianglePercent_DEPRECATED / 100.0f;
		}
	}
}
#endif
