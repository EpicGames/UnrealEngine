// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMerge/MeshInstancingSettings.h"
#include "GameFramework/Actor.h"
#include "Components/InstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshInstancingSettings)

FMeshInstancingSettings::FMeshInstancingSettings()
	: ActorClassToUse(AActor::StaticClass())
	, InstanceReplacementThreshold(2)
	, bSkipMeshesWithVertexColors(true)
	, bUseHLODVolumes(true)
	, ISMComponentToUse(UInstancedStaticMeshComponent::StaticClass())
{
}
