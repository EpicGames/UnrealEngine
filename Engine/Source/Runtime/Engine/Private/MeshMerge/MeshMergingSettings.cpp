// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMerge/MeshMergingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMergingSettings)


FMeshMergingSettings::FMeshMergingSettings()
	: TargetLightMapResolution(256)
	, GutterSize(2)
	, LODSelectionType(EMeshLODSelectionType::CalculateLOD)
	, SpecificLOD(0)
	, bGenerateLightMapUV(true)
	, bComputedLightMapResolution(false)
	, bMergePhysicsData(false)
	, bMergeMeshSockets(false)
	, bMergeMaterials(false)
	, bBakeVertexDataToMesh(false)
	, bUseVertexDataForBakingMaterial(true)
	, bUseTextureBinning(false)
	, bReuseMeshLightmapUVs(true)
	, bMergeEquivalentMaterials(true)
	, bUseLandscapeCulling(false)
	, bIncludeImposters(true)
	, bSupportRayTracing(true)
	, bAllowDistanceField(false)
	, PivotType(EMeshMergePivotType::Automatic)
	, CustomPivotLocation(FVector::ZeroVector)
#if WITH_EDITORONLY_DATA
	, bImportVertexColors_DEPRECATED(false)
	, bCalculateCorrectLODModel_DEPRECATED(false)
	, bExportNormalMap_DEPRECATED(true)
	, bExportMetallicMap_DEPRECATED(false)
	, bExportRoughnessMap_DEPRECATED(false)
	, bExportSpecularMap_DEPRECATED(false)
	, bCreateMergedMaterial_DEPRECATED(false)
	, MergedMaterialAtlasResolution_DEPRECATED(1024)
	, ExportSpecificLOD_DEPRECATED(0)
	, bGenerateNaniteEnabledMesh_DEPRECATED(false)
	, NaniteFallbackTrianglePercent_DEPRECATED(100)
	, bPivotPointAtZero_DEPRECATED(false)
#endif
	, MergeType(EMeshMergeType::MeshMergeType_Default)
{
	for(EUVOutput& OutputUV : OutputUVs)
	{
		OutputUV = EUVOutput::OutputChannel;
	}
}

#if WITH_EDITORONLY_DATA
void FMeshMergingSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		FMeshMergingSettings DefaultObject;
		if (bImportVertexColors_DEPRECATED != DefaultObject.bImportVertexColors_DEPRECATED)
		{
			bBakeVertexDataToMesh = bImportVertexColors_DEPRECATED;
		}

		if (bExportNormalMap_DEPRECATED != DefaultObject.bExportNormalMap_DEPRECATED)
		{
			MaterialSettings.bNormalMap = bExportNormalMap_DEPRECATED;
		}

		if (bExportMetallicMap_DEPRECATED != DefaultObject.bExportMetallicMap_DEPRECATED)
		{
			MaterialSettings.bMetallicMap = bExportMetallicMap_DEPRECATED;
		}
		if (bExportRoughnessMap_DEPRECATED != DefaultObject.bExportRoughnessMap_DEPRECATED)
		{
			MaterialSettings.bRoughnessMap = bExportRoughnessMap_DEPRECATED;
		}
		if (bExportSpecularMap_DEPRECATED != DefaultObject.bExportSpecularMap_DEPRECATED)
		{
			MaterialSettings.bSpecularMap = bExportSpecularMap_DEPRECATED;
		}
		if (MergedMaterialAtlasResolution_DEPRECATED != DefaultObject.MergedMaterialAtlasResolution_DEPRECATED)
		{
			MaterialSettings.TextureSize.X = MergedMaterialAtlasResolution_DEPRECATED;
			MaterialSettings.TextureSize.Y = MergedMaterialAtlasResolution_DEPRECATED;
		}
		if (bCalculateCorrectLODModel_DEPRECATED != DefaultObject.bCalculateCorrectLODModel_DEPRECATED)
		{
			LODSelectionType = EMeshLODSelectionType::CalculateLOD;
		}

		if (ExportSpecificLOD_DEPRECATED != DefaultObject.ExportSpecificLOD_DEPRECATED)
		{
			SpecificLOD = ExportSpecificLOD_DEPRECATED;
			LODSelectionType = EMeshLODSelectionType::SpecificLOD;
		}

		if (bGenerateNaniteEnabledMesh_DEPRECATED)
		{
			NaniteSettings.bEnabled = true;
			NaniteSettings.FallbackPercentTriangles = NaniteFallbackTrianglePercent_DEPRECATED / 100.0f;
		}

		if (bPivotPointAtZero_DEPRECATED)
		{
			PivotType = EMeshMergePivotType::WorldOrigin;
		}
	}
}
#endif
