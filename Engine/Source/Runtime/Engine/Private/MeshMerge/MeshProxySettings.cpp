// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMerge/MeshProxySettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshProxySettings)

FMeshProxySettings::FMeshProxySettings()
	: ScreenSize(300)
	, VoxelSize(3.f)
#if WITH_EDITORONLY_DATA
	, TextureWidth_DEPRECATED(512)
	, TextureHeight_DEPRECATED(512)
	, bExportNormalMap_DEPRECATED(true)
	, bExportMetallicMap_DEPRECATED(false)
	, bExportRoughnessMap_DEPRECATED(false)
	, bExportSpecularMap_DEPRECATED(false)
	, bBakeVertexData_DEPRECATED(false)
	, bGenerateNaniteEnabledMesh_DEPRECATED(false)
	, NaniteProxyTrianglePercent_DEPRECATED(100)
#endif
	, MergeDistance(0)
	, UnresolvedGeometryColor(FColor::Black)
	, MaxRayCastDist(20)
	, HardAngleThreshold(130.f)
	, LightMapResolution(256)
	, NormalCalculationMethod(EProxyNormalComputationMethod::AngleWeighted)
	, LandscapeCullingPrecision(ELandscapeCullingPrecision::Medium)
	, bCalculateCorrectLODModel(false)
	, bOverrideVoxelSize(false)
	, bOverrideTransferDistance(false)
	, bUseHardAngleThreshold(false)
	, bComputeLightMapResolution(false)
	, bRecalculateNormals(true)
	, bUseLandscapeCulling(false)
	, bSupportRayTracing(true)
	, bAllowDistanceField(false)
	, bReuseMeshLightmapUVs(true)
	, bGroupIdenticalMeshesForBaking(false)
	, bCreateCollision(true)
	, bAllowVertexColors(false)
	, bGenerateLightmapUVs(false)
{
	MaterialSettings.MaterialMergeType = EMaterialMergeType::MaterialMergeType_Simplygon;
}

/** Equality operator. */
bool FMeshProxySettings::operator==(const FMeshProxySettings& Other) const
{
	return ScreenSize == Other.ScreenSize
		&& VoxelSize == Other.VoxelSize
		&& MaterialSettings == Other.MaterialSettings
		&& MergeDistance == Other.MergeDistance
		&& UnresolvedGeometryColor == Other.UnresolvedGeometryColor
		&& MaxRayCastDist == Other.MaxRayCastDist
		&& HardAngleThreshold == Other.HardAngleThreshold
		&& LightMapResolution == Other.LightMapResolution
		&& NormalCalculationMethod == Other.NormalCalculationMethod
		&& LandscapeCullingPrecision == Other.LandscapeCullingPrecision
		&& bCalculateCorrectLODModel == Other.bCalculateCorrectLODModel
		&& bOverrideVoxelSize == Other.bOverrideVoxelSize
		&& bOverrideTransferDistance == Other.bOverrideTransferDistance
		&& bUseHardAngleThreshold == Other.bUseHardAngleThreshold
		&& bComputeLightMapResolution == Other.bComputeLightMapResolution
		&& bRecalculateNormals == Other.bRecalculateNormals
		&& bUseLandscapeCulling == Other.bUseLandscapeCulling
		&& bSupportRayTracing == Other.bSupportRayTracing
		&& bAllowDistanceField == Other.bAllowDistanceField
		&& bReuseMeshLightmapUVs == Other.bReuseMeshLightmapUVs
		&& bGroupIdenticalMeshesForBaking == Other.bGroupIdenticalMeshesForBaking
		&& bCreateCollision == Other.bCreateCollision
		&& bAllowVertexColors == Other.bAllowVertexColors
		&& bGenerateLightmapUVs == Other.bGenerateLightmapUVs
		&& NaniteSettings == Other.NaniteSettings;
}

/** Inequality. */
bool FMeshProxySettings::operator!=(const FMeshProxySettings& Other) const
{
	return !(*this == Other);
}

#if WITH_EDITORONLY_DATA
void FMeshProxySettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		MaterialSettings.MaterialMergeType = EMaterialMergeType::MaterialMergeType_Simplygon;

		if (bGenerateNaniteEnabledMesh_DEPRECATED)
		{
			NaniteSettings.bEnabled = true;
			NaniteSettings.FallbackPercentTriangles = NaniteProxyTrianglePercent_DEPRECATED / 100.0f;
		}
	}
}
#endif
