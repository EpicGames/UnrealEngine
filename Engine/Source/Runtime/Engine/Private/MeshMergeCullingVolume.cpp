// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/MeshMergeCullingVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMergeCullingVolume)

AMeshMergeCullingVolume::AMeshMergeCullingVolume(const FObjectInitializer& ObjectInitializer)
:Super(ObjectInitializer)
{
	bNotForClientOrServer = true;

	bColored = true;
	BrushColor = FColor(45, 225, 45);
}


