// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPaintVisualize.h"

#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneManagement.h"
#include "TextureResource.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EVertexColorViewMode::Type GVertexColorViewMode = EVertexColorViewMode::Color;
TWeakObjectPtr<UTexture> GVertexViewModeOverrideTexture;
float GVertexViewModeOverrideUVChannel = 0.0f;
FString GVertexViewModeOverrideOwnerName;
bool ShouldProxyUseVertexColorVisualization(FName OwnerName) { return false; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

namespace MeshPaintVisualize
{ 
	static EMeshPaintVisualizePaintMode::Type GPaintMode = EMeshPaintVisualizePaintMode::VertexColor;
	static EMeshPaintVisualizeShowMode::Type GShowMode = EMeshPaintVisualizeShowMode::ShowAll;
	static EVertexColorViewMode::Type GChannelMode = EVertexColorViewMode::Color;
	
	static TWeakObjectPtr<UTexture> GTextureAsset = nullptr;
	static FRHITexture* GTextureRHI_GameThread = nullptr;
	static FRHITexture* GTextureRHI_RenderThread = nullptr;
	static int32 GTextureCoordinateIndex = 0;

	void SetPaintMode(EMeshPaintVisualizePaintMode::Type PaintMode)
	{
		GPaintMode = PaintMode;
	}

	void SetShowMode(EMeshPaintVisualizeShowMode::Type ShowMode)
	{
		GShowMode = ShowMode;
	}

	EMeshPaintVisualizeShowMode::Type GetShowMode()
	{
		return GShowMode;
	}

	void SetChannelMode(EVertexColorViewMode::Type ChannelMode)
	{
		GChannelMode = ChannelMode;
	}

	EVertexColorViewMode::Type GetChannelMode()
	{
		return GChannelMode;
	}

	void SetTextureAsset(TWeakObjectPtr<UTexture> Texture)
	{
		GTextureAsset = Texture;
		
		UTexture* TexturePtr = Texture.Get();
		FTextureResource* TextureResource = TexturePtr != nullptr ? TexturePtr->GetResource() : nullptr;
		FRHITexture* TextureRHI = TextureResource != nullptr ? TextureResource->GetTexture2DRHI() : nullptr;
		if (TextureRHI != GTextureRHI_GameThread)
		{
			GTextureRHI_GameThread = TextureRHI;
			ENQUEUE_RENDER_COMMAND(SetMeshPaintVisualizeTexture)([TextureRHI](FRHICommandListImmediate& RHICmdList)
			{
				GTextureRHI_RenderThread = TextureRHI;
			});
		}
	}

	FRHITexture* GetTextureAsset_RenderThread()
	{
		return GTextureRHI_RenderThread;
	}

	void SetTextureCoordinateIndex(int32 Index)
	{
		GTextureCoordinateIndex = Index;
	}

	int32 GetTextureCoordinateIndex()
	{
		return GTextureCoordinateIndex;
	}

	FMaterialRenderProxy* GetMaterialRenderProxy(bool bIsSelected, bool bIsHovered)
	{
		UMaterial* VertexColorVisualizationMaterial = nullptr;
		FLinearColor MaterialColor = FLinearColor::White;

		switch (GChannelMode)
		{
		case EVertexColorViewMode::Color:
			VertexColorVisualizationMaterial = GEngine->VertexColorViewModeMaterial_ColorOnly;
			MaterialColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.0f);
			break;
		case EVertexColorViewMode::Alpha:
			VertexColorVisualizationMaterial = GEngine->VertexColorViewModeMaterial_AlphaAsColor;
			MaterialColor = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
			break;
		case EVertexColorViewMode::Red:
			VertexColorVisualizationMaterial = GEngine->VertexColorViewModeMaterial_RedOnly;
			MaterialColor = FLinearColor(1.0f, 0.0f, 0.0f, 0.0f);
			break;
		case EVertexColorViewMode::Green:
			VertexColorVisualizationMaterial = GEngine->VertexColorViewModeMaterial_GreenOnly;
			MaterialColor = FLinearColor(0.0f, 1.0f, 0.0f, 0.0f);
			break;
		case EVertexColorViewMode::Blue:
			VertexColorVisualizationMaterial = GEngine->VertexColorViewModeMaterial_BlueOnly;
			MaterialColor = FLinearColor(0.0f, 0.0f, 1.0f, 0.0f);
			break;
		}

		if (GPaintMode == EMeshPaintVisualizePaintMode::VertexColor && VertexColorVisualizationMaterial != nullptr)
		{
			return new FColoredMaterialRenderProxy(
				VertexColorVisualizationMaterial->GetRenderProxy(),
				GetSelectionColor(FLinearColor::White, bIsSelected, bIsHovered));
		}

		if (GPaintMode == EMeshPaintVisualizePaintMode::TextureColor)
		{
 			return new FColoredMaterialRenderProxy(
 				GEngine->TextureColorViewModeMaterial->GetRenderProxy(),
 				MaterialColor);
		}

#if WITH_EDITORONLY_DATA
		if (GPaintMode == EMeshPaintVisualizePaintMode::TextureAsset && GTextureAsset.IsValid() && GEngine->TexturePaintingMaskMaterial)
		{
			FColoredTexturedMaterialRenderProxy* TextureColorVisualizationMaterialInstance = new FColoredTexturedMaterialRenderProxy(
				GEngine->TexturePaintingMaskMaterial->GetRenderProxy(),
				MaterialColor,
				NAME_Color,
				GTextureAsset.Get(),
				NAME_LinearColor);

			TextureColorVisualizationMaterialInstance->UVChannel = (float)GTextureCoordinateIndex;
			TextureColorVisualizationMaterialInstance->UVChannelParamName = FName("UVChannel");

			return TextureColorVisualizationMaterialInstance;
		}
#endif

		return nullptr;
	}
}
