// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshParticleVertexFactory.cpp: Mesh particle vertex factory implementation
=============================================================================*/

#include "MeshParticleVertexFactory.h"
#include "MeshDrawShaderBindings.h"
#include "ParticleHelper.h"
#include "MeshMaterialShader.h"
#include "GlobalRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MeshUVChannelInfo.h"
#include "Misc/DelayedAutoRegister.h"
#include "LocalVertexFactory.h"

class FMeshParticleVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FMeshParticleVertexFactoryShaderParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		PrevTransformBuffer.Bind(ParameterMap, TEXT("PrevTransformBuffer"));
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		const bool bInstanced = true;
		FMeshParticleVertexFactory* MeshParticleVF = (FMeshParticleVertexFactory*)VertexFactory;
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FMeshParticleUniformParameters>(), MeshParticleVF->GetUniformBuffer() );

		if (FeatureLevel >= ERHIFeatureLevel::SM5)
		{
			ShaderBindings.Add(PrevTransformBuffer, MeshParticleVF->GetPreviousTransformBufferSRV());
		}
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, PrevTransformBuffer);
};

IMPLEMENT_TYPE_LAYOUT(FMeshParticleVertexFactoryShaderParameters);

class FDummyPrevTransformBuffer : public FRenderResource
{
public:
	virtual ~FDummyPrevTransformBuffer() {}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("FDummyPrevTransformBuffer"), 3)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState();

		VB = RHICmdList.CreateBuffer(CreateDesc);
		SRV = RHICmdList.CreateShaderResourceView(
			VB,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
	}

	virtual void ReleaseRHI()
	{
		VB.SafeRelease();
		SRV.SafeRelease();
	}

	virtual FString GetFriendlyName() const
	{
		return TEXT("FDummyPrevTransformBuffer");
	}

	inline FRHIBuffer* GetVB() const
	{
		return VB;
	}

	inline FRHIShaderResourceView* GetSRV() const
	{
		return SRV;
	}

private:
	FBufferRHIRef VB;
	FShaderResourceViewRHIRef SRV;
};

static TGlobalResource<FDummyPrevTransformBuffer> GDummyPrevTransformBuffer;

void FMeshParticleVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride, FDataType& Data, FVertexDeclarationElementList& Elements, FVertexStreamList& InOutStreams)
{
	if (Data.bInitialized)
	{
		// Stream 0 - Instance data
		{
			checkf(InDynamicVertexStride != -1, TEXT("FMeshParticleVertexFactory does not have a valid DynamicVertexStride - likely an empty one was made, but SetStrides was not called"));
			FVertexStream VertexStream;
			VertexStream.VertexBuffer = NULL;
			VertexStream.Stride = 0;
			VertexStream.Offset = 0;
			InOutStreams.Add(VertexStream);

			// @todo metal: this will need a valid stride when we get to instanced meshes!
			Elements.Add(FVertexElement(0, Data.TransformComponent[0].Offset, Data.TransformComponent[0].Type, 8, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.TransformComponent[0].VertexStreamUsage)));
			Elements.Add(FVertexElement(0, Data.TransformComponent[1].Offset, Data.TransformComponent[1].Type, 9, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.TransformComponent[1].VertexStreamUsage)));
			Elements.Add(FVertexElement(0, Data.TransformComponent[2].Offset, Data.TransformComponent[2].Type, 10, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.TransformComponent[2].VertexStreamUsage)));

			Elements.Add(FVertexElement(0, Data.SubUVs.Offset, Data.SubUVs.Type, 11, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.SubUVs.VertexStreamUsage)));
			Elements.Add(FVertexElement(0, Data.SubUVLerpAndRelTime.Offset, Data.SubUVLerpAndRelTime.Type, 12, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.SubUVLerpAndRelTime.VertexStreamUsage)));

			Elements.Add(FVertexElement(0, Data.ParticleColorComponent.Offset, Data.ParticleColorComponent.Type, 14, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.ParticleColorComponent.VertexStreamUsage)));
			Elements.Add(FVertexElement(0, Data.VelocityComponent.Offset, Data.VelocityComponent.Type, 15, InDynamicVertexStride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Data.VelocityComponent.VertexStreamUsage)));
		}

		// Stream 1 - Dynamic parameter
		{
			checkf(InDynamicParameterVertexStride != -1, TEXT("FMeshParticleVertexFactory does not have a valid DynamicParameterVertexStride - likely an empty one was made, but SetStrides was not called"));

			FVertexStream VertexStream;
			VertexStream.VertexBuffer = NULL;
			VertexStream.Stride = 0;
			VertexStream.Offset = 0;
			InOutStreams.Add(VertexStream);

			Elements.Add(FVertexElement(1, 0, VET_Float4, 13, InDynamicParameterVertexStride, true));
		}
	}

	if(Data.PositionComponent.VertexBuffer != NULL)
	{
		Elements.Add(AccessStreamComponent(Data.PositionComponent, 0, InOutStreams));
	}

	// only tangent,normal are used by the stream. the binormal is derived in the shader
	uint8 TangentBasisAttributes[2] = { 1, 2 };
	for(int32 AxisIndex = 0;AxisIndex < 2;AxisIndex++)
	{
		if(Data.TangentBasisComponents[AxisIndex].VertexBuffer != NULL)
		{
			Elements.Add(AccessStreamComponent(Data.TangentBasisComponents[AxisIndex],TangentBasisAttributes[AxisIndex], InOutStreams));
		}
	}

	if (Data.ColorComponentsSRV == nullptr)
	{
		Data.ColorComponentsSRV = GNullColorVertexBuffer.VertexBufferSRV;
		Data.ColorIndexMask = 0;
	}

	// Vertex color
	if(Data.ColorComponent.VertexBuffer != NULL)
	{
		Elements.Add(AccessStreamComponent(Data.ColorComponent,3, InOutStreams));
	}
	else
	{
		//If the mesh has no color component, set the null color buffer on a new stream with a stride of 0.
		//This wastes 4 bytes of bandwidth per vertex, but prevents having to compile out twice the number of vertex factories.
		FVertexStreamComponent NullColorComponent(&GNullColorVertexBuffer, 0, 0, VET_Color, EVertexStreamUsage::ManualFetch);
		Elements.Add(AccessStreamComponent(NullColorComponent, 3, InOutStreams));
	}

	if(Data.TextureCoordinates.Num())
	{
		const int32 BaseTexCoordAttribute = 4;		
		AddStaticMeshTextureCoordinateElements(BaseTexCoordAttribute, Data.TextureCoordinates, Elements, InOutStreams);
	}
}

void FMeshParticleVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	GetVertexElements(GMaxRHIFeatureLevel, DynamicVertexStride, DynamicParameterVertexStride, Data, Elements, Streams);

	// Add a dummy resource to avoid crash due to missing resource
	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5)
	{
		PrevTransformBuffer.NumBytes = 0;
		PrevTransformBuffer.Buffer = GDummyPrevTransformBuffer.GetVB();
		PrevTransformBuffer.SRV = GDummyPrevTransformBuffer.GetSRV();
	}

	if(Streams.Num() > 0)
	{
		InitDeclaration(Elements);
		check(IsValidRef(GetDeclaration()));
	}
}

void FMeshParticleVertexFactory::GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride, FDataType& Data, FVertexDeclarationElementList& Elements)
{
	FVertexStreamList InOutStreams;
	GetVertexElements(FeatureLevel, InDynamicVertexStride, InDynamicParameterVertexStride, Data, Elements, InOutStreams);
}

void FMeshParticleVertexFactory::SetInstanceBuffer(const FVertexBuffer* InstanceBuffer, uint32 StreamOffset, uint32 Stride)
{
	ensure(Stride == DynamicVertexStride);
	Streams[0].VertexBuffer = InstanceBuffer;
	Streams[0].Offset = StreamOffset;
	Streams[0].Stride = Stride;
}

void FMeshParticleVertexFactory::SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, uint32 StreamOffset, uint32 Stride)
{
	if (InDynamicParameterBuffer)
	{
		Streams[1].VertexBuffer = InDynamicParameterBuffer;
		ensure(Stride == DynamicParameterVertexStride);
		Streams[1].Stride = DynamicParameterVertexStride;
		Streams[1].Offset = StreamOffset;
	}
	else
	{
		Streams[1].VertexBuffer = &GNullDynamicParameterVertexBuffer;
		ensure(DynamicParameterVertexStride == 0);
		Streams[1].Stride = 0;
		Streams[1].Offset = 0;
	}
}

uint8* FMeshParticleVertexFactory::LockPreviousTransformBuffer(FRHICommandListBase& RHICmdList, uint32 ParticleCount)
{
	const static uint32 ElementSize = sizeof(FVector4f);
	const static uint32 ParticleSize = ElementSize * 3;
	const uint32 AllocationRequest = ParticleCount * ParticleSize;

	check(!PrevTransformBuffer.MappedBuffer);

	if (AllocationRequest > PrevTransformBuffer.NumBytes)
	{
		PrevTransformBuffer.Release();
		PrevTransformBuffer.Initialize(RHICmdList, TEXT("PrevTransformBuffer"), ElementSize, ParticleCount * 3, PF_A32B32G32R32F, BUF_Dynamic);
	}

	PrevTransformBuffer.Lock(RHICmdList);

	return PrevTransformBuffer.MappedBuffer;
}

void FMeshParticleVertexFactory::UnlockPreviousTransformBuffer(FRHICommandListBase& RHICmdList)
{
	check(PrevTransformBuffer.MappedBuffer);

	PrevTransformBuffer.Unlock(RHICmdList);
}

FRHIShaderResourceView* FMeshParticleVertexFactory::GetPreviousTransformBufferSRV() const
{
	return PrevTransformBuffer.SRV;
}

void FMeshParticleVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FParticleVertexFactoryBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	// Set a define so we can tell in MaterialTemplate.usf when we are compiling a mesh particle vertex factory
	OutEnvironment.SetDefine(TEXT("PARTICLE_MESH_FACTORY"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("PARTICLE_MESH_INSTANCED"), TEXT("1"));

	if (RHISupportsManualVertexFetch(Parameters.Platform))
	{
		OutEnvironment.SetDefineIfUnset(TEXT("MANUAL_VERTEX_FETCH"), TEXT("1"));
	}
}

bool FMeshParticleVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return (Parameters.MaterialParameters.bIsUsedWithMeshParticles || Parameters.MaterialParameters.bIsSpecialEngineMaterial);
}

/**
 * FMeshParticleVertexFactory does not support manual vertex fetch yet so worst case element set is returned to make sure the PSO can be compiled
 */
void FMeshParticleVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	check(VertexInputStreamType == EVertexInputStreamType::Default);

	// Per vertex data
	{
		// Position
		Elements.Add(FVertexElement(0, 0, VET_Float3, 0, 0, false));

		// Normals
		Elements.Add(FVertexElement(1, 0, VET_PackedNormal, 1, 0, false));
		Elements.Add(FVertexElement(2, 4, VET_PackedNormal, 2, 0, false));

		// Color
		Elements.Add(FVertexElement(3, 0, VET_Color, 3, 0, false));

		// Texcoords
		Elements.Add(FVertexElement(4, 0, VET_Half4, 4, 0, false));
		Elements.Add(FVertexElement(5, 4, VET_Half4, 5, 0, false));
		Elements.Add(FVertexElement(6, 8, VET_Half4, 6, 0, false));
		Elements.Add(FVertexElement(7, 12, VET_Half4, 7, 0, false));
	}

	// Per instance data
	{
		// Instance transforms
		Elements.Add(FVertexElement(8, 0, VET_Float4, 8, 0, true));
		Elements.Add(FVertexElement(9, 0, VET_Float4, 9, 0, true));
		Elements.Add(FVertexElement(10, 0, VET_Float4, 10, 0, true));

		// SubUVs
		Elements.Add(FVertexElement(11, 0, VET_Short4, 11, 0, true));

		// SubUVLerpAndRelTime
		Elements.Add(FVertexElement(12, 0, VET_Float2, 12, 0, true));

		// Dynamic parameter
		Elements.Add(FVertexElement(13, 0, VET_Float4, 13, 0, true));

		// Particle Color
		Elements.Add(FVertexElement(14, 0, VET_Float4, 14, 0, true));

		// Particle Velocity
		Elements.Add(FVertexElement(15, 0, VET_Float4, 15, 0, true));
	}
}

void FMeshParticleVertexFactory::SetData(FRHICommandListBase& RHICmdList, const FDataType& InData)
{
	Data = InData;
	UpdateRHI(RHICmdList);
}

void FMeshParticleVertexFactory::SetData(const FDataType& InData)
{
	SetData(FRHICommandListImmediate::Get(), InData);
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FMeshParticleVertexFactory, SF_Vertex, FMeshParticleVertexFactoryShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FMeshParticleVertexFactory,"/Engine/Private/MeshParticleVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPSOPrecaching
);
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FMeshParticleUniformParameters,"MeshParticleVF");
