// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshReductionSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshReductionSettings)

FMeshReductionSettings::FMeshReductionSettings()
	: PercentTriangles(1.0f)
	, MaxNumOfTriangles(MAX_uint32)
	, PercentVertices(1.0f)
	, MaxNumOfVerts(MAX_uint32)
	, MaxDeviation(0.0f)
	, PixelError(8.0f)
	, WeldingThreshold(0.0f)
	, HardAngleThreshold(80.0f)
	, BaseLODModel(0)
	, SilhouetteImportance(EMeshFeatureImportance::Normal)
	, TextureImportance(EMeshFeatureImportance::Normal)
	, ShadingImportance(EMeshFeatureImportance::Normal)
	, bRecalculateNormals(false)
	, bGenerateUniqueLightmapUVs(false)
	, bKeepSymmetry(false)
	, bVisibilityAided(false)
	, bCullOccluded(false)
	, TerminationCriterion(EStaticMeshReductionTerimationCriterion::Triangles)
	, VisibilityAggressiveness(EMeshFeatureImportance::Lowest)
	, VertexColorImportance(EMeshFeatureImportance::Off)
{
}

bool FMeshReductionSettings::operator==(const FMeshReductionSettings& Other) const
{
	return
		TerminationCriterion == Other.TerminationCriterion
		&& PercentVertices == Other.PercentVertices
		&& PercentTriangles == Other.PercentTriangles
		&& MaxNumOfTriangles == Other.MaxNumOfTriangles
		&& MaxNumOfVerts == Other.MaxNumOfVerts
		&& MaxDeviation == Other.MaxDeviation
		&& PixelError == Other.PixelError
		&& WeldingThreshold == Other.WeldingThreshold
		&& HardAngleThreshold == Other.HardAngleThreshold
		&& SilhouetteImportance == Other.SilhouetteImportance
		&& TextureImportance == Other.TextureImportance
		&& ShadingImportance == Other.ShadingImportance
		&& bRecalculateNormals == Other.bRecalculateNormals
		&& BaseLODModel == Other.BaseLODModel
		&& bGenerateUniqueLightmapUVs == Other.bGenerateUniqueLightmapUVs
		&& bKeepSymmetry == Other.bKeepSymmetry
		&& bVisibilityAided == Other.bVisibilityAided
		&& bCullOccluded == Other.bCullOccluded
		&& VisibilityAggressiveness == Other.VisibilityAggressiveness
		&& VertexColorImportance == Other.VertexColorImportance;
}

bool FMeshReductionSettings::operator!=(const FMeshReductionSettings& Other) const
{
	return !(*this == Other);
}
