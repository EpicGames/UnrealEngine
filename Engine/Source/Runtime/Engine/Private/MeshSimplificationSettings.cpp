// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/MeshSimplificationSettings.h"
#include "Modules/ModuleManager.h"
#include "HAL/IConsoleManager.h"
#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSimplificationSettings)

#if WITH_EDITOR
#include "IMeshReductionManagerModule.h"
#endif

static FAutoConsoleVariable CVarMeshReductionModule(
	TEXT("r.MeshReductionModule"),
	TEXT("QuadricMeshReduction"),
	TEXT("Name of what mesh reduction module to choose. If blank it chooses any that exist.\n"),
	ECVF_ReadOnly);

UMeshSimplificationSettings::UMeshSimplificationSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FName UMeshSimplificationSettings::GetContainerName() const
{
	static const FName ContainerName("Project");
	return ContainerName;
}

FName UMeshSimplificationSettings::GetCategoryName() const
{
	static const FName EditorCategoryName("Editor");
	return EditorCategoryName;
}

void UMeshSimplificationSettings::SetMeshReductionModuleName(FName InMeshReductionModuleName)
{
	MeshReductionModuleName = InMeshReductionModuleName;
	CVarMeshReductionModule->Set(*MeshReductionModuleName.ToString());
}

void UMeshSimplificationSettings::PostInitProperties()
{
	Super::PostInitProperties(); 

#if WITH_EDITOR
	// As of this writing (Oct 28, 2025), MeshReductionInterface and MeshUtilities have a
	// circular dependency. Each tries to load the other, but it turns out that
	// MeshUtilities loading MeshReductionInterface works out properly, but MeshReductionInterface
	// loading MeshUtilities does not quite work, since FMeshUtilities::StartupModule()
	// sets its VersionString incorrectly when invoked from inside MeshReductionInterface
	// startup.
	//
	// Explicitly load MeshUtilities first for now to sidestep this mess.
	FModuleManager::Get().LoadModule("MeshUtilities");

	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	if(IsTemplate())
	{
		FModuleManager::Get().LoadModule("MeshReductionInterface");
		ImportConsoleVariableValues();
	}
#endif
}

#if WITH_EDITOR


void UMeshSimplificationSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}


#endif

