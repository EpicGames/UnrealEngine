// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexPainter/MeshVertexPainter.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexPainter)


void FMeshVertexPainter::PaintVerticesSingleColor(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& FillColor, bool bConvertToSRGB)
{
	if (!StaticMeshComponent || !StaticMeshComponent->GetStaticMesh())
	{
		return;
	}

	const int32 NumMeshLODs = StaticMeshComponent->GetStaticMesh()->GetNumLODs();
	StaticMeshComponent->SetLODDataCount(NumMeshLODs, NumMeshLODs);

	const FColor Color = FillColor.ToFColor(bConvertToSRGB);

	uint32 LODIndex = 0;
	for (FStaticMeshComponentLODInfo& LODInfo : StaticMeshComponent->LODData)
	{
		StaticMeshComponent->RemoveInstanceVertexColorsFromLOD(LODIndex);
		check(LODInfo.OverrideVertexColors == nullptr);

		const FStaticMeshLODResources& LODModel = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[LODIndex];
		const FPositionVertexBuffer& PositionVertexBuffer = LODModel.VertexBuffers.PositionVertexBuffer;
		const uint32 NumVertices = PositionVertexBuffer.GetNumVertices();

		LODInfo.OverrideVertexColors = new FColorVertexBuffer;
		LODInfo.OverrideVertexColors->InitFromSingleColor(Color, NumVertices);

		BeginInitResource(LODInfo.OverrideVertexColors);

		LODIndex++;
	}

#if WITH_EDITORONLY_DATA
	StaticMeshComponent->CachePaintedDataIfNecessary();
#endif
	StaticMeshComponent->MarkRenderStateDirty();

	// Explicitly disable the mesh paint tool on the component to prevent stomping the vertex color.
	StaticMeshComponent->bEnableVertexColorMeshPainting = false;
}

void FMeshVertexPainter::PaintVerticesLerpAlongAxis(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& StartColor, const FLinearColor& EndColor, EVertexPaintAxis Axis, bool bConvertToSRGB)
{
	TObjectPtr<UStaticMesh> StaticMesh;
	if (!StaticMeshComponent || !(StaticMesh = StaticMeshComponent->GetStaticMesh()))
	{
		return;
	}
	if (!StaticMesh->bAllowCPUAccess)
	{
#if WITH_EDITOR
		UE_LOGF(LogStaticMesh,
			Warning,
			"bAllowCPUAccess should be set to true for StaticMesh '%ls' before calling PaintVerticesLerpAlongAxis(), otherwise the mesh geometry data isn't accessible when packed.",
			*StaticMesh->GetName());
#else
		UE_LOGF(LogStaticMesh,
			Error,
			"bAllowCPUAccess must be set to true for StaticMesh '%ls' before calling PaintVerticesLerpAlongAxis(), otherwise the mesh geometry data isn't accessible when packed.",
			*StaticMesh->GetName());
		return;
#endif
	}

	const FBoxSphereBounds Bounds = StaticMesh->GetBounds();
	const FBox Box = Bounds.GetBox();
	static_assert(static_cast<int32>(EVertexPaintAxis::X) == 0, "EVertexPaintAxis not correctly defined");
	const float AxisMin = Box.Min.Component(static_cast<int32>(Axis));
	const float AxisMax = Box.Max.Component(static_cast<int32>(Axis));

	const int32 NumMeshLODs = StaticMesh->GetNumLODs();
	StaticMeshComponent->SetLODDataCount(NumMeshLODs, NumMeshLODs);

	uint32 LODIndex = 0;
	for (FStaticMeshComponentLODInfo& LODInfo : StaticMeshComponent->LODData)
	{
		StaticMeshComponent->RemoveInstanceVertexColorsFromLOD(LODIndex);
		check(LODInfo.OverrideVertexColors == nullptr);

		const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[LODIndex];
		const FPositionVertexBuffer& PositionVertexBuffer = LODModel.VertexBuffers.PositionVertexBuffer;
		const uint32 NumVertices = PositionVertexBuffer.GetNumVertices();

		TArray<FColor> VertexColors;
		VertexColors.AddZeroed(NumVertices);

		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const FVector3f& VertexPosition = PositionVertexBuffer.VertexPosition(VertexIndex);
			const FLinearColor Color = FMath::Lerp(StartColor, EndColor, (VertexPosition.Component(static_cast<int32>(Axis)) - AxisMin) / (AxisMax - AxisMin));
			VertexColors[VertexIndex] = Color.ToFColor(bConvertToSRGB);
		}

		LODInfo.OverrideVertexColors = new FColorVertexBuffer;
		LODInfo.OverrideVertexColors->InitFromColorArray(VertexColors);

		BeginInitResource(LODInfo.OverrideVertexColors);

		LODIndex++;
	}

#if WITH_EDITORONLY_DATA
	StaticMeshComponent->CachePaintedDataIfNecessary();
#endif
	StaticMeshComponent->MarkRenderStateDirty();

	// Explicitly disable the mesh paint tool on the component to prevent stomping the vertex color.
	StaticMeshComponent->bEnableVertexColorMeshPainting = false;
}

void FMeshVertexPainter::RemovePaintedVertices(UStaticMeshComponent* StaticMeshComponent)
{
	if (!StaticMeshComponent || !StaticMeshComponent->GetStaticMesh())
	{
		return;
	}

	uint32 LODIndex = 0;
	for (FStaticMeshComponentLODInfo& LODInfo : StaticMeshComponent->LODData)
	{
		StaticMeshComponent->RemoveInstanceVertexColorsFromLOD(LODIndex);
		check(LODInfo.OverrideVertexColors == nullptr);
		LODIndex++;
	}

	StaticMeshComponent->MarkRenderStateDirty();
	StaticMeshComponent->bEnableVertexColorMeshPainting = true;
}

