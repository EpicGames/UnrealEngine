// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexPainter/MeshVertexPainterKismetLibrary.h"
#include "MeshVertexPainter/MeshVertexPainter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexPainterKismetLibrary)


void UMeshVertexPainterKismetLibrary::PaintVerticesSingleColor(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& FillColor, bool bConvertToSRGB)
{
	FMeshVertexPainter::PaintVerticesSingleColor(StaticMeshComponent, FillColor, bConvertToSRGB);
}

void UMeshVertexPainterKismetLibrary::PaintVerticesLerpAlongAxis(UStaticMeshComponent* StaticMeshComponent, const FLinearColor& StartColor, const FLinearColor& EndColor, EVertexPaintAxis Axis, bool bConvertToSRGB)
{
	FMeshVertexPainter::PaintVerticesLerpAlongAxis(StaticMeshComponent, StartColor, EndColor, Axis, bConvertToSRGB);
}

void UMeshVertexPainterKismetLibrary::RemovePaintedVertices(UStaticMeshComponent* StaticMeshComponent)
{
	FMeshVertexPainter::RemovePaintedVertices(StaticMeshComponent);
}

