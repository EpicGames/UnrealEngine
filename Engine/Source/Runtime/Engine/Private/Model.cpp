// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Model.cpp: Unreal model functions
=============================================================================*/

#include "Model.h"

#include "Containers/TransArray.h"
#include "EngineUtils.h"
#include "Engine/Polys.h"
#include "Hash/Blake3.h"
#include "StaticLighting.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Model)

float UModel::BSPTexelScale = 100.0f;

DEFINE_LOG_CATEGORY_STATIC(LogModel, Log, All);

/*-----------------------------------------------------------------------------
	FBspSurf object implementation.
-----------------------------------------------------------------------------*/
#if WITH_EDITOR
/**
 * Returns true if this surface is currently hidden in the editor
 *
 * @return true if this surface is hidden in the editor; false otherwise
 */
bool FBspSurf::IsHiddenEd() const
{
	return bHiddenEdTemporary || bHiddenEdLevel || bHiddenEdLayer;
}

/**
 * Returns true if this surface is hidden at editor startup
 *
 * @return true if this surface is hidden at editor startup; false otherwise
 */
bool FBspSurf::IsHiddenEdAtStartup() const
{
	return ( ( PolyFlags & PF_HiddenEd ) != 0 ); 
}
#endif

/*-----------------------------------------------------------------------------
	Struct serializers.
-----------------------------------------------------------------------------*/
#if WITH_EDITOR
template <typename T>
static void ModelUpdateHash(FBlake3& Builder, const TObjectPtr<T>& Object)
{
	// We don't merge other UObject data into the lighting guid, the lighting guid only handles properties of the UModel changing
	// just record the path for each UObject
	FString PathName = Object.GetPathName();
	if (PathName.IsEmpty())
	{
		Builder.Update(TEXT(""), 0);
	}
	else
	{
		Builder.Update(*PathName, PathName.Len() * sizeof(PathName[0]));
	}
};
static void ModelUpdateHash(FBlake3& Builder, UObject* Object)
{
	// We don't merge other UObject data into the lighting guid, the lighting guid only handles properties of the UModel changing
	// just record the path for each UObject
	FString PathName = Object ? Object->GetPathName() : TEXT("");
	if (PathName.IsEmpty())
	{
		Builder.Update(TEXT(""), 0);
	}
	else
	{
		Builder.Update(*PathName, PathName.Len() * sizeof(PathName[0]));
	}
};
#endif

FArchive& operator<<( FArchive& Ar, FBspSurf& Surf )
{
	Ar << Surf.Material;
	Ar << Surf.PolyFlags;	
	Ar << Surf.pBase << Surf.vNormal;
	Ar << Surf.vTextureU << Surf.vTextureV;
	Ar << Surf.iBrushPoly;
	Ar << Surf.Actor;
	Ar << Surf.Plane;
	Ar << Surf.LightMapScale;
	Ar << Surf.iLightmassIndex;

	// If transacting, we do want to serialize the temporary visibility
	// flags; but not in any other situation
	if ( Ar.IsTransacting() )
	{
		Ar << Surf.bHiddenEdTemporary;
		Ar << Surf.bHiddenEdLevel;
		Ar << Surf.bHiddenEdLayer;
	}

	return Ar;
}

#if WITH_EDITOR
void UpdateHash(FBlake3& Builder, const FBspSurf& Surf)
{
	ModelUpdateHash(Builder, Surf.Material);
	Builder.Update(&Surf.PolyFlags, sizeof(Surf.PolyFlags));
	Builder.Update(&Surf.pBase, sizeof(Surf.pBase));
	Builder.Update(&Surf.vNormal, sizeof(Surf.vNormal));
	Builder.Update(&Surf.vTextureU, sizeof(Surf.vTextureU));
	Builder.Update(&Surf.vTextureV, sizeof(Surf.vTextureV));
	Builder.Update(&Surf.iBrushPoly, sizeof(Surf.iBrushPoly));
	ModelUpdateHash(Builder, Surf.Actor);
	Builder.Update(&Surf.Plane, sizeof(Surf.Plane));
	Builder.Update(&Surf.LightMapScale, sizeof(Surf.LightMapScale));
	Builder.Update(&Surf.iLightmassIndex, sizeof(Surf.iLightmassIndex));
}
#endif

void FBspSurf::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( Material );
	Collector.AddReferencedObject( Actor );
}

FArchive& operator<<( FArchive& Ar, FPoly& Poly )
{
	int32 LegacyNumVertices = Poly.Vertices.Num();
	Ar << Poly.Base << Poly.Normal << Poly.TextureU << Poly.TextureV;
	Ar << Poly.Vertices;
	Ar << Poly.PolyFlags;
	Ar << Poly.Actor << Poly.ItemName;
	Ar << Poly.Material;
	Ar << Poly.iLink << Poly.iBrushPoly;
	Ar << Poly.LightMapScale;
	Ar << Poly.LightmassSettings;
	Ar << Poly.RulesetVariation;
	
	return Ar;
}

FArchive& operator<<( FArchive& Ar, FBspNode& N )
{
	// @warning BulkSerialize: FBSPNode is serialized as memory dump
	// See TArray::BulkSerialize for detailed description of implied limitations.

	// Serialize in the order of variable declaration so the data is compatible with BulkSerialize
	Ar	<< N.Plane;
	Ar	<< N.iVertPool
		<< N.iSurf
		<< N.iVertexIndex
		<< N.ComponentIndex 
		<< N.ComponentNodeIndex
		<< N.ComponentElementIndex;
	
	Ar	<< N.iBack
		<< N.iFront
		<< N.iPlane
		<< N.iCollisionBound
		<< N.iZone[0]
		<< N.iZone[1]
		<< N.NumVertices
		<< N.NodeFlags
		<< N.iLeaf[0]
		<< N.iLeaf[1];

	if( Ar.IsLoading() )
	{
		//@warning: this code needs to be in sync with UModel::Serialize as we use bulk serialization.
		N.NodeFlags &= ~(NF_IsNew|NF_IsFront|NF_IsBack);
	}

	return Ar;
}

FArchive& operator<<( FArchive& Ar, FZoneProperties& P )
{
	Ar	<< P.ZoneActor
		<< P.Connectivity
		<< P.Visibility
		<< P.LastRenderTime;
	return Ar;
}

/**
* Serializer
*
* @param Ar - archive to serialize with
* @param V - vertex to serialize
* @return archive that was used
*/
FArchive& operator<<(FArchive& Ar,FModelVertex& V)
{
	Ar << V.Position;

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::IncreaseNormalPrecision)
	{
		FDeprecatedSerializedPackedNormal Temp;
		Ar << Temp;
		V.TangentX = Temp;
		Ar << Temp;
		V.TangentZ = Temp;
	}
	else
	{
		Ar << V.TangentX;
		Ar << V.TangentZ;
	}
	
	Ar << V.TexCoord;
	Ar << V.ShadowTexCoord;	

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FDepecatedModelVertex& V)
{
	Ar << V.Position;
	Ar << V.TangentX;
	Ar << V.TangentZ;
	Ar << V.TexCoord;
	Ar << V.ShadowTexCoord;

	return Ar;
}

FNodeGroup::~FNodeGroup() = default;

/*---------------------------------------------------------------------------------------
	UModel object implementation.
---------------------------------------------------------------------------------------*/

void UModel::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	const int32 StripVertexBufferFlag = 1;
	FStripDataFlags StripFlags( Ar, GetOuter() && GetOuter()->IsA(ABrush::StaticClass()) ? StripVertexBufferFlag : static_cast<int32>(FStripDataFlags::EStrippedData::None) );

	Ar << Bounds;

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_BSP_UNDO_FIX )
	{
		TTransArray<FVector3f> OldVectors(this);
		TTransArray<FVector3f> OldPoints(this);
		TTransArray<FBspNode> OldNodes(this);
		OldVectors.BulkSerialize(Ar);
		OldPoints.BulkSerialize(Ar);
		OldNodes.BulkSerialize(Ar);

		Vectors = OldVectors;
		Points = OldPoints;
		Nodes = OldNodes;
	}
	else
	{
		Vectors.BulkSerialize(Ar);
		Points.BulkSerialize(Ar);
		Nodes.BulkSerialize(Ar);
	}
	if( Ar.IsLoading() )
	{
		for( int32 NodeIndex=0; NodeIndex<Nodes.Num(); NodeIndex++ )
		{
			Nodes[NodeIndex].NodeFlags &= ~(NF_IsNew|NF_IsFront|NF_IsBack);
		}
	}
	
	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_BSP_UNDO_FIX )
	{
		TTransArray<FBspSurf> OldSurfs(this);
		TTransArray<FVert> OldVerts(this);

		Ar << OldSurfs;
		OldVerts.BulkSerialize(Ar);

		Surfs = OldSurfs;
		Verts = OldVerts;
	}
	else
	{
		Ar << Surfs;
		Verts.BulkSerialize(Ar);
	}

	if( Ar.IsLoading() && Ar.UEVer() < VER_UE4_REMOVE_ZONES_FROM_MODEL)
	{
		int32 NumZones;
		Ar << NumSharedSides << NumZones;

		FZoneProperties DummyZones[FBspNode::MAX_ZONES];
		for( int32 i=0; i<NumZones; i++ )
		{
			Ar << DummyZones[i];
		}
	}
	else
	{
		Ar << NumSharedSides;
	}

#if WITH_EDITOR
	bool bHasEditorOnlyData = !Ar.IsFilterEditorOnly();
	
	if ( Ar.UEVer() < VER_UE4_REMOVE_UNUSED_UPOLYS_FROM_UMODEL )
	{
		bHasEditorOnlyData = true;
	}

	// if we are cooking then don't save this stuff out
	if ( bHasEditorOnlyData )
	{
		Ar << Polys;
		LeafHulls.BulkSerialize( Ar );
		Leaves.BulkSerialize( Ar );
	}
#else
	bool bHasEditorOnlyData = !Ar.IsFilterEditorOnly();
	
	if ( Ar.UEVer() < VER_UE4_REMOVE_UNUSED_UPOLYS_FROM_UMODEL )
	{
		bHasEditorOnlyData = true;
	}

	if((Ar.IsLoading() || Ar.IsSaving()) && bHasEditorOnlyData)
	{
		UPolys* DummyPolys = NULL;
		Ar << DummyPolys;

		TArray<int32> DummyLeafHulls;
		DummyLeafHulls.BulkSerialize( Ar );

		TArray<FLeaf> DummyLeaves;
		DummyLeaves.BulkSerialize( Ar );
	}
#endif

	Ar << RootOutside << Linked;

	if(Ar.IsLoading() && Ar.UEVer() < VER_UE4_REMOVE_ZONES_FROM_MODEL)
	{
		TArray<int32> DummyPortalNodes;
		DummyPortalNodes.BulkSerialize( Ar );
	}

	Ar << NumUniqueVertices; 
	// load/save vertex buffer
	if( StripFlags.IsEditorDataStripped() == false || StripFlags.IsClassDataStripped( StripVertexBufferFlag ) == false )
	{
		Ar << VertexBuffer;
	}

#if WITH_EDITOR
	if(GIsEditor)
	{
		CalculateUniqueVertCount();
	}
#endif // WITH_EDITOR

	// serialize the lighting guid if it's there
	Ar << LightingGuid;

	Ar << LightmassSettings;
}

void UModel::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UModel* This = CastChecked<UModel>(InThis);
#if WITH_EDITOR
	Collector.AddReferencedObject( This->Polys, This );
#endif // WITH_EDITOR
	for( int32 Index = 0; Index < This->Surfs.Num(); Index++ )
	{
		This->Surfs[ Index ].AddReferencedObjects( Collector );
	}

	Super::AddReferencedObjects( This, Collector );
}

#if WITH_EDITOR
void UModel::CalculateUniqueVertCount()
{
	NumUniqueVertices = Points.Num();

	if(NumUniqueVertices == 0 && Polys != NULL)
	{
		TArray<FVector> UniquePoints;

		for(int32 PolyIndex(0); PolyIndex < Polys->Element.Num(); ++PolyIndex)
		{
			for(int32 VertIndex(0); VertIndex < Polys->Element[PolyIndex].Vertices.Num(); ++VertIndex)
			{
				bool bAlreadyAdded(false);
				for(int32 UniqueIndex(0); UniqueIndex < UniquePoints.Num(); ++UniqueIndex)
				{
					if((FVector)Polys->Element[PolyIndex].Vertices[VertIndex] == UniquePoints[UniqueIndex])
					{
						bAlreadyAdded = true;
						break;
					}
				}

				if(!bAlreadyAdded)
				{
					UniquePoints.Push((FVector)Polys->Element[PolyIndex].Vertices[VertIndex]);
				}
			}
		}

		NumUniqueVertices = UniquePoints.Num();
	}
}
#endif // WITH_EDITOR

void UModel::PostLoad()
{
	Super::PostLoad();
	
	if( FApp::CanEverRender() && !HasAnyFlags(RF_ClassDefaultObject) )
	{
		UpdateVertices();
	}

	// If in the editor, initialize each surface to hidden or not depending upon
	// whether the poly flag dictates being hidden at editor startup or not
	if ( GIsEditor )
	{
		for ( TArray<FBspSurf>::TIterator SurfIter( Surfs ); SurfIter; ++SurfIter )
		{
			FBspSurf& CurSurf = *SurfIter;
			CurSurf.bHiddenEdTemporary = ( ( CurSurf.PolyFlags & PF_HiddenEd ) != 0 );
			CurSurf.bHiddenEdLevel = 0;
#if WITH_EDITOR
			CurSurf.bHiddenEdLayer = CurSurf.Actor ? CurSurf.Actor->bHiddenEdLayer : 0;
#else
			CurSurf.bHiddenEdLayer = 0;
#endif
		}

#if WITH_EDITOR
		if (ABrush* Owner = Cast<ABrush>(GetOuter()))
		{
			OwnerLocationWhenLastBuilt = (FVector3f)Owner->GetActorLocation();
			OwnerScaleWhenLastBuilt = (FVector3f)Owner->GetActorScale();
			OwnerRotationWhenLastBuilt = Owner->GetActorRotation();
			bCachedOwnerTransformValid = true;
		}
#endif
	}
}

void UModel::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
#if WITH_EDITOR
	if (!SaveContext.IsProceduralSave())
	{
		// Reconstruct the lighting guid every time the model is saved by the user in editor.
		// ConstructLightingGuid is deterministic so this will not cause spurious changes.
		LightingGuid = ConstructLightingGuid();
	}
#endif
}

#if WITH_EDITOR
void UModel::PostEditUndo()
{
	InvalidSurfaces = true;

	Super::PostEditUndo();
}

void UModel::ModifySurf( int32 InIndex, bool UpdateBrushes )
{
	Modify(false);

	FBspSurf& Surf = Surfs[InIndex];
	if( UpdateBrushes && Surf.Actor )
	{
		Surf.Actor->Brush->Modify(false);
	}
}

void UModel::ModifyAllSurfs( bool UpdateBrushes )
{
	Modify(false);

	if (UpdateBrushes)
	{
		TArray<UModel*> Brushes;
		Brushes.Reset(Surfs.Num());

		for (const FBspSurf& Surf : Surfs)
		{
			if (Surf.Actor)
			{
				check(Surf.Actor->Brush);
				Brushes.AddUnique(Surf.Actor->Brush);
			}
		}

		for (UModel* Brush: Brushes)
		{
			Brush->Modify(false);
		}
	}
}

void UModel::ModifySelectedSurfs( bool UpdateBrushes )
{
	Modify(false);

	if (UpdateBrushes)
	{
		TArray<UModel*> Brushes;
		Brushes.Reset(Surfs.Num());

		for (const FBspSurf& Surf : Surfs)
		{
			if (Surf.Actor && (Surf.PolyFlags & PF_Selected))
			{
				check(Surf.Actor->Brush);
				Brushes.AddUnique(Surf.Actor->Brush);
			}
		}

		for (UModel* Brush : Brushes)
		{
			Brush->Modify(false);
		}
	}
}

bool UModel::HasSelectedSurfaces() const
{
	for (const auto& Surface : Surfs)
	{
		if (Surface.PolyFlags & PF_Selected)
		{
			return true;
		}
	}

	return false;
}
#endif // WITH_EDITOR

bool UModel::Rename( const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags )
{
#if WITH_EDITOR
	// Also rename the UPolys.
	if (NewOuter && Polys && Polys->GetOuter() == GetOuter())
	{
		if (Polys->Rename(*MakeUniqueObjectName(NewOuter, Polys->GetClass()).ToString(), NewOuter, Flags) == false)
		{
			return false;
		}
	}
#endif // WITH_EDITOR

	return Super::Rename( InName, NewOuter, Flags );
}

/**
 * Called after duplication & serialization and before PostLoad. Used to make sure UModel's FPolys
 * get duplicated as well.
 */
void UModel::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
#if WITH_EDITOR
	if( Polys )
	{
		Polys = CastChecked<UPolys>(StaticDuplicateObject( Polys, this ));
	}
#endif // WITH_EDITOR
}

void UModel::BeginDestroy()
{
	Super::BeginDestroy();
	BeginReleaseResources();
}

bool UModel::IsReadyForFinishDestroy()
{
	return ReleaseResourcesFence.IsFenceComplete() && Super::IsReadyForFinishDestroy();
}

void UModel::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	
	// I'm adding extra stuff that haven't been covered by Serialize 
	// I don't have to include VertexFactories (based on Sam Z)
	for(TMap<UMaterialInterface*,TUniquePtr<FRawIndexBuffer16or32> >::TConstIterator IndexBufferIt(MaterialIndexBuffers);IndexBufferIt;++IndexBufferIt)
	{
		CumulativeResourceSize.AddUnknownMemoryBytes(IndexBufferIt->Value->Indices.Num() * sizeof(uint32));
	}
}

/*---------------------------------------------------------------------------------------
	UModel implementation.
---------------------------------------------------------------------------------------*/

#if WITH_EDITOR
bool UModel::Modify( bool bAlwaysMarkDirty/*=true*/ )
{
	bool bSavedToTransactionBuffer = Super::Modify(bAlwaysMarkDirty);

	// We do not reconstruct the LightingGuid on every change, we reconstruct it (deterministically) when saved

	// Modify all child objects.
	if( Polys )
	{
		bSavedToTransactionBuffer = Polys->Modify(bAlwaysMarkDirty) || bSavedToTransactionBuffer;
	}

	return bSavedToTransactionBuffer;
}

FGuid UModel::ConstructLightingGuid() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UModel::ConstructLightingGuid);
	FBlake3 Builder;

	// Add in all variables but only variables that could affect lighting. This includes all of the geometry and materials.
	// It does not include properties in other UObjects (we copy their path only) because the LightingGuid only
	// needs to cover data on *this.
	// It does not include cached data that only optimizes operations that is redundant with other geometry data.
	// It does not include transient data.

	ModelUpdateHash(Builder, Polys);
	if (!Nodes.IsEmpty())
	{
		static_assert(alignof(FBspNode) <= 1 || sizeof(FBspNode) % alignof(FBspNode) == 0, "We rely on zero padding in arrays");
		checkf(Nodes.Num() < 2 || (int64)&Nodes[1] - (int64)&Nodes[0] == sizeof(Nodes[0]), TEXT("We rely on zero padding in arrays"));
		Builder.Update(Nodes.GetData(), Nodes.Num() * sizeof(Nodes[0]));
	}
	if (!Verts.IsEmpty())
	{
		static_assert(alignof(FVert) <= 1 || sizeof(FVert) % alignof(FVert) == 0, "We rely on zero padding in arrays");
		checkf(Verts.Num() < 2 || (int64)&Verts[1] - (int64)&Verts[0] == sizeof(Verts[0]), TEXT("We rely on zero padding in arrays"));
		Builder.Update(Verts.GetData(), Verts.Num() * sizeof(Verts[0]));
	}
	if (!Vectors.IsEmpty())
	{
		static_assert(alignof(FVector3f) <= 1 || sizeof(FVector3f) % alignof(FVector3f) == 0, "We rely on zero padding in arrays");
		checkf(Vectors.Num() < 2 || (int64)&Vectors[1] - (int64)&Vectors[0] == sizeof(Vectors[0]), TEXT("We rely on zero padding in arrays"));
		Builder.Update(Vectors.GetData(), Vectors.Num() * sizeof(Vectors[0]));
	}
	if (!Points.IsEmpty())
	{
		static_assert(alignof(FVector3f) <= 1 || sizeof(FVector3f) % alignof(FVector3f) == 0, "We rely on zero padding in arrays");
		checkf(Points.Num() < 2 || (int64)&Points[1] - (int64)&Points[0] == sizeof(Points[0]), TEXT("We rely on zero padding in arrays"));
		Builder.Update(Points.GetData(), Points.Num() * sizeof(Points[0]));
	}
	for (const FBspSurf& Value : Surfs)
	{
		UpdateHash(Builder, Value);
	}
	if (!LeafHulls.IsEmpty())
	{
		static_assert(alignof(int32) <= 1 || sizeof(int32) % alignof(int32) == 0, "We rely on zero padding in arrays");
		checkf(LeafHulls.Num() < 2 || (int64)&LeafHulls[1] - (int64)&LeafHulls[0] == sizeof(LeafHulls[0]), TEXT("We rely on zero padding in arrays"));
		Builder.Update(LeafHulls.GetData(), LeafHulls.Num() * sizeof(LeafHulls[0]));
	}
	for (const FLeaf& Value : Leaves)
	{
		Builder.Update(&Value, sizeof(Value));
	}
	for (const FLightmassPrimitiveSettings& Value : LightmassSettings)
	{
		Builder.Update(&Value, sizeof(Value));
	}
	for (const TPair<UMaterialInterface*, TUniquePtr<FRawIndexBuffer16or32>>& Pair : MaterialIndexBuffers)
	{
		ModelUpdateHash(Builder, Pair.Key);
		if (Pair.Value)
		{
			for (const uint32& Value : Pair.Value->Indices)
			{
				Builder.Update(&Value, sizeof(Value));
			}
		}
	}
	UpdateHash(Builder, VertexBuffer);
	// ReleaseResourcesFence - Not needed, transient runtime rendering 
	// InvalidSurfaces - Not needed, editor operations support 
	// bOnlyRebuildMaterialIndexBuffers - Not needed, does not impact lighting results
	// bInvalidForStaticLighting - Not needed, does not impact lighting results
	// NumUniqueVertices - Not needed, redundant with this->Verts
	// LightingGuid - Not needed, it's the thing we're calculated and is redundant with all the other data
	// NodeGroups - Not needed, redundant with this->Nodes
	// CachedMappings - Not needed, redundant with this->Surfs
	// NumIncompleteNodeGroups - Not needed, editor operations support
	// LightingLevel - Not needed, redundant with this->Nodes
	// OwnerLocationWhenLastBuilt - Not needed, redundant with this->Verts
	// OwnerRotationWhenLastBuilt - Not needed, redundant with this->Verts
	// OwnerScaleWhenLastBuilt - Not needed, redundant with this->Verts
	// bCachedOwnerTransformValid - Not needed, editor operations support
	// RootOutside  - Not needed, editor operations support
	// Linked - Not needed, editor operations support
	// NumSharedSides - Not needed, editor operations support
	// Bounds - Not needed, redundant with this->Verts

	FBlake3Hash Hash = Builder.Finalize();
	uint32* HashBytes = (uint32*)Hash.GetBytes();
	return FGuid(HashBytes[0], HashBytes[1], HashBytes[2], HashBytes[3]);
}
#endif // WITH_EDITOR

//
// Empty the contents of a model.
//
void UModel::EmptyModel( int32 EmptySurfInfo, int32 EmptyPolys )
{
	Nodes			.Empty();
	Verts			.Empty();

#if WITH_EDITOR
	Leaves			.Empty();
	LeafHulls		.Empty();
#endif // WITH_EDITOR

	if( EmptySurfInfo )
	{
		Vectors.Empty();
		Points.Empty();
		Surfs.Empty();
	}

#if WITH_EDITOR
	if( EmptyPolys )
	{
		Polys = NewObject<UPolys>(GetOuter(), NAME_None, RF_Transactional);
	}
#endif // WITH_EDITOR

	// Init variables.
	NumSharedSides	= 4;
}

//
// Create a new model and allocate all objects needed for it.
//
UModel::UModel(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
	, Nodes()
	, Verts()
	, Vectors()
	, Points()
	, Surfs()
	, VertexBuffer(this)
	, InvalidSurfaces(false)
	, bOnlyRebuildMaterialIndexBuffers(false)
#if WITH_EDITOR
	, bCachedOwnerTransformValid(false)
#endif
{

}

UModel::UModel(FVTableHelper& Helper)
	: Super(Helper)
	, Nodes()
	, Verts()
	, Vectors()
	, Points()
	, Surfs()
	, VertexBuffer(this)
	, InvalidSurfaces(false)
	, bOnlyRebuildMaterialIndexBuffers(false)
#if WITH_EDITOR
	, bCachedOwnerTransformValid(false)
#endif
{

}

void UModel::Initialize(ABrush* Owner, bool InRootOutside)
{
	LightingGuid = FGuid::NewGuid();
	RootOutside = InRootOutside;
	SetFlags( RF_Transactional );
	EmptyModel( 1, 1 );
	if( Owner )
	{
		check(Owner->GetBrushComponent());
		Owner->Brush = this;
#if WITH_EDITOR
		Owner->InitPosRotScale();
#endif
	}
	if( GIsEditor && !FApp::IsGame() )
	{
		UpdateVertices();
	}
}

void UModel::Initialize()
{
#if WITH_EDITOR
	LightingLevel = nullptr;
#endif // WITH_EDITOR
	RootOutside = true;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		EmptyModel(1, 0);
		if (GIsEditor && !FApp::IsGame())
		{
			UpdateVertices();
		}
	}
}

#if WITH_EDITOR
void UModel::BuildBound()
{
	if( Polys && Polys->Element.Num() )
	{
		TArray<FVector> NewPoints;
		for( int32 i=0; i<Polys->Element.Num(); i++ )
			for( int32 j=0; j<Polys->Element[i].Vertices.Num(); j++ )
				NewPoints.Add((FVector)Polys->Element[i].Vertices[j]);
		Bounds = FBoxSphereBounds( NewPoints.GetData(), NewPoints.Num() );
	}
}

void UModel::Transform( ABrush* Owner )
{
	check(Owner);

	for( int32 i=0; i<Polys->Element.Num(); i++ )
		Polys->Element[i].Transform((FVector3f)Owner->GetActorLocation());

}

void UModel::ShrinkModel()
{
	Vectors		.Shrink();
	Points		.Shrink();
	Verts		.Shrink();
	Nodes		.Shrink();
	Surfs		.Shrink();
	if( Polys     ) Polys    ->Element.Shrink();
	LeafHulls	.Shrink();
}

#endif // WITH_EDITOR

void UModel::BeginReleaseResources()
{
	// Release the index buffers.
	for(TMap<UMaterialInterface*,TUniquePtr<FRawIndexBuffer16or32> >::TIterator IndexBufferIt(MaterialIndexBuffers);IndexBufferIt;++IndexBufferIt)
	{
		BeginReleaseResource(IndexBufferIt->Value.Get());
	}

	// Release the vertex buffer and factory.
	BeginReleaseResource(&VertexBuffer.Buffers.PositionVertexBuffer);
	BeginReleaseResource(&VertexBuffer.Buffers.StaticMeshVertexBuffer);

	// Use a fence to keep track of the release progress.
	ReleaseResourcesFence.BeginFence();
}

void UModel::UpdateVertices()
{
	// Wait for pending resource release commands to execute.
	ReleaseResourcesFence.Wait();

	// Don't initialize brush rendering resources on consoles
	if (!GetOuter() || !GetOuter()->IsA(ABrush::StaticClass()) || !FPlatformProperties::RequiresCookedData())
	{
#if WITH_EDITOR
		// rebuild vertex buffer if the resource array is not static 
		if( GIsEditor && !FApp::IsGame() )
		{	
			int32 NumVertices = 0;

			NumVertices = BuildVertexBuffers();

			// We want to check whenever we build the vertex buffer that we have the
			// appropriate number of verts, but since we no longer serialize the total
			// non-unique vertcount we only do this check when building the buffer.
			check(NumVertices == VertexBuffer.Vertices.Num());	
		}
#endif
		VertexBuffer.Buffers.InitModelBuffers(VertexBuffer.Vertices);
		
		//  Empty this if we have cooked data and thus won't need it later to generate collision
		// data etc
		if (FApp::IsGame() && FPlatformProperties::RequiresCookedData())
		{
			VertexBuffer.Vertices.Empty();
		}

		ReleaseResourcesFence.BeginFence();
	}
}

/** 
 *	Compute the "center" location of all the verts 
 */
FVector UModel::GetCenter()
{
	FVector Center(0.f);
	uint32 Cnt = 0;
	for(int32 NodeIndex = 0;NodeIndex < Nodes.Num();NodeIndex++)
	{
		FBspNode& Node = Nodes[NodeIndex];
		uint32 NumVerts = (Node.NodeFlags & PF_TwoSided) ? Node.NumVertices / 2 : Node.NumVertices;
		for(uint32 VertexIndex = 0;VertexIndex < NumVerts;VertexIndex++)
		{
			const FVert& Vert = Verts[Node.iVertPool + VertexIndex];
			const FVector3f& Position = Points[Vert.pVertex];
			Center += (FVector)Position;
			Cnt++;
		}
	}

	if( Cnt > 0 )
	{
		Center /= Cnt;
	}
	
	return Center;
}

#if WITH_EDITOR
/**
* Initialize vertex buffer data from UModel data
* Returns the number of vertices in the vertex buffer.
*/
int32 UModel::BuildVertexBuffers()
{
	// Calculate the size of the vertex buffer and the base vertex index of each node.
	int32 NumVertices = 0;
	for(int32 NodeIndex = 0;NodeIndex < Nodes.Num();NodeIndex++)
	{
		FBspNode& Node = Nodes[NodeIndex];
		FBspSurf& Surf = Surfs[Node.iSurf];
		Node.iVertexIndex = NumVertices;
		NumVertices += (Surf.PolyFlags & PF_TwoSided) ? (Node.NumVertices * 2) : Node.NumVertices;
	}

	// size vertex buffer data
	VertexBuffer.Vertices.Empty(NumVertices);
	VertexBuffer.Vertices.AddUninitialized(NumVertices);

	if(NumVertices > 0)
	{
		// Initialize the vertex data
		FModelVertex* DestVertex = (FModelVertex*)VertexBuffer.Vertices.GetData();
		for(int32 NodeIndex = 0;NodeIndex < Nodes.Num();NodeIndex++)
		{
			FBspNode& Node = Nodes[NodeIndex];
			FBspSurf& Surf = Surfs[Node.iSurf];
			const FVector3f& TextureBase = Points[Surf.pBase];
			const FVector3f& TextureX = Vectors[Surf.vTextureU];
			const FVector3f& TextureY = Vectors[Surf.vTextureV];

			// Use the texture coordinates and normal to create an orthonormal tangent basis.
			FVector3f TangentX = TextureX;
			FVector3f TangentY = TextureY;
			FVector3f TangentZ = Vectors[Surf.vNormal];
			FVector3f::CreateOrthonormalBasis(TangentX,TangentY,TangentZ);

			for(uint32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
			{
				const FVert& Vert = Verts[Node.iVertPool + VertexIndex];
				const FVector3f& Position = Points[Vert.pVertex];
				DestVertex->Position = Position;
				DestVertex->TexCoord.X = ((Position - TextureBase) | TextureX) / UModel::GetGlobalBSPTexelScale();
				DestVertex->TexCoord.Y = ((Position - TextureBase) | TextureY) / UModel::GetGlobalBSPTexelScale();
				DestVertex->ShadowTexCoord = Vert.ShadowTexCoord;
				DestVertex->TangentX = TangentX;
				DestVertex->TangentZ = TangentZ;

				// store the sign of the determinant in TangentZ.W
				DestVertex->TangentZ.W = GetBasisDeterminantSign((FVector)TangentX, (FVector)TangentY, (FVector)TangentZ );

				DestVertex++;
			}

			if(Surf.PolyFlags & PF_TwoSided)
			{
				for(int32 VertexIndex = Node.NumVertices - 1;VertexIndex >= 0;VertexIndex--)
				{
					const FVert& Vert = Verts[Node.iVertPool + VertexIndex];
					const FVector3f& Position = Points[Vert.pVertex];
					DestVertex->Position = Position;
					DestVertex->TexCoord.X = ((Position - TextureBase) | TextureX) / UModel::GetGlobalBSPTexelScale();
					DestVertex->TexCoord.Y = ((Position - TextureBase) | TextureY) / UModel::GetGlobalBSPTexelScale();
					DestVertex->ShadowTexCoord = Vert.BackfaceShadowTexCoord;
					DestVertex->TangentX = TangentX;
					DestVertex->TangentZ = -TangentZ;

					// store the sign of the determinant in TangentZ.W
					DestVertex->TangentZ.W = GetBasisDeterminantSign((FVector)TangentX, (FVector)TangentY, (FVector)-TangentZ );

					DestVertex++;
				}
			}
		}
	}

	return NumVertices;
}

#endif

/**
* Clears local (non RHI) data associated with MaterialIndexBuffers
*/
void UModel::ClearLocalMaterialIndexBuffersData()
{
	TMap<UMaterialInterface*,TUniquePtr<FRawIndexBuffer16or32> >::TIterator MaterialIterator(MaterialIndexBuffers);
	for(; MaterialIterator; ++MaterialIterator)
	{
		MaterialIterator->Value->Indices.Empty();
	}
}
