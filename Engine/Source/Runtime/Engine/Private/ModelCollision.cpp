// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
    ModelCollision.cpp: Simple physics and occlusion testing for editor
=============================================================================*/

#include "Model.h"


/*---------------------------------------------------------------------------------------
   Primitive PointCheck.
---------------------------------------------------------------------------------------*/


void UModel::GetSurfacePlanes(
	const AActor*	Owner,
	TArray<FPlane>& OutPlanes)
{
	if( Nodes.Num() )
	{
		const FMatrix Matrix = 
			// Check for just an Owner matrix
			Owner ? Owner->ActorToWorld().ToMatrixWithScale() :
			// BSP is in world space
			FMatrix::Identity;
		const FMatrix MatrixTA = Matrix.TransposeAdjoint();
		const FMatrix::FReal DetM = Matrix.Determinant();

		for (int32 SurfaceIndex = 0; SurfaceIndex < Surfs.Num(); SurfaceIndex++)
		{
			OutPlanes.Add(FPlane(Surfs[SurfaceIndex].Plane).TransformByUsingAdjointT(Matrix, DetM, MatrixTA));
		}
	}
}


/*---------------------------------------------------------------------------------------
   Point searching.
---------------------------------------------------------------------------------------*/

//
// Find closest vertex to a point at or below a node in the Bsp.  If no vertices
// are closer than MinRadius, returns -1.
//
static float FindNearestVertex
(
	const UModel	&Model, 
	const FVector3f	&SourcePoint,
	FVector3f			&DestPoint, 
	float			MinRadius, 
	int32				iNode, 
	int32				&pVertex
)
{
	float ResultRadius = -1.f;
	while( iNode != INDEX_NONE )
	{
		const FBspNode	*Node	= &Model.Nodes[iNode];
		const int32	    iBack   = Node->iBack;
		const float PlaneDist = Node->Plane.PlaneDot( SourcePoint );
		if( PlaneDist>=-MinRadius && Node->iFront!=INDEX_NONE )
		{
			// Check front.
			const float TempRadius = FindNearestVertex (Model,SourcePoint,DestPoint,MinRadius,Node->iFront,pVertex);
			if (TempRadius >= 0.f) {ResultRadius = TempRadius; MinRadius = TempRadius;};
		}
		if( PlaneDist>-MinRadius && PlaneDist<=MinRadius )
		{
			// Check this node's poly's vertices.
			while( iNode != INDEX_NONE )
			{
				// Loop through all coplanars.
				Node                    = &Model.Nodes	[iNode];
				const FBspSurf* Surf    = &Model.Surfs	[Node->iSurf];
				const FVector3f *Base	    = &Model.Points	[Surf->pBase];
				const float TempRadiusSquared	= FVector3f::DistSquared( SourcePoint, *Base );

				if( TempRadiusSquared < FMath::Square(MinRadius) )
				{
					pVertex = Surf->pBase;
					ResultRadius = MinRadius = FMath::Sqrt(TempRadiusSquared);
					DestPoint = *Base;
				}

				const FVert* VertPool = &Model.Verts[Node->iVertPool];
				for (uint8 B=0; B<Node->NumVertices; B++)
				{
					const FVector3f *Vertex   = &Model.Points[VertPool->pVertex];
					const float TempRadiusSquared2 = FVector3f::DistSquared( SourcePoint, *Vertex );
					if( TempRadiusSquared2 < FMath::Square(MinRadius) )
					{
						pVertex      = VertPool->pVertex;
						ResultRadius = MinRadius = FMath::Sqrt(TempRadiusSquared2);
						DestPoint    = *Vertex;
					}
					VertPool++;
				}
				iNode = Node->iPlane;
			}
		}
		if( PlaneDist > MinRadius )
			break;
		iNode = iBack;
	}
	return ResultRadius;
}

//
// Find Bsp node vertex nearest to a point (within a certain radius) and
// set the location.  Returns distance, or -1.f if no point was found.
//
float UModel::FindNearestVertex
(
	const FVector3f	&SourcePoint,
	FVector3f			&DestPoint,
	float			MinRadius, 
	int32				&pVertex
) const
{
	return Nodes.Num() ? ::FindNearestVertex( *this,SourcePoint,DestPoint,MinRadius,0,pVertex ) : -1.f;
}

/*---------------------------------------------------------------------------------------
	Find Brush Actor
---------------------------------------------------------------------------------------*/
//
// ClipNode - return the node containing a specified location from a number of coplanar nodes.
//
int32 ClipNode(const UModel& Model, int32 iNode, const FVector3f& HitLocation)
{
	while (iNode != INDEX_NONE)
	{
		const FBspNode& Node = Model.Nodes[iNode];
		int32 NumVertices = Node.NumVertices;

		// Only consider this node if it has some vertices.
		if (NumVertices > 0)
		{
			int32 iVertPool = Node.iVertPool;

			FVector PrevPt(Model.Points[Model.Verts[iVertPool + NumVertices - 1].pVertex]);
			FVector Normal(Model.Surfs[Node.iSurf].Plane);
			float PrevDot = 0.f;

			for (int32 i = 0; i<NumVertices; i++)
			{
				FVector Pt(Model.Points[Model.Verts[iVertPool + i].pVertex]);
				float Dot = FPlane(Pt, Normal ^ (Pt - PrevPt)).PlaneDot((FVector)HitLocation);
				// Check for sign change
				if ((Dot < 0.f && PrevDot > 0.f) || (Dot > 0.f && PrevDot < 0.f))
				{
					goto TryNextNode;
				}
				PrevPt = Pt;
				PrevDot = Dot;
			}

			return iNode;
		}

	TryNextNode:;
		iNode = Node.iPlane;
	}
	return INDEX_NONE;
}

//
// Find the iSurf for node the point lies upon. Returns INDEX_NONE if the point does not lie on a surface.
//
static int32 FindSurf(const UModel &Model, const FVector3f &SourcePoint, int32 iNode, float Tolerance)
{
	while (iNode != INDEX_NONE)
	{
		const FBspNode	*Node = &Model.Nodes[iNode];
		const int32	    iBack = Node->iBack;
		const float PlaneDist = Node->Plane.PlaneDot(SourcePoint);
		if (PlaneDist >= -Tolerance && Node->iFront != INDEX_NONE)
		{
			// Check front.
			int32 Front = FindSurf(Model, SourcePoint, Node->iFront, Tolerance);
			if (Front != INDEX_NONE)
			{
				return Front;
			}
		}
		if (PlaneDist > -Tolerance && PlaneDist <= Tolerance)
		{
			iNode = ClipNode(Model, iNode, SourcePoint);
			if (iNode != INDEX_NONE)
			{
				return Model.Nodes[iNode].iSurf;
			}
			else
			{
				return Tolerance;
			}
		}
		if (PlaneDist > UE_KINDA_SMALL_NUMBER)
		{
			break;
		}
		iNode = iBack;
	}
	return INDEX_NONE;
}

//
// Find the brush actor associated with this point, or NULL if the point does not lie on a BSP surface.
//
ABrush* UModel::FindBrush(const FVector3f	&SourcePoint) const
{
	if (Nodes.Num())
	{
		int32 iSurf = FindSurf(*this, SourcePoint, 0, 0.1f);
		if (iSurf != INDEX_NONE)
		{
			return Surfs[iSurf].Actor;
		}
	}
	return nullptr;
}

/*---------------------------------------------------------------------------------------
   Bound filter precompute.
---------------------------------------------------------------------------------------*/

//
// Recursive worker function for UModel::PrecomputeSphereFilter.
//
static void PrecomputeSphereFilter( UModel& Model, int32 iNode, const FPlane& Sphere )
{
	do
	{
		FBspNode* Node   = &Model.Nodes[ iNode ];
		Node->NodeFlags &= ~(NF_IsFront | NF_IsBack);
		float Dist       = Node->Plane.PlaneDot( (FVector3f)Sphere );
		if( Dist < -Sphere.W )
		{
			Node->NodeFlags |= NF_IsBack;
			iNode = Node->iBack;
		}
		else
		{	
			if( Dist > Sphere.W )
				Node->NodeFlags |= NF_IsFront;
			else if( Node->iBack != INDEX_NONE )
				PrecomputeSphereFilter( Model, Node->iBack, Sphere );
			iNode = Node->iFront;
		}
	}
	while( iNode != INDEX_NONE );
}

//
// Precompute the front/back test for a bounding sphere.  Tags all nodes that
// the sphere falls into with a NF_IsBack tag (if the sphere is entirely in back
// of the node), a NF_IsFront tag (if the sphere is entirely in front of the node),
// or neither (if the sphere is split by the node).  This only affects nodes
// that the sphere falls in.  Thus, it is not necessary to perform any cleanup
// after precomputing the filter as long as you're sure the sphere completely
// encloses the object whose filter you're precomputing.
//
void UModel::PrecomputeSphereFilter( const FPlane& Sphere )
{
	if( Nodes.Num() )
		::PrecomputeSphereFilter( *this, 0, Sphere );

}

/**
 * Creates a bounding box for the passed in node
 *
 * @param	Node	Node to create a bounding box for
 * @param	OutBox	The created box
 */
void UModel::GetNodeBoundingBox( const FBspNode& Node, FBox& OutBox ) const
{
	OutBox.Init();
	const int32 FirstVertexIndex = Node.iVertPool;
	for( int32 VertexIndex=0; VertexIndex<Node.NumVertices; VertexIndex++ )
	{
		const FVert&	ModelVert	= Verts[ FirstVertexIndex + VertexIndex ];
		const FVector3f	Vertex		= Points[ ModelVert.pVertex ];
		OutBox += (FVector)Vertex;
	}
}

