// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ModelLight.cpp: Unreal model lighting.
=============================================================================*/

#include "ModelLight.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "Components/LightComponent.h"
#include "Misc/ScopedSlowTask.h"
#include "ComponentReregisterContext.h"
#include "RenderUtils.h"
#include "UnrealEngine.h"
#include "TextureLayout.h"
#include "Collision.h"
#include "LightMap.h"
#include "ShadowMap.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/ModelComponent.h"

//
//	Static vars
//

/** The new BSP elements that are made during lighting, and will be applied to the components when all lighting is done */
TMap<UModelComponent*, TIndirectArray<FModelElement> > UModelComponent::TempBSPElements;

//
//	Definitions.
//

#define SHADOWMAP_MAX_WIDTH			1024
#define SHADOWMAP_MAX_HEIGHT		1024

#define SHADOWMAP_TEXTURE_WIDTH		512
#define SHADOWMAP_TEXTURE_HEIGHT	512

#if (defined(_MSC_VER) || PLATFORM_MAC || PLATFORM_UNIX) && WITH_EDITOR
	/** Whether to allow cropping of unmapped borders in lightmaps and shadowmaps. Controlled by BaseLightmass.ini setting. */
	extern ENGINE_API bool GAllowLightmapCropping;
#endif

/** Sorts the BSP surfaces by descending static lighting texture size. */
struct FBSPSurfaceDescendingTextureSizeSort
{
	static inline int32 Compare( const FBSPSurfaceStaticLighting* A, const FBSPSurfaceStaticLighting* B	)
	{
		return B->SizeX * B->SizeY - A->SizeX * A->SizeY;
	}
};

/**
 * Checks whether a sphere intersects a BSP node.
 * @param	Model - The BSP tree containing the node.
 * @param	NodeIndex - The index of the node in Model.
 * @param	Point - The origin of the sphere.
 * @param	Radius - The radius of the sphere.
 * @return	True if the sphere intersects the BSP node.
 */
static bool SphereOnNode(UModel* Model,uint32 NodeIndex,FVector Point,float Radius)
{
	FBspNode&	Node = Model->Nodes[NodeIndex];
	FBspSurf&	Surf = Model->Surfs[Node.iSurf];

	for(uint32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
	{
		// Create plane perpendicular to both this side and the polygon's normal.
		FVector	Edge(Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex] - Model->Points[Model->Verts[Node.iVertPool + ((VertexIndex + Node.NumVertices - 1) % Node.NumVertices)].pVertex]);
		FVector EdgeNormal = Edge ^ (FVector)Surf.Plane;
		float	VertexDot = Node.Plane.PlaneDot(Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex]);

		// Ignore degenerate edges.
		if(Edge.SizeSquared() < 2.0f*2.0f)
			continue;

		// If point is not behind all the planes created by this polys edges, it's outside the poly.
		if(FVector::PointPlaneDist(Point, (FVector)Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex],EdgeNormal.GetSafeNormal()) > Radius)
			return 0;
	}

	return 1;
}

FBSPSurfaceStaticLighting::FBSPSurfaceStaticLighting(
	const FNodeGroup* InNodeGroup,
	UModel* InModel,
	UModelComponent* InComponent
	):
	FStaticLightingTextureMapping(
		this, 
		InModel, 
		InNodeGroup->SizeX,
		InNodeGroup->SizeY,
		1
	),
	FStaticLightingMesh(
		InNodeGroup->TriangleVertexIndices.Num() / 3,
		InNodeGroup->TriangleVertexIndices.Num() / 3,
		InNodeGroup->Vertices.Num(),
		InNodeGroup->Vertices.Num(),
		0,
		true,
		false,
		InNodeGroup->RelevantLights,
		InComponent,
		InNodeGroup->BoundingBox, 
		InModel->LightingGuid,
		FGuid()
		),
	NodeGroup(InNodeGroup),
	bComplete(false),
	QuantizedData(NULL),
	Model(InModel)
{}

void FBSPSurfaceStaticLighting::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const
{
	OutV0 = NodeGroup->Vertices[NodeGroup->TriangleVertexIndices[TriangleIndex * 3 + 0]];
	OutV1 = NodeGroup->Vertices[NodeGroup->TriangleVertexIndices[TriangleIndex * 3 + 1]];
	OutV2 = NodeGroup->Vertices[NodeGroup->TriangleVertexIndices[TriangleIndex * 3 + 2]];
}

void FBSPSurfaceStaticLighting::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
{
	OutI0 = NodeGroup->TriangleVertexIndices[TriangleIndex * 3 + 0];
	OutI1 = NodeGroup->TriangleVertexIndices[TriangleIndex * 3 + 1];
	OutI2 = NodeGroup->TriangleVertexIndices[TriangleIndex * 3 + 2];
}

FLightRayIntersection FBSPSurfaceStaticLighting::IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const
{
	FHitResult Result(1.0f);

	for(int32 TriangleIndex = 0;TriangleIndex < NodeGroup->TriangleVertexIndices.Num();TriangleIndex += 3)
	{
		const int32 I0 = NodeGroup->TriangleVertexIndices[TriangleIndex + 0];
		const int32 I1 = NodeGroup->TriangleVertexIndices[TriangleIndex + 1];
		const int32 I2 = NodeGroup->TriangleVertexIndices[TriangleIndex + 2];

		const FVector& V0 = NodeGroup->Vertices[I0].WorldPosition;
		const FVector& V1 = NodeGroup->Vertices[I1].WorldPosition;
		const FVector& V2 = NodeGroup->Vertices[I2].WorldPosition;

		if(LineCheckWithTriangle(Result,V2,V1,V0,Start,End,End - Start))
		{
			// Setup a vertex to represent the intersection.
			FStaticLightingVertex IntersectionVertex;
			IntersectionVertex.WorldPosition = Start + (End - Start) * Result.Time;
			IntersectionVertex.WorldTangentZ = Result.Normal;
			return FLightRayIntersection(true,IntersectionVertex);
		}
	}

	return FLightRayIntersection(false,FStaticLightingVertex());
}

#if WITH_EDITOR
void FBSPSurfaceStaticLighting::Apply(FQuantizedLightmapData* InQuantizedData, const TMap<ULightComponent*,FShadowMapData2D*>& InShadowMapData, const FStaticLightingBuildContext* LightingContext)
{
	if(!bComplete)
	{
		// Update the number of surfaces with incomplete static lighting.
		Model->NumIncompleteNodeGroups--;
	}

	// Save the static lighting until all of the component's static lighting has been built.
	ShadowMapData = InShadowMapData;
	QuantizedData = InQuantizedData;
	bComplete = true;

	// If all the surfaces have complete static lighting, apply the component's static lighting.
	if(Model->NumIncompleteNodeGroups == 0)
	{
		Model->ApplyStaticLighting(LightingContext);
	}
}

bool FBSPSurfaceStaticLighting::DebugThisMapping() const
{
	UNREALED_API extern FSelectedLightmapSample GCurrentSelectedLightmapSample;
	const bool bDebug = GCurrentSelectedLightmapSample.Component 
		&& GCurrentSelectedLightmapSample.NodeIndex >= 0
		&& NodeGroup->Nodes.Contains(GCurrentSelectedLightmapSample.NodeIndex)
		// Only allow debugging if the lightmap resolution hasn't changed
		&& GCurrentSelectedLightmapSample.MappingSizeX == SizeX 
		&& GCurrentSelectedLightmapSample.MappingSizeY == SizeY;

	return bDebug;
}
#endif //WITH_EDITOR

FModelElement* UModelComponent::CreateNewTempElement(UModelComponent* Component)
{
	// make the array if needed
	TIndirectArray<FModelElement>* TempElements = TempBSPElements.Find(Component);
	if (TempElements == NULL)
	{
		TempElements = &UModelComponent::TempBSPElements.Add(Component, TIndirectArray<FModelElement>());
	}

	// make it in the temp array
	FModelElement* Element = new FModelElement(Component, NULL);
	TempElements->Add(Element);
	return Element;
}


void UModelComponent::ApplyTempElements(bool bLightingWasSuccessful)
{
	if (bLightingWasSuccessful)
	{
		TArray<UModel*> UpdatedModels;
		TArray<UModelComponent*> UpdatedComponents;

		// apply the temporary lighting elements to the real data
		for (TMap<UModelComponent*, TIndirectArray<FModelElement> >::TIterator It(TempBSPElements); It; ++It)
		{
			UModelComponent* Component = It.Key();
			TIndirectArray<FModelElement>& TempElements = It.Value();

			// replace the current elements with the ones in the temp array
			Component->Elements = TempElements;

			// make sure the element index for the nodes are correct
			for (int32 ElementIndex = 0; ElementIndex < Component->Elements.Num(); ElementIndex++)
			{
				FModelElement& Element = Component->Elements[ElementIndex];
				for (int32 NodeIndex = 0; NodeIndex < Element.Nodes.Num(); NodeIndex++)
				{
					FBspNode& Node = Component->Model->Nodes[Element.Nodes[NodeIndex]];
					Node.ComponentElementIndex = ElementIndex;
				}
			}
			// cache the model/component for updating below
			UpdatedModels.AddUnique(Component->Model);
			UpdatedComponents.AddUnique(Component);
		}

		// Unregister all of the components that are being modified (they will be reregistered at the end of the scope)
		TIndirectArray<FComponentReregisterContext> ComponentContexts;
		for (int32 ComponentIndex = 0; ComponentIndex < UpdatedComponents.Num(); ComponentIndex++)
		{
			UModelComponent* Component = UpdatedComponents[ComponentIndex];

			ComponentContexts.Add(new FComponentReregisterContext(Component));
		}

		// Release all index buffers since they will be modified by BuildRenderData()
		for (int32 ModelIndex = 0; ModelIndex < UpdatedModels.Num(); ModelIndex++)
		{
			UModel* Model = UpdatedModels[ModelIndex];
			for(TMap<UMaterialInterface*,TUniquePtr<FRawIndexBuffer16or32> >::TIterator IndexBufferIt(Model->MaterialIndexBuffers); IndexBufferIt; ++IndexBufferIt)
			{
				BeginReleaseResource(IndexBufferIt->Value.Get());
			}
		}

		// Block until the index buffers have been released
		FlushRenderingCommands();

		// Rebuild rendering data for each modified component
		for (int32 ComponentIndex = 0; ComponentIndex < UpdatedComponents.Num(); ComponentIndex++)
		{
			UModelComponent* Component = UpdatedComponents[ComponentIndex];

			// Build the render data for the new elements.
			Component->BuildRenderData();
		}

		// Initialize all models' index buffers.
		for (int32 ModelIndex = 0; ModelIndex < UpdatedModels.Num(); ModelIndex++)
		{
			UModel* Model = UpdatedModels[ModelIndex];
			for(TMap<UMaterialInterface*,TUniquePtr<FRawIndexBuffer16or32> >::TIterator IndexBufferIt(Model->MaterialIndexBuffers); IndexBufferIt; ++IndexBufferIt)
			{
				BeginInitResource(IndexBufferIt->Value.Get());
			}

			// Mark the model's package as dirty.
			Model->MarkPackageDirty();
		}

		// After this line, the elements in the ComponentContexts array will be destructed, causing components to reregister.
	}

	// the temp lighting is no longer of any use, so clear it out
	TempBSPElements.Empty();
}


#if WITH_EDITOR
void UModelComponent::GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options)
{
	check(0);
}

void UModelComponent::AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const
{
	for (const FModelElement& Element : Elements)
	{
		InGUIDs.Add(Element.MapBuildDataId);
	}
}
#endif

#if WITH_EDITOR
/** A group of BSP surfaces which have the same static lighting relevance. */
class FSurfaceStaticLightingGroup
{
public:

	/** Information about a grouped surface. */
	struct FSurfaceInfo
	{
		FBSPSurfaceStaticLighting* SurfaceStaticLighting;
		uint32 BaseX;
		uint32 BaseY;
	};

	/** The surfaces in the group. */
	TArray<FSurfaceInfo> Surfaces;

	/** The shadow-mapped lights affecting the group. */
	TArray<ULightComponent*> ShadowMappedLights;

	/** The layout of the group's static lighting texture. */
	FTextureLayout TextureLayout;

	/**
	 * Minimal initialization constructor.
	 */
	FSurfaceStaticLightingGroup(uint32 InSizeX,uint32 InSizeY):
		TextureLayout(1,1,InSizeX,InSizeY)
	{}

	/**
	 * Attempts to add a surface to the group.  It may fail if the surface doesn't match the group or won't fit in the group's texture.
	 * @param SurfaceStaticLighting - The static lighting for the surface to add.
	 * @return true if the surface was successfully added.
	 */
	bool AddSurface(FBSPSurfaceStaticLighting* SurfaceStaticLighting)
	{
 		if ( GAllowLightmapCropping && SurfaceStaticLighting->QuantizedData )
		{
			CropUnmappedTexels( SurfaceStaticLighting->QuantizedData->Data, SurfaceStaticLighting->SizeX, SurfaceStaticLighting->SizeY, SurfaceStaticLighting->MappedRect );
		}
		else
		{
			SurfaceStaticLighting->MappedRect = FIntRect( 0, 0, SurfaceStaticLighting->SizeX, SurfaceStaticLighting->SizeY );
		}

		// Attempt to add the surface to the group's texture.
		uint32 PaddedSurfaceBaseX = 0;
		uint32 PaddedSurfaceBaseY = 0;
		if(TextureLayout.AddElement(PaddedSurfaceBaseX,PaddedSurfaceBaseY,SurfaceStaticLighting->MappedRect.Width(),SurfaceStaticLighting->MappedRect.Height()))
		{
			// The surface fits in the group's texture, add it to the group's surface list.
			FSurfaceInfo* SurfaceInfo = new(Surfaces) FSurfaceInfo;
			SurfaceInfo->SurfaceStaticLighting = SurfaceStaticLighting;

			SurfaceInfo->BaseX = PaddedSurfaceBaseX;
			SurfaceInfo->BaseY = PaddedSurfaceBaseY;

			return true;
		}
		else
		{
			// The surface didn't fit in the group's texture, return failure.
			return false;
		}
	}
};
#endif //WITH_EDITOR

void UModelComponent::InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly)
{
	// Save the model state for transactions.
	Modify();

	FComponentReregisterContext ReregisterContext(this);

	Super::InvalidateLightingCacheDetailed(bInvalidateBuildEnqueuedLighting, bTranslationOnly);

	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		FModelElement& Element = Elements[ElementIndex];
		Element.MapBuildDataId = FGuid::NewGuid();
	}
}

void UModelComponent::PropagateLightingScenarioChange()
{
	FComponentRecreateRenderStateContext Context(this);
}

void UModelComponent::GetSurfaceLightMapResolution( int32 SurfaceIndex, int32 QualityScale, int32& Width, int32& Height, FMatrix& WorldToMap, TArray<int32>* GatheredNodes ) const
{
	FBspSurf& Surf = Model->Surfs[SurfaceIndex];

	// Find a plane parallel to the surface.
	FVector3f MapXf, MapYf;
	Surf.Plane.FindBestAxisVectors(MapXf,MapYf);
	FVector3d MapX(MapXf), MapY(MapYf);

	// Find the surface's nodes and the part of the plane they map to.
	bool bFoundNode = false;
	FVector2D MinUV(WORLD_MAX,WORLD_MAX);
	FVector2D MaxUV(-WORLD_MAX,-WORLD_MAX);

	// if the nodes weren't already gathered, then find the ones in this component
	for(int32 NodeIndex = 0; NodeIndex < (GatheredNodes ? GatheredNodes->Num() : Nodes.Num()); NodeIndex++)
	{
		FBspNode& Node = Model->Nodes[GatheredNodes ? (*GatheredNodes)[NodeIndex] : Nodes[NodeIndex]];

		// if they are already gathered, don't check the surface index
		if (GatheredNodes || Node.iSurf == SurfaceIndex)
		{
			// Compute the bounds of the node's vertices on the surface plane.
			for(uint32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
			{
				bFoundNode = true;

				FVector3d	Position = (FVector3d)Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex];
				double X = MapX | Position,
					Y = MapY | Position;
				MinUV.X = FMath::Min(X,MinUV.X);
				MinUV.Y = FMath::Min(Y,MinUV.Y);
				MaxUV.X = FMath::Max(X,MaxUV.X);
				MaxUV.Y = FMath::Max(Y,MaxUV.Y);
			}
		}
	}

	if (bFoundNode)
	{
		double Scale = Surf.LightMapScale * QualityScale;
		MinUV.X = FMath::FloorToFloat(MinUV.X / Scale) * Scale;
		MinUV.Y = FMath::FloorToFloat(MinUV.Y / Scale) * Scale;
		MaxUV.X = FMath::CeilToFloat(MaxUV.X / Scale) * Scale;
		MaxUV.Y = FMath::CeilToFloat(MaxUV.Y / Scale) * Scale;
		// The intent of the above is that Min and Max should be separated by at least Scale, from the floor/ceil,
		// but in the unlucky case that the range is ~0 and exactly aligned to Scale, Min and Max can take the same value.
		// To avoid this causing a divide by zero below, we enforce the at-least-Scale delta explicitly.
		double UVDx = FMath::Max(Scale, (MaxUV.X - MinUV.X));
		double UVDy = FMath::Max(Scale, (MaxUV.Y - MinUV.Y));

		Width = FMath::Clamp(FMath::CeilToInt32((UVDx) / (Surf.LightMapScale * QualityScale)),4,SHADOWMAP_MAX_WIDTH);
		Height = FMath::Clamp(FMath::CeilToInt32((UVDy) / (Surf.LightMapScale * QualityScale)),4,SHADOWMAP_MAX_HEIGHT);
		WorldToMap = FMatrix(
			FPlane(MapX.X / UVDx,	MapY.X / UVDy,	Surf.Plane.X,	0),
			FPlane(MapX.Y / UVDx,	MapY.Y / UVDy,	Surf.Plane.Y,	0),
			FPlane(MapX.Z / UVDx,	MapY.Z / UVDy,	Surf.Plane.Z,	0),
			FPlane(-MinUV.X / UVDx,	-MinUV.Y / UVDy,	-Surf.Plane.W,	1)
			);
	}
	else
	{
		Width = 0;
		Height = 0;
		WorldToMap = FMatrix::Identity;
	}
}


bool UModelComponent::GetLightMapResolution( int32& Width, int32& Height ) const
{
	int32 LightMapArea = 0;
	for(int32 SurfaceIndex = 0;SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
	{
		int32 SizeX;
		int32 SizeY;
		FMatrix WorldToMap;
		GetSurfaceLightMapResolution(SurfaceIndex, 1, SizeX, SizeY, WorldToMap);
		LightMapArea += SizeX * SizeY;
	}

	Width = FMath::TruncToInt( FMath::Sqrt( static_cast<float>(LightMapArea) ) );
	Height = Width;
	return false;
}


int32 UModelComponent::GetStaticLightMapResolution() const
{
	int32 Width;
	int32 Height;
	GetLightMapResolution(Width, Height);

	return FMath::Max<int32>(Width, Height);
}


void UModelComponent::GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const
{
	int32 LightMapWidth	= 0;
	int32 LightMapHeight	= 0;
	GetLightMapResolution( LightMapWidth, LightMapHeight );
	
	// Stored in texture.
	const float MIP_FACTOR = 1.33f;
	ShadowMapMemoryUsage	= FMath::TruncToInt( MIP_FACTOR * LightMapWidth * LightMapHeight ); // G8

	UWorld* World = GetWorld();
	ERHIFeatureLevel::Type FeatureLevel = World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel;
	if (AllowHighQualityLightmaps(FeatureLevel))
	{ 
		LightMapMemoryUsage = FMath::TruncToInt( NUM_HQ_LIGHTMAP_COEF * MIP_FACTOR * LightMapWidth * LightMapHeight ); // DXT5
	}
	else
	{
		LightMapMemoryUsage = FMath::TruncToInt( NUM_LQ_LIGHTMAP_COEF * MIP_FACTOR * LightMapWidth * LightMapHeight / 2 ); // DXT1
	}
	return;
}


typedef	TArray<int32, TInlineAllocator<16>> FPlaneMapItem;

struct FPlaneKey
{
	int32 X;
	int32 Y;
	int32 Z;
	int32 W;

	FPlaneKey(int32 InX, int32 InY, int32 InZ, int32 InW)
		: X(InX)
		, Y(InY)
		, Z(InZ)
		, W(InW)
	{}

	friend FORCEINLINE bool operator == (const FPlaneKey& A, const FPlaneKey& B)
	{
		return A.X == B.X && A.Y == B.Y && A.Z == B.Z && A.W == B.W;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FPlaneKey& Key)
	{
		return HashCombine(static_cast<uint32>(Key.X), HashCombine(static_cast<uint32>(Key.Y), HashCombine(static_cast<uint32>(Key.Z), static_cast<uint32>(Key.W))));
	}
};

class FPlaneMap
{
public:
	FPlaneMap(float InGranularityXYZ, float InGranularityW, float InThreshold, int32 InitialSize = 0)
		: OneOverGranularityXYZ(1.0f / InGranularityXYZ)
		, OneOverGranularityW(1.0f / InGranularityW)
		, Threshold(InThreshold)
	{
		Clear(InitialSize);
	}

	void Clear(int32 InitialSize = 0);

	void AddPlane(const FPlane& Plane, int32 Index);

	const TMap<FPlaneKey, FPlaneMapItem>& GetMap() const { return PlaneMap; }

private:
	float OneOverGranularityXYZ;
	float OneOverGranularityW;
	float Threshold;

	TMap<FPlaneKey, FPlaneMapItem> PlaneMap;
};


void FPlaneMap::Clear(int32 InitialSize)
{
	PlaneMap.Empty(InitialSize);
}


// Given a grid index in one axis, a real position on the grid and a threshold radius,
// return either:
// - the additional grid index it can overlap in that axis, or
// - the original grid index if there is no overlap.
static FORCEINLINE int32 GetAdjacentIndexIfOverlapping(int32 GridIndex, float GridPos, float GridThreshold)
{
	if (GridPos - GridIndex < GridThreshold)
	{
		return GridIndex - 1;
	}
	else if (1.0f - (GridPos - GridIndex) < GridThreshold)
	{
		return GridIndex + 1;
	}
	else
	{
		return GridIndex;
	}
}

void FPlaneMap::AddPlane(const FPlane& Plane, int32 Index)
{
	// Offset applied to the grid coordinates so aligned vertices (the normal case) don't overlap several grid items (taking into account the threshold)
	const float GridOffset = 0.12345f;

	const float AdjustedPlaneX = Plane.X - GridOffset;
	const float AdjustedPlaneY = Plane.Y - GridOffset;
	const float AdjustedPlaneZ = Plane.Z - GridOffset;
	const float AdjustedPlaneW = Plane.W - GridOffset;

	const float GridX = AdjustedPlaneX * OneOverGranularityXYZ;
	const float GridY = AdjustedPlaneY * OneOverGranularityXYZ;
	const float GridZ = AdjustedPlaneZ * OneOverGranularityXYZ;
	const float GridW = AdjustedPlaneW * OneOverGranularityW;

	// Get the grid indices corresponding to the plane components
	const int32 GridIndexX = FMath::FloorToInt(GridX);
	const int32 GridIndexY = FMath::FloorToInt(GridY);
	const int32 GridIndexZ = FMath::FloorToInt(GridZ);
	const int32 GridIndexW = FMath::FloorToInt(GridW);

	PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, GridIndexY, GridIndexZ, GridIndexW)).Add(Index);

	// The grid has a maximum threshold of a certain radius. If the plane is near the edge of a grid item, it may overlap into other items.
	// Add it to all grid items it can be seen from.
	const float GridThresholdXYZ = Threshold * OneOverGranularityXYZ;
	const float GridThresholdW = Threshold * OneOverGranularityW;
	const int32 NeighbourX = GetAdjacentIndexIfOverlapping(GridIndexX, GridX, GridThresholdXYZ);
	const int32 NeighbourY = GetAdjacentIndexIfOverlapping(GridIndexY, GridY, GridThresholdXYZ);
	const int32 NeighbourZ = GetAdjacentIndexIfOverlapping(GridIndexZ, GridZ, GridThresholdXYZ);
	const int32 NeighbourW = GetAdjacentIndexIfOverlapping(GridIndexW, GridW, GridThresholdW);

	const bool bOverlapsInX = (NeighbourX != GridIndexX);
	const bool bOverlapsInY = (NeighbourY != GridIndexY);
	const bool bOverlapsInZ = (NeighbourZ != GridIndexZ);
	const bool bOverlapsInW = (NeighbourW != GridIndexW);

	if (bOverlapsInX)
	{
		PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, GridIndexY, GridIndexZ, GridIndexW)).Add(Index);

		if (bOverlapsInY)
		{
			PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, NeighbourY, GridIndexZ, GridIndexW)).Add(Index);

			if (bOverlapsInZ)
			{
				PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, NeighbourY, NeighbourZ, GridIndexW)).Add(Index);

				if (bOverlapsInW)
				{
					PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, NeighbourY, NeighbourZ, NeighbourW)).Add(Index);
				}
			}
			else if (bOverlapsInW)
			{
				PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, NeighbourY, GridIndexZ, NeighbourW)).Add(Index);
			}
		}
		else
		{
			if (bOverlapsInZ)
			{
				PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, GridIndexY, NeighbourZ, GridIndexW)).Add(Index);

				if (bOverlapsInW)
				{
					PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, GridIndexY, NeighbourZ, NeighbourW)).Add(Index);
				}
			}
			else if (bOverlapsInW)
			{
				PlaneMap.FindOrAdd(FPlaneKey(NeighbourX, GridIndexY, GridIndexZ, NeighbourW)).Add(Index);
			}
		}
	}
	else
	{
		if (bOverlapsInY)
		{
			PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, NeighbourY, GridIndexZ, GridIndexW)).Add(Index);

			if (bOverlapsInZ)
			{
				PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, NeighbourY, NeighbourZ, GridIndexW)).Add(Index);

				if (bOverlapsInW)
				{
					PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, NeighbourY, NeighbourZ, NeighbourW)).Add(Index);
				}
			}
			else if (bOverlapsInW)
			{
				PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, NeighbourY, GridIndexZ, NeighbourW)).Add(Index);
			}
		}
		else
		{
			if (bOverlapsInZ)
			{
				PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, GridIndexY, NeighbourZ, GridIndexW)).Add(Index);

				if (bOverlapsInW)
				{
					PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, GridIndexY, NeighbourZ, NeighbourW)).Add(Index);
				}
			}
			else if (bOverlapsInW)
			{
				PlaneMap.FindOrAdd(FPlaneKey(GridIndexX, GridIndexY, GridIndexZ, NeighbourW)).Add(Index);
			}
		}
	}
}



/**
 * Groups all nodes in the model into NodeGroups (cached in the NodeGroups object)
 *
 * @param Level The level for this model
 * @param Lights The possible lights that will be cached in the NodeGroups
 */
void UModel::GroupAllNodes(ULevel* Level, const TArray<ULightComponentBase*>& Lights)
{
#if WITH_EDITOR
	FScopedSlowTask SlowTask(10);
	SlowTask.MakeDialogDelayed(3.0f);

	// cache the level
	LightingLevel = Level;

	SlowTask.EnterProgressFrame(1);

	// gather all the lights for each component
	TMap<int32, TArray<ULightComponent*> > ComponentRelevantLights;
	for (int32 ComponentIndex = 0; ComponentIndex < Level->ModelComponents.Num(); ComponentIndex++)
	{
		// create a list of lights for the component
		TArray<ULightComponent*>* RelevantLights = &ComponentRelevantLights.Add(ComponentIndex, TArray<ULightComponent*>());

		// Find the lights relevant to the component, and add them to the list of lights for this component
		for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
		{
			ULightComponentBase* LightBase = Lights[LightIndex];
			ULightComponent* Light = Cast<ULightComponent>(LightBase);

			// Only add enabled lights and lights that can potentially be enabled at runtime (toggleable)
			if (Light && (Light->GetVisibleFlag() || (!Light->HasStaticLighting() && Light->AffectsPrimitive(Level->ModelComponents[ComponentIndex]))))
			{
				RelevantLights->Add(Light);
			}
		}
	}

	// make sure the NodeGroups is empty
	for (TMap<int32, FNodeGroup*>::TIterator It(NodeGroups); It; ++It)
	{
		delete It.Value();
	}
	NodeGroups.Empty();

	// caches the nodegroups used by each node
	TArray<FNodeGroup*> ParentNodes;
	ParentNodes.Empty(Nodes.Num());
	ParentNodes.AddZeroed(Nodes.Num());

	// We request this value potentially many times, at what appears to be
	// a high cost (even though the routine is trivial), so cache it first
	const int32 ModelComponentCount = Level->ModelComponents.Num();
	TArray<bool> HasStaticLightingCache;
	HasStaticLightingCache.AddUninitialized(ModelComponentCount);
	for (int32 ComponentIndex = 0; ComponentIndex < ModelComponentCount; ComponentIndex++)
	{
		HasStaticLightingCache[ComponentIndex] = Level->ModelComponents[ComponentIndex]->HasStaticLighting();
	}

	// Prebuild results of comparing two LightmassSettings
	const int32 NumLightmassSettings = LightmassSettings.Num();
	TArray<bool> LightmassSettingsEquality;
	for (int Index1 = 0; Index1 < NumLightmassSettings; Index1++)
	{
		for (int Index2 = 0; Index2 < NumLightmassSettings; Index2++)
		{
			LightmassSettingsEquality.Add(LightmassSettings[Index1] == LightmassSettings[Index2]);
		}
	}

	// We need to form groups of nodes which are nearly coplanar.
	// First, identify nodes whose planes are similar in order to vastly reduce the search space.
	// The FPlaneMap buckets together planes with components within a specified granular range.
	FPlaneMap PlaneMap(1/16.0f, 50.0f, GLightmassDebugOptions.CoplanarTolerance);

	SlowTask.EnterProgressFrame(1);

	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		const FBspNode& Node = Nodes[NodeIndex];

		if (Node.NumVertices > 0 && HasStaticLightingCache.Num() && HasStaticLightingCache[Node.ComponentIndex])
		{
			const FBspSurf& Surf = Surfs[Node.iSurf];
			PlaneMap.AddPlane(FPlane(Surf.Plane), NodeIndex);	// LWC_TODO: Perf pessimization
		}
	}

	SlowTask.EnterProgressFrame(8);

	// Every item in the PlaneMap now contains a list of indices of nodes with similar planes.
	// Now we can do a O(n^2) check to see if any pairs of nodes have planes within the allowed threshold, to be added to the same group.
	{
		FScopedSlowTask InnerTask(PlaneMap.GetMap().Num());
		InnerTask.MakeDialogDelayed(3.0f);

		for (const auto& Pair : PlaneMap.GetMap())
		{
			InnerTask.EnterProgressFrame(1);

			const auto& PlaneMapItem = Pair.Value;

			const int32 NumMapNodes = PlaneMapItem.Num();
			if (NumMapNodes > 1)
			{
				for (int32 MapIndex1 = 0; MapIndex1 < NumMapNodes - 1; MapIndex1++)
				{
					for (int32 MapIndex2 = MapIndex1 + 1; MapIndex2 < NumMapNodes; MapIndex2++)
					{
						const int32 NodeIndex1 = PlaneMapItem[MapIndex1];
						const int32 NodeIndex2 = PlaneMapItem[MapIndex2];
						const FBspNode& Node1 = Nodes[NodeIndex1];
						const FBspNode& Node2 = Nodes[NodeIndex2];
						const FBspSurf& Surf1 = Surfs[Node1.iSurf];
						const FBspSurf& Surf2 = Surfs[Node2.iSurf];

						// if I've already been parented, I don't need to reparent
						if (ParentNodes[NodeIndex1] != nullptr && ParentNodes[NodeIndex2] != nullptr && ParentNodes[NodeIndex1] == ParentNodes[NodeIndex2])
						{
							continue;
						}

						// variable to see check if the 2 nodes are conodes
						bool bNodesAreConodes = false;

						// if we have a tolerance, then join based on coplanar adjacency
						if (GLightmassDebugOptions.bGatherBSPSurfacesAcrossComponents)
						{
							// are these two nodes conodes?
							if (Surf1.LightMapScale == Surf2.LightMapScale)
							{
								if (LightmassSettingsEquality[Surf1.iLightmassIndex * NumLightmassSettings + Surf2.iLightmassIndex])
								{
									if (Surf1.Plane.Equals(Surf2.Plane, GLightmassDebugOptions.CoplanarTolerance))
									{
										// they are coplanar, have the same lightmap res and Lightmass settings, 
										// now we need to check for adjacency which we check for by looking for a shared vertex
										// This is O(n^2) but since there are often only 3 or 4 verts in a poly, this will iterate on average only about 16 times.
										// I doubt it would be any more efficient to use a TSet to check for duplicated indices in this case.
										FVert* VertPool1 = &Verts[Node1.iVertPool];
										for (int32 A = 0; A < Node1.NumVertices && !bNodesAreConodes; A++)
										{
											FVert* VertPool2 = &Verts[Node2.iVertPool];
											for (int32 B = 0; B < Node2.NumVertices && !bNodesAreConodes; B++)
											{
												// if they share a vertex location, they are adjacent (this won't detect adjacency via T-joints)
												if (VertPool1->pVertex == VertPool2->pVertex)
												{
													bNodesAreConodes = true;
												}
												VertPool2++;
											}
											VertPool1++;
										}
									}
								}
							}
						}
						// if coplanar tolerance is < 0, then we join nodes together based on being in the same ModelComponent
						// and from the same surface
						else
						{
							if (Node1.iSurf == Node2.iSurf && Node1.ComponentIndex == Node2.ComponentIndex)
							{
								bNodesAreConodes = true;
							}
						}

						// are Node1 and Node2 conodes - if so, join into a group
						if (bNodesAreConodes)
						{
							// okay, these two nodes are conodes, so we need to stick them together into some pot of nodes
							// look to see if either one are already in a group
							FNodeGroup* NodeGroup = NULL;
							// if both are already in different groups, we need to combine the groups
							if (ParentNodes[NodeIndex1] != NULL && ParentNodes[NodeIndex2] != NULL)
							{
								NodeGroup = ParentNodes[NodeIndex1];

								// merge 2 into 1
								FNodeGroup* NodeGroup2 = ParentNodes[NodeIndex2];
								for (int32 NodeIndex = 0; NodeIndex < NodeGroup2->Nodes.Num(); NodeIndex++)
								{
									NodeGroup->Nodes.Add(NodeGroup2->Nodes[NodeIndex]);
								}
								for (int32 LightIndex = 0; LightIndex < NodeGroup2->RelevantLights.Num(); LightIndex++)
								{
									NodeGroup->RelevantLights.AddUnique(NodeGroup2->RelevantLights[LightIndex]);
								}

								// replace all the users of NodeGroup2 with NodeGroup
								for (int32 GroupIndex = 0; GroupIndex < ParentNodes.Num(); GroupIndex++)
								{
									if (ParentNodes[GroupIndex] == NodeGroup2)
									{
										ParentNodes[GroupIndex] = NodeGroup;
									}
								}

								// the key for the nodegroup is the 0th node (could just be a set now)
								NodeGroups.Remove(NodeGroup2->Nodes[0]);

								// free the now useless nodegroup
								delete NodeGroup2;
							}
							else if (ParentNodes[NodeIndex1] != NULL)
							{
								NodeGroup = ParentNodes[NodeIndex1];
							}
							else if (ParentNodes[NodeIndex2] != NULL)
							{
								NodeGroup = ParentNodes[NodeIndex2];
							}
							// otherwise, make a new group and put them both in it
							else
							{
								NodeGroup = NodeGroups.Add(NodeIndex1, new FNodeGroup());
							}

							// apply both these nodes to the NodeGroup
							for (int32 WhichNode = 0; WhichNode < 2; WhichNode++)
							{
								// operator on each node in this loop
								int32 NodeIndex = WhichNode ? NodeIndex2 : NodeIndex1;

								// track what group the node went into
								ParentNodes[NodeIndex] = NodeGroup;

								// is this node already not yet in the group
								if (NodeGroup->Nodes.Find(NodeIndex) == INDEX_NONE)
								{
									// add it to the group
									NodeGroup->Nodes.Add(NodeIndex);

									// add the relevant lights to the nodegroup
									TArray<ULightComponent*>* RelevantLights = ComponentRelevantLights.Find(Nodes[NodeIndex].ComponentIndex);
									
									// add null check in case there is no light at all and the user presses build lighting.
									for (int32 LightIndex = 0; RelevantLights && LightIndex < RelevantLights->Num(); LightIndex++)
									{
										NodeGroup->RelevantLights.AddUnique((*RelevantLights)[LightIndex]);
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// make a node group for any ungrouped nodes (entries would only be made above if conodes were found)
	for (int32 NodeIndex = 0; NodeIndex < ParentNodes.Num(); NodeIndex++)
	{
		if (ParentNodes[NodeIndex] != NULL)
		{
			continue;
		}

		FNodeGroup* NodeGroup = NodeGroups.Add(NodeIndex, new FNodeGroup());

		// is this node already not yet in the group
		if (NodeGroup->Nodes.Find(NodeIndex) == INDEX_NONE)
		{
			// add it to the group
			NodeGroup->Nodes.Add(NodeIndex);

			// add the relevant lights to the nodegroup
			TArray<ULightComponent*>* RelevantLights = ComponentRelevantLights.Find(Nodes[NodeIndex].ComponentIndex);

			// add null check in case there is no light at all and the user presses build lighting.
			for (int32 LightIndex = 0; RelevantLights && LightIndex < RelevantLights->Num(); LightIndex++)
			{
				NodeGroup->RelevantLights.AddUnique((*RelevantLights)[LightIndex]);
			}
		}
	}
#endif	//WITH_EDITOR
}

/**
 * Applies all of the finished lighting cached in the NodeGroups 
 */
void UModel::ApplyStaticLighting(const FStaticLightingBuildContext* LightingContext)
{
#if WITH_EDITOR
	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GMaxRHIShaderPlatform);

	check(CachedMappings[0]->QuantizedData);

	// Group surfaces based on their static lighting relevance.
	TArray<FSurfaceStaticLightingGroup> SurfaceGroups;
	for (int32 MappingIndex = 0; MappingIndex < CachedMappings.Num(); MappingIndex++)
	{
		FBSPSurfaceStaticLighting* SurfaceStaticLighting = CachedMappings[MappingIndex];

		// Find an existing surface group with the same static lighting relevance.
		FSurfaceStaticLightingGroup* Group = NULL;
		for(int32 GroupIndex = 0;GroupIndex < SurfaceGroups.Num();GroupIndex++)
		{
			FSurfaceStaticLightingGroup& ExistingGroup = SurfaceGroups[GroupIndex];

			// Attempt to add the surface to the group.
			if(ExistingGroup.AddSurface(SurfaceStaticLighting))
			{
				for(TMap<ULightComponent*,FShadowMapData2D*>::TConstIterator ShadowMapDataIt(SurfaceStaticLighting->ShadowMapData);ShadowMapDataIt;++ShadowMapDataIt)
				{
					ExistingGroup.ShadowMappedLights.AddUnique(ShadowMapDataIt.Key());
				}

				Group = &ExistingGroup;
				break;
			}
		}

		// If the surface didn't fit in any existing group, create a new group.
		if(!Group)
		{
			// If the surface is larger than the standard group texture size, create a special group with the texture the same size as the surface.
			uint32 TextureSizeX = SHADOWMAP_TEXTURE_WIDTH;
			uint32 TextureSizeY = SHADOWMAP_TEXTURE_HEIGHT;
			if(SurfaceStaticLighting->SizeX > SHADOWMAP_TEXTURE_WIDTH || SurfaceStaticLighting->SizeY > SHADOWMAP_TEXTURE_HEIGHT)
			{
				TextureSizeX = (SurfaceStaticLighting->SizeX + 3) & ~3;
				TextureSizeY = (SurfaceStaticLighting->SizeY + 3) & ~3;
			}

			// Create the new group.
			Group = ::new(SurfaceGroups) FSurfaceStaticLightingGroup(TextureSizeX,TextureSizeY);

			// Initialize the group's light lists from the surface.
			for(TMap<ULightComponent*,FShadowMapData2D*>::TConstIterator ShadowMapDataIt(SurfaceStaticLighting->ShadowMapData);ShadowMapDataIt;++ShadowMapDataIt)
			{
				Group->ShadowMappedLights.Add(ShadowMapDataIt.Key());
			}

			// Add the surface to the new group.
			verify(Group->AddSurface(SurfaceStaticLighting));
		}
	}

	// Create an element for each surface group.
	for(int32 GroupIndex = 0;GroupIndex < SurfaceGroups.Num();GroupIndex++)
	{
		const FSurfaceStaticLightingGroup& SurfaceGroup = SurfaceGroups[GroupIndex];
		const uint32 GroupSizeX = SurfaceGroup.TextureLayout.GetSizeX();
		const uint32 GroupSizeY = SurfaceGroup.TextureLayout.GetSizeY();

		// initialize new quantized data for the entire group
		FQuantizedLightmapData* GroupQuantizedData = new FQuantizedLightmapData;
		GroupQuantizedData->SizeX = GroupSizeX;
		GroupQuantizedData->SizeY = GroupSizeY;
		GroupQuantizedData->Data.Empty(GroupSizeX * GroupSizeY);
		GroupQuantizedData->Data.AddZeroed(GroupSizeX * GroupSizeY);

		// calculate the new scale for all of the surfaces
		FMemory::Memzero(GroupQuantizedData->Scale, sizeof(GroupQuantizedData->Scale));
		FMemory::Memzero(GroupQuantizedData->Add, sizeof(GroupQuantizedData->Add));

		float MinCoefficient[NUM_STORED_LIGHTMAP_COEF][4];
		float MaxCoefficient[NUM_STORED_LIGHTMAP_COEF][4];

		for( int32 CoefficientIndex = 0; CoefficientIndex < NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2 )
		{
			for( int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++ )
			{
				// Color
				MinCoefficient[ CoefficientIndex ][ ColorIndex ] = 10000.0f;
				MaxCoefficient[ CoefficientIndex ][ ColorIndex ] = 0.0f;

				// Direction
				MinCoefficient[ CoefficientIndex + 1 ][ ColorIndex ] = 10000.0f;
				MaxCoefficient[ CoefficientIndex + 1 ][ ColorIndex ] = -10000.0f;
			}
		}

		for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
		{
			const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceGroup.Surfaces[SurfaceIndex].SurfaceStaticLighting;

			for(int32 CoefficientIndex = 0;CoefficientIndex < NUM_STORED_LIGHTMAP_COEF;CoefficientIndex++)
			{
				for(int32 ColorIndex = 0;ColorIndex < 4;ColorIndex++)
				{
					// The lightmap data for directional coefficients was packed in lightmass with
					// Pack: y = (x - Min) / (Max - Min)
					// We need to solve for Max and Min in order to combine BSP mappings into a lighting group.
					// QuantizedData->Scale and QuantizedData->Add were calculated in lightmass in order to unpack the lightmap data like so
					// Unpack: x = y * UnpackScale + UnpackAdd
					// Which means
					// Scale = Max - Min
					// Add = Min
					// Therefore we can solve for min and max using substitution

					float Scale = SurfaceStaticLighting->QuantizedData->Scale[CoefficientIndex][ColorIndex];
					float Add = SurfaceStaticLighting->QuantizedData->Add[CoefficientIndex][ColorIndex];
					float Min = Add;
					float Max = Scale + Add;

					MinCoefficient[CoefficientIndex][ColorIndex] = FMath::Min(MinCoefficient[CoefficientIndex][ColorIndex], Min);
					MaxCoefficient[CoefficientIndex][ColorIndex] = FMath::Max(MaxCoefficient[CoefficientIndex][ColorIndex], Max);
				}
			}
		}

		// Now calculate the new unpack scale and add based on the composite min and max
		for(int32 CoefficientIndex = 0;CoefficientIndex < NUM_STORED_LIGHTMAP_COEF;CoefficientIndex++)
		{
			for(int32 ColorIndex = 0;ColorIndex < 4;ColorIndex++)
			{
				GroupQuantizedData->Scale[CoefficientIndex][ColorIndex] = FMath::Max(MaxCoefficient[CoefficientIndex][ColorIndex] - MinCoefficient[CoefficientIndex][ColorIndex], UE_DELTA);
				GroupQuantizedData->Add[CoefficientIndex][ColorIndex] = MinCoefficient[CoefficientIndex][ColorIndex];
			}
		}

		// now gather all surfaces together, requantizing using the new Scale above
		for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
		{
			const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo = SurfaceGroup.Surfaces[SurfaceIndex];
			const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceInfo.SurfaceStaticLighting;

			GroupQuantizedData->bHasSkyShadowing |= SurfaceStaticLighting->QuantizedData->bHasSkyShadowing;

			// Copy the surface's light-map into the merged group light-map.
			for(int32 Y = SurfaceStaticLighting->MappedRect.Min.Y; Y < SurfaceStaticLighting->MappedRect.Max.Y; Y++)
			{
				for(int32 X = SurfaceStaticLighting->MappedRect.Min.X; X < SurfaceStaticLighting->MappedRect.Max.X; X++)
				{
					// get source from input, dest from the rectangular offset in the group
					FLightMapCoefficients& SourceSample = SurfaceStaticLighting->QuantizedData->Data[Y * SurfaceStaticLighting->SizeX + X];
					FLightMapCoefficients& DestSample = GroupQuantizedData->Data[(SurfaceInfo.BaseY + Y - SurfaceStaticLighting->MappedRect.Min.Y) * GroupSizeX + (SurfaceInfo.BaseX + X - SurfaceStaticLighting->MappedRect.Min.X)];

					// coverage doesn't change
					DestSample.Coverage = SourceSample.Coverage;

					// Treat alpha special because of residual
					{
						// Decode LogL
						float LogL = (float)SourceSample.Coefficients[0][3] / 255.0f;
						float Residual = (float)SourceSample.Coefficients[1][3] / 255.0f;
						LogL += ( Residual - 0.5f ) / 255.0f;
						LogL = LogL * SurfaceStaticLighting->QuantizedData->Scale[0][3] + SurfaceStaticLighting->QuantizedData->Add[0][3];

						// Encode LogL
						LogL = ( LogL - GroupQuantizedData->Add[0][3] ) / GroupQuantizedData->Scale[0][3];
						Residual = LogL * 255.0f - FMath::RoundToFloat( LogL * 255.0f ) + 0.5f;

						DestSample.Coefficients[0][3] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( LogL * 255.0f ), 0, 255 );
						DestSample.Coefficients[1][3] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Residual * 255.0f ), 0, 255 );
					}
	
					// go over each color coefficient and dequantize and requantize with new Scale/Add
					for(int32 CoefficientIndex = 0;CoefficientIndex < NUM_STORED_LIGHTMAP_COEF;CoefficientIndex++)
					{
						// Don't touch alpha here
						for(int32 ColorIndex = 0;ColorIndex < 3;ColorIndex++)
						{
							// dequantize it
							float Dequantized = (float)SourceSample.Coefficients[CoefficientIndex][ColorIndex] / 255.0f;
							const float Exponent = CoefficientIndex == 0 ? 2.0f : 1.0f;
							Dequantized = FMath::Pow(Dequantized, Exponent);

							const float Unpacked = Dequantized * SurfaceStaticLighting->QuantizedData->Scale[CoefficientIndex][ColorIndex] + SurfaceStaticLighting->QuantizedData->Add[CoefficientIndex][ColorIndex];
							const float Repacked = ( Unpacked - GroupQuantizedData->Add[CoefficientIndex][ColorIndex] ) / GroupQuantizedData->Scale[CoefficientIndex][ColorIndex];

							// requantize it
							DestSample.Coefficients[CoefficientIndex][ColorIndex] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( FMath::Pow( Repacked, 1.0f / Exponent ) * 255.0f ), 0, 255 );
						}
					}

					for(int32 ColorIndex = 0;ColorIndex < 4;ColorIndex++)
					{
						DestSample.SkyOcclusion[ColorIndex] = SourceSample.SkyOcclusion[ColorIndex];
					}

					DestSample.AOMaterialMask = SourceSample.AOMaterialMask;
				}
			}

			// the QuantizedData is expected that AllocateLightMap would take ownership, but since its using a group one, we need to free it
			delete SurfaceStaticLighting->QuantizedData;
		}

		// Calculate the bounds for the lightmap group.
		FBox GroupBox(ForceInit);
		for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
		{
			const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo = SurfaceGroup.Surfaces[SurfaceIndex];
			const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceInfo.SurfaceStaticLighting;
			GroupBox += SurfaceStaticLighting->BoundingBox;
		}
		FBoxSphereBounds GroupLightmapBounds( GroupBox );

		// create the grouped together lightmap, which is used by all elements.
		ELightMapPaddingType PaddingType = GAllowLightmapPadding ? LMPT_PrePadding : LMPT_NoPadding;

		const bool bHasNonZeroData = GroupQuantizedData->HasNonZeroData();

		// We always create a light map if the surface either has any non-zero lighting data, or if the surface has a shadow map.  The runtime
		// shaders are always expecting a light map in the case of a shadow map, even if the lighting is entirely zero.  This is simply to reduce
		// the number of shader permutations to support in the very unlikely case of a unshadowed surfaces that has lighting values of zero.
		const bool bHasRelevantLights = SurfaceGroup.Surfaces.ContainsByPredicate([](const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo) { return SurfaceInfo.SurfaceStaticLighting->RelevantLightsGuid.Num() > 0; });
		const bool bNeedsLightMap = bHasNonZeroData || SurfaceGroup.ShadowMappedLights.Num() > 0 || bHasRelevantLights || GroupQuantizedData->bHasSkyShadowing;

		UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForLevel(LightingLevel);

		// Allocate merged shadow-map data.
		TMap<ULightComponent*,FShadowMapData2D*> GroupShadowMapData;
		for(int32 LightIndex = 0;LightIndex < SurfaceGroup.ShadowMappedLights.Num();LightIndex++)
		{
			GroupShadowMapData.Add(
				SurfaceGroup.ShadowMappedLights[LightIndex],
				new FQuantizedShadowSignedDistanceFieldData2D(GroupSizeX,GroupSizeY)
				);
		}

		// Merge surface shadow-maps into the group shadow-maps.
		for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
		{
			const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo = SurfaceGroup.Surfaces[SurfaceIndex];
			const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceInfo.SurfaceStaticLighting;

			for(TMap<ULightComponent*,FShadowMapData2D*>::TConstIterator ShadowMapDataIt(SurfaceStaticLighting->ShadowMapData);ShadowMapDataIt;++ShadowMapDataIt)
			{
				FShadowMapData2D* GroupShadowMap = GroupShadowMapData.FindRef(ShadowMapDataIt.Key());
				
				if( !GroupShadowMap )
				{
					// No shadow map likely due to light overlap
					continue;
				}

				FShadowMapData2D* SurfaceShadowMap = ShadowMapDataIt.Value();

				FQuantizedShadowSignedDistanceFieldData2D* GroupShadowFactorData = (FQuantizedShadowSignedDistanceFieldData2D*)GroupShadowMap;

				// If the data is already quantized, this will just copy the data
				TArray<FQuantizedSignedDistanceFieldShadowSample> QuantizedData;
				SurfaceShadowMap->Quantize( QuantizedData );

				// Copy the surface's shadow-map into the merged group shadow-map.
				for(int32 Y = SurfaceStaticLighting->MappedRect.Min.Y; Y < SurfaceStaticLighting->MappedRect.Max.Y; Y++)
				{
					for(int32 X = SurfaceStaticLighting->MappedRect.Min.X; X < SurfaceStaticLighting->MappedRect.Max.X; X++)
					{
						const FQuantizedSignedDistanceFieldShadowSample& SourceSample = QuantizedData[Y * SurfaceStaticLighting->SizeX + X];
						FQuantizedSignedDistanceFieldShadowSample& DestSample = (*GroupShadowFactorData)(SurfaceInfo.BaseX + X - SurfaceStaticLighting->MappedRect.Min.X, SurfaceInfo.BaseY + Y - SurfaceStaticLighting->MappedRect.Min.Y);
						DestSample = SourceSample;
					}
				}
			}
		}

		// Create the shadow-maps, which is used by all elements.
		FLightMap2D* LightMap = nullptr;
		FShadowMap2D* ShadowMap = nullptr;

		if (bNeedsLightMap)
		{
			if (bUseVirtualTextures)
			{
				// When using VT lightmaps, shadow data is allocated in the same UV space as lightmap data
				LightMap = FLightMap2D::AllocateLightMap(Registry, GroupQuantizedData, GroupShadowMapData, GroupLightmapBounds, PaddingType, LMF_None);
			}
			else
			{
				TMap<ULightComponent*, FShadowMapData2D*> EmptyShadowMapData;
				LightMap = FLightMap2D::AllocateLightMap(Registry, GroupQuantizedData, EmptyShadowMapData, GroupLightmapBounds, PaddingType, LMF_None);
			}
		}
		
		if(GroupShadowMapData.Num() && !bUseVirtualTextures)
		{
			ShadowMap = FShadowMap2D::AllocateShadowMap( Registry, GroupShadowMapData, GroupLightmapBounds, PaddingType, SMF_None );
		}

		// Apply the surface's static lighting mapping to its vertices.
		for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
		{
			const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo = SurfaceGroup.Surfaces[SurfaceIndex];
			const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceInfo.SurfaceStaticLighting;

			for(int32 NodeIndex = 0;NodeIndex < SurfaceStaticLighting->NodeGroup->Nodes.Num();NodeIndex++)
			{
				const FBspNode& Node = Nodes[SurfaceStaticLighting->NodeGroup->Nodes[NodeIndex]];
				for(int32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
				{
					FVert& Vert = Verts[Node.iVertPool + VertexIndex];
					const FVector3f& WorldPosition = Points[Vert.pVertex];
					const FVector4 StaticLightingTextureCoordinate = SurfaceStaticLighting->NodeGroup->WorldToMap.TransformPosition((FVector)WorldPosition);

					uint32 PaddedSizeX = SurfaceStaticLighting->SizeX;
					uint32 PaddedSizeY = SurfaceStaticLighting->SizeY;
					uint32 BaseX = SurfaceInfo.BaseX - SurfaceStaticLighting->MappedRect.Min.X;
					uint32 BaseY = SurfaceInfo.BaseY - SurfaceStaticLighting->MappedRect.Min.Y;
					if (GLightmassDebugOptions.bPadMappings && GAllowLightmapPadding)
					{
						if ((PaddedSizeX > 2) && (PaddedSizeY > 2))
						{
							PaddedSizeX -= 2;
							PaddedSizeY -= 2;
							BaseX += 1;
							BaseY += 1;
						}
					}

					Vert.ShadowTexCoord.X = (BaseX + StaticLightingTextureCoordinate.X * PaddedSizeX) / (float)GroupSizeX;
					Vert.ShadowTexCoord.Y = (BaseY + StaticLightingTextureCoordinate.Y * PaddedSizeY) / (float)GroupSizeY;
				}
			}
		}

		// we need to go back to the source components and use this lightmap
		TArray<UModelComponent*> Components;
		for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
		{
			const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo = SurfaceGroup.Surfaces[SurfaceIndex];
			const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceInfo.SurfaceStaticLighting;

			// gather all the components that contributed to this mapping
			for(int32 NodeIndex = 0;NodeIndex < SurfaceStaticLighting->NodeGroup->Nodes.Num();NodeIndex++)
			{
				const FBspNode& Node = Nodes[SurfaceStaticLighting->NodeGroup->Nodes[NodeIndex]];
				Components.AddUnique(LightingLevel->ModelComponents[Node.ComponentIndex]);
			}
		}

		// use this lightmap in all of the components that contributed to it
		for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
		{
			UModelComponent* Component = Components[ComponentIndex];

			// Create an element for the surface group.
			FModelElement* Element = UModelComponent::CreateNewTempElement(Component);

			FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(Element->MapBuildDataId, true);
			MeshBuildData.LightMap = LightMap;
			MeshBuildData.ShadowMap = ShadowMap;

 			TSet<FGuid> TempIrrelevantLights;
			for(int32 SurfaceIndex = 0;SurfaceIndex < SurfaceGroup.Surfaces.Num();SurfaceIndex++)
			{
				const FSurfaceStaticLightingGroup::FSurfaceInfo& SurfaceInfo = SurfaceGroup.Surfaces[SurfaceIndex];
				const FBSPSurfaceStaticLighting* SurfaceStaticLighting = SurfaceInfo.SurfaceStaticLighting;

				// Build the list of the element's statically irrelevant lights.
				for(int32 LightIndex = 0;LightIndex < SurfaceStaticLighting->NodeGroup->RelevantLights.Num();LightIndex++)
				{
					ULightComponent* Light = SurfaceStaticLighting->NodeGroup->RelevantLights[LightIndex];

					// Check if the light is stored in the light-map or shadow-map.
					const bool bIsInLightMap = MeshBuildData.LightMap && MeshBuildData.LightMap->ContainsLight(Light->LightGuid);
					const bool bIsInShadowMap = MeshBuildData.ShadowMap && MeshBuildData.ShadowMap->ContainsLight(Light->LightGuid);
					if (!bIsInLightMap && !bIsInShadowMap)
					{
						// Add the light to the statically irrelevant light list if it is in the potentially relevant light list, but didn't contribute to the light-map or a shadow-map.
						TempIrrelevantLights.Add(Light->LightGuid);
					}
				}

				// Add the surfaces' nodes to the element.
				for(int32 NodeIndex = 0;NodeIndex < SurfaceStaticLighting->NodeGroup->Nodes.Num();NodeIndex++)
				{
					const int32 ModelNodeIndex = SurfaceStaticLighting->NodeGroup->Nodes[NodeIndex];
					// Only add nodes from the node group that belong to this component
					if (Nodes[ModelNodeIndex].ComponentIndex == Component->ComponentIndex)
					{
						Element->Nodes.Add(SurfaceStaticLighting->NodeGroup->Nodes[NodeIndex]);
					}
				}
			}

			// Move the data from the set into the array
 			for(TSet<FGuid>::TIterator It(TempIrrelevantLights); It; ++It)
 			{
 				MeshBuildData.IrrelevantLights.Add(*It);
 			}
		}
	}

	// Free the surfaces' static lighting data.
	CachedMappings.Empty();

	// clear the node groups
	for (TMap<int32, FNodeGroup*>::TIterator It(NodeGroups); It; ++It)
	{
		delete It.Value();
	}
	NodeGroups.Empty();

	// Invalidate the model's vertex buffer.
	InvalidSurfaces = true;
#endif	//WITH_EDITOR
}
