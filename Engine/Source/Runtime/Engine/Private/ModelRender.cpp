// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ModelRender.cpp: Unreal model rendering
=============================================================================*/

#include "Engine/Level.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/World.h"
#include "PrimitiveViewRelevance.h"
#include "LightMap.h"
#include "LightSceneProxy.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Model.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/Engine.h"
#include "HModel.h"
#include "Components/ModelComponent.h"
#include "Engine/Brush.h"
#include "Components/ModelComponent.h"
#include "SceneInterface.h"
#include "Hash/Blake3.h"

#include "SceneView.h"

namespace
{
	/** Returns true if a surface should be drawn. This only affects dynamic drawing for selection. */
	FORCEINLINE bool ShouldDrawSurface(const FBspSurf& Surf)
	{
#if WITH_EDITOR
		// Don't draw portal polygons or those hidden within the editor.
		return (Surf.PolyFlags & PF_Portal) == 0 && !Surf.IsHiddenEd();
#else
		return (Surf.PolyFlags & PF_Portal) == 0;
#endif
	}

}	// anon namespace

DEFINE_LOG_CATEGORY_STATIC(LogModelComponent, Log, All);

/*-----------------------------------------------------------------------------
FModelVertexBuffer
-----------------------------------------------------------------------------*/

FModelVertexBuffer::FModelVertexBuffer(UModel* InModel)
:	Model(InModel)
{}

/**
* Serializer for this class
* @param Ar - archive to serialize to
* @param B - data to serialize
*/
FArchive& operator<<(FArchive& Ar,FModelVertexBuffer& B)
{
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::ModelVertexBufferSerialization)
	{
		TResourceArray<FDepecatedModelVertex, VERTEXBUFFER_ALIGNMENT> DepricatedVertices;
		DepricatedVertices.BulkSerialize(Ar);
		B.Vertices.Reset(DepricatedVertices.Num());
		for (const FDepecatedModelVertex& ModelVertex : DepricatedVertices)
		{
			B.Vertices.Add(ModelVertex);
		}
	}
	else if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::IncreaseNormalPrecision)
	{
		TArray<FDepecatedModelVertex> DepricatedVertices;
		DepricatedVertices.BulkSerialize(Ar);
		B.Vertices.Reset(DepricatedVertices.Num());
		for (const FDepecatedModelVertex& ModelVertex : DepricatedVertices)
		{
			B.Vertices.Add(ModelVertex);
		}
	}
	else
	{
		Ar << B.Vertices;
	}

	return Ar;
}

#if WITH_EDITOR
void UpdateHash(FBlake3& Builder, const FModelVertexBuffer& B)
{
	if (!B.Vertices.IsEmpty())
	{
		static_assert(alignof(FModelVertex) <= 1 || sizeof(FModelVertex) % alignof(FModelVertex) == 0, "We rely on zero padding in arrays");
		checkf(B.Vertices.Num() < 2 || (int64)&B.Vertices[1] - (int64)&B.Vertices[0] == sizeof(B.Vertices[0]), TEXT("We rely on zero padding in arrays"));
		Builder.Update(B.Vertices.GetData(), B.Vertices.Num() * sizeof(B.Vertices[0]));
	}
	// B.Buffers - Not needed, transient runtime rendering 
}
#endif
/*-----------------------------------------------------------------------------
UModelComponent
-----------------------------------------------------------------------------*/


void UModelComponent::BuildRenderData()
{
	if (IsRunningDedicatedServer())
	{
		return;
	}

	UModel* TheModel = GetModel();

#if WITH_EDITOR
	const ULevel* Level = CastChecked<ULevel>(GetOuter());
	const bool bIsGameWorld = !Level || !Level->OwningWorld || Level->OwningWorld->IsGameWorld();
#endif

	// Build the component's index buffer and compute each element's bounding box.
	for(int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
	{
		FModelElement& Element = Elements[ElementIndex];

		// Find the index buffer for the element's material.
		TUniquePtr<FRawIndexBuffer16or32>* IndexBufferRef = TheModel->MaterialIndexBuffers.Find(Element.Material);
		if(!IndexBufferRef)
		{
			IndexBufferRef = &TheModel->MaterialIndexBuffers.Emplace(Element.Material,new FRawIndexBuffer16or32());
		}
		FRawIndexBuffer16or32* const IndexBuffer = IndexBufferRef->Get();
		check(IndexBuffer);

		Element.IndexBuffer = IndexBuffer;
		Element.FirstIndex = IndexBuffer->Indices.Num();
		Element.NumTriangles = 0;
		Element.MinVertexIndex = 0xffffffff;
		Element.MaxVertexIndex = 0;
		Element.BoundingBox.Init();
		for(int32 NodeIndex = 0;NodeIndex < Element.Nodes.Num();NodeIndex++)
		{
			const uint16& NodeIdx = Element.Nodes[NodeIndex];
			if( ensureMsgf( TheModel->Nodes.IsValidIndex( NodeIdx ), TEXT( "Invalid Node Index, Idx:%d, Num:%d" ), NodeIdx, TheModel->Nodes.Num() ) )
			{
				const FBspNode& Node = TheModel->Nodes[NodeIdx];
				if (Node.NumVertices == 0)
				{
					Element.MinVertexIndex = 0;
					continue;
				}

				if( ensureMsgf( TheModel->Surfs.IsValidIndex(Node.iSurf), TEXT("Invalid Surf Index, Idx:%d, Num:%d"), Node.iSurf, TheModel->Surfs.Num() ) )
				{
					const FBspSurf& Surf = TheModel->Surfs[Node.iSurf];

#if WITH_EDITOR
					// If we're not in a game world, check the surface visibility
					if(!bIsGameWorld && !ShouldDrawSurface(Surf))
					{
						continue;
					}
#endif

					// Don't put portal polygons in the static index buffer.
					if(Surf.PolyFlags & PF_Portal)
						continue;

					for(uint32 BackFace = 0; BackFace < (uint32)((Surf.PolyFlags & PF_TwoSided) ? 2 : 1); BackFace++)
					{
						for(int32 VertexIndex = 0; VertexIndex < Node.NumVertices; VertexIndex++)
						{
							Element.BoundingBox += (FVector)TheModel->Points[TheModel->Verts[Node.iVertPool + VertexIndex].pVertex];
						}

						for(int32 VertexIndex = 2; VertexIndex < Node.NumVertices; VertexIndex++)
						{
							IndexBuffer->Indices.Add(Node.iVertexIndex + Node.NumVertices * BackFace);
							IndexBuffer->Indices.Add(Node.iVertexIndex + Node.NumVertices * BackFace + VertexIndex);
							IndexBuffer->Indices.Add(Node.iVertexIndex + Node.NumVertices * BackFace + VertexIndex - 1);
							Element.NumTriangles++;
						}
						Element.MinVertexIndex = FMath::Min(Node.iVertexIndex + Node.NumVertices * BackFace, Element.MinVertexIndex);
						Element.MaxVertexIndex = FMath::Max(Node.iVertexIndex + Node.NumVertices * BackFace + Node.NumVertices - 1, Element.MaxVertexIndex);

						uint32 NumModelVertices = TheModel->VertexBuffer.Buffers.StaticMeshVertexBuffer.GetNumVertices();
						if (Element.MaxVertexIndex >= NumModelVertices)
						{
							UE_LOGF(LogModelComponent, Log, "Model %ls has elements that reference missing vertices. MaxVertex=%d, NumModelVertices=%d",
								*GetPathName(), Element.MaxVertexIndex, NumModelVertices);
						}
					}
				}
			}
		}

		IndexBuffer->Indices.Shrink();
		IndexBuffer->ComputeIndexWidth();
	}
}

/**
 * A model component scene proxy.
 */
class FModelSceneProxy final : public FPrimitiveSceneProxy
{
	/** The vertex factory which is used to access VertexBuffer. */
	FLocalVertexFactory VertexFactory;

public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FModelSceneProxy(UModelComponent* InComponent):
		FPrimitiveSceneProxy(InComponent),
		VertexFactory(GetScene().GetFeatureLevel(), "FModelSceneProxy"),
		Component(InComponent),
		CollisionResponse(InComponent->GetCollisionResponseToChannels())
#if WITH_EDITOR
		, CollisionMaterialInstance(GEngine->ShadedLevelColorationUnlitMaterial
			? GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy()
			: nullptr,
			FColor(157,149,223,255))
#endif
	{
		// GetDynamicMeshElements needs to run single pass, due to handling of the bAnySelectedSurfs flag across multiple views
		bSinglePassGDME = true;

		ENQUEUE_RENDER_COMMAND(InitOrUpdateVertexBufferCmd)([VertexBuffer = &InComponent->GetModel()->VertexBuffer](FRHICommandList& RHICmdList)
		{
			if (!VertexBuffer->Buffers.PositionVertexBuffer.IsInitialized())
			{
				VertexBuffer->Buffers.PositionVertexBuffer.InitResource(RHICmdList);
			}
			else if (VertexBuffer->RefCount == 0)
			{
				// Only allow update when no other FModelSceneProxy's is using the vertex buffer
				// otherwise their resources will become invalid
				VertexBuffer->Buffers.PositionVertexBuffer.UpdateRHI(RHICmdList);
			}

			if (!VertexBuffer->Buffers.StaticMeshVertexBuffer.IsInitialized())
			{
				VertexBuffer->Buffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			}
			else if (VertexBuffer->RefCount == 0)
			{
				VertexBuffer->Buffers.StaticMeshVertexBuffer.UpdateRHI(RHICmdList);
			}
		});
		InComponent->GetModel()->VertexBuffer.Buffers.InitModelVF(&VertexFactory);

		OverrideOwnerName(NAME_BSP);
		const TIndirectArray<FModelElement>& SourceElements = Component->GetElements();

		Elements.Empty(SourceElements.Num());
		for(int32 ElementIndex = 0;ElementIndex < SourceElements.Num();ElementIndex++)
		{
			const FModelElement& SourceElement = SourceElements[ElementIndex];
			FElementInfo* Element = new(Elements) FElementInfo(SourceElement, &VertexFactory, ElementIndex);
			MaterialRelevance |= Element->GetMaterial()->GetRelevance(GetScene().GetShaderPlatform());
		}

		bGoodCandidateForCachedShadowmap = CacheShadowDepthsFromPrimitivesUsingWPO() || (!MaterialRelevance.bUsesWorldPositionOffset && !MaterialRelevance.bUsesDisplacement);
	}

	~FModelSceneProxy()
	{
		VertexFactory.ReleaseResource();
	}

	bool IsCollisionView(const FSceneView* View, bool & bDrawCollision) const
	{
		const bool bInCollisionView = View->Family->EngineShowFlags.CollisionVisibility || View->Family->EngineShowFlags.CollisionPawn;
		if (bInCollisionView)
		{
			// use wireframe if collision is enabled, and it's not using complex 
			bDrawCollision = View->Family->EngineShowFlags.CollisionPawn && IsCollisionEnabled() && (CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore);
			bDrawCollision |= View->Family->EngineShowFlags.CollisionVisibility && IsCollisionEnabled() && (CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore);
		}
		else
		{
			bDrawCollision = false;
		}

		return bInCollisionView;
	}

	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent*,TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override
	{
		HHitProxy* ModelHitProxy = new HModel(Component, Component->GetModel());
		OutHitProxies.Add(ModelHitProxy);
		return ModelHitProxy;
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FModelSceneProxy_GetMeshElements);
		bool bAnySelectedSurfs = false;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];

				// View families can vary per view, with bSinglePassGDME == true being set for this class, so get the specific family for the view
				const FSceneViewFamily& ViewSpecificFamily = *View->Family;

				bool bShowSelection = GIsEditor && !View->bIsGameView && ViewSpecificFamily.EngineShowFlags.Selection;
				bool bDynamicBSPTriangles = bShowSelection || IsRichView(ViewSpecificFamily);
				bool bShowBSPTriangles = ViewSpecificFamily.EngineShowFlags.BSPTriangles;
				bool bShowBSP = ViewSpecificFamily.EngineShowFlags.BSP;

#if WITH_EDITOR
				bool bDrawCollision = false;
				const bool bInCollisionView = IsCollisionView(View, bDrawCollision);
				// draw bsp as dynamic when in collision view mode
				if(bInCollisionView)
				{
					bDynamicBSPTriangles = true;
				}
#endif

				// If in a collision view, only draw if we have collision
				if (bDynamicBSPTriangles && bShowBSPTriangles && bShowBSP 
#if WITH_EDITOR
					&& (!bInCollisionView || bDrawCollision)
#endif
					)
				{
					ESceneDepthPriorityGroup DepthPriorityGroup = (ESceneDepthPriorityGroup)GetDepthPriorityGroup(View);

					const FMaterialRenderProxy* MatProxyOverride = nullptr;

#if WITH_EDITOR
					if(bInCollisionView && AllowDebugViewmodes())
					{
						MatProxyOverride = &CollisionMaterialInstance;
					}
#endif

					// If selection is being shown, batch triangles based on whether they are selected or not.
					if (bShowSelection)
					{
						uint32 TotalIndices = 0;

						for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
						{
							const FModelElement& ModelElement = Component->GetElements()[ElementIndex];
							TotalIndices += ModelElement.NumTriangles * 3;
						}

						if (TotalIndices > 0)
						{
							FGlobalDynamicIndexBuffer& DynamicIndexBuffer = Collector.GetDynamicIndexBuffer();
							FGlobalDynamicIndexBuffer::FAllocation IndexAllocation = DynamicIndexBuffer.Allocate<uint32>(TotalIndices);

							if (IndexAllocation.IsValid())
							{
								uint32* Indices = (uint32*)IndexAllocation.Buffer;
								uint32 FirstIndex = IndexAllocation.FirstIndex;

								for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
								{
									const FModelElement& ModelElement = Component->GetElements()[ElementIndex];

									if (ModelElement.NumTriangles > 0)
									{
										const FElementInfo& ProxyElementInfo = Elements[ElementIndex];
										bool bHasSelectedSurfs = false;
										bool bHasHoveredSurfs = false;

										for (uint32 BatchIndex = 0; BatchIndex < 3; BatchIndex++)
										{
											// Three batches total:
											//		Batch 0: Only surfaces that are neither selected, nor hovered
											//		Batch 1: Only selected surfaces
											//		Batch 2: Only hovered surfaces
											const bool bOnlySelectedSurfaces = ( BatchIndex == 1 );
											const bool bOnlyHoveredSurfaces = ( BatchIndex == 2 );

											if (bOnlySelectedSurfaces && !bHasSelectedSurfs)
											{
												continue;
											}

											if (bOnlyHoveredSurfaces && !bHasHoveredSurfs)
											{
												continue;
											}

											uint32 MinVertexIndex = MAX_uint32;
											uint32 MaxVertexIndex = 0;
											uint32 NumIndices = 0;

											for(int32 NodeIndex = 0;NodeIndex < ModelElement.Nodes.Num();NodeIndex++)
											{
												UModel* ComponentModel = Component->GetModel();
												if (ensureMsgf(ComponentModel->Nodes.IsValidIndex(ModelElement.Nodes[NodeIndex]), TEXT("Invalid Node Index, Idx:%d, Num:%d"), ModelElement.Nodes[NodeIndex], ComponentModel->Nodes.Num()))
												{
													FBspNode& Node = ComponentModel->Nodes[ModelElement.Nodes[NodeIndex]];

													if (ensureMsgf(ComponentModel->Surfs.IsValidIndex(Node.iSurf), TEXT("Invalid Surf Index, Idx:%d, Num:%d"), Node.iSurf, ComponentModel->Surfs.Num()))
													{
														FBspSurf& Surf = ComponentModel->Surfs[Node.iSurf];

														if (!ShouldDrawSurface(Surf))
														{
															continue;
														}

														const bool bSurfaceSelected = (Surf.PolyFlags & PF_Selected) == PF_Selected;
														const bool bSurfaceHovered = !bSurfaceSelected && ((Surf.PolyFlags & PF_Hovered) == PF_Hovered);
														bHasSelectedSurfs |= bSurfaceSelected;
														bHasHoveredSurfs |= bSurfaceHovered;

														if (bSurfaceSelected == bOnlySelectedSurfaces && bSurfaceHovered == bOnlyHoveredSurfaces)
														{
															for (uint32 BackFace = 0; BackFace < (uint32)((Surf.PolyFlags & PF_TwoSided) ? 2 : 1); BackFace++)
															{
																for (int32 VertexIndex = 2; VertexIndex < Node.NumVertices; VertexIndex++)
																{
																	*Indices++ = Node.iVertexIndex + Node.NumVertices * BackFace;
																	*Indices++ = Node.iVertexIndex + Node.NumVertices * BackFace + VertexIndex;
																	*Indices++ = Node.iVertexIndex + Node.NumVertices * BackFace + VertexIndex - 1;
																	NumIndices += 3;
																}
																MinVertexIndex = FMath::Min(Node.iVertexIndex + Node.NumVertices * BackFace,MinVertexIndex);
																MaxVertexIndex = FMath::Max(Node.iVertexIndex + Node.NumVertices * BackFace + Node.NumVertices - 1,MaxVertexIndex);
															}
														}
													}
												}
											}

											if (NumIndices > 0)
											{
												FMeshBatch& MeshElement = Collector.AllocateMesh();
												FMeshBatchElement& BatchElement = MeshElement.Elements[0];
												BatchElement.IndexBuffer = IndexAllocation.IndexBuffer;
												MeshElement.VertexFactory = &VertexFactory;
												MeshElement.MaterialRenderProxy = (MatProxyOverride != nullptr) ? MatProxyOverride : ProxyElementInfo.GetMaterial()->GetRenderProxy();
												MeshElement.LCI = &ProxyElementInfo;
												BatchElement.FirstIndex = FirstIndex;
												BatchElement.NumPrimitives = NumIndices / 3;
												BatchElement.MinVertexIndex = MinVertexIndex;
												BatchElement.MaxVertexIndex = MaxVertexIndex;
												BatchElement.VertexFactoryUserData = ProxyElementInfo.GetVertexFactoryUniformBuffer();
												MeshElement.Type = PT_TriangleList;
												MeshElement.DepthPriorityGroup = DepthPriorityGroup;
												MeshElement.bCanApplyViewModeOverrides = true;
												MeshElement.bUseWireframeSelectionColoring = false;
												MeshElement.bUseSelectionOutline = bOnlySelectedSurfaces;
												MeshElement.LODIndex = 0;
												Collector.AddMesh(ViewIndex, MeshElement);
												FirstIndex += NumIndices;
											}
										}

										bAnySelectedSurfs |= bHasSelectedSurfs;
									}
								}
							}
						}
					}
					else
					{
						for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
						{
							const FModelElement& ModelElement = Component->GetElements()[ElementIndex];

							if (ModelElement.NumTriangles > 0)
							{
								FMeshBatch& MeshElement = Collector.AllocateMesh();
								FMeshBatchElement& BatchElement = MeshElement.Elements[0];
								BatchElement.IndexBuffer = ModelElement.IndexBuffer;
								MeshElement.VertexFactory = &VertexFactory;
								MeshElement.MaterialRenderProxy = (MatProxyOverride != nullptr) ? MatProxyOverride : Elements[ElementIndex].GetMaterial()->GetRenderProxy();
								MeshElement.LCI = &Elements[ElementIndex];
								BatchElement.FirstIndex = ModelElement.FirstIndex;
								BatchElement.NumPrimitives = ModelElement.NumTriangles;
								BatchElement.MinVertexIndex = ModelElement.MinVertexIndex;
								BatchElement.MaxVertexIndex = ModelElement.MaxVertexIndex;
								BatchElement.VertexFactoryUserData = Elements[ElementIndex].GetVertexFactoryUniformBuffer();
								MeshElement.Type = PT_TriangleList;
								MeshElement.DepthPriorityGroup = DepthPriorityGroup;
								MeshElement.bCanApplyViewModeOverrides = true;
								MeshElement.bUseWireframeSelectionColoring = false;
								MeshElement.LODIndex = 0;
								Collector.AddMesh(ViewIndex, MeshElement);
							}
						}
					}
				}
			}
		}

		//@todo parallelrendering - remove this legacy state modification
		// Poly selected state is modified in many places, so it's hard to push the selection state to the proxy
		const_cast<FModelSceneProxy*>(this)->SetSelection_RenderThread(bAnySelectedSurfs,false);
	}

	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override
	{
		if (!HasViewDependentDPG())
		{
			// Determine the DPG the primitive should be drawn in.
			ESceneDepthPriorityGroup PrimitiveDPG = GetStaticDepthPriorityGroup();

			PDI->ReserveMemoryForMeshes(Elements.Num());

			for (int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
			{
				const FModelElement& ModelElement = Component->GetElements()[ElementIndex];
				if (ModelElement.NumTriangles > 0)
				{
					FMeshBatch MeshElement;
					FMeshBatchElement& BatchElement = MeshElement.Elements[0];
					BatchElement.IndexBuffer = ModelElement.IndexBuffer;
					MeshElement.VertexFactory = &VertexFactory;
					MeshElement.MaterialRenderProxy = Elements[ElementIndex].GetMaterial()->GetRenderProxy();
					MeshElement.LCI = &Elements[ElementIndex];
					BatchElement.FirstIndex = ModelElement.FirstIndex;
					BatchElement.NumPrimitives = ModelElement.NumTriangles;
					BatchElement.MinVertexIndex = ModelElement.MinVertexIndex;
					BatchElement.MaxVertexIndex = ModelElement.MaxVertexIndex;
					BatchElement.VertexFactoryUserData = Elements[ElementIndex].GetVertexFactoryUniformBuffer();
					MeshElement.Type = PT_TriangleList;
					MeshElement.DepthPriorityGroup = (uint8)PrimitiveDPG;
					MeshElement.LODIndex = 0;
					const bool bValidIndexBuffer = !BatchElement.IndexBuffer || (BatchElement.IndexBuffer && BatchElement.IndexBuffer->IsInitialized() && BatchElement.IndexBuffer->IndexBufferRHI);
					ensure(bValidIndexBuffer);
					if (bValidIndexBuffer)
					{
						PDI->DrawMesh(MeshElement, FLT_MAX);
					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.BSPTriangles && View->Family->EngineShowFlags.BSP;
		bool bShowSelectedTriangles = GIsEditor && !View->bIsGameView && View->Family->EngineShowFlags.Selection;
		bool bCollisionView = View->Family->EngineShowFlags.CollisionPawn || View->Family->EngineShowFlags.CollisionVisibility;
		if (IsRichView(*View->Family) || HasViewDependentDPG() || bCollisionView
			|| (bShowSelectedTriangles && HasSelectedSurfaces()))
		{
			Result.bDynamicRelevance = true;
		}
		else
		{
			Result.bStaticRelevance = true;
		}
		Result.bShadowRelevance = IsShadowCast(View);
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = (VelocityIncludeStationaryPrimitives(View->GetShaderPlatform()) || DrawsVelocity()) && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override
	{
		// Attach the light to the primitive's static meshes.
		bDynamic = true;
		bRelevant = false;
		bLightMapped = true;
		bShadowMapped = true;

		if (Elements.Num() > 0)
		{
			for (int32 ElementIndex = 0;ElementIndex < Elements.Num();ElementIndex++)
			{
				const FElementInfo& LCI = Elements[ElementIndex];
				ELightInteractionType InteractionType = LCI.GetInteraction(LightSceneProxy).GetType();
				if (InteractionType != LIT_CachedIrrelevant)
				{
					bRelevant = true;
					if (InteractionType != LIT_CachedLightMap)
					{
						bLightMapped = false;
					}
					if (InteractionType != LIT_Dynamic)
					{
						bDynamic = false;
					}
				}
			}
		}
		else
		{
			bRelevant = true;
			bLightMapped = false;
		}
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const 
	{ 
		uint32 AdditionalSize = FPrimitiveSceneProxy::GetAllocatedSize();

		AdditionalSize += Elements.GetAllocatedSize();

		return( AdditionalSize ); 
	}

	virtual bool ShowInBSPSplitViewmode() const override
	{
		return true;
	}

private:

	/** Returns true if any surfaces relevant to this component are selected (or hovered). */
	bool HasSelectedSurfaces() const
	{
#if WITH_EDITOR
		if (!ensureMsgf(ABrush::GGeometryRebuildCause == nullptr, TEXT("Attempting to render brushes while they are being updated. Cause: %s"), ABrush::GGeometryRebuildCause))
		{
			return false;
		}
#endif

		UModel* Model = Component->GetModel();

		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			const FModelElement& ModelElement = Component->GetElements()[ElementIndex];
			if (ModelElement.NumTriangles > 0)
			{
				for(int32 NodeIndex = 0;NodeIndex < ModelElement.Nodes.Num();NodeIndex++)
				{
					uint16 ModelNodeIndex = ModelElement.Nodes[NodeIndex];
					// Ensures for debug purposes only; an attempt to catch the cause of UE-36265.
					// Please remove again ASAP as these extra checks can't be fast
					if (ensureMsgf(Model->Nodes.IsValidIndex(ModelNodeIndex), TEXT( "Invalid Node Index, Idx:%d, Num:%d" ), ModelNodeIndex, Model->Nodes.Num() ) )
					{
						FBspNode& Node = Model->Nodes[ModelNodeIndex];
						if (ensureMsgf(Model->Surfs.IsValidIndex(Node.iSurf), TEXT( "Invalid Surf Index, Idx:%d, Num:%d" ), Node.iSurf, Model->Surfs.Num() ) )
						{
							FBspSurf& Surf = Model->Surfs[Node.iSurf];

							if (ShouldDrawSurface(Surf) && (Surf.PolyFlags & (PF_Selected | PF_Hovered)) != 0)
							{
								return true;
							}
						}
					}
				}
			}
		}
		return false;
	}

	UModelComponent* Component;

	class FElementInfo : public FLightCacheInterface
	{
	public:

		/** Initialization constructor. */
		FElementInfo(const FModelElement& InModelElement, const FLocalVertexFactory* InVertexFactory, int32 InElementIndex)
			: FLightCacheInterface()
			, Bounds(InModelElement.BoundingBox)
		{
			const FMeshMapBuildData* MapBuildData = InModelElement.GetMeshMapBuildData();

			if (MapBuildData)
			{
				SetLightMap(MapBuildData->LightMap);
				SetShadowMap(MapBuildData->ShadowMap);
				SetResourceCluster(MapBuildData->ResourceCluster);
				IrrelevantLights = MapBuildData->IrrelevantLights;
			}

			const bool bHasStaticLighting = GetLightMap() != nullptr || GetShadowMap() != nullptr;

			// Determine the material applied to the model element.
			Material = InModelElement.Material;

			// If there isn't an applied material, or if we need static lighting and it doesn't support it, fall back to the default material.
			if(!Material || (bHasStaticLighting && !Material->CheckMaterialUsage(MATUSAGE_StaticLighting)))
			{
				Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}

			{
				TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters>* UniformBufferPtr = &VertexFactoryUniformBuffer;

				ENQUEUE_RENDER_COMMAND(FElementInfoCreateLocalVFUniformBuffer)(
					[UniformBufferPtr, VertexFactory = InVertexFactory, ElementIndex = InElementIndex](FRHICommandListImmediate& RHICmdList)
				{
					*UniformBufferPtr = CreateLocalVFUniformBuffer(VertexFactory, ElementIndex, nullptr, 0, 0);
				});
			}
		}
		
		// FLightCacheInterface.
		virtual FLightInteraction GetInteraction(const FLightSceneProxy* LightSceneProxy) const override
		{
			ELightInteractionType LightInteraction = GetStaticInteraction(LightSceneProxy, IrrelevantLights);
		
			if(LightInteraction != LIT_MAX)
			{
				return FLightInteraction(LightInteraction);
			}

			// Cull the uncached light against the bounding box of the element.
			if(	LightSceneProxy->AffectsBounds(Bounds) )
			{
				return FLightInteraction::Dynamic();
			}
			else
			{
				return FLightInteraction::Irrelevant();
			}
		}

		// Accessors.
		UMaterialInterface* GetMaterial() const { return Material; }

		FRHIUniformBuffer* GetVertexFactoryUniformBuffer() const { return VertexFactoryUniformBuffer; }

	private:

		/** The element's material. */
		UMaterialInterface* Material;

		/** Vertex factory uniform buffer. Needs to be per model element, so every element can have it's unique LODLightmapDataIndex. */
		TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters> VertexFactoryUniformBuffer;

		/** The statically irrelevant lights for this element. */
		TArray<FGuid> IrrelevantLights;

		/** The element's bounding volume. */
		FBoxSphereBounds Bounds;
	};

	TArray<FElementInfo> Elements;

	FMaterialRelevance MaterialRelevance;

	/** Precomputed dynamic mesh batches. */
	struct FDynamicModelMeshBatch : public FMeshBatch
	{
		int32 ModelElementIndex;
		/** true if the batch is selected (we need to override the material)*/
		bool bIsSelectedBatch;
		FDynamicModelMeshBatch( bool bInIsSelectedBatch )
			: FMeshBatch()
			, ModelElementIndex(0)
			, bIsSelectedBatch( bInIsSelectedBatch )
		{
		}
	};

	/** Collision Response of this component**/
	FCollisionResponseContainer CollisionResponse;
#if WITH_EDITOR
	/** Material proxy to use when rendering in a collision view mode */
	FColoredMaterialRenderProxy CollisionMaterialInstance;
#endif
public:
	// Helper functions for LightMap Density view mode
	/**
	 *	Get the number of entires in the Elements array.
	 *
	 *	@return	int32		The number of entries in the array.
	 */
	int32 GetElementCount() const { return Elements.Num(); }

	/**
	 *	Get the element info at the given index.
	 *
	 *	@param	Index			The index of interest
	 *
	 *	@return	FElementInfo*	The element info at that index.
	 *							nullptr if out of range.
	 */
	const FElementInfo* GetElement(int32 Index) const 
	{
		if (Index < Elements.Num())
		{
			return &(Elements[Index]);
		}
		return nullptr;
	}

	virtual void GetLCIs(FLCIArray& LCIs) override
	{
		LCIs.Reserve(Elements.Num());
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
		{
			FLightCacheInterface* LCI = &Elements[ElementIndex];
			LCIs.Push(LCI);
		}
	}
	friend class UModelComponent;
};

void UModelComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		FModelElement& Element = Elements[ElementIndex];
		TUniquePtr<FRawIndexBuffer16or32>* IndexBufferRef = Model->MaterialIndexBuffers.Find(Element.Material);
		Element.IndexBuffer = IndexBufferRef ? IndexBufferRef->Get() : nullptr;
	}

	if (GetModel())
	{
		++GetModel()->VertexBuffer.RefCount;
	}

	Super::CreateRenderState_Concurrent(Context);
}

void UModelComponent::DestroyRenderState_Concurrent()
{
	if (GetModel())
	{
		--GetModel()->VertexBuffer.RefCount;
	}

	Super::DestroyRenderState_Concurrent();
}

FPrimitiveSceneProxy* UModelComponent::CreateSceneProxy()
{
	FPrimitiveSceneProxy* Proxy = ::new FModelSceneProxy(this);
	return Proxy;
}

bool UModelComponent::ShouldRecreateProxyOnUpdateTransform() const
{
	return true;
}

FBoxSphereBounds UModelComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if(Model)
	{
		FBox	BoundingBox(ForceInit);
		for(int32 NodeIndex = 0;NodeIndex < Nodes.Num();NodeIndex++)
		{
			FBspNode& Node = Model->Nodes[Nodes[NodeIndex]];
			for(int32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
			{
				BoundingBox += (FVector)Model->Points[Model->Verts[Node.iVertPool + VertexIndex].pVertex];
			}
		}
		return FBoxSphereBounds(BoundingBox.TransformBy(LocalToWorld));
	}
	else
	{
		return FBoxSphereBounds(LocalToWorld.GetLocation(), FVector::ZeroVector, 0.f);
	}
}
