// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MorphTarget.h"

#include "Engine/SkeletalMesh.h"
#include "EngineUtils.h"
#include "Algo/AnyOf.h"
#include "Interfaces/ITargetPlatform.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Serialization/MemoryArchive.h"
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"

//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(MorphTarget)

FArchive& operator<<(FArchive& Ar, FMorphTargetLODModel& M)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);

	if (!Ar.IsObjectReferenceCollector())
	{
		if (Ar.IsLoading() && Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::AddedMorphTargetSectionIndices)
		{
			Ar << M.Vertices << M.NumBaseMeshVerts;
			M.bGeneratedByEngine = false;
		}
		else if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SaveGeneratedMorphTargetByEngine)
		{
			Ar << M.Vertices << M.NumBaseMeshVerts << M.SectionIndices;
			M.bGeneratedByEngine = false;
		}
		else
		{
			bool bVerticesAreStrippedForCookedBuilds = false;

			if (Ar.IsPersistent() && (Ar.CustomVer(FUE5SpecialProjectStreamObjectVersion::GUID) >= FUE5SpecialProjectStreamObjectVersion::StripMorphTargetSourceDataForCookedBuilds))
			{
				// Strip raw vertex data for cooked builds, but only if we're cooking to disk. 
				bVerticesAreStrippedForCookedBuilds = Ar.IsCooking();
				Ar << bVerticesAreStrippedForCookedBuilds;
			}

			if (bVerticesAreStrippedForCookedBuilds)
			{
				M.NumVertices = M.Vertices.Num();
				Ar << M.NumVertices;
			}
			else
			{
				Ar << M.Vertices;

				if (Ar.IsLoading())
				{
					M.NumVertices = M.Vertices.Num();
				}
			}

			Ar << M.NumBaseMeshVerts << M.SectionIndices << M.bGeneratedByEngine;
		}

		if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MorphTargetCustomImport)
		{
			M.SourceFilename.Empty();
		}
		else
		{
			//Do not cook the source filename, we don't need it at runtime
			if (Ar.IsCooking() && Ar.IsSaving())
			{
				FString EmptySourceFilename;
				Ar << EmptySourceFilename;
			}
			else
			{
				Ar << M.SourceFilename;
			}
		}

		/* Fix the data for the generated by engine flag */
		if(Ar.IsLoading() && M.SourceFilename.StartsWith(FMorphTargetImportedSourceFileInfo::GeneratedByEnginePrefix))
		{
			M.bGeneratedByEngine = true;
			M.SourceFilename.Empty();
		}
	}

	return Ar;
}


//////////////////////////////////////////////////////////////////////////


UMorphTarget::UMorphTarget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UMorphTarget::Serialize( FArchive& Ar )
{
	LLM_SCOPE(ELLMTag::Animation);
	
	Super::Serialize( Ar );
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	FStripDataFlags StripFlags( Ar );
	if( StripFlags.IsAudioVisualDataStripped() )
	{
		return;
	}

	bool bCooked = Ar.IsCooking() && Ar.IsPersistent();
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MorphTargetCookedCPUDataCompressed)
	{
		bCooked = false;
	}
	else
	{
		Ar << bCooked;
	}
	
	// We always store the morph models (save for the raw vertex data when cooking).  
	Ar << MorphLODModels;
	
	if (bCooked)
	{
		// Only platforms that support ES31 rendering need compressed vertex data on the CPU side. Non-ES31 platforms operate on compressed GPU
		// data which is cooked out elsewhere.
		bool bStoreCompressedVertices = Ar.CookingTarget() ? Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::MobileRendering) : false;
		
		Ar << bStoreCompressedVertices;
		
		if (bStoreCompressedVertices)
		{
#if WITH_EDITOR
			if (Ar.IsCooking() && Ar.IsPersistent())
			{
				int32 NumLODs = MorphLODModels.Num();

				static const FSkeletalMeshLODInfo DefaultLODInfo{};
				
				float TangentPrecision = UE::MorphTargetVertexCodec::ComputeTangentPrecision();

				// BaseSkelMesh is technically redundant, since it should just be the outer. To play it safe, we always use it first
				// and just default to the outer when it is null (which can happen).
				USkeletalMesh* SkeletalMesh = BaseSkelMesh;
				if (!SkeletalMesh)
				{
					SkeletalMesh = Cast<USkeletalMesh>(GetOuter());
				}

				Ar << NumLODs;

				for (int32 LODIndex = 0; LODIndex < NumLODs; LODIndex++)
				{
					const FMorphTargetLODModel& LODModel = MorphLODModels[LODIndex];
					
					TArray<UE::MorphTargetVertexCodec::FDeltaBatchHeader> PackedDeltaHeaders;
					TArray<uint32> PackedDeltaData;
					const FSkeletalMeshLODInfo* LODInfo = nullptr;
					if (SkeletalMesh)
					{
						LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
					}
					float PositionPrecision = UE::MorphTargetVertexCodec::ComputePositionPrecision(
						LODInfo ? LODInfo->MorphTargetPositionErrorTolerance : DefaultLODInfo.MorphTargetPositionErrorTolerance);

					UE::MorphTargetVertexCodec::Encode(LODModel.Vertices, nullptr,
						PositionPrecision, TangentPrecision, PackedDeltaHeaders, PackedDeltaData);

					Ar << PackedDeltaHeaders;
					Ar << PackedDeltaData;
					Ar << PositionPrecision;
					Ar << TangentPrecision;
				}
			}
			else
#endif
			{
				int32 NumLODs = 0;
				Ar << NumLODs;

#if !WITH_EDITOR
				CompressedLODModels.Reset(NumLODs);
#endif
			
				for (int32 LODIndex = 0; LODIndex < NumLODs; LODIndex++)
				{
					TArray<UE::MorphTargetVertexCodec::FDeltaBatchHeader> PackedDeltaHeaders;
					TArray<uint32> PackedDeltaData;
					float PositionPrecision = 0.0f;
					float TangentPrecision = 0.0f;

					Ar << PackedDeltaHeaders;
					Ar << PackedDeltaData;
					Ar << PositionPrecision;
					Ar << TangentPrecision;

#if WITH_EDITOR
					FMorphTargetLODModel& MorphModel = MorphLODModels[LODIndex];
					// Unpack the compressed data into the raw vertex data arrays.
					UE::MorphTargetVertexCodec::Decode(PackedDeltaHeaders, PackedDeltaData, PositionPrecision, TangentPrecision, MorphModel.Vertices);
#else
					FMorphTargetCompressedLODModel& CompressedModel = CompressedLODModels.AddDefaulted_GetRef();
					CompressedModel.PackedDeltaHeaders = MoveTemp(PackedDeltaHeaders);
					CompressedModel.PackedDeltaData = MoveTemp(PackedDeltaData);
					CompressedModel.PositionPrecision = PositionPrecision;
					CompressedModel.TangentPrecision = TangentPrecision;
#endif
				}
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void UMorphTarget::DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass)
{
	Super::DeclareCustomVersions(Ar, SpecificSubclass);
	FMorphTargetLODModel MorphLODModel;
	Ar << MorphLODModel;
}
#endif

namespace
{
	void SerializeMorphLODModels(FMemoryArchive& Ar, TArray<FMorphTargetLODModel>& MorphLODModels)
	{
		int32 MorphLODModelNumber = 0;
		if (Ar.IsLoading())
		{
			Ar << MorphLODModelNumber;
			MorphLODModels.Empty(MorphLODModelNumber);
			MorphLODModels.AddDefaulted(MorphLODModelNumber);
		}
		else
		{
			MorphLODModelNumber = MorphLODModels.Num();
			Ar << MorphLODModelNumber;
		}

		for (int32 MorphIndex = 0; MorphIndex < MorphLODModelNumber; ++MorphIndex)
		{
			Ar << MorphLODModels[MorphIndex];
		}
	}
}

void UMorphTarget::SerializeMemoryArchive(FMemoryArchive & Ar)
{
	FName MorphTargetName = GetFName();
	Ar << MorphTargetName;
	SerializeMorphLODModels(Ar, MorphLODModels);
}

#if WITH_EDITOR
void FFinishBuildMorphTargetData::LoadFromMemoryArchive(FMemoryArchive & Ar)
{
	check(Ar.IsLoading());
	
	if (!ensureMsgf(!bApplyMorphTargetsData, TEXT("Error in FFinishBuildMorphTargetData::LoadFromMemoryArchive. The compilation context morph targets data was already set.")))
	{
		MorphLODModelsPerTargetName.Empty();
		bApplyMorphTargetsData = false;
	}
	bApplyMorphTargetsData = true;
	
	int32 MorphTargetNumber = 0;
	Ar << MorphTargetNumber;
	MorphLODModelsPerTargetName.Reserve(MorphTargetNumber);
	for (int32 MorphTargetIndex = 0; MorphTargetIndex < MorphTargetNumber; ++MorphTargetIndex)
	{
		FName MorphTargetName = NAME_None;
		Ar << MorphTargetName;
		TArray<FMorphTargetLODModel>&MorphLODModels = MorphLODModelsPerTargetName.FindOrAdd(MorphTargetName);
		SerializeMorphLODModels(Ar, MorphLODModels);
	}
}
#endif

void UMorphTarget::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITOR
	if (GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::AddedMorphTargetSectionIndices &&
		BaseSkelMesh)
	{
		const int32 MaxLOD = FMath::Min(BaseSkelMesh->GetImportedModel()->LODModels.Num(), MorphLODModels.Num());
		for (int32 LODIndex = 0; LODIndex < MaxLOD; ++LODIndex)
		{
			FMorphTargetLODModel& MorphLODModel = MorphLODModels[LODIndex];
			MorphLODModel.SectionIndices.Empty();
			const FSkeletalMeshLODModel& LODModel = BaseSkelMesh->GetImportedModel()->LODModels[LODIndex];
			TArray<int32> BaseIndexes;
			TArray<int32> LastIndexes;
			for (int32 SectionIdx = 0; SectionIdx < LODModel.Sections.Num(); ++SectionIdx)
			{
				const int32 BaseVertexBufferIndex = LODModel.Sections[SectionIdx].GetVertexBufferIndex();
				BaseIndexes.Add(BaseVertexBufferIndex);
				LastIndexes.Add(BaseVertexBufferIndex + LODModel.Sections[SectionIdx].GetNumVertices());
			}
			// brute force
			for (int32 VertIndex = 0; VertIndex < MorphLODModel.Vertices.Num() && MorphLODModel.SectionIndices.Num() < BaseIndexes.Num(); ++VertIndex)
			{
				int32 SourceVertexIdx = MorphLODModel.Vertices[VertIndex].SourceIdx;
				for (int32 SectionIdx = 0; SectionIdx < BaseIndexes.Num(); ++SectionIdx)
				{
					if (!MorphLODModel.SectionIndices.Contains(SectionIdx))
					{
						if (BaseIndexes[SectionIdx] <= SourceVertexIdx && SourceVertexIdx < LastIndexes[SectionIdx])
						{
							MorphLODModel.SectionIndices.AddUnique(SectionIdx);
							break;
						}
					}
				}
			}
		}
	}
#endif //#if WITH_EDITOR
}

#if WITH_EDITOR
const FMorphTargetDelta* UMorphTarget::GetMorphTargetDelta(int32 LODIndex, int32& OutNumDeltas) const
{
	if(GetMorphLODModels().IsValidIndex(LODIndex))
	{
		// Calling GetMorphLODModels to potentially get from subclasses
		const FMorphTargetLODModel& MorphModel = GetMorphLODModels()[LODIndex];
		OutNumDeltas = MorphModel.Vertices.Num();
		return MorphModel.Vertices.GetData();
	}

	OutNumDeltas = 0;
	return nullptr;
}

TConstArrayView<FMorphTargetDelta> UMorphTarget::GetMorphTargetDeltas(int32 LODIndex) const
{
	if (GetMorphLODModels().IsValidIndex(LODIndex))
	{
		return GetMorphLODModels()[LODIndex].Vertices;
	}

	return {};
}
#endif


FMorphTargetDeltaIterator UMorphTarget::GetDeltaIteratorForLOD(int32 LODIndex) const
{
	if (GetMorphLODModels().IsValidIndex(LODIndex))
	{
#if !WITH_EDITOR
		if (CompressedLODModels.IsValidIndex(LODIndex))
		{
			return FMorphTargetDeltaIterator(CompressedLODModels[LODIndex]);
		}
#endif
		return FMorphTargetDeltaIterator(GetMorphLODModels()[LODIndex].Vertices);
	}

	return {};
}

int32 UMorphTarget::GetNumDeltasForLOD(int32 LODIndex) const
{
	if (GetMorphLODModels().IsValidIndex(LODIndex))
	{
#if WITH_EDITOR
		return GetMorphLODModels()[LODIndex].Vertices.Num();
#else
		return GetMorphLODModels()[LODIndex].NumVertices;
#endif
	}

	return 0;
}

bool UMorphTarget::HasDataForLOD(int32 LODIndex) const
{
	return GetNumDeltasForLOD(LODIndex) != 0;
}

bool UMorphTarget::HasValidData() const
{
	for (const FMorphTargetLODModel& Model : MorphLODModels)
	{
		if (Model.Vertices.Num() > 0)
		{
			return true;
		}
#if !WITH_EDITOR
		// In cooked builds, Model.Vertices is stripped but Model.NumVertices is valid
		else if (Model.NumVertices > 0)
		{
			return true;
		}
#endif
	}

	return false;
}

bool UMorphTarget::HasDataForSection(int32 LODIndex, int32 SectionIndex) const
{
	return HasDataForLOD(LODIndex) && MorphLODModels[LODIndex].SectionIndices.Contains(SectionIndex);
}

void UMorphTarget::EmptyMorphLODModels()
{
	MorphLODModels.Empty();
}

bool UMorphTarget::IsCustomImported(int32 LODIndex) const
{
	if (LODIndex < MorphLODModels.Num())
	{
		// Calling GetMorphLODModels to potentially get from subclasses
		const FMorphTargetLODModel& MorphModel = GetMorphLODModels()[LODIndex];
		return !MorphModel.SourceFilename.IsEmpty();
	}
	return false;
}

const FString& UMorphTarget::GetCustomImportedSourceFilename(int32 LODIndex) const
{
	if (LODIndex < MorphLODModels.Num())
	{
		// Calling GetMorphLODModels to potentially get from subclasses
		const FMorphTargetLODModel& MorphModel = GetMorphLODModels()[LODIndex];
		return MorphModel.SourceFilename;
	}
	static const FString EmptyString;
	return EmptyString;
}

void UMorphTarget::SetCustomImportedSourceFilename(int32 LODIndex, const FString& InSourceFilename)
{
	if (LODIndex < MorphLODModels.Num())
	{
		// Calling GetMorphLODModels to potentially get from subclasses
		FMorphTargetLODModel& MorphModel = GetMorphLODModels()[LODIndex];
		MorphModel.SourceFilename = InSourceFilename;
		MorphModel.bGeneratedByEngine = false;
	}
}

bool UMorphTarget::IsGeneratedByEngine(int32 LODIndex) const
{
	if (LODIndex < MorphLODModels.Num())
	{
		// Calling GetMorphLODModels to potentially get from subclasses
		const FMorphTargetLODModel& MorphModel = GetMorphLODModels()[LODIndex];
		if(MorphModel.SourceFilename.IsEmpty())
		{
			return MorphModel.bGeneratedByEngine;
		}
	}
	return false;
}

void UMorphTarget::SetGeneratedByEngine(int32 LODIndex, bool bInGeneratedByEngine)
{
	if (LODIndex < MorphLODModels.Num())
	{
		// Calling GetMorphLODModels to potentially get from subclasses
		FMorphTargetLODModel& MorphModel = GetMorphLODModels()[LODIndex];
		MorphModel.SourceFilename.Empty();
		MorphModel.bGeneratedByEngine = bInGeneratedByEngine;
	}
}

#if WITH_EDITOR

void UMorphTarget::PopulateDeltas(
	const TArray<FMorphTargetDelta>& Deltas,
	const int32 LODIndex,
	const TArray<FSkelMeshSection>& Sections,
	const bool bCompareNormal,
	const bool bGeneratedByReductionSetting,
	const float InPositionThreshold
	)
{
	// create the LOD entry if it doesn't already exist
	if (LODIndex >= MorphLODModels.Num())
	{
		MorphLODModels.AddDefaulted(LODIndex - MorphLODModels.Num() + 1);
	}

	// morph mesh data to modify
	FMorphTargetLODModel& MorphModel = MorphLODModels[LODIndex];
	// copy the wedge point indices
	// for now just keep every thing 

	// set the original number of vertices
	MorphModel.NumBaseMeshVerts = Deltas.Num();

	// empty morph mesh vertices first
	MorphModel.Vertices.Empty(Deltas.Num());

	// mark if generated by reduction setting, so that we can remove them later if we want to
	// we don't want to delete if it has been imported
	MorphModel.bGeneratedByEngine = bGeneratedByReductionSetting;

	// Still keep this (could remove in long term due to incoming data)
	const float PositionThresholdSquared = FMath::Square(InPositionThreshold);
	for (const FMorphTargetDelta& Delta : Deltas)
	{
		if (Delta.PositionDelta.SizeSquared() > PositionThresholdSquared ||
			( bCompareNormal && Delta.TangentZDelta.SizeSquared() > 0.01f))
		{
			MorphModel.Vertices.Add(Delta);
			for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); ++SectionIdx)
			{
				if (MorphModel.SectionIndices.Contains(SectionIdx))
				{
					continue;
				}
				const uint32 BaseVertexBufferIndex = static_cast<uint32>(Sections[SectionIdx].GetVertexBufferIndex());
				const uint32 LastVertexBufferIndex = BaseVertexBufferIndex + Sections[SectionIdx].GetNumVertices();
				if (BaseVertexBufferIndex <= Delta.SourceIdx && Delta.SourceIdx < LastVertexBufferIndex)
				{
					MorphModel.SectionIndices.AddUnique(SectionIdx);
					break;
				}
			}
		}
	}

	// sort the array of vertices for this morph target based on the base mesh indices
	// that each vertex is associated with. This allows us to sequentially traverse the list
	// when applying the morph blends to each vertex.
	MorphModel.Vertices.Sort(
		[](const FMorphTargetDelta& A, const FMorphTargetDelta& B)
		{
			return A.SourceIdx < B.SourceIdx;
		}
	);

	// remove array slack
	MorphModel.Vertices.Shrink();
	MorphModel.NumVertices = MorphModel.Vertices.Num();
}

void UMorphTarget::RemoveEmptyMorphTargets()
{
	// do not remove LOD 0
	for (int32 CurIndex = MorphLODModels.Num() - 1; CurIndex > 0; --CurIndex)
	{
		//  if it's empty, no reason to keep
		if (MorphLODModels[CurIndex].Vertices.IsEmpty())
		{
			MorphLODModels.RemoveAt(CurIndex);
		}
		else
		{
			// we leave empty ones between. We can't remove between ones. 
			// Once we found valid one, just get out
			break;
		}
	}
}

TUniquePtr<FFinishBuildMorphTargetData> UMorphTarget::CreateFinishBuildMorphTargetData() const
{
	return MakeUnique<FFinishBuildMorphTargetData>();
}

void FFinishBuildMorphTargetData::ApplyEditorData(USkeletalMesh * SkeletalMesh, bool bIsSerializeSaving) const
{
	//Return if we do not need to apply data
	if (!bApplyMorphTargetsData)
	{
		return;
	}

	if (SkeletalMesh->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
	{
		//No need to apply the morph targets if the asset is being deleted.
		//Also acquiring the GCScopeGuard when we are in the destruction process will create a deadlock, if there is code
		//in the BeginDestroy function of the skeletalmesh that touch any property that is lock by the async build.
		return;
	}

	//List of potential issue with this function
	// -This function change the array USkeletalMesh::MorphTargets, if a find reference call happen in same time for this asset(on the game thread)
	//  The find reference can crash because it is accessing this array to find UObject reference.
	//
	// -If this function is call in the pre garbage collector event (we do finish compilation in this event), the call to FindObjectSafe will acquire a GC lock
	//  which will cause a dead lock since the garbage collector already have the gc lock.
	//
	// TODO: remove the UMorphTarget object and replace the data in the import data of the skeletal mesh.
	// During async loading, GC is always inactive so concerns above do not apply. Also, objects aren't yet visible to the game thread so everything should be safe.
	if (!IsInGameThread() && !IsInAsyncLoadingThread() && !IsRunningCookCommandlet())
	{
		UE_LOG(LogSkeletalMesh, Log, TEXT("Calling function FFinishBuildMorphTargetData::ApplyEditorData outside of the game thread is not safe and can deadlock or crash."));
	}
	
	//GC should not be active during the build since we force finish the skinned asset compilation during the pre garbage delegate
	check(!IsGarbageCollectingAndLockingUObjectHashTables());

	FSkeletalMeshModel * SkelMeshModel = SkeletalMesh->GetImportedModel();
	check(SkelMeshModel);
	
	TMap<FName, UMorphTarget*> ExistingMorphTargets;
	for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
	{
		ExistingMorphTargets.Add(MorphTarget->GetFName(), MorphTarget);
	}
	
	TArray<UMorphTarget*> ToDeleteMorphTargets;
	ToDeleteMorphTargets.Append(SkeletalMesh->GetMorphTargets());
	SkeletalMesh->GetMorphTargets().Empty();
	//Rebuild the MorphTarget object
	for (const TPair<FName, TArray<FMorphTargetLODModel>>& TargetNameAndMorphLODModels : MorphLODModelsPerTargetName)
	{
		FName MorphTargetName = TargetNameAndMorphLODModels.Key;
		const TArray<FMorphTargetLODModel>& MorphTargetLODModels = TargetNameAndMorphLODModels.Value;
		int32 MorphLODModelNumber = MorphTargetLODModels.Num();

		UMorphTarget * MorphTarget = ExistingMorphTargets.FindRef(MorphTargetName);
		if (!MorphTarget)
		{
			if (!Algo::AnyOf(MorphTargetLODModels, [](const FMorphTargetLODModel& Model) { return !Model.Vertices.IsEmpty();}))
			{
				//Skip this empty morphtarget
				continue;
			}

			//Reuse the morph target if it already exist and was not garbage collect (it can happen if we play with the morph target threshold build options)
			MorphTarget = FindObjectSafe<UMorphTarget>(SkeletalMesh, *MorphTargetName.ToString(), EFindObjectFlags::ExactClass);
			if (MorphTarget && MorphTarget->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
			{
				MorphTarget = nullptr;
			}

			if (!MorphTarget)
			{
				//When we save the cook result we should never have to build a new morph target
				//When saving cook build, we call GetPlatformSkeletalMeshRenderData in USkeletalMesh::BeginCacheForCookedPlatformData
				//which happen before the serialization of that cook skeletalmesh
				if (!bIsSerializeSaving)
				{
					//Avoid recycling morphtarget with NewObject it cannot be done asynchronously
					//Find the UMorphTarget and simply clear the data if it exist.
					TArray<UObject*> SubObjects;
					GetObjectsWithOuter(SkeletalMesh, SubObjects, EGetObjectsFlags::IncludeNestedObjects);
					for (UObject* SubObject : SubObjects)
					{
						if (SubObject->GetFName() == MorphTargetName)
						{
							if (UMorphTarget* SubMorphTarget = Cast<UMorphTarget>(SubObject))
							{
								MorphTarget = SubMorphTarget;
								MorphTarget->EmptyMorphLODModels();
								MorphTarget->ClearGarbage();
								break;
							}
						}
					}
					//Create a new morph target, if the object do not exist (creating a new uobject is ok to do asynchronously)
					if (!MorphTarget)
					{
						MorphTarget = NewObject<UMorphTarget>(SkeletalMesh, MorphTargetName);
					}
					check(MorphTarget);
				}
				else
				{
					UE_ASSET_LOG(LogSkeletalMesh, Error, SkeletalMesh, TEXT("Cannot cache a skeletal mesh during a serialize if some morph targets need to be created. The solution is to Pre cache the skeletal mesh before the serialization so no morph target get created."));
					continue;
				}
			}
		}
		else
		{
			ToDeleteMorphTargets.Remove(MorphTarget);
		}
		MorphTarget->EmptyMorphLODModels();
		SkeletalMesh->GetMorphTargets().Add(MorphTarget);
		
		MorphTarget->GetMorphLODModels().AddDefaulted(MorphLODModelNumber);
		for (int32 MorphDataIndex = 0; MorphDataIndex < MorphLODModelNumber; ++MorphDataIndex)
		{
			MorphTarget->GetMorphLODModels()[MorphDataIndex] = MorphTargetLODModels[MorphDataIndex];
		}
	}
	//Rebuild the mapping and rehook the curve data
	SkeletalMesh->InitMorphTargets();
	
	//Clear any async flags after the morphtargets have been set to the skeletalmesh
	for (UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
	{
		constexpr EInternalObjectFlags AsyncFlags = EInternalObjectFlags::Async;
		MorphTarget->ClearInternalFlags(AsyncFlags);
	}

	for (UMorphTarget* ToDeleteMorphTarget : ToDeleteMorphTargets)
	{
		ToDeleteMorphTarget->BaseSkelMesh = nullptr;
		ToDeleteMorphTarget->EmptyMorphLODModels();
		ToDeleteMorphTarget->MarkAsGarbage();
	}
}

#endif // WITH_EDITOR


