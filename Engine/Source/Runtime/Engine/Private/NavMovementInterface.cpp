// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/NavMovementInterface.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavMovementInterface)

void INavMovementInterface::StopActiveMovement()
{
	if (!GetNavMovementProperties()->bStopMovementAbortPaths)
	{
		return;
	}

	IPathFollowingAgentInterface* PFAgent = GetPathFollowingAgent();
	if (PFAgent)
	{
		PFAgent->OnUnableToMove(*Cast<UObject>(this));
	}
}

void INavMovementInterface::StopMovementKeepPathing()
{
	FNavMovementProperties* NavMovementProperties = GetNavMovementProperties();
	NavMovementProperties->bStopMovementAbortPaths = false;
	StopMovementImmediately();
	NavMovementProperties->bStopMovementAbortPaths = true;
}

FVector INavMovementInterface::GetNavLocation() const
{
	const INavAgentInterface* MyOwner = Cast<INavAgentInterface>(GetOwnerAsObject());
	return MyOwner ? MyOwner->GetNavAgentLocation() : FNavigationSystem::InvalidLocation; 
}

float INavMovementInterface::GetPathFollowingBrakingDistance(float MaxSpeed) const
{
	const FNavMovementProperties NavMovementProperties = GetNavMovementProperties();
	return NavMovementProperties.bUseFixedBrakingDistanceForPaths ? NavMovementProperties.FixedPathBrakingDistance : MaxSpeed;
}

void INavMovementInterface::SetFixedBrakingDistance(float DistanceToEndOfPath)
{
	if (DistanceToEndOfPath > UE_KINDA_SMALL_NUMBER)
	{
		FNavMovementProperties* NavMovementProperties = GetNavMovementProperties();
		NavMovementProperties->bUseFixedBrakingDistanceForPaths = true;
		NavMovementProperties->FixedPathBrakingDistance = DistanceToEndOfPath;
	}
}

bool INavMovementInterface::UseAccelerationForPathFollowing() const
{
	return GetNavMovementProperties().bUseAccelerationForPaths;
}
