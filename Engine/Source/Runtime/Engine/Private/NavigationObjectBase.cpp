// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/NavigationObjectBase.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/World.h"
#include "Components/BillboardComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Texture2D.h"
#include "GameFramework/PhysicsVolume.h"
#include "Components/CapsuleComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationObjectBase)

DEFINE_LOG_CATEGORY_STATIC(LogNavigationPoint, Log, All);

ANavigationObjectBase::ANavigationObjectBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CapsuleComponent = CreateDefaultSubobject<UCapsuleComponent>(TEXT("CollisionCapsule"));
	CapsuleComponent->ShapeColor = FColor(255, 138, 5, 255);
	CapsuleComponent->bDrawOnlyIfSelected = true;
	CapsuleComponent->InitCapsuleSize(50.0f, 50.0f);
	CapsuleComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	CapsuleComponent->bShouldCollideWhenPlacing = true;
	CapsuleComponent->SetShouldUpdatePhysicsVolume(true);
	CapsuleComponent->Mobility = EComponentMobility::Static;
	RootComponent = CapsuleComponent;
	bCollideWhenPlacing = true;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;

	GoodSprite = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	BadSprite = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite2"));

#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> NavigationTextureObject;
			FName ID_Navigation;
			FText NAME_Navigation;
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> BadNavigationTextureObject;
			FConstructorStatics()
				: NavigationTextureObject(TEXT("/Engine/EditorResources/S_NavP"))
				, ID_Navigation(TEXT("Navigation"))
				, NAME_Navigation(NSLOCTEXT("SpriteCategory", "Navigation", "Navigation"))
				, BadNavigationTextureObject(TEXT("/Engine/EditorResources/Bad"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GoodSprite)
		{
			GoodSprite->Sprite = ConstructorStatics.NavigationTextureObject.Get();
			GoodSprite->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GoodSprite->bHiddenInGame = true;
			GoodSprite->SpriteInfo.Category = ConstructorStatics.ID_Navigation;
			GoodSprite->SpriteInfo.DisplayName = ConstructorStatics.NAME_Navigation;
			GoodSprite->SetupAttachment(CapsuleComponent);
			GoodSprite->SetUsingAbsoluteScale(true);
			GoodSprite->bIsScreenSizeScaled = true;
		}

		if (BadSprite)
		{
			BadSprite->Sprite = ConstructorStatics.BadNavigationTextureObject.Get();
			BadSprite->bHiddenInGame = true;
			BadSprite->SpriteInfo.Category = ConstructorStatics.ID_Navigation;
			BadSprite->SpriteInfo.DisplayName = ConstructorStatics.NAME_Navigation;
			BadSprite->SetUsingAbsoluteScale(true);
			BadSprite->SetupAttachment(CapsuleComponent);
			BadSprite->bIsScreenSizeScaled = true;
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void ANavigationObjectBase::GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const
{
	if (RootComponent == CapsuleComponent && IsRootComponentCollisionRegistered())
	{
		// Note: assuming vertical orientation
		CapsuleComponent->GetScaledCapsuleSize(CollisionRadius, CollisionHalfHeight);
	}
	else
	{
		Super::GetSimpleCollisionCylinder(CollisionRadius, CollisionHalfHeight);
	}
}

#if	WITH_EDITOR
void ANavigationObjectBase::PostEditMove( bool bFinished )
{
	if( bFinished )
	{
		if ( GetWorld()->IsNavigationRebuilt() )
		{
			UE_LOGF(LogNavigationPoint, Log, "PostEditMove Clear paths rebuilt");
		}

		// Validate collision
		Validate();
	}
	
	MarkComponentsRenderStateDirty();
	Super::PostEditMove( bFinished );
}

void ANavigationObjectBase::PostEditUndo()
{
	Super::PostEditUndo();
	
	// undo can move the object without calling post edit move, we need to validate on any movement
	Validate();
}

#endif // WITH_EDITOR

class APhysicsVolume* ANavigationObjectBase::GetNavPhysicsVolume()
{
	if ( CapsuleComponent )
	{
		return CapsuleComponent->GetPhysicsVolume();
	}	

	return GetWorld()->GetDefaultPhysicsVolume();
}

bool ANavigationObjectBase::ShouldBeBased()
{
	APhysicsVolume * PhysicsVolume = GetNavPhysicsVolume();
	return ((PhysicsVolume == NULL || !PhysicsVolume->bWaterVolume) && CapsuleComponent);
}

void ANavigationObjectBase::FindBase()
{
	if ( GetWorld()->HasBegunPlay() )
	{
		return;
	}

	if( ShouldBeBased() )
	{
		// not using find base, because don't want to fail if LD has navigationpoint slightly interpenetrating floor
		FHitResult Hit(1.f);
		
		const float Radius = CapsuleComponent->GetScaledCapsuleRadius();
		FVector const CollisionSlice(Radius, Radius, 1.f);
		
		// check for placement
		const FVector TraceStart = GetActorLocation();
		const FVector TraceEnd = GetActorLocation() - FVector(0.f,0.f, 4.f * CapsuleComponent->GetScaledCapsuleHalfHeight());

		GetWorld()->SweepSingleByObjectType( Hit, TraceStart, TraceEnd, FQuat::Identity, FCollisionObjectQueryParams(ECC_WorldStatic), FCollisionShape::MakeBox(CollisionSlice), FCollisionQueryParams(SCENE_QUERY_STAT(NavFindBase), false));

		// @fixme, ensure object is on the navmesh?
// 		if( Hit.Actor != NULL )
// 		{
// 			if (Hit.Normal.Z > Scout->WalkableFloorZ)
// 			{
// 				const FVector HitLocation = TraceStart + (TraceEnd - TraceStart) * Hit.Time;
// 				TeleportTo(HitLocation + FVector(0.f,0.f,CapsuleComponent->GetScaledCapsuleHalfHeight()-2.f), GetActorRotation(), false, true);
// 			}
// 			else
// 			{
// 				Hit.Actor = NULL;
// 			}
// 		}

		if (GoodSprite)
		{
			GoodSprite->SetVisibility(true);
		}
		if (BadSprite)
		{
			BadSprite->SetVisibility(false);
		}
	}
}


void ANavigationObjectBase::Validate()
{
	if ( ShouldBeBased() && (GoodSprite || BadSprite) )
	{
		FVector OrigLocation = GetActorLocation();
		const float Radius = CapsuleComponent->GetScaledCapsuleRadius();
		FVector const Slice(Radius, Radius, 1.f);

		bool bResult = true;

		// Check for adjustment
		FHitResult Hit(ForceInit);
		const FVector TraceStart = GetActorLocation();
		const FVector TraceEnd = GetActorLocation() - FVector(0.f,0.f, 4.f * CapsuleComponent->GetScaledCapsuleHalfHeight());
		GetWorld()->SweepSingleByChannel(Hit, TraceStart, TraceEnd, FQuat::Identity, ECC_Pawn, FCollisionShape::MakeBox(Slice), FCollisionQueryParams(SCENE_QUERY_STAT(NavObjectBase_Validate),false, this));
		if( Hit.bBlockingHit )
		{
			const FVector HitLocation = TraceStart + (TraceEnd - TraceStart) * Hit.Time;
			FVector Dest = HitLocation + FVector(0.f,0.f,CapsuleComponent->GetScaledCapsuleHalfHeight()-2.f);

			// Move actor (TEST ONLY) to see if navigation point moves
			TeleportTo( Dest, GetActorRotation(), false, true );

			// If only adjustment was down towards the floor, then it is a valid placement
			FVector NewLocation = GetActorLocation();
			bResult = ( NewLocation.X == OrigLocation.X &&  
				NewLocation.Y == OrigLocation.Y && 
				NewLocation.Z <= OrigLocation.Z );

			// Move actor back to original position
			TeleportTo( OrigLocation, GetActorRotation(), false, true );
		}	
		
		// Update sprites by result
		if( GoodSprite )
		{
			GoodSprite->SetVisibility(bResult);
		}
		if( BadSprite )
		{
			BadSprite->SetVisibility(!bResult);
		}
	}

	// Force update of icon
	MarkComponentsRenderStateDirty();
}

void ANavigationObjectBase::GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const
{
	GoalOffset = FVector::ZeroVector;
	GetSimpleCollisionCylinder(GoalRadius, GoalHalfHeight);	
}


