// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/BandwidthTestActor.h"

#include "Net/UnrealNetwork.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BandwidthTestActor)

//-----------------------------------------------------------------------------
//
ABandwidthTestActor::ABandwidthTestActor()
{
	bReplicates = true;
	bAlwaysRelevant = true;
	SetReplicatingMovement(false);

	NetDormancy = DORM_Never;
}

void ABandwidthTestActor::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(ABandwidthTestActor, BandwidthGenerator);
}

void ABandwidthTestActor::StartGeneratingBandwidth(float BandwidthInKilobytesPerSec)
{
	if (BandwidthInKilobytesPerSec <= 0)
	{
		StopGeneratingBandwidth();
		return;
	}

	const float ServerTickRate = GEngine->GetMaxTickRate(0.0f, false);
	if (ServerTickRate <= 0.0f)
	{
		return;
	}

	const float KBPerTick = BandwidthInKilobytesPerSec / ServerTickRate;

	BandwidthGenerator.FillBufferForSize( FMath::TruncToInt(KBPerTick*1000.f) );
	BandwidthGenerator.SpikePeriodInSec = 0.0;
}

void ABandwidthTestActor::StopGeneratingBandwidth()
{
	BandwidthGenerator.FillBufferForSize(0);
	BandwidthGenerator.SpikePeriodInSec = 0.0;
}

void ABandwidthTestActor::StartBandwidthSpike(float SpikeInKilobytes, int32 PeriodInMS)
{
	if (SpikeInKilobytes > 0.0f && PeriodInMS > 0)
	{
		BandwidthGenerator.FillBufferForSize(FMath::TruncToInt(SpikeInKilobytes*1000.f));
		BandwidthGenerator.SpikePeriodInSec = PeriodInMS / 1000.0;
		BandwidthGenerator.OnSpikePeriod();
	}
	else
	{
		StopGeneratingBandwidth();
	}
}

//-----------------------------------------------------------------------------
//
void FBandwidthTestGenerator::FillBufferForSize(int32 SizeInBytes)
{
	const int32 NbOfBytes = SizeInBytes;
	const int32 NbOfKilobytes = SizeInBytes / 1000;
	const int32 SlackInBytes = SizeInBytes - (NbOfKilobytes * 1000);

	ReplicatedBuffers.SetNum(NbOfKilobytes);

	auto FillWithRandomData = [](FBandwidthTestItem& ByteBuffer)
	{
		// Alloc with true random data to reduce Oodle compression
		for (uint8& ByteAlloc : ByteBuffer.Kilobyte)
		{
			ByteAlloc = (uint8)FMath::RandHelper(UINT8_MAX);
		}
	};

	// Fill each array with 1kilobyte
	constexpr int32 BytesInArray = 1000;	
	for(FBandwidthTestItem& Buffer : ReplicatedBuffers )
	{
		Buffer.Kilobyte.Reset(BytesInArray);
		Buffer.Kilobyte.AddUninitialized(BytesInArray);

		FillWithRandomData(Buffer);
	}

	if (SlackInBytes > 0)
	{
		FBandwidthTestItem& SlackBuffer = ReplicatedBuffers.AddDefaulted_GetRef();
		SlackBuffer.Kilobyte.AddUninitialized(SlackInBytes);
		FillWithRandomData(SlackBuffer);
	}
}

bool FBandwidthTestGenerator::NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParams)
{
	if (DeltaParams.Writer)
	{
		// For spiky bandwidth 
		if (SpikePeriodInSec > 0.0)
		{
			const double CurrentSeconds = FPlatformTime::Seconds();
			if (CurrentSeconds < TimeForNextSpike)
			{
				return true;
			}
			
			// Time to send the spike!
			OnSpikePeriod();
		}
		
		for (FBandwidthTestItem& Buffer : ReplicatedBuffers)
		{
			DeltaParams.Writer->Serialize(Buffer.Kilobyte.GetData(), Buffer.Kilobyte.GetTypeSize()*Buffer.Kilobyte.Num());
		}
		
	}
	return true;
}

void FBandwidthTestGenerator::OnSpikePeriod()
{
	if (SpikePeriodInSec > 0.0)
	{
		const double CurrentSeconds = FPlatformTime::Seconds();
		TimeForNextSpike = CurrentSeconds + SpikePeriodInSec;
	}
	else
	{
		TimeForNextSpike = 0.0;
	}
}


/**
 * 
 */
FAutoConsoleCommandWithWorldAndArgs GenerateBandwidth(TEXT("Net.GenerateConstantBandwidth"), 
													  TEXT("Deliver a constant throughput every tick to generate the specified Kilobytes per sec." \
														   "\nUsage:" \
														   "\nNet.GenerateBandwidth KilobytesPerSecond"), 
FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
{
	if (Args.Num() < 1)
	{
		UE_LOGF(LogConsoleResponse, Display, "Missing some parameters");
		return;
	}

	float BandwidthInKilobytesPerSec(0);
	LexTryParseString<float>(BandwidthInKilobytesPerSec, *Args[0]);

	bool bActorExists(false);
	for (TActorIterator<ABandwidthTestActor> It(World); It; ++It)
	{
		bActorExists = true;
		It->StartGeneratingBandwidth(BandwidthInKilobytesPerSec);
	}

	if (!bActorExists)
	{
		ABandwidthTestActor* GeneratorActor = Cast<ABandwidthTestActor>(World->SpawnActor(ABandwidthTestActor::StaticClass()));
		GeneratorActor->StartGeneratingBandwidth(BandwidthInKilobytesPerSec);
	}
}));

FAutoConsoleCommandWithWorldAndArgs GenerateBandwidthSpike(TEXT("Net.GeneratePeriodicBandwidthSpike"),
														  TEXT("Generates a spike of bandwidth every X milliseconds." \
															   "\nUsage:" \
															   "\nNet.GeneratePeriodicBandwidthSpike SpikeInKb PeriodInMS"),
FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray< FString >& Args, UWorld* World)
{
	if (Args.Num() < 2)
	{
		UE_LOGF(LogConsoleResponse, Display, "Missing some parameters");
		return;
	}

	float SpikeInKilobytes(0);
	LexTryParseString<float>(SpikeInKilobytes, *Args[0]);

	int32 PeriodInMS(0);
	LexTryParseString<int32>(PeriodInMS, *Args[1]);

	bool bActorExists(false);
	for (TActorIterator<ABandwidthTestActor> It(World); It; ++It)
	{
		bActorExists = true;
		It->StartBandwidthSpike(SpikeInKilobytes, PeriodInMS);
	}

	if (!bActorExists)
	{
		ABandwidthTestActor* GeneratorActor = Cast<ABandwidthTestActor>(World->SpawnActor(ABandwidthTestActor::StaticClass()));
		GeneratorActor->StartBandwidthSpike(SpikeInKilobytes, PeriodInMS);
	}
}));

FAutoConsoleCommandWithWorldAndArgs CreateBandwidthGenerator(TEXT("Net.CreateBandwidthGenerator"), TEXT(""), FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray< FString >& Args, UWorld* World)
{
	ABandwidthTestActor* GeneratorActor = Cast<ABandwidthTestActor>(World->SpawnActor(ABandwidthTestActor::StaticClass()));
	check(GeneratorActor);
}));


