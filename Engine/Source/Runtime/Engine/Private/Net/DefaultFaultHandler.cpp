// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/DefaultFaultHandler.h"
#include "Net/Core/Connection/NetConnectionFaultRecoveryBase.h"


namespace UE
{
namespace Net
{

/**
 * FDefaultFaultManager
 */

void FDefaultFaultHandler::InitFaultRecovery(FNetConnectionFaultRecoveryBase* InFaultRecovery)
{
	if (FaultRecovery == nullptr)
	{
		FaultRecovery = InFaultRecovery;

		if (FaultRecovery != nullptr)
		{
			FaultRecovery->GetFaultManager().AddResultHandlerPtr(this);
			FaultRecovery->RegisterCounterCategory(ENetFaultCounterCategory::NetworkCorruption, ToInt(ENetFaultCounters::NetConnPacketFault));
		}
	}
}

EHandleNetResult FDefaultFaultHandler::HandleNetResult(FNetResult&& InResult)
{
	EHandleNetResult ReturnVal = EHandleNetResult::Handled;
	FNetCloseResult* CastedResult = Cast<ENetCloseResult>(&InResult);
	ENetCloseResult Result = (CastedResult != nullptr ? CastedResult->GetResult() : ENetCloseResult::Unknown);
	FEscalationCounter CounterIncrement;
	ENetFaultCounters TrackedCounter = ENetFaultCounters::Max;

	switch (Result)
	{
		// Only NetConnection errors which are low level, and only those exclusively resulting from invalid data which may be corruption
		case ENetCloseResult::ZeroLastByte:
		case ENetCloseResult::ZeroSize:
		case ENetCloseResult::ReadHeaderFail:
		case ENetCloseResult::ReadHeaderExtraFail:
		case ENetCloseResult::BunchChannelNameFail:
		case ENetCloseResult::BunchHeaderOverflow:
		case ENetCloseResult::BunchDataOverflow:
		{
			TrackedCounter = ENetFaultCounters::NetConnPacketFault;
			CounterIncrement.Counter++;

			break;
		}


		default:
		{
			ReturnVal = EHandleNetResult::NotHandled;

			break;
		}
	}


	if (ReturnVal != EHandleNetResult::NotHandled)
	{
		if (TrackedCounter != ENetFaultCounters::Max)
		{
			// Static/enum-defined counter index
			FEscalationCounter& FrameCounter = FaultRecovery->GetFrameCounter(ToInt(TrackedCounter));

			FrameCounter.AccumulateCounter(CounterIncrement);

			ReturnVal = FaultRecovery->NotifyHandledFault(MoveTemp(InResult));
		}
	}
		
	return ReturnVal;
}


}
}
