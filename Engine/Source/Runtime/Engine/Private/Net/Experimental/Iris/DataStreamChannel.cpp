// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStreamChannel.h"
#include "HAL/IConsoleManager.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataStreamChannel)

// IWYU pragma: begin_keep
#include "Engine/NetConnection.h"
#include "Iris/IrisConfig.h"
#include "Iris/DataStream/DataStreamManager.h"
#include "Iris/PacketControl/PacketNotification.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Iris/Core/IrisLog.h"

#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/DataChannel.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"


#include "PacketHandler.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Logging/LogMacros.h"
// IWYU pragma: end_keep

namespace UE::Net::Private
{
	static int32 IrisPacketSequenceSafetyMargin = 4;
	static FAutoConsoleVariableRef CVarIrisPacketSequenceSafetyMargin(
		TEXT("net.Iris.PacketSequenceSafetyMargin"),
		IrisPacketSequenceSafetyMargin,
		TEXT("How many packets to spare before considering packet sequence full. This allows a few non-DataStreamChannel packets to be sent without messing up packet acking."
		));

	/**
	 * net.Iris.ReplicationWriterMaxAllowedPacketsIfNotHugeObject
	 * Limit the amount of packets that can be sent in a single batch even if the netspeed allows it.
	 */
	static int32 GReplicationWriterMaxAllowedPacketsIfNotHugeObject = 3;
	static FAutoConsoleVariableRef CVarReplicationWriterMaxAllowedPacketsIfNotHugeObject(TEXT("net.Iris.ReplicationWriterMaxAllowedPacketsIfNotHugeObject"), GReplicationWriterMaxAllowedPacketsIfNotHugeObject,
		TEXT("Allow ReplicationWriter to overcommit data if we have more data to write."));

	static FName ReceivedBunchTransactionName("UDataStreamChannel::ReceivedBunch");
}

UDataStreamChannel::UDataStreamChannel(const FObjectInitializer& ObjectInitializer)
: UChannel(ObjectInitializer)
, WriteRecords(MaxPacketsInFlightCount)
, bIsReadyToHandshake(0U)
, bHandshakeSent(0U)
, bHandshakeComplete(0U)
{
	ChName = FName("DataStream");
	bSupportsParallelTick = true;
}

void UDataStreamChannel::InitDataStreamManager()
{
	check(Connection);

	if (UReplicationSystem* ReplicationSystem = Connection->Driver->GetReplicationSystem())
	{
		DataStreamManager = NewObject<UDataStreamManager>();

		// Make ReplicationSystem aware of new DataStreamManager associated with connection.
		ReplicationSystem->InitDataStreamManager(Connection->GetConnectionHandle().GetParentConnectionId(), DataStreamManager);

		bIsReadyToHandshake = 1U;
	}
	else
	{
		bIsReadyToHandshake = 0U;
	}
}

void UDataStreamChannel::Init(UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags)
{
	Super::Init(InConnection, InChIndex, CreateFlags);
	InitDataStreamManager();
}

void UDataStreamChannel::ReInit()
{
	InitDataStreamManager();
}

bool UDataStreamChannel::CleanUp(const bool bForDestroy, EChannelCloseReason CloseReason)
{
	if (IsValid(DataStreamManager))
	{
		DataStreamManager->Deinit();
		DataStreamManager->MarkAsGarbage();
		DataStreamManager = nullptr;
	}

	WriteRecords.Reset();

	return Super::CleanUp(bForDestroy, CloseReason);
}

void UDataStreamChannel::ReceivedBunch(FInBunch& Bunch)
{
	using namespace UE::Net;

	IRIS_PROFILER_SCOPE(UDataStreamChannel_ReceivedBunch);

	// We are sending dummy bunches until we are open.
	if (!Bunch.GetNumBits())
	{
		return;
	}

	// We do not support partial bunches.
	if (Bunch.bPartial)
	{
		Bunch.SetAtEnd();
		Bunch.SetError();
		return;
	}

	FNetBitStreamReader BitReader;
	BitReader.InitBits(Bunch.GetData(), Bunch.GetNumBits());
	BitReader.Seek(Bunch.GetPosBits());

	FNetSerializationContext SerializationContext(&BitReader);
	SerializationContext.SetPacketId(Connection->InPacketId);

	// For packet stats
	SerializationContext.SetTraceCollector(Connection->GetInTraceCollector());

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	if (Connection->Driver->ShouldReplicateTransactionally())
	{
		UE::RemoteExecutor::ExecuteTransactional(UE::Net::Private::ReceivedBunchTransactionName, [this, &SerializationContext]()
		{
			DataStreamManager->ReadData(SerializationContext);
		});
	}
	else
#endif
	{
		DataStreamManager->ReadData(SerializationContext);
	}

	// Set the bunch at the end
	Bunch.SetAtEnd();

	// If receiving was unsuccessful set bunch in error
	if (SerializationContext.HasErrorOrOverflow())
	{
		if (SerializationContext.GetErrorHandleContext().IsValid())
		{
			TNetResult<ENetCloseResult> NetResult(ENetCloseResult::IrisNetRefHandleError, FString::Printf(TEXT("IrisNetRefHandleError=%s"), *SerializationContext.GetErrorHandleContext().ToString()));
			AddToChainResultPtr(Bunch.ExtendedError, MoveTemp(NetResult));

			UEngineReplicationBridge* ActorBridge = FReplicationSystemUtil::GetActorReplicationBridge(Connection);
			if (ActorBridge)
			{
				ActorBridge->SendErrorWithNetRefHandle(UE::Net::ENetRefHandleError::BitstreamCorrupted, SerializationContext.GetErrorHandleContext(), Connection->GetConnectionHandle().GetParentConnectionId());
			}
		}
		Bunch.SetError();
	}

}

void UDataStreamChannel::SendOpenBunch()
{

	// We send this only once
	if (!bHandshakeSent && NumOutRec == 0)
	{
		// Send dummy data to open the channel
		constexpr int64 MaxBunchBits = 8;
		FOutBunch OutBunch(MaxBunchBits);
		OutBunch.ChName = this->ChName;
		OutBunch.ChIndex = this->ChIndex;
		OutBunch.Channel = this;
		OutBunch.Next = nullptr;
		// Unreliable bunches will be dropped on the receiving side unless the channel is open.
		OutBunch.bReliable = true;

		constexpr bool bAllowMerging = false;
		const FPacketIdRange PacketIds = SendBunch(&OutBunch, bAllowMerging);

		if (PacketIds.First != INDEX_NONE)
		{
			bHandshakeSent = 1U;
		}
	}

}

void UDataStreamChannel::PostTickDispatch()
{
	using namespace UE::Net;

	if (!Connection->Driver->IsUsingIrisReplication() || !bHandshakeComplete)
	{
		return;
	}

	if (IsPacketWindowFull() || !Connection->HasReceivedClientPacket() || (Connection->Handler != nullptr && !Connection->Handler->IsFullyInitialized()))
	{
		return;
	}

	// We probably want separate bandwidth management for iris as we are not pre-filling sendbuffer before call to NetReady.
	if (!IsNetReady())
	{
		return;
	}

#if UE_NET_IRIS_CSV_STATS
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UDataStreamChannel_PostTickDispatch_Write);
#endif

	IRIS_PROFILER_SCOPE(UDataStreamChannel_PostTickDispatch);
	LLM_SCOPE_BYTAG(Iris);

	WriteData(EDataStreamWriteMode::PostTickDispatch);

}

void UDataStreamChannel::Tick()
{
	using namespace UE::Net;

	if (!Connection->Driver->IsUsingIrisReplication() || !bIsReadyToHandshake)
	{
		return;
	}

	if (IsPacketWindowFull() || !Connection->HasReceivedClientPacket() || (Connection->Handler != nullptr && !Connection->Handler->IsFullyInitialized()))
	{
		return;
	}

	// We probably want separate bandwidth management for iris as we are not pre-filling sendbuffer before call to NetReady.
	if (!IsNetReady())
	{
		return;
	}

	// Wait for channel to open
	if (!bHandshakeComplete)
	{
		SendOpenBunch();
		return;
	}

#if UE_NET_IRIS_CSV_STATS
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UDataStreamChannel_Tick_Write);
#endif

	IRIS_PROFILER_SCOPE(UDataStreamChannel_Tick);
	LLM_SCOPE_BYTAG(Iris);

	const bool bWroteData = WriteData(EDataStreamWriteMode::Full);

#if UE_NET_IRIS_ALLOW_DEBUG_DATA
	// If we have debugdata, we write it as separate packets.
	if (DataStreamManager->HasDebugDataToSend() && !IsPacketWindowFull())
	{
		// Need to flush
		if (bWroteData)
		{
			IRIS_PROFILER_SCOPE(UDataStreamChannel_FlushNet);
			Connection->FlushNet();	
		}
		WriteData(EDataStreamWriteMode::DebugData);
	}
#endif
}

bool UDataStreamChannel::WriteData(UE::Net::EDataStreamWriteMode WriteMode)
{
	using namespace UE::Net;

	// Limit the amount of bits to minimum of a bunch and our buffer. NetBitStreamWriter requires the number of bytes to be a multiple of 4.
	const uint32 MaxBitCount = uint32(Connection->GetMaxSingleBunchSizeBits());
	const uint32 MaxBytes = FPlatformMath::Min((MaxBitCount/32U)*4U, (uint32)sizeof(BitStreamBuffer));
	const int64 MaxBunchBits = MaxBytes*8;

	// Try to determine if we have headroom to write more than a single packet if needed.
	UDataStream::FBeginWriteParameters BeginWriteParams
	{
		.WriteMode = WriteMode,
	};

	const SIZE_T RecordCount = WriteRecords.Count();

	if (WriteMode == EDataStreamWriteMode::Full)
	{
		int32 CurrentQueuedBits = Connection->QueuedBits + Connection->SendBuffer.GetNumBits();
		
		// If the connection hasn't gone over the limit yet (negative = bits allowed to send)
		if (CurrentQueuedBits < 0)
		{
			const int32 MaxAllowedPackets = FMath::Min((-CurrentQueuedBits) / MaxBunchBits, UE::Net::Private::GReplicationWriterMaxAllowedPacketsIfNotHugeObject);

			BeginWriteParams.MaxPackets = (uint32)FMath::Max(MaxAllowedPackets, 1);
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (NetConnectionHelper::HasDisabledBandwidthThrottling())
		{
			BeginWriteParams.MaxPackets = 0U;
		}
#endif
	}
	else
	{
		// Only allow 1 packet for PostTickDispatch
		BeginWriteParams.MaxPackets = 1U;
	}

	check(Connection);
	UNetDriver* ConnDriver = Connection->GetDriver();
	UReplicationSystem* RepSystem = ConnDriver ? ConnDriver->GetReplicationSystem() : nullptr;
	ensure(RepSystem);

	//Get a NetStatsContext that is unique per-thread so locking NetStats is not required for the rest of this function
	UE::Net::FScopedNetStatsContext ScopedNetStatsContext(RepSystem);

	// Currently we want to use a full bunch so we flush if we have to
	bool bNeedsPreSendFlush = Connection->SendBuffer.GetNumBits() > MAX_PACKET_HEADER_BITS;

	auto WriteDataFunction = [this, &bNeedsPreSendFlush, &BeginWriteParams, MaxBunchBits, MaxBytes, &ScopedNetStatsContext]() -> UDataStream::EWriteResult
	{
		if (bNeedsPreSendFlush)
		{
			IRIS_PROFILER_SCOPE(UDataStreamChannel_PreSendBunchAndFlushNet);
			Connection->FlushNet();
			bNeedsPreSendFlush = false;
		}

		// Make sure that packet header is written first to ensure that trace data is updated correctly
		if (Connection->SendBuffer.GetNumBits() == 0U)
		{
			Connection->WriteBitsToSendBuffer(nullptr, 0);
		}

		FNetBitStreamWriter BitWriter;
		BitWriter.InitBytes(BitStreamBuffer, MaxBytes);

		FNetSerializationContext SerializationContext(&BitWriter);
		SerializationContext.SetPacketId(Connection->OutPacketId);
		SerializationContext.SetNetStatsContext(ScopedNetStatsContext.GetRawPtr());
		

#if UE_NET_TRACE_ENABLED	
		// For Iris we can use the connection trace collector as long as we make sure that the packet is prepared 
		FNetTraceCollector* Collector = Connection->GetOutTraceCollector();
		SerializationContext.SetTraceCollector(Collector);
		UE_NET_TRACE_BEGIN_BUNCH(Collector);
#endif

		const FDataStreamRecord* Record = nullptr;
		const UDataStream::EWriteResult WriteResult = DataStreamManager->WriteData(SerializationContext, Record);

		if (WriteResult == UDataStream::EWriteResult::NoData || SerializationContext.HasError())
		{
			IRIS_PROFILER_SCOPE(UDataStreamChannel_NoDataSent);
			// Do not report the bunch
			UE_NET_TRACE_DISCARD_BUNCH(Collector);

			if (SerializationContext.HasError())
			{
				FString ErrorMsg = NSLOCTEXT("NetworkErrors", "DataStreamChannelWriteData", "DataStreamChannel failed to write data.").ToString();
				Connection->SendCloseReason(ENetCloseResult::HostClosedConnection);
				FNetControlMessage<NMT_Failure>::Send(Connection, ErrorMsg);
				Connection->FlushNet(true);
				Connection->Close(ENetCloseResult::HostClosedConnection);
			}

			return UDataStream::EWriteResult::NoData;
		}

		// Flush bitstream
		BitWriter.CommitWrites();

		IRIS_PROFILER_SCOPE(UDataStreamChannel_SendBunchAndFlushNet);

		FOutBunch OutBunch(MaxBunchBits);
#if UE_NET_TRACE_ENABLED
		SetTraceCollector(OutBunch, Collector);
#endif
		OutBunch.ChName = this->ChName;
		OutBunch.ChIndex = this->ChIndex;
		OutBunch.Channel = this;
		OutBunch.Next = nullptr;
		// Unreliable bunches will be dropped on the receiving side unless the channel is open.
		OutBunch.bReliable = !OpenAcked;
		OutBunch.SerializeBits(BitStreamBuffer, BitWriter.GetPosBits());

		const int32 PreSendBunchQueuedBits = Connection->QueuedBits + Connection->SendBuffer.GetNumBits();

		constexpr bool bAllowMerging = false;
		const FPacketIdRange PacketIds = SendBunch(&OutBunch, bAllowMerging);

		// Ugly trick to allow DebugData to not count towards bandwidth limit
		if (BeginWriteParams.WriteMode == EDataStreamWriteMode::DebugData)
		{
			Connection->QueuedBits = PreSendBunchQueuedBits - Connection->SendBuffer.GetNumBits();
		}

#if UE_NET_TRACE_ENABLED
		// Since we steal the connection collector, we need to clear it out before the bunch goes out of scope
		SetTraceCollector(OutBunch, nullptr);
#endif

		if (PacketIds.First == INDEX_NONE)
		{
			// Something went wrong.
			check(PacketIds.First != INDEX_NONE);
			return UDataStream::EWriteResult::NoData;
		}

		FDataStreamChannelRecord ChannelRecord;
		ChannelRecord.Record = Record;
		ChannelRecord.PacketId = PacketIds.First;

		WriteRecords.Enqueue(ChannelRecord);

		// If we are allowed to write more data, we need to flush
		bNeedsPreSendFlush = true;

		return WriteResult;
	};

	// Begin the write, if we have nothing todo, just return
	if (DataStreamManager->BeginWrite(BeginWriteParams) == UDataStream::EWriteResult::NoData)
	{
		return false;
	}

	bool bContinueWriting = false;
	do 
	{
		// Write data until we are not allowed to write more
		const UDataStream::EWriteResult Result = WriteDataFunction();

		if (Result == UDataStream::EWriteResult::HasMoreData && IsNetReady() && !IsPacketWindowFull())
		{
			bContinueWriting = true;
		}
		else
		{
			bContinueWriting = false;
		}

	}
	while (bContinueWriting);

	// call end write to cleanup data initialized in BeginWrite	
	DataStreamManager->EndWrite();

	// If we did write data and the current WriteMode is PostTickDispatch we flush the packet here.
	if (WriteMode == EDataStreamWriteMode::PostTickDispatch && bNeedsPreSendFlush)
	{
		IRIS_PROFILER_SCOPE(UDataStreamChannel_FlushNet);
		Connection->FlushNet();	
	}

	return WriteRecords.Count() != RecordCount;
}

bool UDataStreamChannel::CanStopTicking() const
{
	return false;
}

FString UDataStreamChannel::Describe()
{
	return FString("DataStream: ") + UChannel::Describe();
}

void UDataStreamChannel::ReceivedAck(int32 PacketId)
{

	if (!bHandshakeComplete)
	{
		if (bHandshakeSent)
		{
			bHandshakeComplete = 1U;
		}
		return;
	}

	const FDataStreamChannelRecord& ChannelRecord = WriteRecords.Peek();
	if (ensureMsgf((uint32)PacketId == ChannelRecord.PacketId, TEXT("PacketId %d != ChannelRecord.PacketId %d, WriteRecords.Num %d"), PacketId, ChannelRecord.PacketId, (int32)WriteRecords.Count()))
	{
		DataStreamManager->ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus::Delivered, static_cast<const FDataStreamRecord*>(ChannelRecord.Record));
		WriteRecords.Pop();
	}

}

void UDataStreamChannel::ReceivedNak(int32 PacketId)
{

	if (!bHandshakeComplete)
	{
		// Rely on super to resend our open request
		Super::ReceivedNak(PacketId);
		return;
	}

	const FDataStreamChannelRecord& ChannelRecord = WriteRecords.Peek();

	if (ensureMsgf((uint32)PacketId == ChannelRecord.PacketId, TEXT("PacketId %d != ChannelRecord.PacketId %d, WriteRecords.Num %d"), PacketId, ChannelRecord.PacketId, (int32)WriteRecords.Count()))
	{
		DataStreamManager->ProcessPacketDeliveryStatus(UE::Net::EPacketDeliveryStatus::Lost, static_cast<const FDataStreamRecord*>(ChannelRecord.Record));
		WriteRecords.Pop();
	}

}

// Some DataStreams require perfect acking. If the ack sequence window is full we would get NAKs for packets thay may have been received.
bool UDataStreamChannel::IsPacketWindowFull() const
{
	const uint32 IrisPacketSequenceSafetyMarginUnsigned = static_cast<uint32>(FPlatformMath::Max(0, UE::Net::Private::IrisPacketSequenceSafetyMargin));
	if (Connection->IsPacketSequenceWindowFull(IrisPacketSequenceSafetyMarginUnsigned))
	{
		UE_LOGF(LogIris, Verbose, "Packet window full.");
		return true;
	}

	return WriteRecords.Count() >= WriteRecords.AllocatedCapacity();
}

void UDataStreamChannel::AddReferencedObjects(UObject* Object, FReferenceCollector& Collector)
{
	UDataStreamChannel* Channel = CastChecked<UDataStreamChannel>(Object);
	if (Channel->DataStreamManager)
	{
		Collector.AddReferencedObject(Channel->DataStreamManager);
	}

	Super::AddReferencedObjects(Channel, Collector);
}

void UDataStreamChannel::AppendExportBunches(TArray<FOutBunch*>& OutExportBunches)
{
}

TArray<FOutBunch*> UDataStreamChannel::GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags)
{
	// Base version assumed we have a UPackageMapClient, so override it
	return TArray<FOutBunch*>{};
}

void UDataStreamChannel::AppendMustBeMappedGuids(FOutBunch* Bunch)
{
}

bool UDataStreamChannel::HasAcknowledgedAllReliableData() const
{
	if (DataStreamManager)
	{
		return DataStreamManager->HasAcknowledgedAllReliableData();
	}

	return true;
}
