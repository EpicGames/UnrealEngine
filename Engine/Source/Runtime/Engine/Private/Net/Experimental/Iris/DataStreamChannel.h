// Copyright Epic Games, Inc. All Rights Reserved.

// Channel for sending and receiving all Iris data

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Channel.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "DataStreamChannel.generated.h"

class UDataStreamManager;

namespace UE::Net
{
	enum class EDataStreamWriteMode : unsigned;
};

UCLASS(transient, customConstructor, MinimalAPI)
class UDataStreamChannel final : public UChannel
{
	GENERATED_BODY()

	UDataStreamChannel(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	/** Invoked from NetDriver::PostTickDispatch if we have any data that should be written from PostTickDispatch */
	ENGINE_API void PostTickDispatch();

private:
	static void AddReferencedObjects(UObject* Object, FReferenceCollector& Collector);

	// UChannel interface
	ENGINE_API virtual void Init(UNetConnection* InConnection, int32 InChIndex, EChannelCreateFlags CreateFlags) override;
	ENGINE_API virtual void ReInit() override;
	ENGINE_API virtual bool CleanUp(const bool bForDestroy, EChannelCloseReason CloseReason) override;

	/**
	 * Processes the in bound bunch to extract the data streams
	 */
	ENGINE_API virtual void ReceivedBunch(FInBunch& Bunch) override;

	ENGINE_API virtual void Tick() override;

	/** Always ticks */
	ENGINE_API virtual bool CanStopTicking() const override;

	/** Human readable information about the channel */
	ENGINE_API virtual FString Describe() override;

	/** We do not want to append orphaned exportbunches from other channels */
	ENGINE_API virtual void AppendExportBunches(TArray<FOutBunch *>& OutExportBunches) override;
	virtual TArray<FOutBunch*> GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags) override;

	ENGINE_API virtual void AppendMustBeMappedGuids(FOutBunch* Bunch) override;


	/** Packet delivery status handling */
	ENGINE_API virtual void ReceivedAck(int32 PacketId) override;
	ENGINE_API virtual void ReceivedNak(int32 PacketId) override;

	ENGINE_API virtual bool HasAcknowledgedAllReliableData() const override;

private:
	void InitDataStreamManager();

	enum : uint32
	{
		MaxPacketsInFlightCount = 256U,
		BitStreamBufferByteCount = 2048U,
	};

	struct FDataStreamChannelRecord
	{
		const void* Record = nullptr;
		uint32 PacketId = 0U;
	};

	bool IsPacketWindowFull() const;
	void DiscardAllRecords();

	void SendOpenBunch();

	bool WriteData(UE::Net::EDataStreamWriteMode WriteMode);

	TObjectPtr<UDataStreamManager> DataStreamManager = nullptr;

	TResizableCircularQueue<FDataStreamChannelRecord> WriteRecords;

	uint32 BitStreamBuffer[BitStreamBufferByteCount];

	uint32 bIsReadyToHandshake : 1U;
	uint32 bHandshakeSent : 1U;
	uint32 bHandshakeComplete : 1U;
};
