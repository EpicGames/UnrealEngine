// Copyright Epic Games, Inc. All Rights Reserved.
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineReplicationBridge)

#include "Net/Iris/ReplicationSystem/NetActorFactory.h"
#include "Net/Iris/ReplicationSystem/NetSubObjectFactory.h"

#include "Iris/IrisConfig.h"
#include "Iris/IrisConstants.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/NetObjectReference.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Net/Core/NetToken/NetToken.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridgeConfig.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "Iris/ReplicationSystem/ReplicationOperations.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"

#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/ObjectNetSerializer.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"

#include "Iris/Metrics/NetMetrics.h"

#include "AnalyticsEventAttribute.h"

#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "Engine/Level.h"
#include "Engine/World.h"

#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"

#include "HAL/LowLevelMemStats.h"

#include "Net/DataBunch.h"
#include "Net/DataChannel.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Net/Core/Connection/NetCloseResult.h"
#include "Net/Core/Misc/NetSubObjectRegistry.h"
#include "Net/Core/Trace/NetDebugName.h"
#include "Net/NetSubObjectRegistryGetter.h"

#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#include "ProfilingDebugging/AssetMetadataTrace.h"

#include "Templates/Casts.h"

#include "UObject/Package.h"

#include <limits>

extern bool GDefaultUseSubObjectReplicationList;

namespace UE::Net::Private
{

static bool bAlwaysCreateLevelFilteringGroupsForPersistentLevels = true;
static FAutoConsoleVariableRef CVarAlwaysCreateLevelFilteringGroupsForPersistentLevels(
	TEXT("net.Iris.AlwaysCreateLevelFilteringGroupsForPersistentLevels"),
	bAlwaysCreateLevelFilteringGroupsForPersistentLevels,
	TEXT("When true a world's persistent level will have a filter group created such that replication of objects therein can be enabled and disabled at any point in time. This is required for seamless travel but otherwise an unnecessary overhead.")
);

static bool bEnableActorLevelChanges = true;
static FAutoConsoleVariableRef CVarEnableActorLevelChanges(
	TEXT("net.Iris.EnableActorLevelChanges"),
	bEnableActorLevelChanges,
	TEXT("When true the ActorReplicationBridge will process actors that change levels by updating the actor's level groups.")
);

static bool bEnableDynamicNetUpdateFrequency = true;
static FAutoConsoleVariableRef CVarEnableDynamicNetUpdateFrequency(
	TEXT("net.Iris.EnableDynamicNetUpdateFrequency"),
	bEnableDynamicNetUpdateFrequency,
	TEXT("When true changes to AActor::NetUpdateFrequency will be updated in Iris after being registered for replication.")
);

// Due to how it is setup in legacy replication system default is to replicate child subobjects before the actorcomponents
static bool bDefaultReplicateActorComponentSubobjectsAfterComponent = false;
static FAutoConsoleVariableRef CVarDefaultReplicateActorComponentSubobjectsAfterComponent(
		TEXT("net.Iris.DefaultReplicateActorComponentSubobjectsAfterComponent"),
		bDefaultReplicateActorComponentSubobjectsAfterComponent,
		TEXT("If set to true, ChildSubObjects of ActorComponents will replicate after component. Default is false.")
);

static int32 MaxNetRefHandleErrorDiagnosticMessages = 5;
static FAutoConsoleVariableRef CVarMaxNetRefHandleErrorDiagnosticMessages(
	TEXT("net.Iris.MaxNetRefHandleErrorDiagnosticMessages"),
	MaxNetRefHandleErrorDiagnosticMessages,
	TEXT("Maximum number of NMT_IrisNetRefHandleErrorWithDiagnosticMessage control messages a client may send to the server. Once exceeded the bridge falls back to NMT_IrisNetRefHandleError (no diagnostic string).")
);

bool IsActorValidForIrisReplication(const AActor* Actor)
{
	return IsValid(Actor) && !Actor->IsActorBeingDestroyed() && !Actor->IsUnreachable();
}

void ActorReplicationBridgePreUpdateFunction(TArrayView<UObject*> Instances, const UObjectReplicationBridge* Bridge)
{
	const UEngineReplicationBridge* EngineReplicationBridge = CastChecked<const UEngineReplicationBridge>(Bridge);
	if (!EngineReplicationBridge->AllowExecutePreReplication())
	{
		return;
	}
	
	UNetDriver* NetDriver = EngineReplicationBridge->GetNetDriver();
	for (UObject* Instance : Instances)
	{
		AActor* Actor = CastChecked<AActor>(Instance);
		if (IsActorValidForIrisReplication(Actor))
		{
			Actor->CallPreReplication(NetDriver);
		}
	}
}

bool ShouldIncludeRootObjectInLevelGroups(const UObject* RootObject)
{
	// Never filter out PlayerControllers based on level as they are required for travel.
	// Preserves the special case for PlayerControllers from UNetDriver::IsLevelInitializedForActor.
	return !RootObject->IsA<APlayerController>();
}

} // end namespace UE::Net::Private

namespace UE::Net
{

const TCHAR* LexToString(UE::Net::EStopReplicatingReason Reason)
{
	using namespace UE::Net;
	switch (Reason)
	{
		case EStopReplicatingReason::Destroyed:
		{
			return TEXT("Destroyed");
		}
		case EStopReplicatingReason::LevelTransition:
		{
			return TEXT("LevelTransition");
		}
		case EStopReplicatingReason::EndPlayInEditor:
		{
			return TEXT("EndPlayInEditor");
		}
		case EStopReplicatingReason::RemovedFromWorld:
		{
			return TEXT("RemovedFromWorld");
		}
		case EStopReplicatingReason::Quit:
		{
			return TEXT("Quit");
		}
		default:
		{
			return TEXT("Invalid");
		}
	}
}

}

FStopReplicatingActorParams::FStopReplicatingActorParams(EEndPlayReason::Type InEndPlayReason)
{
	using namespace UE::Net;

	EndPlayReason = InEndPlayReason;

	switch (EndPlayReason)
	{
		case EEndPlayReason::Type::Destroyed:
		{
			StopReplicatingReason = EStopReplicatingReason::Destroyed;
			break;
		}
		case EEndPlayReason::Type::LevelTransition:
		{
			StopReplicatingReason = EStopReplicatingReason::LevelTransition;
			break;
		}
		case EEndPlayReason::Type::EndPlayInEditor:
		{
			StopReplicatingReason = EStopReplicatingReason::EndPlayInEditor;
			break;
		}
		case EEndPlayReason::Type::RemovedFromWorld:
		{
			StopReplicatingReason = EStopReplicatingReason::RemovedFromWorld;
			break;
		}
		case EEndPlayReason::Type::Quit:
		{
			StopReplicatingReason = EStopReplicatingReason::Quit;
			break;
		}
		default:
		{
			check(false);
			break;
		}
	}
}

UEngineReplicationBridge::UEngineReplicationBridge()
: UObjectReplicationBridge()
, ActorFactoryId(UE::Net::InvalidNetObjectFactoryId)
, SubObjectFactoryId(UE::Net::InvalidNetObjectFactoryId)
{
	SetInstancePreUpdateFunction(UE::Net::Private::ActorReplicationBridgePreUpdateFunction);

	ActorFactoryName = UNetActorFactory::GetFactoryName();
	SubObjectFactoryName = UNetSubObjectFactory::GetFactoryName();
}

void UEngineReplicationBridge::Initialize(UReplicationSystem* InReplicationSystem)
{
	using namespace UE::Net;

	Super::Initialize(InReplicationSystem);

	ensureMsgf(GDefaultUseSubObjectReplicationList, TEXT("Iris requires replicated actors to use registered subobjectslists. Add \n[SystemSettings]\nnet.SubObjects.DefaultUseSubObjectReplicationList=1\n to your DefaultEngine.ini"));

	ActorFactoryId = FNetObjectFactoryRegistry::GetFactoryIdFromName(ActorFactoryName);
	checkf(ActorFactoryId != InvalidNetObjectFactoryId, TEXT("UNetActorFactory with name %s was not registered"), ToCStr(ActorFactoryName.ToString()));

	SubObjectFactoryId = FNetObjectFactoryRegistry::GetFactoryIdFromName(SubObjectFactoryName);
	checkf(SubObjectFactoryId != InvalidNetObjectFactoryId, TEXT("UNetSubObjectFactory with name %s was not registered"), ToCStr(SubObjectFactoryName.ToString()));

	{
		auto ShouldSpatialize = [](const UClass* Class)
		{
			if (AActor* CDO = Cast<AActor>(Class->GetDefaultObject()))
			{
				return !(CDO->bAlwaysRelevant || CDO->bOnlyRelevantToOwner || CDO->bNetUseOwnerRelevancy);
			}

			return false;
		};

		SetShouldUseDefaultSpatialFilterFunction(ShouldSpatialize);
	}

	{
		auto ClassesAreRelevantEqual = [](const UClass* Class, const UClass* Subclass)
		{
			const AActor* CDO = Cast<AActor>(Class->GetDefaultObject());
			const AActor* SubCDO = Cast<AActor>(Subclass->GetDefaultObject());
			// Same CDO (nullptr)?
			if (CDO == SubCDO)
			{
				return true;
			}

			if (CDO == nullptr || SubCDO == nullptr)
			{
				return false;
			}

			{
				return CDO->bAlwaysRelevant == SubCDO->bAlwaysRelevant
					&& CDO->bOnlyRelevantToOwner == SubCDO->bOnlyRelevantToOwner
					&& CDO->bNetUseOwnerRelevancy == SubCDO->bNetUseOwnerRelevancy;
			}
		};

		SetShouldSubclassUseSameFilterFunction(ClassesAreRelevantEqual);
	}

	ObjectReferencePackageMap = NewObject<UIrisObjectReferencePackageMap>();
}

UEngineReplicationBridge::~UEngineReplicationBridge()
{
}

void UEngineReplicationBridge::Deinitialize()
{
	if (NetDriver)
	{
		NetDriver->OnNetServerMaxTickRateChanged.RemoveAll(this);
		NetDriver->GetOnNetUpdateFrequencyChanged().RemoveAll(this);
		NetDriver = nullptr;
	}
	Super::Deinitialize();
	ObjectReferencePackageMap = nullptr;
	bProtocolMismatchMessageSent = false;
	SentDiagnosticProtocolIds.Empty();
	NumSentDiagnosticMessages = 0;
}

void UEngineReplicationBridge::OnPreSeamlessTravelGarbageCollect()
{
	Super::OnPreSeamlessTravelGarbageCollect();
	ensureMsgf(UE::Net::Private::bAlwaysCreateLevelFilteringGroupsForPersistentLevels, TEXT("Seamless travel requires setting cvar net.Iris.AlwaysCreateLevelFilteringGroupsForPersistentLevels to true"));
}

UE::Net::FNetRefHandle UEngineReplicationBridge::StartReplicatingActor(AActor* Actor)
{
	using namespace UE::Net;

	if (!ensureMsgf(Actor == nullptr || !(Actor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject)), TEXT("Actor %s is a CDO or Archetype and should not be replicated."), ToCStr(GetFullNameSafe(Actor))))
	{
		return FNetRefHandle::GetInvalid();
	}

	const bool bIsNetActor = ULevel::IsNetActor(Actor);
	if (!bIsNetActor)
	{
		UE_LOG_BRIDGEID(LogIris, VeryVerbose, TEXT("Actor %s doesn't have a NetRole."), ToCStr(GetFullNameSafe(Actor)));
		return FNetRefHandle::GetInvalid();
	}

	if (!AllowReplicationOfActorsWithAnyRole())
	{
		if (Actor->GetLocalRole() != ROLE_Authority)
		{
			UE_LOG_BRIDGEID(LogIris, VeryVerbose, TEXT("Actor %s NetRole isn't Authority."), ToCStr(GetFullNameSafe(Actor)));
			return FNetRefHandle::GetInvalid();
		}
	}

	if (Actor->IsActorBeingDestroyed() || !IsValid(Actor) || Actor->IsUnreachable())
	{
		UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Actor %s is being destroyed or unreachable and can't be replicated."), ToCStr(GetFullNameSafe(Actor)));
		return FNetRefHandle::GetInvalid();
	}

	if (!Actor->GetIsReplicated())
	{
		UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Actor %s is not supposed to be replicated."), ToCStr(GetFullNameSafe(Actor)));
		return FNetRefHandle::GetInvalid();
	}

	if (Actor->GetTearOff())
	{
		UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Actor %s is torn off and should not be replicated."), ToCStr(GetFullNameSafe(Actor)));
		return FNetRefHandle::GetInvalid();
	}

	if (!Actor->IsActorInitialized())
	{
		UE_LOG_BRIDGEID(LogIris, Warning, TEXT("Actor %s is not initialized and won't be replicated."), ToCStr(GetFullNameSafe(Actor)));
		return FNetRefHandle::GetInvalid();
	}

	if (!ensureMsgf(Actor->HasActorBegunPlay() || Actor->IsActorBeginningPlay(), TEXT("Actor %s hasn't begun or isn't beginning play in OnReplicationStartedForIris."), ToCStr(GetFullNameSafe(Actor))))
	{
		UE_LOG_BRIDGEID(LogIris, Error, TEXT("Actor %s hasn't begun or isn't beginning play in OnReplicationStartedForIris."), ToCStr(GetFullNameSafe(Actor)));
		return FNetRefHandle::GetInvalid();
	}

	if (!NetDriver)
	{
		UE_LOG_BRIDGEID(LogIris, VeryVerbose, TEXT("There's no NetDriver so nothing can be replicated."));
		return FNetRefHandle::GetInvalid();
	}


	if (!NetDriver->ShouldReplicateActor(Actor))
	{
		UE_LOG_BRIDGEID(LogIris, VeryVerbose, TEXT("Actor %s doesn't want to replicate with NetDriver %s."), ToCStr(GetFullNameSafe(Actor)), ToCStr(NetDriver->GetName()));
		return FNetRefHandle::GetInvalid();
	}

	// Initially dormant actors begin replication when their dormancy is flushed
	const ENetDormancy Dormancy = Actor->NetDormancy;	
	if (Actor->IsNetStartupActor() && (Dormancy == DORM_Initial))
	{
		return FNetRefHandle::GetInvalid();
	}

	FNetRefHandle ExistingHandle = GetReplicatedRefHandle(Actor);
	if (ExistingHandle.IsValid())
	{
		CancelDeferredActor(Actor);
	
		return ExistingHandle;
	}

	if (!Actor->IsUsingRegisteredSubObjectList())
	{
		// Ensure the first time to get attention!
		ensureMsgf(false, TEXT("Actor %s does not replicate subobjects using the registered SubObjectsLists, SubObjects will not replicate properly"), ToCStr(GetFullNameSafe(Actor)));
		UE_LOG_BRIDGEID(LogIris, Warning, TEXT("Actor %s does not replicate subobjects using the registered SubObjectsLists, SubObjects will not replicate properly"), ToCStr(GetFullNameSafe(Actor)));
	}

	FActorReplicationParams ActorParams;
	AActor::FFillReplicationParamsContext Context;
	// Need to call this to get any netfactory override
	Actor->FillReplicationParams(Context, ActorParams);

	// Find the factory ID if this actor does not use the default one.
	FNetObjectFactoryId FactoryId = ActorFactoryId;
	if (ActorParams.NetFactoryName != NAME_None)
	{
		const FNetObjectFactoryId CustomFactoryId = FNetObjectFactoryRegistry::GetFactoryIdFromName(ActorParams.NetFactoryName);
		ensureMsgf(CustomFactoryId != InvalidNetObjectFactoryId, TEXT("Could not find Factory: %s for Actor: %s"), *ActorParams.NetFactoryName.ToString(), *GetNameSafe(Actor));

		FactoryId = CustomFactoryId != InvalidNetObjectFactoryId ? CustomFactoryId : ActorFactoryId;
	}

	FNetRefHandle ActorRefHandle = StartReplicatingRootObject(Actor, FactoryId);

	if (!ActorRefHandle.IsValid())
	{
		const bool bIsFrontendProxy = (GetReplicationSystem()->GetProxyType() == EProxyType::Frontend);

		// If the replication system is a frontend proxy this condition can be encountered if attempting to start replicating
		// an actor that isn't found in the shared object reference cache (i.e. it wasn't replicated to a corresponding backend
		// proxy cache).
		UE_CLOGF(bIsFrontendProxy, LogIris, Warning, "Failed to create NetRefHandle for Actor Named %ls", ToCStr(Actor->GetName()));

		// If the replication system isn't a frontend proxy then this condition is unexpected and will ensure.
		ensureMsgf(bIsFrontendProxy, TEXT("Failed to create NetRefHandle for Actor Named %s"), ToCStr(Actor->GetName()));

		return FNetRefHandle::GetInvalid();
	}

    UE_CLOGF(Actor->bAlwaysRelevant, LogIris, Verbose, "StartReplicatingActor of AlwaysRelevant actor %ls", *PrintObjectFromNetRefHandle(ActorRefHandle));

	// Set owning connection filtering if actor is only relevant to owner
	{
		if (Actor->bOnlyRelevantToOwner && !Actor->bAlwaysRelevant)
		{
			// Only apply owner filter if we haven't force enabled a dynamic filter.
			constexpr bool bRequireForceEnabled = true;
			FName FilterProfile;
			FNetObjectFilterHandle FilterHandle = GetDynamicFilter(Actor->GetClass(), bRequireForceEnabled, FilterProfile);

			if (FilterHandle == InvalidNetObjectFilterHandle)
			{
				GetReplicationSystem()->SetFilter(ActorRefHandle, ToOwnerFilterHandle);
			}
		}
	}

	// Set if this is a NetTemporary
	{
		if (Actor->bNetTemporary)
		{
			GetReplicationSystem()->SetIsNetTemporary(ActorRefHandle);
		}
	}

	// Dormancy, we track all actors that does want to be dormant
	{
		if (Dormancy > DORM_Awake)
		{
			SetObjectWantsToBeDormant(ActorRefHandle, true);
		}
	}

	// Setup Level filtering
	AddRootObjectToContainerGroup(Actor, Actor->GetLevel());

	// If we have registered sub objects we replicate them as well
	const FSubObjectRegistry& ActorSubObjects = FSubObjectRegistryGetter::GetSubObjects(Actor);
	const TArray<FReplicatedComponentInfo>& ReplicatedComponents = FSubObjectRegistryGetter::GetReplicatedComponents(Actor);

	if (ActorSubObjects.GetRegistryList().Num() != 0 || ReplicatedComponents.Num() != 0)
	{
		const FSubObjectReplicationParams SubObjectParams { .RootObjectHandle = ActorRefHandle };
		// Start with the Actor's SubObjects (that is SubObjects that are not ActorComponents)
		for (const FSubObjectRegistry::FEntry& SubObjectInfo : ActorSubObjects.GetRegistryList())
		{
			UObject* SubObjectToReplicate = SubObjectInfo.GetSubObject();
			if (IsValid(SubObjectToReplicate) && SubObjectInfo.NetCondition != ELifetimeCondition::COND_Never)
			{
				FNetRefHandle SubObjectRefHandle = Super::StartReplicatingSubObject(SubObjectToReplicate, SubObjectParams, SubObjectFactoryId);
				if (SubObjectRefHandle.IsValid())
				{
					UObjectReplicationBridge::SetSubObjectNetCondition(SubObjectRefHandle, SubObjectInfo.NetCondition);
				}
			}
		}

		// Now the replicated ActorComponents and their SubObjects
		for (const FReplicatedComponentInfo& RepComponentInfo : ReplicatedComponents)
		{
			if (IsValid(RepComponentInfo.Component) && RepComponentInfo.NetCondition != ELifetimeCondition::COND_Never)
			{
				FReplicationSystemUtil::StartReplicatingActorComponent(Actor, RepComponentInfo.Component);
			}
		}
	}
	
	CancelDeferredActor(Actor);

	return ActorRefHandle;
}

UE::Net::FNetRefHandle UEngineReplicationBridge::StartReplicatingComponent(FNetRefHandle RootObjectHandle, UActorComponent* SubObject)
{
	using namespace UE::Net;

	if (!RootObjectHandle.IsValid())
	{
		return FNetRefHandle::GetInvalid();
	}

	AActor* Owner = SubObject->GetOwner();

	FNetRefHandle ReplicatedComponentHandle = GetReplicatedRefHandle(SubObject);
	const FReplicatedComponentInfo* RepComponentInfo = FSubObjectRegistryGetter::GetReplicatedComponentInfoForComponent(Owner, SubObject);

	if (!ReplicatedComponentHandle.IsValid())
	{
		if (!IsValid(SubObject)
			|| SubObject->IsUnreachable()
			|| !SubObject->GetIsReplicated()
			|| SubObject->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject)
		)
		{
			return FNetRefHandle::GetInvalid();
		}

		if (!SubObject->IsUsingRegisteredSubObjectList())
		{
			UE_LOG_BRIDGEID(LogIris, Warning, TEXT("ActorComponent %s does not replicate subobjects using the registered SubObjectsLists, SubObjects might not replicate properly."), ToCStr(GetFullNameSafe(SubObject)));
		}

		if (RepComponentInfo == nullptr || RepComponentInfo->NetCondition == ELifetimeCondition::COND_Never)
		{
			return FNetRefHandle::GetInvalid();
		}

		// Start replicating the subobject with its owner.
		// TODO: Add control over this from ActorComponent and support it from legacy replication system as well. JIRA UE-237066
		const FSubObjectReplicationParams Params
		{ 
			.RootObjectHandle = RootObjectHandle,
			.ChildSubObjectsReplicationOrder = UE::Net::Private::bDefaultReplicateActorComponentSubobjectsAfterComponent ? EChildSubObjectsReplicationOrder::AfterParent : EChildSubObjectsReplicationOrder::BeforeParent
		};
		ReplicatedComponentHandle = Super::StartReplicatingSubObject(SubObject, Params, SubObjectFactoryId);
	}

	if (!ReplicatedComponentHandle.IsValid())
	{
		return FNetRefHandle::GetInvalid();
	}

	// Update or set any conditionals
	SetSubObjectNetCondition(ReplicatedComponentHandle, RepComponentInfo->NetCondition);

	// Begin replication for any SubObjects registered by the component
	{
		const FSubObjectReplicationParams Params
		{
			.RootObjectHandle = RootObjectHandle,
			.ParentSubObjectHandle = ReplicatedComponentHandle,
		};
	
		for (const FSubObjectRegistry::FEntry& SubObjectInfo : RepComponentInfo->SubObjects.GetRegistryList())
		{
			UObject* SubObjectToReplicate = SubObjectInfo.GetSubObject();
			if (IsValid(SubObjectToReplicate))
			{
				FNetRefHandle SubObjectHandle = Super::StartReplicatingSubObject(SubObjectToReplicate, Params, SubObjectFactoryId);
				if (SubObjectHandle.IsValid())
				{
					SetSubObjectNetCondition(SubObjectHandle, SubObjectInfo.NetCondition);
				}
			}
		}
	}

	CancelDeferredComponent(SubObject);

	return ReplicatedComponentHandle;
}

UE::Net::FNetRefHandle UEngineReplicationBridge::StartReplicatingSubObject(UObject* SubObject, const FSubObjectReplicationParams& Params)
{
	return Super::StartReplicatingSubObject(SubObject, Params, SubObjectFactoryId);
}

void UEngineReplicationBridge::StopReplicatingActor(AActor* Actor, const FStopReplicatingActorParams& Params)
{
	using namespace UE::Net;
	
	CancelDeferredActor(Actor);

	FNetRefHandle RefHandle = GetReplicatedRefHandle(Actor, EGetRefHandleFlags::EvenIfGarbage);
	if (!RefHandle.IsValid())
	{
		// Already not replicated
		return;
	}

	UE_LOGF(LogIris, Verbose, "StopReplicatingActor %ls. Reason %ls ", *PrintObjectFromNetRefHandle(RefHandle), LexToString(Params.StopReplicatingReason));
	ensureMsgf(IsValid(Actor), TEXT("StopReplicatingActor called on invalid actor tied to handle: %s."), *PrintObjectFromNetRefHandle(RefHandle));

	const bool bIsServer = GetReplicationSystem()->IsServer();
	const bool bIsObjectDestroyed = Params.StopReplicatingReason == EStopReplicatingReason::Destroyed;

	// Store destruction info of static actors so their destruction can be replicated when a client loads their level in the future.
	if (bIsServer && bIsObjectDestroyed && RefHandle.IsStatic())
	{
		const UE::Net::FDestructionParameters DestructionParameters
		{
			.Location = Actor->GetActorLocation(),
			.Container = Actor->GetLevel(),
			.bUseDistanceBasedPrioritization = (Actor->bAlwaysRelevant == false),
			.NetFactoryId = ActorFactoryId,
		};
		StoreDestructionInfo(RefHandle, DestructionParameters);
	}
		
	EEndReplicationFlags Flags = ConvertToEndReplicationFlags(Params.StopReplicatingReason);
	if (Actor->bNetTemporary)
	{
		Flags |= EEndReplicationFlags::SkipPendingEndReplicationValidation;
	}
	StopReplicatingNetRefHandle(RefHandle, Flags);
}

EEndReplicationFlags UEngineReplicationBridge::ConvertToEndReplicationFlags(UE::Net::EStopReplicatingReason StopReplicatingReason) const
{
	using namespace UE::Net;

	EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::None;
	if (StopReplicatingReason == EStopReplicatingReason::Destroyed || StopReplicatingReason == EStopReplicatingReason::ProxyReuse)
	{
		EndReplicationFlags |= EEndReplicationFlags::Destroy | EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId;
	}
	
	if (StopReplicatingReason == EStopReplicatingReason::ProxyReuse)
	{
		EndReplicationFlags |= EEndReplicationFlags::ProxyReuse;
	}

	// If we are shutting down we do not need to validate that we are not detaching remote instances by accident.
	const bool bIsShuttingDown = (StopReplicatingReason == EStopReplicatingReason::EndPlayInEditor) || (StopReplicatingReason == EStopReplicatingReason::Quit) || NetDriver->GetPendingDestruction();

	// Streaming out a level can happen prior to all actors in it being destroyed/ending replication through replication so we don't want to validate they're detached by accident.
	const bool bIsStreamingOutLevel = (StopReplicatingReason == EStopReplicatingReason::RemovedFromWorld);

	if (bIsShuttingDown || bIsStreamingOutLevel)
	{
		EndReplicationFlags |= EEndReplicationFlags::SkipPendingEndReplicationValidation;
	}

	return EndReplicationFlags;
}

void UEngineReplicationBridge::StopReplicatingComponent(UActorComponent* ActorComponent, EEndReplicationFlags EndReplicationFlags)
{
	using namespace UE::Net;
	
	CancelDeferredComponent(ActorComponent);

	FNetRefHandle ComponentHandle = GetReplicatedRefHandle(ActorComponent, EGetRefHandleFlags::EvenIfGarbage);
	if (ComponentHandle.IsValid())
	{
		UE_LOGF(LogIris, Verbose, "StopReplicatingComponent for %ls %ls.", *GetNameSafe(ActorComponent), *ComponentHandle.ToString());
		ensureMsgf(IsValid(ActorComponent), TEXT("StopReplicatingActor called on invalid component tied to handle: %s."), *ComponentHandle.ToString());
		
		StopReplicatingNetRefHandle(ComponentHandle, EndReplicationFlags);
	}
}

void UEngineReplicationBridge::GetInitialDependencies(FNetRefHandle Handle, FNetDependencyInfoArray& OutDependencies) const
{
	using namespace UE::Net;

	uint32 DependencyCount = 0U;

	// $TODO: Cache create dependencies and do the lookup based on index

	// Handles with static names does not have have any initial dependencies (other than the reference itself)
	if (Handle.IsDynamic())
	{
		UObject* Object = GetReplicatedObject(Handle);
		if (!Object)
		{
			return;
		}

		if (AActor* Actor = Cast<AActor>(Object))
		{
			UObject* Archetype = nullptr;
			UObject* ActorLevel = nullptr;

			// ChildActor's need to be spawned from the ChildActorTemplate otherwise any non-replicated 
			// customized properties will be incorrect on the Client.
			if (UChildActorComponent* CAC = Actor->GetParentComponent())
			{
				Archetype = CAC->GetSpawnableChildActorTemplate();
			}
			if (Archetype == nullptr)
			{
				Archetype = Actor->GetArchetype();
			}
			ActorLevel = Actor->GetLevel();

			check( Archetype != nullptr );
			check( Actor->NeedsLoadForClient() );			// We have no business sending this unless the client can load
			check( Archetype->NeedsLoadForClient() );		// We have no business sending this unless the client can load

			// Add initial dependencies
	
			// Archetype
			OutDependencies.Emplace(FNetDependencyInfo(GetOrCreateObjectReference(Archetype)));
			
			// Level if it differs from the persistent level
			const bool bUsePersistentLevel = NetDriver->GetWorld()->PersistentLevel == ActorLevel;
			if (!bUsePersistentLevel)
			{
				OutDependencies.Emplace(FNetDependencyInfo(GetOrCreateObjectReference(ActorLevel)));
			}
		}
		else
		{
			// Add initial dependencies

			// SubObjects either have a dependency on their path relative the owner or a reference to their class
			if (const bool bIsNameStableForNetworking = Object->IsNameStableForNetworking())
			{
				OutDependencies.Emplace(FNetDependencyInfo(GetOrCreateObjectReference(Object)));
			}
			else
			{
				OutDependencies.Emplace(FNetDependencyInfo(GetOrCreateObjectReference(Object->GetClass())));
			}
		}
	}
}

void UEngineReplicationBridge::SetNetDriver(UNetDriver* const InNetDriver)
{
	if (NetDriver)
	{
		NetDriver->OnNetServerMaxTickRateChanged.RemoveAll(this);
		NetDriver->GetOnNetUpdateFrequencyChanged().RemoveAll(this);
	}

	NetDriver = InNetDriver;
	if (InNetDriver != nullptr)
	{
		SetMaxTickRate(static_cast<float>(FPlatformMath::Max(InNetDriver->GetNetServerMaxTickRate(), 0)));

		InNetDriver->OnNetServerMaxTickRateChanged.AddUObject(this, &UEngineReplicationBridge::OnMaxTickRateChanged);
		InNetDriver->GetOnNetUpdateFrequencyChanged().AddUObject(this, &UEngineReplicationBridge::OnNetUpdateFrequencyChanged);

		const FName RequiredChannelName = UObjectReplicationBridgeConfig::GetConfig()->GetRequiredNetDriverChannelClassName();
		
		if (!RequiredChannelName.IsNone())
		{
			const bool bRequiredChannelIsConfigured = InNetDriver->ChannelDefinitions.FindByPredicate(
				[RequiredChannelName](const FChannelDefinition& rhs)
				{
					return rhs.ClassName == RequiredChannelName;
				}) != nullptr;

			checkf(bRequiredChannelIsConfigured, TEXT("ObjectReplication needs the netdriver channel %s to work. Add this channel to the netdriver channel definitions config"), *RequiredChannelName.ToString());
		}

		bAllowExecutePreReplication = InNetDriver->IsPreReplicationEnabled();
		bAllowReplicationOfActorsWithAnyRole = InNetDriver->IsReplicationOfActorsWithAnyRoleEnabled();
	}
}

void UEngineReplicationBridge::OnMaxTickRateChanged(UNetDriver* InNetDriver, int32 NewMaxTickRate, int32 OldMaxTickRate)
{
	SetMaxTickRate(static_cast<float>(FPlatformMath::Max(InNetDriver->GetNetServerMaxTickRate(), 0)));
}

bool UEngineReplicationBridge::RemapPathForPIE(uint32 ConnectionId, FString& Str, bool bReading) const
{
	if (ConnectionId == UE::Net::InvalidConnectionId)
	{
		return GEngine->NetworkRemapPath(static_cast<UNetConnection*>(nullptr), Str, bReading);
	}
	else
	{
		UObject* UserData = GetReplicationSystem()->GetConnectionUserData(ConnectionId);
		UNetConnection* NetConnection = Cast<UNetConnection>(UserData);
		return GEngine->NetworkRemapPath(NetConnection, Str, bReading);
	}
}

bool UEngineReplicationBridge::ObjectContainerHasFinishedLoading(UObject* Object) const
{
	const UWorld* DriverWorld = Object && NetDriver ? NetDriver->GetWorld() : nullptr;
	if (DriverWorld)
	{
		// get the level for the object
		const ULevel* Level = Object->GetTypedOuter<ULevel>();

		if (Level != nullptr && Level != DriverWorld->PersistentLevel)
		{
			return Level->bIsVisible;
		}
	}

	return true;
}

UEngineReplicationBridge* UEngineReplicationBridge::Create(UNetDriver* NetDriver)
{
	UEngineReplicationBridge* Bridge = NewObject<UEngineReplicationBridge>(GetTransientPackage(), UEngineReplicationBridge::StaticClass());
	Bridge->SetNetDriver(NetDriver);

	return Bridge;
}

bool UEngineReplicationBridge::CanCreateDestructionInfo() const
{
	if(ensure(NetDriver))
	{
		return NetDriver->CanCreateDestructionInfo();
	}

	return false;
}

bool UEngineReplicationBridge::IsClassReplicatedByDefault(const UClass* Class) const
{
	if (Class && Class->IsChildOf(AActor::StaticClass()))
	{
		const AActor* CDO = Cast<AActor>(Class->GetDefaultObject());
		return CDO && CDO->GetIsReplicated();
	}

	return Super::IsClassReplicatedByDefault(Class);
}

void UEngineReplicationBridge::WakeUpObjectInstantiatedFromRemote(AActor* Actor) const
{
	// If the actor is already awake or can't be woken up then return immediately.
	if (Actor->NetDormancy <= DORM_Awake)
	{
		return;
	}

	ENetDormancy OldDormancy = Actor->NetDormancy;
	Actor->NetDormancy = DORM_Awake;

	if (!NetDriver)
	{
		return;
	}

	if (FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(NetDriver->GetWorld()))
	{
		for (FNamedNetDriver& Driver : WorldContext->ActiveNetDrivers)
		{
			if (Driver.NetDriver != nullptr && Driver.NetDriver != NetDriver && Driver.NetDriver->ShouldReplicateActor(Actor))
			{
				Driver.NetDriver->NotifyActorClientDormancyChanged(Actor, OldDormancy);
			}
		}
	}
}

void UEngineReplicationBridge::StartReplicatingActorDefer(AActor* Instance)
{
	if (bIsFlushDeferredRunning)
	{
		UE_LOGF(LogIris, Warning, "Cannot call UEngineReplicationBridge::StartReplicatingActorDefer() for %ls while flushing.", *GetNameSafe(Instance));
		return;
	}

	FDeferredStartStop* Def = DeferredStartStopCallsActors.Find(Instance);
	if (Def == nullptr)
	{
		// If this is the first call for this actor make sure it hasn't already started replication so it can be started.
		const bool bCanCallStart = !GetReplicatedRefHandle(Instance).IsValid();
		if (!bCanCallStart)
		{
			UE_LOGF(LogIris, Warning, "UEngineReplicationBridge::StartReplicatingActorDefer: First call for %ls failed because it's already replicating.", *GetNameSafe(Instance));
			return;
		}

		Def = &DeferredStartStopCallsActors.Add(Instance);
	}

	check(Def);

	const int32 NewCount = (Def->Count + 1);
	
	// Prevent deferring multiple, consecutive calls to start replication for the same actor.
	if (NewCount <= 1)
	{
		Def->Count = NewCount;
		UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::StartReplicatingActorDefer: %ls", *GetNameSafe(Instance));
	}
	else
	{
		UE_LOGF(LogIris, Warning, "UEngineReplicationBridge::StartReplicatingActorDefer: Already deferred this call for %ls", *GetNameSafe(Instance));
	}
}

void UEngineReplicationBridge::StopReplicatingActorDefer(AActor* Instance, const FStopReplicatingActorParams& Params)
{
	if (bIsFlushDeferredRunning)
	{
		UE_LOGF(LogIris, Warning, "Cannot call UEngineReplicationBridge::StopReplicatingActorDefer() for %ls while flushing.", *GetNameSafe(Instance));
		return;
	}

	FDeferredStartStop* Def = DeferredStartStopCallsActors.Find(Instance);
	if (Def == nullptr)
	{
		// If this is the first call for this actor make sure it's already started replication so it can be stopped.
		const bool bCanCallStop = GetReplicatedRefHandle(Instance).IsValid();
		if (!bCanCallStop)
		{
			UE_CLOGF(Instance->GetIsReplicated(), LogIris, Warning, "UEngineReplicationBridge::StopReplicatingActorDefer: First call for %ls failed because it's already not replicating.", *GetNameSafe(Instance));
			return;
		}

		Def = &DeferredStartStopCallsActors.Add(Instance);
	}

	check(Def);

	const int32 NewCount = (Def->Count - 1);

	// Prevent deferring multiple, consecutive calls to stop replication for the same actor.
	if (NewCount >= -1)
	{
		Def->Count = NewCount;
		Def->StopReplicatingActorParams = Params;
		UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::StopReplicatingActorDefer: %ls", *GetNameSafe(Instance));
	}
	else
	{
		UE_LOGF(LogIris, Warning, "UEngineReplicationBridge::StopReplicatingActorDefer: Already deferred this call for %ls", *GetNameSafe(Instance));
	}
}

void UEngineReplicationBridge::StartReplicatingComponentDefer(const UObject* RootObject, UActorComponent* Component)
{
	if (bIsFlushDeferredRunning)
	{
		UE_LOGF(LogIris, Warning, "Cannot call UEngineReplicationBridge::StartReplicatingComponentDefer() for %ls while flushing.", *GetNameSafe(Component));
		return;
	}

	FDeferredStartStop* Def = DeferredStartStopCallsComponents.Find(Component);
	if (Def == nullptr)
	{
		// If this is the first call for this component, make sure it hasn't already started replication so it can be started.
		const bool bCanCallStart = !GetReplicatedRefHandle(Component).IsValid();
		if (!bCanCallStart)
		{
			UE_LOGF(LogIris, Warning, "UEngineReplicationBridge::StartReplicatingComponentDefer: First call for %ls failed because it's already replicating.", *GetNameSafe(Component));
			return;
		}

		Def = &DeferredStartStopCallsComponents.Add(Component);
	}

	check(Def);

	const int32 NewCount = (Def->Count + 1);

	// Prevent deferring multiple, consecutive calls to start replication for the same component.
	if (NewCount <= 1)
	{
		Def->Count = NewCount;
		Def->RootObject = RootObject;
		UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::StartReplicatingComponentDefer: %ls", *GetNameSafe(Component));
	}
	else
	{
		UE_LOGF(LogIris, Warning, "UEngineReplicationBridge::StartReplicatingComponentDefer: Already deferred this call for %ls", *GetNameSafe(Component));
	}
}

void UEngineReplicationBridge::StopReplicatingComponentDefer(UActorComponent* Component, EEndReplicationFlags EndReplicationFlags)
{
	if (bIsFlushDeferredRunning)
	{
		UE_LOGF(LogIris, Warning, "Cannot call UEngineReplicationBridge::StopReplicatingComponentDefer() for %ls while flushing.", *GetNameSafe(Component));
		return;
	}

	FDeferredStartStop* Def = DeferredStartStopCallsComponents.Find(Component);
	if (Def == nullptr)
	{
		// If this is the first call for this component, make sure it's already started replication so it can be stopped.
		const bool bCanCallStop = GetReplicatedRefHandle(Component).IsValid();
		if (!bCanCallStop)
		{
			UE_CLOGF(Component->GetIsReplicated(), LogIris, Warning, "UEngineReplicationBridge::StopReplicatingComponentDefer: First call for %ls failed because it's already not replicating.", *GetNameSafe(Component));
			return;
		}

		Def = &DeferredStartStopCallsComponents.Add(Component);
	}

	check(Def);

	const int32 NewCount = (Def->Count - 1);

	// Prevent deferring multiple, consecutive calls to stop replication for the same component.
	if (NewCount >= -1)
	{
		Def->Count = NewCount;
		Def->StopReplicatingComponentFlags = EndReplicationFlags;
		UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::StopReplicatingComponentDefer: %ls", *GetNameSafe(Component));
	}
	else
	{
		UE_LOGF(LogIris, Warning, "UEngineReplicationBridge::StopReplicatingComponentDefer: Already deferred this call for %ls", *GetNameSafe(Component));
	}
}

void UEngineReplicationBridge::FlushDeferred()
{
	bIsFlushDeferredRunning = true;

	if (DeferredStartStopCallsActors.Num() > 0 || DeferredStartStopCallsComponents.Num() > 0)
	{
		UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::FlushDeferred: Actors=%d Components=%d", DeferredStartStopCallsActors.Num(), DeferredStartStopCallsComponents.Num());
	}

	// Actor replication needs to be processed before component replication in case the component depends
	// on an owning actor that hasn't started replicating yet.

	for (TPair<TObjectKey<AActor>, FDeferredStartStop>& Pair : DeferredStartStopCallsActors)
	{
		TObjectKey ActorKey = Pair.Key;
		FDeferredStartStop& Def = Pair.Value;

		// No start or stop was called, or the number of start and stop calls cancel out (i.e. do nothing).
		if (Def.Count == 0)
		{
			continue;
		}

		AActor* Actor = ActorKey.ResolveObjectPtr();
		if (!Actor)
		{
			// The actor no longer exists.
			continue;
		}

		// Call UEngineReplicationBridge::StartReplicatingActor()
		if (Def.Count > 0)
		{
			UE::Net::FNetRefHandle NewNetRefHandle = StartReplicatingActor(Actor);
			if (NewNetRefHandle.IsValid())
			{
				Actor->OnReplicationStartedForIris(AActor::FOnReplicationStartedParams());
			}
				
			UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::FlushDeferred: Called StartReplicatingActor for %ls (success=%d)", *Actor->GetName(), NewNetRefHandle.IsValid());
		}
				
		// Call UEngineReplicationBridge::StopReplicatingActor()
		else if (Def.Count < 0)
		{
			// Cache locally in case the DeferredStartStopCallsActors map is changed as a side-effect of the remaining logic.
			FStopReplicatingActorParams StopReplicatingActorParams = Def.StopReplicatingActorParams;

			AActor::FOnStopReplicationParams OnStopReplicationParams = { .StopReplicatingParams = StopReplicatingActorParams };
			Actor->OnStopReplicationForIris(OnStopReplicationParams);
			StopReplicatingActor(Actor, StopReplicatingActorParams);

			UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::FlushDeferred: Called StopReplicatingActor for %ls", *Actor->GetName());
		}
	}

	for (TPair<TObjectKey<UActorComponent>, FDeferredStartStop>& Pair : DeferredStartStopCallsComponents)
	{
		TObjectKey<UActorComponent> ActorComponentKey = Pair.Key;
		FDeferredStartStop& Def = Pair.Value;

		// No start or stop was called, or the number of start and stop calls cancel out (i.e. do nothing).
		if (Def.Count == 0)
		{
			continue;
		}

		UActorComponent* ActorComponent = ActorComponentKey.ResolveObjectPtr();
		if (!ActorComponent)
		{
			// The component no longer exists.
			continue;
		}

		// Call UEngineReplicationBridge::StartReplicatingComponent()
		if (Def.Count > 0)
		{
			UObject* RootObject = Def.RootObject.ResolveObjectPtr();
			if (RootObject)
			{
				FNetRefHandle RootObjectHandle = GetReplicatedRefHandle(RootObject);
				if (!RootObjectHandle.IsValid())
				{
					UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::FlushDeferred: Cannot find handle for component's root object %ls", *RootObject->GetName());
					continue;
				}

				UE::Net::FNetRefHandle NewNetRefHandle = StartReplicatingComponent(RootObjectHandle, ActorComponent);
				if (NewNetRefHandle.IsValid())
				{
					ActorComponent->OnReplicationStartedForIris(UActorComponent::FOnReplicationStartedParams());
				}

				UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::FlushDeferred: Called StartReplicatingComponent for %ls (success=%d)", *ActorComponent->GetName(), NewNetRefHandle.IsValid());
			}
		}

		// Call UEngineReplicationBridge::StopReplicatingComponent()
		else if (Def.Count < 0)
		{
			ActorComponent->OnStopReplicationForIris(UActorComponent::FOnStopReplicationParams());
			StopReplicatingComponent(ActorComponent, Def.StopReplicatingComponentFlags);

			UE_LOGF(LogIris, Verbose, "UEngineReplicationBridge::FlushDeferred: Called StopReplicatingComponent for %ls", *ActorComponent->GetName());
		}
	}

	bIsFlushDeferredRunning = false;

	DeferredStartStopCallsActors.Reset();
	DeferredStartStopCallsComponents.Reset();
}

void UEngineReplicationBridge::OnProtocolMismatchDetected(FNetRefHandle ObjectHandle, const TArray<uint8>& CDOStateBytes)
{
	Super::OnProtocolMismatchDetected(ObjectHandle, CDOStateBytes);

	// As a client tell the server we could not bind this specific NetRefHandle
	if (NetDriver && NetDriver->ServerConnection)
	{
		uint64 RawHandleId = ObjectHandle.GetId();

		if (!bProtocolMismatchMessageSent && CDOStateBytes.Num() > 0)
		{
			// We must const_cast because Send requires non-const. CDOStateBytes is NOT modified.
			FNetControlMessage<NMT_IrisProtocolMismatchWithCDOState>::Send(NetDriver->ServerConnection, RawHandleId, const_cast<TArray<uint8>&>(CDOStateBytes));
			bProtocolMismatchMessageSent = true;
		}
		else
		{
			FNetControlMessage<NMT_IrisProtocolMismatch>::Send(NetDriver->ServerConnection, RawHandleId);
		}
	}
}

void UEngineReplicationBridge::OnProtocolMismatchReported(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes)
{
	Super::OnProtocolMismatchReported(RefHandle, ConnectionId, ClientCDOStateBytes);

	// If we are the server force the client to disconnect since not replicating a critical class will prevent the game from working.
	if (NetDriver && NetDriver->IsServer())
	{
		const UObject* ReplicatedObject = GetReplicatedObject(RefHandle);

		// If the object instance doesn't exist anymore, pass a null class anyway in case the config wants to disconnect on ALL class types.
		const UClass* ObjectClass = ReplicatedObject ? ReplicatedObject->GetClass() : nullptr;

		if (IsClassCritical(ObjectClass))
		{
			if (UNetConnection* ClientConnection = NetDriver->GetConnectionByHandle(UE::Net::FConnectionHandle(ConnectionId)))
			{
				FString ErrorMsg = FString::Printf(TEXT("Protocol mismatch: %s:%s. Class: %s"), *RefHandle.ToString(), *GetNameSafe(ReplicatedObject), *GetNameSafe(ObjectClass));
				UE_LOGF(LogIris, Error, "%ls: Closing connection due to: %ls", ToCStr(ClientConnection->Describe()), ToCStr(ErrorMsg));
				{
					UE::Net::FNetCloseResult CloseReason = ENetCloseResult::IrisProtocolMismatch;
					ClientConnection->SendCloseReason(MoveTemp(CloseReason));
				}

				FNetControlMessage<NMT_Failure>::Send(ClientConnection, ErrorMsg);
				ClientConnection->FlushNet(true);

				{
					UE::Net::FNetCloseResult CloseReason = ENetCloseResult::IrisProtocolMismatch;
					ClientConnection->Close(MoveTemp(CloseReason));
				}
			}
		}
	}
}

void UEngineReplicationBridge::SendErrorWithNetRefHandle(UE::Net::ENetRefHandleError ErrorType, FNetRefHandle RefHandle, uint32 ConnectionId, TConstArrayView<const FNetRefHandle> ExtraNetRefHandle, const UE::Net::FNetErrorContext& ErrorContext)
{
	if (NetDriver)
	{
		if (UNetConnection* ClientConnection = NetDriver->GetConnectionByHandle(UE::Net::FConnectionHandle(ConnectionId)))
		{
			uint64 HandleId = RefHandle.GetId();

			TArray<uint64> ArrayIds;
			ArrayIds.Reserve(ExtraNetRefHandle.Num());
			for(const FNetRefHandle ExtraHandle : ExtraNetRefHandle)
			{
				ArrayIds.Emplace(ExtraHandle.GetId());
			}

			const FString& DiagnosticMessage = ErrorContext.GetErrorDiagnostic();
			const UE::Net::FReplicationProtocolIdentifier ProtocolId = ErrorContext.GetErrorProtocolId();

			const bool bAlreadySentForProtocol = SentDiagnosticProtocolIds.Contains(ProtocolId);
			const bool bGlobalCapReached = NumSentDiagnosticMessages >= UE::Net::Private::MaxNetRefHandleErrorDiagnosticMessages;

			if (DiagnosticMessage.IsEmpty() || bAlreadySentForProtocol || bGlobalCapReached)
			{
				FNetControlMessage<NMT_IrisNetRefHandleError>::Send(ClientConnection, ErrorType, HandleId, ArrayIds);
			}
			else
			{
				FNetControlMessage<NMT_IrisNetRefHandleErrorWithDiagnosticMessage>::Send(ClientConnection, ErrorType, HandleId, ArrayIds, const_cast<FString&>(DiagnosticMessage));
				SentDiagnosticProtocolIds.Add(ProtocolId);
				++NumSentDiagnosticMessages;
			}
		}
		else
		{
			UE_LOGF(LogIris, Error, "UEngineReplicationBridge::SendErrorWithNetRefHandle could not find Connection for id:%u", ConnectionId);
		}
	}
}

void UEngineReplicationBridge::ActorChangedLevel(const AActor* Actor, const ULevel* PreviousLevel)
{
	if (!UE::Net::Private::bEnableActorLevelChanges)
	{
		return;
	}

	UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("ActorChangedLevel: Actor %s from PreviousLevel %s"), *GetFullNameSafe(Actor), *GetFullNameSafe(PreviousLevel));

	if (!UE::Net::Private::ShouldIncludeRootObjectInLevelGroups(Actor))
	{
		return;
	}

	// Remove from previous level group
	if (PreviousLevel)
	{
		const UPackage* const PreviousLevelPackage = PreviousLevel->GetOutermost();
		const FName PreviousLevelPackageName = PreviousLevelPackage->GetFName();

		const UE::Net::FNetRefHandle ActorRefHandle = GetReplicatedRefHandle(Actor);
		UE::Net::FNetObjectGroupHandle PreviousContainerGroup = GetContainerGroup(PreviousLevel);
		if (GetReplicationSystem()->IsInGroup(PreviousContainerGroup, ActorRefHandle))
		{
			UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("ActorChangedLevel: removing %s from GroupIndex: %u PreviousLevel: %s"), *ActorRefHandle.ToString(), PreviousContainerGroup.GetGroupIndex(), ToCStr(PreviousLevelPackageName.ToString()));
			GetReplicationSystem()->RemoveFromGroup(PreviousContainerGroup, ActorRefHandle);
		}
		else if (PreviousContainerGroup.IsValid())
		{
			UE_LOG_BRIDGEID(LogIris, Warning, TEXT("ActorChangedLevel: %s not found in GroupIndex: %u PreviousLevel: %s"), *ActorRefHandle.ToString(), PreviousContainerGroup.GetGroupIndex(), ToCStr(PreviousLevelPackageName.ToString()));
		}
	}

	AddRootObjectToContainerGroup(Actor, Actor->GetLevel());
}

void UEngineReplicationBridge::OnNetUpdateFrequencyChanged(const AActor* Actor)
{
	if (UE::Net::Private::bEnableDynamicNetUpdateFrequency)
	{
		const UE::Net::FNetRefHandle ActorHandle = GetReplicatedRefHandle(Actor);

		if (ActorHandle != UE::Net::FNetRefHandle::GetInvalid())
		{
			SetPollFrequency(ActorHandle, Actor->GetNetUpdateFrequency());
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UEngineReplicationBridge::AddRootObjectToLevelGroup(const UObject* RootObject, const ULevel* Level)
{
	AddRootObjectToContainerGroup(RootObject, Level);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UEngineReplicationBridge::AddRootObjectToContainerGroup(const UObject* RootObject, const UObject* Container)
{
	if (!UE::Net::Private::ShouldIncludeRootObjectInLevelGroups(RootObject))
	{
		return;
	}
	
	// For seamless travel we need to maintain a level group for all kinds of levels, including persistent ones.
	if (const ULevel* Level = Cast<ULevel>(Container))
	{
		if (UE::Net::Private::bAlwaysCreateLevelFilteringGroupsForPersistentLevels || !Level->IsPersistentLevel() || (Level != NetDriver->GetWorld()->PersistentLevel))
		{
			const UPackage* const LevelPackage = Level->GetOutermost();
			const FString PackageName = LevelPackage->GetName();

			UE::Net::FNetObjectGroupHandle LevelGroup = GetContainerGroup(Level);
			if (!LevelGroup.IsValid())
			{
				// When a level is unloaded and reloaded immediately and when LevelStreaming.ShouldReuseUnloadedButStillAroundLevels is false, 
				// we can end up with 2 different sets of level pointers that are have the same original package name.
				// Make the package name unique so a new group gets assigned to it.
				FString LevelGroupPackageUniqueName = PackageName;
				int32 Index = 0;
				while (GetReplicationSystem()->FindGroup(*LevelGroupPackageUniqueName).IsValid())
				{
					++Index;
					LevelGroupPackageUniqueName = FString::Printf(TEXT("%s_%d"), *PackageName, Index);
				}

				LevelGroup = CreateContainerGroup(Level, *LevelGroupPackageUniqueName);

				UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Created new GroupIndex: %u for Level: %s using Package Unique Name: %s"), LevelGroup.GetGroupIndex(), ToCStr(PackageName), ToCStr(LevelGroupPackageUniqueName));

				// Update the filtering status of the group based on current level visibility for all connections
				NetDriver->UpdateGroupFilterStatusForLevel(Level, LevelGroup);
			}

			const UE::Net::FNetRefHandle RootObjectRefHandle = GetReplicatedRefHandle(RootObject);

			// Add object to group
			UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Added %s to GroupIndex: %u Level: %s"), *RootObjectRefHandle.ToString(), LevelGroup.GetGroupIndex(), ToCStr(PackageName));
			GetReplicationSystem()->AddToGroup(LevelGroup, RootObjectRefHandle);
		}
	}
	else if (const UPackage* Package = Cast<UPackage>(Container))
	{
		UE::Net::FNetObjectGroupHandle ContainerGroup = GetContainerGroup(Package);

		if (!ContainerGroup.IsValid())
		{
			const FName ContainerName = Package->GetFName();
			ContainerGroup = CreateContainerGroup(Package, ContainerName);

			UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Created new GroupIndex: %u for Level: %s using Package Unique Name: %s"), ContainerGroup.GetGroupIndex(), ToCStr(ContainerName.ToString()), ToCStr(ContainerName.ToString()));
		}

		const UE::Net::FNetRefHandle RootObjectRefHandle = GetReplicatedRefHandle(RootObject);

		// Add object to group
		UE_LOG_BRIDGEID(LogIris, Verbose, TEXT("Added %s to GroupIndex: %u Container: %s"), *RootObjectRefHandle.ToString(), ContainerGroup.GetGroupIndex(), ToCStr(Package->GetPackage()->GetName()));
		GetReplicationSystem()->AddToGroup(ContainerGroup, RootObjectRefHandle);
	}
	else
	{
		ensureMsgf(false, TEXT("Iris requires replicated container group to be a ULevel or a UPackage, Container: %s"), ToCStr(GetFullNameSafe(Container)));
	}
}

FString UEngineReplicationBridge::PrintConnectionInfo(uint32 ConnectionId) const
{
	if (NetDriver)
	{
		if (UNetConnection* ClientConnection = NetDriver->GetConnectionByHandle(UE::Net::FConnectionHandle(ConnectionId)))
		{
			return FString::Printf(TEXT("ConnectionId:%u ViewTarget: %s Named: %s"), ConnectionId, *GetNameSafe(ClientConnection->ViewTarget), *ClientConnection->Describe());
		}
		else
		{
			return FString::Printf(TEXT("ConnectionId:%u no NetConnection found"), ConnectionId);
		}
	}
	else
	{
		return FString::Printf(TEXT("ConnectionId:%u no NetDriver attached"), ConnectionId);
	}
}

void UEngineReplicationBridge::ConsumeNetMetrics(TArray<FAnalyticsEventAttribute>& OutAttrs)
{
	using namespace UE::Net;

	FNetMetrics MetricsCollector;

	GetReplicationSystem()->CollectNetMetrics(MetricsCollector);

	const TMap<FName, FNetMetric>& Metrics = MetricsCollector.GetMetrics();

	for (auto& It : Metrics)
	{
		FName MetricName = It.Key;
		const FNetMetric& Metric = It.Value;

		switch (Metric.GetDataType())
		{
			case FNetMetric::EDataType::Signed:
			{
				const int32 Value = Metric.GetSigned();
				OutAttrs.Add(FAnalyticsEventAttribute(MetricName.ToString(), Value));
			} break;

			case FNetMetric::EDataType::Unsigned:
			{
				const uint32 Value = Metric.GetUnsigned();
				OutAttrs.Add(FAnalyticsEventAttribute(MetricName.ToString(), Value));
			} break;

			case FNetMetric::EDataType::Double:
			{
				const double Value = Metric.GetDouble();
				OutAttrs.Add(FAnalyticsEventAttribute(MetricName.ToString(), Value));
			} break;

			default:
			{
				checkNoEntry();
			} break;
		}
	}

	GetReplicationSystem()->ResetNetMetrics();
}
bool UEngineReplicationBridge::AllowExecutePreReplication() const
{
	return bAllowExecutePreReplication;
}

bool UEngineReplicationBridge::AllowReplicationOfActorsWithAnyRole() const
{
	return bAllowReplicationOfActorsWithAnyRole;
}

void UEngineReplicationBridge::CancelDeferredActor(AActor* Actor)
{
	if (bIsFlushDeferredRunning)
	{
		return;
	}

	UE_CLOGF(DeferredStartStopCallsActors.Contains(Actor), LogIris, Verbose, "UEngineReplicationBridge::CancelDeferredActor: Cancelling deferred %ls.", *GetNameSafe(Actor));
	DeferredStartStopCallsActors.Remove(Actor);
}

void UEngineReplicationBridge::CancelDeferredComponent(UActorComponent* ActorComponent)
{
	if (bIsFlushDeferredRunning)
	{
		return;
	}

	UE_CLOGF(DeferredStartStopCallsComponents.Contains(ActorComponent), LogIris, Verbose, "UEngineReplicationBridge::CancelDeferredComponent: Cancelling deferred %ls.", *GetNameSafe(ActorComponent));
	DeferredStartStopCallsComponents.Remove(ActorComponent);
}