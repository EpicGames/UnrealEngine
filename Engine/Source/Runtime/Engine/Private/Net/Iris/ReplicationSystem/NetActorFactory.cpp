// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Iris/ReplicationSystem/NetActorFactory.h"

#include "Engine/Level.h"
#include "Engine/NetDriver.h"
#include "Engine/World.h"

#include "GameFramework/Actor.h"

#include "HAL/UnrealMemory.h"
#include "HAL/LowLevelMemStats.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "Net/DataBunch.h"
#include "Net/Core/Connection/ConnectionHandle.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"

#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "Stats/Stats.h"

#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetActorFactory)

namespace UE::Net::Private
{
	static int32 SerializeNewActorMaxBunchSize = 1024;
	static FAutoConsoleVariableRef CVarSerializeNewActorMaxBunchSize(
	TEXT("net.Iris.SerializeNewActorMaxBunchSize"),
	SerializeNewActorMaxBunchSize,
	TEXT("Max allowed bits that can be added to the creation header via OnSerializeNewActor."));

	static FName NetActorFactoryName("NetActorFactory");

	EActorNetSpawnInfoFlags GetSpawnInfoFlags()
	{
		// Disable performance warnings on FindConsoleVariable. We we call it once per NetActorFactory that is created, but in tests this can be thousands of times.
		constexpr bool bWarnOnFrequentFindCVar = false;

		// Init spawninfo flags from CVARs
		EActorNetSpawnInfoFlags Flags = EActorNetSpawnInfoFlags::None;
		{
			bool bQuantizeActorScaleOnSpawn = false;
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.QuantizeActorScaleOnSpawn"), bWarnOnFrequentFindCVar);
			if (ensure(CVar))
			{
				bQuantizeActorScaleOnSpawn = CVar->GetBool();
			}
			Flags |= bQuantizeActorScaleOnSpawn ? EActorNetSpawnInfoFlags::QuantizeScale : EActorNetSpawnInfoFlags::None;
		}

		{
			bool bQuantizeActorLocationOnSpawn = true;
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.QuantizeActorLocationOnSpawn"), bWarnOnFrequentFindCVar);
			if (ensure(CVar))
			{
				bQuantizeActorLocationOnSpawn = CVar->GetBool();
			}
			Flags |= bQuantizeActorLocationOnSpawn ? EActorNetSpawnInfoFlags::QuantizeLocation : EActorNetSpawnInfoFlags::None;
		}

		{
			bool bQuantizeActorVelocityOnSpawn = true;
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.QuantizeActorVelocityOnSpawn"), bWarnOnFrequentFindCVar);
			if (ensure(CVar))
			{
				bQuantizeActorVelocityOnSpawn = CVar->GetBool();
			}
			Flags |= bQuantizeActorVelocityOnSpawn ? EActorNetSpawnInfoFlags::QuantizeVelocity : EActorNetSpawnInfoFlags::None;
		}

		return Flags;
	}

	
} // end namespace UE::Net::Private

//------------------------------------------------------------------------
// UNetActorFactory
//------------------------------------------------------------------------

FName UNetActorFactory::GetFactoryName()
{
	return UE::Net::Private::NetActorFactoryName;
}

void UNetActorFactory::OnInit()
{
	Super::OnInit();
	SpawnInfoFlags = UE::Net::Private::GetSpawnInfoFlags();
}

void UNetActorFactory::FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams)
{
	using namespace UE::Net;

	AActor* Actor = Cast<AActor>(Context.Object);
	if (!Actor)
	{
		UE_LOGF(LogIris, Error, "UNetActorFactory only supports AActor derived classes. Got %ls", ToCStr(GetNameSafe(Context.Object ? Context.Object->GetClass() : nullptr)));
		ensureMsgf(false, TEXT("UNetActorFactory only supports AActor derived classes. Got %s"), ToCStr(GetNameSafe(Context.Object ? Context.Object->GetClass() : nullptr)));
		return;
	}

	AActor::FFillReplicationParamsContext ActorParamsContext;
	FActorReplicationParams ActorParams;
	Actor->FillReplicationParams(ActorParamsContext, ActorParams);

	OutParams.bNeedsPreUpdate = true,
		OutParams.bNeedsWorldLocationUpdate = true,
		OutParams.bIsDormant = Actor->NetDormancy > DORM_Awake,
		OutParams.StaticPriority = (Actor->bAlwaysRelevant || Actor->bOnlyRelevantToOwner) ? Actor->NetPriority : 0.0f,
		OutParams.PollFrequency = Actor->GetNetUpdateFrequency();

#if !UE_BUILD_SHIPPING
	ensureMsgf(!(Actor->bAlwaysRelevant || Actor->bOnlyRelevantToOwner) || OutParams.StaticPriority >= 1.0f, TEXT("Very low NetPriority %.02f for always relevant or owner relevant Actor %s. Set it to 1.0f or higher."), Actor->NetPriority, ToCStr(Actor->GetName()));
#endif

	// Assign a specific filter if requested
	if (ActorParams.FilterType != FActorReplicationParams::ConfigFilter)
	{
		OutParams.bUseExplicitDynamicFilter = true;

		switch (ActorParams.FilterType)
		{
		case FActorReplicationParams::AlwaysRelevant:
			OutParams.ExplicitDynamicFilterName = NAME_None;
			break;

		case FActorReplicationParams::DefaultSpatial:
			OutParams.ExplicitDynamicFilterName = Bridge->GetDefaultSpatialFilterName();
			break;

		case FActorReplicationParams::ExplicitFilter:
			ensureMsgf(!ActorParams.ExplicitDynamicFilterName.IsNone(), TEXT("FActorReplicationParams::ExplicitDynamicFilterName must be defined when FilterType is ExplicitFilter"));
			OutParams.ExplicitDynamicFilterName = ActorParams.ExplicitDynamicFilterName;
			break;
		}
	}
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetActorFactory::CreateAndFillHeader(UE::Net::FNetRefHandle Handle)
{
	using namespace UE::Net;

	AActor* Actor = Cast<AActor>(Bridge->GetReplicatedObject(Handle));
	
	if (!Actor)
	{
		ensureMsgf(Actor, TEXT("UNetActorFactory::CreateAndFillHeader could not find actor tied to handle: %s"), *Bridge->PrintObjectFromNetRefHandle(Handle));
		return nullptr;
	}

	const FNetObjectReference ActorReference = Bridge->GetOrCreateObjectReference(Actor);

	TUniquePtr<FBaseActorNetCreationHeader> BaseHeader;

	// Allocate the right header type for this actor
	if (!ActorReference.GetRefHandle().IsDynamic())
	{
		BaseHeader.Reset(new FStaticActorNetCreationHeader);
	}
	else
	{
		if (Bridge->IsNetRefHandlePreRegistered(Handle))
		{
			BaseHeader.Reset(new FPreRegisteredActorNetCreationHeader);
		}
		else
		{
			BaseHeader.Reset(new FDynamicActorNetCreationHeader);
		}
	}


	const bool bSuccess = FillHeader(BaseHeader.Get(), Handle, Actor);
	
	return bSuccess ? MoveTemp(BaseHeader) : nullptr;
}

bool UNetActorFactory::FillHeader(UE::Net::FBaseActorNetCreationHeader* BaseHeader, UE::Net::FNetRefHandle Handle, AActor* Actor)
{
	using namespace UE::Net;

	if (!BaseHeader->IsDynamic())
	{
		FStaticActorNetCreationHeader* Header = static_cast<FStaticActorNetCreationHeader*>(BaseHeader);
		
		// Refer by path for static actors
		Header->ObjectReference = Bridge->GetOrCreateObjectReference(Actor);;
	}
	else if (BaseHeader->IsPreregistered())
	{
		// Nothing to fill!
	}
	else
	{
		FDynamicActorNetCreationHeader* Header = static_cast<FDynamicActorNetCreationHeader*>(BaseHeader);

		// This is more or less a straight copy from ClientPackageMap and needs to be updated accordingly
		UObject* Archetype = nullptr;
		UObject* ActorLevel = nullptr;

		// ChildActor's need to be spawned from the ChildActorTemplate otherwise any non-replicated 
		// customized properties will be incorrect on the Client.
		if (const UChildActorComponent* CAC = Actor->GetParentComponent())
		{
			Archetype = CAC->GetSpawnableChildActorTemplate();
		}

		if (Archetype == nullptr)
		{
			Archetype = Actor->GetArchetype();
		}

		ActorLevel = Actor->GetLevel();

		check(Archetype != nullptr);
		check(Actor->NeedsLoadForClient());			// We have no business sending this unless the client can load
		check(Archetype->NeedsLoadForClient());		// We have no business sending this unless the client can load

		UEngineReplicationBridge* EngineBridge = CastChecked<UEngineReplicationBridge>(Bridge);

		// Fill in Header
		Header->ArchetypeReference = Bridge->GetOrCreateObjectReference(Archetype);
		Header->bUsePersistentLevel = (UE::Net::Private::SerializeNewActorOverrideLevel == 0) || (EngineBridge->GetNetDriver()->GetWorld()->PersistentLevel == ActorLevel);

		if (!Header->bUsePersistentLevel)
		{
			Header->LevelReference = Bridge->GetOrCreateObjectReference(ActorLevel);
		}

		if (const USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			Header->SpawnInfo.Location = FRepMovement::RebaseOntoZeroOrigin(Actor->GetActorLocation(), Actor);
			Header->SpawnInfo.Rotation = Actor->GetActorRotation();
			Header->SpawnInfo.Scale = Actor->GetActorScale();
			FVector Scale = Actor->GetActorScale();

			if (USceneComponent* AttachParent = RootComponent->GetAttachParent())
			{
				// If this actor is attached, when the scale is serialized on the client, the attach parent property won't be set yet.
				// USceneComponent::SetWorldScale3D (which got called by AActor::SetActorScale3D, which we used to do but no longer).
				// would perform this transformation so that what is sent is relative to the parent. If we don't do this, we will
				// apply the world scale on the client, which will then get applied a second time when the attach parent property is received.
				FTransform ParentToWorld = AttachParent->GetSocketTransform(RootComponent->GetAttachSocketName());
				Scale = Scale * ParentToWorld.GetSafeScaleReciprocal(ParentToWorld.GetScale3D());
			}

			Header->SpawnInfo.Scale = Scale;
			Header->SpawnInfo.Velocity = Actor->GetVelocity();
		}
		else
		{
			check(!Header->SpawnInfo.Location.ContainsNaN());
		}
	}

	// Finally fill the custom actor creation data for all.
	{
		FOutBunch Bunch(UE::Net::Private::SerializeNewActorMaxBunchSize);
		Actor->OnSerializeNewActor(Bunch);
		BaseHeader->CustomCreationDataBitCount = IntCastChecked<uint16>(Bunch.GetNumBits());
		if (BaseHeader->CustomCreationDataBitCount > 0)
		{
			BaseHeader->CustomCreationData.SetNumZeroed(Align(Bunch.GetNumBytes(), 4));
			FMemory::Memcpy(BaseHeader->CustomCreationData.GetData(), Bunch.GetData(), Bunch.GetNumBytes());
		}
	}

	return true;
}

bool UNetActorFactory::SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header) 
{
	using namespace UE::Net;

	FNetBitStreamWriter* Writer = Context.Serialization.GetBitStreamWriter();

	const FBaseActorNetCreationHeader* BaseActorHeader = static_cast<const FBaseActorNetCreationHeader*>(Header);

	// Serialize the type of header it is (static -> preregistered -> dynamic)
	const bool bIsDynamic = Writer->WriteBool(BaseActorHeader->IsDynamic());
	
	if (bIsDynamic)
	{
		const bool bIsPreRegistered = Writer->WriteBool(BaseActorHeader->IsPreregistered());
	}

	// Serialize the content of the header
	return BaseActorHeader->Serialize(Context);
	
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetActorFactory::CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context)
{
	using namespace UE::Net;

	TUniquePtr<FBaseActorNetCreationHeader> Header;

	FNetBitStreamReader* Writer = Context.Serialization.GetBitStreamReader();

	// Deserialize the type of header and create it
	const bool bIsDynamic = Writer->ReadBool();
	if (bIsDynamic)
	{
		const bool bIsPreRegistered = Writer->ReadBool();

		if (!bIsPreRegistered)
		{
			Header.Reset(new FDynamicActorNetCreationHeader);
		}
		else
		{
			Header.Reset(new FPreRegisteredActorNetCreationHeader);
		}
	}
	else
	{
		Header.Reset(new FStaticActorNetCreationHeader);
	}

	// Deserialize the content of the header
	Header->Deserialize(Context);
	
	return Header;
}

UNetObjectFactory::FInstantiateResult UNetActorFactory::InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	LLM_SCOPE(ELLMTag::EngineMisc);
	IRIS_PROFILER_SCOPE(NetActorFactory_InstantiateReplicatedObjectFromHeader);

	UEngineReplicationBridge* ActorBridge = CastChecked<UEngineReplicationBridge>(Bridge);
	UNetDriver* NetDriver = ActorBridge->GetNetDriver();

	const FBaseActorNetCreationHeader* BaseHeader = static_cast<const FBaseActorNetCreationHeader*>(Header);

	// For static actors, just find the object using the path
	if (!BaseHeader->IsDynamic())
	{
		const FStaticActorNetCreationHeader* StaticHeader = static_cast<const FStaticActorNetCreationHeader*>(Header);

		AActor* Actor = Cast<AActor>(ActorBridge->ResolveObjectReference(StaticHeader->ObjectReference, Context.ResolveContext));
		if (!Actor)
		{
			FInstantiateResult InstantiateResult;
			if (UE_LOG_ACTIVE(LogIris, Error))
			{
				const FString FailedObjectReferenceStr = ActorBridge->DescribeObjectReference(StaticHeader->ObjectReference, Context.ResolveContext);
				UE_LOGF(LogIris, Error, "UNetActorFactory::InstantiateNetObjectFromHeader Failed to resolve ObjectReference: %ls . Could not find static actor.", *FailedObjectReferenceStr);
				InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Static actor not found. ObjectReference: %s"), *FailedObjectReferenceStr);
			}
			return InstantiateResult;
		}

		UE_LOGF(LogIris, Verbose, "UNetActorFactory::InstantiateNetObjectFromHeader Found static Actor: %ls using ObjectReference: %ls", ToCStr(Actor->GetPathName()), *ActorBridge->DescribeObjectReference(StaticHeader->ObjectReference, Context.ResolveContext));

		FInstantiateResult Result {  .Instance = Actor };
			
		if (NetDriver->ShouldClientDestroyActor(Actor))
		{
			Result.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;
		}

		return Result;
	}

	// If pre-registered, look for the existing instance.
	if (BaseHeader->IsPreregistered())
	{
		// Nothing to read in the header for now

		AActor* FoundActor = Cast<AActor>(Bridge->GetPreRegisteredObject(Context.Handle));
		if (!FoundActor)
		{
			FInstantiateResult InstantiateResult;
			if (UE_LOG_ACTIVE(LogIris, Error))
			{
				UE_LOGF(LogIris, Error, "UNetActorFactory::InstantiateReplicatedObjectFromHeader Unable to find pre-registered actor: %ls", *Context.Handle.ToString());
				InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Pre-registered actor not found. Handle: %s"), *Context.Handle.ToString());
			}
			return InstantiateResult;
		}

		FInstantiateResult Result{ .Instance = FoundActor };

		if (NetDriver->ShouldClientDestroyActor(FoundActor))
		{
			Result.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;
		}

		return Result;
	}

	// For dynamic actors, spawn a new actor using the provided information, or find a pre-registered actor
	const FDynamicActorNetCreationHeader* DynamicHeader = static_cast<const FDynamicActorNetCreationHeader*>(Header);

	// Find archetype
	AActor* Archetype = Cast<AActor>(Bridge->ResolveObjectReference(DynamicHeader->ArchetypeReference, Context.ResolveContext));
	if (!Archetype)
	{
		FInstantiateResult InstantiateResult;
		if (UE_LOG_ACTIVE(LogIris, Error))
		{
			const FString FailedArchetypeReferenceStr = Bridge->DescribeObjectReference(DynamicHeader->ArchetypeReference, Context.ResolveContext);
			UE_LOGF(LogIris, Error, "UNetActorFactory::InstantiateNetObjectFromHeader Unable to instantiate actor, failed to resolve archetype: %ls", *FailedArchetypeReferenceStr);
			InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Actor archetype not found. Archetype: %s"), *FailedArchetypeReferenceStr);
		}
		return InstantiateResult;
	}

	// Find level
	ULevel* Level = nullptr;
	if (!DynamicHeader->bUsePersistentLevel)
	{
		Level = Cast<ULevel>(Bridge->ResolveObjectReference(DynamicHeader->LevelReference, Context.ResolveContext));
	}

	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Archetype->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Archetype->GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(Archetype, Archetype->GetClass());
			
	// For streaming levels, it's possible that the owning level has been made not-visible but is still loaded.
	// In that case, the level will still be found but the owning world will be invalid.
	// If that happens, wait to spawn the Actor until the next time the level is streamed in.
	// At that point, the Server should resend any dynamic Actors.
			
	check(Level == nullptr || Level->GetWorld() != nullptr);

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Template = Archetype;
	SpawnInfo.OverrideLevel = Level;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bRemoteOwned = true;
	SpawnInfo.bNoFail = true;

	UWorld* World = NetDriver->GetWorld();
	FVector SpawnLocation = FRepMovement::RebaseOntoLocalOrigin(DynamicHeader->SpawnInfo.Location, World->OriginLocation);
		
	AActor* Actor = World->SpawnActorAbsolute(Archetype->GetClass(), FTransform(DynamicHeader->SpawnInfo.Rotation, SpawnLocation), SpawnInfo);

	// For Iris we expect that we will be able to spawn the actor as streaming always is controlled from server
	if (!Actor)
	{
		FInstantiateResult InstantiateResult;
		if (UE_LOG_ACTIVE(LogIris, Error))
		{
			const FString ArchetypeNameStr = GetNameSafe(Archetype);
			const FString ObjectReferenceStr = Bridge->DescribeObjectReference(DynamicHeader->ArchetypeReference, Context.ResolveContext);
			ensureMsgf(Actor, TEXT("UNetActorFactory::InstantiateNetObjectFromHeader SpawnActor failed. Used Archetype: %s ObjectReference: %s"), *ArchetypeNameStr, *ObjectReferenceStr);
			InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("SpawnActor failed. Archetype: %s, ObjectReference: %s"), *ArchetypeNameStr, *ObjectReferenceStr);
		}
		else
		{
			// We still want to ensure if the LogIris Error is disabled, but only construct the strings if ensures are enabled.
			ensureMsgf(Actor, TEXT("UNetActorFactory::InstantiateNetObjectFromHeader SpawnActor failed. Used Archetype: %s ObjectReference: %s"), *GetNameSafe(Archetype), *Bridge->DescribeObjectReference(DynamicHeader->ArchetypeReference, Context.ResolveContext));
		}
		return InstantiateResult;
	}

	static constexpr float Epsilon = UE_KINDA_SMALL_NUMBER;

	// Set Velocity if it differs from Default
	if (!DynamicHeader->SpawnInfo.Velocity.Equals(DefaultSpawnInfo.Velocity, Epsilon))
	{
		Actor->PostNetReceiveVelocity(DynamicHeader->SpawnInfo.Velocity);
	}
					
	// Set Scale if it differs from Default
	if (!DynamicHeader->SpawnInfo.Scale.Equals(DefaultSpawnInfo.Scale, Epsilon))
	{
		Actor->SetActorRelativeScale3D(DynamicHeader->SpawnInfo.Scale);
	}

	FInstantiateResult Result { .Instance = Actor };

	if (NetDriver->ShouldClientDestroyActor(Actor))
	{
		Result.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;
	}

	UE_LOGF(LogIris, Verbose, "UNetActorFactory::InstantiateNetObjectFromHeader Spawned Actor: %ls using Archetype: %ls", *Actor->GetPathName(), *GetNameSafe(Archetype));

	// Mark actor as pending its PostNetInit
	Actor->SetActorIsPendingPostNetInit(true);

	return Result;
}

void UNetActorFactory::PostInstantiation(const FPostInstantiationContext& Context)
{
	using namespace UE::Net;

	AActor* Actor = CastChecked<AActor>(Context.Instance);
	if (!Actor)
	{
		return;
	}

	UEngineReplicationBridge* ActorBridge = CastChecked<UEngineReplicationBridge>(Bridge);
	UNetDriver* NetDriver = ActorBridge->GetNetDriver();

	const FBaseActorNetCreationHeader* BaseHeader = static_cast<const FBaseActorNetCreationHeader*>(Context.Header);

	// OnActorChannelOpen
	{
		UNetConnection* Connection = NetDriver->GetConnectionByHandle(FConnectionHandle(Context.ConnectionId));
		FInBunch Bunch(Connection, const_cast<uint8*>(BaseHeader->CustomCreationData.GetData()), BaseHeader->CustomCreationDataBitCount);
		Actor->OnActorChannelOpen(Bunch, Connection);

		if (Bunch.IsError() || Bunch.GetBitsLeft() != 0)
		{
			UE_LOGF(LogIris, Error, "UNetActorFactory::PostInstantiation deserialization error in OnActorChannelOpen for Actor: %ls", *Actor->GetPathName());
			check(false);
			return;
		}
	}

	// Wake up from dormancy. This is important for client replays.
	ActorBridge->WakeUpObjectInstantiatedFromRemote(Actor);
}

void UNetActorFactory::PostInit(const FPostInitContext& Context)
{
	AActor* Actor = CastChecked<AActor>(Context.Instance);
	if (Actor)
	{
		// We now know that we have applied initial state
		Actor->SetActorIsPendingPostNetInit(false);
	
		// PostNetInit is only called for dynamic actors
		if (Context.Handle.IsDynamic())
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(PostNetInit);
			QUICK_SCOPE_CYCLE_COUNTER(Stat_PostNetInit);
			LLM_SCOPE_BYNAME(TEXT("UObject/NetworkPostInit"));
			LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Actor->GetPackage(), ELLMTagSet::Assets);
			LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Actor->GetClass(), ELLMTagSet::AssetClasses);
			UE_TRACE_METADATA_SCOPE_ASSET(Actor, Actor->GetClass());
			Actor->PostNetInit();
		}
	}
}

void UNetActorFactory::SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated)
{
	AActor* RootActor = Cast<AActor>(Bridge->GetReplicatedObject(RootObject));
	UObject* SubObject = Bridge->GetReplicatedObject(SubObjectCreated);
	if (IsValid(RootActor) && IsValid(SubObject))
	{
		RootActor->OnSubobjectCreatedFromReplication(SubObject);
	}
}

void UNetActorFactory::SubObjectDetachedFromReplication(const FDetachContext& Context, const FSubObjectDetachContext& SubObjectContext)
{
	using namespace UE::Net;

	if (!Context.DetachedInstance)
	{
		return;
	}

	if (Context.Reason == EDetachReason::TornOff)
	{
		return;
	}

	const bool bIsStable = Context.DetachedInstance->IsNameStableForNetworking();
	const bool bWillBeDestroyed = !bIsStable;

	if (!bWillBeDestroyed)
	{
		return;
	}

	if (AActor* ActorOwner = Cast<AActor>(SubObjectContext.RootObject))
	{
		ActorOwner->OnSubobjectDestroyFromReplication(Context.DetachedInstance);
	}
}

void UNetActorFactory::DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext)
{
	using namespace UE::Net;

	AActor* Actor = Cast<AActor>(Context.DetachedInstance);
	if (!Actor)
	{
		// Nothing to do since the actor instance has been detached already
		return;
	}

	UEngineReplicationBridge* ActorBridge = CastChecked<UEngineReplicationBridge>(Bridge);
	UNetDriver* NetDriver = ActorBridge->GetNetDriver();

	const bool bIsDynamic = !Actor->IsNameStableForNetworking();
	const bool bCanBeDestroyed = (bIsDynamic || Context.Reason == EDetachReason::StaticDestroyed) && NetDriver->ShouldClientDestroyActor(Actor);

	if (Context.Reason == EDetachReason::TornOff)
	{
		if (!NetDriver->ShouldClientDestroyTearOffActors())
		{
			NetDriver->ClientSetActorTornOff(Actor);
		}
		else if (bCanBeDestroyed)
		{
			Actor->PreDestroyFromReplication();
			Actor->Destroy(true);
		}
	}
	else if (bCanBeDestroyed)
	{
		// Note that subobjects have already been detached by the ReplicationBridge
		Actor->PreDestroyFromReplication();
		Actor->Destroy(true);
	}
}

TOptional<UNetActorFactory::FWorldInfoData> UNetActorFactory::GetWorldInfo(const FWorldInfoContext& Context) const
{
	using namespace UE::Net;

	const AActor* Actor = Cast<AActor>(Context.Instance);
	if (!Actor)
	{
		// This is typical when an actor has been destroyed locally but not replicated to all clients yet.
		UE_LOGF(
			LogIris,
			VeryVerbose,
			"UNetActorFactory::GetWorldInfo requested for non-existent actor (likely destroyed): %ls | NetRefHandle: %ls",
			*GetNameSafe(Context.Instance),
			*Context.Handle.ToString());
		return NullOpt;
	}

	FWorldInfoData OutData;
	
	if (EnumHasAnyFlags(Context.InfoRequested, EWorldInfoRequested::Location))
	{
		OutData.WorldLocation = Actor->GetActorLocation();
	}
	if (EnumHasAnyFlags(Context.InfoRequested, EWorldInfoRequested::CullDistance))
	{
		OutData.CullDistance = Actor->GetNetCullDistanceSquared() > 0.0f ? FMath::Sqrt(Actor->GetNetCullDistanceSquared()) : 0.0f;
	}

	return OutData;
}

 namespace UE::Net
 {

//------------------------------------------------------------------------
// FBaseActorNetCreationHeader
//------------------------------------------------------------------------

EActorNetSpawnInfoFlags FBaseActorNetCreationHeader::GetFactorySpawnFlags(const UNetActorFactory* ActorFactory) const
{
	return ActorFactory->SpawnInfoFlags;
}

bool FBaseActorNetCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.Serialization.GetBitStreamWriter();

	const bool bHasCustomCreationData = Writer->WriteBool(CustomCreationDataBitCount > 0);
	if (bHasCustomCreationData)
	{
		Writer->WriteBits(CustomCreationDataBitCount - 1U, 16U);
		Writer->WriteBitStream(reinterpret_cast<const uint32*>(CustomCreationData.GetData()), 0U, CustomCreationDataBitCount);
	}

	return Writer->IsOverflown() == false;
}

bool FBaseActorNetCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	FNetBitStreamReader* Reader = Context.Serialization.GetBitStreamReader();

	const bool bHasCustomCreationData = Reader->ReadBool();
	if (bHasCustomCreationData)
	{
		CustomCreationDataBitCount = 1U + Reader->ReadBits(16U);
		CustomCreationData.SetNumZeroed(((CustomCreationDataBitCount + 31U) & ~31U) >> 3U);
		Reader->ReadBitStream(reinterpret_cast<uint32*>(CustomCreationData.GetData()), CustomCreationDataBitCount);
	}

	return Reader->IsOverflown() == false;
}

//------------------------------------------------------------------------
// FStaticActorNetCreationHeader
//------------------------------------------------------------------------

bool FStaticActorNetCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	WriteFullNetObjectReference(Context.Serialization, ObjectReference);

	return Super::Serialize(Context);
}

bool FStaticActorNetCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	ReadFullNetObjectReference(Context.Serialization, ObjectReference);
	
	return Super::Deserialize(Context);
}


FString FStaticActorNetCreationHeader::ToString() const
{
	return FString::Printf(TEXT("\n\tFStaticActorNetCreationHeader (ProtocolId:0x%x):\n\t"
								"ObjectReference=%s\n\t"
								"CustomCreationData=%u bits"),
						   		GetProtocolId(),
						   		*ObjectReference.ToString(),
								CustomCreationDataBitCount);
}

//------------------------------------------------------------------------
// FDynamicActorNetCreationHeader
//------------------------------------------------------------------------

bool FDynamicActorNetCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const UNetActorFactory* ActorFactory = CastChecked<UNetActorFactory>(Context.Factory);

	FNetBitStreamWriter* Writer = Context.Serialization.GetBitStreamWriter();

	// Write Archetype and LevelPath
	WriteFullNetObjectReference(Context.Serialization, ArchetypeReference);
		
	// Only write the LevelPath if it differs from the persistent level
	if (!Writer->WriteBool(bUsePersistentLevel))
	{
		WriteFullNetObjectReference(Context.Serialization, LevelReference);
	}

	const EActorNetSpawnInfoFlags SpawnFlags = GetFactorySpawnFlags(ActorFactory);
	const bool bQuantizeLocation = EnumHasAnyFlags(SpawnFlags, EActorNetSpawnInfoFlags::QuantizeLocation);
	const bool bQuantizeScale = EnumHasAnyFlags(SpawnFlags, EActorNetSpawnInfoFlags::QuantizeScale);
	const bool bQuantizeVelocity = EnumHasAnyFlags(SpawnFlags, EActorNetSpawnInfoFlags::QuantizeVelocity);

	// Write actor spawn info
	WriteConditionallyQuantizedVector(Writer, SpawnInfo.Location, ActorFactory->DefaultSpawnInfo.Location, bQuantizeLocation);
	WriteConditionallyQuantizedVector(Writer, SpawnInfo.Scale, ActorFactory->DefaultSpawnInfo.Scale, bQuantizeScale);
	WriteConditionallyQuantizedVector(Writer, SpawnInfo.Velocity, ActorFactory->DefaultSpawnInfo.Velocity, bQuantizeVelocity);

	// For rotation we use 0.001f for Rotation comparison to keep consistency with old behavior.
	static constexpr float RotationEpsilon = 0.001f;
	WriteRotator(Writer, SpawnInfo.Rotation, ActorFactory->DefaultSpawnInfo.Rotation, RotationEpsilon);

	return Super::Serialize(Context);
}

bool FDynamicActorNetCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const UNetActorFactory* ActorFactory = CastChecked<UNetActorFactory>(Context.Factory);

	FNetBitStreamReader* Reader = Context.Serialization.GetBitStreamReader();

	// Read Archetype
	ReadFullNetObjectReference(Context.Serialization, ArchetypeReference);
		
	bUsePersistentLevel = Reader->ReadBool();
	if (!bUsePersistentLevel)
	{
		ReadFullNetObjectReference(Context.Serialization, LevelReference);
	}

	// Read actor spawn info
	ReadConditionallyQuantizedVector(Reader, SpawnInfo.Location, ActorFactory->DefaultSpawnInfo.Location);
	ReadConditionallyQuantizedVector(Reader, SpawnInfo.Scale, ActorFactory->DefaultSpawnInfo.Scale);
	ReadConditionallyQuantizedVector(Reader, SpawnInfo.Velocity, ActorFactory->DefaultSpawnInfo.Velocity);

	ReadRotator(Reader, SpawnInfo.Rotation, ActorFactory->DefaultSpawnInfo.Rotation);
	
	return Super::Deserialize(Context);
}


FString FDynamicActorNetCreationHeader::ToString() const
{
	return FString::Printf(TEXT("\n\tFDynamicActorNetCreationHeader (ProtocolId:0x%x):\n\t"
								"ArchetypeReference=%s\n\t"
								"SpawnInfo.Location=%s\n\t"
								"SpawnInfo.Rotation=%s\n\t"
								"SpawnInfo.Scale=%s\n\t"
								"SpawnInfo.Velocity=%s\n\t"		
								"bUsePersistentLevel=%s\n\t"
								"LevelReference=%s\n\t"
								"CustomCreationData=%u bits"),
						   		GetProtocolId(),
								*ArchetypeReference.ToString(),
								*SpawnInfo.Location.ToCompactString(),
								*SpawnInfo.Rotation.ToCompactString(),
								*SpawnInfo.Scale.ToCompactString(),
								*SpawnInfo.Velocity.ToCompactString(),		
								bUsePersistentLevel?TEXT("True"):TEXT("False"),
								*LevelReference.ToString(),
								CustomCreationDataBitCount);
}

//------------------------------------------------------------------------
// FPreRegisteredActorNetCreationHeader
//------------------------------------------------------------------------
bool FPreRegisteredActorNetCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	// Nothing to serialize
	return Super::Serialize(Context);
}

bool FPreRegisteredActorNetCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	// Nothing to deserialize
	return Super::Deserialize(Context);
}

FString FPreRegisteredActorNetCreationHeader::ToString() const
{
	return FString::Printf(TEXT("\n\tFPreRegisteredActorNetCreationHeader (ProtocolId:0x%x):\n\t"
								"CustomCreationData=%u bits"),
						   		GetProtocolId(),
						   		CustomCreationDataBitCount);
}

} // end namespace UE::Net


