// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Iris/ReplicationSystem/NetEngineFactories.h"

#include "Net/Iris/ReplicationSystem/NetActorFactory.h"
#include "Net/Iris/ReplicationSystem/NetSubObjectFactory.h"
#include "Net/Iris/ReplicationSystem/NetRootObjectFactory.h"

#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"

namespace UE::Net
{

namespace Private
{
	static bool bAreFactoriesRegistered = false;
}

void InitEngineNetObjectFactories()
{
	using namespace UE::Net::Private;

	if (ensure(bAreFactoriesRegistered == false))
	{
		FNetObjectFactoryRegistry::RegisterFactory(UNetActorFactory::StaticClass(), UNetActorFactory::GetFactoryName());
		FNetObjectFactoryRegistry::RegisterFactory(UNetSubObjectFactory::StaticClass(), UNetSubObjectFactory::GetFactoryName());
		FNetObjectFactoryRegistry::RegisterFactory(UNetRootObjectFactory::StaticClass(), UNetRootObjectFactory::GetFactoryName());
		bAreFactoriesRegistered = true;
	}
}

void ShutdownEngineNetObjectFactories()
{
	using namespace UE::Net::Private;

	if (bAreFactoriesRegistered)
	{
		FNetObjectFactoryRegistry::UnregisterFactory(UNetActorFactory::GetFactoryName());
		FNetObjectFactoryRegistry::UnregisterFactory(UNetSubObjectFactory::GetFactoryName());
		FNetObjectFactoryRegistry::UnregisterFactory(UNetRootObjectFactory::GetFactoryName());
		bAreFactoriesRegistered = false;
	}
}

} // end namespace UE::Net
