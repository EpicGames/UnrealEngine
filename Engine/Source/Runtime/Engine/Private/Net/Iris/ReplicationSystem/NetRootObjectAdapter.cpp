// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Iris/ReplicationSystem/NetRootObjectAdapter.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/NetDriver.h"

#include "Iris/Core/IrisLog.h"

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"

#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/NetRootObjectFactory.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

namespace UE::Net::Private
{
	static const FName NotRoutedFilter(TEXT("NotRouted"));
}

namespace UE::Net
{

FNetRootObjectAdapter::FNetRootObjectAdapter(const FRootObjectSettings& Settings)
{
	Configure(Settings);
}

void FNetRootObjectAdapter::InitAdapter(UObject* ReplicatedObject)
{
	check(ReplicatedObject);

	if (!ReplicatedObject->IsSupportedForNetworking())
	{
		ensureMsgf(false, TEXT("Can't replicate %s unless IsSupportedForNetworking is true"), *GetNameSafe(ReplicatedObject));
		return;
	}

	if (ReplicatedObject->IsNameStableForNetworking() && !ReplicatedObject->IsFullNameStableForNetworking())
	{
		UE_LOGF(LogIris, Error, "RootObject %ls is stable, but not all of it's Outer are stable. This is not supported and the object needs to be forced dynamic to be able to replicate autonomously.", *GetNameSafe(ReplicatedObject));

		ensureMsgf(false, TEXT("Cannot replicate %s because it's stable but not all of it's Outers are stable."), *GetNameSafe(ReplicatedObject));
		return;
	}

	WeakReplicatedObject = ReplicatedObject;
	check(WeakReplicatedObject.IsValid());

	bIsInitialized = true;
}

void FNetRootObjectAdapter::DeinitAdapter()
{
	ensureMsgf(!bIsReplicating, TEXT("DeinitAdapter on %s called while the object is still replicating. The object will stay in the replication system!"), *GetNameSafe(WeakReplicatedObject.Get()));

	WeakReplicatedObject.Reset();
	WeakLevel.Reset();

	bIsInitialized = false;
}

void FNetRootObjectAdapter::Configure(const FRootObjectSettings& Settings)
{
	if (bIsReplicating)
	{
		ensureMsgf(!bIsReplicating, TEXT("Cannot configure: %s after it started replicating"), *GetNameSafe(WeakReplicatedObject.Get()));
		return;
	}

	ensureMsgf(Settings.bIsAlwaysRelevant != Settings.bIsNotRouted, TEXT("Wrong config for: %s. Setting must be either exclusively AlwaysRelevant or bNotRouted."), *GetNameSafe(WeakReplicatedObject.Get()));

	bIsAlwaysRelevant = Settings.bIsAlwaysRelevant;
	bIsNotRouted = Settings.bIsNotRouted;
	bOnlyReplicateWhenLinked = Settings.bOnlyReplicateWhenLinked;

	ExplicitNetFactoryName = Settings.FactoryName;
}

void FNetRootObjectAdapter::SetAttachedLevel(ULevel* InLevel, ELevelValidation LevelValidation)
{
	TWeakObjectPtr<ULevel> NewWeakLevel(InLevel);

	// Nothing to do if the level didn't change.
	if (NewWeakLevel.IsValid() && NewWeakLevel == WeakLevel)
	{
		return;
	}

	if (bIsReplicating)
	{
		ensureMsgf(!bIsReplicating, TEXT("Cannot set attached level after object: %s is replicating"), *GetNameSafe(WeakReplicatedObject.Get()));
		return;
	}

#if DO_ENSURE
	if (LevelValidation == ELevelValidation::All)
	{
		if (WeakReplicatedObject.IsValid())
		{
			ULevel* RepObjectLevel = FindLevelForObject(WeakReplicatedObject.Get());

			ensureMsgf(RepObjectLevel == nullptr || RepObjectLevel == InLevel, TEXT("RootObjectAdapter for %s is using level: %s but the object has level: %s as an outer. If that is intentional pass 'IgnoreObjectOuter' to disable this warning."),
				*GetNameSafe(WeakReplicatedObject.Get()), *GetNameSafe(InLevel->OwningWorld), *GetNameSafe(RepObjectLevel->OwningWorld));
		}
	}
#endif 

	WeakLevel = InLevel;
	ensureMsgf(InLevel == nullptr || WeakLevel.IsValid(), TEXT("Cannot attach to an invalid level"));
}

FName FNetRootObjectAdapter::GetNetFactoryName() const
{
	// If no name was set, use the default RootObjectFactory
	return ExplicitNetFactoryName.IsNone() ? UNetRootObjectFactory::GetFactoryName() : ExplicitNetFactoryName;
}

void FNetRootObjectAdapter::SetNetFactoryName(FName InFactoryName)
{
	if (ensureMsgf(!bIsReplicating, TEXT("Cannot change factories after replication started")))
	{
		ExplicitNetFactoryName = InFactoryName;
	}
}

bool FNetRootObjectAdapter::ValidateRootObject()
{
	if (!bIsInitialized)
	{
		ensureMsgf(false, TEXT("FNetRootObjectAdapter must be initialized before starting replication"));
		return false;
	}

	UObject* ReplicatedObject = WeakReplicatedObject.Get();
	if (!ReplicatedObject)
	{
		ensureMsgf(false, TEXT("FNetRootObjectAdapter cannot start replication without a valid object"));
		return false;
	}

	// Validate the attached level
	{
		ULevel* AttachedLevel = WeakLevel.Get();

		// No levels set yet, find one by asking the object
		if (AttachedLevel == nullptr)
		{
			ULevel* RepObjectLevel = FindLevelForObject(ReplicatedObject);

			if (!RepObjectLevel)
			{
				ensureMsgf(false, TEXT("FNetRootObjectAdapter needs a ULevel to replicate object: %s"), *GetNameSafe(ReplicatedObject));
				return false;
			}

			SetAttachedLevel(RepObjectLevel, ELevelValidation::IgnoreObjectOuter);
		}
	}

	ULevel* Level = WeakLevel.Get();
	if (!Level)
	{
		return false;
	}

	UWorld* World = Level->GetWorld();
	if (!World)
	{
		ensureMsgf(World, TEXT("Level: %s does not have an owning world"), *GetNameSafe(Level));
		return false;
	}
	
	if (World->bIsTearingDown)
	{
		UE_LOGF(LogIris, Warning, "%s The world of %ls is being torn down.", __FUNCTION__, *GetNameSafe(ReplicatedObject));
		return false;
	}

	return true;
}

ULevel* FNetRootObjectAdapter::FindLevelForObject(UObject* InObject) const
{
	if (!InObject)
	{
		return nullptr;
	}

	// Start by asking the object its world
	UWorld* World = InObject->GetWorld();
	if (World && World->PersistentLevel)
	{
		return World->PersistentLevel;
	}
	
	// Fallback to directly polling its Outer's
	return InObject->GetTypedOuter<ULevel>();
}

void FNetRootObjectAdapter::StartReplication(ULevel* InLevel)
{
	if (bIsReplicating)
	{
		// Already replicating, nothing to do
		return;
	}

	if (InLevel)
	{
		SetAttachedLevel(InLevel);
	}

	// Make sure the object can be replicated
	const bool bValidForReplication = ValidateRootObject();
	
	if (!bValidForReplication)
	{
		return;
	}

	UObject* ReplicatedObject = WeakReplicatedObject.Get();
	check(ReplicatedObject);
	ULevel* Level = WeakLevel.Get();
	check(Level);
	UWorld* World = Level->GetWorld();
	check(World);

	UE_LOGF(LogIris, Verbose, "%s for: %ls (level: %ls)", __FUNCTION__, *GetNameSafe(ReplicatedObject), *GetNameSafe(Level));

	// Make sure Iris is enabled in the GameNetDriver
	if (World->GetNetDriver() && !World->GetNetDriver()->IsUsingIrisReplication())
	{
		ensureMsgf(false, TEXT("NetRootObjectAdapter is only compatible with the Iris replication system."));
		return;
	}

	const FName NetFactoryName = GetNetFactoryName();
	const FNetObjectFactoryId NetFactoryId = FNetObjectFactoryRegistry::GetFactoryIdFromName(NetFactoryName);
	if (UNLIKELY(NetFactoryId == InvalidNetObjectFactoryId))
	{
		ensureMsgf(NetFactoryId != InvalidNetObjectFactoryId, TEXT("NetFactory: %s is not a valid factory name"), *NetFactoryName.ToString());
		return;
	}

	FReplicationSystemUtil::ForEachServerBridge(World, [&](UEngineReplicationBridge* Bridge)
	{
		const FNetRefHandle NetRefHandle = Bridge->StartReplicatingRootObject(ReplicatedObject, NetFactoryId);

		if (!NetRefHandle.IsValid())
		{
			UE_LOGF(LogIris, Error, "%s Error starting replication for RootObject: %ls", __FUNCTION__, *GetNameSafe(ReplicatedObject));
			ensure(false);
			return;
		}

		bIsReplicating = true;

		// Setup level filtering
		Bridge->AddRootObjectToContainerGroup(ReplicatedObject, Level);

		//TODO: Support a subobject list in the adapter ?
	});
}

void FNetRootObjectAdapter::StopReplication()
{
	if (!bIsReplicating)
	{
		// Was never replicating, nothing to do
		return;
	}

	ULevel* Level = WeakLevel.Get();
	if (!Level)
	{
		ensureMsgf(Level, TEXT("StopReplication for %s called after his level was destroyed"), *GetNameSafe(WeakReplicatedObject.Get()));
		return;
	}

	UWorld* World = Level->GetWorld();
	if (!World)
	{
		ensureMsgf(World, TEXT("Level: %s does not have an owning world"), *GetNameSafe(Level));
		return;
	}

	UObject* ReplicatedObject = WeakReplicatedObject.GetEvenIfUnreachable();
	if (!ReplicatedObject)
	{
		UE_CLOGF(!WeakReplicatedObject.IsExplicitlyNull(), LogIris, Warning, "%s failed because the root object was destroyed before StopReplication", __FUNCTION__);
		
		return;
	}

	UE_LOGF(LogIris, Verbose, "%s for: %ls (level: %ls)", __FUNCTION__, *GetNameSafe(ReplicatedObject), *GetNameSafe(Level));

	// Warn if we were still attached to dependents
	if (!DependentRootObjects.IsEmpty())
	{
		UE_LOGF(LogIris, Warning, "%s for: %ls while still dependent to %d root objects. Dependencies will be cleared.", __FUNCTION__, *GetNameSafe(ReplicatedObject), DependentRootObjects.Num());

		// Iris dependencies will be destroyed by StopReplicatingNetRefHandle so let's clear the refcounts here
		DependentRootObjects.Empty();
	}

	FReplicationSystemUtil::ForEachServerBridge(World, [&](UEngineReplicationBridge* Bridge)
	{
		FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(ReplicatedObject, EGetRefHandleFlags::EvenIfGarbage);
		if (!RefHandle.IsValid())
		{
			// Already not replicated
			return;
		}

		//TODO: These default flags should probably be automatic or given by the bridge.
		const EEndReplicationFlags DefaultFlags = EEndReplicationFlags::Destroy | EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId;

		// Stop replicating the object
		Bridge->StopReplicatingNetRefHandle(RefHandle, DefaultFlags);
	});

	bIsReplicating = false;
}

void FNetRootObjectAdapter::RelevantWith(const UObject* OtherReplicatedObject)
{
	check(OtherReplicatedObject);

	if (!bIsInitialized)
	{
		ensureMsgf(bIsInitialized, TEXT("RootObjectAdapter cannot replicate unless it's initialized to an object"));
		return;
	}

	UObject* MyReplicatedObject = WeakReplicatedObject.Get();
	if (!MyReplicatedObject)
	{
		ensureMsgf(MyReplicatedObject, TEXT("RootObjectAdapter assigned to a destroyed object. Something went wrong"));
		return;
	}

	if (MyReplicatedObject == OtherReplicatedObject)
	{
		ensureMsgf(MyReplicatedObject != OtherReplicatedObject, TEXT("Cannot call RelevantWith on yourself: %s"), *GetNameSafe(MyReplicatedObject));
		return;
	}

	const bool bValidForReplication = ValidateRootObject();
	
	if (!bValidForReplication)
	{
		return;
	}

	ULevel* Level = WeakLevel.Get();
	check(Level);
	UWorld* World = Level->GetWorld();
	check(World);
	
	UE_LOGF(LogIris, Verbose, "%s for: %ls | Attached to %ls | (level: %ls)", __FUNCTION__, *GetNameSafe(MyReplicatedObject), *GetNameSafe(OtherReplicatedObject), *GetNameSafe(Level));



	FReplicationSystemUtil::ForEachServerBridge(World, [&](UEngineReplicationBridge* Bridge)
	{
		// Find the handle of the object we will be dependent to
		const FNetRefHandle OtherRefHandle = Bridge->GetReplicatedRefHandle(OtherReplicatedObject);
		if (!OtherRefHandle.IsValid())
		{
			UE_LOGF(LogIris, Error, "%s Cannot replicate: %ls with: %ls because the other object is not replicating", __FUNCTION__, *GetNameSafe(MyReplicatedObject), *GetNameSafe(OtherReplicatedObject));
			ensure(false);
			return;
		}

		// Find the handle of it's root object
		const FNetRefHandle ParentHandle = Bridge->GetRootObjectOfAnyObject(OtherRefHandle);
		if (!ParentHandle.IsValid())
		{
			UE_LOGF(LogIris, Error, "%s Could not find the root object handle to replicate %ls with %ls", __FUNCTION__, *GetNameSafe(MyReplicatedObject), *GetNameSafe(OtherReplicatedObject));
			ensure(false);
			return;
		}

		// Make sure our object is replicating when it first gets associated with another root object.
		if (!bIsReplicating)
		{
			StartReplication(Level);
		}

		if (!bIsReplicating)
		{
			// Something went wrong
			return;
		}

		// Find our handle
		const FNetRefHandle MyRefHandle = Bridge->GetReplicatedRefHandle(MyReplicatedObject);
		if (!MyRefHandle.IsValid())
		{
			return;
		}

		const FObjectKey DependentKey = Bridge->GetReplicatedObject(ParentHandle);

		uint32& RootObjectRefCount = DependentRootObjects.FindOrAdd(DependentKey);
		++RootObjectRefCount;

		// Add the Iris dependency the first time we are linked with that root object
		if (RootObjectRefCount == 1)
		{
			Bridge->AddDependentObject(ParentHandle, MyRefHandle);
		}
	});
}

void FNetRootObjectAdapter::RemoveRelevantWith(const UObject* OtherReplicatedObject)
{
	UObject* MyReplicatedObject = WeakReplicatedObject.Get();
	if (!MyReplicatedObject)
	{
		ensureMsgf(MyReplicatedObject, TEXT("RootObjectAdapter assigned to a destroyed object. Something went wrong"));
		return;
	}

	if (MyReplicatedObject == OtherReplicatedObject)
	{
		ensureMsgf(MyReplicatedObject != OtherReplicatedObject, TEXT("Cannot StopReplicatingWith yourself: %s"), *GetNameSafe(OtherReplicatedObject));
		return;
	}

	ULevel* Level = WeakLevel.Get();
	if (!Level)
	{
		ensureMsgf(Level, TEXT("RootObjectAdapter cannot be replicated without a valid level associated to it: %s."), *GetNameSafe(WeakReplicatedObject.Get()));
		return;
	}

	UE_LOGF(LogIris, Verbose, "%s for: %ls | Dettaching from: %ls | (level: %ls)", __FUNCTION__, *GetNameSafe(MyReplicatedObject), *GetNameSafe(OtherReplicatedObject), *GetNameSafe(Level));

	UWorld* World = Level->GetWorld();
	check(World);

	if (!bIsReplicating)
	{
		// Nothing more to do if we stopped replicating already
		return;
	}


	FReplicationSystemUtil::ForEachServerBridge(World, [&](UEngineReplicationBridge* Bridge)
	{
		FObjectKey DependentKey;

		// Find the handle of the object we will be dependent to
		const FNetRefHandle OtherRefHandle = Bridge->GetReplicatedRefHandle(OtherReplicatedObject);
		const FNetRefHandle ParentHandle = Bridge->GetRootObjectOfAnyObject(OtherRefHandle);

		if (ParentHandle.IsValid())
		{
			DependentKey = Bridge->GetReplicatedObject(ParentHandle);
		}
		else
		{
			// This usually means the other object has stopped replicating before this.
			// This is not directly a problem because Iris will have removed the links at that time.
			// But it also means we can't find the root object pointer via Iris
			UE_LOGF(LogIris, VeryVerbose, "%s Could not find root object handle to replicate %ls with %ls", __FUNCTION__, *GetNameSafe(MyReplicatedObject), *GetNameSafe(OtherReplicatedObject));

			DependentKey = OtherReplicatedObject;
		}

		{
			uint32* RootObjectRefCount = DependentRootObjects.Find(DependentKey);

			ensureMsgf(RootObjectRefCount != nullptr, TEXT("Could not find dependent object: %s in dependency map of %s"), *GetNameSafe(DependentKey.ResolveObjectPtr()), *GetNameSafe(MyReplicatedObject));
	
			// Remove the Iris dependency if it's the last reference
			if (!RootObjectRefCount || (*RootObjectRefCount) <= 1)
			{
				// Find our handle
				const FNetRefHandle MyRefHandle = Bridge->GetReplicatedRefHandle(MyReplicatedObject);
				ensureMsgf(MyRefHandle.IsValid(), TEXT("No netrefhandle assigned to %s"), *GetNameSafe(MyReplicatedObject));

				// Only need to remove the dependency if both objects are still replicating
				if (MyRefHandle.IsValid() && ParentHandle.IsValid())
				{
					Bridge->RemoveDependentObject(ParentHandle, MyRefHandle);
				}
				
				DependentRootObjects.Remove(DependentKey);
			}
			else
			{
				--(*RootObjectRefCount);
			}
		}
	});

	// If we are no longer replicated via anybody else, remove ourself from the replication system.
	if (DependentRootObjects.IsEmpty() && bOnlyReplicateWhenLinked)
	{
		StopReplication();
	}
}

void FNetRootObjectAdapter::FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) const
{
	OutParams.bUseClassConfigDynamicFilter = false;
	OutParams.PollFrequency = UNetRootObjectFactory::GetDefaultNetUpdateFrequency();

	if (bIsNotRouted)
	{
		OutParams.bUseExplicitDynamicFilter = true;
		OutParams.ExplicitDynamicFilterName = UE::Net::Private::NotRoutedFilter;
	}
	else if (bIsAlwaysRelevant)
	{
		OutParams.bUseExplicitDynamicFilter = true;
		OutParams.ExplicitDynamicFilterName = NAME_None;
	}
	else
	{
		//TODO: Custom filter support
		//      Spatial filter support
	}
}

} // end namespace UE::Net
