// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Iris/ReplicationSystem/NetRootObjectFactory.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetRootObjectFactory)

namespace UE::Net::Private
{
	static const FName RootObjectFactoryName("RootObjectFactory");

	float DefaultRootObjectNetUpdateFrequency = 30.f;
	static FAutoConsoleVariableRef CVarDefaultRootObjectNetUpdateFrequency(
		TEXT("net.Iris.DefaultRootObjectNetUpdateFrequency"),
		DefaultRootObjectNetUpdateFrequency,
		TEXT("The default net update frequency of objects using the NetRootObjectFactory."));
}

float UNetRootObjectFactory::GetDefaultNetUpdateFrequency()
{
	return UE::Net::Private::DefaultRootObjectNetUpdateFrequency;
}

FName UNetRootObjectFactory::GetFactoryName()
{
	return UE::Net::Private::RootObjectFactoryName;
}

void UNetRootObjectFactory::OnInit()
{
	Super::OnInit();

	FactoryExtensionInterfaceClass = UNetRootObjectFactoryExtension::StaticClass();
}

UNetObjectFactory::FInstantiateResult UNetRootObjectFactory::InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header)
{
	using namespace UE::Net;

	const FNetRootObjectCreationHeader* RootObjectHeader = static_cast<const FNetRootObjectCreationHeader*>(Header);

	// For stable objects, find the object via its pathname and bind it
	if (RootObjectHeader->bIsStable)
	{
		UObject* RootObjectPtr = Bridge->ResolveObjectReference(RootObjectHeader->Reference, Context.ResolveContext);

		if (!RootObjectPtr)
		{
			FInstantiateResult InstantiateResult;
			if (UE_LOG_ACTIVE(LogIris, Error))
			{
				const FString RootObjectReferenceStr = Bridge->DescribeObjectReference(RootObjectHeader->Reference, Context.ResolveContext);
				UE_LOGF(LogIris, Error, "%s: Failed to resolve stable reference: %ls . Could not find stable object to bind to.", __FUNCTION__, *RootObjectReferenceStr);
				InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Stable object not found. Reference: %s"), *RootObjectReferenceStr);
			}
			return InstantiateResult;
		}

		UE_LOGF(LogIris, Verbose, "%s: Found stable object: %ls with Handle: %ls via: %ls", __FUNCTION__, *GetNameSafe(RootObjectPtr), *Context.Handle.ToString(), *Bridge->DescribeObjectReference(RootObjectHeader->Reference, Context.ResolveContext));

		return FInstantiateResult{ .Instance = RootObjectPtr };
	}

	// For preregistered objects, find the object in the list
	if (RootObjectHeader->bIsPreregistered)
	{
		// Nothing useable in the header

		UObject* RegisteredObject = Bridge->GetPreRegisteredObject(Context.Handle);
		if (!RegisteredObject)
		{
			FInstantiateResult InstantiateResult;
			if (UE_LOG_ACTIVE(LogIris, Error))
			{
				UE_LOGF(LogIris, Error, "%s Unable to find pre-registered root object: %ls", __FUNCTION__, *Context.Handle.ToString());
				InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Pre-registered object not found. Handle %s"), *Context.Handle.ToString());
			}
			return InstantiateResult;
		}

		UE_LOGF(LogIris, Verbose, "%s: Found pre-registered object: %ls with Handle: %ls", __FUNCTION__, *GetNameSafe(RegisteredObject), *Context.Handle.ToString());

		return FInstantiateResult{ .Instance = RegisteredObject };
	}

	// For dynamic objects, spawn them from the template
	
	// Find the template
	UObject* Template = Bridge->ResolveObjectReference(RootObjectHeader->Reference, Context.ResolveContext);
	if (!Template)
	{
		FInstantiateResult InstantiateResult;
		if (UE_LOG_ACTIVE(LogIris, Error))
		{
			const FString TemplateReferenceStr = Bridge->DescribeObjectReference(RootObjectHeader->Reference, Context.ResolveContext);
			UE_LOGF(LogIris, Error, "%s Unable to instantiate root object, failed to resolve template: %ls", __FUNCTION__, *TemplateReferenceStr);
			InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Dynamic object template not found. TemplateReference: %s"), *TemplateReferenceStr);
		}
		return InstantiateResult;
	}

	UObject* DynamicRootObject = NewObject<UObject>(GetTransientPackageAsObject(), Template->GetClass(), NAME_None, RF_NoFlags, Template);

	FInstantiateResult Result
	{ 
		.Instance = DynamicRootObject,
		.Template = Template, // Force the template to be used as the default init state source. It's possible GetArchetype() will be different then the template
	};

	// Dynamic objects may be destroyed.
	Result.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;

	UE_LOGF(LogIris, Verbose, "%s: Spawned %ls from template: %ls with Handle: %ls", __FUNCTION__, *GetNameSafe(DynamicRootObject), *GetNameSafe(Template), *Context.Handle.ToString());
			
	return Result;
}

void UNetRootObjectFactory::DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext)
{
	using namespace UE::Net;

	UE_LOGF(LogIris, Verbose, "%s: %ls | Reason: %ls", __FUNCTION__, *GetNameSafe(Context.DetachedInstance), LexToString(Context.Reason));

	if (!Context.DetachedInstance)
	{
		return;
	}

	if (Context.Reason == EDetachReason::TornOff)
	{
		return;
	}
	
	const bool bIsStable = Context.DetachedInstance->IsNameStableForNetworking();
	const bool bCanBeDestroyed = !bIsStable || Context.Reason == EDetachReason::StaticDestroyed;
	
	if (bCanBeDestroyed)
	{
		Context.DetachedInstance->PreDestroyFromReplication();

		//TODO: Some objects will want to opt-out of being MarkAsGarbage to support verse semantics
		Context.DetachedInstance->MarkAsGarbage();
	}
}

TOptional<UNetObjectFactory::FWorldInfoData> UNetRootObjectFactory::GetWorldInfo(const FWorldInfoContext& Context) const
{
	// TODO: Could add support for this via the INetRootObjectFactoryExtension.

	ensureMsgf(false, TEXT("UNetRootObjectFactory::GetWorldInfo does not support object: %s with a world location. Classes that want to support spatial filtering need to implement the function in their own factory."), *GetNameSafe(Context.Instance));
	return NullOpt;
}

void UNetRootObjectFactory::FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams)
{
	// If the object itself implements INetRootObjectFactoryExtension we get the replication params from there
	INetRootObjectFactoryExtension* FactoryExtension = static_cast<INetRootObjectFactoryExtension*>(Context.Object->GetInterfaceAddress(FactoryExtensionInterfaceClass));

	if (FactoryExtension)
	{
		FactoryExtension->FillRootObjectReplicationParams(Context, OutParams);
		return;
	}

	// It should suffice to ensure once. A single replicated instance of this class would ensure and assuming any testing whatsoever was performed the ensure should have triggered then rather than being hit in a live environment for the first time.
	ensureMsgf(false, TEXT("Class %s doesn't have an INetRootObjectFactoryExtension associated with it."), ToCStr(Context.Object->GetClass()->GetName()));

	// Use our factory defaults when they differ from FRootObjectReplicationParams defaults.
	OutParams.PollFrequency = UE::Net::Private::DefaultRootObjectNetUpdateFrequency;
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetRootObjectFactory::CreateAndFillHeader(UE::Net::FNetRefHandle Handle)
{
	using namespace UE::Net;

	UObject* RootObjectPtr = Bridge->GetReplicatedObject(Handle);

	if (!RootObjectPtr)
	{
		UE_LOGF(LogIris, Error, "%s: no instance associated with handle: %ls", __FUNCTION__, *Handle.ToString());
		return nullptr;
	}

	TUniquePtr<FNetRootObjectCreationHeader> RootObjectHeader(new FNetRootObjectCreationHeader);

	const bool bSuccess = FillHeader(RootObjectHeader.Get(), Handle, RootObjectPtr);

	if (!bSuccess)
	{
		return nullptr;
	}
	
	return RootObjectHeader;
}

bool UNetRootObjectFactory::FillHeader(UE::Net::FNetRootObjectCreationHeader* RootObjectHeader, UE::Net::FNetRefHandle Handle, const UObject* RootObjectPtr)
{
	using namespace UE::Net;

	const FNetObjectReference RootObjectReference = Bridge->GetOrCreateObjectReference(RootObjectPtr);

	if (!RootObjectReference.IsValid())
	{
		UE_LOGF(LogIris, Error, "%s: could not create NetReference for %ls (handle: %ls)", __FUNCTION__, *GetNameSafe(RootObjectPtr), *Handle.ToString());
		return false;
	}

	// Stable object
	if (Handle.IsStatic() || RootObjectPtr->IsNameStableForNetworking())
	{
		RootObjectHeader->bIsStable = true;
		RootObjectHeader->Reference = RootObjectReference;

		return true;
	}

	// Preregistered object
	if (Bridge->IsNetRefHandlePreRegistered(Handle))
	{
		RootObjectHeader->bIsPreregistered = true;

		return true;
	}

	// Dynamic object
	const UObject* Archetype = RootObjectPtr->GetArchetype();
	checkf(Archetype, TEXT("No archetype found for: %s"), *GetNameSafe(RootObjectPtr));
	checkf(Archetype->NeedsLoadForClient(), TEXT("Archetype: %s of object: %s will not exist on the client"), *GetNameSafe(Archetype), *GetNameSafe(RootObjectPtr));
		
	const FNetObjectReference ArchetypeReference = Bridge->GetOrCreateObjectReference(Archetype);
	if (!ArchetypeReference.IsValid())
	{
		UE_LOGF(LogIris, Error, "%s: could not create NetReference for archetype: %ls", __FUNCTION__, *GetNameSafe(Archetype));
		return false;
	}

	RootObjectHeader->Reference = ArchetypeReference;
	
	return true;
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetRootObjectFactory::CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context)
{
	using namespace UE::Net;

	TUniquePtr<FNetRootObjectCreationHeader> RootObjectHeader(new FNetRootObjectCreationHeader);
	const bool bSuccess = RootObjectHeader->Deserialize(Context);

	if (!bSuccess)
	{
		return nullptr;
	}
	
	return RootObjectHeader;
}

bool UNetRootObjectFactory::SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header)
{
	using namespace UE::Net;

	const FNetRootObjectCreationHeader* RootObjectHeader = static_cast<const FNetRootObjectCreationHeader*>(Header);
	return RootObjectHeader->Serialize(Context);
}

//------------------------------------------------------------------------
// FNetRootObjectCreationHeader
//------------------------------------------------------------------------

namespace UE::Net
{

FString FNetRootObjectCreationHeader:: ToString() const
{
	return FString::Printf(TEXT("\n\tFNetRootObjectCreationHeader (ProtocolId:0x%x):\n\t"
								"%s\n\t"
								"Reference=%s")
						   , GetProtocolId()
						   , bIsStable ? TEXT("Stable") : (bIsPreregistered ? TEXT("Preregistered") : TEXT("Dynamic"))
						   , *Reference.ToString()
						   );
}

bool FNetRootObjectCreationHeader::Serialize(const UE::Net::FCreationHeaderContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.Serialization.GetBitStreamWriter();
	
	Writer->WriteBool(bIsStable);

	if (bIsStable)
	{
		WriteFullNetObjectReference(Context.Serialization, Reference);
	}
	else
	{
		Writer->WriteBool(bIsPreregistered);

		if (!bIsPreregistered)
		{
			WriteFullNetObjectReference(Context.Serialization, Reference);
		}
	}

	return Writer->IsOverflown() == false;
}

bool FNetRootObjectCreationHeader::Deserialize(const UE::Net::FCreationHeaderContext& Context)
{
	FNetBitStreamReader* Reader = Context.Serialization.GetBitStreamReader();

	bIsStable = Reader->ReadBool();

	if (bIsStable)
	{
		ReadFullNetObjectReference(Context.Serialization, Reference);
	}
	else
	{
		bIsPreregistered = Reader->ReadBool();

		if (!bIsPreregistered)
		{
			ReadFullNetObjectReference(Context.Serialization, Reference);
		}
	}

	return Reader->IsOverflown() == false;
}

} // end namespace UE::Net

