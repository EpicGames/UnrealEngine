// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Iris/ReplicationSystem/NetSubObjectFactory.h"

#include "GameFramework/Actor.h"

#include "HAL/UnrealMemory.h"

#include "Iris/Core/IrisLog.h"

#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetBitStreamReader.h"
#include "Iris/Serialization/NetBitStreamWriter.h"
#include "Iris/Serialization/NetSerializationContext.h"
#include "Iris/Serialization/ObjectNetSerializer.h"

#include "UObject/Package.h"

#include "Net/DataBunch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetSubObjectFactory)

namespace UE::Net::Private
{
	static bool bWarnWhenOuterIsReplaced = false;
	static FAutoConsoleVariableRef CVarWarnWhenOuterIsReplaced(
		TEXT("net.Iris.WarnWhenOuterIsReplaced"),
		bWarnWhenOuterIsReplaced,
		TEXT("Log a warning when a subobject's original outer is not found on the client and replaced with the RootObject.")
	);

	static FName SubObjectFactoryName("NetSubObjectFactory");
}

FName UNetSubObjectFactory::GetFactoryName()
{
	return UE::Net::Private::SubObjectFactoryName;
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetSubObjectFactory::CreateAndFillHeader(UE::Net::FNetRefHandle Handle)
{
	using namespace UE::Net;

	UObject* SubObject = Bridge->GetReplicatedObject(Handle);
	if (!SubObject)
	{
		ensureMsgf(SubObject, TEXT("UNetSubObjectFactory::CreateAndFillHeader could not find object tied to handle: %s"), *Bridge->PrintObjectFromNetRefHandle(Handle));
		return nullptr;
	}

	FNetObjectReference ObjectRef = Bridge->GetOrCreateObjectReference(SubObject);

	if (ObjectRef.GetRefHandle().IsStatic() || SubObject->IsNameStableForNetworking())
	{
		// No more information needed since we don't need to spawn the object on the remote.
		TUniquePtr<FNetStaticSubObjectCreationHeader> Header(new FNetStaticSubObjectCreationHeader);
		Header->ObjectReference = ObjectRef;
		return Header;
	}

	TUniquePtr<FNetDynamicSubObjectCreationHeader> Header(new FNetDynamicSubObjectCreationHeader);
	FNetDynamicSubObjectCreationHeader* SubObjectHeader = static_cast<FNetDynamicSubObjectCreationHeader*>(Header.Get());

	const bool bSuccess = FillDynamicHeader(SubObjectHeader, SubObject, Handle);
	if (UNLIKELY(!bSuccess))
	{
		return nullptr;
	}

	return Header;
}

bool UNetSubObjectFactory::FillDynamicHeader(UE::Net::FNetDynamicSubObjectCreationHeader* DynamicHeader, const UObject* SubObject, UE::Net::FNetRefHandle Handle)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	check(SubObject->NeedsLoadForClient() ); // We have no business sending this unless the client can load
	check(SubObject->GetClass()->NeedsLoadForClient());	// We have no business sending this unless the client can load

	// Set the template to clone the object from
	{
		UObject* Archetype = nullptr;
		Archetype = SubObject->GetArchetype();

		DynamicHeader->TemplateReference = Bridge->GetOrCreateObjectReference(Archetype);

		if (!DynamicHeader->TemplateReference.GetRefHandle().IsStatic())
		{
			UE_LOGF(LogIris, Error, "Archetype %ls for subobject %ls (%ls) should be a stable name otherwise the client can't use it", *GetNameSafe(Archetype), *Bridge->PrintObjectFromNetRefHandle(Handle), *GetNameSafe(SubObject->GetClass()->GetAuthoritativeClass()));
			return false;
		}
	}

	// Find the right Outer
	UObject* OuterObject = SubObject->GetOuter();	
	if (OuterObject == GetTransientPackage())
	{
		DynamicHeader->bOuterIsTransientLevel = true;
	}
	else 
	{
		FNetRefHandle RootObjectHandle = Bridge->GetRootObjectOfSubObject(Handle);
		UObject* RootObject = Bridge->GetReplicatedObject(RootObjectHandle);
		check(RootObject);

		if (OuterObject == RootObject)
		{
			DynamicHeader->bOuterIsRootObject = true;
		}
		else
		{
			DynamicHeader->OuterReference = Bridge->GetOrCreateObjectReference(OuterObject);

			// If the Outer is not net-referenceable, use the RootObject instead
			if (!DynamicHeader->OuterReference.IsValid())
			{
				UE_CLOGF(bWarnWhenOuterIsReplaced, LogIris, Warning, "UNetSubObjectFactory::CreateAndFillHeader subobject: %ls has an Outer: %ls that is not stable or replicated. Clients will use RootObject: %ls as the Outer instead", 
					*Bridge->PrintObjectFromNetRefHandle(Handle), 
					*GetNameSafe(OuterObject),
					*GetNameSafe(RootObject));

				DynamicHeader->bOuterIsRootObject = true;
			}
		}
	}

	return true;
}

TUniquePtr<UE::Net::FNetObjectCreationHeader> UNetSubObjectFactory::CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context)
{
	using namespace UE::Net;

	FNetBitStreamReader* Reader = Context.Serialization.GetBitStreamReader();

	if (Reader->ReadBool())
	{
		TUniquePtr<FNetDynamicSubObjectCreationHeader> Header(new FNetDynamicSubObjectCreationHeader);
		Header->Deserialize(Context);
		return Header;
	}
	else
	{
		TUniquePtr<FNetStaticSubObjectCreationHeader> Header(new FNetStaticSubObjectCreationHeader);
		Header->Deserialize(Context);
		return Header;
	}
}


UNetSubObjectFactory::FInstantiateResult UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header)
{
	using namespace UE::Net;
	using namespace UE::Net::Private;

	const FNetBaseSubObjectCreationHeader* BaseHeader = static_cast<const FNetBaseSubObjectCreationHeader*>(Header);

	if (!BaseHeader->IsDynamic())
	{
		const FNetStaticSubObjectCreationHeader* SubObjectHeader = static_cast<const FNetStaticSubObjectCreationHeader*>(BaseHeader);

		// Resolve by finding object relative to owner. We do not allow this object to be destroyed.
		UObject* SubObject = Bridge->ResolveObjectReference(SubObjectHeader->ObjectReference, Context.ResolveContext);

		if (!SubObject)
		{
			FInstantiateResult InstantiateResult;
			if (UE_LOG_ACTIVE(LogIris, Error))
			{
				const FString SubObjectReferenceStr = Bridge->DescribeObjectReference(SubObjectHeader->ObjectReference, Context.ResolveContext);
				const FString OwnerStr = Bridge->PrintObjectFromNetRefHandle(Context.RootObjectOfSubObject);
				const FString RootObjectStr = GetPathNameSafe(Bridge->GetReplicatedObject(Context.RootObjectOfSubObject));
				UE_LOGF(LogIris, Error, "UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader %ls: Failed to find static or stable name object referenced by SubObject: %ls, Owner: %ls, RootObject: %ls", *Context.Handle.ToString(), *SubObjectReferenceStr, *OwnerStr, *RootObjectStr);
				InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Static subobject not found. SubObjectReference: %s, Owner: %s, RootObject: %s"), *SubObjectReferenceStr, *OwnerStr, *RootObjectStr);
			}
			return InstantiateResult;
		}

		UE_LOGF(LogIris, Verbose, "UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader %ls: Found static or stable name SubObject using path %ls", *Context.Handle.ToString(), ToCStr(SubObject->GetPathName()));

		return FInstantiateResult{ .Instance = SubObject };
	}

	// For dynamic objects we have to spawn them

	const FNetDynamicSubObjectCreationHeader* SubObjectHeader = static_cast<const FNetDynamicSubObjectCreationHeader*>(Header);
		
	UObject* RootObject = Bridge->GetReplicatedObject(Context.RootObjectOfSubObject);

	// Find the archetype of the subobject
	UObject* Template = Bridge->ResolveObjectReference(SubObjectHeader->TemplateReference, Context.ResolveContext);
	if (!Template)
	{
		FInstantiateResult InstantiateResult;
		if (UE_LOG_ACTIVE(LogIris, Error))
		{
			const FString TemplateReferenceStr = Bridge->DescribeObjectReference(SubObjectHeader->TemplateReference, Context.ResolveContext);
			UE_LOGF(LogIris, Error, "UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader Unable to resolve template: %ls, cannot instantiate dynamic subobject %ls", *TemplateReferenceStr, *Context.Handle.ToString());
			InstantiateResult.FailureDiagnosticMessage = FString::Printf(TEXT("Dynamic subobject template not found. TemplateReference: %s"), *TemplateReferenceStr);
		}

		ensure(Template);

		return InstantiateResult;
	}
			
	// Find the proper Outer
	UObject* OuterObject = nullptr;
	if (SubObjectHeader->bOuterIsTransientLevel)
	{
		OuterObject = GetTransientPackage();
	}
	else if (SubObjectHeader->bOuterIsRootObject)
	{
		OuterObject = RootObject;
	}				
	else
	{
		OuterObject = Bridge->ResolveObjectReference(SubObjectHeader->OuterReference, Context.ResolveContext);

		if (!OuterObject)
		{
			// Fallback to the rootobject instead
			OuterObject = RootObject;

			UE_CLOGF(bWarnWhenOuterIsReplaced, LogIris, Warning, "UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader Failed to find Outer %ls for dynamic subobject %ls (template: %ls). Original outer will be replaced with %ls instead.", *Bridge->DescribeObjectReference(SubObjectHeader->OuterReference, Context.ResolveContext), *Context.Handle.ToString(), *GetNameSafe(Template), *GetNameSafe(RootObject));
		}
	}

	// Instantiate the SubObject
	UObject* SubObj = nullptr;
	UClass* SubObjClass = nullptr;

	SubObjClass = Template->GetClass();
	SubObj = NewObject<UObject>(OuterObject, SubObjClass, NAME_None, RF_NoFlags, Template);

	// Sanity check some things
	checkf(SubObj != nullptr, TEXT("UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader: Subobject is NULL after instantiating. Class: %s, Outer %s, Actor %s"), *GetNameSafe(SubObjClass), *GetNameSafe(OuterObject), *GetNameSafe(RootObject));
	checkf(!OuterObject || SubObj->IsIn(OuterObject), TEXT("UNetSubObjectFactory::InstantiateReplicatedObjectFromHeader: Subobject is not in Outer. SubObject: %s, Outer %s, Actor %s"), *SubObj->GetName(), *GetNameSafe(OuterObject), *GetNameSafe(RootObject));

	FInstantiateResult Result
	{ 
		.Instance = SubObj,
		.Template = Template, // Force the template to be used as the default init state source. It's possible GetArchetype() will be different then the template
	};

	// Call OnSubObjectCreatedFromReplication after the state has been applied to the owning actor
	Result.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::ShouldCallSubObjectCreatedFromReplication;

	// Created objects may be destroyed.
	Result.Flags |= EReplicationBridgeCreateNetRefHandleResultFlags::AllowDestroyInstanceFromRemote;
			
	return Result;
}

bool UNetSubObjectFactory::SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header)
{
	using namespace UE::Net;
	const FNetBaseSubObjectCreationHeader* SubObjectHeader = static_cast<const FNetBaseSubObjectCreationHeader*>(Header);
	
	FNetBitStreamWriter* Writer = Context.Serialization.GetBitStreamWriter();
	
	Writer->WriteBool(SubObjectHeader->IsDynamic());

	return SubObjectHeader->Serialize(Context);
}

void UNetSubObjectFactory::SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated)
{
	ensureMsgf(false, TEXT("NetSubObjectFactory::SubObjectCreatedFromReplication should never be called since subobjects cannot have their own subobject list. RootObject: %s, SubObjectCreated: %s"), *Bridge->PrintObjectFromNetRefHandle(RootObject), *Bridge->PrintObjectFromNetRefHandle(SubObjectCreated));
}

void UNetSubObjectFactory::DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext)
{
	using namespace UE::Net;

	UObject* DetachedSubObject = Context.DetachedInstance;
	if (!DetachedSubObject)
	{
		// The instance has already been detached, nothing else to do here
		return;
	}

	if (Context.Reason == EDetachReason::TornOff)
	{
		// If the SubObject is being torn off it is up to owning actor to clean it up properly
		return;
	}

	if (!SubObjectContext.IsSet())
	{
		ensureMsgf(false, TEXT("The SubObjectContext should always exist for the NetSubObjectFactory"));
		return;
	}

	const bool bIsStableSubObject = DetachedSubObject->IsNameStableForNetworking();

	//TODO: Handle StaticDestroyed reason
	const bool bCanBeDestroyed = !bIsStableSubObject;

	//TODO: Decide if there are conditions where we could do things differently here when looking at the root object detach reason...

	if (!bCanBeDestroyed)
	{
		// Nothing to do if the subobject shouldn't be destroyed
		return;
	}

	DetachedSubObject->PreDestroyFromReplication();
	DetachedSubObject->MarkAsGarbage();
}

TOptional<UNetObjectFactory::FWorldInfoData> UNetSubObjectFactory::GetWorldInfo(const FWorldInfoContext& Context) const
{
	ensureMsgf(false, TEXT("UNetSubObjectFactory::GetWorldInfo called but subobjects should never support this. Instance: %s, NetRefHandle: %s"), *GetNameSafe(Context.Instance), *Bridge->PrintObjectFromNetRefHandle(Context.Handle));
	return NullOpt;
}

void UNetSubObjectFactory::FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams&)
{
	ensureMsgf(false, TEXT("UNetSubObjectFactory::FillRootObjectReplicationParams called but subobjects should never support this. Instance: %s"), *GetNameSafe(Context.Object));
}

//------------------------------------------------------------------------

namespace UE::Net
{

//------------------------------------------------------------------------
// FNetStaticSubObjectCreationHeader
//------------------------------------------------------------------------
FString FNetStaticSubObjectCreationHeader::ToString() const
{
	return FString::Printf(TEXT("\n\tFNetStaticSubObjectCreationHeader (ProtocolId:0x%x):\n\t"
								"ObjectReference=%s\n\t"),								
								GetProtocolId(),
								*ObjectReference.ToString());

}

bool FNetStaticSubObjectCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	WriteFullNetObjectReference(Context.Serialization, ObjectReference);
	return true;
}

bool FNetStaticSubObjectCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	ReadFullNetObjectReference(Context.Serialization, ObjectReference);
	return true;
}

//------------------------------------------------------------------------
// FNetDynamicSubObjectCreationHeader
//------------------------------------------------------------------------

FString FNetDynamicSubObjectCreationHeader::ToString() const
{
	return FString::Printf(TEXT("\n\tFNetDynamicSubObjectCreationHeader (ProtocolId:0x%x):\n\t"
								"TemplateReference=%s\n\t"
								"OuterReference=%s\n\t"
								"bUsePersistenLevel=%u\n\t"
								"bOuterIsTransientLevel=%u\n\t"
								"bOuterIsRootObject=%u\n\t"),
								GetProtocolId(),
								*TemplateReference.ToString(),
								*OuterReference.ToString(),
								bUsePersistentLevel,
								bOuterIsTransientLevel,
								bOuterIsRootObject);

}

bool FNetDynamicSubObjectCreationHeader::Serialize(const FCreationHeaderContext& Context) const
{
	FNetBitStreamWriter* Writer = Context.Serialization.GetBitStreamWriter();

	WriteFullNetObjectReference(Context.Serialization, TemplateReference);

	if (!Writer->WriteBool(bOuterIsTransientLevel))
	{
		if (!Writer->WriteBool(bOuterIsRootObject))
		{
			WriteFullNetObjectReference(Context.Serialization, OuterReference);
		}
	}

	return true;
}

bool FNetDynamicSubObjectCreationHeader::Deserialize(const FCreationHeaderContext& Context)
{
	FNetBitStreamReader* Reader = Context.Serialization.GetBitStreamReader();

	ReadFullNetObjectReference(Context.Serialization, TemplateReference);

	bOuterIsTransientLevel = Reader->ReadBool();
	if (!bOuterIsTransientLevel)
	{
		bOuterIsRootObject = Reader->ReadBool();
		if (!bOuterIsRootObject)
		{
			ReadFullNetObjectReference(Context.Serialization, OuterReference);
		}
	}

	return true;
}

} // end namespace UE::Net
