// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "Engine/World.h"

#include "EngineUtils.h"

#include "GameFramework/PlayerController.h"

#include "HAL/IConsoleManager.h"

#include "Iris/Core/IrisLog.h"
#include "Iris/Core/IrisProfiler.h"

#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"

#include "Net/Core/Connection/ConnectionHandle.h"
#include "Net/Core/Misc/NetConditionGroupManager.h"
#include "Net/Core/NetHandle/NetHandleManager.h"

#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/NetSubObjectRegistryGetter.h"
#include "Net/Subsystems/NetworkSubsystem.h"

#include "Templates/Casts.h"
#include "Templates/Function.h"

namespace UE::Net::Private
{
	bool DoesTypeMatch(FReplicationSystemUtil::EType Type, UReplicationSystem* System)
	{
		switch(Type)
		{
			case FReplicationSystemUtil::EType::Any:
			{
				return true;
			}

			case FReplicationSystemUtil::EType::Server:
			{
				return System->IsServer();
			}

			case FReplicationSystemUtil::EType::Client:
			{
				return !System->IsServer();
			}

			default:
			{
				ensureMsgf(false, TEXT("EType entry not handled"));
				return false;
			}
		}
	}
}

namespace UE::Net
{

void FReplicationSystemUtil::ForEachBridge(const UWorld* World, TFunctionRef<void(UEngineReplicationBridge*)> Function, EType Type)
{
	using namespace UE::Net::Private;

	if (!World)
	{
		return;
	}

	UGameInstance* GameInstance = World->GetGameInstance();
	if (!GameInstance)
	{
		return;
	}

	UEngine* Engine = GameInstance->GetEngine();
	if (!Engine)
	{
		return;
	}

	const FWorldContext* Context = Engine->GetWorldContextFromWorld(World);
	if (!Context)
	{
		return;
	}

	for (const FNamedNetDriver& NamedNetDriver : Context->ActiveNetDrivers)
	{
		if (UNetDriver* NetDriver = NamedNetDriver.NetDriver)
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (DoesTypeMatch(Type, ReplicationSystem))
				{
					// There should always be a bridge attached to the system
					UEngineReplicationBridge* ReplicationBridge = CastChecked<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge());

					Function(ReplicationBridge);
				}
			}
		}
	}
}

void FReplicationSystemUtil::ForEachSystem(const UWorld* World, TFunctionRef<void(UReplicationSystem*)> Function, EType Type)
{
	using namespace UE::Net::Private;

	if (!World)
	{
		return;
	}

	UGameInstance* GameInstance = World->GetGameInstance();
	if (!GameInstance)
	{
		return;
	}

	UEngine* Engine = GameInstance->GetEngine();
	if (!Engine)
	{
		return;
	}

	const FWorldContext* Context = Engine->GetWorldContextFromWorld(World);
	if (!Context)
	{
		return;
	}

	for (const FNamedNetDriver& NamedNetDriver : Context->ActiveNetDrivers)
	{
		if (UNetDriver* NetDriver = NamedNetDriver.NetDriver)
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (DoesTypeMatch(Type, ReplicationSystem))
				{
					Function(ReplicationSystem);
				}
			}
		}
	}
}

void FReplicationSystemUtil::ForEachSystemAndBridge(const UWorld* World, TFunctionRef<void(UReplicationSystem*, UEngineReplicationBridge*)> Function, EType Type)
{
	using namespace UE::Net::Private;

	if (!World)
	{
		return;
	}

	UGameInstance* GameInstance = World->GetGameInstance();
	if (!GameInstance)
	{
		return;
	}

	UEngine* Engine = GameInstance->GetEngine();
	if (!Engine)
	{
		return;
	}

	const FWorldContext* Context = Engine->GetWorldContextFromWorld(World);
	if (!Context)
	{
		return;
	}

	for (const FNamedNetDriver& NamedNetDriver : Context->ActiveNetDrivers)
	{
		if (UNetDriver* NetDriver = NamedNetDriver.NetDriver)
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (DoesTypeMatch(Type, ReplicationSystem))
				{
					// There should always be a bridge attached to the system
					UEngineReplicationBridge* ReplicationBridge = CastChecked<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge());

					Function(ReplicationSystem, ReplicationBridge);
				}
			}
		}
	}
}

void FReplicationSystemUtil::ForEachReplicationSystem(const UEngine* Engine, const UWorld* World, TFunctionRef<void(UReplicationSystem*)> Function)
{
	if (Engine == nullptr || World == nullptr)
	{
		return;
	}

	if (const FWorldContext* Context = Engine->GetWorldContextFromWorld(World))
	{
		for (const FNamedNetDriver& NamedNetDriver : Context->ActiveNetDrivers)
		{
			if (UNetDriver* NetDriver = NamedNetDriver.NetDriver)
			{
				if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
				{
					Function(ReplicationSystem);
				}
			}
		}
	}
}

void FReplicationSystemUtil::ForEachReplicationSystem(TFunctionRef<void(UReplicationSystem*)> Function)
	{
		for (UReplicationSystem* ReplicationSystem : FReplicationSystemFactory::GetAllReplicationSystems())
		{
			if (ReplicationSystem != nullptr)
			{
				Function(ReplicationSystem);
			}
		}
	}

UReplicationSystem* FReplicationSystemUtil::GetReplicationSystem(const AActor* Actor)
{
	if (!Actor)
	{
		return nullptr;
	}

	if (const UNetConnection* NetConnection = Actor->GetNetConnection())
	{
		if (NetConnection->Driver)
		{
			return NetConnection->Driver->GetReplicationSystem();
		}
	}

	UNetDriver* NetDriver = Actor->GetNetDriver();
	if (!Actor->GetWorld() || !NetDriver)
	{
		return nullptr;
	}

	return NetDriver->GetReplicationSystem();
}

UReplicationSystem* FReplicationSystemUtil::GetReplicationSystem(const UNetDriver* NetDriver)
{
	return NetDriver ? NetDriver->GetReplicationSystem() : nullptr;
}

UReplicationSystem* FReplicationSystemUtil::GetReplicationSystem(const UWorld* World)
{
	UNetDriver* NetDriver = World ? World->GetNetDriver() : nullptr;
	return NetDriver ? NetDriver->GetReplicationSystem() : nullptr;
}

UEngineReplicationBridge* FReplicationSystemUtil::GetActorReplicationBridge(const AActor* Actor)
{
	if (UReplicationSystem* ReplicationSystem = GetReplicationSystem(Actor))
	{
		return Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge());
	}
	else
	{
		return nullptr;
	}
}

UEngineReplicationBridge* FReplicationSystemUtil::GetActorReplicationBridge(const UNetConnection* NetConnection)
{
	const UNetDriver* Driver = NetConnection ? NetConnection->GetDriver() : nullptr;
	if (const UReplicationSystem* ReplicationSystem = Driver ? Driver->GetReplicationSystem() : nullptr)
	{
		return Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge());
	}
	else
	{
		return nullptr;
	}
}

UEngineReplicationBridge* FReplicationSystemUtil::GetEngineReplicationBridge(const UWorld* World)
{
	UReplicationSystem* ReplicationSystem = GetReplicationSystem(World);	
	return Cast<UEngineReplicationBridge>(ReplicationSystem ? ReplicationSystem->GetReplicationBridge() : nullptr);
}

FNetHandle FReplicationSystemUtil::GetNetHandle(const UObject* ReplicatedObject)
{
	return FNetHandleManager::GetNetHandle(ReplicatedObject);
}

void FReplicationSystemUtil::StartReplicatingActor(AActor* Actor)
{
	bool bCallReplicationStarted = false;
	if (const UWorld* World = Actor->GetWorld())
	{
		ForEachReplicationSystem(GEngine, World, [Actor, &bCallReplicationStarted](UReplicationSystem* ReplicationSystem)
		{
			if (ReplicationSystem->IsServer())
			{
				if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
				{
					const FNetRefHandle ReplicatedNetRefHandle = Bridge->StartReplicatingActor(Actor);
					if (ReplicatedNetRefHandle.IsValid())
					{
						bCallReplicationStarted = true;
					}
				}
			}
		});
	}

	if (bCallReplicationStarted)
	{
		AActor::FOnReplicationStartedParams Params;
		Actor->OnReplicationStartedForIris(Params);	
	}
}

void FReplicationSystemUtil::StopReplicatingActor(AActor* Actor, const FStopReplicatingActorParams& StopReplicatingParams)
{
	if (!Actor)
	{
		return;
	}

	// Invoke OnStopReplication before we destroy NetRefHandles etc.
	AActor::FOnStopReplicationParams OnStopReplicationParams = { .StopReplicatingParams = StopReplicatingParams };
	Actor->OnStopReplicationForIris(OnStopReplicationParams);

	// If the call is coming from for example destroying an actor then a formerly associated NetRefHandle will no longer be valid.
	// The bridge itself will verify that the actor is replicated by it so there's no reason to check that either.
	ForEachReplicationSystem([Actor, StopReplicatingParams](UReplicationSystem* ReplicationSystem)
	{
		if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
		{
			Bridge->StopReplicatingActor(Actor, StopReplicatingParams);
		}
	});
}

void FReplicationSystemUtil::StartReplicatingActorComponent(const AActor* Actor, UActorComponent* ActorComp)
{
	const FNetHandle ActorHandle = GetNetHandle(Actor);
	bool bCallReplicationStarted = false;

	// If the actor doesn't have a valid handle we assume it's not replicated by any ReplicationSystem
	if (!ActorHandle.IsValid())
	{
		return;
	}

	if (const UWorld* World = Actor->GetWorld())
	{
		ForEachReplicationSystem(GEngine, World, [ActorHandle, ActorComp, &bCallReplicationStarted](UReplicationSystem* ReplicationSystem)
		{
			if (ReplicationSystem->IsServer())
			{
				if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
				{
					const FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
					if (ActorRefHandle.IsValid())
					{
						const FNetRefHandle ComponentHandle = Bridge->StartReplicatingComponent(ActorRefHandle, ActorComp);
						if (ComponentHandle.IsValid())
						{
							bCallReplicationStarted = true;
						}
					}
				}
			}
		});
	}

	if (bCallReplicationStarted)
	{
		UActorComponent::FOnReplicationStartedParams Params;
		ActorComp->OnReplicationStartedForIris(Params);
	}
}

void FReplicationSystemUtil::StartReplicatingActorSubObject(const AActor* Actor, UObject* ActorSubObject, ELifetimeCondition NetCondition)
{
	if (NetCondition == ELifetimeCondition::COND_Never)
	{
		return;
	}

	// Assume an actor without NetHandle isn't replicated.
	const FNetHandle ActorHandle = GetNetHandle(Actor);
	if (!ActorHandle.IsValid())
	{
		return;
	}

	if (const UWorld* World = Actor->GetWorld())
	{
		ForEachReplicationSystem(GEngine, World, [ActorHandle, ActorSubObject, NetCondition](UReplicationSystem* ReplicationSystem)
		{
			if (ReplicationSystem->IsServer())
			{
				if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
				{
					const FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
					if (ActorRefHandle.IsValid())
					{
						const UObjectReplicationBridge::FSubObjectReplicationParams Params { .RootObjectHandle = ActorRefHandle };
						const FNetRefHandle SubObjectRefHandle = Bridge->StartReplicatingSubObject(ActorSubObject, Params);
						if (SubObjectRefHandle.IsValid() && NetCondition != ELifetimeCondition::COND_None)
						{
							Bridge->SetSubObjectNetCondition(SubObjectRefHandle, NetCondition);
						}
					}
				}
			}
		});
	}
}

void FReplicationSystemUtil::StartReplicatingActorComponentSubObject(UActorComponent* ActorComponent, UObject* SubObject, ELifetimeCondition NetCondition)
{
	const AActor* Actor = ActorComponent->GetOwner();
	if (Actor && NetCondition != ELifetimeCondition::COND_Never)
	{		
		const FNetHandle ActorHandle = GetNetHandle(Actor);
		if (!ActorHandle.IsValid())
		{
			return;
		}

		if (const UWorld* World = Actor->GetWorld())
		{
			ForEachReplicationSystem(GEngine, World, [ActorHandle, ActorComponent, SubObject, NetCondition](UReplicationSystem* ReplicationSystem)
			{
				if (ReplicationSystem->IsServer())
				{
					if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
					{
						const FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
						const FNetRefHandle ActorComponentRefHandle = Bridge->GetReplicatedRefHandle(ActorComponent);
						if (ActorRefHandle.IsValid() && ActorComponentRefHandle.IsValid())
						{
							const UObjectReplicationBridge::FSubObjectReplicationParams Params
							{ 
								.RootObjectHandle = ActorRefHandle,
								.ParentSubObjectHandle = ActorComponentRefHandle
							};
							const FNetRefHandle SubObjectRefHandle = Bridge->StartReplicatingSubObject(SubObject, Params);
							if (SubObjectRefHandle.IsValid() && NetCondition != ELifetimeCondition::COND_None)
							{
								Bridge->SetSubObjectNetCondition(SubObjectRefHandle, NetCondition);
							}
						}
					}
				}
			});
		}
	}
}

void FReplicationSystemUtil::StopReplicatingActorComponent(UActorComponent* ActorComponent)
{
	if (!ensure(ActorComponent))
	{
		return;
	}

	// Invoke OnStopReplication before we destroy NetRefHandles etc.
	UActorComponent::FOnStopReplicationParams OnStopReplicationParams;
	ActorComponent->OnStopReplicationForIris(OnStopReplicationParams);

	ForEachReplicationSystem([ActorComponent](UReplicationSystem* ReplicationSystem)
	{
		if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
		{
			constexpr EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId;
			Bridge->StopReplicatingComponent(ActorComponent, EndReplicationFlags);
		}
	});
}

void FReplicationSystemUtil::StopReplicatingActorSubObject(const AActor* Actor, UObject* SubObject)
{
	ForEachReplicationSystem([SubObject](UReplicationSystem* ReplicationSystem)
	{
		if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
		{				
			constexpr EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::Destroy | EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId;
			Bridge->StopReplicatingNetObject(SubObject, EndReplicationFlags);
		}
	});
}

void FReplicationSystemUtil::SetNetConditionForActorSubObject(const AActor* Actor, UObject* SubObject, ELifetimeCondition NetCondition)
{
	if (!IsValid(Actor) || !IsValid(SubObject))
	{
		return;
	}

	const FNetHandle ActorHandle = GetNetHandle(Actor);
	if (!ActorHandle.IsValid())
	{
		return;
	}

	if (const UWorld* World = Actor->GetWorld())
	{
		ForEachReplicationSystem(GEngine, World, [SubObject, NetCondition](UReplicationSystem* ReplicationSystem)
		{
			if (ReplicationSystem->IsServer())
			{
				if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
				{
					const FNetRefHandle SubObjectRefHandle = Bridge->GetReplicatedRefHandle(SubObject);
					if (SubObjectRefHandle.IsValid())
					{
						Bridge->SetSubObjectNetCondition(SubObjectRefHandle, NetCondition);
					}
				}
			}
		});
	}
}

void FReplicationSystemUtil::StopReplicatingActorComponentSubObject(UActorComponent* ActorComponent, UObject* SubObject)
{
	StopReplicatingActorSubObject(ActorComponent->GetOwner(), SubObject);
}

void FReplicationSystemUtil::AddDependentActor(const AActor* Parent, AActor* Child, EDependentObjectSchedulingHint SchedulingHint)
{
	check(Parent);

	const UWorld* World = Parent->GetWorld();
	if (!World || !World->GetNetDriver() || !World->GetNetDriver()->IsUsingIrisReplication())
	{
		return;
	}

	// Can only add dependent actors on already replicating actors
	FNetHandle ParentHandle = GetNetHandle(Parent);
	if (!ensureMsgf(ParentHandle.IsValid(), TEXT("FReplicationSystemUtil::AddDependentActor Parent %s is not replicated. Cannot attach child %s as dependent"), *GetPathNameSafe(Parent), *GetNameSafe(Child)))
	{
		return;
	}

	ForEachReplicationSystem(GEngine, World, [ParentHandle, Child, SchedulingHint](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
			{
				const FNetRefHandle ParentRefHandle = Bridge->GetReplicatedRefHandle(ParentHandle);
				if (ParentRefHandle.IsValid())
				{
					FNetRefHandle ChildRefHandle = Bridge->GetReplicatedRefHandle(Child);
					if (!ChildRefHandle.IsValid())
					{
						// Note: direct call to bridge, might be that it should call ReplicationUtil::StartReplicatingActor instead 
						ChildRefHandle = Bridge->StartReplicatingActor(Child);
					}
					if (ensureMsgf(ChildRefHandle.IsValid(), TEXT("FReplicationSystemUtil::AddDependentActor Child %s is not replicated"), *GetPathNameSafe(Child)))
					{
						Bridge->AddDependentObject(ParentRefHandle, ChildRefHandle, SchedulingHint);
					}
				}
			}
		}
	});
}

void FReplicationSystemUtil::AddDependentActor(const AActor* Parent, AActor* Child)
{
	AddDependentActor(Parent, Child, EDependentObjectSchedulingHint::Default);
}

void FReplicationSystemUtil::RemoveDependentActor(const AActor* Parent, AActor* Child)
{
	check(Parent);
	const UWorld* World = Parent->GetWorld();
	if (!World || !World->GetNetDriver() || !World->GetNetDriver()->IsUsingIrisReplication())
	{
		return;
	}

	const FNetHandle ParentHandle = GetNetHandle(Parent);
	const FNetHandle ChildHandle = GetNetHandle(Child);
	if (!ParentHandle.IsValid() || !ChildHandle.IsValid())
	{
		return;
	}

	ForEachReplicationSystem(GEngine, World, [ParentHandle, ChildHandle](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
			{
				const FNetRefHandle ParentRefHandle = Bridge->GetReplicatedRefHandle(ParentHandle);
				if (ParentRefHandle.IsValid())
				{
					const FNetRefHandle ChildRefHandle = Bridge->GetReplicatedRefHandle(ChildHandle);
					Bridge->RemoveDependentObject(ParentRefHandle, ChildRefHandle);
				}
			}
		}
	});
}

void FReplicationSystemUtil::SetNetConditionForActorComponent(const UActorComponent* ActorComponent, ELifetimeCondition NetCondition)
{
	if (!IsValid(ActorComponent))
	{
		return;
	}

	const AActor* Actor = ActorComponent->GetOwner();
	const FNetHandle ActorHandle = GetNetHandle(Actor);
	if (!ActorHandle.IsValid())
	{
		return;
	}

	if (const UWorld* World = Actor->GetWorld())
	{
		ForEachReplicationSystem(GEngine, World, [ActorComponent, NetCondition](UReplicationSystem* ReplicationSystem)
		{
			if (ReplicationSystem->IsServer())
			{
				if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
				{
					const FNetRefHandle ActorComponentRefHandle = Bridge->GetReplicatedRefHandle(ActorComponent);
					if (ActorComponentRefHandle.IsValid())
					{
						Bridge->SetSubObjectNetCondition(ActorComponentRefHandle, NetCondition);
					}
				}
			}
		});
	}
}

void FReplicationSystemUtil::StartReplicatingActorsInWorldForNetDriver(UWorld* World, UNetDriver* NetDriver)
{
	// We only do this if the world already is initialized
	// Normally we rely on Begin/EndPlay to control if an actor is replicated or not.
	if (World && World->bIsWorldInitialized)
	{
		for (FActorIterator Iter(World); Iter; ++Iter)
		{
			AActor* Actor = *Iter;
			if (IsValid(Actor) && Actor->HasActorBegunPlay() && NetDriver->ShouldReplicateActor(Actor) && ULevel::IsNetActor(Actor))
			{
				StartReplicatingActor(Actor);
			}
		}
	}
}

void FReplicationSystemUtil::NotifyActorDormancyChange(UReplicationSystem* ReplicationSystem, AActor* Actor, ENetDormancy OldDormancyState)
{
	if (!ReplicationSystem || !ReplicationSystem->IsServer())
	{
		return;
	}

	const ENetDormancy Dormancy = Actor->NetDormancy;
	const bool bIsPendingDormancy = (Dormancy > DORM_Awake);

	if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
	{
		const FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(Actor);
		if (ActorRefHandle.IsValid())
		{
			Bridge->SetObjectWantsToBeDormant(ActorRefHandle, bIsPendingDormancy);
		}
	}
}

void FReplicationSystemUtil::FlushNetDormancy(UReplicationSystem* ReplicationSystem, AActor* Actor, bool bWasDormInitial)
{
	if (!ReplicationSystem || !ReplicationSystem->IsServer())
	{
		return;
	}

	if (!Actor->IsActorInitialized())
	{
		UE_LOGF(LogIris, Verbose, "FReplicationSystemUtil::FlushNetDormancy called on %ls that isn't fully initialized yet. Ignoring.", ToCStr(GetFullNameSafe(Actor)));
		return;
	}

	if (!Actor->GetIsReplicated())
	{
		ensureMsgf(Actor->GetIsReplicated(), TEXT("FReplicationSystemUtil::FlushNetDormancy Actor: %s is not replicated"), ToCStr(GetFullNameSafe(Actor)));
		return;
	}

	// Handle DormInitial actors
	if (bWasDormInitial && (Actor->HasActorBegunPlay() || Actor->IsActorBeginningPlay()))
	{
		// Start Replication for DORM_Initial actors the first time their dormancy is flushed
		// (since it's not called when they BeginPlay).
		// 
		// We still don't want to call Start Replication before BeginPlay though:
		// -The actor and its components/subobjects may not be completely set up for replication yet
		// -If the actor is DormInitial, and is flushed before BeginPlay, its dormancy state will change to DormantAll and it will Start Replication normally

		StartReplicatingActor(Actor);
	}

	const FNetHandle ActorHandle = GetNetHandle(Actor);
	if (ActorHandle.IsValid())
	{
		if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
		{
			const FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
			if (ActorRefHandle.IsValid())
			{
				Bridge->NetFlushDormantObject(ActorRefHandle);
			}
		}
	}
}

void FReplicationSystemUtil::UpdateSubObjectGroupMemberships(const UObject* SubObject, const UWorld* World)
{
	if (const UNetworkSubsystem* NetSubsystem = World ? World->GetSubsystem<UNetworkSubsystem>() : nullptr)
	{
		FObjectKey SubObjectKey(SubObject);
		ForEachReplicationSystem(GEngine, World, [NetSubsystem, SubObject, SubObjectKey](UReplicationSystem* ReplicationSystem)
		{
			if (ReplicationSystem->IsServer())
			{
				if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
				{
					FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(SubObject);
					if (RefHandle.IsValid())
					{
						for (const FName NetGroup : NetSubsystem->GetNetConditionGroupManager().GetSubObjectNetConditionGroups(SubObjectKey))
						{
							FNetObjectGroupHandle SubObjectGroupHandle = ReplicationSystem->GetOrCreateSubObjectFilter(NetGroup);
							ReplicationSystem->AddToGroup(SubObjectGroupHandle, RefHandle);
						}
					}
				}
			}
		});
	}
}

void FReplicationSystemUtil::RemoveSubObjectGroupMembership(const APlayerController* PC, const FName NetGroup)
{
	if (IsSpecialNetConditionGroup(NetGroup))
	{
		return;
	}
	
	// We assume the player controller is tied to a single connection.
	if (UNetConnection* Conn = PC->GetNetConnection())
	{
		if (UReplicationSystem* ReplicationSystem = Conn->GetDriver() ? Conn->GetDriver()->GetReplicationSystem() : nullptr)
		{
			ReplicationSystem->SetSubObjectFilterStatus(NetGroup, Conn->GetConnectionHandle(), ENetFilterStatus::Disallow);
		}
	}
}

void FReplicationSystemUtil::UpdateSubObjectGroupMemberships(const APlayerController* PC)
{
	// We assume the player controller is tied to a single connection.
	if (UNetConnection* Conn = PC->GetNetConnection())
	{
		if (UReplicationSystem* ReplicationSystem = Conn->GetDriver() ? Conn->GetDriver()->GetReplicationSystem() : nullptr)
		{
			const FConnectionHandle ConnectionHandle = Conn->GetConnectionHandle();
			for (const FName NetGroup : PC->GetNetConditionGroups())
			{
				if (!IsSpecialNetConditionGroup(NetGroup))
				{
					FNetObjectGroupHandle SubObjectGroupHandle = ReplicationSystem->GetOrCreateSubObjectFilter(NetGroup);
					ReplicationSystem->SetSubObjectFilterStatus(NetGroup, ConnectionHandle, ENetFilterStatus::Allow);
				}
			}
		}
	}
}

void FReplicationSystemUtil::SetReplicationCondition(FNetHandle NetHandle, EReplicationCondition Condition, bool bEnableCondition)
{
	ForEachReplicationSystem([NetHandle, Condition, bEnableCondition](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
			{
				FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(NetHandle);
				if (RefHandle.IsValid())
				{
					ReplicationSystem->SetReplicationCondition(RefHandle, Condition, bEnableCondition);
				}
			}
		}
	});
}

void FReplicationSystemUtil::SetStaticPriority(const AActor* Actor, float Priority)
{
	FNetHandle ActorHandle = GetNetHandle(Actor);
	if (!ActorHandle.IsValid())
	{
		return;
	}
	
	ForEachReplicationSystem([ActorHandle, Priority](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
			{
				const FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
				ReplicationSystem->SetStaticPriority(ActorRefHandle, Priority);
			}
		}
	});
}

void FReplicationSystemUtil::SetCullDistanceOverride(const AActor* Actor, float CullDist)
{
	FNetHandle ActorHandle = GetNetHandle(Actor);
	if (!ActorHandle.IsValid())
	{
		return;
	}
	
	ForEachReplicationSystem([ActorHandle, CullDist](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
			{
				FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
				if (RefHandle.IsValid())
				{
					ReplicationSystem->SetCullDistanceOverride(RefHandle, CullDist);
				}
			}
		}
	});
}

void FReplicationSystemUtil::ClearCullDistanceOverride(const AActor* Actor)
{
	FNetHandle ActorHandle = GetNetHandle(Actor);
	if (!ActorHandle.IsValid())
	{
		return;
	}
	
	ForEachReplicationSystem([ActorHandle](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
			{
				FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(ActorHandle);
				if (RefHandle.IsValid())
				{
					ReplicationSystem->ClearCullDistanceOverride(RefHandle);
				}
			}
		}
	});
}

void FReplicationSystemUtil::SetPollFrequency(const UObject* Object, float PollFrequency)
{
	FNetHandle NetHandle = GetNetHandle(Object);
	if (!NetHandle.IsValid())
	{
		return;
	}
	
	ForEachReplicationSystem([NetHandle, PollFrequency](UReplicationSystem* ReplicationSystem)
	{
		if (ReplicationSystem->IsServer())
		{
			if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
			{
				FNetRefHandle RefHandle = Bridge->GetReplicatedRefHandle(NetHandle);
				if (RefHandle.IsValid())
				{
					Bridge->SetPollFrequency(RefHandle, PollFrequency);
				}
			}
		}
	});
}

void FDeferredReplicationSystemCalls::StartReplicatingActorDefer(AActor* Actor)
{
	if (UWorld* World = Actor->GetWorld())
	{
		if (UNetDriver* NetDriver = World->GetNetDriver())
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
				{
					Bridge->StartReplicatingActorDefer(Actor);
				}
			}
		}
	}
}

void FDeferredReplicationSystemCalls::StopReplicatingActorDefer(AActor* Actor, const FStopReplicatingActorParams& StopReplicatingActorParams)
{
	if (UWorld* World = Actor->GetWorld())
	{
		if (UNetDriver* NetDriver = World->GetNetDriver())
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
				{
					Bridge->StopReplicatingActorDefer(Actor, StopReplicatingActorParams);
				}
			}
		}
	}
}

void FDeferredReplicationSystemCalls::StartReplicatingActorComponentDefer(const AActor* Actor, UActorComponent* ActorComponent)
{
	if (UWorld* World = ActorComponent->GetWorld())
	{
		if (UNetDriver* NetDriver = World->GetNetDriver())
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
				{
					Bridge->StartReplicatingComponentDefer(Actor, ActorComponent);
				}
			}
		}
	}
}

void FDeferredReplicationSystemCalls::StopReplicatingActorComponentDefer(UActorComponent* ActorComponent, EEndReplicationFlags EndReplicationFlags)
{
	if (UWorld* World = ActorComponent->GetWorld())
	{
		if (UNetDriver* NetDriver = World->GetNetDriver())
		{
			if (UReplicationSystem* ReplicationSystem = NetDriver->GetReplicationSystem())
			{
				if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
				{
					Bridge->StopReplicatingComponentDefer(ActorComponent, EndReplicationFlags);
				}
			}
		}
	}
}

void FDeferredReplicationSystemCalls::FlushDeferred(UEngineReplicationBridge* Bridge)
{
	Bridge->FlushDeferred();
}

} // end namespace UE::Net

