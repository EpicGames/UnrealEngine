// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetAnalyticsTypes.h"
#include "HAL/IConsoleManager.h"

namespace NetAnalyticsCVars
{
	static int32 NumberOfConsecutiveDroppedPacketsToConsiderBurst = 3;
	static FAutoConsoleVariableRef CVarNumberOfDroppedPacketsToConsiderBurst(
		TEXT("NetAnalytics.NumberOfConsecutiveDroppedPacketsToConsiderBurst"),
		NumberOfConsecutiveDroppedPacketsToConsiderBurst,
		TEXT("The number of packets lost in a row (in or out) for us to consider the frame as having bursts of packet loss.")
		TEXT("Not affected by NetAnalytics.MinimumNumberOfPacketsForBurstTracking.")
	);

	static int32 MinimumNumberOfPacketsForBurstTracking = 5;
	static FAutoConsoleVariableRef CVarMinimumNumberOfPacketsForBurstTracking(
		TEXT("NetAnalytics.MinimumNumberOfPacketsForBurstTracking"),
		MinimumNumberOfPacketsForBurstTracking,
		TEXT("The minimum number of packets that must have been notified (in our out) in order to consider a frame for packet loss by percentage.")
		TEXT("See NetAnalytics.PercentOfDroppedPacketsToConsiderBurst")
	);

	static float PercentOfDroppedPacketsToConsiderBurst = 0.2f;
	static FAutoConsoleVariableRef CVarPercentOfDroppedPacketsToConsiderBurst(
		TEXT("NetAnalytics.PercentOfDroppedPacketsToConsiderBurst"),
		PercentOfDroppedPacketsToConsiderBurst,
		TEXT("The percentage of packets lost in a frame (in or out) for us to consider the frame as having bursts of packet loss.\n")
		TEXT("See NetAnalytics.MinimumNumberOfPacketsForBurstTracking.")
	);
}

FDelinquencyAnalytics::FDelinquencyAnalytics(const uint32 InNumberOfTopOffendersToTrack) :
	TotalTime(0.f),
	NumberOfTopOffendersToTrack(InNumberOfTopOffendersToTrack)
{
	TopOffenders.Reserve(NumberOfTopOffendersToTrack);
}

FDelinquencyAnalytics::FDelinquencyAnalytics(FDelinquencyAnalytics&& Other)
	: TopOffenders(MoveTemp(Other.TopOffenders))
	, AllDelinquents(MoveTemp(Other.AllDelinquents))
	, TotalTime(Other.TotalTime)
	, NumberOfTopOffendersToTrack(Other.NumberOfTopOffendersToTrack)
{
	TopOffenders.Reserve(NumberOfTopOffendersToTrack);
}

void FDelinquencyAnalytics::Add(FDelinquencyNameTimePair&& ToTrack)
{
	struct FHelper
	{
		static bool Compare(const FDelinquencyNameTimePair& LHS, const FDelinquencyNameTimePair& RHS)
		{
			return LHS.TimeSeconds > RHS.TimeSeconds;
		}
	};

	// Regardless of whether or not this item has been seen before, there was a new entry
	// so we'll add that time to the total.
	TotalTime += ToTrack.TimeSeconds;

	// TODO: We might consider tracking the code below as totals instead of the max time for a single event.
	// For example, an Actor could end up queueing bunches several times within a reporting window, and each
	// of those events would add to the TotalTime, but below we will only have the time of the longest period
	// where bunches were queued.

	if (NumberOfTopOffendersToTrack == 0)
	{
		AllDelinquents.Emplace(MoveTemp(ToTrack));
	}
	else if (TopOffenders.Num() == 0)
	{
		TopOffenders.Add(ToTrack);
		AllDelinquents.Emplace(MoveTemp(ToTrack));
		return;
	}
	else
	{
		if (FDelinquencyNameTimePair* AlreadyTracked = AllDelinquents.Find(ToTrack))
		{
			// We found an entry, so check its time.
			if (AlreadyTracked->TimeSeconds >= ToTrack.TimeSeconds)
			{
				// We already have tracked a worse offense for this entry, so there's nothing more
				// we need to do.
				return;
			}
			else if (TopOffenders.Num() > 0)
			{
				const float LeastOffensiveTime = TopOffenders.Last().TimeSeconds;
				if (AlreadyTracked->TimeSeconds >= LeastOffensiveTime)
				{
					// Our previous offense would have been tracked in the TopOffenders list, so go ahead and remove it.
					// We're sorted highest to lowest, but are using greater than to actually compare, so UpperBound should
					// return the index above which of the entry just before us.
					// However, it's possible that other entries have the same time, so we need to make sure we remove the
					// correct one.
					int32 MaybeOurEntry = Algo::UpperBound<>(TopOffenders, ToTrack, &FHelper::Compare);
					while (true)
					{
						// Sanity check that we have actually found our entry.
						// If we hit the end of the list, or we see an entry that should be less offensive, then
						// we've missed our entry.
						if (MaybeOurEntry == TopOffenders.Num() || TopOffenders[MaybeOurEntry].TimeSeconds < AlreadyTracked->TimeSeconds)
						{
							// It's possible that multiple entries have the same delinquency time.
							// If our current entry matches the LeastOffensiveTime, it's possible that we were in the
							// top offenders list at one point but were pushed out when more offensive entries were added.
							// If our time doesn't match the least offensive time, we should definitely be in the list.
							ensureMsgf(AlreadyTracked->TimeSeconds == LeastOffensiveTime, TEXT("FDelinquencyAnalytics::Add - Unable to find expected entry %s:%f, list may not be sorted!"), *AlreadyTracked->Name.ToString(), AlreadyTracked->TimeSeconds);
							break;
						}

						// We found our entry, so we're done.
						else if (TopOffenders[MaybeOurEntry].Name == AlreadyTracked->Name)
						{
							TopOffenders.RemoveAt(MaybeOurEntry, EAllowShrinking::No);
							break;
						}

						++MaybeOurEntry;
					}
				}
			}
		}

		AllDelinquents.Add(ToTrack);

		const int32 LocalNumberOfTopOffendersToTrack = static_cast<int32>(NumberOfTopOffendersToTrack);
		const int32 InsertAt = Algo::UpperBound<>(TopOffenders, ToTrack, &FHelper::Compare);

		// Check to see if this time was ranked in our top offenders.
		if (InsertAt < LocalNumberOfTopOffendersToTrack)
		{
			// If we're going to displace a previous top offender, remove the least offensive.
			if (LocalNumberOfTopOffendersToTrack == TopOffenders.Num())
			{
				TopOffenders.RemoveAt(TopOffenders.Num() - 1, EAllowShrinking::No);
			}

			TopOffenders.InsertUninitialized(InsertAt, 1);
			TopOffenders[InsertAt] = MoveTemp(ToTrack);
		}
	}
}

void FDelinquencyAnalytics::Reset()
{
	TopOffenders.Reset();
	AllDelinquents.Reset();
	TotalTime = 0;
}

void FDelinquencyAnalytics::CountBytes(FArchive& Ar) const
{
	TopOffenders.CountBytes(Ar);
	AllDelinquents.CountBytes(Ar);
}

void FNetConnectionSaturationAnalytics::TrackFrame(const bool bIsSaturated)
{
	++NumberOfTrackedFrames;

	if (bIsSaturated)
	{
		++NumberOfSaturatedFrames;
		++CurrentRunOfSaturatedFrames;
        if (CurrentRunOfSaturatedFrames > LongestRunOfSaturatedFrames)
        {
            LongestRunOfSaturatedFrames = CurrentRunOfSaturatedFrames;
        }
	}
	else
	{
		CurrentRunOfSaturatedFrames = 0;
	}
}

void FNetConnectionSaturationAnalytics::TrackReplication(const bool bIsSaturated)
{
	++NumberOfReplications;

	if (bIsSaturated)
	{
		++NumberOfSaturatedReplications;
		++CurrentRunOfSaturatedReplications;
        if (CurrentRunOfSaturatedReplications > LongestRunOfSaturatedReplications)
        {
            LongestRunOfSaturatedReplications = CurrentRunOfSaturatedReplications;
        }
	}
	else
	{
		CurrentRunOfSaturatedReplications = 0;
	}
}

void FNetConnectionSaturationAnalytics::Reset()
{
	NumberOfSaturatedFrames = 0;
	NumberOfTrackedFrames = 0;
	LongestRunOfSaturatedFrames = 0;

	NumberOfReplications = 0;
	NumberOfSaturatedReplications = 0;
	LongestRunOfSaturatedReplications = 0;

	CurrentRunOfSaturatedFrames = 0;
	CurrentRunOfSaturatedReplications = 0;
}

void FNetConnectionPacketAnalytics::Reset()
{
	bSawPacketLossBurstThisFrame = false;
	NumberOfAcksThisFrame = 0;
	NumberOfNaksThisFrame = 0;
	NumberOfMissingPacketsThisFrame = 0;
	NumberOfPacketsThisFrame = 0;
	CurrentRunOfDroppedOutPackets = 0;
	LongestRunOfDroppedOutPackets = 0;
	LongestRunOfDroppedInPackets = 0;
	NumberOfFramesWithBurstsOfPacketLoss = 0;
	NumberOfFramesWithNoPackets = 0;
	NumberOfTrackedFrames = 0;
}

void FNetConnectionPacketAnalytics::TrackAck(int32 PacketId)
{
	++NumberOfAcksThisFrame;

	// We will only know about NAKs when we receive ACKs, so we only need to update this here.
	if (UNLIKELY(CurrentRunOfDroppedOutPackets > 0))
	{
		if (NetAnalyticsCVars::NumberOfConsecutiveDroppedPacketsToConsiderBurst > 0)
		{
			bSawPacketLossBurstThisFrame |= CurrentRunOfDroppedOutPackets >= static_cast<uint32>(NetAnalyticsCVars::NumberOfConsecutiveDroppedPacketsToConsiderBurst);
		}

		LongestRunOfDroppedOutPackets = FMath::Max<>(CurrentRunOfDroppedOutPackets, LongestRunOfDroppedOutPackets);
		CurrentRunOfDroppedOutPackets = 0;
	}
}

void FNetConnectionPacketAnalytics::TrackNak(int32 PacketId)
{
	++NumberOfNaksThisFrame;
	++CurrentRunOfDroppedOutPackets;
}

void FNetConnectionPacketAnalytics::TrackInPacket(uint32 InPacketId, uint32 NumberOfMissingPackets)
{
	++NumberOfPacketsThisFrame;

	// We will only know about missing packets when we receive a packet, so we only need to update this here.
	if (UNLIKELY(NumberOfMissingPackets))
	{
		if (NetAnalyticsCVars::NumberOfConsecutiveDroppedPacketsToConsiderBurst > 0)
		{
			bSawPacketLossBurstThisFrame |= NumberOfMissingPackets >= static_cast<uint32>(NetAnalyticsCVars::NumberOfConsecutiveDroppedPacketsToConsiderBurst);
		}

		NumberOfMissingPackets += NumberOfMissingPacketsThisFrame;
		LongestRunOfDroppedInPackets = FMath::Max<>(NumberOfMissingPackets, LongestRunOfDroppedInPackets);
	}
}

void FNetConnectionPacketAnalytics::Tick()
{
	++NumberOfTrackedFrames;

	const int32 NumberOfInPackets = NumberOfPacketsThisFrame + NumberOfMissingPacketsThisFrame;
	const int32 NumberOfOutPackets = NumberOfAcksThisFrame + NumberOfNaksThisFrame;

	if (NumberOfInPackets == 0 && NumberOfOutPackets == 0)
	{
		++NumberOfFramesWithNoPackets;
		return;
	}

	if (!bSawPacketLossBurstThisFrame &&
		NetAnalyticsCVars::MinimumNumberOfPacketsForBurstTracking > 0 &&
		NetAnalyticsCVars::PercentOfDroppedPacketsToConsiderBurst > 0.f)
	{
		const float PercentInPacketsLost = (NumberOfInPackets > 0) ? static_cast<float>(NumberOfMissingPacketsThisFrame) / static_cast<float>(NumberOfInPackets) : 0.f;
		const float PercentOutPacketsLost = (NumberOfOutPackets > 0) ? static_cast<float>(NumberOfNaksThisFrame) / static_cast<float>(NumberOfOutPackets) : 0.f;

		bSawPacketLossBurstThisFrame = (PercentInPacketsLost >= NetAnalyticsCVars::PercentOfDroppedPacketsToConsiderBurst) ||
			(PercentOutPacketsLost >= NetAnalyticsCVars::PercentOfDroppedPacketsToConsiderBurst);
	}

	if (bSawPacketLossBurstThisFrame)
	{
		++NumberOfFramesWithBurstsOfPacketLoss;
	}

	bSawPacketLossBurstThisFrame = false;
	NumberOfAcksThisFrame = 0;
	NumberOfNaksThisFrame = 0;
	NumberOfMissingPacketsThisFrame = 0;
	NumberOfPacketsThisFrame = 0;
}
