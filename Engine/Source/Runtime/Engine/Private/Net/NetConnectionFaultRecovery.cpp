// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/NetConnectionFaultRecovery.h"
#include "Engine/NetConnection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetConnectionFaultRecovery)


/**
 * UNetFaultConfig
 */

void UNetFaultConfig::InitConfigDefaultsInternal()
{
	bEnabled = true;

	EscalationSeverity.Empty();

	EscalationSeverity.Add(TEXT("Normal"));
	EscalationSeverity.Add(TEXT("Fault"));
	EscalationSeverity.Add(TEXT("PersistentFault"));
	EscalationSeverity.Add(TEXT("DisconnectCountdown"));
	EscalationSeverity.Add(TEXT("Disconnect"));
}

namespace UE
{
namespace Net
{

/**
 * FNetConnectionFaultRecovery
 */

void FNetConnectionFaultRecovery::InitDefaults(FString InConfigContext, UNetConnection* InConnection)
{
	ConfigContext = InConfigContext;
	Connection = InConnection;

	if (Connection != nullptr)
	{
		LastInTotalHandlerPackets = Connection->GetInTotalHandlerPackets();

		DefaultFaultHandler.InitFaultRecovery(this);
	}
}

void FNetConnectionFaultRecovery::TickRealtime(double TimeSeconds)
{
	if (NetFaultEscalationManager.IsValid())
	{
		if (Connection != nullptr && !NetFaultEscalationManager->IsDormant())
		{
			FEscalationCounter& FrameCounter = NetFaultEscalationManager->GetFrameCounter((int32)ENetFaultCounters::PacketCount);
			const int32 NewInTotalHandlerPacket = Connection->GetInTotalHandlerPackets();

			FrameCounter.Counter += (NewInTotalHandlerPacket - LastInTotalHandlerPackets);
			LastInTotalHandlerPackets = NewInTotalHandlerPacket;
		}

		if (NetFaultEscalationManager->DoesRequireTick())
		{
			NetFaultEscalationManager->TickRealtime(TimeSeconds);
		}
	}

	if (Connection != nullptr && (TimeSeconds - LastPerSecPacketCheck) > 1.0)
	{
		const int32 NewInTotalHandlerPacket = Connection->GetInTotalHandlerPackets();

		InTotalHandlerPacketsPerSec = NewInTotalHandlerPacket - LastInTotalHandlerPacketsPerSec;
		LastInTotalHandlerPacketsPerSec = NewInTotalHandlerPacket;
		LastPerSecPacketCheck = TimeSeconds;
	}
}

bool FNetConnectionFaultRecovery::DoesRequireTick() const
{
	// Technically DoesRequireTick/!IsDormant are the same, but checking in case that changes in the future
	return NetFaultEscalationManager.IsValid();
}

void FNetConnectionFaultRecovery::InitEscalationManager()
{
	if (!NetFaultEscalationManager.IsValid())
	{
		NetFaultEscalationManager = MakeUnique<FNetFaultEscalationHandler>();
		NetFaultEscalationManager->SetNotifySeverityUpdate(
			[this](const FEscalationState& OldState, const FEscalationState& NewState, ESeverityUpdate UpdateType)
			{
				NotifySeverityUpdate(OldState, NewState, UpdateType);
			});

		NetFaultEscalationManager->Init(ConfigContext);

		if (Connection != nullptr)
		{
			NetFaultEscalationManager->SetManagerContext(Connection->LowLevelGetRemoteAddress(true));
		}

		const int32 MaxStaticCounter = (int32)ENetFaultCounters::Max;

		if (LastCounterIndex != MaxStaticCounter)
		{
			const int32 FirstAddIdx = NetFaultEscalationManager->AddNewCounter(LastCounterIndex - MaxStaticCounter);

			// Cached CounterIndex values need to be accurate
			check(FirstAddIdx == MaxStaticCounter);
		}

		for (const FPendingCategoryRegister& CurEntry : PendingCategories)
		{
			NetFaultEscalationManager->RegisterCounterCategory(CurEntry.CategoryIndex, CurEntry.CounterIndex);
		}

		PendingCategories.Empty();
	}
}

void FNetConnectionFaultRecovery::NotifySeverityUpdate(const FEscalationState& OldState, const FEscalationState& NewState,
														ESeverityUpdate UpdateType)
{
	const FNetFaultState& NetFaultState = static_cast<const FNetFaultState&>(NewState);
	bool bEscalated = (UpdateType == ESeverityUpdate::Escalate || UpdateType == ESeverityUpdate::AutoEscalate);

	if (bEscalated)
	{
		// Prime the dormant packet counter, so quota checks have a usable packet count to work with
		if (OldState.bDormant && InTotalHandlerPacketsPerSec > 0)
		{
			FEscalationCounter& FrameCounter = NetFaultEscalationManager->GetFrameCounter((int32)ENetFaultCounters::PacketCount);

			FrameCounter.Counter += InTotalHandlerPacketsPerSec;
		}

		if (NetFaultState.bCloseConnection && Connection != nullptr)
		{
			TPimplPtr<FNetResult, EPimplPtrMode::DeepCopy> FinalFaultList;

			AddToChainResultPtr(FinalFaultList, ENetCloseResult::FaultDisconnect);
			AddToChainResultPtr(FinalFaultList, ENetCloseResult::NotRecoverable);
			AddToChainResultPtr(FinalFaultList, MoveTemp(*TrackedFaults));

			Connection->Close(MoveTemp(*FinalFaultList));

			TrackedFaultEnumHashes.Empty();

			bDisconnected = true;
		}
	}
	else
	{
		if (NetFaultState.bDormant && TrackedFaults.IsValid())
		{
			if (Connection != nullptr)
			{
				FNetConnAnalyticsVars& AnalyticsVars = Connection->AnalyticsVars;

				for (FNetCloseResult::FConstIterator It(*TrackedFaults); It; ++It)
				{
					int32& CountValue = AnalyticsVars.RecoveredFaults.FindOrAdd(It->DynamicToString(ENetResultString::ResultEnumOnly));

					CountValue++;
				}
			}

			TrackedFaults.Reset();
			TrackedFaultEnumHashes.Empty();
		}
	}
}

}
}

