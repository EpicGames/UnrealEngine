// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetConnectionIdHandler.h"

FNetConnectionIdHandler::FNetConnectionIdHandler()
: IdHint(0)
{
}

FNetConnectionIdHandler::~FNetConnectionIdHandler()
{
}

void FNetConnectionIdHandler::Init(uint32 IdCount)
{
	check(IdCount > 0);
	UsedIds.Init(false, IdCount + 1);
	// Treat 0 as an invalid connection ID. It's used by NetConnection CDOs.
	UsedIds[0] = true;
}

uint32 FNetConnectionIdHandler::Allocate()
{
	uint32 Id = UsedIds.FindAndSetFirstZeroBit(IdHint);
	ensureMsgf(Id != INDEX_NONE, TEXT("Out of connection IDs. All %d are in use."), UsedIds.Num());
	IdHint = Id + 1;
	return Id != INDEX_NONE ? Id : 0U;
}

void FNetConnectionIdHandler::Free(uint32 Id)
{
	check(Id != 0 && Id < uint32(UsedIds.Num()));

	UsedIds[Id] = (Id == 0U);
	IdHint = FMath::Min(IdHint, Id);
}

uint32 FNetConnectionIdHandler::GetMaxConnectionIdCount() const
{
	return UsedIds.Num();
}

