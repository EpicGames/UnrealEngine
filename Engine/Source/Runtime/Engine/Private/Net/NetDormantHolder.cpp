// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetDormantHolder.h"

#include "Net/DataReplication.h"
#include "Engine/NetworkObjectList.h"

namespace UE::Net::Private
{

/*-----------------------------------------------------------------------------
	FDormantObjectReplicator
-----------------------------------------------------------------------------*/

FDormantObjectReplicator::FDormantObjectReplicator(FObjectKey InObjectKey)
	: ObjectKey(InObjectKey)
	, Replicator(MakeShared<FObjectReplicator>())
{
	//nothing
}

FDormantObjectReplicator::FDormantObjectReplicator(FObjectKey InObjectKey, const TSharedRef<FObjectReplicator>& ExistingReplicator)
	: ObjectKey(InObjectKey)
	, Replicator(ExistingReplicator)
{
	//nothing
}

/*-----------------------------------------------------------------------------
	FDormantReplicatorHolder
-----------------------------------------------------------------------------*/

bool FDormantReplicatorHolder::DoesReplicatorExist(AActor* DormantActor, UObject* ReplicatedObject) const
{
	if (const FActorDormantReplicators* ActorReplicators = ActorReplicatorSet.Find(DormantActor))
	{
		const FObjectKey SubObjectKey = ReplicatedObject;

		return (ActorReplicators->DormantReplicators.Find(SubObjectKey) != nullptr);
	}

	return false;
}

TSharedPtr<FObjectReplicator> FDormantReplicatorHolder::FindReplicator(AActor* DormantActor, UObject* ReplicatedObject)
{
	TSharedPtr<FObjectReplicator> ReplicatorPtr;

	if (FActorDormantReplicators* ActorReplicators = ActorReplicatorSet.Find(DormantActor))
	{
		const FObjectKey SubObjectKey = ReplicatedObject;
		
		if (FDormantObjectReplicator* SubObjectReplicator = ActorReplicators->DormantReplicators.Find(SubObjectKey))
		{
			ReplicatorPtr = SubObjectReplicator->Replicator;
		}
	}

	return ReplicatorPtr;
}

TSharedPtr<FObjectReplicator> FDormantReplicatorHolder::FindAndRemoveReplicator(AActor* DormantActor, UObject* ReplicatedObject)
{
	TSharedPtr<FObjectReplicator> ReplicatorPtr;

	if (FActorDormantReplicators* ActorReplicators = ActorReplicatorSet.Find(DormantActor))
	{
		const FObjectKey SubObjectKey = ReplicatedObject;
		FSetElementId Index = ActorReplicators->DormantReplicators.FindId(ReplicatedObject);

		if (Index.IsValidId())
		{
			ReplicatorPtr = ActorReplicators->DormantReplicators[Index].Replicator;
			ActorReplicators->DormantReplicators.Remove(Index);
		}
	}

	return ReplicatorPtr;
}

const TSharedRef<FObjectReplicator>& FDormantReplicatorHolder::CreateAndStoreReplicator(AActor* DormantActor, UObject* ReplicatedObject, bool& bOverwroteExistingReplicator)
{
	FActorDormantReplicators& ActorReplicators = ActorReplicatorSet.FindOrAdd(FActorDormantReplicators(DormantActor));

	const FObjectKey SubObjectKey = ReplicatedObject;	

	// Add a new replicator tied to this object. 
	// If there was already a replicator for the same object in the set, it will be destroyed and overwritten by this new one.
	FSetElementId Index = ActorReplicators.DormantReplicators.Add(FDormantObjectReplicator(SubObjectKey), &bOverwroteExistingReplicator);

	return ActorReplicators.DormantReplicators[Index].Replicator;
}

void FDormantReplicatorHolder::StoreReplicator(AActor* DormantActor, UObject* ReplicatedObject, const TSharedRef<FObjectReplicator>& ObjectReplicator)
{
	FActorDormantReplicators& ActorReplicators = ActorReplicatorSet.FindOrAdd(FActorDormantReplicators(DormantActor));

	ActorReplicators.DormantReplicators.Add(FDormantObjectReplicator(ReplicatedObject, ObjectReplicator));
}

bool FDormantReplicatorHolder::RemoveStoredReplicator(AActor* DormantActor, FObjectKey ReplicatedObjectKey)
{
	FSetElementId Index = ActorReplicatorSet.FindId(DormantActor);
	const bool bIsFound = Index.IsValidId();
	if (bIsFound)
	{
		ActorReplicatorSet[Index].DormantReplicators.Remove(ReplicatedObjectKey);

		// Cleanup the actor entry if its not holding any other replicators
		if (ActorReplicatorSet[Index].DormantReplicators.IsEmpty())		
		{
			ActorReplicatorSet.Remove(Index);
		}
	}

	return bIsFound;
}

void FDormantReplicatorHolder::CleanupAllReplicatorsOfActor(AActor* DormantActor)
{
	ActorReplicatorSet.Remove(DormantActor);
}

void FDormantReplicatorHolder::CleanupStaleObjects(FNetworkObjectList& NetworkObjectList, UObject* ReferenceOwner)
{
#if UE_REPLICATED_OBJECT_REFCOUNTING
	TArray<TWeakObjectPtr<UObject>, TInlineAllocator<16>> CleanedUpObjects;
#endif

	for (FActorReplicatorSet::TIterator ActorSetIt = ActorReplicatorSet.CreateIterator(); ActorSetIt; ++ActorSetIt)
	{
		for (FActorDormantReplicators::FObjectReplicatorSet::TIterator ReplicatorSetIt = ActorSetIt->DormantReplicators.CreateIterator(); ReplicatorSetIt; ++ReplicatorSetIt)
		{
			FDormantObjectReplicator& DormantReplicator = *ReplicatorSetIt;
			TWeakObjectPtr<UObject> DormantObjectPtr = DormantReplicator.Replicator->GetWeakObjectPtr();

			if (!DormantObjectPtr.IsValid())
			{
#if UE_REPLICATED_OBJECT_REFCOUNTING
				// Keep track of the cleaned up object if it's a subobject and not the main actor
				if (ActorSetIt->OwnerActorKey != DormantReplicator.ObjectKey)
				{
					CleanedUpObjects.Add(DormantObjectPtr);
				}
#endif

				ReplicatorSetIt.RemoveCurrent();
			}
		}

#if UE_REPLICATED_OBJECT_REFCOUNTING
		if (CleanedUpObjects.Num() > 0)
		{
			NetworkObjectList.RemoveMultipleSubObjectChannelReference(ActorSetIt->OwnerActorKey, CleanedUpObjects, ReferenceOwner);
			CleanedUpObjects.Reset();
		}
#endif

		if (ActorSetIt->DormantReplicators.IsEmpty())
		{
			ActorSetIt.RemoveCurrent();
		}
	}
}

void FDormantReplicatorHolder::ForEachDormantReplicator(UE::Net::FExecuteForEachDormantReplicator Function)
{
	for (const FActorDormantReplicators& ActorReplicators : ActorReplicatorSet)
	{
		for (const FDormantObjectReplicator& DormantReplicator : ActorReplicators.DormantReplicators)
		{
			Function(ActorReplicators.OwnerActorKey, DormantReplicator.ObjectKey, DormantReplicator.Replicator);
		}
	}
}

void FDormantReplicatorHolder::ForEachDormantReplicatorOfActor(AActor* DormantActor, UE::Net::FExecuteForEachDormantReplicator Function)
{
	if (FActorDormantReplicators* ActorReplicators = ActorReplicatorSet.Find(DormantActor))
	{
		for (const FDormantObjectReplicator& DormantReplicator : ActorReplicators->DormantReplicators)
		{
			Function(ActorReplicators->OwnerActorKey, DormantReplicator.ObjectKey, DormantReplicator.Replicator);
		}
	}
}

void FDormantReplicatorHolder::EmptySet()
{
	ActorReplicatorSet.Empty();
}

void FDormantReplicatorHolder::CountBytes(FArchive& Ar) const
{
	ActorReplicatorSet.CountBytes(Ar);
	for (const FActorDormantReplicators& ActorReplicators : ActorReplicatorSet)
	{
		ActorReplicators.CountBytes(Ar);
	}

	FlushedObjectMap.CountBytes(Ar);

	for (auto FlushedIt = FlushedObjectMap.CreateConstIterator(); FlushedIt; ++FlushedIt)
	{
		FlushedIt.Value().CountBytes(Ar);
	}
}

UE::Net::FDormantObjectMap* FDormantReplicatorHolder::FindFlushedObjectsForActor(AActor* Actor)
{
	return FlushedObjectMap.Find(Actor);
}

UE::Net::FDormantObjectMap& FDormantReplicatorHolder::FindOrAddFlushedObjectsForActor(AActor* Actor)
{
	return FlushedObjectMap.FindOrAdd(Actor);
}

void FDormantReplicatorHolder::ClearFlushedObjectsForActor(AActor* Actor)
{
	FlushedObjectMap.Remove(Actor);
}

} //end namespace UE::Net::Private