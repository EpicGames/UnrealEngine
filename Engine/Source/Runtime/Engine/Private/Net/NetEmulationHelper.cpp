// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetEmulationHelper.h"

#include "Engine/Engine.h"
#include "Engine/NetworkSettings.h"
#include "Engine/NetDriver.h"
#include "Misc/ConfigCacheIni.h"

#if DO_ENABLE_NET_TEST

namespace UE::Net::Private::NetEmulationHelper
{

/** Global that stores the network emulation values outside the NetDriver lifetime */
static TOptional<FPacketSimulationSettings> PersistentPacketSimulationSettings;

void CreatePersistentSimulationSettings()
{
	if (!PersistentPacketSimulationSettings.IsSet())
	{
		PersistentPacketSimulationSettings.Emplace(FPacketSimulationSettings());
	}
}

void ApplySimulationSettingsOnNetDrivers(UWorld* World, const FPacketSimulationSettings& Settings)
{
	// Execute on all active NetDrivers
	FWorldContext& Context = GEngine->GetWorldContextFromWorldChecked(World);
	for (const FNamedNetDriver& ActiveNetDriver : Context.ActiveNetDrivers)
	{
		if (ActiveNetDriver.NetDriver)
		{
			ActiveNetDriver.NetDriver->SetPacketSimulationSettings(Settings);
		}
	}
}

bool HasPersistentPacketEmulationSettings()
{
	return PersistentPacketSimulationSettings.IsSet();
}

void ApplyPersistentPacketEmulationSettings(UNetDriver* NetDriver)
{
	NetDriver->SetPacketSimulationSettings(PersistentPacketSimulationSettings.GetValue());
}

FAutoConsoleCommandWithWorldArgsAndOutputDevice NetEmulationPktEmulationProfile(TEXT("NetEmulation.PktEmulationProfile"), TEXT("Apply a preconfigured emulation profile."),
FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Output)
{
	bool bProfileApplied(false);
	if (Args.Num() > 0)
	{
		FString CmdParams = FString::Printf(TEXT("PktEmulationProfile=%s"), *(Args[0]));

		CreatePersistentSimulationSettings();

		bProfileApplied = PersistentPacketSimulationSettings.GetValue().ParseSettings(*CmdParams, nullptr);

		if (bProfileApplied)
		{
			ApplySimulationSettingsOnNetDrivers(World, PersistentPacketSimulationSettings.GetValue());
		}
		else
		{
			Output.Log(FString::Printf(TEXT("EmulationProfile: %s was not found in Engine.ini"), *(Args[0])));
		}
	}
	else
	{
		Output.Log(FString::Printf(TEXT("Missing emulation profile name")));
	}

	if (!bProfileApplied)
	{
		if (const UNetworkSettings* NetworkSettings = GetDefault<UNetworkSettings>())
		{
			Output.Log(TEXT("List of some supported emulation profiles:"));
			for (const FNetworkEmulationProfileDescription& ProfileDesc : NetworkSettings->NetworkEmulationProfiles)
			{
				Output.Log(FString::Printf(TEXT("%s"), *ProfileDesc.ProfileName));
			}
		}
	}
}));

FAutoConsoleCommandWithWorld NetEmulationOff(TEXT("NetEmulation.Off"), TEXT("Turn off network emulation"),
FConsoleCommandWithWorldDelegate::CreateStatic([](UWorld* World)
{
	CreatePersistentSimulationSettings();
	PersistentPacketSimulationSettings.GetValue().ResetSettings();
	ApplySimulationSettingsOnNetDrivers(World, PersistentPacketSimulationSettings.GetValue());
}));


FAutoConsoleCommandWithWorldArgsAndOutputDevice NetEmulationDropNothingFunction(TEXT("NetEmulation.DropNothing"), TEXT("Disables any RPC drop settings previously set."),
FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Output)
{
	UNetDriver* NetDriver = World->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	NetDriver->SendRPCDel.Unbind();
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice NetEmulationDropAnyUnreliableFunction(TEXT("NetEmulation.DropAnyUnreliable"), TEXT("Drop any sent unreliable RPCs. (optional)<0-100> to set the drop percentage (default is 20)."),
FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Output)
{
	UNetDriver* NetDriver = World->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	float DropPercentage = 20.f;
	if (Args.Num() > 0)
	{
		float DropParam = DropPercentage;
		LexFromString(DropParam, *Args[0]);
		if (DropParam > 0.0f)
		{
			DropPercentage = FMath::Min(DropParam, 100.f);
		}
	}

	NetDriver->SendRPCDel.BindLambda([DropPercentage](AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject, bool& bOutBlockSendRPC)
	{
		if ((Function->FunctionFlags & FUNC_NetReliable) == 0)
		{
			const float RandValue = FMath::FRand();
			bOutBlockSendRPC = RandValue > (DropPercentage * 0.01f);

			if (SubObject)
			{
				UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls::%ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(SubObject), *GetNameSafe(Function));
			}
			else
			{
				UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(Function));
			}
		}
	});

	UE_LOGF(LogNetTraffic, Warning, "Will start dropping %.2f%% of all unreliable RPCs", DropPercentage);
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice NetEmulationDropUnreliableOfActorClassFunction(TEXT("NetEmulation.DropUnreliableOfActorClass"),
	TEXT("Drop random unreliable RPCs sent on actors of the given class type. "
		"<ActorClassName> Class name to match with (can be a substring). "
		"(optional)<0-100> to set the drop percentage (default is 20)."),
FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Output)
{
	UNetDriver* NetDriver = World->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	if (Args.Num() < 1)
	{
		UE_LOGF(LogNet, Warning, "No class name parameter passed to NetEmulation.DropUnreliableOfActorClass");
		return;
	}

	FString ClassNameParam = Args[0];

	float DropPercentage = 20.f;
	if (Args.Num() > 1)
	{
		float DropParam = DropPercentage;
		LexFromString(DropParam, *Args[1]);
		if (DropParam > 0.0f)
		{
			DropPercentage = FMath::Min(DropParam, 100.f);
		}
	}

	NetDriver->SendRPCDel.BindLambda([ClassNameParam, DropPercentage](AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject, bool& bOutBlockSendRPC)
	{
		if ((Function->FunctionFlags & FUNC_NetReliable) == 0)
		{
			const float RandValue = FMath::FRand();
			if (RandValue <= (DropPercentage * 0.01f))
			{
				return;
			}

			bool bMatches = false;

			UClass* Class = Actor->GetClass();
			while (Class && !bMatches)
			{
				bMatches = Class->GetName().Contains(*ClassNameParam);
				Class = Class->GetSuperClass();
			}

			if (bMatches)
			{
				bOutBlockSendRPC = true;

				if (SubObject)
				{
					UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls::%ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(SubObject), *GetNameSafe(Function));
				}
				else
				{
					UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(Function));
				}
			}
		}
	});

	UE_LOGF(LogNetTraffic, Warning, "Will start dropping %.2f%% of all unreliable RPCs of actors of class: %ls", DropPercentage, *ClassNameParam);
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice NetEmulationDropUnreliableRPCFunction(TEXT("NetEmulation.DropUnreliableRPC"),
	TEXT("Drop randomly the unreliable RPCs of the given name. "
		"<RPCName> The name of the RPC (can be a substring). "
		"(optional)<0-100> to set the drop percentage (default is 20)."),
FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Output)
{
	UNetDriver* NetDriver = World->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	if (Args.Num() < 1)
	{
		UE_LOGF(LogNet, Warning, "No RPC name parameter passed to NetEmulation.DropUnreliableRPC");
		return;
	}

	FString RPCNameParam = Args[0];

	float DropPercentage = 20.f;
	if (Args.Num() > 1)
	{
		float DropParam = DropPercentage;
		LexFromString(DropParam, *Args[1]);
		if (DropParam > 0.0f)
		{
			DropPercentage = FMath::Min(DropParam, 100.f);
		}
	}

	NetDriver->SendRPCDel.BindLambda([RPCNameParam, DropPercentage](AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject, bool& bOutBlockSendRPC)
	{
		if ((Function->FunctionFlags & FUNC_NetReliable) == 0)
		{
			const float RandValue = FMath::FRand();
			if (RandValue <= (DropPercentage * 0.01f))
			{
				return;
			}

			if (Function->GetName().Contains(*RPCNameParam))
			{
				bOutBlockSendRPC = true;

				if (SubObject)
				{
					UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls::%ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(SubObject), *GetNameSafe(Function));
				}
				else
				{
					UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(Function));
				}
			}
		}
	});

	UE_LOGF(LogNetTraffic, Warning, "Will start dropping %.2f%% of all unreliable RPCs named: %ls", DropPercentage, *RPCNameParam);
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice NetEmulationDropUnreliableOfSubObjectClassFunction(TEXT("NetEmulation.DropUnreliableOfSubObjectClass"),
	TEXT("Drop randomly the unreliable RPCs of a subobject of the given class. "
		"<SubObjectClassName> The name of the RPC (can be a substring). "
		"(optional)<0-100> to set the drop percentage (default is 20)."),
FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World, FOutputDevice& Output)
{
	UNetDriver* NetDriver = World->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	if (Args.Num() < 1)
	{
		UE_LOGF(LogNet, Warning, "No SubObject name parameter passed to NetEmulation.DropUnreliableOfSubObjectClass");
		return;
	}

	FString SubObjectClassNameParam = Args[0];

	float DropPercentage = 20.f;
	if (Args.Num() > 1)
	{
		float DropParam = DropPercentage;
		LexFromString(DropParam, *Args[1]);
		if (DropParam > 0.0f)
		{
			DropPercentage = FMath::Min(DropParam, 100.f);
		}
	}

	NetDriver->SendRPCDel.BindLambda([SubObjectClassNameParam, DropPercentage](AActor* Actor, UFunction* Function, void* Parameters, FOutParmRec* OutParms, FFrame* Stack, UObject* SubObject, bool& bOutBlockSendRPC)
	{
		if (SubObject && (Function->FunctionFlags & FUNC_NetReliable) == 0)
		{
			const float RandValue = FMath::FRand();
			if (RandValue <= (DropPercentage * 0.01f))
			{
				return;
			}

			bool bMatches = false;

			UClass* Class = SubObject->GetClass();
			while (Class && !bMatches)
			{
				bMatches = Class->GetName().Contains(*SubObjectClassNameParam);
				Class = Class->GetSuperClass();
			}

			if (bMatches)
			{
				bOutBlockSendRPC = true;

				UE_LOGF(LogNetTraffic, Log, "      Dropped unreliable RPC %ls::%ls : %ls", *GetFullNameSafe(Actor), *GetNameSafe(SubObject), *GetNameSafe(Function));
			}
		}
	});

	UE_LOGF(LogNetTraffic, Warning, "Will start dropping %.2f%% of all unreliable RPCs for subobjects: %ls", DropPercentage, *SubObjectClassNameParam);
}));

#define BUILD_NETEMULATION_CONSOLE_COMMAND(CommandName, CommandHelp) FAutoConsoleCommandWithWorldAndArgs NetEmulation##CommandName(TEXT("NetEmulation."#CommandName), TEXT(CommandHelp), \
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic([](const TArray<FString>& Args, UWorld* World) \
	{ \
		if (Args.Num() > 0) \
		{ \
			CreatePersistentSimulationSettings(); \
			FString CmdParams = FString::Printf(TEXT(#CommandName"=%s"), *(Args[0])); \
			PersistentPacketSimulationSettings.GetValue().ParseSettings(*CmdParams, nullptr); \
			ApplySimulationSettingsOnNetDrivers(World, PersistentPacketSimulationSettings.GetValue()); \
		} \
	}));

	BUILD_NETEMULATION_CONSOLE_COMMAND(PktLoss, "Simulates network packet loss");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktOrder, "Simulates network packets received out of order");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktDup, "Simulates sending/receiving duplicate network packets");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktLag, "Simulates network packet lag");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktLagVariance, "Simulates variable network packet lag");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktLagMin, "Sets minimum outgoing packet latency");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktLagMax, "Sets maximum outgoing packet latency)");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktIncomingLagMin, "Sets minimum incoming packet latency");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktIncomingLagMax, "Sets maximum incoming packet latency");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktIncomingLoss, "Simulates incoming packet loss");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktJitter, "Simulates outgoing packet jitter");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktBufferBloatInMS, "Simulates outgoing buffer bloat");
	BUILD_NETEMULATION_CONSOLE_COMMAND(PktIncomingBufferBloatInMS, "Simulates incoming buffer bloat");


} // end namespace UE::Net::Private::NetEmulationHelper

#endif //#if DO_ENABLE_NET_TEST

class FPacketSimulationConsoleCommandVisitor
{
public:
	static void OnPacketSimulationConsoleCommand(const TCHAR* Name, IConsoleObject* CVar, TArray<IConsoleObject*>& Sink)
	{
		Sink.Add(CVar);
	}
};

/** reads in settings from the .ini file
 * @note: overwrites all previous settings
 */
void FPacketSimulationSettings::LoadConfig(const TCHAR* OptionalQualifier)
{
#if DO_ENABLE_NET_TEST
	ConfigHelperInt(TEXT("PktLoss"), PktLoss, OptionalQualifier);

	ConfigHelperInt(TEXT("PktLossMinSize"), PktLossMinSize, OptionalQualifier);
	ConfigHelperInt(TEXT("PktLossMaxSize"), PktLossMaxSize, OptionalQualifier);

	bool InPktOrder = !!PktOrder;
	ConfigHelperBool(TEXT("PktOrder"), InPktOrder, OptionalQualifier);
	PktOrder = int32(InPktOrder);

	ConfigHelperInt(TEXT("PktLag"), PktLag, OptionalQualifier);
	ConfigHelperInt(TEXT("PktLagVariance"), PktLagVariance, OptionalQualifier);

	ConfigHelperInt(TEXT("PktLagMin"), PktLagMin, OptionalQualifier);
	ConfigHelperInt(TEXT("PktLagMax"), PktLagMax, OptionalQualifier);

	ConfigHelperInt(TEXT("PktDup"), PktDup, OptionalQualifier);

	ConfigHelperInt(TEXT("PktIncomingLagMin"), PktIncomingLagMin, OptionalQualifier);
	ConfigHelperInt(TEXT("PktIncomingLagMax"), PktIncomingLagMax, OptionalQualifier);
	ConfigHelperInt(TEXT("PktIncomingLoss"), PktIncomingLoss, OptionalQualifier);

	ConfigHelperInt(TEXT("PktJitter"), PktJitter, OptionalQualifier);
	
	ConfigHelperInt(TEXT("PktBufferBloatInMS"), PktBufferBloatInMS, OptionalQualifier);
	ConfigHelperInt(TEXT("PktIncomingBufferBloatInMS"), PktIncomingBufferBloatInMS, OptionalQualifier);

	ValidateSettings();
#endif //DO_ENABLE_NET_TEST
}

bool FPacketSimulationSettings::LoadEmulationProfile(const TCHAR* ProfileName)
{
#if DO_ENABLE_NET_TEST
	const FString SectionName = FString::Printf(TEXT("%s.%s"), TEXT("PacketSimulationProfile"), ProfileName);

	TArray<FString> SectionConfigs;
	bool bSectionExists = GConfig->GetSection(*SectionName, SectionConfigs, GEngineIni);
	if (!bSectionExists)
	{
		UE_LOGF(LogNet, Log, "EmulationProfile [%ls] was not found in %ls. Packet settings were not changed", *SectionName, *GEngineIni);
		return false;
	}

	ResetSettings();

	UScriptStruct* ThisStruct = FPacketSimulationSettings::StaticStruct();

	for (const FString& ConfigVar : SectionConfigs)
	{
		FString VarName;
		FString VarValue;
		if (ConfigVar.Split(TEXT("="), &VarName, &VarValue))
		{
			// If using the one line struct definition
			if (VarName.Equals(TEXT("PacketSimulationSettings"), ESearchCase::IgnoreCase))
			{
				ThisStruct->ImportText(*VarValue, this, nullptr, 0, (FOutputDevice*)GWarn, TEXT("FPacketSimulationSettings"));
			}
			else if (FProperty* StructProperty = ThisStruct->FindPropertyByName(FName(*VarName, FNAME_Find)))
			{
				StructProperty->ImportText_InContainer(*VarValue, this, nullptr, 0);
			}
			else
			{
				UE_LOGF(LogNet, Warning, "FPacketSimulationSettings::LoadEmulationProfile could not find property named %ls", *VarName);
			}
		}
	}

	ValidateSettings();
#endif //DO_ENABLE_NET_TEST
	return true;
}

void FPacketSimulationSettings::ResetSettings()
{
#if DO_ENABLE_NET_TEST
	* this = FPacketSimulationSettings();
#endif
}

void FPacketSimulationSettings::ValidateSettings()
{
#if DO_ENABLE_NET_TEST

	PktLoss = FMath::Clamp<int32>(PktLoss, 0, 100);

	PktOrder = FMath::Clamp<int32>(PktOrder, 0, 1);

	PktLagMin = FMath::Max(PktLagMin, 0);
	PktLagMax = FMath::Max(PktLagMin, PktLagMax);

	PktDup = FMath::Clamp<int32>(PktDup, 0, 100);

	PktIncomingLagMin = FMath::Max(PktIncomingLagMin, 0);
	PktIncomingLagMax = FMath::Max(PktIncomingLagMin, PktIncomingLagMax);
	PktIncomingLoss = FMath::Clamp<int32>(PktIncomingLoss, 0, 100);

	PktBufferBloatInMS = FMath::Max(PktBufferBloatInMS, 0);
	PktIncomingBufferBloatInMS = FMath::Max(PktIncomingBufferBloatInMS, 0);
#endif
}

bool FPacketSimulationSettings::ConfigHelperInt(const TCHAR* Name, int32& Value, const TCHAR* OptionalQualifier)
{
#if DO_ENABLE_NET_TEST

	if (OptionalQualifier)
	{
		if (GConfig->GetInt(TEXT("PacketSimulationSettings"), *FString::Printf(TEXT("%s%s"), OptionalQualifier, Name), Value, GEngineIni))
		{
			return true;
		}
	}

	if (GConfig->GetInt(TEXT("PacketSimulationSettings"), Name, Value, GEngineIni))
	{
		return true;
	}
#endif
	return false;
}

bool FPacketSimulationSettings::ConfigHelperBool(const TCHAR* Name, bool& Value, const TCHAR* OptionalQualifier)
{
#if DO_ENABLE_NET_TEST
	if (OptionalQualifier)
	{
		if (GConfig->GetBool(TEXT("PacketSimulationSettings"), *FString::Printf(TEXT("%s%s"), OptionalQualifier, Name), Value, GEngineIni))
		{
			return true;
		}
	}

	if (GConfig->GetBool(TEXT("PacketSimulationSettings"), Name, Value, GEngineIni))
	{
		return true;
	}
#endif
	return false;
}

/**
 * Reads the settings from a string: command line or an exec
 *
 * @param Stream the string to read the settings from
 */
bool FPacketSimulationSettings::ParseSettings(const TCHAR* Cmd, const TCHAR* OptionalQualifier)
{
	// note that each setting is tested.
	// this is because the same function will be used to parse the command line as well
	bool bParsed = false;

#if DO_ENABLE_NET_TEST
	FString EmulationProfileName;
	if (FParse::Value(Cmd, TEXT("PktEmulationProfile="), EmulationProfileName))
	{
		UE_LOGF(LogNet, Log, "Applying EmulationProfile %ls", *EmulationProfileName);
		bParsed = LoadEmulationProfile(*EmulationProfileName);
	}
	if (ParseHelper(Cmd, TEXT("PktLoss="), PktLoss, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLoss set to %d", PktLoss);
	}
	if (ParseHelper(Cmd, TEXT("PktLossMinSize="), PktLossMinSize, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLossMinSize set to %d", PktLossMinSize);
	}
	if (ParseHelper(Cmd, TEXT("PktLossMaxSize="), PktLossMaxSize, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLossMaxSize set to %d", PktLossMaxSize);
	}
	if (ParseHelper(Cmd, TEXT("PktOrder="), PktOrder, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktOrder set to %d", PktOrder);
	}
	if (ParseHelper(Cmd, TEXT("PktLag="), PktLag, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLag set to %d", PktLag);
	}
	if (ParseHelper(Cmd, TEXT("PktDup="), PktDup, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktDup set to %d", PktDup);
	}
	if (ParseHelper(Cmd, TEXT("PktLagVariance="), PktLagVariance, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLagVariance set to %d", PktLagVariance);
	}
	if (ParseHelper(Cmd, TEXT("PktLagMin="), PktLagMin, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLagMin set to %d", PktLagMin);
	}
	if (ParseHelper(Cmd, TEXT("PktLagMax="), PktLagMax, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktLagMax set to %d", PktLagMax);
	}
	if (ParseHelper(Cmd, TEXT("PktIncomingLagMin="), PktIncomingLagMin, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktIncomingLagMin set to %d", PktIncomingLagMin);
	}
	if (ParseHelper(Cmd, TEXT("PktIncomingLagMax="), PktIncomingLagMax, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktIncomingLagMax set to %d", PktIncomingLagMax);
	}
	if (ParseHelper(Cmd, TEXT("PktIncomingLoss="), PktIncomingLoss, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktIncomingLoss set to %d", PktIncomingLoss);
	}
	if (ParseHelper(Cmd, TEXT("PktJitter="), PktJitter, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktJitter set to %d", PktJitter);
	}
	if (ParseHelper(Cmd, TEXT("PktBufferBloatInMS="), PktBufferBloatInMS, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktBufferBloatInMS set to %d", PktBufferBloatInMS);
	}
	if (ParseHelper(Cmd, TEXT("PktIncomingBufferBloatInMS="), PktIncomingBufferBloatInMS, OptionalQualifier))
	{
		bParsed = true;
		UE_LOGF(LogNet, Log, "PktIncomingBufferBloatInMS set to %d", PktIncomingBufferBloatInMS);
	}

	ValidateSettings();
#endif //DO_ENABLE_NET_TEST
	return bParsed;
}

bool FPacketSimulationSettings::ParseHelper(const TCHAR* Cmd, const TCHAR* Name, int32& Value, const TCHAR* OptionalQualifier)
{
#if DO_ENABLE_NET_TEST
	if (OptionalQualifier)
	{
		if (FParse::Value(Cmd, *FString::Printf(TEXT("%s%s"), OptionalQualifier, Name), Value))
		{
			return true;
		}
	}

	if (FParse::Value(Cmd, Name, Value))
	{
		return true;
	}
#endif
	return false;
}
