// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/NetDriver.h"

class UWorld;

#if DO_ENABLE_NET_TEST

namespace UE::Net::Private::NetEmulationHelper
{
	void CreatePersistentSimulationSettings();
	void ApplySimulationSettingsOnNetDrivers(UWorld* World, const FPacketSimulationSettings& Settings);
	bool HasPersistentPacketEmulationSettings();
	void ApplyPersistentPacketEmulationSettings(UNetDriver* NetDriver);
} //namespace UE::Net::Private::NetEmulationHelper

#endif //#if DO_ENABLE_NET_TEST
