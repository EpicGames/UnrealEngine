// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetIDVariant.h"
#include "EngineLogs.h"

namespace UE::Net
{

FNetIDVariant::FNetIDVariant(FNetworkGUID NetGUID)
	: Variant(TInPlaceType<FNetworkGUID>(), NetGUID)
{
}

FNetIDVariant::FNetIDVariant(FNetRefHandle NetRefHandle)
	: Variant(TInPlaceType<FNetRefHandle>(), NetRefHandle)
{
}

bool FNetIDVariant::operator==(const FNetIDVariant& RHS) const
{
	if (Variant.GetIndex() != RHS.Variant.GetIndex())
	{
		return false;
	}

	if (Variant.IsType<FNetworkGUID>())
	{
		return Variant.Get<FNetworkGUID>() == RHS.Variant.Get<FNetworkGUID>();
	}
	else if (Variant.IsType<FNetRefHandle>())
	{
		return Variant.Get<FNetRefHandle>() == RHS.Variant.Get<FNetRefHandle>();
	}

	return true;
}

bool FNetIDVariant::IsValid() const
{
	if (Variant.IsType<FNetworkGUID>())
	{
		return Variant.Get<FNetworkGUID>().IsValid();
	}
	else
	if (Variant.IsType<FNetRefHandle>())
	{
		return Variant.Get<FNetRefHandle>().IsValid();
	}
	else
	{
		return false;
	}
}

FArchive& operator<<(FArchive& Ar, FNetIDVariant& NetID)
{
	constexpr SIZE_T ExpectedSize = 3;
	static_assert(TVariantSize_V<FNetIDVariant::FVariantType> == ExpectedSize, "FNetIDVariant variant size changed, potential serialization incompatibility.");

	uint32 TypeIndex = static_cast<uint32>(NetID.Variant.GetIndex());

	Ar.SerializeInt(TypeIndex, TVariantSize_V<FNetIDVariant::FVariantType>);

	if (Ar.IsSaving())
	{
		if (NetID.Variant.IsType<FNetworkGUID>())
		{
			FNetworkGUID WriteGUID = NetID.Variant.Get<FNetworkGUID>();
			Ar << WriteGUID;
		}
		else if (NetID.Variant.IsType<FNetRefHandle>())
		{
			FNetRefHandle WriteHandle = NetID.Variant.Get<FNetRefHandle>();
			Ar << WriteHandle;
		}
	}
	else
	{
		if (TypeIndex == FNetIDVariant::FVariantType::IndexOfType<FNetworkGUID>())
		{
			FNetworkGUID ReadGUID;
			Ar << ReadGUID;
			NetID.Variant.Set<FNetworkGUID>(ReadGUID);
		}
		else if (TypeIndex == FNetIDVariant::FVariantType::IndexOfType<FNetRefHandle>())
		{
			FNetRefHandle ReadRefHandle;
			Ar << ReadRefHandle;
			NetID.Variant.Set<FNetRefHandle>(ReadRefHandle);
		}
		else if(TypeIndex == FNetIDVariant::FVariantType::IndexOfType<FNetIDVariant::FEmptyID>())
		{
			NetID.Variant = FNetIDVariant::FVariantType();
		}
		else
		{
			// Invalid TypeIndex
			UE_LOGF(LogNet, Warning, "Unknown TypeIndex %u reading an FNetIDVariant.", TypeIndex);
			Ar.SetError();
		}
	}

	return Ar;
}

FString FNetIDVariant::ToString() const
{
	if (Variant.IsType<FNetworkGUID>())
	{
		return Variant.Get<FNetworkGUID>().ToString();
	}
	else if (Variant.IsType<FNetRefHandle>())
	{
		return Variant.Get<FNetRefHandle>().ToString();
	}
	return FString("Invalid");
}

}