// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetNameDebug.h"

#if UE_NET_REPACTOR_NAME_DEBUG
#include "UObject/Class.h"
#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"


#ifndef UE_NET_REPACTOR_NAME_DEBUG_PROFILING
	#define UE_NET_REPACTOR_NAME_DEBUG_PROFILING 0
#endif

#if UE_NET_REPACTOR_NAME_DEBUG_PROFILING
	DECLARE_CYCLE_STAT(TEXT("DemoAsync_NameDebug"), STAT_NetAsyncDemoNameDebug, STATGROUP_Net);
#endif


namespace UE::Net
{
	int32 GCVarNetActorNameDebug = 0;
	int32 GCVarNetSubObjectNameDebug = 0;
	static int32 GCVarNetSubObjectFullNameDebugUnsafe = 0;
	static int32 GCVarNetAsyncDemoNameDebugChance = 10;
	static int32 GAsyncDemoNameDebugCounter = 0;

	static FAutoConsoleVariableRef CVarNetActorNameDebug(
		TEXT("net.ActorNameDebug"),
		GCVarNetActorNameDebug,
		TEXT("When turned on the name of actors being replicated for replays, will be stored on the stack for crashdumps."));

	static FAutoConsoleVariableRef CVarNetSubObjectNameDebug(
		TEXT("net.SubObjectNameDebug"),
		GCVarNetSubObjectNameDebug,
		TEXT("When turned on the name of subobjects being replicated for replays, will be stored on the stack for crashdumps."));

	static FAutoConsoleVariableRef CVarNetSubObjectFullNameDebugUnsafe(
		TEXT("net.SubObjectFullNameDebugUnsafe"),
		GCVarNetSubObjectFullNameDebugUnsafe,
		TEXT("When turned on the full name of subobjects being replicated for replays, will be stored on the stack for crashdumps. ")
		TEXT("This is unsafe, as when debugging for invalid objects, this will trigger the crash early."));

	static FAutoConsoleVariableRef CVarNetAsyncDemoNameDebugChance(
		TEXT("net.AsyncDemoNameDebugChance"),
		GCVarNetAsyncDemoNameDebugChance,
		TEXT("When actor/subobject demo name debugging is enabled, this is the percentage chance per replication that names will be cached. ")
		TEXT("This is to minimize the performance impact - and also represents the percentage of crash dumps that will have data on the stack."));

#if UE_NET_REPACTOR_NAME_DEBUG_PROFILING
	static int32 GCVarNetAsyncDemoNameDebugProfiling = 0;

	static FAutoConsoleVariableRef CVarNetAsyncDemoNameDebugProfiling(
		TEXT("net.AsyncDemoNameDebugProfiling"),
		GCVarNetAsyncDemoNameDebugProfiling,
		TEXT("Enables profiling of 'net.ActorNameDebug' and 'net.SubObjectNameDebug' name caching."));
#endif
}

namespace UE::Net::Private
{
	bool HasMetAsyncDemoNameDebugChance()
	{
		bool bReturnVal = false;

		GAsyncDemoNameDebugCounter--;

		if (GAsyncDemoNameDebugCounter <= 0)
		{
			const int32 Variance = static_cast<int32>(GCVarNetAsyncDemoNameDebugChance * 0.5);

			bReturnVal = GAsyncDemoNameDebugCounter == 0;
			GAsyncDemoNameDebugCounter = (GCVarNetAsyncDemoNameDebugChance - Variance) + FMath::RandRange(0, Variance) + 1;
		}

		return bReturnVal;
	}

	/** Stores a name in FNameDebugBuffer, replacing any existing value */
	void StoreName_Internal(FNameDebugBuffer& Buffer, FName InName)
	{
		const FNameEntry* NameEntry = InName.GetDisplayNameEntry();

		Buffer.bIsWide = NameEntry->IsWide();
		Buffer.Length = NameEntry->GetNameLength();

		if (Buffer.bIsWide)
		{
			NameEntry->GetWideName(*reinterpret_cast<WIDECHAR(*)[NAME_SIZE]>(const_cast<WIDECHAR*>(Buffer.Wide)));
		}
		else
		{
			NameEntry->GetAnsiName(*reinterpret_cast<ANSICHAR(*)[NAME_SIZE]>(const_cast<ANSICHAR*>(Buffer.Ansi)));
		}
	}

	/** Appends a name to FNameDebugBuffer - not safe to use before a 'Store*' function initializes the buffer */
	void AppendName_Internal(FNameDebugBuffer& Buffer, FName InName)
	{
		const FNameEntry* NameEntry = InName.GetDisplayNameEntry();
		const bool bNewWide = NameEntry->IsWide();
		const int32 AppendLength = NameEntry->GetNameLength();

		if (Buffer.Length == 0)
		{
			Buffer.bIsWide = bNewWide;
		}

		if (Buffer.bIsWide == bNewWide && (Buffer.Length + AppendLength + 1) < FNameDebugBuffer::MaxNameAppend)
		{
			const int32 StartOffset = Buffer.Length;

			Buffer.Length += AppendLength + 1;

			if (Buffer.bIsWide)
			{
				Buffer.Wide[StartOffset] = '.';

				NameEntry->GetWideName(*reinterpret_cast<WIDECHAR(*)[NAME_SIZE]>(const_cast<WIDECHAR*>(Buffer.Wide) + StartOffset + 1));
			}
			else
			{
				Buffer.Ansi[StartOffset] = '.';

				NameEntry->GetAnsiName(*reinterpret_cast<ANSICHAR(*)[NAME_SIZE]>(const_cast<ANSICHAR*>(Buffer.Ansi) + StartOffset + 1));
			}
		}
	}

	/** Equivalent to UObjectBaseUtility::GetPathName - not safe to use before a 'Store*' function initializes the buffer */
	void AppendPathName_Internal(FNameDebugBuffer& Buffer, const UObject* InObj)
	{
		const UObject* CurOuter = InObj->GetOuter();

		if (CurOuter != nullptr)
		{
			AppendPathName_Internal(Buffer, CurOuter);
		}

		AppendName_Internal(Buffer, InObj->GetFName());
	}

	void AppendFullName_Internal(FNameDebugBuffer& Buffer, const UObject* InObj)
	{
		AppendName_Internal(Buffer, InObj->GetClass()->GetFName());
		AppendPathName_Internal(Buffer, InObj);
	}

	void StoreFullName(PTRINT BufferPtr, const UObject* InObj)
	{
		if (BufferPtr != 0)
		{
			FNameDebugBuffer& Buffer = *reinterpret_cast<FNameDebugBuffer*>(BufferPtr);

#if UE_NET_REPACTOR_NAME_DEBUG_PROFILING
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetAsyncDemoNameDebug, GCVarNetAsyncDemoNameDebugProfiling);
#endif

			StoreName_Internal(Buffer, InObj->GetClass()->GetFName());
			AppendPathName_Internal(Buffer, InObj);
		}
	}

	void StoreSubObjectName(PTRINT BufferPtr, FName ClassName, FName ObjName, const UObject* InObj)
	{
		if (BufferPtr != 0)
		{
			FNameDebugBuffer& Buffer = *reinterpret_cast<FNameDebugBuffer*>(BufferPtr);

#if UE_NET_REPACTOR_NAME_DEBUG_PROFILING
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_NetAsyncDemoNameDebug, GCVarNetAsyncDemoNameDebugProfiling);
#endif

			StoreName_Internal(Buffer, ClassName);
			AppendName_Internal(Buffer, ObjName);

			if (GCVarNetSubObjectFullNameDebugUnsafe)
			{
				AppendFullName_Internal(Buffer, InObj);
			}
		}
	}
}
#endif
