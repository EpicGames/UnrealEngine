// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/CoreNetTypes.h"

#if UE_NET_REPACTOR_NAME_DEBUG
#include "UObject/NameTypes.h"
#include "Net/Core/Misc/NetSubObjectRegistry.h"

/**
 * Alloca's an FNameDebugBuffer to a PTRINT - to be used by one of the 'Store*' name debug buffer functions.
 *
 * Uses PTRINT, because actual pointers get optimized away, even when volatile. Copy the PTRINT value, and paste the address to a debug Memory window.
 */
#define UE_NET_ALLOCA_NAME_DEBUG_BUFFER() \
	reinterpret_cast<PTRINT>(FMemory_Alloca_Aligned(sizeof(FNameDebugBuffer), alignof(FNameDebugBuffer)))

namespace UE::Net
{
	extern ENGINE_API int32 GCVarNetActorNameDebug;
	extern ENGINE_API int32 GCVarNetSubObjectNameDebug;
};

namespace UE::Net::Private
{
	/** Minimal FName data, kept on stack for crash dumps - volatile to prevent optimization - no construction */
	struct FNameDebugBuffer
	{
		static constexpr int32 MaxNameAppend = 128;

		union
		{
			volatile ANSICHAR Ansi[NAME_SIZE + MaxNameAppend];
			volatile WIDECHAR Wide[NAME_SIZE + MaxNameAppend];
		};

		int32 Length;
		bool bIsWide;
	};


	/** Returns true 'GCVarNetAsyncDemoNameDebugChance' percent of the time - but with some added determinism and variance */
	bool HasMetAsyncDemoNameDebugChance();

	/** Equivalent to UObjectBaseUtility::GetFullName */
	void StoreFullName(PTRINT BufferPtr, const UObject* InObj);

	/** Safely stores a subobject class and name in FNameDebugBuffer - optionally attempting an unsafe append of the full name after */
	void StoreSubObjectName(PTRINT BufferPtr, FName ClassName, FName ObjName, const UObject* InObj);

	inline void StoreSubObjectName(PTRINT BufferPtr, const FSubObjectRegistry::FEntry& SubObjectInfo)
	{
		StoreSubObjectName(BufferPtr, SubObjectInfo.SubObjectClassName, SubObjectInfo.SubObjectName, SubObjectInfo.GetSubObject());
	}

	inline void StoreSubObjectName(PTRINT BufferPtr, const FReplicatedComponentInfo& ComponentInfo)
	{
		StoreSubObjectName(BufferPtr, ComponentInfo.ComponentClassName, ComponentInfo.ComponentName, (UObject*)ComponentInfo.Component);
	}
};
#endif
