// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetPacketNotify.h"
#include "CoreGlobals.h"
#include "Serialization/BitReader.h"
#include "Net/Util/SequenceHistory.h"
#include "Net/Util/SequenceNumber.h"

FNetPacketNotify::FNetPacketNotify()
	: AckRecord(64)
	, WrittenHistoryWordCount(0)
{
}

FNetPacketNotify::SequenceNumberT::DifferenceT FNetPacketNotify::GetCurrentSequenceHistoryLength() const
{
	if (InAckSeq >= InAckSeqAck)
	{
		return FMath::Min(SequenceNumberT::Diff(InAckSeq, InAckSeqAck), (SequenceNumberT::DifferenceT)SequenceHistoryT::Size);
	}
	else
	{
		// Worst case send full history
		return (SequenceNumberT::DifferenceT)SequenceHistoryT::Size;
	}
}

bool FNetPacketNotify::WillSequenceFitInSequenceHistory(SequenceNumberT Seq) const
{
	if (Seq >= InAckSeqAck)
	{
		return (SIZE_T)SequenceNumberT::Diff(Seq, InAckSeqAck) <= SequenceHistoryT::Size;
	}

	return false;
}

bool FNetPacketNotify::GetHasUnacknowledgedAcks() const
{
	for (SequenceNumberT::DifferenceT It = 0, EndIt = GetCurrentSequenceHistoryLength(); It < EndIt; ++It)
	{
		if (InSeqHistory.IsDelivered(It))
		{
			return true;
		}
	}
	return false;
}

FNetPacketNotify::SequenceNumberT FNetPacketNotify::UpdateInAckSeqAck(SequenceNumberT::DifferenceT AckCount, SequenceNumberT AckedSeq)
{
	if ((SIZE_T)AckCount <= AckRecord.Count())
	{
		if (AckCount > 1)
		{
			AckRecord.PopNoCheck(AckCount - 1);
		}

		FSentAckData AckData = AckRecord.PeekNoCheck();
		AckRecord.PopNoCheck();

		// verify that we have a matching sequence number
		if (AckData.OutSeq == AckedSeq)
		{
			return AckData.InAckSeq;
		}
	}

	// Pessimistic view, should never occur but we do want to know about it if it would
	ensureMsgf(false, TEXT("FNetPacketNotify::UpdateInAckSeqAck - Failed to find matching AckRecord for %u"), AckedSeq.Get());
	
	return SequenceNumberT(AckedSeq.Get() - MaxSequenceHistoryLength);
}

void FNetPacketNotify::Init(SequenceNumberT InitialInSeq, SequenceNumberT InitialOutSeq)
{
	InSeqHistory.Reset();
	InSeq = InitialInSeq;
	InAckSeq = InitialInSeq;
	InAckSeqAck = InitialInSeq;
	OutSeq = InitialOutSeq;
	OutAckSeq = SequenceNumberT(InitialOutSeq.Get() - 1);
	WaitingForFlushSeqAck = OutAckSeq;
}

void FNetPacketNotify::SetWaitForSequenceHistoryFlush()
{
	SequenceNumberT FlushSeq = OutSeq;
	// If we're in the middle of writing a header (WriteHeader called but not yet Committed), we must wait for the next OutSeq since ackdata might have changed before it is commited.
	// This handles the edge case where flush triggers between WriteHeader and CommitAndIncrementOutSeq.
	if (WrittenHistoryWordCount != 0)
	{
		++FlushSeq;
	}
	UE_LOG_PACKET_NOTIFY_WARNING(TEXT("FNetPacketNotify::SetWaitForSequenceHistoryFlush - Wait for ack of next OutSeq: %u"), FlushSeq.Get());
	WaitingForFlushSeqAck = FlushSeq;
}

void FNetPacketNotify::AckSeq(SequenceNumberT AckedSeq, bool IsAck)
{
	check( AckedSeq == InSeq);
	while (AckedSeq > InAckSeq)
	{
		++InAckSeq;

		const bool bReportAcked = InAckSeq == AckedSeq ? IsAck : false;

		UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::AckSeq - AckedSeq: %u, IsAck %u, AckHistorySize: %d"), InAckSeq.Get(), bReportAcked ? 1u : 0u, GetCurrentSequenceHistoryLength());

		InSeqHistory.AddDeliveryStatus(bReportAcked);
	}
}

FNetPacketNotify::SequenceNumberT::DifferenceT FNetPacketNotify::InternalUpdate(const FNotificationHeader& NotificationData, SequenceNumberT::DifferenceT InSeqDelta)
{
	if (!WillSequenceFitInSequenceHistory(NotificationData.Seq) && !IsWaitingForSequenceHistoryFlush())
	{
		// If we will overflow our outgoing ack sequence history and it contains accepted processed packets we must initiate a re-sync of ack sequence history.
		// This is done by ignoring any new packets until we are in sync again.
		// This would typically only occur in situations where we would have had huge packet loss or spikes on the receiving end.
		if (GetHasUnacknowledgedAcks())
		{
			SetWaitForSequenceHistoryFlush();

			// Mark everything we can as lost up until the end of the sequence history, as we will not accept anything until we are confirmed as flushed
			SequenceNumberT NewInSeqToAck(NotificationData.Seq);
			NewInSeqToAck = SequenceNumberT(InAckSeqAck.Get() + MaxSequenceHistoryLength);

			if (NewInSeqToAck >= InAckSeq)
			{
				// Need to set InSeq to clamped value before naks
				InSeq = NewInSeqToAck;
				AckSeq(NewInSeqToAck, false);

				UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::Update - Waiting for sequence history flush - Rejected: %u Last InAckSeq: %u"), NotificationData.Seq.Get(), InAckSeq.Get());
			}
		}
		else
		{
			// We can reset if we have no previous acks and then can safely synthesize nacks on the receiving end
			const SequenceNumberT NewInAckSeqAck(NotificationData.Seq.Get() - 1);
			UE_LOG_PACKET_NOTIFY_WARNING(TEXT("FNetPacketNotify::Reset SequenceHistory - As it is empty. New InAckSeqAck: %u Old: %u"), NewInAckSeqAck.Get(), InAckSeqAck.Get());
			InAckSeqAck = NewInAckSeqAck;
			InSeqHistory.Reset();			
		}
	}

	// Update InSeq
	InSeq = NotificationData.Seq;

	if (IsWaitingForSequenceHistoryFlush())
	{
		UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::Update - Waiting for sequence history flush - Ignoring current InSeq: %u Last acked InSeq: %u"), NotificationData.Seq.Get(), InAckSeq.Get());
	}

	return InSeqDelta;
};

namespace 
{
	struct FPackedHeader
	{
		using SequenceNumberT = FNetPacketNotify::SequenceNumberT;

		static_assert(FNetPacketNotify::SequenceNumberBits <= 14, "SequenceNumbers must be smaller than 14 bits to fit history word count");

		enum { HistoryWordCountBits = 4 };
		enum { SeqMask				= (1 << FNetPacketNotify::SequenceNumberBits) - 1 };
		enum { HistoryWordCountMask	= (1 << HistoryWordCountBits) - 1 };
		enum { AckSeqShift			= HistoryWordCountBits };
		enum { SeqShift				= AckSeqShift + FNetPacketNotify::SequenceNumberBits };
		enum { ResetHistoryValue	= HistoryWordCountMask };
		
		// Validate expected WordCount, we reserve the last word in order to indicate if we are trying to reset the ack window
		static_assert(FNetPacketNotify::SequenceHistoryT::WordCount <= ((1 << HistoryWordCountBits) -1), "Sequence history word count (minus one) must fit within HistoryWordCountBits. Check FNetPacketNotify::MaxSequenceHistoryLength.");

		static uint32 Pack(SequenceNumberT Seq, SequenceNumberT AckedSeq, SIZE_T HistoryWordCount, bool bResetHistory)
		{
			// We reserve ResetHistoryValue to express if we are currently resetting sequence ack history
			check(HistoryWordCount != ResetHistoryValue);
			if (bResetHistory)
			{
				check(HistoryWordCount + 1 == FNetPacketNotify::SequenceHistoryT::WordCount);
				HistoryWordCount = ResetHistoryValue;
			}

			uint32 Packed = 0u;

			Packed |= Seq.Get() << SeqShift;
			Packed |= AckedSeq.Get() << AckSeqShift;
			Packed |= HistoryWordCount & HistoryWordCountMask;

			return Packed;
		}

		static SequenceNumberT GetSeq(uint32 Packed) { return SequenceNumberT(Packed >> SeqShift & SeqMask); }
		static SequenceNumberT GetAckedSeq(uint32 Packed) { return SequenceNumberT(Packed >> AckSeqShift & SeqMask); }
		static SIZE_T GetHistoryWordCount(uint32 Packed) { return (Packed & HistoryWordCountMask); }
	};
}

// These methods must always write and read the exact same number of bits, that is the reason for not using WriteInt/WrittedWrappedInt
bool FNetPacketNotify::WriteHeader(FBitWriter& Writer, bool bRefresh)
{
	// We always write at least 1 word
	SIZE_T CurrentHistoryWordCount = FMath::Clamp<SIZE_T>((GetCurrentSequenceHistoryLength() + SequenceHistoryT::BitsPerWord - 1u) / SequenceHistoryT::BitsPerWord, 1u, SequenceHistoryT::WordCount);

	// If we are waiting for SequenceHistoryFlush we will always write the full sequence history
	const bool bIsWaitingForSequenceHistoryFlush = IsWaitingForSequenceHistoryFlush();
	if (bIsWaitingForSequenceHistoryFlush && CurrentHistoryWordCount != SequenceHistoryT::WordCount)
	{
		CurrentHistoryWordCount = SequenceHistoryT::WordCount;
	}

	// We can only do a refresh if we do not need more space for the history
	if (bRefresh && (CurrentHistoryWordCount > WrittenHistoryWordCount))
	{
		return false;
	}

	// How many words of ack data should we write? If this is a refresh we must write the same size as the original header
	WrittenHistoryWordCount = bRefresh ? WrittenHistoryWordCount : CurrentHistoryWordCount;

	// This is the last InSeq we have acknowledged at this time
	WrittenInAckSeq = InAckSeq;

	SequenceNumberT::SequenceT Seq = OutSeq.Get();

	// If we are waiting for sequencehistory flush we send the latest recvd InSeq, otherwise we send the latest accepted or rejected sequence.
	SequenceNumberT::SequenceT AckedSeq = bIsWaitingForSequenceHistoryFlush ? InSeq.Get() : InAckSeq.Get();

	// Pack data into a uint
	uint32 PackedHeader = FPackedHeader::Pack(Seq, AckedSeq, WrittenHistoryWordCount - 1, bIsWaitingForSequenceHistoryFlush);

	// Write packed header
	Writer << PackedHeader;

	// Write ack history
	InSeqHistory.Write(Writer, WrittenHistoryWordCount);

	// Write AckedHistorySeq
	uint16 AckedHistorySeq = InAckSeq.Get();
	Writer << AckedHistorySeq;

	UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::WriteHeader - Seq %u, AckedSeq %u bReFresh %u bWaitingForSequenceHistoryFlush %u AckedHistorySeq %u HistorySizeInWords %" SIZE_T_FMT), Seq, AckedSeq, bRefresh ? 1u : 0u, bIsWaitingForSequenceHistoryFlush ? 1u : 0u, AckedHistorySeq, WrittenHistoryWordCount);

	return true;
}

bool FNetPacketNotify::ReadHeader(FNotificationHeader& Data, FBitReader& Reader) const
{
	// Read packed header
	uint32 PackedHeader = 0;	
	Reader << PackedHeader;

	// Unpack
	Data.Seq = FPackedHeader::GetSeq(PackedHeader);
	Data.AckedSeq = FPackedHeader::GetAckedSeq(PackedHeader);
	const SIZE_T HistoryWordCount = FPackedHeader::GetHistoryWordCount(PackedHeader);
	if (HistoryWordCount == FPackedHeader::ResetHistoryValue)
	{
		Data.HistoryWordCount = SequenceHistoryT::WordCount;
		Data.bRemoteIsWaitingForHistoryReset = true;
	}
	else
	{
		Data.HistoryWordCount = FPlatformMath::Min(HistoryWordCount + 1, SequenceHistoryT::WordCount);
		Data.bRemoteIsWaitingForHistoryReset = false;
	}
	
	// Read ack history
	Data.History.Read(Reader, Data.HistoryWordCount);

	// Read AckedHistorySeq
	uint16 AckedHistorySeq = Data.AckedSeq.Get();
	Reader << AckedHistorySeq;
	Data.AckedHistorySeq = SequenceNumberT(AckedHistorySeq);

	UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::ReadHeader - Seq %u, AckedSeq %u RemoteWaitingForFlush %u AckedHistorySeq %u HistorySizeInWords %" SIZE_T_FMT), Data.Seq.Get(), Data.AckedSeq.Get(), Data.bRemoteIsWaitingForHistoryReset ? 1U : 0U, Data.AckedHistorySeq.Get(), Data.HistoryWordCount);

	return Reader.IsError() == false;
}

FNetPacketNotify::SequenceNumberT FNetPacketNotify::CommitAndIncrementOutSeq()
{
	// we have not written a header...this is a fail.
	check(WrittenHistoryWordCount != 0);

	// Add entry to the ack-record so that we can update the InAckSeqAck when we received the ack for this OutSeq.
	AckRecord.Enqueue( {OutSeq, WrittenInAckSeq } );
	WrittenHistoryWordCount = 0u;
	
	return ++OutSeq;
}

