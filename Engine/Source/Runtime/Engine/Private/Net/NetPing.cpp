// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetPing.h"
#include "Stats/Stats.h"
#include "Icmp.h"
#include "Net/DataChannel.h"
#include "Stats/StatsTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetPing)


#if STATS
DECLARE_STATS_GROUP(TEXT("Ping"), STATGROUP_Ping, STATCAT_Advanced)

// Special values: -1 for ping types which are not enabled, -2 for ping types which have been disabled due to failure/timeout
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Ping (Incl. Frame)"), STAT_Ping_RoundTripInclFrame, STATGROUP_Ping);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Ping (Excl. Frame)"), STAT_Ping_RoundTripExclFrame, STATGROUP_Ping);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Ping (ICMP)"), STAT_Ping_ICMP, STATGROUP_Ping);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Ping Timeouts (ICMP)"), STAT_PingTimeouts_ICMP, STATGROUP_Ping);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Ping Timeout Percent (ICMP)"), STAT_PingTimeoutPercent_ICMP, STATGROUP_Ping);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Ping (UDP QoS)"), STAT_Ping_UDPQoS, STATGROUP_Ping);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Ping Timeouts (UDP QoS)"), STAT_PingTimeouts_UDPQoS, STATGROUP_Ping);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Ping Timeout Percent (UDP QoS)"), STAT_PingTimeoutPercent_UDPQoS, STATGROUP_Ping);

DECLARE_CYCLE_STAT(TEXT("UpdatePing Time"), STAT_Ping_UpdatePingTime, STATGROUP_Ping);

#endif


namespace UE::Net
{

FNetPing::FOnAlternatePingAddressReceivedSignature FNetPing::OnAlternatePingAddressReceivedDelegate;

// CVars

static TAutoConsoleVariable<int32> CVarNetPingEnabled(
	TEXT("net.NetPingEnabled"), 0,
	TEXT("Whether or not the NetPing ping handling interface is enabled. Used for centralized ping tracking, and ICMP/UDP ping. ")
	TEXT("(Valid values: 0 = Off, 1 = Enabled for client, 2 = Enabled for server and client, 3 = Enabled for server only)"));

static TAutoConsoleVariable<FString> CVarNetPingTypes(
	TEXT("net.NetPingTypes"), TEXT(""),
	TEXT("A comma-delimited list of EPingType pings to enable, and (optionally) the EPingAverageType averaging to apply to the ping ")
	TEXT("(e.g: \"RoundTrip=None,RoundTripExclFrame=PlayerStateAvg,ICMP=MovingAverage\")."));

static int32 GNetPingICMPInterval = 5.0;

static FAutoConsoleVariableRef CVarNetPingICMPInterval(
	TEXT("net.NetPingICMPInterval"), GNetPingICMPInterval,
	TEXT("Specifies the interval (in seconds) for performing ICMP pings."));

static int32 GNetPingUDPInterval = 5.0;

static FAutoConsoleVariableRef CVarNetPingUDPInterval(
	TEXT("net.NetPingUDPInterval"), GNetPingUDPInterval,
	TEXT("Specifies the interval (in seconds) for performing UDP pings."));

static int32 GNetPingUDPPort = 22222;

static FAutoConsoleVariableRef CVarNetPingUDPPort(
	TEXT("net.NetPingUDPPort"), GNetPingUDPPort,
	TEXT("For 'UDPQoS' ping type, sets the port used for pinging."));

static TAutoConsoleVariable<int32> CVarNetPingTimeoutDisableThreshold(
	TEXT("net.NetPingTimeoutDisableThreshold"), 15,
	TEXT("The number of times to send an ICMP/UDP ping when at a failure/timeout rate of 100%, before giving up and disabling pings."));

static int32 GNetPingMaxOverrideCount = 8;

static FAutoConsoleVariableRef CVarNetPingMaxOverrideCount(
	TEXT("net.NetPingMaxOverrideCount"), GNetPingMaxOverrideCount,
	TEXT("Specifies the maximum number of times the ping address may be overridden before being ignored"));

#if !UE_BUILD_SHIPPING
static int32 GNetPingDebugDump = 0;

static FAutoConsoleVariableRef CVarNetPingDebugDump(
	TEXT("net.NetPingDebugDump"), GNetPingDebugDump,
	TEXT("Whether or not to dump NetPing ping values to log every 5 seconds."));
#endif

namespace Private
{

/**
 * Base class for QoS-derived ping type handling
 */
class FNetPingQoSBase
{
protected:
	FNetPingQoSBase() = delete;

	/**
	 * Base constructor
	 *
	 * @param InPingType	The QoS ping type this instance will handle
	 * @param InOwner		The FNetPing instance which owns this ping type handler
	 */
	FNetPingQoSBase(EPingType InPingType, FNetPing* InOwner);

	/**
	 * Resets the ping type handler, discarding any active pings
	 */
	void Reset();

	/**
	 * Handles receiving a QoS ping result.
	 *
	 * @param Result	The QoS ping result to handle
	 */
	void PingResult(FIcmpEchoResult Result);

public:
	/**
	 * Release an in-progress QoS ping handler from its Owner
	 */
	void Release();


private:
	/** The QoS ping type this instance handles */
	const EPingType PingType;

	/** The FNetPing instance which owns this ping type handler */
	FNetPing* Owner = nullptr;

	/** Whether or not to discard the next ping value result (typically after a reset) */
	bool bDiscardNextPing = false;

protected:
	/** Whether or not a ping is in progress */
	bool bPingInProgress = false;

	/** The timestamp at which the last ping began */
	double PingTimestamp = 0.0;
};

/**
 * Handles QoS ICMP pings
 */
class FNetPingICMP : public FNetPingQoSBase, public TSharedFromThis<FNetPingICMP>
{
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

private:
	FNetPingICMP() = delete;

	/**
	 * Base constructor
	 *
	 * @param InOwner	The FNetPing instance which owns this ping type handler
	 */
	FNetPingICMP(FNetPing* InOwner);

public:
	/**
	 * Singleton for retrieving an instance to the sole ICMP ping handler.
	 *
	 * NOTE: Only the GameNetDriver client can obtain the ICMP ping handler,
	 * to prevent conflicting results from multiple active ICMP pings.
	 * This is a precautionary restriction, it might be possible to support this without conflicting results.
	 *
	 * @param InNetConn		The NetConnection whose FNetPing instance is trying to obtain the ICMP ping handler.
	 * @return				Returns a shared pointer to the ICMP ping handler, or nullptr if not available.
	 */
	static TSharedPtr<FNetPingICMP> GetNetPingICMP(UNetConnection* InNetConn);

	/**
	 * Perform an ICMP ping.
	 *
	 * @param CurTimeSeconds	The current time, in seconds.
	 * @param PingAddress		The address (without port) to ping.
	 */
	void Ping(double CurTimeSeconds, FString PingAddress);
};

/**
 * Handles QoS UDP pings
 */
class FNetPingUDPQoS : public FNetPingQoSBase, public TSharedFromThis<FNetPingUDPQoS>
{
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

private:
	FNetPingUDPQoS() = delete;

	/**
	 * Base constructor
	 *
	 * @param InOwner		The FNetPing instance which owns this ping type handler
	 */
	FNetPingUDPQoS(FNetPing* InOwner);

public:
	/**
	 * Creates a UDPQoS ping handler.
	 *
	 * @param InOwner	The FNetPing instance that will own the UDPQoS ping handler.
	 * @return			Returns a shared pointer to the UDPQoS ping handler.
	 */
	static TSharedPtr<FNetPingUDPQoS> CreateNetPingUDPQoS(FNetPing* InOwner);

	/**
	 * Perform a UDPQoS ping.
	 *
	 * @param CurTimeSeconds	The current time, in seconds.
	 * @param PingAddress		The address (with port) to ping.
	 */
	void Ping(double CurTimeSeconds, FString PingAddress);
};

}

/**
 * FNetPing::FMovingAverageStorage
 */

FNetPing::FMovingAverageStorage::FMovingAverageStorage()
{
	PingAccumulator.SetConsumer(&PingValues);
}

/**
 * FNetPing::FPlayerStateAvgStorage
 */

void FNetPing::FPlayerStateAvgStorage::Reset()
{
	PingCount = 0;
	PingTimeouts = 0;

	PingValues.Reset();

	CurPingBucket = 0;

	for (int32 BucketIdx=0; BucketIdx<UE_ARRAY_COUNT(PingBucket); BucketIdx++)
	{
		PingBucket[BucketIdx] = {};
	}

	CurPingBucketTimestamp = 0.0;
}

void FNetPing::FPlayerStateAvgStorage::UpdatePing(double TimeVal, float InPing)
{
	// APlayerState::UpdatePing copy

	// Limit the size of the ping, to avoid overflowing PingBucket values
	InPing = FMath::Min(1.1f, InPing);

	PingValues.PeekMeasurement(TimeVal, static_cast<double>(InPing));

	float InPingInMs = InPing * 1000.f;

	if ((TimeVal - CurPingBucketTimestamp) >= 1.0)
	{
		// Trigger ping recalculation now, while all buckets are 'full'
		//	(misses the latest ping update, but averages a full 4 seconds data)
		RecalculateAvgPing();

		CurPingBucket = (CurPingBucket + 1) % UE_ARRAY_COUNT(PingBucket);
		CurPingBucketTimestamp = TimeVal;


		PingBucket[CurPingBucket].PingSum = FMath::FloorToInt(InPingInMs);
		PingBucket[CurPingBucket].PingCount = 1;
	}
	// Limit the number of pings we accept per-bucket, to avoid overflowing PingBucket values
	else if (PingBucket[CurPingBucket].PingCount < 7)
	{
		PingBucket[CurPingBucket].PingSum += FMath::FloorToInt(InPingInMs);
		PingBucket[CurPingBucket].PingCount++;
	}
}

void FNetPing::FPlayerStateAvgStorage::RecalculateAvgPing()
{
	// APlayerState::RecalculateAvgPing copy
	int32 Sum = 0;
	int32 Count = 0;

	for (uint8 i=0; i<UE_ARRAY_COUNT(PingBucket); i++)
	{
		Sum += PingBucket[i].PingSum;
		Count += PingBucket[i].PingCount;
	}

	const double FinalPingSeconds = (Count > 0 ? (static_cast<double>(Sum) / (static_cast<double>(Count) * 1000.0)) : 0.0);

	PingValues.AddSample(FinalPingSeconds);
}

/**
 * FNetPing
 */

FNetPing::FNetPing(UNetConnection* InOwner)
	: Owner(InOwner)
{
}

FNetPing::~FNetPing()
{
	if (NetPingICMP.IsValid())
	{
		NetPingICMP->Release();
	}

	if (NetPingUDPQoS.IsValid())
	{
		NetPingUDPQoS->Release();
	}
}

TPimplPtr<FNetPing> FNetPing::CreateNetPing(UNetConnection* InOwner)
{
	if (const UNetDriver* Driver = InOwner != nullptr ? InOwner->Driver.Get() : nullptr)
	{
		const int32 EnabledVal = CVarNetPingEnabled.GetValueOnAnyThread();
		const bool bEnabledForClient = EnabledVal == 1 || EnabledVal == 2;
		const bool bEnabledForServer = EnabledVal == 2 || EnabledVal == 3;
		const bool bIsServer = Driver->IsServer();
		const bool bIsGameDriver = (Driver->NetDriverName == NAME_GameNetDriver || Driver->NetDriverName == NAME_PendingNetDriver);

		if (bIsGameDriver && ((!bIsServer && bEnabledForClient) || (bIsServer && bEnabledForServer)))
		{
			TPimplPtr<FNetPing> ReturnVal = MakePimpl<FNetPing>(InOwner);

			ReturnVal->Init();

			return ReturnVal;
		}
	}

	return nullptr;
}

void FNetPing::Init()
{
	using namespace UE::Net::Private;

	static FString LastPingTypesCVar;
	static EPingType LastPingTypes = EPingType::None;
	static EPingAverageType LastPingAverageTypes[static_cast<uint32>(EPingType::Count)] = { EPingAverageType::None };

	FString PingTypesCVar = CVarNetPingTypes.GetValueOnAnyThread();

	if (UNLIKELY(LastPingTypesCVar != PingTypesCVar))
	{
		LastPingTypesCVar = PingTypesCVar;

		TArray<FString> PingTypeList;
		UEnum* PingTypeEnum = StaticEnum<EPingType>();
		UEnum* PingAvgEnum = StaticEnum<EPingAverageType>();

		PingTypesCVar.ParseIntoArray(PingTypeList, TEXT(","));

		for (FString CurPingAndAvg : PingTypeList)
		{
			FString CurPingType;
			FString CurPingAvg;

			if (CurPingAndAvg.Split(TEXT("="), &CurPingType, &CurPingAvg))
			{
				int64 PingTypeVal = PingTypeEnum->GetValueByNameString(CurPingType);
				int64 PingAvgVal = PingAvgEnum->GetValueByNameString(CurPingAvg);

				if (PingTypeVal != INDEX_NONE && PingAvgVal != INDEX_NONE)
				{
					const EPingType FinalPingTypeVal = static_cast<EPingType>(PingTypeVal);
					const EPingAverageType FinalPingAvgVal = static_cast<EPingAverageType>(PingAvgVal);

					PingTypes |= FinalPingTypeVal;

					PingAverageTypes[PingTypeToIdx(FinalPingTypeVal)] = FinalPingAvgVal;
				}
				else
				{
					UE_LOGF(LogNet, Error, "FNetPing: Invalid 'net.NetPingTypes' entry: PingType: %ls (Valid: %i), "
							"PingAverageType: %ls (Valid: %i)", ToCStr(CurPingType), (int32)(PingTypeVal != INDEX_NONE),
							ToCStr(CurPingAvg), (int32)(PingAvgVal != INDEX_NONE))
				}
			}
			else
			{
				UE_LOGF(LogNet, Error, "FNetPing: Invalid 'net.NetPingTypes' entry '%ls' (Full CVar: %ls)",
						ToCStr(CurPingAndAvg), ToCStr(PingTypesCVar));
			}
		}

		LastPingTypes = PingTypes;

		for (int32 AvgIdx=0; AvgIdx<UE_ARRAY_COUNT(PingAverageTypes); AvgIdx++)
		{
			LastPingAverageTypes[AvgIdx] = PingAverageTypes[AvgIdx];
		}
	}
	else
	{
		PingTypes = LastPingTypes;

		for (int32 AvgIdx=0; AvgIdx<UE_ARRAY_COUNT(PingAverageTypes); AvgIdx++)
		{
			PingAverageTypes[AvgIdx] = LastPingAverageTypes[AvgIdx];
		}
	}


	auto GetNewStorageIdx = [this](EPingAverageType AvgType) -> uint8
		{
			if (AvgType == EPingAverageType::None)
			{
				return PingNoAverageStorage.AddDefaulted();
			}
			else if (AvgType == EPingAverageType::MovingAverage)
			{
				return PingMovingAverageStorage.AddDefaulted();
			}
			else //if (AvgType == EPingAverageType::PlayerStateAvg)
			{
				return PingPlayerStateAvgStorage.AddDefaulted();
			}
		};

	if (EnumHasAnyFlags(PingTypes, EPingType::RoundTrip))
	{
		const int32 PingTypeIdx = PingTypeToIdx(EPingType::RoundTrip);

		PingAverageStorageIdx[PingTypeIdx] = GetNewStorageIdx(PingAverageTypes[PingTypeIdx]);
	}

	if (EnumHasAnyFlags(PingTypes, EPingType::RoundTripExclFrame))
	{
		const int32 PingTypeIdx = PingTypeToIdx(EPingType::RoundTripExclFrame);

		PingAverageStorageIdx[PingTypeIdx] = GetNewStorageIdx(PingAverageTypes[PingTypeIdx]);
	}

	if (UNetConnection* ServerConn = (Owner && Owner->Driver ? Owner->Driver->ServerConnection : nullptr))
	{
		if (EnumHasAnyFlags(PingTypes, EPingType::ICMP))
		{
			const int32 PingTypeIdx = PingTypeToIdx(EPingType::ICMP);

			PingAverageStorageIdx[PingTypeIdx] = GetNewStorageIdx(PingAverageTypes[PingTypeIdx]);

			NetPingICMP = FNetPingICMP::GetNetPingICMP(ServerConn);
		}

		if (EnumHasAnyFlags(PingTypes, EPingType::UDPQoS))
		{
			const int32 PingTypeIdx = PingTypeToIdx(EPingType::UDPQoS);

			PingAverageStorageIdx[PingTypeIdx] = GetNewStorageIdx(PingAverageTypes[PingTypeIdx]);

			NetPingUDPQoS = FNetPingUDPQoS::CreateNetPingUDPQoS(this);
		}
	}

	// Negative ping to represent no values read
#if STATS
	SET_DWORD_STAT(STAT_Ping_RoundTripInclFrame, (EnumHasAnyFlags(PingTypes, EPingType::RoundTrip) ? 0 : -1));
	SET_DWORD_STAT(STAT_Ping_RoundTripExclFrame, (EnumHasAnyFlags(PingTypes, EPingType::RoundTripExclFrame) ? 0 : -1));
	SET_DWORD_STAT(STAT_Ping_ICMP, (EnumHasAnyFlags(PingTypes, EPingType::ICMP) ? 0 : -1));
	SET_DWORD_STAT(STAT_Ping_UDPQoS, (EnumHasAnyFlags(PingTypes, EPingType::UDPQoS) ? 0 : -1));
#endif
}

void FNetPing::UpdatePing(EPingType PingType, double TimeVal, double PingValue)
{
	using namespace UE::Net::Private;

#if STATS
	SCOPE_CYCLE_COUNTER(STAT_Ping_UpdatePingTime);
#endif

	if (EnumHasAnyFlags(PingTypes, PingType))
	{
		const int32 PingTypeIdx = PingTypeToIdxRuntime(PingType);
		const uint8 StorageIdx = PingAverageStorageIdx[PingTypeIdx];
		const EPingAverageType CurAvgType = PingAverageTypes[PingTypeIdx];
		const int32 bUpdateCount = static_cast<int32>(!EnumHasAnyFlags(PingType, EPingType::ICMP | EPingType::UDPQoS));

		if (CurAvgType == EPingAverageType::None)
		{
			FNoAverageStorage& CurStorage = PingNoAverageStorage[StorageIdx];

			CurStorage.PingValues.AddSample(PingValue);

			CurStorage.PingCount += bUpdateCount;

#if STATS
			UpdatePingStats(PingType, CurStorage.PingValues.GetCurrent());
#endif
		}
		else if (CurAvgType == EPingAverageType::MovingAverage)
		{
			FMovingAverageStorage& CurStorage = PingMovingAverageStorage[StorageIdx];

			CurStorage.PingAccumulator.AddMeasurement(TimeVal, PingValue);

			CurStorage.PingCount += bUpdateCount;

#if STATS
			UpdatePingStats(PingType, CurStorage.PingValues.GetCurrent());
#endif
		}
		else //if (CurAvgType == EPingAverageType::PlayerStateAvg)
		{
			FPlayerStateAvgStorage& CurStorage = PingPlayerStateAvgStorage[StorageIdx];

			CurStorage.UpdatePing(TimeVal, static_cast<float>(PingValue));

			CurStorage.PingCount += bUpdateCount;

#if STATS
			UpdatePingStats(PingType, CurStorage.PingValues.GetCurrent());
#endif
		}
	}
}

void FNetPing::UpdatePingTimeout(EPingType PingType)
{
#if STATS
	SCOPE_CYCLE_COUNTER(STAT_Ping_UpdatePingTime);
#endif

	if (EnumHasAnyFlags(PingTypes, PingType))
	{
		FAverageStorageBase& CurStorageBase = GetPingStorageBase(PingType);

		CurStorageBase.PingTimeouts++;

#if STATS
		const double TimeoutPercent =
			(static_cast<double>(CurStorageBase.PingTimeouts) / static_cast<double>(CurStorageBase.PingCount)) * 100.0;

		UpdateTimeoutStats(PingType, CurStorageBase.PingTimeouts, TimeoutPercent);
#endif

		if (CurStorageBase.PingTimeouts == CurStorageBase.PingCount &&
			CurStorageBase.PingTimeouts >= static_cast<uint32>(CVarNetPingTimeoutDisableThreshold.GetValueOnAnyThread()))
		{
			FString FailureAddress;

			if (PingType == EPingType::ICMP)
			{
				FailureAddress = GetICMPAddr();

				UE_LOGF(LogNet, Warning, "FNetPing: ICMP ping failed with 100%% timeout after '%i' tries. Disabling. "
					"Ping address: %ls", CurStorageBase.PingTimeouts, ToCStr(FailureAddress));

#if STATS
				// -2 for failed pings
				SET_DWORD_STAT(STAT_Ping_ICMP, -2);
#endif

				NetPingICMP.Reset();
			}
			else if (PingType == EPingType::UDPQoS)
			{
				FailureAddress = GetUDPQoSAddr();

				UE_LOGF(LogNet, Warning, "FNetPing: UDPQoS ping failed with 100%% timeout after '%i' tries. Disabling. "
					"Ping address: %ls", CurStorageBase.PingTimeouts, ToCStr(FailureAddress));

#if STATS
				// -2 for failed pings
				SET_DWORD_STAT(STAT_Ping_UDPQoS, -2);
#endif

				NetPingUDPQoS.Reset();
			}

			PingTypes &= ~PingType;

			// Notify the server of the failed ping
			if (FailureAddress.Len() > 0 && Owner)
			{
				const uint32 PingTypeVal = static_cast<uint32>(PingType);
				ENetPingControlMessage MessageType = ENetPingControlMessage::PingFailure;
				FString MessageStr = FString::Printf(TEXT("%i=%s"), PingTypeVal, ToCStr(FailureAddress));

				FNetControlMessage<NMT_NetPing>::Send(Owner.Get(), MessageType, MessageStr);
			}
		}
	}
}

#if STATS
void FNetPing::UpdatePingStats(EPingType PingType, double CurrentValue)
{
	const int32 CurrentValueMS = FMath::TruncToInt(CurrentValue * 1000.0);

	if (PingType == EPingType::RoundTrip)
	{
		SET_DWORD_STAT(STAT_Ping_RoundTripInclFrame, CurrentValueMS);
	}
	else if (PingType == EPingType::RoundTripExclFrame)
	{
		SET_DWORD_STAT(STAT_Ping_RoundTripExclFrame, CurrentValueMS);
	}
	else
	{
		const FAverageStorageBase& CurStorageBase = GetPingStorageBase(PingType);
		const double TimeoutPercent = (CurStorageBase.PingTimeouts == 0 ? 0.0 :
			(static_cast<double>(CurStorageBase.PingTimeouts) / static_cast<double>(CurStorageBase.PingCount)) * 100.0);

		if (PingType == EPingType::ICMP)
		{
			SET_DWORD_STAT(STAT_Ping_ICMP, CurrentValueMS);
			SET_FLOAT_STAT(STAT_PingTimeoutPercent_ICMP, TimeoutPercent);
		}
		else if (PingType == EPingType::UDPQoS)
		{
			SET_DWORD_STAT(STAT_Ping_UDPQoS, CurrentValueMS);
			SET_FLOAT_STAT(STAT_PingTimeoutPercent_UDPQoS, TimeoutPercent);
		}
	}
}

void FNetPing::UpdateTimeoutStats(EPingType PingType, uint32 NumTimeouts, double TimeoutPercent)
{
	if (PingType == EPingType::ICMP)
	{
		SET_DWORD_STAT(STAT_PingTimeouts_ICMP, NumTimeouts);
		SET_FLOAT_STAT(STAT_PingTimeoutPercent_ICMP, TimeoutPercent);
	}
	else if (PingType == EPingType::UDPQoS)
	{
		SET_DWORD_STAT(STAT_PingTimeouts_UDPQoS, NumTimeouts);
		SET_FLOAT_STAT(STAT_PingTimeoutPercent_UDPQoS, TimeoutPercent);
	}
}
#endif

const FSampleMinMaxAvg& FNetPing::GetPingStorageValues(EPingType PingType) const
{
	using namespace UE::Net::Private;

	check(EnumHasAnyFlags(PingTypes, PingType));

	const int32 PingTypeIdx = PingTypeToIdxRuntime(PingType);
	const uint8 StorageIdx = PingAverageStorageIdx[PingTypeIdx];
	const EPingAverageType CurAvgType = PingAverageTypes[PingTypeIdx];

	if (CurAvgType == EPingAverageType::None)
	{
		return PingNoAverageStorage[StorageIdx].PingValues;
	}
	else if (CurAvgType == EPingAverageType::MovingAverage)
	{
		return PingMovingAverageStorage[StorageIdx].PingValues;
	}
	else //if (CurAvgType == EPingAverageType::PlayerStateAvg)
	{
		return PingPlayerStateAvgStorage[StorageIdx].PingValues;
	}
}

FNetPing::FAverageStorageBase& FNetPing::GetPingStorageBase(EPingType PingType)
{
	using namespace UE::Net::Private;

	check(EnumHasAnyFlags(PingTypes, PingType));

	const int32 PingTypeIdx = PingTypeToIdxRuntime(PingType);
	const uint8 StorageIdx = PingAverageStorageIdx[PingTypeIdx];
	const EPingAverageType CurAvgType = PingAverageTypes[PingTypeIdx];

	if (CurAvgType == EPingAverageType::None)
	{
		return PingNoAverageStorage[StorageIdx];
	}
	else if (CurAvgType == EPingAverageType::MovingAverage)
	{
		return PingMovingAverageStorage[StorageIdx];
	}
	else //if (CurAvgType == EPingAverageType::PlayerStateAvg)
	{
		return PingPlayerStateAvgStorage[StorageIdx];
	}
}

FPingValues FNetPing::GetPingValues(EPingType PingType) const
{
	FPingValues ReturnVal;

	if (EnumHasAnyFlags(PingTypes, PingType))
	{
		const FSampleMinMaxAvg& CurStorageValues = GetPingStorageValues(PingType);
		const double CurStorageMin = CurStorageValues.GetMin();
		const double CurStorageMax = CurStorageValues.GetMax();

		if (CurStorageValues.GetSampleCount() > 0)
		{
			ReturnVal.Current = CurStorageValues.GetCurrent();
		}

		if (CurStorageMin != std::numeric_limits<double>::max())
		{
			ReturnVal.Min = CurStorageMin;
		}

		if (CurStorageMax != std::numeric_limits<double>::min())
		{
			ReturnVal.Max = CurStorageMax;
		}
	}

	return ReturnVal;
}

void FNetPing::ResetPingValues(EPingType PingType)
{
	using namespace UE::Net::Private;

	if (EnumHasAnyFlags(PingTypes, PingType))
	{
		const int32 PingTypeIdx = PingTypeToIdxRuntime(PingType);
		const uint8 StorageIdx = PingAverageStorageIdx[PingTypeIdx];
		const EPingAverageType CurAvgType = PingAverageTypes[PingTypeIdx];

		if (CurAvgType == EPingAverageType::None)
		{
			FNoAverageStorage& CurStorage = PingNoAverageStorage[StorageIdx];

			CurStorage.PingCount = 0;
			CurStorage.PingTimeouts = 0;
			CurStorage.PingValues.Reset();
		}
		else if (CurAvgType == EPingAverageType::MovingAverage)
		{
			FMovingAverageStorage& CurStorage = PingMovingAverageStorage[StorageIdx];

			CurStorage.PingCount = 0;
			CurStorage.PingTimeouts = 0;
			CurStorage.PingValues.Reset();
			CurStorage.PingAccumulator.Reset();
		}
		else //if (CurAvgType == EPingAverageType::PlayerStateAvg)
		{
			PingPlayerStateAvgStorage[StorageIdx].Reset();
		}
	}
}

void FNetPing::TickRealtime(double CurTimeSeconds)
{
	if (CachedRemoteAddr.Len() > 0 && CurTimeSeconds > NextRemoteAddrCheckTimestamp)
	{
		UpdateRemoteAddr(CurTimeSeconds);
	}

	if (EnumHasAnyFlags(PingTypes, EPingType::ICMP) && CurTimeSeconds > NextPingTimerICMPTimestamp)
	{
		NextPingTimerICMPTimestamp = CurTimeSeconds + GNetPingICMPInterval;

		PingTimerICMP(CurTimeSeconds);
	}

	if (EnumHasAnyFlags(PingTypes, EPingType::UDPQoS) && CurTimeSeconds > NextPingTimerUDPTimestamp)
	{
		NextPingTimerUDPTimestamp = CurTimeSeconds + GNetPingUDPInterval;

		PingTimerUDP(CurTimeSeconds);
	}

#if !UE_BUILD_SHIPPING
	static double LastDebugPingDump = 0.0;

	if (!!GNetPingDebugDump && (CurTimeSeconds - LastDebugPingDump) > 5.0)
	{
		LastDebugPingDump = CurTimeSeconds;

		UE_LOGF(LogNet, Log, "FNetPing Debug Dump:");

		if (EnumHasAnyFlags(PingTypes, EPingType::RoundTrip))
		{
			FPingValues CurPingValues = GetPingValues(EPingType::RoundTrip);

			UE_LOGF(LogNet, Log, " - RoundTrip: Min: %f, Max: %f, Current: %f", CurPingValues.Min, CurPingValues.Max,
					CurPingValues.Current);
		}

		if (EnumHasAnyFlags(PingTypes, EPingType::RoundTripExclFrame))
		{
			FPingValues CurPingValues = GetPingValues(EPingType::RoundTripExclFrame);

			UE_LOGF(LogNet, Log, " - RoundTripExclFrame: Min: %f, Max: %f, Current: %f", CurPingValues.Min, CurPingValues.Max,
					CurPingValues.Current);
		}

		if (EnumHasAnyFlags(PingTypes, EPingType::ICMP))
		{
			FPingValues CurPingValues = GetPingValues(EPingType::ICMP);
			const FAverageStorageBase& CurStorageBase = GetPingStorageBase(EPingType::ICMP);
			const double TimeoutPercent =
				(static_cast<double>(CurStorageBase.PingTimeouts) / static_cast<double>(CurStorageBase.PingCount)) * 100.0;

			UE_LOGF(LogNet, Log, " - ICMP: Min: %f, Max: %f, Current: %f, Timeouts: %i, Timeout Percent: %f",
					CurPingValues.Min, CurPingValues.Max, CurPingValues.Current, CurStorageBase.PingTimeouts,
					TimeoutPercent);
		}

		if (EnumHasAnyFlags(PingTypes, EPingType::UDPQoS))
		{
			FPingValues CurPingValues = GetPingValues(EPingType::UDPQoS);
			const FAverageStorageBase& CurStorageBase = GetPingStorageBase(EPingType::UDPQoS);
			const double TimeoutPercent =
				(static_cast<double>(CurStorageBase.PingTimeouts) / static_cast<double>(CurStorageBase.PingCount)) * 100.0;

			UE_LOGF(LogNet, Log, " - UDP: Min: %f, Max: %f, Current: %f, Timeouts: %i, Timeout Percent: %f",
					CurPingValues.Min, CurPingValues.Max, CurPingValues.Current, CurStorageBase.PingTimeouts,
					TimeoutPercent);
		}
	}
#endif
}

const TCHAR* FNetPing::GetRemoteAddr()
{
	if (LIKELY(CachedRemoteAddr.Len() > 0))
	{
		return *CachedRemoteAddr;
	}

	UpdateRemoteAddr(FPlatformTime::Seconds());

	return CachedRemoteAddr.Len() > 0 ? *CachedRemoteAddr : TEXT("");
}

FString FNetPing::GetICMPAddr()
{
	return (OverridePingICMPAddress.Len() > 0 ? OverridePingICMPAddress : GetRemoteAddr());
}

FString FNetPing::GetUDPQoSAddr()
{
	TStringBuilder<256> PingIPStr;

	if (OverridePingUDPQoSAddress.Len() > 0)
	{
		PingIPStr.Append(ToCStr(OverridePingUDPQoSAddress));
	}
	else
	{
		PingIPStr.Append(ToCStr(GetRemoteAddr()));
	}


	PingIPStr.AppendChar(TEXT(':'));
	PingIPStr.Appendf(TEXT("%i"), (OverridePingUDPQosPort != INDEX_NONE ? OverridePingUDPQosPort : GNetPingUDPPort));

	return PingIPStr.ToString();
}

void FNetPing::UpdateRemoteAddr(double CurTimeSeconds)
{
	NextRemoteAddrCheckTimestamp = CurTimeSeconds + 1.0;

	if (UNetConnection* ServerConn = (Owner && Owner->Driver ? Owner->Driver->ServerConnection : nullptr))
	{
		TSharedPtr<const FInternetAddr> CurRemoteAddr = ServerConn->GetRemoteAddr();

		if (WeakRemoteAddr != CurRemoteAddr)
		{
			WeakRemoteAddr = CurRemoteAddr;
			CachedRemoteAddr = ServerConn->LowLevelGetRemoteAddress();
		}
	}
}

void FNetPing::PingTimerICMP(double CurTimeSeconds)
{
	if (EnumHasAnyFlags(PingTypes, EPingType::ICMP) && NetPingICMP.IsValid() && Owner && Owner->Driver && Owner->Driver->ServerConnection)
	{
		FAverageStorageBase& CurStorageBase = GetPingStorageBase(EPingType::ICMP);

		CurStorageBase.PingCount++;

		NetPingICMP->Ping(CurTimeSeconds, GetICMPAddr());
	}
}

void FNetPing::PingTimerUDP(double CurTimeSeconds)
{
	if (EnumHasAnyFlags(PingTypes, EPingType::UDPQoS) && NetPingUDPQoS.IsValid() && Owner && Owner->Driver && Owner->Driver->ServerConnection)
	{
		FAverageStorageBase& CurStorageBase = GetPingStorageBase(EPingType::UDPQoS);

		CurStorageBase.PingCount++;

		NetPingUDPQoS->Ping(CurTimeSeconds, GetUDPQoSAddr());
	}
}

void FNetPing::HandleNetPingControlMessage(UNetConnection* Connection, ENetPingControlMessage MessageType, FString MessageStr)
{
	if (Connection)
	{
		if (UNetDriver* Driver = Connection->Driver)
		{
			bool bIsClient = Driver->ServerConnection == Connection;
	
			auto IsAddrValid = [](const FString& InAddr)->bool
			{
				if (InAddr.Len() < 32)
				{
					for (const TCHAR CurChar : InAddr)
					{
						// Valid characters
						if ((CurChar >= TEXT('0') && CurChar <= TEXT('9')) || CurChar == TEXT(':') || CurChar == TEXT('.'))
						{
							continue;
						}
	
						return false;
					}
				}
	
				return true;
			};
	
			if (bIsClient)
			{
				if (MessageType == ENetPingControlMessage::SetPingAddress)
				{
					if (FNetPing* NetPing = Driver->ServerConnection->GetNetPing())
					{
						TArray<FString> Args;

						if (MessageStr.ParseIntoArray(Args, TEXT("=")) == 2 && Args[0].IsNumeric())
						{
							const EPingType PingType = static_cast<EPingType>(FCString::Atoi(*Args[0]));
							FString& PingAddress = Args[1];

							if (IsValidPingType(PingType) && IsAddrValid(PingAddress))
							{
								NetPing->ServerSetPingAddress(PingType, PingAddress, ENetPingTargetEndpoint::DefaultEndpoint);
							}
						}
					}
				}
				else if (MessageType == ENetPingControlMessage::SetAlternatePingAddress)
				{
					if (FNetPing* NetPing = Driver->ServerConnection->GetNetPing())
					{
						TArray<FString> Args;

						if (MessageStr.ParseIntoArray(Args, TEXT("=")) == 2 && Args[0].IsNumeric())
						{
							const EPingType PingType = static_cast<EPingType>(FCString::Atoi(*Args[0]));
							FString& PingAddress = Args[1];

							if (IsValidPingType(PingType) && IsAddrValid(PingAddress))
							{
								NetPing->ServerSetPingAddress(PingType, PingAddress, ENetPingTargetEndpoint::AlternateEndpoint);

								if (NetPing->HasAlternatePingAddress(PingType))
								{
									OnAlternatePingAddressReceivedDelegate.Broadcast(Connection, PingType, PingAddress);
								}
							}
						}
					}
				}
			}
			else
			{
				if (MessageType == ENetPingControlMessage::PingFailure)
				{
					TArray<FString> Args;
	
					if (MessageStr.ParseIntoArray(Args, TEXT("=")) == 2 && Args[0].IsNumeric())
					{
						const EPingType PingType = static_cast<EPingType>(FCString::Atoi(*Args[0]));
						FString& PingAddress = Args[1];
	
						if (IsValidPingType(PingType) && IsAddrValid(PingAddress))
						{
							if (PingType == EPingType::ICMP)
							{
								Connection->AnalyticsVars.AddFailedPingAddressICMP(PingAddress);
							}
							else if (PingType == EPingType::UDPQoS)
							{
								Connection->AnalyticsVars.AddFailedPingAddressUDP(PingAddress);
							}
						}
					}
				}
			}
		}
	}
}

void FNetPing::ServerSetPingAddress(EPingType PingType, FString PingAddress, ENetPingTargetEndpoint TargetEndpoint /*= ENetPingTargetEndpoint::DefaultEndpoint*/)
{
	if (OverridePingAddressCount >= GNetPingMaxOverrideCount)
	{
		UE_LOGF(LogNet, Warning, "ServerSetPingAddress: OverridePingAddressCount %d exceeds GNetPingMaxOverrideCount: %d", OverridePingAddressCount, GNetPingMaxOverrideCount);
		return;
	}

	if (EnumHasAnyFlags(PingTypes, PingType))
	{
		OverridePingAddressCount++;

		if (PingType == EPingType::ICMP)
		{
			if (PingAddress.Find(TEXT(":")) == INDEX_NONE)
			{
				if (TargetEndpoint != ENetPingTargetEndpoint::DefaultEndpoint)
				{
					UE_LOGF(LogNet, Log, "ServerSetPingAddress: EPingType::ICMP PingAddress: %ls", ToCStr(PingAddress));


					OverridePingICMPAddress = PingAddress;

					ResetPingValues(EPingType::ICMP);
				}
				else
				{
					UE_LOGF(LogNet, Warning, "ServerSetPingAddress: EPingType::ICMP does not currently support alternate ping endpoints");
				}
			}
			else
			{
				UE_LOGF(LogNet, Warning, "ServerSetPingAddress: EPingType::ICMP should not specify a port number: %ls",
						ToCStr(PingAddress));
			}
		}
		// For UDPQoS, it's valid to pass a string setting only the port: ":11111"
		else if (PingType == EPingType::UDPQoS)
		{
			if (PingAddress.Len() > 0)
			{
				TArray<FString> IPPort;
				bool bUseAlternateAddress = (TargetEndpoint == ENetPingTargetEndpoint::AlternateEndpoint);

				FString & OverridePingAddress = bUseAlternateAddress ? AlternatePingUDPQoSAddress : OverridePingUDPQoSAddress;
				int32 & OverridePingPort = bUseAlternateAddress ? AlternatePingUDPQoSPort : OverridePingUDPQosPort;

				if (PingAddress.ParseIntoArray(IPPort, TEXT(":")) <= 2)
				{
					if (IPPort[0].Len() > 0)
					{
						OverridePingAddress = IPPort[0];
					}

					if (IPPort.Num() == 2 && IPPort[1].IsNumeric())
					{
						OverridePingPort = FCString::Atoi(ToCStr(IPPort[1]));
					}

					UE_LOGF(LogNet, Log, "ServerSetPingAddress: EPingType::UDPQoS PingAddress: %ls, PingPort: %i",
							ToCStr(OverridePingAddress), OverridePingPort);

					ResetPingValues(EPingType::UDPQoS);
				}
				else
				{
					UE_LOGF(LogNet, Warning, "ServerSetPingAddress: EPingType::UDPQoS PingAddress split into too many components: %ls",
							ToCStr(PingAddress));
				}
			}
		}
	}
}

void FNetPing::UseAlternatePingAddress(EPingType PingType)
{
	if (PingType == EPingType::UDPQoS && !AlternatePingUDPQoSAddress.IsEmpty() &&
		EnumHasAnyFlags(PingTypes, PingType))
	{
		OverridePingUDPQoSAddress = AlternatePingUDPQoSAddress;

		if (AlternatePingUDPQoSPort != INDEX_NONE)
		{
			OverridePingUDPQosPort = AlternatePingUDPQoSPort;
		}

		UE_LOGF(LogNet, Log, "UseAlternatePingAddress: EPingType::UDPQoS PingAddress: %ls, PingPort: %i",
				ToCStr(OverridePingUDPQoSAddress), OverridePingUDPQosPort);

		ResetPingValues(EPingType::UDPQoS);
	}
}

bool FNetPing::HasAlternatePingAddress(EPingType PingType) const
{
	if (PingType == EPingType::UDPQoS)
	{
		return !AlternatePingUDPQoSAddress.IsEmpty();
	}

	return false;
}

namespace Private
{

/**
 * FNetPingQoSBase
 */
FNetPingQoSBase::FNetPingQoSBase(EPingType InPingType, FNetPing* InOwner)
	: PingType(InPingType)
	, Owner(InOwner)
{
}

void FNetPingQoSBase::Release()
{
	Owner = nullptr;
}

void FNetPingQoSBase::Reset()
{
	if (bPingInProgress)
	{
		bDiscardNextPing = true;
	}

	PingTimestamp = 0.0;
}

void FNetPingQoSBase::PingResult(FIcmpEchoResult Result)
{
	bPingInProgress = false;

	if (!bDiscardNextPing && Owner != nullptr)
	{
		if (Result.Status == EIcmpResponseStatus::Success)
		{
			const double CurPing = static_cast<double>(Result.Time);
			const double ApproxTimeSeconds = PingTimestamp + CurPing;

			Owner->UpdatePing(PingType, ApproxTimeSeconds, CurPing);
		}
		else
		{
			// Not always a timeout, but treat all failures as such anyway.
			Owner->UpdatePingTimeout(PingType);
		}
	}

	bDiscardNextPing = false;
}

/**
 * FNetPingICMP
 */

FNetPingICMP::FNetPingICMP(FNetPing* InOwner)
	: FNetPingQoSBase(EPingType::ICMP, InOwner)
{
}

TSharedPtr<FNetPingICMP> FNetPingICMP::GetNetPingICMP(UNetConnection* InNetConn)
{
	static TSharedPtr<FNetPingICMP> NetPingSingleton;

	if (FNetPing* InOwner = InNetConn != nullptr ? InNetConn->GetNetPing() : nullptr)
	{
		FName NetDriverName = (InNetConn->Driver != nullptr ? InNetConn->Driver->NetDriverName : NAME_None);

		// The expected reference count, if no NetDriver has a handle to NetPingSingleton
		const int32 IsUniqueRefCount = 1 + (int32)(NetPingSingleton.IsValid() && NetPingSingleton->bPingInProgress);

		// Only provide an FNetPingICMP interface for GameNetDriver's, when no other NetDriver has a handle to one.
		if ((NetDriverName == NAME_GameNetDriver || NetDriverName == NAME_PendingNetDriver) &&
			(!NetPingSingleton.IsValid() || NetPingSingleton.GetSharedReferenceCount() == IsUniqueRefCount))
		{
			if (!NetPingSingleton.IsValid())
			{
				NetPingSingleton = MakeShared<FNetPingICMP>(InOwner);
			}
			else
			{
				NetPingSingleton->Reset();
			}

			return NetPingSingleton;
		}
	}

	return TSharedPtr<FNetPingICMP>();
}

void FNetPingICMP::Ping(double CurTimeSeconds, FString PingAddress)
{
	if (!bPingInProgress)
	{
		bPingInProgress = true;
		PingTimestamp = CurTimeSeconds;
		TSharedPtr<FNetPingICMP> Self = AsShared();

		FIcmp::IcmpEcho(PingAddress, static_cast<float>(GNetPingICMPInterval),
			[Self = MoveTemp(Self)](FIcmpEchoResult Result)
			{
				if (Self.IsValid())
				{
					Self->PingResult(Result);
				}
			});
	}
}

/**
 * FNetPingUDPQoS
 */

FNetPingUDPQoS::FNetPingUDPQoS(FNetPing* InOwner)
	: FNetPingQoSBase(EPingType::UDPQoS, InOwner)
{
}

TSharedPtr<FNetPingUDPQoS> FNetPingUDPQoS::CreateNetPingUDPQoS(FNetPing* InOwner)
{
	return MakeShared<FNetPingUDPQoS>(InOwner);
}

void FNetPingUDPQoS::Ping(double CurTimeSeconds, FString PingAddress)
{
	if (!bPingInProgress)
	{
		bPingInProgress = true;
		PingTimestamp = CurTimeSeconds;
		TSharedPtr<FNetPingUDPQoS> Self = AsShared();

		FUDPPing::UDPEcho(PingAddress, static_cast<float>(GNetPingUDPInterval),
			[Self = MoveTemp(Self)](FIcmpEchoResult Result)
			{
				if (Self.IsValid())
				{
					Self->PingResult(Result);
				}
			});
	}
}

}

}
