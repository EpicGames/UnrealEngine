// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetPushModelHelpers.h"
#include "Net/Core/PushModel/PushModel.h"
#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"
#include "Net/Core/PushModel/PushModelMacros.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetPushModelHelpers)

void UNetPushModelHelpers::MarkPropertyDirty(UObject* Object, FName PropertyName)
{
#if WITH_PUSH_MODEL
	if (Object && IS_PUSH_MODEL_ENABLED())
	{
		UClass* Class = Object->GetClass();
		if (Class->HasAnyClassFlags(CLASS_ReplicationDataIsSetUp))
		{
			// If this is too slow to be practical, we might be able build a lookup from ClassName+PropertyName -> RepIndex.
			// It would be safest to invalidate the lookup on a hot reload / recompile (if possible), blueprint recompile,
			// and map change.
			FProperty* Property = FindFProperty<FProperty>(Class, PropertyName);
			if (Property == nullptr)
			{
				UE_LOGF(LogNet, Warning, "UNetPushModelHelpers::MarkPropertyDirty: Unable to find Property %ls in Class %ls", *PropertyName.ToString(), *Class->GetPathName());
			}
			else if ((Property->PropertyFlags & CPF_Net) == 0)
			{
				UE_LOGF(LogNet, Warning, "UNetPushModelHelpers::MarkPropertyDirty: Property %ls in class %ls is not Replicated.", *PropertyName.ToString(), *Class->GetPathName());
			}
			else
			{
				MARK_PROPERTY_DIRTY_UNSAFE(Object, Property->RepIndex);
			}
		}
	}
#endif
}

#if WITH_PUSH_MODEL
namespace UEPushModelPrivate
{
#if WITH_PUSH_VALIDATION_SUPPORT
	static bool bCheckPushBPRepIndexAgainstName = false;
	FAutoConsoleVariableRef CVarCheckPushBPRepIndexAgainstName(
		TEXT("Net.CheckPushBPRepIndexAgainstName"),
		bCheckPushBPRepIndexAgainstName,
		TEXT("When enabled, validates that BP generated values passed to MarkPropertyDirtyFromRepIndex match the actual property data")
	);
#else
	static constexpr bool bCheckPushBPRepIndexAgainstName = false;
#endif
}
#endif

void UNetPushModelHelpers::MarkPropertyDirtyFromRepIndex(UObject* Object, int32 RepIndex, FName PropertyName)
{
#if WITH_PUSH_MODEL
	if (Object && IS_PUSH_MODEL_ENABLED())
	{
		UClass* Class = Object->GetClass();
		if (Class->HasAnyClassFlags(CLASS_ReplicationDataIsSetUp))
		{
			if (RepIndex < INDEX_NONE || RepIndex >= Class->ClassReps.Num())
			{
				UE_LOGF(LogNet, Warning, "UNetPushModelHelpers::MarkPropertyDirtyFromRepIndex: Invalid Rep Index. Class %ls RepIndex %d", *Class->GetPathName(), RepIndex);
			}
			else
			{
#if WITH_PUSH_VALIDATION_SUPPORT
				checkf(!UEPushModelPrivate::bCheckPushBPRepIndexAgainstName || Class->ClassReps[RepIndex].Property->GetFName() == PropertyName,
					TEXT("Property and RepIndex don't match! Object=%s, RepIndex=%d, InPropertyName=%s, FoundPropertyName=%s"),
						*Object->GetPathName(), RepIndex, *PropertyName.ToString(), *(Class->ClassReps[RepIndex].Property->GetName()));
#endif
	
				MARK_PROPERTY_DIRTY_UNSAFE(Object, RepIndex);
			}
		}
	}
#endif
}
