// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "Net/Core/Misc/NetSubObjectRegistry.h"

namespace UE::Net
{

/** Helper class to restrict access to the subobject list of actors and actor components */
class FSubObjectRegistryGetter final
{
public:

	FSubObjectRegistryGetter() = delete;
	~FSubObjectRegistryGetter() = delete;

	using FSubObjectRegistry = UE::Net::FSubObjectRegistry;

	static const FSubObjectRegistry& GetSubObjects(const AActor* InActor)
	{
		return InActor->ReplicatedSubObjects;
	}

	static const FSubObjectRegistry* GetSubObjectsOfActorComponent(const AActor* InActor, UActorComponent* InActorComp)
	{
		const UE::Net::FReplicatedComponentInfo* ComponentInfo = InActor->ReplicatedComponentsInfo.FindByKey(InActorComp);
		return ComponentInfo ? &(ComponentInfo->SubObjects) : nullptr;
	}

	static const TArray<UE::Net::FReplicatedComponentInfo>& GetReplicatedComponents(AActor* InActor)
	{
		return InActor->ReplicatedComponentsInfo;
	}

	static const UE::Net::FReplicatedComponentInfo* GetReplicatedComponentInfoForComponent(AActor* InActor, UActorComponent* InActorComp)
	{
		return InActor->ReplicatedComponentsInfo.FindByKey(InActorComp);
	}

	static bool IsSubObjectInRegistry(AActor* InActor, UActorComponent* InActorComp, UObject* InSubObject)
	{
		UE::Net::FReplicatedComponentInfo* ComponentInfo = InActor->ReplicatedComponentsInfo.FindByKey(InActorComp);
		return ComponentInfo ? ComponentInfo->SubObjects.IsSubObjectInRegistry(InSubObject) : false;
	}

	/** Look for the subobject in the subobject list of the actor or any of its replicated components. */
	static bool IsSubObjectInAnyRegistry(AActor* InActor, UObject* InSubObject)
	{
		if (InActor->ReplicatedSubObjects.IsSubObjectInRegistry(InSubObject))
		{
			return true;
		}

		for (const UE::Net::FReplicatedComponentInfo& ComponentInfo : InActor->ReplicatedComponentsInfo)
		{
			if (ComponentInfo.Component == InSubObject || ComponentInfo.SubObjects.IsSubObjectInRegistry(InSubObject))
			{
				return true;
			}
		}

		return false;
	}

	static void InitReplicatedComponentsList(AActor* InActor)
	{
		InActor->BuildReplicatedComponentsInfo();
	}

	static uint32 CountReplicatedSubObjectsOfActor(AActor* InActor)
	{
		// Start with the actor's subobjects;
		uint32 NumReplicatedObjects = (uint32)InActor->ReplicatedSubObjects.Num();
		// Add the replicated components
		NumReplicatedObjects += (uint32)InActor->ReplicatedComponentsInfo.Num();

		// Finally add the subobjects of the replicated components	
		for (const UE::Net::FReplicatedComponentInfo& ReplicatedComponentInfo : InActor->ReplicatedComponentsInfo)
		{
			NumReplicatedObjects += (uint32)ReplicatedComponentInfo.SubObjects.Num();
		}

		return NumReplicatedObjects;
	}
};

}
