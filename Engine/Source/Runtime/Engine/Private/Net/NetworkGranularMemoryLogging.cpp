// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetworkGranularMemoryLogging.h"

#if UE_WITH_NETWORK_GRANULAR_MEM_TRACKING

#include "Serialization/ArchiveCountMem.h"
#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"
#include "Containers/Ticker.h"
#include "Stats/Stats.h"
#include "Containers/Ticker.h"

namespace UE::Net::Private::GranularMemoryTracking
{
	static TAutoConsoleVariable<int32> CVarUseGranularNetworkTracking(
		TEXT("Net.UseGranularNetworkTracking"),
		0,
		TEXT("When enabled, Obj List will print out highly detailed information about Network Memory Usage")
	);

	struct FNetworkMemoryTrackingScope
	{
		FNetworkMemoryTrackingScope(const FString& ScopeName)
			: ScopeName(ScopeName)
		{
		}

		void GenerateRows(const FString& Prefix, TArray<FString>& OutRows)
		{
			const FString NewPrefix = (Prefix.IsEmpty()) ? ScopeName : (Prefix + TEXT("::") + ScopeName);

			SubScopes.KeySort(TLess<FString>());
			for (auto It = SubScopes.CreateIterator(); It; ++It)
			{
				It.Value()->GenerateRows(NewPrefix, OutRows);
			}

			struct FField
			{
				const FString* Name;
				uint64 Bytes;
			};

			auto FieldLessThan = [](const FField& LHS, const FField& RHS)
			{
				return *(LHS.Name) < *(RHS.Name);
			};

			TArray<FField> FieldValues;
			FieldValues.Reserve(Fields.Num() + 2);
			FieldValues.Add({ &NewPrefix, TotalBytes });

			static const FString CountName = "Count";
			FieldValues.Add({ &CountName, Count });

			for (auto ConstIt = Fields.CreateConstIterator(); ConstIt; ++ConstIt)
			{
				FieldValues.Add({ &ConstIt.Key(), ConstIt.Value() });
			}

			// Keep our total scope data in place, but sort the rest of the fields.
			Algo::Sort(MakeArrayView(FieldValues.GetData() + 2, FieldValues.Num() - 2), FieldLessThan);

			FString ReportRow = FString::Printf(TEXT("%s\r\n%s"),
				*FString::JoinBy(FieldValues, TEXT(","), [](const FField& Field) { return *Field.Name; }),
				*FString::JoinBy(FieldValues, TEXT(","), [](const FField& Field) { return FString::Printf(TEXT("%llu"), Field.Bytes); }));

			OutRows.Emplace(ReportRow);
		}

		FNetworkMemoryTrackingScope* FindOrAddScope(const FString& NewScopeName)
		{
			TUniquePtr<FNetworkMemoryTrackingScope>& FoundScope = SubScopes.FindOrAdd(NewScopeName);
			if (!FoundScope.IsValid())
			{
				FoundScope = MakeUnique<FNetworkMemoryTrackingScope>(NewScopeName);
			}

			return FoundScope.Get();
		}

		void AddBytesToField(const FString& FieldName, const uint64 Bytes)
		{
			Fields.FindOrAdd(FieldName) += Bytes;
			TotalBytes += Bytes;
		}

		uint64 GetTotalBytes() const
		{
			return TotalBytes;
		}

		void IncrementCount()
		{
			++Count;
		}

		const FString ScopeName;

	private:

		TMap<FString, TUniquePtr<FNetworkMemoryTrackingScope>> SubScopes;
		TMap<FString, uint64> Fields;
		uint32 Count = 0;
		uint64 TotalBytes = 0u;
	};

	struct FNetworkMemoryTrackingScopeStack
	{
		static TUniquePtr<FNetworkMemoryTrackingScopeStack>& Get()
		{
			static TUniquePtr<FNetworkMemoryTrackingScopeStack> Stack;

			if (!Stack.IsValid())
			{
				Stack.Reset(new FNetworkMemoryTrackingScopeStack());
			}

			return Stack;
		}

		void PushScope(const FString& ScopeName)
		{
			if (CurrentScope)
			{
				CurrentScope = CurrentScope->FindOrAddScope(ScopeName);
			}
			else if (TUniquePtr<FNetworkMemoryTrackingScope>* ExistingScope = TopLevelScopes.Find(ScopeName))
			{
				CurrentScope = ExistingScope->Get();
			}
			else
			{
				CurrentScope = TopLevelScopes.Add(ScopeName, MakeUnique<FNetworkMemoryTrackingScope>(ScopeName)).Get();
			}

			CurrentScope->IncrementCount();

			ScopeStack.Push(CurrentScope);
		}

		void PopScope()
		{
			ScopeStack.Pop();
			CurrentScope = (ScopeStack.Num() > 0) ? ScopeStack.Top() : nullptr;
		}

		void TrackWork(const FString& WorkName, const uint64 WorkBytes)
		{
			static const FString UnknownScopeName("UNKNOWN");

			auto GetCorrectScope = [this]() -> FNetworkMemoryTrackingScope &
			{
				if (CurrentScope)
				{
					return *CurrentScope;
				}

				if (TUniquePtr<FNetworkMemoryTrackingScope> * UnknownScope = TopLevelScopes.Find(UnknownScopeName))
				{
					return **UnknownScope;
				}

				return *TopLevelScopes.Emplace(UnknownScopeName, MakeUnique<FNetworkMemoryTrackingScope>(UnknownScopeName));
			};

			GetCorrectScope().AddBytesToField(WorkName, WorkBytes);
		}

	private:

		FNetworkMemoryTrackingScopeStack() :
			TickHandle(FTSTicker::GetCoreTicker().AddTicker(TEXT("NetworkGranularMemoryLogging::FNetworkMemoryTrackingScopeStack"), 0, &FNetworkMemoryTrackingScopeStack::OnTick))
		{
		}

		static bool OnTick(float DeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FNetworkMemoryTrackingScopeStack_OnTick);

			TUniquePtr<FNetworkMemoryTrackingScopeStack>& ScopeStack = Get();
			ScopeStack->TopLevelScopes.KeySort(TLess<FString>());

			const FString EmptyPrefix;
			TArray<FString> Rows;
			TArray<FString> TopLevelKBRows;

			
			for (auto It = ScopeStack->TopLevelScopes.CreateIterator(); It; ++It)
			{
				It.Value()->GenerateRows(EmptyPrefix, Rows);
				TopLevelKBRows.Add(FString::Printf(TEXT("%s KB\r\n%llu"), *It.Key(), FMath::DivideAndRoundUp(It.Value()->GetTotalBytes(), (uint64)1024)));
			}

			Rows.Append(TopLevelKBRows);

			// TODO: Replace \r\n with some platform specific newline macro.
			UE_LOGF(LogNet, Warning, "\r\n%ls", *FString::Join(Rows, TEXT("\r\n\r\n")));

			FTSTicker::RemoveTicker(ScopeStack->TickHandle);
			ScopeStack.Reset();
			return true;
		}

		FTSTicker::FDelegateHandle TickHandle;
		TMap<FString, TUniquePtr<FNetworkMemoryTrackingScope>> TopLevelScopes;
		FNetworkMemoryTrackingScope* CurrentScope = nullptr;
		TArray<FNetworkMemoryTrackingScope*> ScopeStack;
	};

	static const bool ShouldTrackMemory(const FArchive& Ar)
	{
		return CVarUseGranularNetworkTracking.GetValueOnAnyThread() && Ar.IsCountingMemory() && FString("FArchiveCountMem").Equals(Ar.GetArchiveName());
	}

	FScopeMarker::FScopeMarker(FArchive& InAr, FString&& InScopeName) :
		Ar(InAr),
		ScopeName(InScopeName),
		ScopeStack(ShouldTrackMemory(Ar) ? FNetworkMemoryTrackingScopeStack::Get().Get() : nullptr)
	{
		if (ScopeStack)
		{
			ScopeStack->PushScope(ScopeName);
		}
	}

	FScopeMarker::~FScopeMarker()
	{
		if (ScopeStack)
		{
			ScopeStack->PopScope();
		}
	}

	void FScopeMarker::BeginWork()
	{
		if (ScopeStack)
		{
			PreWorkPos = ((FArchiveCountMem&)Ar).GetMax();
		}
	}

	void FScopeMarker::EndWork(const FString& WorkName)
	{
		if (ScopeStack)
		{
			LogCustomWork(WorkName, ((FArchiveCountMem&)Ar).GetMax() - PreWorkPos);
		}
	}

	void FScopeMarker::LogCustomWork(const FString& WorkName, const uint64 Bytes) const
	{
		if (ScopeStack)
		{
			// UE_LOGF(LogNet, Log, "%ls: %ls is %llu bytes", *ScopeName, *WorkName, Bytes);
			ScopeStack->TrackWork(WorkName, Bytes);
		}
	}
};

#endif // UE_WITH_NETWORK_GRANULAR_MEM_TRACKING
