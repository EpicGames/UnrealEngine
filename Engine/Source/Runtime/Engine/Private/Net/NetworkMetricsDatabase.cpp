// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetworkMetricsDatabase.h"
#include "Net/NetworkMetricsMutators.h"
#include "EngineStats.h"
#include "Iris/IrisConfig.h"
#include "Logging/LogMacros.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "StructUtils/InstancedStruct.h"

#if USE_SERVER_PERF_COUNTERS
#include "PerfCountersModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkMetricsDatabase)

DEFINE_LOG_CATEGORY_STATIC(LogNetworkMetrics, Error, All);

void UNetworkMetricsDatabase::CreateFloat(const FName MetricName, float DefaultValue)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	if (ensureMsgf(!MetricTypes.Contains(MetricName), TEXT("Metric %s already exists in the database."), *MetricName.ToString()))
	{
		UE::Net::FNetworkMetric<float> Metric;
		Metric.Name = MetricName;
		Metric.Value = DefaultValue;
		MetricFloats.Add(MetricName, Metric);
		MetricTypes.Add(MetricName, EMetricType::Float);
	}
}

void UNetworkMetricsDatabase::CreateInt(const FName MetricName, int64 DefaultValue)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	if (ensureMsgf(!MetricTypes.Contains(MetricName), TEXT("Metric %s already exists in the database."), *MetricName.ToString()))
	{
		UE::Net::FNetworkMetric<int64> Metric;
		Metric.Name = MetricName;
		Metric.Value = DefaultValue;
		MetricInts.Add(MetricName, Metric);
		MetricTypes.Add(MetricName, EMetricType::Integer);
	}
}

bool UNetworkMetricsDatabase::SetFloat(const FName MetricName, float Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<float>* Metric = MetricFloats.Find(MetricName);
	if (Metric)
	{
		Metric->Value = Value;
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::SetFloat Cannot find float metric %ls.", *MetricName.ToString());
		return false;
	}
}

bool UNetworkMetricsDatabase::SetMinFloat(const FName MetricName, float Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<float>* Metric = MetricFloats.Find(MetricName);
	if (Metric)
	{
		Metric->Value = FMath::Min<float>(Value, Metric->Value);
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::SetMinFloat Cannot find float metric %ls.", *MetricName.ToString());
		return false;
	}
}

bool UNetworkMetricsDatabase::SetMaxFloat(const FName MetricName, float Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<float>* Metric = MetricFloats.Find(MetricName);
	if (Metric)
	{
		Metric->Value = FMath::Max<float>(Value, Metric->Value);
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::SetMaxFloat Cannot find float metric %ls.", *MetricName.ToString());
		return false;
	}
}

float UNetworkMetricsDatabase::GetFloat(const FName MetricName)
{
	UE_IRIS_PARALLEL_EXPR(UE::TReadScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<float>* Metric = MetricFloats.Find(MetricName);
	if (Metric)
	{
		return Metric->Value;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::GetFloat Cannot find float metric %ls.", *MetricName.ToString());
		return 0.0f;
	}
}

bool UNetworkMetricsDatabase::SetInt(const FName MetricName, int64 Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<int64>* Metric = MetricInts.Find(MetricName);
	if (Metric)
	{
		Metric->Value = Value;
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::SetInt Cannot find integer metric %ls.", *MetricName.ToString());
		return false;
	}
}

bool UNetworkMetricsDatabase::SetMinInt(const FName MetricName, int64 Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<int64>* Metric = MetricInts.Find(MetricName);
	if (Metric)
	{
		Metric->Value = FMath::Min<int64>(Value, Metric->Value);
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::SetMinInt Cannot find integer metric %ls.", *MetricName.ToString());
		return false;
	}
}

bool UNetworkMetricsDatabase::SetMaxInt(const FName MetricName, int64 Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<int64>* Metric = MetricInts.Find(MetricName);
	if (Metric)
	{
		Metric->Value = FMath::Max<int64>(Value, Metric->Value);
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::SetMaxInt Cannot find integer metric %ls.", *MetricName.ToString());
		return false;
	}
}

bool UNetworkMetricsDatabase::IncrementInt(const FName MetricName, int64 Value)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<int64>* Metric = MetricInts.Find(MetricName);
	if (Metric)
	{
		Metric->Value += Value;
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::IncrementInt Cannot find integer metric %ls.", *MetricName.ToString());
		return false;
	}
}

int64 UNetworkMetricsDatabase::GetInt(const FName MetricName)
{
	UE_IRIS_PARALLEL_EXPR(UE::TReadScopeLock LockGuard(NetworkMetricLock));
	UE::Net::FNetworkMetric<int64>* Metric = MetricInts.Find(MetricName);
	if (Metric)
	{
		return Metric->Value;
	}
	else
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::GetInt Cannot find integer metric %ls.", *MetricName.ToString());
		return 0;
	}
}

bool UNetworkMetricsDatabase::Contains(const FName MetricName) const
{
	UE_IRIS_PARALLEL_EXPR(UE::TReadScopeLock LockGuard(NetworkMetricLock));
	return MetricTypes.Contains(MetricName);
}

void UNetworkMetricsDatabase::AddMetricToSnapshot(UE::Net::FNetworkMetricSnapshot& Snapshot, const FName MetricName, const EMetricType MetricType)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	if (MetricType == EMetricType::Integer)
	{
		const UE::Net::FNetworkMetric<int64>* Metric = MetricInts.Find(MetricName);
		if (ensureMsgf(Metric, TEXT("Unable to find metric %s when reporting to listeners."), *MetricName.ToString()))
		{
			Snapshot.MetricInts.Add(*Metric);
		}
	}
	else if (MetricType == EMetricType::Float)
	{
		const UE::Net::FNetworkMetric<float>* Metric = MetricFloats.Find(MetricName);
		if (ensureMsgf(Metric, TEXT("Unable to find metric %s when reporting to listeners."), *MetricName.ToString()))
		{
			Snapshot.MetricFloats.Add(*Metric);
		}
	}
}

void UNetworkMetricsDatabase::ProcessListeners()
{
	double CurrentTimeSeconds = FPlatformTime::Seconds();

	UE::Net::FNetworkMetricSnapshot Snapshot;
	Snapshot.MetricFloats.Reserve(MetricFloats.Num());
	Snapshot.MetricInts.Reserve(MetricInts.Num());

	for (const TPair<TWeakObjectPtr<UNetworkMetricsBaseListener>, TSet<FNameAndType>>& Elem: ListenersToMetrics)
	{	
		UNetworkMetricsBaseListener* Listener = Elem.Key.Get();
		const TSet<FNameAndType>& ListenerMetrics = Elem.Value;

		if (ensure(Listener))
		{
			// For mutator listeners generate & process snapshots every frame. Do this before calling Report() on listeners
			// so that they can take this frame's data into account.
			for (TInstancedStruct<FNetworkMetricsMutator>& Mutator : Listener->GetMutators())
			{
				if (ensure(Mutator.IsValid()))
				{
					Snapshot.Reset();

					for (const FName MetricName : Mutator.Get().GetAllMetricNames())
					{
						EMetricType* MetricType = MetricTypes.Find(MetricName);
						if (MetricType)
						{
							AddMetricToSnapshot(Snapshot, MetricName, *MetricType);
						}
					}

					Mutator.GetMutable().ProcessFrame(Snapshot);
				}
			}

			if (!HasReportIntervalPassed(CurrentTimeSeconds, Listener))
			{
				continue;
			}

			Snapshot.Reset();

			for (const FNameAndType& NameAndType : ListenerMetrics)
			{
				const FName& MetricName = NameAndType.Key;
				const EMetricType& MetricType = NameAndType.Value;

				AddMetricToSnapshot(Snapshot, MetricName, MetricType);
			}

			// Add mutator values to the snapshot
			for (TInstancedStruct<FNetworkMetricsMutator>& Mutator : Listener->GetMutators())
			{
				if (ensure(Mutator.IsValid()))
				{
					FNetworkMetricsMutator::FValueVariant MutatorVariant = Mutator.Get().GetValue();
					if (MutatorVariant.IsType<int64>())
					{
						UE::Net::FNetworkMetric<int64> MutatorInt;
						MutatorInt.Name = Mutator.Get().GetName();
						MutatorInt.Value = MutatorVariant.Get<int64>();
						Snapshot.MetricInts.Add(MutatorInt);
					}
					else if (MutatorVariant.IsType<float>())
					{
						UE::Net::FNetworkMetric<float> MutatorFloat;
						MutatorFloat.Name = Mutator.Get().GetName();
						MutatorFloat.Value = MutatorVariant.Get<float>();
						Snapshot.MetricFloats.Add(MutatorFloat);
					}
					else
					{
						ensureMsgf(false, TEXT("Mutator %s returned value of unknown type."), *Mutator.Get().GetName().ToString());
					}
				}
			}

			Listener->Report(Snapshot);
		}
	}
}

void UNetworkMetricsDatabase::Reset()
{
	{
		UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
		MetricInts.Reset();
		MetricFloats.Reset();
		MetricTypes.Reset();
	}
	
	ResetListeners();
}

void UNetworkMetricsDatabase::ResetListeners()
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	LastReportListener.Reset();
	ListenersToMetrics.Reset();
}

void UNetworkMetricsDatabase::Register(const FName MetricName, TWeakObjectPtr<UNetworkMetricsBaseListener> Reporter)
{
	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	if (!ensure(Reporter.IsValid()))
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::Register: Cannot register metric %ls with invalid listener.", *MetricName.ToString());
		return;
	}

	const EMetricType* MetricType = MetricTypes.Find(MetricName);

	if (ensureMsgf(MetricType, TEXT("Cannot find metric %s to register listener."), *MetricName.ToString()))
	{
		// Make sure the metric name doesn't collide with a mutator name that's already registered to the listener.
		for (TInstancedStruct<FNetworkMetricsMutator>& Mutator : Reporter->GetMutators())
		{
			if (ensure(Mutator.IsValid()))
			{
				if (Mutator.Get().GetName() == MetricName)
				{
					UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::Register: Cannot register metric %ls whose name collides with a registered mutator for listener %ls.", *MetricName.ToString(), *Reporter->GetClass()->GetName());
					ensureMsgf(false, TEXT("UNetworkMetricsDatabase::Register: Cannot register metric %s whose name collides with a registered mutator for listener %s."), *MetricName.ToString(), *Reporter->GetClass()->GetName());
					return;
				}
			}
		}

		TSet<FNameAndType>& ListenerMetrics = ListenersToMetrics.FindOrAdd(Reporter);
		ListenerMetrics.Add(FNameAndType(MetricName, *MetricType));
		LastReportListener.Add(Reporter, 0.0);
	}
}

void UNetworkMetricsDatabase::RegisterMutator(TWeakObjectPtr<UNetworkMetricsBaseListener> Listener, const TInstancedStruct<FNetworkMetricsMutator>& Mutator)
{
	if (!ensure(Listener.IsValid()))
	{
		UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::RegisterMutator: Cannot register mutator %ls with invalid listener.", *Mutator.Get().GetName().ToString());
		return;
	}

	for (const FName MutatorMetricName : Mutator.Get().GetAllMetricNames())
	{
		if (!Contains(MutatorMetricName))
		{
			UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::RegisterMutator: Cannot register mutator %ls with an unknown metric '%ls'.", *Mutator.Get().GetName().ToString(), *MutatorMetricName.ToString());
			return;
		}
	}

	// Make sure the mutator name doesn't collide with a mutator name that's already registered to the listener.
	for (TInstancedStruct<FNetworkMetricsMutator>& ExistingMutator : Listener->GetMutators())
	{
		if (ensure(ExistingMutator.IsValid()))
		{
			if (ExistingMutator.Get().GetName() == Mutator.Get().GetName())
			{
				UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::RegisterMutator: Cannot register mutator %ls whose name collides with an existing mutator for listener %ls.", *Mutator.Get().GetName().ToString(), *Listener->GetClass()->GetName());
				ensureMsgf(false, TEXT("UNetworkMetricsDatabase::RegisterMutator: Cannot register mutator %s whose name collides with an existing mutator for listener %s."), *Mutator.Get().GetName().ToString(), *Listener->GetClass()->GetName());
				return;
			}
		}
	}

	UE_IRIS_PARALLEL_EXPR(UE::TWriteScopeLock LockGuard(NetworkMetricLock));
	// Make sure the mutator name doesn't collide with a metric name that's already registered to the listener.
	TSet<FNameAndType>* ListenerMetrics = ListenersToMetrics.Find(Listener);
	if (ListenerMetrics)
	{
		for (const FNameAndType& NameAndType : *ListenerMetrics)
		{
			if (Mutator.Get().GetName() == NameAndType.Key)
			{
				UE_LOGF(LogNetworkMetrics, Warning, "UNetworkMetricsDatabase::RegisterMutator: Cannot register mutator %ls whose name collides with a registered metric for listener %ls.", *Mutator.Get().GetName().ToString(), *Listener->GetClass()->GetName());
				ensureMsgf(false, TEXT("UNetworkMetricsDatabase::RegisterMutator: Cannot register mutator %s whose name collides with a registered metric for listener %s."), *Mutator.Get().GetName().ToString(), *Listener->GetClass()->GetName());
				return;
			}
		}
	}
	else
	{
		// Add entries to the listener maps so this listener is processed even if it doesn't have any metrics directly registered.
		ListenerMetrics = &ListenersToMetrics.Add(Listener);
		LastReportListener.Add(Listener, 0.0);
	}

	Listener->GetMutators().Add(Mutator);
}

bool UNetworkMetricsDatabase::HasReportIntervalPassed(double CurrentTimeSeconds, UNetworkMetricsBaseListener* Listener)
{
	if (ensureMsgf(Listener->GetInterval() >= 0, TEXT("Listener has a negative reporting time interval.")))
	{
		double* LastReportSeconds = LastReportListener.Find(Listener);

		if (ensure(LastReportSeconds))
		{
			double DurationSeconds = CurrentTimeSeconds - *LastReportSeconds;

			if (DurationSeconds >= Listener->GetInterval())
			{
				*LastReportSeconds = CurrentTimeSeconds;
				return true;
			}
		}
	}

	return false;
}

UNetworkMetricsBaseListener::UNetworkMetricsBaseListener() :
	UObject(),
	IntervalSeconds(0)
{

}

UNetworkMetricsBaseListener::~UNetworkMetricsBaseListener() = default;

void UNetworkMetricsLog::Report(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
	FString MetricsStr;

	for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
	{
		MetricsStr += FString::Printf(TEXT("%s=%lld "), *Metric.Name.ToString(), Metric.Value);
	}

	for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
	{
		MetricsStr += FString::Printf(TEXT("%s=%2.2f "), *Metric.Name.ToString(), Metric.Value);
	}

	UE_CLOGF(!MetricsStr.IsEmpty(), LogNetworkMetrics, Log, "Network Metrics: %ls", *MetricsStr);

	for (TInstancedStruct<FNetworkMetricsMutator>& Mutator : GetMutators())
	{
		Mutator.GetMutable().Reset();
	}
}

UNetworkMetricsCSV::UNetworkMetricsCSV() :
	CategoryIndex(-1)
{
	// The default CSV category for networking metrics.
	SetCategory("Networking");
}

void UNetworkMetricsCSV::SetCategory(const FString& CategoryName)
{
#if CSV_PROFILER_STATS
	int32 Index = FCsvProfiler::GetCategoryIndex(CategoryName);
	if (ensureMsgf(Index != 1, TEXT("Unable to find CSV category %s"), *CategoryName))
	{
		CategoryIndex = Index;
	}
#endif
}

void UNetworkMetricsCSV::Report(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
#if CSV_PROFILER_STATS
	if (ensureMsgf(CategoryIndex != -1, TEXT("SetCategory() must be called before being registered as a listener.")))
	{
		for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
		{
			if (ensureMsgf(IntFitsIn<int32>(Metric.Value), TEXT("Integer metric %s truncated when reporting to CSV."), *Metric.Name.ToString()))
			{
				FCsvProfiler::RecordCustomStat(Metric.Name, CategoryIndex, static_cast<int32>(Metric.Value), ECsvCustomStatOp::Set);
			}
		}

		for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
		{
			FCsvProfiler::RecordCustomStat(Metric.Name, CategoryIndex, Metric.Value, ECsvCustomStatOp::Set);
		}
	}
#endif
}

void UNetworkMetricsPerfCounters::Report(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
#if USE_SERVER_PERF_COUNTERS
	IPerfCounters* PerfCounters = IPerfCountersModule::Get().GetPerformanceCounters();

	if (PerfCounters)
	{
		for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
		{
			if (ensureMsgf(IntFitsIn<int32>(Metric.Value), TEXT("Integer metric %s truncated when reporting to PerfCounters."), *Metric.Name.ToString()))
			{
				PerfCounters->Set(Metric.Name.ToString(), static_cast<uint32>(Metric.Value));
			}
		}

		for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
		{
			PerfCounters->Set(Metric.Name.ToString(), Metric.Value);
		}
	}
#endif
}

UNetworkMetricsStats::UNetworkMetricsStats()
{

}

void UNetworkMetricsStats::Report(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
#if STATS
	if (!FThreadStats::IsCollectingData())
	{
		return;
	}

	const int32 TotalMetrics = Snapshot.MetricFloats.Num() + Snapshot.MetricInts.Num();

	// An instance of UNetworkMetricStats is bound to a specific Stat value defined with the DEFINE_STAT macro 
	// so there should only be one metric provided to this function.
	if (ensureMsgf(TotalMetrics <= 1, TEXT("UNetworkMetricsStats should only be registered to listen to one metric.")))
	{
		if (LIKELY(TotalMetrics == 1))
		{
			for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
			{
				SET_DWORD_STAT_FName(StatName, Metric.Value);
			}

			for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
			{
				SET_FLOAT_STAT_FName(StatName, Metric.Value);
			}
		}
	}
#endif
}
