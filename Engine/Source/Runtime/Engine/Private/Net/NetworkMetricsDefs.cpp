// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetworkMetricsDefs.h"

namespace UE::Net::Metric
{

const FName AddedConnections("AddedConnections");
const FName ClosedConnectionsDueToReliableBufferOverflow("ClosedConnectionsDueToReliableBufferOverflow");
const FName InPackets("InPackets");
const FName OutPackets("OutPackets");
const FName InPacketsPerFrame("InPacketsPerFrame");
const FName OutPacketsPerFrame("OutPacketsPerFrame");
const FName InLostPacketsFoundPerFrame("InLostPacketsFoundPerFrame");
const FName OutLostPacketsFoundPerFrame("OutLostPacketsFoundPerFrame");
const FName InPacketsClientPerSecondAvg("InPacketsClientPerSecondAvg");
const FName InPacketsClientPerSecondMax("InPacketsClientPerSecondMax");
const FName InPacketsClientPerSecondMin("InPacketsClientPerSecondMin");
const FName OutPacketsClientPerSecondAvg("OutPacketsClientPerSecondAvg");
const FName OutPacketsClientPerSecondMax("OutPacketsClientPerSecondMax");
const FName OutPacketsClientPerSecondMin("OutPacketsClientPerSecondMin");
const FName InPacketsClientAvg("InPacketsClientAvg");
const FName InPacketsClientMax("InPacketsClientMax");
const FName OutPacketsClientAvg("OutPacketsClientAvg");
const FName OutPacketsClientMax("OutPacketsClientMax");
const FName InRate("InRate");
const FName OutRate("OutRate");
const FName InRateClientAvg("InRateClientAvg");
const FName InRateClientMax("InRateClientMax");
const FName InRateClientMin("InRateClientMin");
const FName OutRateClientAvg("OutRateClientAvg");
const FName OutRateClientMax("OutRateClientMax");
const FName OutRateClientMin("OutRateClientMin");
const FName InPacketsLost("InPacketsLost");
const FName OutPacketsLost("OutPacketsLost");
const FName AverageJitterInMS("AverageJitterInMS");
const FName InBunches("InBunches");
const FName OutBunches("OutBunches");
const FName Ping("Ping");
const FName AvgPing("AvgPing");
const FName MinPing("MinPing");
const FName MaxPing("MaxPing");
const FName OutKBytes("OutKBytes");
const FName OutNetGUIDKBytesSec("OutNetGUIDKBytesSec");
const FName ServerReplicateActorTimeMS("ServerReplicateActorTimeMS");
const FName GatherPrioritizeTimeMS("GatherPrioritizeTimeMS");
const FName ReplicateActorTimeMS("ReplicateActorTimeMS");
const FName NumReplicatedActors("NumReplicatedActors");
const FName Connections("Connections");
const FName NumReplicateActorCallsPerConAvg("NumReplicateActorCallsPerConAvg");
const FName NumberOfActiveActors("NumberOfActiveActors");
const FName NumberOfFullyDormantActors("NumberOfFullyDormantActors");
const FName NumSkippedObjectEmptyUpdates("NumSkippedObjectEmptyUpdates");
const FName NumOpenChannels("NumOpenChannels");
const FName NumTickingChannels("NumTickingChannels");
const FName SatConnections("SatConnections");
const FName SharedSerializationPropertyHit("SharedSerializationPropertyHit");
const FName SharedSerializationPropertyMiss("SharedSerializationPropertyMiss");
const FName SharedSerializationRPCHit("SharedSerializationRPCHit");
const FName SharedSerializationRPCMiss("SharedSerializationRPCMiss");
const FName NumClientUpdateLevelVisibility("NumClientUpdateLevelVisibility");
const FName Channels("Channels");
const FName NumActorChannels("NumActorChannels");
const FName NumDormantActors("NumDormantActors");
const FName NumActors("NumActors");
const FName NumNetActors("NumNetActors");
const FName NumNetGUIDsAckd("NumNetGUIDsAckd");
const FName NumNetGUIDsPending("NumNetGUIDsPending");
const FName NumNetGUIDsUnAckd("NumNetGUIDsUnAckd");
const FName NetSaturated("NetSaturated");
const FName MaxPacketOverhead("MaxPacketOverhead");
const FName NetGUIDInRate("NetGUIDInRate");
const FName NetGUIDOutRate("NetGUIDOutRate");
const FName NetNumClients("NetNumClients");
const FName NumClients("NumClients");
const FName NumConnections("NumConnections");
const FName NumAllConnections("NumAllConnections");
const FName NumConsideredActors("NumConsideredActors");
const FName NumInitiallyDormantActors("NumInitiallyDormantActors");
const FName PrioritizedActors("PrioritizedActors");
const FName NumRelevantDeletedActors("NumRelevantDeletedActors");
const FName NumReplicatedActorBytes("NumReplicatedActorBytes");
const FName ImportedNetGuids("ImportedNetGuids");
const FName PendingOuterNetGuids("PendingOuterNetGuids");
const FName NetInBunchTimeOvershootPercent("NetInBunchTimeOvershootPercent");
const FName UnmappedReplicators("UnmappedReplicators");
const FName PercentInVoice("PercentInVoice");
const FName PercentOutVoice("PercentOutVoice");
const FName VoiceBytesRecv("VoiceBytesRecv");
const FName VoiceBytesSent("VoiceBytesSent");
const FName VoicePacketsRecv("VoicePacketsRecv");
const FName VoicePacketsSent("VoicePacketsSent");
const FName PingBucketInt0("PingBucketInt0");
const FName PingBucketInt1("PingBucketInt1");
const FName PingBucketInt2("PingBucketInt2");
const FName PingBucketInt3("PingBucketInt3");
const FName PingBucketInt4("PingBucketInt4");
const FName PingBucketInt5("PingBucketInt5");
const FName PingBucketInt6("PingBucketInt6");
const FName PingBucketInt7("PingBucketInt7");
const FName OutgoingReliableMessageQueueMaxSize("OutgoingReliableMessageQueueMaxSize");
const FName IncomingReliableMessageQueueMaxSize("IncomingReliableMessageQueueMaxSize");
const FName RawPing("RawPing");
const FName RawPingClientMin("RawPingClientMin");
const FName RawPingClientMax("RawPingClientMax");
const FName RawPingClientAvg("RawPingClientAvg");
const FName NumPlayerControllersForReplication("NumPlayerControllersForReplication");
const FName NumNoPawnPlayerControllersForReplication("NumNoPawnPlayerControllersForReplication");
const FName NumPawnsForReplication("NumPawnsForReplication");
const FName NumReplicationViewers("NumReplicationViewers");

}

