// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/NetworkMetricsMutators.h"
#include "Net/NetworkMetricsDatabase.h"
#include "Net/NetworkMetricsDefs.h"
#include "EngineStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkMetricsMutators)

FName FNetworkMetricsMutator::GetName() const
{
	if (!NameOverride.IsNone())
	{
		return NameOverride;
	}

	return GetNameInternal();
}

FName FNetworkMetricsMutatorAvg::GetNameInternal() const
{
	// Lazily init the name because there's no good initialization hook after configs are loaded and MetricName is set.
	if (MutatorName.IsNone())
	{
		MutatorName = FName(MetricName.GetPlainNameString() + TEXT("Avg"));
	}

	return MutatorName;
}

void FNetworkMetricsMutatorAvg::ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
	for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
	{
		AddIntSample(Metric.Value);
	}

	for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
	{
		AddFloatSample(Metric.Value);
	}
}

void FNetworkMetricsMutatorAvg::Reset()
{
	IntTotal = 0;
	FloatTotal = 0.0f;
	NumSamples = 0;
}

FNetworkMetricsMutator::FValueVariant FNetworkMetricsMutatorAvg::GetValue() const
{
	if (NumSamples != 0)
	{
		return FValueVariant(TInPlaceType<float>(), (static_cast<float>(IntTotal) + FloatTotal) / NumSamples);
	}

	return FValueVariant(TInPlaceType<float>(), 0.0f);
}

void FNetworkMetricsMutatorAvg::AddIntSample(int64 Sample)
{
	IntTotal += Sample;
	++NumSamples;
}

void FNetworkMetricsMutatorAvg::AddFloatSample(float Sample)
{
	FloatTotal += Sample;
	++NumSamples;
}

FName FNetworkMetricsMutatorMin::GetNameInternal() const
{
	// Lazily init the name because there's no good initialization hook after configs are loaded and MetricName is set.
	if (MutatorName.IsNone())
	{
		MutatorName = FName(MetricName.GetPlainNameString() + TEXT("Min"));
	}

	return MutatorName;
}

void FNetworkMetricsMutatorMin::ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
	// This mutator only supports a single input metric so we can assume it will be the only one in the snapshot.
	ensureMsgf(Snapshot.MetricInts.Num() + Snapshot.MetricFloats.Num() == 1, TEXT("FNetworkMetricsMutatorMin::ProcessFrame expects a single metric"));

	for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
	{
		if (MinValue.IsType<int64>())
		{
			MinValue.Set<int64>(FMath::Min(Metric.Value, MinValue.Get<int64>()));
		}
		else
		{
			MinValue.Set<int64>(FMath::Min(Metric.Value, std::numeric_limits<int64>::max()));
		}
	}

	for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
	{
		if (MinValue.IsType<float>())
		{
			MinValue.Set<float>(FMath::Min(Metric.Value, MinValue.Get<float>()));
		}
		else
		{
			MinValue.Set<float>(FMath::Min(Metric.Value, std::numeric_limits<float>::max()));
		}
	}
}

void FNetworkMetricsMutatorMin::Reset()
{
	MinValue.Set<int64>(std::numeric_limits<int64>::max());
}

FNetworkMetricsMutator::FValueVariant FNetworkMetricsMutatorMin::GetValue() const
{
	return MinValue;
}

FName FNetworkMetricsMutatorMax::GetNameInternal() const
{
	// Lazily init the name because there's no good initialization hook after configs are loaded and MetricName is set.
	if (MutatorName.IsNone())
	{
		MutatorName = FName(MetricName.GetPlainNameString() + TEXT("Max"));
	}

	return MutatorName;
}

void FNetworkMetricsMutatorMax::ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
	// This mutator only supports a single input metric so we can assume it will be the only one in the snapshot.
	ensureMsgf(Snapshot.MetricInts.Num() + Snapshot.MetricFloats.Num() == 1, TEXT("FNetworkMetricsMutatorMax::ProcessFrame expects a single metric"));

	for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
	{
		if (MaxValue.IsType<int64>())
		{
			MaxValue.Set<int64>(FMath::Max(Metric.Value, MaxValue.Get<int64>()));
		}
		else
		{
			MaxValue.Set<int64>(FMath::Max(Metric.Value, std::numeric_limits<int64>::lowest()));
		}
	}

	for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
	{
		if (MaxValue.IsType<float>())
		{
			MaxValue.Set<float>(FMath::Max(Metric.Value, MaxValue.Get<float>()));
		}
		else
		{
			MaxValue.Set<float>(FMath::Max(Metric.Value, std::numeric_limits<float>::lowest()));
		}
	}
}

void FNetworkMetricsMutatorMax::Reset()
{
	MaxValue.Set<int64>(std::numeric_limits<int64>::lowest());
}

FNetworkMetricsMutator::FValueVariant FNetworkMetricsMutatorMax::GetValue() const
{
	return MaxValue;
}

TArrayView<const FName> FNetworkMetricsMutatorPercent::GetAllMetricNames() const
{
	if (SourceMetricNames.IsEmpty())
	{
		SourceMetricNames.Add(NumeratorName);
		SourceMetricNames.Add(DenominatorName);
	}

	return MakeArrayView(SourceMetricNames);
}

FName FNetworkMetricsMutatorPercent::GetNameInternal() const
{
	// Lazily init the name because there's no good initialization hook after configs are loaded and MetricName is set.
	if (MutatorName.IsNone())
	{
		MutatorName = FName(NumeratorName.GetPlainNameString() + DenominatorName.GetPlainNameString() + TEXT("Pct"));
	}

	return MutatorName;
}

template<class T>
void FNetworkMetricsMutatorPercent::UpdateFromSnapshotValue(FValueVariant& InOutValue, T MetricValue)
{
	if (InOutValue.IsType<T>())
	{
		InOutValue.Get<T>() += MetricValue;
	}
	else
	{
		InOutValue.Set<T>(MetricValue);
	}
}

void FNetworkMetricsMutatorPercent::ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot)
{
	// This function assumes metrics with the same name won't change types at runtime
	for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
	{
		if (Metric.Name == NumeratorName)
		{
			UpdateFromSnapshotValue<int64>(Numerator, Metric.Value);
		}

		if (Metric.Name == DenominatorName)
		{
			UpdateFromSnapshotValue<int64>(Denominator, Metric.Value);
		}
	}

	for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
	{
		if (Metric.Name == NumeratorName)
		{
			UpdateFromSnapshotValue<float>(Numerator, Metric.Value);
		}

		if (Metric.Name == DenominatorName)
		{
			UpdateFromSnapshotValue<float>(Denominator, Metric.Value);
		}
	}
}

void FNetworkMetricsMutatorPercent::Reset()
{
	Numerator = FValueVariant(TInPlaceType<int64>(), 0);
	Denominator = FValueVariant(TInPlaceType<int64>(), 0);
}

FNetworkMetricsMutator::FValueVariant FNetworkMetricsMutatorPercent::GetValue() const
{
	const float FloatDenominator = Denominator.IsType<float>() ? Denominator.Get<float>() : static_cast<float>(Denominator.Get<int64>());

	if (FMath::IsNearlyZero(FloatDenominator))
	{
		return FValueVariant(TInPlaceType<float>(), 0.0f);
	}

	const float FloatNumerator = Numerator.IsType<float>() ? Numerator.Get<float>() : static_cast<float>(Numerator.Get<int64>());

	return FValueVariant(TInPlaceType<float>(), (FloatNumerator / FloatDenominator) * 100.0f);
}

FNetworkMetricsMutatorOutPacketLoss::FNetworkMetricsMutatorOutPacketLoss()
	: Super()
{
	SetNumeratorMetricName(UE::Net::Metric::OutLostPacketsFoundPerFrame);
	SetDenominatorMetricName(UE::Net::Metric::OutPacketsPerFrame);
}

FNetworkMetricsMutatorInPacketLoss::FNetworkMetricsMutatorInPacketLoss()
	: Super()
{
	SetNumeratorMetricName(UE::Net::Metric::InLostPacketsFoundPerFrame);
	SetDenominatorMetricName(UE::Net::Metric::InPacketsPerFrame);
}
