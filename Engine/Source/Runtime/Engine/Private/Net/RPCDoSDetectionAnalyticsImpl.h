// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Net/Core/Analytics/RPCDoSDetectionAnalytics.h"


// Typedefs

/**
 * Callback passed in by the NetConnection, for getting the current World for analytics referencing.
 *
 * @return		Returns the current World the NetConnection inhabits
 */
using FGetWorld = TUniqueFunction<UWorld*()>;


/**
 * RPC DoS Detection implementation for basic aggregated net analytics data
 */
class FRPCDoSAnalyticsSenderImpl final : public FRPCDoSAnalyticsSender
{
public:
	void Init(FGetWorld&& InWorldFunc);

	virtual void SendAnalytics(FNetAnalyticsAggregator* Aggregator, FRPCDoSAnalyticsData* Data) override final;

	virtual void FireEvent_ServerRPCDoSEscalation(FNetAnalyticsAggregator* Aggregator, FRPCDoSAnalyticsData* Data, int32 SeverityIndex,
													const FString& SeverityCategory, int32 WorstCountPerSec, double WorstTimePerSec,
													const FString& InPlayerIP, const FString& InPlayerUID, const TArray<FName>& InRPCGroup,
													double InRPCGroupTime=0.0) override final;

private:
	/** Callback used for getting the current World the NetDriver and RPC DoS Detection is associated with */
	FGetWorld WorldFunc;
};

