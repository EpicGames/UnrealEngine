// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/ReplayPlaylistTracker.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "Net/UnrealNetwork.h"

FReplayPlaylistTracker::FReplayPlaylistTracker(const FReplayPlaylistParams& Params, UGameInstance* InGameInstance) :
	bIsStartingReplay(false),
	CurrentReplay(0),
	Playlist(Params.Playlist),
	AdditionalOptions(Params.InitialOptions.NewAdditionalOptions.Get(TArray<FString>())),
	WorldOverride(const_cast<UWorld*>(Params.InitialOptions.NewWorldOverride.Get(nullptr))),
	GameInstance(InGameInstance),
	PreStartNextReplay(Params.PreStartNextPlaylistReplayDelegate)
{
	const FString QuoteString("\"");
	for (int32 i = Playlist.Num() - 1; i >= 0; --i)
	{
		FString& Replay = Playlist[i];
		Replay.TrimStartAndEndInline();

		// Remove leading and trailing quotes.
		const bool bMismatchedQuotes = Replay.StartsWith(QuoteString) != Replay.EndsWith(QuoteString);
		if (Replay.RemoveFromStart(QuoteString) || Replay.RemoveFromEnd(QuoteString))
		{
			// Log the untouched replay.
			UE_LOGF(LogDemo, Log, "FReplayPlaylistTracker: Removed mismatched quotes from %ls", *Params.Playlist[i]);
		}

		// Now that we've maybe removed quotes, try trimming one last time.
		Replay.TrimStartAndEndInline();

		if (Replay.Len() == 0)
		{
			UE_LOGF(LogDemo, Log, "FReplayPlaylistTracker: Removed invalid replay %ls", *Params.Playlist[i]);
			Playlist.RemoveAt(i, EAllowShrinking::No);
		}
	}

	if (Playlist.Num() == 0)
	{
		UE_LOGF(LogDemo, Warning, "FReplayPlaylistTracker: No valid replays in playlist.")
	}
}

void FReplayPlaylistTracker::Reset()
{
	if (UDemoNetDriver* LocalDemoNetDriver = DemoNetDriver.Get())
	{
		DemoNetDriver.Reset();

		FNetworkReplayDelegates::OnReplayPlaybackFailure.Remove(OnDemoPlaybackFailedHandle);
		FNetworkReplayDelegates::OnReplayPlaybackComplete.Remove(OnDemoPlaybackFinishedHandle);
		FNetworkReplayDelegates::OnReplayRecordingComplete.Remove(OnDemoStoppedHandle);

		LocalDemoNetDriver->SetPlayingPlaylist(nullptr);
	}

	OnDemoPlaybackFailedHandle.Reset();
	OnDemoPlaybackFinishedHandle.Reset();
	OnDemoStoppedHandle.Reset();

	// When we are starting a new replay, the old DemoNetDriver will be torn down.
	// It's fine to handle the above code, as we won't reassociate until we know
	// the replay has started successfully, we definitely don't want to reset
	// our current replay.
	if (!bIsStartingReplay)
	{
		CurrentReplay = 0;
	}
}

bool FReplayPlaylistTracker::Start()
{
	// Don't log this case, as it will be called out earlier.
	if (Playlist.Num() > 0 && CurrentReplay < Playlist.Num())
	{
		UWorld* World = WorldOverride.Get();
		World = World ? World : GameInstance->GetWorld();

		bool bStartedPlayingSuccessfully = false;
		{
			TGuardValue<bool> GuardStarting(bIsStartingReplay, true);
			bStartedPlayingSuccessfully = GameInstance->PlayReplay(Playlist[CurrentReplay], World, AdditionalOptions);
		}

		if (bStartedPlayingSuccessfully)
		{
			TSharedRef<ThisClass> This = AsShared();

			if (UDemoNetDriver * LocalDemoNetDriver = World->GetDemoNetDriver())
			{
				DemoNetDriver = LocalDemoNetDriver;

				OnDemoPlaybackFailedHandle = FNetworkReplayDelegates::OnReplayPlaybackFailure.AddSP(AsShared(), &ThisClass::OnDemoPlaybackFailed);
				OnDemoPlaybackFinishedHandle = FNetworkReplayDelegates::OnReplayPlaybackComplete.AddSP(AsShared() , &ThisClass::OnDemoPlaybackFinished);
				OnDemoStoppedHandle = FNetworkReplayDelegates::OnReplayRecordingComplete.AddSP(AsShared(), &ThisClass::OnDemoStopped);

				LocalDemoNetDriver->SetPlayingPlaylist(This);

				UE_LOGF(LogDemo, Log, "FReplayPlaylistTracker::Start: Successfully started Replay=%ls, CurrentReplayIdx=%d", *Playlist[CurrentReplay], CurrentReplay);
				return true;
			}
			else
			{
				UE_LOGF(LogDemo, Error, "FReplayPlaylistTracker::Start: Unable to retrieve DemoNetDriver, replay may be started, but playlist won't continue");
			}
		}
		else
		{
			UE_LOGF(LogDemo, Warning, "FReplayPlaylistTracker::Start: Failed to play replay.");
		}
	}

	return false;
}

bool FReplayPlaylistTracker::Restart()
{
	CurrentReplay = 0;
	return Start();
}

void FReplayPlaylistTracker::OnDemoPlaybackFailed(UWorld* InWorld, const UE::Net::TNetResult<EReplayResult>& Result)
{
	UWorld* World = WorldOverride.Get();
	World = World ? World : GameInstance->GetWorld();

	if (World == InWorld)
	{
		Quit();
	}
}

void FReplayPlaylistTracker::OnDemoStopped(UWorld* InWorld)
{
	UWorld* World = WorldOverride.Get();
	World = World ? World : GameInstance->GetWorld();

	if (World == InWorld)
	{
		Quit();
	}
}

void FReplayPlaylistTracker::Quit()
{
	Reset();
}

void FReplayPlaylistTracker::OnDemoPlaybackFinished(UWorld* InWorld)
{
	UWorld* World = WorldOverride.Get();
	World = World ? World : GameInstance->GetWorld();

	if (World == InWorld)
	{
		PlayNextReplay();
	}
}

void FReplayPlaylistTracker::PlayNextReplay()
{
	bool bWasSuccesful = false;
	if (UGameInstance * LocalGameInstance = GameInstance.Get())
	{
		CurrentReplay = FMath::Min(Playlist.Num(), CurrentReplay + 1);

		if (CurrentReplay < Playlist.Num())
		{
			if (PreStartNextReplay.IsBound())
			{
				FReplayPlaylistUpdatedOptions NewOptions;
				PreStartNextReplay.Execute(*this, NewOptions);

				if (NewOptions.NewWorldOverride.IsSet())
				{
					UE_LOGF(LogDemo, Log, "FReplayPlaylistTracker::PlayNextReplay: Updated World Override, NewWorldOverride=%ls, OldWorldOverride=%ls",
						*GetPathNameSafe(NewOptions.NewWorldOverride.GetValue()),
						*GetPathNameSafe(WorldOverride.Get()));

					WorldOverride = NewOptions.NewWorldOverride.GetValue();
				}
				if (NewOptions.NewAdditionalOptions.IsSet())
				{
					UE_LOGF(LogDemo, Log, "FReplayPlaylistTracker::PlayNextReplay: Updated AdditionalOptions, NewOptions=[%ls], OldOptions=[%ls]",
						*FString::Join(NewOptions.NewAdditionalOptions.GetValue(), TEXT(",")),
						*FString::Join(AdditionalOptions, TEXT(",")));

					AdditionalOptions = MoveTemp(NewOptions.NewAdditionalOptions.GetValue());
				}
			}

			bWasSuccesful = Start();
		}
		else
		{
			// If we've reached the end of our playlist, we don't need to do anything.
			// The DemoNetDriver may fire every the finish notification every frame,
			// and we don't want to inadvertently reset our state (restarting our playlist).
			// Once the demo has been closed out, we should get the Stopped (or error) delegate callbacks,
			// and that's what will clean us up.
			bWasSuccesful = true;
		}
	}
	if (!bWasSuccesful)
	{
		Quit();
	}
}
