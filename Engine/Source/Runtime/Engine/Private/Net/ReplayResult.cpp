// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/ReplayResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplayResult)

#define CASE_EREPLAYRESULT_TO_TEXT_RET(txt) case txt: ReturnVal = TEXT(#txt); break;

const TCHAR* LexToString(EReplayResult Result)
{
	const TCHAR* ReturnVal = TEXT("::Invalid");

	switch (Result)
	{
		FOREACH_ENUM_EREPLAYRESULT(CASE_EREPLAYRESULT_TO_TEXT_RET)
	}

	while (*ReturnVal != ':')
	{
		ReturnVal++;
	}

	ReturnVal += 2;

	return ReturnVal;
}

#undef CASE_EREPLAYRESULT_TO_TEXT_RET
