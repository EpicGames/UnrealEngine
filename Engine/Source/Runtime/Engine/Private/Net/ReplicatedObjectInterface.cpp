// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/ReplicatedObjectInterface.h"
#include "UObject/Interface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplicatedObjectInterface)


UReplicatedObjectInterface::UReplicatedObjectInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
