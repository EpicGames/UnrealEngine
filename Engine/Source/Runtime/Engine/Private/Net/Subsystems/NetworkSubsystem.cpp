// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/Subsystems/NetworkSubsystem.h"

#include "Net/NetworkGranularMemoryLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkSubsystem)


void UNetworkSubsystem::Serialize(FArchive& Ar)
{
	if (Ar.IsCountingMemory())
	{
		GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UNetworkSubsystem::Serialize");

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GroupsManager", GroupsManager.CountBytes(Ar));
	}
}
