// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Net/Util/SequenceNumber.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNetSequenceNumberTest, "Net.Misc.SequenceNumberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FNetSequenceNumberTest::RunTest(const FString& Parameters)
{
	typedef TSequenceNumber<3, uint16> FSequence3;

	// Test construct
	{
		FSequence3 Seq;
		TestEqual(TEXT("SequenceNumbers - Seq() == 0"), 0u, Seq.Get());		
	}

	// Test construct with value
	{
		FSequence3 Seq(8);
		TestEqual(TEXT("SequenceNumbers - Seq(8) == 0"), 0u, Seq.Get());		
	}

	// Test postIncrement
	{
		FSequence3 Seq;
		Seq++;
		TestEqual("SequenceNumbers - Seq()++ == 1", 1, Seq.Get());
	}

	// Test preIncrement
	{
		FSequence3 Seq;
		++Seq;
		TestEqual("SequenceNumbers - ++Seq == 1", 1, Seq.Get());
	}

	// Test wraparound
	{
		FSequence3 Seq(7);
		Seq++;
		TestEqual("SequenceNumbers - Seq(7) + 1 == 0", 0, Seq.Get());
	}

	// Test equal and not equal
	{
		FSequence3 A(2);
		FSequence3 B(2);
		FSequence3 C(1);

		TestTrue("SequenceNumbers - operator ==", A == B);
		TestTrue("SequenceNumbers - operator ==", B == A);
		TestTrue("SequenceNumbers - operator !=", A != C);
		TestTrue("SequenceNumbers - operator !=", C != A);
		TestFalse("SequenceNumbers - operator !=", A != B);
	}

	// Test operator >
	{
		FSequence3 Zero(0);
		FSequence3 HalfMinusOne(FSequence3::SeqNumberHalf-1);
		FSequence3 Half(FSequence3::SeqNumberHalf);
		FSequence3 MaxSeq(FSequence3::SeqNumberCount - 1);

		TestTrue("SequenceNumbers - operator> HalfMinusOne > Zero", HalfMinusOne > Zero);
		TestTrue("SequenceNumbers - operator> Half > HalfMinusOne", Half > HalfMinusOne);
		TestFalse("SequenceNumbers - operator> Half > Zero", Half > Zero);

		TestFalse("SequenceNumbers - operator> MaxSeq > Zero", MaxSeq > Zero);
		TestTrue("SequenceNumbers - operator> Zero > MaxSeq", Zero > MaxSeq);
	}

	// Test operator >=
	{
		FSequence3 Zero(0);
		FSequence3 HalfMinusOne(FSequence3::SeqNumberHalf - 1);
		FSequence3 Half(FSequence3::SeqNumberHalf);
		FSequence3 MaxSeq(FSequence3::SeqNumberCount - 1);

		TestTrue("SequenceNumbers - operator> Zero >= Zero", Zero >= Zero); //-V501 PVS - Testing overloaded operator
		TestTrue("SequenceNumbers - operator> MaxSeq >= MaxSeq", MaxSeq >= MaxSeq); //-V501 PVS - Testing overloaded operator

		TestTrue("SequenceNumbers - operator> HalfMinusOne >= Zero", HalfMinusOne >= Zero);
		TestTrue("SequenceNumbers - operator> Half >= HalfMinusOne", Half >= HalfMinusOne);
		TestFalse("SequenceNumbers - operator> Half >= Zero", Half >= Zero);

		TestFalse("SequenceNumbers - operator> MaxSeq >= Zero", MaxSeq >= Zero);
		TestTrue("SequenceNumbers - operator> Zero >= MaxSeq", Zero >= MaxSeq);
	}

	// Test operator <
	{
		FSequence3 Zero(0);
		FSequence3 HalfMinusOne(FSequence3::SeqNumberHalf - 1);
		FSequence3 Half(FSequence3::SeqNumberHalf);
		FSequence3 MaxSeq(FSequence3::SeqNumberCount - 1);

		TestFalse("SequenceNumbers - operator> HalfMinusOne < Zero", HalfMinusOne < Zero);
		TestFalse("SequenceNumbers - operator> Half < HalfMinusOne", Half < HalfMinusOne);
		TestTrue("SequenceNumbers - operator> Half < Zero", Half < Zero);

		TestTrue("SequenceNumbers - operator> MaxSeq < Zero", MaxSeq < Zero);
		TestFalse("SequenceNumbers - operator> Zero < MaxSeq", Zero < MaxSeq);
	}

	// Test operator <=
	{
		FSequence3 Zero(0);
		FSequence3 HalfMinusOne(FSequence3::SeqNumberHalf - 1);
		FSequence3 Half(FSequence3::SeqNumberHalf);
		FSequence3 MaxSeq(FSequence3::SeqNumberCount - 1);

		TestTrue("SequenceNumbers - operator> Zero <= Zero", Zero <= Zero); //-V501 PVS - Testing overloaded operator
		TestTrue("SequenceNumbers - operator> MaxSeq <= MaxSeq", MaxSeq <= MaxSeq); //-V501 PVS - Testing overloaded operator

		TestFalse("SequenceNumbers - operator> HalfMinusOne < Zero", HalfMinusOne < Zero);
		TestFalse("SequenceNumbers - operator> Half < HalfMinusOne", Half < HalfMinusOne);
		TestTrue("SequenceNumbers - operator> Half < Zero", Half < Zero);

		TestTrue("SequenceNumbers - operator> MaxSeq < Zero", MaxSeq < Zero);
		TestFalse("SequenceNumbers - operator> Zero < MaxSeq", Zero < MaxSeq);
	}

	// Test operator +
	{
		FSequence3 Seq(5);
		FSequence3 Result(Seq + FSequence3(3));

		TestEqual("SequenceNumbers - Seq(5) + 3 == 0", 0, Result.Get());
	}

	// Test operator -
	{
		FSequence3 Seq(5);
		FSequence3 Result(Seq - FSequence3(7));

		TestEqual("SequenceNumbers - Seq(5) - 7 == 6", 6, Result.Get());
	}
	
	// Test diff sequence numbers
	{		
		// Valid sequence = 0-7, max distance between sequence number in order to determine order is half the sequence space (0-3)
		for (int32 I = 0; I < FSequence3::SeqNumberCount; ++I)
		{
			FSequence3 Seq(I);
			FSequence3 Ref(I);
			
			for (int32 U = 0; U < FSequence3::SeqNumberCount; ++U)
			{	
				FSequence3::DifferenceT Diff = FSequence3::Diff(Seq, Ref);

				if (U < FSequence3::SeqNumberHalf)
				{
					if (!(Diff == U))
					{
						TestTrue(TEXT("SequenceNumbers - Expected Diff "), Diff == U);
					}
				}
				else
				{
					if (!(Diff == (U - FSequence3::SeqNumberCount)))
					{
						TestTrue(TEXT("SequenceNumbers - Expected Diff"), Diff == (U - FSequence3::SeqNumberCount));
					}
				}
				++Seq;
			}
		}

		check(!HasAnyErrors());
	}

	return true;
}

#endif // #if WITH_DEV_AUTOMATION_TESTS