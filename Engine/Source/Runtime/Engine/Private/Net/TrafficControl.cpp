// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "Net/TrafficControl.h"
#include "CoreGlobals.h"
#include "EngineLogs.h"

FNetworkCongestionControl::FNetworkCongestionControl(double ConfiguredNetSpeed, uint32 MaxPackets) :
	Analyzer(this),
	MinRTT(10),
	NetSpeed(ConfiguredNetSpeed),
	MaxPacketsAllowedInFlight(MaxPackets)
{}

void FNetworkCongestionControl::UpdateMinRTT(double Timestamp)
{
	double LatestRTT = Analyzer.GetLatestRTT();
	if (LatestRTT <= MinRTT)
	{
		MinRTT = LatestRTT;
	}
	else
	{
		MinRTT = 0.9 * MinRTT + 0.1 * LatestRTT;
	}
}

double FNetworkCongestionControl::GetMinRTT() const
{
	return MinRTT;
}

void FNetworkCongestionControl::OnAck(const FAckSample& AckSample)
{
	Analyzer.OnAck(AckSample);
	UpdateMinRTT(AckSample.Timestamp);
}

bool FNetworkCongestionControl::IsReadyToSend(double Timestamp) const
{
	uint32 BytesInFlight = Analyzer.GetBytesInFlight();
	uint32 PacketsInFlight = Analyzer.GetPacketsInFlight();
	double CongestionWindow = GetCongestionWindow();
	if (BytesInFlight >= CongestionWindow || PacketsInFlight >= MaxPacketsAllowedInFlight)
	{
		UE_LOG(LogNetTraffic, Verbose,
			TEXT("%" UINT64_FMT ": Not ready to send, BytesInFlight: %d, PacketsInFlight: %d, CongestionWindow: %f, MinRTT: %f"),
			GFrameCounter, BytesInFlight, PacketsInFlight, CongestionWindow, GetMinRTT());
		return false;
	}

	UE_LOGF(LogNetTraffic, VeryVerbose,
		"BytesInFlight: %d, PacketsInFlight: %d, CongestionWindow: %f, MinRTT: %f",
		BytesInFlight, PacketsInFlight, CongestionWindow, GetMinRTT());
	return true;
}

void FNetworkCongestionControl::OnSend(const FSeqSample& SeqSample)
{
	Analyzer.OnSend(SeqSample);
}

double FNetworkCongestionControl::GetCongestionWindow() const
{
	return GetMinRTT() * NetSpeed;
}

FNetworkTrafficAnalyzer::FNetworkTrafficAnalyzer(FNetworkCongestionControl* InTrafficControlModule) :
	TrafficControlModule(InTrafficControlModule),
	OutPacketRecords(8),
	LatestUplinkBandwidth(0),
	LatestRTT(0),
	OutSeq(0),
	OutAckedSeq(0),
	BytesInFlight(0),
	bInitialized(false)
{}

void FNetworkTrafficAnalyzer::OnSend(const FSeqSample& SeqSample)
{
	SeqT Seq(SeqSample.Seq);

	if (!bInitialized || (Seq == OutSeq + SeqT(1)))
	{
		OutSeq = SeqT(Seq);

		// First packet sent
		if (!bInitialized)
		{
			OutAckedSeq = OutSeq - SeqT(1);
		}

		bInitialized = true;
		check(OutSeq != OutAckedSeq);
	}
	else
	{
		UE_LOGF(LogNet, Error, "Sending out of order packet, expected: %d, actual: %d", (OutSeq + SeqT(1)).Get(), Seq.Get());
		return;
	}

	BytesInFlight += SeqSample.PacketSize;
	OutPacketRecords.Enqueue(SeqSample);

	UE_LOGF(LogNet, VeryVerbose, "FNetworkTrafficAnalyzer::OnSend, BytesInFlight: %d, "
		"PacketsInFlight: %d, OutSeq: %d, OutAckedSeq: %d",
		BytesInFlight, GetPacketsInFlight(), OutSeq.Get(), OutAckedSeq.Get());
}

void FNetworkTrafficAnalyzer::OnAck(const FAckSample& AckSample)
{
	SeqT Ack(AckSample.Ack);

	int32 AckNum = SeqT::Diff(Ack, OutAckedSeq);
	if (AckNum <= 0)
	{
		return;
	}

	UE_LOGF(LogNet, VeryVerbose, "FNetworkTrafficAnalyzer::OnAck, Ack: %d, AckNum: %d",
		Ack.Get(), AckNum);

	check((OutAckedSeq + SeqT(1)) == SeqT(OutPacketRecords.Peek().Seq));

	// Process history acks
	for (int i = 1; i < AckNum; ++i)
	{
		const FSeqSample& PacketRecord = OutPacketRecords.Peek();
		BytesInFlight -= PacketRecord.PacketSize;
		OutPacketRecords.Pop();
	}

	// Process current ack
	const FSeqSample& PacketRecord = OutPacketRecords.Peek();
	check(Ack == SeqT(PacketRecord.Seq));
	BytesInFlight -= PacketRecord.PacketSize;
	LatestRTT = AckSample.Timestamp - PacketRecord.Timestamp;
	if (LatestRTT > 0)
	{
		LatestUplinkBandwidth = PacketRecord.PacketSize / LatestRTT;
	}
	OutPacketRecords.Pop();

	OutAckedSeq = Ack;

	UE_LOGF(LogNet, VeryVerbose, "FNetworkTrafficAnalyzer::OnAck, BytesInFlight: %d, "
		"PacketsInFlight: %d, LatestRTT: %f, LatestUplinkBandwidth: %f, OutSeq: %d, OutAckedSeq: %d",
		BytesInFlight, GetPacketsInFlight(), LatestRTT, LatestUplinkBandwidth, OutSeq.Get(), OutAckedSeq.Get());
}
