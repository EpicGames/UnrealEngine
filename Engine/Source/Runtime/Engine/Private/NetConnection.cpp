// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetConnection.cpp: Unreal connection base class.
=============================================================================*/

#include "Engine/NetConnection.h"
#include "Engine/ReplicationDriver.h"
#include "Engine/GameInstance.h"
#include "EngineStats.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "UObject/Package.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LevelStreaming.h"
#include "PacketHandlers/StatelessConnectHandlerComponent.h"
#include "Engine/LocalPlayer.h"
#include "Stats/StatsTrace.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Net/UnrealNetwork.h"
#include "Net/NetworkProfiler.h"
#include "Net/DataReplication.h"
#include "Engine/ActorChannel.h"
#include "Engine/ChildConnection.h"
#include "Engine/VoiceChannel.h"
#include "Misc/App.h"
#include "Net/DataChannel.h"
#include "Engine/PackageMapClient.h"
#include "Engine/NetworkObjectList.h"
#include "EncryptionComponent.h"
#include "Net/PerfCountersHelpers.h"
#include "GameDelegates.h"
#include "Misc/PackageName.h"
#include "Templates/Greater.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectIterator.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "SocketSubsystem.h"
#include "HAL/LowLevelMemStats.h"
#include "Net/NetPing.h"
#include "LevelUtils.h"
#include "Net/Core/Connection/RPCDoSDetection.h"
#include "Net/RPCDoSDetectionAnalyticsImpl.h"
#include "Net/NetConnectionFaultRecovery.h"
#include "Net/NetSubObjectRegistryGetter.h"
#include "Net/RepLayout.h"
#include "Net/Subsystems/NetworkSubsystem.h"
#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Iris/ReplicationSystem/NetTokenStore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetConnection)

DECLARE_LLM_MEMORY_STAT(TEXT("NetConnection"), STAT_NetConnectionLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(NetConnection, NAME_None, TEXT("Networking"), GET_STATFNAME(STAT_NetConnectionLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));

static TAutoConsoleVariable<int32> CVarPingExcludeFrameTime(TEXT("net.PingExcludeFrameTime"), 0,
	TEXT("If true, game frame times are subtracted from calculated ping to approximate actual network ping"));

static TAutoConsoleVariable<int32> CVarPingUsePacketRecvTime(TEXT("net.PingUsePacketRecvTime"), 0,
	TEXT("Use OS or Receive Thread packet receive time, for calculating the ping. Excludes frame time."));

static TAutoConsoleVariable<int32> CVarPingPlayerFacingPingType(TEXT("net.PlayerFacingPingType"), int32(EPingType::RoundTripExclFrame),
	TEXT("The preferred ping type reported to the player controller. 1 = RoundTrip (including server frame time), 2 = RoundTripExclFrame, 4 = ICMP, 8 = UDPQoS"));

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<int32> CVarPingDisplayServerTime(TEXT("net.PingDisplayServerTime"), 0,
	TEXT("Show server frame time. Not available in shipping builds."));
#endif

static TAutoConsoleVariable<int32> CVarTickAllOpenChannels(TEXT("net.TickAllOpenChannels"), 0,
	TEXT("If nonzero, each net connection will tick all of its open channels every tick. Leaving this off will improve performance."));

static TAutoConsoleVariable<int32> CVarRandomizeSequence(TEXT("net.RandomizeSequence"), 1,
	TEXT("Randomize initial packet sequence, can provide some obfuscation"));

static TAutoConsoleVariable<int32> CVarMaxChannelSize(TEXT("net.MaxChannelSize"), 0,
	TEXT("The maximum number of network channels allowed across the entire server, if <= 0 the connection DefaultMaxChannelSize will be used."));

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<int32> CVarForceNetFlush(TEXT("net.ForceNetFlush"), 0,
	TEXT("Immediately flush send buffer when written to (helps trace packet writes - WARNING: May be unstable)."));
#endif

static TAutoConsoleVariable<int32> CVarNetDoPacketOrderCorrection(TEXT("net.DoPacketOrderCorrection"), 1,
	TEXT("Whether or not to try to fix 'out of order' packet sequences, by caching packets and waiting for the missing sequence."));

static TAutoConsoleVariable<int32> CVarNetPacketOrderCorrectionEnableThreshold(TEXT("net.PacketOrderCorrectionEnableThreshold"), 1,
	TEXT("The number of 'out of order' packet sequences that need to occur, before correction is enabled."));

static TAutoConsoleVariable<int32> CVarNetPacketOrderMaxMissingPackets(TEXT("net.PacketOrderMaxMissingPackets"), 3,
	TEXT("The maximum number of missed packet sequences that is allowed, before treating missing packets as lost."));

static TAutoConsoleVariable<int32> CVarNetPacketOrderMaxCachedPackets(TEXT("net.PacketOrderMaxCachedPackets"), 32,
	TEXT("(NOTE: Must be power of 2!) The maximum number of packets to cache while waiting for missing packet sequences, before treating missing packets as lost."));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarDisableBandwithThrottling(TEXT("net.DisableBandwithThrottling"), 0,
	TEXT("Forces IsNetReady to always return true. Not available in shipping or test builds."));
#endif

TAutoConsoleVariable<int32> CVarNetEnableCongestionControl(TEXT("net.EnableCongestionControl"), 0,
	TEXT("Enables congestion control module."));

static TAutoConsoleVariable<int32> CVarLogUnhandledFaults(TEXT("net.LogUnhandledFaults"), 1,
	TEXT("Whether or not to warn about unhandled net faults (could be deliberate, depending on implementation). 0 = off, 1 = log once, 2 = log always."));

static int32 GNetCloseTimingDebug = 0;

static FAutoConsoleVariableRef CVarCloseTimingDebug(TEXT("net.CloseTimingDebug"), GNetCloseTimingDebug,
	TEXT("Logs the last packet send/receive and TickFlush/TickDispatch times, on connection close - for debugging blocked send/recv paths."));

static bool GCVarLogAllConnectionCleanup = false;
static FAutoConsoleVariableRef CVarLogAllConnectionCleanup(TEXT("net.Connection.LogAllCleanup"), GCVarLogAllConnectionCleanup, TEXT("When true log every connection CleanUp even when it was a normal socket closure."));

static TAutoConsoleVariable<bool> CVarSkipMissingLevelDisconnect( TEXT("net.SkipMissingLevelDisconnect"), false, TEXT("If true skip disconnecting a player if they have a level that doesn't exist on server"));

int32 BufferBloatCooldownTimeInMS = 1000;
static FAutoConsoleVariableRef CVarBufferBloatCooldownTimeInMS(
	TEXT("netEmulation.BufferBloatCooldownTimeInMS"),
	BufferBloatCooldownTimeInMS,
	TEXT("Cooldown time in milliseconds during which buffer bloat won't be active."),
	ECVF_Default);

extern int32 GNetDormancyValidate;
extern bool GbNetReuseReplicatorsForDormantObjects;

DECLARE_CYCLE_STAT(TEXT("NetConnection SendAcks"), Stat_NetConnectionSendAck, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("NetConnection Tick"), Stat_NetConnectionTick, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("NetConnection ReceivedNak"), Stat_NetConnectionReceivedNak, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("NetConnection NetConnectionReceivedAcks"), Stat_NetConnectionReceivedAcks, STATGROUP_Net);

namespace NetConnectionHelper
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bool HasDisabledBandwidthThrottling()
	{
		return CVarDisableBandwithThrottling.GetValueOnAnyThread() > 0;
	}
#endif
}

namespace UE::Net::Connection::Private
{
	struct FValidateLevelVisibilityResult
	{
		bool bLevelExists = false;
		const ULevelStreaming* StreamingLevel;
		UPackage* Package = nullptr;
		FLinkerLoad* Linker = nullptr;
	};

	const FValidateLevelVisibilityResult ValidateLevelVisibility(UWorld* World, const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Net_ValidateLevelVisibility);

		// Verify that we were passed a valid level name
		// If we have a linker we know it has been loaded off disk successfully
		// If we have a file it is fine too
		// If its in our own streaming level list, its good

		FValidateLevelVisibilityResult Result;

		auto FindStreamingLevel = [World, &Result](FName InPackageName)
		{
			FString PackageNameStr = InPackageName.ToString();
			Result.Package = FindPackage(nullptr, *PackageNameStr);
#if WITH_EDITOR
			if (Result.Package && Result.Package->IsDynamicPIEPackagePending())
			{
				// Package is a dynamic PIE package with pending external objects still loading, mimic same behavior as if it was AsyncLoad
				Result.Package = nullptr;
			}
#endif
			Result.Linker = FLinkerLoad::FindExistingLinkerForPackage(Result.Package);

			Result.StreamingLevel = FLevelUtils::FindStreamingLevel(World, InPackageName);
			Result.bLevelExists = (Result.Linker != nullptr) || (Result.StreamingLevel != nullptr);

			if (!Result.bLevelExists)
			{
				Result.bLevelExists = FLevelUtils::IsValidStreamingLevel(World, *PackageNameStr);
			}
		};

		FindStreamingLevel(LevelVisibility.PackageName);

		if (!Result.bLevelExists)
		{
			Result.bLevelExists = (!FPackageName::IsMemoryPackage(LevelVisibility.PackageName.ToString()) && FPackageName::DoesPackageExist(LevelVisibility.FileName.ToString()));
		}

		// If the level is not found, try resolving it using the optional LevelVisibility.CookPackageName.
		// This is useful when the in-memory PackageName differs from the cooked package name.
		// A common example is when a PIE client sends PIE memory-generated packages to a cooked server.
		if (!Result.bLevelExists && (!LevelVisibility.CookPackageName.IsNone() && LevelVisibility.CookPackageName != LevelVisibility.PackageName))
		{
			FindStreamingLevel(LevelVisibility.CookPackageName);
		}

		return Result;
	}

	/** Maximum possible clock time value with available bits. If that value is sent than the jitter clock time is ignored */
	constexpr uint32 MaxJitterClockTimeValue = (1 << NetConnectionHelper::NumBitsForJitterClockTimeInHeader) - 1;

	/** Maximum possible precision of the jitter stat. Set to 1 second since we only send the milliseconds of the local clock time */
	constexpr int32 MaxJitterPrecisionInMS = 1000;

	/** Used to reduce the impact on the average jitter of a divergent value */
	constexpr double JitterNoiseReduction = 16.0;

	/**
	 * Return the milliseconds of a ClockTime
	 * Ex: 3.45 seconds => 450 milliseconds
	 */
	int32 GetClockTimeMilliseconds(double ClockTimeInSeconds)
	{
		return FMath::TruncToInt((ClockTimeInSeconds - FMath::FloorToDouble(ClockTimeInSeconds)) * 1000.0);
	}
	
	// Attempts to add two values while detecting and preventing 32-bit signed integer overflow and underflow, used for QueuedBits.
	// Returns true if an overflow would have occurred, false if not.
	bool Add_DetectOverflow_Clamp(int64 Original, int64 Change, int32& Result)
	{
		const int64 Result64 = Original + Change;

		if (Result64 > std::numeric_limits<int32>::max())
		{
			Result = std::numeric_limits<int32>::max();
			return true;
		}

		if (Result64 < std::numeric_limits<int32>::min())
		{
			Result = std::numeric_limits<int32>::min();
			return true;
		}

		Result = static_cast<int32>(Result64);
		return false;
	}

	bool bGracefulCloseEnabled = true;
	FAutoConsoleVariableRef CVarNetGracefulCloseEnabled(TEXT("net.GracefulCloseEnabled"), bGracefulCloseEnabled, TEXT("If enabled, connections will wait for reliable bunches to be acked before cleaning up."));

	int32 bTrackFlushedDormantObjects = true;
	FAutoConsoleVariableRef CVarNetTrackFlushedDormantObjects(TEXT("net.TrackFlushedDormantObjects"), bTrackFlushedDormantObjects, TEXT("If enabled, track dormant subobjects when dormancy is flushed, so they can be properly deleted if destroyed prior to the next ReplicateActor."));

	int32 bEnableFlushDormantSubObjects = true;
	FAutoConsoleVariableRef CVarNetFlushDormantSubObjects(TEXT("net.EnableFlushDormantSubObjects"), bEnableFlushDormantSubObjects, TEXT("If enabled, FlushNetDormancy will flush replicated subobjects in addition to replicated components. Only applies to objects using the replicated subobject list."));

	int32 bEnableFlushDormantSubObjectsCheckConditions = true;
	FAutoConsoleVariableRef CVarNetFlushDormantSubObjectsCheckConditions(TEXT("net.EnableFlushDormantSubObjectsCheckConditions"), bEnableFlushDormantSubObjectsCheckConditions, TEXT("If enabled, when net.EnableFlushDormantSubObjects is also true a dormancy flush will also check replicated subobject conditions"));

	int32 bFlushDormancyUseDefaultStateForUnloadedLevels = true;
	FAutoConsoleVariableRef CVarFlushDormancyUseDefaultStateForUnloadedLevels(TEXT("net.FlushDormancyUseDefaultStateForUnloadedLevels"), bFlushDormancyUseDefaultStateForUnloadedLevels, TEXT("If enabled, dormancy flushing will init replicators with default object state if the client doesn't have the actor's level loaded."));

	// Tracking for dormancy-flushed subobjects for correct deletion, see UE-77163
	void TrackFlushedSubObject(FDormantObjectMap& InOutFlushedGuids, UObject* FlushedObject, const TSharedPtr<FNetGUIDCache>& GuidCache)
	{
		if (Connection::Private::bTrackFlushedDormantObjects)
		{
			// Searching for the guid because the value on the replicator built in FlushDormancyForObject can be invalid until the next replication
			// We can then safely ignore any object that still has no guid, since it won't have ever been replicated
			FNetworkGUID ObjectNetGUID = GuidCache->GetNetGUID(FlushedObject);
			if (ObjectNetGUID.IsValid())
			{
				InOutFlushedGuids.Add(ObjectNetGUID, FlushedObject);
			}
		}
	}

	// Flushes dormancy for registered subobjects of either an actor or component
	void FlushDormancyForSubObjects(UNetConnection* Connection, AActor* Actor, const UE::Net::FSubObjectRegistry& SubObjects, FDormantObjectMap& InOutFlushedGuids, const TStaticBitArray<COND_Max>& ConditionMap, const FNetConditionGroupManager* NetConditionGroupManager)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetConnection_FlushDormancyForSubObjects);

		for (const FSubObjectRegistry::FEntry& SubObjectInfo : SubObjects.GetRegistryList())
		{
			UObject* SubObject = SubObjectInfo.GetSubObject();
			if (ensureMsgf(IsValid(SubObject), TEXT("Found invalid subobject (%s) registered in %s"), *GetNameSafe(SubObject), *Actor->GetName()))
			{
				if (!bEnableFlushDormantSubObjectsCheckConditions ||
					!NetConditionGroupManager ||
					UActorChannel::CanSubObjectReplicateToClient(Connection->PlayerController, SubObjectInfo.NetCondition, SubObjectInfo.Key, ConditionMap, *NetConditionGroupManager))
				{
					Connection->FlushDormancyForObject(Actor, SubObject);
					TrackFlushedSubObject(InOutFlushedGuids, SubObject, Connection->Driver->GetNetGuidCache());
				}
			}
		}
	}
}

// ChannelRecord Implementation
namespace FChannelRecordImpl
{

// Push ChannelRecordEntry for packet if PacketID differs from last PacketId
static void PushPacketId(FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId);

// Push written ChannelIndex to WrittenChanneRecord, Push new packetId if PackedIdDiffers from Last pushed entry */
static void PushChannelRecord(FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId, int32 ChannelIndex);

static bool HasChannelRecordsToConsumeForPacket(const FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId);

// Consume all FChannelRecordEntries for the given PacketId and execute a function with the signature (void*)(int32 PacketId, uint32 Channelndex) for each entry
template<class Functor>
static void ConsumeChannelRecordsForPacket(FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId, Functor&& Func);

// Consume all FChannelRecordEntries and execute a function with the signature (void*)(uint32 Channelndex) each entry
template<class Functor>
static void ConsumeAllChannelRecords(FWrittenChannelsRecord& WrittenChannelsRecord, Functor&& Func);

// Returns allocated size of the record
static SIZE_T CountBytes(FWrittenChannelsRecord& WrittenChannelsRecord);

};

namespace UE::Net::Private
{
	extern bool bTrackDormantObjectsByLevel;

#if UE_SUPPORT_PARALLEL_IRIS
	static FTransactionallySafeCriticalSection SharedNetTickMutex; //Shared across all NetConnections to protect sections of the Tick logic that cannot run in parallel
#endif // UE_SUPPORT_PARALLEL_IRIS
}

/*-----------------------------------------------------------------------------
	UNetConnection implementation.
-----------------------------------------------------------------------------*/

UNetConnection* UNetConnection::GNetConnectionBeingCleanedUp = NULL;

UNetConnection::UNetConnection(const FObjectInitializer& ObjectInitializer)
:	UPlayer(ObjectInitializer)
,	Driver				( nullptr )
,	PackageMapClass		( UPackageMapClient::StaticClass() )
,	PackageMap			( nullptr )
,	ViewTarget			( nullptr )
,   OwningActor			( nullptr )
,	MaxPacket			( 0 )
,	bInternalAck		( false )
,	bReplay				( false )
,	bForceInitialDirty	( false )
,	bUnlimitedBunchSizeAllowed ( false )
,	RemoteAddr			( nullptr )
,	MaxPacketHandlerBits ( 0 )
,	State				( USOCK_Invalid )
,	Handler()
,	StatelessConnectComponent()
,	PacketOverhead		( 0 )
PRAGMA_DISABLE_DEPRECATION_WARNINGS
,	ResponseId			( 0 ) // variable is unused and will be deleted
PRAGMA_ENABLE_DEPRECATION_WARNINGS
,	QueuedBits			( 0 )
,	TickCount			( 0 )
,	LastProcessedFrame	( 0 )
,	LastOSReceiveTime	()
,	bIsOSReceiveTimeLocal(false)
,   PreviousJitterTimeDelta(0)
,	PreviousPacketSendTimeInS(0.0)
,	AllowMerge			( false )
,	TimeSensitive		( false )
,	LastOutBunch		( nullptr )
,	SendBunchHeader		( MAX_BUNCH_HEADER_BITS )

,	StatPeriod			( 1.f  )
,	AvgLag				( 0 )
,	LagAcc				( 0 )
,	LagCount			( 0 )
,	LastTime			( 0 )
,	FrameTime			( 0 )
,	CumulativeTime		( 0 )
,	AverageFrameTime	( 0 )
,	CountedFrames		( 0 )
,	InBytes				( 0 )
,	OutBytes			( 0 )
,	InTotalBytes		( 0 )
,	OutTotalBytes		( 0 ) 
,	InPackets			( 0 )
,	OutPackets			( 0 )
,	InTotalPackets		( 0 )
,	OutTotalPackets		( 0 )
,	InBytesPerSecond	( 0 )
,	OutBytesPerSecond	( 0 )
,	InPacketsPerSecond	( 0 )
,	OutPacketsPerSecond	( 0 )
,	InTotalPacketsLost	( 0 )
,	OutTotalPacketsLost	( 0 )
,	OutTotalAcks		( 0 )
,	StatPeriodCount		( 0 )
,	AverageJitterInMS	(0.0f)
,	AnalyticsVars		()
,	NetAnalyticsData	()
,	SendBuffer			( 0 )
,	InPacketId			( -1 )
,	OutPacketId			( 0 ) // must be initialized as OutAckPacketId + 1 so loss of first packet can be detected
,	OutAckPacketId		( -1 )
,	bLastHasServerFrameTime( false )
,	DefaultMaxChannelSize(32767)
,	InitOutReliable		( 0 )
,	InitInReliable		( 0 )
,	PackageVersionUE( GPackageFileUEVersion )
,	PackageVersionLicenseeUE( GPackageFileLicenseeUEVersion )
,	ResendAllDataState( EResendAllDataState::None )
#if !UE_BUILD_SHIPPING
,	ReceivedRawPacketDel()
#endif
,	PlayerOnlinePlatformName( NAME_None )
,	ClientWorldPackageName( NAME_None )
,	LastNotifiedPacketId( -1 )
,	OutTotalNotifiedPackets(0)
,	HasDirtyAcks(0u)
,	bHasWarnedAboutChannelLimit(false)
,	bConnectionPendingCloseDueToSocketSendFailure(false)
,	PacketOrderCache()
,	PacketOrderCacheStartIdx(0)
,	PacketOrderCacheCount(0)
,	bFlushingPacketOrderCache(false)
{
	NetworkCustomVersions = FNetworkVersion::GetNetworkCustomVersions();
}

UNetConnection::UNetConnection(FVTableHelper& Helper)
	: Super(Helper)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UNetConnection::~UNetConnection() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UNetConnection::InitChannelData()
{
	LLM_SCOPE_BYTAG(NetConnection);

	if (!ensureMsgf(Channels.Num() == 0, TEXT("InitChannelData: Already initialized!")))
	{
		return;
	}
	
	check(Driver);
	check(!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject | EObjectFlags::RF_ArchetypeObject));

	int32 ChannelSize = CVarMaxChannelSize.GetValueOnAnyThread();
	if (ChannelSize <= 0)
	{
		// set from the connection default
		ChannelSize = DefaultMaxChannelSize;
	
		// allow the driver to override
		const int32 MaxChannelsOverride = Driver->GetMaxChannelsOverride();
		if (MaxChannelsOverride > 0)
	{
			ChannelSize = MaxChannelsOverride;
		}
	}

	UE_LOGF(LogNet, Log, "%ls setting maximum channels to: %d", *GetNameSafe(this), ChannelSize);

	Channels.AddDefaulted(ChannelSize);
	OutReliable.AddDefaulted(ChannelSize);
	InReliable.AddDefaulted(ChannelSize);

	if (!IsInternalAck())
	{
		PendingOutRec.AddDefaulted(ChannelSize);
	}

		PacketNotify.Init(InPacketId, OutPacketId);
	}	

/**
 * Initialize common settings for this connection instance
 *
 * @param InDriver the net driver associated with this connection
 * @param InSocket the socket associated with this connection
 * @param InURL the URL to init with
 * @param InState the connection state to start with for this connection
 * @param InMaxPacket the max packet size that will be used for sending
 * @param InPacketOverhead the packet overhead for this connection type
 */
void UNetConnection::InitBase(UNetDriver* InDriver,class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	using namespace UE::Net;

	// Oodle depends upon this
	check(InMaxPacket <= MAX_PACKET_SIZE);

	// Owning net driver
	Driver = InDriver;

	InitChannelData();

	// Cache instance id
#if UE_NET_TRACE_ENABLED
	NetTraceId = Driver->GetNetTraceId();
#endif

	SetConnectionHandle(UE::Net::FConnectionHandle(InDriver->AllocateConnectionId()));

	const double DriverElapsedTime = Driver->GetElapsedTime();

	// Stats
	StatUpdateTime			= DriverElapsedTime;
	LastReceiveTime			= DriverElapsedTime;
	LastReceiveRealtime		= 0.0;			// These are set to 0 and initialized on our first tick to deal with scenarios where
	LastGoodPacketRealtime	= 0.0;			// notable time may elapse between init and first use
	LastTime				= 0.0;
	LastSendTime			= DriverElapsedTime;
	LastTickTime			= DriverElapsedTime;
	LastRecvAckTimestamp	= DriverElapsedTime;
	ConnectTimestamp		= DriverElapsedTime;

	// Analytics
	TSharedPtr<FNetAnalyticsAggregator>& AnalyticsAggregator = Driver->AnalyticsAggregator;
	const bool bIsReplay = IsReplay();
	const bool bIsServer = Driver->IsServer();

	if (AnalyticsAggregator.IsValid() && !bIsReplay)
	{
		const TCHAR* AnalyticsDataName = (bIsServer ? TEXT("Core.ServerNetConn") : TEXT("Core.ClientNetConn"));

		NetAnalyticsData = REGISTER_NET_ANALYTICS(AnalyticsAggregator, FNetConnAnalyticsData, AnalyticsDataName);
	}

	NetConnectionHistogram.InitHitchTracking();

	// Current state
	SetConnectionState(InState);
	// Copy the URL
	URL = InURL;

	// Use the passed in values
	MaxPacket = InMaxPacket;
	PacketOverhead = InPacketOverhead;

	check(MaxPacket > 0 && PacketOverhead > 0);

	bLoggedFlushNetQueuedBitsOverflow = false;

	// Reset Handler
	Handler.Reset();

	InitHandler();

	FaultRecovery = MakeUnique<FNetConnectionFaultRecovery>();

	FaultRecovery->InitDefaults((Driver != nullptr ? Driver->GetNetDriverDefinition().ToString() : TEXT("")), this);

	if (CVarLogUnhandledFaults.GetValueOnAnyThread() != 0)
	{
		FaultRecovery->FaultManager.SetUnhandledResultCallback([](FNetResult&& InResult)
			{
				static TArray<uint32> LoggedResults;

				const int32 LogVal = CVarLogUnhandledFaults.GetValueOnAnyThread();
				const bool bUniqueLog = LogVal == 1;
				const bool bAlwaysLog = LogVal == 2;
				const uint32 ResultHash = GetTypeHash(InResult);

				if (bAlwaysLog || (bUniqueLog && !LoggedResults.Contains(ResultHash)))
				{
					if (bUniqueLog)
					{
						LoggedResults.AddUnique(ResultHash);
					}

					UE_LOGF(LogNet, Warning, "NetConnection FaultManager: Unhandled fault: %ls",
							ToCStr(InResult.DynamicToString(ENetResultString::WithChain)));
				}

				return EHandleNetResult::NotHandled;
			});
	}

#if DO_ENABLE_NET_TEST
	// Copy the command line settings from the net driver
	UpdatePacketSimulationSettings();
#endif

	// Other parameters.
	CurrentNetSpeed = URL.HasOption(TEXT("LAN")) ? GetDefault<UPlayer>()->ConfiguredLanSpeed : GetDefault<UPlayer>()->ConfiguredInternetSpeed;

	if ( CurrentNetSpeed == 0 )
	{
		CurrentNetSpeed = 2600;
	}
	else
	{
		CurrentNetSpeed = FMath::Max<int32>(CurrentNetSpeed, 1800);
	}

	// Init RemoteNetTokenState.
	if (Driver->GetNetTokenStore())
	{
		Driver->GetNetTokenStore()->InitRemoteNetTokenStoreState(GetConnectionHandle().GetParentConnectionId());
	}

	// Create package map.
	UPackageMapClient* PackageMapClient = NewObject<UPackageMapClient>(this, PackageMapClass);

	if (ensure(PackageMapClient != nullptr))
	{
		// We access GuidCache directly here to not trigger ensure when running iris.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PackageMapClient->Initialize(this, Driver->GuidCache);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		PackageMap = PackageMapClient;
	}

	if (bIsServer && !bIsReplay)
	{
		TSharedPtr<FRPCDoSAnalyticsSenderImpl> RPCDoSAnalyticsSender = MakeShared<FRPCDoSAnalyticsSenderImpl>();

		RPCDoSAnalyticsSender->Init(
			[WorldPtr = TWeakObjectPtr<UWorld>(InDriver->GetWorld())]() -> UWorld*
			{
				return WorldPtr.IsValid() ? WorldPtr.Get() : nullptr;
			});

		RPCDoS = MakeUnique<FRPCDoSDetection>();

		RPCDoS->Init(Driver->GetNetDriverDefinition(), AnalyticsAggregator, RPCDoSAnalyticsSender,
			[this]() -> FString
			{
				return LowLevelGetRemoteAddress(true);
			},
			[this]() -> FString
			{
				APlayerState* PS = PlayerController != nullptr ? PlayerController->PlayerState : nullptr;

				return PS != nullptr ? PS->GetUniqueId().ToString() : TEXT("");
			},
			[this]()
			{
				Close(ENetCloseResult::RPCDoS);
			});
	}

	NetPing = FNetPing::CreateNetPing(this);

	UE_NET_TRACE_CONNECTION_CREATED(NetTraceId, GetConnectionHandle().GetParentConnectionId());
	UE_NET_TRACE_CONNECTION_STATE_UPDATED(NetTraceId, GetConnectionHandle().GetParentConnectionId(), static_cast<uint8>(GetConnectionState()));

	if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
	{
		ReplicationSystem->AddConnection(GetConnectionHandle().GetParentConnectionId());
		ReplicationSystem->SetConnectionUserData(GetConnectionHandle().GetParentConnectionId(), this);

		if (RPCDoS.IsValid())
		{
			ReplicationSystem->SetConnectionRPCDoS(GetConnectionHandle().GetParentConnectionId(), RPCDoS.Get());
		}
	}
}

/**
 * Initializes an "addressless" connection with the passed in settings
 *
 * @param InDriver the net driver associated with this connection
 * @param InState the connection state to start with for this connection
 * @param InURL the URL to init with
 * @param InConnectionSpeed Optional connection speed override
 */
void UNetConnection::InitConnection(UNetDriver* InDriver, EConnectionState InState, const FURL& InURL, int32 InConnectionSpeed, int32 InMaxPacket)
{
	using namespace UE::Net;

	Driver = InDriver;

	InitChannelData();

	// Cache instance id
#if UE_NET_TRACE_ENABLED
	NetTraceId = Driver->GetNetTraceId();
#endif

	SetConnectionHandle(UE::Net::FConnectionHandle(InDriver->AllocateConnectionId()));

	// We won't be sending any packets, so use a default size
	MaxPacket = (InMaxPacket == 0 || InMaxPacket > MAX_PACKET_SIZE) ? MAX_PACKET_SIZE : InMaxPacket;
	PacketOverhead = 0;
	SetConnectionState(InState);

#if DO_ENABLE_NET_TEST
	// Copy the command line settings from the net driver
	UpdatePacketSimulationSettings();
#endif

	// Get the 
	if (InConnectionSpeed)
	{
		CurrentNetSpeed = InConnectionSpeed;
	}
	else
	{

		CurrentNetSpeed =  URL.HasOption(TEXT("LAN")) ? GetDefault<UPlayer>()->ConfiguredLanSpeed : GetDefault<UPlayer>()->ConfiguredInternetSpeed;
		if ( CurrentNetSpeed == 0 )
		{
			CurrentNetSpeed = 2600;
		}
		else
		{
			CurrentNetSpeed = FMath::Max<int32>(CurrentNetSpeed, 1800);
		}
	}

	FaultRecovery = MakeUnique<FNetConnectionFaultRecovery>();
	FaultRecovery->InitDefaults((Driver != nullptr ? Driver->GetNetDriverDefinition().ToString() : TEXT("")), this);

	// Create package map.
	auto PackageMapClient = NewObject<UPackageMapClient>(this);
	PackageMapClient->Initialize(this, Driver->GetNetGuidCache());
	PackageMap = PackageMapClient;

	UE_NET_TRACE_CONNECTION_CREATED(NetTraceId, GetConnectionHandle().GetParentConnectionId());
	UE_NET_TRACE_CONNECTION_STATE_UPDATED(NetTraceId, GetConnectionHandle().GetParentConnectionId(), static_cast<uint8>(GetConnectionState()));
}

void UNetConnection::InitHandler()
{
	using namespace UE::Net;

	LLM_SCOPE_BYTAG(NetConnection);

	check(!Handler.IsValid());

#if !UE_BUILD_SHIPPING
	if (!FParse::Param(FCommandLine::Get(), TEXT("NoPacketHandler")))
#endif
	{
		Handler = MakeUnique<PacketHandler>();

		if (Handler.IsValid())
		{
			UE::Handler::Mode Mode = Driver->ServerConnection != nullptr ? UE::Handler::Mode::Client : UE::Handler::Mode::Server;

			FPacketHandlerNotifyAddHandler NotifyAddHandler;
			
			NotifyAddHandler.BindLambda([this](TSharedPtr<HandlerComponent>& NewHandler)
				{
					if (NewHandler.IsValid())
					{
						NewHandler->InitFaultRecovery(GetFaultRecovery());
					}
				});

			Handler->InitializeDelegates(FPacketHandlerLowLevelSendTraits::CreateUObject(this, &UNetConnection::LowLevelSend),
											MoveTemp(NotifyAddHandler));

			Handler->NotifyAnalyticsProvider(Driver->AnalyticsProvider, Driver->AnalyticsAggregator);
			Handler->Initialize(Mode, MaxPacket * 8, false, nullptr, nullptr, Driver->GetNetDriverDefinition());


			// Add handling for the stateless connect handshake, for connectionless packets, as the outermost layer
			TSharedPtr<HandlerComponent> NewComponent =
				Handler->AddHandler(TEXT("Engine.EngineHandlerComponentFactory(StatelessConnectHandlerComponent)"), true);

			StatelessConnectComponent = StaticCastSharedPtr<StatelessConnectHandlerComponent>(NewComponent);

			if (StatelessConnectComponent.IsValid())
			{
				StatelessConnectHandlerComponent* CurComponent = StatelessConnectComponent.Pin().Get();
				
				CurComponent->SetDriver(Driver);
				CurComponent->SetHandshakeFailureCallback([this](FStatelessHandshakeFailureInfo FailureInfo){this->OnStatelessHandshakeFailure(FailureInfo);});
			}


			Handler->InitializeComponents();

			MaxPacketHandlerBits = Handler->GetTotalReservedPacketBits();
		}
	}


#if !UE_BUILD_SHIPPING
	uint32 MaxPacketBits = MaxPacket * 8;
	uint32 ReservedTotal = MaxPacketHandlerBits + MAX_PACKET_HEADER_BITS + MAX_PACKET_TRAILER_BITS;

	SET_DWORD_STAT(STAT_MaxPacket, MaxPacketBits);
	SET_DWORD_STAT(STAT_MaxPacketMinusReserved, MaxPacketBits - ReservedTotal);
	SET_DWORD_STAT(STAT_PacketReservedTotal, ReservedTotal);
	SET_DWORD_STAT(STAT_PacketReservedNetConnection, MAX_PACKET_HEADER_BITS + MAX_PACKET_TRAILER_BITS);
	SET_DWORD_STAT(STAT_PacketReservedPacketHandler, MaxPacketHandlerBits);
#endif
}

void UNetConnection::InitSequence(int32 IncomingSequence, int32 OutgoingSequence)
{
	checkf(InReliable.Num() > 0 && OutReliable.Num() > 0, TEXT("InitChannelData must be called prior to InitSequence."));

	// Make sure the sequence hasn't already been initialized on the server, and ignore multiple initializations on the client
	check(InPacketId == -1 || Driver->ServerConnection != nullptr);

	if (InPacketId == -1 && CVarRandomizeSequence.GetValueOnAnyThread() > 0)
	{
		// Initialize the base UNetConnection packet sequence (not very useful/effective at preventing attacks)
		InPacketId = IncomingSequence - 1;
		OutPacketId = OutgoingSequence;
		OutAckPacketId = OutgoingSequence - 1;
		LastNotifiedPacketId = OutAckPacketId;

		// Initialize the reliable packet sequence (more useful/effective at preventing attacks)
		InitInReliable = IncomingSequence & (MAX_CHSEQUENCE - 1);
		InitOutReliable = OutgoingSequence & (MAX_CHSEQUENCE - 1);

		InReliable.Init(InitInReliable, InReliable.Num());
		OutReliable.Init(InitOutReliable, OutReliable.Num());

		PacketNotify.Init(InPacketId, OutPacketId);

		UE_LOGF(LogNet, Verbose, "InitSequence: IncomingSequence: %i, OutgoingSequence: %i, InitInReliable: %i, InitOutReliable: %i", IncomingSequence, OutgoingSequence, InitInReliable, InitOutReliable);
	}
}

void UNetConnection::OnStatelessHandshakeFailure(UE::Net::FStatelessHandshakeFailureInfo HandshakeFailureInfo)
{
	using namespace UE::Net;

	if (HandshakeFailureInfo.FailureReason == EHandshakeFailureReason::WrongVersion)
	{
		const bool bUpgradeSuccess = HandleReceiveNetUpgrade(HandshakeFailureInfo.RemoteNetworkVersion, HandshakeFailureInfo.RemoteNetworkFeatures, ENetUpgradeSource::StatelessHandshake);

		if (bUpgradeSuccess)
		{
			// Restart the handshake since the network driver is now compatible
			StatelessConnectComponent.Pin().Get()->NotifyHandshakeBegin();
		}
	}
}

void UNetConnection::NotifyAnalyticsProvider()
{
	if (Handler.IsValid())
	{
		Handler->NotifyAnalyticsProvider(Driver->AnalyticsProvider, Driver->AnalyticsAggregator);
	}
}

void UNetConnection::NotifyConnectionUpdated()
{
	if (const uint32 MyConnectionId = GetConnectionHandle().GetParentConnectionId() && OwningActor && RemoteAddr)
	{
		UE_NET_TRACE_CONNECTION_UPDATED(NetTraceId, MyConnectionId, *(RemoteAddr->ToString(true)), *(OwningActor->GetName()));
	}
}

int32 UNetConnection::GetLastNotifiedPacketId() const
{
	return LastNotifiedPacketId;
}

AActor* UNetConnection::GetConnectionViewTarget() const
{
	AActor* TempViewTarget = PlayerController ? PlayerController->GetViewTarget() : nullptr;
	return (TempViewTarget && TempViewTarget->GetWorld()) ? TempViewTarget : ToRawPtr(OwningActor);
}

void UNetConnection::EnableEncryption(const FEncryptionData& EncryptionData)
{
	if (Handler.IsValid())
	{
		UE_LOGF(LogNet, Verbose, "UNetConnection::EnableEncryption, %ls", *Describe());

		TSharedPtr<FEncryptionComponent> EncryptionComponent = Handler->GetEncryptionComponent();
		if (EncryptionComponent.IsValid())
		{
			EncryptionComponent->SetEncryptionData(EncryptionData);
			EncryptionComponent->EnableEncryption();
		}
		else
		{
			UE_LOGF(LogNet, Warning, "UNetConnection::EnableEncryption, encryption component not found!");
		}
	}
}

void UNetConnection::EnableEncryptionServer(const FEncryptionData& EncryptionData)
{
	if (GetConnectionState() != USOCK_Invalid && GetConnectionState() != USOCK_Closed && Driver)
	{
		SendClientEncryptionAck();
		EnableEncryption(EncryptionData);
	}
	else
	{
		UE_LOGF(LogNet, Log, "UNetConnection::EnableEncryptionServer, connection in invalid state. %ls", *Describe());
	}
}

void UNetConnection::SendClientEncryptionAck()
{
	if (GetConnectionState() != USOCK_Invalid && GetConnectionState() != USOCK_Closed && Driver)
	{
		FNetControlMessage<NMT_EncryptionAck>::Send(this);
		FlushNet();
	}
	else
	{
		UE_LOGF(LogNet, Log, "UNetConnection::SendClientEncryptionAck, connection in invalid state. %ls", *Describe());
	}
}

void UNetConnection::SetEncryptionData(const FEncryptionData& EncryptionData)
{
	if (Handler.IsValid())
	{
		UE_LOGF(LogNet, Verbose, "UNetConnection::SetEncryptionData, %ls", *Describe());

		TSharedPtr<FEncryptionComponent> EncryptionComponent = Handler->GetEncryptionComponent();
		if (EncryptionComponent.IsValid())
		{
			EncryptionComponent->SetEncryptionData(EncryptionData);
		}
		else
		{
			UE_LOGF(LogNet, Warning, "UNetConnection::SetEncryptionData, encryption component not found!");
		}
	}
}

void UNetConnection::EnableEncryption()
{
	if (Handler.IsValid())
	{
		UE_LOGF(LogNet, Verbose, "UNetConnection::EnableEncryption, %ls", *Describe());

		TSharedPtr<FEncryptionComponent> EncryptionComponent = Handler->GetEncryptionComponent();
		if (EncryptionComponent.IsValid())
		{
			EncryptionComponent->EnableEncryption();
		}
		else
		{
			UE_LOGF(LogNet, Warning, "UNetConnection::EnableEncryption, encryption component not found!");
		}
	}
}

bool UNetConnection::IsEncryptionEnabled() const
{
	if (Handler.IsValid())
	{
		TSharedPtr<FEncryptionComponent> EncryptionComponent = Handler->GetEncryptionComponent();
		if (EncryptionComponent.IsValid())
		{
			return EncryptionComponent->IsEncryptionEnabled();
		}
	}

	return false;
}

void UNetConnection::Serialize( FArchive& Ar )
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UNetConnection::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("UObject", UObject::Serialize(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PackageName", Ar << PackageMap;);
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Channels",
		for (UChannel* Channel : Channels)
		{
			Ar << Channel;
		}
	);

	if (Ar.IsCountingMemory())
	{
		// TODO: We don't currently track:
		//		StatelessConnectComponents
		//		AnalyticsVars
		//		AnalyticsData
		//		Histogram data.
		// These are probably insignificant, though.

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Channels", Channels.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Challenge", Challenge.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ClientResponse", ClientResponse.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RequestURL", RequestURL.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SendBuffer", SendBuffer.CountMemory(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Channels", Channels.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("OutReliable", OutReliable.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("InReliable", InReliable.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingOutRec", PendingOutRec.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ActorChannels", ActorChannels.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DestroyedStartupOrDormantActorGUIDs", DestroyedStartupOrDormantActorGUIDs.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("KeepProcessingActorChannelBunchesMap",
			KeepProcessingActorChannelBunchesMap.CountBytes(Ar);
			for (const auto& KeepProcessingActorChannelBunchesPair : KeepProcessingActorChannelBunchesMap)
			{
				KeepProcessingActorChannelBunchesPair.Value.CountBytes(Ar);
			}
		);

		// ObjectReplicators are going to be counted by UNetDriver::Serialize AllOwnedReplicators.
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DormantReplicatorSet", DormantReplicatorSet.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ClientVisibleLevelNames", ClientVisibleLevelNames.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ClientVisibleActorOuters", ClientVisibleActorOuters.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ActorsStarvedByClassTimeMap",
			ActorsStarvedByClassTimeMap.CountBytes(Ar);
			for (auto& ActorsStarvedByClassTimePair : ActorsStarvedByClassTimeMap)
			{
				Ar << ActorsStarvedByClassTimePair.Key;
				ActorsStarvedByClassTimePair.Value.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("IgnoringChannels", IgnoringChannels.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("OutgoingBunches", OutgoingBunches.CountBytes(Ar));
		
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ChannelRecord",
			const SIZE_T SizeAllocatedByChannelRecord = FChannelRecordImpl::CountBytes(ChannelRecord);
			Ar.CountBytes(SizeAllocatedByChannelRecord, SizeAllocatedByChannelRecord)
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LastOut", LastOut.CountMemory(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SendBunchHeader", SendBunchHeader.CountMemory(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PacketHandler",
			if (Handler.IsValid())
			{
				// PacketHandler already counts its size.
				Handler->CountBytes(Ar);
			}
		);

#if DO_ENABLE_NET_TEST
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Delayed",
			Delayed.CountBytes(Ar);
			for (const FDelayedPacket& Packet : Delayed)
			{
				Packet.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DelayedIncoming", 
			DelayedIncomingPackets.CountBytes(Ar);
			for (const FDelayedIncomingPacket& DelayedPacket : DelayedIncomingPackets)
			{
				DelayedPacket.CountBytes(Ar);
			}
		);
#endif
	}
}

uint32 UNetConnection::GetClientHandshakeId() const
{
	return ClientHandshakeId;
}

void UNetConnection::SetClientHandshakeId(uint32 InClientHandshakeId)
{
	ClientHandshakeId = InClientHandshakeId;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EConnectionState const UNetConnection::GetConnectionState() const
{
	return State;
}

void UNetConnection::SetConnectionState(EConnectionState ConnectionState)
{
	State = ConnectionState;
	UE_NET_TRACE_CONNECTION_STATE_UPDATED(NetTraceId, GetConnectionHandle().GetParentConnectionId(), static_cast<uint8>(State));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const TCHAR* LexToString(const EConnectionState Value)
{
	switch (Value)
	{
	case EConnectionState::USOCK_Closed:
		return TEXT("Closed");
	case EConnectionState::USOCK_Open:
		return TEXT("Open");
	case EConnectionState::USOCK_Pending:
		return TEXT("Pending");
	case EConnectionState::USOCK_Invalid:
	default:
		return TEXT("Invalid");
	}
}

bool UNetConnection::IsClosingOrClosed() const
{
	const EConnectionState CurrentState = GetConnectionState();
	return CurrentState == USOCK_Closing || CurrentState == USOCK_Closed;
}

void UNetConnection::Close(FNetResult&& CloseReason)
{
	if (IsInternalAck())
	{
		SetReserveDestroyedChannels(false);
		SetIgnoreReservedChannels(false);
	}

	if (Driver != nullptr && GetConnectionState() != USOCK_Closed)
	{
		NETWORK_PROFILER(GNetworkProfiler.TrackEvent(TEXT("CLOSE"), *(GetName() + TEXT(" ") + LowLevelGetRemoteAddress()), this));

		UE_IRIS_PARALLEL_EXPR(UE::TScopeLock ScopeLock(GetSharedNetTickMutex()));

		TStringBuilder<2048> CloseStr;

		CloseStr.Append(TEXT("UNetConnection::Close: "));
		CloseStr.Append(ToCStr(Describe()));
		CloseStr.Appendf(TEXT(", Channels: %i, Time: "), OpenChannels.Num());
		CloseStr.Append(ToCStr(FDateTime::UtcNow().ToString(TEXT("%Y.%m.%d-%H.%M.%S"))));

		if (GNetCloseTimingDebug)
		{
			const double CurTime = FPlatformTime::Seconds();
			const double TimeSinceRecv = LastReceiveRealtime != 0.0 && InTotalPackets != 0 ? (CurTime - LastReceiveRealtime) : -1.0;
			const double TimeSinceSend = PreviousPacketSendTimeInS != 0.0 ? (CurTime - PreviousPacketSendTimeInS) : -1.0;
			const double TimeSinceTickFlush = LastTime != 0.0 ? (CurTime - LastTime) : -1.0;
			const double TimeSinceTickDispatch = CurTime - Driver->LastTickDispatchRealtime;

			CloseStr.Appendf(TEXT(", TimeSinceTickDispatch: %f, TimeSinceRecv: %f, TimeSinceTickFlush: %f, TimeSinceSend: %f"),
								TimeSinceTickDispatch, TimeSinceRecv, TimeSinceTickFlush, TimeSinceSend);
		}

		UE_LOGF(LogNet, Log, "%ls", ToCStr(CloseStr.ToString()));

		SendCloseReason(MoveTemp(CloseReason));

		if (Channels[0] != nullptr)
		{
			Channels[0]->Close(EChannelCloseReason::Destroyed);
		}

		SetConnectionState(EConnectionState::USOCK_Closed);

		const bool bReadyToSend = (Handler == nullptr || Handler->IsFullyInitialized()) && HasReceivedClientPacket();

		if (bReadyToSend)
		{
			FlushNet();
		}

		if (NetAnalyticsData.IsValid())
		{
			NetAnalyticsData->CommitAnalytics(AnalyticsVars);
		}

		if (RPCDoS.IsValid())
		{
			RPCDoS->NotifyClose();
		}

		NetPing.Reset();
	}

	LogCallLastTime		= 0;
	LogCallCount		= 0;
	LogSustainedCount	= 0;
}

void UNetConnection::GracefulClose(FNetResult&& CloseReason)
{
	bPendingDestroy = true;

	if (Channels[0] && GetConnectionState() != USOCK_Closed)
	{
		if (UE::Net::Connection::Private::bGracefulCloseEnabled && GetConnectionState() != USOCK_Closing)
		{
			UE_LOGF(LogNet, Log, "Initiating graceful close. CloseReason: %ls. %ls", ToCStr(CloseReason.DynamicToString(UE::Net::ENetResultString::WithChain)), *Describe());

			PendingGracefulCloseResult = MakePimpl<FNetResult, EPimplPtrMode::DeepCopy>(MoveTemp(CloseReason));

			SetConnectionState(EConnectionState::USOCK_Closing);
			GracefulCloseTimeoutDeadline = Driver->GetElapsedTime() + Driver->GracefulCloseConnectionTimeout;

			if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
			{
				ReplicationSystem->SetConnectionGracefullyClosing(ConnectionHandle.GetParentConnectionId());
			}

			// Might as well try to close immediately in case there isn't any pending reliable data.
			TryClosePendingGracefulClose();
		}
		else
		{
			Close(MoveTemp(CloseReason));
		}
	}
}

void UNetConnection::TryClosePendingGracefulClose()
{
	if (GetConnectionState() == USOCK_Closing)
	{
		bool bAllChannelsReadyToClose = true;

		for (TObjectPtr<class UChannel> Channel : OpenChannels)
		{
			if (Channel != nullptr && !Channel->HasAcknowledgedAllReliableData())
			{
				bAllChannelsReadyToClose = false;
				break;
			}
		}

		if (bAllChannelsReadyToClose)
		{
			UE_LOGF(LogNet, Log, "Graceful close complete, closing connection: %ls", *Describe());
			Close(MoveTemp(*PendingGracefulCloseResult));
		}
	}
}

void UNetConnection::HandleNetResultOrClose(ENetCloseResult InResult)
{
	using namespace UE::Net;

	const EHandleNetResult RecoveryResult = (FaultRecovery.IsValid() ? FaultRecovery->HandleNetResult(InResult) : EHandleNetResult::NotHandled);

	if (RecoveryResult == EHandleNetResult::NotHandled)
	{
		Close(InResult);
	}
}

void UNetConnection::SendCloseReason(FNetResult&& CloseReason)
{
	using namespace UE::Net;

	FString CloseReasonsStr;
	FNetCloseResult* CastedCloseReason = Cast<ENetCloseResult>(&CloseReason);

	if (CastedCloseReason == nullptr || *CastedCloseReason != ENetCloseResult::Unknown)
	{
		UE_LOGF(LogNet, Log, "UNetConnection::SendCloseReason:");

		for (FNetCloseResult::FConstIterator It(CloseReason); It; ++It)
		{
			if (CloseReasonsStr.Len() > 0)
			{
				CloseReasonsStr += TEXT(',');
			}

			CloseReasonsStr += It->DynamicToString(ENetResultString::ResultEnumOnly);

			UE_LOGF(LogNet, Log, " - %ls", ToCStr(It->DynamicToString()));
		}
	}

	const bool bReadyToSend = (Handler == nullptr || Handler->IsFullyInitialized()) && HasReceivedClientPacket();

	if (Channels.Num() > 0 && Channels[0] != nullptr && bReadyToSend && !CloseReasonsStr.IsEmpty())
	{
		FNetControlMessage<NMT_CloseReason>::Send(this, CloseReasonsStr);
	}

	if (NetAnalyticsData.IsValid())
	{
		if (!AnalyticsVars.CloseReason.IsValid())
		{
			AnalyticsVars.CloseReason = MakeUnique<FNetResult>(MoveTemp(CloseReason));
		}
		// Sometimes SendCloseReason is called separately to Close, both with the same CloseReason, when remote disconnect is anticipated
		else if (*AnalyticsVars.CloseReason != CloseReason)
		{
			AnalyticsVars.CloseReason->AddChainResult(MoveTemp(CloseReason));
		}
	}
}

void UNetConnection::HandleReceiveCloseReason(const FString& CloseReasonList)
{
	// Multiple NMT_CloseReason's can be received in some circumstances, but only process the first to limit spam potential.
	if (!bReceivedCloseReason)
	{
		bool bValid = true;
		auto IsAlnum_NoLocale = [](TCHAR Char) -> bool
			{
				return (Char >= TEXT('A') && Char <= TEXT('Z')) || (Char >= TEXT('a') && Char <= TEXT('z')) ||
						(Char >= TEXT('0') && Char <= TEXT('9'));
			};

		for (const TCHAR CurChar : CloseReasonList)
		{
			if (!IsAlnum_NoLocale(CurChar) && CurChar != TEXT('_') && CurChar != TEXT(','))
			{
				bValid = false;
				break;
			}
		}

		if (bValid)
		{
			const bool bRemoteIsServer = Driver && Driver->ServerConnection;
			TArray<FString> CloseReasonsAlloc;
			TArray<FString>& CloseReasonsArray = ((!bRemoteIsServer && NetAnalyticsData.IsValid()) ?
													AnalyticsVars.ClientCloseReasons : CloseReasonsAlloc);

			CloseReasonList.ParseIntoArray(CloseReasonsArray, TEXT(","));

			if (CloseReasonsArray.Num() < 128)
			{
				UE_LOGF(LogNet, Log, "NMT_CloseReason: (%ls Disconnect Reasons) %ls", (bRemoteIsServer ? TEXT("Server") : TEXT("Client")),
						ToCStr(LowLevelGetRemoteAddress(true)));

				for (const FString& CurReason : CloseReasonsArray)
				{
					UE_LOGF(LogNet, Log, " - %ls", ToCStr(CurReason));
				}
			}
			else
			{
				CloseReasonsArray.Empty();
			}

			bReceivedCloseReason = true;
		}
	}
}

bool UNetConnection::HandleReceiveNetUpgrade(uint32 RemoteNetworkVersion, EEngineNetworkRuntimeFeatures RemoteNetworkFeatures, UE::Net::ENetUpgradeSource NetUpgradeSource)
{
	bool bUpgradeSucceeded = false;
	const uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();

	const bool bIsNetCLValid = FNetworkVersion::IsNetworkCompatible(LocalNetworkVersion, RemoteNetworkVersion);

	// If the NetCL is not identical, we cannot upgrade and must disconnect
	if (!bIsNetCLValid || Driver == nullptr)
	{
		bUpgradeSucceeded = false;
	}
	else
	{
		Driver->TryUpgradeNetworkFeatures(RemoteNetworkFeatures);
		bUpgradeSucceeded = Driver->GetNetworkRuntimeFeatures() == RemoteNetworkFeatures;
	}

	if (!bUpgradeSucceeded)
	{
		const EEngineNetworkRuntimeFeatures LocalNetworkFeatures = Driver ? Driver->GetNetworkRuntimeFeatures() : EEngineNetworkRuntimeFeatures::None;

		TStringBuilder<128> LocalFeaturesDescription;
		TStringBuilder<128> RemoteFeaturesDescription;
		FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(LocalNetworkFeatures, LocalFeaturesDescription);
		FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(RemoteNetworkFeatures, RemoteFeaturesDescription);

		UE_LOGF(LogNet, Error, "Server is incompatible with the local version of the game: RemoteNetworkVersion=%u vs LocalNetworkVersion=%u, RemoteNetworkFeatures=%ls vs LocalNetworkFeatures=%ls",
			RemoteNetworkVersion, LocalNetworkVersion,
			RemoteFeaturesDescription.ToString(), LocalFeaturesDescription.ToString());

		const FString ConnectionError = NSLOCTEXT("Engine", "ClientOutdated", "The match you are trying to join is running an incompatible version of the game.  Please try upgrading your game version.").ToString();
		GEngine->BroadcastNetworkFailure(GetWorld(), Driver, ENetworkFailure::OutdatedClient, ConnectionError);

		if (NetUpgradeSource == UE::Net::ENetUpgradeSource::StatelessHandshake)
		{
			Close(ENetCloseResult::OutdatedClient);
		}
	}

	return bUpgradeSucceeded;
}

FString UNetConnection::Describe()
{
	// `LowLevelGetRemoteAddress` just returns a human readable description of the remote address
	// and will, on some implementations, call into system stuff to do that (think getting the IP
	// address or stuff like that). So we need to call this in the open as a result.
	FString RemoteAddress = AutoRTFM::Open([&]
		{
			return LowLevelGetRemoteAddress(true);
		});

	return FString::Printf( TEXT( "[UNetConnection] RemoteAddr: %s, Name: %s, Driver: %s, IsServer: %s, PC: %s, Owner: %s, UniqueId: %s" ),
			*RemoteAddress,
			*GetName(),
			Driver ? *Driver->GetDescription() : TEXT( "NULL" ),
			Driver && Driver->IsServer() ? TEXT( "YES" ) : TEXT( "NO" ),
			PlayerController ? *PlayerController->GetName() : TEXT( "NULL" ),
			OwningActor ? *OwningActor->GetName() : TEXT( "NULL" ),
			*PlayerId.ToDebugString());
}

void UNetConnection::CleanUp()
{
	// Remove UChildConnection(s)
	for (int32 i = 0; i < Children.Num(); i++)
	{
		Children[i]->CleanUp();
	}
	Children.Empty();

	const bool bWasSocketClosed = GetConnectionState() == USOCK_Closed;

	UE_CLOGF((GCVarLogAllConnectionCleanup || !bWasSocketClosed), LogNet, Log, "UNetConnection::Cleanup: Closing open connection. %ls", *Describe());

	Close(ENetCloseResult::Cleanup);

	if (Driver != nullptr)
	{
		// Remove from driver.
		if (Driver->ServerConnection)
		{
			check(Driver->ServerConnection == this);
			Driver->ServerConnection = NULL;
		}
		else
		{
			check(Driver->ServerConnection == NULL);
			Driver->RemoveClientConnection(this);

#if USE_SERVER_PERF_COUNTERS
			if (IPerfCountersModule::IsAvailable())
			{
				PerfCountersIncrement(TEXT("RemovedConnections"));
			}
#endif
		}
	}

	// Kill all channels.
	for (int32 i = OpenChannels.Num() - 1; i >= 0; i--)
	{
		UChannel* OpenChannel = OpenChannels[i];
		if (OpenChannel != NULL)
		{
			OpenChannel->ConditionalCleanUp(true, EChannelCloseReason::Destroyed);
		}
	}

	// Cleanup any straggler KeepProcessingActorChannelBunchesMap channels
	for (const auto& MapKeyValuePair : KeepProcessingActorChannelBunchesMap)
	{
		for (UActorChannel* CurChannel : MapKeyValuePair.Value)
		{
			CurChannel->ConditionalCleanUp(true, EChannelCloseReason::Destroyed);
		}
	}

	if (ConnectionHandle.IsParentConnection())
	{
		UE_NET_TRACE_CONNECTION_CLOSED(NetTraceId, ConnectionHandle.GetParentConnectionId());
	}

	if (Driver != nullptr)
	{
		// It would be nicer to have the Driver handle this internally, but unfortunately the ServerConnection member is public.
		// Otherwise we'd be able to do the appropriate logic in Add/Remove Client/ServerConnection
		if (ConnectionHandle.IsParentConnection())
		{
			if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
			{
				ReplicationSystem->RemoveConnection(ConnectionHandle.GetParentConnectionId());
			}
			Driver->FreeConnectionId(ConnectionHandle.GetParentConnectionId());
		}
	}

	KeepProcessingActorChannelBunchesMap.Empty();

	PackageMap = NULL;

	if (GIsRunning)
	{
		DestroyOwningActor();
	}

	CleanupDormantActorState();

	Handler.Reset();

	SetClientLoginState(EClientLoginState::CleanedUp);

	Driver = nullptr;
	MarkAsGarbage();

#if UE_NET_TRACE_ENABLED	
	UE_NET_TRACE_DESTROY_COLLECTOR(InTraceCollector);
	UE_NET_TRACE_DESTROY_COLLECTOR(OutTraceCollector);
	InTraceCollector = nullptr;
	OutTraceCollector = nullptr;
#endif

	if (CachedNoPawnPlayerController)
	{
		CachedNoPawnPlayerController->Destroy();
		CachedNoPawnPlayerController = nullptr;
	}
}

void UNetConnection::DestroyOwningActor()
{
	if (OwningActor != nullptr)
	{
		// Cleanup/Destroy the connection actor & controller
		if (!OwningActor->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			// UNetConnection::CleanUp can be called from UNetDriver::FinishDestroyed that is called from GC.
			OwningActor->OnNetCleanup(this);
		}
		OwningActor = nullptr;
		PlayerController = nullptr;
	}
	else
	{
		if (ClientLoginState < EClientLoginState::ReceivedJoin)
		{
			UE_LOGF(LogNet, Log, "UNetConnection::PendingConnectionLost. %ls bPendingDestroy=%d ", *Describe(), bPendingDestroy);
			FGameDelegates::Get().GetPendingConnectionLostDelegate().Broadcast(PlayerId);
		}
	}
}

UChildConnection::UChildConnection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UChildConnection::CleanUp()
{
	if (GIsRunning)
	{
		if (OwningActor != NULL)
		{
			if ( !OwningActor->HasAnyFlags( RF_BeginDestroyed | RF_FinishDestroyed ) )
			{
				// Cleanup/Destroy the connection actor & controller	
				OwningActor->OnNetCleanup(this);
			}

			OwningActor = NULL;
			PlayerController = NULL;
		}
	}
	PackageMap = NULL;
	Driver = NULL;
}

void UChildConnection::RemoveLocalPlayer(UGameInstance* GameInstance, ULocalPlayer* LocalPlayer)
{
	GameInstance->RemoveLocalPlayer(LocalPlayer);
}

void UChildConnection::CloseAndRemoveChild(UE::Net::ECloseChildFlags Flags)
{
	UE_LOGF(LogNet, Log, "[%ls:%ls] Closing child connection %ls.", *GetNameSafe(GetDriver()), *GetNameSafe(Parent), *GetName());
	
	if (EnumHasAnyFlags(Flags, UE::Net::ECloseChildFlags::SendCloseMessage))
	{
		if (PlayerController)
		{
			int32 NetPlayerIndexParam = static_cast<int32>(PlayerController->NetPlayerIndex);
			FNetControlMessage<NMT_CloseChildConnection>::Send(Parent, NetPlayerIndexParam);

			UE_LOGF(LogNet, Log, "[%ls:%ls] Sending NMT_CloseChildConnection with NetPlayerIndex %d.", 
				   *GetNameSafe(GetDriver()), *GetNameSafe(Parent), NetPlayerIndexParam);
		}
		else
		{
			UE_LOGF(LogNet, Warning, "[%ls:%ls] Cannot send NMT_CloseChildConnection because there's no valid player controller.", 
				   *GetNameSafe(GetDriver()), *GetNameSafe(Parent));
		}
	}

	if (PlayerController)
	{
		if (ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer())
		{
			if (UWorld* World = GetWorld())
			{
				if (UGameInstance* GameInstance = World->GetGameInstance())
				{
					// A local player may be removed from the game before the proxy driver is cleaned up in the case where the
					// game is shutting down or closing.
					const bool bIsRegisteredLocalPlayer = GameInstance->GetLocalPlayers().Find(LocalPlayer) != INDEX_NONE;

					if (bIsRegisteredLocalPlayer)
					{
						UE_LOGF(LogNet, Log, "[%ls:%ls] Remove local player %ls.", *GetNameSafe(GetDriver()), *GetNameSafe(Parent), *GetNameSafe(LocalPlayer));

						RemoveLocalPlayer(GameInstance, LocalPlayer);
					}
				}
			}
		}
	}

	if (Parent)
	{
		Parent->RemoveChild(this);
		
		if (EnumHasAnyFlags(Flags, UE::Net::ECloseChildFlags::DestroyActors))
		{
			CleanUp();
		}
	}
}

void UChildConnection::InitChildConnection(UNetDriver* InDriver, UNetConnection* InParent)
{
	Driver = InDriver;
	URL = FURL();
	SetConnectionState(InParent->GetConnectionState());
	URL.Host = InParent->URL.Host;
	Parent = InParent;
	PackageMap = InParent->PackageMap;
	CurrentNetSpeed = InParent->CurrentNetSpeed;

	InitChannelData();

	AssignConnectionHandle();
}

void UChildConnection::SetProxyConnection(bool bEnabled)
{
	UE_LOGF(LogNet, Error, "Cannot set the child connection %ls to be a proxy connection.", *GetName());
}

bool UChildConnection::IsProxyConnection() const
{
	return Parent->IsProxyConnection();
}

void UChildConnection::AssignConnectionHandle()
{
	if (const UNetConnection* ParentConn = Parent.Get())
	{
		// Don't assume the children remain on a particular index or that the array isn't ever shrinking. This code will assign as low an ID as possible.
		const uint32 MaxNeededChildId = ParentConn->Children.Num() + 1;
		TBitArray<> UsedChildIds;
		UsedChildIds.SetNum(static_cast<int32>(MaxNeededChildId) + 1, false);
		constexpr uint32 InvalidChildId = 0;
		UsedChildIds[InvalidChildId] = true;
		for (const UChildConnection* ChildConn : ParentConn->Children)
		{
			const uint32 ChildId = Cast<UNetConnection>(ChildConn)->GetConnectionHandle().GetChildConnectionId();
			// If the ChildId is outside our small range we don't need to know it's used and the bitarray isn't large enough to store the information.
			if (ChildId <= MaxNeededChildId)
			{
				UsedChildIds[ChildId] = true;
			}
		}

		// There will be at least one unused child ID.
		int32 ChildId = UsedChildIds.Find(false);
		if (ChildId == INDEX_NONE)
		{
			ensureMsgf(false, TEXT("Logic failed assigning child connection ID."));
			ChildId = InvalidChildId;
		}
		
		SetConnectionHandle(UE::Net::FConnectionHandle(ParentConn->GetConnectionHandle().GetParentConnectionId(), static_cast<uint32>(ChildId)));
	}
}

void UChildConnection::GracefulClose(FNetCloseResult&& CloseReason)
{
	if (Parent)
	{
		Parent->GracefulClose(MoveTemp(CloseReason));
	}
}

void UNetConnection::FinishDestroy()
{
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		CleanUp();
	}

	Super::FinishDestroy();
}

void UNetConnection::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UNetConnection* This = CastChecked<UNetConnection>(InThis);

	// Let GC know that we're referencing some UChannel objects
	for (auto& Channel : This->Channels)
	{
		Collector.AddReferencedObject( Channel, This );
	}

	// Let GC know that we're referencing some UActorChannel objects
	for ( auto It = This->KeepProcessingActorChannelBunchesMap.CreateIterator(); It; ++It )
	{
		auto& ChannelArray = It.Value();
		for ( auto& CurChannel : ChannelArray )
		{
			Collector.AddReferencedObject( CurChannel, This );
		}
	}

	Super::AddReferencedObjects(This, Collector);
}

UWorld* UNetConnection::GetWorld() const
{
	UWorld* World = nullptr;
	if (Driver)
	{
		World = Driver->GetWorld();
	}

	if (!World && OwningActor)
	{
		World = OwningActor->GetWorld();
	}

	return World;
}

#if UE_ALLOW_EXEC_COMMANDS
bool UNetConnection::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	if ( Super::Exec( InWorld, Cmd,Ar) )
	{
		return true;
	}
	else if ( GEngine->Exec( InWorld, Cmd, Ar ) )
	{
		return true;
	}
	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

void UNetConnection::AssertValid()
{
	// Make sure this connection is in a reasonable state.
	check(IsClosingOrClosed() || GetConnectionState() == USOCK_Pending || GetConnectionState() == USOCK_Open);

}

FNetLevelVisibilityTransactionId UNetConnection::UpdateLevelStreamStatusChangedTransactionId(const ULevelStreaming* LevelObject, const FName PackageName, bool bShouldBeVisible)
{
	// Increment transactionId
	FNetLevelVisibilityTransactionId& TransactionId = ClientPendingStreamingStatusRequest.FindOrAdd(PackageName, FNetLevelVisibilityTransactionId());
	TransactionId.IncrementTransactionIndex();

	// Disable replication of actors on the invisible level
	// We want to do this now to avoid having data in flight that the client will not be able to receive.
	if (bShouldBeVisible == false)
	{
		ClientVisibleLevelNames.Remove(PackageName);
		ClientMakingVisibleLevelNames.Remove(PackageName);
		if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
		{
			if (const ULevel* Level = LevelObject->GetLoadedLevel())
			{
				const UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridge();
				if (UE::Net::FNetObjectGroupHandle GroupHandle = ReplicationBridge->GetContainerGroup(Level); GroupHandle.IsValid())
				{
					ReplicationSystem->SetGroupFilterStatus(GroupHandle, GetConnectionHandle().GetParentConnectionId(), UE::Net::ENetFilterStatus::Disallow);
				}
			}
		}
		else
		{
			UpdateCachedLevelVisibility(PackageName);
		}
	}

	return TransactionId;
}

bool UNetConnection::ClientHasInitializedLevel(const ULevel* TestLevel) const
{
	checkSlow(Driver);
	checkSlow(Driver->IsServer());

	// This function is called a lot, basically for every replicated actor every time it replicates, on every client connection
	// Each client connection has a different visibility state (what levels are currently loaded for them).

	const FName PackageName = TestLevel->GetPackage()->GetFName();

	if (const bool* bIsVisible = ClientVisibleActorOuters.Find(PackageName))
	{
		return *bIsVisible;
	}

	// The level was not in the acceleration map so we perform the "legacy" function and 
	// cache the result so that we don't do this every time:
	return UpdateCachedLevelVisibility(PackageName);
}

bool UNetConnection::ClientHasInitializedLevelFor(const AActor* TestActor) const
{
	return ClientHasInitializedLevel(TestActor->GetLevel());
}

bool UNetConnection::UpdateCachedLevelVisibility(const FName& PackageName) const
{
	bool bVisible = false;
	
	if (PackageName.IsNone())
	{
		bVisible = true;
	}
	else if ((PackageName == ClientWorldPackageName) && (Driver->GetWorldPackage()->GetFName() == ClientWorldPackageName))
	{
		bVisible = true;
	}
	else
	{
		bVisible = ClientVisibleLevelNames.Contains(PackageName);
	}

	ClientVisibleActorOuters.FindOrAdd(PackageName) = bVisible;

	return bVisible;
}

void UNetConnection::UpdateAllCachedLevelVisibility() const
{
	// Update our acceleration map
	for (const TPair<FName, bool>& VisPair : ClientVisibleActorOuters)
	{
		UpdateCachedLevelVisibility(VisPair.Key);
	}
}

void UNetConnection::UpdateLevelVisibility(const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
{
	if (Driver && Driver->GetWorld())
	{
		// If we are doing seamless travel we need to defer visibility updates until after the server has completed loading the level
		// otherwise we might end up in a situation where visibility is not correctly updated
		if (Driver->GetWorld()->IsInSeamlessTravel())
		{
			PendingUpdateLevelVisibility.FindOrAdd(LevelVisibility.PackageName) = LevelVisibility;
		}
	}
	UpdateLevelVisibilityInternal(LevelVisibility);

	NotifyConnectionUpdated();
}

void UNetConnection::UpdateLevelVisibilityInternal(const FUpdateLevelVisibilityLevelInfo& LevelVisibility)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetConnection_UpdateLevelVisibilityInternal)

	using namespace UE::Net::Connection::Private;

	GNumClientUpdateLevelVisibility++;

	bool bVerifiedTransaction = true;
	if (ULevelStreaming::ShouldServerUseMakingVisibleTransactionRequest())
	{
		// If we have a valid server instigated VisibilityRequest we verify it against our pending transactions before treating the level as visible
		// This is to avoid issues with multiple possibly conflicting requests in flight.
		const FNetLevelVisibilityTransactionId VisibilityRequestId = LevelVisibility.VisibilityRequestId;
		if (VisibilityRequestId.IsValid() && !VisibilityRequestId.IsClientTransaction())
		{
			if (FNetLevelVisibilityTransactionId* PendingTransactionIndex = ClientPendingStreamingStatusRequest.Find(LevelVisibility.PackageName))
			{
				bVerifiedTransaction = VisibilityRequestId == *PendingTransactionIndex;
			}
		}
	}

	// add or remove the level package name from the list, as requested
	if (LevelVisibility.bIsVisible)
	{
		// verify that we were passed a valid level name
		FValidateLevelVisibilityResult VisibilityResult = ValidateLevelVisibility(GetWorld(), LevelVisibility);

		if (bVerifiedTransaction && VisibilityResult.bLevelExists)
		{
			// Verify that server knows this package
			UE_CLOGF(!FLevelUtils::IsServerStreamingLevelVisible(GetWorld(), LevelVisibility.PackageName), LogPlayerController, Warning, "ServerUpdateLevelVisibility() Added '%ls', but level is not visible on server.", *LevelVisibility.PackageName.ToString());
			
			ClientMakingVisibleLevelNames.Remove(LevelVisibility.PackageName);
			ClientVisibleLevelNames.Add(LevelVisibility.PackageName);
			UE_LOGF(LogPlayerController, Verbose, "ServerUpdateLevelVisibility() Added '%ls'", *LevelVisibility.PackageName.ToString());

			UpdateCachedLevelVisibility(LevelVisibility.PackageName);

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_NetUpdateLevelVisibility_DestructionInfos);

				// Any destroyed actors that were destroyed prior to the streaming level being unloaded for the client will not be in the connections
				// destroyed actors list when the level is reloaded, so seek them out and add in
				const TSet<FNetworkGUID>& DestructionInfoGuids = Driver->GetDestroyedStartupOrDormantActors(LevelVisibility.PackageName);
				
				for (const FNetworkGUID& DestroyedGuid : DestructionInfoGuids)
				{
					AddDestructionInfo(Driver->DestroyedStartupOrDormantActors[DestroyedGuid].Get());
				}
			}

			QUICK_SCOPE_CYCLE_COUNTER(STAT_NetUpdateLevelVisibility_UpdateDormantActors);

			// Any dormant actor that has changes flushed or made before going dormant needs to be updated on the client 
			// when the streaming level is loaded, so mark them active for this connection
			UWorld* LevelWorld = nullptr;

			if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
			{
				// We need to be careful here if the server would load a level after the client, 
				// for that to work we would need to update group filter status after level is loaded
				const ULevel* Level = VisibilityResult.StreamingLevel ? VisibilityResult.StreamingLevel->GetLoadedLevel() : nullptr;
				if (Level)
				{
					const UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridge();
					if (UE::Net::FNetObjectGroupHandle GroupHandle = ReplicationBridge->GetContainerGroup(Level); GroupHandle.IsValid())
					{
						ReplicationSystem->SetGroupFilterStatus(GroupHandle, GetConnectionHandle().GetParentConnectionId(), UE::Net::ENetFilterStatus::Allow);
					}
				}
			}
			else 
			{
				LevelWorld = VisibilityResult.Package ? (UWorld*)FindObjectWithOuter(VisibilityResult.Package, UWorld::StaticClass()) : nullptr;

				FNetworkObjectList& NetworkObjectList = Driver->GetNetworkObjectList();

				if (UE::Net::Private::bTrackDormantObjectsByLevel)
				{
					NetworkObjectList.FlushDormantActors(this, LevelVisibility.PackageName);
				}
				else
				{
					if (LevelWorld && LevelWorld->PersistentLevel)
					{
						for (AActor* Actor : LevelWorld->PersistentLevel->Actors)
						{
							// Dormant Initial actors have no changes. Dormant Never and Awake will be sent normal, so we only need
							// to mark Dormant All Actors as (temporarily) active to get the update sent over
							if (Actor && Actor->GetIsReplicated() && (Actor->NetDormancy == DORM_DormantAll))
							{
								NetworkObjectList.MarkActive(Actor, this, Driver);
							}
						}
					}
				}
			}

			if (ReplicationConnectionDriver)
			{
				ReplicationConnectionDriver->NotifyClientVisibleLevelNamesAdd(LevelVisibility.PackageName, LevelWorld);
			}
		}
		else if (!VisibilityResult.bLevelExists)
		{
			FString PackageNameStr = LevelVisibility.PackageName.ToString();
			FString FileNameStr = LevelVisibility.FileName.ToString();

			UE_LOGF(LogPlayerController, Warning, "ServerUpdateLevelVisibility() ignored non-existant package. PackageName='%ls', FileName='%ls'",
					ToCStr(PackageNameStr), ToCStr(FileNameStr));

			// Added CVAR to address #sh-06-13-desert-biome-server-kicks-ext. Should be removed once a proper solution is found
			if (!CVarSkipMissingLevelDisconnect.GetValueOnAnyThread() && !LevelVisibility.bSkipCloseOnError)
			{
				TStringBuilder<1024> VisibilityParms;

				VisibilityParms.Append(PackageNameStr);
				VisibilityParms.AppendChar(TEXT(','));
				VisibilityParms.Append(FileNameStr);

				Close({ENetCloseResult::MissingLevelPackage, VisibilityParms.ToString()});
			}
		}
	}
	else if (LevelVisibility.bTryMakeVisible)
	{
		if (FLevelUtils::SupportsMakingVisibleTransactionRequests(GetWorld()))
		{
			const FNetLevelVisibilityTransactionId VisibilityRequestId = LevelVisibility.VisibilityRequestId;
			check(VisibilityRequestId.IsValid() && VisibilityRequestId.IsClientTransaction());

			// Only consider visible levels with their streaming level in the LoadedVisible state and returning ShouldBeVisible()
			ULevelStreaming* ServerVisibleStreamingLevel = FLevelUtils::GetServerVisibleStreamingLevel(GetWorld(), LevelVisibility.PackageName);
			if (ServerVisibleStreamingLevel && ServerVisibleStreamingLevel->ShouldBeVisible() && (ServerVisibleStreamingLevel->GetLevelStreamingState() == ELevelStreamingState::LoadedVisible))
			{
				ClientMakingVisibleLevelNames.Add(LevelVisibility.PackageName);
				check(!ClientVisibleLevelNames.Contains(LevelVisibility.PackageName));
			}
		}
	}
	else
	{
		ClientMakingVisibleLevelNames.Remove(LevelVisibility.PackageName);
		ClientVisibleLevelNames.Remove(LevelVisibility.PackageName);
		UE_LOGF(LogPlayerController, Verbose, "ServerUpdateLevelVisibility() Removed '%ls'", *LevelVisibility.PackageName.ToString());
		if (ReplicationConnectionDriver)
		{
			ReplicationConnectionDriver->NotifyClientVisibleLevelNamesRemove(LevelVisibility.PackageName);
		}

		if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
		{
			// See if the level is loaded, if it is disable filtering, if not filter should already be removed
			if (const ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(GetWorld(), LevelVisibility.PackageName))
			{
				if (const ULevel* Level = StreamingLevel->GetLoadedLevel())
				{
					// $IRIS: $TODO: Implement forced scope-update for a group/connection to ensure that we treat replicated objects as destroyed if the filter status of a level is disabled/enabled on the same frame
					// The reason for this is that the client currently destroys the instances rather then managing this through the replication system
					// If we implement a way to re-instantiate instances on the client we might be able to persist the state
					const UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridge();
					if (UE::Net::FNetObjectGroupHandle GroupHandle = ReplicationBridge->GetContainerGroup(Level); GroupHandle.IsValid())
					{
						ReplicationSystem->SetGroupFilterStatus(GroupHandle, GetConnectionHandle().GetParentConnectionId(), UE::Net::ENetFilterStatus::Disallow);
					}
				}
			}
		}

		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetUpdateLevelVisibility_LevelUnloadChannelClose);

		// Close any channels now that have actors that were apart of the level the client just unloaded
		for (auto It = ActorChannels.CreateIterator(); It; ++It)
		{
			UActorChannel* Channel = It.Value();

			check(Channel->OpenedLocally);

			if (Channel->Actor && Channel->Actor->GetLevel() && Channel->Actor->GetLevel()->GetOutermost()->GetFName() == LevelVisibility.PackageName)
			{
				It.RemoveCurrent(); // Remove safely here since Channel->Close will remove unsafely
				Channel->Close(EChannelCloseReason::LevelUnloaded);
			}
		}

		// If the server is not sending override levels to clients, clients won't have another way to destroy spawned,
		// dormant actors in streaming levels that go invisible. Send them a destruction info so that they clean it up.
		if (!UE::Net::Private::SerializeNewActorOverrideLevel)
		{
			const ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(GetWorld(), LevelVisibility.PackageName);
			const ULevel* Level = StreamingLevel ? StreamingLevel->GetLoadedLevel() : nullptr;

			if (Level)
			{
				for (AActor* ThisActor : Level->Actors)
				{
					// Only do this for spawned actors
					if (ThisActor && !ThisActor->IsNetStartupActor())
					{
						Driver->SendDestructionInfoForLevelUnloadIfDormant(ThisActor, this);
					}
				}
			}
		}

		UpdateCachedLevelVisibility(LevelVisibility.PackageName);
	}
}

void UNetConnection::UpdateContainerVisibility(UObject* Container, bool bVisible)
{
	UpdateContainerVisibilityInternal(Container, bVisible);
}

void UNetConnection::UpdateContainerVisibilityInternal(UObject* Container, bool bVisible)
{
	if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
	{
		const UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridge();
		UE::Net::FNetObjectGroupHandle ContainerGroupHandle = ReplicationBridge->GetContainerGroup(Container);
		if (ContainerGroupHandle.IsValid())
		{
			ReplicationSystem->SetGroupFilterStatus(ContainerGroupHandle, GetConnectionHandle().GetParentConnectionId(), bVisible ? UE::Net::ENetFilterStatus::Allow : UE::Net::ENetFilterStatus::Disallow);
		}
	}
}

void UNetConnection::SetClientWorldPackageName(FName NewClientWorldPackageName)
{
	ClientWorldPackageName = NewClientWorldPackageName;
	
	if (Driver->IsUsingIrisReplication())
	{
		Driver->UpdateGroupFilterStatusForAllLevelsForConnection(GetConnectionHandle());
	}
	else
	{
		UpdateAllCachedLevelVisibility();
	}
}

void UNetConnection::ValidateSendBuffer()
{
	if ( SendBuffer.IsError() )
	{
		UE_LOG( LogNetTraffic, Fatal, TEXT( "UNetConnection::ValidateSendBuffer: Out.IsError() == true. NumBits: %" INT64_FMT ", NumBytes: %" INT64_FMT ", MaxBits: %" INT64_FMT ), SendBuffer.GetNumBits(), SendBuffer.GetNumBytes(), SendBuffer.GetMaxBits() );
	}
}

void UNetConnection::InitSendBuffer()
{
	check(MaxPacket > 0);

	int32 FinalBufferSize = (MaxPacket * 8) - MaxPacketHandlerBits;

	// Initialize the one outgoing buffer.
	if (FinalBufferSize == SendBuffer.GetMaxBits())
	{
		// Reset all of our values to their initial state without a malloc/free
		SendBuffer.Reset();
	}
	else
	{
		// First time initialization needs to allocate the buffer
		SendBuffer = FBitWriter(FinalBufferSize);
	}

	HeaderMarkForPacketInfo.Reset();

	ResetPacketBitCounts();

	ValidateSendBuffer();
}

void UNetConnection::ReceivedRawPacket( void* InData, int32 Count )
{
	using namespace UE::Net;

#if !UE_BUILD_SHIPPING
	// Add an opportunity for the hook to block further processing
	bool bBlockReceive = false;

	ReceivedRawPacketDel.ExecuteIfBound(InData, Count, bBlockReceive);

	if (bBlockReceive)
	{
		return;
	}
#endif

#if DO_ENABLE_NET_TEST
	// Opportunity for packet loss burst simulation to drop the incoming packet.
	if (Driver && Driver->IsSimulatingPacketLossBurst())
	{
		return;
	}
#endif

	uint8* Data = (uint8*)InData;

	++InTotalHandlerPackets;

	if (Handler.IsValid())
	{
		FReceivedPacketView PacketView;

		PacketView.DataView = {Data, Count, ECountUnits::Bytes};

		EIncomingResult IncomingResult = Handler->Incoming(PacketView);

		if (IncomingResult == EIncomingResult::Success)
		{
			Count = PacketView.DataView.NumBytes();

			if (Count > 0)
			{
				Data = PacketView.DataView.GetMutableData();
			}
			// This packed has been consumed
			else
			{
				return;
			}
		}
		else
		{
			UE_LOGF(LogNet, Warning, "Packet failed PacketHandler processing.");


			FInPacketTraits& Traits = PacketView.Traits;
			const bool bErrorNotRecoverable = !Traits.ExtendedError.IsValid() ||
												Traits.ExtendedError->HasChainResult(ENetCloseResult::NotRecoverable);
			bool bCloseConnection = bErrorNotRecoverable;

			if (!bErrorNotRecoverable)
			{
				const EHandleNetResult RecoveryResult = (FaultRecovery.IsValid() ?
					FaultRecovery->FaultManager.HandleNetResult(MoveTemp(*Traits.ExtendedError)) :
					EHandleNetResult::NotHandled);

				bCloseConnection = RecoveryResult == EHandleNetResult::NotHandled;
			}

			if (bCloseConnection)
			{
				Close(AddToAndConsumeChainResultPtr(Traits.ExtendedError, ENetCloseResult::PacketHandlerIncomingError));
			}

			return;
		}
		
		// See if we receive a packet that wasn't fully consumed by the handler before the handler is initialized.
		if (!Handler->IsFullyInitialized())
		{
			UE_LOGF(LogNet, Warning, "PacketHander isn't fully initialized and also didn't fully consume a packet! This will cause the connection to try to send a packet before the initial packet sequence has been established. Ignoring. Connection: %ls", *Describe());
			return;
		}
	}


	// Handle an incoming raw packet from the driver.
	UE_LOGF(LogNetTraffic, Verbose, "%6.3f: Received %i", FPlatformTime::Seconds() - GStartTime, Count );
	int32 PacketBytes = Count + PacketOverhead;
	InBytes += PacketBytes;
	InTotalBytes += PacketBytes;
	++InPackets;
	++InPacketsThisFrame;
	++InTotalPackets;

	if (Driver)
	{
		Driver->InBytes += PacketBytes;
		Driver->InTotalBytes += PacketBytes;
		Driver->InPackets++;
		Driver->InTotalPackets++;
	}

	if (Count > 0)
	{
		uint8 LastByte = Data[Count-1];

		if (LastByte != 0)
		{
			int32 BitSize = (Count * 8) - 1;

			// Bit streaming, starts at the Least Significant Bit, and ends at the MSB.
			while (!(LastByte & 0x80))
			{
				LastByte *= 2;
				BitSize--;
			}


			FBitReader Reader(Data, BitSize);

			// Set the network version on the reader
			SetNetVersionsOnArchive(Reader);

			if (Handler.IsValid())
			{
				Handler->IncomingHigh(Reader);
			}

			if (Reader.GetBitsLeft() > 0)
			{
				ReceivedPacket(Reader);

				// Check if the out of order packet cache needs flushing
				FlushPacketOrderCache();
			}
		}
		// MalformedPacket - Received a packet with 0's in the last byte
		else
		{
			UE_LOGF(LogNet, Warning, "Received packet with 0's in last byte of packet");

			HandleNetResultOrClose(ENetCloseResult::ZeroLastByte);
		}
	}
	// MalformedPacket - Received a packet of 0 bytes
	else 
	{
		UE_LOGF(LogNet, Warning, "Received zero-size packet");

		HandleNetResultOrClose(ENetCloseResult::ZeroSize);
	}
}

void UNetConnection::PreTickDispatch()
{
	const bool bIsServer = Driver->ServerConnection == nullptr;

	if (!IsReplay())
	{
		const double LastTickDispatchRealtime = Driver->LastTickDispatchRealtime;

		if (bIsServer && RPCDoS.IsValid())
		{
			RPCDoS->PreTickDispatch(LastTickDispatchRealtime);
		}

		if (FaultRecovery.IsValid() && FaultRecovery->DoesRequireTick())
		{
			FaultRecovery->TickRealtime(LastTickDispatchRealtime);
		}
	}
}

void UNetConnection::PostTickDispatch()
{
	const bool bIsServer = Driver != nullptr && Driver->ServerConnection == nullptr;

	if (!IsInternalAck())
	{
#if DO_ENABLE_NET_TEST
		ReinjectDelayedPackets();
#endif

		FlushPacketOrderCache(/*bFlushWholeCache=*/true);
		PacketAnalytics.Tick();

		// Make sure to clear the AckPacketId for the next frame
		OutFirstAckPacketId = -1;
	}

	if (RPCDoS.IsValid() && bIsServer && !IsReplay())
	{
		RPCDoS->PostTickDispatch();
	}
}

void UNetConnection::FlushPacketOrderCache(bool bFlushWholeCache/*=false*/)
{
	if (PacketOrderCache.IsSet() && PacketOrderCacheCount > 0)
	{
		TCircularBuffer<TUniquePtr<FBitReader>>& Cache = PacketOrderCache.GetValue();
		int32 CacheEndIdx = PacketOrderCache->GetPreviousIndex(PacketOrderCacheStartIdx);
		bool bEndOfCacheSet = Cache[CacheEndIdx].IsValid();

		bFlushingPacketOrderCache = true;

		// If the end of the cache has had its value set, this forces the flushing of the whole cache, no matter how many missing sequences there are.
		// The reason for this (other than making space in the cache), is that when we receive a sequence that is out of range of the cache,
		// it is stored at the end, and so the cache index no longer lines up with the sequence number - which it needs to.
		bFlushWholeCache = bFlushWholeCache || bEndOfCacheSet;

		while (PacketOrderCacheCount > 0)
		{
			TUniquePtr<FBitReader>& CurCachePacket = Cache[PacketOrderCacheStartIdx];

			if (CurCachePacket.IsValid())
			{
				UE_LOGF(LogNet, VeryVerbose, "'Out of Order' Packet Cache, replaying packet with cache index: %i (bFlushWholeCache: %i)", PacketOrderCacheStartIdx, (int32)bFlushWholeCache);

#if DO_ENABLE_NET_TEST
				// Packets in the packet order cache have already had packet simulation applied. Make sure it's not applied again.
				TGuardValue<bool> ReinjectingGuard(bIsReinjectingDelayedPackets, true);
#endif

				constexpr bool bIsReinjectedPacket = true;
				ReceivedPacket(*CurCachePacket.Get(), bIsReinjectedPacket);

				CurCachePacket.Reset();

				PacketOrderCacheCount--;
			}
			// Advance the cache only up to the first missing packet, unless flushing the whole cache
			else if (!bFlushWholeCache)
			{
				break;
			}

			PacketOrderCacheStartIdx = PacketOrderCache->GetNextIndex(PacketOrderCacheStartIdx);
		}

		bFlushingPacketOrderCache = false;
	}
}

#if DO_ENABLE_NET_TEST
void UNetConnection::ReinjectDelayedPackets()
{
	if (DelayedIncomingPackets.Num() > 0)
	{
		const double CurrentTime = FPlatformTime::Seconds();

		TGuardValue<bool> ReinjectingGuard(bIsReinjectingDelayedPackets, true);

		uint32 NbReinjected(0);
		for (FDelayedIncomingPacket& DelayedPacket : DelayedIncomingPackets)
		{
			if (DelayedPacket.ReinjectionFrameCount > 0)
			{
				if (DelayedPacket.ReinjectionFrameCount > 1)
				{
					DelayedPacket.ReinjectionFrameCount--;
					break;
				}
			}			
			else if (DelayedPacket.ReinjectionTime > CurrentTime)
			{
				break;
			}

			++NbReinjected;
			ReceivedPacket(*DelayedPacket.PacketData.Get());
		}

		// Delete processed packets
		DelayedIncomingPackets.RemoveAt(0, NbReinjected, EAllowShrinking::No);
	}
}
#endif //#if DO_ENABLE_NET_TEST

uint32 GNetOutBytes = 0;

void UNetConnection::FlushNet(bool bIgnoreSimulation)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FlushNet);

	check(Driver);

	// Update info.
	ValidateSendBuffer();
	LastEnd = FBitWriterMark();
	TimeSensitive = 0;

	// If there is any pending data to send, send it.
	if (SendBuffer.GetNumBits() || HasDirtyAcks || ( Driver->GetElapsedTime() - LastSendTime > Driver->KeepAliveTime && !IsInternalAck() && GetConnectionState() != USOCK_Closed))
	{
		// Due to the PacketHandler handshake code, servers must never send the client data,
		// before first receiving a client control packet (which is taken as an indication of a complete handshake).
		if (!HasReceivedClientPacket() && CVarRandomizeSequence.GetValueOnAnyThread() != 0)
		{
			UE_LOGF(LogNet, Log, "Attempting to send data before handshake is complete. %ls", *Describe());

			Close(ENetCloseResult::PrematureSend);
			InitSendBuffer();

			return;
		}


		FOutPacketTraits Traits;

		// If sending keepalive packet or just acks, still write the packet header
		if (SendBuffer.GetNumBits() == 0)
		{
			WriteBitsToSendBuffer( NULL, 0 );		// This will force the packet header to be written

			Traits.bIsKeepAlive = true;
			AnalyticsVars.OutKeepAliveCount++;
		}


		// @todo #JohnB: Since OutgoingHigh uses SendBuffer, its ReservedPacketBits needs to be modified to account for this differently
		if (Handler.IsValid())
		{
			Handler->OutgoingHigh(SendBuffer);
		}

		const double PacketSentTimeInS = FPlatformTime::Seconds();

		// Write the UNetConnection-level termination bit
		SendBuffer.WriteBit(1);

		// Refresh outgoing header with latest data
		if ( !IsInternalAck() )
		{
			// if we update ack, we also update received ack associated with outgoing seq
			// so we know how many ack bits we need to write (which is updated in received packet)
			WritePacketHeader(SendBuffer);

			WriteFinalPacketInfo(SendBuffer, PacketSentTimeInS);
		}

		ValidateSendBuffer();

		const int32 NumStrayBits = SendBuffer.GetNumBits();

		// @todo: This is no longer accurate, given potential for PacketHandler termination bit and bit padding
		//NumPaddingBits += (NumStrayBits != 0) ? (8 - NumStrayBits) : 0;

		Traits.NumAckBits = NumAckBits;
		Traits.NumBunchBits = NumBunchBits;

		NETWORK_PROFILER(GNetworkProfiler.FlushOutgoingBunches(this));

		// Flush and destroy collector
#if UE_NET_TRACE_ENABLED	
		if (OutTraceCollector)
		{
			UE_NET_TRACE_FLUSH_COLLECTOR(OutTraceCollector, NetTraceId, GetConnectionHandle().GetParentConnectionId(), ENetTracePacketType::Outgoing);
			UE_NET_TRACE_DESTROY_COLLECTOR(OutTraceCollector);
			OutTraceCollector = nullptr;
		}
		// Report end of packet
		UE_NET_TRACE_PACKET_SEND(NetTraceId, GetConnectionHandle().GetParentConnectionId(), OutPacketId, SendBuffer.GetNumBits());
#endif

		// Send now.
#if DO_ENABLE_NET_TEST
		bool bWasPacketEmulated(false);

		// if the connection is closing/being destroyed/etc we need to send immediately regardless of settings
		// because we won't be around to send it delayed
		if (GetConnectionState() != USOCK_Closed && !IsGarbageCollecting() && !bIgnoreSimulation && !IsInternalAck())
		{
			bWasPacketEmulated = CheckOutgoingPacketEmulation(PacketSentTimeInS, Traits);
		}

		if (!bWasPacketEmulated)
		{
#endif
			// Checked in FlushNet() so each child class doesn't have to implement this
			if (Driver->IsNetResourceValid())
			{
				LowLevelSend(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits);
			}
#if DO_ENABLE_NET_TEST
			if (PacketSimulationSettings.PktDup && FMath::FRand() * 100.f < PacketSimulationSettings.PktDup)
			{
				// Checked in FlushNet() so each child class doesn't have to implement this
				if (Driver->IsNetResourceValid())
				{
					LowLevelSend((char*) SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits);
				}
			}
		}
#endif
		// Update stuff.
		const int32 Index = OutPacketId & (UE_ARRAY_COUNT(OutLagPacketId)-1);

		// Remember the actual time this packet was sent out, so we can compute ping when the ack comes back
		OutLagPacketId[Index]			= OutPacketId;
		OutLagTime[Index]				= PacketSentTimeInS;
		OutBytesPerSecondHistory[Index]	= FMath::Min(OutBytesPerSecond / 1024, 255);
		
		// Increase outgoing sequence number
		if (!IsInternalAck())
		{
			PacketNotify.CommitAndIncrementOutSeq();
		}

		// Make sure that we always push an ChannelRecordEntry for each transmitted packet even if it is empty
		FChannelRecordImpl::PushPacketId(ChannelRecord, OutPacketId);

		++OutPackets;
		++OutPacketsThisFrame;
		++OutTotalPackets;
		{
			UE_IRIS_PARALLEL_EXPR(Driver->LockNetDriverStats(true));
			Driver->OutPackets++;
			Driver->OutTotalPackets++;
			UE_IRIS_PARALLEL_EXPR(Driver->UnlockNetDriverStats(true));
		}

		//Record the first packet time in the histogram
		if (!bFlushedNetThisFrame)
		{
			double LastPacketTimeDiffInMs = (Driver->GetElapsedTime() - LastSendTime) * 1000.0;
			NetConnectionHistogram.AddMeasurement(LastPacketTimeDiffInMs);
		}

		LastSendTime = Driver->GetElapsedTime();

		const int32 PacketBytes = SendBuffer.GetNumBytes() + PacketOverhead;

		if (NetworkCongestionControl.IsSet())
		{
			NetworkCongestionControl.GetValue().OnSend({ PacketSentTimeInS, OutPacketId, PacketBytes });
		}
		
		++OutPacketId; 

		if (!IsReplay())
		{
			int32 NewQueuedBits = 0;
			const bool bWouldOverflow = UE::Net::Connection::Private::Add_DetectOverflow_Clamp(QueuedBits, PacketBytes * 8, NewQueuedBits);

			if (bWouldOverflow && !bLoggedFlushNetQueuedBitsOverflow)
			{
				UE_LOGF(LogNet, Log, "UNetConnection::FlushNet: QueuedBits overflow detected! QueuedBits: %d, PacketBytes: %d. %ls", QueuedBits, PacketBytes, *Describe());
				bLoggedFlushNetQueuedBitsOverflow = true;
			}

			QueuedBits = NewQueuedBits;
		}

		OutBytes += PacketBytes;
		OutTotalBytes += PacketBytes;
		UE_IRIS_PARALLEL_EXPR(Driver->LockNetDriverStats(true));
		Driver->OutBytes += PacketBytes;
		Driver->OutTotalBytes += PacketBytes;
		GNetOutBytes += PacketBytes;
		UE_IRIS_PARALLEL_EXPR(Driver->UnlockNetDriverStats(true));

		AnalyticsVars.OutAckOnlyCount += (NumAckBits > 0 && NumBunchBits == 0);

		bFlushedNetThisFrame = true;

		InitSendBuffer();
	}
}

#if DO_ENABLE_NET_TEST
bool UNetConnection::CheckOutgoingPacketEmulation(double CurrentTimeInS, FOutPacketTraits& Traits)
{
	if (ShouldDropOutgoingPacketForLossSimulation(SendBuffer.GetNumBits()))
	{
		UE_LOGF(LogNet, VeryVerbose, "Dropping outgoing packet at %f", CurrentTimeInS);
		return true;
	}

	if (PacketSimulationSettings.PktFrameDelay > 0)
	{
		FDelayedPacket& DelayedPacket = Delayed.Emplace_GetRef(FDelayedPacket(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits));
		DelayedPacket.DelayFrameCount = PacketSimulationSettings.PktFrameDelay;

		// No more emulation applied
		return true;
	}

	if (PacketSimulationSettings.PktOrder)
	{
		Delayed.Emplace(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits);

		for (int32 i = Delayed.Num() - 1; i >= 0; i--)
		{
			if (FMath::FRand() > 0.50)
			{
				// Checked in FlushNet() so each child class doesn't have to implement this
				if (Driver->IsNetResourceValid())
				{
					LowLevelSend((char*)&Delayed[i].Data[0], Delayed[i].SizeBits, Delayed[i].Traits);
				}

				Delayed.RemoveAt(i);
			}
		}

		return true;
	}

	FDelayedPacket* DelayedPacket = nullptr;

	if (PacketSimulationSettings.PktJitter != 0)
	{
		bSendDelayedPacketsOutofOrder = true;

		DelayedPacket = &(Delayed.Emplace_GetRef(FDelayedPacket(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits)));

		// In order to cause jitter, send one packet at min latency and the next packet at high latency
		bool bIsLowLatency = (OutPacketId % 2) == 0;

		if (bIsLowLatency)
		{
			// ExtraLag goes from [PktLagMin, PktLagMin+PktLagVariance]
			const double LagVariance = 2.0f * (FMath::FRand() - 0.5f) * double(PacketSimulationSettings.PktLagVariance);
			const double ExtraLag = (double(PacketSimulationSettings.PktLagMin) + LagVariance) / 1000.f;
			DelayedPacket->SendTime = CurrentTimeInS + ExtraLag;
		}
		else
		{
			// ExtraLag goes from [PktLagMin+PktJitter, PktLagMin+PktJitter+PktLagVariance]
			const double LagVariance = 2.0f * (FMath::FRand() - 0.5f) * double(PacketSimulationSettings.PktLagVariance);
			const double ExtraLag = (double(PacketSimulationSettings.PktLagMin + PacketSimulationSettings.PktJitter) + LagVariance) / 1000.f;
			DelayedPacket->SendTime = CurrentTimeInS + ExtraLag;
		}
	}
	else if (PacketSimulationSettings.PktLag)
	{
		DelayedPacket = &(Delayed.Emplace_GetRef(FDelayedPacket(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits)));

		// ExtraLag goes from PktLag + [-PktLagVariance, PktLagVariance]
		const double LagVariance = 2.0f * (FMath::FRand() - 0.5f) * double(PacketSimulationSettings.PktLagVariance);
		const double ExtraLag = (double(PacketSimulationSettings.PktLag) + LagVariance) / 1000.f;
		DelayedPacket->SendTime = CurrentTimeInS + ExtraLag;
	}
	else if (PacketSimulationSettings.PktLagMin > 0 || PacketSimulationSettings.PktLagMax > 0)
	{
		DelayedPacket = &(Delayed.Emplace_GetRef(FDelayedPacket(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits)));

		// ExtraLag goes from [PktLagMin, PktLagMax]
		const double LagVariance = FMath::FRand() * double(PacketSimulationSettings.PktLagMax - PacketSimulationSettings.PktLagMin);
		const double ExtraLag = (double(PacketSimulationSettings.PktLagMin) + LagVariance) / 1000.f;
		DelayedPacket->SendTime = CurrentTimeInS + ExtraLag;
	}

	// Delay the packet until the next buffer flush or later
	if (PacketSimulationSettings.PktBufferBloatInMS > 0)
	{
		if (!DelayedPacket)
		{
			DelayedPacket = &(Delayed.Emplace_GetRef(FDelayedPacket(SendBuffer.GetData(), SendBuffer.GetNumBits(), Traits)));
		}

		if (NextBufferBloatFlushOut < CurrentTimeInS && CurrentTimeInS >= NextBufferBloatActiveTimeOut)
		{
			// Find the time at which we flush all delayed packets, and the next time we restart the buffer bloat accumulation
			NextBufferBloatFlushOut = CurrentTimeInS + (PacketSimulationSettings.PktBufferBloatInMS/ 1000.0);
			NextBufferBloatActiveTimeOut = NextBufferBloatFlushOut + (BufferBloatCooldownTimeInMS / 1000.0);
		}

		// Only keep the latest delayed time
		DelayedPacket->SendTime = FMath::Max(NextBufferBloatFlushOut, DelayedPacket->SendTime);
	}
	
	return DelayedPacket != nullptr;
}
#endif

#if DO_ENABLE_NET_TEST
bool UNetConnection::ShouldDropOutgoingPacketForLossSimulation(int64 NumBits) const
{
	return Driver->IsSimulatingPacketLossBurst() || 
		(PacketSimulationSettings.PktLoss > 0 && 
         PacketSimulationSettings.ShouldDropPacketOfSize(NumBits) && 
         FMath::FRand() * 100.f < PacketSimulationSettings.PktLoss);
}
#endif

int32 UNetConnection::IsNetReady(bool bSaturate)
{
	if (!IsReplay() && bSaturate)
	{
		// Note that this doesn't immediately saturate, but instead forces the buffer to be on the threshold of being saturated.
		// So IsNetReady will be true but the next serialized bit will make it go over the limit and then IsNetReady will return false
		// Due to this confusion between name and effect it was removed as an option.
		QueuedBits = -SendBuffer.GetNumBits();
	}

	return (int32)IsNetReady();
}

bool UNetConnection::IsNetReady() const
{
	if (IsReplay())
	{
		return true;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (NetConnectionHelper::HasDisabledBandwidthThrottling())
	{
		return true;
	}
#endif

	if (NetworkCongestionControl.IsSet())
	{
		return NetworkCongestionControl.GetValue().IsReadyToSend(Driver->GetElapsedTime());
	}

	return QueuedBits + SendBuffer.GetNumBits() <= 0;
}

bool UNetConnection::IsPacketSequenceWindowFull(uint32 SafetyMargin)
{
	return PacketNotify.IsSequenceWindowFull(SafetyMargin);
}

void UNetConnection::ReadInput( float DeltaSeconds )
{}

void UNetConnection::ReceivedAck(int32 AckPacketId, FChannelsToClose& OutChannelsToClose)
{
	UE_LOGF(LogNetTraffic, Verbose, "   Received ack %i", AckPacketId);

	SCOPE_CYCLE_COUNTER(Stat_NetConnectionReceivedAcks);
	
	// Advance OutAckPacketId
	OutAckPacketId = AckPacketId;

	// Store the first Ack we received in the packet
	if (OutFirstAckPacketId == -1)
	{
		OutFirstAckPacketId = AckPacketId;

		// On the first ACK of the first packet of a frame, reset the RawPing
		if (InPacketsThisFrame == 1)
		{
			RawPingInSeconds = 0.0;
		}
	}

	// Process the bunch.
	LastRecvAckTimestamp = Driver->GetElapsedTime();

	PacketAnalytics.TrackAck(AckPacketId);

	if (PackageMap)
	{
		PackageMap->ReceivedAck( AckPacketId );
	}

	auto AckChannelFunc = [this, &OutChannelsToClose](int32 AckedPacketId, uint32 ChannelIndex)
	{
		UChannel* const Channel = Channels[ChannelIndex];

		if (Channel)
		{
			if (Channel->OpenPacketId.Last == AckedPacketId) // Necessary for unreliable "bNetTemporary" channels.
			{
				Channel->OpenAcked = 1;
			}
				
			for (FOutBunch* OutBunch = Channel->OutRec; OutBunch; OutBunch = OutBunch->Next)
			{
				if (OutBunch->bOpen)
				{
					UE_LOGF(LogNet, VeryVerbose, "Channel %i reset Ackd because open is reliable. ", Channel->ChIndex );
					Channel->OpenAcked  = 0; // We have a reliable open bunch, don't let the above code set the OpenAcked state,
											// it must be set in UChannel::ReceivedAcks to verify all open bunches were received.
				}

				if (OutBunch->PacketId == AckedPacketId)
				{
					OutBunch->ReceivedAck = 1;
				}
			}
			Channel->ReceivedAck(AckedPacketId);
			EChannelCloseReason CloseReason;
			if (Channel->ReceivedAcks(CloseReason))
			{
				const FChannelCloseInfo Info = {ChannelIndex, CloseReason};
				OutChannelsToClose.Emplace(Info);
			}	
		}
	};

	// Invoke AckChannelFunc on all channels written for this PacketId
	FChannelRecordImpl::ConsumeChannelRecordsForPacket(ChannelRecord, AckPacketId, AckChannelFunc);
}

void UNetConnection::ReceivedNak( int32 NakPacketId )
{
	UE_LOGF(LogNetTraffic, Verbose, "   Received nak %i", NakPacketId);

	UE_NET_TRACE_PACKET_DROPPED(NetTraceId, GetConnectionHandle().GetParentConnectionId(), NakPacketId, ENetTracePacketType::Outgoing);

	SCOPE_CYCLE_COUNTER(Stat_NetConnectionReceivedNak);

	PacketAnalytics.TrackNak(NakPacketId);

	// Update pending NetGUIDs
	PackageMap->ReceivedNak(NakPacketId);

	auto NakChannelFunc = [this](int32 NackedPacketId, uint32 ChannelIndex)
	{
		UChannel* const Channel = Channels[ChannelIndex];
		if (Channel)
		{
			Channel->ReceivedNak(NackedPacketId);
			if (Channel->OpenPacketId.InRange(NackedPacketId))
			{
				Channel->ReceivedAcks(); //warning: May destroy Channel.
			}
		}
	};

	// Invoke NakChannelFunc on all channels written for this PacketId
	FChannelRecordImpl::ConsumeChannelRecordsForPacket(ChannelRecord, NakPacketId, NakChannelFunc);

	// Stats
	++OutPacketsLost;
	++OutTotalPacketsLost;
	++Driver->OutPacketsLost;
	++Driver->OutTotalPacketsLost;
}

// IMPORTANT:
// WritePacketHeader must ALWAYS write the exact same number of bits as we go back and rewrite the header
// right before we put the packet on the wire.
void UNetConnection::WritePacketHeader(FBitWriter& Writer)
{
	// If this is a header refresh, we only serialize the updated serial number information
	const bool bIsHeaderUpdate = Writer.GetNumBits() > 0u;

	// Header is always written first in the packet
	FBitWriterMark Reset;
	FBitWriterMark Restore(Writer);
	Reset.PopWithoutClear(Writer);
	
	// Write notification header or refresh the header if used space is the same.
	bool bWroteHeader = PacketNotify.WriteHeader(Writer, bIsHeaderUpdate);

#if !UE_BUILD_SHIPPING
	checkf(Writer.GetNumBits() <= MAX_PACKET_RELIABLE_SEQUENCE_HEADER_BITS, TEXT("WritePacketHeader exceeded the max allowed bits. Wrote %" INT64_FMT ". Max %d"), Writer.GetNumBits(), MAX_PACKET_RELIABLE_SEQUENCE_HEADER_BITS);
#endif

	// Jump back to where we came from.
	if (bIsHeaderUpdate)
	{
		Restore.PopWithoutClear(Writer);

		// if we wrote the header and successfully refreshed the header status we no longer has any dirty acks
		if (bWroteHeader)
		{
			HasDirtyAcks = 0u;
		}
	}
}

void UNetConnection::WriteDummyPacketInfo(FBitWriter& Writer)
{
#if !UE_BUILD_SHIPPING
	const int32 BitsWrittenPrePacketInfo = Writer.GetNumBits();
#endif

	// The first packet of a frame will include the packet info payload
	const bool bHasPacketInfoPayload = bFlushedNetThisFrame == false;
	Writer.WriteBit(bHasPacketInfoPayload);

	if (bHasPacketInfoPayload)
	{
		// Pre-insert the bits since the final time values will be calculated and inserted right before LowLevelSend
		HeaderMarkForPacketInfo.Init(Writer);
		uint32 DummyJitterClockTime(0);
		Writer.SerializeBits(&DummyJitterClockTime, NetConnectionHelper::NumBitsForJitterClockTimeInHeader);

#if !UE_BUILD_SHIPPING
		checkf(Writer.GetNumBits() - BitsWrittenPrePacketInfo == (1 + NetConnectionHelper::NumBitsForJitterClockTimeInHeader), TEXT("WriteDummyPacketInfo did not write the expected nb of bits: Wrote %" INT64_FMT ", Expected %d"),
			   Writer.GetNumBits() - BitsWrittenPrePacketInfo, (1 + NetConnectionHelper::NumBitsForJitterClockTimeInHeader));
#endif

		const uint8 bHasServerFrameTime = Driver->IsServer() ? bLastHasServerFrameTime : (CVarPingExcludeFrameTime.GetValueOnAnyThread() > 0 ? 1u : 0u);
		Writer.WriteBit(bHasServerFrameTime);

		if (bHasServerFrameTime && Driver->IsServer())
		{
			uint8 DummyFrameTimeByte(0);
			Writer << DummyFrameTimeByte;
		}
	}

	bSendBufferHasDummyPacketInfo = bHasPacketInfoPayload;

#if !UE_BUILD_SHIPPING
	checkf(Writer.GetNumBits() - BitsWrittenPrePacketInfo <= MAX_PACKET_INFO_HEADER_BITS, TEXT("WriteDummyPacketInfo exceeded the max allowed bits. Wrote %" INT64_FMT ". Max %d"), Writer.GetNumBits() - BitsWrittenPrePacketInfo, MAX_PACKET_INFO_HEADER_BITS);
#endif
}

void UNetConnection::WriteFinalPacketInfo(FBitWriter& Writer, const double PacketSentTimeInS)
{
	if (bSendBufferHasDummyPacketInfo == false)
	{
		// PacketInfo payload is not included in this SendBuffer; nothing to rewrite
		return;
	}

	check(HeaderMarkForPacketInfo.GetNumBits() != 0);

	FBitWriterMark CurrentMark(Writer);

	// Go back to write over the dummy bits
	HeaderMarkForPacketInfo.PopWithoutClear(Writer);

#if !UE_BUILD_SHIPPING
	const int32 BitsWrittenPreJitterClock = Writer.GetNumBits();
#endif

	// Write Jitter clock time
	{
		const double DeltaSendTimeInMS = (PacketSentTimeInS - PreviousPacketSendTimeInS) * 1000.0;

		uint32 ClockTimeMilliseconds = 0;

		// If the delta is over our max precision, we send MAX value and jitter will be ignored by the receiver.
		if (DeltaSendTimeInMS >= UE::Net::Connection::Private::MaxJitterPrecisionInMS)
		{
			ClockTimeMilliseconds = UE::Net::Connection::Private::MaxJitterClockTimeValue;
		}
		else
		{
			// Get the fractional part (milliseconds) of the clock time
			ClockTimeMilliseconds = UE::Net::Connection::Private::GetClockTimeMilliseconds(PacketSentTimeInS);

			// Ensure we don't overflow
			ClockTimeMilliseconds &= UE::Net::Connection::Private::MaxJitterClockTimeValue;
		}

		Writer.SerializeInt(ClockTimeMilliseconds, UE::Net::Connection::Private::MaxJitterClockTimeValue + 1);

#if !UE_BUILD_SHIPPING
		checkf(Writer.GetNumBits() - BitsWrittenPreJitterClock == NetConnectionHelper::NumBitsForJitterClockTimeInHeader, TEXT("WriteFinalPacketInfo did not write the expected nb of bits: Wrote %" INT64_FMT ", Expected %d"),
			   Writer.GetNumBits() - BitsWrittenPreJitterClock, NetConnectionHelper::NumBitsForJitterClockTimeInHeader);
#endif

		PreviousPacketSendTimeInS = PacketSentTimeInS;
	}

	// Write server frame time
	{
		const uint8 bHasServerFrameTime = Driver->IsServer() ? bLastHasServerFrameTime : (CVarPingExcludeFrameTime.GetValueOnGameThread() > 0 ? 1u : 0u);
		Writer.WriteBit(bHasServerFrameTime);

		if (bHasServerFrameTime && Driver->IsServer())
		{
			// Write data used to calculate link latency
			uint8 FrameTimeByte = FMath::Min(FMath::FloorToInt(FrameTime * 1000), 255);
			Writer << FrameTimeByte;
		}
	}
	
	HeaderMarkForPacketInfo.Reset();
	
	// Revert to the correct bit writing place
	CurrentMark.PopWithoutClear(Writer);
}

bool UNetConnection::ReadPacketInfo(FBitReader& Reader, bool bHasPacketInfoPayload, FEngineNetworkCustomVersion::Type EngineNetVer)
{
	using namespace UE::Net;

	// If this packet did not contain any packet info, nothing else to read
	if (!bHasPacketInfoPayload)
	{
		const bool bCanContinueReading = !Reader.IsError();
		return bCanContinueReading;
	}

	const bool bHasServerFrameTime = Reader.ReadBit() == 1u;
	double ServerFrameTime = 0.0;

	if ( !Driver->IsServer() )
	{
		if ( bHasServerFrameTime )
		{
			uint8 FrameTimeByte	= 0;
			Reader << FrameTimeByte;
			// As a client, our request was granted, read the frame time
			ServerFrameTime = ( double )FrameTimeByte / 1000;
		}
	}
	else
	{
		bLastHasServerFrameTime = bHasServerFrameTime;
	}

	if (EngineNetVer < FEngineNetworkCustomVersion::JitterInHeader)
	{
		uint8 RemoteInKBytesPerSecondByte = 0;
		Reader << RemoteInKBytesPerSecondByte;
	}

	// limit to known size to know the size of the packet header
	if ( Reader.IsError() )
	{
		return false;
	}

	if (OutFirstAckPacketId == -1)
	{
		// This means we received a packet without new ACKs in it.
		// Continue reading it but we don't need to update the Ping.
		return true;
	}

	// Find the received time of the packet
	double PacketReceiveTime = 0.0;
	FTimespan& RecvTimespan = LastOSReceiveTime.Timestamp;

	if (!RecvTimespan.IsZero() && Driver != nullptr && CVarPingUsePacketRecvTime.GetValueOnAnyThread())
	{
		if (bIsOSReceiveTimeLocal)
		{
			PacketReceiveTime = RecvTimespan.GetTotalSeconds();
		}
		else if (ISocketSubsystem* SocketSubsystem = Driver->GetSocketSubsystem())
		{
			PacketReceiveTime = SocketSubsystem->TranslatePacketTimestamp(LastOSReceiveTime);
		}
	}

	// Use the driver's LastTickDispatchRealtime because it is set closer to the beginning of the frame - we don't care about the time so far of the current frame to process the packet
	// This time needs to be a realtime value based on FPlatformTime::Seconds so that it's compatible with the packet send time.
	const double CurrentTime = (PacketReceiveTime != 0.0 ? PacketReceiveTime : Driver->LastTickDispatchRealtime);

	// Update ping

	const bool bExcludeFrameTime = !!CVarPingExcludeFrameTime.GetValueOnAnyThread();

	// Go trought all the ACKs we received
	for (int32 AckId = OutFirstAckPacketId; AckId <= OutAckPacketId; ++AckId)
	{
		// Map the Id to the Index(0-255) storing the sent time 
		const int32 AckIndex = AckId & (UE_ARRAY_COUNT(OutLagPacketId) - 1);
		
		if (OutLagPacketId[AckIndex] != AckId)
		{
			// The Ids were already processed, nothing more to do here
			return true;
		}

		// Clear the id to tell it was processed.
		OutLagPacketId[AckIndex] = -1;

		const double PacketSendTime = OutLagTime[AckIndex];
		const double RTT = (CurrentTime - PacketSendTime);
		const double RTTExclFrame = RTT - (bExcludeFrameTime ? ServerFrameTime : 0.0);
		const double PacketPing = FMath::Max(RTTExclFrame, 0.0);

		LagAcc += PacketPing;
		LagCount++;

		// Keep the highest ping as the RawPing value for this frame
		RawPingInSeconds = FMath::Max(RawPingInSeconds, PacketPing);

		if (PlayerController)
		{
			// PlayerController is valid and local (as opposed to currently loaned out)
			UE_CLOGF(PlayerController.GetResidence() == EResidence::Remote, LogNet, Warning, "%s: PlayerController is unexpectedly a RemoteObject", __func__);
			if (PlayerController.GetResidence() != EResidence::Remote)
			{
				double PlayerFacingPing = RTTExclFrame;
				switch(EPingType(CVarPingPlayerFacingPingType.GetValueOnAnyThread()))
				{
					case EPingType::ICMP:
						if (NetPing.IsValid())
						{
							PlayerFacingPing = NetPing->GetPingValues(EPingType::ICMP).Current;
						}
						break;
					case EPingType::UDPQoS:
						if (NetPing.IsValid())
						{
							PlayerFacingPing = NetPing->GetPingValues(EPingType::UDPQoS).Current;
						}
						break;
					case EPingType::RoundTrip:
						PlayerFacingPing = RTT;
						break;
					case EPingType::RoundTripExclFrame:
					default:
						PlayerFacingPing = RTTExclFrame;
						break;
				}
				
				PlayerController->UpdatePing(FMath::Max(PlayerFacingPing, 0.f));
			}
		}

		if (NetPing.IsValid())
		{
			NetPing->UpdatePing(EPingType::RoundTrip, CurrentTime, FMath::Max(RTT, 0.0)); 
			NetPing->UpdatePing(EPingType::RoundTripExclFrame, CurrentTime, PacketPing);
		}

		if (NetworkCongestionControl.IsSet())
		{
			NetworkCongestionControl.GetValue().OnAck({ CurrentTime, OutAckPacketId });
		}
	}

	OutFirstAckPacketId = -1;

	return true;
}

FNetworkGUID UNetConnection::GetActorGUIDFromOpenBunch(FInBunch& Bunch)
{
	// NOTE: This could break if this is a PartialBunch and the ActorGUID wasn't serialized.
	//			Seems unlikely given the aggressive Flushing + increased MTU on InternalAck.

	// Any GUIDs / Exports will have been read already for InternalAck connections,
	// but we may have to skip over must-be-mapped GUIDs before we can read the actor GUID.

	if (Bunch.bHasMustBeMappedGUIDs)
	{
		uint16 NumMustBeMappedGUIDs = 0;
		Bunch << NumMustBeMappedGUIDs;

		for (int32 i = 0; i < NumMustBeMappedGUIDs; i++)
		{
			FNetworkGUID NetGUID;
			Bunch << NetGUID;
		}
	}

	NET_CHECKSUM( Bunch );

	FNetworkGUID ActorGUID;
	Bunch << ActorGUID;

	return ActorGUID;
}

#if DO_ENABLE_NET_TEST
bool UNetConnection::CheckIncomingPacketEmulation(double CurrentTimeInS, FBitReader& Reader)
{
	if (PacketSimulationSettings.PktIncomingLoss)
	{
		if (FMath::FRand() * 100.f < PacketSimulationSettings.PktIncomingLoss)
		{
			UE_LOGF(LogNet, VeryVerbose, "Dropped incoming packet at %f", CurrentTimeInS);
			return true;
		}
	}

	FDelayedIncomingPacket* DelayedPacket = nullptr;

	if (PacketSimulationSettings.PktIncomingLagMin > 0 || PacketSimulationSettings.PktIncomingLagMax > 0)
	{
		// ExtraLagInSec goes from [PktIncomingLagMin, PktIncomingLagMax]
		const double LagVarianceInMS = FMath::FRand() * double(PacketSimulationSettings.PktIncomingLagMax - PacketSimulationSettings.PktIncomingLagMin);
		const double ExtraLagInSec = (double(PacketSimulationSettings.PktIncomingLagMin) + LagVarianceInMS) / 1000.f;

		DelayedPacket = &(DelayedIncomingPackets.AddDefaulted_GetRef());

		DelayedPacket->PacketData = MakeUnique<FBitReader>(Reader);
		DelayedPacket->ReinjectionTime = CurrentTimeInS + ExtraLagInSec;

		UE_LOGF(LogNet, VeryVerbose, "Delaying incoming packet for %f seconds", ExtraLagInSec);
	}
	else if (PacketSimulationSettings.PktIncomingFrameDelay)
	{
		DelayedPacket = &(DelayedIncomingPackets.AddDefaulted_GetRef());

		DelayedPacket->PacketData = MakeUnique<FBitReader>(Reader);
		DelayedPacket->ReinjectionFrameCount = PacketSimulationSettings.PktIncomingFrameDelay;

		UE_LOGF(LogNet, VeryVerbose, "Delaying incoming packet for %u frames", PacketSimulationSettings.PktIncomingFrameDelay);

		// Not compatible with BufferBloat
		return true;
	}

	if (PacketSimulationSettings.PktIncomingBufferBloatInMS > 0)
	{
		if (!DelayedPacket)
		{
			DelayedPacket = &(DelayedIncomingPackets.AddDefaulted_GetRef());
			DelayedPacket->PacketData = MakeUnique<FBitReader>(Reader);
		}

		if (NextBufferBloatFlushIn < CurrentTimeInS && CurrentTimeInS >= NextBufferBloatActiveTimeIn)
		{
			NextBufferBloatFlushIn = CurrentTimeInS + (PacketSimulationSettings.PktIncomingBufferBloatInMS / 1000.f);
			NextBufferBloatActiveTimeIn = NextBufferBloatFlushIn + (BufferBloatCooldownTimeInMS / 1000.0);
		}

		// Only keep the latest delayed time
		DelayedPacket->ReinjectionTime = FMath::Max(NextBufferBloatFlushIn, DelayedPacket->ReinjectionTime);
	}

	return DelayedPacket != nullptr;

}
#endif //DO_ENABLE_NET_TEST

void UNetConnection::ReceivedPacket( FBitReader& Reader, bool bIsReinjectedPacket, bool bDispatchPacket )
{
	using namespace UE::Net;

	SCOPED_NAMED_EVENT(UNetConnection_ReceivedPacket, FColor::Green);
	AssertValid();

	// Handle PacketId.
	if( Reader.IsError() )
	{
		ensureMsgf(false, TEXT("Packet too small") );
		return;
	}

	// Record the packet time to the histogram
	const double CurrentReceiveTimeInS = FPlatformTime::Seconds();

#if DO_ENABLE_NET_TEST
	if (!IsInternalAck() && !bIsReinjectingDelayedPackets)
	{
		const bool bIsPackedEmulated = CheckIncomingPacketEmulation(CurrentReceiveTimeInS, Reader);

		if (bIsPackedEmulated)
		{
			// Packet was delayed or dropped, nothing to do here anymore
			return;
		}
	}
#endif //#if DO_ENABLE_NET_TEST


	FBitReaderMark ResetReaderMark(Reader);

	ValidateSendBuffer();

	if (!bIsReinjectedPacket)
	{
		const double LastPacketTimeDiffInMs = (CurrentReceiveTimeInS - LastReceiveRealtime) * 1000.0;
		NetConnectionHistogram.AddMeasurement(LastPacketTimeDiffInMs);

		// Update receive time to avoid timeout.
		LastReceiveTime = Driver->GetElapsedTime();
		LastReceiveRealtime = CurrentReceiveTimeInS;
	}

	FChannelsToClose ChannelsToClose;

	const FEngineNetworkCustomVersion::Type PacketEngineNetVer = static_cast<FEngineNetworkCustomVersion::Type>(Reader.EngineNetVer());

	// If we choose to not process this packet we need to restore it
	const int32 OldInPacketId = InPacketId;

	if (IsInternalAck())
	{
		++InPacketId;
	}	
	else
	{
		// Read packet header
		FNetPacketNotify::FNotificationHeader Header;
		if (!PacketNotify.ReadHeader(Header, Reader))
		{
			UE_LOGF(LogNet, Warning, "Failed to read PacketHeader");

			HandleNetResultOrClose(ENetCloseResult::ReadHeaderFail);

			return;
		}

		bool bHasPacketInfoPayload = true;

		if (PacketEngineNetVer >= FEngineNetworkCustomVersion::JitterInHeader)
		{
			bHasPacketInfoPayload = Reader.ReadBit() == 1u;

			if (bHasPacketInfoPayload)
			{
#if !UE_BUILD_SHIPPING
				const int32 BitsReadPreJitterClock = Reader.GetPosBits();
#endif
				// Read jitter clock time from the packet header
				uint32 PacketJitterClockTimeMS = 0;
				Reader.SerializeInt(PacketJitterClockTimeMS, UE::Net::Connection::Private::MaxJitterClockTimeValue + 1);

#if !UE_BUILD_SHIPPING
				static double LastJitterLogTime = 0.0;

				if (((Reader.GetPosBits() - BitsReadPreJitterClock) != NetConnectionHelper::NumBitsForJitterClockTimeInHeader) &&
					((CurrentReceiveTimeInS - LastJitterLogTime) > 5.0))
				{
					UE_LOG(LogNet, Warning, TEXT("JitterClockTime did not read the expected nb of bits. Read %" INT64_FMT ", Expected %d"),
							Reader.GetPosBits() - BitsReadPreJitterClock, NetConnectionHelper::NumBitsForJitterClockTimeInHeader);

					LastJitterLogTime = CurrentReceiveTimeInS;
				}
#endif

				if (!bIsReinjectedPacket)
				{
					ProcessJitter(PacketJitterClockTimeMS);
				}
			}			
		}

		const int32 PacketSequenceDelta = PacketNotify.GetSequenceDelta(Header);

		// If the packetID is higher then the last one
		if (PacketSequenceDelta > 0)
		{
			const bool bPacketOrderCacheActive = !bFlushingPacketOrderCache && PacketOrderCache.IsSet();
			const bool bCheckForMissingSequence = bPacketOrderCacheActive && PacketOrderCacheCount == 0;
			const bool bFillingPacketOrderCache = bPacketOrderCacheActive && PacketOrderCacheCount > 0;
			const int32 MaxMissingPackets = (bCheckForMissingSequence ? CVarNetPacketOrderMaxMissingPackets.GetValueOnAnyThread() : 0);

			const int32 MissingPacketCount = PacketSequenceDelta - 1;

			// If there is a gap in the Id sequence, cache the packet so they can be processed in the right order
			if (bFillingPacketOrderCache || (bCheckForMissingSequence && MissingPacketCount > 0 && MissingPacketCount <= MaxMissingPackets))
			{
				int32 LinearCacheIdx = PacketSequenceDelta - 1;
				int32 CacheCapacity = PacketOrderCache->Capacity();
				bool bLastCacheEntry = LinearCacheIdx >= (CacheCapacity - 1);

				// The last cache entry is only set, when we've reached capacity or when we receive a sequence which is out of bounds of the cache
				LinearCacheIdx = bLastCacheEntry ? (CacheCapacity - 1) : LinearCacheIdx;

				int32 CircularCacheIdx = PacketOrderCacheStartIdx;

				for (int32 LinearDec=LinearCacheIdx; LinearDec > 0; LinearDec--)
				{
					CircularCacheIdx = PacketOrderCache->GetNextIndex(CircularCacheIdx);
				}

				TUniquePtr<FBitReader>& CurCachePacket = PacketOrderCache.GetValue()[CircularCacheIdx];

				// Reset the reader to its initial position, and cache the packet
				if (!CurCachePacket.IsValid())
				{
					UE_LOGF(LogNet, VeryVerbose, "'Out of Order' Packet Cache, caching sequence order '%i' (capacity: %i)", LinearCacheIdx, CacheCapacity);

					CurCachePacket = MakeUnique<FBitReader>(Reader);

					PacketOrderCacheCount++;
					TotalOutOfOrderPacketsRecovered++;
					AnalyticsVars.IncreaseOutOfOrderPacketsRecoveredCount();
					Driver->IncreaseTotalOutOfOrderPacketsRecovered();

					ResetReaderMark.Pop(*CurCachePacket);
				}
				else
				{
					TotalOutOfOrderPacketsDuplicate++;
					AnalyticsVars.IncreaseOutOfOrderPacketsDuplicateCount();
					Driver->IncreaseTotalOutOfOrderPacketsDuplicate();
				}

				return;
			}

			// Process acks
			// Lambda to dispatch delivery notifications, 
			auto HandlePacketNotification = [&Header, &ChannelsToClose, this](FNetPacketNotify::SequenceNumberT AckedSequence, bool bDelivered)
			{
				// Increase LastNotifiedPacketId, this is a full packet Id
				++LastNotifiedPacketId;
				++OutTotalNotifiedPackets;
				Driver->IncreaseOutTotalNotifiedPackets();

				// Sanity check
				if (FNetPacketNotify::SequenceNumberT(LastNotifiedPacketId) != AckedSequence)
				{
					UE_LOGF(LogNet, Error, "LastNotifiedPacketId != AckedSequence");

					Close(ENetCloseResult::AckSequenceMismatch);

					return;
				}

				if (!FChannelRecordImpl::HasChannelRecordsToConsumeForPacket(ChannelRecord, LastNotifiedPacketId))
				{
					UE_LOGF(LogNet, Error, "Could not find matching record for packet %d", LastNotifiedPacketId);

					Close(ENetCloseResult::AckSequenceMismatch);

					return;
				}

				if (bDelivered)
				{
					ReceivedAck(LastNotifiedPacketId, ChannelsToClose);
				}
				else
				{
					ReceivedNak(LastNotifiedPacketId);
				};
			};

			// Update incoming sequence data and deliver packet notifications
			// Packet is only accepted if both the incoming sequence number and incoming ack data are valid		
			const int32 UpdatedPacketSequenceDelta = PacketNotify.Update(Header, HandlePacketNotification);
			if (PacketNotify.IsWaitingForSequenceHistoryFlush())
			{
				// Mark acks dirty
				++HasDirtyAcks;

				// Since we did not necessarily ack or nack all packets we need to update the InPacketId to reflect this.
				InPacketId = OldInPacketId + UpdatedPacketSequenceDelta;

				return;
			}

			InPacketsLost += MissingPacketCount;
			InTotalPacketsLost += MissingPacketCount;
			Driver->InPacketsLost += MissingPacketCount;
			Driver->InTotalPacketsLost += MissingPacketCount;
			InPacketId += PacketSequenceDelta;

			check(FNetPacketNotify::SequenceNumberT(InPacketId).Get() == Header.Seq.Get());

			PacketAnalytics.TrackInPacket(InPacketId, MissingPacketCount);
		}
		// PacketID was earlier then a previously read packet, drop it entirely because it is now obsolete
		else
		{
			TotalOutOfOrderPacketsLost++;
			AnalyticsVars.IncreaseOutOfOrderPacketsLostCount();
			Driver->IncreaseTotalOutOfOrderPacketsLost();

			if (!PacketOrderCache.IsSet() && CVarNetDoPacketOrderCorrection.GetValueOnAnyThread() != 0)
			{
				int32 EnableThreshold = CVarNetPacketOrderCorrectionEnableThreshold.GetValueOnAnyThread();

				if (TotalOutOfOrderPacketsLost >= EnableThreshold)
				{
					UE_LOGF(LogNet, Verbose, "Hit threshold of %i 'out of order' packet sequences. Enabling out of order packet correction.", EnableThreshold);

					int32 CacheSize = FMath::RoundUpToPowerOfTwo(CVarNetPacketOrderMaxCachedPackets.GetValueOnAnyThread());

					PacketOrderCache.Emplace(CacheSize);
				}
			}

			// Protect against replay attacks
			// We already protect against this for reliable bunches, and unreliable properties
			// The only bunch we would process would be unreliable RPC's, which could allow for replay attacks
			// So rather than add individual protection for unreliable RPC's as well, just kill it at the source, 
			// which protects everything in one fell swoop
			return;
		}

		// Extra information associated with the header (read only after acks have been processed)
		if (PacketSequenceDelta > 0 && !ReadPacketInfo(Reader, bHasPacketInfoPayload, PacketEngineNetVer))
		{
			UE_LOGF(LogNet, Warning, "Failed to read extra PacketHeader information");

			HandleNetResultOrClose(ENetCloseResult::ReadHeaderExtraFail);

			return;
		}
	}

	// Setup InTraceCollector of net tracing is enabled
#if  UE_NET_TRACE_ENABLED
	InTraceCollector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);

	// Trace Packet header bits
	UE_NET_TRACE(PacketHeaderAndInfo, GetInTraceCollector(), ResetReaderMark.GetPos(), Reader.GetPosBits(), ENetTraceVerbosity::Trace);
#endif

	const bool bIgnoreRPCs = Driver->ShouldIgnoreRPCs();

	bool bSkipAck = false;

	bool bHasBunchErrors = false;

	const bool bIsServer = Driver->ServerConnection == nullptr;
	double PostReceiveTime = 0.0;

	{
		if (RPCDoS.IsValid() && bIsServer && !IsReplay())
		{
			RPCDoS->PreReceivedPacket(CurrentReceiveTimeInS);
		}

		ON_SCOPE_EXIT
		{
			PostReceiveTime = FPlatformTime::Seconds();

			if (RPCDoS.IsValid() && bIsServer && !IsReplay())
			{
				RPCDoS->PostReceivedPacket(PostReceiveTime);
			}
		};

		if (bDispatchPacket)
		{
			DispatchPacket(Reader, InPacketId, bSkipAck, bHasBunchErrors);
		}

		// Close/clean-up channels pending close due to received acks.
		for (FChannelCloseInfo& Info : ChannelsToClose)
		{
			if (UChannel* Channel = Channels[Info.Id])
			{
				Channel->ConditionalCleanUp(false, Info.CloseReason);
			}
		}

		ValidateSendBuffer();
	}

	// Acknowledge the packet.
	if (!bSkipAck)
	{
		LastGoodPacketRealtime = PostReceiveTime;
	}

	if(!IsInternalAck())
	{
		// We always call AckSequence even if we are explicitly rejecting the packet as this updates the expected InSeq used to drive future acks.
		if (bSkipAck)
		{
			// Explicit Nak, we treat this packet as dropped but we still report it to the sending side as quickly as possible
			PacketNotify.NakSeq(InPacketId);
		}
		else
		{
			PacketNotify.AckSeq(InPacketId);

			// Keep stats happy
			++OutTotalAcks;
			++Driver->OutTotalAcks;
		}

		if (Driver->IsSendingImmediateAcks())
		{
			// If enabled, we want to let the other side know about the ack, so even if there are no other outgoing data when we tick the connection we will send an ackpacket.
			TimeSensitive = 1;
			++HasDirtyAcks;
		}
	}

	// Flush trace content collector
#if UE_NET_TRACE_ENABLED
	if (InTraceCollector)
	{
		UE_NET_TRACE_FLUSH_COLLECTOR(InTraceCollector, NetTraceId, GetConnectionHandle().GetParentConnectionId(), ENetTracePacketType::Incoming);
		UE_NET_TRACE_DESTROY_COLLECTOR(InTraceCollector);
		InTraceCollector = nullptr;
	}
#endif

	// Trace end marker of this incoming data packet.
	UE_NET_TRACE_PACKET_RECV(NetTraceId, GetConnectionHandle().GetParentConnectionId(), InPacketId, Reader.GetNumBits());

	if ((bHasBunchErrors || bSkipAck) && !IsInternalAck())
	{
		// Trace packet as lost on receiving end to indicate bSkipAck or errors in traced data
		UE_NET_TRACE_PACKET_DROPPED(NetTraceId, GetConnectionHandle().GetParentConnectionId(), InPacketId, ENetTracePacketType::Incoming);
	}
}

void UNetConnection::DispatchPacket( FBitReader& Reader, int32 PacketId, bool& bOutSkipAck, bool& bOutHasBunchErrors )
{
	// Track channels that were rejected while processing this packet - used to avoid sending multiple close-channel bunches,
	// which would cause a disconnect serverside
	TArray<int32> RejectedChans;

	const bool bIsServer = Driver->ServerConnection == nullptr;
	const bool bIgnoreRPCs = Driver->ShouldIgnoreRPCs();

	const FEngineNetworkCustomVersion::Type PacketEngineNetVer = static_cast<FEngineNetworkCustomVersion::Type>(Reader.EngineNetVer());

	// We want to be able to read replays that do not support export extentions.
	const bool bHasPartialCustomExportsFinalBit = !(IsInternalAck() && PacketEngineNetVer < FEngineNetworkCustomVersion::CustomExports);

	// Disassemble and dispatch all bunches in the packet.
	while( !Reader.AtEnd() && GetConnectionState()!=USOCK_Closed )
	{
		// For demo backwards compatibility, old replays still have this bit
		if (IsInternalAck() && PacketEngineNetVer < FEngineNetworkCustomVersion::AcksIncludedInHeader)
		{
			const bool IsAckDummy = Reader.ReadBit() == 1u;
		}

		// Parse the bunch.
		int32 StartPos = Reader.GetPosBits();
	
		// Process Received data
		{
			// Parse the incoming data.
			FInBunch Bunch( this );
			int32 IncomingStartPos		= Reader.GetPosBits();
			uint8 bControl				= Reader.ReadBit();
			Bunch.PacketId				= PacketId;
			Bunch.bOpen					= bControl ? Reader.ReadBit() : 0;
			Bunch.bClose				= bControl ? Reader.ReadBit() : 0;
		
			if (PacketEngineNetVer < FEngineNetworkCustomVersion::ChannelCloseReason)
			{
				const uint8 bDormant = Bunch.bClose ? Reader.ReadBit() : 0;
				Bunch.CloseReason = bDormant ? EChannelCloseReason::Dormancy : EChannelCloseReason::Destroyed;
			}
			else
			{
				Bunch.CloseReason = Bunch.bClose ? (EChannelCloseReason)Reader.ReadInt((uint32)EChannelCloseReason::MAX) :
													EChannelCloseReason::Destroyed;
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Removing this will require a new network version for replay compatabilty
			Bunch.bIsReplicationPaused = Reader.ReadBit();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			Bunch.bReliable				= Reader.ReadBit();

			if (PacketEngineNetVer < FEngineNetworkCustomVersion::MaxActorChannelsCustomization)
			{
				static const int OLD_MAX_ACTOR_CHANNELS = 10240;
				Bunch.ChIndex = Reader.ReadInt(OLD_MAX_ACTOR_CHANNELS);
			}
			else
			{
				uint32 ChIndex;
				Reader.SerializeIntPacked(ChIndex);

				if (ChIndex >= (uint32)Channels.Num())
				{
					UE_LOGF(LogNet, Warning, "Bunch channel index exceeds channel limit");

					Close(ENetCloseResult::BunchBadChannelIndex);

					return;
				}

				Bunch.ChIndex = ChIndex;
			}

			const int32 CachedChIndex = Bunch.ChIndex;

			// if flag is set, remap channel index values, we're fast forwarding a replay checkpoint
			// and there should be no bunches for existing channels
			if (IsInternalAck() && bAllowExistingChannelIndex && (PacketEngineNetVer >= FEngineNetworkCustomVersion::ReplayDormancy))
			{
				if (ChannelIndexMap.Contains(Bunch.ChIndex))
				{
					Bunch.ChIndex = ChannelIndexMap[Bunch.ChIndex];
				}
				else
				{
					if (Channels[Bunch.ChIndex])
					{
						// this should be an open bunch if its the first time we've seen it
						if (Bunch.bOpen && (!Bunch.bPartial || Bunch.bPartialInitial))
						{
							FBitReaderMark Mark(Reader);

							Reader.ReadBit(); // bHasPackageMapExports
							Reader.ReadBit(); // bHasMustBeMappedGUIDs

							const uint8 bPartial = Reader.ReadBit(); // bPartial

							if (bPartial)
							{
								Reader.ReadBit(); // bPartialInitial
								if (bHasPartialCustomExportsFinalBit)
								{
									Reader.ReadBit(); // bPartialCustomExportsFinal
								}
								Reader.ReadBit(); // bPartialFinal
							}

							FName ChName;
							UPackageMap::StaticSerializeName(Reader, ChName);

							Mark.Pop(Reader);

							int32 FreeIndex = GetFreeChannelIndex(ChName);

							if (FreeIndex != INDEX_NONE)
							{
								UE_LOGF(LogNetTraffic, Verbose, "Adding channel mapping %d to %d", Bunch.ChIndex, FreeIndex);
								ChannelIndexMap.Add(Bunch.ChIndex, FreeIndex);
								Bunch.ChIndex = FreeIndex;
							}
							else
							{
								UE_LOGF(LogNetTraffic, Warning, "Unable to find free channel index");
								continue;
							}
						}
					}
				}
			}

			Bunch.bHasPackageMapExports	= Reader.ReadBit();
			Bunch.bHasMustBeMappedGUIDs	= Reader.ReadBit();
			Bunch.bPartial				= Reader.ReadBit();

			if (Bunch.bReliable)
			{
				if (IsInternalAck())
				{
					// We can derive the sequence for 100% reliable connections
					Bunch.ChSequence = InReliable[Bunch.ChIndex] + 1;
				}
				else
				{
					// If this is a reliable bunch, use the last processed reliable sequence to read the new reliable sequence
					Bunch.ChSequence = MakeRelative(Reader.ReadInt(MAX_CHSEQUENCE), InReliable[Bunch.ChIndex], MAX_CHSEQUENCE);
				}
			} 
			else if (Bunch.bPartial)
			{
				// If this is an unreliable partial bunch, we simply use packet sequence since we already have it
				Bunch.ChSequence = PacketId;
			}
			else
			{
				Bunch.ChSequence = 0;
			}

			Bunch.bPartialInitial = Bunch.bPartial ? Reader.ReadBit() : 0;
			Bunch.bPartialCustomExportsFinal = Bunch.bPartial && bHasPartialCustomExportsFinalBit ? Reader.ReadBit() : 0;
			Bunch.bPartialFinal = Bunch.bPartial ? Reader.ReadBit() : 0;

			if (PacketEngineNetVer < FEngineNetworkCustomVersion::ChannelNames)
			{
				uint32 ChType = (Bunch.bReliable || Bunch.bOpen) ? Reader.ReadInt(CHTYPE_MAX) : CHTYPE_None;
				switch (ChType)
				{
					case CHTYPE_Control:
						Bunch.ChName = NAME_Control;
						break;
					case CHTYPE_Voice:
						Bunch.ChName = NAME_Voice;
						break;
					case CHTYPE_Actor:
						Bunch.ChName = NAME_Actor;
						break;
				}
			}
			else
			{
				if (Bunch.bReliable || Bunch.bOpen)
				{
					UPackageMap::StaticSerializeName(Reader, Bunch.ChName);

					if( Reader.IsError() )
					{
						UE_LOGF(LogNet, Warning, "Channel name serialization failed.");

						HandleNetResultOrClose(ENetCloseResult::BunchChannelNameFail);

						return;
					}
				}
				else
				{
					Bunch.ChName = NAME_None;
				}
			}

			UChannel* Channel = Channels[Bunch.ChIndex];

			// If there's an existing channel and the bunch specified it's channel type, make sure they match.
			if (Channel && (Bunch.ChName != NAME_None) && (Bunch.ChName != Channel->ChName))
			{
				UE_LOGF(LogNet, Error,
						"Existing channel at index %d with type \"%ls\" differs from the incoming bunch's expected channel type, \"%ls\".",
						Bunch.ChIndex, *Channel->ChName.ToString(), *Bunch.ChName.ToString());

				Close(ENetCloseResult::BunchWrongChannelType);

				return;
			}

			int32 BunchDataBits  = Reader.ReadInt( UNetConnection::MaxPacket*8 );

			if ((Bunch.bClose || Bunch.bOpen) && UE_LOG_ACTIVE(LogNetDormancy,VeryVerbose) )
			{
				UE_LOGF(LogNetDormancy, VeryVerbose, "Received: %ls", *Bunch.ToString());
			}

			if (UE_LOG_ACTIVE(LogNetTraffic,VeryVerbose))
			{
				UE_LOGF(LogNetTraffic, VeryVerbose, "Received: %ls", *Bunch.ToString());
			}

			const int32 HeaderPos = Reader.GetPosBits();

			if(Reader.IsError())
			{
				UE_LOGF(LogNet, Warning, "Bunch header overflowed");

				HandleNetResultOrClose(ENetCloseResult::BunchHeaderOverflow);

				return;
			}

			// Trace bunch read
			UE_NET_TRACE_BUNCH_SCOPE(InTraceCollector, Bunch, StartPos, HeaderPos - StartPos);

			// Iris requires bitstream data allocations to be a multiple of four bytes.
			const int64 BunchDataBits32BitAligned = (BunchDataBits + 31) & ~31;
			Bunch.ResetData(Reader, BunchDataBits, BunchDataBits32BitAligned);
			if (Reader.IsError())
			{
				// Bunch claims it's larger than the enclosing packet.
				UE_LOG(LogNet, Warning, TEXT("Bunch data overflowed (%i %i+%i/%" INT64_FMT ")"), IncomingStartPos, HeaderPos, BunchDataBits,
						Reader.GetNumBits());

				HandleNetResultOrClose(ENetCloseResult::BunchDataOverflow);

				return;
			}

			if (Bunch.bHasPackageMapExports)
			{
				// Clients still send NetGUID.IsDefault()/FExportFlags.bHasPath packets to the server, separate from this check
				if (Driver->IsServer())
				{
					UE_LOGF(LogNetTraffic, Error, "UNetConnection::ReceivedPacket: Server received bHasPackageMapExports packet.");

					Close(ENetCloseResult::BunchServerPackageMapExports);

					return;
				}

				Driver->NetGUIDInBytes += (BunchDataBits + (HeaderPos - IncomingStartPos)) >> 3;

				if (IsInternalAck())
				{
					// NOTE - For replays, we do this even earlier, to try and load this as soon as possible,
					// in case there's an issue creating the channel. If a replay fails to create a channel, we want to salvage as much as possible
					Cast<UPackageMapClient>(PackageMap)->ReceiveNetGUIDBunch(Bunch);

					if (Bunch.IsError())
					{
						UE_LOGF(LogNetTraffic, Error, "UNetConnection::ReceivedPacket: Bunch.IsError() after ReceiveNetGUIDBunch. ChIndex: %i",
								Bunch.ChIndex);
					}
				}
			}

			if (Bunch.bReliable)
			{
				UE_LOGF(LogNetTraffic, Verbose, "   Reliable Bunch, Channel %i Sequence %i: Size %.1f+%.1f",
						Bunch.ChIndex, Bunch.ChSequence, (HeaderPos-IncomingStartPos)/8.f, (Reader.GetPosBits()-HeaderPos)/8.f);
			}
			else
			{
				UE_LOGF(LogNetTraffic, Verbose, "   Unreliable Bunch, Channel %i: Size %.1f+%.1f",
						Bunch.ChIndex, (HeaderPos-IncomingStartPos)/8.f, (Reader.GetPosBits()-HeaderPos)/8.f);
			}

			if (Bunch.bOpen)
			{
				UE_LOGF(LogNetTraffic, Verbose, "   bOpen Bunch, Channel %i Sequence %i: Size %.1f+%.1f",
						Bunch.ChIndex, Bunch.ChSequence, (HeaderPos-IncomingStartPos)/8.f, (Reader.GetPosBits()-HeaderPos)/8.f );
			}

			if ( Channels[Bunch.ChIndex] == NULL && ( Bunch.ChIndex != 0 || Bunch.ChName != NAME_Control ) )
			{
				// Can't handle other channels until control channel exists.
				if ( Channels[0] == NULL )
				{
					UE_LOGF(LogNetTraffic, Log,
							"ReceivedPacket: Received non-control bunch before control channel was created. ChIndex: %i, ChName: %ls",
							Bunch.ChIndex, *Bunch.ChName.ToString());

					Close(ENetCloseResult::BunchPrematureControlChannel);

					return;
				}
				// on the server, if we receive bunch data for a channel that doesn't exist while we're still logging in,
				// it's either a broken client or a new instance of a previous connection,
				// so reject it
				else if ( PlayerController == NULL && Driver->ClientConnections.Contains( this ) )
				{
					UE_LOGF(LogNet, Warning,
							"ReceivedPacket: Received non-control bunch before player controller was assigned. ChIndex: %i, ChName: %ls",
							Bunch.ChIndex, *Bunch.ChName.ToString());

					Close(ENetCloseResult::BunchPrematureChannel);

					return;
				}
			}
			// ignore control channel close if it hasn't been opened yet
			if ( Bunch.ChIndex == 0 && Channels[0] == NULL && Bunch.bClose && Bunch.ChName == NAME_Control )
			{
				UE_LOGF(LogNetTraffic, Log, "ReceivedPacket: Received control channel close before open");

				Close(ENetCloseResult::BunchPrematureControlClose);

				return;
			}

			// Receiving data.

			// We're on a 100% reliable connection and we are rolling back some data.
			// In that case, we can generally ignore these bunches.
			if (IsInternalAck() && bAllowExistingChannelIndex)
			{
				if (PacketEngineNetVer < FEngineNetworkCustomVersion::ReplayDormancy)
				{
					if (Channel)
					{
						// This was an open bunch for a channel that's already opened.
						// We can ignore future bunches from this channel.
						const bool bNewlyOpenedActorChannel = Bunch.bOpen && (Bunch.ChName == NAME_Actor) &&
																(!Bunch.bPartial || Bunch.bPartialInitial);

						if (bNewlyOpenedActorChannel)
						{
							FNetworkGUID ActorGUID = GetActorGUIDFromOpenBunch(Bunch);

							if (!Bunch.IsError())
							{
								IgnoringChannels.Add(Bunch.ChIndex, ActorGUID);
							}
							else
							{
								UE_LOGF(LogNetTraffic, Error,
										"UNetConnection::ReceivedPacket: Unable to read actor GUID for ignored bunch. (Channel %d)",
										Bunch.ChIndex);
							}
						}

						if (IgnoringChannels.Contains(Bunch.ChIndex))
						{
							if (Bunch.bClose && (!Bunch.bPartial || Bunch.bPartialFinal))
							{
								FNetworkGUID ActorGUID = IgnoringChannels.FindAndRemoveChecked(Bunch.ChIndex);
								if (ActorGUID.IsStatic())
								{
									UObject* FoundObject = Driver->GetNetGuidCache()->GetObjectFromNetGUID(ActorGUID, false);
									if (AActor* StaticActor = Cast<AActor>(FoundObject))
									{
										DestroyIgnoredActor(StaticActor);
									}
									else
									{
										ensure(FoundObject == nullptr);

										UE_LOGF(LogNetTraffic, Log,
												"UNetConnection::ReceivedPacket: Unable to find static actor to cleanup for ignored bunch. " "(Channel %d NetGUID %ls)", Bunch.ChIndex, *ActorGUID.ToString());
									}
								}
							}

							UE_LOGF(LogNetTraffic, Log, "Ignoring bunch for already open channel: %i", Bunch.ChIndex);
							continue;
						}
					}
				}
				else
				{
					const bool bCloseBunch = Bunch.bClose && (!Bunch.bPartial || Bunch.bPartialFinal);

					if (bCloseBunch && ChannelIndexMap.Contains(CachedChIndex))
					{
						check(ChannelIndexMap[CachedChIndex] == Bunch.ChIndex);

						UE_LOGF(LogNetTraffic, Verbose, "Removing channel mapping %d to %d", CachedChIndex, Bunch.ChIndex);
						ChannelIndexMap.Remove(CachedChIndex);
					}
				}
			}

			// Ignore if reliable packet has already been processed.
			if ( Bunch.bReliable && Bunch.ChSequence <= InReliable[Bunch.ChIndex] )
			{
				UE_LOGF(LogNetTraffic, Log, "UNetConnection::ReceivedPacket: Received outdated bunch (Channel %d Current Sequence %i)",
						Bunch.ChIndex, InReliable[Bunch.ChIndex]);

				check( !IsInternalAck() );		// Should be impossible with 100% reliable connections
				continue;
			}
		
			// If opening the channel with an unreliable packet, check that it is "bNetTemporary", otherwise discard it
			if( !Channel && !Bunch.bReliable )
			{
				// Unreliable bunches that open channels should be bOpen && (bClose || bPartial)
				// NetTemporary usually means one bunch that is unreliable (bOpen and bClose):	1(bOpen, bClose)
				// But if that bunch export NetGUIDs, it will get split into 2 bunches:			1(bOpen, bPartial) - 2(bClose).
				// (the initial actor bunch itself could also be split into multiple bunches. So bPartial is the right check here)

				const bool ValidUnreliableOpen = Bunch.bOpen && (Bunch.bClose || Bunch.bPartial);
				if (!ValidUnreliableOpen)
				{
					if ( IsInternalAck() )
					{
						// Should be impossible with 100% reliable connections
						UE_LOGF(LogNetTraffic, Error,
								"      Received unreliable bunch before open with reliable connection (Channel %d Current Sequence %i)",
								Bunch.ChIndex, InReliable[Bunch.ChIndex]);
					}
					else
					{
						// Simply a log (not a warning, since this can happen under normal conditions, like from a re-join, etc)
						UE_LOGF(LogNetTraffic, Log, "      Received unreliable bunch before open (Channel %d Current Sequence %i)",
								Bunch.ChIndex, InReliable[Bunch.ChIndex]);
					}

					// Since we won't be processing this packet, don't ack it
					// We don't want the sender to think this bunch was processed when it really wasn't
					bOutSkipAck = true;
					continue;
				}
			}

			// Create channel if necessary.
			if (Channel == nullptr)
			{
				if (RejectedChans.Contains(Bunch.ChIndex))
				{
					UE_LOGF(LogNetTraffic, Log,
							"      Ignoring Bunch for ChIndex %i, as the channel was already rejected while processing this packet.",
							Bunch.ChIndex);

					continue;
				}

				// Validate channel type.
				if ( !Driver->IsKnownChannelName( Bunch.ChName ) )
				{
					UE_LOGF(LogNet, Warning, "ReceivedPacket: Connection unknown channel type (%ls)", *Bunch.ChName.ToString());

					Close(ENetCloseResult::UnknownChannelType);

					return;
				}

				// Ignore incoming data on channel types clients can't create. Can occur if we've incoming data when server is closing a channel
				if ( Driver->IsServer() && (Driver->ChannelDefinitionMap[Bunch.ChName].bClientOpen == false) )
				{
					UE_LOGF(LogNetTraffic, Warning, "      Ignoring Bunch Create received from client since only server is allowed to "
						"create this type of channel: Bunch  %i: ChName %ls, ChSequence: %i, bReliable: %i, bPartial: %i, "
						"bPartialInitial: %i, bPartialFinal: %i", Bunch.ChIndex, *Bunch.ChName.ToString(), Bunch.ChSequence,
							(int)Bunch.bReliable, (int)Bunch.bPartial, (int)Bunch.bPartialInitial, (int)Bunch.bPartialFinal);

					RejectedChans.AddUnique(Bunch.ChIndex);

					continue;
				}

				// peek for guid
				if (IsInternalAck() && bIgnoreActorBunches)
				{
					if (Bunch.bOpen && (!Bunch.bPartial || Bunch.bPartialInitial) && (Bunch.ChName == NAME_Actor))
					{
						FBitReaderMark Mark(Bunch);
						FNetworkGUID ActorGUID = GetActorGUIDFromOpenBunch(Bunch);
						Mark.Pop(Bunch);

						if (ActorGUID.IsValid() && !ActorGUID.IsDefault())
						{
							if (IgnoredBunchGuids.Contains(ActorGUID))
							{
								UE_LOGF(LogNetTraffic, Verbose, "Adding Channel: %i to ignore list, ignoring guid: %ls",
										Bunch.ChIndex, *ActorGUID.ToString());

								IgnoredBunchChannels.Add(Bunch.ChIndex);

								continue;
							}
							else
							{
								if (IgnoredBunchChannels.Remove(Bunch.ChIndex))
								{
									UE_LOGF(LogNetTraffic, Verbose, "Removing Channel: %i from ignore list, got new guid: %ls",
											Bunch.ChIndex, *ActorGUID.ToString());
								}
							}
						}
						else
						{
							UE_LOGF(LogNetTraffic, Warning, "Open bunch with invalid actor guid, Channel: %i", Bunch.ChIndex);
						}
					}
					else
					{
						if (IgnoredBunchChannels.Contains(Bunch.ChIndex))
						{
							UE_LOGF(LogNetTraffic, Verbose, "Ignoring bunch on channel: %i", Bunch.ChIndex);
							continue;
						}
					}
				}

				// Reliable (either open or later), so create new channel.
				UE_LOGF(LogNetTraffic, Log, "      Bunch Create %i: ChName %ls, ChSequence: %i, bReliable: %i, bPartial: %i, " "bPartialInitial: %i, bPartialFinal: %i", Bunch.ChIndex, *Bunch.ChName.ToString(), Bunch.ChSequence,
						(int)Bunch.bReliable, (int)Bunch.bPartial, (int)Bunch.bPartialInitial, (int)Bunch.bPartialFinal);

				Channel = CreateChannelByName( Bunch.ChName, EChannelCreateFlags::None, Bunch.ChIndex );

				// Notify the server of the new channel.
				if( Driver->Notify == nullptr || !Driver->Notify->NotifyAcceptingChannel( Channel ) )
				{
					// Channel refused, so close it, flush it, and delete it.
					UE_LOGF(LogNet, Verbose, "      NotifyAcceptingChannel Failed! Channel: %ls", *Channel->Describe() );

					RejectedChans.AddUnique(Bunch.ChIndex);

					FOutBunch CloseBunch( Channel, true );
					check(!CloseBunch.IsError());
					check(CloseBunch.bClose);
					CloseBunch.bReliable = 1;
					Channel->SendBunch( &CloseBunch, false );
					FlushNet();
					Channel->ConditionalCleanUp(false, EChannelCloseReason::Destroyed);
					if( Bunch.ChIndex==0 )
					{
						UE_LOGF(LogNetTraffic, Log, "Channel 0 create failed" );
						SetConnectionState(EConnectionState::USOCK_Closed);
					}
					continue;
				}
			}

			Bunch.bIgnoreRPCs = bIgnoreRPCs;

			// Dispatch the raw, unsequenced bunch to the channel.
			bool bLocalSkipAck = false;

			if (Channel != nullptr)
			{
				// Warning: May destroy channel
				Channel->ReceivedRawBunch(Bunch, bLocalSkipAck);
			}

			if ( bLocalSkipAck )
			{
				bOutSkipAck = true;
			}
			Driver->InBunches++;
			Driver->InTotalBunches++;
			Driver->InTotalReliableBunches += Bunch.bReliable ? 1 : 0;

			if (Bunch.IsCriticalError() || Bunch.IsError())
			{
				bOutHasBunchErrors = true;

				// Disconnect if we received a corrupted packet from the client (eg server crash attempt).
				if (bIsServer)
				{
					UE_LOGF(LogNetTraffic, Error, "Received corrupted packet data with SequenceId: %d from client %ls. Disconnecting.", PacketId, *LowLevelGetRemoteAddress());

					Close(AddToAndConsumeChainResultPtr(Bunch.ExtendedError, ENetCloseResult::CorruptData));

					return;
				}
				else
				{
					UE_LOGF(LogNetTraffic, Error, "Received corrupted packet data with SequenceId: %d from server %ls", PacketId, *LowLevelGetRemoteAddress());
					// If Iris reports errors they are unrecoverable.
					if (Driver->GetReplicationSystem() != nullptr)
					{
						ensureMsgf(false, TEXT("Received corrupted packet data. Iris cannot recover from this."));
						Close(AddToAndConsumeChainResultPtr(Bunch.ExtendedError, ENetCloseResult::CorruptData));
						return;
					}
				}
			}
			// In replay, if the bunch generated an error but the channel isn't actually open, clean it up so the channel index remains free
			if (IsInternalAck() && Bunch.IsError() && Channel && !Channel->OpenedLocally && !Channel->OpenAcked)
			{
				UE_LOGF(LogNetTraffic, Warning, "Replay cleaning up channel that couldn't be opened: %ls", *Channel->Describe());
				Channel->ConditionalCleanUp(true, EChannelCloseReason::Destroyed);
			}
		}
	}
}

void UNetConnection::RestoreRemappedChannel(const int32 ChIndex)
{
	TArray<int32, TInlineAllocator<16>> RemapChain;

	if (ChannelIndexMap.Contains(ChIndex))
	{
		RemapChain.Add(ChIndex);

		while (const int32* ChainIndex = ChannelIndexMap.FindKey(RemapChain[RemapChain.Num() - 1]))
		{
			const int32 FoundIdx = RemapChain.Find(*ChainIndex);
			
			if (!ensureMsgf(FoundIdx == INDEX_NONE, TEXT("RestoreRemappedChannel: Loop detected in channel remaps!")))
			{
				break;
			}

			RemapChain.Add(*ChainIndex);
		}
	}

	for (int32 i = RemapChain.Num() - 1; i >= 0; --i)
	{
		const int32 SourceIndex = RemapChain[i];

		if (ChannelIndexMap.Contains(SourceIndex))
		{
			const int32 RemappedIndex = ChannelIndexMap[SourceIndex];

			UChannel* SrcChannel = Channels[RemappedIndex];
			UChannel* DstChannel = Channels[SourceIndex];

			// this channel should still exist, but the location we want to swap it back to should be empty
			if (ensureMsgf(SrcChannel && !DstChannel, TEXT("Source should exist: [%s] Destination should be null: [%s]"), SrcChannel ? *SrcChannel->Describe() : TEXT("null"), DstChannel ? *DstChannel->Describe() : TEXT("null")))
			{
				SrcChannel->ChIndex = SourceIndex;

				Swap(Channels[SourceIndex], Channels[RemappedIndex]);
				Swap(InReliable[SourceIndex], InReliable[RemappedIndex]);

				ChannelIndexMap.Remove(SourceIndex);
			}
		}
	}
}

void UNetConnection::SetAllowExistingChannelIndex(bool bAllow)
{
	check(IsInternalAck() && IsReplay());

	bAllowExistingChannelIndex = bAllow;
	IgnoringChannels.Reset();

	if (!bAllow)
	{
		// We could have remapped a channel that remained open because we were temporarily using its index before it was closed
		// Go ahead and restore those indices now (Channels + InReliable)
		TArray<int32> RemappedKeys;
		ChannelIndexMap.GenerateKeyArray(RemappedKeys);

		// clean up any broken channels first
		for (const int32 ChIndex : RemappedKeys)
		{
			if (UChannel* Channel = Channels[ChIndex])
			{
				// It is possible the index we want to swap back to wasn't cleaned up because the channel was marked broken by backwards compatibility
				if (Channel->Broken)
				{
					if (UActorChannel* ActorChannel = Cast<UActorChannel>(Channel))
					{
						// look for a queued close bunch
						for (FInBunch* InBunch : ActorChannel->QueuedBunches)
						{
							if (InBunch && InBunch->bClose)
							{
								UE_LOGF(LogNet, Warning, "SetAllowExistingChannelIndex:  Cleaning up broken channel: %ls", *ActorChannel->Describe());
								ActorChannel->ConditionalCleanUp(true, InBunch->CloseReason);
								break;
							}
						}

						// broken channel but the actor never spawned, should be safe to remove
						if (ActorChannel->Actor == nullptr)
						{
							UE_LOGF(LogNet, Warning, "SetAllowExistingChannelIndex:  Cleaning up broken channel: %ls", *ActorChannel->Describe());
							ActorChannel->ConditionalCleanUp(true, EChannelCloseReason::Destroyed);
						}
					}
				}
			}
		}

		ChannelIndexMap.GenerateKeyArray(RemappedKeys);

		// now try to remap the channels
		for (const int32 Key : RemappedKeys)
		{
			RestoreRemappedChannel(Key);
		}
	}

	ChannelIndexMap.Reset();
}

void UNetConnection::SetProxyConnection(bool bEnabled)
{
	bIsProxyConnection = bEnabled;

	if (UReplicationSystem* DriverReplicationSystem = Driver->GetReplicationSystem())
	{
		const uint32 ConnectionId = GetConnectionHandle().GetParentConnectionId();
		DriverReplicationSystem->SetProxyConnection(ConnectionId, bEnabled);
	}
}

bool UNetConnection::IsProxyConnection() const
{
	return bIsProxyConnection;
}

void UNetConnection::SetIgnoreActorBunches(bool bInIgnoreActorBunches, TSet<FNetworkGUID>&& InIgnoredBunchGuids)
{
	check(IsInternalAck());
	bIgnoreActorBunches = bInIgnoreActorBunches;

	IgnoredBunchChannels.Empty();
	InIgnoredBunchGuids.Empty();

	if (bIgnoreActorBunches)
	{
		IgnoredBunchGuids = MoveTemp(InIgnoredBunchGuids);
	}
}

void UNetConnection::SetReserveDestroyedChannels(bool bInReserveChannels)
{
	check(IsInternalAck());
	bReserveDestroyedChannels = bInReserveChannels;
}

void UNetConnection::SetIgnoreReservedChannels(bool bInIgnoreReservedChannels)
{
	check(IsInternalAck());
	bIgnoreReservedChannels = bInIgnoreReservedChannels;

	ReservedChannels.Empty();
}

void UNetConnection::PrepareWriteBitsToSendBuffer(const int32 SizeInBits, const int32 ExtraSizeInBits)
{
	ValidateSendBuffer();

#if !UE_BUILD_SHIPPING
	// Now that the stateless handshake is responsible for initializing the packet sequence numbers,
	//	we can't allow any packets to be written to the send buffer until after this has completed
	if (CVarRandomizeSequence.GetValueOnAnyThread() > 0)
	{
		checkf(!Handler.IsValid() || Handler->IsFullyInitialized(), TEXT("Attempted to write to send buffer before packet handler was fully initialized. Connection: %s"), *Describe());
	}
#endif

	const int32 TotalSizeInBits = SizeInBits + ExtraSizeInBits;

	// Flush if we can't add to current buffer
	if ( TotalSizeInBits > GetFreeSendBufferBits() )
	{
		FlushNet();
	}

#if UE_NET_TRACE_ENABLED
	// If tracing is enabled setup the NetTraceCollector for outgoing data
	if (SendBuffer.GetNumBits() == 0)
	{
		OutTraceCollector = UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace);
	}
#endif

	// If this is the start of the queue, make sure to add the packet id
	if ( SendBuffer.GetNumBits() == 0 && !IsInternalAck() )
	{
		UE_NET_TRACE_SCOPE(PacketHeaderAndInfo, SendBuffer, OutTraceCollector, ENetTraceVerbosity::Trace);

		// Write Packet Header, before sending the packet we will go back and rewrite the data
		WritePacketHeader(SendBuffer);

		// Pre-write the bits for the packet info
		WriteDummyPacketInfo(SendBuffer);

		// We do not allow the first bunch to merge with the ack data as this will "revert" the ack data.
		AllowMerge = false;
	
		// Update stats for PacketIdBits and ackdata (also including the data used for packet RTT and saturation calculations)
		int64 BitsWritten = SendBuffer.GetNumBits();
		NumPacketIdBits += FNetPacketNotify::SequenceNumberT::SeqNumberBits;
		NumAckBits += BitsWritten - FNetPacketNotify::SequenceNumberT::SeqNumberBits;

		// Report stats to profiler
		NETWORK_PROFILER( GNetworkProfiler.TrackSendAck( NumAckBits, this ) );

		ValidateSendBuffer();
	}
}

int32 UNetConnection::WriteBitsToSendBufferInternal( 
	const uint8 *	Bits, 
	const int32		SizeInBits, 
	const uint8 *	ExtraBits, 
	const int32		ExtraSizeInBits,
	EWriteBitsDataType DataType)
{
	// Remember start position in case we want to undo this write, no meaning to undo the header write as this is only used to pop bunches and the header should not count towards the bunch
	// Store this after the possible flush above so we have the correct start position in the case that we do flush
	LastStart = FBitWriterMark( SendBuffer );

	// Add the bits to the queue
	if ( SizeInBits )
	{
		SendBuffer.SerializeBits( const_cast< uint8* >( Bits ), SizeInBits );
		ValidateSendBuffer();
	}

	// Add any extra bits
	if ( ExtraSizeInBits )
	{
		SendBuffer.SerializeBits( const_cast< uint8* >( ExtraBits ), ExtraSizeInBits );
		ValidateSendBuffer();
	}

	const int32 RememberedPacketId = OutPacketId;

	switch ( DataType )
	{
		case EWriteBitsDataType::Bunch:
			NumBunchBits += SizeInBits + ExtraSizeInBits;
			break;
		default:
			break;
	}

	// Flush now if we are full
	if (GetFreeSendBufferBits() == 0
#if !UE_BUILD_SHIPPING
		|| CVarForceNetFlush.GetValueOnAnyThread() != 0
#endif
		)
	{
		FlushNet();
	}

	return RememberedPacketId;
}

int32 UNetConnection::WriteBitsToSendBuffer( 
	const uint8 *	Bits, 
	const int32		SizeInBits, 
	const uint8 *	ExtraBits, 
	const int32		ExtraSizeInBits,
	EWriteBitsDataType DataType)
{
	// Flush packet as needed and begin new packet
	PrepareWriteBitsToSendBuffer(SizeInBits, ExtraSizeInBits);

	// Write the data and flush if the packet is full, return value is the packetId into which the data was written
	return WriteBitsToSendBufferInternal(Bits, SizeInBits, ExtraBits, ExtraSizeInBits, DataType);
}

/** Returns number of bits left in current packet that can be used without causing a flush  */
int64 UNetConnection::GetFreeSendBufferBits()
{
	// If we haven't sent anything yet, make sure to account for the packet header + trailer size
	// Otherwise, we only need to account for trailer size
	const int32 ExtraBits = ( SendBuffer.GetNumBits() > 0 ) ? MAX_PACKET_TRAILER_BITS : MAX_PACKET_HEADER_BITS + MAX_PACKET_TRAILER_BITS;

	const int32 NumberOfFreeBits = SendBuffer.GetMaxBits() - ( SendBuffer.GetNumBits() + ExtraBits );

	check( NumberOfFreeBits >= 0 );

	return NumberOfFreeBits;
}

void UNetConnection::PopLastStart()
{
	UE_NET_TRACE_POP_SEND_BUNCH(OutTraceCollector);

	NumBunchBits -= SendBuffer.GetNumBits() - LastStart.GetNumBits();
	LastStart.Pop(SendBuffer);
	NETWORK_PROFILER(GNetworkProfiler.PopSendBunch(this));
}

TSharedPtr<FObjectReplicator> UNetConnection::CreateReplicatorForNewActorChannel(UObject* Object)
{
	TSharedPtr<FObjectReplicator> NewReplicator = MakeShareable(new FObjectReplicator());
	NewReplicator->InitWithObject( Object, this, true );
	return NewReplicator;
}

int32 UNetConnection::SendRawBunch(FOutBunch& Bunch, bool InAllowMerge, const FNetTraceCollector* BunchCollector)
{
	ValidateSendBuffer();
	check(!Bunch.ReceivedAck);
	check(!Bunch.IsError());
	UE_IRIS_PARALLEL_EXPR(Driver->LockNetDriverStats(true));
	Driver->OutBunches++;
	Driver->OutTotalBunches++;
	Driver->OutTotalReliableBunches += Bunch.bReliable ? 1 : 0;
	UE_IRIS_PARALLEL_EXPR(Driver->UnlockNetDriverStats(true));

	// Build header.
	SendBunchHeader.Reset();

	const bool bIsOpenOrClose = Bunch.bOpen || Bunch.bClose;
	const bool bIsOpenOrReliable = Bunch.bOpen || Bunch.bReliable;

	SendBunchHeader.WriteBit(bIsOpenOrClose);
	if (bIsOpenOrClose)
	{
		SendBunchHeader.WriteBit(Bunch.bOpen);
		SendBunchHeader.WriteBit(Bunch.bClose);
		if (Bunch.bClose)
		{
			uint32 Value = (uint32)Bunch.CloseReason;
			SendBunchHeader.SerializeInt(Value, (uint32)EChannelCloseReason::MAX);
		}
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SendBunchHeader.WriteBit(Bunch.bIsReplicationPaused);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SendBunchHeader.WriteBit(Bunch.bReliable);

	uint32 ChIndex = Bunch.ChIndex;
	SendBunchHeader.SerializeIntPacked(ChIndex); 

	SendBunchHeader.WriteBit(Bunch.bHasPackageMapExports);
	SendBunchHeader.WriteBit(Bunch.bHasMustBeMappedGUIDs);
	SendBunchHeader.WriteBit(Bunch.bPartial);

	if (Bunch.bReliable && !IsInternalAck())
	{
		SendBunchHeader.WriteIntWrapped(Bunch.ChSequence, MAX_CHSEQUENCE);
	}

	if (Bunch.bPartial)
	{
		SendBunchHeader.WriteBit(Bunch.bPartialInitial);
		SendBunchHeader.WriteBit(Bunch.bPartialCustomExportsFinal);
		SendBunchHeader.WriteBit(Bunch.bPartialFinal);
	}

	if (bIsOpenOrReliable)
	{
		UPackageMap::StaticSerializeName(SendBunchHeader, Bunch.ChName);
	}
	
	SendBunchHeader.WriteIntWrapped(Bunch.GetNumBits(), UNetConnection::MaxPacket * 8);

#if DO_CHECK
	if (UNLIKELY(SendBunchHeader.IsError()))
	{
		const bool bDidReplicateChannelName = bIsOpenOrReliable;
		const bool bDoesChannelNameReplicateAsString = !Bunch.ChName.ToEName() || !ShouldReplicateAsInteger(*Bunch.ChName.ToEName(), Bunch.ChName);

		checkf(false, TEXT("SendBunchHeader Error: Bunch = %s,  Channel Name Serialized As String: %d"),
			*Bunch.ToString(), !!(bDidReplicateChannelName && bDoesChannelNameReplicateAsString));
	}
#endif

	check(!SendBunchHeader.IsError());

	// Remember start position.
	AllowMerge = InAllowMerge;
	Bunch.Time = Driver->GetElapsedTime();

	if (bIsOpenOrClose && UE_LOG_ACTIVE(LogNetDormancy,VeryVerbose))
	{
		UE_LOGF(LogNetDormancy, VeryVerbose, "Sending: %ls", *Bunch.ToString());
	}

	if (UE_LOG_ACTIVE(LogNetTraffic, VeryVerbose))
	{
		UE_LOGF(LogNetTraffic, VeryVerbose, "Sending: %ls", *Bunch.ToString());
	}

	NETWORK_PROFILER(GNetworkProfiler.PushSendBunch(this, &Bunch, SendBunchHeader.GetNumBits(), Bunch.GetNumBits()));

	const int32 BunchHeaderBits = SendBunchHeader.GetNumBits();
	const int32 BunchBits = Bunch.GetNumBits();

	// If the bunch does not fit in the current packet, 
	// flush packet now so that we can report collected stats in the correct scope
	PrepareWriteBitsToSendBuffer(BunchHeaderBits, BunchBits);

	// We want to mark the packet in which we write the data as TimeSensitive
	// Note: we want to mark the packet as TimeSensitive here, as PrepareWriteBitsToSendBuffer might flush the packet
	TimeSensitive = 1;

	// Report bunch
	UE_NET_TRACE_END_BUNCH(OutTraceCollector, Bunch, Bunch.ChName, 0, BunchHeaderBits, BunchBits, BunchCollector);

	// Write the bits to the buffer and remember the packet id used
	Bunch.PacketId = WriteBitsToSendBufferInternal(SendBunchHeader.GetData(), BunchHeaderBits, Bunch.GetData(), BunchBits, EWriteBitsDataType::Bunch);

	// Track channels that wrote data to this packet.
	FChannelRecordImpl::PushChannelRecord(ChannelRecord, Bunch.PacketId, Bunch.ChIndex);

	UE_LOG(LogNetTraffic, Verbose, TEXT("UNetConnection::SendRawBunch. ChIndex: %d. Bits: %" INT64_FMT ". PacketId: %d"), Bunch.ChIndex, Bunch.GetNumBits(), Bunch.PacketId);

	if (PackageMap && (Bunch.bHasPackageMapExports || Bunch.bPartialCustomExportsFinal))
	{
		PackageMap->NotifyBunchCommit(Bunch.PacketId, &Bunch);
	}

	if (Bunch.bHasPackageMapExports)
	{
		UE_IRIS_PARALLEL_EXPR(Driver->LockNetDriverStats(true));
		Driver->NetGUIDOutBytes += (SendBunchHeader.GetNumBits() + Bunch.GetNumBits()) >> 3;
		UE_IRIS_PARALLEL_EXPR(Driver->UnlockNetDriverStats(true));
	}

	if (bAutoFlush)
	{
		FlushNet();
	}

	return Bunch.PacketId;
}

int32 UNetConnection::GetFreeChannelIndex(const FName& ChName) const
{
	int32 ChIndex = INDEX_NONE;
	int32 FirstChannel = 1;

	const int32 StaticChannelIndex = Driver->ChannelDefinitionMap[ChName].StaticChannelIndex;
	if (StaticChannelIndex != INDEX_NONE)
	{
		FirstChannel = StaticChannelIndex;
	}

	// Search the channel array for an available location
	for (ChIndex = FirstChannel; ChIndex < Channels.Num(); ChIndex++)
	{
		const bool bIgnoreReserved = bIgnoreReservedChannels && ReservedChannels.Contains(ChIndex);
		const bool bIgnoreRemapped = bAllowExistingChannelIndex && ChannelIndexMap.Contains(ChIndex);

		if (!Channels[ChIndex] && !bIgnoreReserved && !bIgnoreRemapped)
		{
			break;
		}
	}

	if (ChIndex == Channels.Num())
	{
		ChIndex = INDEX_NONE;
	}

	return ChIndex;
}

UChannel* UNetConnection::CreateChannelByName(const FName& ChName, EChannelCreateFlags CreateFlags, int32 ChIndex)
{
	LLM_SCOPE_BYTAG(NetConnection);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetConnection_CreateChannelByName);

	check(Driver->IsKnownChannelName(ChName));
	checkf(GetUChildConnection() == nullptr, TEXT("Creating channels on child connections is unsupported: ChName: %s ChIndex: %d Connection: %s"), *ChName.ToString(), ChIndex, *Describe());
	AssertValid();

	// If no channel index was specified, find the first available.
	if (ChIndex == INDEX_NONE)
	{
		ChIndex = GetFreeChannelIndex(ChName);

		// Fail to create if the channel array is full
		if (ChIndex == INDEX_NONE)
		{
			if (!bHasWarnedAboutChannelLimit)
			{
				bHasWarnedAboutChannelLimit = true;
				UE_LOGF(LogNetTraffic, Warning, "No free channel could be found in the channel list (current limit is %d channels) for connection with owner %ls. Consider increasing the max channels allowed using net.MaxChannelSize.", Channels.Num(), *GetNameSafe(OwningActor));
			}

			return nullptr;
		}
	}

	// Make sure channel is valid.
	check(Channels.IsValidIndex(ChIndex));
	check(Channels[ChIndex] == nullptr);

	// Create channel.
	UChannel* Channel = Driver->GetOrCreateChannelByName(ChName);
	check(Channel);
	Channel->Init( this, ChIndex, CreateFlags );
	Channels[ChIndex] = Channel;
	OpenChannels.Add(Channel);

	if (Driver->ChannelDefinitionMap[ChName].bTickOnCreate)
	{
		StartTickingChannel(Channel);
	}

	UE_LOGF(LogNetTraffic, Log, "Created channel %i of type %ls", ChIndex, *ChName.ToString());

	return Channel;
}

/**
 * @return Finds the voice channel for this connection or NULL if none
 */
UVoiceChannel* UNetConnection::GetVoiceChannel()
{
	check(Driver);
	if (!Driver->IsKnownChannelName(NAME_Voice))
	{
		return nullptr;
	}

	int32 VoiceChannelIndex = Driver->ChannelDefinitionMap[NAME_Voice].StaticChannelIndex;
	check(Channels.IsValidIndex(VoiceChannelIndex));

	return Channels[VoiceChannelIndex] != nullptr && Channels[VoiceChannelIndex]->ChName == NAME_Voice ?
		Cast<UVoiceChannel>(Channels[VoiceChannelIndex]) : nullptr;
}

float UNetConnection::GetTimeoutValue()
{
	check(Driver);
#if !UE_BUILD_SHIPPING
	if (Driver->bNoTimeouts)
	{
		// APlayerController depends on this timeout to destroy itself and free up
		// its resources, so we have to handle this case here as well
		return bPendingDestroy ? 2.f : UE_MAX_FLT;
	}
#endif

	float Timeout = Driver->InitialConnectTimeout;

	if ((GetConnectionState() != USOCK_Pending) && (bPendingDestroy || (OwningActor && OwningActor->UseShortConnectTimeout())))
	{
		const float ConnectionTimeout = Driver->ConnectionTimeout;

		// If the connection is pending destroy give it 2 seconds to try to finish sending any reliable packets
		Timeout = bPendingDestroy ? 2.f : ConnectionTimeout;
	}

	// Longtimeouts allows a multiplier to be added to get correct disconnection behavior
	// with with additional leniancy when required. Implicit in debug/editor builds
	static bool LongTimeouts = FParse::Param(FCommandLine::Get(), TEXT("longtimeouts"));

	if (Driver->TimeoutMultiplierForUnoptimizedBuilds > 0 
		&& (LongTimeouts || WITH_EDITOR || UE_BUILD_DEBUG)
		)
	{
		Timeout *= Driver->TimeoutMultiplierForUnoptimizedBuilds;
	}

	return Timeout;
}

void UNetConnection::Tick(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(Stat_NetConnectionTick);

	AssertValid();

	// Acquire a coarse grain lock which protects access to shared state such as NetDriver internals, lag simulation, disconnect logic etc
	// It is unlocked whilst ticking individual channels which is where the most time consuming work is done
	UE_IRIS_PARALLEL_EXPR(UE::TScopeLock ScopeLock(GetSharedNetTickMutex()));

	// Get frame time.
	const double CurrentRealtimeSeconds = FPlatformTime::Seconds();

	// Lag simulation.
#if DO_ENABLE_NET_TEST
	UpdateDelayedPackets(CurrentRealtimeSeconds);
#endif

	// if this is 0 it's our first tick since init, so start our real-time tracking from here
	if (LastTime == 0.0)
	{
		LastTime = CurrentRealtimeSeconds;
		LastReceiveRealtime = CurrentRealtimeSeconds;
		LastGoodPacketRealtime = CurrentRealtimeSeconds;
	}

	FrameTime = CurrentRealtimeSeconds - LastTime;
	const int32 MaxNetTickRate = Driver->MaxNetTickRate;
	float EngineTickRate = Driver->CachedEngineMaxTickRate;
	// We want to make sure the DesiredTickRate stays at <= 0 if there's no tick rate limiting of any kind, since it's used later in the function for bandwidth limiting.
	if (MaxNetTickRate > 0 && EngineTickRate <= 0.0f)
	{
		EngineTickRate = MAX_flt;
	}
	const float MaxNetTickRateFloat = MaxNetTickRate > 0 ? float(MaxNetTickRate) : MAX_flt;
	const float DesiredTickRate = FMath::Clamp(EngineTickRate, 0.0f, MaxNetTickRateFloat);
	// Apply net tick rate limiting if the desired net tick rate is strictly less than the engine tick rate.
	if (!IsInternalAck() && MaxNetTickRateFloat < EngineTickRate && DesiredTickRate > 0.0f)
	{
		const float MinNetFrameTime = 1.0f/DesiredTickRate;
		if (FrameTime < MinNetFrameTime)
		{
			return;
		}
	}

	LastTime = CurrentRealtimeSeconds;
	CumulativeTime += FrameTime;
	CountedFrames++;
	if(CumulativeTime > 1.f)
	{
		AverageFrameTime = CumulativeTime / CountedFrames;
		CumulativeTime = 0;
		CountedFrames = 0;
	}

	// Pretend everything was acked, for 100% reliable connections or demo recording.
	if (IsInternalAck())
	{
		const bool bIsServer = Driver->IsServer();
		OutAckPacketId = OutPacketId;

		LastReceiveTime = Driver->GetElapsedTime();
		LastReceiveRealtime = FPlatformTime::Seconds();
		LastGoodPacketRealtime = FPlatformTime::Seconds();

		// Consume all records
		auto InternnalAckChannelFunc = [this, bIsServer](uint32 ChannelIndex)
		{
			UChannel* Channel = Channels[ChannelIndex];
			if (Channel)
			{
				for (FOutBunch* OutBunch=Channel->OutRec; OutBunch; OutBunch=OutBunch->Next)
				{
					OutBunch->ReceivedAck = 1;
				}

				if (bIsServer || Channel->OpenedLocally)
				{
					Channel->OpenAcked = 1;
				}

				Channel->ReceivedAcks();
			}
		};

		FChannelRecordImpl::ConsumeAllChannelRecords(ChannelRecord, InternnalAckChannelFunc);
	}

	// Update stats.
	if ( CurrentRealtimeSeconds - StatUpdateTime > StatPeriod )
	{
		// Update stats.
		const float RealTime = CurrentRealtimeSeconds - StatUpdateTime;
		if( LagCount )
		{
			AvgLag = LagAcc/LagCount;
		}

		InBytesPerSecond = FMath::TruncToInt(static_cast<float>(InBytes) / RealTime);
		OutBytesPerSecond = FMath::TruncToInt(static_cast<float>(OutBytes) / RealTime);
		InPacketsPerSecond = FMath::TruncToInt(static_cast<float>(InPackets) / RealTime);
		OutPacketsPerSecond = FMath::TruncToInt(static_cast<float>(OutPackets) / RealTime);

		// Add TotalPacketsLost to total since InTotalPackets only counts ACK packets
		InPacketsLossPercentage.UpdateLoss(InPacketsLost, InTotalPackets + InTotalPacketsLost, StatPeriodCount);

		// Using OutTotalNotifiedPackets so we do not count packets that are still in transit.
		OutPacketsLossPercentage.UpdateLoss(OutPacketsLost, OutTotalNotifiedPackets, StatPeriodCount);

		// Init counters.
		LagAcc = 0;
		StatUpdateTime = CurrentRealtimeSeconds;
		LagCount = 0;
		InPacketsLost = 0;
		OutPacketsLost = 0;
		InBytes = 0;
		OutBytes = 0;
		InPackets = 0;
		OutPackets = 0;

		++StatPeriodCount;
	}

	if (bConnectionPendingCloseDueToSocketSendFailure)
	{
		Close(ENetCloseResult::SocketSendFailure);

		bConnectionPendingCloseDueToSocketSendFailure = false;

		// early out
		return;
	}

	// Compute time passed since last update.
	const double DriverElapsedTime = Driver->GetElapsedTime();
	const double DeltaTime	= DriverElapsedTime - LastTickTime;
	LastTickTime			= DriverElapsedTime;

	// Handle timeouts.
	const float Timeout = GetTimeoutValue();

	const bool bReceiveTimedOut = (CurrentRealtimeSeconds - LastReceiveRealtime) > Timeout;
	const bool bGracefulCloseTimedOut = (GetConnectionState() == USOCK_Closing) && (DriverElapsedTime > GracefulCloseTimeoutDeadline);

	if (bReceiveTimedOut || bGracefulCloseTimedOut)
	{
		const TCHAR* const TimeoutString = TEXT("UNetConnection::Tick: Connection TIMED OUT. Closing connection.");
		const TCHAR* const DestroyString = TEXT("UNetConnection::Tick: Connection closing during pending destroy, not all shutdown traffic may have been negotiated");

		const TCHAR* ErrorString = bPendingDestroy ? DestroyString : TimeoutString;

		if (bGracefulCloseTimedOut)
		{
			ErrorString = TEXT("UNetConnection::Tick: Connection graceful close timed out, not all shutdown traffic was negotiated");
		}

		// Compute true realtime since packet was received (as well as truly processed)
		const double Seconds = FPlatformTime::Seconds();

		const double ReceiveRealtimeDelta = Seconds - LastReceiveRealtime;
		const double GoodRealtimeDelta = Seconds - LastGoodPacketRealtime;

		// Timeout.
		FString Error = FString::Printf(TEXT("%s. Elapsed: %2.2f, Real: %2.2f, Good: %2.2f, DriverTime: %2.2f, Threshold: %2.2f, %s"),
			ErrorString,
			DriverElapsedTime - LastReceiveTime,
			ReceiveRealtimeDelta,
			GoodRealtimeDelta,
			DriverElapsedTime,
			bGracefulCloseTimedOut ? Driver->GracefulCloseConnectionTimeout : Timeout,
			*Describe());
		
		static double LastTimePrinted = 0.0f;
		if (FPlatformTime::Seconds() - LastTimePrinted > GEngine->NetErrorLogInterval)
		{
			UE_LOGF(LogNet, Warning, "%ls", *Error);
			LastTimePrinted = FPlatformTime::Seconds();
		}

		HandleConnectionTimeout(Error);

		if (Driver == NULL)
		{
			// Possible that the Broadcast above caused someone to kill the net driver, early out
			return;
		}
	}
	else
	{
		// We should never need more ticking channels than open channels
		checkf(ChannelsToTick.Num() <= OpenChannels.Num(), TEXT("More ticking channels (%d) than open channels (%d) for net connection!"), ChannelsToTick.Num(), OpenChannels.Num())

		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetConnection_TickChannels);

		// Tick the channels.
		if (CVarTickAllOpenChannels.GetValueOnAnyThread() == 0)
		{
			for (int32 i = ChannelsToTick.Num()-1; i >= 0; i--)
			{
				UE_IRIS_PARALLEL_EXPR(UE::TScopeUnlock<FTransactionallySafeCriticalSection> ScopeUnlock(ChannelsToTick[i]->SupportsParallelTick() ? &GetSharedNetTickMutex() : nullptr ));
				ChannelsToTick[i]->Tick();

				if (ChannelsToTick[i]->CanStopTicking())
				{
					ChannelsToTick.RemoveAt(i);
				}
			}
		}
		else
		{
			for (int32 i = OpenChannels.Num() - 1; i >= 0; i--)
			{
				UE_IRIS_PARALLEL_EXPR(UE::TScopeUnlock<FTransactionallySafeCriticalSection> ScopeUnlock(OpenChannels[i]->SupportsParallelTick() ? &GetSharedNetTickMutex() : nullptr ));
				if (OpenChannels[i])
				{
					OpenChannels[i]->Tick();
				}
				else
				{
					UE_LOGF(LogNet, Warning, "UNetConnection::Tick: null channel in OpenChannels array. %ls", *Describe());
				}
			}
		}

		for ( auto ProcessingActorMapIter = KeepProcessingActorChannelBunchesMap.CreateIterator(); ProcessingActorMapIter; ++ProcessingActorMapIter )
		{
			auto& ActorChannelArray = ProcessingActorMapIter.Value();
			for ( int32 ActorChannelIdx = 0; ActorChannelIdx < ActorChannelArray.Num(); ++ActorChannelIdx )
			{
				UActorChannel* CurChannel = ActorChannelArray[ActorChannelIdx];
				
				bool bRemoveChannel = false;
				if ( IsValid(CurChannel) )
				{
					check( CurChannel->ChIndex == -1 );
					if ( CurChannel->ProcessQueuedBunches() )
					{
						// Since we are done processing bunches, we can now actually clean this channel up
						CurChannel->ConditionalCleanUp(false, CurChannel->QueuedCloseReason);

						bRemoveChannel = true;
						UE_LOGF( LogNet, VeryVerbose, "UNetConnection::Tick: Removing from KeepProcessingActorChannelBunchesMap. Num: %i", KeepProcessingActorChannelBunchesMap.Num() );
					}

				}
				else
				{
					bRemoveChannel = true;
					UE_LOGF( LogNet, Verbose, "UNetConnection::Tick: Removing from KeepProcessingActorChannelBunchesMap before done processing bunches. Num: %i", KeepProcessingActorChannelBunchesMap.Num() );
				}

				// Remove the actor channel from the array
				if ( bRemoveChannel )
				{
					ActorChannelArray.RemoveAt( ActorChannelIdx, EAllowShrinking::No);
					--ActorChannelIdx;
				}
			}

			if ( ActorChannelArray.Num() == 0 )
			{
				ProcessingActorMapIter.RemoveCurrent();
			}
		}

		// If gracefully closing, check whether all channels are ready for close
		TryClosePendingGracefulClose();

		// If channel 0 has closed, mark the connection as closed.
		if (Channels[0] == nullptr && (OutReliable[0] != InitOutReliable || InReliable[0] != InitInReliable))
		{
			SetConnectionState(EConnectionState::USOCK_Closed);
		}
	}

	// Flush.
	if ( TimeSensitive || (Driver->GetElapsedTime() - LastSendTime) > Driver->KeepAliveTime)
	{
		bool bHandlerHandshakeComplete = !Handler.IsValid() || Handler->IsFullyInitialized();

		// Delay any packet sends on the server, until we've verified that a packet has been received from the client.
		if (bHandlerHandshakeComplete && HasReceivedClientPacket())
		{
			UE_IRIS_PARALLEL_EXPR(UE::TScopeUnlock<FTransactionallySafeCriticalSection> ScopeUnlock(&GetSharedNetTickMutex()));
			FlushNet();
		}
	}

	// Tick Handler
	if (Handler.IsValid())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetConnection_TickPacketHandler)

		Handler->Tick(FrameTime);

		// Resend any queued up raw packets (these come from the reliability handler)
		BufferedPacket* ResendPacket = Handler->GetQueuedRawPacket();

		if (ResendPacket && Driver->IsNetResourceValid())
		{
			Handler->SetRawSend(true);

			while (ResendPacket != nullptr)
			{
				LowLevelSend(ResendPacket->Data, ResendPacket->CountBits, ResendPacket->Traits);
				ResendPacket = Handler->GetQueuedRawPacket();
			}

			Handler->SetRawSend(false);
		}

		BufferedPacket* QueuedPacket = Handler->GetQueuedPacket();

		/* Send all queued packets */
		while(QueuedPacket != nullptr)
		{
			if (Driver->IsNetResourceValid())
			{
				LowLevelSend(QueuedPacket->Data, QueuedPacket->CountBits, QueuedPacket->Traits);
			}
			delete QueuedPacket;
			QueuedPacket = Handler->GetQueuedPacket();
		}
	}

	// Update queued byte count.
	// this should be at the end so that the cap is applied *after* sending (and adjusting QueuedBytes for) any remaining data for this tick

	SaturationAnalytics.TrackFrame(!IsNetReady());

	if (!IsReplay())
	{
		// Clamp DeltaTime for bandwidth limiting so that if there is a hitch, we don't try to send
		// a large burst on the next frame, which can cause another hitch if a lot of additional replication occurs.
		float BandwidthDeltaTime = DeltaTime;
		if (DesiredTickRate != 0.0f)
		{
			BandwidthDeltaTime = FMath::Clamp(BandwidthDeltaTime, 0.0f, 1.0f / DesiredTickRate);
		}

		float DeltaBits = CurrentNetSpeed * BandwidthDeltaTime * 8.f;

		int32 NewQueuedBits = 0;
		const int64 DeltaQueuedBits = -FMath::TruncToInt(DeltaBits);
		const bool bWouldOverflow = UE::Net::Connection::Private::Add_DetectOverflow_Clamp(QueuedBits, DeltaQueuedBits, NewQueuedBits);

		ensureMsgf(!bWouldOverflow, TEXT("UNetConnection::Tick: QueuedBits overflow detected! QueuedBits: %d, change: %lld, BandwidthDeltaTime: %.4f, DesiredTickRate: %.2f"),
			QueuedBits, DeltaQueuedBits, BandwidthDeltaTime, DesiredTickRate);

		QueuedBits = NewQueuedBits;

		const int64 AllowedLag = 2 * DeltaQueuedBits;

		if (QueuedBits < AllowedLag)
		{
			int32 NewAllowedLag = 0;
			const bool bAllowedLagOverflow = UE::Net::Connection::Private::Add_DetectOverflow_Clamp(0, AllowedLag, NewAllowedLag);

			ensureMsgf(!bAllowedLagOverflow, TEXT("UNetConnection::Tick: AllowedLag overflow detected! AllowedLag: %lld, BandwidthDeltaTime: %.4f, DesiredTickRate: %.2f"),
				AllowedLag, BandwidthDeltaTime, DesiredTickRate);

			QueuedBits = NewAllowedLag;
		}
	}

	bFlushedNetThisFrame = false;

	if (NetPing.IsValid())
	{
		NetPing->TickRealtime(CurrentRealtimeSeconds);
	}
}

#if DO_ENABLE_NET_TEST
void UNetConnection::UpdateDelayedPackets(const double CurrentRealtimeSeconds)
{
	for (int32 Index = 0; Index < Delayed.Num(); ++Index)
	{
		FDelayedPacket& Packet = Delayed[Index];
		bool bSendPacket = false;

		if (Packet.DelayFrameCount > 0)
		{
			if (Packet.DelayFrameCount == 1)
			{
				bSendPacket = true;
			}
			else
			{
				Packet.DelayFrameCount--;
			}
		}
		else if (CurrentRealtimeSeconds > Packet.SendTime)
		{
			bSendPacket = true;
		}

		if (bSendPacket)
		{
			LowLevelSend((char*)&(Packet.Data[0]), Packet.SizeBits, Packet.Traits);
			Packet.bSent = true;
		}
		else if (!bSendDelayedPacketsOutofOrder)
		{
			// Break now instead of continuing to iterate through the list. Otherwise it causes out of order packets due to the random variance added to each latency
			break;
		}
	}

	Delayed.RemoveAll([](const FDelayedPacket& Packet)
	{
		return Packet.bSent;
	});
}
#endif // DO_ENABLE_NET_TEST

void UNetConnection::HandleConnectionTimeout(const FString& Error)
{
	if (!bPendingDestroy)
	{
		GEngine->BroadcastNetworkFailure(Driver->GetWorld(), Driver, ENetworkFailure::ConnectionTimeout, Error);
	}

	Close(ENetCloseResult::ConnectionTimeout);

#if USE_SERVER_PERF_COUNTERS
	PerfCountersIncrement(TEXT("TimedoutConnections"));
#endif

}

void UNetConnection::HandleClientPlayer( APlayerController *PC, UNetConnection* NetConnection )
{
	check(Driver->GetWorld());

	// Hook up the Viewport to the new player actor.
	ULocalPlayer*	LocalPlayer = NULL;
	if (GetLocalPlayerFunction())
	{
		LocalPlayer = GetLocalPlayerFunction()(NetConnection, PC);
	}
	else
	{
		if (FLocalPlayerIterator It(GEngine, Driver->GetWorld()); It)
		{
			LocalPlayer = *It;
		}
	}

	// Detach old player if it's in the same level.
	check(LocalPlayer);
	if( LocalPlayer->PlayerController && LocalPlayer->PlayerController->GetLevel() == PC->GetLevel())
	{
		if (LocalPlayer->PlayerController->GetLocalRole() == ROLE_Authority)
		{
			// local placeholder PC while waiting for connection to be established
			LocalPlayer->PlayerController->GetWorld()->DestroyActor(LocalPlayer->PlayerController);
		}
		else
		{
			// tell the server the swap is complete
			// we cannot use a replicated function here because the server has already transferred ownership and will reject it
			// so use a control channel message
			int32 Index = INDEX_NONE;
			FNetControlMessage<NMT_PCSwap>::Send(this, Index);
		}
		LocalPlayer->PlayerController->Player = NULL;
		LocalPlayer->PlayerController->NetConnection = NULL;
		LocalPlayer->PlayerController = NULL;
	}

	LocalPlayer->CurrentNetSpeed = CurrentNetSpeed;

	// Init the new playerpawn.
	PC->SetRole(ROLE_AutonomousProxy);
	PC->NetConnection = NetConnection;
	PC->SetPlayer(LocalPlayer);
	UE_LOGF(LogNet, Verbose, "%ls setplayer %ls",*PC->GetName(),*LocalPlayer->GetName());
	LastReceiveTime = Driver->GetElapsedTime();
	SetConnectionState(EConnectionState::USOCK_Open);
	SetClientHandshakeId(PC->GetClientHandshakeId());
	PlayerController = PC;
	OwningActor = PC;

	{
		// Enable replication
		if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
		{
			ReplicationSystem->SetReplicationEnabledForConnection(GetConnectionHandle().GetParentConnectionId(), true);
		}
	}

	UWorld* World = PlayerController->GetWorld();
	// if we have already loaded some sublevels, tell the server about them
	{
		TArray<FUpdateLevelVisibilityLevelInfo> LevelVisibilities;
		for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
		{
			if (LevelStreaming)
			{
				const ULevel* Level = LevelStreaming->GetLoadedLevel();
				if ( Level && Level->bIsVisible && !Level->bClientOnlyVisible )
				{
					FUpdateLevelVisibilityLevelInfo& LevelVisibility = *new( LevelVisibilities ) FUpdateLevelVisibilityLevelInfo(Level, true);
					LevelVisibility.PackageName = PC->NetworkRemapPath(LevelVisibility.PackageName, false);
				}
			}
		}
		if( LevelVisibilities.Num() > 0 )
		{
			PC->ServerUpdateMultipleLevelsVisibility( LevelVisibilities );
		}
	}

	// if we have splitscreen viewports, ask the server to join them as well
	bool bSkippedFirst = false;
	for (FLocalPlayerIterator It(GEngine, Driver->GetWorld()); It; ++It)
	{
		if (*It != LocalPlayer)
		{
			// send server command for new child connection
			TArray<FString> Options;
			It->SendSplitJoin(Options);
		}
	}

	NetConnection->GetOnConnectionOpened().Broadcast(this, PC, NetConnection);
	
	NotifyConnectionUpdated();
}

UChildConnection* UNetConnection::GetChildConnectionWithNetPlayerIndex(const int32 NetPlayerIndex)
{
	TObjectPtr<UChildConnection>* ChildConnectionPtr = ChildrenLookup.Find(NetPlayerIndex);
	if (!ChildConnectionPtr)
	{
		return nullptr;
	}

	UChildConnection* ChildConnection = *ChildConnectionPtr;

	ensure(!ChildConnection->PlayerController || (ChildConnection->PlayerController && ChildConnection->PlayerController->NetPlayerIndex == NetPlayerIndex));
	return ChildConnection;
}

bool UNetConnection::GetNetPlayerIndex(UChildConnection* ChildConnection, int32* OutNetPlayerIndex)
{
	if (OutNetPlayerIndex == nullptr)
	{
		return false;
	}

	for (TPair<int32, TObjectPtr<UChildConnection>>& Pair : ChildrenLookup)
	{
		if (Pair.Value == ChildConnection)
		{
			*OutNetPlayerIndex = Pair.Key;
			return true;
		}
	}

	return false;
}

UChildConnection* UNetConnection::CreateChild(int32* OutNetPlayerIndex)
{
	const int32 NetPlayerIndex = NextServerNetPlayerIndex;
	NextServerNetPlayerIndex++;

	UE_LOGF(LogNet, Log, "[%ls:%ls] Assigning NetPlayerIndex %d to a new child connection.", *GetNameSafe(GetDriver()), *GetName(), NetPlayerIndex);

	UChildConnection* ChildConnection = CreateChildWithNetPlayerIndex(NetPlayerIndex);

	if (OutNetPlayerIndex)
	{
		*OutNetPlayerIndex = NetPlayerIndex;
	}

	return ChildConnection;
}

UChildConnection* UNetConnection::CreateChildWithNetPlayerIndex(int32 NetPlayerIndex)
{
	// A NetPlayerIndex of 0 is reserved for the parent connection so the NetPlayerIndex for
	// child connections should always begin at 1.
	ensure(NetPlayerIndex > 0);

	UChildConnection* ChildConnection = NewObject<UChildConnection>(GetTransientPackage(), GetDriver()->ChildNetConnectionClass);
	ChildConnection->InitChildConnection(GetDriver(), this);
	
	UE_LOGF(LogNet, Log, "[%ls:%ls] Creating child connection %ls NetPlayerIndex %d", *GetNameSafe(GetDriver()), *GetName(), *GetNameSafe(ChildConnection), NetPlayerIndex);
	
	ensure(Children.Find(ChildConnection) == INDEX_NONE);
	Children.Add(ChildConnection);

	ensure(!ChildrenLookup.Contains(NetPlayerIndex));
	ChildrenLookup.Add(NetPlayerIndex, ChildConnection);

	return ChildConnection;
}

void UNetConnection::RemoveChild(UChildConnection* ChildConnection)
{
	ensure(Children.Find(ChildConnection) != INDEX_NONE);

	Children.Remove(ChildConnection);
		
	int32 NetPlayerIndexToRemove = -1;

	if (PlayerController && ChildConnection->PlayerController)
	{
		NetPlayerIndexToRemove = ChildConnection->PlayerController->NetPlayerIndex;
	}
	else
	{
		for (TPair<int32, TObjectPtr<UChildConnection>>& Pair : ChildrenLookup)
		{
			if (Pair.Value == ChildConnection)
			{
				NetPlayerIndexToRemove = Pair.Key;
				break;
			}
		}
	}
	
	UE_LOGF(LogNet, Log, "Removing child connection %ls from parent connection %ls using NetPlayerIndex %d.", *GetNameSafe(ChildConnection), *GetNameSafe(this), NetPlayerIndexToRemove);

	if (NetPlayerIndexToRemove >= 0)
	{
		// A NetPlayerIndex of 0 is reserved for the parent connection so it should never be found in the lookup table.
		ensure(NetPlayerIndexToRemove > 0);

		ensure(ChildrenLookup.Contains(NetPlayerIndexToRemove) && *ChildrenLookup.Find(NetPlayerIndexToRemove) == ChildConnection);
		ChildrenLookup.Remove(NetPlayerIndexToRemove);
	}
}

void UChildConnection::HandleClientPlayer(APlayerController* PC, UNetConnection* NetConnection)
{
	ULocalPlayer* NewPlayer = NULL;
	if (NetConnection->GetLocalPlayerFunction())
	{
		NewPlayer = NetConnection->GetLocalPlayerFunction()(NetConnection, PC);
	}
	else
	{
		// Find the local player with the same split screen identifier as the new player controller.
		for (FLocalPlayerIterator It(GEngine, Driver->GetWorld()); It; ++It)
		{
			ULocalPlayer* LocalPlayer = *It;
			if (PC->GetLocalPlayerConnectionIdentifier() == LocalPlayer->GetConnectionIdentifier())
			{
				NewPlayer = *It;
				break;
			}
		}
	}

	if (!ensure(NewPlayer != NULL))
	{
		UE_LOGF(LogNet, Error, "Failed to find LocalPlayer for received PlayerController '%ls' with index %d. PlayerControllers:", *PC->GetName(), int32(PC->NetPlayerIndex));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		check( PC->GetWorld() );
		for (TActorIterator<APlayerController> It(PC->GetWorld()); It; ++It)
		{
			if (It->GetLocalRole() < ROLE_Authority)
			{
				UE_LOGF(LogNet, Log, " - %ls", *It->GetFullName());
			}
		}
#endif
		if (ensure(Parent != nullptr))
		{
			Parent->Close(ENetCloseResult::BadChildConnectionIndex);
		}
		return; // avoid crash
	}

	// Detach old player.
	check(NewPlayer);
	if (NewPlayer->PlayerController != NULL)
	{
		if (NewPlayer->PlayerController->GetLocalRole() == ROLE_Authority)
		{
			// local placeholder PC while waiting for connection to be established
			NewPlayer->PlayerController->GetWorld()->DestroyActor(NewPlayer->PlayerController);
		}
		else
		{
			// tell the server the swap is complete
			// we cannot use a replicated function here because the server has already transferred ownership and will reject it
			// so use a control channel message
			int32 Index = Parent->Children.Find(this);
			FNetControlMessage<NMT_PCSwap>::Send(Parent, Index);
		}
		NewPlayer->PlayerController->Player = NULL;
		NewPlayer->PlayerController->NetConnection = NULL;
		NewPlayer->PlayerController = NULL;
	}

	NewPlayer->CurrentNetSpeed = CurrentNetSpeed;

	// Init the new playerpawn.
	PC->SetRole(ROLE_AutonomousProxy);
	PC->NetConnection = NetConnection;
	PC->SetPlayer(NewPlayer);
	UE_LOGF(LogNet, Verbose, "%ls setplayer %ls", *PC->GetName(), *NewPlayer->GetName());
	PlayerController = PC;
	OwningActor = PC;
	SetClientHandshakeId(PC->GetClientHandshakeId());

	NetConnection->GetOnConnectionOpened().Broadcast(this, PC, NetConnection);

	NotifyConnectionUpdated();
}

#if DO_ENABLE_NET_TEST
void UNetConnection::UpdatePacketSimulationSettings(void)
{
	check(Driver);
	PacketSimulationSettings = Driver->PacketSimulationSettings;
}
#endif

void UNetConnection::SetLocalPlayerFunction(TFunction<ULocalPlayer*(UNetConnection*, APlayerController*)> InPlayerConnectionFunction)
{
	PlayerConnectionFunction = InPlayerConnectionFunction;
}

TFunction<ULocalPlayer*(UNetConnection*, APlayerController*)> UNetConnection::GetLocalPlayerFunction()
{
	return PlayerConnectionFunction;
}

/**
 * Called to determine if a voice packet should be replicated to this
 * connection or any of its child connections
 *
 * @param Sender - the sender of the voice packet
 *
 * @return true if it should be sent on this connection, false otherwise
 */
bool UNetConnection::ShouldReplicateVoicePacketFrom(const FUniqueNetId& Sender)
{
	if (PlayerController &&
		// Has the handshaking of the mute list completed?
		PlayerController->MuteList.bHasVoiceHandshakeCompleted)
	{	
		// Check with the owning player controller first.
		if (Sender.IsValid() &&
			// Determine if the server should ignore replication of voice packets that are already handled by a peer connection
			//(!Driver->AllowPeerVoice || !Actor->HasPeerConnection(Sender)) &&
			// Determine if the sender was muted for the local player 
			PlayerController->IsPlayerMuted(Sender) == false)
		{
			// The parent wants to allow, but see if any child connections want to mute
			for (int32 Index = 0; Index < Children.Num(); Index++)
			{
				if (Children[Index]->ShouldReplicateVoicePacketFrom(Sender) == false)
				{
					// A child wants to mute, so skip
					return false;
				}
			}
			// No child wanted to block it so accept
			return true;
		}
	}
	// Not able to handle voice yet or player is muted on this connection
	return false;
}

void UNetConnection::ResetGameWorldState()
{
	//Clear out references and do whatever else so that nothing holds onto references that it doesn't need to.
	ResetDestructionInfos();
	ClientPendingStreamingStatusRequest.Empty();
	ClientVisibleLevelNames.Empty();
	ClientMakingVisibleLevelNames.Empty();
	KeepProcessingActorChannelBunchesMap.Empty();
	CleanupDormantActorState();
	ClientVisibleActorOuters.Empty();

	// Clear the view target as this may be from an out of date world
	ViewTarget = nullptr;

	// Update any level visibility requests received during the transition
	// This can occur if client loads faster than the server
	for (const auto& Pending : PendingUpdateLevelVisibility)
	{
		UpdateLevelVisibilityInternal(Pending.Value);
	}
	PendingUpdateLevelVisibility.Empty();
}

void UNetConnection::CleanupDormantActorState()
{
	ClearDormantReplicatorsReference();

	DormantReplicatorSet.EmptySet();
}

void UNetConnection::ClearDormantReplicatorsReference()
{
	using namespace UE::Net::Private;

#if UE_REPLICATED_OBJECT_REFCOUNTING
	if (!Driver)
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>, TInlineAllocator<16>> SubObjectsToRemove;

	// Find all the dormant subobject replicators still held by this connection and remove our reference to them.
	for (const FActorDormantReplicators& ActorDormantReplicators : DormantReplicatorSet.ActorReplicatorSet)
	{
		FObjectKey OwnerKey = ActorDormantReplicators.OwnerActorKey;

		for (const FDormantObjectReplicator& DormantObject : ActorDormantReplicators.DormantReplicators)
		{
			// Only if it's a subobject and not the actor itself
			if (DormantObject.ObjectKey != OwnerKey)
			{
				SubObjectsToRemove.Add(DormantObject.Replicator->GetWeakObjectPtr());
			}
		}

		if (!SubObjectsToRemove.IsEmpty())
		{
			Driver->GetNetworkObjectList().RemoveMultipleSubObjectChannelReference(ActorDormantReplicators.OwnerActorKey, SubObjectsToRemove, this);
			SubObjectsToRemove.Reset();
		}
	}
#endif
}

void UNetConnection::FlushDormancy(AActor* Actor)
{
	using namespace UE::Net;
	using namespace Connection::Private;

	UE_LOGF(LogNetDormancy, Verbose, "FlushDormancy: %ls. Connection: %ls", *Actor->GetName(), *GetName());
	
	if (Driver->GetNetworkObjectList().MarkActive(Actor, this, Driver))
	{
		FlushDormancyForObject(Actor, Actor);

		const FSubObjectRegistry& ActorSubObjects = FSubObjectRegistryGetter::GetSubObjects(Actor);
		const TArray<FReplicatedComponentInfo>& ReplicatedComponents = FSubObjectRegistryGetter::GetReplicatedComponents(Actor);

		FDormantObjectMap FlushedGuids;
		if (Connection::Private::bTrackFlushedDormantObjects)
		{
			// This doesn't reserve space for subobjects of components, but avoids iterating the component list twice.
			FlushedGuids.Reserve(ReplicatedComponents.Num() + ActorSubObjects.GetRegistryList().Num());
		}

		TStaticBitArray<COND_Max> ConditionMap;
		const FNetConditionGroupManager* NetConditionGroupManager = nullptr;

		if (bEnableFlushDormantSubObjectsCheckConditions)
		{
			// Fill in the flags used by conditional subobjects
			FReplicationFlags RepFlags;
		
			// Since a dormant object won't necessarily have an FObjectReplicator or channel, we can't check for
			// initial replication in the normal way. So always flush for NetInitial to be safe, it may create
			// an extra replicator but the condition will be checked again before the object is actually replicated.
			RepFlags.bNetInitial = true;

			UNetConnection* OwningConnection = Actor->GetNetConnection();
			RepFlags.bNetOwner = (OwningConnection == this || (OwningConnection != nullptr && OwningConnection->IsA(UChildConnection::StaticClass()) && ((UChildConnection*)OwningConnection)->Parent == this));
		
			RepFlags.bNetSimulated = (Actor->GetRemoteRole() == ROLE_SimulatedProxy);
			RepFlags.bRepPhysics = Actor->GetReplicatedMovement().bRepPhysics;
			RepFlags.bReplay = bReplay;

			ConditionMap = UE::Net::BuildConditionMapFromRepFlags(RepFlags);
			
			const UWorld* const World = Actor->GetWorld();
			const UNetworkSubsystem* const NetworkSubsystem = World ? World->GetSubsystem<UNetworkSubsystem>() : nullptr;
			NetConditionGroupManager = NetworkSubsystem ? &NetworkSubsystem->GetNetConditionGroupManager() : nullptr;
			ensureMsgf(NetConditionGroupManager, TEXT("UNetConnection::FlushDormancy: couldn't find a NetConditionGroupManager for %s."), *Actor->GetName());
		}

		if (bEnableFlushDormantSubObjects)
		{
			FlushDormancyForSubObjects(this, Actor, ActorSubObjects, FlushedGuids, ConditionMap, NetConditionGroupManager);
		}

		for (const FReplicatedComponentInfo& RepComponentInfo : ReplicatedComponents)
		{
			UActorComponent* ActorComp = RepComponentInfo.Component;

			if (ensureMsgf(IsValid(ActorComp), TEXT("Found invalid replicated component (%s) registered in %s"), *GetNameSafe(ActorComp), *Actor->GetName()))
			{
				if (!bEnableFlushDormantSubObjectsCheckConditions ||
					!NetConditionGroupManager ||
					UActorChannel::CanSubObjectReplicateToClient(PlayerController, RepComponentInfo.NetCondition, RepComponentInfo.Key, ConditionMap, *NetConditionGroupManager))
				{
					FlushDormancyForObject(Actor, ActorComp);
					TrackFlushedSubObject(FlushedGuids, ActorComp, Driver->GetNetGuidCache());

					if (bEnableFlushDormantSubObjects)
					{
						const FSubObjectRegistry* const ComponentSubObjects = FSubObjectRegistryGetter::GetSubObjectsOfActorComponent(Actor, ActorComp);
						if (ComponentSubObjects)
						{
							FlushDormancyForSubObjects(this, Actor, *ComponentSubObjects, FlushedGuids, ConditionMap, NetConditionGroupManager);
						}
					}
				}
			}
		}

		if (bTrackFlushedDormantObjects && !FlushedGuids.IsEmpty())
		{
			FDormantObjectMap& DormantObjects = DormantReplicatorSet.FindOrAddFlushedObjectsForActor(Actor);
			DormantObjects.Append(FlushedGuids);
		}
	}

	// If channel is pending dormancy, cancel it
			
	// If the close bunch was already sent, that is fine, by reseting the dormant flag
	// here, the server will not add the actor to the dormancy list when it closes the channel 
	// after it gets the client ack. The result is the channel will close but be open again
	// right away
	UActorChannel* Ch = FindActorChannelRef(Actor);

	if ( Ch != nullptr )
	{
		UE_LOGF( LogNetDormancy, Verbose, "    Found Channel[%d] '%ls'. Reseting Dormancy. Ch->Closing: %d", Ch->ChIndex, *Ch->Describe(), Ch->Closing );

		Ch->Dormant = false;
		Ch->bPendingDormancy = false;
		Ch->bIsInDormancyHysteresis = false;
	}

}

void UNetConnection::ForcePropertyCompare( AActor* Actor )
{
	UActorChannel* Ch = FindActorChannelRef( Actor );

	if ( Ch != nullptr )
	{
		Ch->bForceCompareProperties = true;
	}
}

void UNetConnection::FlushDormancyForObject(AActor* DormantActor, UObject* ReplicatedObject)
{
	using namespace UE::Net::Connection::Private;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetConnection_FlushDormancyForObject)

	bool bReuseReplicators = false;

	if (GbNetReuseReplicatorsForDormantObjects && Driver && Driver->IsServer())
	{
		bReuseReplicators = true;
	}

	if (GNetDormancyValidate == 1)
	{
		TSharedPtr<FObjectReplicator> Replicator = DormantReplicatorSet.FindReplicator(DormantActor, ReplicatedObject);
		if (Replicator.IsValid())
		{
			Replicator->ValidateAgainstState(ReplicatedObject);
		}
	}

	// If we want to reuse replicators, make sure they exist for this object
	if (bReuseReplicators)
	{
		bReuseReplicators = DormantReplicatorSet.DoesReplicatorExist(DormantActor, ReplicatedObject);
	}

	// If we need to create a new replicator when flushing
	if (!bReuseReplicators)
	{
		bool bOverwroteExistingReplicator = false;
		const TSharedRef<FObjectReplicator>& ObjectReplicatorRef = DormantReplicatorSet.CreateAndStoreReplicator(DormantActor, ReplicatedObject, bOverwroteExistingReplicator);
		
		bool bUseDefaultState = false;
		
		// Init using the object's current state if the client has this actor's level loaded. If the level
		// is unloaded, we need to use the default state since the client has no current state.
		if (bFlushDormancyUseDefaultStateForUnloadedLevels && !ClientHasInitializedLevel(DormantActor->GetLevel()))
		{
			bUseDefaultState = true;
		}

		ObjectReplicatorRef->InitWithObject(ReplicatedObject, this, bUseDefaultState);

#if UE_REPLICATED_OBJECT_REFCOUNTING
		// Add a refcount only when we did not create a replicator on top of an existing one.
		if (Driver && DormantActor != ReplicatedObject && !bOverwroteExistingReplicator)
		{
			Driver->GetNetworkObjectList().AddSubObjectChannelReference(DormantActor, ReplicatedObject, this);
		}
#endif

		// Flush the must be mapped GUIDs, the initialization may add them, but they're phantom and will be remapped when actually sending
		if (UPackageMapClient* PackageMapClient = CastChecked<UPackageMapClient>(PackageMap))
		{
			TArray< FNetworkGUID >& MustBeMappedGuidsInLastBunch = PackageMapClient->GetMustBeMappedGuidsInLastBunch();
			MustBeMappedGuidsInLastBunch.Reset();
		}
	}
}

/** Wrapper for setting the current client login state, so we can trap for debugging, and verbosity purposes. */
void UNetConnection::SetClientLoginState( const EClientLoginState::Type NewState )
{
	if ( ClientLoginState == NewState )
	{
		UE_LOGF(LogNet, Verbose, "UNetConnection::SetClientLoginState: State same: %ls", EClientLoginState::ToString( NewState ) );
		return;
	}

	UE_CLOGF((Driver == nullptr || !Driver->DDoS.CheckLogRestrictions()), LogNet, Verbose,
				"UNetConnection::SetClientLoginState: State changing from %ls to %ls",
				EClientLoginState::ToString(ClientLoginState), EClientLoginState::ToString(NewState));

	if (UReplicationSystem* ReplicationSystem = Driver->GetReplicationSystem())
	{
		if (NewState == EClientLoginState::ReceivedJoin)
		{
			// Enable replication
			ReplicationSystem->SetReplicationEnabledForConnection(GetConnectionHandle().GetParentConnectionId(), true);
		}
	}

	ClientLoginState = NewState;
}

/** Wrapper for setting the current expected client login msg type. */
void UNetConnection::SetExpectedClientLoginMsgType(const uint8 NewType)
{
	const bool bLogRestricted = Driver != nullptr && Driver->DDoS.CheckLogRestrictions();

	if ( ExpectedClientLoginMsgType == NewType )
	{
		UE_CLOGF(!bLogRestricted, LogNet, Verbose, "UNetConnection::SetExpectedClientLoginMsgType: Type same: [%d]%ls",
				NewType, FNetControlMessageInfo::IsRegistered(NewType) ? FNetControlMessageInfo::GetName(NewType) : TEXT("UNKNOWN"));

		return;
	}

	UE_CLOGF(!bLogRestricted, LogNet, Verbose,
		"UNetConnection::SetExpectedClientLoginMsgType: Type changing from [%d]%ls to [%d]%ls", 
		ExpectedClientLoginMsgType,
		FNetControlMessageInfo::IsRegistered(ExpectedClientLoginMsgType) ? FNetControlMessageInfo::GetName(ExpectedClientLoginMsgType) : TEXT("UNKNOWN"),
		NewType,
		FNetControlMessageInfo::IsRegistered(NewType) ? FNetControlMessageInfo::GetName(NewType) : TEXT("UNKNOWN"));

	ExpectedClientLoginMsgType = NewType;
}

/** This function validates that ClientMsgType is the next expected msg type. */
bool UNetConnection::IsClientMsgTypeValid( const uint8 ClientMsgType )
{
	if ( ClientLoginState == EClientLoginState::LoggingIn )
	{
		// If client is logging in, we are expecting a certain msg type each step of the way
		if ( ClientMsgType != ExpectedClientLoginMsgType )
		{
			// Not the expected msg type
			UE_LOGF(LogNet, Log, "UNetConnection::IsClientMsgTypeValid FAILED: (ClientMsgType != ExpectedClientLoginMsgType) Remote Address=%ls", *LowLevelGetRemoteAddress());
			return false;
		}
	} 
	else
	{
		// Once a client is logged in, we no longer expect any of the msg types below
		if ( ClientMsgType == NMT_Hello || ClientMsgType == NMT_Login )
		{
			// We don't want to see these msg types once the client is fully logged in
			UE_LOGF(LogNet, Log, "UNetConnection::IsClientMsgTypeValid FAILED: Invalid msg after being logged in - Remote Address=%ls", *LowLevelGetRemoteAddress());
			return false;
		}
	}

	return true;
}

/**
* This function tracks the number of log calls per second for this client, 
* and disconnects the client if it detects too many calls are made per second
*/
bool UNetConnection::TrackLogsPerSecond()
{
	const double NewTime = FPlatformTime::Seconds();

	const double LogCallTotalTime = NewTime - LogCallLastTime;

	LogCallCount++;

	static const double LOG_AVG_THRESHOLD				= 0.5;		// Frequency to check threshold
	static const double	MAX_LOGS_PER_SECOND_INSTANT		= 60;		// If they hit this limit, they will instantly get disconnected
	static const double	MAX_LOGS_PER_SECOND_SUSTAINED	= 5;		// If they sustain this logs/second for a certain count, they get disconnected
	static const double	MAX_SUSTAINED_COUNT				= 10;		// If they sustain MAX_LOGS_PER_SECOND_SUSTAINED for this count, they get disconnected (5 seconds currently)

	if ( LogCallTotalTime > LOG_AVG_THRESHOLD )
	{
		const double LogsPerSecond = (double)LogCallCount / LogCallTotalTime;

		LogCallLastTime = NewTime;
		LogCallCount	= 0;

		if ( LogsPerSecond > MAX_LOGS_PER_SECOND_INSTANT )
		{
			// Hit this instant limit, we instantly disconnect them
			UE_LOGF(LogNet, Warning, "TrackLogsPerSecond instant FAILED. LogsPerSecond: %f, RemoteAddr: %ls",
					(float)LogsPerSecond, *LowLevelGetRemoteAddress());

			Close(ENetCloseResult::LogLimitInstant);

#if USE_SERVER_PERF_COUNTERS
			PerfCountersIncrement(TEXT("ClosedConnectionsDueToMaxBadRPCsLimit"));
#endif
			return false;
		}

		if ( LogsPerSecond > MAX_LOGS_PER_SECOND_SUSTAINED )
		{
			// Hit the sustained limit, count how many times we get here
			LogSustainedCount++;

			// Warn that we are approaching getting disconnected (will be useful when going over historical logs)
			UE_LOGF(LogNet, Warning,
					"TrackLogsPerSecond: LogsPerSecond > MAX_LOGS_PER_SECOND_SUSTAINED. LogSustainedCount: %i, LogsPerSecond: %f, RemoteAddr: %ls",
					LogSustainedCount, (float)LogsPerSecond, *LowLevelGetRemoteAddress() );

			if ( LogSustainedCount > MAX_SUSTAINED_COUNT )
			{
				// Hit the sustained limit for too long, disconnect them
				UE_LOGF(LogNet, Warning, "TrackLogsPerSecond: LogSustainedCount > MAX_SUSTAINED_COUNT. LogsPerSecond: %f, RemoteAddr: %ls",
						(float)LogsPerSecond, *LowLevelGetRemoteAddress());

				Close(ENetCloseResult::LogLimitSustained);

#if USE_SERVER_PERF_COUNTERS
				PerfCountersIncrement(TEXT("ClosedConnectionsDueToMaxBadRPCsLimit"));
#endif
				return false;
			}
		}
		else
		{
			// Reset sustained count since they are not above the threshold
			LogSustainedCount = 0;
		}
	}

	return true;
}

void UNetConnection::ResetPacketBitCounts()
{
	NumPacketIdBits = 0;
	NumBunchBits = 0;
	NumAckBits = 0;
	NumPaddingBits = 0;
}

void UNetConnection::SetPlayerOnlinePlatformName(const FName InPlayerOnlinePlatformName)
{
	PlayerOnlinePlatformName = InPlayerOnlinePlatformName;
}

void UNetConnection::DestroyIgnoredActor(AActor* Actor)
{
	if (Driver && Driver->World)
	{
		Driver->World->DestroyActor(Actor, true);
	}
}

void UNetConnection::StoreDormantReplicator(AActor* OwnerActor, UObject* Object, const TSharedRef<FObjectReplicator>& ObjectReplicator)
{
	ObjectReplicator->ReleaseStrongReference();

	if (ensureMsgf(OwnerActor, TEXT("StoreDormantReplicator cannot receive a null owner while storing %s"), *GetNameSafe(Object)))
	{
		DormantReplicatorSet.StoreReplicator(OwnerActor, Object, ObjectReplicator);
	}
}

TSharedPtr<FObjectReplicator> UNetConnection::FindAndRemoveDormantReplicator(AActor* OwnerActor, UObject* Object)
{
	const FObjectKey ObjectKey(Object);

	TSharedPtr<FObjectReplicator> ReplicatorPtr = DormantReplicatorSet.FindAndRemoveReplicator(OwnerActor, Object);

	if (ReplicatorPtr.IsValid())
	{
		// Only return the replicator if the object is still valid, otherwise just remove it from the cache and allow the caller to create a new one
		if (UObject* StrongPtr = ReplicatorPtr->GetWeakObjectPtr().Get())
		{
			// Reassign the strong pointer for GC/faster resolve
			ReplicatorPtr->SetObject(StrongPtr);
		}
		else
		{
			ReplicatorPtr.Reset();
		}
	}

	return ReplicatorPtr;
}

void UNetConnection::RemoveDormantReplicator(AActor* Actor, UObject* Object)
{
	const FObjectKey ObjectKey = Object;

	const bool bWasStored = DormantReplicatorSet.RemoveStoredReplicator(Actor, ObjectKey);

#if UE_REPLICATED_OBJECT_REFCOUNTING
	// If this is a subobject (and not the main actor) remove the reference
	if (bWasStored && Object != Actor)
	{
		if (Driver)
		{
			Driver->GetNetworkObjectList().RemoveSubObjectChannelReference(Actor, Object, this);
		}
	}
#endif
}

void UNetConnection::ExecuteOnAllDormantReplicators(UE::Net::FExecuteForEachDormantReplicator ExecuteFunction)
{
	DormantReplicatorSet.ForEachDormantReplicator(ExecuteFunction);
}

void UNetConnection::ExecuteOnAllDormantReplicatorsOfActor(AActor* OwnerActor, UE::Net::FExecuteForEachDormantReplicator ExecuteFunction)
{
	DormantReplicatorSet.ForEachDormantReplicatorOfActor(OwnerActor, ExecuteFunction);	
}

void UNetConnection::CleanupDormantReplicatorsForActor(AActor* Actor)
{
	if (Actor)
	{
#if UE_REPLICATED_OBJECT_REFCOUNTING
		if (Driver)
		{
			TArray<TWeakObjectPtr<UObject>, TInlineAllocator<16>> RemovedObjects;
			auto ExecuteFunction = [&RemovedObjects](FObjectKey OwnerActorKey, FObjectKey ObjectKey, const TSharedRef<FObjectReplicator>& ReplicatorRef)
			{
				// If it's the replicator of a subobject and not the main actor
				if (OwnerActorKey != ObjectKey)
				{
					RemovedObjects.Add(ReplicatorRef->GetWeakObjectPtr());
				}
			};

			DormantReplicatorSet.ForEachDormantReplicatorOfActor(Actor, ExecuteFunction);

			if (RemovedObjects.Num() > 0)
			{
				Driver->GetNetworkObjectList().RemoveMultipleSubObjectChannelReference(Actor, RemovedObjects, this);
			}
		}
#endif

		DormantReplicatorSet.CleanupAllReplicatorsOfActor(Actor);
	}
}

void UNetConnection::CleanupStaleDormantReplicators()
{
	if (ensure(Driver))	
	{
		DormantReplicatorSet.CleanupStaleObjects(Driver->GetNetworkObjectList(), this);
	}
}

UE::Net::FDormantObjectMap* UNetConnection::GetDormantFlushedObjectsForActor(AActor* Actor)
{
	return DormantReplicatorSet.FindFlushedObjectsForActor(Actor);
}

void UNetConnection::ClearDormantFlushedObjectsForActor(AActor* Actor)
{
	DormantReplicatorSet.ClearFlushedObjectsForActor(Actor);
}

void UNetConnection::SetPendingCloseDueToReplicationFailure()
{
	bConnectionPendingCloseDueToReplicationFailure = true;
}

void UNetConnection::SetPendingCloseDueToSocketSendFailure()
{
	bConnectionPendingCloseDueToSocketSendFailure = true;
}

void UNetConnection::ConsumeQueuedActorDelinquencyAnalytics(FNetQueuedActorDelinquencyAnalytics& Out)
{
	if (UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(PackageMap))
	{
		return PackageMapClient->ConsumeQueuedActorDelinquencyAnalytics(Out);
	}
	else
	{
		Out.Reset();
	}
}

const FNetQueuedActorDelinquencyAnalytics& UNetConnection::GetQueuedActorDelinquencyAnalytics() const
{
	static FNetQueuedActorDelinquencyAnalytics Empty;

	if (UPackageMapClient const * const PackageMapClient = Cast<UPackageMapClient>(PackageMap))
	{
		return PackageMapClient->GetQueuedActorDelinquencyAnalytics();
	}
	
	return Empty;
}

void UNetConnection::ResetQueuedActorDelinquencyAnalytics()
{
	if (UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(PackageMap))
	{
		PackageMapClient->ResetQueuedActorDelinquencyAnalytics();
	}
}

void UNetConnection::ConsumeSaturationAnalytics(FNetConnectionSaturationAnalytics& Out)
{
	Out = MoveTemp(SaturationAnalytics);
	SaturationAnalytics.Reset();
}

const FNetConnectionSaturationAnalytics& UNetConnection::GetSaturationAnalytics() const
{
	return SaturationAnalytics;
}

void UNetConnection::ResetSaturationAnalytics()
{
	SaturationAnalytics.Reset();
}

void UNetConnection::ConsumePacketAnalytics(FNetConnectionPacketAnalytics& Out)
{
	Out = MoveTemp(PacketAnalytics);
	PacketAnalytics.Reset();
}

const FNetConnectionPacketAnalytics& UNetConnection::GetPacketAnalytics() const
{
	return PacketAnalytics;
}

void UNetConnection::ResetPacketAnalytics()
{
	PacketAnalytics.Reset();
}

void UNetConnection::TrackReplicationForAnalytics(const bool bWasSaturated)
{
	++TickCount;
	SaturationAnalytics.TrackReplication(bWasSaturated);
}

void UNetConnection::ProcessJitter(uint32 PacketJitterClockTimeMS)
{
	if (PacketJitterClockTimeMS >= UE::Net::Connection::Private::MaxJitterClockTimeValue)
	{
		// Ignore this packet for jitter calculations
		return;
	}

	const int32 CurrentClockTimeMilliseconds = UE::Net::Connection::Private::GetClockTimeMilliseconds(LastReceiveRealtime);

	// Get the delta between the sent and receive clock time.
	int32 CurrentJitterTimeDelta = CurrentClockTimeMilliseconds - (int32)PacketJitterClockTimeMS;
	
	if (CurrentJitterTimeDelta < 0)
	{
		// Account for wrap-around
		CurrentJitterTimeDelta += UE::Net::Connection::Private::MaxJitterPrecisionInMS;
	}

	// Get the difference between the last two deltas
	const float CurrentJitter = FMath::Abs(CurrentJitterTimeDelta - PreviousJitterTimeDelta);
	

	// Add the value to the average and smooth it out to reduce divergences
	const float PreviousJitterInMS = AverageJitterInMS;
	AverageJitterInMS = PreviousJitterInMS + ((CurrentJitter - PreviousJitterInMS) / UE::Net::Connection::Private::JitterNoiseReduction);

	//UE_LOGF(LogNetTraffic, Verbose, "Avg Jitter: %f | CurrentJitter: %f | CurrentTimeDelta: %d | PreviousTimeDelta: %d | ReceivedClockMilli %d | LocalClockMilli %d", AverageJitterInMS, CurrentJitter, CurrentJitterTimeDelta, PreviousJitterTimeDelta, PacketJitterClockTimeMS, CurrentClockTimeMilliseconds);

	// Store for next process
	PreviousJitterTimeDelta = CurrentJitterTimeDelta;
}

void UNetConnection::SendChallengeControlMessage()
{
	if (GetConnectionState() != USOCK_Invalid && GetConnectionState() != USOCK_Closed && Driver)
	{
		Challenge = FString::Printf(TEXT("%08X"), FPlatformTime::Cycles());
		SetExpectedClientLoginMsgType(NMT_Login);
		FNetControlMessage<NMT_Challenge>::Send(this, Challenge);
		FlushNet();
	}
	else
	{
		UE_LOGF(LogNet, Log, "UWorld::SendChallengeControlMessage: connection in invalid state. %ls", *Describe());
	}
}

void UNetConnection::SendChallengeControlMessage(const FEncryptionKeyResponse& Response)
{
	if (GetConnectionState() != USOCK_Invalid && GetConnectionState() != USOCK_Closed && Driver)
	{
		if (Response.Response == EEncryptionResponse::Success)
		{
			EnableEncryptionServer(Response.EncryptionData);
			SendChallengeControlMessage();
		}
		else
		{
			FString ResponseStr(LexToString(Response.Response));
			UE_LOGF(LogNet, Warning, "UWorld::SendChallengeControlMessage: encryption failure [%ls] %ls", *ResponseStr, *Response.ErrorMsg);

			SendCloseReason(ENetCloseResult::EncryptionFailure);
			FNetControlMessage<NMT_Failure>::Send(this, ResponseStr);
			FlushNet(true);
			Close(ENetCloseResult::EncryptionFailure);
		}
	}
	else
	{
		UE_LOGF(LogNet, Warning, "UNetConnection::SendChallengeControlMessage: connection in invalid state. %ls", *Describe());
	}
}

uint32 UNetConnection::GetParentConnectionId() const
{
	return GetConnectionHandle().GetParentConnectionId();
}

void UNetConnection::NotifyActorDestroyed(AActor* Actor, bool IsSeamlessTravel /* = false */)
{
	// Remove it from any dormancy lists
	CleanupDormantReplicatorsForActor(Actor);
	ClearDormantFlushedObjectsForActor(Actor);
}

void UNetConnection::NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason)
{
	UReplicationConnectionDriver* const ConnectionDriver = GetReplicationConnectionDriver();
	if (ConnectionDriver)
	{
		ConnectionDriver->NotifyActorChannelCleanedUp(Channel);
	}
}

void UNetConnection::SetNetVersionsOnArchive(FArchive& Ar) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetSetVersionsOnArchive);
	Ar.SetEngineNetVer(GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid));
	Ar.SetGameNetVer(GetNetworkCustomVersion(FGameNetworkCustomVersion::Guid));

	const FCustomVersionArray& AllVersions = NetworkCustomVersions.GetAllVersions();
	for (const FCustomVersion& Version : AllVersions)
	{
		Ar.SetCustomVersion(Version.Key, Version.Version, Version.GetFriendlyName());
	}

	Ar.SetUEVer(PackageVersionUE);
	Ar.SetLicenseeUEVer(PackageVersionLicenseeUE);
	// Base archives only store FEngineVersionBase, but net connections store FEngineVersion.
	// This will slice off the branch name and anything else stored in FEngineVersion.
	Ar.SetEngineVer(EngineVersion);
}

uint32 UNetConnection::GetNetworkCustomVersion(const FGuid& VersionGuid) const
{
	const FCustomVersion* CustomVer = NetworkCustomVersions.GetVersion(VersionGuid);
	return CustomVer != nullptr ? CustomVer->Version : 0;
}

void UNetConnection::SetNetworkCustomVersions(const FCustomVersionContainer& CustomVersions)
{
	const FCustomVersionArray& AllVersions = CustomVersions.GetAllVersions();
	for (const FCustomVersion& Version : AllVersions)
	{
		NetworkCustomVersions.SetVersion(Version.Key, Version.Version, Version.GetFriendlyName());
	}
}

#if UE_SUPPORT_PARALLEL_IRIS
FTransactionallySafeCriticalSection& UNetConnection::GetSharedNetTickMutex()
{
	return UE::Net::Private::SharedNetTickMutex;
}
#endif // UE_SUPPORT_PARALLEL_IRIS

/*-----------------------------------------------------------------------------
	USimulatedClientNetConnection.
-----------------------------------------------------------------------------*/

USimulatedClientNetConnection::USimulatedClientNetConnection( const FObjectInitializer& ObjectInitializer ) : Super( ObjectInitializer )
{
	SetInternalAck(true);
	SetUnlimitedBunchSizeAllowed(true);	// here to avoid changing the previous behavior controlled by SetInternalAck, but probably not necessary
}

void USimulatedClientNetConnection::HandleClientPlayer( class APlayerController* PC, class UNetConnection* NetConnection )
{
	SetConnectionState(EConnectionState::USOCK_Open);
	PlayerController = PC;
	OwningActor = PC;

	NotifyConnectionUpdated();
}

// ----------------------------------------------------------------

static void	AddSimulatedNetConnections(const TArray<FString>& Args, UWorld* World)
{
	int32 ConnectionCount = 99;
	if (Args.Num() > 0)
	{
		LexFromString(ConnectionCount, *Args[0]);
	}

	// Search for server game net driver. Do it this way so we can cheat in PIE
	UNetDriver* BestNetDriver = nullptr;
	for (TObjectIterator<UNetDriver> NetDriverIt; NetDriverIt; ++NetDriverIt)
	{
		if (NetDriverIt->NetDriverName == NAME_GameNetDriver && NetDriverIt->IsServer())
		{
			BestNetDriver = *NetDriverIt;
			break;
		}
	}

	if (!BestNetDriver)
	{
		return;
	}

	AActor* DefaultViewTarget = nullptr;
	APlayerController* PC = nullptr;
	for (auto Iterator = BestNetDriver->GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		PC = Iterator->Get();
		if (PC)
		{
			DefaultViewTarget = PC->GetViewTarget();
			break;
		}
	}
	

	UE_LOGF(LogNet, Display, "Adding %d Simulated Connections...", ConnectionCount);
	while(ConnectionCount-- > 0)
	{
		USimulatedClientNetConnection* Connection = NewObject<USimulatedClientNetConnection>();
		Connection->InitConnection( BestNetDriver, USOCK_Open, BestNetDriver->GetWorld()->URL, 1000000 );
		Connection->InitSendBuffer();
		BestNetDriver->AddClientConnection( Connection );
		Connection->HandleClientPlayer(PC, Connection);
		Connection->SetClientWorldPackageName(BestNetDriver->GetWorldPackage()->GetFName());
	}	
}

static void	RemoveSimulatedNetConnections(const TArray<FString>& Args, UWorld* World)
{
	int32 ConnectionCount = -1;
	if (Args.Num() > 0)
	{
		LexFromString(ConnectionCount, *Args[0]);
	}

	// Search for server game net driver. Do it this way so we can cheat in PIE
	UNetDriver* BestNetDriver = nullptr;
	for (TObjectIterator<UNetDriver> NetDriverIt; NetDriverIt; ++NetDriverIt)
	{
		if (NetDriverIt->NetDriverName == NAME_GameNetDriver && NetDriverIt->IsServer())
		{
			BestNetDriver = *NetDriverIt;
			break;
		}
	}

	if (!BestNetDriver)
	{
		return;
	}

	int32 RemovedConnections(0);
	for (TObjectIterator<USimulatedClientNetConnection> SimulatedNetConnectionIt; SimulatedNetConnectionIt; ++SimulatedNetConnectionIt)
	{
		USimulatedClientNetConnection* Connection = *SimulatedNetConnectionIt;
		if (Connection && IsValidChecked(Connection) && !Connection->IsUnreachable())
		{
			Connection->Close();
			Connection->MarkAsGarbage();

			RemovedConnections++;
			if (ConnectionCount > 0 && RemovedConnections >= ConnectionCount)
			{
				break;
			}
		}
	}

	UE_LOGF(LogNet, Display, "Removed %d Simulated Connections...", RemovedConnections);
}

FAutoConsoleCommandWithWorldAndArgs AddimulatedConnectionsCmd(TEXT("net.SimulateConnections"), TEXT("Starts a Simulated Net Driver"),	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(AddSimulatedNetConnections) );

FAutoConsoleCommandWithWorldAndArgs RemoveSimulatedConnectionsCmd(TEXT("net.DisconnectSimulatedConnections"), TEXT("Disconnects some simulated connections (0 = all)"), FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(RemoveSimulatedNetConnections));

FAutoConsoleCommandWithWorldAndArgs ForceOnePacketPerBunch(TEXT("net.ForceOnePacketPerBunch"), 
	TEXT("When set to true it will enable AutoFlush on all connections and force a packet to be sent for every bunch we create. This forces one packet per replicated actor and can help find rare ordering bugs"), 
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
{
	bool bEnable = true;
	if (Args.Num() > 0)
	{
		LexTryParseString<bool>(bEnable, *Args[0]);
	}

	UE_LOGF(LogNet, Display, "ForceOnePacketPerBunch set to %ls", bEnable?TEXT("true"):TEXT("false"));

	for (TObjectIterator<UNetConnection> It; It; ++It)
	{
		// Only set autoflush for GameNetDriver connections
		if (It->Driver == World->NetDriver)
		{
			It->SetAutoFlush(bEnable);
		}
	}
}));


// ----------------------------------------------------------------


static void	PrintActorReportFunc(const TArray<FString>& Args, UWorld* InWorld)
{
	// Search for server game net driver. Do it this way so we can cheat in PIE
	UNetDriver* BestNetDriver = nullptr;
	for (TObjectIterator<UNetDriver> NetDriverIt; NetDriverIt; ++NetDriverIt)
	{
		if (NetDriverIt->NetDriverName == NAME_GameNetDriver && NetDriverIt->IsServer())
		{
			BestNetDriver = *NetDriverIt;
			break;
		}
	}

	int32 TotalCount = 0;
	
	TMap<UClass*, int32> ClassCount;
	TMap<UClass*, int32> ActualClassCount;
	TMap<ENetDormancy, int32> DormancyCount;
	FBox BoundingBox;

	TMap<AActor*, int32> RawActorPtrMap;
	TMap<TWeakObjectPtr<AActor>, int32> WeakPtrMap;
	TMap<FObjectKey, int32> ObjKeyMap;

	UWorld* World = BestNetDriver ? BestNetDriver->GetWorld() : InWorld;
	if (!World)
		return;

	
	for (TActorIterator<AActor> It(World); It; ++It)
	{
		AActor* Actor = *It;
		if (Actor->GetIsReplicated() == false)
		{
			continue;
		}

		TotalCount++;
		DormancyCount.FindOrAdd(Actor->NetDormancy)++;

		BoundingBox += Actor->GetActorLocation();

		UClass* CurrentClass = Actor->GetClass();

		ActualClassCount.FindOrAdd(CurrentClass)++;

		while(CurrentClass)
		{
			ClassCount.FindOrAdd(CurrentClass)++;
			CurrentClass = CurrentClass->GetSuperClass();
		}

		RawActorPtrMap.Add(Actor) = FMath::Rand();
		WeakPtrMap.Add(Actor) = FMath::Rand();
		ObjKeyMap.Add(FObjectKey(Actor)) = FMath::Rand();
	}

	ClassCount.ValueSort(TGreater<int32>());
	ActualClassCount.ValueSort(TGreater<int32>());

	UE_LOGF(LogNet, Display, "Class Count (includes inheritance)");
	for (auto MapIt : ClassCount)
	{
		UE_LOGF(LogNet, Display, "%ls - %d", *GetNameSafe(MapIt.Key), MapIt.Value);
	}


	UE_LOGF(LogNet, Display, "");
	UE_LOGF(LogNet, Display, "Class Count (actual clases)");
	for (auto MapIt : ActualClassCount)
	{
		UE_LOGF(LogNet, Display, "%ls - %d", *GetNameSafe(MapIt.Key), MapIt.Value);
	}

	UE_LOGF(LogNet, Display, "");
	UE_LOGF(LogNet, Display, "Complete Bounding Box: %ls", *BoundingBox.ToString());
	UE_LOGF(LogNet, Display, "                 Size: %ls", *BoundingBox.GetSize().ToString());

	UE_LOGF(LogNet, Display, "");

	for (auto MapIt : DormancyCount)
	{
		UE_LOGF(LogNet, Display, "%ls - %d", *UEnum::GetValueAsString(TEXT("/Script/Engine.ENetDormancy"), MapIt.Key), MapIt.Value);
	}

	UE_LOGF(LogNet, Display, "");
	UE_LOGF(LogNet, Display, "Total Replicated Actor Count: %d", TotalCount);


	UE_LOGF(LogNet, Display, "");
	UE_LOGF(LogNet, Display, "Raw Actor Map: ");
	RawActorPtrMap.Dump(*GLog);

	UE_LOGF(LogNet, Display, "");
	UE_LOGF(LogNet, Display, "Weak Ptr Map: ");
	WeakPtrMap.Dump(*GLog);

	UE_LOGF(LogNet, Display, "");
	UE_LOGF(LogNet, Display, "ObjectKey Map: ");
	ObjKeyMap.Dump(*GLog);
}

FAutoConsoleCommandWithWorldAndArgs PrintActorReportCmd(TEXT("net.ActorReport"), TEXT(""),	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(PrintActorReportFunc) );

/*-----------------------------------------------------------------------------
	FChannelRecordImpl
-----------------------------------------------------------------------------*/

namespace FChannelRecordImpl
{

void PushPacketId(FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId)
{
	if (PacketId != WrittenChannelsRecord.LastPacketId)
	{
		FWrittenChannelsRecord::FChannelRecordEntry PacketEntry = { uint32(PacketId), 1u };
		WrittenChannelsRecord.ChannelRecord.Enqueue(PacketEntry);
		WrittenChannelsRecord.LastPacketId = PacketId;
	}
}

void PushChannelRecord(FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId, int32 ChannelIndex)
{
	PushPacketId(WrittenChannelsRecord, PacketId);

	FWrittenChannelsRecord::FChannelRecordEntry ChannelEntry = { uint32(ChannelIndex), 0u };
	WrittenChannelsRecord.ChannelRecord.Enqueue(ChannelEntry);
}

SIZE_T CountBytes(FWrittenChannelsRecord& WrittenChannelsRecord)
{
	return WrittenChannelsRecord.ChannelRecord.AllocatedCapacity() * sizeof(FWrittenChannelsRecord::FChannelRecordEntry);
}

bool HasChannelRecordsToConsumeForPacket(const FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId)
{
	const FWrittenChannelsRecord::FChannelRecordEntryQueue& Record = WrittenChannelsRecord.ChannelRecord;
	if (!Record.IsEmpty())
	{
		FWrittenChannelsRecord::FChannelRecordEntry PacketEntry = Record.Peek();
		return PacketEntry.IsSequence == 1u && PacketEntry.Value == (uint32)PacketId;
	}

	return false;
}

template<class Functor>
void ConsumeChannelRecordsForPacket(FWrittenChannelsRecord& WrittenChannelsRecord, int32 PacketId, Functor&& Func)
{
	const int32 ExpectedSeq = PacketId;
	uint32 PreviousChannelIndex = uint32(-1);

	FWrittenChannelsRecord::FChannelRecordEntryQueue& Record = WrittenChannelsRecord.ChannelRecord;

	// We should ALWAYS have data when we get here
	FWrittenChannelsRecord::FChannelRecordEntry PacketEntry = Record.Peek();
	Record.Pop();

	// Verify that we got the expected packetId
	check(PacketEntry.IsSequence == 1u && PacketEntry.Value == (uint32)PacketId);

	while (!Record.IsEmpty() && Record.PeekNoCheck().IsSequence == 0u)
	{
		const FWrittenChannelsRecord::FChannelRecordEntry Entry = Record.PeekNoCheck();
		Record.PopNoCheck();

		const uint32 ChannelIndex = Entry.Value;

		// Only process channel once per packet
		if (ChannelIndex != PreviousChannelIndex)
		{
			Func(PacketId, ChannelIndex);
			PreviousChannelIndex = ChannelIndex;
		}
	}
}

template<class Functor>
void ConsumeAllChannelRecords(FWrittenChannelsRecord& WrittenChannelsRecord, Functor&& Func)
{
	// Consume all records
	uint32 PreviousChannelIndex = uint32(-1);
	FWrittenChannelsRecord::FChannelRecordEntryQueue& Record = WrittenChannelsRecord.ChannelRecord;

	while (!Record.IsEmpty())
	{
		const FWrittenChannelsRecord::FChannelRecordEntry Entry = Record.PeekNoCheck();
		Record.PopNoCheck();

		const uint32 ChannelIndex = Entry.Value;

		// if the channel writes data multiple non-consecutive times between ticks, the func will be invoked multiple times which should not be an issue.
		if (Entry.IsSequence == 0u && ChannelIndex != PreviousChannelIndex)
		{
			Func(ChannelIndex);
			PreviousChannelIndex = ChannelIndex;
		}
	}
}

}

/**
 * FScopedRepContext
 */

FScopedRepContext::FScopedRepContext(UNetConnection* InConnection, AActor* InActor)
	: Connection(InConnection)
{
	if (Connection)
	{
		check(!Connection->RepContextActor);
		check(!Connection->RepContextLevel);

		Connection->RepContextActor = InActor;
		if (InActor)
		{
			Connection->RepContextLevel = InActor->GetLevel();
		}
	}
}

