// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkDriver.cpp: Unreal network driver base class.
=============================================================================*/

#include "Engine/NetDriver.h"

#include "AnalyticsEventAttribute.h"
#include "Engine/GameInstance.h"
#include "Engine/ServerStatReplicator.h"
#include "Misc/App.h"
#include "EngineStats.h"
#include "GameFramework/GameModeBase.h"
#include "Misc/ConfigCacheIni.h"
#include "Net/Core/NetHandle/NetHandleManager.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "UObject/UObjectIterator.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "PacketHandlers/StatelessConnectHandlerComponent.h"
#include "Engine/LocalPlayer.h"
#include "DrawDebugHelpers.h"
#include "Stats/StatsTrace.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Net/NetworkProfiler.h"
#include "Engine/PackageMapClient.h"
#include "Net/NetIDVariant.h"
#include "Net/RepLayout.h"
#include "Net/DataReplication.h"
#include "Engine/ControlChannel.h"
#include "Engine/ActorChannel.h"
#include "Engine/VoiceChannel.h"
#include "Engine/NetworkObjectList.h"
#include "GameFramework/GameNetworkManager.h"
#include "Net/OnlineEngineInterface.h"
#include "NetworkingDistanceConstants.h"
#include "Engine/ChildConnection.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/Core/Misc/GuidReferences.h"
#include "Net/Core/PropertyConditions/PropertyConditions.h"
#include "Net/DataChannel.h"
#include "GameFramework/PlayerState.h"
#include "Net/PerfCountersHelpers.h"
#include "Stats/StatsMisc.h"
#include "Engine/ReplicationDriver.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/NetworkSettings.h"
#include "Net/NetEmulationHelper.h"
#include "Net/NetworkMetricsDatabase.h"
#include "Net/NetworkMetricsDefs.h"
#include "Net/NetworkMetricsConfig.h"
#include "Net/NetworkMetricsMutators.h"
#include "Net/NetSubObjectRegistryGetter.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Templates/Greater.h"
#include "UObject/RemoteExecutor.h"
#include "UObject/Stack.h"
#include "Iris/IrisConfig.h"
#include "Iris/Core/IrisDebugging.h"
#include "Iris/Core/IrisProfiler.h"
#include "Iris/Core/IrisMemoryTracker.h"
#include "Net/Experimental/Iris/DataStreamChannel.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ReplicationView.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilter.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#include "Iris/ReplicationSystem/NameTokenStore.h"
#include "Iris/ReplicationSystem/StringTokenStore.h"
#include "GameplayTagTokenStore.h"
#include "Interfaces/ReplicatedObjectInterface.h"


#if USE_SERVER_PERF_COUNTERS
#endif
#include "GenericPlatform/GenericPlatformCrashContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetDriver)

#if WITH_EDITOR
#include "Editor.h"
#else
#include "HAL/LowLevelMemStats.h"
#include "UObject/Package.h"
#endif

DECLARE_LLM_MEMORY_STAT(TEXT("NetDriver"), STAT_NetDriverLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(NetDriver, NAME_None, TEXT("Networking"), GET_STATFNAME(STAT_NetDriverLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));

// Default net driver stats
DEFINE_STAT(STAT_Ping);
DEFINE_STAT(STAT_Channels);
DEFINE_STAT(STAT_MaxPacketOverhead);
DEFINE_STAT(STAT_InRate);
DEFINE_STAT(STAT_OutRate);
DEFINE_STAT(STAT_InRateClientMax);
DEFINE_STAT(STAT_InRateClientMin);
DEFINE_STAT(STAT_InRateClientAvg);
DEFINE_STAT(STAT_InPacketsClientMax);
DEFINE_STAT(STAT_InPacketsClientMin);
DEFINE_STAT(STAT_InPacketsClientAvg);
DEFINE_STAT(STAT_OutRateClientMax);
DEFINE_STAT(STAT_OutRateClientMin);
DEFINE_STAT(STAT_OutRateClientAvg);
DEFINE_STAT(STAT_OutPacketsClientMax);
DEFINE_STAT(STAT_OutPacketsClientMin);
DEFINE_STAT(STAT_OutPacketsClientAvg);
DEFINE_STAT(STAT_NetNumClients);
DEFINE_STAT(STAT_InPackets);
DEFINE_STAT(STAT_OutPackets);
DEFINE_STAT(STAT_InBunches);
DEFINE_STAT(STAT_OutBunches);
DEFINE_STAT(STAT_OutLoss);
DEFINE_STAT(STAT_InLoss);
DEFINE_STAT(STAT_NumConsideredActors);
DEFINE_STAT(STAT_PrioritizedActors);
DEFINE_STAT(STAT_NumReplicatedActors);
DEFINE_STAT(STAT_NumReplicatedActorBytes);
DEFINE_STAT(STAT_NumRelevantDeletedActors);
DEFINE_STAT(STAT_NumActorChannels);
DEFINE_STAT(STAT_NumActors);
DEFINE_STAT(STAT_NumNetActors);
DEFINE_STAT(STAT_NumDormantActors);
DEFINE_STAT(STAT_NumInitiallyDormantActors);
DEFINE_STAT(STAT_NumNetGUIDsAckd);
DEFINE_STAT(STAT_NumNetGUIDsPending);
DEFINE_STAT(STAT_NumNetGUIDsUnAckd);
DEFINE_STAT(STAT_ObjPathBytes);
DEFINE_STAT(STAT_NetGUIDInRate);
DEFINE_STAT(STAT_NetGUIDOutRate);
DEFINE_STAT(STAT_NetSaturated);
DEFINE_STAT(STAT_ImportedNetGuids);
DEFINE_STAT(STAT_PendingOuterNetGuids);
DEFINE_STAT(STAT_UnmappedReplicators);
DEFINE_STAT(STAT_OutgoingReliableMessageQueueMaxSize);
DEFINE_STAT(STAT_IncomingReliableMessageQueueMaxSize);

// Voice specific stats
DEFINE_STAT(STAT_VoiceBytesSent);
DEFINE_STAT(STAT_VoiceBytesRecv);
DEFINE_STAT(STAT_VoicePacketsSent);
DEFINE_STAT(STAT_VoicePacketsRecv);
DEFINE_STAT(STAT_PercentInVoice);
DEFINE_STAT(STAT_PercentOutVoice);

#if !UE_BUILD_SHIPPING
// Packet stats
DEFINE_STAT(STAT_MaxPacket);
DEFINE_STAT(STAT_MaxPacketMinusReserved);
DEFINE_STAT(STAT_PacketReservedTotal);
DEFINE_STAT(STAT_PacketReservedNetConnection);
DEFINE_STAT(STAT_PacketReservedPacketHandler);
DEFINE_STAT(STAT_PacketReservedHandshake);
#endif

DECLARE_CYCLE_STAT(TEXT("NetDriver AddClientConnection"), Stat_NetDriverAddClientConnection, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("NetDriver ProcessRemoteFunction"), STAT_NetProcessRemoteFunc, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("Process Prioritized Actors Time"), STAT_NetProcessPrioritizedActorsTime, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("NetDriver TickFlush"), STAT_NetTickFlush, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("NetDriver TickFlush GatherStats"), STAT_NetTickFlushGatherStats, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("NetDriver TickFlush GatherStatsPerfCounters"), STAT_NetTickFlushGatherStatsPerfCounters, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("ReceiveRPC_ProcessRemoteFunction"), STAT_NetReceiveRPC_ProcessRemoteFunction, STATGROUP_Game);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Net bunch time % overshoot frames"), STAT_NetInBunchTimeOvershootPercent, STATGROUP_Net);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Saturated Connections"), STAT_NumSaturatedConnections, STATGROUP_Net);
DECLARE_DWORD_COUNTER_STAT(TEXT("SharedSerialization RPC Hit"), STAT_SharedSerializationRPCHit, STATGROUP_Net);
DECLARE_DWORD_COUNTER_STAT(TEXT("SharedSerialization RPC Miss"), STAT_SharedSerializationRPCMiss, STATGROUP_Net);
DECLARE_DWORD_COUNTER_STAT(TEXT("Empty Object Replicate Properties Skipped"), STAT_NumSkippedObjectEmptyUpdates, STATGROUP_Net);
DECLARE_DWORD_COUNTER_STAT(TEXT("SharedSerialization Property Hit"), STAT_SharedSerializationPropertyHit, STATGROUP_Net);
DECLARE_DWORD_COUNTER_STAT(TEXT("SharedSerialization Property Miss"), STAT_SharedSerializationPropertyMiss, STATGROUP_Net);

DEFINE_LOG_CATEGORY_STATIC(LogNetSyncLoads, Log, All);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(NETCORE_API, Networking);

int32 GNumSaturatedConnections; // Counter for how many connections are skipped/early out due to bandwidth saturation
int32 GNumSharedSerializationHit;
int32 GNumSharedSerializationMiss;
int32 GNumSkippedObjectEmptyUpdates;
int32 GNumReplicationViewers;

extern int32 GNetRPCDebug;

namespace UE::Net::Private
{
	/** 
	* ConnectionTickTask is used to run each individual connection's Tick step in parallel
	* When running these tasks, everything in NetConnection->Tick is expected to be thread safe
	*/
	struct FConnectionTickTaskData
	{
		float DeltaSeconds = 0.0f;

		FConnectionTickTaskData(float InDeltaSeconds)
			: DeltaSeconds(InDeltaSeconds)
		{}
	};

	class FConnectionTickTask
	{
	public:
		FConnectionTickTask(UNetConnection* InConnection, FConnectionTickTaskData& InConnectionTickTaskData)
			: NetConnection(InConnection)
			, ConnectionTickTaskData(InConnectionTickTaskData)
		{
		}

		FORCENOINLINE void DoTask() const
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ReplicationNetConnectionTickTask);

			NetConnection->Tick(ConnectionTickTaskData.DeltaSeconds);
		}

	protected:
		UNetConnection* NetConnection = nullptr;
		FConnectionTickTaskData& ConnectionTickTaskData;
	};

	struct FAutoDestructProperty
	{
		FProperty* Property = nullptr;
		void* LocalParameters = nullptr;

		FAutoDestructProperty(FProperty* InProp, void* InLocalParams) 
			: Property(InProp)
			, LocalParameters(InLocalParams)
		{
			check(Property->HasAnyPropertyFlags(CPF_OutParm));
		}

		FAutoDestructProperty(FAutoDestructProperty&& rhs)
			: Property(rhs.Property)
			, LocalParameters(rhs.LocalParameters)
		{
			rhs.Property = nullptr;
			rhs.LocalParameters = nullptr;
		}

		~FAutoDestructProperty()
		{
			if (Property)
			{
				Property->DestroyValue_InContainer(LocalParameters);
			}
		}
	};

	TArray<UE::Net::Private::FAutoDestructProperty> CopyOutParametersToLocalParameters(UFunction* Function, FOutParmRec* OutParms, void* LocalParms, UObject* TargetObj)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CopyOutParametersToLocalParameters);
		TArray<UE::Net::Private::FAutoDestructProperty> CopiedProperties;

		// Look for CPF_OutParm's, we'll need to copy these into the local parameter memory manually
		// The receiving side will pull these back out when needed
		for (TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & (CPF_Parm | CPF_ReturnParm)) == CPF_Parm; ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_OutParm))
			{
				if (OutParms == nullptr)
				{
					UE_LOGF(LogNet, Warning, "Missing OutParms. Property: %ls, Function: %ls, Actor: %ls", *It->GetName(), *GetNameSafe(Function), *GetFullNameSafe(TargetObj));
					continue;
				}

				FOutParmRec* Out = OutParms;

				checkSlow(Out);

				while (Out->Property != *It)
				{
					Out = Out->NextOutParm;
					checkSlow(Out);
				}

				void* Dest = It->ContainerPtrToValuePtr<void>(LocalParms);

				const int32 CopySize = It->GetElementSize() * It->ArrayDim;

				check(((uint8*)Dest - (uint8*)LocalParms) + CopySize <= Function->ParmsSize);

				It->InitializeValue(Dest);
				It->CopyCompleteValue(Dest, Out->PropAddr);

				CopiedProperties.Emplace(*It, LocalParms);
			}
		}

		return CopiedProperties;
	}

	static bool bIgnoreStaticActorDestruction = false;

	void ApplyReplicationSystemConfig(const FNetDriverReplicationSystemConfig& ReplicationSystemConfig, UReplicationSystem::FReplicationSystemParams& OutParams)
	{
		if (ReplicationSystemConfig.MaxReplicatedObjectCount != 0)
		{
			OutParams.MaxReplicatedObjectCount = ReplicationSystemConfig.MaxReplicatedObjectCount;
		}

		if (ReplicationSystemConfig.InitialNetObjectListCount != 0)
		{
			OutParams.InitialNetObjectListCount = ReplicationSystemConfig.InitialNetObjectListCount;
		}

		if (ReplicationSystemConfig.NetObjectListGrowCount != 0)
		{
			OutParams.NetObjectListGrowCount = ReplicationSystemConfig.NetObjectListGrowCount;
		}

		if (ReplicationSystemConfig.PreAllocatedMemoryBuffersObjectCount != 0)
		{
			OutParams.PreAllocatedMemoryBuffersObjectCount = ReplicationSystemConfig.PreAllocatedMemoryBuffersObjectCount;
		}
		
		if (ReplicationSystemConfig.MaxReplicationWriterObjectCount!= 0)
		{
			OutParams.MaxReplicationWriterObjectCount = ReplicationSystemConfig.MaxReplicationWriterObjectCount;
		}

		if (ReplicationSystemConfig.MaxDeltaCompressedObjectCount != 0)
		{
			OutParams.MaxDeltaCompressedObjectCount = ReplicationSystemConfig.MaxDeltaCompressedObjectCount;
		}

		if (ReplicationSystemConfig.MaxNetObjectGroupCount != 0)
		{
			OutParams.MaxNetObjectGroupCount = ReplicationSystemConfig.MaxNetObjectGroupCount;
		}

		OutParams.bAllowParallelTasks = ReplicationSystemConfig.bAllowParallelTasks;
	}

	bool IsGuidInOuterChain(const FNetGUIDCache& GuidCache, const FNetGuidCacheObject* CacheObj, FNetworkGUID GuidMatch)
	{
		while (CacheObj && CacheObj->OuterGUID.IsValid())
		{
			if (CacheObj->OuterGUID == GuidMatch)
			{
				return true;
			}

			CacheObj = GuidCache.GetCacheObject(CacheObj->OuterGUID);
		}

		return false;
	}

	static float ClientIncomingBunchFrameTimeLimitMS = 0.0f;

	static FAutoConsoleVariableRef CVarClientIncomingBunchFrameTimeLimitMS(
		TEXT("net.ClientIncomingBunchFrameTimeLimitMS"),
		ClientIncomingBunchFrameTimeLimitMS,
		TEXT("Time in milliseconds to limit client incoming bunch processing to. If 0, no limit. As long as we're below the limit, will start processing another bunch. A single bunch that takes a while to process can overshoot the limit. ")
		TEXT("After the limit is hit, remaining bunches in a packet are queued, and the IpNetDriver will not process any more packets in the current frame."));

	int32 SerializeNewActorOverrideLevel = 1;
	static FAutoConsoleVariableRef CVarNetSerializeNewActorOverrideLevel(
		TEXT("net.SerializeNewActorOverrideLevel"),
		SerializeNewActorOverrideLevel,
		TEXT("If true, servers will serialize a spawned, replicated actor's level so the client attempts to spawn it into that level too. If false, clients will spawn all these actors into the persistent level."));

	int32 SendDormantDestructionOnRemoval = 1;
	static FAutoConsoleVariableRef CVarNetSendDormantDestructionOnRemoval(
		TEXT("net.SendDormantDestructionOnRemoval"),
		SendDormantDestructionOnRemoval,
		TEXT("If true and net.SerializeNewActorOverrideLevel = 0, servers will send destruction info for dormant destroyed actors in invisible levels, regardless of how they were removed. This ensures they destroy on the client, in-case the they (or the levels they are in) were removed unexpectedly."));

	bool bUpdateReplicationSystemWithNoConnections = true;
	static FAutoConsoleVariableRef CVarNetUpdateReplicationSystemWithNoConnections(
		TEXT("net.iris.UpdateReplicationSystemWithNoConnections"),
		bUpdateReplicationSystemWithNoConnections,
		TEXT("If true ReplicationSystem::NetUpdate will be called even if we have no connections to ensure that we do not accumulate pending work."));

	bool bAllowParallelNetTickAtRuntime = false;
	static FAutoConsoleVariableRef CVarNetAllowParallelNetTick(
		TEXT("net.iris.AllowParallelNetTick"),
		bAllowParallelNetTickAtRuntime,
		TEXT("If true, NetConnection::Tick can be run in parallel. This is designed as a runtime disable flag to be used alongside config changes. To enable parallel tick, it also requires the NetDriver config bCanUseParallelNetConnectionTick=true and ReplicationSystemConfig bAllowParallelTasks=true."));
	
#if UE_WITH_REMOTE_OBJECT_HANDLE
	bool bServerReplicateActorsGranularTransactions = true;
	static FAutoConsoleVariableRef CVarServerReplicateActorsGranularTransactions(
		TEXT("net.ServerReplicateActorsGranularTransactions"),
		bServerReplicateActorsGranularTransactions,
		TEXT("If true and bReplicateTransactionally is true, ServerReplicateActors will use a transaction per connection instead of one transaction for the whole function."));
#endif

	static void DumpActiveNetActors(UWorld* InWorld)
	{
		UNetDriver *NetDriver = InWorld->NetDriver;
		if (!NetDriver)
		{
			return;
		}

		UE_LOGF(LogConsoleResponse, Log, "----------------------------");
		UE_LOGF(LogConsoleResponse, Log, "Active network actors in %ls:", *NetDriver->GetName());

		TMap<UClass*, int32> ActorClassCount;

		for (const TSharedPtr<FNetworkObjectInfo>& ObjectInfo : NetDriver->GetNetworkObjectList().GetActiveObjects())
		{
			AActor* Actor = ObjectInfo->WeakActor.Get();
			if (!Actor)
			{
				continue;
			}

			UE_LOGF(LogConsoleResponse, Log, " * %ls (%ls)", *Actor->GetName(), *Actor->GetClass()->GetName());

			int32& ClassCount = ActorClassCount.FindOrAdd(Actor->GetClass(), 0);
			ClassCount++;
		}

		UE_LOGF(LogConsoleResponse, Log, "----------------------------");

		UE_LOGF(LogConsoleResponse, Log, "Number of each active actor class type:");

		for (TPair<UClass*, int32> ClassCount : ActorClassCount)
		{
			UE_LOGF(LogConsoleResponse, Log, " %ls: %d", *ClassCount.Key->GetName(), ClassCount.Value);
		}

		UE_LOGF(LogConsoleResponse, Log, "----------------------------");
		UE_LOGF(LogConsoleResponse, Log, "Total: %d", NetDriver->GetNetworkObjectList().GetActiveObjects().Num());
		UE_LOGF(LogConsoleResponse, Log, "----------------------------");
	}

	FAutoConsoleCommandWithWorld DumpActiveNetActorsCommand(
		TEXT("net.DumpActiveNetActors"), 
		TEXT("Dumps all of the active net actors in the replication system."),
		FConsoleCommandWithWorldDelegate::CreateStatic(DumpActiveNetActors)
	);

} //namespace UE::Net::Private

namespace UE::Net
{
	FScopedIgnoreStaticActorDestruction::FScopedIgnoreStaticActorDestruction()
	{
		bCachedValue = UE::Net::Private::bIgnoreStaticActorDestruction;
		UE::Net::Private::bIgnoreStaticActorDestruction = true;
	}

	FScopedIgnoreStaticActorDestruction::~FScopedIgnoreStaticActorDestruction()
	{
		UE::Net::Private::bIgnoreStaticActorDestruction = bCachedValue;
	}

	bool ShouldIgnoreStaticActorDestruction()
	{
		return UE::Net::Private::bIgnoreStaticActorDestruction;
	}

	uint32 CountReplicatedSubObjectsOfActor(AActor* ReplicatedActor)
	{
		return UE::Net::FSubObjectRegistryGetter::CountReplicatedSubObjectsOfActor(ReplicatedActor);
	}

	bool bDiscardTornOffActorRPCs = true;
	FAutoConsoleVariableRef CVarNetDiscardTornOffActorRPCs(
		TEXT("net.DiscardTornOffActorRPCs"),
		bDiscardTornOffActorRPCs,
		TEXT("If enabled, discard RPCs if the actor has been torn off."),
		ECVF_Default);
} // end namespace UE::Net

#if UE_BUILD_SHIPPING
#define DEBUG_REMOTEFUNCTION(Format, ...)
#else
#define DEBUG_REMOTEFUNCTION(Format, ...) UE_LOG(LogNet, VeryVerbose, Format, __VA_ARGS__);
#endif

// CVars
int32 GInitialNetworkId = -1;
static FAutoConsoleVariableRef CVarInitialNetworkId(
	TEXT("net.InitialNetworkId"),
	GInitialNetworkId,
	TEXT("The initial value to use when assigning network ids (FNetRefHandle or FNetGUID). If -1 then not explicitly set and the default for the replication system is used."),
	ECVF_Default);

int32 GSetNetDormancyEnabled = 1;
static FAutoConsoleVariableRef CVarSetNetDormancyEnabled(
	TEXT("net.DormancyEnable"),
	GSetNetDormancyEnabled,
	TEXT("Enables Network Dormancy System for reducing CPU and bandwidth overhead of infrequently updated actors\n")
	TEXT("1 Enables network dormancy. 0 disables network dormancy."),
	ECVF_Default);

int32 GNetDormancyValidate = 0;
static FAutoConsoleVariableRef CVarNetDormancyValidate(
	TEXT("net.DormancyValidate"),
	GNetDormancyValidate,
	TEXT("Validates that dormant actors do not change state while in a dormant state (on server only)")
	TEXT("0: Dont validate. 1: Validate on wake up. 2: Validate on each net update"),
	ECVF_Default);

bool GbNetReuseReplicatorsForDormantObjects = false;
static FAutoConsoleVariableRef CVarNetReuseReplicatorsForDormantObjects(
	TEXT("Net.ReuseReplicatorsForDormantObjects"),
	GbNetReuseReplicatorsForDormantObjects,
	TEXT("When true, Server's will persist and attempt to reuse replicators for Dormant Actors and Objects. This can cut down on bandwidth by preventing redundant information from being sent when waking objects from Dormancy."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarNetDebugDraw(
	TEXT("net.DebugDraw"),
	0,
	TEXT("Draws debug information for network dormancy and relevancy\n")
	TEXT("1 Enables network debug drawing. 0 disables."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarNetDebugDrawCullDistance(
	TEXT("net.DebugDrawCullDistance"),
	0.f,
	TEXT("Cull distance for net.DebugDraw. World Units")
	TEXT("Max world units an actor can be away from the local view to draw its dormancy status. Zero disables culling"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarUseAdaptiveNetUpdateFrequency(
	TEXT( "net.UseAdaptiveNetUpdateFrequency" ), 
	0, 
	TEXT( "If 1, NetUpdateFrequency will be calculated based on how often actors actually send something when replicating" ) );

TAutoConsoleVariable<int32> CVarNetAllowEncryption(
	TEXT("net.AllowEncryption"),
	1,
	TEXT("If true, the engine will attempt to load an encryption PacketHandler component and fill in the EncryptionToken parameter ")
	TEXT("of the NMT_Hello message based on the ?EncryptionToken= URL option and call callbacks if it's non-empty. ")
	TEXT("Additionally, a value of '2' will make the EncryptionToken required - which is enforced serverside. ")
	TEXT("(0 = Disabled, 1 = Allowed (default), 2 = Required)"));

static TAutoConsoleVariable<int32> CVarActorChannelPool(
	TEXT("net.ActorChannelPool"),
	1,
	TEXT("If nonzero, actor channels will be pooled to save memory and object creation cost."));

static TAutoConsoleVariable<int32> CVarAllowReliableMulticastToNonRelevantChannels(
	TEXT("net.AllowReliableMulticastToNonRelevantChannels"),
	1,
	TEXT("Allow Reliable Server Multicasts to be sent to non-Relevant Actors, as long as their is an existing ActorChannel."));

static int32 GNetControlChannelDestructionInfo = 0;
static FAutoConsoleVariableRef CVarNetControlChannelDestructionInfo(
	TEXT("net.ControlChannelDestructionInfo"),
	GNetControlChannelDestructionInfo,
	TEXT("If enabled, send destruction info updates via the control channel instead of creating a new actor channel.")
	TEXT("0: Old behavior, use an actor channel. 1: New behavior, use the control channel"),
	ECVF_Default);

static int32 GNetResetAckStatePostSeamlessTravel = 0;
static FAutoConsoleVariableRef CVarNetResetAckStatePostSeamlessTravel(
	TEXT("net.ResetAckStatePostSeamlessTravel"),
	GNetResetAckStatePostSeamlessTravel,
	TEXT("If 1, the server will reset the ack state of the package map after seamless travel. Increases bandwidth usage, but may resolve some issues with GUIDs not being available on clients after seamlessly traveling."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarAddNetDriverInfoToNetAnalytics(
	TEXT("net.AddNetDriverInfoToNetAnalytics"),
	false,
	TEXT("Automatically add NetDriver information to the NetAnalytics cache"));

static int32 GNetDisableRandomNetUpdateDelay = 0;
static FAutoConsoleVariableRef CVarNetDisableRandomNetUpdateDelay(
	TEXT("net.DisableRandomNetUpdateDelay"),
	GNetDisableRandomNetUpdateDelay,
	TEXT("If 1, the server will not add a random delay to an actors' NextUpdateTime."),
	ECVF_Default);

static bool GbEnableNetStats = false;
static FAutoConsoleVariableRef CVarEnableNetStats(
	TEXT("net.EnableNetStats"),
	GbEnableNetStats,
	TEXT("Enable the calculation of network stats and storing them in the network metrics database."),
	ECVF_Default);

static bool GbEnableSupportForNoPawnConnection = false;
static FAutoConsoleVariableRef CVarEnableSupportForNoPawnConnection(
	TEXT("net.EnableSupportForNoPawnConnection"),
	GbEnableSupportForNoPawnConnection,
	TEXT("Enable server support for NMT_JoinNoPawn and NMT_JoinNoPawnSplit control messages."),
	ECVF_Default);

static bool GbEnableActorCountInStats = false;
static FAutoConsoleVariableRef CVarEnableActorCountInStats(
	TEXT("net.EnableActorCountInStats"),
	GbEnableActorCountInStats,
	TEXT("Enable the tracking of pawns and player controller counts in the network metrics database."),
	ECVF_Default);

namespace UE::Net
{
	static FString GRequiredEncryptionNetDriverDefNames_Internal = "all";
	static TArray<FName> GRequiredEncryptionNetDriverDefNames;

	// Safe to update static here, but not CVar system cached value
	static void ParseRequiredEncryptionCVars(IConsoleVariable* =nullptr)
	{
		GRequiredEncryptionNetDriverDefNames.Reset();

		if (GRequiredEncryptionNetDriverDefNames_Internal.TrimStartAndEnd() == TEXT("all"))
		{
			if (GEngine != nullptr)
			{
				for (const FNetDriverDefinition& CurDef : GEngine->NetDriverDefinitions)
				{
					if (CurDef.DefName != NAME_DemoNetDriver)
					{
						GRequiredEncryptionNetDriverDefNames.Add(CurDef.DefName);
					}
				}
			}
		}
		else
		{
			TArray<FString> NetDriverNames;
			GRequiredEncryptionNetDriverDefNames_Internal.ParseIntoArray(NetDriverNames, TEXT(","));

			for (const FString& CurName : NetDriverNames)
			{
				GRequiredEncryptionNetDriverDefNames.Add(FName(CurName.TrimStartAndEnd()));
			};
		}
	}

	static FAutoConsoleVariableRef CVarNetRequiredEncryptionNetDriverDefNames(
		TEXT("net.RequiredEncryptionNetDriverDefNames"),
		GRequiredEncryptionNetDriverDefNames_Internal,
		TEXT("Comma-delimited list of NetDriverDefinition's where 'IsEncryptionRequired' will return true, when 'net.AllowEncryption' is 2. ")
		TEXT("(specifying 'all' will enable this for all NetDriverDefinition's)"),
		FConsoleVariableDelegate::CreateStatic(&ParseRequiredEncryptionCVars));

	struct FServerReplicateActors_ForConnectionParams
	{
		UNetConnection* Connection;
		TArray<FNetViewer>& ConnectionViewers;
		float DeltaSeconds;
		const TArray<FNetworkObjectInfo*>& ConsiderList;
		int32& InOutUpdated;
		bool bCPUSaturated;
	};

	namespace Private
	{
		static int32 CleanUpRenamedDynamicActors = 0;

		static FAutoConsoleVariableRef CVarNetCleanUpRenamedDynamicActors(
			TEXT("net.CleanUpRenamedDynamicActors"),
			CleanUpRenamedDynamicActors,
			TEXT("When enabled, dynamic actors that change outers via Rename() on the server will close their channel ")
			TEXT("or send a destruction info to clients without the actor's new level loaded."));

		static FName StatNameRawPing("RawPing");
	}

} // end namespace UE::Net



/*-----------------------------------------------------------------------------
	UNetDriver implementation.
-----------------------------------------------------------------------------*/

UNetDriver::UNetDriver(const FObjectInitializer& ObjectInitializer)
:	UObject(ObjectInitializer)
,	MaxInternetClientRate(10000)
, 	MaxClientRate(15000)
,   ServerConnection(nullptr)
,	ClientConnections()
,	MappedClientConnections()
,	RecentlyDisconnectedClients()
,	RecentlyDisconnectedTrackingTime(0)
,	ConnectionlessHandler()
,	StatelessConnectComponent()
,	AnalyticsProvider()
,	AnalyticsAggregator()
,   World(nullptr)
,	NetDriverDefinition(NAME_None)
,	MaxChannelsOverride(0)
,   Notify(nullptr)
,	ElapsedTime( 0.0 )
,	bInTick(false)
,	bPendingDestruction(false)
#if DO_ENABLE_NET_TEST
,   bForcedPacketSettings(false)
#endif
,   bDidHitchLastFrame(false)
,   bHasReplayConnection(false)
,   bMaySendProperties(false)
,   bSkipServerReplicateActors(false)
,   bSkipClearVoicePackets(false)
,   bNoTimeouts(false)
,   bNeverApplyNetworkEmulationSettings(false)
,   bIsPeer(false)
,   ProfileStats(false)
,   bSkipLocalStats(false)
,   bCollectNetStats(false)
,   bIsStandbyCheckingEnabled(false)
,   bHasStandbyCheatTriggered(false)
,	LastTickDispatchRealtime( 0.f )
,	InBytes(0)
,	InTotalBytes(0)
,	OutBytes(0)
,	OutTotalBytes(0)
,	NetGUIDOutBytes(0)
,	NetGUIDInBytes(0)
,	InPackets(0)
,	PrevInPackets(0)
,	InTotalPackets(0)
,	OutPackets(0)
,	PrevOutPackets(0)
,	OutTotalPackets(0)
,	InBunches(0)
,	OutBunches(0)
,	InTotalBunches(0)
,	OutTotalBunches(0)
,	OutTotalReliableBunches(0)
,	InTotalReliableBunches(0)
,	InPacketsLost(0)
,	PrevInPacketsLost(0)
,	InTotalPacketsLost(0)
,	OutPacketsLost(0)
,	PrevOutPacketsLost(0)
,	OutTotalPacketsLost(0)
,	StatUpdateTime(0.0)
,	StatPeriod(1.f)
,	TotalRPCsCalled(0)
,	OutTotalAcks(0)
,	LastCleanupTime(0.0)
,	NetTag(0)
#if NET_DEBUG_RELEVANT_ACTORS
,	DebugRelevantActors(false)
#endif // NET_DEBUG_RELEVANT_ACTORS
#if !UE_BUILD_SHIPPING
,	SendRPCDel()
#endif
,	ProcessQueuedBunchesCurrentFrameMilliseconds(0.0f)
,	DDoS()
,	LocalAddr(nullptr)
,	ProxyType(UE::Net::EProxyType::None)
,	DummyNetworkObjectInfo(MakePimpl<FNetworkObjectInfo>())
,	NetworkObjects(new FNetworkObjectList)
,	LagState(ENetworkLagState::NotLagging)
,	DuplicateLevelID(INDEX_NONE)
,	PacketLossBurstEndTime(-1.0f)
,	OutTotalNotifiedPackets(0u)
,	NetTraceId(NetTraceInvalidGameInstanceId)
{
	UpdateDelayRandomStream.Initialize(FApp::bUseFixedSeed ? GetFName() : NAME_None);
}

UNetDriver::UNetDriver(FVTableHelper& Helper)
	: Super(Helper)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UNetDriver::~UNetDriver() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UNetDriver::InitPacketSimulationSettings()
{
#if DO_ENABLE_NET_TEST
	using namespace UE::Net::Private::NetEmulationHelper;

	if (bNeverApplyNetworkEmulationSettings)
	{
		return;
	}

	if (bForcedPacketSettings)
	{
		return;
	}

	if (HasPersistentPacketEmulationSettings())
	{
		ApplyPersistentPacketEmulationSettings(this);
		return;
	}

	// Read the settings from .ini and command line, with the command line taking precedence	
	PacketSimulationSettings.ResetSettings();
	PacketSimulationSettings.LoadConfig(*NetDriverDefinition.ToString());
	PacketSimulationSettings.ParseSettings(FCommandLine::Get(), *NetDriverDefinition.ToString());
#endif
}

#if DO_ENABLE_NET_TEST
bool UNetDriver::IsSimulatingPacketLossBurst() const
{
	return PacketLossBurstEndTime > ElapsedTime;
}
#endif

void UNetDriver::PostInitProperties()
{
	Super::PostInitProperties();

	// By default we're the game net driver and any child ones must override this
	NetDriverName = NAME_GameNetDriver;

	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		// Init ConnectionIdHandler
		{
			constexpr uint32 IdCount = 1024;
			ConnectionIdHandler.Init(IdCount);
		}

		InitPacketSimulationSettings();

		
		RoleProperty		= FindFieldChecked<FProperty>( AActor::StaticClass(), TEXT("Role"      ) );
		RemoteRoleProperty	= FindFieldChecked<FProperty>( AActor::StaticClass(), TEXT("RemoteRole") );

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (GInitialNetworkId >= 0)
		{
			GuidCache = TSharedPtr<FNetGUIDCache>(new FNetGUIDCache(this, GInitialNetworkId));
		}
		else
		{
			GuidCache = TSharedPtr<FNetGUIDCache>(new FNetGUIDCache(this));
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		NetCache			= TSharedPtr< FClassNetCacheMgr >( new FClassNetCacheMgr() );

		ProfileStats		= FParse::Param(FCommandLine::Get(),TEXT("profilestats"));

		EvaluateNoTimeouts();

		OnLevelRemovedFromWorldHandle = FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &UNetDriver::OnLevelRemovedFromWorld);
		OnLevelAddedToWorldHandle = FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UNetDriver::OnLevelAddedToWorld);
		PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UNetDriver::PostGarbageCollect);
		ReportSyncLoadDelegateHandle = FNetDelegates::OnSyncLoadDetected.AddUObject(this, &UNetDriver::ReportSyncLoad);

		LoadChannelDefinitions();
	}

	NotifyGameInstanceUpdated();
}

void UNetDriver::EvaluateNoTimeouts()
{
#if !UE_BUILD_SHIPPING
	bNoTimeouts = bNoTimeouts || FParse::Param(FCommandLine::Get(), TEXT("NoTimeouts")) ? true : false;
#endif // !UE_BUILD_SHIPPING

#if WITH_EDITOR
	// Do not time out in PIE since the server is local.
	bNoTimeouts = bNoTimeouts || (GEditor && GEditor->PlayWorld);
#endif // WITH_EDITOR
}

void UNetDriver::PostReloadConfig(FProperty* PropertyToLoad)
{
	Super::PostReloadConfig(PropertyToLoad);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		EvaluateNoTimeouts();
	}

	LoadChannelDefinitions();
}

void UNetDriver::LoadChannelDefinitions()
{
	LLM_SCOPE_BYTAG(NetDriver);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (ServerConnection != nullptr)
		{
			UE_LOGF(LogNet, Warning, "Loading channel definitions while a server connection is active.");
		}

		if (ClientConnections.Num() != 0)
		{
			UE_LOGF(LogNet, Warning, "Loading channel definitions while client connections are active.");
		}

		TSet<int32> StaticChannelIndices;

		ChannelDefinitionMap.Reset();

		for (FChannelDefinition& ChannelDef : ChannelDefinitions)
		{
			UE_CLOGF(!ChannelDef.ChannelName.ToEName() || !ShouldReplicateAsInteger(*ChannelDef.ChannelName.ToEName(), ChannelDef.ChannelName),
				LogNet, Warning, "Channel name will be serialized as a string: %ls", *ChannelDef.ChannelName.ToString());
			UE_CLOGF(ChannelDefinitionMap.Contains(ChannelDef.ChannelName), LogNet, Error, "Channel name is defined multiple times: %ls", *ChannelDef.ChannelName.ToString());
			UE_CLOGF(StaticChannelIndices.Contains(ChannelDef.StaticChannelIndex), LogNet, Error, "Channel static index is already in use: %ls %i", *ChannelDef.ChannelName.ToString(), ChannelDef.StaticChannelIndex);

			ChannelDef.ChannelClass = StaticLoadClass(UChannel::StaticClass(), nullptr, *ChannelDef.ClassName.ToString(), nullptr, LOAD_Quiet);
			if (ChannelDef.ChannelClass != nullptr)
			{
				ChannelDefinitionMap.Add(ChannelDef.ChannelName, ChannelDef);
			}

			if (ChannelDef.StaticChannelIndex != INDEX_NONE)
			{
				StaticChannelIndices.Add(ChannelDef.StaticChannelIndex);
			}
		}

		ensureMsgf(IsKnownChannelName(NAME_Control), TEXT("Control channel type is not properly defined."));
	}
}

void UNetDriver::NotifyGameInstanceUpdated()
{
	const UWorld* LocalWorld = GetWorld();

#if UE_NET_TRACE_ENABLED
	if (GetNetTraceId() != NetTraceInvalidGameInstanceId && LocalWorld && LocalWorld->WorldType)
	{
		FString InstanceName = FString::Printf(TEXT("%s (%s)"), *GetNameSafe(LocalWorld->GetGameInstance()), LexToString(LocalWorld->WorldType.GetValue()));
		UE_NET_TRACE_UPDATE_INSTANCE(GetNetTraceId(), IsServer(), *InstanceName);
	}
#endif

#if WITH_EDITOR
	if (ReplicationSystem && LocalWorld)
	{
		ReplicationSystem->SetPIEInstanceID(World->GetPackage()->GetPIEInstanceID());
	}
#endif
}

bool UNetDriver::HasSupportForNoPawnConnection()
{
	return GbEnableSupportForNoPawnConnection || bSupportNoPawnConnection;
}

TObjectPtr<UNetConnection> UNetDriver::FindConnection(FUniqueNetIdRepl& PlayerId)
{
	UNetConnection* FoundConnection = nullptr;

	// Find the connection with the same PlayerId.
	ForEachClientConnection([PlayerId, &FoundConnection](UNetConnection* ClientConnection)
	{
		if (ClientConnection->PlayerId == PlayerId)
		{
			FoundConnection = ClientConnection;
			return true;
		}

		return false;
	});

	return FoundConnection;
}

void UNetDriver::AssertValid()
{
}

/*static*/ bool UNetDriver::IsAdaptiveNetUpdateFrequencyEnabled()
{
	const bool bUseAdapativeNetFrequency = CVarUseAdaptiveNetUpdateFrequency.GetValueOnAnyThread() > 0;
	return bUseAdapativeNetFrequency;
}

void UNetDriver::AddNetworkActor(AActor* Actor)
{
	LLM_SCOPE_BYTAG(NetDriver);
	ensureMsgf(Actor == nullptr || !(Actor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject)), TEXT("%s is a CDO or Archetype and should not be replicated."), *GetFullNameSafe(Actor));

	UE_LOGF(LogNetLifecycle, Log, "[%ls] Registering for replication: Actor: %ls", *GetName(), *GetFullNameSafe(Actor));

	// We currently only want to do this on the server as we want to replicate the actor to the client
	// but we rely on the old system for actor instantiation
	if (ReplicationSystem)
	{
		return;
	}

	if (!IsDormInitialStartupActor(Actor))
	{
		GetNetworkObjectList().FindOrAdd(Actor, this);
		if (ReplicationDriver)
		{
			ReplicationDriver->AddNetworkActor(Actor);
		}
	}
}

void UNetDriver::SetRoleSwapOnReplicate(AActor* Actor, bool bSwapRoles)
{
	TSharedPtr<FNetworkObjectInfo> InfoPtr = GetNetworkObjectList().Find(Actor);
	if (InfoPtr.IsValid())
	{
		InfoPtr->bSwapRolesOnReplicate = bSwapRoles;
	}

	if (ReplicationDriver)
	{
		ReplicationDriver->SetRoleSwapOnReplicate(Actor, bSwapRoles);
	}
}

FNetworkObjectInfo* UNetDriver::FindOrAddNetworkObjectInfo(const AActor* InActor)
{
	ensureMsgf(InActor == nullptr || !(InActor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject)), TEXT("%s is a CDO or Archetype and should not be replicated."), *GetFullNameSafe(InActor));

	bool bWasAdded = false;

	if (ReplicationSystem)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FindNetworkObjectInfo(InActor);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (TSharedPtr<FNetworkObjectInfo>* InfoPtr = GetNetworkObjectList().FindOrAdd(const_cast<AActor*>(InActor), this, &bWasAdded))
	{
		if (bWasAdded && ReplicationDriver)
		{
			ReplicationDriver->AddNetworkActor(const_cast<AActor*>(InActor));
		}

		return InfoPtr->Get();
	}

	return nullptr;
}

FNetworkObjectInfo* UNetDriver::FindNetworkObjectInfo(const AActor* InActor)
{
	if (ReplicationSystem)
	{
		if (const UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
		{
			const UE::Net::FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(InActor);
			
			// Return default initialized NetworkObjectInfo so the presence of a pointer can at least indicate whether the actor is replicated or not.
			if (ActorRefHandle.IsValid())
			{
				*DummyNetworkObjectInfo = FNetworkObjectInfo();
				return DummyNetworkObjectInfo.Get();
			}
		}

		return nullptr;
	}

	return GetNetworkObjectList().Find(InActor).Get();
}

bool UNetDriver::IsNetworkActorUpdateFrequencyThrottled(const FNetworkObjectInfo& InNetworkActor) const
{
	bool bThrottled = false;
	if (IsAdaptiveNetUpdateFrequencyEnabled())
	{
		// Must have been replicated once for this to happen (and for OptimalNetUpdateDelta to have been set)
		const AActor* Actor = InNetworkActor.Actor;
		if (Actor && InNetworkActor.LastNetReplicateTime != 0)
		{
			const float ExpectedNetDelay = (1.0f / Actor->GetNetUpdateFrequency());
			if (InNetworkActor.OptimalNetUpdateDelta > ExpectedNetDelay)
			{
				bThrottled = true;
			}
		}
	}

	return bThrottled;
}

bool UNetDriver::IsNetworkActorUpdateFrequencyThrottled(const AActor* InActor) const
{
	bool bThrottled = false;
	if (InActor && IsAdaptiveNetUpdateFrequencyEnabled())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (const FNetworkObjectInfo* NetActor = FindNetworkObjectInfo(InActor))
		{
			bThrottled = IsNetworkActorUpdateFrequencyThrottled(*NetActor);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return bThrottled;
}

void UNetDriver::CancelAdaptiveReplication(const AActor* InActor)
{
	if (InActor && IsAdaptiveNetUpdateFrequencyEnabled())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FNetworkObjectInfo* NetActor = FindNetworkObjectInfo(InActor))
		{
			CancelAdaptiveReplication(*NetActor);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UNetDriver::CancelAdaptiveReplication(FNetworkObjectInfo& InNetworkActor)
{
	if (IsAdaptiveNetUpdateFrequencyEnabled())
	{
		if (AActor* Actor = InNetworkActor.Actor)
		{
			if (UWorld* ActorWorld = Actor->GetWorld())
			{
				const float ExpectedNetDelay = (1.0f / Actor->GetNetUpdateFrequency());
				const float NewUpdateTime = ActorWorld->GetTimeSeconds() + FMath::FRandRange(0.5f, 1.0f) * ExpectedNetDelay;

				// Only allow the next update to be sooner than the current one
				InNetworkActor.NextUpdateTime = FMath::Min(InNetworkActor.NextUpdateTime, (double)NewUpdateTime);
				InNetworkActor.OptimalNetUpdateDelta = ExpectedNetDelay;
				// TODO: we really need a way to cancel the throttling completely. OptimalNetUpdateDelta is going to be recalculated based on LastNetReplicateTime.
			}
		}
	}
}

bool UNetDriver::IsPendingNetUpdate(const AActor* InActor) const
{
	if (World)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (const FNetworkObjectInfo* NetActor = FindNetworkObjectInfo(InActor))
		{
			return NetActor->bPendingNetUpdate || (NetActor->NextUpdateTime < World->GetTimeSeconds());
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
		
	return false;
}

static TAutoConsoleVariable<int32> CVarOptimizedRemapping( TEXT( "net.OptimizedRemapping" ), 1, TEXT( "Uses optimized path to remap unmapped network guids" ) );
static TAutoConsoleVariable<int32> CVarMaxClientGuidRemaps( TEXT( "net.MaxClientGuidRemaps" ), 100, TEXT( "Max client resolves of unmapped network guids per tick" ) );

namespace UE
{
	namespace Net
	{
		int32 FilterGuidRemapping = 1;
		static FAutoConsoleVariableRef CVarFilterGuidRemapping(TEXT("net.FilterGuidRemapping"), FilterGuidRemapping, TEXT("Remove destroyed and parent guids from unmapped list"));
	};
};

bool CVar_NetDriver_ReportGameTickFlushTime = false;
static FAutoConsoleVariableRef CVarNetDriverReportTickFlushTime(TEXT("net.ReportGameTickFlushTime"), CVar_NetDriver_ReportGameTickFlushTime, TEXT("Record and report to the perf tracking system the processing time of the GameNetDriver's TickFlush."), ECVF_Default);

/** Accounts for the network time we spent in the game driver. */
double GTickFlushGameDriverTimeSeconds = 0.0;

bool ShouldEnableScopeSecondsTimers()
{
#if STATS
	return true;
#elif CSV_PROFILER_STATS
	return CVar_NetDriver_ReportGameTickFlushTime || FCsvProfiler::Get()->IsCapturing();
#else
	return CVar_NetDriver_ReportGameTickFlushTime;
#endif
}

void UNetDriver::TickFlush(float DeltaSeconds)
{
	LLM_SCOPE_BYTAG(NetDriver);

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NetworkOutgoing);
	SCOPE_CYCLE_COUNTER(STAT_NetTickFlush);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*NetDriverDefinition.ToString());

	bool bEnableTimer = (NetDriverName == NAME_GameNetDriver) && ShouldEnableScopeSecondsTimers();
	if (bEnableTimer)
	{
		GTickFlushGameDriverTimeSeconds = 0.0;
	}
	FSimpleScopeSecondsCounter ScopedTimer(GTickFlushGameDriverTimeSeconds, bEnableTimer);

	// Cached here to avoid calling ShouldThrottleCPUUsage during parallel steps
	CachedEngineMaxTickRate = GEngine->GetMaxTickRate(0.0f, false);

	if (IsServer() && (ClientConnections.Num() > 0 || UE::Net::Private::bUpdateReplicationSystemWithNoConnections) && !bSkipServerReplicateActors)
	{
		if (ReplicationSystem)
		{
			if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
			{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
				if (bReplicateTransactionally)
				{
					static const FName FlushDeferredName("UEngineReplicationBridge::FlushDeferred");
					UE::RemoteExecutor::ExecuteTransactional(FlushDeferredName, [Bridge]() {
						UE::Net::FDeferredReplicationSystemCalls::FlushDeferred(Bridge);
					});
				}
				else
#endif
				{
					UE::Net::FDeferredReplicationSystemCalls::FlushDeferred(Bridge);
				}
			}
		}

		// Update all clients.
#if WITH_SERVER_CODE
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ServerReplicateActors);

		if (ReplicationSystem)
		{
			InternalIrisUpdateTransactional(DeltaSeconds);
		}
		else if (ClientConnections.Num() > 0)
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
			if (bReplicateTransactionally && !UE::Net::Private::bServerReplicateActorsGranularTransactions)
			{
				static FName TransactionalWorkName("ServerReplicateActors");
				UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, DeltaSeconds]()
				{
					ServerReplicateActors(DeltaSeconds);
				});
			}
			else
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
			{
				ServerReplicateActors(DeltaSeconds);
			}
		}
#endif // WITH_SERVER_CODE
	}
	else if (!IsServer() && ServerConnection != nullptr)
	{
		if (ReplicationSystem)
		{
			UpdateIrisReplicationViews();
			ReplicationSystem->NetUpdate(DeltaSeconds);
		}
	}

	// Reset queued bunch amortization timer
	ProcessQueuedBunchesCurrentFrameMilliseconds = 0.0f;

	// Poll all sockets.
	if (ServerConnection)
	{
		// Queue client voice packets in the server's voice channel
		ProcessLocalClientPackets();
		ServerConnection->Tick(DeltaSeconds);
	}
	else
	{
		// Queue up any voice packets the server has locally
		ProcessLocalServerPackets();
	}

	// Check if this NetDriver is allowed to execute UNetConnection::Tick via multithread tasks
	bool bCanRunClientTickInParallel = 
		IsServer() && bIsUsingParallelNetTick && ReplicationSystem && ReplicationSystemConfigServer.bAllowParallelTasks && UE::Net::Private::bAllowParallelNetTickAtRuntime;

	// Do we have enough connections to run in multithread
	bCanRunClientTickInParallel = bCanRunClientTickInParallel && (ClientConnections.Num() >= ReplicationSystemConfigServer.MinConnectionsForParallelTick);

	if (bCanRunClientTickInParallel)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDriver_TickClientConnections)

		// Set up a Task for each ClientConnection to run its Tick in parallel
		TArray<UE::Tasks::TTask<void>> TasksToComplete;
		TArray<UE::Net::Private::FConnectionTickTaskData> TaskData;
		TaskData.Init(UE::Net::Private::FConnectionTickTaskData(DeltaSeconds), ClientConnections.Num());
		int32 CurrentConnectionIndex = 0;

		ReplicationSystem->StartParallelPhase();

		for (UNetConnection* Connection : ClientConnections)
		{
			IRIS_PROFILER_SCOPE(UConnectionTick_PrepareAndDispatchTask);
			
			UE::Net::Private::FConnectionTickTaskData& Data = TaskData[CurrentConnectionIndex];

			TasksToComplete.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Connection, &Data]()
				{
					UE::Net::Private::FConnectionTickTask NewTask = UE::Net::Private::FConnectionTickTask(Connection, Data);
					NewTask.DoTask();
				}
			));

			CurrentConnectionIndex++;
		}
	
		// Block until all tick tasks are done
		UE::Tasks::Wait(TasksToComplete);

		ReplicationSystem->StopParallelPhase();
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDriver_TickClientConnections)

		for (UNetConnection* Connection : ClientConnections)
		{
			Connection->Tick(DeltaSeconds);
		}
	}

	if (ConnectionlessHandler.IsValid())
	{
		ConnectionlessHandler->Tick(DeltaSeconds);

		FlushHandler();
	}
	
#if UE_NET_TRACE_ENABLED
	FNetTrace::FinishConnectionTick(GetNetTraceId());
#endif // UE_NET_TRACE_ENABLED

	if (CVarNetDebugDraw.GetValueOnAnyThread() > 0)
	{
		DrawNetDriverDebug();
	}

	if (!IsUsingIrisReplication())
	{
		if (ShouldReplicateTransactionally())
		{
			static const FName NAME_UpdateUnmappedObjects { "UpdateUnmappedObjects" };
			UE::RemoteExecutor::ExecuteTransactional(NAME_UpdateUnmappedObjects, [this]()
			{
				UpdateUnmappedObjects();
			});
		}
		else
		{
			UpdateUnmappedObjects();
		}

		CleanupStaleDormantReplicators();
	}

	UpdateNetworkStats();

	ResetNetConnectionStats();

	// Send the current values of metrics to all of the metrics listeners.
	GetMetrics()->ProcessListeners();

	// Update the lag state
	UpdateNetworkLagState();
}

void UNetDriver::InternalIrisUpdateTransactional(float DeltaSeconds)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	if (bReplicateTransactionally)
	{
		const FName TransactionalWorkName("UNetDriver::InternalIrisUpdate");
		UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, DeltaSeconds]()
			{
				InternalIrisUpdate(DeltaSeconds);
			});
	}
	else
#endif
	{
		InternalIrisUpdate(DeltaSeconds);
	}
}

void UNetDriver::InternalIrisUpdate(float DeltaSeconds)
{
	UpdateIrisReplicationViews();
	SendClientMoveAdjustments();
	ReplicationSystem->NetUpdate(DeltaSeconds);
}

void UNetDriver::UpdateUnmappedObjects()
{
	CSV_SCOPED_TIMING_STAT(Networking, UpdateUnmappedObjects);
	SCOPE_CYCLE_COUNTER(STAT_NetUpdateUnmappedObjectsTime);

	if (CVarOptimizedRemapping.GetValueOnAnyThread() && GetNetGuidCache().IsValid())
	{
		// Go over recently imported network guids, and see if there are any replicators that need to map them
		TSet<FNetworkGUID>& ImportedNetGuidsRef = GetNetGuidCache()->ImportedNetGuids;
		TMap<FNetworkGUID, TSet<FNetworkGUID>>& PendingOuterNetGuidsRef = GetNetGuidCache()->PendingOuterNetGuids;

		GetMetrics()->SetInt(UE::Net::Metric::ImportedNetGuids, ImportedNetGuidsRef.Num());
		GetMetrics()->SetInt(UE::Net::Metric::PendingOuterNetGuids, PendingOuterNetGuidsRef.Num());
		GetMetrics()->SetInt(UE::Net::Metric::UnmappedReplicators, UnmappedReplicators.Num());

		TSet<FObjectReplicator*> ForceUpdateReplicators;

		for (FObjectReplicator* Replicator : UnmappedReplicators)
		{
			if (Replicator->bForceUpdateUnmapped)
			{
				Replicator->bForceUpdateUnmapped = false;
				ForceUpdateReplicators.Add(Replicator);
			}
		}

		if (ImportedNetGuidsRef.Num() || ForceUpdateReplicators.Num())
		{
			int32 NumRemapTests = 0;
			const int32 MaxRemaps = IsServer() ? 0 : CVarMaxClientGuidRemaps.GetValueOnAnyThread();

			TArray<FNetworkGUID> UnmappedGuids;
			TArray<FNetworkGUID> NewlyMappedGuids;

			auto ImportedNetGuidsIter = ImportedNetGuidsRef.CreateIterator();
			for (; ImportedNetGuidsIter; ++ImportedNetGuidsIter)
			{
				const FNetworkGUID& NetworkGuid = *ImportedNetGuidsIter;
				bool bMappedOrBroken = false;

				if (GetNetGuidCache()->GetObjectFromNetGUID(NetworkGuid, false) != nullptr)
				{
					if (UE::Net::Private::bRemapStableSubobjects)
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_NetRemapStableSubobjects);

						// Import any unmapped, stably-named guids that are inners of the GUID that just mapped.
						// These are tracked separately from the normal ImportedNetGuids since they are often
						// default subobjects created in constructors and there's no other hook to import them.
						const TArray<FNetworkGUID>* Inners = GetNetGuidCache()->FindUnmappedStablyNamedGuidsWithOuter(NetworkGuid);

						if (Inners)
						{
							TSet<FNetworkGUID>& PendingGuidsRef = PendingOuterNetGuidsRef.FindOrAdd(NetworkGuid);
							PendingGuidsRef.Append(*Inners);

							// Now that they're on the pending import list, they will stay there until mapped. Can remove from the outer-to-inner map.
							GetNetGuidCache()->RemoveUnmappedStablyNamedGuidsWithOuter(NetworkGuid);
						}
					}

					NewlyMappedGuids.Add(NetworkGuid);
					bMappedOrBroken = true;
				}

				if (GetNetGuidCache()->IsGUIDBroken(NetworkGuid, false))
				{
					bMappedOrBroken = true;
				}

				if (!bMappedOrBroken)
				{
					const FNetworkGUID OuterGUID = GetNetGuidCache()->GetOuterNetGUID(NetworkGuid);

					// we're missing the outer, stop checking until we map it
					if ((UE::Net::FilterGuidRemapping != 0) && OuterGUID.IsValid() && !OuterGUID.IsDefault() && !GetNetGuidCache()->IsGUIDLoaded(OuterGUID) && !GetNetGuidCache()->IsGUIDPending(OuterGUID))
					{
						UE_LOGF(LogNetPackageMap, Log, "Missing outer (%ls) for unmapped guid (%ls), marking pending", *GetNetGuidCache()->Describe(OuterGUID), *GetNetGuidCache()->Describe(NetworkGuid));

						TSet<FNetworkGUID>& PendingGuidsRef = PendingOuterNetGuidsRef.FindOrAdd(OuterGUID);
						PendingGuidsRef.Add(NetworkGuid);
					}
					else
					{
						if (ensure(NetworkGuid.IsValid()))
						{
							UnmappedGuids.Add(NetworkGuid);
						}
					}
				}

				++NumRemapTests;

				if ((MaxRemaps > 0) && (NumRemapTests >= MaxRemaps))
				{
					break;
				}
			}

			// If there are any that were left unprocessed, then re-create the set with them at the beginning
			if (ImportedNetGuidsIter)
			{
				TSet<FNetworkGUID> DeferredGuids;
				DeferredGuids.Reserve(ImportedNetGuidsRef.Num());

				for (; ImportedNetGuidsIter; ++ImportedNetGuidsIter)
				{
					DeferredGuids.Add(*ImportedNetGuidsIter);
				}

				Swap(ImportedNetGuidsRef, DeferredGuids);
			}
			else
			{
				ImportedNetGuidsRef.Empty();
			}

			// attempt to resolve dependent guids next tick (outer is now mapped)
			for (const FNetworkGUID& NetGuid : NewlyMappedGuids)
			{
				if (TSet<FNetworkGUID>* DependentGuids = PendingOuterNetGuidsRef.Find(NetGuid))
				{
					UE_LOGF(LogNetPackageMap, Log, "Newly mapped outer (%ls) removing from pending", *GetNetGuidCache()->FullNetGUIDPath(NetGuid));

					ImportedNetGuidsRef.Append(*DependentGuids);
					PendingOuterNetGuidsRef.Remove(NetGuid);
				}
			}

			// any tested guids that could not yet be mapped are added back to the list
			if (UnmappedGuids.Num())
			{
				ImportedNetGuidsRef.Append(UnmappedGuids);
			}

			if (NewlyMappedGuids.Num() || ForceUpdateReplicators.Num())
			{
				TSet<FObjectReplicator*> ReplicatorsToUpdate = MoveTemp(ForceUpdateReplicators);

				for (const FNetworkGUID& NetGuid : NewlyMappedGuids)
				{
					if (TSet<FObjectReplicator*>* Replicators = GuidToReplicatorMap.Find(NetGuid))
					{
						ReplicatorsToUpdate.Append(*Replicators);
					}
				}

				for (FObjectReplicator* Replicator : ReplicatorsToUpdate)
				{
					if (UnmappedReplicators.Contains(Replicator))
					{
						bool bHasMoreUnmapped = false;
						Replicator->UpdateUnmappedObjects(bHasMoreUnmapped);

						if (!bHasMoreUnmapped)
						{
							UnmappedReplicators.Remove(Replicator);
						}
					}
				}
			}
		}
	}
	else
	{
		// Update properties that are unmapped, try to hook up the object pointers if they exist now
		for (auto It = UnmappedReplicators.CreateIterator(); It; ++It)
		{
			FObjectReplicator* Replicator = *It;

			bool bHasMoreUnmapped = false;

			Replicator->UpdateUnmappedObjects(bHasMoreUnmapped);

			if (!bHasMoreUnmapped)
			{
				// If there are no more unmapped objects, we can also stop checking
				It.RemoveCurrent();
			}
		}
	}
}

void UNetDriver::CleanupStaleDormantReplicators()
{
	// Go over RepChangedPropertyTrackerMap periodically, and remove entries that no longer have valid objects
	// Unfortunately if you mark an object as pending kill, it will no longer find itself in this map,
	// so we do this as a fail safe to make sure we never leak memory from this map

	const double CleanupTimeSeconds = 10.0;
	const double CurrentRealtimeSeconds = FPlatformTime::Seconds();

	if (CurrentRealtimeSeconds - LastCleanupTime > CleanupTimeSeconds)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NetUpdateCleanupTime);

		for (auto It = ReplicationChangeListMap.CreateIterator(); It; ++It)
		{
			if (!It.Key().GetWeakObjectPtr().IsValid())
			{
				It.RemoveCurrent();
			}
		}

		for (UNetConnection* const ClientConnection : ClientConnections)
		{
			if (ClientConnection)
			{
				ClientConnection->CleanupStaleDormantReplicators();
			}
		}

		if (ServerConnection)
		{
			ServerConnection->CleanupStaleDormantReplicators();
		}

		LastCleanupTime = CurrentRealtimeSeconds;
	}

	if (QueuedBunchFailsafeNumChannels > 0)
	{
		UE_LOGF(LogNet, Log, "UNetDriver::TickFlush: %u channel(s) exceeded net.QueuedBunchTimeFailsafeSeconds and flushed their entire queue(s) this frame.", QueuedBunchFailsafeNumChannels);
		QueuedBunchFailsafeNumChannels = 0;
	}
}

void UNetDriver::UpdateNetworkLagState()
{
	ENetworkLagState::Type OldLagState = LagState;

	// Percentage of the timeout time that a connection is considered "lagging"
	const float TimeoutPercentThreshold = 0.75f;

	if ( IsServer() )
	{
		// Server network lag detection

		// See if all clients connected to us are lagging. If so there might be network connection problems.
		// Only trigger this if there are a few connections since a single client could have just crashed or disconnected suddenly,
		// and is less likely to happen with multiple clients simultaneously.
		int32 NumValidConnections = 0;
		int32 NumLaggingConnections = 0;
		for (UNetConnection* Connection : ClientConnections)
		{
			if (Connection)
			{
				NumValidConnections++;

				const double HalfTimeout = Connection->GetTimeoutValue() * TimeoutPercentThreshold;
				const double DeltaTimeSinceLastMessage = ElapsedTime - Connection->LastReceiveTime;
				if (DeltaTimeSinceLastMessage > HalfTimeout)
				{
					NumLaggingConnections++;
				}
			}
		}

		if (NumValidConnections >= 2 && NumValidConnections == NumLaggingConnections)
		{
			// All connections that we could measure are lagging and there are enough to know it is not likely the fault of the clients.
			LagState = ENetworkLagState::Lagging;
		}
		else
		{
			// We have at least one non-lagging client or we don't have enough clients to know if the server is lagging.
			LagState = ENetworkLagState::NotLagging;
		}
	}
	else
	{
		// Client network lag detection.
		
		// Just check the server connection.
		if (ensure(ServerConnection))
		{
			const double HalfTimeout = ServerConnection->GetTimeoutValue() * TimeoutPercentThreshold;
			const double DeltaTimeSinceLastMessage = ElapsedTime - ServerConnection->LastReceiveTime;
			if (DeltaTimeSinceLastMessage > HalfTimeout)
			{
				// We have exceeded half our timeout. We are lagging.
				LagState = ENetworkLagState::Lagging;
			}
			else
			{
				// Not lagging yet. We have received a message recently.
				LagState = ENetworkLagState::NotLagging;
			}
		}
	}

	if (OldLagState != LagState)
	{
		GEngine->BroadcastNetworkLagStateChanged(GetWorld(), this, LagState);
	}
}

/**
 * Determines which other connections should receive the voice packet and
 * queues the packet for those connections. Used for sending both local/remote voice packets.
 *
 * @param VoicePacket the packet to be queued
 * @param CameFromConn the connection this packet came from (NULL if local)
 */
void UNetDriver::ReplicateVoicePacket(TSharedPtr<FVoicePacket> VoicePacket, UNetConnection* CameFromConn)
{
	// Iterate the connections and see if they want the packet
	for (int32 Index = 0; Index < ClientConnections.Num(); Index++)
	{
		UNetConnection* Conn = ClientConnections[Index];
		// Skip the originating connection
		if (CameFromConn != Conn)
		{
			// If server then determine if it should replicate the voice packet from another sender to this connection
			const bool bReplicateAsServer = !bIsPeer && Conn->ShouldReplicateVoicePacketFrom(*VoicePacket->GetSender());
			// If client peer then determine if it should send the voice packet to another client peer
			//const bool bReplicateAsPeer = (bIsPeer && AllowPeerVoice) && Conn->ShouldReplicateVoicePacketToPeer(Conn->PlayerId);

			if (bReplicateAsServer)// || bReplicateAsPeer)
			{
				UVoiceChannel* VoiceChannel = Conn->GetVoiceChannel();
				if (VoiceChannel != NULL)
				{
					// Add the voice packet for network sending
					VoiceChannel->AddVoicePacket(VoicePacket);
				}
			}
		}
	}
}

/**
 * Process any local talker packets that need to be sent to clients
 */
void UNetDriver::ProcessLocalServerPackets()
{
	if (World)
	{
		int32 NumLocalTalkers = UOnlineEngineInterface::Get()->GetNumLocalTalkers(World);
		// Process all of the local packets
		for (int32 Index = 0; Index < NumLocalTalkers; Index++)
		{
			// Returns a ref counted copy of the local voice data or NULL if nothing to send
			TSharedPtr<FVoicePacket> LocalPacket = UOnlineEngineInterface::Get()->GetLocalPacket(World, Index);
			// Check for something to send for this local talker
			if (LocalPacket.IsValid())
			{
				// See if anyone wants this packet
				ReplicateVoicePacket(LocalPacket, NULL);

				// once all local voice packets are processed then call ClearVoicePackets()
			}
		}
	}
}

/**
 * Process any local talker packets that need to be sent to the server
 */
void UNetDriver::ProcessLocalClientPackets()
{
	if (World)
	{
		int32 NumLocalTalkers = UOnlineEngineInterface::Get()->GetNumLocalTalkers(World);
		if (NumLocalTalkers)
		{
			UVoiceChannel* VoiceChannel = ServerConnection->GetVoiceChannel();
			if (VoiceChannel)
			{
				// Process all of the local packets
				for (int32 Index = 0; Index < NumLocalTalkers; Index++)
				{
					// Returns a ref counted copy of the local voice data or NULL if nothing to send
					TSharedPtr<FVoicePacket> LocalPacket = UOnlineEngineInterface::Get()->GetLocalPacket(World, Index);
					// Check for something to send for this local talker
					if (LocalPacket.IsValid())
					{
						// If there is a voice channel to the server, submit the packets
						//if (ShouldSendVoicePacketsToServer())
						{
							// Add the voice packet for network sending
							VoiceChannel->AddVoicePacket(LocalPacket);
						}

						// once all local voice packets are processed then call ClearLocalVoicePackets()
					}
				}
			}
		}
	}
}

void UNetDriver::PostTickFlush()
{
	// We are still ticking here
	{
		TGuardValue<bool> GuardInNetTick(bInTick, true);
		if (ReplicationSystem && (ServerConnection != nullptr || (ClientConnections.Num() > 0) || UE::Net::Private::bUpdateReplicationSystemWithNoConnections))
		{
			ReplicationSystem->PostSendUpdate();
		}

		if (World && !bSkipClearVoicePackets)
		{
			UOnlineEngineInterface::Get()->ClearVoicePackets(World);
		}
	}

	// All ticks are finished, look if a destruction request was queued
	if (bPendingDestruction)
	{
		check(IsInTick() == false);
		if (World)
		{
			GEngine->DestroyNamedNetDriver(World, NetDriverName);
		}
		else
		{
			UE_LOGF(LogNet, Error, "NetDriver %ls pending destruction without valid world.", *NetDriverName.ToString());
		}
		bPendingDestruction = false;
	}
}

bool UNetDriver::InitConnectionClass(void)
{
	if (NetConnectionClass == NULL && NetConnectionClassName != TEXT(""))
	{
		NetConnectionClass = LoadClass<UNetConnection>(NULL,*NetConnectionClassName,NULL,LOAD_None,NULL);
		if (NetConnectionClass == NULL)
		{
			UE_LOGF(LogNet, Error,"Failed to load class '%ls'",*NetConnectionClassName);
		}
	}

	ChildNetConnectionClass = UChildConnection::StaticClass();

	return NetConnectionClass != NULL;
}

bool UNetDriver::InitReplicationDriverClass()
{
	if (ReplicationDriverClass == nullptr && !ReplicationDriverClassName.IsEmpty())
	{
		ReplicationDriverClass = LoadClass<UReplicationDriver>(nullptr, *ReplicationDriverClassName, nullptr, LOAD_None, nullptr);
		if (ReplicationDriverClass == nullptr)
		{
			UE_LOGF(LogNet, Error,"Failed to load class '%ls'", *ReplicationDriverClassName);
		}
	}

	return ReplicationDriverClass != nullptr;
}

bool UNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	// Read any timeout overrides from the URL
	if (const TCHAR* InitialConnectTimeoutOverride = URL.GetOption(TEXT("InitialConnectTimeout="), nullptr))
	{
		float ParsedValue;
		LexFromString(ParsedValue, InitialConnectTimeoutOverride);
		if (ParsedValue != 0.0f)
		{
			InitialConnectTimeout = ParsedValue;
		}
	}
	if (const TCHAR* ConnectionTimeoutOverride = URL.GetOption(TEXT("ConnectionTimeout="), nullptr))
	{
		float ParsedValue;
		LexFromString(ParsedValue, ConnectionTimeoutOverride);
		if (ParsedValue != 0.0f)
		{
			ConnectionTimeout = ParsedValue;
		}
	}
	if (URL.HasOption(TEXT("NoTimeouts")))
	{
		bNoTimeouts = true;
	}

	LastTickDispatchRealtime = FPlatformTime::Seconds();
	bool bSuccess = InitConnectionClass();

	if (!bInitAsClient)
	{
		ConnectionlessHandler.Reset();
		
		if (!IsUsingIrisReplication())
		{
			InitReplicationDriverClass();
			SetReplicationDriver(UReplicationDriver::CreateReplicationDriver(this, URL, GetWorld()));
		}

		DDoS.Init(FMath::Clamp(GetNetServerMaxTickRate(), 1, 1000));

		DDoS.NotifySeverityEscalation.BindLambda(
			[this](FString SeverityCategory)
		{
			GEngine->BroadcastNetworkDDosSEscalation(this->GetWorld(), this, SeverityCategory);
		});
	}

#if DO_ENABLE_NET_TEST
	bool bSettingFound(false);
	FPacketSimulationSettings PacketSettings;

	for (const FString& URLOption : URL.Op)
	{
		bSettingFound |= PacketSettings.ParseSettings(*URLOption);
	}

	if( bSettingFound )
	{
		SetPacketSimulationSettings(PacketSettings);
	}
#endif //#if DO_ENABLE_NET_TEST

	Notify = InNotify;

	// At the time being using NetTokens and NetTokenStore is only available when compiling with iris.
	// Init NetToken stores
	if (!GetNetTokenStore())
	{
		using namespace UE::Net;

		NetTokenStore = MakeUnique<FNetTokenStore>();

		FNetTokenStore::FInitParams NetTokenStoreInitParams;
		NetTokenStoreInitParams.Authority = !bInitAsClient ? FNetToken::ENetTokenAuthority::Authority : FNetToken::ENetTokenAuthority::None;
		NetTokenStoreInitParams.MaxConnections = ConnectionIdHandler.GetMaxConnectionIdCount();
		NetTokenStore->Init(NetTokenStoreInitParams);

		// TODO: make this configurable from config so that users can provide their own custom NetTokenStores
		// Order is important, changing order will require new netversion
		// When we move it to the config TypeId will be made explicit
		NetTokenStore->CreateAndRegisterDataStore<FStringTokenStore>();
		NetTokenStore->CreateAndRegisterDataStore<FNameTokenStore>();
		NetTokenStore->CreateAndRegisterDataStore<FGameplayTagTokenStore>();
		OnNetTokenStoreReadyDelegate.Broadcast(this);
	}
	
	if (IsUsingIrisReplication() && !ReplicationSystem)
	{
		CreateReplicationSystem(bInitAsClient);
	}

	
	if (NetDriverDefinition == NAME_GameNetDriver)
	{
		UpdateCrashContext(ECrashContextUpdate::UpdateRepModel);

#if WITH_PUSH_MODEL
		// Disable PushModel globally when the GameNetDriver is running with Iris.
		// Temp until we fully integrate Iris PushModel support.
		const bool bAllowPushModelHandles = !IsUsingIrisReplication();
		UEPushModelPrivate::SetHandleCreationAllowed(bAllowPushModelHandles);
#endif
	}

	UE_LOGF(LogNet, Log, "InitBase %ls (NetDriverDefinition %ls) using replication model %ls", *NetDriverName.ToString(), *NetDriverDefinition.ToString(), *GetReplicationModelName());
	
	InitNetTraceId();

	// NotifyGameInstanceUpdate might have been called prior to setting up the NetTraceId so we call it again
	NotifyGameInstanceUpdated();

	if (!bInitAsClient)
	{
		InitDestroyedStartupActors();
	}

	CachedGlobalNetTravelCount = GEngine->GetGlobalNetTravelCount();

	// Add all of the metrics used by the networking system and register metrics listeners.
	SetupNetworkMetrics();

	if (ShouldRegisterMetricsDatabaseListeners())
	{
		SetupNetworkMetricsListeners(bInitAsClient);
	}

	return bSuccess;
}

void UNetDriver::SetupNetworkMetrics()
{
	NetworkMetricsDatabase = NewObject<UNetworkMetricsDatabase>();

	// The total number of connections added since the driver was started.
	GetMetrics()->CreateInt(UE::Net::Metric::AddedConnections, 0);

	// The total number of connections that were closed because of a reliable buffer overflow since the driver was started.
	GetMetrics()->CreateInt(UE::Net::Metric::ClosedConnectionsDueToReliableBufferOverflow, 0);

	// The number of incoming packets per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InPackets, 0);

	// The number of outgoing packets per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutPackets, 0);

	// The number of incoming packets per frame across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsPerFrame, 0);

	// The number of outgoing packets per frame across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsPerFrame, 0);

	// The number of incoming packets lost per frame across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InLostPacketsFoundPerFrame, 0);

	// The number of outgoing packets lost per frame across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutLostPacketsFoundPerFrame, 0);

	// The average/min/max incoming packets per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsClientPerSecondAvg, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsClientPerSecondMax, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsClientPerSecondMin, 0);
	
	// The average/min/max outgoing packets per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsClientPerSecondAvg, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsClientPerSecondMax, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsClientPerSecondMin, 0);
	
	// (Server only) The average/max incoming packets per stat period across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsClientAvg, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsClientMax, 0);

	// (Server only) The average/max outgoing packets per stat period across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsClientAvg, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsClientMax, 0);

	// The total number of incoming bytes per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InRate, 0);

	// The total number of outgoing bytes per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutRate, 0);

	// (Server only) The average/min/max incoming bytes across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InRateClientAvg, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::InRateClientMax, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::InRateClientMin, 0);

	// (Server only) The average/min/max outgoing bytes per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutRateClientAvg, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::OutRateClientMax, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::OutRateClientMin, 0);

	// The percentage of incoming/outgoing packets per second that have been lost.
	GetMetrics()->CreateInt(UE::Net::Metric::InPacketsLost, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::OutPacketsLost, 0);

	// (Client only) Current jitter for this connection in milliseconds
	GetMetrics()->CreateFloat(UE::Net::Metric::AverageJitterInMS, 0);

	// The number of incoming bunches per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::InBunches, 0);

	// The number of outgoing bunches per second across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::OutBunches, 0);

	// (Client only) The current ping in milliseconds to the server.
	GetMetrics()->CreateInt(UE::Net::Metric::Ping, 0);

	// The average/min/max ping across all connections.
	GetMetrics()->CreateFloat(UE::Net::Metric::AvgPing, 0.0f);
	GetMetrics()->CreateFloat(UE::Net::Metric::MinPing, 0.0f);
	GetMetrics()->CreateFloat(UE::Net::Metric::MaxPing, 0.0f);

	// (Non-Iris only) The number of kilobytes sent in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::OutKBytes, 0);

	// (Non-Iris only) The number of outgoing kilobytes that made up NetGUID bunches in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::OutNetGUIDKBytesSec, 0);
	
	// (Non-Iris only) The time, in milliseconds, spent by the server performing replication in the last frame. This value represents the sum of ReplicateActorTimeMS 
	// and GatherPrioritizeTimeMS (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::ServerReplicateActorTimeMS, 0.0f);
	// (Non-Iris only) The time, in milliseconds, spent preparing to replicate actors during server replication in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::GatherPrioritizeTimeMS, 0.0f);
	// (Non-Iris only) The time, in milliseconds, spent replicating actors during server replication in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::ReplicateActorTimeMS, 0.0f);

	// (Non-Iris only) (Server only) The number of actors replicated in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::NumReplicatedActors, 0);

	// The number of active connections in the last frame.
	GetMetrics()->CreateInt(UE::Net::Metric::Connections, 0);

	// (Non-Iris only) The average number of replicated actors per connection in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::NumReplicateActorCallsPerConAvg, 0.0f);

	// (Non-Iris only) The number of networked actors in the world during the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::NumberOfActiveActors, 0);

	// (Non-Iris only) (Server only) The number of dormant actors in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::NumberOfFullyDormantActors, 0);

	// (Non-Iris only) (Server only) The number of network objects that skipped replication across all connections in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::NumSkippedObjectEmptyUpdates, 0);

	// (Non-Iris only) (Server only) The number of open channels across all connections in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::NumOpenChannels, 0);

	// (Non-Iris only) (Server only) The number of open channels that have dormant actors or queued bunches across all connections in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::NumTickingChannels, 0);

	// (Non-Iris only) (Server only) The number of connections that are skipped due to bandwidth saturation in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::SatConnections, 0);

	// (Non-Iris only) (Server only) The number of property updates that used shared serialization in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::SharedSerializationPropertyHit, 0);

	// (Non-Iris only) (Server only) The number of property updates that couldn't used shared serialization in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::SharedSerializationPropertyMiss, 0);

	// (Non-Iris only) (Server only) The number of RPCs that used shared serialization in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::SharedSerializationRPCHit, 0);

	// (Non-Iris only) (Server only) The number of RPCs that couldn't use shared serialization in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateInt(UE::Net::Metric::SharedSerializationRPCMiss, 0);

	// (Non-Iris only) The number of received level visibility requests (for level changes) in the last frame (only if GReplicateActorTimingEnabled is true).
	GetMetrics()->CreateFloat(UE::Net::Metric::NumClientUpdateLevelVisibility, 0);

	// The number of open channels.
	GetMetrics()->CreateInt(UE::Net::Metric::Channels, 0);

	// The number of actor channels.
	GetMetrics()->CreateInt(UE::Net::Metric::NumActorChannels, 0);

	// The number of dormant actors.
	GetMetrics()->CreateInt(UE::Net::Metric::NumDormantActors, 0);

	// The number of actors in the world.
	GetMetrics()->CreateInt(UE::Net::Metric::NumActors, 0);

	// The number of networked actors in the world.
	GetMetrics()->CreateInt(UE::Net::Metric::NumNetActors, 0);

	// The number of NetGUID that have been acked, are unacked or are pending ack.
	GetMetrics()->CreateInt(UE::Net::Metric::NumNetGUIDsAckd, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::NumNetGUIDsPending, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::NumNetGUIDsUnAckd, 0);

	// 1 if one or more connections are unable to send packets due to bandwidth saturation.
	// 0 if all connections are ready to send packets.
	GetMetrics()->CreateInt(UE::Net::Metric::NetSaturated, 0);

	// The additional overhead, in bytes, associated with each packet (e.g. a UDP header).
	GetMetrics()->CreateInt(UE::Net::Metric::MaxPacketOverhead, 0);

	// The number of incoming/outgoing bytes per second used for NetGUID bunches.
	GetMetrics()->CreateInt(UE::Net::Metric::NetGUIDInRate, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::NetGUIDOutRate, 0);

	// (Server only) The number of active client connections (can be smaller than NumConnections).
	GetMetrics()->CreateInt(UE::Net::Metric::NetNumClients, 0);

	// (Server only) The number of active client connections (can be smaller than NumConnections).
	GetMetrics()->CreateInt(UE::Net::Metric::NumClients, 0);

	// The number of parent connections.
	GetMetrics()->CreateInt(UE::Net::Metric::NumConnections, 0);

	// The number of connections including any child connections.
	GetMetrics()->CreateInt(UE::Net::Metric::NumAllConnections, 0);

	// (Server only) The number of actors considered for replication across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::NumConsideredActors, 0);

	// (Server only) The number of actors that are configured to be initially dormant (DORM_Initial).
	GetMetrics()->CreateInt(UE::Net::Metric::NumInitiallyDormantActors, 0);

	// (Server only) The number of prioritized considered actors across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::PrioritizedActors, 0);

	// (Server only) The number of prioritized actors that are deleted actors across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::NumRelevantDeletedActors, 0);

	// (Non-Iris only) (Server only) The number of bytes sent when replicating actors across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::NumReplicatedActorBytes, 0);

	// The number of game objects that aren't currently mapped to a replicator across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::UnmappedReplicators, 0);

	// The percentage of incoming/outgoing bytes per second that are used for voice communications across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::PercentInVoice, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PercentOutVoice, 0);

	// The number of incoming/outgoing bytes/packets per second used by voice communications across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::VoiceBytesRecv, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::VoiceBytesSent, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::VoicePacketsRecv, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::VoicePacketsSent, 0);

	// (Server only) A histogram of ping across all connections.
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt0, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt1, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt2, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt3, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt4, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt5, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt6, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PingBucketInt7, 0);

	// The maximum size of the incoming/outgoing reliable message queues across all channels and connections in the last frame.
	GetMetrics()->CreateInt(UE::Net::Metric::OutgoingReliableMessageQueueMaxSize, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::IncomingReliableMessageQueueMaxSize, 0);

	GetMetrics()->CreateInt(UE::Net::Metric::ImportedNetGuids, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::PendingOuterNetGuids, 0);
	GetMetrics()->CreateInt(UE::Net::Metric::NetInBunchTimeOvershootPercent, 0);

	// (Client only) The highest maximum ping processed every frame. If no pings are processed in the frame it keeps the last value set.
	GetMetrics()->CreateInt(UE::Net::Metric::RawPing, 0);

	// (Server only) The minimum RawPing from among all client connections each frame.
	GetMetrics()->CreateInt(UE::Net::Metric::RawPingClientMin, 0);

	// (Server only) The maximum RawPing from among all client connections each frame.
	GetMetrics()->CreateInt(UE::Net::Metric::RawPingClientMax, 0);

	// (Server only) The average RawPing from among all client connections each frame.
	GetMetrics()->CreateFloat(UE::Net::Metric::RawPingClientAvg, 0.0f);

	// The number of instances of APlayerController registered for replication, excluding ANoPawnPlayerController.
	GetMetrics()->CreateInt(UE::Net::Metric::NumPlayerControllersForReplication, 0);

	// The number of instances of ANoPawnPlayerController registered for replication.
	GetMetrics()->CreateInt(UE::Net::Metric::NumNoPawnPlayerControllersForReplication, 0);

	// The number of instances of APawn registered for replication.
	GetMetrics()->CreateInt(UE::Net::Metric::NumPawnsForReplication, 0);

	// (Generic only) The number of FNetViewers used during replication.
	GetMetrics()->CreateInt(UE::Net::Metric::NumReplicationViewers, 0);
}

void UNetDriver::SetupNetworkMetricsListeners(bool bIsClient)
{
	// Register metrics in the ini configuration file with a listener.
	const UNetworkMetricsConfig* Config = GetDefault<UNetworkMetricsConfig>();

	const bool bIrisEnabled = IsUsingIrisReplication();

	for (const FNetworkMetricConfig& MetricConfig : Config->Listeners)
	{
		// Allow config file to selectively enable/disable listeners
		bool bEnabled = true;

		switch(MetricConfig.EnableMode)
		{
			case ENetworkMetricEnableMode::EnableForAllReplication:
			{
				bEnabled = true;
			} break;

			case ENetworkMetricEnableMode::EnableForIrisOnly:
			{
				bEnabled = bIrisEnabled;
			} break;

			case ENetworkMetricEnableMode::EnableForNonIrisOnly:
			{
				bEnabled = !bIsUsingIris;
			} break;

			case ENetworkMetricEnableMode::ServerOnly:
			{
				bEnabled = !bIsClient;
			} break;

			case ENetworkMetricEnableMode::ClientOnly:
			{
				bEnabled = bIsClient;
			} break;

			default:
			{
				ensureMsgf(false, TEXT("Missing implementation"));
			} break;
		}

		if (!bEnabled)
		{
			continue;
		}

		FName ListenerClassName = *MetricConfig.Class.ToString();
		TObjectPtr<UNetworkMetricsBaseListener>* Listener = NetworkMetricsListeners.Find(ListenerClassName);

		// Create the listener if it's not in the cache.
		if (Listener == nullptr)
		{
			const UClass* ListenerClass = MetricConfig.Class.Get();
			if (ListenerClass == nullptr)
			{
				UE_LOGF(LogNet, Warning, "NetDriver::SetupNetworkMetricsListeners: Unknown metric listener %ls provided in config for metric %ls.", *MetricConfig.Class.ToString(), *MetricConfig.MetricName.ToString());
				continue;
			}

			Listener = &NetworkMetricsListeners.Add(ListenerClassName, NewObject<UNetworkMetricsBaseListener>(this, ListenerClass));
		}

		if (MetricConfig.MetricName != NAME_None)
		{
			if (!GetMetrics()->Contains(MetricConfig.MetricName))
			{
				UE_LOGF(LogNet, Warning, "NetDriver::SetupNetworkMetricListeners: Cannot register metric listener %ls to an unknown metric '%ls'.", *MetricConfig.Class.ToString(), *MetricConfig.MetricName.ToString());
				continue;
			}

			GetMetrics()->Register(MetricConfig.MetricName, Listener->Get());

			UE_LOGF(LogNet, Verbose, "Registering network metrics listener %ls for metric %ls", *ListenerClassName.ToString(), *MetricConfig.MetricName.ToString());
		}

		if (MetricConfig.Mutator.IsValid())
		{
			GetMetrics()->RegisterMutator(Listener->Get(), MetricConfig.Mutator);

			UE_LOGF(LogNet, Verbose, "%ls: registering network metrics mutator %ls with listener %ls", *NetDriverName.ToString(), *MetricConfig.Mutator.Get().GetName().ToString(), *ListenerClassName.ToString());
		}
	}

	// The listeners for the Stats system are hardcoded here because Stats is always expected to be running and some/all of the metrics should
	// not be disabled through a configuration file.
#if STATS
	RegisterStatsListener(UE::Net::Metric::ImportedNetGuids, GET_STATFNAME(STAT_ImportedNetGuids));
	RegisterStatsListener(UE::Net::Metric::PendingOuterNetGuids, GET_STATFNAME(STAT_PendingOuterNetGuids));
	RegisterStatsListener(UE::Net::Metric::UnmappedReplicators, GET_STATFNAME(STAT_UnmappedReplicators));
	RegisterStatsListener(UE::Net::Metric::NumInitiallyDormantActors, GET_STATFNAME(STAT_NumInitiallyDormantActors));
	RegisterStatsListener(UE::Net::Metric::NumConsideredActors, GET_STATFNAME(STAT_NumConsideredActors));
	RegisterStatsListener(UE::Net::Metric::PrioritizedActors, GET_STATFNAME(STAT_PrioritizedActors));
	RegisterStatsListener(UE::Net::Metric::NumRelevantDeletedActors, GET_STATFNAME(STAT_NumRelevantDeletedActors));
	RegisterStatsListener(UE::Net::Metric::SharedSerializationRPCHit, GET_STATFNAME(STAT_SharedSerializationRPCHit));
	RegisterStatsListener(UE::Net::Metric::SharedSerializationRPCMiss, GET_STATFNAME(STAT_SharedSerializationRPCMiss));
	RegisterStatsListener(UE::Net::Metric::NumReplicatedActors, GET_STATFNAME(STAT_NumReplicatedActors));
	RegisterStatsListener(UE::Net::Metric::SatConnections, GET_STATFNAME(STAT_NumSaturatedConnections));
	RegisterStatsListener(UE::Net::Metric::NumSkippedObjectEmptyUpdates, GET_STATFNAME(STAT_NumSkippedObjectEmptyUpdates));
	RegisterStatsListener(UE::Net::Metric::SharedSerializationPropertyHit, GET_STATFNAME(STAT_SharedSerializationPropertyHit));
	RegisterStatsListener(UE::Net::Metric::SharedSerializationPropertyMiss, GET_STATFNAME(STAT_SharedSerializationPropertyMiss));
	RegisterStatsListener(UE::Net::Metric::NumReplicatedActorBytes, GET_STATFNAME(STAT_NumReplicatedActorBytes));
	RegisterStatsListener(UE::Net::Metric::Ping, GET_STATFNAME(STAT_Ping));
	RegisterStatsListener(UE::Net::Metric::Channels, GET_STATFNAME(STAT_Channels));
	RegisterStatsListener(UE::Net::Metric::MaxPacketOverhead, GET_STATFNAME(STAT_MaxPacketOverhead));
	RegisterStatsListener(UE::Net::Metric::OutPacketsLost, GET_STATFNAME(STAT_OutLoss));
	RegisterStatsListener(UE::Net::Metric::InPacketsLost, GET_STATFNAME(STAT_InLoss));
	RegisterStatsListener(UE::Net::Metric::InRate, GET_STATFNAME(STAT_InRate));
	RegisterStatsListener(UE::Net::Metric::OutRate, GET_STATFNAME(STAT_OutRate));
	RegisterStatsListener(UE::Net::Metric::InRateClientMax, GET_STATFNAME(STAT_InRateClientMax));
	RegisterStatsListener(UE::Net::Metric::InRateClientMin, GET_STATFNAME(STAT_InRateClientMin));
	RegisterStatsListener(UE::Net::Metric::InRateClientAvg, GET_STATFNAME(STAT_InRateClientAvg));
	RegisterStatsListener(UE::Net::Metric::InPacketsClientPerSecondMax, GET_STATFNAME(STAT_InPacketsClientMax));
	RegisterStatsListener(UE::Net::Metric::InPacketsClientPerSecondMin, GET_STATFNAME(STAT_InPacketsClientMin));
	RegisterStatsListener(UE::Net::Metric::InPacketsClientPerSecondAvg, GET_STATFNAME(STAT_InPacketsClientAvg));
	RegisterStatsListener(UE::Net::Metric::OutRateClientMax, GET_STATFNAME(STAT_OutRateClientMax));
	RegisterStatsListener(UE::Net::Metric::OutRateClientMin, GET_STATFNAME(STAT_OutRateClientMin));
	RegisterStatsListener(UE::Net::Metric::OutRateClientAvg, GET_STATFNAME(STAT_OutRateClientAvg));
	RegisterStatsListener(UE::Net::Metric::OutPacketsClientPerSecondMax, GET_STATFNAME(STAT_OutPacketsClientMax));
	RegisterStatsListener(UE::Net::Metric::OutPacketsClientPerSecondMin, GET_STATFNAME(STAT_OutPacketsClientMin));
	RegisterStatsListener(UE::Net::Metric::OutPacketsClientPerSecondAvg, GET_STATFNAME(STAT_OutPacketsClientAvg));
	RegisterStatsListener(UE::Net::Metric::NetNumClients, GET_STATFNAME(STAT_NetNumClients));
	RegisterStatsListener(UE::Net::Metric::InPackets, GET_STATFNAME(STAT_InPackets));
	RegisterStatsListener(UE::Net::Metric::OutPackets, GET_STATFNAME(STAT_OutPackets));
	RegisterStatsListener(UE::Net::Metric::InBunches, GET_STATFNAME(STAT_InBunches));
	RegisterStatsListener(UE::Net::Metric::OutBunches, GET_STATFNAME(STAT_OutBunches));
	RegisterStatsListener(UE::Net::Metric::NetGUIDInRate, GET_STATFNAME(STAT_NetGUIDInRate));
	RegisterStatsListener(UE::Net::Metric::NetGUIDOutRate, GET_STATFNAME(STAT_NetGUIDOutRate));
	RegisterStatsListener(UE::Net::Metric::VoicePacketsSent, GET_STATFNAME(STAT_VoicePacketsSent));
	RegisterStatsListener(UE::Net::Metric::VoicePacketsRecv, GET_STATFNAME(STAT_VoicePacketsRecv));
	RegisterStatsListener(UE::Net::Metric::VoiceBytesSent, GET_STATFNAME(STAT_VoiceBytesSent));
	RegisterStatsListener(UE::Net::Metric::VoiceBytesRecv, GET_STATFNAME(STAT_VoiceBytesRecv));
	RegisterStatsListener(UE::Net::Metric::PercentInVoice, GET_STATFNAME(STAT_PercentInVoice));
	RegisterStatsListener(UE::Net::Metric::PercentOutVoice, GET_STATFNAME(STAT_PercentOutVoice));
	RegisterStatsListener(UE::Net::Metric::NumActorChannels, GET_STATFNAME(STAT_NumActorChannels));
	RegisterStatsListener(UE::Net::Metric::NumDormantActors, GET_STATFNAME(STAT_NumDormantActors));
	RegisterStatsListener(UE::Net::Metric::NumActors, GET_STATFNAME(STAT_NumActors));
	RegisterStatsListener(UE::Net::Metric::NumNetActors, GET_STATFNAME(STAT_NumNetActors));
	RegisterStatsListener(UE::Net::Metric::NumNetGUIDsAckd, GET_STATFNAME(STAT_NumNetGUIDsAckd));
	RegisterStatsListener(UE::Net::Metric::NumNetGUIDsPending, GET_STATFNAME(STAT_NumNetGUIDsPending));
	RegisterStatsListener(UE::Net::Metric::NumNetGUIDsUnAckd, GET_STATFNAME(STAT_NumNetGUIDsUnAckd));
	RegisterStatsListener(UE::Net::Metric::NetSaturated, GET_STATFNAME(STAT_NetSaturated));
	RegisterStatsListener(UE::Net::Metric::NetInBunchTimeOvershootPercent, GET_STATFNAME(STAT_NetInBunchTimeOvershootPercent));
	RegisterStatsListener(UE::Net::Metric::GatherPrioritizeTimeMS, GET_STATFNAME(STAT_NetServerGatherPrioritizeRepActorsTime));
	RegisterStatsListener(UE::Net::Metric::OutgoingReliableMessageQueueMaxSize, GET_STATFNAME(STAT_OutgoingReliableMessageQueueMaxSize));
	RegisterStatsListener(UE::Net::Metric::IncomingReliableMessageQueueMaxSize, GET_STATFNAME(STAT_IncomingReliableMessageQueueMaxSize));
#endif
}

FString UNetDriver::GetReplicationModelName() const
{
	if (UReplicationDriver* RepDriver = GetReplicationDriver())
	{
		return RepDriver->GetClass()->GetName();
	}
	else if (ReplicationSystem)
	{
		return TEXT("Iris");
	}
	else
	{
		return TEXT("Generic");
	}
}

void UNetDriver::InitConnectionlessHandler()
{
	check(!ConnectionlessHandler.IsValid());

#if !UE_BUILD_SHIPPING
	if (!FParse::Param(FCommandLine::Get(), TEXT("NoPacketHandler")))
#endif
	{
		ConnectionlessHandler = MakeUnique<PacketHandler>(&DDoS);

		if (ConnectionlessHandler.IsValid())
		{
			ConnectionlessHandler->NotifyAnalyticsProvider(AnalyticsProvider, AnalyticsAggregator);
			ConnectionlessHandler->Initialize(UE::Handler::Mode::Server, MAX_PACKET_SIZE, true, nullptr, nullptr, NetDriverDefinition);

			// Add handling for the stateless connect handshake, for connectionless packets, as the outermost layer
			TSharedPtr<HandlerComponent> NewComponent =
				ConnectionlessHandler->AddHandler(TEXT("Engine.EngineHandlerComponentFactory(StatelessConnectHandlerComponent)"), true);

			StatelessConnectComponent = StaticCastSharedPtr<StatelessConnectHandlerComponent>(NewComponent);

			if (StatelessConnectComponent.IsValid())
			{
				StatelessConnectComponent.Pin()->SetDriver(this);
			}

			ConnectionlessHandler->InitializeComponents();
		}
	}
}

void UNetDriver::FlushHandler()
{
	BufferedPacket* QueuedPacket = ConnectionlessHandler->GetQueuedConnectionlessPacket();

	while (QueuedPacket != nullptr)
	{
		LowLevelSend(QueuedPacket->Address, QueuedPacket->Data, QueuedPacket->CountBits, QueuedPacket->Traits);

		delete QueuedPacket;

		QueuedPacket = ConnectionlessHandler->GetQueuedConnectionlessPacket();
	}
}

void UNetDriver::EnableExecuteRPCFunctions(bool bEnable)
{
	bExecuteRPCFunctions = bEnable;
}

bool UNetDriver::IsExecuteRPCFunctionsEnabled() const
{
	return bExecuteRPCFunctions;
}

void UNetDriver::EnablePreReplication(bool bEnable)
{
	bCallPreReplication = bEnable;
}

bool UNetDriver::IsPreReplicationEnabled() const
{
	return bCallPreReplication;
}

void UNetDriver::EnableRoleSwappingOfAllActors(bool bEnable)
{
	bSwapRolesOfAllActors = bEnable;
}

bool UNetDriver::IsRoleSwappingOnAllActorsEnabled() const
{
	return bSwapRolesOfAllActors;
}

void UNetDriver::EnableReplicationOfActorsWithAnyRole(bool bEnable)
{
	bReplicationOfActorsWithAnyRole = bEnable;
}

bool UNetDriver::IsReplicationOfActorsWithAnyRoleEnabled() const
{
	return bReplicationOfActorsWithAnyRole;
}

void UNetDriver::SetProxyType(UE::Net::EProxyType InProxyType)
{
	ProxyType = InProxyType;
}

UE::Net::EProxyType UNetDriver::GetProxyType() const
{
	return ProxyType;
}

ENetMode UNetDriver::GetNetMode() const
{
	// Special case for PIE and game worlds - forcing dedicated server behavior if requested
#if WITH_EDITOR
	if (World && (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game) && IsServer())
	{
		//@todo: world context won't be valid during seamless travel CopyWorldData
		FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);
		if (WorldContext && WorldContext->RunAsDedicated)
		{
			return NM_DedicatedServer;
		}
	}
#endif

	// Normal
	return (IsServer() ? (GIsClient ? NM_ListenServer : NM_DedicatedServer) : NM_Client);
}

void UNetDriver::RegisterTickEvents(class UWorld* InWorld)
{
	if (InWorld)
	{
		TickDispatchDelegateHandle  = InWorld->OnTickDispatch ().AddUObject(this, &UNetDriver::InternalTickDispatch);
		PostTickDispatchDelegateHandle	= InWorld->OnPostTickDispatch().AddUObject(this, &UNetDriver::PostTickDispatch);
		TickFlushDelegateHandle     = InWorld->OnTickFlush    ().AddUObject(this, &UNetDriver::InternalTickFlush);
		PostTickFlushDelegateHandle		= InWorld->OnPostTickFlush	 ().AddUObject(this, &UNetDriver::PostTickFlush);
	}
}

void UNetDriver::UnregisterTickEvents(class UWorld* InWorld)
{
	if (InWorld)
	{
		InWorld->OnTickDispatch ().Remove(TickDispatchDelegateHandle);
		InWorld->OnPostTickDispatch().Remove(PostTickDispatchDelegateHandle);
		InWorld->OnTickFlush().Remove(TickFlushDelegateHandle);
		InWorld->OnPostTickFlush().Remove(PostTickFlushDelegateHandle);
	}
}

void UNetDriver::InternalTickDispatch(float DeltaSeconds)
{
	TGuardValue<bool> GuardInNetTick(bInTick, true);

	if (ReplicationSystem)
	{
		ReplicationSystem->PreReceiveUpdate();
	}

	TickDispatch(DeltaSeconds);

	if (ReplicationSystem)
	{
		ReplicationSystem->PostReceiveUpdate();
	}
}

void UNetDriver::InternalTickFlush(float DeltaSeconds)
{
	TGuardValue<bool> GuardInNetTick(bInTick, true);
	TickFlush(DeltaSeconds);
}

static bool bCVarLogPendingGuidsOnShutdown = false;
static FAutoConsoleVariableRef CVarLogPendingGuidsOnShutdown(
	TEXT("Net.LogPendingGuidsOnShutdown"),
	bCVarLogPendingGuidsOnShutdown,
	TEXT("")
);

/** Shutdown all connections managed by this net driver */
void UNetDriver::Shutdown()
{
	// Client closing connection to server
	if (ServerConnection)
	{
		const UPackageMapClient* const ClientPackageMap = Cast<UPackageMapClient>(ServerConnection->PackageMap);
		const bool bLogGuids = bCVarLogPendingGuidsOnShutdown && (ClientPackageMap != nullptr);
		TSet<FNetworkGUID> GuidsToLog;

		for (UChannel* Channel : ServerConnection->OpenChannels)
		{
			 if (UActorChannel * ActorChannel = Cast<UActorChannel>(Channel))
			 {
				 if (bLogGuids)
				 {
					 GuidsToLog.Append(ActorChannel->PendingGuidResolves);
				 }
				 ActorChannel->CleanupReplicators();
			 }
		}

		if (GuidsToLog.Num() > 0)
		{
			TArray<FString> GuidStrings;
			GuidStrings.Reserve(GuidsToLog.Num());
			for (const FNetworkGUID& GuidToLog : GuidsToLog)
			{
				FString FullNetGUIDPath = ClientPackageMap->GetFullNetGUIDPath(GuidToLog);
				if (!FullNetGUIDPath.IsEmpty())
				{
					GuidStrings.Emplace(MoveTemp(FullNetGUIDPath));
				}
				else
				{
					GuidStrings.Emplace(GuidToLog.ToString());
				}
			}

			UE_LOGF(LogNet, Warning, "NetDriver::Shutdown: Pending Guids: \n%ls", *FString::Join(GuidStrings, TEXT("\n")));
		}
		

		// Calls Channel[0]->Close to send a close bunch to server
		ServerConnection->Close();
		ServerConnection->FlushNet();
	}

	// Server closing connections with clients
	if (ClientConnections.Num() > 0)
	{
		FString ErrorMsg = NSLOCTEXT("NetworkErrors", "HostClosedConnection", "Host closed the connection.").ToString();

		for (UNetConnection* CurClient : ClientConnections)
		{
			CurClient->SendCloseReason(ENetCloseResult::HostClosedConnection);
			FNetControlMessage<NMT_Failure>::Send(CurClient, ErrorMsg);
			CurClient->FlushNet(true);
		}

		for (int32 ClientIndex = ClientConnections.Num() - 1; ClientIndex >= 0; ClientIndex--)
		{
			if (ClientConnections[ClientIndex]->PlayerController)
			{
				APawn* Pawn = ClientConnections[ClientIndex]->PlayerController->GetPawn();
				if( Pawn )
				{
					Pawn->Destroy( true );
				}
			}

			// Calls Close() internally and removes from ClientConnections
			ClientConnections[ClientIndex]->CleanUp();
		}
	}

	// Empty our replication map here before we're destroyed, 
	// even though we use AddReferencedObjects to keep the referenced properties
	// in here from being collected, when we're all GC'd the order seems non-deterministic
	RepLayoutMap.Empty();
	ReplicationChangeListMap.Empty();

	// Clean up the actor channel pool
	for (TObjectPtr<UChannel>& Channel : ActorChannelPool)
	{
		Channel->MarkAsGarbage();
	}
	ActorChannelPool.Empty();

	ConnectionlessHandler.Reset();

	SetReplicationDriver(nullptr);
	SetReplicationSystem(nullptr);

	// End NetTrace session for this instance
	UE_NET_TRACE_END_SESSION(GetNetTraceId());

	SendNetAnalytics();

	// Clear repmodel flags for clients or listen servers when shutting down since they are considered offline.
	// For dedicated servers let's keep the flag in case we crash post-shutdown.
	if (GetNetMode() != NM_DedicatedServer)
	{
		UpdateCrashContext(ECrashContextUpdate::ClearRepModel);
	}
	else
	{
		UpdateCrashContext(ECrashContextUpdate::Default);
	}

	NetworkMetricsDatabase->Reset();
	NetworkMetricsListeners.Reset();
}

bool UNetDriver::IsServer() const
{
	// Client connections ALWAYS set the server connection object in InitConnect()
	// @todo ONLINE improve this with a bool
	return ServerConnection == NULL;
}

void UNetDriver::SendNetAnalytics()
{
	if (!AnalyticsAggregator.IsValid())
	{
		return;
	}

	// Add the default NetDriver information if requested
	if (CVarAddNetDriverInfoToNetAnalytics.GetValueOnAnyThread())
	{
		SetNetAnalyticsAttributes(TEXT("NetDriverName"), NetDriverName.ToString());
		SetNetAnalyticsAttributes(TEXT("NetDriverDefinition"), NetDriverDefinition.ToString());
		SetNetAnalyticsAttributes(TEXT("ReplicationModel"), GetReplicationModelName());
		SetNetAnalyticsAttributes(TEXT("NetMode"), ToString(GetNetMode()));
	}

	auto GameAttributes = [this](TArray<FAnalyticsEventAttribute>& OutAttributes)
	{
		OutAttributes.Reserve(OutAttributes.Num() + CachedNetAnalyticsAttributes.Num());
		for (auto It=CachedNetAnalyticsAttributes.CreateConstIterator(); It; ++It)
		{
			OutAttributes.Emplace(FAnalyticsEventAttribute(It.Key(), It.Value()));
		}
	};

	AnalyticsAggregator->SetAnalyticsAppender(GameAttributes);
	AnalyticsAggregator->SendAnalytics();
	AnalyticsAggregator.Reset();
}

EEngineNetworkRuntimeFeatures UNetDriver::GetNetworkRuntimeFeatures() const
{
	EEngineNetworkRuntimeFeatures NetDriverFeatures = EEngineNetworkRuntimeFeatures::None;
	
	if (IsUsingIrisReplication())
	{
		EnumAddFlags(NetDriverFeatures, EEngineNetworkRuntimeFeatures::IrisEnabled);
	}

	const bool bWithRemoteObjectReferences = !!UE_WITH_REMOTE_OBJECT_HANDLE;
	if (bWithRemoteObjectReferences && bUseRemoteObjectRefs)	
	{
		EnumAddFlags(NetDriverFeatures, EEngineNetworkRuntimeFeatures::UseRemoteObjectReferences);
	}

	return NetDriverFeatures;
}

void UNetDriver::TryUpgradeNetworkFeatures(EEngineNetworkRuntimeFeatures RemoteFeatures)
{
	const bool bIsRemoteUsingIris = EnumHasAnyFlags(RemoteFeatures, EEngineNetworkRuntimeFeatures::IrisEnabled);

	// If our netdriver needs to enable Iris
	if (bIsRemoteUsingIris && !IsUsingIrisReplication())
	{
		SetReplicationDriver(nullptr);

		constexpr bool bInitWithIris = true;
		constexpr bool bUsingParallelNetTick = false;
		PostCreation(bInitWithIris, bUsingParallelNetTick);

		RecreateIrisSystem();
		
		ReinitBase();

		if (ensureMsgf(ReplicationSystem, TEXT("NetworkFeature Upgraded failed to create repSystem for %s"), *GetDescription()))
		{
			checkf(ServerConnection, TEXT("NMT_Upgrade control message received with no server connection on %s"), *GetDescription());
			
			// Tell Iris and the Datastream about the current netconnection
			ReplicationSystem->AddConnection(ServerConnection->GetConnectionHandle().GetParentConnectionId());
			ReplicationSystem->SetConnectionUserData(ServerConnection->GetConnectionHandle().GetParentConnectionId(), ServerConnection);
			for (UChannel* Channel : ServerConnection->Channels)
			{
				if (Channel)
				{
					Channel->ReInit();
				}
			}
		}

	}
	// If the remote was not using Iris but we did
	else if(!bIsRemoteUsingIris && IsUsingIrisReplication())
	{
		DestroyIrisSystem();

		constexpr bool bInitWithoutIris = false;
		constexpr bool bUsingParallelNetTick = false;
		PostCreation(bInitWithoutIris, bUsingParallelNetTick);

		ReinitBase();

		for (UChannel* Channel : ServerConnection->Channels)
		{
			if (Channel)
			{
				Channel->ReInit();
			}
		}
	}

	ensureMsgf(GetNetworkRuntimeFeatures() == RemoteFeatures, TEXT("UNetDriver::TryUpgradeNetworkFeatures failed for %s"), *GetDescription());
}

// ----------------------------------------------------------------------------------------
//	RPC Tracking (receiving) via CSV profiler
// ----------------------------------------------------------------------------------------
CSV_DEFINE_CATEGORY(ReplicationRPCs, WITH_SERVER_CODE);

#ifndef RPC_CSV_TRACKER	// Defines if RPC CSV tracking is compiled in. Default is only on server builds.
#define RPC_CSV_TRACKER (CSV_PROFILER_STATS && WITH_SERVER_CODE)
#endif

/** Helper struct for tracking RPC (receive) timing */
struct FRPCCSVTracker
{
	const double StatThreshold = 0.001; // 1MS minimum accumulated threshold for a stat to be dumped

	void NotifyRPCProcessed(UFunction* Func, double Time)
	{
#if RPC_CSV_TRACKER
		if (FItem* Item = FunctionMap.Find(Func))
		{
			Item->Time += Time;
		}
		else
		{
			FunctionMap.Emplace( Func, FItem(Func, Time) );
		}
#endif
	}

	void EndTickDispatch()
	{
#if RPC_CSV_TRACKER
		FCsvProfiler* Profiler = FCsvProfiler::Get();
		if (Profiler->IsCapturing())
		{
			// Record stat and reset accumulated time
			for (auto& It : FunctionMap)
			{
				FItem& Item = It.Value;
				if (Item.Time >= StatThreshold)
				{
					Profiler->RecordCustomStat(Item.Stat, CSV_CATEGORY_INDEX(ReplicationRPCs), static_cast<float>(Item.Time * 1000.0), ECsvCustomStatOp::Set);
				}
				Item.Time = 0.0;
			}
		}
#endif
	}

	struct FItem
	{
		FItem(UFunction* Func, double InTime)
		{
#if RPC_CSV_TRACKER
			Stat = FName(*Func->GetName());
			Time = InTime;
#endif
		}
		double Time;
		FName Stat;
	};

	TMap<UFunction*, FItem>	FunctionMap;
	
} GRPCCSVTracker;

// ----------------------------------------------------------------------------------------

void UNetDriver::TickDispatch( float DeltaTime )
{
	SendCycles=0;

	// Manage realtime values
	{
		const double CurrentRealtime = FPlatformTime::Seconds();
		const float DeltaRealtime = CurrentRealtime - LastTickDispatchRealtime;
		LastTickDispatchRealtime = CurrentRealtime;

		// Check to see if too much time is passing between ticks
		// Setting this to somewhat large value for now, but small enough to catch blocking calls that are causing timeouts
		constexpr float TickLogThreshold = 5.0f;
		bDidHitchLastFrame = (DeltaTime > TickLogThreshold || DeltaRealtime > TickLogThreshold);

		UE_CLOGF(bDidHitchLastFrame, LogNet, Log, "UNetDriver::TickDispatch: Very long time between ticks. DeltaTime: %2.2f, Realtime: %2.2f. %ls", DeltaTime, DeltaRealtime, *GetName() );
	}

	// Get new time.
	ElapsedTime += DeltaTime;

	IncomingBunchProcessingElapsedFrameTimeMS = 0.0f;

	// Checks for standby cheats if enabled	
	UpdateStandbyCheatStatus();

	ResetNetworkMetrics();

	if (ServerConnection == nullptr)
	{
		// Delete any straggler connections
		{
			QUICK_SCOPE_CYCLE_COUNTER(UNetDriver_TickDispatch_CheckClientConnectionCleanup)

			for (int32 ConnIdx=ClientConnections.Num()-1; ConnIdx>=0; ConnIdx--)
			{
				UNetConnection* CurConn = ClientConnections[ConnIdx];

				if (IsValid(CurConn))
				{
					if (CurConn->GetConnectionState() == USOCK_Closed)
					{
						CurConn->CleanUp();
					}
					else
					{
						CurConn->PreTickDispatch();
					}
				}
			}
		}

		// Clean up recently disconnected client tracking
		if (RecentlyDisconnectedClients.Num() > 0)
		{
			int32 NumToRemove = 0;

			for (const FDisconnectedClient& CurElement : RecentlyDisconnectedClients)
			{
				if ((LastTickDispatchRealtime - CurElement.DisconnectTime) >= RecentlyDisconnectedTrackingTime)
				{
					verify(MappedClientConnections.Remove(CurElement.Address) == 1);

					NumToRemove++;
				}
				else
				{
					break;
				}
			}

			if (NumToRemove > 0)
			{
				RecentlyDisconnectedClients.RemoveAt(0, NumToRemove);
			}
		}
	}
	else if (IsValid(ServerConnection))
	{
		ServerConnection->PreTickDispatch();
	}

#if RPC_CSV_TRACKER
	GReceiveRPCTimingEnabled = (NetDriverName == NAME_GameNetDriver && ShouldEnableScopeSecondsTimers()) && (ServerConnection==nullptr);
#endif
}

void UNetDriver::PostTickDispatch()
{
	TGuardValue<bool> GuardInNetTick(bInTick, true);

	if (ShouldReplicateTransactionally())
	{
		ensureMsgf(IsServer(), TEXT("We are not expecting Clients to replicate transactionally"));
		if (ReplicatorsWithPendingTransactions.Num() > 0)
		{
			TSet<FObjectReplicator*> TransactingReplicators; 
			Swap(TransactingReplicators, ReplicatorsWithPendingTransactions);

			for (FObjectReplicator* Replicator : TransactingReplicators)
			{
				static const FName WorkName { TEXT("EnqueuePendingTransactionalRPCs") };
				constexpr bool bIsTransactional = true;
				UE::RemoteExecutor::EnqueueWork(WorkName, bIsTransactional, [Replicator, &TransactingReplicators]()
				{
					if (const UObject* ReplicatorObject = Replicator->GetObject())
					{
						// Help us keep track of this object's execution time
						UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(ReplicatorObject->GetFName());
						UE_AUTORTFM_ONABORT()
						{
							UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT("Abort"));
						};
						UE_AUTORTFM_ONCOMMIT()
						{
							UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT("Commit"));
						};

						Replicator->EnqueuePendingTransactionalRPCs();
					}

					TransactingReplicators.Remove(Replicator);
				});
			}

			// Actually do all of the work
			static const FName RegionName { __func__ };
			UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(RegionName);
			UE::RemoteExecutor::ExecutePendingWork();
			UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT(""));

			ensureMsgf(TransactingReplicators.IsEmpty(), TEXT("We expected all TransactingReplicators to be serviced by now!"));
			ensureMsgf(ReplicatorsWithPendingTransactions.IsEmpty(), TEXT("Execution of RPCs or pending work caused more RPCs to be enqueued -- those will be delayed another frame"));
		}
	}

	// Flush out of order packet caches for connections that did not receive the missing packets during TickDispatch
	if (ServerConnection != nullptr)
	{
		if (IsValid(ServerConnection))
		{
			ServerConnection->PostTickDispatch();
		}
	}

	TArray<UNetConnection*> ClientConnCopy = ClientConnections;
	for (UNetConnection* CurConn : ClientConnCopy)
	{
		if (IsValid(CurConn))
		{
			CurConn->PostTickDispatch();
		}
	}

	PostDispatchSendUpdate();

	if (ReplicationDriver)
	{
		ReplicationDriver->PostTickDispatch();
	}

	if (GReceiveRPCTimingEnabled)
	{
		GRPCCSVTracker.EndTickDispatch();
		GReceiveRPCTimingEnabled = false;
	}
}

void UNetDriver::NotifyRPCProcessed(UFunction* Function, UNetConnection* Connection, double ElapsedTimeSeconds)
{
	GRPCCSVTracker.NotifyRPCProcessed(Function, ElapsedTimeSeconds);
}

bool UNetDriver::IsLevelInitializedForActor(const AActor* InActor, const UNetConnection* InConnection) const
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	check(InActor);
	check(InConnection);
	check(World == InActor->GetWorld());
#endif

	UPackage* WorldPackageRawPtr = GetWorldPackage();
	const bool bWorldHasSamePackageName = WorldPackageRawPtr != nullptr && InConnection->GetClientWorldPackageName() == WorldPackageRawPtr->GetFName();

	// we can't create channels while the client is in the wrong world
	if (bWorldHasSamePackageName && InConnection->ClientHasInitializedLevel(InActor->GetLevel()))
	{
		return true;
	}

	// exception: Special case for PlayerControllers as they are required for the client to travel to the new world correctly			
	const APlayerController* AsPlayerController = Cast<APlayerController>(InActor);
	if (!AsPlayerController)
	{
		return false;
	}

	bool bIsConnectionPC = false;
	// Make sure we have Parent Connection
	const UNetConnection* const ParentConnection = InConnection->IsA<UChildConnection>() ? Cast<UChildConnection>(InConnection)->Parent : InConnection;

	// Check Parent Connection
	if (InActor == ParentConnection->PlayerController)
	{
		bIsConnectionPC = true;
	}
	// Check Child Connections
	else
	{
		for (const UChildConnection* const ChildConnection : ParentConnection->Children)
		{
			if (InActor == ChildConnection->PlayerController)
			{
				bIsConnectionPC = true;
				break;
			}
		}
	}

	return bIsConnectionPC;
}

//
// Internal RPC calling.
//
void UNetDriver::InternalProcessRemoteFunction(
	AActor* Actor,
	UObject* SubObject,
	UNetConnection* Connection,
	UFunction* Function,
	void* Parms,
	FOutParmRec* OutParms,
	FFrame* Stack,
	const bool bIsServer)
{
	EProcessRemoteFunctionFlags UnusedFlags = EProcessRemoteFunctionFlags::None;
	InternalProcessRemoteFunctionPrivate(Actor, SubObject, Connection, Function, Parms, OutParms, Stack, bIsServer, UnusedFlags);
}

void UNetDriver::InternalProcessRemoteFunctionPrivate(
	AActor* Actor,
	UObject* SubObject,
	UNetConnection* Connection,
	UFunction* Function,
	void* Parms,
	FOutParmRec* OutParms,
	FFrame* Stack,
	const bool bIsServer,
	EProcessRemoteFunctionFlags& Flags)
{
	// get the top most function
	while (Function->GetSuperFunction())
	{
		Function = Function->GetSuperFunction();
	}

	// If saturated and function is unimportant, skip it. Note unreliable multicasts are queued at the actor channel level so they are not gated here.
	if (!(Function->FunctionFlags & FUNC_NetReliable) && (!(Function->FunctionFlags & FUNC_NetMulticast)) && (!Connection->IsNetReady()))
	{
		DEBUG_REMOTEFUNCTION(TEXT("Network saturated, not calling %s::%s"), *GetNameSafe(Actor), *GetNameSafe(Function));
		return;
	}

	// Route RPC calls to actual connection
	if (Connection->GetUChildConnection())
	{
		Connection = ((UChildConnection*)Connection)->Parent;
	}

	// Prevent RPC calls to gracefully-closing connections
	const EConnectionState ConnectionState = Connection->GetConnectionState();
	if (ConnectionState == USOCK_Closing)
	{
		DEBUG_REMOTEFUNCTION(TEXT("Attempting to call RPC on a closing connection. Not calling %s::%s"), *GetNameSafe(Actor), *GetNameSafe(Function));
		return;
	}

	// Prevent RPC calls to closed connections
	if (ConnectionState == USOCK_Closed)
	{
		DEBUG_REMOTEFUNCTION(TEXT("Attempting to call RPC on a closed connection. Not calling %s::%s"), *GetNameSafe(Actor), *GetNameSafe(Function));
		return;
	}

	if (World == nullptr)
	{
		DEBUG_REMOTEFUNCTION(TEXT("Attempting to call RPC with a null World on the net driver. Not calling %s::%s"), *GetNameSafe(Actor), *GetNameSafe(Function));
		return;
	}

	// If we have a subobject, thats who we are actually calling this on. If no subobject, we are calling on the actor.
	UObject* TargetObj = SubObject ? SubObject : Actor;

	// Make sure this function exists for both parties.
	const FClassNetCache* ClassCache = NetCache->GetClassNetCache( TargetObj->GetClass() );
	if (!ClassCache)
	{
		DEBUG_REMOTEFUNCTION(TEXT("ClassNetCache empty, not calling %s::%s"), *GetNameSafe(Actor), *GetNameSafe(Function));
		return;
	}
		
	const FFieldNetCache* FieldCache = ClassCache->GetFromField(Function);
	if (!FieldCache)
	{
		DEBUG_REMOTEFUNCTION(TEXT("FieldCache empty, not calling %s::%s"), *GetNameSafe(Actor), *GetNameSafe(Function));
		return;
	}

	// Get the actor channel.
	UActorChannel* Ch = Connection->FindActorChannelRef(Actor);
	if (!Ch)
	{
		if (bIsServer)
		{
			if (Actor->IsPendingKillPending())
			{
				// Don't try opening a channel for me, I am in the process of being destroyed. Ignore my RPCs.
				return;
			}

			if (IsLevelInitializedForActor(Actor, Connection))
			{
				Ch = Cast<UActorChannel>(Connection->CreateChannelByName(NAME_Actor, EChannelCreateFlags::OpenedLocally));
			}
			else
			{
				UE_LOGF(LogNet, Verbose, "Can't send function '%ls' on actor '%ls' because client hasn't loaded the level '%ls' containing it", *GetNameSafe(Function), *GetNameSafe(Actor), *GetNameSafe(Actor->GetLevel()));
				return;
			}
		}

		if (!Ch)
		{
			return;
		}

		if (bIsServer)
		{
			Ch->SetChannelActor(Actor, ESetChannelActorFlags::None);
		}
	}

	ProcessRemoteFunctionForChannelPrivate(Ch, ClassCache, FieldCache, TargetObj, Connection, Function, Parms, OutParms, Stack, bIsServer, ERemoteFunctionSendPolicy::Default, Flags);
}

void UNetDriver::ProcessRemoteFunctionForChannel(
	UActorChannel* Ch,
	const FClassNetCache* ClassCache,
	const FFieldNetCache* FieldCache,
	UObject* TargetObj,
	UNetConnection* Connection,
	UFunction* Function,
	void* Parms,
	FOutParmRec* OutParms,
	FFrame* Stack,
	const bool bIsServer,
	const ERemoteFunctionSendPolicy SendPolicy)
{
	EProcessRemoteFunctionFlags UnusedFlags = EProcessRemoteFunctionFlags::None;
	ProcessRemoteFunctionForChannelPrivate(Ch, ClassCache, FieldCache, TargetObj, Connection, Function, Parms, OutParms, Stack, bIsServer, SendPolicy, UnusedFlags);
}

void UNetDriver::ProcessRemoteFunctionForChannel(
	UActorChannel* Ch,
	const FClassNetCache* ClassCache,
	const FFieldNetCache* FieldCache,
	UObject* TargetObj,
	UNetConnection* Connection,
	UFunction* Function,
	void* Parms,
	FOutParmRec* OutParms,
	FFrame* Stack,
	const bool bIsServer,
	const ERemoteFunctionSendPolicy SendPolicy,
	EProcessRemoteFunctionFlags& RemoteFunctionFlags)
{
	ProcessRemoteFunctionForChannelPrivate(Ch, ClassCache, FieldCache, TargetObj, Connection, Function, Parms, OutParms, Stack, bIsServer, SendPolicy, RemoteFunctionFlags);
}

void UNetDriver::ProcessRemoteFunctionForChannelPrivate(
	UActorChannel* Ch,
	const FClassNetCache* ClassCache,
	const FFieldNetCache* FieldCache,
	UObject* TargetObj,
	UNetConnection* Connection,
	UFunction* Function,
	void* Parms,
	FOutParmRec* OutParms,
	FFrame* Stack,
	const bool bIsServer,
	const ERemoteFunctionSendPolicy SendPolicy,
	EProcessRemoteFunctionFlags& RemoteFunctionFlags)
{
	// ---------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
	// This function should be kept fast! Assume this is getting called multiple times at once. Don't look things up/recalc them if they do not change per connection/actor.
	// ---------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------

	// The connection must be part of the same driver.
	ensure(Connection->GetDriver() == this);

	if (Ch->Closing)
	{
		return;
	}

	// Make sure initial channel-opening replication has taken place.
	if (Ch->OpenPacketId.First == INDEX_NONE)
	{
		if (!bIsServer)
		{
			DEBUG_REMOTEFUNCTION(TEXT("Initial channel replication has not occurred, not calling %s::%s"), *GetFullNameSafe(TargetObj), *GetNameSafe(Function));
			return;
		}

		// triggering replication of an Actor while already in the middle of replication can result in invalid data being sent and is therefore illegal
		if (Ch->bIsReplicatingActor)
		{
			FString Error(FString::Printf(TEXT("Attempt to replicate function '%s' on Actor '%s' while it is in the middle of variable replication!"), *GetNameSafe(Function), *GetFullNameSafe(TargetObj)));
			UE_LOGF(LogScript, Error, "%ls", *Error);
			ensureMsgf(false, TEXT("%s"), *Error);
			return;
		}

		// Bump the ReplicationFrame value to invalidate any properties marked as "unchanged".
		// We only want to do this at most once per invocation of ProcessRemoteFunctionForChannel, to prevent cases
		// where invoking a Client Multicast RPC Spams PreReplication and Property Comparisons.
		if (!EnumHasAnyFlags(RemoteFunctionFlags, EProcessRemoteFunctionFlags::ReplicatedActor))
		{
			ReplicationFrame++;

			if (bCallPreReplication)
			{
				Ch->GetActor()->CallPreReplication(this);
			}

			RemoteFunctionFlags |= EProcessRemoteFunctionFlags::ReplicatedActor;
		}

		if (!Ch->IsActorReadyForReplication())
		{
			// Build the list of replicated components since the actor isn't BeginPlay yet. Otherwise none of the components would be replicated by this initial ReplicateActor.
			UE::Net::FSubObjectRegistryGetter::InitReplicatedComponentsList(Ch->GetActor());
		}

		Ch->SetForcedSerializeFromRPC(true);
		Ch->ReplicateActor();
		Ch->SetForcedSerializeFromRPC(false);
	}

	// Clients may be "closing" this connection but still processing bunches, we can't send anything if we have an invalid ChIndex.
	if (Ch->ChIndex == -1)
	{
		ensure(!bIsServer);
		return;
	}

	FScopedRepContext RepContext(Connection, Ch->GetActor());
	
	// Form the RPC preamble.
	FOutBunch Bunch(Ch, 0);

	// Reliability.
	//warning: RPC's might overflow, preventing reliable functions from getting thorough.
	if (Function->FunctionFlags & FUNC_NetReliable)
	{
		Bunch.bReliable = 1;
	}

	// verify we haven't overflowed unacked bunch buffer (Connection is not net ready)
	//@warning: needs to be after parameter evaluation for script stack integrity
	if (Bunch.IsError())
	{
		if (!Bunch.bReliable)
		{
			// Not reliable, so not fatal. This can happen a lot in debug builds at startup if client is slow to get in game
			UE_LOGF(LogNet, Warning, "Can't send function '%ls' on '%ls': Reliable buffer overflow. FieldCache->FieldNetIndex: %d Max %d. Ch MaxPacket: %d", *GetNameSafe(Function), *GetFullNameSafe(TargetObj), FieldCache->FieldNetIndex, ClassCache->GetMaxIndex(), Ch->Connection->MaxPacket );
		}
		else
		{
			// The connection has overflowed the reliable buffer. We cannot recover from this. Disconnect this user.
			UE_LOGF(LogNet, Warning, "Closing connection. Can't send function '%ls' on '%ls': Reliable buffer overflow. FieldCache->FieldNetIndex: %d Max %d. Ch MaxPacket: %d.", *GetNameSafe(Function), *GetFullNameSafe(TargetObj), FieldCache->FieldNetIndex, ClassCache->GetMaxIndex(), Ch->Connection->MaxPacket );

			FString ErrorMsg = NSLOCTEXT("NetworkErrors", "ClientReliableBufferOverflow", "Outgoing reliable buffer overflow").ToString();

			Connection->SendCloseReason(ENetCloseResult::RPCReliableBufferOverflow);
			FNetControlMessage<NMT_Failure>::Send(Connection, ErrorMsg);
			Connection->FlushNet(true);
			Connection->Close(ENetCloseResult::RPCReliableBufferOverflow);
#if USE_SERVER_PERF_COUNTERS
			GetMetrics()->IncrementInt(UE::Net::Metric::ClosedConnectionsDueToReliableBufferOverflow, 1);
#endif
		}
		return;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	extern TAutoConsoleVariable<int32> CVarNetReliableDebug;

	if (CVarNetReliableDebug.GetValueOnAnyThread() > 0)
	{
		Bunch.DebugString = FString::Printf(TEXT("%.2f RPC: %s - %s"), Connection->Driver->GetElapsedTime(), *GetFullNameSafe(TargetObj), *GetNameSafe(Function));
	}
#endif

	// Init bunch for rpc
#if UE_NET_TRACE_ENABLED
	SetTraceCollector(Bunch, UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace));
#endif

	bool LogAsWarning = (GNetRPCDebug == 1);
	if (LogAsWarning)
	{
		// Suppress spammy engine RPCs. This could be made a configable list in the future.
		if (Function->GetName().Contains(TEXT("ServerUpdateCamera"))) LogAsWarning = false;
		if (Function->GetName().Contains(TEXT("ClientAckGoodMove"))) LogAsWarning = false;
		if (Function->GetName().Contains(TEXT("ServerMove"))) LogAsWarning = false;
	}

	FNetBitWriter TempWriter( Bunch.PackageMap, 0 );

	// Create NetToken export scope for TempBitWriter, appending exports to Bunch.NetTokensPendingExport
	UE::Net::FNetTokenExportScope ExportScope(TempWriter, GetNetTokenStore(), Bunch.NetTokensPendingExport, "ProcessRemoteFunctionForChannelPrivate");
	
#if UE_NET_TRACE_ENABLED
	// Create trace collector if tracing is enabled for the target bunch
	SetTraceCollector(TempWriter, GetTraceCollector(Bunch) ? UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace) : nullptr);
    ON_SCOPE_EXIT { UE_NET_TRACE_DESTROY_COLLECTOR(GetTraceCollector(TempWriter)); };
#endif // UE_NET_TRACE_ENABLED

	// Use the replication layout to send the rpc parameter values
	TSharedPtr<FRepLayout> RepLayout = GetFunctionRepLayout(Function);
	RepLayout->SendPropertiesForRPC(Function, Ch, TempWriter, Parms);

	if (TempWriter.IsError())
	{
		if (LogAsWarning)
		{
			UE_LOGF(LogNet, Warning, "Error: Can't send function '%ls' on '%ls': Failed to serialize properties", *GetNameSafe(Function), *GetFullNameSafe(TargetObj));
		}
		else
		{
			UE_LOGF(LogNet, Log, "Error: Can't send function '%ls' on '%ls': Failed to serialize properties", *GetNameSafe(Function), *GetFullNameSafe(TargetObj));
		}
	}
	else
	{
		// Make sure net field export group is registered
		FNetFieldExportGroup* NetFieldExportGroup = Ch->GetOrCreateNetFieldExportGroupForClassNetCache(TargetObj);

		int32 HeaderBits	= 0;
		int32 ParameterBits	= 0;
		
		bool QueueBunch = false;
		switch (SendPolicy)
		{
			case ERemoteFunctionSendPolicy::Default:
				QueueBunch = ( !Bunch.bReliable && Function->FunctionFlags & FUNC_NetMulticast );
				break;
			case ERemoteFunctionSendPolicy::ForceQueue:
				QueueBunch = true;
				break;
			case ERemoteFunctionSendPolicy::ForceSend:
				QueueBunch = false;
				break;
		}

		if (QueueBunch)
		{
			Ch->WriteFieldHeaderAndPayload(Bunch, ClassCache, FieldCache, NetFieldExportGroup, TempWriter);
			ParameterBits = Bunch.GetNumBits();
		}
		else
		{
			Ch->PrepareForRemoteFunction(TargetObj);

			FNetBitWriter TempBlockWriter(Bunch.PackageMap, 0);

#if UE_NET_TRACE_ENABLED
			UE_NET_TRACE_OBJECT_SCOPE(Ch->ActorNetGUID, Bunch, GetTraceCollector(Bunch), ENetTraceVerbosity::Trace);

			// Ugliness to get data reported correctly, we basically fold the data from TempWriter into TempBlockWriter and then to Bunch
			// Create trace collector if tracing is enabled for the target bunch			
			SetTraceCollector(TempBlockWriter, GetTraceCollector(Bunch) ? UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Trace) : nullptr);
#endif
			
			Ch->WriteFieldHeaderAndPayload(TempBlockWriter, ClassCache, FieldCache, NetFieldExportGroup, TempWriter);
			ParameterBits = TempBlockWriter.GetNumBits();
			HeaderBits = Ch->WriteContentBlockPayload(TargetObj, Bunch, false, TempBlockWriter);

			UE_NET_TRACE_DESTROY_COLLECTOR(GetTraceCollector(TempBlockWriter));
		}

		// Send the bunch.
		if( Bunch.IsError() )
		{
			UE_LOGF(LogNet, Log, "Error: Can't send function '%ls' on '%ls': RPC bunch overflowed (too much data in parameters?)", *GetNameSafe(Function), *GetFullNameSafe(TargetObj));
			ensureMsgf(false,TEXT("Error: Can't send function '%s' on '%s': RPC bunch overflowed (too much data in parameters?)"), *GetNameSafe(Function), *GetFullNameSafe(TargetObj));
		}
		else if (Ch->Closing)
		{
			UE_LOGF(LogNetTraffic, Log, "RPC bunch on closing channel" );
		}
		else
		{
			// Make sure we're tracking all the bits in the bunch
			check(Bunch.GetNumBits() == HeaderBits + ParameterBits);

			if (QueueBunch)
			{
				// Unreliable multicast functions are queued and sent out during property replication
				if (LogAsWarning)
				{
					UE_LOGF(LogNetTraffic, Warning,	"      Queing unreliable multicast RPC: %ls::%ls [%.1f bytes]", *GetFullNameSafe(TargetObj), *GetNameSafe(Function), Bunch.GetNumBits() / 8.f);
				}
				else
				{
					UE_LOGF(LogNetTraffic, Log,		"      Queing unreliable multicast RPC: %ls::%ls [%.1f bytes]", *GetFullNameSafe(TargetObj), *GetNameSafe(Function), Bunch.GetNumBits() / 8.f);
				}

				NETWORK_PROFILER(GNetworkProfiler.TrackQueuedRPC(Connection, TargetObj, Ch->Actor, Function, HeaderBits, ParameterBits, 0));
				Ch->QueueRemoteFunctionBunch(TargetObj, Function, Bunch);
			}
			else
			{
				if (LogAsWarning)
				{
					UE_LOGF(LogNetTraffic, Warning,	"      Sent RPC: %ls::%ls [%.1f bytes]", *GetFullNameSafe(TargetObj), *GetNameSafe(Function), Bunch.GetNumBits() / 8.f);
				}
				else
				{
					UE_LOGF(LogNetTraffic, Log,		"      Sent RPC: %ls::%ls [%.1f bytes]", *GetFullNameSafe(TargetObj), *GetNameSafe(Function), Bunch.GetNumBits() / 8.f);
				}

				NETWORK_PROFILER(GNetworkProfiler.TrackSendRPC(Ch->Actor, Function, HeaderBits, ParameterBits, 0, Connection));
				constexpr bool bDoMerge = true;
				FPacketIdRange PacketIdRange = Ch->SendBunch(&Bunch, bDoMerge);

				if (UNLIKELY(PacketIdRange.First == INDEX_NONE && PacketIdRange.Last == INDEX_NONE))
				{
					UE_LOGF(LogNetTraffic, Error, "      ERROR: Failed to send RPC: %ls::%ls [%.1f bytes]", *GetFullNameSafe(TargetObj), *GetNameSafe(Function), Bunch.GetNumBits() / 8.f);
				}

			}
		}
	}

	if (Connection->IsInternalAck())
	{
		Connection->FlushNet();
	}
}

void UNetDriver::UpdateStandbyCheatStatus(void)
{
#if WITH_SERVER_CODE
	// Only the server needs to check
	if (ServerConnection == NULL && ClientConnections.Num())
	{
		QUICK_SCOPE_CYCLE_COUNTER(UNetDriver_TickDispatch_UpdateStandbyCheatStatus)

		// Only check for cheats if enabled and one wasn't previously detected
		if (bIsStandbyCheckingEnabled &&
			bHasStandbyCheatTriggered == false &&
			ClientConnections.Num() > 2)
		{
			int32 CountBadTx = 0;
			int32 CountBadRx = 0;
			int32 CountBadPing = 0;
			
			UWorld* FoundWorld = NULL;
			// Look at each connection checking for a receive time and an ack time
			for (int32 Index = 0; Index < ClientConnections.Num(); Index++)
			{
				UNetConnection* NetConn = ClientConnections[Index];
				// Don't check connections that aren't fully formed (still loading & no controller)
				// Controller won't be present until the join message is sent, which is after loading has completed
				if (NetConn)
				{
					APlayerController* PlayerController = NetConn->PlayerController;
					if(PlayerController)
					{
						UWorld* PlayerControllerWorld = PlayerController->GetWorld();
						if( PlayerControllerWorld && 
						   PlayerControllerWorld->GetTimeSeconds() - PlayerController->CreationTime > JoinInProgressStandbyWaitTime &&
							// Ignore players with pending delete (kicked/timed out, but connection not closed)
							PlayerController->IsPendingKillPending() == false)
						{
							if (!FoundWorld)
							{
								FoundWorld = PlayerControllerWorld;
							}
							else
							{
								check(FoundWorld == PlayerControllerWorld);
							}
							if (ElapsedTime - NetConn->LastReceiveTime > StandbyRxCheatTime)
							{
								CountBadRx++;
							}
							if (ElapsedTime - NetConn->GetLastRecvAckTime() > StandbyTxCheatTime)
							{
								CountBadTx++;
							}
							// Check for host tampering or crappy upstream bandwidth
							if (PlayerController->PlayerState &&
								PlayerController->PlayerState->ExactPing > BadPingThreshold)
							{
								CountBadPing++;
							}
						}
					}
				}
			}
			
			if (FoundWorld)
			{
				AGameNetworkManager* const NetworkManager = FoundWorld->NetworkManager;
				if (NetworkManager)
				{
					// See if we hit the percentage required for either TX or RX standby detection
					if (float(CountBadRx) / float(ClientConnections.Num()) > PercentMissingForRxStandby)
					{
						bHasStandbyCheatTriggered = true;
						NetworkManager->StandbyCheatDetected(STDBY_Rx);
					}
					else if (float(CountBadPing) / float(ClientConnections.Num()) > PercentForBadPing)
					{
						bHasStandbyCheatTriggered = true;
						NetworkManager->StandbyCheatDetected(STDBY_BadPing);
					}
					// Check for the host not sending to the clients
					else if (float(CountBadTx) / float(ClientConnections.Num()) > PercentMissingForTxStandby)
					{
						bHasStandbyCheatTriggered = true;
						NetworkManager->StandbyCheatDetected(STDBY_Tx);
					}
				}
			}
		}
	}
#endif
}


void UNetDriver::SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider)
{
	AnalyticsProvider = InProvider;

	if (AnalyticsProvider.IsValid())
	{
		if (AnalyticsAggregator.IsValid())
		{
			// If reinitializing an existing aggregator, hotfix its config
			AnalyticsAggregator->InitConfig();

			ensure(AnalyticsAggregator->GetAnalyticsProvider() == AnalyticsProvider);
		}
		else
		{
			AnalyticsAggregator = MakeShareable(new FNetAnalyticsAggregator(InProvider, NetDriverDefinition));

			AnalyticsAggregator->Init();
		}
	}
	else
	{
		AnalyticsAggregator.Reset();
	}

	if (ConnectionlessHandler.IsValid())
	{
		ConnectionlessHandler->NotifyAnalyticsProvider(AnalyticsProvider, AnalyticsAggregator);
	}

	if (ServerConnection != nullptr)
	{
		ServerConnection->NotifyAnalyticsProvider();
	}

	for (UNetConnection* CurConn : ClientConnections)
	{
		if (CurConn != nullptr)
		{
			CurConn->NotifyAnalyticsProvider();
		}
	}
}

void UNetDriver::Serialize( FArchive& Ar )
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UNetDriver::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("UNetDriver::Super", Super::Serialize(Ar));

	if (Ar.IsCountingMemory())
	{
		// TODO: We don't currently track:
		//		StatelessConnectComponents
		//		PacketHandlers
		//		Network Address Bytes
		//		AnalyticsData
		//		NetworkNotify
		//		Delegate Handles
		//		DDoSDetection data
		// These are probably insignificant, though.

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("MappedClientConnection", MappedClientConnections.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RecentlyDisconnectedClients", RecentlyDisconnectedClients.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GuidCache",
			if (FNetGUIDCache const * const LocalGuidCache = GetNetGuidCache().Get())
			{
				LocalGuidCache->CountBytes(Ar);
			}
		);
		
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LocalNetCache",
			if (FClassNetCacheMgr const * const LocalNetCache = NetCache.Get())
			{
				LocalNetCache->CountBytes(Ar);
			}
		);

#if NET_DEBUG_RELEVANT_ACTORS
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LastPrioritizedActors", LastPrioritizedActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LastRelevantActors", LastRelevantActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LastSentActors", LastSentActors.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("LastNonRelevantActors", LastNonRelevantActors.CountBytes(Ar));
#endif // NET_DEBUG_RELEVANT_ACTORS

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DestroyedStartupOrDormantActors",
			DestroyedStartupOrDormantActors.CountBytes(Ar);

			for (const auto& DestroyedStartupOrDormantActorPair : DestroyedStartupOrDormantActors)
			{
				if (FActorDestructionInfo const * const DestructionInfo = DestroyedStartupOrDormantActorPair.Value.Get())
				{
					Ar.CountBytes(sizeof(FActorDestructionInfo), sizeof(FActorDestructionInfo));
					DestructionInfo->PathName.CountBytes(Ar);
				}
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DestroyedStartupOrDormantActorsByLevel", 
			DestroyedStartupOrDormantActorsByLevel.CountBytes(Ar);

			for (const auto& DestroyedPair : DestroyedStartupOrDormantActorsByLevel)
			{
				DestroyedPair.Value.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RenamedStartupActors", RenamedStartupActors.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RepLayoutMap",
			RepLayoutMap.CountBytes(Ar);

			for (const auto& RepLayoutPair : RepLayoutMap)
			{
				if (FRepLayout const * const RepLayout = RepLayoutPair.Value.Get())
				{
					Ar.CountBytes(sizeof(FRepLayout), sizeof(FRepLayout));
					RepLayout->CountBytes(Ar);
				}
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ReplicationChangeListMap", 
			ReplicationChangeListMap.CountBytes(Ar);

			for (const auto& ReplicationChangeListPair : ReplicationChangeListMap)
			{
				if (FReplicationChangelistMgr const* const ChangelistMgr = ReplicationChangeListPair.Value.Get())
				{
					Ar.CountBytes(sizeof(FReplicationChangelistMgr), sizeof(FReplicationChangelistMgr));
					ChangelistMgr->CountBytes(Ar);
				}
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("GuidToReplicatorMap", GuidToReplicatorMap.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("UnmappedReplicators", UnmappedReplicators.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("AllOwnedReplicators",
		
			// AllOwnedReplicators is the superset of all initialized FObjectReplicators,
			// including DormantReplicators, Replicators on Channels, and UnmappedReplicators.
			AllOwnedReplicators.CountBytes(Ar);
			
			for (FObjectReplicator const * const OwnedReplicator : AllOwnedReplicators)
			{
				if (OwnedReplicator)
				{
					Ar.CountBytes(sizeof(FObjectReplicator), sizeof(FObjectReplicator));
					OwnedReplicator->CountBytes(Ar);
				}
			}
		);

		// Replicators are owned by UActorChannels, and so we don't track them here.

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetworkObjects",
			if (FNetworkObjectList const * const NetObjList = NetworkObjects.Get())
	{
				Ar.CountBytes(sizeof(FNetworkObjectList), sizeof(FNetworkObjectList));
				NetworkObjects->CountBytes(Ar);
			}
		);
	}
}

void UNetDriver::FinishDestroy()
{
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		// Make sure we tell listeners we are no longer lagging in case they set something up when lagging started.
		if (GEngine && LagState != ENetworkLagState::NotLagging)
		{
			LagState = ENetworkLagState::NotLagging;
			GEngine->BroadcastNetworkLagStateChanged(GetWorld(), this, LagState);
		}

		// Destroy server connection.
		if( ServerConnection )
		{
			ServerConnection->CleanUp();
		}
		// Destroy client connections.
		while( ClientConnections.Num() )
		{
			UNetConnection* ClientConnection = ClientConnections[0];
			ClientConnection->CleanUp();
		}
		// Low level destroy.
		LowLevelDestroy();

		// Delete the guid cache
		GetNetGuidCache().Reset();

		FWorldDelegates::LevelRemovedFromWorld.Remove(OnLevelRemovedFromWorldHandle);
		FWorldDelegates::LevelAddedToWorld.Remove(OnLevelAddedToWorldHandle);
		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
		FNetDelegates::OnSyncLoadDetected.Remove(ReportSyncLoadDelegateHandle);
	}
	else
	{
		check(ServerConnection==NULL);
		check(ClientConnections.Num()==0);
		check(!GetNetGuidCache().IsValid());
	}

	// Make sure we've properly shut down all of the FObjectReplicator's
	check( GuidToReplicatorMap.Num() == 0 );
	check( TotalTrackedGuidMemoryBytes == 0 );
	check( UnmappedReplicators.Num() == 0 );

	Super::FinishDestroy();
}

void UNetDriver::LowLevelDestroy()
{
	// We are closing down all our sockets and low level communications.
	// Sever the link with UWorld to ensure we don't tick again
	SetWorld(NULL);

	if(GetNetGuidCache().IsValid())
	{
		GetNetGuidCache()->ReportSyncLoadedGUIDs();
	}
}

FString UNetDriver::LowLevelGetNetworkNumber()
{
	return LocalAddr.IsValid() ? LocalAddr->ToString(true) : FString();
}

#if !UE_BUILD_SHIPPING
bool UNetDriver::HandleSocketsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Print list of open connections.
	Ar.Logf( TEXT("%s Connections:"), *GetDescription() );
	if( ServerConnection )
	{
		Ar.Logf( TEXT("   Server %s"), *ServerConnection->LowLevelDescribe() );
		for( int32 i=0; i<ServerConnection->OpenChannels.Num(); i++ )
		{
			UChannel* const Channel = ServerConnection->OpenChannels[i];
			if (Channel)
			{
				Ar.Logf( TEXT("      Channel %i: %s"), Channel->ChIndex, *Channel->Describe() );
			}
		}
	}
#if WITH_SERVER_CODE
	for( int32 i=0; i<ClientConnections.Num(); i++ )
	{
		UNetConnection* Connection = ClientConnections[i];
		Ar.Logf( TEXT("   Client %s"), *Connection->LowLevelDescribe() );
		for( int32 j=0; j<Connection->OpenChannels.Num(); j++ )
		{
			UChannel* const Channel = Connection->OpenChannels[j];
			if (Channel)
			{
				Ar.Logf( TEXT("      Channel %i: %s"), Channel->ChIndex, *Channel->Describe() );
			}
		}
	}
#endif
	return true;
}

bool UNetDriver::HandlePackageMapCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Print packagemap for open connections
	Ar.Logf(TEXT("Package Map:"));
	if (ServerConnection != NULL)
	{
		Ar.Logf(TEXT("   Server %s"), *ServerConnection->LowLevelDescribe());
		ServerConnection->PackageMap->LogDebugInfo(Ar);
	}
#if WITH_SERVER_CODE
	for (int32 i = 0; i < ClientConnections.Num(); i++)
	{
		UNetConnection* Connection = ClientConnections[i];
		Ar.Logf( TEXT("   Client %s"), *Connection->LowLevelDescribe() );
		Connection->PackageMap->LogDebugInfo(Ar);
	}
#endif
	return true;
}

bool UNetDriver::HandleNetFloodCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UNetConnection* TestConn = NULL;
	if (ServerConnection != NULL)
	{
		TestConn = ServerConnection;
	}
#if WITH_SERVER_CODE
	else if (ClientConnections.Num() > 0)
	{
		TestConn = ClientConnections[0];
	}
#endif
	if (TestConn != NULL)
	{
		Ar.Logf(TEXT("Flooding connection 0 with control messages"));

		for (int32 i = 0; i < 256 && TestConn->GetConnectionState() == USOCK_Open; i++)
		{
			FNetControlMessage<NMT_Netspeed>::Send(TestConn, TestConn->CurrentNetSpeed);
			TestConn->FlushNet();
		}
	}
	return true;
}

bool UNetDriver::HandleNetDebugTextCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Send a text string for testing connection
	FString TestStr = FParse::Token(Cmd,false);
	if (ServerConnection != NULL)
	{
		UE_LOGF(LogNet, Log, "%ls sending NMT_DebugText [%ls] to [%ls]", 
			*GetDescription(),*TestStr, *ServerConnection->LowLevelDescribe());

		FNetControlMessage<NMT_DebugText>::Send(ServerConnection,TestStr);
		ServerConnection->FlushNet(true);
	}
#if WITH_SERVER_CODE
	else
	{
		for (int32 ClientIdx=0; ClientIdx < ClientConnections.Num(); ClientIdx++)
		{
			UNetConnection* Connection = ClientConnections[ClientIdx];
			if (Connection)
			{
				UE_LOGF(LogNet, Log, "%ls sending NMT_DebugText [%ls] to [%ls]", 
					*GetDescription(),*TestStr, *Connection->LowLevelDescribe());

				FNetControlMessage<NMT_DebugText>::Send(Connection,TestStr);
				Connection->FlushNet(true);
			}
		}
	}
#endif // WITH_SERVER_CODE
	return true;
}

bool UNetDriver::HandleNetDisconnectCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString Msg = NSLOCTEXT("NetworkErrors", "NETDISCONNECTMSG", "NETDISCONNECT MSG").ToString();
	if (ServerConnection != NULL)
	{
		UE_LOGF(LogNet, Log, "%ls disconnecting connection from host [%ls]", 
			*GetDescription(),*ServerConnection->LowLevelDescribe());

		ServerConnection->SendCloseReason(ENetCloseResult::Disconnect);
		FNetControlMessage<NMT_Failure>::Send(ServerConnection, Msg);
		ServerConnection->FlushNet(true);
		ServerConnection->Close(ENetCloseResult::Disconnect);
	}
#if WITH_SERVER_CODE
	else
	{
		for (int32 ClientIdx=0; ClientIdx < ClientConnections.Num(); ClientIdx++)
		{
			UNetConnection* Connection = ClientConnections[ClientIdx];
			if (Connection)
			{
				UE_LOGF(LogNet, Log, "%ls disconnecting from client [%ls]", 
					*GetDescription(),*Connection->LowLevelDescribe());

				Connection->SendCloseReason(ENetCloseResult::Disconnect);
				FNetControlMessage<NMT_Failure>::Send(Connection, Msg);
				Connection->FlushNet(true);
			}
		}

		// Copy ClientConnections to close each connection, in case closing modifies the ClientConnnection list
		const TArray<UNetConnection*> ClientConnCopy = ClientConnections;

		for (UNetConnection* Connection : ClientConnCopy)
		{
			if (Connection != nullptr)
			{
				Connection->Close(ENetCloseResult::Disconnect);
			}
		}
	}
#endif
	return true;
}

bool UNetDriver::HandleNetDumpServerRPCCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
#if WITH_SERVER_CODE
	for ( TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt )
	{
		bool bHasNetFields = false;

		ClassIt->SetUpRuntimeReplicationData();

		for ( int32 i = 0; i < ClassIt->NetFields.Num(); i++ )
		{
			UFunction * Function = Cast<UFunction>( ClassIt->NetFields[i] );

			if ( Function != NULL && Function->FunctionFlags & FUNC_NetServer )
			{
				bHasNetFields = true;
				break;
			}
		}

		if ( !bHasNetFields )
		{
			continue;
		}

		Ar.Logf( TEXT( "Class: %s" ), *ClassIt->GetName() );

		for ( int32 i = 0; i < ClassIt->NetFields.Num(); i++ )
		{
			UFunction * Function = Cast<UFunction>( ClassIt->NetFields[i] );

			if ( Function != NULL && Function->FunctionFlags & FUNC_NetServer )
			{
				const FClassNetCache * ClassCache = NetCache->GetClassNetCache( *ClassIt );

				const FFieldNetCache * FieldCache = ClassCache->GetFromField( Function );

				TArray< FProperty * > Parms;

				for ( TFieldIterator<FProperty> It( Function ); It && ( It->PropertyFlags & ( CPF_Parm | CPF_ReturnParm ) ) == CPF_Parm; ++It )
				{
					Parms.Add( *It );
				}

				if ( Parms.Num() == 0 )
				{
					Ar.Logf( TEXT( "    [0x%03x] %s();" ), FieldCache->FieldNetIndex, *Function->GetName() );
					continue;
				}

				FString ParmString;

				for ( int32 j = 0; j < Parms.Num(); j++ )
				{
					if ( CastField<FStructProperty>( Parms[j] ) )
					{
						ParmString += CastField<FStructProperty>( Parms[j] )->Struct->GetName();
					}
					else
					{
						ParmString += Parms[j]->GetClass()->GetName();
					}

					ParmString += TEXT( " " );

					ParmString += Parms[j]->GetName();

					if ( j < Parms.Num() - 1 )
					{
						ParmString += TEXT( ", " );
					}
				}						

				Ar.Logf( TEXT( "    [0x%03x] %s( %s );" ), FieldCache->FieldNetIndex, *Function->GetName(), *ParmString );
			}
		}
	}
#endif
	return true;
}

bool UNetDriver::HandleNetDumpDormancy(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FProperty* Property = FindFProperty<FProperty>(AActor::StaticClass(), TEXT("NetDormancy"));
	check(Property != nullptr);
	Ar.Logf(TEXT(""));
	TArray<AActor*> Actors;
	if (FParse::Command(&Cmd, TEXT("ACTIVE")))
	{
		for (auto It = GetNetworkObjectList().GetActiveObjects().CreateConstIterator(); It; ++It)
		{
			FNetworkObjectInfo* ActorInfo = (*It).Get();
			Actors.Add(ActorInfo->Actor);
		}
		Ar.Logf(TEXT("Logging network dormancy for %d active network objects"), Actors.Num());
		Ar.Logf(TEXT(""));
	}
	else if (FParse::Command(&Cmd, TEXT("ALLDORMANT")))
	{
		for (auto It = GetNetworkObjectList().GetDormantObjectsOnAllConnections().CreateConstIterator(); It; ++It)
		{
			FNetworkObjectInfo* ActorInfo = (*It).Get();
			Actors.Add(ActorInfo->Actor);
		}
		Ar.Logf(TEXT("Logging network dormancy for %d dormant network objects"), Actors.Num());
		Ar.Logf(TEXT(""));
	}
	else
	{
		for (auto It = GetNetworkObjectList().GetAllObjects().CreateConstIterator(); It; ++It)
		{
			FNetworkObjectInfo* ActorInfo = (*It).Get();
			Actors.Add(ActorInfo->Actor);
		}
		Ar.Logf(TEXT("Logging network dormancy for %d all network objects"), Actors.Num());
		Ar.Logf(TEXT(""));
	}
	// Iterate through the objects reporting on their dormancy status
	for (AActor* ThisActor : Actors)
	{
		uint8* BaseData = (uint8*)ThisActor;
		FString ResultStr;
		for (int32 i = 0; i < Property->ArrayDim; i++)
		{
			Property->ExportText_InContainer(i, ResultStr, BaseData, BaseData, ThisActor, PPF_IncludeTransient);
		}

		Ar.Logf(TEXT("%s.%s = %s"), *ThisActor->GetFullName(), *Property->GetName(), *ResultStr);
	}
	Ar.Logf(TEXT(""));
	return true;
}

bool UNetDriver::HandleDumpSubObjectsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_SERVER_CODE
	for(UNetConnection* ClientConnection : ClientConnections)
	{
		Ar.Logf(TEXT("Logging subobjects for %d open actor channels on %s"), ClientConnection->OpenChannels.Num(), *GetNameSafe(ClientConnection));
		Ar.Logf(TEXT(""));

		for (const UChannel* Channel : ClientConnection->OpenChannels)
		{
			if (const UActorChannel* ActorChannel = Cast<UActorChannel>(Channel))
			{
				if (ActorChannel->ReplicationMap.Num() > 1)
				{
					Ar.Logf(TEXT("   Actor: %s"), *GetFullNameSafe(ActorChannel->Actor));

					for (auto RepComp = ActorChannel->ReplicationMap.CreateConstIterator(); RepComp; ++RepComp)
					{
						const TSharedRef<FObjectReplicator>& LocalReplicator = RepComp.Value();
						const UObject* Obj = LocalReplicator->GetWeakObjectPtr().Get();

						if (Obj != ActorChannel->Actor)
						{
							Ar.Logf(TEXT("       Object: %s Class: %s"), *GetNameSafe(Obj), *GetNameSafe(LocalReplicator->ObjectClass));
						}
					}
				}
			}
		}
	}
#endif
	return true;
}

bool UNetDriver::HandleDumpRepLayoutFlagsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_SERVER_CODE

#if WITH_PUSH_MODEL
	const bool bPartialPushDump = FParse::Command(&Cmd, TEXT("PARTIALPUSHMODEL"));
#endif

	Ar.Logf(TEXT("Logging replayout flags for: %s"), *GetNameSafe(this));
	Ar.Logf(TEXT(""));

	for (auto LayoutIt = RepLayoutMap.CreateConstIterator(); LayoutIt; ++LayoutIt)
	{
		const TSharedPtr<FRepLayout>& RepLayout = LayoutIt.Value();
		if (RepLayout.IsValid())
		{
			FString FlagStr;

			const ERepLayoutFlags Flags = RepLayout->GetFlags();

			for (uint32 i = 0; i < sizeof(ERepLayoutFlags) * 8; ++i)
			{
				ERepLayoutFlags Flag = (ERepLayoutFlags)(1 << i);

				if (EnumHasAnyFlags(Flags, Flag))
				{
					FlagStr += (FlagStr.IsEmpty() ? TEXT("") : TEXT("|"));
					FlagStr += LexToString(Flag);
				}
			}

			if (FlagStr.IsEmpty())
			{
				FlagStr = TEXT("None");
			}

			Ar.Logf(TEXT("  Owner: %s"), *GetFullNameSafe(RepLayout->GetOwner()));
			Ar.Logf(TEXT("    Flags: %s"), *FlagStr);

#if WITH_PUSH_MODEL
			if (bPartialPushDump && EnumHasAnyFlags(Flags, ERepLayoutFlags::PartialPushSupport))
			{
				int32 ParentCount = RepLayout->GetNumParents();

				Ar.Logf(TEXT("    Parent Properties: %d Non-push:"), ParentCount);

				for (int32 i = 0; i < ParentCount; ++i)
				{
					if ((RepLayout->GetParentCondition(i) != COND_Never) && !RepLayout->IsPushModelProperty(i) && (RepLayout->GetParentArrayIndex(i) == 0))
					{
						if (const FProperty* Property = RepLayout->GetParentProperty(i))
						{
							Ar.Logf(TEXT("      Property: %s Class: %s"), *Property->GetName(), *GetNameSafe(Property->GetOwnerClass()));
						}
					}
				}
			}
#endif
		}
	}
#endif
	return true;
}

bool UNetDriver::HandlePushModelMemCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_PUSH_MODEL
	UEPushModelPrivate::LogMemory(Ar);
#endif
	return true;
}

bool UNetDriver::HandlePropertyConditionsMemCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	UE::Net::Private::FNetPropertyConditionManager::Get().LogMemory(Ar);
	return true;
}
#endif // !UE_BUILD_SHIPPING

void UNetDriver::HandlePacketLossBurstCommand( int32 DurationInMilliseconds )
{
#if DO_ENABLE_NET_TEST
	UE_LOGF(LogNet, Log, "%ls simulating packet loss burst. Dropping incoming and outgoing packets for %d milliseconds.", *GetName(), DurationInMilliseconds);

	PacketLossBurstEndTime = ElapsedTime + (DurationInMilliseconds / 1000.0);
#endif
}

#if !UE_BUILD_SHIPPING
void HandleNetFlushAllDormancy(UNetDriver* InNetDriver, UWorld* InWorld)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDriver_HandleNetFlushAllDormancy);

	// TODO: Maybe prevent this from happening on Beacons or Demo Net Drivers?
	if (InNetDriver && InWorld && InNetDriver->ServerConnection == nullptr)
	{
		const FNetworkObjectList& NetworkObjectList = InNetDriver->GetNetworkObjectList();
		const int32 NumberOfActorsDormantOnAllConnections = NetworkObjectList.GetDormantObjectsOnAllConnections().Num();

		TSet<AActor*> ActorsToReset;
		ActorsToReset.Reserve(NumberOfActorsDormantOnAllConnections);

		// Just iterate over all network objects here to make sure we get Actors that are Pending Dormancy.
		for (const TSharedPtr<FNetworkObjectInfo>& NetworkObjectInfo : NetworkObjectList.GetAllObjects())
		{
			if (NetworkObjectInfo.IsValid())
			{
				if (AActor* Actor = NetworkObjectInfo->WeakActor.Get())
				{
					if (Actor->NetDormancy.GetValue() > (int32)DORM_Awake)
					{
						ActorsToReset.Add(Actor);
					}
				}
			}
		}

		const double StartTime = FPlatformTime::Seconds();

		for (AActor* Actor : ActorsToReset)
		{
			Actor->FlushNetDormancy();
		}

		const double TotalTime = FPlatformTime::Seconds() - StartTime;


		float CurrentDormancyHysteresis = -1.f;
		if (IConsoleVariable* CurrentDormancyHysteresisCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.DormancyHysteresis")))
		{
			CurrentDormancyHysteresis = CurrentDormancyHysteresisCVar->GetFloat();
		}

		UE_LOGF(LogNet, Warning, "HandleNetFlushAllDormancy: NumberOfActors: %d, NumberOfActorsDormantOnAllConnections: %d, TimeForFlush: %lf, DormancyHysteresis: %f",
			ActorsToReset.Num(), NumberOfActorsDormantOnAllConnections, TotalTime, CurrentDormancyHysteresis);
	}
}
#endif

bool UNetDriver::Exec_Dev( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
#if !UE_BUILD_SHIPPING
	int32 PacketLossBurstMilliseconds = 0;

	if( FParse::Command(&Cmd,TEXT("SOCKETS")) )
	{
		return HandleSocketsCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("PACKAGEMAP")))
	{
		return HandlePackageMapCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("NETFLOOD")))
	{
		return HandleNetFloodCommand( Cmd, Ar );
	}
#if DO_ENABLE_NET_TEST
	// This will allow changing the Pkt* options at runtime
	else if (bNeverApplyNetworkEmulationSettings == false && 
			 PacketSimulationSettings.ParseSettings(Cmd, *NetDriverDefinition.ToString()))
	{
		OnPacketSimulationSettingsChanged();
		return true;
	}
#endif
	else if (FParse::Command(&Cmd, TEXT("NETDEBUGTEXT")))
	{
		return HandleNetDebugTextCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("NETDISCONNECT")))
	{
		return HandleNetDisconnectCommand( Cmd, Ar );	
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPSERVERRPC")))
	{
		return HandleNetDumpServerRPCCommand( Cmd, Ar );	
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPDORMANCY")))
	{
		return HandleNetDumpDormancy(Cmd, Ar);
	}
	else if (FParse::Value(Cmd, TEXT("PKTLOSSBURST="), PacketLossBurstMilliseconds))
	{
		HandlePacketLossBurstCommand(PacketLossBurstMilliseconds);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("FLUSHALLNETDORMANCY")))
	{
		HandleNetFlushAllDormancy(this, InWorld);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPSUBOBJECTS")))
	{
		HandleDumpSubObjectsCommand(Cmd, Ar);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPREPLAYOUTFLAGS")))
	{
		HandleDumpRepLayoutFlagsCommand(Cmd, Ar);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("PUSHMODELMEM")))
	{
		HandlePushModelMemCommand(Cmd, Ar);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("PROPERTYCONDITIONMEM")))
	{
		HandlePropertyConditionsMemCommand(Cmd, Ar);
		return true;
	}
	else
#endif // !UE_BUILD_SHIPPING
	{
		return false;
	}
}

bool UNetDriver::SendDestructionInfoForLevelUnloadIfDormant(AActor* ThisActor, UNetConnection* Connection)
{
	TSharedPtr<FNetworkObjectInfo> FoundInfo = GetNetworkObjectList().Find(ThisActor);
	if (FoundInfo && Connection)
	{
		const bool bDormantOrRecentlyDormant = FoundInfo->DormantConnections.Contains(Connection) || FoundInfo->RecentlyDormantConnections.Contains(Connection);

		if (bDormantOrRecentlyDormant)
		{
			UE_LOGF(LogNet, Verbose, "Sending destruction info for dormant actor level unload: %ls, to connection: %ls", *GetFullNameSafe(ThisActor), *Connection->Describe());

			FActorDestructionInfo DestructInfo;
			DestructInfo.DestroyedPosition = ThisActor->GetActorLocation();
			DestructInfo.NetGUID = GetNetGuidCache()->GetNetGUID(ThisActor);
			DestructInfo.Level = ThisActor->GetLevel();
			DestructInfo.ObjOuter = ThisActor->GetOuter();
			DestructInfo.PathName = ThisActor->GetName();
			DestructInfo.StreamingLevelName = NAME_None; // currently unused
			DestructInfo.Reason = EChannelCloseReason::LevelUnloaded;

			SendDestructionInfo(Connection, &DestructInfo);
			GetNetworkObjectList().MarkActive(ThisActor, Connection, this);
			return true;
		}
	}

	return false;
}

FActorDestructionInfo* UNetDriver::CreateDestructionInfo(AActor* ThisActor, FActorDestructionInfo *DestructionInfo )
{
	if (DestructionInfo)
	{
		return DestructionInfo;
	}

	FNetworkGUID NetGUID = GetNetGuidCache()->GetOrAssignNetGUID( ThisActor );
	if (NetGUID.IsDefault())
	{
		UE_LOGF(LogNet, Error, "CreateDestructionInfo got an invalid NetGUID for %ls", *ThisActor->GetName());
		return nullptr;
	}

	TUniquePtr<FActorDestructionInfo>& NewInfoPtr = DestroyedStartupOrDormantActors.FindOrAdd(NetGUID);
	if (NewInfoPtr.IsValid() == false)
	{
		NewInfoPtr = MakeUnique<FActorDestructionInfo>();
	}

	FActorDestructionInfo &NewInfo = *NewInfoPtr;
	NewInfo.DestroyedPosition = ThisActor->GetActorLocation();
	NewInfo.NetGUID = NetGUID;
	NewInfo.Level = ThisActor->GetLevel();
	NewInfo.ObjOuter = ThisActor->GetOuter();
	NewInfo.PathName = ThisActor->GetName();

	// Look for renamed actor now so we can clear it after the destroy is queued
	FName RenamedPath = RenamedStartupActors.FindRef(ThisActor->GetFName());
	if (RenamedPath != NAME_None)
	{
		NewInfo.PathName = RenamedPath.ToString();
	}

	if (NewInfo.Level.IsValid() && !NewInfo.Level->IsPersistentLevel() )
	{
		NewInfo.StreamingLevelName = NewInfo.Level->GetOutermost()->GetFName();
	}
	else
	{
		NewInfo.StreamingLevelName = NAME_None;
	}

	TSet<FNetworkGUID>& DestroyedGuidsForLevel = DestroyedStartupOrDormantActorsByLevel.FindOrAdd(NewInfo.StreamingLevelName);
	DestroyedGuidsForLevel.Add(NetGUID);

	NewInfo.Reason = EChannelCloseReason::Destroyed;

	return &NewInfo;
}

bool UNetDriver::CanCreateDestructionInfo() const
{
	return !GIsReconstructingBlueprintInstances && !UE::Net::ShouldIgnoreStaticActorDestruction();
}

void UNetDriver::NotifyActorDestroyed(AActor* ThisActor, bool IsSeamlessTravel, EChannelCloseReason CloseReason)
{
	if (ReplicationSystem)
	{
		// For actors that are DORM_Initial when destroyed we need to create and replicate a destruction info.
		if (ReplicationSystem->IsServer())
		{
			const bool bActorHasRole = ThisActor->GetRemoteRole() != ROLE_None;
			const bool bShouldCreateDestructionInfoForInitiallyDormantActor = IsDormInitialStartupActor(ThisActor) && bActorHasRole && !IsSeamlessTravel;

			if (bShouldCreateDestructionInfoForInitiallyDormantActor && CanCreateDestructionInfo())
			{
				if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
				{
					const UE::Net::FNetObjectReference Reference = Bridge->GetOrCreateObjectReference(ThisActor);
					if (Reference.GetRefHandle().IsValid() && Reference.GetRefHandle().IsStatic())
					{
						const UE::Net::FDestructionParameters Params
						{ 
							.Location = ThisActor->GetActorLocation(), 
							.Container = ThisActor->GetLevel(), 
							.bUseDistanceBasedPrioritization = true,
							.NetFactoryId = Bridge->GetActorFactoryId()
						};
						Bridge->AddStaticDestructionInfo(ThisActor->GetName(), ThisActor->GetOuter(), Params);
					}
				}
			}
		}

		// For the other cases, Iris handles replication of destroy through EndReplication
		return;
	}

	const bool bIsServer = IsServer();
	
	if (bIsServer)
	{
		FActorDestructionInfo* DestructionInfo = nullptr;
		bool IsStaticReplicator = false;
		if (IReplicatedObjectInterface* ReplicatedObject = Cast<IReplicatedObjectInterface>(ThisActor))
		{
			IsStaticReplicator = ReplicatedObject->IsStaticReplicator();
		}

		const bool bIsActorStatic = !GetNetGuidCache()->IsDynamicObject( ThisActor ) || IsStaticReplicator;
		const bool bActorHasRole = ThisActor->GetRemoteRole() != ROLE_None;
		const bool bShouldCreateDestructionInfo = bIsServer && bIsActorStatic && bActorHasRole && !IsSeamlessTravel;

		if (bShouldCreateDestructionInfo && CanCreateDestructionInfo())
		{
			UE_LOGF(LogNet, VeryVerbose, "NotifyActorDestroyed %ls - StartupActor", *ThisActor->GetPathName() );
			DestructionInfo = CreateDestructionInfo(ThisActor, DestructionInfo);
		}

		const FNetworkObjectInfo* NetworkObjectInfo = GetNetworkObjectList().Find( ThisActor ).Get();

		for( int32 i=ClientConnections.Num()-1; i>=0; i-- )
		{
			UNetConnection* Connection = ClientConnections[i];
			if( ThisActor->bNetTemporary )
				Connection->SentTemporaries.Remove( ThisActor );
			UActorChannel* Channel = Connection->FindActorChannelRef(ThisActor);
			if( Channel )
			{
				check(Channel->OpenedLocally);
				Channel->bClearRecentActorRefs = false;
				Channel->Close(CloseReason);
			}
			else
			{
				const bool bDormantOrRecentlyDormant = NetworkObjectInfo && (NetworkObjectInfo->DormantConnections.Contains(Connection) || NetworkObjectInfo->RecentlyDormantConnections.Contains(Connection));

				if (bShouldCreateDestructionInfo || bDormantOrRecentlyDormant)
				{
					// Make a new destruction info if necessary. It is necessary if the actor is dormant or recently dormant because
					// even though the client knew about the actor at some point, it doesn't have a channel to handle destruction.
					DestructionInfo = CreateDestructionInfo(ThisActor, DestructionInfo);
					if (DestructionInfo)
					{
						Connection->AddDestructionInfo(DestructionInfo);
					}
				}
			}

			Connection->NotifyActorDestroyed(ThisActor);
		}
	}

	if (ServerConnection)
	{
		ServerConnection->NotifyActorDestroyed(ThisActor);
	}

	NetworkObjects->OnActorDestroyed(ThisActor);

	// Remove this actor from the network object list
	RemoveNetworkActor( ThisActor );
}

void UNetDriver::NotifySubObjectDestroyed(UObject* SubObject)
{
	UE::Net::Private::FNetPropertyConditionManager::Get().NotifyObjectDestroyed(SubObject);
}

void UNetDriver::RemoveNetworkActor(AActor* Actor)
{
	LLM_SCOPE_BYTAG(NetDriver);
	UE_LOGF(LogNetLifecycle, Log, "[%ls] Deregistering for replication: Actor: %ls", *GetName(), *GetFullNameSafe(Actor));

	// This is a hack: Sometimes world streaming will stream out levels even though some players can still see them
	// Usually this is not a problem since the clients will remove the actors when it's told the level is gone. 
	// But when SerializeNewActorOverrideLevel is off, the client thinks the actors are in the main level, so they don't get removed.
	// If the client streams the world back in the actors start to double-up D: 
	// Ideally world streaming would never stream out worlds until all clients can't see them, but that change is too risky now. 
	// So for now, notify all the removed actors to make sure the client cleans them up. 
	if (IsServer() 
		&& !UE::Net::Private::SerializeNewActorOverrideLevel && UE::Net::Private::SendDormantDestructionOnRemoval
		&& IsValid(Actor) && !Actor->IsActorBeingDestroyed()
		&& IsValid(Actor->GetLevel()) && !Actor->GetLevel()->bIsVisible
		&& !Actor->IsNetStartupActor())
	{
		for (int32 i = ClientConnections.Num() - 1; i >= 0; i--)
		{
			UNetConnection* Connection = ClientConnections[i];
			if (Connection && Connection->ClientHasInitializedLevel(Actor->GetLevel()))
			{
				SendDestructionInfoForLevelUnloadIfDormant(Actor, Connection);
			}
		}
	}

	// Remove from renamed list if destroyed
	RenamedStartupActors.Remove(Actor->GetFName());

	if (ReplicationDriver)
	{
		ReplicationDriver->RemoveNetworkActor(Actor);
	}

	// In case a network actor is removed while the ConsiderList is in use, remove the stale entries.
	// This can happen when bReplicateTransactionally is true and ServerReplicateActors_ForConnection aborts.
	CurrentConsiderList.RemoveAllSwap([Actor](const FNetworkObjectInfo* Info)
		{
			return Info->Actor == Actor;
		},
		EAllowShrinking::No);

	GetNetworkObjectList().Remove(Actor);
}

void UNetDriver::RemoveInvalidNetworkActor(const AActor* BadActor)
{
	LLM_SCOPE_BYTAG(NetDriver);

	if (ReplicationDriver)
	{
		ensureMsgf(false, TEXT("Not calling UReplicationDriver::RemoveNetworkActor with bad actor pointer: %p"), BadActor);
	}

	// In case a network actor is removed while the ConsiderList is in use, remove the stale entries.
	// This can happen when bReplicateTransactionally is true and ServerReplicateActors_ForConnection aborts.
	CurrentConsiderList.RemoveAllSwap([BadActor](const FNetworkObjectInfo* Info)
		{
			return Info->Actor == BadActor;
		},
		EAllowShrinking::No);

	GetNetworkObjectList().RemoveInvalid(BadActor);
}

void UNetDriver::DeleteSubObjectOnClients(AActor* Actor, UObject* SubObject)
{
	if (ReplicationSystem)
	{
		if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
		{
			constexpr EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::Destroy | EEndReplicationFlags::DestroyNetHandle | EEndReplicationFlags::ClearNetPushId;
			Bridge->StopReplicatingNetObject(SubObject, EndReplicationFlags);
		}
	}
	else
	{
#if UE_REPLICATED_OBJECT_REFCOUNTING
		NetworkObjects->SetSubObjectForDeletion(Actor, SubObject);
#endif
	}
}

void UNetDriver::TearOffSubObjectOnClients(AActor* Actor, UObject* SubObject)
{
	if (ReplicationSystem)
	{
		if (UEngineReplicationBridge* Bridge = Cast<UEngineReplicationBridge>(ReplicationSystem->GetReplicationBridge()))
		{
			Bridge->StopReplicatingNetObject(SubObject, EEndReplicationFlags::TearOff);
		}
	}
	else
	{
#if UE_REPLICATED_OBJECT_REFCOUNTING
		NetworkObjects->SetSubObjectForTearOff(Actor, SubObject);
#endif
	}
}

void UNetDriver::NotifyActorRenamed(AActor* ThisActor, FName PreviousName)
{
	NotifyActorRenamed(ThisActor, nullptr, PreviousName);
}

void UNetDriver::NotifyActorRenamed(AActor* ThisActor, UObject* PreviousOuter, FName PreviousName)
{
	LLM_SCOPE_BYTAG(NetDriver);

	const bool bIsServer = IsServer();
	const bool bIsActorStatic = !GetNetGuidCache()->IsDynamicObject(ThisActor);
	const bool bActorHasRole = ThisActor->GetRemoteRole() != ROLE_None;

#if WITH_EDITOR
	// When recompiling and reinstancing a Blueprint, we rename the old actor out of the way, which would cause this code to emit a warning during PIE
	// Since that old actor is about to die (on both client and server) in the reinstancing case, it's safe to skip
	// We also want to skip when we are reconstructing Blueprint Instances as we dont want to store renaming as components are destroyed and created again
	if (GCompilingBlueprint || GIsReconstructingBlueprintInstances)
	{
		return;
	}
#endif

	if (bActorHasRole)
	{
		if (bIsActorStatic)
		{
			if (bIsServer)
			{
				FName OriginalName = RenamedStartupActors.FindRef(PreviousName);
				if (OriginalName != NAME_None)
				{
					PreviousName = OriginalName;
				}

				RenamedStartupActors.Add(ThisActor->GetFName(), PreviousName);

				UE_LOGF(LogNet, Log, "NotifyActorRenamed StartupActor: %ls PreviousName: %ls", *ThisActor->GetName(), *PreviousName.ToString());
			}
			else 
			{
				UE_LOGF(LogNet, Warning, "NotifyActorRenamed on client, StartupActor: %ls", *ThisActor->GetName());
			}
		}
		else
		{
			const bool bActorChangedOuter = ThisActor->GetOuter() != PreviousOuter;

			if (bIsServer && bActorChangedOuter)
			{
				UE_LOGF(LogNet, Log, "NotifyActorRenamed on server, dynamic actor changed outer: %ls PreviousOuter: %ls PreviousName: %ls", *GetFullNameSafe(ThisActor), *GetFullNameSafe(PreviousOuter), *PreviousName.ToString());

				// Forward change to ReplicationDriver so it can update its state
				if (ReplicationDriver)
				{
					ReplicationDriver->NotifyActorRenamed(ThisActor, PreviousOuter, PreviousName);
				}

				if (ReplicationSystem)
				{
					if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
					{
						Bridge->ActorChangedLevel(ThisActor, Cast<ULevel>(PreviousOuter));
					}
				}
				else if (UE::Net::Private::CleanUpRenamedDynamicActors)
				{
					// Close the channel or send destruction info for connections that don't have the new level visible.
					for (UNetConnection* Connection : ClientConnections)
					{
						if (!Connection->ClientHasInitializedLevel(ThisActor->GetLevel()))
						{
							UActorChannel* Channel = Connection->FindActorChannelRef(ThisActor);
							if (Channel)
							{
								ensureMsgf(Channel->OpenedLocally, TEXT("Channel for %s not opened locally on rename."), *GetFullNameSafe(ThisActor));

								UE_LOGF(LogNet, Verbose, "Closing channel for renamed actor: %ls to connection: %ls", *GetFullNameSafe(ThisActor), *Connection->Describe());
								Channel->Close(EChannelCloseReason::LevelUnloaded);
							}
							else
							{
								SendDestructionInfoForLevelUnloadIfDormant(ThisActor, Connection);
							}
						}
					}
				}
			}
		}
	}
}

// This method will be called when a Streaming Level is about to be Garbage Collected.
void UNetDriver::NotifyStreamingLevelUnload(ULevel* Level)
{
	if (IsServer())
	{
		for (AActor* Actor : Level->Actors)
		{
			if (Actor && Actor->IsNetStartupActor())
			{
				NotifyActorLevelUnloaded(Actor);
			}
		}

		if (ReplicationSystem)
		{
			ReplicationSystem->NotifyStreamingLevelUnload(Level);
		}
		
		TArray<FNetworkGUID> RemovedGUIDs;
		for (auto It = DestroyedStartupOrDormantActors.CreateIterator(); It; ++It)
		{
			FActorDestructionInfo* DestructInfo = It->Value.Get();
			if (DestructInfo->Level == Level && DestructInfo->NetGUID.IsStatic())
			{
				for (UNetConnection* Connection : ClientConnections)
				{
					Connection->RemoveDestructionInfo(DestructInfo);
				}

				RemovedGUIDs.Add(It->Key);
				It.RemoveCurrent();
			}
		}

		RemoveDestroyedGuidsByLevel(Level, RemovedGUIDs);
	}	

	if (Level->LevelScriptActor)
	{
		RemoveClassRepLayoutReferences(Level->LevelScriptActor->GetClass());
		ReplicationChangeListMap.Remove(Level->LevelScriptActor);
	}

	if (ServerConnection && ServerConnection->PackageMap)
	{
		UE_LOGF(LogNet, Verbose, "NotifyStreamingLevelUnload: %ls", *Level->GetFullName() );

		if (Level->LevelScriptActor)
		{
			UActorChannel * Channel = ServerConnection->FindActorChannelRef((AActor*)Level->LevelScriptActor);
			if (Channel)
			{
				UE_LOGF(LogNet, Log, "NotifyStreamingLevelUnload: BREAKING");

				Channel->BreakAndReleaseReferences();
			}
		}

		ServerConnection->PackageMap->NotifyStreamingLevelUnload(Level);
	}

	for( int32 i=ClientConnections.Num()-1; i>=0; i-- )
	{
		UNetConnection* Connection = ClientConnections[i];
		if (Connection && Connection->PackageMap)
		{
			Connection->PackageMap->NotifyStreamingLevelUnload(Level);
		}
	}
}

/** Called when an actor is being unloaded during a seamless travel or do due level streaming 
 *  The main point is that it calls the normal NotifyActorDestroyed to destroy the channel on the server
 *	but also removes the Actor reference, sets broken flag, and cleans up actor class references on clients.
 */
void UNetDriver::NotifyActorLevelUnloaded( AActor* TheActor )
{
	// server
	NotifyActorDestroyed(TheActor, true);
	// client
	if (ServerConnection != NULL)
	{
		// we can't kill the channel until the server says so, so just clear the actor ref and break the channel
		UActorChannel* Channel = ServerConnection->FindActorChannelRef(TheActor);
		if (Channel != NULL)
		{
			ServerConnection->RemoveActorChannel(TheActor);
			Channel->BreakAndReleaseReferences();
		}
	}
}

void UNetDriver::NotifyActorTearOff(AActor* Actor)
{
	if (ReplicationDriver)
	{
		ReplicationDriver->NotifyActorTearOff(Actor);
	}
	if (ReplicationSystem)
	{
		if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
		{
			// Set the actor to be torn-off during the next update of the replication system
			const UE::Net::FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(Actor);
			ReplicationSystem->TearOffNextUpdate(ActorRefHandle);
		}
	}
}

void UNetDriver::NotifyActorIsTraveling(AActor* TravelingActor)
{
	if (ReplicationSystem)
	{
		return;
	}

	NetworkObjects->OnActorIsTraveling(TravelingActor);
}

void UNetDriver::ForceNetUpdate(AActor* Actor)
{
	// Let Replication Driver handle it if it exists
	if (ReplicationDriver)
	{
		ReplicationDriver->ForceNetUpdate(Actor);
		return;
	}

	if (ReplicationSystem)
	{
		if (UObjectReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridge())
		{
			const UE::Net::FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(Actor);
			ReplicationSystem->ForceNetUpdate(ActorRefHandle);
		}
		return;
	}
	
	// Legacy implementation
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FNetworkObjectInfo* NetActor = FindNetworkObjectInfo(Actor))
	{
		NetActor->NextUpdateTime = World ? (World->TimeSeconds - 0.01f) : 0.0;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UNetDriver::ForceAllActorsNetUpdateTime(float NetUpdateTimeOffset, TFunctionRef<bool(const AActor* const)> ValidActorTestFunc)
{
	for (auto It = GetNetworkObjectList().GetAllObjects().CreateConstIterator(); It; ++It)
	{
		FNetworkObjectInfo* NetActorInfo = (*It).Get();
		if (NetActorInfo)
		{
			const AActor* const Actor = NetActorInfo->WeakActor.Get();
			if (Actor)
			{
				if (ValidActorTestFunc(Actor))
				{
					// Only allow the next update to be sooner than the current one
					const double NewUpdateTime = World->TimeSeconds + NetUpdateTimeOffset * FMath::FRand();
					NetActorInfo->NextUpdateTime = FMath::Min(NetActorInfo->NextUpdateTime, NewUpdateTime);
				}
			}
		}
	}
}

/** UNetDriver::FlushActorDormancy(AActor* Actor)
 *	 Flushes the actor from the NetDriver's dormant list and/or cancels pending dormancy on the actor channel.
 *
 *	 This does not change the Actor's actual NetDormant state. If a dormant actor is Flushed, it will net update at least one more
 *	 time, and then go back to dormant.
 */
void UNetDriver::FlushActorDormancy(AActor* Actor, bool bWasDormInitial)
{
	// Note: Going into dormancy is completely handled in ServerReplicateActor. We want to avoid
	// event-based handling of going into dormancy, because we have to deal with connections joining in progress.
	// It is better to have ::ServerReplicateActor check the AActor and UChannel's states to determined if an actor
	// needs to be moved into dormancy. The same amount of work will be done (1 time per connection when an actor goes dorm)
	// and we avoid having to do special things when a new client joins.
	//
	// Going out of dormancy can be event based like this since it only affects clients already joined. Its more efficient in this
	// way too, since we dont have to check every dormant actor in ::ServerReplicateActor to see if it needs to go out of dormancy

	if (GSetNetDormancyEnabled == 0)
		return;

	check(Actor);

	if (ReplicationDriver)
	{
		ReplicationDriver->FlushNetDormancy(Actor, bWasDormInitial);
	}

	if (ReplicationSystem)
	{
		UE::Net::FReplicationSystemUtil::FlushNetDormancy(ReplicationSystem, Actor, bWasDormInitial);
	}
	else
	{
		FlushActorDormancyInternal(Actor);
	}
}

void UNetDriver::NotifyActorDormancyChange(AActor* Actor, ENetDormancy OldDormancyState)
{
	if (ReplicationDriver)
	{
		ReplicationDriver->NotifyActorDormancyChange(Actor, OldDormancyState);
	}

	if (ReplicationSystem)
	{
		UE::Net::FReplicationSystemUtil::NotifyActorDormancyChange(ReplicationSystem, Actor, OldDormancyState);
	}
	else
	{
		FlushActorDormancyInternal(Actor);
	}
}

void UNetDriver::FlushActorDormancyInternal(AActor *Actor)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDriver_FlushActorDormancy);
	// Go through each connection and remove the actor from the dormancy list
	for (int32 i=0; i < ClientConnections.Num(); ++i)
	{
		UNetConnection *NetConnection = ClientConnections[i];
		if(NetConnection != NULL)
		{
			NetConnection->FlushDormancy(Actor);
		}
	}
}

void UNetDriver::ForcePropertyCompare( AActor* Actor )
{
#if WITH_SERVER_CODE
	check( Actor );

	for ( int32 i=0; i < ClientConnections.Num(); ++i )
	{
		UNetConnection *NetConnection = ClientConnections[i];
		if ( NetConnection != NULL )
		{
			NetConnection->ForcePropertyCompare( Actor );
		}
	}
#endif // WITH_SERVER_CODE
}

void UNetDriver::ForceActorRelevantNextUpdate(AActor* Actor)
{
#if WITH_SERVER_CODE
	check(Actor);
	
	GetNetworkObjectList().ForceActorRelevantNextUpdate(Actor, this);
#endif // WITH_SERVER_CODE
}

UChildConnection* UNetDriver::CreateChild(UNetConnection* Parent)
{
	return Parent->CreateChild(nullptr);
}

void UNetDriver::PostGarbageCollect()
{
	// We can't perform this logic in AddReferencedObjects because destroying GCObjects
	// during Garbage Collection is illegal (@see UGCObjectReferencer::RemoveObject).
	// FRepLayout's are GC objects, and either map could be holding onto the last reference
	// to a given RepLayout.

	for (auto It = RepLayoutMap.CreateIterator(); It; ++It)
	{
		if (!It.Key().IsValid())
		{
			It.RemoveCurrent();
		}
	}

	for (auto It = ReplicationChangeListMap.CreateIterator(); It; ++It)
	{
		if (!It.Key().GetWeakObjectPtr().IsValid())
		{
			It.RemoveCurrent();
		}
	}

	for (FObjectReplicator* Replicator : AllOwnedReplicators)
	{
		if (!Replicator->GetWeakObjectPtr().IsValid())
		{
			Replicator->ReleaseStrongReference();
		}
	}
}

void UNetDriver::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UNetDriver* This = CastChecked<UNetDriver>(InThis);
	Super::AddReferencedObjects(This, Collector);

	// TODO: It feels like we could get away without FRepLayout needing to track references.
	//			E.G., if we detected that FObjectReplicator / FReplicationChangelistMgrs detected
	//			their associated objects were destroyed, we could destroy the Shadow Buffers
	//			which should be the last thing referencing the FRepLayout and the Properties.
	for (TPair<TWeakObjectPtr<UObject>, TSharedPtr<FRepLayout>>& Pair : This->RepLayoutMap)
	{
		Pair.Value->AddReferencedObjects(Collector);
	}
	
	for (FObjectReplicator* Replicator : This->AllOwnedReplicators)
	{
		Collector.AddStableReference(&Replicator->ObjectClass);
	}

	for (auto& Pair : This->MappedClientConnections)
	{
		Collector.AddStableReference(&Pair.Value);
	}

	if (This->GetNetGuidCache().IsValid())
	{
		This->GetNetGuidCache()->CollectReferences(Collector);
	}
}


#if DO_ENABLE_NET_TEST

void UNetDriver::SetPacketSimulationSettings(const FPacketSimulationSettings& NewSettings)
{
	if (bNeverApplyNetworkEmulationSettings)
	{
		return;
	}

	// Once set this prevents driver changes from reloading the packet emulation settings
	bForcedPacketSettings = true;

	PacketSimulationSettings = NewSettings;
	OnPacketSimulationSettingsChanged();
}

void UNetDriver::OnPacketSimulationSettingsChanged()
{
	if (ServerConnection)
	{
		ServerConnection->UpdatePacketSimulationSettings();
	}
#if WITH_SERVER_CODE
	for (UNetConnection* ClientConnection : ClientConnections)
	{
		if (ClientConnection)
		{
			ClientConnection->UpdatePacketSimulationSettings();
		}
	}
#endif
}

#endif //DO_ENABLE_NET_TEST

FNetViewer::FNetViewer(UNetConnection* InConnection, float DeltaSeconds) :
	Connection(InConnection),
	InViewer(InConnection->PlayerController ? InConnection->PlayerController : InConnection->OwningActor),
	ViewTarget(InConnection->ViewTarget),
	ViewLocation(ForceInit),
	ViewDir(ForceInit)
{
	check(InConnection->OwningActor);
	check(!InConnection->PlayerController || (InConnection->PlayerController == InConnection->OwningActor));

	APlayerController* ViewingController = InConnection->PlayerController;

	// Get viewer coordinates.
	ViewLocation = ViewTarget->GetActorLocation();
	if (ViewingController)
	{
		FRotator ViewRotation = ViewingController->GetControlRotation();
		ViewingController->GetPlayerViewPoint(ViewLocation, ViewRotation);
		ViewDir = ViewRotation.Vector();
	}
}

FNetViewer::FNetViewer(AController* InController)
	: Connection(nullptr)
	, InViewer(InController)
	, ViewTarget(nullptr)
	, ViewLocation(ForceInit)
	, ViewDir(ForceInit)
{
	if (InController)
	{
		ViewTarget = InController->GetViewTarget();
		if (ViewTarget)
		{
			ViewLocation = ViewTarget->GetActorLocation();
		}

		FRotator ViewRotation = InController->GetControlRotation();
		InController->GetPlayerViewPoint(ViewLocation, ViewRotation);
		ViewDir = ViewRotation.Vector();
	}
}


FActorPriority::FActorPriority(UNetConnection* InConnection, UActorChannel* InChannel, FNetworkObjectInfo* InActorInfo, const TArray<struct FNetViewer>& Viewers, bool bLowBandwidth)
	: ActorInfo(InActorInfo), Channel(InChannel), DestructionInfo(NULL)
{	
	const float Time = Channel ? (InConnection->Driver->GetElapsedTime() - Channel->LastUpdateTime) : InConnection->Driver->SpawnPrioritySeconds;
	// take the highest priority of the viewers on this connection
	Priority = 0;
	for (int32 i = 0; i < Viewers.Num(); i++)
	{
		Priority = FMath::Max<int32>(Priority, FMath::RoundToInt(65536.0f * ActorInfo->Actor->GetNetPriority(Viewers[i].ViewLocation, Viewers[i].ViewDir, Viewers[i].InViewer, Viewers[i].ViewTarget, InChannel, Time, bLowBandwidth)));
	}
}

FActorPriority::FActorPriority(UNetConnection* InConnection, FActorDestructionInfo * Info, const TArray<FNetViewer>& Viewers )
	: ActorInfo(NULL), Channel(NULL), DestructionInfo(Info)
{
	
	Priority = 0;

	for (int32 i = 0; i < Viewers.Num(); i++)
	{
		float Time  = InConnection->Driver->SpawnPrioritySeconds;

		FVector Dir = DestructionInfo->DestroyedPosition - Viewers[i].ViewLocation;
		float DistSq = Dir.SizeSquared();
		
		// adjust priority based on distance and whether actor is in front of viewer
		if ( (Viewers[i].ViewDir | Dir) < 0.f )
		{
			if ( DistSq > NEARSIGHTTHRESHOLDSQUARED )
				Time *= 0.2f;
			else if ( DistSq > CLOSEPROXIMITYSQUARED )
				Time *= 0.4f;
		}
		else if ( DistSq > MEDSIGHTTHRESHOLDSQUARED )
			Time *= 0.4f;

		Priority = FMath::Max<int32>(Priority, 65536.0f * Time);
	}
}

namespace NetCmds
{
	static FAutoConsoleVariable MaxConnectionsToTickPerServerFrame( TEXT( "net.MaxConnectionsToTickPerServerFrame" ), 0, TEXT( "When non-zero, the maximum number of channels that will have changed replicated to them per server update" ) );
}

#if WITH_SERVER_CODE
int32 UNetDriver::ServerReplicateActors_PrepConnections( const float DeltaSeconds )
{
	int32 NumClientsToTick = ClientConnections.Num();

	// by default only throttle update for listen servers unless specified on the commandline
	static bool bForceClientTickingThrottle = FParse::Param( FCommandLine::Get(), TEXT( "limitclientticks" ) );
	if ( (bForceClientTickingThrottle || GetNetMode() == NM_ListenServer) && bTickingThrottleEnabled )
	{
		// determine how many clients to tick this frame based on GEngine->NetTickRate (always tick at least one client), double for lan play
		// FIXME: DeltaTimeOverflow is a static, and will conflict with other running net drivers, we investigate storing it on the driver itself!
		static float DeltaTimeOverflow = 0.f;
		// updates are doubled for lan play
		static bool LanPlay = FParse::Param( FCommandLine::Get(), TEXT( "lanplay" ) );
		//@todo - ideally we wouldn't want to tick more clients with a higher deltatime as that's not going to be good for performance and probably saturate bandwidth in hitchy situations, maybe 
		// come up with a solution that is greedier with higher framerates, but still won't risk saturating server upstream bandwidth
		float ClientUpdatesThisFrame = GEngine->NetClientTicksPerSecond * ( DeltaSeconds + DeltaTimeOverflow ) * ( LanPlay ? 2.f : 1.f );
		NumClientsToTick = FMath::Min<int32>( NumClientsToTick, FMath::TruncToInt( ClientUpdatesThisFrame ) );
		//UE_LOGF(LogNet, Log, "%2.3f: Ticking %d clients this frame, %2.3f/%2.4f",GetWorld()->GetTimeSeconds(),NumClientsToTick,DeltaSeconds,ClientUpdatesThisFrame);
		if ( NumClientsToTick == 0 )
		{
			// if no clients are ticked this frame accumulate the time elapsed for the next frame
			DeltaTimeOverflow += DeltaSeconds;
			return 0;
		}
		DeltaTimeOverflow = 0.f;
	}

	if( NetCmds::MaxConnectionsToTickPerServerFrame->GetInt() > 0 )
	{
		NumClientsToTick = FMath::Min( ClientConnections.Num(), NetCmds::MaxConnectionsToTickPerServerFrame->GetInt() );
	}

	bool bFoundReadyConnection = false;

	for ( int32 ConnIdx = 0; ConnIdx < ClientConnections.Num(); ConnIdx++ )
	{
		UNetConnection* Connection = ClientConnections[ConnIdx];
		check( Connection );
		check( Connection->GetConnectionState() == USOCK_Pending || Connection->GetConnectionState() == USOCK_Open || Connection->IsClosingOrClosed());
		checkSlow( Connection->GetUChildConnection() == NULL );

		// Handle not ready channels.
		//@note: we cannot add Connection->IsNetReady(0) here to check for saturation, as if that's the case we still want to figure out the list of relevant actors
		//			to reset their NetUpdateTime so that they will get sent as soon as the connection is no longer saturated
		AActor* OwningActor = Connection->OwningActor;
		if ( OwningActor != NULL && Connection->GetConnectionState() == USOCK_Open && ( Connection->Driver->GetElapsedTime() - Connection->LastReceiveTime < 1.5 ) )
		{
			check( World == OwningActor->GetWorld() );

			bFoundReadyConnection = true;

			// the view target is what the player controller is looking at OR the owning actor itself when using beacons
			AActor* DesiredViewTarget = OwningActor;
			if (Connection->PlayerController)
			{
				if (AActor* ViewTarget = Connection->PlayerController->GetViewTarget())
				{
					if (ViewTarget->GetWorld())
					{
						// It is safe to use the player controller's view target.
						DesiredViewTarget = ViewTarget;
					}
					else
					{
						// Log an error, since this means the view target for the player controller no longer has a valid world (this can happen
						// if the player controller's view target was in a sublevel instance that has been unloaded).
						UE_LOGF(LogNet, Warning, "Player controller %ls's view target (%ls) no longer has a valid world! Was it unloaded as part a level instance?",
							*Connection->PlayerController->GetName(), *ViewTarget->GetName());
					}
				}
			}
			Connection->ViewTarget = DesiredViewTarget;

			for ( int32 ChildIdx = 0; ChildIdx < Connection->Children.Num(); ChildIdx++ )
			{
				UNetConnection *Child = Connection->Children[ChildIdx];
				APlayerController* ChildPlayerController = Child->PlayerController;
				AActor* DesiredChildViewTarget = Child->OwningActor;

				if (ChildPlayerController)
				{
					AActor* ChildViewTarget = ChildPlayerController->GetViewTarget();

					if (ChildViewTarget && ChildViewTarget->GetWorld())
					{
						DesiredChildViewTarget = ChildViewTarget;
					}
				}

				Child->ViewTarget = DesiredChildViewTarget;
			}
		}
		else
		{
			Connection->ViewTarget = NULL;
			for ( int32 ChildIdx = 0; ChildIdx < Connection->Children.Num(); ChildIdx++ )
			{
				Connection->Children[ChildIdx]->ViewTarget = NULL;
			}
		}
	}

	return bFoundReadyConnection ? NumClientsToTick : 0;
}

void UNetDriver::ServerReplicateActors_BuildConsiderList( TArray<FNetworkObjectInfo*>& OutConsiderList, const float ServerTickTime )
{
	SCOPE_CYCLE_COUNTER( STAT_NetConsiderActorsTime );

	UE_LOGF( LogNetTraffic, Log, "ServerReplicateActors_BuildConsiderList, Building ConsiderList at WorldTime: %f ServerTickTime: %f", World->GetTimeSeconds(), ServerTickTime );

	int32 NumInitiallyDormant = 0;

	const bool bUseAdapativeNetFrequency = IsAdaptiveNetUpdateFrequencyEnabled();

	TArray<AActor*> ActorsToRemove;

	for ( const TSharedPtr<FNetworkObjectInfo>& ObjectInfo : GetNetworkObjectList().GetActiveObjects() )
	{
		FNetworkObjectInfo* ActorInfo = ObjectInfo.Get();

		if ( !ActorInfo->bPendingNetUpdate && World->TimeSeconds <= ActorInfo->NextUpdateTime )
		{
			UE_LOGF(LogNetTraffic, VeryVerbose, "Skipping actor: %ls. bPendingNetUpdate: %ls | NextUpdateTime: %f (currently %f)", *GetNameSafe(ActorInfo->Actor), ActorInfo->bPendingNetUpdate?TEXT("true"):TEXT("false"),ActorInfo->NextUpdateTime, World->TimeSeconds);
			continue;		// It's not time for this actor to perform an update, skip it
		}

		AActor* Actor = ActorInfo->Actor;

		if ( Actor->IsPendingKillPending() )
		{
			// Actors aren't allowed to be placed in the NetworkObjectList if they are PendingKillPending.
			// Actors should also be unconditionally removed from the NetworkObjectList when UWorld::DestroyActor is called.
			// If this is happening, it means code is not destructing Actors properly, and that's not OK.
			UE_LOGF( LogNet, Warning, "Actor %ls was found in the NetworkObjectList, but is PendingKillPending", *Actor->GetName() );
			ActorsToRemove.Add( Actor );
			continue;
		}

		if ( Actor->GetRemoteRole() == ROLE_None )
		{
			UE_LOGF(LogNetTraffic, VeryVerbose, "Skipping Actor %ls - it has no remote role!", *Actor->GetName());
			ActorsToRemove.Add( Actor );
			continue;
		}

		// This actor may belong to a different net driver, make sure this is the correct one
		// (this can happen when using beacon net drivers for example)
		if (Actor->GetNetDriverName() != NetDriverName)
		{
			UE_LOGF(LogNetTraffic, Error, "Actor %ls in wrong network actors list! (Has net driver '%ls', expected '%ls')",
					*Actor->GetName(), *Actor->GetNetDriverName().ToString(), *NetDriverName.ToString());

			continue;
		}

		// Verify the actor is actually initialized (it might have been intentionally spawn deferred until a later frame)
		if ( !Actor->IsActorInitialized() )
		{
			UE_LOGF(LogNetTraffic, VeryVerbose, "Skipping Actor %ls - it not initialized!", *Actor->GetName());
			continue;
		}

		// Don't send actors that may still be streaming in or out
		ULevel* Level = Actor->GetLevel();
		if ( Level->HasVisibilityChangeRequestPending() || Level->bIsAssociatingLevel )
		{
			UE_LOGF(LogNetTraffic, VeryVerbose, "Skipping Actor %ls - it has a pending visibility change!", *Actor->GetName());
			continue;
		}

		if ( IsDormInitialStartupActor(Actor) )
		{
			// This stat isn't that useful in its current form when using NetworkActors list
			// We'll want to track initially dormant actors some other way to track them with stats
			SCOPE_CYCLE_COUNTER( STAT_NetInitialDormantCheckTime );
			NumInitiallyDormant++;
			ActorsToRemove.Add( Actor );
			UE_LOGF(LogNetTraffic, VeryVerbose, "Skipping Actor %ls - its initially dormant!", *Actor->GetName() );
			continue;
		}

		checkSlow( Actor->NeedsLoadForClient() ); // We have no business sending this unless the client can load
		checkSlow( World == Actor->GetWorld() );

		// Set defaults if this actor is replicating for first time
		if ( ActorInfo->LastNetReplicateTime == 0 )
		{
			UE_LOGF(LogNetTraffic, VeryVerbose, "Setting Actor %ls - initial replication update time!", *Actor->GetName());
			ActorInfo->LastNetReplicateTime = World->TimeSeconds;
			ActorInfo->OptimalNetUpdateDelta = 1.0f / Actor->GetNetUpdateFrequency();
		}

		const float ScaleDownStartTime = 2.0f;
		const float ScaleDownTimeRange = 5.0f;

		const float LastReplicateDelta = World->TimeSeconds - ActorInfo->LastNetReplicateTime;

		if ( LastReplicateDelta > ScaleDownStartTime )
		{
			if ( Actor->GetMinNetUpdateFrequency() == 0.0f )
			{
				Actor->SetMinNetUpdateFrequency(2.0f);
			}

			// Calculate min delta (max rate actor will update), and max delta (slowest rate actor will update)
			const float MinOptimalDelta = 1.0f / Actor->GetNetUpdateFrequency();									  // Don't go faster than NetUpdateFrequency
			const float MaxOptimalDelta = FMath::Max( 1.0f / Actor->GetMinNetUpdateFrequency(), MinOptimalDelta ); // Don't go slower than MinNetUpdateFrequency (or NetUpdateFrequency if it's slower)

			// Interpolate between MinOptimalDelta/MaxOptimalDelta based on how long it's been since this actor actually sent anything
			const float Alpha = FMath::Clamp( ( LastReplicateDelta - ScaleDownStartTime ) / ScaleDownTimeRange, 0.0f, 1.0f );
			ActorInfo->OptimalNetUpdateDelta = FMath::Lerp( MinOptimalDelta, MaxOptimalDelta, Alpha );
		}

		// Setup ActorInfo->NextUpdateTime, which will be the next time this actor will replicate properties to connections
		// NOTE - We don't do this if bPendingNetUpdate is true, since this means we're forcing an update due to at least one connection
		//	that wasn't to replicate previously (due to saturation, etc)
		// NOTE - This also means all other connections will force an update (even if they just updated, we should look into this)
		if ( !ActorInfo->bPendingNetUpdate )
		{
			UE_LOGF( LogNetTraffic, Log, "actor %ls requesting new net update, time: %2.3f", *Actor->GetName(), World->TimeSeconds );

			const float NextUpdateDelta = bUseAdapativeNetFrequency ? ActorInfo->OptimalNetUpdateDelta : 1.0f / Actor->GetNetUpdateFrequency();

			float RandDelay = GNetDisableRandomNetUpdateDelay ? 0.0f : UpdateDelayRandomStream.FRand() * ServerTickTime;

			// then set the next update time
			ActorInfo->NextUpdateTime = World->TimeSeconds + RandDelay + NextUpdateDelta;

			// and mark when the actor first requested an update
			//@note: using ElapsedTime because it's compared against UActorChannel.LastUpdateTime which also uses that value
			ActorInfo->LastNetUpdateTimestamp = ElapsedTime;
		}

		// and clear the pending update flag assuming all clients will be able to consider it
		ActorInfo->bPendingNetUpdate = false;

		// add it to the list to consider below
		// For performance reasons, make sure we don't resize the array. It should already be appropriately sized above!
		ensure( OutConsiderList.Num() < OutConsiderList.Max() );
		OutConsiderList.Add( ActorInfo );

		// Call PreReplication on all actors that will be considered
		if (bCallPreReplication)
		{
			Actor->CallPreReplication(this);
		}
	}

	for ( AActor* Actor : ActorsToRemove )
	{
		RemoveNetworkActor( Actor );
	}

	// Update stats
	GetMetrics()->SetInt(UE::Net::Metric::NumInitiallyDormantActors, NumInitiallyDormant);
	GetMetrics()->SetInt(UE::Net::Metric::NumConsideredActors, OutConsiderList.Num());
}

// Returns true if this actor should replicate to *any* of the passed in connections
static FORCEINLINE_DEBUGGABLE bool IsActorRelevantToConnection( const AActor* Actor, const TArray<FNetViewer>& ConnectionViewers )
{
	for ( int32 viewerIdx = 0; viewerIdx < ConnectionViewers.Num(); viewerIdx++ )
	{
		if ( Actor->IsNetRelevantFor( ConnectionViewers[viewerIdx].InViewer, ConnectionViewers[viewerIdx].ViewTarget, ConnectionViewers[viewerIdx].ViewLocation ) )
		{
			return true;
		}
	}

	return false;
}

// Returns true if this actor is owned by, and should replicate to *any* of the passed in connections
static FORCEINLINE_DEBUGGABLE UNetConnection* IsActorOwnedByAndRelevantToConnection( const AActor* Actor, const TArray<FNetViewer>& ConnectionViewers, bool& bOutHasNullViewTarget )
{
	const AActor* ActorOwner = Actor->GetNetOwner();

	bOutHasNullViewTarget = false;

	for ( int i = 0; i < ConnectionViewers.Num(); i++ )
	{
		UNetConnection* ViewerConnection = ConnectionViewers[i].Connection;

		if ( ViewerConnection->ViewTarget == nullptr )
		{
			bOutHasNullViewTarget = true;
		}

		if ( ActorOwner == ViewerConnection->PlayerController ||
			 ( ViewerConnection->PlayerController && ActorOwner == ViewerConnection->PlayerController->GetPawn() ) ||
			 (ViewerConnection->ViewTarget && ViewerConnection->ViewTarget->IsRelevancyOwnerFor( Actor, ActorOwner, ViewerConnection->OwningActor ) ) )
		{
			return ViewerConnection;
		}
	}

	return nullptr;
}

// Returns true if this actor is considered dormant (and all properties caught up) to the current connection
static FORCEINLINE_DEBUGGABLE bool IsActorDormant( FNetworkObjectInfo* ActorInfo, const TWeakObjectPtr<UNetConnection>& Connection )
{
	// If actor is already dormant on this channel, then skip replication entirely
	return ActorInfo->DormantConnections.Contains( Connection );
}

// Returns true if this actor wants to go dormant for a particular connection
static FORCEINLINE_DEBUGGABLE bool ShouldActorGoDormant( AActor* Actor, const TArray<FNetViewer>& ConnectionViewers, UActorChannel* Channel, const float Time, const bool bLowNetBandwidth )
{
	if ( Actor->NetDormancy <= DORM_Awake || !Channel || Channel->bPendingDormancy || Channel->Dormant )
	{
		// Either shouldn't go dormant, or is already dormant
		return false;
	}

	if ( Actor->NetDormancy == DORM_DormantPartial )
	{
		for ( int32 viewerIdx = 0; viewerIdx < ConnectionViewers.Num(); viewerIdx++ )
		{
			if ( !Actor->GetNetDormancy( ConnectionViewers[viewerIdx].ViewLocation, ConnectionViewers[viewerIdx].ViewDir, ConnectionViewers[viewerIdx].InViewer, ConnectionViewers[viewerIdx].ViewTarget, Channel, Time, bLowNetBandwidth ) )
			{
				return false;
			}
		}
	}

	return true;
}

int32 UNetDriver::ServerReplicateActors_PrioritizeActors( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, const TArray<FNetworkObjectInfo*>& ConsiderList, const bool bCPUSaturated, FActorPriority*& OutPriorityList, FActorPriority**& OutPriorityActors )
{
	SCOPE_CYCLE_COUNTER( STAT_NetPrioritizeActorsTime );

	// Get list of visible/relevant actors.

	NetTag++;

	// Set up to skip all sent temporary actors
	for ( int32 j = 0; j < Connection->SentTemporaries.Num(); j++ )
	{
		Connection->SentTemporaries[j]->NetTag = NetTag;
	}

	// Make list of all actors to consider.
	check( World == Connection->OwningActor->GetWorld() );

	int32 FinalSortedCount = 0;
	int32 DeletedCount = 0;

	// Make weak ptr once for IsActorDormant call
	TWeakObjectPtr<UNetConnection> WeakConnection(Connection);

	const int32 MaxSortedActors = ConsiderList.Num() + DestroyedStartupOrDormantActors.Num();
	if ( MaxSortedActors > 0 )
	{
		OutPriorityList = new ( FMemStack::Get(), MaxSortedActors ) FActorPriority;
		OutPriorityActors = new ( FMemStack::Get(), MaxSortedActors ) FActorPriority*;

		check( World == Connection->ViewTarget->GetWorld() );

		AGameNetworkManager* const NetworkManager = World->NetworkManager;
		const bool bLowNetBandwidth = NetworkManager ? NetworkManager->IsInLowBandwidthMode() : false;

		for ( FNetworkObjectInfo* ActorInfo : ConsiderList )
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			// If we can run transactions entries in the ConsiderList might become invalid
			if (bReplicateTransactionally && !ActorInfo->WeakActor.IsValid())
			{
				continue;
			}
#endif

			AActor* Actor = ActorInfo->Actor;

			UActorChannel* Channel = Connection->FindActorChannelRef( ActorInfo->WeakActor );

			// Skip actor if not relevant and theres no channel already.
			// Historically Relevancy checks were deferred until after prioritization because they were expensive (line traces).
			// Relevancy is now cheap and we are dealing with larger lists of considered actors, so we want to keep the list of
			// prioritized actors low.
			if (!Channel)
			{
				if (!IsLevelInitializedForActor(Actor, Connection))
				{
					// If the level this actor belongs to isn't loaded on client, don't bother sending
					continue;
				}

				if (!IsActorRelevantToConnection(Actor, ConnectionViewers))
				{
					// If not relevant (and we don't have a channel), skip
					continue;
				}
			}

			UNetConnection* PriorityConnection = Connection;

			if ( Actor->bOnlyRelevantToOwner )
			{
				// This actor should be owned by a particular connection, see if that connection is the one passed in
				bool bHasNullViewTarget = false;

				PriorityConnection = IsActorOwnedByAndRelevantToConnection( Actor, ConnectionViewers, bHasNullViewTarget );

				if ( PriorityConnection == nullptr )
				{
					// Not owned by this connection, if we have a channel, close it, and continue
					// NOTE - We won't close the channel if any connection has a NULL view target.
					//	This is to give all connections a chance to own it
					if ( !bHasNullViewTarget && Channel != NULL && ElapsedTime - Channel->RelevantTime >= RelevantTimeout )
					{
						Channel->Close(EChannelCloseReason::Relevancy);
					}

					// This connection doesn't own this actor
					continue;
				}
			}
			else if ( GSetNetDormancyEnabled != 0 )
			{
				// Skip Actor if dormant
				if ( IsActorDormant( ActorInfo, WeakConnection ) )
				{
					continue;
				}

				// See of actor wants to try and go dormant
				if ( ShouldActorGoDormant( Actor, ConnectionViewers, Channel, ElapsedTime, bLowNetBandwidth ) )
				{
					CA_ASSUME(Channel); // ShouldActorGoDormant returns false if Channel is null, but analyzers don't seem to see that

					// Channel is marked to go dormant now once all properties have been replicated (but is not dormant yet)
					Channel->StartBecomingDormant();
				}
			}

			// Actor is relevant to this connection, add it to the list
			// NOTE - We use NetTag to make sure SentTemporaries didn't already mark this actor to be skipped
			if ( Actor->NetTag != NetTag )
			{
				UE_LOGF( LogNetTraffic, Log, "Consider %ls alwaysrelevant %d frequency %f ", *Actor->GetName(), Actor->bAlwaysRelevant, Actor->GetNetUpdateFrequency() );

				Actor->NetTag = NetTag;

				OutPriorityList[FinalSortedCount] = FActorPriority( PriorityConnection, Channel, ActorInfo, ConnectionViewers, bLowNetBandwidth );
				OutPriorityActors[FinalSortedCount] = OutPriorityList + FinalSortedCount;

				FinalSortedCount++;

#if NET_DEBUG_RELEVANT_ACTORS
				if ( DebugRelevantActors )
				{
					LastPrioritizedActors.Add( Actor );
				}
#endif // NET_DEBUG_RELEVANT_ACTORS
			}
		}

		// Add in deleted actors
		for ( auto It = Connection->GetDestroyedStartupOrDormantActorGUIDs().CreateConstIterator(); It; ++It )
		{
			FActorDestructionInfo& DInfo = *DestroyedStartupOrDormantActors.FindChecked( *It );
			OutPriorityList[FinalSortedCount] = FActorPriority( Connection, &DInfo, ConnectionViewers );
			OutPriorityActors[FinalSortedCount] = OutPriorityList + FinalSortedCount;
			FinalSortedCount++;
			DeletedCount++;
		}

		// Sort by priority
		Algo::SortBy(MakeArrayView(OutPriorityActors, FinalSortedCount), &FActorPriority::Priority, TGreater<>());
	}

	UE_LOGF( LogNetTraffic, Log, "ServerReplicateActors_PrioritizeActors: Potential %04i ConsiderList %03i FinalSortedCount %03i", MaxSortedActors, ConsiderList.Num(), FinalSortedCount );

	// Setup stats
	GetMetrics()->SetInt(UE::Net::Metric::PrioritizedActors, FinalSortedCount);
	GetMetrics()->SetInt(UE::Net::Metric::NumRelevantDeletedActors, DeletedCount);

	return FinalSortedCount;
}

int32 UNetDriver::ServerReplicateActors_ProcessPrioritizedActors(UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, FActorPriority** PriorityActors, const int32 FinalSortedCount, int32& OutUpdated )
{
	TInterval<int32> ActorsIndexRange(0, FinalSortedCount);
	return ServerReplicateActors_ProcessPrioritizedActorsRange(Connection, ConnectionViewers, PriorityActors, ActorsIndexRange, OutUpdated);
}

int32 UNetDriver::ServerReplicateActors_ProcessPrioritizedActorsRange( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, FActorPriority** PriorityActors, const TInterval<int32>& ActorsIndexRange, int32& OutUpdated, bool bIgnoreSaturation )
{
	SCOPE_CYCLE_COUNTER(STAT_NetProcessPrioritizedActorsTime);

	int32 ActorUpdatesThisConnection		= 0;
	int32 ActorUpdatesThisConnectionSent	= 0;
	int32 FinalRelevantCount				= 0;

	if (!Connection->IsNetReady() && !bIgnoreSaturation)
	{
		GNumSaturatedConnections++;
		// Connection saturated, don't process any actors
		return 0;
	}

	for ( int32 j = ActorsIndexRange.Min; j < ActorsIndexRange.Min + ActorsIndexRange.Max; j++ )
	{
		FNetworkObjectInfo*	ActorInfo = PriorityActors[j]->ActorInfo;

		// Deletion entry
		if ( ActorInfo == NULL && PriorityActors[j]->DestructionInfo )
		{
			// Make sure client has streaming level loaded
			if ( PriorityActors[j]->DestructionInfo->StreamingLevelName != NAME_None && !Connection->ClientVisibleLevelNames.Contains( PriorityActors[j]->DestructionInfo->StreamingLevelName ) )
			{
				// This deletion entry is for an actor in a streaming level the connection doesn't have loaded, so skip it
				continue;
			}

			FinalRelevantCount++;
			UE_LOGF( LogNetTraffic, Log, "Server replicate actor creating destroy channel for NetGUID <%ls,%ls> Priority: %d", *PriorityActors[j]->DestructionInfo->NetGUID.ToString(), *PriorityActors[j]->DestructionInfo->PathName, PriorityActors[j]->Priority );

			SendDestructionInfo(Connection, PriorityActors[j]->DestructionInfo);

			Connection->RemoveDestructionInfo( PriorityActors[j]->DestructionInfo );		// Remove from connections to-be-destroyed list (close bunch of reliable, so it will make it there)
			continue;
		}

#if !( UE_BUILD_SHIPPING || UE_BUILD_TEST )
		static IConsoleVariable* DebugObjectCvar = IConsoleManager::Get().FindConsoleVariable( TEXT( "net.PackageMap.DebugObject" ) );
		static IConsoleVariable* DebugAllObjectsCvar = IConsoleManager::Get().FindConsoleVariable( TEXT( "net.PackageMap.DebugAll" ) );
		if ( ActorInfo &&
			 ( ( DebugObjectCvar && !DebugObjectCvar->GetString().IsEmpty() && ActorInfo->Actor->GetName().Contains( DebugObjectCvar->GetString() ) ) ||
			   ( DebugAllObjectsCvar && DebugAllObjectsCvar->GetInt() != 0 ) ) )
		{
			UE_LOGF( LogNetPackageMap, Log, "Evaluating actor for replication %ls", *ActorInfo->Actor->GetName() );
		}
#endif

		// Normal actor replication
		UActorChannel* Channel = PriorityActors[j]->Channel;
		UE_LOGF( LogNetTraffic, Log, " Maybe Replicate %ls", ActorInfo ? *ActorInfo->Actor->GetName() : TEXT("None") );
		if ( !Channel || Channel->Actor ) //make sure didn't just close this channel
		{
			AActor* Actor = ActorInfo->Actor;
			bool bIsRelevant = false;

			const bool bLevelInitializedForActor = IsLevelInitializedForActor( Actor, Connection );

			// only check visibility on already visible actors every 1.0 + 0.5R seconds or every RelevantTimeout if it's lower then 1sec.
			// bTearOff actors should never be checked
			if ( bLevelInitializedForActor )
			{
				const float MinVisibilityTimeout = FMath::Min(RelevantTimeout, 1.0f);
				if ( !Actor->GetTearOff() && ( !Channel || ElapsedTime - Channel->RelevantTime >= MinVisibilityTimeout) )
				{
					if ( IsActorRelevantToConnection( Actor, ConnectionViewers ) )
					{
						bIsRelevant = true;
					}
#if NET_DEBUG_RELEVANT_ACTORS
					else if ( DebugRelevantActors )
					{
						LastNonRelevantActors.Add( Actor );
					}
#endif // NET_DEBUG_RELEVANT_ACTORS
				}
			}
			else
			{
				// Actor is no longer relevant because the world it is/was in is not loaded by client
				// exception: player controllers should never show up here
				UE_LOGF( LogNetTraffic, Log, "- Level not initialized for actor %ls", *Actor->GetName() );
			}

			// if the actor is now relevant or was recently relevant
			const bool bIsRecentlyRelevant = bIsRelevant || ( Channel && ElapsedTime - Channel->RelevantTime < RelevantTimeout ) || (ActorInfo->ForceRelevantFrame >= Connection->LastProcessedFrame);

			if ( bIsRecentlyRelevant )
			{
				FinalRelevantCount++;

				TOptional<FScopedActorRoleSwap> SwapGuard;
				if (ActorInfo->bSwapRolesOnReplicate || IsRoleSwappingOnAllActorsEnabled())
				{
					SwapGuard = FScopedActorRoleSwap(Actor);
				}

				// Find or create the channel for this actor.
				// we can't create the channel if the client is in a different world than we are
				// or the package map doesn't support the actor's class/archetype (or the actor itself in the case of serializable actors)
				// or it's an editor placed actor and the client hasn't initialized the level it's in
				if ( Channel == NULL && GetNetGuidCache()->SupportsObject( Actor->GetClass() ) && GetNetGuidCache()->SupportsObject( Actor->IsNetStartupActor() ? Actor : Actor->GetArchetype() ) )
				{
					if ( bLevelInitializedForActor )
					{
						// Create a new channel for this actor.
						Channel = (UActorChannel*)Connection->CreateChannelByName( NAME_Actor, EChannelCreateFlags::OpenedLocally );
						if ( Channel )
						{
							Channel->SetChannelActor(Actor, ESetChannelActorFlags::None);
						}
					}
					// if we couldn't replicate it for a reason that should be temporary, and this Actor is updated very infrequently, make sure we update it again soon
					else if ( Actor->GetNetUpdateFrequency() < 1.0f )
					{
						UE_LOGF( LogNetTraffic, Log, "Unable to replicate %ls", *Actor->GetName() );
						ActorInfo->NextUpdateTime = World->TimeSeconds + 0.2f * FMath::FRand();
					}
				}

				if ( Channel )
				{
					// if it is relevant then mark the channel as relevant for a short amount of time
					if ( bIsRelevant )
					{
						Channel->RelevantTime = ElapsedTime + 0.5 * UpdateDelayRandomStream.FRand();
					}
					// if the channel isn't saturated
					if ( Channel->IsNetReady() || bIgnoreSaturation)
					{
						// replicate the actor
						UE_LOGF( LogNetTraffic, Log, "- Replicate %ls. %d", *Actor->GetName(), PriorityActors[j]->Priority );

#if NET_DEBUG_RELEVANT_ACTORS
						if ( DebugRelevantActors )
						{
							LastRelevantActors.Add( Actor );
						}
#endif // NET_DEBUG_RELEVANT_ACTORS

						double ChannelLastNetUpdateTime = Channel->LastUpdateTime;

						if ( Channel->ReplicateActor() )
						{
#if USE_SERVER_PERF_COUNTERS
							// A channel time of 0.0 means this is the first time the actor is being replicated, so we don't need to record it
							if (ChannelLastNetUpdateTime > 0.0)
							{
								Connection->GetActorsStarvedByClassTimeMap().FindOrAdd(Actor->GetClass()->GetName()).Add((World->RealTimeSeconds - ChannelLastNetUpdateTime) * 1000.0f);
							}
#endif

							ActorUpdatesThisConnectionSent++;

#if NET_DEBUG_RELEVANT_ACTORS
							if ( DebugRelevantActors )
							{
								LastSentActors.Add( Actor );
							}
#endif // NET_DEBUG_RELEVANT_ACTORS

							// Calculate min delta (max rate actor will upate), and max delta (slowest rate actor will update)
							const float MinOptimalDelta				= 1.0f / Actor->GetNetUpdateFrequency();
							const float MaxOptimalDelta				= FMath::Max( 1.0f / Actor->GetMinNetUpdateFrequency(), MinOptimalDelta );
							const float DeltaBetweenReplications	= ( World->TimeSeconds - ActorInfo->LastNetReplicateTime );

							// Choose an optimal time, we choose 70% of the actual rate to allow frequency to go up if needed
							ActorInfo->OptimalNetUpdateDelta = FMath::Clamp( DeltaBetweenReplications * 0.7f, MinOptimalDelta, MaxOptimalDelta );
							ActorInfo->LastNetReplicateTime = World->TimeSeconds;
						}
						ActorUpdatesThisConnection++;
						OutUpdated++;
					}
					else
					{
						UE_LOGF( LogNetTraffic, Log, "- Channel saturated, forcing pending update for %ls", *Actor->GetName() );
						// otherwise force this actor to be considered in the next tick again
						Actor->ForceNetUpdate();
					}
					// second check for channel saturation
					if (!Connection->IsNetReady() && !bIgnoreSaturation)
					{
						// We can bail out now since this connection is saturated, we'll return how far we got though
						GNumSaturatedConnections++;
						return j;
					}
				}
			}

			// If the actor wasn't recently relevant, or if it was torn off, close the actor channel if it exists for this connection
			if ( ( !bIsRecentlyRelevant || Actor->GetTearOff() ) && Channel != NULL )
			{
				// Non startup (map) actors have their channels closed immediately, which destroys them.
				// Startup actors get to keep their channels open.

				// Fixme: this should be a setting
				if ( !bLevelInitializedForActor || !Actor->IsNetStartupActor() )
				{
					UE_LOGF( LogNetTraffic, Log, "- Closing channel for no longer relevant actor %ls", *Actor->GetName() );
					Channel->Close(Actor->GetTearOff() ? EChannelCloseReason::TearOff : EChannelCloseReason::Relevancy);
				}
			}
		}
	}

	return ActorsIndexRange.Max;
}

void UNetDriver::ServerReplicateActors_MarkRelevantActors( UNetConnection* Connection, const TArray<FNetViewer>& ConnectionViewers, int32 StartActorIndex, int32 EndActorIndex, FActorPriority** PriorityActors )
{
	// relevant actors that could not be processed this frame are marked to be considered for next frame
	for ( int32 k=StartActorIndex; k<EndActorIndex; k++ )
	{
		if (!PriorityActors[k]->ActorInfo)
		{
			// A deletion entry, skip it because we dont have anywhere to store a 'better give higher priority next time'
			continue;
		}
	
		AActor* Actor = PriorityActors[k]->ActorInfo->Actor;
	
		UActorChannel* Channel = PriorityActors[k]->Channel;
		
		UE_LOGF(LogNetTraffic, Verbose, "Saturated. %ls", *Actor->GetName());
		if (Channel != NULL && ElapsedTime - Channel->RelevantTime <= 1.0)
		{
			UE_LOGF(LogNetTraffic, Log, " Saturated. Mark %ls NetUpdateTime to be checked for next tick", *Actor->GetName());
			PriorityActors[k]->ActorInfo->bPendingNetUpdate = true;
		}
		else if ( IsActorRelevantToConnection( Actor, ConnectionViewers ) )
		{
			// If this actor was relevant but didn't get processed, force another update for next frame
			UE_LOGF( LogNetTraffic, Log, " Saturated. Mark %ls NetUpdateTime to be checked for next tick", *Actor->GetName() );
			PriorityActors[k]->ActorInfo->bPendingNetUpdate = true;
			if ( Channel != NULL )
			{
				Channel->RelevantTime = ElapsedTime + 0.5 * UpdateDelayRandomStream.FRand();
			}
		}
	
		// If the actor was forced to relevant and didn't get processed, try again on the next update;
		if (PriorityActors[k]->ActorInfo->ForceRelevantFrame >= Connection->LastProcessedFrame)
		{
			PriorityActors[k]->ActorInfo->ForceRelevantFrame = ReplicationFrame+1;
		}
	}
}

#endif

void UNetDriver::ServerReplicateActors_ForConnection(UE::Net::FServerReplicateActors_ForConnectionParams& Params)
{
#if WITH_SERVER_CODE
	const int32 LocalNumSaturated = GNumSaturatedConnections;

	auto ShouldUseConnectionAsViewer = [](UNetConnection* Connection)
	{
		if (Connection)
		{
			if (ANoPawnPlayerController* NoPawnPlayerController = Cast<ANoPawnPlayerController>(Connection->PlayerController))
			{
				return NoPawnPlayerController->IsEnableReplicationToConnection();
			}
		}

		return true;
	};

	// Make a list of viewers this connection should consider (this connection and children of this connection)
	Params.ConnectionViewers.Reset();

	if (ShouldUseConnectionAsViewer(Params.Connection))
	{
		new(Params.ConnectionViewers)FNetViewer(Params.Connection, Params.DeltaSeconds);
	}

	for (int32 ViewerIndex = 0; ViewerIndex < Params.Connection->Children.Num(); ViewerIndex++)
	{
		if (Params.Connection->Children[ViewerIndex]->ViewTarget != NULL)
		{
			if (ShouldUseConnectionAsViewer(Params.Connection->Children[ViewerIndex]))
			{
				new(Params.ConnectionViewers)FNetViewer(Params.Connection->Children[ViewerIndex], Params.DeltaSeconds);
			}
		}
	}

	GNumReplicationViewers += Params.ConnectionViewers.Num();

	// send ClientAdjustment if necessary
	// we do this here so that we send a maximum of one per packet to that client; there is no value in stacking additional corrections
	if (Params.Connection->PlayerController)
	{
		Params.Connection->PlayerController->SendClientAdjustment();
	}

	for (int32 ChildIdx = 0; ChildIdx < Params.Connection->Children.Num(); ChildIdx++)
	{
		if (Params.Connection->Children[ChildIdx]->PlayerController != NULL)
		{
			Params.Connection->Children[ChildIdx]->PlayerController->SendClientAdjustment();
		}
	}

	FMemMark RelevantActorMark(FMemStack::Get());

	const bool bProcessConsiderListIsBound = OnProcessConsiderListOverride.IsBound();

	if (bProcessConsiderListIsBound)
	{
		OnProcessConsiderListOverride.Execute({ Params.DeltaSeconds, Params.Connection, Params.bCPUSaturated }, Params.InOutUpdated, Params.ConsiderList);
	}

	if (!bProcessConsiderListIsBound)
	{
		FActorPriority* PriorityList = NULL;
		FActorPriority** PriorityActors = NULL;

		// Get a sorted list of actors for this connection
		const int32 FinalSortedCount = ServerReplicateActors_PrioritizeActors(Params.Connection, Params.ConnectionViewers, Params.ConsiderList, Params.bCPUSaturated, PriorityList, PriorityActors);

		// Process the sorted list of actors for this connection
		TInterval<int32> ActorsIndexRange(0, FinalSortedCount);
		const int32 LastProcessedActor = ServerReplicateActors_ProcessPrioritizedActorsRange(Params.Connection, Params.ConnectionViewers, PriorityActors, ActorsIndexRange, Params.InOutUpdated);

		ServerReplicateActors_MarkRelevantActors(Params.Connection, Params.ConnectionViewers, LastProcessedActor, FinalSortedCount, PriorityActors);
	}

	RelevantActorMark.Pop();

	Params.ConnectionViewers.Reset();

	Params.Connection->LastProcessedFrame = ReplicationFrame;

	const bool bWasSaturated = GNumSaturatedConnections > LocalNumSaturated;
	Params.Connection->TrackReplicationForAnalytics(bWasSaturated);
#endif
}

int64 UNetDriver::SendDestructionInfo(UNetConnection* Connection, FActorDestructionInfo* DestructionInfo)
{
	LLM_SCOPE_BYTAG(NetDriver);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetSendDestructionInfo);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(SendDestructionInfo);

	int64 NumBits = 0;

	checkf(Connection, TEXT("SendDestructionInfo called with invalid connection: %s"), *GetDescription());
	checkf(DestructionInfo, TEXT("SendDestructionInfo called with invalid destruction info: %s"), *GetDescription());

	UE_LOGF(LogNetTraffic, VeryVerbose, "Sending Destruction Info %ls", *GetDescription());

	FScopedRepContext RepContext(Connection, DestructionInfo->Level.Get());

	if (GNetControlChannelDestructionInfo != 0 && !Connection->IsReplay())
	{
		if (UControlChannel* ControlChan = Cast<UControlChannel>(Connection->Channels[0]))
		{
			NumBits = ControlChan->SendDestructionInfo(DestructionInfo);
		}
	}
	else
	{
		UActorChannel* Channel = (UActorChannel*)Connection->CreateChannelByName(NAME_Actor, EChannelCreateFlags::OpenedLocally);
		if (Channel)
		{
			// Send a close bunch on the new channel
			if(!Channel->Closing && (Connection->GetConnectionState() == USOCK_Open || Connection->GetConnectionState() == USOCK_Pending))
			{
				// Outer must be valid to call PackageMap->WriteObject. In the case of streaming out levels, this can go null out of from underneath us. In that case, just skip the destruct info.
				// We assume that if server unloads a level that clients will to and this will implicitly destroy all actors in it, so not worried about leaking actors client side here.
				if (UObject* ObjOuter = DestructionInfo->ObjOuter.Get())
				{
					// Send a close notify, and wait for ack.
					FOutBunch CloseBunch(Channel, true);
					check(!CloseBunch.IsError());
					check(CloseBunch.bClose);
					CloseBunch.bReliable = 1;
					CloseBunch.CloseReason = DestructionInfo->Reason;

					// Serialize DestructInfo
					NET_CHECKSUM(CloseBunch); // This is to mirror the Checksum in UPackageMapClient::SerializeNewActor
					Connection->PackageMap->WriteObject(CloseBunch, ObjOuter, DestructionInfo->NetGUID, DestructionInfo->PathName);

					UE_LOG(LogNetTraffic, Log, TEXT("SendDestructionInfo: Channel %d. NetGUID <%s> Path: %s. Bits: %" INT64_FMT), Channel->ChIndex, *DestructionInfo->NetGUID.ToString(), *DestructionInfo->PathName, CloseBunch.GetNumBits());
					UE_LOG(LogNetDormancy, Verbose, TEXT("SendDestructionInfo: Channel %d. NetGUID <%s> Path: %s. Bits: %" INT64_FMT), Channel->ChIndex, *DestructionInfo->NetGUID.ToString(), *DestructionInfo->PathName, CloseBunch.GetNumBits());

					constexpr bool bDontMerge = false;
					Channel->SendBunch(&CloseBunch, bDontMerge);

					NumBits = CloseBunch.GetNumBits();
				}
			}
		}
	}

	return NumBits;
}

void UNetDriver::ReportSyncLoad(const FNetSyncLoadReport& Report)
{
	if (Report.NetDriver != this)
	{
		return;
	}

	switch(Report.Type)
	{
		case ENetSyncLoadType::ActorSpawn:
		{
			UE_LOGF(LogNetSyncLoads, Log, "Spawning actor %ls caused a sync load of %ls!", *GetNameSafe(Report.OwningObject), *GetFullNameSafe(Report.LoadedObject));
			break;
		}

		case ENetSyncLoadType::PropertyReference:
		{
			UE_LOGF(LogNetSyncLoads, Log, "%ls of object %ls caused a sync load of %ls!", *GetFullNameSafe(Report.Property), *GetNameSafe(Report.OwningObject), *GetFullNameSafe(Report.LoadedObject));
			break;
		}

		case ENetSyncLoadType::Unknown:
		{
			UE_LOGF(LogNetSyncLoads, Log, "%ls was sync loaded by an unknown source.", *GetFullNameSafe(Report.LoadedObject));
			break;
		}

		default:
		{
			UE_LOGF(LogNetSyncLoads, Error, "%ls was sync loaded but its type %d isn't supported by UNetDriver::ReportSyncLoad.", *GetFullNameSafe(Report.LoadedObject), static_cast<int32>(Report.Type));
			break;
		}
	}
}


// -------------------------------------------------------------------------------------------------------------------------
//	Replication profiling (server cpu) helpers.
//		-CSV stat cateogry "Replication". Not enabled by default. (use -csvCategories=Replication to enable from cmd line)
//		-Auto CSV capturing can be enabled with -ReplicationCSVCaptureFrames=X. This will start a CSV capture once there
//			is one client connected and will auto terminate the process once X frames have elapsed.
// -------------------------------------------------------------------------------------------------------------------------


// Replication CSV category is always enabled (client and server).
CSV_DEFINE_CATEGORY(Replication, true);

double GReplicationGatherPrioritizeTimeSeconds;
double GServerReplicateActorTimeSeconds;

int32 GNumClientConnections;
int32 GNumClientUpdateLevelVisibility;

struct FReplicationAutoCapture
{
	int32 CaptureFrames=-1;
	int32 KillProcessFrames = -1;
	bool StartedCapture = false;

	void DoFrame()
	{
#if CSV_PROFILER_STATS
		if (CaptureFrames == -1)
		{
			// First time see if we want to auto capture
			CaptureFrames = 0;
			FParse::Value(FCommandLine::Get(), TEXT("ReplicationCSVCaptureFrames="), CaptureFrames);
		}
		
		if (CaptureFrames > 0)
		{			
			if (!StartedCapture)
			{
				StartedCapture = true;
				FCsvProfiler::Get()->EnableCategoryByString(TEXT("Replication"));
				FCsvProfiler::Get()->BeginCapture();
			}
				
			if (--CaptureFrames <= 0)
			{
				FCsvProfiler::Get()->EndCapture();
				KillProcessFrames = 60;
			}

		}
		else if (KillProcessFrames > 0)
		{
			// Kill process when finished
			--KillProcessFrames;
			if (KillProcessFrames == 0)
			{
				GLog->Flush();
				FPlatformMisc::RequestExit(true, TEXT("FReplicationAutoCapture"));
			}
		}
#endif
	}
};
FReplicationAutoCapture GReplicationAutoCapture;

#if CSV_PROFILER_STATS
struct FScopedNetDriverStats
{
	FScopedNetDriverStats(UNetDriver* InNetDriver) : NetDriver(InNetDriver)
	{
		GReplicationAutoCapture.DoFrame();

		// Set GReplicateActorTimingEnabled for this frame. (This will determine if actor channels do cycle counting while replicating)
		GReplicateActorTimingEnabled = ShouldEnableScopeSecondsTimers();
		if (GReplicateActorTimingEnabled)
		{
			GReplicateActorTimeSeconds = 0;
			GNumReplicateActorCalls = 0;
			GNumSaturatedConnections = 0;
			GNumSkippedObjectEmptyUpdates = 0;
			GNumReplicationViewers = 0;

			GReplicationGatherPrioritizeTimeSeconds = 0.f;
			GServerReplicateActorTimeSeconds = 0.f;

			// Whatever these values currently are were (mostly) set by RPCs (technically something else could have force ReplicateActor to be called but this is rare).
			InNetDriver->GetMetrics()->SetInt(UE::Net::Metric::SharedSerializationRPCHit, GNumSharedSerializationHit);
			InNetDriver->GetMetrics()->SetInt(UE::Net::Metric::SharedSerializationRPCMiss, GNumSharedSerializationMiss);

			const FNetworkObjectList& NetworkObjectList = NetDriver->GetNetworkObjectList();
			InNetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumberOfActiveActors, NetworkObjectList.GetActiveObjects().Num());
			InNetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumberOfFullyDormantActors, NetworkObjectList.GetDormantObjectsOnAllConnections().Num());

			StartTime = FPlatformTime::Seconds();
			StartOutBytes = GNetOutBytes;
		}
	}

	~FScopedNetDriverStats()
	{
		if (GReplicateActorTimingEnabled)
		{
			const double TotalTime = FPlatformTime::Seconds() - StartTime;

			GServerReplicateActorTimeSeconds = TotalTime;
			GReplicationGatherPrioritizeTimeSeconds = TotalTime - GReplicateActorTimeSeconds;

			uint32 FrameOutBytes = GNetOutBytes - StartOutBytes;

			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::ServerReplicateActorTimeMS, static_cast<float>(GServerReplicateActorTimeSeconds * 1000.0));
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumReplicatedActors, GNumReplicateActorCalls);
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::SatConnections, GNumSaturatedConnections);
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumSkippedObjectEmptyUpdates, GNumSkippedObjectEmptyUpdates);
			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::GatherPrioritizeTimeMS, static_cast<float>(GReplicationGatherPrioritizeTimeSeconds * 1000.0));
			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::ReplicateActorTimeMS, static_cast<float>(GReplicateActorTimeSeconds * 1000.0));
			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::NumReplicateActorCallsPerConAvg, (static_cast<float>(GNumReplicateActorCalls) / static_cast<float>(GNumClientConnections)));
			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::OutKBytes, static_cast<float>(FrameOutBytes) / 1024.f);
			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::OutNetGUIDKBytesSec, static_cast<float>(NetDriver->NetGUIDOutBytes) / 1024.f);
			NetDriver->GetMetrics()->SetFloat(UE::Net::Metric::NumClientUpdateLevelVisibility, static_cast<float>(GNumClientUpdateLevelVisibility));
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::SharedSerializationPropertyHit, GNumSharedSerializationHit);
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::SharedSerializationPropertyMiss, GNumSharedSerializationMiss);
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumReplicationViewers, GNumReplicationViewers);

			int32 NumOpenChannels = 0;
			int32 NumTickingChannels = 0;
			int32 MaxOutPackets = 0;
			int32 MaxInPackets = 0;

			for (const UNetConnection* ClientConnection : NetDriver->ClientConnections)
			{
				NumOpenChannels += ClientConnection->OpenChannels.Num();
				NumTickingChannels += ClientConnection->GetNumTickingChannels();
				MaxOutPackets = FMath::Max(MaxOutPackets, ClientConnection->OutPackets);
			}

			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumOpenChannels, NumOpenChannels);
			NetDriver->GetMetrics()->SetInt(UE::Net::Metric::NumTickingChannels, NumTickingChannels);

			// Note: we want to reset this at the end of the frame since the RPC stats are incremented at the top (recv)
			GNumSharedSerializationHit = 0;
			GNumSharedSerializationMiss = 0;
			GNumClientUpdateLevelVisibility = 0;
		}
	}
	
	double StartTime;
	uint32 StartOutBytes;
	UNetDriver* NetDriver;
};
#endif

// -------------------------------------------------------------------------------------------------------------------------
//	ServerReplicateActors: this is main function to replicate actors to client connections. It can be "outsourced" to a Replication Driver.
// -------------------------------------------------------------------------------------------------------------------------

int32 UNetDriver::ServerReplicateActors(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_NetServerRepActorsTime);

#if WITH_SERVER_CODE
	if ( ClientConnections.Num() == 0 )
	{
		return 0;
	}

	GetMetrics()->SetInt(UE::Net::Metric::NumReplicatedActors,0 );
	GetMetrics()->SetInt(UE::Net::Metric::NumReplicatedActorBytes, 0);

#if CSV_PROFILER_STATS
	FScopedNetDriverStats NetDriverStats(this);
	GNumClientConnections = ClientConnections.Num();
#endif
	
	if (ReplicationDriver)
	{
		return ReplicationDriver->ServerReplicateActors(DeltaSeconds);
	}

	check( World );

	// Bump the ReplicationFrame value to invalidate any properties marked as "unchanged" for this frame.
	ReplicationFrame++;

	int32 Updated = 0;

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	const bool bUseGranularTransactions = (ShouldReplicateTransactionally() && UE::Net::Private::bServerReplicateActorsGranularTransactions);
#else
	constexpr bool bUseGranularTransactions = false;
#endif

	const int32 NumClientsToTick = [this, DeltaSeconds, bUseGranularTransactions]()
		{
			if (bUseGranularTransactions)
			{
				int32 LocalNumClientsToTick = 0;
				static FName TransactionalWorkName("ServerReplicateActors_PrepConnections");
				UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, DeltaSeconds, &LocalNumClientsToTick]()
					{
						LocalNumClientsToTick = ServerReplicateActors_PrepConnections(DeltaSeconds);
					});
				return LocalNumClientsToTick;
			}
			else
			{
				return ServerReplicateActors_PrepConnections(DeltaSeconds);
			}
		}();


	if ( NumClientsToTick == 0 )
	{
		// No connections are ready this frame
		return 0;
	}

	AWorldSettings* WorldSettings = World->GetWorldSettings();

	bool bCPUSaturated		= false;
	float ServerTickTime	= GEngine->GetMaxTickRate( DeltaSeconds );
	if ( ServerTickTime == 0.f )
	{
		ServerTickTime = DeltaSeconds;
	}
	else
	{
		ServerTickTime	= 1.f/ServerTickTime;
		bCPUSaturated	= DeltaSeconds > 1.2f * ServerTickTime;
	}

	ensureMsgf(CurrentConsiderList.IsEmpty(), TEXT("UNetDriver::ServerReplicateActors: CurrentConsiderList isn't empty before starting replication."));

	CurrentConsiderList.Reserve( GetNetworkObjectList().GetActiveObjects().Num() );
	ON_SCOPE_EXIT
	{
		// Deallocate the array while the NetDriver doesn't need it.
		CurrentConsiderList.Empty();
	};

	// Build the consider list (actors that are ready to replicate)
	if (bUseGranularTransactions)
	{
		static const FName TransactionalWorkName("ServerReplicateActors_BuildConsiderList");
		UE::RemoteExecutor::ExecuteTransactional(TransactionalWorkName, [this, ServerTickTime]()
			{
				ServerReplicateActors_BuildConsiderList(CurrentConsiderList, ServerTickTime);
			});
	}
	else
	{
		ServerReplicateActors_BuildConsiderList(CurrentConsiderList, ServerTickTime);
	}

	TSet<UNetConnection*> ConnectionsToClose;

	FMemMark Mark( FMemStack::Get() );

	if (OnPreConsiderListUpdateOverride.IsBound())
	{
		OnPreConsiderListUpdateOverride.Execute({ DeltaSeconds, nullptr, bCPUSaturated }, Updated, CurrentConsiderList);
	}

	for ( int32 i=0; i < ClientConnections.Num(); i++ )
	{
		UNetConnection* Connection = ClientConnections[i];
		check(Connection);

		// net.DormancyValidate can be set to 2 to validate all dormant actors against last known state before going dormant
		if ( GNetDormancyValidate == 2 )
		{
			auto ValidateFunction = [](FObjectKey OwnerActorKey, FObjectKey ObjectKey, const TSharedRef<FObjectReplicator>& ReplicatorRef)
			{
				FObjectReplicator& Replicator = ReplicatorRef.Get();

				// We will call FObjectReplicator::ValidateAgainstState multiple times for
				// the same object (once for itself and again for each subobject).
				if (Replicator.OwningChannel != nullptr)
				{
					Replicator.ValidateAgainstState(Replicator.OwningChannel->GetActor());
				}
			};
				
			Connection->ExecuteOnAllDormantReplicators(ValidateFunction);
		}

		// if this client shouldn't be ticked this frame
		if (i >= NumClientsToTick)
		{
			//UE_LOGF(LogNet, Log, "skipping update to %ls",*Connection->GetName());
			// then mark each considered actor as bPendingNetUpdate so that they will be considered again the next frame when the connection is actually ticked
			for (int32 ConsiderIdx = 0; ConsiderIdx < CurrentConsiderList.Num(); ConsiderIdx++)
			{
				AActor* Actor = CurrentConsiderList[ConsiderIdx]->Actor;
				// if the actor hasn't already been flagged by another connection,
				if (Actor != NULL && !CurrentConsiderList[ConsiderIdx]->bPendingNetUpdate)
				{
					// find the channel
					UActorChannel* Channel = Connection->FindActorChannelRef(CurrentConsiderList[ConsiderIdx]->WeakActor);
					// and if the channel last update time doesn't match the last net update time for the actor
					if (Channel != NULL && Channel->LastUpdateTime < CurrentConsiderList[ConsiderIdx]->LastNetUpdateTimestamp)
					{
						//UE_LOGF(LogNet, Log, "flagging %ls for a future update",*Actor->GetName());
						// flag it for a pending update
						CurrentConsiderList[ConsiderIdx]->bPendingNetUpdate = true;
					}
				}
			}
			// clear the time sensitive flag to avoid sending an extra packet to this connection
			Connection->TimeSensitive = false;
		}
		else if (Connection->ViewTarget)
		{
			UE::Net::FServerReplicateActors_ForConnectionParams Params =
			{
				.Connection = Connection,
				.ConnectionViewers = WorldSettings->ReplicationViewers,
				.DeltaSeconds = DeltaSeconds,
				.ConsiderList = CurrentConsiderList,
				.InOutUpdated = Updated,
				.bCPUSaturated = bCPUSaturated
			};

			if (bUseGranularTransactions)
			{
				static const FName TransactionalWorkName("ServerReplicateActors_ForConnection");
				UE::RemoteExecutor::EnqueueWork(TransactionalWorkName, true, [this, ConstParams = Params]()
					{
						UE::Net::FServerReplicateActors_ForConnectionParams Params = ConstParams;
						ServerReplicateActors_ForConnection(Params);
					});
			}
			else
			{
				ServerReplicateActors_ForConnection(Params);
			}
		}

		if (Connection->GetPendingCloseDueToReplicationFailure())
		{
			ConnectionsToClose.Add(Connection);
		}
	}

	if (bUseGranularTransactions)
	{
		UE::RemoteExecutor::ExecutePendingWork();
	}

	if (OnPostConsiderListUpdateOverride.IsBound())
	{
		OnPostConsiderListUpdateOverride.ExecuteIfBound( { DeltaSeconds, nullptr, bCPUSaturated }, Updated, CurrentConsiderList);
	}

	// shuffle the list of connections if not all connections were ticked
	if (NumClientsToTick < ClientConnections.Num())
	{
		int32 NumConnectionsToMove = NumClientsToTick;
		while (NumConnectionsToMove > 0)
		{
			// move all the ticked connections to the end of the list so that the other connections are considered first for the next frame
			UNetConnection *Connection = ClientConnections[0];
			ClientConnections.RemoveAt(0,1);
			ClientConnections.Add(Connection);
			NumConnectionsToMove--;
		}
	}
	Mark.Pop();

#if NET_DEBUG_RELEVANT_ACTORS
	if (DebugRelevantActors)
	{
		PrintDebugRelevantActors();
		LastPrioritizedActors.Empty();
		LastSentActors.Empty();
		LastRelevantActors.Empty();
		LastNonRelevantActors.Empty();

		DebugRelevantActors  = false;
	}
#endif // NET_DEBUG_RELEVANT_ACTORS

	for (UNetConnection* ConnectionToClose : ConnectionsToClose)
	{
		ConnectionToClose->Close();
	}

	return Updated;
#else
	return 0;
#endif // WITH_SERVER_CODE
}

UChannel* UNetDriver::GetOrCreateChannelByName(const FName& ChName)
{
	UChannel* RetVal = nullptr;
	if (ChName == NAME_Actor && CVarActorChannelPool.GetValueOnAnyThread() != 0)
	{
		while (ActorChannelPool.Num() > 0 && RetVal == nullptr)
		{
			RetVal = ActorChannelPool.Pop();
			if (RetVal && RetVal->GetClass() != ChannelDefinitionMap[ChName].ChannelClass)
			{
				// Channel type Changed since this channel was added to the pool. Throw it away.
				RetVal->MarkAsGarbage();
				RetVal = nullptr;
			}
		}
		if (RetVal)
		{
			check(RetVal->GetClass() == ChannelDefinitionMap[ChName].ChannelClass);
			check(IsValid(RetVal));
			RetVal->bPooled = false;
		}
	}

	if (!RetVal)
	{
		RetVal = InternalCreateChannelByName(ChName);
	}

	return RetVal;
}

UChannel* UNetDriver::InternalCreateChannelByName(const FName& ChName)
{
	LLM_SCOPE_BYTAG(NetChannel);
	return NewObject<UChannel>(this, ChannelDefinitionMap[ChName].ChannelClass);
}

void UNetDriver::RegisterStatsListener(const FName MetricName, const FName StatName)
{
	FName ListenerKeyName = FName(FString::Printf(TEXT("NetworkMetricsStats_%s"), *StatName.ToString()));
	if (ensureMsgf(!NetworkMetricsListeners.Contains(ListenerKeyName), TEXT("Stat metrics listener %s already added to the database."), *StatName.ToString()))
	{
		UNetworkMetricsStats* Listener = NewObject<UNetworkMetricsStats>();
		Listener->SetStatName(StatName);
		Listener->SetInterval(StatPeriod);
		NetworkMetricsListeners.Add(ListenerKeyName, Listener);
		GetMetrics()->Register(MetricName, Listener);
	}
}

void UNetDriver::ResetNetworkMetrics()
{
	GetMetrics()->SetInt(UE::Net::Metric::OutgoingReliableMessageQueueMaxSize, 0);
	GetMetrics()->SetInt(UE::Net::Metric::IncomingReliableMessageQueueMaxSize, 0);
}

void UNetDriver::ReleaseToChannelPool(UChannel* Channel)
{
	LLM_SCOPE_BYTAG(NetChannel);
	check(IsValid(Channel));
	if (Channel->ChName == NAME_Actor && CVarActorChannelPool.GetValueOnAnyThread() != 0)
	{
		ActorChannelPool.Push(Channel);

		Channel->AddedToChannelPool();
	}
}

void UNetDriver::SetNetDriverName(FName NewNetDriverNamed)
{
	NetDriverName = NewNetDriverNamed;
}

void UNetDriver::SetNetDriverDefinition(FName NewNetDriverDefinition)
{
	NetDriverDefinition = NewNetDriverDefinition;

	if (const FNetDriverDefinition* DriverDef = GEngine->NetDriverDefinitions.FindByPredicate([this](const FNetDriverDefinition& Def) { return (Def.DefName == NetDriverDefinition); }))
	{
		MaxChannelsOverride = DriverDef->MaxChannelsOverride;
	}

	InitPacketSimulationSettings();
}

void UNetDriver::ReinitBase()
{
	UE_LOGF(LogNet, Log, "Re-InitBase %ls (NetDriverDefinition %ls) using replication model %ls", *NetDriverName.ToString(), *NetDriverDefinition.ToString(), *GetReplicationModelName());
}

void UNetDriver::PostCreation(bool bInitializeWithIris, bool bEnableParallelNetTick)
{
	bIsUsingIris = bInitializeWithIris;
	bIsUsingParallelNetTick = bEnableParallelNetTick;

	if (NetDriverDefinition == NAME_GameNetDriver)
	{
		//Add to CSV whether we're using Iris on the GameNetDriver or not
		CSV_METADATA(TEXT("Iris"), IsUsingIrisReplication() ? TEXT("1") : TEXT("0"));
	}
}

#if NET_DEBUG_RELEVANT_ACTORS
void UNetDriver::PrintDebugRelevantActors()
{
	struct SLocal
	{
		static void AggregateAndPrint( TArray< TWeakObjectPtr<AActor> >	&List, FString txt )
		{
			TMap< TWeakObjectPtr<UClass>, int32>	ClassSummary;
			TMap< TWeakObjectPtr<UClass>, int32>	SuperClassSummary;

			for (auto It = List.CreateIterator(); It; ++It)
			{
				if (AActor* Actor = It->Get())
				{

					ClassSummary.FindOrAdd(Actor->GetClass())++;
					if (Actor->GetClass()->GetSuperStruct())
					{
						SuperClassSummary.FindOrAdd( Actor->GetClass()->GetSuperClass() )++;
					}
				}
			}

			struct FCompareActorClassCount
			{
				FORCEINLINE bool operator()( int32 A, int32 B ) const
				{
					return A < B;
				}
			};


			ClassSummary.ValueSort(FCompareActorClassCount());
			SuperClassSummary.ValueSort(FCompareActorClassCount());

			UE_LOGF(LogNet, Warning, "------------------------------" );
			UE_LOGF(LogNet, Warning, " %ls Class Summary", *txt );
			UE_LOGF(LogNet, Warning, "------------------------------" );

			for (auto It = ClassSummary.CreateIterator(); It; ++It)
			{
				UE_LOGF(LogNet, Warning, "%4d - %ls (%ls)", It.Value(), *It.Key()->GetName(), It.Key()->GetSuperStruct() ? *It.Key()->GetSuperStruct()->GetName() : TEXT("NULL") );
			}

			UE_LOGF(LogNet, Warning, "---------------------------------" );
			UE_LOGF(LogNet, Warning, " %ls Parent Class Summary ", *txt );
			UE_LOGF(LogNet, Warning, "------------------------------" );

			for (auto It = SuperClassSummary.CreateIterator(); It; ++It)
			{
				UE_LOGF(LogNet, Warning, "%4d - %ls (%ls)", It.Value(), *It.Key()->GetName(), It.Key()->GetSuperStruct() ? *It.Key()->GetSuperStruct()->GetName() : TEXT("NULL") );
			}

			UE_LOGF(LogNet, Warning, "---------------------------------" );
			UE_LOGF(LogNet, Warning, " %ls Total: %d", *txt, List.Num() );
			UE_LOGF(LogNet, Warning, "---------------------------------" );
		}
	};

	SLocal::AggregateAndPrint( LastPrioritizedActors, TEXT(" Prioritized Actor") );
	SLocal::AggregateAndPrint( LastRelevantActors, TEXT(" Relevant Actor") );
	SLocal::AggregateAndPrint( LastNonRelevantActors, TEXT(" NonRelevant Actor") );
	SLocal::AggregateAndPrint( LastSentActors, TEXT(" Sent Actor") );

	UE_LOGF(LogNet, Warning, "---------------------------------" );
	UE_LOGF(LogNet, Warning, " Num Connections: %d", ClientConnections.Num() );

	UE_LOGF(LogNet, Warning, "---------------------------------" );
}
#endif // NET_DEBUG_RELEVANT_ACTORS

void UNetDriver::DrawNetDriverDebug()
{
#if ENABLE_DRAW_DEBUG
	UNetConnection *Connection = (ServerConnection ? ToRawPtr(ServerConnection) : (ClientConnections.Num() >= 1 ? ToRawPtr(ClientConnections[0]) : NULL));
	if (!Connection)
	{
		return;
	}

	UWorld* LocalWorld = GetWorld();
	if (!LocalWorld)
	{
		return;
	}

	if (!IsInGameThread())
	{
		return;
	}

	ULocalPlayer*	LocalPlayer = NULL;
	if(FLocalPlayerIterator It(GEngine, LocalWorld);It)
	{
		LocalPlayer = *It;
	}
	if (!LocalPlayer)
	{
		return;
	}

	const float MaxDebugDrawDistanceSquare = FMath::Square(CVarNetDebugDrawCullDistance.GetValueOnAnyThread());

	const FVector Extent(20.f);
	// Used to draw additional boxes to show more state
	const FBox BoxExpansion = FBox(&Extent, 1);

	const UObjectReplicationBridge* Bridge = ReplicationSystem ? ReplicationSystem->GetReplicationBridge() : nullptr;

	for (FActorIterator It(LocalWorld); It; ++It)
	{
		// Skip drawing non-replicated actors and controllers since their bounds are
		// bogus and represented by the Character/Pawn
		if (!It->GetIsReplicated() || It->IsA(AController::StaticClass()) || It->IsA(AInfo::StaticClass()))
		{
			continue;
		}

		FBox Box = It->GetComponentsBoundingBox();
		// Skip actors without bounds in the world
		if (Box.GetExtent().IsNearlyZero())
		{
			continue;
		}

		FColor ExtraStateDrawColor;

		bool bWasCulled = false;
		const bool bIsAlwaysRelevant = It->bAlwaysRelevant;
		// Determine if we need to show that it was network culled or not
		if (!bIsAlwaysRelevant && !It->bOnlyRelevantToOwner)
		{
			const float DistanceSquared = (It->GetActorLocation() - LocalPlayer->LastViewLocation).SizeSquared();
			// Check for culling from this view based upon the CVar
			if (MaxDebugDrawDistanceSquare > 0.f && MaxDebugDrawDistanceSquare < DistanceSquared)
			{
				continue;
			}

			if (DistanceSquared > It->GetNetCullDistanceSquared())
			{
				bWasCulled = true;
				ExtraStateDrawColor = FColor::White;
			}
		}
		else if (bIsAlwaysRelevant)
		{
			ExtraStateDrawColor = FColor::Red;
		}
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FNetworkObjectInfo* NetworkObjectInfo = Connection->Driver->FindNetworkObjectInfo( *It );
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FColor DrawColor = FColor::White;
		if ( NetworkObjectInfo && NetworkObjectInfo->DormantConnections.Contains( Connection ) )
		{
			DrawColor = FColor::Blue;
		}
		else if (Connection->FindActorChannelRef(*It) != NULL)
		{
			DrawColor = FColor::Green;
		}
		else
		{
			// A replicated actor that is not on a channel and not dormant
			DrawColor = FColor::Orange;
		}

		// Draw NetGUID or NetHandle
		FVector AboveActor(0.f, 0.f, 32.f);
#if !UE_BUILD_SHIPPING
		if (Bridge)
		{
			AActor* Actor = *It;
			const UE::Net::FNetRefHandle ActorRefHandle = Bridge->GetReplicatedRefHandle(Actor);
			if (ActorRefHandle.IsValid() && UE::Net::IrisDebugHelper::FilterDebuggedObject(Actor))
			{
				DrawColor = FColor::Green;
				DrawDebugString(Actor->GetWorld(), AboveActor, ActorRefHandle.ToString(), Actor, FColor::White, 0.f);
			}
			else
			{
				continue;
			}
		}
		else
		{
			FNetworkGUID NetGUID = Connection->PackageMap->GetNetGUIDFromObject(*It);
			if (NetGUID.IsValid())
			{
				DrawDebugString(LocalWorld, AboveActor, NetGUID.ToString(), *It, FColor::White, 0.f);
			}
		}
#endif

		DrawDebugBox(LocalWorld, Box.GetCenter(), Box.GetExtent(), FQuat::Identity, DrawColor, false);
		const bool bDrawSecondBox = bWasCulled || bIsAlwaysRelevant;
		// Draw a second box around the object if it was net culled
		if (bDrawSecondBox)
		{
			Box += BoxExpansion;
			DrawDebugBox(LocalWorld, Box.GetCenter(), Box.GetExtent(), FQuat::Identity, ExtraStateDrawColor, false);
		}
	}

	// Draw state
#if !UE_BUILD_SHIPPING
	if (Bridge)
	{
		UE::Net::FNetRefHandle DebugRefHandle = UE::Net::IrisDebugHelper::GetDebugNetRefHandle();
		if (DebugRefHandle.IsValid() && Bridge)
		{
			UE::Net::FNetRefHandle RootObjectHandle = Bridge->GetRootObjectOfSubObject(DebugRefHandle);
			UObject* Object = Bridge->GetReplicatedObject(RootObjectHandle.IsValid() ? RootObjectHandle : DebugRefHandle);
			if (AActor* Actor = Cast<AActor>(Object))
			{
				DrawDebugString(LocalWorld, FVector(0.f, 0.f, 0.f), UE::Net::IrisDebugHelper::DebugNetObjectStateToString(DebugRefHandle.GetId(), ReplicationSystem->GetId()), Actor, FColor::White, 0.f);
			}
		}
	}
#endif // !UE_BUILD_SHIPPING


#endif
}

bool UNetDriver::NetObjectIsDynamic(const UObject *Object) const
{
	const UActorComponent *ActorComponent = Cast<const UActorComponent>(Object);
	if (ActorComponent)
	{
		// Actor components are dynamic if their owning actor is.
		return NetObjectIsDynamic(Object->GetOuter());
	}

	const AActor *Actor = Cast<const AActor>(Object);
	if (!Actor || Actor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || Actor->IsNetStartupActor())
	{
		return false;
	}

	return true;
}

void UNetDriver::AddClientConnection(UNetConnection* NewConnection)
{
	LLM_SCOPE_BYTAG(NetDriver);

	SCOPE_CYCLE_COUNTER(Stat_NetDriverAddClientConnection);

	UE_CLOGF(!DDoS.CheckLogRestrictions(), LogNet, Log, "AddClientConnection: Added client connection: %ls", *NewConnection->Describe());

	ClientConnections.Add(NewConnection);

	TSharedPtr<const FInternetAddr> ConnAddr = NewConnection->GetRemoteAddr();

	if (ConnAddr.IsValid())
	{
		MappedClientConnections.Add(ConnAddr.ToSharedRef(), NewConnection);

		// On the off-chance of the same IP:Port being reused, check RecentlyDisconnectedClients
		int32 RecentDisconnectIdx = RecentlyDisconnectedClients.IndexOfByPredicate(
			[&ConnAddr](const FDisconnectedClient& CurElement)
			{
				return *ConnAddr == *CurElement.Address;
			});

		if (RecentDisconnectIdx != INDEX_NONE)
		{
			RecentlyDisconnectedClients.RemoveAt(RecentDisconnectIdx);
		}
	}

	if (ReplicationDriver)
	{
		ReplicationDriver->AddClientConnection(NewConnection);
	}

#if USE_SERVER_PERF_COUNTERS
	GetMetrics()->IncrementInt(UE::Net::Metric::AddedConnections, 1);
#endif

	CreateInitialServerChannels(NewConnection);

	// When new connections join, we need to make sure to add all fully dormant actors back to the network list, so they can get processed for the new connection
	// They'll eventually fall back off to this list when they are dormant on the new connection
	GetNetworkObjectList().HandleConnectionAdded();

	for (auto It = DestroyedStartupOrDormantActors.CreateIterator(); It; ++It)
	{
		if (It.Key().IsStatic())
		{
			UE_LOGF(LogNet, VeryVerbose, "Adding actor NetGUID <%ls> to new connection's destroy list", *It.Key().ToString());
			NewConnection->AddDestructionInfo(It.Value().Get());
		}
	}

	if (!bHasReplayConnection && NewConnection->IsReplay())
	{
		bHasReplayConnection = true;
	}

	UpdateCrashContext();
}

void UNetDriver::ForEachClientConnection(TFunction<bool(TObjectPtr<UNetConnection>)> Func)
{
	for (TObjectPtr<UNetConnection> ParentConnection : ClientConnections)
	{
		if (Func(ParentConnection))
		{
			return;
		}
		
		for (TObjectPtr<UNetConnection> ChildConnection : ParentConnection->Children)
		{
			if (Func(ChildConnection))
			{
				return;
			}
		}
	}
}

void UNetDriver::UpdateCrashContext(ECrashContextUpdate UpdateType)
{
	if (NetDriverDefinition != NAME_GameNetDriver)
	{
		return;
	}

	int32 NumClients = 0;

	for (const UNetConnection* Connection : ClientConnections)
	{
		if (Connection && !Connection->IsReplay())
		{
			++NumClients;
		}
	}

	static FString Attrib_NumClients = "NumClients";
	FGenericCrashContext::SetEngineData(Attrib_NumClients, LexToString(NumClients));

	static FString CrashContext_ReplicationModel = "ReplicationModel";

	if (UpdateType == ECrashContextUpdate::UpdateRepModel)
	{		
		FGenericCrashContext::SetEngineData(CrashContext_ReplicationModel, *GetReplicationModelName());
	}
	else if (UpdateType == ECrashContextUpdate::ClearRepModel)
	{
		FGenericCrashContext::SetEngineData(CrashContext_ReplicationModel, TEXT(""));
	}
}

void UNetDriver::CreateReplicatedStaticActorDestructionInfo(ULevel* Level, const FReplicatedStaticActorDestructionInfo& Info)
{
	check(Level);

	if (ReplicationSystem)
	{
		if (UEngineReplicationBridge* Bridge = ReplicationSystem->GetReplicationBridgeAs<UEngineReplicationBridge>())
		{
			// Add explicit destruction info for this object
			const UE::Net::FDestructionParameters Params
			{
				.Location = Info.DestroyedPosition,
				.Container = Level,				
				.bUseDistanceBasedPrioritization = true,
				.NetFactoryId = Bridge->GetActorFactoryId()
			};

			Bridge->AddStaticDestructionInfo(Info.PathName.ToString(), Info.ObjOuter.Get(), Params);
		}
		return;
	}

	FNetworkGUID NetGUID = GetNetGuidCache()->AssignNewNetGUIDFromPath_Server(Info.PathName.ToString(), Info.ObjOuter.Get(), Info.ObjClass);
	if (NetGUID.IsDefault())
	{
		UE_LOGF(LogNet, Error, "CreateReplicatedStaticActorDestructionInfo got an invalid NetGUID for %ls", *Info.PathName.ToString());
		return;
	}

	UE_LOGF(LogNet, VeryVerbose, "CreateReplicatedStaticActorDestructionInfo %ls %ls %ls %ls", *GetName(), *Level->GetName(), *Info.PathName.ToString(), *NetGUID.ToString());
	
	TUniquePtr<FActorDestructionInfo>& NewInfoPtr = DestroyedStartupOrDormantActors.FindOrAdd(NetGUID);
	if (NewInfoPtr.IsValid() == false)
	{
		NewInfoPtr = MakeUnique<FActorDestructionInfo>();
	}
	
	FActorDestructionInfo &NewInfo = *NewInfoPtr;
	NewInfo.DestroyedPosition = Info.DestroyedPosition;
	NewInfo.NetGUID = NetGUID;
	NewInfo.Level = Level;
	NewInfo.ObjOuter = Info.ObjOuter.Get();
	NewInfo.PathName = Info.PathName.ToString();

	// Look for renamed actor now so we can clear it after the destroy is queued
	FName RenamedPath = RenamedStartupActors.FindRef(Info.PathName);
	if (RenamedPath != NAME_None)
	{
		NewInfo.PathName = RenamedPath.ToString();
	}

	if (NewInfo.Level.IsValid() && !NewInfo.Level->IsPersistentLevel() )
	{
		NewInfo.StreamingLevelName = NewInfo.Level->GetOutermost()->GetFName();
	}
	else
	{
		NewInfo.StreamingLevelName = NAME_None;
	}

	TSet<FNetworkGUID>& DestroyedGuidsForLevel = DestroyedStartupOrDormantActorsByLevel.FindOrAdd(NewInfo.StreamingLevelName);
	DestroyedGuidsForLevel.Add(NetGUID);

	NewInfo.Reason = EChannelCloseReason::Destroyed;
}

void UNetDriver::InitDestroyedStartupActors()
{
	LLM_SCOPE_BYTAG(NetDriver);

	if (World)
	{
		// add startup actors destroyed before the creation of this net driver
		for (auto LevelIt(World->GetLevelIterator()); LevelIt; ++LevelIt)
		{
			ULevel* Level = *LevelIt;
			if (Level)
			{
				const TArray<FReplicatedStaticActorDestructionInfo>& DestroyedReplicatedStaticActors = Level->GetDestroyedReplicatedStaticActors();
				for(const FReplicatedStaticActorDestructionInfo& CurInfo : DestroyedReplicatedStaticActors)
				{
					CreateReplicatedStaticActorDestructionInfo(Level, CurInfo);
				}
			}
		}
	}
}

void UNetDriver::NotifyActorClientDormancyChanged(AActor* Actor, ENetDormancy OldDormancyState)
{
	if (IsServer())
	{
		AddNetworkActor(Actor);
		NotifyActorDormancyChange(Actor, OldDormancyState);
	}
}

void UNetDriver::ClientSetActorDormant(AActor* Actor)
{
	const bool bIsServer = IsServer();

	if (Actor && !bIsServer)
	{
		ENetDormancy OldDormancy = Actor->NetDormancy;

		Actor->NetDormancy = DORM_DormantAll;

		if (World)
		{
			if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(World))
			{
				for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr && Driver.NetDriver != this && Driver.NetDriver->ShouldReplicateActor(Actor))
					{
						Driver.NetDriver->NotifyActorClientDormancyChanged(Actor, OldDormancy);
					}
				}
			}
		}
	}
}

void UNetDriver::NotifyActorFullyDormantForConnection(AActor* Actor, UNetConnection* Connection)
{
	GetNetworkObjectList().MarkDormant(Actor, Connection, IsServer() ? ClientConnections.Num() : 1, this);
	if (ReplicationDriver)
	{
		ReplicationDriver->NotifyActorFullyDormantForConnection(Actor, Connection);
	}
}

void UNetDriver::RemoveClientConnection(UNetConnection* ClientConnectionToRemove)
{
	check(ClientConnectionToRemove);
	verify(ClientConnections.Remove(ClientConnectionToRemove) == 1);

	TSharedPtr<const FInternetAddr> AddrToRemove = ClientConnectionToRemove->GetRemoteAddr();

	if (AddrToRemove.IsValid())
	{
		constexpr bool bPrintPort = true;

		TSharedRef<const FInternetAddr> ConstAddrRef = AddrToRemove.ToSharedRef();

		if (RecentlyDisconnectedTrackingTime > 0)
		{
			const bool bIsAddressTracked = MappedClientConnections.Find(ConstAddrRef) != nullptr;

			if (ensureMsgf(bIsAddressTracked, TEXT("RemoveClientConnection for %s was not in MappedClientConnections list"), *(ConstAddrRef->ToString(bPrintPort))))
			{
				UE_LOGF(LogNet, Log, "UNetDriver::RemoveClientConnection - Removed address %ls from MappedClientConnections for: %ls", *(ConstAddrRef->ToString(bPrintPort)), *ClientConnectionToRemove->Describe());

				RecentlyDisconnectedClients.Add(FDisconnectedClient(ConstAddrRef, FPlatformTime::Seconds()));

				// Mark recently disconnected clients as nullptr (don't wait for GC), and keep the MappedClientConections entry for a while.
				// Required for identifying/ignoring packets from recently disconnected clients, with the same performance as for NetConnection's (important for DDoS detection)
				MappedClientConnections.Emplace(ConstAddrRef, nullptr);
			}
		}
		else
		{
			UE_LOGF(LogNet, Log, "UNetDriver::RemoveClientConnection - Removed address %ls from MappedClientConnections for: %ls", *(ConstAddrRef->ToString(bPrintPort)), *ClientConnectionToRemove->Describe());
			verify(MappedClientConnections.Remove(ConstAddrRef) == 1);
		}
	}

	if (ReplicationDriver)
	{
		ReplicationDriver->RemoveClientConnection(ClientConnectionToRemove);
	}

	bHasReplayConnection = false;

	for (UNetConnection* ClientConn : ClientConnections)
	{
		if (ClientConn && ClientConn->IsReplay())
		{
			bHasReplayConnection = true;
			break;
		}
	}

	UpdateCrashContext();
}

void UNetDriver::SetWorld(class UWorld* InWorld)
{
	if (World)
	{
		// Remove old world association
		UnregisterTickEvents(World);
		World = nullptr;
		WorldPackage = nullptr;
		Notify = nullptr;

		GetNetworkObjectList().Reset();
	}

	if (InWorld)
	{
		// Setup new world association
		World = InWorld;
		WorldPackage = InWorld->GetOutermost();
		Notify = InWorld;
		RegisterTickEvents(InWorld);

		if (IsUsingIrisReplication())
		{
			if (bSkipStartReplicatingWorldForIris)
			{
				// We skipped, reset the flag in case there is another call.
				bSkipStartReplicatingWorldForIris = false;
			}
			else
			{
				UpdateGroupFilterStatusForAllLevelsForAllConnections();				
				UE::Net::FReplicationSystemUtil::StartReplicatingActorsInWorldForNetDriver(InWorld, this);
			}
		}
		else
		{
			GetNetworkObjectList().AddInitialObjects(InWorld, this);
		}

		NotifyGameInstanceUpdated();
	}

	if (ReplicationDriver)
	{
		ReplicationDriver->SetRepDriverWorld(InWorld);
		ReplicationDriver->InitializeActorsInWorld(InWorld);
	}
}

void UNetDriver::ResetGameWorldState()
{
	DestroyedStartupOrDormantActors.Empty();
	DestroyedStartupOrDormantActorsByLevel.Empty();
	RenamedStartupActors.Empty();

	if ( NetCache.IsValid() )
	{
		NetCache->ClearClassNetCache();	// Clear the cache net: it will recreate itself after seamless travel
	}

	GetNetworkObjectList().ResetDormancyState();

	if (ServerConnection)
	{
		ServerConnection->ResetGameWorldState();
	}
	for (auto It = ClientConnections.CreateIterator(); It; ++It)
	{
		(*It)->ResetGameWorldState();
	}

	if (ReplicationDriver)
	{
		ReplicationDriver->ResetGameWorldState();
	}
}

void UNetDriver::CleanPackageMaps()
{
	if ( GetNetGuidCache().IsValid() )
	{ 
		GetNetGuidCache()->CleanReferences();
	}

	if (GNetResetAckStatePostSeamlessTravel)
	{
		for (UNetConnection* Connection : ClientConnections)
		{
			if (Connection)
			{ 
				if (UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Connection->PackageMap))
				{
					PackageMapClient->ResetAckState();
				}
			}
		}
	}
}

void UNetDriver::RemoveClassRepLayoutReferences(UClass* Class)
{
	RepLayoutMap.Remove(Class);

	for (auto Func : TFieldRange<UFunction>(Class, EFieldIteratorFlags::ExcludeSuper))
	{
		if (Func && Func->HasAnyFunctionFlags(EFunctionFlags::FUNC_Net))
		{
			RepLayoutMap.Remove(Func);
		}
	}
}

void UNetDriver::CleanupWorldForSeamlessTravel()
{
	if (World != nullptr)
	{
		for (auto LevelIt(World->GetLevelIterator()); LevelIt; ++LevelIt)
		{
			if (const ULevel* Level = *LevelIt)
			{
				if (Level->LevelScriptActor)
				{
					// workaround for this not being called on clients
					if (ServerConnection != nullptr)
					{
						NotifyActorLevelUnloaded(Level->LevelScriptActor);
					}

					RemoveClassRepLayoutReferences(Level->LevelScriptActor->GetClass());

					ReplicationChangeListMap.Remove(Level->LevelScriptActor);
				}

				// This is currently necessary because the actor iterator used in the seamless travel handler 
				// skips over AWorldSettings actors for an unknown reason.
				AWorldSettings* WorldSettings = Level->GetWorldSettings(false);

				if (WorldSettings != nullptr)
				{
					NotifyActorLevelUnloaded(WorldSettings);
				}
			}
		}
	}
}

void UNetDriver::PreSeamlessTravelGarbageCollect()
{
	// Destruction of our current World is imminent. Reset anything tied to it.
	ResetGameWorldState();

	// Clear out the network objects list, the traveling actors are in SeamlessTravelingObjects
	NetworkObjects->Reset();

	if (ReplicationSystem)
	{
		ReplicationSystem->PreSeamlessTravelGarbageCollect();
	}
}

void UNetDriver::PostSeamlessTravelGarbageCollect()
{
	CleanPackageMaps();

	NetworkObjects->OnPostSeamlessTravel();

	if (ReplicationSystem)
	{
		ReplicationSystem->PostSeamlessTravelGarbageCollect();
	}
}

void UNetDriver::SetReplicationDriver(UReplicationDriver* NewReplicationDriver)
{
	if (ReplicationDriver)
	{
		ReplicationDriver->TearDown();

		// Though the NetDriver does not set the ConnectionReplicationConnectionDriver,
		// We are going to clear it here so that references to the old replication driver
		// do not leak. (E.g, a new replication driver may not set ConnectionReplicationDriver)
		for (UNetConnection* Connection : ClientConnections)
		{
			if (Connection)
			{
				Connection->TearDownReplicationConnectionDriver();
			}		
		}
		if (ServerConnection)
		{
			// Clients shouldn't currently ever have a repdriver but this could change in the future
			ServerConnection->TearDownReplicationConnectionDriver();
		}
	}

	if (IsUsingIrisReplication() && NewReplicationDriver)
	{
		checkf(false, TEXT("ReplicationDriver (%s) are not supported with a NetDriver (%s) configured to use Iris"), *NewReplicationDriver->GetName(), *NetDriverName.ToString());
		NewReplicationDriver->MarkAsGarbage();
		return;
	}

	ReplicationDriver = NewReplicationDriver;
	if (ReplicationDriver)
	{
		ReplicationDriver->SetRepDriverWorld( GetWorld() );
		ReplicationDriver->InitForNetDriver(this);
		ReplicationDriver->InitializeActorsInWorld( GetWorld() );
	}

	NotifyGameInstanceUpdated();
}

UNetConnection* UNetDriver::GetConnectionById(uint32 ParentConnectionId) const
{
	if (ServerConnection != nullptr && ServerConnection->GetConnectionHandle().GetParentConnectionId() == ParentConnectionId)
	{
		return ServerConnection;
	}

	for (UNetConnection* Connection : ClientConnections)
	{
		if (Connection && Connection->GetConnectionHandle().GetParentConnectionId() == ParentConnectionId)
		{
			return Connection;
		}
	}

	return nullptr;
}

UNetConnection* UNetDriver::GetConnectionByHandle(UE::Net::FConnectionHandle ConnectionHandle) const
{
	if (ServerConnection != nullptr && ServerConnection->GetConnectionHandle() == ConnectionHandle)
	{
		return ServerConnection;
	}

	for (UNetConnection* Connection : ClientConnections)
	{
		if (Connection && Connection->GetConnectionHandle() == ConnectionHandle)
		{
			return Connection;
		}
	}

	return nullptr;
}

bool UNetDriver::InitReplicationBridgeClass()
{
	if (ReplicationBridgeClass != nullptr)
	{
		return true;
	}

	if (!ReplicationBridgeClassName.IsEmpty())
	{
		ReplicationBridgeClass = LoadClass<UObjectReplicationBridge>(nullptr, *ReplicationBridgeClassName, nullptr, LOAD_None, nullptr);
		if (ReplicationBridgeClass == nullptr)
		{
			UE_LOGF(LogNet, Error, "Failed to load class '%ls'", *ReplicationBridgeClassName);
			return false;
		}
	}
	else
	{
		// Fall back on ActorReplicationBridge
		ReplicationBridgeClass = UEngineReplicationBridge::StaticClass();
	}

	return ReplicationBridgeClass != nullptr;
}

void UNetDriver::UpdateGroupFilterStatusForLevel(const ULevel* Level, UE::Net::FNetObjectGroupHandle LevelGroupHandle)
{
	using namespace UE::Net;

	if (ReplicationSystem == nullptr || !ensure(ReplicationSystem->IsValidGroup(LevelGroupHandle)))
	{
		return;
	}

	const FName WorldPackageName = GetWorldPackage()->GetFName();
	const FName LevelPackageName = Level->GetOutermost()->GetFName();
	const bool bIsPersistentLevel = Level->IsPersistentLevel();

	// Build connection mask
	FNetBitArray ConnectionMask(ReplicationSystem->GetMaxConnectionCount());
		
	for (const UNetConnection* Connection : ClientConnections)
	{
		// For a level to be visible the client world need to match ours. 
		const bool bIsSameWorld = (WorldPackageName == Connection->GetClientWorldPackageName());
		// If the world matches then the level is loaded if it's persistent or if the client has streamed it in.
		const bool bIsVisible = bIsSameWorld && (bIsPersistentLevel || Connection->ClientVisibleLevelNames.Contains(LevelPackageName));
		ConnectionMask.SetBitValue(Connection->GetConnectionHandle().GetParentConnectionId(), bIsVisible);
	}

	// Set group filter status
	ReplicationSystem->SetGroupFilterStatus(LevelGroupHandle, ConnectionMask, ENetFilterStatus::Allow);
}

void UNetDriver::DisallowReplicationOfAllLevelGroupsForConnection(UE::Net::FConnectionHandle ConnectionHandle)
{
	if (!ConnectionHandle.IsValid() || ConnectionHandle.IsChildConnection())
	{
		return;
	}

	const UObjectReplicationBridge* Bridge = ReplicationSystem ? ReplicationSystem->GetReplicationBridge() : nullptr;
	if (Bridge == nullptr)
	{
		return;
	}

	const uint32 ConnectionId = ConnectionHandle.GetParentConnectionId();
	const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>& ContainerGroups = Bridge->GetAllContainerGroups();
	for (const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>::ElementType& ContaineAndGroups : ContainerGroups)
	{
		ReplicationSystem->SetGroupFilterStatus(ContaineAndGroups.Value, ConnectionId, UE::Net::ENetFilterStatus::Disallow);
	}
}

void UNetDriver::UpdateGroupFilterStatusForAllLevelsForConnection(UE::Net::FConnectionHandle ConnectionHandle)
{
	if (!ConnectionHandle.IsValid() || ConnectionHandle.IsChildConnection())
	{
		return;
	}

	const UObjectReplicationBridge* Bridge = ReplicationSystem ? ReplicationSystem->GetReplicationBridge() : nullptr;
	if (Bridge == nullptr)
	{
		return;
	}

	const UNetConnection* Connection = GetConnectionByHandle(ConnectionHandle);
	if (!ensureMsgf(Connection, TEXT("Unknown connection with parent ID %u child ID %u"), ConnectionHandle.GetParentConnectionId(), ConnectionHandle.GetChildConnectionId()))
	{
		return;
	}

	// If client and server are in different worlds we disallow replication of world objects.
	if (GetWorldPackage()->GetFName() != Connection->GetClientWorldPackageName())
	{
		DisallowReplicationOfAllLevelGroupsForConnection(ConnectionHandle);
		return;
	}

	const uint32 ConnectionId = ConnectionHandle.GetParentConnectionId();
	const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>& ContainerGroups = Bridge->GetAllContainerGroups();
	for (const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>::ElementType& ContainerAndGroup : ContainerGroups)
	{
		if (const ULevel* Level = Cast<ULevel>(ContainerAndGroup.Key.ResolveObjectPtr()))
		{
			const bool bIsVisible = (Level->IsPersistentLevel() || Connection->ClientVisibleLevelNames.Contains(Level->GetOutermost()->GetFName()));
			ReplicationSystem->SetGroupFilterStatus(ContainerAndGroup.Value, ConnectionId, bIsVisible ? UE::Net::ENetFilterStatus::Allow : UE::Net::ENetFilterStatus::Disallow);
		}
	}
}

void UNetDriver::UpdateGroupFilterStatusForAllLevelsForAllConnections()
{
	using namespace UE::Net;

	if (ClientConnections.IsEmpty())
	{
		return;
	}

	const UObjectReplicationBridge* Bridge = ReplicationSystem ? ReplicationSystem->GetReplicationBridge() : nullptr;
	if (Bridge == nullptr)
	{
		return;
	}

	const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>& ContainerGroups = Bridge->GetAllContainerGroups();
	for (const TMap<FObjectKey, UE::Net::FNetObjectGroupHandle>::ElementType& ContainerAndGroup : ContainerGroups)
	{
		if (const ULevel* Level = Cast<ULevel>(ContainerAndGroup.Key.ResolveObjectPtr()))
		{
			UpdateGroupFilterStatusForLevel(Level, ContainerAndGroup.Value);
		}
	}
}

void UNetDriver::SetReplicationSystem(UReplicationSystem* InReplicationSystem)
{
	if (ReplicationSystem)
	{
		UE::Net::FReplicationSystemFactory::DestroyReplicationSystem(ReplicationSystem);
	}

	ReplicationSystem = InReplicationSystem;

	if (ReplicationSystem)
	{
		// When we run using Iris, we use ReplicationSystemId as our unique identifier
		NetTraceId = ReplicationSystem->GetId();
		
#if UE_NET_TRACE_ENABLED
		//We explicitly make sure NetTrace has a corresponding GameInstance set up, as it means we can assume the owning map is read-only during parallel phases of Iris
		FNetTrace::SetupInstanceForNetTraceId(NetTraceId);
#endif // UE_NET_TRACE_ENABLED

		UE::Net::FReplicationSystemUtil::StartReplicatingActorsInWorldForNetDriver(GetWorld(), this);
	}
}

void UNetDriver::RestartIrisSystem()
{
	if (ReplicationSystem == nullptr)
	{
		ensureMsgf(false, TEXT("RestartIrisSystem called while no system existed."));
		return;
	}

	if (ClientConnections.Num() > 0)
	{
		ensureMsgf(false, TEXT("RestartIrisSystem called while there were active connections."));
		return;
	}

	DestroyIrisSystem();

	CreateReplicationSystem(!IsServer());
}

void UNetDriver::DestroyIrisSystem()
{
	if (ReplicationSystem == nullptr)
	{
		ensureMsgf(false, TEXT("DestroyIrisSystem called while no system existed."));
		return;
	}

	if (ClientConnections.Num() > 0)
	{
		ensureMsgf(false, TEXT("DestroyIrisSystem called while there were active connections."));
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDriver_DestroyIrisSystem);

	UE::Net::FReplicationSystemFactory::DestroyReplicationSystem(ReplicationSystem);
	ReplicationSystem = nullptr;
}

void UNetDriver::RecreateIrisSystem()
{
	CreateReplicationSystem(!IsServer());
}

void UNetDriver::CreateReplicationSystem(bool bInitAsClient)
{
	LLM_SCOPE_BYTAG(Iris);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NetDriver_CreateReplicationSystem);

	const bool bBridgeClassExists = InitReplicationBridgeClass();

	if (!bBridgeClassExists)
	{
		ensureMsgf(false, TEXT("InitReplicationBridgeClass could not load configured class %s"), *ReplicationBridgeClassName);
		return;
	}

	UEngineReplicationBridge* ReplicationBridge = NewObject<UEngineReplicationBridge>(GetTransientPackage(), ReplicationBridgeClass);
	if (ReplicationBridge)
	{
		ReplicationBridge->SetNetDriver(this);

		FNetDriverReplicationSystemConfig& RepSystemConfig = bInitAsClient ? ReplicationSystemConfigClient : ReplicationSystemConfigServer;
		// Ask the GameInstance if it wants to override any configs
		if (const UWorld* LocalWorld = GetWorld())
		{
			if (UGameInstance* GameInstance = LocalWorld->GetGameInstance())
			{
				GameInstance->OverrideIrisReplicationSystemConfig(RepSystemConfig, !bInitAsClient);
			}
		}

		// Create ReplicationSystem
		UReplicationSystem::FReplicationSystemParams Params;
		Params.ReplicationBridge = ReplicationBridge;
		Params.bIsServer = !bInitAsClient;
		Params.bAllowObjectReplication = !bInitAsClient;
		Params.bUseRemoteObjectReferences = IsUsingRemoteObjectReferences();
		Params.bReplicateTransactionally = ShouldReplicateTransactionally();
		Params.ForwardNetRPCCallDelegate.BindUObject(this, &UNetDriver::ForwardRemoteFunction);
		UE::Net::Private::ApplyReplicationSystemConfig(RepSystemConfig, Params);
		Params.bAllowExecuteRPC = IsExecuteRPCFunctionsEnabled();
		Params.bAllowExecuteRepNotifies = !ShouldSkipRepNotifies();
		Params.bAlwaysSwapRolesOnReplication = IsRoleSwappingOnAllActorsEnabled();
		Params.NetTokenStore = NetTokenStore.Get();
		Params.ProxyType = ProxyType;

		if (GInitialNetworkId > 0)
		{
			Params.InitialNetRefHandleIndex = GInitialNetworkId;
		}

		SetReplicationSystem(UE::Net::FReplicationSystemFactory::CreateReplicationSystem(Params));
	}
	else
	{
		UE_LOGF(LogNet, Error, "Failed to initialize ReplicationSystem");
	}
}

void UNetDriver::UpdateIrisReplicationViews() const
{
	using namespace UE::Net;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateIrisReplicationViews);

	FReplicationView::FView DefaultView;
	{
		const APlayerCameraManager* CameraManager = GetDefault<APlayerCameraManager>();
		DefaultView.FoVRadians = FMath::DegreesToRadians(CameraManager->DefaultFOV);
	}

	const bool bUpdateConnectionViewTarget = IsServer();

	auto FillIrisReplicationViews = [&DefaultView, bUpdateConnectionViewTarget](TArrayView<UNetConnection*> AllConnections, FReplicationView& OutReplicationView)
	{
		for (UNetConnection* AnyConnection : AllConnections)
		{
			// As we no longer call ServerReplicateActors we must make sure to update the ViewTarget
			if (bUpdateConnectionViewTarget)
			{
				AnyConnection->ViewTarget = AnyConnection->GetConnectionViewTarget();
			}

			const AActor* ViewTarget = AnyConnection->ViewTarget;
			const APlayerController* ViewingController = AnyConnection->PlayerController;
			if (ViewTarget == nullptr && ViewingController == nullptr)
			{
				continue;
			}

			FReplicationView::FView& View = OutReplicationView.Views.AddZeroed_GetRef();
			View.FoVRadians = DefaultView.FoVRadians;

			if (ViewTarget)
			{
				View.Pos = ViewTarget->GetActorLocation();
				View.Dir = ViewTarget->GetActorRotation().Vector();
				View.ViewTarget = UE::Net::FNetHandleManager::GetNetHandle(ViewTarget);
			}
			if (ViewingController)
			{
				FRotator ViewRotation = ViewingController->GetControlRotation();
				ViewingController->GetPlayerViewPoint(View.Pos, ViewRotation);
				View.Dir = ViewRotation.Vector();
				View.Controller = UE::Net::FNetHandleManager::GetNetHandle(ViewingController);

				if (const APlayerCameraManager* CameraManager = ViewingController->PlayerCameraManager)
				{
					View.FoVRadians = FMath::DegreesToRadians(CameraManager->GetFOVAngle());
				}
			}
		}
	};

	if (IsServer())
	{
		FReplicationView ReplicationView;
		TArray<UNetConnection*, TInlineAllocator<UE_IRIS_INLINE_VIEWS_PER_CONNECTION>> AllConnections;

		for (UNetConnection* ClientConnection : ClientConnections)
		{
			AllConnections.Add(ClientConnection);
			for (UNetConnection* Children : ClientConnection->Children)
			{
				AllConnections.Add(Children);
			}

			FillIrisReplicationViews(AllConnections, ReplicationView);

			ReplicationSystem->SetReplicationView(ClientConnection->GetConnectionHandle().GetParentConnectionId(), ReplicationView);

			ReplicationView.Views.Reset();
			AllConnections.Reset();
		}
	}
	else
	{
		FReplicationView ReplicationView;
		TArray<UNetConnection*, TInlineAllocator<UE_IRIS_INLINE_VIEWS_PER_CONNECTION>> AllConnections;

		AllConnections.Add(ServerConnection);
		for (UNetConnection* Children : ServerConnection->Children)
		{
			AllConnections.Add(Children);
		}

		FillIrisReplicationViews(AllConnections, ReplicationView);

		ReplicationSystem->SetReplicationView(ServerConnection->GetConnectionHandle().GetParentConnectionId(), ReplicationView);
	}
}

void UNetDriver::SendClientMoveAdjustments()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SendClientMoveAdjustments);

	for (UNetConnection* Connection : ClientConnections)
	{
		if (Connection == nullptr || Connection->ViewTarget == nullptr)
		{
			continue;
		}

		if (APlayerController* PC = Connection->PlayerController)
		{
			PC->SendClientAdjustment();
		}

		for (UNetConnection* ChildConnection : Connection->Children)
		{
			if (ChildConnection == nullptr)
			{
				continue;
			}

			if (APlayerController* PC = ChildConnection->PlayerController)
			{
				PC->SendClientAdjustment();
			}
		}
	}
}

void UNetDriver::PostDispatchSendUpdate()
{
	if (!ReplicationSystem)
	{
		return;
	}
	
	if (!IsKnownChannelName(NAME_DataStream))
	{
		return;
	}

	ReplicationSystem->TickPostReceive();

	ReplicationSystem->SendUpdate([this](TArrayView<uint32> ConnectionsToSend)
	{
		const int32 DataStreamChannelIndex = ChannelDefinitionMap[NAME_DataStream].StaticChannelIndex;

		for (uint32 ConnId : ConnectionsToSend)
		{
			UNetConnection* NetConnection = GetConnectionByHandle(UE::Net::FConnectionHandle(ConnId));
			if (NetConnection && NetConnection->Channels.IsValidIndex(DataStreamChannelIndex))
			{
				if (UDataStreamChannel* DataStreamChannel = Cast<UDataStreamChannel>(NetConnection->Channels[DataStreamChannelIndex]))
				{
					DataStreamChannel->PostTickDispatch();
				}
			}
		}
	});

	ReplicationSystem->PostSendUpdate();
}


void UNetDriver::InitNetTraceId()
{
	if (IsUsingIrisReplication())
	{
		return;
	}

	if (NetTraceId == NetTraceInvalidGameInstanceId)
	{
		// We just need to make sure that all active NetDriver`s running in the same game instance uses different NetTraceId`s.
		static uint8 CurrentNetTraceId = 0;
		NetTraceId = 128 + ((++CurrentNetTraceId) & 127);
	}
}

#if NET_DEBUG_RELEVANT_ACTORS
static void	DumpRelevantActors( UWorld* InWorld )
{
	UNetDriver *NetDriver = InWorld->NetDriver;
	if (!NetDriver)
	{
		return;
	}

	NetDriver->DebugRelevantActors = true;
}
#endif // NET_DEBUG_RELEVANT_ACTORS

TSharedPtr<FRepChangedPropertyTracker> UNetDriver::FindOrCreateRepChangedPropertyTracker(UObject* Obj)
{
	check(IsServer() || MaySendProperties());

	const FObjectKey ObjectKey = Obj;
	checkf(ObjectKey.ResolveObjectPtr() != nullptr, TEXT("Unresolveable object %s received in FindOrCreateRepChangedPropertyTracker"), *GetNameSafe(Obj));

	return UE::Net::Private::FNetPropertyConditionManager::Get().FindOrCreatePropertyTracker(ObjectKey);
}

TSharedPtr<FRepChangedPropertyTracker> UNetDriver::FindRepChangedPropertyTracker(UObject* Obj)
{
	return UE::Net::Private::FNetPropertyConditionManager::Get().FindPropertyTracker(Obj);
}

TSharedPtr<FRepLayout> UNetDriver::GetObjectClassRepLayout( UClass * Class )
{
	TSharedPtr<FRepLayout>* RepLayoutPtr = RepLayoutMap.Find(Class);

	if (!RepLayoutPtr) 
	{
		ECreateRepLayoutFlags Flags = MaySendProperties() ? ECreateRepLayoutFlags::MaySendProperties : ECreateRepLayoutFlags::None;
		RepLayoutPtr = &RepLayoutMap.Add(Class, FRepLayout::CreateFromClass(Class, ServerConnection, Flags));
	}

	return *RepLayoutPtr;
}

TSharedPtr<FRepLayout> UNetDriver::GetFunctionRepLayout(UFunction * Function)
{
	TSharedPtr<FRepLayout>* RepLayoutPtr = RepLayoutMap.Find(Function);

	if (!RepLayoutPtr) 
	{
		ECreateRepLayoutFlags Flags = MaySendProperties() ? ECreateRepLayoutFlags::MaySendProperties : ECreateRepLayoutFlags::None;

		// Use a temp shared ptr to hold onto a previous value in case we abort. The adding to RepLayoutMap copies this TSharedPtr, and will end up taking a ref count and owning it
		TSharedPtr<FRepLayout> NewLayoutPtr;
		UE_AUTORTFM_OPEN{
			NewLayoutPtr = FRepLayout::CreateFromFunction(Function, ServerConnection, Flags);
		};

		RepLayoutPtr = &RepLayoutMap.Add(Function, NewLayoutPtr);
	}

	return *RepLayoutPtr;
}

TSharedPtr<FRepLayout> UNetDriver::GetStructRepLayout( UStruct * Struct )
{
	TSharedPtr<FRepLayout>* RepLayoutPtr = RepLayoutMap.Find(Struct);

	if (!RepLayoutPtr) 
	{
		ECreateRepLayoutFlags Flags = MaySendProperties() ? ECreateRepLayoutFlags::MaySendProperties : ECreateRepLayoutFlags::None;
		RepLayoutPtr = &RepLayoutMap.Add(Struct, FRepLayout::CreateFromStruct(Struct, ServerConnection, Flags));
	}

	return *RepLayoutPtr;
}

TSharedPtr<FReplicationChangelistMgr> UNetDriver::GetReplicationChangeListMgr(UObject* Object)
{
	check(IsServer() || MaySendProperties());

	TSharedPtr<FReplicationChangelistMgr>* ReplicationChangeListMgrPtr = ReplicationChangeListMap.Find(Object);

	if (!ReplicationChangeListMgrPtr)
	{
		const TSharedPtr<const FRepLayout> RepLayout = GetObjectClassRepLayout(Object->GetClass());
		ReplicationChangeListMgrPtr = &ReplicationChangeListMap.Add(Object, RepLayout->CreateReplicationChangelistMgr(Object, GetCreateReplicationChangelistMgrFlags()));
	}

	return *ReplicationChangeListMgrPtr;
}

void UNetDriver::RemoveDestroyedGuidsByLevel(const ULevel* Level, const TArray<FNetworkGUID>& RemovedGUIDs)
{
	check(Level);

	const FName StreamingLevelName = !Level->IsPersistentLevel() ? Level->GetOutermost()->GetFName() : NAME_None;

	if (TSet<FNetworkGUID>* DestroyedGuidsForLevel = DestroyedStartupOrDormantActorsByLevel.Find(StreamingLevelName))
	{
		for (const FNetworkGUID& NetGUID : RemovedGUIDs)
		{
			DestroyedGuidsForLevel->Remove(NetGUID);
		}
	}
}

// This method will be called when Streaming Levels become Visible.
void UNetDriver::OnLevelAddedToWorld(ULevel* Level, UWorld* InWorld)
{
	// Actors will be re-added to the network list when ULevel::SortActorList is called.
}

// This method will be called when Streaming Levels are hidden.
void UNetDriver::OnLevelRemovedFromWorld(class ULevel* Level, class UWorld* InWorld)
{
	if (Level && InWorld == GetWorld())
	{
		if (!IsServer())
		{
			for (AActor* Actor : Level->Actors)
			{
				if (Actor)
				{
					// Always call this on clients.
					// It won't actually destroy the Actor, but it will do some cleanup of the channel, etc.
					NotifyActorLevelUnloaded(Actor);
				}
			}
		}	
		else
		{
			for (AActor* Actor : Level->Actors)
			{
				if (Actor)
				{
					// Keep Startup actors alive, because they haven't been destroyed yet.
					// Technically, Dynamic actors may not have been destroyed either, but this
					// resembles relevancy.
					if (!Actor->IsNetStartupActor())
					{
						NotifyActorLevelUnloaded(Actor);
					}
					else
					{
						// We still want to remove Startup actors from the Network list so they aren't processed anymore.
						GetNetworkObjectList().Remove(Actor);
					}
				}
			}

			TArray<FNetworkGUID> RemovedGUIDs;
			for (auto It = DestroyedStartupOrDormantActors.CreateIterator(); It; ++It)
			{
				FActorDestructionInfo* DestructInfo = It->Value.Get();
				
				// Until the level is actually unloaded / reloaded, any Static Actors that have been
				// destroyed will not be recreated, so we still need to track them.
				if (DestructInfo->Level == Level && !DestructInfo->NetGUID.IsStatic())
				{
					for (UNetConnection* Connection : ClientConnections)
					{
						Connection->RemoveDestructionInfo(DestructInfo);
					}

					RemovedGUIDs.Add(It->Key);
					It.RemoveCurrent();
				}
			}

			RemoveDestroyedGuidsByLevel(Level, RemovedGUIDs);
		}
	}
}

bool UNetDriver::ShouldUpdateStats() const
{
	return NetDriverName == NAME_GameNetDriver;
}

bool UNetDriver::ShouldRegisterMetricsDatabaseListeners() const
{
	return NetDriverDefinition == NAME_GameNetDriver;
}

bool UNetDriver::ShouldSkipRepNotifies() const
{
#if !UE_BUILD_SHIPPING
	if (SkipRepNotifiesDel.IsBound())
	{
		return SkipRepNotifiesDel.Execute();
	}
#endif

	return false;
}

void UNetDriver::RequestNetDriverDestruction()
{
	if (GetPendingDestruction())
	{
		// destruction already queued
		return;
	}

	// If we are ticking, wait before triggering our destruction 
	if (IsInTick())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SetPendingDestruction(true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		if (ensureMsgf(World, TEXT("RequestNetDriverDestruction on %s failed due to no world assigned"), *GetDescription()))
		{
			GEngine->DestroyNamedNetDriver(World, NetDriverName);
		}		
	}
}

void UNetDriver::SetPendingDestruction(bool bDestroy)
{
	UE_LOGF(LogNet, Log, "SetPendingDestruction: %d on %ls)", bDestroy, *GetDescription());
	bPendingDestruction = bDestroy;
}
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------

int32 CVar_RepDriver_Enable = 1;
static FAutoConsoleVariableRef CVarRepManagerEnable(TEXT("Net.RepDriver.Enable"), CVar_RepDriver_Enable, TEXT("Enables Replication Driver. 0 will fallback to legacy NetDriver implementation."), ECVF_Default );

UReplicationDriver::UReplicationDriver()
{
}

UReplicationDriver* UReplicationDriver::CreateReplicationDriver(UNetDriver* NetDriver, const FURL& URL, UWorld* World)
{
	if (CreateReplicationDriverDelegate().IsBound())
	{
		return CreateReplicationDriverDelegate().Execute(NetDriver, URL, World);
	}

	// If we initialize the server map from the initial commandline, this will run before the "-execcmds" option has been handled. So explicitly check for force cmd line options
	static bool FirstTime = true;
	if (FirstTime)
	{
		FirstTime = false;
		if (FParse::Param(FCommandLine::Get(), TEXT("RepDriverEnable")))
		{
			CVar_RepDriver_Enable = 1;
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("RepDriverDisable")))
		{
			CVar_RepDriver_Enable = 0;
		}
	}

	if (CVar_RepDriver_Enable == 0)
	{
		return nullptr;
	}

	UClass* ReplicationDriverClass = NetDriver ? ToRawPtr(NetDriver->ReplicationDriverClass) : nullptr;
	if (ReplicationDriverClass == nullptr)
	{
		UE_LOGF(LogNet, Log, "ReplicationDriverClass is null! Not using ReplicationDriver.");
		return nullptr;
	}

	return NewObject<UReplicationDriver>(GetTransientPackage(), ReplicationDriverClass);
}

FCreateReplicationDriver& UReplicationDriver::CreateReplicationDriverDelegate()
{
	static FCreateReplicationDriver Delegate;
	return Delegate;
}

// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------
// --------------------------------------------------------------------------------------------------------------------------------------------


void UNetDriver::ProcessRemoteFunction(
	class AActor* Actor,
	UFunction* Function,
	void* Parameters,
	FOutParmRec* OutParms,
	FFrame* Stack,
	class UObject* SubObject)
{
	if (Actor->IsActorBeingDestroyed())
	{
		UE_LOGF(LogNet, Warning, "UNetDriver::ProcessRemoteFunction: Remote function %ls called from actor %ls while actor is being destroyed. Function will not be processed.", *Function->GetName(), *Actor->GetName());
		return;
	}

	if (UE::Net::bDiscardTornOffActorRPCs && Actor->GetTearOff())
	{
		UE_LOGF(LogNet, Warning, "UNetDriver::ProcessRemoteFunction: Remote function %ls called from actor %ls while actor is torn off. Function will not be processed.", *Function->GetName(), *Actor->GetName());
		return;
	}

#if !UE_BUILD_SHIPPING
	SCOPE_CYCLE_COUNTER(STAT_NetProcessRemoteFunc);
	SCOPE_CYCLE_UOBJECT(Function, Function);

	{
		UObject* TestObject = (SubObject == nullptr) ? Actor : SubObject;
		checkf(IsInGameThread(), TEXT("Attempted to call ProcessRemoteFunction from a thread other than the game thread, which is not supported.  Object: %s Function: %s"), *GetPathNameSafe(TestObject), *GetNameSafe(Function));
		ensureMsgf(TestObject->IsSupportedForNetworking() || TestObject->IsNameStableForNetworking(), TEXT("Attempted to call ProcessRemoteFunction with object that is not supported for networking. Object: %s Function: %s"), *TestObject->GetPathName(), *Function->GetName());
	}

	bool bBlockSendRPC = false;

	SendRPCDel.ExecuteIfBound(Actor, Function, Parameters, OutParms, Stack, SubObject, bBlockSendRPC);

	if (!bBlockSendRPC)
#endif
	{
		const bool bIsServer = IsServer();
		const bool bIsServerMulticast = bIsServer && (Function->FunctionFlags & FUNC_NetMulticast);

		++TotalRPCsCalled;

		// Copy Any Out Params to Local Params 
		TArray<UE::Net::Private::FAutoDestructProperty> LocalOutParms;
		if (Stack == nullptr)
		{
			// If we have a subobject, thats who we are actually calling this on. If no subobject, we are calling on the actor.
			UObject* TargetObj = SubObject ? SubObject : Actor;
			LocalOutParms = UE::Net::Private::CopyOutParametersToLocalParameters(Function, OutParms, Parameters, TargetObj);
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (!UE::RemoteObject::Handle::IsOwned(Actor) || !UE::RemoteObject::Handle::IsOwned(SubObject))
		{
			UE::RemoteObject::Serialization::Network::EnqueueRPC(Actor, SubObject, Function, Parameters);
			return;
		}
#endif

		if (ReplicationSystem)
		{
			if (bIsServerMulticast)
			{
				if (ReplicationSystem->SendRPC(Actor, SubObject, Function, Parameters))
				{
					return;
				}
			}
			else
			{
				if (UNetConnection* Connection = Actor->GetNetConnection())
				{
					if (ReplicationSystem->SendRPC(Connection->GetConnectionHandle().GetParentConnectionId(), Actor, SubObject, Function, Parameters))
					{
						return;
					}
				}
				else
				{
					UE_LOGF(LogNet, Verbose, "SendRPC %ls::%ls dropped because Actor does not have a NetConnection", *Actor->GetName(), *Function->GetName());
				}
			}

			// If we are using Iris replication, we should never fall back on normal replication path
			return;
		}

		// Forward to replication Driver if there is one
		if (ReplicationDriver && ReplicationDriver->ProcessRemoteFunction(Actor, Function, Parameters, OutParms, Stack, SubObject))
		{
			return;
		}

		// RepDriver didn't handle it, default implementation
		UNetConnection* Connection = nullptr;
		if (bIsServerMulticast)
		{
			TSharedPtr<FRepLayout> RepLayout = GetFunctionRepLayout(Function);

			// Multicast functions go to every client
			EProcessRemoteFunctionFlags RemoteFunctionFlags = EProcessRemoteFunctionFlags::None;
			TArray<UNetConnection*> UniqueRealConnections;
			for (int32 i = 0; i < ClientConnections.Num(); ++i)
			{
				Connection = ClientConnections[i];
				if (Connection && Connection->ViewTarget)
				{
					// Only send or queue multicasts if the actor is relevant to the connection
					FNetViewer Viewer(Connection, 0.f);

					if (Connection->GetUChildConnection() != nullptr)
					{
						Connection = ((UChildConnection*)Connection)->Parent;
					}

					// It's possible that an actor is not relevant to a specific connection, but the channel is still alive (due to hysteresis).
					// However, it's also possible that the Actor could become relevant again before the channel ever closed, and in that case we
					// don't want to lose Reliable RPCs.
					if (Actor->IsNetRelevantFor(Viewer.InViewer, Viewer.ViewTarget, Viewer.ViewLocation) ||
						((Function->FunctionFlags & FUNC_NetReliable) && !!CVarAllowReliableMulticastToNonRelevantChannels.GetValueOnGameThread() && Connection->FindActorChannelRef(Actor)))
					{
						// We don't want to call this unless necessary, and it will internally handle being called multiple times before a clear
						// Builds any shared serialization state for this rpc
						RepLayout->BuildSharedSerializationForRPC(Parameters, GetNetTokenStore());

						InternalProcessRemoteFunctionPrivate(Actor, SubObject, Connection, Function, Parameters, OutParms, Stack, bIsServer, RemoteFunctionFlags);
					}
				}
			}

			// Finished sending this multicast rpc, clear any shared state
			RepLayout->ClearSharedSerializationForRPC();

			// Return here so we don't call InternalProcessRemoteFunction again at the bottom of this function
			return;
		}

		// Send function data to remote.
		Connection = Actor->GetNetConnection();
		if (Connection)
		{
			if (ServerConnection)
			{
				Connection = ServerConnection;
			}
			InternalProcessRemoteFunction(Actor, SubObject, Connection, Function, Parameters, OutParms, Stack, bIsServer);
		}
		else
		{
			UE_LOGF(LogNet, Warning, "UNetDriver::ProcessRemoteFunction: No owning connection for actor %ls. Function %ls will not be processed.", *Actor->GetName(), *Function->GetName());
		}
	}
}

void UNetDriver::CreateInitialClientChannels()
{
	if (ServerConnection != nullptr)
	{
		for (const FChannelDefinition& ChannelDef : ChannelDefinitions)
		{
			if (ChannelDef.bInitialClient && (ChannelDef.ChannelClass != nullptr))
			{
				ServerConnection->CreateChannelByName(ChannelDef.ChannelName, EChannelCreateFlags::OpenedLocally, ChannelDef.StaticChannelIndex);
			}
		}
	}
}

void UNetDriver::CreateInitialServerChannels(UNetConnection* ClientConnection)
{
	if (ClientConnection != nullptr)
	{
		for (const FChannelDefinition& ChannelDef : ChannelDefinitions)
		{
			if (ChannelDef.bInitialServer && (ChannelDef.ChannelClass != nullptr))
			{
				ClientConnection->CreateChannelByName(ChannelDef.ChannelName, EChannelCreateFlags::OpenedLocally, ChannelDef.StaticChannelIndex);
			}
		}
	}
}

void UNetDriver::AddObjectToNetworkIdCache(UObject* Object)
{
	if (!GuidCache)
	{
		return;
	}

	if (!ensure(IsValid(Object)))
	{
		UE_LOGF(LogNet, Warning, "Cannot add an invalid object %ls to the network id cache.", *GetNameSafe(Object));
		return;
	}

	FNetworkGUID NetGUID = GuidCache->GetOrAssignNetGUID(Object);
	UE_LOGF(LogNet, Verbose, "Adding object %ls (NetGUID=%ls) to the NetGUID cache.", *Object->GetName(), *NetGUID.ToString());

	auto AddSubObjectsToNetGUIDCache = [this](const UE::Net::FSubObjectRegistry* SubObjects)
	{
		for (const UE::Net::FSubObjectRegistry::FEntry& SubObjectInfo : SubObjects->GetRegistryList())
		{
			UObject* SubObject = SubObjectInfo.GetSubObject();
			if (ensure(IsValid(SubObject)))
			{
				FNetworkGUID SubObjectNetGUID = GuidCache->GetOrAssignNetGUID(SubObject);
				UE_LOGF(LogNet, Verbose, "Adding sub-object %ls (NetGUID=%ls) to the NetGUID cache.", *SubObject->GetName(), *SubObjectNetGUID.ToString());
			}
			else
			{
				UE_LOGF(LogNet, Warning, "Cannot add an invalid sub-object %ls to the network id cache.", *GetNameSafe(SubObject));
			}
		}
	};

	if (AActor* Actor = Cast<AActor>(Object))
	{
		const UE::Net::FSubObjectRegistry& ActorSubObjects = UE::Net::FSubObjectRegistryGetter::GetSubObjects(Actor);
		AddSubObjectsToNetGUIDCache(&ActorSubObjects);
	}
	else if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
	{
		const UE::Net::FSubObjectRegistry* ActorComponentSubObjects = UE::Net::FSubObjectRegistryGetter::GetSubObjectsOfActorComponent(ActorComponent->GetOwner(), ActorComponent);
		if (ActorComponentSubObjects)
		{
			AddSubObjectsToNetGUIDCache(ActorComponentSubObjects);
		}
	}
}

void UNetDriver::RemoveObjectFromNetworkIdCache(UObject* Object)
{
	if (!Object)
	{
		return;
	}

	if (!GuidCache)
	{
		return;
	}

	if (GuidCache->RemoveNetGUID(Object))
	{
		UE_LOGF(LogNet, Verbose, "Removed object %ls from the NetGUID cache.", *Object->GetName());
	}
	else
	{
		UE_LOGF(LogNet, Verbose, "Cannot find object %ls in the NetGUID cache.", *Object->GetName());
	}

	auto RemoveSubObjectsFromNetGUIDCache = [this](const UE::Net::FSubObjectRegistry* SubObjects)
	{
		for (const UE::Net::FSubObjectRegistry::FEntry& SubObjectInfo : SubObjects->GetRegistryList())
		{
			UObject* SubObject = SubObjectInfo.GetSubObject();
			if (SubObject)
			{
				if (GuidCache->RemoveNetGUID(SubObject))
				{
					UE_LOGF(LogNet, Verbose, "Removed sub-object %ls from the NetGUID cache.", *SubObject->GetName());
				}
				else
				{
					UE_LOGF(LogNet, Verbose, "Cannot find sub-object %ls in the NetGUID cache.", *SubObject->GetName());
				}
			}
		}
	};

	if (AActor* Actor = Cast<AActor>(Object))
	{
		const UE::Net::FSubObjectRegistry& ActorSubObjects = UE::Net::FSubObjectRegistryGetter::GetSubObjects(Actor);
		RemoveSubObjectsFromNetGUIDCache(&ActorSubObjects);
	}
	else if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
	{
		const UE::Net::FSubObjectRegistry* ActorComponentSubObjects = UE::Net::FSubObjectRegistryGetter::GetSubObjectsOfActorComponent(ActorComponent->GetOwner(), ActorComponent);
		if (ActorComponentSubObjects)
		{
			RemoveSubObjectsFromNetGUIDCache(ActorComponentSubObjects);
		}
	}
}

bool UNetDriver::ShouldReplicateFunction(AActor* Actor, UFunction* Function) const
{
	return (Actor && Actor->GetNetDriverName() == NetDriverName);
}

bool UNetDriver::ShouldForwardFunction(AActor* Actor, UFunction* Function, void* Parms) const
{
	return !IsServer();
}

void UNetDriver::ForwardRemoteFunction(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parms)
{
	AActor* OwningActor = Cast<AActor>(RootObject);
	if (!OwningActor)
	{
		//  Only actors can forward RPCs to the demo netdriver
		return;
	}

	if (!ShouldForwardFunction(OwningActor, Function, Parms))
	{
		return;
	}

	if (FWorldContext* Context = GEngine->GetWorldContextFromWorld(GetWorld()))
	{
		for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
		{
			if (Driver.NetDriver != nullptr && Driver.NetDriver != this && Driver.NetDriver->ShouldReplicateFunction(OwningActor, Function))
			{
				SCOPE_CYCLE_COUNTER(STAT_NetReceiveRPC_ProcessRemoteFunction);
				Driver.NetDriver->ProcessRemoteFunction(OwningActor, Function, Parms, static_cast<FOutParmRec*>(nullptr), static_cast<FFrame*>(nullptr), SubObject);
			}
		}
	}
}

bool UNetDriver::ShouldReplicateActor(AActor* Actor) const
{
	return (Actor && Actor->GetNetDriverName() == NetDriverName);
}

bool UNetDriver::ShouldCallRemoteFunction(UObject* Object, UFunction* Function, const FReplicationFlags& RepFlags) const
{
	return ((!IsServer() || RepFlags.bNetOwner) && !RepFlags.bIgnoreRPCs);
}

bool UNetDriver::ShouldClientDestroyActor(AActor* Actor, EChannelCloseReason CloseReason) const
{
	return (Actor && !Actor->IsA(ALevelScriptActor::StaticClass()));
}

void UNetDriver::NotifyActorChannelOpen(UActorChannel* Channel, AActor* Actor)
{

}

void UNetDriver::NotifyActorChannelCleanedUp(UActorChannel* Channel, EChannelCloseReason CloseReason)
{
	if (Channel && Channel->Connection)
	{
		Channel->Connection->NotifyActorChannelCleanedUp(Channel, CloseReason);
	}
}

void UNetDriver::ClientSetActorTornOff(AActor* Actor)
{
	check(Actor);

	if (IsServer())
	{
		UE_LOGF(LogNet, Error, "ClientSetActorTornOff should only be called on clients.");
		return;
	}

	if (Actor->GetIsReplicated() && (Actor->GetRemoteRole() == ROLE_Authority))
	{
		Actor->SetRole(ROLE_Authority);
		Actor->SetReplicates(false);

		if (Actor->GetWorld() != nullptr && !IsEngineExitRequested())
		{
			Actor->TornOff();
		}

		if (World)
		{
			if (FWorldContext* const Context = GEngine->GetWorldContextFromWorld(World))
			{
				for (FNamedNetDriver& Driver : Context->ActiveNetDrivers)
				{
					if (Driver.NetDriver != nullptr && Driver.NetDriver->ShouldReplicateActor(Actor))
					{
						Driver.NetDriver->NotifyActorTornOff(Actor);
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogNet, Warning, "ClientSetActorTornOff called with invalid actor: %ls", *GetNameSafe(Actor));
	}
}

void UNetDriver::NotifyActorTornOff(AActor* Actor)
{

}

void UNetDriver::ConsumeAsyncLoadDelinquencyAnalytics(FNetAsyncLoadDelinquencyAnalytics& Out)
{
	if (FNetGUIDCache* LocalGuidCache = GetNetGuidCache().Get())
	{
		LocalGuidCache->ConsumeAsyncLoadDelinquencyAnalytics(Out);
	}
	else
	{
		Out.Reset();
	}
}

const FNetAsyncLoadDelinquencyAnalytics& UNetDriver::GetAsyncLoadDelinquencyAnalytics() const
{
	static const FNetAsyncLoadDelinquencyAnalytics Empty(0);

	if (FNetGUIDCache const * const LocalGuidCache = GetNetGuidCache().Get())
	{
		return LocalGuidCache->GetAsyncLoadDelinquencyAnalytics();
	}

	return Empty;
}

void UNetDriver::ResetAsyncLoadDelinquencyAnalytics()
{
	if (FNetGUIDCache* LocalGuidCache = GetNetGuidCache().Get())
	{
		LocalGuidCache->ResetAsyncLoadDelinquencyAnalytics();
	}
}

bool UNetDriver::DidHitchLastFrame() const
{
	return bDidHitchLastFrame;
}

bool UNetDriver::IsDormInitialStartupActor(AActor* Actor)
{
	return Actor && Actor->IsNetStartupActor() && (Actor->NetDormancy == DORM_Initial);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UNetDriver::SetNetServerMaxTickRate(int32 InServerMaxTickRate)
{
	if (NetServerMaxTickRate != InServerMaxTickRate)
	{
		const int32 OldTickRate = NetServerMaxTickRate;
		NetServerMaxTickRate = InServerMaxTickRate;

		DDoS.SetMaxTickRate(NetServerMaxTickRate);

		OnNetServerMaxTickRateChanged.Broadcast(this, NetServerMaxTickRate, OldTickRate);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

ECreateReplicationChangelistMgrFlags UNetDriver::GetCreateReplicationChangelistMgrFlags() const
{
	return ECreateReplicationChangelistMgrFlags::None;
}

static TAutoConsoleVariable<int32> CVarRelinkMappedReferences(TEXT("net.RelinkMappedReferences"), 1, TEXT(""));

void UNetDriver::MoveMappedObjectToUnmapped(const UObject* Object)
{
	if (!Object)
	{
		return;
	}

	if (!CVarRelinkMappedReferences.GetValueOnGameThread())
	{
		return;
	}

	if (IsServer())
	{
		return;
	}

	// Find all replicators that are referencing this object, and make sure to mark the references as unmapped
	// This is so when/if this object is instantiated again (using same network guid), we can re-establish the old references
	FNetworkGUID NetGuid = GetNetGuidCache()->NetGUIDLookup.FindRef(const_cast<UObject*>(Object));

	if (NetGuid.IsValid())
	{
		TSet<FObjectReplicator*>* Replicators = GuidToReplicatorMap.Find(NetGuid);

		if (Replicators != nullptr)
		{
			for (FObjectReplicator* Replicator : *Replicators)
			{
				if (Replicator->MoveMappedObjectToUnmapped(NetGuid))
				{
					UnmappedReplicators.Add(Replicator);
				}
				else if (!UnmappedReplicators.Contains(Replicator))
				{
					UE_LOGF(LogNet, Warning, "UActorChannel::MoveMappedObjectToUnmapped: MoveMappedObjectToUnmapped didn't find object: %ls", *GetPathNameSafe(Replicator->GetObject()));
				}
			}
		}
	}
}

bool UNetDriver::IsEncryptionRequired() const
{
	using namespace UE::Net;

	if (GRequiredEncryptionNetDriverDefNames.Num() == 0)
	{
		ParseRequiredEncryptionCVars();
	}

	const FWorldContext* const WorldContext = (NetDriverName == NAME_PendingNetDriver ? GEngine->GetWorldContextFromPendingNetGameNetDriver(this) :
												GEngine->GetWorldContextFromWorld(World));
	const int32 AllowEncryption = CVarNetAllowEncryption.GetValueOnGameThread();
	const bool bDevDisableEncryptionCheck = !IsServer() && !(UE_BUILD_SHIPPING || UE_BUILD_TEST); // Clientside only
	const bool bPIEDisableEncryptionCheck = WorldContext != nullptr && WorldContext->WorldType == EWorldType::PIE;
	const bool bNetDriverDefNameRequiresEncryption = GRequiredEncryptionNetDriverDefNames.Contains(NetDriverDefinition);

	return DoesSupportEncryption() && bNetDriverDefNameRequiresEncryption && AllowEncryption == 2 && !bDevDisableEncryptionCheck &&
			!bPIEDisableEncryptionCheck;
}

float UNetDriver::GetIncomingBunchFrameProcessingTimeLimit() const
{
	using namespace UE::Net::Private;

	if (!IsServer())
	{
		return ClientIncomingBunchFrameTimeLimitMS;
	}

	return 0.0f;
}

bool UNetDriver::HasExceededIncomingBunchFrameProcessingTime() const
{
	const float TimeLimit = GetIncomingBunchFrameProcessingTimeLimit();
	return (TimeLimit > 0.0f) && (IncomingBunchProcessingElapsedFrameTimeMS > TimeLimit);
}

void UNetDriver::UpdateNetworkStats()
{
	using namespace UE::Net::Private;

	bool bCollectServerStats = false;
#if USE_SERVER_PERF_COUNTERS || STATS
	bCollectServerStats = true;
#endif

	if (bCollectNetStats || bCollectServerStats || GbEnableNetStats)
	{
		SCOPE_CYCLE_COUNTER(STAT_NetTickFlushGatherStats);

		++StatUpdateFrames;

		NumFramesOverIncomingBunchTimeLimit += HasExceededIncomingBunchFrameProcessingTime() ? 1 : 0;

		GetMetrics()->SetInt(UE::Net::Metric::InPacketsPerFrame, InPackets - PrevInPackets);
		GetMetrics()->SetInt(UE::Net::Metric::OutPacketsPerFrame, OutPackets - PrevOutPackets);
		GetMetrics()->SetInt(UE::Net::Metric::InLostPacketsFoundPerFrame, InPacketsLost - PrevInPacketsLost);
		GetMetrics()->SetInt(UE::Net::Metric::OutLostPacketsFoundPerFrame, OutPacketsLost - PrevOutPacketsLost);

		PrevInPackets = InPackets;
		PrevOutPackets = OutPackets;
		PrevInPacketsLost = InPacketsLost;
		PrevOutPacketsLost = OutPacketsLost;

		const double CurrentRealtimeSeconds = FPlatformTime::Seconds();
		// Update network stats (only main game net driver for now) if stats or perf counters are used
		if (ShouldUpdateStats() &&
			CurrentRealtimeSeconds - StatUpdateTime > StatPeriod)
		{
			int32 ClientInBytesMax = 0;
			int32 ClientInBytesMin = 0;
			int32 ClientInBytesAvg = 0;
			int32 ClientInPacketsMax = 0;
			int32 ClientInPacketsMin = 0;
			int32 ClientInPacketsAvg = 0;
			int32 ClientsInPacketsThisFrameAvg = 0;
			int32 ClientsInPacketsThisFrameMax = 0;
			int32 ClientOutBytesMax = 0;
			int32 ClientOutBytesMin = 0;
			int32 ClientOutBytesAvg = 0;
			int32 ClientOutPacketsMax = 0;
			int32 ClientOutPacketsMin = 0;
			int32 ClientOutPacketsAvg = 0;
			int32 ClientsOutPacketsThisFrameAvg = 0;
			int32 ClientsOutPacketsThisFrameMax = 0;
			int NumClients = 0;
			int32 MaxPacketOverhead = 0;
			int32 InBunchTimeOvershootPercent = 0;

			// these need to be updated even if we are not collecting stats, since they get reported to analytics/QoS
			for (const UNetConnection* Client : ClientConnections)
			{
				if (Client)
				{
#define UpdatePerClientMinMaxAvg(VariableName) \
				Client##VariableName##Max = FMath::Max(Client##VariableName##Max, Client->VariableName##PerSecond); \
				if (Client##VariableName##Min == 0 || Client->VariableName##PerSecond < Client##VariableName##Min) \
				{ \
					Client##VariableName##Min = Client->VariableName##PerSecond; \
				} \
				Client##VariableName##Avg += Client->VariableName##PerSecond; \

					UpdatePerClientMinMaxAvg(InBytes);
					UpdatePerClientMinMaxAvg(OutBytes);
					UpdatePerClientMinMaxAvg(InPackets);
					UpdatePerClientMinMaxAvg(OutPackets);

					MaxPacketOverhead = FMath::Max(Client->PacketOverhead, MaxPacketOverhead);
#undef UpdatePerClientMinMaxAvg

					ClientsInPacketsThisFrameAvg += Client->InPacketsThisFrame;
					ClientsOutPacketsThisFrameAvg += Client->OutPacketsThisFrame;
					ClientsInPacketsThisFrameMax = FMath::Max(ClientsInPacketsThisFrameMax, Client->InPacketsThisFrame);
					ClientsOutPacketsThisFrameMax = FMath::Max(ClientsOutPacketsThisFrameMax, Client->OutPacketsThisFrame);

					++NumClients;
				}
			}

			if (NumClients > 1)
			{
				ClientInBytesAvg /= NumClients;
				ClientInPacketsAvg /= NumClients;
				ClientOutBytesAvg /= NumClients;
				ClientOutPacketsAvg /= NumClients;
				ClientsInPacketsThisFrameAvg  /= NumClients;
				ClientsOutPacketsThisFrameAvg /= NumClients;
			}

			int32 Ping = 0;
			int32 NumOpenChannels = 0;
			int32 NumActorChannels = 0;
			int32 NumDormantActors = 0;
			int32 NumActors = 0;
			int32 AckCount = 0;
			int32 UnAckCount = 0;
			int32 PendingCount = 0;
			int32 NetSaturated = 0;
			int32 NumActiveNetActors = GetNetworkObjectList().GetActiveObjects().Num();

			if (
#if STATS
				FThreadStats::IsCollectingData() ||
#endif
				bCollectNetStats || GbEnableNetStats)
			{
				const float RealTime = CurrentRealtimeSeconds - StatUpdateTime;

				// Use the elapsed time to keep things scaled to one measured unit
				InBytes = FMath::TruncToInt(InBytes / RealTime);
				OutBytes = FMath::TruncToInt(OutBytes / RealTime);

				NetGUIDOutBytes = FMath::TruncToInt(NetGUIDOutBytes / RealTime);
				NetGUIDInBytes = FMath::TruncToInt(NetGUIDInBytes / RealTime);

				// Save off for stats later

				InBytesPerSecond = InBytes;
				OutBytesPerSecond = OutBytes;

				InPackets = FMath::TruncToInt(InPackets / RealTime);
				OutPackets = FMath::TruncToInt(OutPackets / RealTime);
				InBunches = FMath::TruncToInt(InBunches / RealTime);
				OutBunches = FMath::TruncToInt(OutBunches / RealTime);
				OutPacketsLost = FMath::TruncToInt(100.f * OutPacketsLost / FMath::Max((float)OutPackets, 1.f));
				InPacketsLost = FMath::TruncToInt(100.f * InPacketsLost / FMath::Max((float)InPackets + InPacketsLost, 1.f));

				GetMetrics()->SetInt(UE::Net::Metric::InPackets, InPackets);
				GetMetrics()->SetInt(UE::Net::Metric::OutPackets, OutPackets);
				GetMetrics()->SetInt(UE::Net::Metric::InBunches, InBunches);
				GetMetrics()->SetInt(UE::Net::Metric::OutBunches, OutBunches);
				GetMetrics()->SetInt(UE::Net::Metric::OutPacketsLost, OutPacketsLost);
				GetMetrics()->SetInt(UE::Net::Metric::InPacketsLost, InPacketsLost);

				if (ServerConnection && ServerConnection->PlayerController && ServerConnection->PlayerController->PlayerState)
				{
					Ping = FMath::TruncToInt(ServerConnection->PlayerController->PlayerState->ExactPing);
				}

				if (ServerConnection)
				{
					NumOpenChannels = ServerConnection->OpenChannels.Num();
					GetMetrics()->SetFloat(UE::Net::Metric::AverageJitterInMS, ServerConnection->GetAverageJitterInMS());
				}

				for (int32 i = 0; i < ClientConnections.Num(); i++)
				{
					NumOpenChannels += ClientConnections[i]->OpenChannels.Num();
				}

				// Use the elapsed time to keep things scaled to one measured unit
				VoicePacketsSent = FMath::TruncToInt(VoicePacketsSent / RealTime);
				VoicePacketsRecv = FMath::TruncToInt(VoicePacketsRecv / RealTime);
				VoiceBytesSent = FMath::TruncToInt(VoiceBytesSent / RealTime);
				VoiceBytesRecv = FMath::TruncToInt(VoiceBytesRecv / RealTime);

				// Determine voice percentages
				VoiceInPercent = (InBytes > 0) ? FMath::TruncToInt(100.f * (float)VoiceBytesRecv / (float)InBytes) : 0;
				VoiceOutPercent = (OutBytes > 0) ? FMath::TruncToInt(100.f * (float)VoiceBytesSent / (float)OutBytes) : 0;

				// Percent of frames this stat period that the incoming bunch processing time limit was exceeded (0-100)
				InBunchTimeOvershootPercent = (StatUpdateFrames > 0) ? FMath::TruncToInt(100.0f * (float)NumFramesOverIncomingBunchTimeLimit / (float)StatUpdateFrames) : 0;

				if (World)
				{
					NumActors = World->GetActorCount();
				}
				
				// Handle these stats to do this differently for client & server
				if (ServerConnection != nullptr)
				{
					NumActorChannels = ServerConnection->ActorChannelsNum();
					NumDormantActors = GetNetworkObjectList().GetNumDormantActorsForConnection(ServerConnection);
#if STATS
					ServerConnection->PackageMap->GetNetGUIDStats(AckCount, UnAckCount, PendingCount);
#endif
					NetSaturated = ServerConnection->IsNetReady() ? 0 : 1;
				}
				else
				{
					int32 SharedDormantActors = GetNetworkObjectList().GetDormantObjectsOnAllConnections().Num();
					for (int32 i = 0; i < ClientConnections.Num(); i++)
					{
						NumActorChannels += ClientConnections[i]->ActorChannelsNum();
						NumDormantActors += GetNetworkObjectList().GetNumDormantActorsForConnection(ClientConnections[i]);
#if STATS
						int32 ClientAckCount = 0;
						int32 ClientUnAckCount = 0;
						int32 ClientPendingCount = 0;
						ClientConnections[i]->PackageMap->GetNetGUIDStats(AckCount, UnAckCount, PendingCount);
						AckCount += ClientAckCount;
						UnAckCount += ClientUnAckCount;
						PendingCount += ClientPendingCount;
#endif
						NetSaturated |= ClientConnections[i]->IsNetReady() ? 0 : 1;
					}
					// Subtract out the duplicate counted dormant actors
					NumDormantActors -= FMath::Max(0, ClientConnections.Num() - 1) * SharedDormantActors;
				}
			}
			
			GetMetrics()->SetInt(UE::Net::Metric::Ping, Ping);
			GetMetrics()->SetInt(UE::Net::Metric::InRate, InBytesPerSecond);
			GetMetrics()->SetInt(UE::Net::Metric::OutRate, OutBytesPerSecond);

#if STATS
			if (!bSkipLocalStats)
			{
				// Copy the net status values over
				GetMetrics()->SetInt(UE::Net::Metric::Channels, NumOpenChannels);
				GetMetrics()->SetInt(UE::Net::Metric::MaxPacketOverhead, MaxPacketOverhead);

				GetMetrics()->SetInt(UE::Net::Metric::NetNumClients, NumClients);

				GetMetrics()->SetInt(UE::Net::Metric::NetGUIDInRate, NetGUIDInBytes);
				GetMetrics()->SetInt(UE::Net::Metric::NetGUIDOutRate, NetGUIDOutBytes);

				GetMetrics()->SetInt(UE::Net::Metric::VoicePacketsSent, VoicePacketsSent);
				GetMetrics()->SetInt(UE::Net::Metric::VoicePacketsRecv, VoicePacketsRecv);
				GetMetrics()->SetInt(UE::Net::Metric::VoiceBytesSent, VoiceBytesSent);
				GetMetrics()->SetInt(UE::Net::Metric::VoiceBytesRecv, VoiceBytesRecv);

				GetMetrics()->SetInt(UE::Net::Metric::PercentInVoice, VoiceInPercent);
				GetMetrics()->SetInt(UE::Net::Metric::PercentOutVoice, VoiceOutPercent);

				GetMetrics()->SetInt(UE::Net::Metric::NumActorChannels, NumActorChannels);
				GetMetrics()->SetInt(UE::Net::Metric::NumDormantActors, NumDormantActors);
				GetMetrics()->SetInt(UE::Net::Metric::NumActors, NumActors);
				GetMetrics()->SetInt(UE::Net::Metric::NumNetActors, NumActiveNetActors);
				GetMetrics()->SetInt(UE::Net::Metric::NumNetGUIDsAckd, AckCount);
				GetMetrics()->SetInt(UE::Net::Metric::NumNetGUIDsPending, UnAckCount);
				GetMetrics()->SetInt(UE::Net::Metric::NumNetGUIDsUnAckd, PendingCount);
				GetMetrics()->SetInt(UE::Net::Metric::NetSaturated, NetSaturated);

				GetMetrics()->SetInt(UE::Net::Metric::NetInBunchTimeOvershootPercent, InBunchTimeOvershootPercent);
			}

			// If we are to replicate server stats out to an observer, then set those values
			if (AGameModeBase* GMBase = World->GetAuthGameMode())
			{
				if (GMBase->ServerStatReplicator != nullptr)
				{
					GMBase->ServerStatReplicator->Channels = NumOpenChannels;
					GMBase->ServerStatReplicator->MaxPacketOverhead = MaxPacketOverhead;
					GMBase->ServerStatReplicator->OutLoss = OutPacketsLost;
					GMBase->ServerStatReplicator->InLoss = InPacketsLost;
					GMBase->ServerStatReplicator->InRate = InBytesPerSecond;
					GMBase->ServerStatReplicator->OutRate = OutBytesPerSecond;
					GMBase->ServerStatReplicator->InRateClientMax = ClientInBytesMax;
					GMBase->ServerStatReplicator->InRateClientMin = ClientInBytesMin;
					GMBase->ServerStatReplicator->InRateClientAvg = ClientInBytesAvg;
					GMBase->ServerStatReplicator->InPacketsClientMax = ClientInPacketsMax;
					GMBase->ServerStatReplicator->InPacketsClientMin = ClientInPacketsMin;
					GMBase->ServerStatReplicator->InPacketsClientAvg = ClientInPacketsAvg;
					GMBase->ServerStatReplicator->OutRateClientMax = ClientOutBytesMax;
					GMBase->ServerStatReplicator->OutRateClientMin = ClientOutBytesMin;
					GMBase->ServerStatReplicator->OutRateClientAvg = ClientOutBytesAvg;
					GMBase->ServerStatReplicator->OutPacketsClientMax = ClientOutPacketsMax;
					GMBase->ServerStatReplicator->OutPacketsClientMin = ClientOutPacketsMin;
					GMBase->ServerStatReplicator->OutPacketsClientAvg = ClientOutPacketsAvg;
					GMBase->ServerStatReplicator->NetNumClients = NumClients;
					GMBase->ServerStatReplicator->InPackets = InPackets;
					GMBase->ServerStatReplicator->OutPackets = OutPackets;
					GMBase->ServerStatReplicator->InBunches = InBunches;
					GMBase->ServerStatReplicator->OutBunches = OutBunches;
					GMBase->ServerStatReplicator->NetGUIDInRate = NetGUIDInBytes;
					GMBase->ServerStatReplicator->NetGUIDOutRate = NetGUIDOutBytes;
					GMBase->ServerStatReplicator->VoicePacketsSent = VoicePacketsSent;
					GMBase->ServerStatReplicator->VoicePacketsRecv = VoicePacketsRecv;
					GMBase->ServerStatReplicator->VoiceBytesSent = VoiceBytesSent;
					GMBase->ServerStatReplicator->VoiceBytesRecv = VoiceBytesRecv;
					GMBase->ServerStatReplicator->PercentInVoice = VoiceInPercent;
					GMBase->ServerStatReplicator->PercentOutVoice = VoiceOutPercent;
					GMBase->ServerStatReplicator->NumActorChannels = NumActorChannels;
					GMBase->ServerStatReplicator->NumDormantActors = NumDormantActors;
					GMBase->ServerStatReplicator->NumActors = NumActors;
					GMBase->ServerStatReplicator->NumNetActors = NumActiveNetActors;
					GMBase->ServerStatReplicator->NumNetGUIDsAckd = AckCount;
					GMBase->ServerStatReplicator->NumNetGUIDsPending = UnAckCount;
					GMBase->ServerStatReplicator->NumNetGUIDsUnAckd = PendingCount;
					GMBase->ServerStatReplicator->NetSaturated = NetSaturated;
				}
			}
#endif // STATS

#if USE_SERVER_PERF_COUNTERS
			{
				SCOPE_CYCLE_COUNTER(STAT_NetTickFlushGatherStatsPerfCounters);

				// Update total connections
				GetMetrics()->SetInt(UE::Net::Metric::Connections, ClientConnections.Num());
				GetMetrics()->SetInt(UE::Net::Metric::NumConnections, ClientConnections.Num());

				const int kNumBuckets = 8;	// evenly spaced with increment of 30 ms; last bucket collects all off-scale pings as well
				if (ClientConnections.Num() > 0)
				{
					int32 NumAllConnections = 0;

					// Update per connection statistics
					float MinPing = UE_MAX_FLT;
					float AvgPing = 0;
					float MaxPing = -UE_MAX_FLT;
					float PingCount = 0;

					int32 Buckets[kNumBuckets] = { 0 };

					for (int32 i = 0; i < ClientConnections.Num(); i++)
					{
						UNetConnection* Connection = ClientConnections[i];

						NumAllConnections++;

						if (Connection)
						{
							NumAllConnections += Connection->Children.Num();
						}

						if (Connection != nullptr)
						{
							if (Connection->PlayerController && Connection->PlayerController->PlayerState)
							{
#if UE_WITH_REMOTE_OBJECT_HANDLE
								if (Connection->PlayerController.GetResidence() == EResidence::Remote)
								{
									ensureMsgf(false, TEXT("Expecting the Connection's PlayerController to be locally owned by now"));
									continue;
								}
								else if (Connection->PlayerController->PlayerState.GetResidence() == EResidence::Remote)
								{
									continue;
								}
#endif

								// Ping value calculated per client
								float ConnPing = Connection->PlayerController->PlayerState->ExactPing;

								int Bucket = FMath::Max(0, FMath::Min(kNumBuckets - 1, (static_cast<int>(ConnPing) / 30)));
								++Buckets[Bucket];

								if (ConnPing < MinPing)
								{
									MinPing = ConnPing;
								}

								if (ConnPing > MaxPing)
								{
									MaxPing = ConnPing;
								}

								AvgPing += ConnPing;
								PingCount++;
							}
						}
					}

					if (PingCount > 0)
					{
						AvgPing /= static_cast<float>(PingCount);
					}

					GetMetrics()->SetInt(UE::Net::Metric::NumAllConnections, NumAllConnections);

					GetMetrics()->SetFloat(UE::Net::Metric::AvgPing, AvgPing);
					GetMetrics()->SetFloat(UE::Net::Metric::MaxPing, MaxPing);
					GetMetrics()->SetFloat(UE::Net::Metric::MinPing, MinPing);

					// Unrolling the loop through ping buckets to prevent dynamically creating 
					// the metric name string from the ping bucket id using FString::Printf().
					if (ensure(kNumBuckets == 8))
					{
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt0, Buckets[0]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt1, Buckets[1]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt2, Buckets[2]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt3, Buckets[3]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt4, Buckets[4]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt5, Buckets[5]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt6, Buckets[6]);
						GetMetrics()->IncrementInt(UE::Net::Metric::PingBucketInt7, Buckets[7]);
					}
				}
				else
				{
					GetMetrics()->SetInt(UE::Net::Metric::NumAllConnections, 0);

					GetMetrics()->SetFloat(UE::Net::Metric::AvgPing, 0.0f);
					GetMetrics()->SetFloat(UE::Net::Metric::MaxPing, -FLT_MAX);
					GetMetrics()->SetFloat(UE::Net::Metric::MinPing, FLT_MAX);

					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt0, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt1, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt2, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt3, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt4, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt5, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt6, 0);
					GetMetrics()->SetInt(UE::Net::Metric::PingBucketInt7, 0);
				}

				// set the per connection stats (these are calculated earlier).
				// Note that NumClients may be != NumConnections. Also, if NumClients is 0, the rest of counters should be 0 as well
				GetMetrics()->SetInt(UE::Net::Metric::NumClients, NumClients);
				GetMetrics()->SetInt(UE::Net::Metric::MaxPacketOverhead, MaxPacketOverhead);
				GetMetrics()->SetInt(UE::Net::Metric::InRateClientMax, ClientInBytesMax);
				GetMetrics()->SetInt(UE::Net::Metric::InRateClientMin, ClientInBytesMin);
				GetMetrics()->SetInt(UE::Net::Metric::InRateClientAvg, ClientInBytesAvg);
				GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientPerSecondMax, ClientInPacketsMax);
				GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientPerSecondMin, ClientInPacketsMin);
				GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientPerSecondAvg, ClientInPacketsAvg);
				GetMetrics()->SetInt(UE::Net::Metric::OutRateClientMax, ClientOutBytesMax);
				GetMetrics()->SetInt(UE::Net::Metric::OutRateClientMin, ClientOutBytesMin);
				GetMetrics()->SetInt(UE::Net::Metric::OutRateClientAvg, ClientOutBytesAvg);
				GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientPerSecondMax, ClientOutPacketsMax);
				GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientPerSecondMin, ClientOutPacketsMin);
				GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientPerSecondAvg, ClientOutPacketsAvg);
			}
#endif // USE_SERVER_PERF_COUNTERS

			GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientAvg, ClientsInPacketsThisFrameAvg);
			GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientMax, ClientsInPacketsThisFrameMax);
			GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientAvg, ClientsOutPacketsThisFrameAvg);
			GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientMax, ClientsOutPacketsThisFrameMax);

			// Reset everything
			InBytes = 0;
			OutBytes = 0;
			NetGUIDOutBytes = 0;
			NetGUIDInBytes = 0;
			InPackets = 0;
			PrevInPackets = 0;
			OutPackets = 0;
			PrevOutPackets = 0;
			InBunches = 0;
			OutBunches = 0;
			OutPacketsLost = 0;
			PrevOutPacketsLost = 0;
			InPacketsLost = 0;
			PrevInPacketsLost = 0;
			VoicePacketsSent = 0;
			VoiceBytesSent = 0;
			VoicePacketsRecv = 0;
			VoiceBytesRecv = 0;
			VoiceInPercent = 0;
			VoiceOutPercent = 0;
			NumFramesOverIncomingBunchTimeLimit = 0;
			StatUpdateTime = CurrentRealtimeSeconds;
			StatUpdateFrames = 0;

			// Record the number of player controllers in the world and registered for replication.
			if (GbEnableActorCountInStats)
			{
				int32 NumPlayerControllersForReplication = 0;
				int32 NumNoPawnPlayerControllersForReplication = 0;
				int32 NumPawnsForReplication = 0;

				for (const TSharedPtr<FNetworkObjectInfo>& ObjectInfo : GetNetworkObjectList().GetActiveObjects())
				{
					AActor* Actor = ObjectInfo->Actor;

					if (Actor->IsA(ANoPawnPlayerController::StaticClass()))
					{
						NumNoPawnPlayerControllersForReplication++;
					}
					else if (Actor->IsA(APlayerController::StaticClass()))
					{
						NumPlayerControllersForReplication++;
					}
					else if (Actor->IsA(APawn::StaticClass()))
					{
						NumPawnsForReplication++;
					}
				}

				GetMetrics()->SetInt(UE::Net::Metric::NumPlayerControllersForReplication, NumPlayerControllersForReplication);
				GetMetrics()->SetInt(UE::Net::Metric::NumNoPawnPlayerControllersForReplication, NumNoPawnPlayerControllersForReplication);
				GetMetrics()->SetInt(UE::Net::Metric::NumPawnsForReplication, NumPawnsForReplication);
			}
		}
		else
		{

			// CSV stats need to be collected every frame

			int32 ClientsInPacketsThisFrameAvg = 0;
			int32 ClientsInPacketsThisFrameMax = 0;
			int32 ClientsOutPacketsThisFrameAvg = 0;
			int32 ClientsOutPacketsThisFrameMax = 0;

			// Collect raw pings from all clients
			int32 ClientsRawPingThisFrameMin = std::numeric_limits<int32>::max();
			int32 ClientsRawPingThisFrameMax = 0;
			float ClientsRawPingThisFrameAvg = 0;

			int32 NumClients = 0;

			for (const UNetConnection* Client : ClientConnections)
			{
				if (Client)
				{
					++NumClients;

#if CSV_PROFILER_STATS
					ClientsInPacketsThisFrameAvg += Client->InPacketsThisFrame;
					ClientsOutPacketsThisFrameAvg += Client->OutPacketsThisFrame;
					ClientsInPacketsThisFrameMax = FMath::Max(ClientsInPacketsThisFrameMax, Client->InPacketsThisFrame);
					ClientsOutPacketsThisFrameMax = FMath::Max(ClientsOutPacketsThisFrameMax, Client->OutPacketsThisFrame);
#endif  //#if CSV_PROFILER_STATS

					const float RawPingValueInMS = Client->RawPingInSeconds * 1000.f;
					ClientsRawPingThisFrameMin = FMath::Min(ClientsRawPingThisFrameMin, static_cast<int32>(RawPingValueInMS));
					ClientsRawPingThisFrameMax = FMath::Max(ClientsRawPingThisFrameMax, static_cast<int32>(RawPingValueInMS));
					ClientsRawPingThisFrameAvg += RawPingValueInMS;
				}
			}

			if (NumClients > 1)
			{
				ClientsInPacketsThisFrameAvg /= NumClients;
				ClientsOutPacketsThisFrameAvg /= NumClients;

				ClientsRawPingThisFrameAvg /= NumClients;
			}

#if CSV_PROFILER_STATS
			if (ServerConnection)
			{
				const int32 RawPingValueInMS = (int32)(ServerConnection->RawPingInSeconds * 1000.f);
				GetMetrics()->SetInt(UE::Net::Metric::RawPing, RawPingValueInMS);
			}

			GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientAvg, ClientsInPacketsThisFrameAvg);
			GetMetrics()->SetInt(UE::Net::Metric::InPacketsClientMax, ClientsInPacketsThisFrameMax);
			GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientAvg, ClientsOutPacketsThisFrameAvg);
			GetMetrics()->SetInt(UE::Net::Metric::OutPacketsClientMax, ClientsOutPacketsThisFrameMax);
#endif  //#if CSV_PROFILER_STATS

			if (!ClientConnections.IsEmpty())
			{
				GetMetrics()->SetInt(UE::Net::Metric::RawPingClientMin, ClientsRawPingThisFrameMin);
				GetMetrics()->SetInt(UE::Net::Metric::RawPingClientMax, ClientsRawPingThisFrameMax);
				GetMetrics()->SetFloat(UE::Net::Metric::RawPingClientAvg, ClientsRawPingThisFrameAvg);
			}
		}
	} // bCollectNetStats ||(USE_SERVER_PERF_COUNTERS) || STATS

}


void UNetDriver::ResetNetConnectionStats()
{
	if (ServerConnection)
	{
		ServerConnection->InPacketsThisFrame = 0;
		ServerConnection->OutPacketsThisFrame = 0;
	}
	else
	{
		for (UNetConnection* Client : ClientConnections)
		{
			if (Client)
			{
				Client->InPacketsThisFrame = 0;
				Client->OutPacketsThisFrame = 0;
			}
		}
	}
}

UE::Net::FNetIDVariant UNetDriver::CreateNetIDForObject(UObject* Object)
{
	using namespace UE::Net;

	if (bIsUsingIris)
	{
		if (ReplicationSystem)
		{
			UObjectReplicationBridge* ObjectBridge = ReplicationSystem->GetReplicationBridge();
			if (ObjectBridge)
			{
				FNetRefHandle AllocatedRefHandle = ObjectBridge->PreRegisterNewObjectReferenceHandle(Object);
				return FNetIDVariant(AllocatedRefHandle);
			}
			else
			{
				UE_LOGF(LogNet, Warning, "UNetDriver::CreateNetIDForObject: failed to assign a NetRefHandle for object %ls, no ObjectBridge. NetDriver %ls.", *GetFullNameSafe(Object), *GetFullName());
			}
		}
		else
		{
			UE_LOGF(LogNet, Warning, "UNetDriver::CreateNetIDForObject: failed to assign a NetRefHandle for object %ls, no ReplicationSystem. NetDriver %ls.", *GetFullNameSafe(Object), *GetFullName());
			ensureMsgf(false, TEXT("ReplicationSystem is null"));
		}

		return FNetIDVariant(FNetRefHandle());
	}
	else
	{
		FNetworkGUID GUID = GuidCache->AssignNewNetGUID_Server(Object);
		return FNetIDVariant(GUID);
	}
}

void UNetDriver::AssignNetIDToObject(UE::Net::FNetIDVariant NetID, const UObject* Object)
{
	if (bIsUsingIris)
	{
		if (NetID.GetVariant().IsType<UE::Net::FNetRefHandle>())
		{
			if (ReplicationSystem)
			{
				UObjectReplicationBridge* ObjectBridge = ReplicationSystem->GetReplicationBridge();
				if (ObjectBridge)
				{
					ObjectBridge->PreRegisterObjectWithReferenceHandle(Object, NetID.GetVariant().Get<UE::Net::FNetRefHandle>());
				}
				else
				{
					UE_LOGF(LogNet, Warning, "UNetDriver::AssignNetIDToObject: failed to register NetID %ls for object %ls with net driver %ls, no ObjectBridge.", *NetID.ToString(), *GetFullNameSafe(Object), *GetFullName());
				}
			}
			else
			{
				UE_LOGF(LogNet, Warning, "UNetDriver::AssignNetIDToObject: failed to register NetID %ls for object %ls with net driver %ls, no ReplicationSystem.", *NetID.ToString(), *GetFullNameSafe(Object), *GetFullName());
				ensureMsgf(false, TEXT("NReplicationSystem is null"));
			}
		}
		else
		{
			UE_LOGF(LogNet, Warning, "UNetDriver::AssignNetIDToObject: NetID %ls for object %ls is not an FNetRefHandle but NetDriver %ls is using Iris.", *NetID.ToString(), *GetFullNameSafe(Object), *GetFullName());
			ensureMsgf(false, TEXT("NetID is not an FNetRefHandle"));
		}
	}
	else
	{
		if (NetID.GetVariant().IsType<FNetworkGUID>())
		{
			GuidCache->RegisterNetGUID_Client(NetID.GetVariant().Get<FNetworkGUID>(), Object);
		}
		else
		{
			const FString ErrorMessage = FString::Printf(TEXT("UNetDriver::AssignNetIDToObject: NetID %s for object %s is not a NetGUID but NetDriver %s is using generic replication."), *NetID.ToString(), *GetFullNameSafe(Object), *GetFullName());
			UE_LOGF(LogNet, Warning, "%ls", *ErrorMessage);
			ensureMsgf(false, TEXT("%s"), *ErrorMessage);
		}
	}
}

#if NET_DEBUG_RELEVANT_ACTORS
FAutoConsoleCommandWithWorld	DumpRelevantActorsCommand(
	TEXT("net.DumpRelevantActors"), 
	TEXT( "Dumps information on relevant actors during next network update" ), 
	FConsoleCommandWithWorldDelegate::CreateStatic(DumpRelevantActors)
	);
#endif // NET_DEBUG_RELEVANT_ACTORS

/**
 * Exec handler that routes online specific execs to the proper subsystem
 *
 * @param InWorld World context
 * @param Cmd 	the exec command being executed
 * @param Ar 	the archive to log results to
 *
 * @return true if the handler consumed the input, false to continue searching handlers
 */
static bool NetDriverExec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bHandled = false;

	// Ignore any execs that don't start with NET
	if (FParse::Command(&Cmd, TEXT("NET")))
	{
		UNetDriver* NamedDriver = NULL;
		TCHAR TokenStr[128];

		// Route the command to a specific beacon if a name is specified or all of them otherwise
		if (FParse::Token(Cmd, TokenStr, UE_ARRAY_COUNT(TokenStr), true))
		{
			NamedDriver = GEngine->FindNamedNetDriver(InWorld, FName(TokenStr));
			if (NamedDriver != NULL)
			{
				bHandled = NamedDriver->Exec(InWorld, Cmd, Ar);
			}
			else
			{
				FWorldContext &Context = GEngine->GetWorldContextFromWorldChecked(InWorld);

				Cmd -= FCString::Strlen(TokenStr);
				for (int32 NetDriverIdx=0; NetDriverIdx < Context.ActiveNetDrivers.Num(); NetDriverIdx++)
				{
					NamedDriver = Context.ActiveNetDrivers[NetDriverIdx].NetDriver;
					if (NamedDriver)
					{
						bHandled |= NamedDriver->Exec(InWorld, Cmd, Ar);
					}
				}
			}
		}
	}

	return bHandled;
}

/** Our entry point for all net driver related exec routing */
FStaticSelfRegisteringExec NetDriverExecRegistration(NetDriverExec);


namespace UE::Net::Private
{

FAutoConsoleCommand PrintNetConnectionInfoCommand(
TEXT("net.PrintNetConnections"),
TEXT("Prints information on all net connections of a NetDriver.  Defaults to the GameNetDriver. Choose a different driver via NetDriverName= or NetDriverDefinition="),
FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray< FString >& Args)
{
	FName NetDriverName;

	// Default to print GameNetDriver if no params are set
	FName NetDriverDef = NAME_GameNetDriver;

	if (const FString* DriverNameStr = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("NetDriverName=")); }))
	{
		FParse::Value(**DriverNameStr, TEXT("NetDriverName="), NetDriverName);
	}
	else if (const FString* DriverDefStr = Args.FindByPredicate([](const FString& Str) { return Str.Contains(TEXT("NetDriverDef=")); }))
	{
		FParse::Value(**DriverDefStr, TEXT("NetDriverDef="), NetDriverDef);
	}

	auto PrintNetConnections = [](UNetDriver* NetDriver)
	{
		if (NetDriver->ServerConnection)
		{
			UE_LOGF(LogNet, Display, "Printing Server Connection for: %ls (Definition=%ls)", *NetDriver->NetDriverName.ToString(), *NetDriver->GetNetDriverDefinition().ToString());

			UE_LOGF(LogNet, Display, "\tServerConnection: ConnectionId=%u ViewTarget=%ls FullDescription=%ls Child=false", 
				NetDriver->ServerConnection->GetConnectionHandle().GetParentConnectionId(), 
				*GetNameSafe(NetDriver->ServerConnection->ViewTarget),
				*NetDriver->ServerConnection->Describe()
			);

			for (UNetConnection* ChildNetConnection : NetDriver->ServerConnection->Children)
			{
				UE_LOGF(LogNet, Display, "\tServerConnection: ConnectionId=%u ViewTarget=%ls FullDescription=%ls Child=true", 
					   ChildNetConnection->GetConnectionHandle().GetParentConnectionId(), 
					   *GetNameSafe(ChildNetConnection->ViewTarget),
					   *ChildNetConnection->Describe()
				);
			}
		}
		else
		{
			UE_LOGF(LogNet, Display, "Printing Client Connections (%u) for: %ls (Definition=%ls)", NetDriver->ClientConnections.Num(), *NetDriver->NetDriverName.ToString(), *NetDriver->GetNetDriverDefinition().ToString());

			for (UNetConnection* NetConnection : NetDriver->ClientConnections)
			{
				UE_LOGF(LogNet, Display, "\tClientConnection: ConnectionId=%u ViewTarget=%ls NetId=%ls FullDescription=%ls Child=false", 
					NetConnection->GetConnectionHandle().GetParentConnectionId(), 
					*GetNameSafe(NetConnection->ViewTarget),
					*NetConnection->PlayerId.ToDebugString(), 
					*NetConnection->Describe()
				);

				for (UNetConnection* ChildNetConnection : NetConnection->Children)
				{
					UE_LOGF(LogNet, Display, "\tClientConnection: ConnectionId=%u ViewTarget=%ls NetId=%ls FullDescription=%ls Child=true", 
						   ChildNetConnection->GetConnectionHandle().GetParentConnectionId(), 
						   *GetNameSafe(ChildNetConnection->ViewTarget),
						   *ChildNetConnection->PlayerId.ToDebugString(), 
						   *ChildNetConnection->Describe()
					);
				}
			}
		}
	};

	for (TObjectIterator<UNetDriver> It; It; ++It)
	{
		if (UNetDriver* NetDriver = *It)
		{
			if (NetDriverName != NAME_None)
			{
				if (NetDriver->NetDriverName == NetDriverName)
				{
					PrintNetConnections(NetDriver);
				}
			}
			else if (NetDriverDef != NAME_None)
			{
				if (NetDriver->GetNetDriverDefinition() == NetDriverDef)
				{
					PrintNetConnections(NetDriver);
				}
			}
			// Print all NetDrivers when both names are none
			else
			{
				PrintNetConnections(NetDriver);
			}
		}
	}
}));

} // end namespace UE::Net::Private