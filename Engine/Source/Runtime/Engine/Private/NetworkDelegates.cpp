// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/NetworkDelegates.h"

FNetDelegates::FReceivedNetworkEncryptionToken FNetDelegates::OnReceivedNetworkEncryptionToken;
FNetDelegates::FReceivedNetworkEncryptionAck FNetDelegates::OnReceivedNetworkEncryptionAck;
FNetDelegates::FReceivedNetworkEncryptionFailure FNetDelegates::OnReceivedNetworkEncryptionFailure;
FNetDelegates::FOnPendingNetGameConnectionCreated FNetDelegates::OnPendingNetGameConnectionCreated;
FNetDelegates::FNetworkCheatDetected FNetDelegates::OnNetworkCheatDetected;
FNetDelegates::FOnSyncLoadDetected FNetDelegates::OnSyncLoadDetected;