// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/NetworkObjectList.h"
#include "Engine/NetConnection.h"
#include "EngineUtils.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Engine/ActorChannel.h"
#include "UObject/Package.h"

namespace UE::Net::Private
{
	bool bTrackDormantObjectsByLevel = false;
	static FAutoConsoleVariableRef CVarNetTrackDormantObjectsByLevel(
		TEXT("net.TrackDormantObjectsByLevel"),
		bTrackDormantObjectsByLevel,
		TEXT("When true, network object list will maintain a set of dormant actors per connnection per level."),
		ECVF_Default);

	static const TCHAR* LexToString(ENetSubObjectStatus SubObjectStatus)
	{
		switch (SubObjectStatus)
		{
		case ENetSubObjectStatus::Active: return TEXT("Active");
		case ENetSubObjectStatus::TearOff: return TEXT("TearOff");
		case ENetSubObjectStatus::Delete: return TEXT("Delete");
		}
		return TEXT("Missing");
	}
}

#if UE_REPLICATED_OBJECT_REFCOUNTING
void FNetworkObjectList::FActorSubObjectReferences::CountBytes(FArchive& Ar) const
{
	ActiveSubObjectChannelReferences.CountBytes(Ar);
	InvalidSubObjectChannelReferences.CountBytes(Ar);
}
#endif

void FNetworkObjectList::AddInitialObjects(UWorld* const World, UNetDriver* NetDriver)
{
	if (World == nullptr || NetDriver == nullptr)
	{
		return;
	}

	if (NetDriver->GetReplicationSystem())
	{
		return;
	}


	for (FActorIterator Iter(World); Iter; ++Iter)
	{
		AActor* Actor = *Iter;
		if (IsValid(Actor) && ULevel::IsNetActor(Actor) && !UNetDriver::IsDormInitialStartupActor(Actor))
		{
			FindOrAdd(Actor, NetDriver);
		}
	}
}

TSharedPtr<FNetworkObjectInfo> FNetworkObjectList::Find(AActor* const Actor)
{
	if (Actor == nullptr)
	{
		return nullptr;
	}

	if (TSharedPtr<FNetworkObjectInfo>* InfoPtr = AllNetworkObjects.Find(Actor))
	{
		return *InfoPtr;
	}

	return TSharedPtr<FNetworkObjectInfo>();
}

TSharedPtr<FNetworkObjectInfo>* FNetworkObjectList::FindOrAdd(AActor* const Actor, UNetDriver* NetDriver, bool* OutWasAdded)
{
	if (!IsValid(Actor) ||

		// This implies the actor was added either added sometime during UWorld::DestroyActor,
		// or was potentially previously destroyed (and its Index points to a diferent, non-PendingKill object).
		!ensureAlwaysMsgf(!Actor->IsActorBeingDestroyed(), TEXT("Attempting to add an actor that's being destroyed to the NetworkObjectList Actor=%s NetDriverName=%s"), *Actor->GetPathName(), NetDriver ? *NetDriver->NetDriverName.ToString() : TEXT("None")))
	{
		return nullptr;
	}

	if (NetDriver && NetDriver->GetReplicationSystem())
	{
		return nullptr;
	}

	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfo = AllNetworkObjects.Find(Actor);
	
	if (NetworkObjectInfo == nullptr)
	{
		// We do a name check here so we don't add an actor to a network list that it shouldn't belong to
		if (NetDriver && NetDriver->ShouldReplicateActor(Actor))
		{
			NetworkObjectInfo = &AllNetworkObjects[AllNetworkObjects.Emplace(new FNetworkObjectInfo(Actor))];
			ActiveNetworkObjects.Add(*NetworkObjectInfo);

			UE_LOGF(LogNetDormancy, VeryVerbose, "FNetworkObjectList::Add: Adding actor. Actor: %ls, Total: %i, Active: %i, NetDriverName: %ls", *Actor->GetName(), AllNetworkObjects.Num(), ActiveNetworkObjects.Num(), *NetDriver->NetDriverName.ToString());

			if (OutWasAdded)
			{
				*OutWasAdded = true;
			}
		}
	}
	else
	{
		UE_LOGF(LogNetDormancy, VeryVerbose, "FNetworkObjectList::Add: Already contained. Actor: %ls, Total: %i, Active: %i, NetDriverName: %ls", *Actor->GetName(), AllNetworkObjects.Num(), ActiveNetworkObjects.Num(), NetDriver ? *NetDriver->NetDriverName.ToString() : TEXT("None"));
		if (OutWasAdded)
		{
			*OutWasAdded = false;
		}
	}
	
	check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());

	return NetworkObjectInfo;
}

void FNetworkObjectList::Remove(const AActor* const Actor)
{
	if (Actor == nullptr)
	{
		return;
	}

	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = AllNetworkObjects.Find(Actor);

	if (NetworkObjectInfoPtr == nullptr)
	{
		// Sanity check that we're not on the other lists either
		check(!ActiveNetworkObjects.Contains(Actor));
		check(!ObjectsDormantOnAllConnections.Contains(Actor));
		check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());
		return;
	}

	const FName PackageName = (UE::Net::Private::bTrackDormantObjectsByLevel ? Actor->GetLevel()->GetPackage()->GetFName() : FName());

	FNetworkObjectInfo* NetworkObjectInfo = NetworkObjectInfoPtr->Get();

	// Lower the dormant object count for each connection this object is dormant on
	for (auto ConnectionIt = NetworkObjectInfo->DormantConnections.CreateIterator(); ConnectionIt; ++ConnectionIt)
	{
		UNetConnection* Connection = (*ConnectionIt).Get();

		if (Connection == nullptr || Connection->GetConnectionState() == USOCK_Closed)
		{
			if (UE::Net::Private::bTrackDormantObjectsByLevel)
			{
				DormantObjectsPerConnection.Remove(Connection);
			}

			ConnectionIt.RemoveCurrent();
			continue;
		}

		int32& NumDormantObjectsPerConnectionRef = NumDormantObjectsPerConnection.FindOrAdd(Connection);
		check(NumDormantObjectsPerConnectionRef > 0);
		NumDormantObjectsPerConnectionRef--;

		if (UE::Net::Private::bTrackDormantObjectsByLevel)
		{
			if (TMap<FName, FNetworkObjectSet>* DormantObjectsByLevel = DormantObjectsPerConnection.Find(Connection))
			{
				if (FNetworkObjectSet* DormantObjects = DormantObjectsByLevel->Find(PackageName))
				{
					DormantObjects->Remove(Actor);
					
					if (DormantObjects->Num() == 0)
					{
						DormantObjectsByLevel->Remove(PackageName);
					}
				}
			}
		}
	}

	// Remove this object from all lists
	AllNetworkObjects.Remove(Actor);
	ActiveNetworkObjects.Remove(Actor);
	ObjectsDormantOnAllConnections.Remove(Actor);

	if (UE::Net::Private::bTrackDormantObjectsByLevel)
	{
		if (FNetworkObjectSet* FullyDormant = FullyDormantObjectsByLevel.Find(PackageName))
		{
			FullyDormant->Remove(Actor);

			if (FullyDormant->Num() == 0)
			{
				FullyDormantObjectsByLevel.Remove(PackageName);
			}
		}
	}

	check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());
}

void FNetworkObjectList::RemoveInvalid(const AActor* const BadActor)
{
	if (BadActor == nullptr)
	{
		return;
	}

	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = AllNetworkObjects.Find(BadActor);

	if (NetworkObjectInfoPtr == nullptr)
	{
		// Sanity check that we're not on the other lists either
		check(!ActiveNetworkObjects.Contains(BadActor));
		check(!ObjectsDormantOnAllConnections.Contains(BadActor));
		check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());
		return;
	}

	FNetworkObjectInfo* NetworkObjectInfo = NetworkObjectInfoPtr->Get();

	// Lower the dormant object count for each connection this object is dormant on
	for (auto ConnectionIt = NetworkObjectInfo->DormantConnections.CreateIterator(); ConnectionIt; ++ConnectionIt)
	{
		UNetConnection* Connection = (*ConnectionIt).Get();

		if (Connection == nullptr || Connection->GetConnectionState() == USOCK_Closed)
		{
			if (UE::Net::Private::bTrackDormantObjectsByLevel)
			{
				DormantObjectsPerConnection.Remove(Connection);
			}

			ConnectionIt.RemoveCurrent();
			continue;
		}

		int32& NumDormantObjectsPerConnectionRef = NumDormantObjectsPerConnection.FindOrAdd(Connection);
		check(NumDormantObjectsPerConnectionRef > 0);
		NumDormantObjectsPerConnectionRef--;

		// Remove the actor from all connection specific dormancy object sets
		if (UE::Net::Private::bTrackDormantObjectsByLevel)
		{
			if (TMap<FName, FNetworkObjectSet>* DormantObjectsByLevel = DormantObjectsPerConnection.Find(Connection))
			{
				for (TMap<FName, FNetworkObjectSet>::TIterator It = DormantObjectsByLevel->CreateIterator(); It; ++It)
				{
					FNetworkObjectSet& ObjectSet = It.Value();
					// If we found the correct object set to remove the actor from we're done
					if (ObjectSet.Remove(BadActor) > 0)
					{
						// There's no point in keeping empty sets around
						if (ObjectSet.IsEmpty())
						{
							It.RemoveCurrent();
						}
						break;
					}
				}
			}
		}
	}

	// Remove this object from all lists
	AllNetworkObjects.Remove(BadActor);
	ActiveNetworkObjects.Remove(BadActor);
	ObjectsDormantOnAllConnections.Remove(BadActor);

	// Remove the actor from the fully dormant object sets
	if (UE::Net::Private::bTrackDormantObjectsByLevel)
	{
		for (TMap<FName, FNetworkObjectSet>::TIterator It = FullyDormantObjectsByLevel.CreateIterator(); It; ++It)
		{
			FNetworkObjectSet& ObjectSet = It.Value();
			ObjectSet.Remove(BadActor);
			if (ObjectSet.IsEmpty())
			{
				It.RemoveCurrent();
			}
		}
	}

	check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());
}

void FNetworkObjectList::MarkDormant(AActor* const Actor, UNetConnection* const Connection, const int32 NumConnections, UNetDriver* NetDriver)
{
	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = FindOrAdd(Actor, NetDriver);

	if (NetworkObjectInfoPtr == nullptr)
	{
		return;		// Actor doesn't belong to this net driver name
	}

	const FName PackageName = Actor->GetLevel()->GetPackage()->GetFName();

	FNetworkObjectInfo* NetworkObjectInfo = NetworkObjectInfoPtr->Get();

	// Add the connection to the list of dormant connections (if it's not already on the list)
	if (!NetworkObjectInfo->DormantConnections.Contains(Connection))
	{
		check(ActiveNetworkObjects.Contains(Actor));

		NetworkObjectInfo->DormantConnections.Add(Connection);

		// Keep track of the number of dormant objects on each connection
		int32& NumDormantObjectsPerConnectionRef = NumDormantObjectsPerConnection.FindOrAdd(Connection);
		NumDormantObjectsPerConnectionRef++;

		if (UE::Net::Private::bTrackDormantObjectsByLevel)
		{
			// make sure the connection map exists
			TMap<FName, FNetworkObjectSet>& DormantObjectsByLevel = DormantObjectsPerConnection.FindOrAdd(Connection);

			// if not fully dormant
			if (NetworkObjectInfo->DormantConnections.Num() != NumConnections)
			{
				FNetworkObjectSet& DormantObjects = DormantObjectsByLevel.FindOrAdd(PackageName);
				DormantObjects.Add(*NetworkObjectInfoPtr);
			}
		}

		UE_LOGF(LogNetDormancy, Log, "FNetworkObjectList::MarkDormant: Actor is now dormant. Actor: %ls. NumDormant: %i, Connection: %ls", *Actor->GetName(), NumDormantObjectsPerConnectionRef, *Connection->GetName());
	}

	// Clean up DormantConnections list (remove possible GC'd connections)
	for (auto ConnectionIt = NetworkObjectInfo->DormantConnections.CreateIterator(); ConnectionIt; ++ConnectionIt)
	{
		if ((*ConnectionIt).Get() == nullptr || (*ConnectionIt).Get()->GetConnectionState() == USOCK_Closed)
		{
			if (UE::Net::Private::bTrackDormantObjectsByLevel)
			{
				DormantObjectsPerConnection.Remove((*ConnectionIt).Get());
			}

			ConnectionIt.RemoveCurrent();
		}
	}

	// At this point, after removing null references, we should never be over the connection count
	check(NetworkObjectInfo->DormantConnections.Num() <= NumConnections);

	// If the number of dormant connections now matches the number of actual connections, we can remove this object from the active list
	if (NetworkObjectInfo->DormantConnections.Num() == NumConnections)
	{
		ObjectsDormantOnAllConnections.Add(*NetworkObjectInfoPtr);

		if (UE::Net::Private::bTrackDormantObjectsByLevel)
		{
			FNetworkObjectSet& FullyDormant = FullyDormantObjectsByLevel.FindOrAdd(PackageName);
			FullyDormant.Add(*NetworkObjectInfoPtr);

			// Remove from connection object lists
			for (auto ConnectionIt = NetworkObjectInfo->DormantConnections.CreateIterator(); ConnectionIt; ++ConnectionIt)
			{
				if (TMap<FName, FNetworkObjectSet>* DormantObjectsByLevel = DormantObjectsPerConnection.Find((*ConnectionIt).Get()))
				{
					if (FNetworkObjectSet* DormantObjects = DormantObjectsByLevel->Find(PackageName))
					{
						const int32 NumRemoved = DormantObjects->Remove(Actor);
						checkf((NumRemoved > 0) || (Connection == (*ConnectionIt).Get()), TEXT("Actor not found in Connection->Level->Dormant map: %s"), *GetNameSafe(Actor));

						if (DormantObjects->Num() == 0)
						{
							DormantObjectsByLevel->Remove(PackageName);
						}
					}
				}
			}
		}

		ActiveNetworkObjects.Remove(Actor);

		UE_LOGF(LogNetDormancy, Log, "FNetworkObjectList::MarkDormant: Actor is now dormant on all connections. Actor: %ls. Total: %i, Active: %i, Connection: %ls", *Actor->GetName(), AllNetworkObjects.Num(), ActiveNetworkObjects.Num(), *Connection->GetName());
	}

	check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());
}

bool FNetworkObjectList::MarkActiveInternal(const TSharedPtr<FNetworkObjectInfo>& ObjectInfo, UNetConnection* const Connection, UNetDriver* NetDriver)
{
	FNetworkObjectInfo* NetworkObjectInfo = ObjectInfo.Get();
	AActor* Actor = NetworkObjectInfo->Actor;

	const FName PackageName = Actor->GetLevel()->GetPackage()->GetFName();

	// Remove from the ObjectsDormantOnAllConnections if needed
	if (ObjectsDormantOnAllConnections.Remove(Actor) > 0)
	{
		// Put this object back on the active list
		ActiveNetworkObjects.Add(ObjectInfo);

		UE_LOGF(LogNetDormancy, Log, "FNetworkObjectList::MarkDormant: Actor is no longer dormant on all connections. Actor: %ls. Total: %i, Active: %i, Connection: %ls", *Actor->GetName(), AllNetworkObjects.Num(), ActiveNetworkObjects.Num(), *Connection->GetName());

		if (UE::Net::Private::bTrackDormantObjectsByLevel)
		{
			if (FNetworkObjectSet* FullyDormant = FullyDormantObjectsByLevel.Find(PackageName))
			{
				const int32 NumRemoved = FullyDormant->Remove(Actor);
				checkf(NumRemoved > 0, TEXT("Actor not found in full Level->Dormant map: %s"), *GetNameSafe(Actor));

				if (FullyDormant->Num() == 0)
				{
					FullyDormantObjectsByLevel.Remove(PackageName);
				}

				// add back into per connection maps
				for (auto ConnectionIt = NetworkObjectInfo->DormantConnections.CreateIterator(); ConnectionIt; ++ConnectionIt)
				{
					TMap<FName, FNetworkObjectSet>& DormantObjectsByLevel = DormantObjectsPerConnection.FindOrAdd((*ConnectionIt).Get());
					FNetworkObjectSet& DormantObjects = DormantObjectsByLevel.FindOrAdd(PackageName);
					DormantObjects.Add(ObjectInfo);
				}
			}
		}
	}

	check((ActiveNetworkObjects.Num() + ObjectsDormantOnAllConnections.Num()) == AllNetworkObjects.Num());

	// Remove connection from the dormant connection list
	if (NetworkObjectInfo->DormantConnections.Remove(Connection) > 0)
	{
		// Add the connection to the list of recently dormant connections
		NetworkObjectInfo->RecentlyDormantConnections.Add(Connection);

		int32& NumDormantObjectsPerConnectionRef = NumDormantObjectsPerConnection.FindOrAdd(Connection);
		check(NumDormantObjectsPerConnectionRef > 0);
		NumDormantObjectsPerConnectionRef--;

		if (UE::Net::Private::bTrackDormantObjectsByLevel)
		{
			if (TMap<FName, FNetworkObjectSet>* DormantObjectsByLevel = DormantObjectsPerConnection.Find(Connection))
			{
				if (FNetworkObjectSet* DormantObjects = DormantObjectsByLevel->Find(PackageName))
				{
					const int32 NumRemoved = DormantObjects->Remove(Actor);
					checkf(NumRemoved > 0, TEXT("Actor not found in Connection->Level->Dormant map: %s"), *GetNameSafe(Actor));

					if (DormantObjects->Num() == 0)
					{
						DormantObjectsByLevel->Remove(PackageName);
					}
				}
			}
		}

		UE_LOGF(LogNetDormancy, Log, "FNetworkObjectList::MarkActive: Actor is no longer dormant. Actor: %ls. NumDormant: %i, Connection: %ls", *Actor->GetName(), NumDormantObjectsPerConnectionRef, *Connection->GetName());
		return true;
	}

	return false;
}

bool FNetworkObjectList::MarkActive(AActor* const Actor, UNetConnection* const Connection, UNetDriver* NetDriver)
{
	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = FindOrAdd(Actor, NetDriver);

	if (NetworkObjectInfoPtr == nullptr)
	{
		return false;		// Actor doesn't belong to this net driver name
	}

	return MarkActiveInternal(*NetworkObjectInfoPtr, Connection, NetDriver);
}

void FNetworkObjectList::MarkDirtyForReplay(AActor* const Actor)
{
	if (Actor)
	{
		if (TSharedPtr<FNetworkObjectInfo>* InfoPtr = AllNetworkObjects.Find(Actor))
		{
			if (FNetworkObjectInfo* ObjectInfo = InfoPtr->Get())
			{
				ObjectInfo->bDirtyForReplay = true;
			}
		}
	}
}

void FNetworkObjectList::ResetReplayDirtyTracking()
{
	for (auto It = AllNetworkObjects.CreateIterator(); It; ++It)
	{
		if (FNetworkObjectInfo* ObjectInfo = (*It).Get())
		{
			ObjectInfo->bDirtyForReplay = false;
		}
	}
}

void FNetworkObjectList::ClearRecentlyDormantConnection(AActor* const Actor, UNetConnection* const Connection, UNetDriver* NetDriver)
{
	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = FindOrAdd(Actor, NetDriver);

	if (NetworkObjectInfoPtr == nullptr)
	{
		return;		// Actor doesn't belong to this net driver name
	}

	FNetworkObjectInfo* NetworkObjectInfo = NetworkObjectInfoPtr->Get();

	NetworkObjectInfo->RecentlyDormantConnections.Remove(Connection);
}

void FNetworkObjectList::OnActorIsTraveling(AActor* TravelingAtor)
{
	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = AllNetworkObjects.Find(TravelingAtor);
	if (NetworkObjectInfoPtr)
	{
		SeamlessTravelingObjects.Add(*NetworkObjectInfoPtr);
	}
}

void FNetworkObjectList::OnPostSeamlessTravel()
{
	for (const TSharedPtr<FNetworkObjectInfo>& SeamlessTravelingActorInfo : SeamlessTravelingObjects)
	{
		// Make sure the actor didn't re-register itself during seamless travel
		if (AllNetworkObjects.Find(SeamlessTravelingActorInfo->Actor) != nullptr)
		{
			AActor* TravelingActor = SeamlessTravelingActorInfo->Actor;
			ensureMsgf(false, TEXT("Seamless traveling actor %s was readded to the netdriver before seamless travel was finished!"), *GetNameSafe(TravelingActor));
			
			// Clean our lists of this duplicate info.
			AllNetworkObjects.Remove(TravelingActor);
			ActiveNetworkObjects.Remove(TravelingActor);
			ObjectsDormantOnAllConnections.Remove(TravelingActor);
		}

		//Ensure the actor's dormant connections lists are empty when seamlessly traveling
		SeamlessTravelingActorInfo->DormantConnections.Empty();
		SeamlessTravelingActorInfo->RecentlyDormantConnections.Empty();

		// Add the saved object info back into the active list
		AllNetworkObjects.Add(SeamlessTravelingActorInfo);
		ActiveNetworkObjects.Add(SeamlessTravelingActorInfo);
	}

	SeamlessTravelingObjects.Empty();
}

void FNetworkObjectList::HandleConnectionAdded()
{
	// When a new connection is added, we must add all objects back to the active list so the new connection will process it
	// Once the objects is dormant on that connection, it will then be removed from the active list again
	for (auto It = ObjectsDormantOnAllConnections.CreateIterator(); It; ++It)
	{
		ActiveNetworkObjects.Add(*It);
	}

	ObjectsDormantOnAllConnections.Empty();
	FullyDormantObjectsByLevel.Empty();
}

void FNetworkObjectList::ResetDormancyState()
{
	// Reset all state related to dormancy, and move all objects back on to the active list
	ObjectsDormantOnAllConnections.Empty();
	FullyDormantObjectsByLevel.Empty();

	ActiveNetworkObjects = AllNetworkObjects;

	for (auto It = AllNetworkObjects.CreateIterator(); It; ++It)
	{
		FNetworkObjectInfo* NetworkObjectInfo = ( *It ).Get();

		NetworkObjectInfo->DormantConnections.Empty();
		NetworkObjectInfo->RecentlyDormantConnections.Empty();
	}

	NumDormantObjectsPerConnection.Empty();
	DormantObjectsPerConnection.Empty();
}

int32 FNetworkObjectList::GetNumDormantActorsForConnection(UNetConnection* const Connection) const
{
	const int32 *Count = NumDormantObjectsPerConnection.Find( Connection );

	return (Count != nullptr) ? *Count : 0;
}

void FNetworkObjectList::ForceActorRelevantNextUpdate(AActor* const Actor, UNetDriver* NetDriver)
{
	TSharedPtr<FNetworkObjectInfo>* NetworkObjectInfoPtr = FindOrAdd(Actor, NetDriver);

	if (NetworkObjectInfoPtr == nullptr)
	{
		return;		// Actor doesn't belong to this net driver name
	}

	FNetworkObjectInfo* NetworkObjectInfo = NetworkObjectInfoPtr->Get();

	NetworkObjectInfo->ForceRelevantFrame = NetDriver->ReplicationFrame + 1;
}

void FNetworkObjectList::Reset()
{
	// Reset all state
	AllNetworkObjects.Empty();
	ActiveNetworkObjects.Empty();
	ObjectsDormantOnAllConnections.Empty();
	NumDormantObjectsPerConnection.Empty();
	FullyDormantObjectsByLevel.Empty();
	DormantObjectsPerConnection.Empty();
}

void FNetworkObjectInfo::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FNetworkObjectInfo::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DormantConnections", DormantConnections.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("RecentlyDormantConnections", RecentlyDormantConnections.CountBytes(Ar));
}

void FNetworkObjectList::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FNetworkObjectList::CountBytes");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ActiveNetworkObjects", ActiveNetworkObjects.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ObjectsDormantOnAllConnections", ObjectsDormantOnAllConnections.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NumDormantObjectsPerConnection", NumDormantObjectsPerConnection.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("FullyDormantObjectsByLevel",
		FullyDormantObjectsByLevel.CountBytes(Ar);
		for (const TPair<FName, FNetworkObjectSet>& LevelPair : FullyDormantObjectsByLevel)
		{
			LevelPair.Value.CountBytes(Ar);
		}
	);
	
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DormantObjectsPerConnection",
		DormantObjectsPerConnection.CountBytes(Ar);
		for (const TPair<TObjectKey<UNetConnection>, TMap<FName, FNetworkObjectSet>>& ConnectionPair : DormantObjectsPerConnection)
		{
			ConnectionPair.Value.CountBytes(Ar);

			for (const TPair<FName, FNetworkObjectSet>& LevelPair : ConnectionPair.Value)
			{
				LevelPair.Value.CountBytes(Ar);
			}
		}
	);

	// ObjectsDormantOnAllConnections and ActiveNetworkObjects are both sub sets of AllNetworkObjects
	// and only have pointers back to the data there.
	// So, to avoid double (or triple) counting, only explicit count the elements from AllNetworkObjects.
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("AllNetworkObjects",
		AllNetworkObjects.CountBytes(Ar);
		for (const TSharedPtr<FNetworkObjectInfo>& SharedInfo : AllNetworkObjects)
		{
			if (FNetworkObjectInfo const* const Info = SharedInfo.Get())
			{
				Ar.CountBytes(sizeof(FNetworkObjectInfo), sizeof(FNetworkObjectInfo));
				Info->CountBytes(Ar);
			}
		}
	);

#if UE_REPLICATED_OBJECT_REFCOUNTING
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("SubObjectChannelReferences", SubObjectChannelReferences.CountBytes(Ar));
	for (const FActorSubObjectReferences& SubObjectRef : SubObjectChannelReferences)
	{
		SubObjectRef.CountBytes(Ar);
	}
#endif
}

void FNetworkObjectList::FlushDormantActors(UNetConnection* const Connection, const FName& PackageName)
{
	FNetworkObjectSet DormantActors;

	// fully dormant actors
	if (const FNetworkObjectSet* FullyDormantObjects = FullyDormantObjectsByLevel.Find(PackageName))
	{
		DormantActors.Append(*FullyDormantObjects);
	}

	// connection specific dormant actors
	if (const TMap<FName, FNetworkObjectSet>* DormantObjectsByLevel = DormantObjectsPerConnection.Find(Connection))
	{
		if (const FNetworkObjectSet* DormantObjects = DormantObjectsByLevel->Find(PackageName))
		{
			DormantActors.Append(*DormantObjects);
		}
	}

	for (const TSharedPtr<FNetworkObjectInfo>& ActorInfo : DormantActors)
	{
		MarkActiveInternal(ActorInfo, Connection, Connection->Driver);
	}
}

void FNetworkObjectList::OnActorDestroyed(AActor* DestroyedActor)
{
#if UE_REPLICATED_OBJECT_REFCOUNTING
	SubObjectChannelReferences.Remove(DestroyedActor);
#endif
}

#if UE_REPLICATED_OBJECT_REFCOUNTING

FNetworkObjectList::FActorInvalidSubObjectView FNetworkObjectList::FindActorInvalidSubObjects(AActor* OwnerActor) const
{
	if (const FActorSubObjectReferences* SubObjectsRefInfo = SubObjectChannelReferences.Find(OwnerActor))
	{
		return FActorInvalidSubObjectView(SubObjectsRefInfo->InvalidSubObjectDirtyCount, &(SubObjectsRefInfo->InvalidSubObjectChannelReferences));
	}
	else
	{
		return FActorInvalidSubObjectView(0);
	}
}

void FNetworkObjectList::SetSubObjectForDeletion(AActor* Actor, UObject* SubObject)
{
	InvalidateSubObject(Actor, SubObject, ENetSubObjectStatus::Delete);
}

void FNetworkObjectList::SetSubObjectForTearOff(AActor* Actor, UObject* SubObject)
{
	InvalidateSubObject(Actor, SubObject, ENetSubObjectStatus::TearOff);
}

void FNetworkObjectList::InvalidateSubObject(AActor* Actor, UObject* SubObject, ENetSubObjectStatus InvalidStatus)
{
	ensure(InvalidStatus != ENetSubObjectStatus::Active);

	if (FActorSubObjectReferences* SubObjectsRefInfo = SubObjectChannelReferences.Find(Actor))
	{
		const TWeakObjectPtr<UObject> SubObjectPtr = SubObject;
		const FSetElementId FoundId = SubObjectsRefInfo->ActiveSubObjectChannelReferences.FindId(SubObjectPtr);

		if (FoundId.IsValidId())
		{
			// Flag its new state
			SubObjectsRefInfo->ActiveSubObjectChannelReferences[FoundId].Status = InvalidStatus;

			// Move the reference to the destroyed list.
			SubObjectsRefInfo->InvalidSubObjectChannelReferences.Emplace(MoveTemp(SubObjectsRefInfo->ActiveSubObjectChannelReferences[FoundId]));
			SubObjectsRefInfo->ActiveSubObjectChannelReferences.Remove(FoundId);

			// Increase dirty count so channels can refresh this list
			SubObjectsRefInfo->InvalidSubObjectDirtyCount = FMath::Max(SubObjectsRefInfo->InvalidSubObjectDirtyCount + 1, 1); // Wrap around to 1 because 0 is the default actor channel value
		}
	}
}

void FNetworkObjectList::AddSubObjectChannelReference(AActor* OwnerActor, UObject* ReplicatedSubObject, UObject* ReferenceOwner)
{
	FActorSubObjectReferences* SubObjectsRefInfo = SubObjectChannelReferences.Find(OwnerActor);

	if (SubObjectsRefInfo == nullptr)
	{
		// Create an entry for this actor
		FSetElementId Index = SubObjectChannelReferences.Emplace(FActorSubObjectReferences(OwnerActor));
		SubObjectsRefInfo = &(SubObjectChannelReferences[Index]);
	}

	const TWeakObjectPtr<UObject> SubObjectPtr = ReplicatedSubObject;

	// Find the subobject in the active list first
	FSubObjectChannelReference* ChannelReferences = SubObjectsRefInfo->ActiveSubObjectChannelReferences.Find(SubObjectPtr);

	if (ChannelReferences == nullptr)
	{
		// Look if the object could be in the invalid list
		ChannelReferences = SubObjectsRefInfo->InvalidSubObjectChannelReferences.FindByKey(SubObjectPtr);
	}

	if (ChannelReferences)
	{
#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
		ensureMsgf(!ChannelReferences->RegisteredOwners.Contains(ReferenceOwner), TEXT("SubObject %s (0x%p) owned by %s was already referenced by (0x%p) %s"), *GetNameSafe(ReplicatedSubObject), ReplicatedSubObject, *GetNameSafe(OwnerActor), ReferenceOwner, *GetNameSafe(ReferenceOwner));
		ChannelReferences->RegisteredOwners.Add(ReferenceOwner);
#endif

		ChannelReferences->ChannelRefCount++;
		check(ChannelReferences->ChannelRefCount < MAX_uint16);
	}
	else
	{
		bool bWasSet = false;
		FSetElementId ElementId = SubObjectsRefInfo->ActiveSubObjectChannelReferences.Emplace(FSubObjectChannelReference(SubObjectPtr), &bWasSet);

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
		SubObjectsRefInfo->ActiveSubObjectChannelReferences[ElementId].RegisteredOwners.Add(ReferenceOwner);
#endif
	}
}

void FNetworkObjectList::RemoveMultipleSubObjectChannelReference(FObjectKey OwnerActorKey, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToRemove, UObject* ReferenceOwner)
{
	FSetElementId FoundIndex = SubObjectChannelReferences.FindId(OwnerActorKey);
	if (FoundIndex.IsValidId())
	{
		FActorSubObjectReferences& SubObjectsRefInfo = SubObjectChannelReferences[FoundIndex];

		for (const TWeakObjectPtr<UObject>& SubObjectPtr : SubObjectsToRemove)
		{
			HandleRemoveAnySubObjectChannelRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);
		}

		if (SubObjectsRefInfo.HasNoSubObjects())
		{
			SubObjectChannelReferences.Remove(FoundIndex);
		}
	}
}

void FNetworkObjectList::RemoveMultipleInvalidSubObjectChannelReference(FObjectKey OwnerActorKey, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToRemove, UObject* ReferenceOwner)
{
	FSetElementId FoundIndex = SubObjectChannelReferences.FindId(OwnerActorKey);
	if (FoundIndex.IsValidId())
	{
		FActorSubObjectReferences& SubObjectsRefInfo = SubObjectChannelReferences[FoundIndex];
	
		for (const TWeakObjectPtr<UObject>& SubObjectPtr : SubObjectsToRemove)
		{
			HandleRemoveInvalidSubObjectRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);
		}

		if (SubObjectsRefInfo.HasNoSubObjects())
		{
			SubObjectChannelReferences.Remove(FoundIndex);
		}
	}
}

void FNetworkObjectList::RemoveMultipleActiveSubObjectChannelReference(FObjectKey OwnerActorKey, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToRemove, UObject* ReferenceOwner)
{
	FSetElementId FoundIndex = SubObjectChannelReferences.FindId(OwnerActorKey);
	if (FoundIndex.IsValidId())
	{
		FActorSubObjectReferences& SubObjectsRefInfo = SubObjectChannelReferences[FoundIndex];

		for (const TWeakObjectPtr<UObject>& SubObjectPtr : SubObjectsToRemove)
		{
			bool bWasRemoved = HandleRemoveActiveSubObjectRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);

			// If somehow the object wasn't in the active list, check if it was in the invalid list just to be sure.
			// It's possible for an object set to be deleted to have it's uobject ptr become innacessible before we replicate it's owner.
			// Calling this here would act as a safeguard to ensure its reference is removed even if the tear off/force delete command was never sent.
			if (!bWasRemoved)
			{
				HandleRemoveInvalidSubObjectRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);
			}
		}

		if (SubObjectsRefInfo.HasNoSubObjects())
		{
			SubObjectChannelReferences.Remove(FoundIndex);
		}
	}
}

void FNetworkObjectList::RemoveSubObjectChannelReference(AActor* OwnerActor, const TWeakObjectPtr<UObject>& SubObjectPtr, UObject* ReferenceOwner)
{
	FSetElementId FoundIndex = SubObjectChannelReferences.FindId(OwnerActor);
	if (FoundIndex.IsValidId())
	{
		FActorSubObjectReferences& SubObjectsRefInfo = SubObjectChannelReferences[FoundIndex];
		HandleRemoveAnySubObjectChannelRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);

		if (SubObjectsRefInfo.HasNoSubObjects())
		{
			SubObjectChannelReferences.Remove(FoundIndex);
		}
	}
}

void FNetworkObjectList::HandleRemoveAnySubObjectChannelRef(FActorSubObjectReferences& SubObjectsRefInfo, const TWeakObjectPtr<UObject>& SubObjectPtr, UObject* ReferenceOwner)
{
	check(SubObjectPtr.IsExplicitlyNull() == false);

	// Look in the active set
	bool bWasRemoved = HandleRemoveActiveSubObjectRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);

	// Look in the destroyed array
	if (!bWasRemoved)
	{
		bWasRemoved = HandleRemoveInvalidSubObjectRef(SubObjectsRefInfo, SubObjectPtr, ReferenceOwner);
	}

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
	ensureMsgf(bWasRemoved, TEXT("HandleRemoveAnySubObjectChannelRef could not find any references for %s (0x%p) owned by %s"), *GetNameSafe(SubObjectPtr.GetEvenIfUnreachable()), SubObjectPtr.GetEvenIfUnreachable(), *GetNameSafe(SubObjectsRefInfo.ActorKey.ResolveObjectPtrEvenIfUnreachable()));
#endif
}

bool FNetworkObjectList::HandleRemoveActiveSubObjectRef(FActorSubObjectReferences& SubObjectsRefInfo, const TWeakObjectPtr<UObject>& SubObjectPtr, UObject* ReferenceOwner)
{
	FSetElementId FoundId = SubObjectsRefInfo.ActiveSubObjectChannelReferences.FindId(SubObjectPtr);
	if (FoundId.IsValidId())
	{
		FSubObjectChannelReference& ActiveReference = SubObjectsRefInfo.ActiveSubObjectChannelReferences[FoundId];

		check(ActiveReference.SubObjectPtr.HasSameIndexAndSerialNumber(SubObjectPtr));

		ActiveReference.ChannelRefCount--;

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
		int32 Removed = ActiveReference.RegisteredOwners.RemoveSingleSwap(ReferenceOwner);
		ensureMsgf(Removed > 0, TEXT("Removed ACTIVE ref for Subobject %s (0x%p) owned by %s but it was never referenced by the connection (0x%p) %s"), *GetNameSafe(SubObjectPtr.GetEvenIfUnreachable()), SubObjectPtr.GetEvenIfUnreachable(), *GetNameSafe(SubObjectsRefInfo.ActorKey.ResolveObjectPtrEvenIfUnreachable()), ReferenceOwner, *GetNameSafe(ReferenceOwner));
#endif

		if (ActiveReference.ChannelRefCount == 0)
		{
			SubObjectsRefInfo.ActiveSubObjectChannelReferences.Remove(FoundId);
		}

		return true;
	}

	return false;
}

bool FNetworkObjectList::HandleRemoveInvalidSubObjectRef(FActorSubObjectReferences& SubObjectsRefInfo, const TWeakObjectPtr<UObject>& SubObjectPtr, UObject* ReferenceOwner)
{
	const int32 Index = SubObjectsRefInfo.InvalidSubObjectChannelReferences.IndexOfByKey(SubObjectPtr);
	if (Index != INDEX_NONE)
	{
		FSubObjectChannelReference& InvalidReference = SubObjectsRefInfo.InvalidSubObjectChannelReferences[Index];
		InvalidReference.ChannelRefCount--;

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
		int32 Removed = InvalidReference.RegisteredOwners.RemoveSingleSwap(ReferenceOwner);
		ensureMsgf(Removed > 0, TEXT("Removed INVALID ref for Subobject %s (0x%p) owned by %s but it was never referenced by the connection (0x%p) %s"), *GetNameSafe(SubObjectPtr.GetEvenIfUnreachable()), SubObjectPtr.GetEvenIfUnreachable(), *GetNameSafe(SubObjectsRefInfo.ActorKey.ResolveObjectPtrEvenIfUnreachable()), ReferenceOwner, *GetNameSafe(ReferenceOwner));
#endif
		if (InvalidReference.ChannelRefCount == 0)
		{
			SubObjectsRefInfo.InvalidSubObjectChannelReferences.RemoveAtSwap(Index);
		}

		return true;
	}

	return false;
}

#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS
void FNetworkObjectList::SwapMultipleReferencesForDormancy(AActor* OwnerActor, const TArrayView<TWeakObjectPtr<UObject>>& SubObjectsToSwap, UActorChannel* PreviousChannelRefOwner, UNetConnection* NewConnectionRefOwner)
{
	if (FActorSubObjectReferences* SubObjectsRefInfo = SubObjectChannelReferences.Find(OwnerActor))
	{
		for (const TWeakObjectPtr<UObject>& SubObjectPtr : SubObjectsToSwap)
		{
			HandleSwapReferenceForDormancy(SubObjectsRefInfo, SubObjectPtr, PreviousChannelRefOwner, NewConnectionRefOwner);
		}
	}
}

void FNetworkObjectList::SwapReferenceForDormancy(AActor* OwnerActor, UObject* ReplicatedSubObject, UNetConnection* PreviousConnectionRefOwner, UActorChannel* NewChannelRefOwner)
{
	if (FActorSubObjectReferences* SubObjectsRefInfo = SubObjectChannelReferences.Find(OwnerActor))
	{
		const TWeakObjectPtr<UObject> SubObjectPtr = ReplicatedSubObject;
		HandleSwapReferenceForDormancy(SubObjectsRefInfo, SubObjectPtr, PreviousConnectionRefOwner, NewChannelRefOwner);
	}
}

void FNetworkObjectList::HandleSwapReferenceForDormancy(FActorSubObjectReferences* SubObjectsRefInfo, const TWeakObjectPtr<UObject>& SubObjectPtr, UObject* PreviousRefOwner, UObject* NewRefOwner)
{
	check(SubObjectsRefInfo);

	// First check the active list
	FSubObjectChannelReference* SubObjectChannelRef = SubObjectsRefInfo->ActiveSubObjectChannelReferences.Find(SubObjectPtr);

	if (!SubObjectChannelRef)
	{
		// Then check the inactive list
		SubObjectChannelRef = SubObjectsRefInfo->InvalidSubObjectChannelReferences.FindByKey(SubObjectPtr);
	}

	if (SubObjectChannelRef)
	{
		int32 Removed = SubObjectChannelRef->RegisteredOwners.RemoveSingleSwap(PreviousRefOwner);

		ensureMsgf(Removed > 0, TEXT("SwapReferencesForDormancy could not find reference to previous reference (0x%p) %s for subobject %s (0x%p) owned by %s. Swapping to (0x%p) %s"), 
			PreviousRefOwner, *GetNameSafe(PreviousRefOwner), *GetNameSafe(SubObjectPtr.GetEvenIfUnreachable()), SubObjectPtr.GetEvenIfUnreachable(), *GetNameSafe(SubObjectsRefInfo->ActorKey.ResolveObjectPtrEvenIfUnreachable()), NewRefOwner, *GetNameSafe(NewRefOwner));

		ensureMsgf(!SubObjectChannelRef->RegisteredOwners.Contains(NewRefOwner), TEXT("SwapReferencesForDormancy found new reference (0x%p) %s was already registered to %s (0x%p) owned by %s"), 
			NewRefOwner, *GetNameSafe(NewRefOwner), *GetNameSafe(SubObjectPtr.GetEvenIfUnreachable()), SubObjectPtr.GetEvenIfUnreachable(), *GetNameSafe(SubObjectsRefInfo->ActorKey.ResolveObjectPtrEvenIfUnreachable()));

		SubObjectChannelRef->RegisteredOwners.Add(NewRefOwner);
	}
	else
	{
		ensureMsgf(false, TEXT("SwapReferencesForDormancy could not find any references to %s (0x%p) owned by %s. Swapping from (0x%p) %s to (0x%p) %s"), 
			*GetNameSafe(SubObjectPtr.GetEvenIfUnreachable()), SubObjectPtr.GetEvenIfUnreachable(), *GetNameSafe(SubObjectsRefInfo->ActorKey.ResolveObjectPtrEvenIfUnreachable()), PreviousRefOwner, *GetNameSafe(PreviousRefOwner), NewRefOwner, *GetNameSafe(NewRefOwner));
	}
}
#endif //#if DO_REPLICATED_OBJECT_CHANNELREF_CHECKS

#endif //#if UE_REPLICATED_OBJECT_REFCOUNTING
