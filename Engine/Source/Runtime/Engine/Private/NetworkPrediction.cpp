// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/NetworkPredictionInterface.h"
#include "UObject/Interface.h"


UNetworkPredictionInterface::UNetworkPredictionInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
