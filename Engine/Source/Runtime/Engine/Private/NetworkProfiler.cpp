// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkProfiler.cpp: server network profiling support.
=============================================================================*/

#include "Net/NetworkProfiler.h"
#include "AutoRTFM.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Engine/World.h"
#include "TimerManager.h"
#include "Serialization/TransactionallySafeArchiveWriter.h"
#include "UObject/UnrealType.h"

#if USE_NETWORK_PROFILER

CSV_DEFINE_CATEGORY(NetworkProfiler, true);

#define SCOPE_LOCK_REF(X) UE::TScopeLock ScopeLock(X);

struct FNetworkProfilerCVarHelper
{
	static bool& GetNetProfilerIsComparisonTrackingEnabled()
	{
		return FNetworkProfiler::bIsComparisonTrackingEnabled;
	}
};

bool FNetworkProfiler::bIsComparisonTrackingEnabled = false;
static FAutoConsoleVariableRef CVarProfilerUseComparisonTracking(
	TEXT("Net.ProfilerUseComparisonTracking"),
	FNetworkProfilerCVarHelper::GetNetProfilerIsComparisonTrackingEnabled(),
	TEXT("")
);


/**
 * Whether to track the raw network data or not.
 */
#define NETWORK_PROFILER_TRACK_RAW_NETWORK_DATA		0

#include "Engine/NetConnection.h"

/** Global network profiler instance. */
FNetworkProfiler GNetworkProfiler;

/** Magic value, determining that file is a network profiler file.				*/
#define NETWORK_PROFILER_MAGIC						0x1DBF348C

/** Version history of network profiler. Entries added on serialization changes. */
enum class ENetworkProfilerVersionHistory : uint32
{
	Initial = 10,
	ChannelTypesAsStrings = 11,
	AddressesAsStrings = 12,
	LargeRPCFix = 13,				// Changed RPC size tracking to packed 32 bits instead of plain uint16s, to handle very large RPCs.
	PushModelTracking = 14,			// Adding some new data to help profile Push Model changes, and to help identify things that could utilize Push Model effectively.

	// New history items go above here.

	VersionPlusOne,
	Latest = VersionPlusOne - 1
};

static const FString UnknownName("UnknownName");

static const double MaxTempFileAgeSeconds = 60.0 * 60.0 * 24.0 * 5.0;

enum ENetworkProfilingPayloadType
{
	NPTYPE_FrameMarker			= 0,	// Frame marker, signaling beginning of frame.	
	NPTYPE_SocketSendTo,				// FSocket::SendTo
	NPTYPE_SendBunch,					// UChannel::SendBunch
	NPTYPE_SendRPC,						// Sending RPC
	NPTYPE_ReplicateActor,				// Replicated object	
	NPTYPE_ReplicateProperty,			// Property being replicated.
	NPTYPE_EndOfStreamMarker,			// End of stream marker		
	NPTYPE_Event,						// Event
	NPTYPE_RawSocketData,				// Raw socket data being sent
	NPTYPE_SendAck,						// Ack being sent
	NPTYPE_WritePropertyHeader,			// Property header being written
	NPTYPE_ExportBunch,					// Exported GUIDs
	NPTYPE_MustBeMappedGuids,			// Must be mapped GUIDs
	NPTYPE_BeginContentBlock,			// Content block headers
	NPTYPE_EndContentBlock,				// Content block footers
	NPTYPE_WritePropertyHandle,			// Property handles
	NPTYPE_ConnectionChanged,			// Connection changed
	NPTYPE_NameReference,				// New reference to name
	NPTYPE_ConnectionReference,			// New reference to connection
	NPTYPE_PropertyComparison,			// Data about property comparions.
	NPTYPE_ReplicatePropertiesMetadata	// Full set of top level properties for an object.
};

/*=============================================================================
	FNetworkProfilerHeader implementation.
=============================================================================*/

FNetworkProfilerHeader::FNetworkProfilerHeader()
	: Magic(NETWORK_PROFILER_MAGIC)
	, Version(ENetworkProfilerVersionHistory::Latest)
{
}

void FNetworkProfilerHeader::Reset(const FURL& InURL)
{
	FParse::Value( FCommandLine::Get(), TEXT( "NETWORKPROFILER=" ), Tag );
	GameName = FApp::GetProjectName();
	URL = InURL.ToString();
}

FArchive& operator << ( FArchive& Ar, FNetworkProfilerHeader& Header )
{
	check( Ar.IsSaving() );
	Ar	<< Header.Magic
		<< Header.Version;
	Header.Tag.SerializeAsANSICharArray( Ar );
	Header.GameName.SerializeAsANSICharArray( Ar );
	Header.URL.SerializeAsANSICharArray( Ar );
	return Ar;
}

/*=============================================================================
	FNetworkProfiler implementation.
=============================================================================*/

/**
 * Constructor, initializing member variables.
 */
FNetworkProfiler::FNetworkProfiler()
:	FileWriter( nullptr )
,	bHasNoticeableNetworkTrafficOccured(false)
,	bIsTrackingEnabled(false)
,	bShouldTrackingBeEnabled(false)
,	LastAddress( nullptr )
,	IgnorePropertyCount(0)
{
}

/**
 * Returns index of passed in name into name array. If not found, adds it.
 *
 * @param	Name	Name to find index for
 * @return	Index of passed in name
 */
int32 FNetworkProfiler::GetNameTableIndex( const FString& Name )
{
	// Index of name in name table.
	int32 Index = INDEX_NONE;

	// Use index if found.
	int32* IndexPtr = NameToNameTableIndexMap.Find( Name );
	if( IndexPtr )
	{
		Index = *IndexPtr;
	}
	// Encountered new name, add to array and set index mapping.
	else
	{
		Index = NameArray.Num();
		new(NameArray)FString(Name);
		NameToNameTableIndexMap.Add(*Name,Index);

		// Write out the name reference token
		uint8 Type = NPTYPE_NameReference;
		( *FileWriter ) << Type;
		Name.SerializeAsANSICharArray( *FileWriter );
	}

	check(Index!=INDEX_NONE);
	return Index;
}

/**
* Returns index of passed in name into name array. If not found, adds it.
*
* @param	Address	Address string to find index for
* @return	Index of passed in name
*/
int32 FNetworkProfiler::GetAddressTableIndex( const FString& Address )
{
	// Index of name in name table.
	int32 Index = INDEX_NONE;

	// Use index if found.
	int32* IndexPtr = AddressTableIndexMap.Find( Address );
	if ( IndexPtr )
	{
		Index = *IndexPtr;
	}
	// Encountered new name, add to array and set index mapping.
	else
	{
		Index = AddressTableIndexMap.Num();
		AddressTableIndexMap.Add( Address, Index );

		// Write out the name reference token
		uint8 Type = NPTYPE_ConnectionReference;
		( *FileWriter ) << Type;
		Address.SerializeAsANSICharArray( *FileWriter );
	}

	check( Index != INDEX_NONE );
	return Index;
}

/**
 * Enables/ disables tracking. Emits a session changes if disabled.
 *
 * @param	bShouldEnableTracking	Whether tracking should be enabled or not
 */
void FNetworkProfiler::EnableTracking( bool bShouldEnableTracking )
{
	bShouldTrackingBeEnabled = bShouldEnableTracking;

	if( bShouldEnableTracking )
	{
		UE_LOGF(LogNet, Log, "Network Profiler: ENABLED");
	}

	// Flush existing session in progress if we're disabling tracking and it was enabled.
	if( bIsTrackingEnabled && !bShouldEnableTracking )
	{
		TrackSessionChange(false,FURL());
	}
}

/**
 * Marks the beginning of a frame.
 */
void FNetworkProfiler::TrackFrameBegin()
{
	if( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);
		uint8 Type = NPTYPE_FrameMarker;
		(*FileWriter) << Type;
		float RelativeTime=  (float)(FPlatformTime::Seconds() - GStartTime);
		(*FileWriter) << RelativeTime;
		LastAddress = nullptr;
	}
}

/**
* Tracks when connection address changes, and will emit token if different from last connection
*
* @param	Connection				Destination address
*/
void FNetworkProfiler::SetCurrentConnection( UNetConnection* Connection )
{
	if ( bIsTrackingEnabled && Connection != nullptr )
	{
		uint32 Index = INDEX_NONE;

		const TSharedPtr<const FInternetAddr> ConnectionAddr = Connection->GetRemoteAddr();
		if ( ConnectionAddr.IsValid() )
		{
			if ( LastAddress != ConnectionAddr )
			{
				Index = GetAddressTableIndex(ConnectionAddr->ToString(true));
			}
		}
		else
		{
			static const FString InvalidConnection = FString("Invalid Connection");
			Index = GetAddressTableIndex(InvalidConnection);
		}

		if (Index != INDEX_NONE)
		{
			uint8 Type = NPTYPE_ConnectionChanged;
			(*FileWriter) << Type;
			(*FileWriter).SerializeIntPacked(Index);

			LastAddress = ConnectionAddr;
		}
	}
}

/**
 * Tracks and RPC being sent.
 * 
 * @param	Actor		Actor RPC is being called on
 * @param	Function	Function being called
 * @param	NumBits		Number of bits serialized into bunch for this RPC
 */
void FNetworkProfiler::TrackSendRPC(const AActor* Actor, const UFunction* Function, uint32 NumHeaderBits, uint32 NumParameterBits, uint32 NumFooterBits, UNetConnection* Connection)
{
	if (bIsTrackingEnabled)
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection(Connection);

		uint32 ActorNameTableIndex = GetNameTableIndex(GetNameSafe(Actor->GetClass()));
		uint32 FunctionNameTableIndex = GetNameTableIndex(Function->GetName());

		uint8 Type = NPTYPE_SendRPC;
		(*FileWriter) << Type;
		(*FileWriter).SerializeIntPacked(ActorNameTableIndex);
		(*FileWriter).SerializeIntPacked(FunctionNameTableIndex);
		(*FileWriter).SerializeIntPacked(NumHeaderBits);
		(*FileWriter).SerializeIntPacked(NumParameterBits);
		(*FileWriter).SerializeIntPacked(NumFooterBits);
	}
}

void FNetworkProfiler::TrackQueuedRPC(UNetConnection* Connection, UObject* TargetObject, const AActor* Actor, const UFunction* Function, uint32 NumHeaderBits, uint32 NumParameterBits, uint32 NumFooterBits)
{
	if (bIsTrackingEnabled)
	{
		SCOPE_LOCK_REF(CriticalSection);
		
		FQueuedRPCInfo Info;

		Info.ActorNameIndex = GetNameTableIndex(GetNameSafe(Actor->GetClass()));
		Info.FunctionNameIndex = GetNameTableIndex(Function->GetName());

		Info.Connection = Connection;
		Info.TargetObject = TargetObject;
		Info.NumHeaderBits = NumHeaderBits;
		Info.NumParameterBits = NumParameterBits;
		Info.NumFooterBits = NumFooterBits;

		QueuedRPCs.Add(Info);
	}
}

void FNetworkProfiler::FlushQueuedRPCs(UNetConnection* Connection, UObject* TargetObject)
{
	if (bIsTrackingEnabled)
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection(Connection);

		for (int i = QueuedRPCs.Num() - 1; i >= 0; --i)
		{
			if (QueuedRPCs[i].Connection == Connection && QueuedRPCs[i].TargetObject == TargetObject)
			{
				uint8 Type = NPTYPE_SendRPC;
				(*FileWriter) << Type;
				(*FileWriter).SerializeIntPacked(QueuedRPCs[i].ActorNameIndex);
				(*FileWriter).SerializeIntPacked(QueuedRPCs[i].FunctionNameIndex);
				(*FileWriter).SerializeIntPacked(QueuedRPCs[i].NumHeaderBits);
				(*FileWriter).SerializeIntPacked(QueuedRPCs[i].NumParameterBits);
				(*FileWriter).SerializeIntPacked(QueuedRPCs[i].NumFooterBits);

				QueuedRPCs.RemoveAtSwap(i);
			}
		}
	}
}

/**
 * Low level FSocket::Send information.
 *
 * @param	SocketDesc				Description of socket data is being sent to
 * @param	Data					Data sent
 * @param	BytesSent				Bytes actually being sent
 */
void FNetworkProfiler::TrackSocketSend( const FString& SocketDesc, const void* Data, uint16 BytesSent )
{
	if( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);
		uint32 DummyIp = 0;
		//TrackSocketSendToCore( SocketDesc, Data, BytesSent, DummyIp );
	}
}

/**
 * Low level FSocket::SendTo information.
 *
 * @param	SocketDesc				Description of socket data is being sent to
 * @param	Data					Data sent
 * @param	BytesSent				Bytes actually being sent
 * @param	Connection				Destination address
 */
void FNetworkProfiler::TrackSocketSendTo(
	const FString& SocketDesc,
	const void* Data,
	uint16 BytesSent,
	uint16 NumPacketIdBits,
	uint16 NumBunchBits,
	uint16 NumAckBits,
	uint16 NumPaddingBits,
	UNetConnection* Connection )
{
	if( bIsTrackingEnabled )
	{
		TrackSocketSendToCore( SocketDesc, Data, BytesSent, NumPacketIdBits, NumBunchBits, NumAckBits, NumPaddingBits, Connection);
	}
}

/**
 * Low level FSocket::SendTo information.
 *
 * @param	SocketDesc				Description of socket data is being sent to
 * @param	Data					Data sent
 * @param	BytesSent				Bytes actually being sent
 * @param	Connection				Destination address
 */
void FNetworkProfiler::TrackSocketSendToCore(
	const FString& SocketDesc,
	const void* Data,
	uint16 BytesSent,
	uint16 NumPacketIdBits,
	uint16 NumBunchBits,
	uint16 NumAckBits,
	uint16 NumPaddingBits,
	UNetConnection* Connection )
{
	if( bIsTrackingEnabled )
	{
		// Low level socket code is called from multiple threads.
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection( Connection );

		uint32 NameTableIndex = GetNameTableIndex( SocketDesc );

		uint8 Type = NPTYPE_SocketSendTo;
		(*FileWriter) << Type;
		(*FileWriter).SerializeIntPacked( NameTableIndex );
		(*FileWriter) << BytesSent;
		(*FileWriter) << NumPacketIdBits;
		(*FileWriter) << NumBunchBits;
		(*FileWriter) << NumAckBits;
		(*FileWriter) << NumPaddingBits;
#if NETWORK_PROFILER_TRACK_RAW_NETWORK_DATA
		Type = NPTYPE_RawSocketData;
		(*FileWriter) << Type;
		(*FileWriter) << BytesSent;
		check( FileWriter->IsSaving() );
		FileWriter->Serialize( const_cast<void*>(Data), BytesSent );
#endif
		bHasNoticeableNetworkTrafficOccured = true;
	}
}

/**
 * Mid level UChannel::SendBunch information.
 * 
 * @param	Channel		UChannel data is being sent to/ on
 * @param	OutBunch	FOutBunch being sent
 * @param	NumBits		Num bits to serialize for this bunch (not including merging)
 */
void FNetworkProfiler::TrackSendBunch( FOutBunch* OutBunch, uint16 NumBits, UNetConnection* Connection )
{
	if( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF( CriticalSection );

		SetCurrentConnection( Connection );

		uint8 Type = NPTYPE_SendBunch;
		(*FileWriter) << Type;
		uint16 ChannelIndex = OutBunch->ChIndex;
		(*FileWriter) << ChannelIndex;
		uint32 NameTableIndex = GetNameTableIndex(OutBunch->ChName.ToString());
		(*FileWriter).SerializeIntPacked(NameTableIndex);
		(*FileWriter) << NumBits;
	}
}

void FNetworkProfiler::PushSendBunch( UNetConnection* Connection, FOutBunch* OutBunch, uint16 NumHeaderBits, uint16 NumPayloadBits )
{
	if ( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);
		OutgoingBunches.FindOrAdd(Connection).Emplace(OutBunch->ChIndex, GetNameTableIndex(OutBunch->ChName.ToString()), NumHeaderBits, NumPayloadBits);
	}
}

void FNetworkProfiler::PopSendBunch( UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);
		if ( OutgoingBunches.Contains(Connection) && OutgoingBunches[Connection].Num() > 0 )
		{
			OutgoingBunches[Connection].Pop();
		}
	}
}

void FNetworkProfiler::FlushOutgoingBunches( UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF( CriticalSection );

		SetCurrentConnection( Connection );

		if ( OutgoingBunches.Contains( Connection ) )
		{
			for ( FSendBunchInfo& BunchInfo : OutgoingBunches[Connection] )
			{
				uint8 Type = NPTYPE_SendBunch;
				(*FileWriter) << Type;
				(*FileWriter) << BunchInfo.ChannelIndex;
				(*FileWriter).SerializeIntPacked(BunchInfo.ChannelTypeNameIndex);
				(*FileWriter) << BunchInfo.NumHeaderBits;
				(*FileWriter) << BunchInfo.NumPayloadBits;
			}

			OutgoingBunches[Connection].SetNum(0);
		}
	}
}

/**
 * Track actor being replicated.
 *
 * @param	Actor		Actor being replicated
 */
void FNetworkProfiler::TrackReplicateActor( const AActor* Actor, FReplicationFlags RepFlags, uint32 Cycles, UNetConnection* Connection )
{
	if (bIsTrackingEnabled)
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection(Connection);

		uint32 NameTableIndex = GetNameTableIndex(GetNameSafe(Actor->GetClass()));

		uint8 Type = NPTYPE_ReplicateActor;
		(*FileWriter) << Type;
		uint8 NetFlags = (RepFlags.bNetInitial << 1) | (RepFlags.bNetOwner << 2);
		(*FileWriter) << NetFlags;
		(*FileWriter).SerializeIntPacked(NameTableIndex);
		float TimeInMS = FPlatformTime::ToMilliseconds(Cycles);	// FIXME: We may want to just pass in cycles to profiler to we don't lose precision
		(*FileWriter) << TimeInMS;
		// Use actor replication as indication whether session is worth keeping or not.
		bHasNoticeableNetworkTrafficOccured = true;
	}
}

template<typename Allocator>
static void ProfilerSerializeBitArray(FArchive& Ar, TBitArray<Allocator>& ToWrite)
{
	uint32 NumBits = ToWrite.Num();
	Ar.SerializeIntPacked(NumBits);

	uint32* Data = ToWrite.GetData();
	const uint32 NumDWords = ((NumBits + NumBitsPerDWORD - 1) >> NumBitsPerDWORDLogTwo);
	for (uint32 i = 0; i < NumDWords; ++i)
	{
		Ar.SerializeIntPacked(Data[i]);
	}
}

void FNetworkProfiler::TrackCompareProperties_Unsafe(const FString& ObjectName, uint32 Cycles, TBitArray<>& PropertiesCompared, TBitArray<>& PropertiesThatChanged, TArray<uint8>& PropertyNameExportData)
{
	uint32 ObjectNameTableIndex = GetNameTableIndex(ObjectName);
	uint8 Type = NPTYPE_PropertyComparison;
	float TimeInMS = FPlatformTime::ToMilliseconds(Cycles);

	(*FileWriter) << Type;
	(*FileWriter).SerializeIntPacked(ObjectNameTableIndex);
	(*FileWriter) << TimeInMS;
	ProfilerSerializeBitArray(*FileWriter, PropertiesCompared);
	ProfilerSerializeBitArray(*FileWriter, PropertiesThatChanged);

	if (PropertyNameExportData.Num() == 0)
	{
		uint32 Size = 0;
		(*FileWriter).SerializeIntPacked(Size);
	}
	else
	{
		(*FileWriter).Serialize(PropertyNameExportData.GetData(), PropertyNameExportData.Num());
	}

}

void FNetworkProfiler::TrackReplicatePropertiesMetadata(const UObject* Object, TBitArray<>& InactiveProperties, bool bSentAllProperties, UNetConnection* Connection)
{
	if (IsComparisonTrackingEnabled())
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection(Connection);

		uint32 ObjectNameTableIndex = GetNameTableIndex(GetNameSafe(Object));
		uint8 Type = NPTYPE_ReplicatePropertiesMetadata;
		uint8 Flags = (bSentAllProperties ? 1 : 0);
		
		(*FileWriter) << Type;
		(*FileWriter).SerializeIntPacked(ObjectNameTableIndex);
		(*FileWriter) << Flags;
		ProfilerSerializeBitArray(*FileWriter, InactiveProperties);
	}
}

void FNetworkProfiler::SerializeReplicateProperty( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection )
{
	SetCurrentConnection( Connection );

	uint8 Type = NPTYPE_ReplicateProperty;
	(*FileWriter) << Type;
	(*FileWriter).SerializeIntPacked( NameTableIndex );
	(*FileWriter) << NumBits;
}

/**
 * Track property being replicated.
 *
 * @param	Property	Property being replicated
 * @param	NumBits		Number of bits used to replicate this property
 */
void FNetworkProfiler::TrackReplicateProperty( const FProperty* Property, uint16 NumBits, UNetConnection* Connection )
{
	if(bIsTrackingEnabled && !!!IgnorePropertyCount)
	{
		UE::TScopeLock ScopeLock(CriticalSection);

		uint32 NameTableIndex = GetNameTableIndex( Property->GetName() );

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_ONCOMMIT(this, NameTableIndex, NumBits, Connection)
			{
				UE::TScopeLock ScopeLock2(CriticalSection);

				SerializeReplicateProperty(NameTableIndex, NumBits, Connection);
			};
		}
		else
		{
			SerializeReplicateProperty(NameTableIndex, NumBits, Connection);
		}
	}
}

void FNetworkProfiler::SerializeWritePropertyHeader( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection )
{
	SetCurrentConnection( Connection );

	uint8 Type = NPTYPE_WritePropertyHeader;
	(*FileWriter) << Type;
	(*FileWriter).SerializeIntPacked( NameTableIndex );
	(*FileWriter) << NumBits;
}

void FNetworkProfiler::TrackWritePropertyHeader( const FProperty* Property, uint16 NumBits, UNetConnection* Connection )
{
	if( bIsTrackingEnabled )
	{
		UE::TScopeLock ScopeLock(CriticalSection);

		uint32 NameTableIndex = GetNameTableIndex( Property->GetName() );

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_ONCOMMIT(this, NameTableIndex, NumBits, Connection)
			{
				UE::TScopeLock ScopeLock2(CriticalSection);

				SerializeWritePropertyHeader(NameTableIndex, NumBits, Connection);
			};
		}
		else
		{
			SerializeWritePropertyHeader(NameTableIndex, NumBits, Connection);
		}
	}
}

/**
 * Track event occurring, like e.g. client join/ leave
 *
 * @param	EventName			Name of the event
 * @param	EventDescription	Additional description/ information for event
 */
void FNetworkProfiler::TrackEvent( const FString& EventName, const FString& EventDescription, UNetConnection* Connection )
{
	if( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection( Connection );

		uint32 EventNameNameTableIndex = GetNameTableIndex( EventName );
		uint32 EventDescriptionNameTableIndex = GetNameTableIndex( EventDescription );

		uint8 Type = NPTYPE_Event;
		(*FileWriter) << Type;
		(*FileWriter).SerializeIntPacked( EventNameNameTableIndex );
		(*FileWriter).SerializeIntPacked( EventDescriptionNameTableIndex );
	}
}

/**
 * Called when the server first starts listening and on round changes or other
 * similar game events. We write to a dummy file that is renamed when the current
 * session ends.
 *
 * @param	bShouldContinueTracking		Whether to continue tracking
 * @param	InURL						URL used for new session
 */
void FNetworkProfiler::TrackSessionChange( bool bShouldContinueTracking, const FURL& InURL )
{
	if ( bIsTrackingEnabled || bShouldTrackingBeEnabled )
	{
		UE_LOGF( LogNet, Log, "Network Profiler: TrackSessionChange.  InURL: %ls", *InURL.ToString() );

		// Session change might occur while other thread uses low level networking.
		SCOPE_LOCK_REF(CriticalSection);

		// End existing tracking session.
		if( FileWriter )
		{	
			UE_LOGF( LogNet, Log, "Network Profiler: Closing session file for '%ls'", *CurrentHeader.GetURL() );

			if ( !bHasNoticeableNetworkTrafficOccured )
			{
				UE_LOGF( LogNet, Warning, "Network Profiler: Nothing important happened" );
			}

			// Write end of stream marker.
			uint8 Type = NPTYPE_EndOfStreamMarker;
			( *FileWriter ) << Type;

			if (OnNetworkProfileFinished().IsBound())
			{
				OnNetworkProfileFinished().Broadcast(FileWriter->GetArchiveName());
			}

			// Clean up.
			delete FileWriter;
			FileWriter = nullptr;
			bHasNoticeableNetworkTrafficOccured = false;
		}

		if( bShouldContinueTracking )
		{
			// Start a new tracking session.
			check( FileWriter == nullptr );

			static int32 Salt = 0;
			Salt++;		// Use a salt to solve the issue where this function is called so fast it produces the same time (seems to happen during seamless travel)

			FString FileName;

			if (NextFileName.IsEmpty())
			{
				FileName = FApp::GetProjectName() + FString::Printf(TEXT("-Pid%i"), FPlatformProcess::GetCurrentProcessId()) + TEXT("-") + FDateTime::Now().ToString() + FString::Printf(TEXT("[%i]"), Salt) + TEXT(".nprof");
			}
			else
			{
				FileName = MoveTemp(NextFileName);
			}
			const FString FinalFileName = FPaths::ProfilingDir() + FileName;

			IFileManager::Get().MakeDirectory( *FPaths::GetPath( FinalFileName ) );
			FileWriter = new FTransactionallySafeArchiveWriter( TUniquePtr<FArchive>( IFileManager::Get().CreateFileWriter( *FinalFileName, FILEWRITE_EvenIfReadOnly )));
			check( FileWriter );
			
			UE_LOGF(LogNet, Log, "Network Profiler: Creating session file at location %ls", *FinalFileName);

			// Reset the arrays and maps so that they will match up for the new profile.
			NameToNameTableIndexMap.Reset();
			NameArray.Reset();
			AddressTableIndexMap.Reset();

			CurrentHeader.Reset(InURL);

			// Serialize a header of the proper size, overwritten when session ends.
			(*FileWriter) << CurrentHeader;

			//Mark that tracking truly is enabled now
			bIsTrackingEnabled = bShouldTrackingBeEnabled = true;

			if (OnNetworkProfileStarted().IsBound())
			{
				OnNetworkProfileStarted().Broadcast(FileWriter->GetArchiveName());
			}
		}
		else
		{
			//Mark that tracking should not be enabled
			bIsTrackingEnabled = bShouldTrackingBeEnabled = false;
		}
	}
}

void FNetworkProfiler::TrackSendAck( uint16 NumBits, UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection( Connection );

		uint8 Type = NPTYPE_SendAck;
		(*FileWriter) << Type;
		(*FileWriter) << NumBits;
	}
}

void FNetworkProfiler::TrackMustBeMappedGuids( uint16 NumGuids, uint16 NumBits, UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection( Connection );

		uint8 Type = NPTYPE_MustBeMappedGuids;
		(*FileWriter) << Type;
		(*FileWriter) << NumGuids;
		(*FileWriter) << NumBits;
	}
}

void FNetworkProfiler::TrackExportBunch( uint16 NumBits, UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		SCOPE_LOCK_REF(CriticalSection);

		SetCurrentConnection( Connection );

		uint8 Type = NPTYPE_ExportBunch;
		(*FileWriter) << Type;
		(*FileWriter) << NumBits;
	}
}

void FNetworkProfiler::SerializeBeginContentBlock( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection )
{
	SetCurrentConnection( Connection );

	uint8 Type = NPTYPE_BeginContentBlock;
	(*FileWriter) << Type;
	(*FileWriter).SerializeIntPacked( NameTableIndex );
	(*FileWriter) << NumBits;
}

void FNetworkProfiler::TrackBeginContentBlock( UObject* Object, uint16 NumBits, UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		UE::TScopeLock ScopeLock(CriticalSection);

		uint32 NameTableIndex = GetNameTableIndex( Object != nullptr ? Object->GetName() : UnknownName );

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_ONCOMMIT(this, NameTableIndex, NumBits, Connection)
			{
				UE::TScopeLock ScopeLock2(CriticalSection);

				SerializeBeginContentBlock(NameTableIndex, NumBits, Connection);
			};
		}
		else
		{
			SerializeBeginContentBlock(NameTableIndex, NumBits, Connection);
		}
	 }
}

void FNetworkProfiler::SerializeEndContentBlock( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection )
{
	SetCurrentConnection( Connection );

	uint8 Type = NPTYPE_EndContentBlock;
	(*FileWriter) << Type;
	(*FileWriter).SerializeIntPacked( NameTableIndex );
	(*FileWriter) << NumBits;
}

void FNetworkProfiler::TrackEndContentBlock( UObject* Object, uint16 NumBits, UNetConnection* Connection )
{
	if ( bIsTrackingEnabled )
	{
		UE::TScopeLock ScopeLock(CriticalSection);

		uint32 NameTableIndex = GetNameTableIndex( Object != nullptr ? Object->GetName() : UnknownName );

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_ONCOMMIT(this, NameTableIndex, NumBits, Connection)
			{
				UE::TScopeLock ScopeLock2(CriticalSection);

				SerializeEndContentBlock(NameTableIndex, NumBits, Connection);
			};
		}
		else
		{
			SerializeEndContentBlock(NameTableIndex, NumBits, Connection);
		}
	}
}

void FNetworkProfiler::SerializeWritePropertyHandle( uint16 NumBits, UNetConnection* Connection )
{
	SetCurrentConnection( Connection );

	uint8 Type = NPTYPE_WritePropertyHandle;
	(*FileWriter) << Type;
	(*FileWriter) << NumBits;
}

void FNetworkProfiler::TrackWritePropertyHandle( uint16 NumBits, UNetConnection* Connection )
{
	if (bIsTrackingEnabled && !!!IgnorePropertyCount)
	{
		UE_AUTORTFM_ONCOMMIT(this, NumBits, Connection)
		{
			SCOPE_LOCK_REF(CriticalSection);

			SerializeWritePropertyHandle(NumBits, Connection);
		};
	}
}

void FNetworkProfiler::SetNextFileName(const FString& FileName)
{
	NextFileName = FileName;
}

/**
 * Processes any network profiler specific exec commands
 *
 * @param InWorld	The world in this context
 * @param Cmd		The command to parse
 * @param Ar		The output device to write data to
 *
 * @return			True if processed, false otherwise
*/
bool FNetworkProfiler::Exec( UWorld * InWorld, const TCHAR* Cmd, FOutputDevice & Ar )
{
	if (FParse::Command(&Cmd, TEXT("ENABLE")))
	{
		EnableTracking(true);
	}
	else if (FParse::Command(&Cmd, TEXT("DISABLE")))
	{
		EnableTracking(false);
	} 
	else if (FParse::Command(&Cmd, TEXT("AUTOSTOP")))
	{
		EnableTracking(true);

		// Default to 30secs
		float AutoStopTime = 30.f;
		FParse::Value(Cmd, TEXT("TIME="), AutoStopTime);

		if( InWorld )
		{
			InWorld->GetTimerManager().SetTimer(AutoStopTimerHandle, FTimerDelegate::CreateRaw(this, &FNetworkProfiler::AutoStopTracking), AutoStopTime, false);
		}
	}
	else 
	{
		// Default to toggle
		EnableTracking( !bIsTrackingEnabled );
	}

	// If we are tracking, and we don't have a file writer, force one now 
	if ( bShouldTrackingBeEnabled && FileWriter == nullptr ) 
	{
		TrackSessionChange( true, InWorld != nullptr ? InWorld->URL : FURL() );
		if ( FileWriter == nullptr )
		{
			UE_LOGF(LogNet, Warning, "FNetworkProfiler::Exec: FAILED to create file writer!");
			EnableTracking( false );
		}
	}

	return true;
}

void FNetworkProfiler::AutoStopTracking()
{
	if (bIsTrackingEnabled)
	{
		EnableTracking(false);
	}
}

#endif	//#if USE_NETWORK_PROFILER

