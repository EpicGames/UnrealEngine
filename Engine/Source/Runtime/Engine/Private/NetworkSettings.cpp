// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/NetworkSettings.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkSettings)

UNetworkSettings::UNetworkSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Network");
}

static FName NetworkConsoleVariableFName("ConsoleVariable");

void UNetworkSettings::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITOR
void UNetworkSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
#endif // #if WITH_EDITOR


