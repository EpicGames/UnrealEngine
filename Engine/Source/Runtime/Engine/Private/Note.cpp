// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Note.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Note)

ANote::ANote(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("Arrow"));
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> NoteTextureObject;
			FName ID_Notes;
			FText NAME_Notes;
			FConstructorStatics()
				: NoteTextureObject(TEXT("/Engine/EditorResources/S_Note"))
				, ID_Notes(TEXT("Notes"))
				, NAME_Notes(NSLOCTEXT("SpriteCategory", "Notes", "Notes"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(150, 200, 255);

			ArrowComponent->ArrowSize = 0.5f;
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Notes;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Notes;
			ArrowComponent->Mobility = EComponentMobility::Static;
			ArrowComponent->bIsScreenSizeScaled = true;
			ArrowComponent->SetupAttachment(RootComponent);
		}

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.NoteTextureObject.Get();
			SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Notes;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Notes;
			SpriteComponent->SetupAttachment(RootComponent);
			SpriteComponent->Mobility = EComponentMobility::Static;
		}
	}
#endif // WITH_EDITORONLY_DATA

	SetHidden(true);
	SetCanBeDamaged(false);
}

