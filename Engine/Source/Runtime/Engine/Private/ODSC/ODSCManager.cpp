// Copyright Epic Games, Inc. All Rights Reserved.

#include "ODSC/ODSCManager.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "ODSCLog.h"
#include "ODSCThread.h"
#include "Containers/BackgroundableTicker.h"
#include "MaterialShared.h"
#include "Materials/MaterialInstance.h"
#include "Materials/Material.h"

// For GetMaxSupportedFeatureLevel
#include "DataDrivenShaderPlatformInfo.h"

// For GetCachedScalabilityCVars
#include "UnrealEngine.h"

#include "Engine/Console.h"
#include "Engine/GameViewportClient.h"
#include "EngineUtils.h"

DEFINE_LOG_CATEGORY(LogODSC);

// FODSCManager

static TAutoConsoleVariable<int32> CVarODSCRecompileMode(
	TEXT("ODSC.recompilemode"),
	1,
	TEXT("Highly experimental - Changes how recompileshaders behaves in cooked build\n")
	TEXT("0 (legacy): Gathers all visible materials in a single frame and compiles all permutations for them\n")
	TEXT("1 (default): Compile only the permutations that are requested by the renderer. Faster iteration but more prone to hitching because of MDC recaching\n")
	);

static int32 GODSCUseDefaultMaterialOnRecompile = 0;
static FAutoConsoleVariableRef CVarODSCUseDefaultMaterialOnRecompile(
	TEXT("ODSC.usedefaultmaterialonrecompile"),
	GODSCUseDefaultMaterialOnRecompile,
	TEXT("Indicates if the default material should be used while waiting for a shader to be compiled by ODSC\n")
	TEXT("0 (default) - use the default material only if the permutation is misssing\n")
	TEXT("1 - use the default material even if a permutation exists\n")
	TEXT("Setting to 1 can be useful when changing uniform buffer layout on some shaders (SHADER_PARAMETER_STRUCT for example) and avoid recooking.\n")
	TEXT("This won't work if the default material's layout is changed\n"),
	ECVF_RenderThreadSafe
);

FODSCManager* GODSCManager = nullptr;

class FODSCManagerAccess
{
public:
	static void ODSCLogMissedMaterials(const TArray<FString>& Args)
	{
#if WITH_ODSC && !NO_LOGGING
		if (FODSCManager::IsODSCActive())
		{
			TArray<FString> MaterialPaths;
			GODSCManager->Thread->RetrieveMissedMaterials(MaterialPaths);
			UConsole* ViewportConsole = (GEngine->GameViewport != nullptr) ? GEngine->GameViewport->ViewportConsole : nullptr;
			FConsoleOutputDevice StrOut(ViewportConsole);

			for (const FString& MaterialKey : MaterialPaths)
			{
				StrOut.CategorizedLogf(LogODSC.GetCategoryName(), ELogVerbosity::Error, TEXT("ODSC missed material: %s"), *MaterialKey);
			}
		}
#endif
	}
};

static FAutoConsoleCommand ODSCLogMissedMaterialsCmd(
	TEXT("odsc.logmissedmaterials"),
	TEXT("Logs materials that were not found by the ODSC server"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FODSCManagerAccess::ODSCLogMissedMaterials)
);

namespace ODSCManagerPrivate
{
#if WITH_ODSC
	static thread_local int32 ODSCSuspendForceRecompileCount=0;
	static thread_local FPrimitiveSceneInfo* CurrentPrimitiveSceneInfo=nullptr;
#endif
}

void FODSCManager::SetCurrentPrimitiveSceneInfo(FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
#if WITH_ODSC
	ODSCManagerPrivate::CurrentPrimitiveSceneInfo = PrimitiveSceneInfo;
#endif
}

void FODSCManager::ResetCurrentPrimitiveSceneInfo()
{
#if WITH_ODSC
	ODSCManagerPrivate::CurrentPrimitiveSceneInfo = nullptr;
#endif
}

FODSCManager::FODSCManager()
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
{
	if (IsODSCEnabled())
	{
		FString Host;
		const bool bHostSet = FParse::Value(FCommandLine::Get(), TEXT("-odschost="), Host);

		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FODSCManager::OnEnginePreExit);
		Thread = new FODSCThread(Host);
		Thread->StartThread();
		OnScreenMessagesHandle = FCoreDelegates::OnGetOnScreenMessages.AddLambda([this](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
			{
				if (Thread)
				{
					FString LocalErrorMessage;
					RetrieveErrorMessage(LocalErrorMessage);
					if (!LocalErrorMessage.IsEmpty())
					{
						OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error, FText::FromString(LocalErrorMessage));
					}

					bool bIsConnectedToODSCServer;
					bool bHasPendingGlobalShaders;
					uint32 NumPendingMaterialsRecompile;
					uint32 NumPendingMaterialsShaders;
					bool bHasShaderData = Thread->GetPendingShaderData(bIsConnectedToODSCServer, bHasPendingGlobalShaders, NumPendingMaterialsRecompile, NumPendingMaterialsShaders);
					if (bIsConnectedToODSCServer && bHasShaderData)
					{
						FString Message = "Recompiling shaders (";
						if (bHasPendingGlobalShaders)
						{
							Message += "global";
						}

						if (NumPendingMaterialsRecompile > 0)
						{
							Message += FString::Printf(TEXT(" %d materials"), NumPendingMaterialsRecompile);
						}

						if (NumPendingMaterialsShaders > 0)
						{
							Message += FString::Printf(TEXT("%d pipelines"), NumPendingMaterialsShaders);
						}

						Message += ")";
						OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(Message));
					}
					else if (!bIsConnectedToODSCServer)
					{
						OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error, FText::FromString(FString::Printf(TEXT("Not connected to %s"), *Thread->GetODSCHostIP())));
					}
				}
			}
		);
	}
}

FODSCManager::~FODSCManager()
{
	if (OnScreenMessagesHandle.IsValid())
	{
		FCoreDelegates::OnGetOnScreenMessages.Remove(OnScreenMessagesHandle);
	}

	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	StopThread();
}

void FODSCManager::OnEnginePreExit()
{
	StopThread();
}

void FODSCManager::StopThread()
{
	if (Thread)
	{
		Thread->StopThread();
		delete Thread;
		Thread = nullptr;
	}
}

bool FODSCManager::Tick(float DeltaSeconds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FODSCManager_Tick);

	if (IsHandlingRequests())
	{
		Thread->Wakeup();

		TArray<FODSCMessageHandler*> CompletedThreadedRequests;
		Thread->GetCompletedRequests(CompletedThreadedRequests);

		bool bFlushAsyncLoading = HasAsyncLoadingInstances();

		if (CompletedThreadedRequests.Num() && bFlushAsyncLoading)
		{
			FlushAsyncLoading();
		}

		// Finish and remove any completed requests
		for (FODSCMessageHandler* CompletedRequest : CompletedThreadedRequests)
		{
			check(CompletedRequest);
			ProcessCookOnTheFlyShaders(false, CompletedRequest->GetMeshMaterialMaps(), CompletedRequest->GetMaterialsToLoad(), CompletedRequest->GetGlobalShaderMap());
			delete CompletedRequest;
		}
		// keep ticking
		return true;
	}
	// stop ticking
	return false;
}

void FODSCManager::AddThreadedRequest(
	const TArray<FString>& MaterialsToCompile,
	const FString& ShaderTypesToLoad,
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel,
	ODSCRecompileCommand RecompileCommandType,
	const FString& RequestedMaterialName,
	const FShaderCompilerFlags& ExtraCompilerFlags
)
{
	if (IsHandlingRequests())
	{
		ClearErrorMessage();

		Thread->AddRequest(TArray<FString>(), FString(), ShaderPlatform, FeatureLevel, QualityLevel, ODSCRecompileCommand::ResetMaterialCache);

		if ((RecompileCommandType == ODSCRecompileCommand::Material || RecompileCommandType == ODSCRecompileCommand::Changed)
			&& (CVarODSCRecompileMode.GetValueOnAnyThread() > 0))
		{
			Thread->ResetMaterialsODSCData(ShaderPlatform);

			// Rendering commands got flushed by ResetMaterialsODSCData
			MaterialNameToRecompile = FName(*RequestedMaterialName);	
			
			// when we ask for "changed", we want both materials and global
			if (RecompileCommandType == ODSCRecompileCommand::Changed)
			{
				Thread->AddRequest(TArray<FString>(), FString(), ShaderPlatform, FeatureLevel, QualityLevel, ODSCRecompileCommand::Changed, ExtraCompilerFlags);
			}
		}
		else
		{
			Thread->AddRequest(MaterialsToCompile, ShaderTypesToLoad, ShaderPlatform, FeatureLevel, QualityLevel, RecompileCommandType, ExtraCompilerFlags);
		}
	}
}

void FODSCManager::AddThreadedShaderPipelineRequest(
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel,
	const FMaterial* Material,
	const FString& VertexFactoryName,
	const FString& PipelineName,
	const TArray<FString>& ShaderTypeNames,
	int32 PermutationId,
	const TArray<FShaderId>& RequestShaderIds)
{
#if WITH_ODSC
	// Testing ODSCManagerPrivate::ODSCSuspendForceRecompileCount since a map from ODSC may actually be in use and we request PSO precaching
	if (IsHandlingRequests() && ODSCManagerPrivate::ODSCSuspendForceRecompileCount == 0)
	{
		Thread->AddShaderPipelineRequest(ShaderPlatform, FeatureLevel, QualityLevel, Material, ODSCManagerPrivate::CurrentPrimitiveSceneInfo, VertexFactoryName, PipelineName, ShaderTypeNames, PermutationId, RequestShaderIds);
	}
#endif
}

bool FODSCManager::IsODSCEnabled()
{
	FString Host;
	const bool bODSCEnabled = FParse::Value(FCommandLine::Get(), TEXT("-odschost="), Host);

	if (IsRunningCookOnTheFly() || bODSCEnabled)
	{
		return true;
	}
	return false;
}

void FODSCManager::RegisterMaterialInstance(const UMaterialInstance* MaterialInstance)
{
	if (IsODSCActive() && MaterialInstance->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
	{
		FScopeLock Lock(&GODSCManager->MaterialInstancesCachedUniformExpressionsCS);
		TWeakObjectPtr<const UMaterialInstance>& MaterialInstanceSoftPtr = GODSCManager->MaterialInstancesCachedUniformExpressions.FindOrAdd(MaterialInstance);
		MaterialInstanceSoftPtr = TWeakObjectPtr<const UMaterialInstance>(MaterialInstance);
	}
} 

void FODSCManager::UnregisterMaterialInstance(const UMaterialInstance* MaterialInstance)
{
	if (GODSCManager != nullptr)
	{
		FScopeLock Lock(&GODSCManager->MaterialInstancesCachedUniformExpressionsCS);
		GODSCManager->MaterialInstancesCachedUniformExpressions.Remove(MaterialInstance);
	}
}

bool FODSCManager::HasAsyncLoadingInstances()
{
	FScopeLock Lock(&MaterialInstancesCachedUniformExpressionsCS);

	bool bHasAsyncLoadingInstances = false;
	for (auto Iter = MaterialInstancesCachedUniformExpressions.CreateIterator(); Iter; ++Iter)
	{
		const UMaterialInstance* MI = Iter.Value().Get();

		if (MI == nullptr || !MI->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
		{
			Iter.RemoveCurrent();
			continue;
		}

		bHasAsyncLoadingInstances = true;
	}

	return bHasAsyncLoadingInstances;
}


void FODSCManager::SuspendODSCForceRecompile()
{
#if WITH_ODSC
	check(ODSCManagerPrivate::ODSCSuspendForceRecompileCount >= 0);
	++ODSCManagerPrivate::ODSCSuspendForceRecompileCount;
#endif
}

void FODSCManager::ResumeODSCForceRecompile()
{
#if WITH_ODSC
	--ODSCManagerPrivate::ODSCSuspendForceRecompileCount;
	check(ODSCManagerPrivate::ODSCSuspendForceRecompileCount >= 0);
#endif
}

bool FODSCManager::ShouldForceRecompileInternal(const FMaterialShaderMap* MaterialShaderMap, const FMaterial* Material)
{
#if WITH_ODSC
	if (!FPlatformProperties::RequiresCookedData() || CVarODSCRecompileMode.GetValueOnAnyThread() == 0  || ODSCManagerPrivate::ODSCSuspendForceRecompileCount > 0)
	{
		return false;
	}

	if (MaterialShaderMap->IsFromODSC())
	{
		return false;
	}

	if (Material->IsDefaultMaterial() && !Material->IsLightFunction())
	{
		return false;
	}

	if (!MaterialNameToRecompile.IsNone())
	{
		EODSCMetaDataType ODSCMetaData = (EODSCMetaDataType)Material->GetODSCMetaData();

		// No locking on the material ODSC metadata: the dependency chain/material names are not supposed to be changing on a per frame basis
		if (ODSCMetaData == EODSCMetaDataType::Default)
		{
			UMaterialInterface* EngineMaterialInterface = Material->GetMaterialInterface();
			TSet<UMaterialInterface*> MaterialDependencies;
			EngineMaterialInterface->GetDependencies(MaterialDependencies);

			bool bHasDependencies = false;
			for (auto Iter : MaterialDependencies)
			{
				UMaterialInterface* MIDep = Iter;
				if (MIDep && MIDep->GetFName() == MaterialNameToRecompile)
				{
					bHasDependencies = true;
				}
			}

			ODSCMetaData = (bHasDependencies) ? EODSCMetaDataType::IsDependentOnMaterialName : EODSCMetaDataType::IsNotDependentOnMaterialName;
			Material->SetODSCMetaData((uint8)ODSCMetaData);
		}
		return ODSCMetaData == EODSCMetaDataType::IsDependentOnMaterialName;
	}

	// when running with cook on the fly there are full shader-maps built by the cooker already
	// ODSC will only occur for this material if a shader is missing or changed
	if (IsRunningCookOnTheFly())
	{
		return false;	
	}

	return true;
#else
	return false;
#endif
}

bool FODSCManager::CheckIfRequestAlreadySent(const TArray<FShaderId>& RequestShaderIds, const FMaterial* Material) const
{
	if (Thread)
	{
		return Thread->CheckIfRequestAlreadySent(RequestShaderIds, Material);
	}
	return false;
}

void FODSCManager::UnregisterMaterialName(const FMaterial* Material)
{
	if (IsODSCActive())
	{
		GODSCManager->Thread->UnregisterMaterialName(Material);
	}
}

void FODSCManager::RegisterMaterialShaderMaps(const FString& MaterialName, const TArray<TRefCountPtr<FMaterialShaderMap>>& LoadedShaderMaps)
{
	if (IsODSCActive())
	{
		GODSCManager->Thread->RegisterMaterialShaderMaps(MaterialName, LoadedShaderMaps);
	}
}

FMaterialShaderMap* FODSCManager::FindMaterialShaderMap(const FString& MaterialName, const FMaterialShaderMapId& ShaderMapId)
{
	if (IsODSCActive())
	{
		return GODSCManager->Thread->FindMaterialShaderMap(MaterialName, ShaderMapId);
	}
	return nullptr;
}

void FODSCManager::TryLoadGlobalShaders(EShaderPlatform ShaderPlatform)
{
	check(IsODSCActive());
	ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);
	const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	Thread->AddRequest(TArray<FString>(), FString(), ShaderPlatform, TargetFeatureLevel, ActiveQualityLevel, ODSCRecompileCommand::Changed);
	Thread->Wakeup();
	Thread->WaitUntilAllRequestsDone();
	TArray<FODSCMessageHandler*> CompletedThreadedRequests;
	Thread->GetCompletedRequests(CompletedThreadedRequests);
	// Finish and remove any completed requests
	for (FODSCMessageHandler* CompletedRequest : CompletedThreadedRequests)
	{
		check(CompletedRequest);
		ProcessCookOnTheFlyShaders(false, CompletedRequest->GetMeshMaterialMaps(), CompletedRequest->GetMaterialsToLoad(), CompletedRequest->GetGlobalShaderMap());
		delete CompletedRequest;
	}
}

void FODSCManager::ReportODSCError(const FString& InErrorMessage)
{
	if (GODSCManager && !InErrorMessage.IsEmpty())
	{
		FScopeLock Lock(&GODSCManager->ErrorMessageCS);
		GODSCManager->ErrorMessage += InErrorMessage;
		GODSCManager->ErrorMessage += FString("\n");
	}
}

bool FODSCManager::UseDefaultMaterialOnRecompile()
{
	return GODSCUseDefaultMaterialOnRecompile > 0;
}

void FODSCManager::RetrieveErrorMessage(FString& OutErrorMessage)
{
	FScopeLock Lock(&ErrorMessageCS);
	OutErrorMessage = ErrorMessage;
}

void FODSCManager::ClearErrorMessage()
{
	FScopeLock Lock(&ErrorMessageCS);
	ErrorMessage.Empty();
}
