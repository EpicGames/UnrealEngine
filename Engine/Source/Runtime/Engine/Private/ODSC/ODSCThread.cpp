// Copyright Epic Games, Inc. All Rights Reserved.

#include "ODSCThread.h"
#include "CookOnTheFly.h"
#include "ODSCLog.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "MaterialShared.h"
#include "Materials/MaterialInterface.h"
#include "UObject/UObjectIterator.h"
#include "PrimitiveSceneInfo.h"
#include "Components/PrimitiveComponent.h"


static int32 GODSCMaxPayloadPerRequests = 200;
static FAutoConsoleVariableRef CVarODSCMaxPayloadPerRequests(
	TEXT("ODSC.MaxPayloadPerRequests"),
	GODSCMaxPayloadPerRequests,
	TEXT("Specify how many shaders compilations requests we send in a single message to the ODSC server \n")
);

FODSCRequestPayload::FODSCRequestPayload(
	EShaderPlatform InShaderPlatform,
	ERHIFeatureLevel::Type InFeatureLevel,
	EMaterialQualityLevel::Type InQualityLevel,
	const FString& InMaterialName,
	const FString& InVertexFactoryName,
	const FString& InPipelineName,
	const TArray<FString>& InShaderTypeNames,
	int32 InPermutationId,
	const FString& InRequestHash
)
: ShaderPlatform(InShaderPlatform)
, FeatureLevel(InFeatureLevel)
, QualityLevel(InQualityLevel)
, MaterialName(InMaterialName)
, VertexFactoryName(InVertexFactoryName)
, PipelineName(InPipelineName)
, ShaderTypeNames(std::move(InShaderTypeNames))
, PermutationId(InPermutationId)
, RequestHash(InRequestHash)
{

}

FArchive& operator<<(FArchive& Ar, FODSCRequestPayload& Payload)
{
	int32 iShaderPlatform = static_cast<int32>(Payload.ShaderPlatform);
	int32 iFeatureLevel = static_cast<int32>(Payload.FeatureLevel);
	int32 iQualityLevel = static_cast<int32>(Payload.QualityLevel);

	Ar << iShaderPlatform;
	Ar << iFeatureLevel;
	Ar << iQualityLevel;
	Ar << Payload.MaterialName;
	Ar << Payload.VertexFactoryName;
	Ar << Payload.PipelineName;
	Ar << Payload.ShaderTypeNames;
	Ar << Payload.PermutationId;
	Ar << Payload.RequestHash;

	if (Ar.IsLoading())
	{
		Payload.ShaderPlatform = static_cast<EShaderPlatform>(iShaderPlatform);
		Payload.FeatureLevel = static_cast<ERHIFeatureLevel::Type>(iFeatureLevel);
		Payload.QualityLevel = static_cast<EMaterialQualityLevel::Type>(iQualityLevel);
	}

	return Ar;
}

FODSCMessageHandler::FODSCMessageHandler(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQualityLevel, ODSCRecompileCommand InRecompileCommandType)
:	ShaderPlatform(InShaderPlatform),
	FeatureLevel(InFeatureLevel),
	QualityLevel(InQualityLevel),
	RecompileCommandType(InRecompileCommandType)
{
}

FODSCMessageHandler::FODSCMessageHandler(
	const TArray<FString>& InMaterials,
	const FString& ShaderTypesToLoad,
	EShaderPlatform InShaderPlatform,
	ERHIFeatureLevel::Type InFeatureLevel,
	EMaterialQualityLevel::Type InQualityLevel,
	ODSCRecompileCommand InRecompileCommandType,
	const FShaderCompilerFlags& InExtraCompilerFlags)
:	MaterialsToLoad(std::move(InMaterials)),
	ShaderTypesToLoad(ShaderTypesToLoad),
	ShaderPlatform(InShaderPlatform),
	FeatureLevel(InFeatureLevel),
	QualityLevel(InQualityLevel),
	RecompileCommandType(InRecompileCommandType),
	ExtraCompilerFlags(InExtraCompilerFlags)
{
}

void FODSCMessageHandler::FillPayload(FArchive& Payload)
{
	// When did we start this request?
	RequestStartTime = FPlatformTime::Seconds();

	int32 ConvertedShaderPlatform = static_cast<int32>(ShaderPlatform);
	int32 ConvertedFeatureLevel = static_cast<int32>(FeatureLevel);
	int32 ConvertedQualityLevel = static_cast<int32>(QualityLevel);

	Payload << MaterialsToLoad;
	Payload << ShaderTypesToLoad;
	Payload << ExtraCompilerFlags;
	Payload << ConvertedShaderPlatform;
	Payload << ConvertedFeatureLevel;
	Payload << ConvertedQualityLevel;
	Payload << RecompileCommandType;
	Payload << RequestBatch;
}

void FODSCMessageHandler::ProcessResponse(FArchive& Response)
{
	UE_LOGF(LogODSC, Display, "Received response in %lf seconds.", FPlatformTime::Seconds() - RequestStartTime);

	// pull back the compiled mesh material data (if any)
	Response << OutMeshMaterialMaps;
	Response << OutGlobalShaderMap;
}

void FODSCMessageHandler::AddPayload(const FODSCRequestPayload& Payload)
{
	RequestBatch.Add(Payload);
}

const TArray<FString>& FODSCMessageHandler::GetMaterialsToLoad() const
{
	return MaterialsToLoad;
}

const TArray<uint8>& FODSCMessageHandler::GetMeshMaterialMaps() const
{
	return OutMeshMaterialMaps;
}

const TArray<uint8>& FODSCMessageHandler::GetGlobalShaderMap() const
{
	return OutGlobalShaderMap;
}

bool FODSCMessageHandler::ReloadGlobalShaders() const
{
	return RecompileCommandType == ODSCRecompileCommand::Global;
}

FODSCThread::FODSCThread(const FString& HostIP)
	: Thread(nullptr)
	, WakeupEvent(FPlatformProcess::GetSynchEventFromPool(true))
	, AllRequestsDoneEvent(FPlatformProcess::GetSynchEventFromPool(true))
	, ODSCHostIP(HostIP)
{
	UE_LOGF(LogODSC, Log, "ODSC Thread active.");

	bHasDefaultConnection = (FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly")).GetDefaultServerConnection() != nullptr);
	if (!bHasDefaultConnection)
	{
		ConnectToODSCHost();
	}
}

FODSCThread::~FODSCThread()
{
	StopThread();

	FPlatformProcess::ReturnSynchEventToPool(AllRequestsDoneEvent);
	AllRequestsDoneEvent = nullptr;
	FPlatformProcess::ReturnSynchEventToPool(WakeupEvent);
	WakeupEvent = nullptr;
}

bool FODSCThread::ConnectToODSCHost()
{
	// If we don't have a default connection make a specific connection to the HostIP provided.
	UE::Cook::FCookOnTheFlyHostOptions CookOnTheFlyHostOptions;
	CookOnTheFlyHostOptions.Hosts.Add(ODSCHostIP);
	CookOnTheFlyServerConnection = FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly")).ConnectToServer(CookOnTheFlyHostOptions);
	if (!CookOnTheFlyServerConnection)
	{
		UE_LOGF(LogODSC, Warning, "Failed to connect to cook on the fly server.");
		return false;
	}
	return CookOnTheFlyServerConnection != nullptr && CookOnTheFlyServerConnection->IsConnected();
}

bool FODSCThread::CheckODSCConnection()
{
	// If we have a default connection that already exists, send directly to that.
	if ((CookOnTheFlyServerConnection == nullptr) || (!CookOnTheFlyServerConnection->IsConnected()))
	{
		// Losing connection when exit is requested is expected, do not try to reconnect
		if (ExitRequest.GetValue())
		{
			return false;
		}

		UE_LOGF(LogODSC, Display, "Detected that CookOnTheFlyServerConnection has been lost, trying again");
		if (!ConnectToODSCHost())
		{
			return false;
		}
	}
	return CookOnTheFlyServerConnection != nullptr && CookOnTheFlyServerConnection->IsConnected();
}

void FODSCThread::StartThread()
{
	Thread = FRunnableThread::Create(this, TEXT("ODSCThread"), 128 * 1024, TPri_Normal);
}

void FODSCThread::StopThread()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}
}

void FODSCThread::Tick()
{
	Process();
}

void FODSCThread::ResetMaterialsODSCData(EShaderPlatform InShaderPlatform)
{
#if WITH_ODSC
	FlushRenderingCommands();

	{
		FWriteScopeLock WriteLock(RequestHashesRWLock);

		// this will stop the rendering thread, and reattach components, in the destructor
		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::Default);
		RequestHashes.Empty();

		for (TObjectIterator<UMaterialInterface> It; It; ++It)
		{ 
			UMaterialInterface* Material = *It;
			if (Material)
			{
				const FMaterialResource* MaterialResource = Material->GetMaterialResource(InShaderPlatform);
				if (MaterialResource && MaterialResource->GetGameThreadShaderMap())
				{
					MaterialResource->GetGameThreadShaderMap()->SetIsFromODSC(false);
#if WITH_ODSC
					MaterialResource->SetODSCMetaData((uint8)0);
#endif
				}
				UpdateContext.AddMaterialInterface(Material);
			}
		}
	}
#endif
}

FODSCThread::FODSCShaderId::FODSCShaderId(const FShaderId& ShaderId)
: ShaderTypeHashedName(ShaderId.Type ? ShaderId.Type->GetHashedName() : 0)
, VFTypeHashedName(ShaderId.VFType ? ShaderId.VFType->GetHashedName() : 0)
, ShaderPipelineName(ShaderId.ShaderPipelineName)
, PermutationId(ShaderId.PermutationId)
, Platform(ShaderId.Platform)
{}

bool FODSCThread::CheckIfRequestAlreadySent(const TArray<FShaderId>& RequestShaderIds, const FMaterial* Material) const
{
	FReadScopeLock ReadLock(RequestHashesRWLock);
	const FName* CachedMaterialName = ODSCPointerToNames.Find((UPTRINT)Material);
	if (CachedMaterialName == nullptr)
	{
		return false;
	}

	const FODSCShaderMapData* ODSCShaderMapData = RequestHashes.Find(*CachedMaterialName);
	if (ODSCShaderMapData == nullptr)
	{
		return false;
	}

	for (const FShaderId& ShaderId : RequestShaderIds)
	{
		if (!ODSCShaderMapData->CurrentRequests.Contains(ShaderId))
		{
			return false;
		}
	}

	return true;
}

void FODSCThread::UnregisterMaterialName(const FMaterial* Material)
{
	FWriteScopeLock WriteLock(RequestHashesRWLock);
	ODSCPointerToNames.Remove((UPTRINT)Material);
}

void FODSCThread::RegisterMaterialShaderMaps(const FString& MaterialName, const TArray<TRefCountPtr<FMaterialShaderMap>>& LoadedShaderMaps)
{
	FWriteScopeLock WriteLock(RequestHashesRWLock);

	FODSCShaderMapData& ODSCShaderMapData = RequestHashes.FindOrAdd(FName(MaterialName));

	ODSCShaderMapData.MaterialShaderMaps = LoadedShaderMaps;

	for (FMaterialShaderMap* MaterialShaderMap : LoadedShaderMaps)
	{
		TMap<FShaderId, TShaderRef<FShader>> ShadersInMap;
		MaterialShaderMap->GetShaderList(ShadersInMap);
		for (auto Iter : ShadersInMap)
		{
			// The shadermap we receive contains all the requests the client sent until now, so it's possible they already got removed
			if (ODSCShaderMapData.CurrentRequests.Find(Iter.Key))
			{
				ODSCShaderMapData.CurrentRequests.Remove(Iter.Key);
			}
		}
	}

}

FMaterialShaderMap* FODSCThread::FindMaterialShaderMap(const FString& MaterialName, const FMaterialShaderMapId& ShaderMapId) const
{
	FReadScopeLock ReadLock(RequestHashesRWLock);
	const FODSCShaderMapData* ODSCShaderMapData = RequestHashes.Find(FName(MaterialName));
	if (ODSCShaderMapData == nullptr)
	{
		return nullptr;
	}

	for (FMaterialShaderMap* MaterialShaderMap : ODSCShaderMapData->MaterialShaderMaps)
	{
		const FMaterialShaderMapId& ExistingShaderMapId = MaterialShaderMap->GetShaderMapId();
		bool bFeatureLevelMatch = (ExistingShaderMapId.FeatureLevel == ShaderMapId.FeatureLevel);
		bool bQualityLevelMatch = (ShaderMapId.QualityLevel == EMaterialQualityLevel::Num || ExistingShaderMapId.QualityLevel == EMaterialQualityLevel::Num
								   || ShaderMapId.QualityLevel == ExistingShaderMapId.QualityLevel);

		if (bFeatureLevelMatch && bQualityLevelMatch)
		{
			return MaterialShaderMap;
		}
	}

	return nullptr;
}

void FODSCThread::RetrieveMissedMaterials(TArray<FString>& OutMaterialPaths) const
{
	FReadScopeLock ReadLock(RequestHashesRWLock);
	for (auto Iter = RequestHashes.CreateConstIterator(); Iter; ++Iter)
	{
		const FODSCShaderMapData& ODSCShaderMapData = Iter.Value();
		if (!ODSCShaderMapData.CurrentRequests.IsEmpty())
		{
			FString MaterialKey(Iter.Key().ToString());
			if (ODSCShaderMapData.ActorPath.IsValid())
			{
				MaterialKey += ":::";
				MaterialKey += ODSCShaderMapData.ActorPath.ToString();
			}
			OutMaterialPaths.Add(MaterialKey);
		}
	}
}

void FODSCThread::AddRequest(
	const TArray<FString>& MaterialsToCompile,
	const FString& ShaderTypesToLoad,
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel,
	ODSCRecompileCommand RecompileCommandType,
	const FShaderCompilerFlags& ExtraCompilerFlags)
{
	PendingMaterialThreadedRequests.Enqueue(new FODSCMessageHandler(MaterialsToCompile, ShaderTypesToLoad, ShaderPlatform, FeatureLevel, QualityLevel, RecompileCommandType, ExtraCompilerFlags));
}

void FODSCThread::AddShaderPipelineRequest(
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type QualityLevel,
	const FMaterial* Material,
	const FPrimitiveSceneInfo* PrimitiveSceneInfo,
	const FString& VertexFactoryName,
	const FString& PipelineName,
	const TArray<FString>& ShaderTypeNames,
	int32 PermutationId,
	const TArray<FShaderId>& RequestShaderIds
)
{
	bool bShouldAddRequest = false;
	bool bDefaultMaterial = false;

	FString ActorPath;
	{
		FWriteScopeLock WriteLock(RequestHashesRWLock);

		FName& CachedMaterialName = ODSCPointerToNames.FindOrAdd((UPTRINT)Material);
		if (CachedMaterialName.IsNone())
		{
			CachedMaterialName = FName(Material->GetFullPath());
		}
		
		bDefaultMaterial = Material->IsDefaultMaterial();

		FODSCShaderMapData& ODSCShaderMapData = RequestHashes.FindOrAdd(CachedMaterialName);

		for (const FShaderId& ShaderId : RequestShaderIds)
		{
			bool bAlreadyInSet = false;
			ODSCShaderMapData.CurrentRequests.Add(ShaderId, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				bShouldAddRequest = true;
			}
		}

		// for default materials, we request all the permutations anyway
		if (bDefaultMaterial && ODSCShaderMapData.CurrentRequests.Num() > 1)
		{
			bShouldAddRequest = false;
		}

		if (bShouldAddRequest)
		{
#if WITH_ODSC && IS_MONOLITHIC
			if (PrimitiveSceneInfo)
			{
				AActor* OwningActor = PrimitiveSceneInfo->GetComponentForDebugOnly() ? PrimitiveSceneInfo->GetComponentForDebugOnly()->GetOwner() : nullptr;
				if (OwningActor)
				{
					ActorPath = OwningActor->GetPathName();
				}
			}
#endif
		}

		if (!ActorPath.IsEmpty())
		{
			ODSCShaderMapData.ActorPath = FName(ActorPath);
		}
	}

	if (bShouldAddRequest)
	{
		SCOPED_NAMED_EVENT(AddShaderPipelineRequest_AddRequest, FColor::Emerald);

		FString MaterialName = Material->GetFullPath();

		if (!bDefaultMaterial && !ActorPath.IsEmpty())
		{
			MaterialName += ":::";
			MaterialName += ActorPath;
		}

		FString RequestString = (MaterialName + VertexFactoryName + PipelineName);
		for (const auto& ShaderTypeName : ShaderTypeNames)
		{
			RequestString += ShaderTypeName;
		}
		const FString RequestHash = FMD5::HashAnsiString(*RequestString);
		if (bDefaultMaterial)
		{
			TArray<FString> MaterialsToCompile = {MaterialName};
			FString ShaderTypesToLoad;
			PendingMaterialThreadedRequests.Enqueue(new FODSCMessageHandler(MaterialsToCompile, ShaderTypesToLoad, ShaderPlatform, FeatureLevel, QualityLevel, ODSCRecompileCommand::Material));
		}
		else
		{
			PendingMeshMaterialThreadedRequests.Enqueue(FODSCRequestPayload(ShaderPlatform, FeatureLevel, QualityLevel, MaterialName, VertexFactoryName, PipelineName, ShaderTypeNames, PermutationId, RequestHash));
		}
	}
}

void FODSCThread::GetCompletedRequests(TArray<FODSCMessageHandler*>& OutCompletedRequests)
{
	check(IsInGameThread());
	FODSCMessageHandler* Request = nullptr;
	while (CompletedThreadedRequests.Dequeue(Request))
	{
		OutCompletedRequests.Add(Request);
	}
}

void FODSCThread::Wakeup()
{
	AllRequestsDoneEvent->Reset();
	WakeupEvent->Trigger();
}

void FODSCThread::WaitUntilAllRequestsDone()
{
	AllRequestsDoneEvent->Wait();
}

bool FODSCThread::Init()
{
	return true;
}

uint32 FODSCThread::Run()
{
	while (!ExitRequest.GetValue())
	{
		if (WakeupEvent->Wait())
		{
			Process();
		}
	}
	return 0;
}

void FODSCThread::Stop()
{
	ExitRequest.Set(true);
	WakeupEvent->Trigger();
}

void FODSCThread::Exit()
{

}

void CollectPendingMeshMaterialRequests(TQueue<FODSCRequestPayload, EQueueMode::Mpsc>& PendingMeshMaterialThreadedRequests, TArray<FODSCMessageHandler*>& PendingRequestsPipeline, 
										std::atomic<uint32>& NumPendingMaterialsShaders, const int32 MaxPayloadPerRequests)
{
	TArray<FODSCRequestPayload> PayloadsToAggregate;
	FODSCRequestPayload Payload;
	while (PendingMeshMaterialThreadedRequests.Dequeue(Payload))
	{
		PayloadsToAggregate.Add(Payload);
	}

	if (PayloadsToAggregate.Num())
	{
		FODSCMessageHandler* RequestHandler = new FODSCMessageHandler(PayloadsToAggregate[0].ShaderPlatform, PayloadsToAggregate[0].FeatureLevel, PayloadsToAggregate[0].QualityLevel, ODSCRecompileCommand::Material);
		for (const FODSCRequestPayload& payload : PayloadsToAggregate)
		{
			if (RequestHandler->NumPayloads() >= MaxPayloadPerRequests)
			{
				PendingRequestsPipeline.Add(RequestHandler);
				NumPendingMaterialsShaders += RequestHandler->NumPayloads();
				RequestHandler = new FODSCMessageHandler(PayloadsToAggregate[0].ShaderPlatform, PayloadsToAggregate[0].FeatureLevel, PayloadsToAggregate[0].QualityLevel, ODSCRecompileCommand::Material);;
			}

			RequestHandler->AddPayload(payload);
		}
		PendingRequestsPipeline.Add(RequestHandler);
		NumPendingMaterialsShaders += RequestHandler->NumPayloads();
	}
}

void FODSCThread::Process()
{
	MaxPayloadPerRequests = FMath::Max(1, GODSCMaxPayloadPerRequests);

	// cache all pending material/global requests
	{
		FODSCMessageHandler* Request = nullptr;
		while (PendingMaterialThreadedRequests.Dequeue(Request))
		{
			PendingRequestsMaterialAndGlobal.Add(Request);
		}
	}

	if (bHasDefaultConnection)
	{
		bIsConnectedToODSCServer = true;
	}
	else
	{
		bIsConnectedToODSCServer = CheckODSCConnection();
	}

	ON_SCOPE_EXIT
	{
		// SendMessageToServer is synchronous, so when we're here, we know we've processed all the requests
		WakeupEvent->Reset();
		AllRequestsDoneEvent->Trigger();
	};

	// Early out to avoid trying to connect (and most likely fail) for every compilation request
	if (!bIsConnectedToODSCServer)
	{
		return;
	}

	// cache material requests.
	TArray<FODSCMessageHandler*> RequestsToStart = MoveTemp(PendingRequestsMaterialAndGlobal);
	bool bHasGlobalShaders = false;
	uint32 NumMaterials = 0;

	for (FODSCMessageHandler* NextRequest : RequestsToStart)
	{
		if (NextRequest->GetRecompileCommandType() != ODSCRecompileCommand::Material)
		{
			bHasGlobalShaders = true;
		}
		else
		{
			NumMaterials += NextRequest->GetMaterialsToLoad().Num();
		}
	}

	bHasPendingGlobalShaders.store(bHasGlobalShaders, std::memory_order_release);
	NumPendingMaterialsRecompile.store(NumMaterials, std::memory_order_release);

	// process any material or recompile change shader requests or global shader compile requests.
	for (FODSCMessageHandler* NextRequest : RequestsToStart)
	{
		// send the info, the handler will process the response (and update shaders, etc)
		if (SendMessageToServer(NextRequest))
		{
			CompletedThreadedRequests.Enqueue(NextRequest);
		}
		else
		{
			PendingRequestsMaterialAndGlobal.Add(NextRequest);
		}

	}

	bHasPendingGlobalShaders.store(false, std::memory_order_release);
	NumPendingMaterialsRecompile.store(0, std::memory_order_release);

	RequestsToStart = MoveTemp(PendingRequestsPipeline);

	NumPendingMaterialsShaders.store(0, std::memory_order_release);

	CollectPendingMeshMaterialRequests(PendingMeshMaterialThreadedRequests, RequestsToStart, NumPendingMaterialsShaders, MaxPayloadPerRequests);

	int32 NextRequestIndex = 0;

	while (NextRequestIndex < RequestsToStart.Num())
	{
		FODSCMessageHandler* NextRequest = RequestsToStart[NextRequestIndex];
		if (SendMessageToServer(NextRequest))
		{
			CompletedThreadedRequests.Enqueue(NextRequest);
		}
		else
		{
			PendingRequestsPipeline.Add(NextRequest);
		}
		NumPendingMaterialsShaders -= NextRequest->NumPayloads();

		CollectPendingMeshMaterialRequests(PendingMeshMaterialThreadedRequests, RequestsToStart, NumPendingMaterialsShaders, MaxPayloadPerRequests);
		++NextRequestIndex;
	}

	NumPendingMaterialsShaders.store(0, std::memory_order_release);
}

bool FODSCThread::SendMessageToServer(IPlatformFile::IFileServerMessageHandler* Handler)
{
	if (bHasDefaultConnection)
	{
		IFileManager::Get().SendMessageToServer(TEXT("RecompileShaders"), Handler);
		return true;
	}

	if (!CheckODSCConnection())
	{
		return false;
	}

	// We don't have a default COTF connection so use our specific connection to send our command.
	UE::Cook::FCookOnTheFlyRequest Request(UE::Cook::ECookOnTheFlyMessage::RecompileShaders);
	{
		TUniquePtr<FArchive> Ar = Request.WriteBody();
		Handler->FillPayload(*Ar);
	}

	UE::Cook::FCookOnTheFlyResponse Response = CookOnTheFlyServerConnection->SendRequest(Request).Get();
	if (Response.IsOk())
	{
		TUniquePtr<FArchive> Ar = Response.ReadBody();
		Handler->ProcessResponse(*Ar);
		return true;
	}
	else
	{
		UE_LOGF(LogODSC, Display, "Received error response from CookOnTheFlyServerConnection; disconnecting");
		CookOnTheFlyServerConnection.Reset();
		return false;
	}
}

bool FODSCThread::GetPendingShaderData(bool& bOutIsConnectedToODSCServer, bool& bOutHasPendingGlobalShaders, uint32& OutNumPendingMaterialsRecompile, uint32& OutNumPendingMaterialsShaders) const
{
	bOutIsConnectedToODSCServer = bIsConnectedToODSCServer.load(std::memory_order_acquire);
	bOutHasPendingGlobalShaders = bHasPendingGlobalShaders.load(std::memory_order_acquire);
	OutNumPendingMaterialsRecompile = NumPendingMaterialsRecompile.load(std::memory_order_acquire);
	OutNumPendingMaterialsShaders = NumPendingMaterialsShaders.load(std::memory_order_acquire);
	return bOutHasPendingGlobalShaders || OutNumPendingMaterialsRecompile > 0 || OutNumPendingMaterialsShaders > 0;
}
