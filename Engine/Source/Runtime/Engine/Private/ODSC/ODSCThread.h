// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "Misc/SingleThreadRunnable.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Containers/Queue.h"
#include "ShaderCompiler.h"
#include "RHIDefinitions.h"

class FEvent;
class FRunnableThread;
class FMaterialShaderMap;
class FMaterialShaderMapId;
class FPrimitiveSceneInfo;

namespace UE
{
	namespace Cook
	{
		class ICookOnTheFlyServerConnection;
		class FCookOnTheFlyMessage;
	}
}

enum class EODSCMetaDataType
{
	Default=0, // The material hasn't been seen by ODSCManager yet
	IsDependentOnMaterialName,
	IsNotDependentOnMaterialName,
};

class FODSCMessageHandler : public IPlatformFile::IFileServerMessageHandler
{
public:
	FODSCMessageHandler(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQualityLevel, ODSCRecompileCommand InRecompileCommandType);
	FODSCMessageHandler(
		const TArray<FString>& InMaterials,
		const FString& ShaderTypesToLoad,
		EShaderPlatform InShaderPlatform,
		ERHIFeatureLevel::Type InFeatureLevel,
		EMaterialQualityLevel::Type InQualityLevel,
		ODSCRecompileCommand InRecompileCommandType,
		const FShaderCompilerFlags& InExtraCompilerFlags = FShaderCompilerFlags()
	);

	/** Subclass fills out an archive to send to the server */
	virtual void FillPayload(FArchive& Payload) override;

	/** Subclass pulls data response from the server */
	virtual void ProcessResponse(FArchive& Response) override;

	void AddPayload(const FODSCRequestPayload& Payload);

	const TArray<FString>& GetMaterialsToLoad() const;
	const TArray<uint8>& GetMeshMaterialMaps() const;
	const TArray<uint8>& GetGlobalShaderMap() const;
	bool ReloadGlobalShaders() const;
	ODSCRecompileCommand GetRecompileCommandType() const { return RecompileCommandType; };
	int32 NumPayloads() const { return RequestBatch.Num(); }

private:
	/** The time when this command was issued.  This isn't serialized to the cooking server. */
	double RequestStartTime = 0.0;

	/** The materials we send over the network and expect maps for on the return */
	TArray<FString> MaterialsToLoad;

	/** The names of shader type file names to compile shaders for. */
	FString ShaderTypesToLoad;

	/** Which shader platform we are compiling for */
	EShaderPlatform ShaderPlatform;

	/** Which feature level to compile for. */
	ERHIFeatureLevel::Type FeatureLevel;

	/** Which material quality level to compile for. */
	EMaterialQualityLevel::Type QualityLevel;

	/** Whether or not to recompile changed shaders */
	ODSCRecompileCommand RecompileCommandType = ODSCRecompileCommand::None;

	/** Extra compiler flags sent to the shader compiler. This can be used to request individual shaders to be updated without optimizations for debugging. */
	FShaderCompilerFlags ExtraCompilerFlags = FShaderCompilerFlags();

	/** The payload for compiling a specific set of shaders. */
	TArray<FODSCRequestPayload> RequestBatch;

	/** The serialized shader maps from across the network */
	TArray<uint8> OutMeshMaterialMaps;

	/** The serialized global shader map from across the network */
	TArray<uint8> OutGlobalShaderMap;
};

/**
 * Manages ODSC thread
 * Handles sending requests to the cook on the fly server and communicating results back to the Game Thread.
 */
class FODSCThread
	: FRunnable, FSingleThreadRunnable
{
public:

	FODSCThread(const FString& HostIP);
	virtual ~FODSCThread();

	/**
	 * Start the ODSC thread.
	 */
	void StartThread();

	/**
	 * Stop the ODSC thread.  Blocks until thread has stopped.
	 */
	void StopThread();

	//~ Begin FSingleThreadRunnable Interface
	// Cannot be overriden to ensure identical behavior with the threaded tick
	virtual void Tick() override final;
	//~ End FSingleThreadRunnable Interface

	/**
	 * Add a shader compile request to be processed by this thread.
	 *
	 * @param MaterialsToCompile - List of material names to submit compiles for.
	 * @param ShaderTypesToLoad - List of shader types to submit compiles for.
	 * @param ShaderPlatform - Which shader platform to compile for.
	 * @param RecompileCommandType - Whether we should recompile changed or global shaders.
	 *
	 * @return false if no longer needs ticking
	 */
	void AddRequest(
		const TArray<FString>& MaterialsToCompile,
		const FString& ShaderTypesToLoad,
		EShaderPlatform ShaderPlatform,
		ERHIFeatureLevel::Type FeatureLevel,
		EMaterialQualityLevel::Type QualityLevel,
		ODSCRecompileCommand RecompileCommandType,
		const FShaderCompilerFlags& ExtraCompilerFlags = FShaderCompilerFlags()
	);

	/**
	 * Add a request to compile a pipeline (VS/PS) of shaders.  The results are submitted and processed in an async manner.
	 *
	 * @param ShaderPlatform - Which shader platform to compile for.
	 * @param FeatureLevel - Which feature level to compile for.
	 * @param QualityLevel - Which material quality level to compile for.
	 * @param MaterialName - The name of the material to compile.
	 * @param VertexFactoryName - The name of the vertex factory type we should compile.
	 * @param PipelineName - The name of the shader pipeline we should compile.
	 * @param ShaderTypeNames - The shader type names of all the shader stages in the pipeline.
	 * @param PermutationId - The permutation ID of the shader we should compile.
	 *
	 * @return false if no longer needs ticking
	 */
	void AddShaderPipelineRequest(
		EShaderPlatform ShaderPlatform,
		ERHIFeatureLevel::Type FeatureLevel,
		EMaterialQualityLevel::Type QualityLevel,
		const FMaterial* Material,
		const FPrimitiveSceneInfo* PrimitiveSceneInfo,
		const FString& VertexFactoryName,
		const FString& PipelineName,
		const TArray<FString>& ShaderTypeNames,
		int32 PermutationId,
		const TArray<FShaderId>& RequestShaderIds
	);

	/**
	 * Get completed requests.  Clears internal arrays.  Called on Game thread.
	 *
	 * @param OutCompletedRequests array of requests that have been completed
	 */
	void GetCompletedRequests(TArray<FODSCMessageHandler*>& OutCompletedRequests);

	/**
	* Wakeup the thread to process requests.
	*/
	void Wakeup();
	
	/**
	* Wait until all added requests are processed. Must be called after Wakeup.
	*/
	void WaitUntilAllRequestsDone();

	bool GetPendingShaderData(bool& bOutIsConnectedToODSCServer, bool& bOutHasPendingGlobalShaders, uint32& OutNumPendingMaterialsRecompile, uint32& OutNumPendingMaterialsShaders) const;

	void ResetMaterialsODSCData(EShaderPlatform InShaderPlatform);

	bool CheckIfRequestAlreadySent(const TArray<FShaderId>& RequestShaderIds, const FMaterial* Material) const;
	const FString& GetODSCHostIP() const { return ODSCHostIP; };

	void UnregisterMaterialName(const FMaterial* Material);
	void RegisterMaterialShaderMaps(const FString& MaterialName, const TArray<TRefCountPtr<FMaterialShaderMap>>& LoadedShaderMaps);
	FMaterialShaderMap* FindMaterialShaderMap(const FString& MaterialName, const FMaterialShaderMapId& ShaderMapId) const;

	void RetrieveMissedMaterials(TArray<FString>& OutMaterialPaths) const;

protected:

	//~ Begin FRunnable Interface
	virtual bool Init() override;
	virtual uint32 Run() override final;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable Interface

	/** signal request to stop and exit thread */
	FThreadSafeCounter ExitRequest;

private:

	/**
	 * Responsible for sending and waiting on compile requests with the cook on the fly server.
	 *
	 */
	void Process();

	bool ConnectToODSCHost();
	bool CheckODSCConnection();

	/**
	 * Threaded requests that are waiting to be processed on the ODSC thread.
	 * Added to on (any) non-ODSC thread, processed then cleared on ODSC thread.
	 */
	TQueue<FODSCMessageHandler*, EQueueMode::Mpsc> PendingMaterialThreadedRequests;

	/**
	 * Threaded requests that are waiting to be processed on the ODSC thread.
	 * Added to on (any) non-ODSC thread, processed then cleared on ODSC thread.
	 */
	TQueue<FODSCRequestPayload, EQueueMode::Mpsc> PendingMeshMaterialThreadedRequests;

	/**
	 * Threaded requests that have completed and are waiting for the game thread to process.
	 * Added to on ODSC thread, processed then cleared on game thread (Single producer, single consumer)
	 */
	TQueue<FODSCMessageHandler*, EQueueMode::Spsc> CompletedThreadedRequests;

	/** Lock to access the RequestHashes TMap */
	mutable FRWLock RequestHashesRWLock;
	FCriticalSection RequestHashCriticalSection;

	struct FODSCShaderId
	{
	public:
		inline FODSCShaderId() {}
		FODSCShaderId(const FShaderId& ShaderId);

		FHashedName ShaderTypeHashedName = 0;
		FHashedName VFTypeHashedName = 0;
		FHashedName ShaderPipelineName = 0;
		int32 PermutationId = 0;
		uint32 Platform : SP_NumBits = SP_NumPlatforms;

		friend inline uint32 GetTypeHash( const FODSCShaderId& Id )
		{
			return HashCombine(
				GetTypeHash(Id.ShaderTypeHashedName),
				HashCombine(GetTypeHash(Id.VFTypeHashedName),
							HashCombine(GetTypeHash(Id.ShaderPipelineName),
										HashCombine(GetTypeHash(Id.PermutationId), GetTypeHash(Id.Platform)))));
		}

		friend bool operator==(const FODSCShaderId& X, const FODSCShaderId& Y)
		{
			return X.ShaderTypeHashedName == Y.ShaderTypeHashedName
			&& X.ShaderPipelineName == Y.ShaderPipelineName
			&& X.VFTypeHashedName == Y.VFTypeHashedName
			&& X.PermutationId == Y.PermutationId 
			&& X.Platform == Y.Platform;
		}

		friend bool operator!=(const FODSCShaderId& X, const FODSCShaderId& Y)
		{
			return !(X == Y);
		}
	};


	struct FODSCShaderMapData
	{
    	/** All the shadermaps owned by the material (quality level / feature level) */
		TArray<TRefCountPtr<FMaterialShaderMap>> MaterialShaderMaps;
    	/** Hashes for all Pending or Completed requests.  This is so we avoid making the same request multiple times. */
		TSet<FODSCShaderId> CurrentRequests;
		
		FName ActorPath;
	};

    /** Requests seen for a given material name */
	TMap<FName, FODSCShaderMapData> RequestHashes;

    /** FMaterial* -> FName cache to avoid the expensive operation of calling FMaterialResource::GetFullPath and convert it to FName */
	TMap<UPTRINT, FName> ODSCPointerToNames; 

	/** Pointer to Runnable Thread */
	FRunnableThread* Thread = nullptr;

	/** Holds an event signaling the thread to wake up. */
	FEvent* WakeupEvent;
	
	/** Holds an event signaling when all the requests are processed*/
	FEvent* AllRequestsDoneEvent;

	bool SendMessageToServer(IPlatformFile::IFileServerMessageHandler* Handler);

	TArray<FODSCMessageHandler*> PendingRequestsMaterialAndGlobal;
	TArray<FODSCMessageHandler*> PendingRequestsPipeline;

	/** Special connection to the cooking server.  This is only used to send recompileshaders commands on. */
	TUniquePtr<UE::Cook::ICookOnTheFlyServerConnection> CookOnTheFlyServerConnection;

	FString ODSCHostIP;

	int32 MaxPayloadPerRequests = INT32_MAX;

	std::atomic<bool> bIsConnectedToODSCServer = false;
	std::atomic<bool> bHasPendingGlobalShaders = false;
	std::atomic<uint32> NumPendingMaterialsRecompile = 0;
	std::atomic<uint32> NumPendingMaterialsShaders = 0;
	bool bHasDefaultConnection = false;
};
