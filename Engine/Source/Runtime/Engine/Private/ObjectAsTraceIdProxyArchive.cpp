// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectAsTraceIdProxyArchive.h"
#include "Serialization/Archive.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Serialization/SerializedPropertyScope.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UnrealType.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Misc/PackageName.h"
#include "ObjectTrace.h"

/*----------------------------------------------------------------------------
	FObjectAsTraceIdProxyArchive.
----------------------------------------------------------------------------*/

FObjectAsTraceIdProxyArchive::FObjectAsTraceIdProxyArchive( FArchive& InInnerArchive)
	: FNameAsStringProxyArchive(InInnerArchive)
{
}

FObjectAsTraceIdProxyArchive::~FObjectAsTraceIdProxyArchive() = default;

/**
 * Serialize the given UObject* as a Traced Object Id
 */
void FObjectAsTraceIdProxyArchive::Write(const UObject* Obj)
{
#if OBJECT_TRACE_ENABLED
	if (Obj)
	{
		// first ensure the object has been traced 
		if (const UClass* Class = Cast<UClass>(Obj))
		{
			TRACE_TYPE(Class);
		}
		TRACE_OBJECT(Obj);
		// then serialize the Object as a Trace Object id;
		uint64 Id = FObjectTrace::GetObjectId(Obj);
		InnerArchive << Id;
	}
	else
#endif
	{
		// for null pointer, output 0
		uint64 zero = 0;
		InnerArchive << zero;
	}
}

FArchive& FObjectAsTraceIdProxyArchive::operator<<(UObject*& Obj)
{
	if (IsLoading())
	{
		// Loading isn't implemented for this class since we can't depend on GameplayProvider from here
	}
	else
	{
		Write(Obj);
	}
	return *this;
}


FArchive& FObjectAsTraceIdProxyArchive::operator<<(FWeakObjectPtr& Obj)
{
	return FArchiveUObject::SerializeWeakObjectPtr(*this, Obj);
}

FArchive& FObjectAsTraceIdProxyArchive::operator<<(FSoftObjectPtr& Value)
{
	if (IsLoading())
	{
		// Reset before serializing to clear the internal weak pointer. 
		Value.ResetWeakPtr();
	}
	*this << Value.GetUniqueID();
	return *this;
}

FArchive& FObjectAsTraceIdProxyArchive::operator<<(FSoftObjectPath& Value)
{
	Value.SerializePath(*this);
	return *this;
}

FArchive& FObjectAsTraceIdProxyArchive::operator<<(FObjectPtr& Obj)
{
	return FArchiveUObject::SerializeObjectPtr(*this, Obj);
}