// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectCacheContext.h"
#include "Engine/SkinnedAsset.h"
#include "UObject/UObjectIterator.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/TextureCollection.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Containers/Queue.h"
#include "Logging/LogMacros.h"
#include "HAL/LowLevelMemStats.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Components/ComponentInterfaceIterator.h"
#include "Components/InstancedSkinnedMeshComponent.h"

#define LOCTEXT_NAMESPACE "ObjectCache"

LLM_DEFINE_TAG(ObjectReverseLookupCache);

DEFINE_LOG_CATEGORY_STATIC(LogObjectCache, Log, All);

namespace ObjectCacheContextImpl {

EInternalObjectFlags GetObjectCacheInternalFlagsExclusion()
{
	// We never want to return objects that are invalid or still being worked on by other threads
	return EInternalObjectFlags::Unreachable | EInternalObjectFlags::Garbage | EInternalObjectFlags_AsyncLoading | EInternalObjectFlags::Async;
}

} // namespace ObjectCacheContextImpl

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"

static TAutoConsoleVariable<int32> CVarObjectReverseLookupMode(
	TEXT("Editor.ObjectReverseLookupMode"),
	1,
	TEXT("0 - Reverse lookup tables are computed every time they are needed (slower behavior) \n")
	TEXT("1 - Maintain permanent reverse lookup tables (faster behavior) \n")
	TEXT("2 - Comparison mode (slowest to do validation between both mode) \n"),
	ECVF_ReadOnly /* Can only be enabled from the command-line */
);

static TAutoConsoleVariable<int32> CVarObjectReverseLookupMask(
	TEXT("Editor.ObjectReverseLookupMask"),
	-1,
	TEXT("-1 - Does validation on all objects type (slowest) \n")
	TEXT("0  - Skip validation on all objects type \n")
	TEXT("1  - Perform validation on Texture to Textures lookups \n")
	TEXT("2  - Perform validation on Texture to Materials lookups \n")
	TEXT("4  - Perform validation on Material to Primitives lookups \n")
	TEXT("8  - Perform validation on StaticMesh To Components lookups \n")
	TEXT("16 - Perform validation on Material to Materials lookups \n")
	TEXT("32 - Perform validation on Texture Collection to Materials lookups \n")
	TEXT("64 - Perform validation on SkinnedAsset to Components lookups \n"),
	ECVF_Default /* Can only be enabled from the command-line */
);

namespace ObjectCacheContextImpl { void Validate(); }

static FAutoConsoleCommand CVarObjectReverseLookupValidate(
	TEXT("Editor.ObjectReverseLookupValidate"),
	TEXT("Compare objects contained in the reverse lookup against the old scanning method to see if there is any discrepenties."),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			ObjectCacheContextImpl::Validate();
		}
	)
);

namespace ObjectCacheContextImpl {

enum class EObjectReverseLookupMask : int32
{
	None = 0,
	TextureToTextureLookupCache = 1,
	TextureToMaterialLookupCache = 2,
	MaterialToPrimitiveLookupCache = 4,
	StaticMeshToComponentLookupCache = 8,
	MaterialToMaterialsLookupCache = 16,
	TextureCollectionToMaterialLookupCache = 32,
	SkinnedAssetToComponentLookupCache = 64,
	All = -1
};

ENUM_CLASS_FLAGS(EObjectReverseLookupMask);

enum class EObjectReverseLookupMode
{
	Temporary = 0,
	Permanent = 1,
	Comparison = 2
};

static bool GIsGetObjectReverseLookupModeInitialized = false;
EObjectReverseLookupMode GetObjectReverseLookupMode()
{
	if (!GIsGetObjectReverseLookupModeInitialized)
	{
		int32 Mode = 0;
		if (FParse::Value(FCommandLine::Get(), TEXT("-ObjectReverseLookupMode="), Mode))
		{
			CVarObjectReverseLookupMode->Set(Mode, ECVF_SetByCommandline);
		}
		GIsGetObjectReverseLookupModeInitialized = true;
	}

	return (EObjectReverseLookupMode)CVarObjectReverseLookupMode.GetValueOnAnyThread();
}

static bool GIsGetObjectReverseLookupMaskInitialized = false;
EObjectReverseLookupMask GetObjectReverseLookupMask()
{
	if (!GIsGetObjectReverseLookupMaskInitialized)
	{
		int32 Mask = 0;
		if (FParse::Value(FCommandLine::Get(), TEXT("-ObjectReverseLookupMask="), Mask))
		{
			CVarObjectReverseLookupMask->Set(Mask, ECVF_SetByCommandline);
		}
		GIsGetObjectReverseLookupMaskInitialized = true;
	}

	return (EObjectReverseLookupMask)CVarObjectReverseLookupMask.GetValueOnAnyThread();
}

bool IsReverseLookupComparisonActive(EObjectReverseLookupMask Mask)
{
	return GetObjectReverseLookupMode() == EObjectReverseLookupMode::Comparison && (GetObjectReverseLookupMask() & Mask) != EObjectReverseLookupMask::None;
}

bool ShouldUseLookupTable(EObjectReverseLookupMask Mask)
{
	const EObjectReverseLookupMode Mode = GetObjectReverseLookupMode();
	switch (Mode)
	{
		case EObjectReverseLookupMode::Comparison:
			return !IsReverseLookupComparisonActive(Mask);
		case EObjectReverseLookupMode::Permanent:
			return true;
		default:
			return false;
	}
}

struct FDefaultObjectSearchPredicate
{
	EInternalObjectFlags ExcludedInternalFlags;

	FDefaultObjectSearchPredicate()
		: ExcludedInternalFlags(GetObjectCacheInternalFlagsExclusion())
	{
	}

	bool operator() (IPrimitiveComponent* Object) const
	{
		return Object->GetUObject() == nullptr || !Object->GetUObject()->HasAnyInternalFlags(ExcludedInternalFlags);
	}

	bool operator() (IStaticMeshComponent* Object) const
	{
		return Object->GetPrimitiveComponentInterface()->GetUObject() == nullptr || !Object->GetPrimitiveComponentInterface()->GetUObject()->HasAnyInternalFlags(ExcludedInternalFlags);
	}

	bool operator() (ISkinnedMeshComponent* Object) const
	{
		return Object->GetPrimitiveComponentInterface()->GetUObject() == nullptr || !Object->GetPrimitiveComponentInterface()->GetUObject()->HasAnyInternalFlags(ExcludedInternalFlags);
	}
	
	bool operator() (UObject* Object) const
	{
		return Object==nullptr || !Object->HasAnyInternalFlags(ExcludedInternalFlags);
	}
};

template <typename FromType, typename ToType>
class FObjectReverseLookupCache
{
public:
	void RemoveFrom(FromType* InFrom)
	{
		UE::TWriteScopeLock ScopeLock(Lock);

		TRawSet<ToType*>* ToObjects = FromToMapping.Find(InFrom);
		if (ToObjects)
		{
			for (ToType* ToObject : *ToObjects)
			{
				TRawSet<FromType*>* FromObjects = ToFromMapping.Find(ToObject);
				if (FromObjects)
				{
					UE_LOGF(LogObjectCache, VeryVerbose, "Lookup mapping removed %ls -> %ls", *InFrom->GetFullName(), *ToObject->GetFullName());
					FromObjects->Remove(InFrom);
				}
			}

			FromToMapping.Remove(InFrom);
		}
	}

	void Update(ToType* InTo, const TArray<FromType*>& NewFromObjects)
	{
		LLM_SCOPE_BYTAG(ObjectReverseLookupCache);
		UE::TWriteScopeLock ScopeLock(Lock);

		// Remove the old reverse lookup mappings as they can't be read from the
		// objects anymore.
		TRawSet<FromType*>* OldFromObjects = ToFromMapping.Find(InTo);
		if (OldFromObjects)
		{
			for (FromType* OldFrom : *OldFromObjects)
			{
				TRawSet<ToType*>* ToObjects = FromToMapping.Find(OldFrom);
				if (ToObjects)
				{
					ToObjects->Remove(InTo);
				}
			}
		}

		if (NewFromObjects.Num())
		{
			TRawSet<FromType*>& FromObjects = ToFromMapping.FindOrAdd(InTo);
			FromObjects.Reset();

			for (FromType* From : NewFromObjects)
			{
				if (From != nullptr)
				{
					UE_LOGF(LogObjectCache, VeryVerbose, "Lookup mapping added %ls -> %ls", *GetFullName(From), *GetFullName(InTo));
					FromToMapping.FindOrAdd(From).FindOrAdd(InTo);
					FromObjects.FindOrAdd(From);
				}
			}
		}
		else
		{
			ToFromMapping.Remove(InTo);
		}
	}

	template <typename PredicateType = FDefaultObjectSearchPredicate>
	TObjectCacheIterator<ToType> GetFrom(FromType* InFrom, const PredicateType& Predicate = PredicateType()) const
	{
		UE::TReadScopeLock ScopeLock(Lock);

		// Until we can get some kind of thread-safe CopyOnWrite behavior
		// we need to extract a copy to avoid race conditions.
		TArray<ToType*> OutTo;
		if (const TRawSet<ToType*>* Set = FromToMapping.Find(InFrom))
		{
			OutTo.Reserve(Set->Num());
			for (ToType* To : *Set)
			{
				if (Predicate(To))
				{
					OutTo.Add(To);
				}
			}
		}

		return TObjectCacheIterator<ToType>(MoveTemp(OutTo));
	}

	template <typename PredicateType = FDefaultObjectSearchPredicate>
	TObjectCacheIterator<FromType> GetTo(ToType* InTo, const PredicateType& Predicate = PredicateType()) const
	{
		UE::TReadScopeLock ScopeLock(Lock);

		// Until we can get some kind of thread-safe CopyOnWrite behavior,
		// we need to extract a copy to avoid race conditions.
		TArray<FromType*> OutFrom;
		if (const TRawSet<FromType*>* Set = ToFromMapping.Find(InTo))
		{
			OutFrom.Reserve(Set->Num());
			for (FromType* From : *Set)
			{
				if (Predicate(From))
				{
					OutFrom.Add(From);
				}
			}
		}

		return TObjectCacheIterator<FromType>(MoveTemp(OutFrom));
	}

	template<class T>
	FString GetFullName(T* Obj)
	{
		return Obj->GetFullName();
	}

	template<>
	FString GetFullName(IStaticMeshComponent* Interface)
	{
		return Interface->GetPrimitiveComponentInterface()->GetFullName();
	}

	template<>
	FString GetFullName(ISkinnedMeshComponent* Interface)
	{
		return Interface->GetPrimitiveComponentInterface()->GetFullName();
	}

	int32 Compare(const FObjectReverseLookupCache<FromType, ToType>& Other)
	{
		UE::TWriteScopeLock ScopeLockA(Other.Lock);
		UE::TWriteScopeLock ScopeLockB(Lock);

		int32 ErrorCount = 0;
		for (const auto& Kvp : FromToMapping)
		{
			const TRawSet<ToType*>* ToObjects = Other.FromToMapping.Find(Kvp.Key);
			for (const auto& To : Kvp.Value)
			{
				bool bIsPresent = ToObjects ? ToObjects->Contains(To) : false;
				if (!bIsPresent)
				{
					UE_LOGF(LogObjectCache, Display, "Missing a direct lookup from %ls to %ls mapping", *GetFullName(Kvp.Key), *GetFullName(To));
					ErrorCount++;
				}
			}
		}

		for (const auto& Kvp : ToFromMapping)
		{
			const TRawSet<FromType*>* FromObjects = Other.ToFromMapping.Find(Kvp.Key);
			for (const auto& From : Kvp.Value)
			{
				bool bIsPresent = FromObjects ? FromObjects->Contains(From) : false;
				if (!bIsPresent)
				{
					UE_LOGF(LogObjectCache, Display, "Missing a reverse lookup from %ls to %ls mapping", *GetFullName(Kvp.Key), *GetFullName(From));
					ErrorCount++;
				}
			}
		}

		return ErrorCount;
	}
private:
	mutable FTransactionallySafeRWLock Lock;

	// We use raw pointer to compare UObject here both for performance reason
	// and because we can't use TObjectKey. Some objects are reallocated in-place
	// with a different serial number but without changing their pointer.
	// See: StaticAllocateObject for implementation detail.
	// This trick is used to avoid having to track down all referencers of a UObject*
	// just to update it, and is used for instance to replace the content of a UStaticMesh
	// during a reimport without having to update any UStaticMeshComponent.
	// Because it is the job of this class to maintain pointer to pointer lookups, we
	// can't use the serial number of any object in our map, and by extention, we can't
	// use TObjectKey nor FObjectKey to store our keys.
	// We also assume that any UObject pointer in these maps are always valid
	// because we require to be notified when those objects are destroyed.
	template<typename ElementType>
	struct TRawObjectKeyFuncs : BaseKeyFuncs<ElementType, void*, false>
	{
		typedef typename TTypeTraits<void*>::ConstPointerType KeyInitType;
		typedef typename TCallTraits<ElementType>::ParamType ElementInitType;

		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element) {	return (void*)Element; }
		static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B) { return A == B; }
		static FORCEINLINE uint32 GetKeyHash(KeyInitType Key) { return GetTypeHash(Key); }
	};

	template <typename InElementType>
	using TRawSet = TSet<InElementType, TRawObjectKeyFuncs<InElementType>>;

	template <typename InKeyType, typename InValueType>
	using TRawMap = TMap<InKeyType, InValueType, FDefaultSetAllocator, TDefaultMapKeyFuncs<void*, InValueType, false>>;

	TRawMap<FromType*, TRawSet<ToType*>> FromToMapping;
	TRawMap<ToType*, TRawSet<FromType*>> ToFromMapping;
};

FObjectReverseLookupCache<UMaterialInterface, UMaterialInstance>   GMaterialToMaterialInstanceLookupCache; // Parent -> Children
FObjectReverseLookupCache<UTexture, UMaterialInterface>            GTextureToMaterialLookupCache;
FObjectReverseLookupCache<UTextureCollection, UMaterialInterface>  GTextureCollectionToMaterialLookupCache;
FObjectReverseLookupCache<UTexture, UTexture>                      GTextureToTextureLookupCache; // CompositeTexture -> Texture
FObjectReverseLookupCache<UStaticMesh, IStaticMeshComponent>       GStaticMeshToComponentLookupCache;
FObjectReverseLookupCache<USkinnedAsset, ISkinnedMeshComponent>    GSkinnedAssetToComponentLookupCache;
FObjectReverseLookupCache<UMaterialInterface, IPrimitiveComponent> GMaterialToPrimitiveLookupCache;

void GetReferencedTextures(UMaterialInterface* MaterialInterface, TSet<UTexture*>& OutReferencedTextures);
void GetReferencedTextureCollections(UMaterialInterface* MaterialInterface, TSet<UTextureCollection*>& OutReferencedTextureCollections);

void Validate()
{
	// Scan and compare UStaticMesh -> IStaticMeshComponent
	int32 ErrorCount = 0;
	{
		FObjectReverseLookupCache<UStaticMesh, IStaticMeshComponent> TempLookup;
		for (TComponentInterfaceIterator<IStaticMeshComponent> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			if (It->GetStaticMesh())
			{
				TempLookup.Update(*It, { It->GetStaticMesh() } );
			}
		}
		ErrorCount += TempLookup.Compare(GStaticMeshToComponentLookupCache);
	}

	// Scan and compare USkinnedAsset -> ISkinnedMeshComponent
	{
		FObjectReverseLookupCache<USkinnedAsset, ISkinnedMeshComponent> TempLookup;
		for (TComponentInterfaceIterator<ISkinnedMeshComponent> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			if (It->GetSkinnedAsset())
			{
				TempLookup.Update(*It, { It->GetSkinnedAsset() } );
			}
		}
		ErrorCount += TempLookup.Compare(GSkinnedAssetToComponentLookupCache);
	}

	// Scan and compare UTexture -> UMaterialInterface
	{
		FObjectReverseLookupCache<UTexture, UMaterialInterface> TempLookup;
		for (TObjectIterator<UMaterialInterface> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			TSet<UTexture*> ReferencedTextures;
			GetReferencedTextures(*It, ReferencedTextures);
			TempLookup.Update(*It, ReferencedTextures.Array());
		}
		ErrorCount += TempLookup.Compare(GTextureToMaterialLookupCache);
	}

	// Scan and compare UTextureCollection -> UMaterialInterface
	{
		FObjectReverseLookupCache<UTextureCollection, UMaterialInterface> TempLookup;
		for (TObjectIterator<UMaterialInterface> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			TSet<UTextureCollection*> ReferencedTextureCollections;
			GetReferencedTextureCollections(*It, ReferencedTextureCollections);
			TempLookup.Update(*It, ReferencedTextureCollections.Array());
		}
		ErrorCount += TempLookup.Compare(GTextureCollectionToMaterialLookupCache);
	}

	// Scan and compare UTexture -> UTexture
	{
		FObjectReverseLookupCache<UTexture, UTexture> TempLookup;
		for (TObjectIterator<UTexture> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			UTexture* Texture = *It;
			if (Texture->GetCompositeTexture() != nullptr && Texture->GetCompositeTexture() != Texture)
			{
				TempLookup.Update(*It, { It->GetCompositeTexture() });
			}
		}
		ErrorCount += TempLookup.Compare(GTextureToTextureLookupCache);
	}

	// Scan and compare UMaterialInterface -> UPrimitiveComponent
	{
		FObjectReverseLookupCache<UMaterialInterface, IPrimitiveComponent> TempLookup;
		for (TComponentInterfaceIterator<IPrimitiveComponent> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			IPrimitiveComponent* Component = *It;
			if (Component->IsRenderStateCreated() && Component->GetSceneProxy() && !Component->IsRenderStateDirty())
			{
				TArray<UMaterialInterface*> UsedMaterials;
				Component->GetUsedMaterials(UsedMaterials);
				TempLookup.Update(Component, UsedMaterials);
			}
		}
		ErrorCount += TempLookup.Compare(GMaterialToPrimitiveLookupCache);
	}

	// Scan and compare UMaterialInterface -> UMaterialInterface
	{
		FObjectReverseLookupCache<UMaterialInterface, UMaterialInstance> TempLookup;
		for (TObjectIterator<UMaterialInstance> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			if (It->Parent)
			{
				TempLookup.Update(*It, { It->Parent });
			}
		}
		ErrorCount += TempLookup.Compare(GMaterialToMaterialInstanceLookupCache);
	}

	UE_LOGF(LogObjectCache, Display, "Object Cache verification found %d discrepenties", ErrorCount);
}

} // namespace ObjectCacheContextImpl

#endif // #if WITH_EDITOR

namespace ObjectCacheContextImpl {

	void GetReferencedTextures(UMaterialInterface* MaterialInterface, TSet<UTexture*>& OutReferencedTextures)
	{
		TSet<UTexture*> Textures;
		for (UObject* TextureObject : MaterialInterface->GetReferencedTextures())
		{
			UTexture* Texture = Cast<UTexture>(TextureObject);
			if (Texture)
			{
				OutReferencedTextures.FindOrAdd(Texture);
			}
		}

		// GetReferencedTextures() doesn't return overrides anymore and we need all referenced textures.
		UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
		while (MaterialInstance)
		{
			for (const FTextureParameterValue& TextureParam : MaterialInstance->TextureParameterValues)
			{
				if (TextureParam.ParameterValue)
				{
					OutReferencedTextures.FindOrAdd(TextureParam.ParameterValue);
				}
			}

			MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
		}
	}

	void GetReferencedTextureCollections(UMaterialInterface* MaterialInterface, TSet<UTextureCollection*>& OutReferencedTextureCollections)
	{
		TSet<UTextureCollection*> TextureCollections;
		for (UTextureCollection* TextureCollection : MaterialInterface->GetReferencedTextureCollections())
		{
			if (TextureCollection)
			{
				OutReferencedTextureCollections.FindOrAdd(TextureCollection);
			}
		}

		UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
		while (MaterialInstance)
		{
			for (const FTextureCollectionParameterValue& TextureCollectionParam : MaterialInstance->TextureCollectionParameterValues)
			{
				if (TextureCollectionParam.ParameterValue)
				{
					OutReferencedTextureCollections.FindOrAdd(TextureCollectionParam.ParameterValue);
				}
			}

			MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
		}
	}

	// This is the old method of finding dependent materials by scanning all MaterialInstance currently reachable
	// Adapted from FMaterialUpdateContext::~FMaterialUpdateContext() / MaterialShared.cpp
	void GetMaterialsAffectedByMaterials_Iteration(TArrayView<UMaterialInterface*> InMaterials, TSet<UMaterialInterface*>& OutAffectedMaterials)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetMaterialsAffectedByMaterials_Iteration);

		TSet<UMaterialInterface*> Materials;
		Materials.Reserve(InMaterials.Num());
		for (UMaterialInterface* Material : InMaterials)
		{
			OutAffectedMaterials.FindOrAdd(Material);
			Materials.FindOrAdd(Material);
		}

		TSet<UMaterialInterface*> AffectedMaterials;
		for (TObjectIterator<UMaterialInstance> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			UMaterialInstance* MaterialInstance = *It;
			UMaterial* BaseMaterial = MaterialInstance->GetMaterial();

			if (Materials.Contains(BaseMaterial))
			{
				// Check to see if this instance is dependent on any of the material interfaces we directly updated.
				for (UMaterialInterface* MaterialInterface : InMaterials)
				{
					if (MaterialInstance->IsDependent(MaterialInterface))
					{
						OutAffectedMaterials.FindOrAdd(MaterialInstance);
						break;
					}
				}
			}
		}
	}

#if WITH_EDITOR
	// New method by recursively iterating on a reverse lookup from parent -> children relationships until the graph is fully resolved
	void GetMaterialsAffectedByMaterials_Lookup(TArrayView<UMaterialInterface*> InMaterials, TSet<UMaterialInterface*>& OutAffectedMaterials)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetMaterialsAffectedByMaterials_Lookup);

		TQueue<UMaterialInterface*> Todo;
		for (UMaterialInterface* Material : InMaterials)
		{
			Todo.Enqueue(Material);
		}

		UMaterialInterface* NextItem;
		while (Todo.Dequeue(NextItem))
		{
			OutAffectedMaterials.FindOrAdd(NextItem);
			for (UMaterialInstance* Child : GMaterialToMaterialInstanceLookupCache.GetFrom(NextItem))
			{
				bool bIsAlreadyPresent = false;
				OutAffectedMaterials.FindOrAdd(Child, &bIsAlreadyPresent);
				if (!bIsAlreadyPresent)
				{
					Todo.Enqueue(Child);
				}
			}
		}
	}
#endif // #if WITH_EDITOR

	bool DoesPrimitiveDependsOnMaterials(FObjectCacheContext& Context, IPrimitiveComponent* PrimitiveComponent, TArrayView<UMaterialInterface*> InMaterials)
	{
		// Note: relying on GetUsedMaterials to be accurate, or else we won't propagate to the right primitives and the renderer will crash later
		// FPrimitiveSceneProxy::VerifyUsedMaterial is used to make sure that all materials used for rendering are reported in GetUsedMaterials
		TObjectCacheIterator<UMaterialInterface> UsedMaterials = Context.GetUsedMaterials(PrimitiveComponent);
		if (!UsedMaterials.IsEmpty())
		{
			for (UMaterialInterface* UpdatedMaterial : InMaterials)
			{
				for (UMaterialInterface* UsedMaterial : UsedMaterials)
				{
					if (UsedMaterial && (UsedMaterial == UpdatedMaterial || UsedMaterial->IsDependent(UpdatedMaterial)))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	// This is the old method of finding primitives that depends on a set of materials by scanning all currently reachable Primitives' UsedMaterials
	// Adapted from FShaderCompilingManager::PropagateMaterialChangesToPrimitives / ShaderCompiler.cpp
	void GetPrimitivesAffectedByMaterials_Iteration(FObjectCacheContext& Context, TArrayView<UMaterialInterface*> InMaterials, TSet<IPrimitiveComponent*>& OutAffectedPrimitives)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetPrimitivesAffectedByMaterials_Iteration);

		for (IPrimitiveComponent* PrimitiveComponent : Context.GetPrimitiveComponents())
		{
			if (PrimitiveComponent->IsRenderStateCreated() && PrimitiveComponent->GetSceneProxy())
			{
				if (DoesPrimitiveDependsOnMaterials(Context, PrimitiveComponent, InMaterials))
				{
					OutAffectedPrimitives.FindOrAdd(PrimitiveComponent);
				}
			}
		}
	}

#if WITH_EDITOR
	// This is the new method of finding primitives that depends on a set of materials by using reverse lookups
	void GetPrimitivesAffectedByMaterials_Lookup(TArrayView<UMaterialInterface*> InMaterials, TSet<IPrimitiveComponent*>& OutAffectedPrimitives)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetPrimitivesAffectedByMaterials_Lookup);

		TSet<UMaterialInterface*> AffectedMaterials;
		GetMaterialsAffectedByMaterials_Lookup(InMaterials, AffectedMaterials);

		for (UMaterialInterface* MaterialInterface : AffectedMaterials)
		{
			for (IPrimitiveComponent* Primitive : GMaterialToPrimitiveLookupCache.GetFrom(MaterialInterface))
			{
				OutAffectedPrimitives.FindOrAdd(Primitive);
			}
		}
	}

	void GetTexturesAffectedByTexture_Iteration(UTexture* InTexture, TSet<UTexture*>& OutAffectedTextures)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetTextureAffectedByTexture_Iteration);

		for (TObjectIterator<UTexture> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			UTexture* Texture = *It;

			if (Texture != InTexture && Texture->GetCompositeTexture() == InTexture && Texture->CompositeTextureMode != CTM_Disabled)
			{
				OutAffectedTextures.Add(Texture);
			}
		}
	}

#endif // #if WITH_EDITOR

} // namespace ObjectCacheContextImpl

TObjectCacheIterator<IPrimitiveComponent> FObjectCacheContext::GetPrimitiveComponents()
{
	using namespace ObjectCacheContextImpl;

	if (!PrimitiveComponents.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputePrimitiveComponents);

		TArray<IPrimitiveComponent*> Array;
		Array.Reserve(4096);
		for (TComponentInterfaceIterator<IPrimitiveComponent> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			Array.Add(*It);
		}
		PrimitiveComponents = MoveTemp(Array);
	}

	return TObjectCacheIterator<IPrimitiveComponent>(MakeArrayView(PrimitiveComponents.GetValue()));
}

TObjectCacheIterator<UTexture> FObjectCacheContext::GetUsedTextures(UMaterialInterface* MaterialInterface)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::TextureToMaterialLookupCache))
	{
		return GTextureToMaterialLookupCache.GetTo(MaterialInterface);
	}
#endif

	TSet<UTexture*>* Textures = MaterialUsedTextures.Find(MaterialInterface);
	if (Textures == nullptr)
	{
		Textures = &MaterialUsedTextures.Add(MaterialInterface);
		ObjectCacheContextImpl::GetReferencedTextures(MaterialInterface, *Textures);
	}

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::TextureToMaterialLookupCache))
	{
		TSet<UTexture*> LookupResult;
		for (UTexture* Texture : GTextureToMaterialLookupCache.GetTo(MaterialInterface))
		{
			LookupResult.FindOrAdd(Texture);
			checkf(Textures->Contains(Texture), TEXT("Permanent Object Cache has an additional texture %s on material %s"), *Texture->GetFullName(), *MaterialInterface->GetFullName());
		}

		for (UTexture* Texture : *Textures)
		{
			checkf(LookupResult.Contains(Texture), TEXT("Permanent Object Cache is missing a texture %s on material %s"), *Texture->GetFullName(), *MaterialInterface->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<UTexture>(Textures->Array());
}

TObjectCacheIterator<UTextureCollection> FObjectCacheContext::GetUsedTextureCollections(UMaterialInterface* MaterialInterface)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::TextureCollectionToMaterialLookupCache))
	{
		return GTextureCollectionToMaterialLookupCache.GetTo(MaterialInterface);
	}
#endif

	TSet<UTextureCollection*>* TextureCollections = MaterialUsedTextureCollections.Find(MaterialInterface);
	if (TextureCollections == nullptr)
	{
		TextureCollections = &MaterialUsedTextureCollections.Add(MaterialInterface);
		ObjectCacheContextImpl::GetReferencedTextureCollections(MaterialInterface, *TextureCollections);
	}

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::TextureCollectionToMaterialLookupCache))
	{
		TSet<UTextureCollection*> LookupResult;
		for (UTextureCollection* TextureCollection : GTextureCollectionToMaterialLookupCache.GetTo(MaterialInterface))
		{
			LookupResult.FindOrAdd(TextureCollection);
			checkf(TextureCollections->Contains(TextureCollection), TEXT("Permanent Object Cache has an additional texture collection %s on material %s"), *TextureCollection->GetFullName(), *MaterialInterface->GetFullName());
		}

		for (UTextureCollection* TextureCollection : *TextureCollections)
		{
			checkf(LookupResult.Contains(TextureCollection), TEXT("Permanent Object Cache is missing a texture collection %s on material %s"), *TextureCollection->GetFullName(), *MaterialInterface->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<UTextureCollection>(TextureCollections->Array());
}


template<typename UComponentType, typename UAssetType>
template <typename ObjectIteratorType>
TObjectCacheIterator<UComponentType> FObjectCacheContext::TComponentTypeCache<UComponentType,UAssetType>::Get()
{
	using namespace ObjectCacheContextImpl;

	if (!Components.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeSkinnedMeshComponents);

		TArray<UComponentType*> Array;
		Array.Reserve(4096);
		for (ObjectIteratorType It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			Array.Add(*It);
		}
		Components = MoveTemp(Array);
	}
	return TObjectCacheIterator<UComponentType>(MakeArrayView(Components.GetValue()));
}

template<typename UComponentType, typename UAssetType>
template <typename ObjectIteratorType, typename GetNumFuncType, typename GetFuncType>
TObjectCacheIterator<UComponentType> FObjectCacheContext::TComponentTypeCache<UComponentType,UAssetType>::Get(UAssetType *AssetType, GetNumFuncType GetNumFunc, GetFuncType GetFunc)
{
	const TSet<UComponentType*>& Result = GetSet<ObjectIteratorType>(AssetType, GetNumFunc, GetFunc);
	return TObjectCacheIterator<UComponentType>(Result.Array());
}


template<typename UComponentType, typename UAssetType>
template <typename ObjectIteratorType, typename GetNumFuncType, typename GetFuncType>
const TSet<UComponentType* >& FObjectCacheContext::TComponentTypeCache<UComponentType,UAssetType>::GetSet(UAssetType *AssetType, GetNumFuncType GetNumFunc, GetFuncType GetFunc)
{
	if (!AssetToComponents.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeSkinnedMeshToComponents);

		TMap<TObjectKey<UAssetType>, TSet<UComponentType*>> TempMap;
		TempMap.Reserve(8192);
		for (UComponentType* Component : Get<ObjectIteratorType>())
		{
			int32 Num = GetNumFunc(Component);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				TempMap.FindOrAdd(GetFunc(Component, Index)).FindOrAdd(Component);
			}
		}
		AssetToComponents = MoveTemp(TempMap);
	}

	static TSet<UComponentType*> EmptySet;
	TSet<UComponentType*>* Set = AssetToComponents.GetValue().Find(AssetType);
	const TSet<UComponentType*>& Result = Set ? *Set : EmptySet;

	return Result;
}


TObjectCacheIterator<ISkinnedMeshComponent> FObjectCacheContext::GetSkinnedMeshComponents()
{
	return SkinnedMeshCache.Get<TComponentInterfaceIterator<ISkinnedMeshComponent>>();
}

TObjectCacheIterator<ISkinnedMeshComponent> FObjectCacheContext::GetSkinnedMeshComponents(USkinnedAsset* InSkinnedAsset)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::SkinnedAssetToComponentLookupCache))
	{
		return GSkinnedAssetToComponentLookupCache.GetFrom(InSkinnedAsset);
	}
#endif

	const TSet<ISkinnedMeshComponent*>& ComputeResult = SkinnedMeshCache.GetSet<TComponentInterfaceIterator<ISkinnedMeshComponent>>(InSkinnedAsset, 
		[](ISkinnedMeshComponent*) { return 1; },
		[](ISkinnedMeshComponent* SkinnedMeshComponent, int32 Index) { return SkinnedMeshComponent->GetSkinnedAsset(); });

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::SkinnedAssetToComponentLookupCache))
	{
		TSet<ISkinnedMeshComponent*> LookupResult;
		for (ISkinnedMeshComponent* Component : GSkinnedAssetToComponentLookupCache.GetFrom(InSkinnedAsset))
		{
			LookupResult.FindOrAdd(Component);
			checkf(ComputeResult.Contains(Component), TEXT("Permanent Object Cache has an additional component %s for skinned asset %s"), *Component->GetPrimitiveComponentInterface()->GetFullName(), *InSkinnedAsset->GetFullName());
		}

		for (ISkinnedMeshComponent* Component : ComputeResult)
		{
			checkf(LookupResult.Contains(Component), TEXT("Permanent Object Cache is missing a component %p %s for skinned asset %p %s"), Component , *Component->GetPrimitiveComponentInterface()->GetFullName(), InSkinnedAsset ,*InSkinnedAsset->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<ISkinnedMeshComponent>(ComputeResult.Array());
}

TObjectCacheIterator<UInstancedSkinnedMeshComponent> FObjectCacheContext::GetInstancedSkinnedMeshComponents()
{
	return InstancedSkinnedMeshCache.Get<TObjectIterator<UInstancedSkinnedMeshComponent>>();
}

TObjectCacheIterator<UInstancedSkinnedMeshComponent> FObjectCacheContext::GetInstancedSkinnedMeshComponents(UTransformProviderData* InTransformProvider)
{
	return InstancedSkinnedMeshCache.Get<TObjectIterator<UInstancedSkinnedMeshComponent>>(InTransformProvider,
		[](UInstancedSkinnedMeshComponent* Component) { return 1; },
		[](UInstancedSkinnedMeshComponent* Component, int32) -> UTransformProviderData* {
			UTransformProviderData* Provider = Component->GetTransformProvider();
			return Provider ? Provider->GetRootTransformProvider() : nullptr;
		});
}

TObjectCacheIterator<UMaterialInterface> FObjectCacheContext::GetUsedMaterials(IPrimitiveComponent* Component)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::MaterialToPrimitiveLookupCache))
	{
		return GMaterialToPrimitiveLookupCache.GetTo(Component);
	}
#endif

	TSet<UMaterialInterface*>* Materials = PrimitiveComponentToMaterial.Find(Component);
	if (Materials == nullptr)
	{
		Materials = &PrimitiveComponentToMaterial.Add(Component);
		TArray<UMaterialInterface*> OutMaterials;
		// GetUsedMaterials can sometimes return nullptr... need to filter them out
		Component->GetUsedMaterials(OutMaterials, true);
		for (UMaterialInterface* MaterialInterface : OutMaterials)
		{
			if (MaterialInterface)
			{
				Materials->FindOrAdd(MaterialInterface);
			}
		}
	}

#if WITH_EDITOR
	// The only case where reverse lookup is allowed to diverge is when the render state is dirty meaning
	// UsedMaterials might have been modified but the SceneProxy hasn't been recreated yet.
	// This is not a problem as we're using the reverse proxy to know which component needs to be dirtied anyway.
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::MaterialToPrimitiveLookupCache) && Component->IsRenderStateCreated() && Component->GetSceneProxy() && !Component->IsRenderStateDirty())
	{
		TSet<UMaterialInterface*> LookupResult;
		for (UMaterialInterface* Material : GMaterialToPrimitiveLookupCache.GetTo(Component))
		{
			LookupResult.FindOrAdd(Material);
			checkf(Materials->Contains(Material), TEXT("Permanent Object Cache has an additional material %s for component %s"), *Material->GetFullName(), *Component->GetFullName());
		}

		for (UMaterialInterface* Material : *Materials)
		{
			checkf(LookupResult.Contains(Material), TEXT("Permanent Object Cache is missing a material %s on component %s"), *Material->GetFullName(), *Component->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<UMaterialInterface>(Materials->Array());
}

TObjectCacheIterator<IStaticMeshComponent> FObjectCacheContext::GetStaticMeshComponents()
{
	return StaticMeshCache.Get<TComponentInterfaceIterator<IStaticMeshComponent>>();
}

TObjectCacheIterator<IStaticMeshComponent> FObjectCacheContext::GetStaticMeshComponents(UStaticMesh* InStaticMesh)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::StaticMeshToComponentLookupCache))
	{
		return GStaticMeshToComponentLookupCache.GetFrom(InStaticMesh);
	}
#endif

	const TSet<IStaticMeshComponent*>& ComputeResult = StaticMeshCache.GetSet<TComponentInterfaceIterator<IStaticMeshComponent>>(InStaticMesh,
		[](IStaticMeshComponent*) { return 1; },
		[](IStaticMeshComponent* StaticMeshComponent, int32 Index) { return StaticMeshComponent->GetStaticMesh(); });

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::StaticMeshToComponentLookupCache))
	{
		TSet<IStaticMeshComponent*> LookupResult;
		for (IStaticMeshComponent* Component : GStaticMeshToComponentLookupCache.GetFrom(InStaticMesh))
		{
			LookupResult.FindOrAdd(Component);
			checkf(ComputeResult.Contains(Component), TEXT("Permanent Object Cache has an additional component %s for staticmesh %s"), *Component->GetPrimitiveComponentInterface()->GetFullName(), *InStaticMesh->GetFullName());
		}

		for (IStaticMeshComponent* Component : ComputeResult)
		{
			checkf(LookupResult.Contains(Component), TEXT("Permanent Object Cache is missing a component %p %s for staticmesh %p %s"), Component , *Component->GetPrimitiveComponentInterface()->GetFullName(), InStaticMesh ,*InStaticMesh->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<IStaticMeshComponent>(ComputeResult.Array());
}

TObjectCacheIterator<UMaterialInterface> FObjectCacheContext::GetMaterialsAffectedByTexture(UTexture* InTexture)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::TextureToMaterialLookupCache))
	{
		return GTextureToMaterialLookupCache.GetFrom(InTexture);
	}
#endif

	if (!TextureToMaterials.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeMaterialsAffectedByTexture);

		TMap<TObjectKey<UTexture>, TSet<UMaterialInterface*>> TempMap;
		TempMap.Reserve(8192);
		for (TObjectIterator<UMaterialInterface> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			UMaterialInterface* MaterialInterface = *It;
			for (UTexture* Texture : GetUsedTextures(MaterialInterface))
			{
				TempMap.FindOrAdd(Texture).FindOrAdd(MaterialInterface);
			}
		}
		TextureToMaterials = MoveTemp(TempMap);
	}

	static TSet<UMaterialInterface*> EmptySet;
	TSet<UMaterialInterface*>* Set = TextureToMaterials.GetValue().Find(InTexture);
	const TSet<UMaterialInterface*>& ComputeResult = Set ? *Set : EmptySet;

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::TextureToMaterialLookupCache))
	{
		TSet<UMaterialInterface*> LookupResult;
		for (UMaterialInterface* Material : GTextureToMaterialLookupCache.GetFrom(InTexture))
		{
			LookupResult.FindOrAdd(Material);
			checkf(ComputeResult.Contains(Material), TEXT("Permanent Object Cache has an additional material %s for texture %s"), *Material->GetFullName(), *InTexture->GetFullName());
		}

		for (UMaterialInterface* Material : ComputeResult)
		{
			checkf(LookupResult.Contains(Material), TEXT("Permanent Object Cache is missing a material %s for texture %s"), *Material->GetFullName(), *InTexture->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<UMaterialInterface>(ComputeResult.Array());
}

TObjectCacheIterator<UMaterialInterface> FObjectCacheContext::GetMaterialsAffectedByTextureCollection(UTextureCollection* InTextureCollection)
{
	using namespace ObjectCacheContextImpl;
#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::TextureCollectionToMaterialLookupCache))
	{
		return GTextureCollectionToMaterialLookupCache.GetFrom(InTextureCollection);
	}
#endif

	if (!TextureCollectionToMaterials.IsSet())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeMaterialsAffectedByTexture);

		TMap<TObjectKey<UTextureCollection>, TSet<UMaterialInterface*>> TempMap;
		TempMap.Reserve(8192);
		for (TObjectIterator<UMaterialInterface> It(RF_ClassDefaultObject, true /*bIncludeDerivedClasses*/, GetObjectCacheInternalFlagsExclusion()); It; ++It)
		{
			UMaterialInterface* MaterialInterface = *It;
			for (UTextureCollection* TextureCollection : GetUsedTextureCollections(MaterialInterface))
			{
				TempMap.FindOrAdd(TextureCollection).FindOrAdd(MaterialInterface);
			}
		}
		TextureCollectionToMaterials = MoveTemp(TempMap);
	}

	static TSet<UMaterialInterface*> EmptySet;
	TSet<UMaterialInterface*>* Set = TextureCollectionToMaterials.GetValue().Find(InTextureCollection);
	const TSet<UMaterialInterface*>& ComputeResult = Set ? *Set : EmptySet;

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::TextureCollectionToMaterialLookupCache))
	{
		TSet<UMaterialInterface*> LookupResult;
		for (UMaterialInterface* Material : GTextureCollectionToMaterialLookupCache.GetFrom(InTextureCollection))
		{
			LookupResult.FindOrAdd(Material);
			checkf(ComputeResult.Contains(Material), TEXT("Permanent Object Cache has an additional material %s for texture %s"), *Material->GetFullName(), *InTextureCollection->GetFullName());
		}

		for (UMaterialInterface* Material : ComputeResult)
		{
			checkf(LookupResult.Contains(Material), TEXT("Permanent Object Cache is missing a material %s for texture %s"), *Material->GetFullName(), *InTextureCollection->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<UMaterialInterface>(ComputeResult.Array());
}

#if WITH_EDITOR

TObjectCacheIterator<UTexture> FObjectCacheContext::GetTexturesAffectedByTexture(UTexture* InTexture)
{
	using namespace ObjectCacheContextImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FObjectCacheContext::GetTexturesAffectedByTexture);

	// Filter out if CompositeTextureMode is not active
	auto Predicate = [](UTexture* Texture) { return Texture->CompositeTextureMode != CTM_Disabled; };

	if (ShouldUseLookupTable(EObjectReverseLookupMask::TextureToTextureLookupCache))
	{
		return GTextureToTextureLookupCache.GetFrom(InTexture, Predicate);
	}

	TSet<UTexture*> IterationResults;
	ObjectCacheContextImpl::GetTexturesAffectedByTexture_Iteration(InTexture, IterationResults);

	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::TextureToTextureLookupCache))
	{
		TSet<UTexture*> LookupResults;
		for (UTexture* Texture : GTextureToTextureLookupCache.GetFrom(InTexture, Predicate))
		{
			LookupResults.Add(Texture);
			checkf(IterationResults.Contains(Texture), TEXT("Permanent Object Cache has an additional element %s"), *Texture->GetFullName());
		}

		for (UTexture* Texture : IterationResults)
		{
			checkf(LookupResults.Contains(Texture), TEXT("Permanent Object Cache is missing element %s"), *Texture->GetFullName());
		}
	}

	return TObjectCacheIterator<UTexture>(IterationResults.Array());
}

#endif

TObjectCacheIterator<UMaterialInterface> FObjectCacheContext::GetMaterialsAffectedByMaterials(TArrayView<UMaterialInterface*> InMaterials)
{
	using namespace ObjectCacheContextImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FObjectCacheContext::GetMaterialsAffectedByMaterials);

#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::MaterialToMaterialsLookupCache))
	{
		TSet<UMaterialInterface*> LookupResults;
		ObjectCacheContextImpl::GetMaterialsAffectedByMaterials_Lookup(InMaterials, LookupResults);
		return TObjectCacheIterator<UMaterialInterface>(LookupResults.Array());
	}
#endif

	TSet<UMaterialInterface*> IterationResults;
	ObjectCacheContextImpl::GetMaterialsAffectedByMaterials_Iteration(InMaterials, IterationResults);

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::MaterialToMaterialsLookupCache))
	{
		TSet<UMaterialInterface*> LookupResults;
		ObjectCacheContextImpl::GetMaterialsAffectedByMaterials_Lookup(InMaterials, LookupResults);

		for (UMaterialInterface* MaterialInterface : LookupResults)
		{
			checkf(IterationResults.Contains(MaterialInterface), TEXT("Permanent Object Cache has an additional element %s"), *MaterialInterface->GetFullName());
		}

		for (UMaterialInterface* MaterialInterface : IterationResults)
		{
			checkf(LookupResults.Contains(MaterialInterface), TEXT("Permanent Object Cache is missing element %s"), *MaterialInterface->GetFullName());
		}
	}
#endif

	return TObjectCacheIterator<UMaterialInterface>(IterationResults.Array());
}

TObjectCacheIterator<IPrimitiveComponent> FObjectCacheContext::GetPrimitivesAffectedByMaterials(TArrayView<UMaterialInterface*> InMaterials)
{
	using namespace ObjectCacheContextImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FObjectCacheContext::GetPrimitivesAffectedByMaterials);

#if WITH_EDITOR
	if (ShouldUseLookupTable(EObjectReverseLookupMask::MaterialToPrimitiveLookupCache))
	{
		TSet<IPrimitiveComponent*> AffectedPrimitives;
		ObjectCacheContextImpl::GetPrimitivesAffectedByMaterials_Lookup(InMaterials, AffectedPrimitives);
		return TObjectCacheIterator<IPrimitiveComponent>(AffectedPrimitives.Array());
	}
#endif

	TSet<IPrimitiveComponent*> IterationResults;
	ObjectCacheContextImpl::GetPrimitivesAffectedByMaterials_Iteration(*this, InMaterials, IterationResults);

#if WITH_EDITOR
	if (IsReverseLookupComparisonActive(EObjectReverseLookupMask::MaterialToPrimitiveLookupCache))
	{
		TSet<IPrimitiveComponent*> LookupResults;
		ObjectCacheContextImpl::GetPrimitivesAffectedByMaterials_Lookup(InMaterials, LookupResults);

		// The only case where reverse lookup is allowed to diverge is when the render state is dirty meaning
		// UsedMaterials might have been modified but the SceneProxy hasn't been recreated yet.
		// This is not a problem as we're using the reverse proxy to know which component needs to be dirtied anyway.
		for (IPrimitiveComponent* Component : LookupResults)
		{
			if (Component->IsRenderStateCreated() && Component->GetSceneProxy() && !Component->IsRenderStateDirty())
			{
				checkf(IterationResults.Contains(Component), TEXT("Permanent Object Cache has an additional element %s"), *Component->GetFullName());
			}
		}

		for (IPrimitiveComponent* Component : IterationResults)
		{
			if (Component->IsRenderStateCreated() && Component->GetSceneProxy() && !Component->IsRenderStateDirty())
			{
				checkf(LookupResults.Contains(Component), TEXT("Permanent Object Cache is missing element %s"), *Component->GetFullName());
			}
		}
	}
#endif

	return TObjectCacheIterator<IPrimitiveComponent>(IterationResults.Array());
}

TObjectCacheIterator<IPrimitiveComponent> FObjectCacheContext::GetPrimitivesAffectedByMaterial(UMaterialInterface* InMaterialInterface)
{
	UMaterialInterface* InplaceArray[1] = { InMaterialInterface };
	return GetPrimitivesAffectedByMaterials(InplaceArray);
}

namespace ObjectCacheContextScopeImpl
{
	static thread_local TUniquePtr<FObjectCacheContext> Current = nullptr;
}

FObjectCacheContextScope::FObjectCacheContextScope()
{
	using namespace ObjectCacheContextScopeImpl;
	if (Current == nullptr)
	{
		Current.Reset(new FObjectCacheContext());
		bIsOwner = true;
	}
}

FObjectCacheContextScope::~FObjectCacheContextScope()
{
	using namespace ObjectCacheContextScopeImpl;
	if (bIsOwner)
	{
		Current.Reset();
	}
}

FObjectCacheContext& FObjectCacheContextScope::GetContext()
{
	using namespace ObjectCacheContextScopeImpl;
	return *Current;
}

#if WITH_EDITOR

namespace ObjectCacheContextImpl {

struct FObjectCacheEventSinkPrivate
{
	static void BeginQueueNotifyEvents();
	static void ProcessQueuedNotifyEvents();
	static void EndQueueNotifyEvents();

	static void NotifyUsedMaterialsChanged_Concurrent(const IPrimitiveComponent* PrimitiveComponent, const TArray<UMaterialInterface*>& UsedMaterials);
	static void NotifyRenderStateChanged_Concurrent(const IPrimitiveComponent*);
	static void NotifyMaterialChanged_Concurrent(UMaterialInterface*);
	static void NotifyStaticMeshChanged_Concurrent(IStaticMeshComponent*);
	static void NotifySkinnedMeshChanged_Concurrent(ISkinnedMeshComponent*);
	static void NotifyMaterialDestroyed_Concurrent(UMaterialInterface*);
	static void NotifyCompositeTextureChanged_Concurrent(UTexture*);

	enum ECacheEventType
	{
		EMaterialDestroyed,
		EMaterialsChanged,
		ERenderStateChanged,
		EMaterialChanged,
		EStaticMeshChanged,
		ECompositeTextureChanged,
		ESkinnedAssetChanged,
	};

	struct FNotifyEvent
	{
		ECacheEventType EventType;
		UMaterialInterface* MaterialInterface;
		const IPrimitiveComponent* PrimitiveComponent;
		IStaticMeshComponent* StaticMeshComponent;
		ISkinnedMeshComponent* SkinnedMeshComponent;
		TArray<UMaterialInterface*> UsedMaterials;
		UTexture* Texture;
		UTextureCollection* TextureCollection;
	};

	static std::atomic<bool> bShouldQueueSinkEvents;

	static bool ShouldQueueSinkEvents()
	{
		// Loading thread events can't be queued as it could cause race condition with async loading when the queue is processed
		return bShouldQueueSinkEvents.load(std::memory_order_relaxed) && !IsInAsyncLoadingThread();
	}

	using TEventAllocator = TLockFreeClassAllocator<FNotifyEvent, PLATFORM_CACHE_LINE_SIZE>;
	using TEventList = TLockFreePointerListFIFO<FNotifyEvent, PLATFORM_CACHE_LINE_SIZE>;
	static TEventAllocator& GetAllocator();
	static TEventList& GetNotifyEvents();
	static void AddNotifyEvent(FObjectCacheEventSinkPrivate::ECacheEventType EventType, UMaterialInterface* MaterialInterface, const IPrimitiveComponent* PrimitiveComponent, IStaticMeshComponent* StaticMeshComponent, ISkinnedMeshComponent* SkinnedMeshComponent, const TArray<UMaterialInterface*>* UsedMaterials, UTexture* Texture);
};

} // namespace ObjectCacheContextImpl

void FObjectCacheEventSink::BeginQueueNotifyEvents()
{
	using namespace ObjectCacheContextImpl;

	FObjectCacheEventSinkPrivate::BeginQueueNotifyEvents();
}

void FObjectCacheEventSink::ProcessQueuedNotifyEvents()
{
	using namespace ObjectCacheContextImpl;

	FObjectCacheEventSinkPrivate::ProcessQueuedNotifyEvents();
}

void FObjectCacheEventSink::EndQueueNotifyEvents()
{
	using namespace ObjectCacheContextImpl;

	FObjectCacheEventSinkPrivate::EndQueueNotifyEvents();
}

void FObjectCacheEventSink::NotifyCompositeTextureChanged_Concurrent(UTexture* InTexture)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::ECompositeTextureChanged, nullptr, nullptr, nullptr, nullptr, nullptr, InTexture);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifyCompositeTextureChanged_Concurrent(InTexture);
		}
	}
}
void FObjectCacheEventSink::NotifyMaterialDestroyed_Concurrent(UMaterialInterface* MaterialInterface)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{

		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::EMaterialDestroyed, MaterialInterface, nullptr, nullptr, nullptr, nullptr, nullptr);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifyMaterialDestroyed_Concurrent(MaterialInterface);
		}
	}
}

void FObjectCacheEventSink::NotifyUsedMaterialsChanged_Concurrent(const IPrimitiveComponent* PrimitiveComponent, const TArray<UMaterialInterface*>& UsedMaterials)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::EMaterialsChanged, nullptr, PrimitiveComponent, nullptr, nullptr, &UsedMaterials, nullptr);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifyUsedMaterialsChanged_Concurrent(PrimitiveComponent, UsedMaterials);
		}
	}
}

void FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(UActorComponent* ActorComponent)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(ActorComponent))
		{
			if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
			{
				FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::ERenderStateChanged, nullptr, Primitive->GetPrimitiveComponentInterface(), nullptr, nullptr, nullptr, nullptr);
			}
			else
			{
				FObjectCacheEventSinkPrivate::NotifyRenderStateChanged_Concurrent(Primitive->GetPrimitiveComponentInterface());
			}
		}
	}
}

void FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(IPrimitiveComponent* PrimitiveComponent)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::ERenderStateChanged, nullptr, PrimitiveComponent, nullptr, nullptr, nullptr, nullptr);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifyRenderStateChanged_Concurrent(PrimitiveComponent);
		}
	}
}


void FObjectCacheEventSink::NotifyMaterialChanged_Concurrent(UMaterialInterface* MaterialInterface)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::EMaterialChanged, MaterialInterface, nullptr, nullptr, nullptr, nullptr, nullptr);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifyMaterialChanged_Concurrent(MaterialInterface);
		}
	}
}

void FObjectCacheEventSink::NotifyStaticMeshChanged_Concurrent(IStaticMeshComponent* StaticMeshComponent)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::EStaticMeshChanged, nullptr, nullptr, StaticMeshComponent, nullptr, nullptr, nullptr);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifyStaticMeshChanged_Concurrent(StaticMeshComponent);
		}
	}
}

void FObjectCacheEventSink::NotifySkinnedMeshChanged_Concurrent(ISkinnedMeshComponent* SkinnedMeshComponent)
{
	using namespace ObjectCacheContextImpl;

	if (GetObjectReverseLookupMode() != EObjectReverseLookupMode::Temporary)
	{
		if (FObjectCacheEventSinkPrivate::ShouldQueueSinkEvents())
		{
			FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::ESkinnedAssetChanged, nullptr, nullptr, nullptr, SkinnedMeshComponent, nullptr, nullptr);
		}
		else
		{
			FObjectCacheEventSinkPrivate::NotifySkinnedMeshChanged_Concurrent(SkinnedMeshComponent);
		}
	}
}

namespace ObjectCacheContextImpl {

std::atomic<bool> FObjectCacheEventSinkPrivate::bShouldQueueSinkEvents(false);

FObjectCacheEventSinkPrivate::TEventAllocator& FObjectCacheEventSinkPrivate::GetAllocator()
{
	static FObjectCacheEventSinkPrivate::TEventAllocator TheAllocator;
	return TheAllocator;
}

FObjectCacheEventSinkPrivate::TEventList& FObjectCacheEventSinkPrivate::GetNotifyEvents()
{
	static TEventList NotifyEvents;
	return NotifyEvents;
}

void FObjectCacheEventSinkPrivate::AddNotifyEvent(FObjectCacheEventSinkPrivate::ECacheEventType EventType, UMaterialInterface* MaterialInterface, const IPrimitiveComponent* PrimitiveComponent, IStaticMeshComponent* StaticMeshComponent, ISkinnedMeshComponent* SkinnedMeshComponent, const TArray<UMaterialInterface*>* UsedMaterials, UTexture* Texture)
{

	FNotifyEvent* Event = GetAllocator().New();
	Event->EventType = EventType;
	Event->MaterialInterface = MaterialInterface;
	Event->PrimitiveComponent = PrimitiveComponent;
	Event->StaticMeshComponent = StaticMeshComponent;
	Event->SkinnedMeshComponent = SkinnedMeshComponent;
	Event->Texture = Texture;

	if (EventType == EMaterialsChanged)
	{
		check(UsedMaterials);
		Event->UsedMaterials = *UsedMaterials;
	}

	GetNotifyEvents().Push(Event);
}

void FObjectCacheEventSinkPrivate::BeginQueueNotifyEvents()
{
	check(!bShouldQueueSinkEvents);
	bShouldQueueSinkEvents = true;
}

void FObjectCacheEventSinkPrivate::ProcessQueuedNotifyEvents()
{
	check(bShouldQueueSinkEvents);

	TEventList& Events = GetNotifyEvents();
	TEventAllocator& Allocator = GetAllocator();

	while (FNotifyEvent* Event = Events.Pop())
	{
		switch (Event->EventType)
		{
		case EMaterialDestroyed:
			NotifyMaterialDestroyed_Concurrent(Event->MaterialInterface);
			break;
		case EMaterialsChanged:
			NotifyUsedMaterialsChanged_Concurrent(Event->PrimitiveComponent, Event->UsedMaterials);
			break;
		case ERenderStateChanged:
			NotifyRenderStateChanged_Concurrent(Event->PrimitiveComponent);
			break;
		case EMaterialChanged:
			NotifyMaterialChanged_Concurrent(Event->MaterialInterface);
			break;
		case EStaticMeshChanged:
			NotifyStaticMeshChanged_Concurrent(Event->StaticMeshComponent);
			break;
		case ECompositeTextureChanged:
			NotifyCompositeTextureChanged_Concurrent(Event->Texture);
			break;
		case ESkinnedAssetChanged:
			NotifySkinnedMeshChanged_Concurrent(Event->SkinnedMeshComponent);
			break;
		}

		Allocator.Free(Event);
	}
}

void FObjectCacheEventSinkPrivate::EndQueueNotifyEvents()
{
	check(bShouldQueueSinkEvents);
	ProcessQueuedNotifyEvents();
	bShouldQueueSinkEvents = false;
}


void FObjectCacheEventSinkPrivate::NotifyCompositeTextureChanged_Concurrent(UTexture* Texture)
{
	using namespace ObjectCacheContextImpl;

#if WITH_EDITOR
	// Stop tracking textures that are being destroyed
	if (Texture->HasAnyFlags(RF_BeginDestroyed) || Texture->GetCompositeTexture() == nullptr || Texture->GetCompositeTexture() == Texture)
	{
		GTextureToTextureLookupCache.Update(Texture, {});
	}
	else
	{
		GTextureToTextureLookupCache.Update(Texture, { Texture->GetCompositeTexture() });
	}
#endif
}

void FObjectCacheEventSinkPrivate::NotifyMaterialDestroyed_Concurrent(UMaterialInterface* MaterialInterface)
{
	using namespace ObjectCacheContextImpl;
	check(MaterialInterface);

	// If a material is destroyed, remove it from the cache so we don't return it anymore
	// This can happen if a component doesn't update its render state after modifying its used materials
	// (i.e. LandscapeComponent won't update it's render state when modifying MobileMaterialInterfaces during cook since they aren't meaningful for rendering)
	// The net result is that the SceneProxy->UsedMaterialsForVerification will continue to hold an invalid material interface object but if it's not used for rendering, no harm will be done...
	GMaterialToPrimitiveLookupCache.RemoveFrom(MaterialInterface);

	// Remove any Material to Texture that might be present in the cache for this Material
	GTextureToMaterialLookupCache.Update(MaterialInterface, {});

	// Remove any Material to Texture that might be present in the cache for this Material
	GTextureCollectionToMaterialLookupCache.Update(MaterialInterface, {});

	// Cleanup up any mapping that this material could have with other materials
	if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface))
	{
		GMaterialToMaterialInstanceLookupCache.Update(MaterialInstance, {});
	}
}

void FObjectCacheEventSinkPrivate::NotifyUsedMaterialsChanged_Concurrent(const IPrimitiveComponent* PrimitiveComponent, const TArray<UMaterialInterface*>& UsedMaterials)
{
	check(PrimitiveComponent);
	using namespace ObjectCacheContextImpl;

	GMaterialToPrimitiveLookupCache.Update(const_cast<IPrimitiveComponent*>(PrimitiveComponent), UsedMaterials);
}

void FObjectCacheEventSinkPrivate::NotifyRenderStateChanged_Concurrent(const IPrimitiveComponent* PrimitiveComponent)
{
	check(PrimitiveComponent);
	using namespace ObjectCacheContextImpl;

	// Clear mappings whenever the render state changes until NotifyUsedMaterialsChanged is called during SceneProxy creation
	GMaterialToPrimitiveLookupCache.Update(const_cast<IPrimitiveComponent*>(PrimitiveComponent), {});
}

void FObjectCacheEventSinkPrivate::NotifyMaterialChanged_Concurrent(UMaterialInterface* MaterialInterface)
{
	using namespace ObjectCacheContextImpl;

	if (MaterialInterface->HasAnyFlags(RF_BeginDestroyed))
	{
		NotifyMaterialDestroyed_Concurrent(MaterialInterface);
	}
	else
	{
		{
			TSet<UTexture*> Textures;
			ObjectCacheContextImpl::GetReferencedTextures(MaterialInterface, Textures);
			GTextureToMaterialLookupCache.Update(MaterialInterface, Textures.Array());
		}

		{
			TSet<UTextureCollection*> TextureCollections;
			ObjectCacheContextImpl::GetReferencedTextureCollections(MaterialInterface, TextureCollections);
			GTextureCollectionToMaterialLookupCache.Update(MaterialInterface, TextureCollections.Array());
		}

		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface))
		{
			if (MaterialInstance->Parent)
			{
				GMaterialToMaterialInstanceLookupCache.Update(MaterialInstance, { MaterialInstance->Parent });
			}
			else
			{
				GMaterialToMaterialInstanceLookupCache.Update(MaterialInstance, { });
			}
		}
	}
}

void FObjectCacheEventSinkPrivate::NotifyStaticMeshChanged_Concurrent(IStaticMeshComponent* StaticMeshComponent)
{
	check(StaticMeshComponent);
	using namespace ObjectCacheContextImpl;

	// Stop tracking components that are being destroyed
	if (StaticMeshComponent->GetPrimitiveComponentInterface()->GetUObject() && (StaticMeshComponent->GetPrimitiveComponentInterface()->GetUObject()->HasAnyFlags(RF_BeginDestroyed) || StaticMeshComponent->GetStaticMesh() == nullptr))
	{
		GStaticMeshToComponentLookupCache.Update(StaticMeshComponent, {});
	}
	else
	{
		GStaticMeshToComponentLookupCache.Update(StaticMeshComponent, { StaticMeshComponent->GetStaticMesh() });
	}
}

void FObjectCacheEventSinkPrivate::NotifySkinnedMeshChanged_Concurrent(ISkinnedMeshComponent* SkinnedMeshComponent)
{
	check(SkinnedMeshComponent);
	using namespace ObjectCacheContextImpl;

	// Stop tracking components that are being destroyed
	if (SkinnedMeshComponent->GetPrimitiveComponentInterface()->GetUObject() && (SkinnedMeshComponent->GetPrimitiveComponentInterface()->GetUObject()->HasAnyFlags(RF_BeginDestroyed) || SkinnedMeshComponent->GetSkinnedAsset() == nullptr))
	{
		GSkinnedAssetToComponentLookupCache.Update(SkinnedMeshComponent, {});
	}
	else
	{
		GSkinnedAssetToComponentLookupCache.Update(SkinnedMeshComponent, { SkinnedMeshComponent->GetSkinnedAsset() });
	}
}

} // namespace ObjectCacheContextImpl

#endif

#undef LOCTEXT_NAMESPACE