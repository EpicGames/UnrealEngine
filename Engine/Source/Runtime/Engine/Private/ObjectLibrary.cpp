// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ObjectLibrary.h"
#include "Blueprint/BlueprintSupport.h"
#include "Modules/ModuleManager.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Misc/PackageName.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StreamableManager.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectLibrary)

UObjectLibrary::UObjectLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsFullyLoaded = false;
	bUseWeakReferences = false;
	bIncludeOnlyOnDiskAssets = true;
	bRecursivePaths = true;

#if WITH_EDITOR
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		bIsGlobalAsyncScanEnvironment = AssetRegistry.IsSearchAllAssets();

		if ( bIsGlobalAsyncScanEnvironment )
		{
			// Listen for when the asset registry has finished discovering files
			AssetRegistry.OnFilesLoaded().AddUObject(this, &UObjectLibrary::OnAssetRegistryFilesLoaded);
		}
	}
#endif
}

#if WITH_EDITOR
void UObjectLibrary::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// If a base class is set, null out any references to Objects not of that class.
	if(ObjectBaseClass != NULL)
	{
		for(int32 i=0; i<Objects.Num(); i++)
		{
			if (bHasBlueprintClasses)
			{			
				if(Objects[i] != NULL)
				{
					UClass* BlueprintClass = Cast<UClass>(Objects[i]);
					if (!BlueprintClass)
					{
						UBlueprintCore* Blueprint = Cast<UBlueprintCore>(Objects[i]);
						BlueprintClass = Blueprint ? *Blueprint->GeneratedClass : nullptr;
						// replace BP with BPGC
						Objects[i] = BlueprintClass;
					}

					if (!BlueprintClass)
					{
						// Only blueprints
						Objects[i] = NULL;
					} 
					else if (!BlueprintClass->IsChildOf(ObjectBaseClass))
					{
						// Wrong base class
						Objects[i] = NULL;
					}
				}
			}
			else
			{
				if(Objects[i] != NULL && !Objects[i]->IsA(ObjectBaseClass))
				{
					Objects[i] = NULL;
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UObjectLibrary::PostLoad()
{
	Super::PostLoad();

	if (bHasBlueprintClasses)
	{
		// replace BP with BPGC
		for (int32 i = 0; i < Objects.Num(); i++)
		{
			UBlueprintCore* Blueprint = Cast<UBlueprintCore>(Objects[i]);
			if (Blueprint)
			{
				UClass* BlueprintClass = Blueprint->GeneratedClass;
				Objects[i] = (BlueprintClass && BlueprintClass->IsChildOf(ObjectBaseClass))
					? BlueprintClass
					: nullptr;
			}
		}
	}
}
#endif // WITH_EDITOR

class UObjectLibrary* UObjectLibrary::CreateLibrary(UClass* InBaseClass, bool bInHasBlueprintClasses, bool bInUseWeak)
{
	UObjectLibrary *NewLibrary = NewObject<UObjectLibrary>();

	NewLibrary->ObjectBaseClass = InBaseClass;
	NewLibrary->bHasBlueprintClasses = bInHasBlueprintClasses;
	NewLibrary->UseWeakReferences(bInUseWeak);

	return NewLibrary;
}

void UObjectLibrary::UseWeakReferences(bool bSetUseWeak)
{
	if (bSetUseWeak == bUseWeakReferences)
	{
		return;
	}

	bUseWeakReferences = bSetUseWeak;

	if (bUseWeakReferences)
	{
		// Convert existing strong references
		for(int32 i=0; i<Objects.Num(); i++)
		{
			if(Objects[i] != NULL)
			{
				WeakObjects.AddUnique(Objects[i]);
			}
		}
		Objects.Empty();
	}
	else
	{
		// Convert existing weak references
		for(int32 i=0; i<WeakObjects.Num(); i++)
		{
			if(WeakObjects[i].Get() != NULL)
			{
				Objects.AddUnique(WeakObjects[i].Get());
			}
		}
		WeakObjects.Empty();
	}
}

bool UObjectLibrary::AddObject(UObject *NewObject)
{
	if (NewObject == NULL)
	{
		return false;
	}

	if (ObjectBaseClass != NULL)
	{
		if (bHasBlueprintClasses)
		{
			UClass* BlueprintClass = Cast<UClass>(NewObject);

			if (!BlueprintClass)
			{
				// Only blueprints
				return false;
			} 
			else if (!BlueprintClass->IsChildOf(ObjectBaseClass))
			{
				// Wrong base class
				return false;
			}
		}
		else
		{
			if (!NewObject->IsA(ObjectBaseClass))
			{
				return false;
			}
		}
	}

	if (bUseWeakReferences)
	{
		if (WeakObjects.Contains(NewObject))
		{
			return false;
		}

		if (WeakObjects.Add(NewObject) != INDEX_NONE)
		{
			Modify();
			OnObjectAddedEvent.Broadcast(NewObject);
			return true;
		}
	}
	else
	{
		if (Objects.Contains(NewObject))
		{
			return false;
		}

		if (Objects.Add(NewObject) != INDEX_NONE)
		{
			Modify();
			OnObjectAddedEvent.Broadcast(NewObject);
			return true;
		}
	}
	
	return false;
}

bool UObjectLibrary::RemoveObject(UObject *ObjectToRemove)
{
	if (!bUseWeakReferences)
	{
		if (Objects.Remove(ObjectToRemove) != 0)
		{
			Modify();
			OnObjectRemovedEvent.Broadcast(ObjectToRemove);
			return true;
		}
	}
	else
	{
		if (WeakObjects.Remove(ObjectToRemove) != 0)
		{
			Modify();
			OnObjectRemovedEvent.Broadcast(ObjectToRemove);
			return true;
		}
	}

	return false;
}

int32 UObjectLibrary::LoadAssetsFromPaths(const TArray<FString>& Paths)
{
	int32 Count = 0;

	if (bIsFullyLoaded)
	{
		// We already ran this
		return 0; 
	}

	bIsFullyLoaded = true;
	
	for (int PathIndex = 0; PathIndex < Paths.Num(); PathIndex++)
	{
		TArray<UObject*> LoadedObjects;
		FString Path = Paths[PathIndex];
		if (EngineUtils::FindOrLoadAssetsByPath(Path, LoadedObjects, bHasBlueprintClasses ? EngineUtils::ATL_Class : EngineUtils::ATL_Regular))
		{
			for (int32 i = 0; i < LoadedObjects.Num(); ++i)
			{
				UObject* Object = LoadedObjects[i];

				if (Object == NULL || (ObjectBaseClass && !Object->IsA(ObjectBaseClass)))
				{
					// Incorrect type, skip
					continue;
				}
		
				AddObject(Object);
				Count++;
			}
		}
	}
	return Count;
}

int32 UObjectLibrary::LoadBlueprintsFromPaths(const TArray<FString>& Paths)
{
	int32 Count = 0;

	if (!bHasBlueprintClasses)
	{
		return 0;
	}

	if (bIsFullyLoaded)
	{
		// We already ran this
		return 0; 
	}

	bIsFullyLoaded = true;

	for (int PathIndex = 0; PathIndex < Paths.Num(); PathIndex++)
	{
		TArray<UObject*> LoadedObjects;
		FString Path = Paths[PathIndex];

		if (EngineUtils::FindOrLoadAssetsByPath(Path, LoadedObjects, EngineUtils::ATL_Class))
		{
			for (int32 i = 0; i < LoadedObjects.Num(); ++i)
			{
				auto Class = Cast<UBlueprintGeneratedClass>(LoadedObjects[i]);

				if (Class == NULL || (ObjectBaseClass && !Class->IsChildOf(ObjectBaseClass)))
				{
					continue;
				}

				AddObject(Class);
				Count++;
			}
		}
	}
	return Count;
}

int32 UObjectLibrary::LoadAssetDataFromPaths(const TArray<FString>& Paths, bool bForceSynchronousScan)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

#if WITH_EDITOR
	// Cooked data has the asset data already set up
	const bool bShouldDoSynchronousScan = !bIsGlobalAsyncScanEnvironment || bForceSynchronousScan;
	if ( bShouldDoSynchronousScan )
	{
		AssetRegistry.ScanPathsSynchronous(Paths);
	}
	else
	{
		if ( AssetRegistry.IsLoadingAssets() )
		{
			// Keep track of the paths we asked for so once assets are discovered we will refresh the list
			for ( const FString& Path : Paths )
			{
				DeferredAssetDataPaths.AddUnique(Path);
			}
		}
	}
#endif

	FARFilter ARFilter;
	if ( ObjectBaseClass )
	{
		ARFilter.ClassPaths.Add(ObjectBaseClass->GetClassPathName());

#if WITH_EDITOR
		// Add any old names to the list in case things haven't been resaved
		// @todo make FLinkerLoad support long class names
		TArray<FString> OldNames = FLinkerLoad::FindPreviousPathNamesForClass(ObjectBaseClass->GetPathName(), false);
		for (const FString& OldName : OldNames)
		{
			FTopLevelAssetPath OldPathName(OldName);
			check(!OldPathName.IsNull() || OldName.IsEmpty());
			ARFilter.ClassPaths.Add(OldPathName);
		}
#endif

		ARFilter.bRecursiveClasses = true;
	}

	for (int PathIndex = 0; PathIndex < Paths.Num(); PathIndex++)
	{
		ARFilter.PackagePaths.Add(FName(*Paths[PathIndex]));
	}

	ARFilter.bRecursivePaths = bRecursivePaths;
	ARFilter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;

	AssetDataList.Empty();
	AssetRegistry.GetAssets(ARFilter, AssetDataList);

	return AssetDataList.Num();
}

int32 UObjectLibrary::LoadBlueprintAssetDataFromPaths(const TArray<FString>& Paths, bool bForceSynchronousScan)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	if (!bHasBlueprintClasses)
	{
		return 0;
	}

#if WITH_EDITOR
	{
		// Cooked data has the asset data already set up
		const bool bShouldDoSynchronousScan = !bIsGlobalAsyncScanEnvironment || bForceSynchronousScan;
		if ( bShouldDoSynchronousScan )
		{
			// The calls into AssetRegistery require /Game/ instead of /Game.
			// The calls further below, when setting up the ARFilters, do not want the trailing /.
			// (note: this is only an annoying edge case with /Game. Subfolders will work in both cases without the trailing /".
			TArray<FString> LongFileNamePaths = Paths;
			for (FString& Str : LongFileNamePaths)
			{
				if (Str.EndsWith(TEXT("/")) == false)
				{
					Str += TEXT("/");
				}
			}
			AssetRegistry.ScanPathsSynchronous(LongFileNamePaths);
		}
		else
		{
			if ( AssetRegistry.IsLoadingAssets() )
			{
				// Keep track of the paths we asked for so once assets are discovered we will refresh the list
				for (const FString& Path : Paths)
				{
					DeferredAssetDataPaths.AddUnique(Path);
				}
			}
		}
	}
#endif

	FARFilter ARFilter;
	ARFilter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	
	for (int PathIndex = 0; PathIndex < Paths.Num(); PathIndex++)
	{
		ARFilter.PackagePaths.Add(FName(*Paths[PathIndex]));
	}
	
	ARFilter.bRecursivePaths = bRecursivePaths;
	ARFilter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;

	/* GetDerivedClassNames doesn't work yet
	if ( ObjectBaseClass )
	{
		TArray<FName> SearchClassNames;
		TSet<FName> ExcludedClassNames;
		TSet<FName> DerivedClassNames;
		SearchClassNames.Add(ObjectBaseClass->GetFName());
		AssetRegistry.GetDerivedClassNames(SearchClassNames, ExcludedClassNames, DerivedClassNames);

		const FName GeneratedClassName = FName("GeneratedClass");
		for ( auto DerivedClassIt = DerivedClassNames.CreateConstIterator(); DerivedClassIt; ++DerivedClassIt )
		{
			ARFilter.TagsAndValues.Add(GeneratedClassName, (*DerivedClassIt).ToString());
		}
	}*/

	AssetDataList.Empty();
	AssetRegistry.GetAssets(ARFilter, AssetDataList);

	// Filter out any blueprints found whose parent class is not derived from ObjectBaseClass
	if (ObjectBaseClass)
	{
		TSet<FTopLevelAssetPath> DerivedClassNames;
		TArray<FTopLevelAssetPath> ClassNames;
		ClassNames.Add(ObjectBaseClass->GetClassPathName());
		AssetRegistry.GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);

		for(int32 AssetIdx=AssetDataList.Num() - 1; AssetIdx >= 0; --AssetIdx)
		{
			FAssetData& Data = AssetDataList[AssetIdx];

			bool bShouldRemove = true;
			const FString ParentClassFromData = Data.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
			if (!ParentClassFromData.IsEmpty())
			{
				const FString ClassObjectPath = FPackageName::ExportTextPathToObjectPath(ParentClassFromData);
				if (DerivedClassNames.Contains(FTopLevelAssetPath(ClassObjectPath)))
				{
					// This asset is derived from ObjectBaseClass. Keep it.
					bShouldRemove = false;
				}
			}

			if ( bShouldRemove )
			{
				AssetDataList.RemoveAt(AssetIdx);
			}
		}
	}

	return AssetDataList.Num();
}

int32 UObjectLibrary::LoadAssetsFromAssetData()
{
	int32 Count = 0;
	bool bPreloadObjects = !WITH_EDITOR;

	if (bIsFullyLoaded)
	{
		// We already ran this
		return 0; 
	}

	bIsFullyLoaded = true;

	// Preload the packages with an async call, faster in cooked builds
	if (bPreloadObjects)
	{
		TArray<FSoftObjectPath> AssetsToStream;
		AssetsToStream.Reserve(AssetDataList.Num());
		for (int32 AssetIdx = 0; AssetIdx < AssetDataList.Num(); AssetIdx++)
		{
			FAssetData& Data = AssetDataList[AssetIdx];
			AssetsToStream.AddUnique(FSoftObjectPath(FTopLevelAssetPath(Data.PackageName, NAME_None)));
		}

		if (AssetsToStream.Num())
		{
			FStreamableManager Streamable;
			
			// This will either use loadobject or async load + flush as appropriate
			Streamable.RequestSyncLoad(MoveTemp(AssetsToStream));
		}

	}

	for(int32 AssetIdx=0; AssetIdx<AssetDataList.Num(); AssetIdx++)
	{
		FAssetData& Data = AssetDataList[AssetIdx];

		UObject *LoadedObject = NULL;
		
		if (!bHasBlueprintClasses)
		{
			LoadedObject = Data.GetAsset();

			checkSlow(!LoadedObject || !ObjectBaseClass || LoadedObject->IsA(ObjectBaseClass));

			if (!LoadedObject)
			{
				UE_LOGF(LogEngine, Warning, "Failed to load %ls referenced in %ls", *Data.PackageName.ToString(), ObjectBaseClass ? *ObjectBaseClass->GetName() : TEXT("Unnamed"));
			}
		}
		else
		{
			UPackage* Package = Data.GetPackage();
			if (Package)
			{
				TArray<UObject*> ObjectsInPackage;
				GetObjectsWithOuter(Package, ObjectsInPackage);
				for (UObject* PotentialBPGC : ObjectsInPackage)
				{
					if (auto LoadedBPGC = Cast<UBlueprintGeneratedClass>(PotentialBPGC))
					{
						checkSlow(!ObjectBaseClass || LoadedBPGC->IsChildOf(ObjectBaseClass));
						LoadedObject = LoadedBPGC;
						break; //there is usually only one BGPC in a package
					}
				}
			}
		}

		if (LoadedObject)
		{
			AddObject(LoadedObject);
			Count++;
		}
	}

	return Count;
}

void UObjectLibrary::ClearLoaded()
{
	bIsFullyLoaded = false;
	AssetDataList.Empty();
	Objects.Empty();
	WeakObjects.Empty();
}

void UObjectLibrary::GetAssetDataList(TArray<FAssetData>& OutAssetData)
{
	OutAssetData = AssetDataList;
}

#if WITH_EDITOR
void UObjectLibrary::OnAssetRegistryFilesLoaded()
{
	if ( DeferredAssetDataPaths.Num() )
	{
		if ( bHasBlueprintClasses )
		{
			LoadBlueprintAssetDataFromPaths(DeferredAssetDataPaths, false);
		}
		else
		{
			LoadAssetDataFromPaths(DeferredAssetDataPaths, false);
		}

		DeferredAssetDataPaths.Empty();
	}
}
#endif // WITH_EDITOR

