// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectPropertyTrace.h"
#include "ObjectTrace.h"
#include "Containers/Ticker.h"
#include "UObject/UnrealType.h"

#if WITH_ENGINE

#if OBJECT_PROPERTY_TRACE_ENABLED
UE_TRACE_CHANNEL(ObjectProperties, "Per-frame snapshots of UObject property values (type, name, parent hierarchy, value) for registered objects. Requires the Object channel. Enables property inspection over time in Unreal Insights.")

UE_TRACE_EVENT_BEGIN(Object, ClassPropertyStringId, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint32, Id)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Value)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Object, PropertiesStart2)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Object, PropertiesEnd)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Object, PropertyValue3)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, ObjectId)
	UE_TRACE_EVENT_FIELD(int32, ParentId)
	UE_TRACE_EVENT_FIELD(uint32, TypeId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint32, ParentNameId)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Value)
UE_TRACE_EVENT_END()

namespace ObjectPropertyTrace
{
	static FTSTicker::FDelegateHandle TickerHandle;
	static TArray<TWeakObjectPtr<const UObject>> Objects;

	static uint32 CurrentClassPropertyStringId = 0;
	static TMap<FString, uint32> StringIdMap;
	typedef TFunctionRef<void(const FString& /*InTypeString*/, const FString& /*InValue*/, int32 /*InId*/, int32 /*InParentId*/, FStringBuilderBase& /*InPropertyNameBuilder*/)> IterateFunction;

	static uint32 TraceStringId(const FString& InString)
	{
		if(const uint32* ExistingIdPtr = StringIdMap.Find(InString))
		{
			return *ExistingIdPtr;
		}

		if (InString.IsEmpty())
		{
			return 0;	// Return invalid id.
		}
		else
		{
			const uint32 NewId = StringIdMap.Add(InString, ++CurrentClassPropertyStringId);

			UE_TRACE_LOG(Object, ClassPropertyStringId, ObjectProperties, InString.Len() * sizeof(TCHAR))
				<< ClassPropertyStringId.Id(NewId)
				<< ClassPropertyStringId.Value(*InString, InString.Len());

			return NewId;
		}
	}

	/**
	 * Helper to ensure FStringBuilderBase clears any added strings at the end of a scope.
	 * 
	 * Used to support creating one single FStringBuilderBase when tracing an object's properties recursively.
	 */
	struct FScopedPropertyName
	{
		FScopedPropertyName(FStringBuilderBase& InBuilder) : StartSize(InBuilder.Len()), Builder(InBuilder)
		{
		}

		~FScopedPropertyName()
		{
			Builder.RemoveSuffix(FMath::Max(Builder.Len() - StartSize, 0));
		}
		
		int32 StartSize;
		FStringBuilderBase& Builder;
	};
	
	static void IteratePropertiesRecursive(FProperty* InProperty, const void* InContainer, const FString& InKey, IterateFunction InFunction, int32& InId, int32 InParentId, FStringBuilderBase& InPropertyNameBuilder)
	{
		// Handle container properties
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			FScriptArrayHelper_InContainer Helper(ArrayProperty, InContainer);

			int32 ArrayRootId = ++InId;
			FString SizeString = FString::Printf(TEXT("{Num = %d}"), Helper.Num());
			
			// Build property name id
			FScopedPropertyName ScopedPropertyName(InPropertyNameBuilder);
			ScopedPropertyName.Builder.AppendChar(TEXT('.'));
			ScopedPropertyName.Builder.Append(InProperty->GetName());

			// Trace array property
			InFunction(InProperty->GetCPPType(), SizeString, ArrayRootId, InParentId, InPropertyNameBuilder);

			// Trace container items
			for (int32 DynamicIndex = 0; DynamicIndex < Helper.Num(); ++DynamicIndex)
			{
				const void* ValuePtr = Helper.GetRawPtr(DynamicIndex);
				FString KeyString = FString::Printf(TEXT("[%d]"), DynamicIndex);
				
				IteratePropertiesRecursive(ArrayProperty->Inner, ValuePtr, KeyString, InFunction, InId, ArrayRootId, InPropertyNameBuilder);
			}
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
		{
			FScriptMapHelper_InContainer Helper(MapProperty, InContainer);

			int32 MapRootId = ++InId;
			FString SizeString = FString::Printf(TEXT("{Num = %d}"), Helper.Num());

			// Build property name id
			FScopedPropertyName ScopedPropertyName(InPropertyNameBuilder);
			ScopedPropertyName.Builder.AppendChar(TEXT('.'));
			ScopedPropertyName.Builder.Append(InProperty->GetName());
			
			InFunction(InProperty->GetCPPType(), SizeString, MapRootId, InParentId, InPropertyNameBuilder);
			
			for (FScriptMapHelper::FIterator It(Helper); It; ++It)
			{
				int32 MapEntryId = ++InId;
				FString KeyString = FString::Printf(TEXT("[%d]"), It.GetLogicalIndex());
				FString TypeString = FString::Printf(TEXT("{%s, %s}"), *MapProperty->KeyProp->GetCPPType(), *MapProperty->ValueProp->GetCPPType());

				// Build pair parent name id
				FScopedPropertyName ScopedParentPairName(InPropertyNameBuilder);
				ScopedParentPairName.Builder.AppendChar(TEXT('.'));
				ScopedParentPairName.Builder.Append(KeyString);

				InFunction(TypeString, TEXT("{...}"), MapEntryId, MapRootId, InPropertyNameBuilder);

				const void* PairPtr = Helper.GetPairPtr(It);

				IteratePropertiesRecursive(MapProperty->KeyProp, PairPtr, MapProperty->KeyProp->GetName(), InFunction, InId, MapEntryId, InPropertyNameBuilder);

				IteratePropertiesRecursive(MapProperty->ValueProp, PairPtr, MapProperty->ValueProp->GetName(), InFunction, InId, MapEntryId, InPropertyNameBuilder);
			}
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
		{
			FScriptSetHelper_InContainer Helper(SetProperty, InContainer);

			int32 SetRootId = ++InId;
			FString SizeString = FString::Printf(TEXT("{Num = %d}"), Helper.Num());

			// Build property name id
			FScopedPropertyName ScopedPropertyName(InPropertyNameBuilder);
			ScopedPropertyName.Builder.AppendChar(TEXT('.'));
			ScopedPropertyName.Builder.Append(InProperty->GetName());

			// Trace set property
			InFunction(InProperty->GetCPPType(), SizeString, SetRootId, InParentId, InPropertyNameBuilder);

			// Trace container items
			for (FScriptSetHelper::FIterator It(Helper); It; ++It)
			{
				const void* ValuePtr = Helper.GetElementPtr(It);
				FString KeyString = FString::Printf(TEXT("[%d]"), It.GetLogicalIndex());

				IteratePropertiesRecursive(SetProperty->ElementProp, ValuePtr, KeyString, InFunction, InId, SetRootId, InPropertyNameBuilder);
			}
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			int32 StructRootId = ++InId;

			// Build property name id
			FScopedPropertyName ScopedPropertyName(InPropertyNameBuilder);
			ScopedPropertyName.Builder.AppendChar(TEXT('.'));
			ScopedPropertyName.Builder.Append(InProperty->GetName());

			// Trace struct property
			InFunction(InProperty->GetCPPType(), TEXT("{...}"), StructRootId, InParentId, InPropertyNameBuilder);
			
			// Recurse
			const void* StructContainer = StructProperty->ContainerPtrToValuePtr<const void>(InContainer);
			for(TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				IteratePropertiesRecursive(*It, StructContainer, It->GetName(), InFunction, InId, StructRootId, InPropertyNameBuilder);
			}
		}
		else
		{
			// Normal property
			int32 PropertyParentId = InParentId;
			if(InProperty->ArrayDim > 1)
			{
				// Handle static array header
				PropertyParentId = ++InId;
				FString SizeString = FString::Printf(TEXT("{Num = %d}"), InProperty->ArrayDim);

				// Build property name id
				FScopedPropertyName ScopedPropertyName(InPropertyNameBuilder);
				ScopedPropertyName.Builder.AppendChar(TEXT('.'));
				ScopedPropertyName.Builder.Append(InProperty->GetName());

				// Trace property
				InFunction(InProperty->GetCPPType(), SizeString, PropertyParentId, InParentId,  InPropertyNameBuilder);
			}

			for (int32 StaticIndex = 0; StaticIndex != InProperty->ArrayDim; ++StaticIndex)
			{
				const void* ValuePtr = InProperty->ContainerPtrToValuePtr<const void>(InContainer, StaticIndex);

				FString KeyString = InProperty->ArrayDim == 1 ? InKey : FString::Printf(TEXT("[%d]"), StaticIndex);

				// Build property name id
				FScopedPropertyName ScopedPropertyName(InPropertyNameBuilder);
				ScopedPropertyName.Builder.AppendChar(TEXT('.'));
				ScopedPropertyName.Builder.Append(KeyString);

				// Trace property
				FString ValueString;
				InProperty->ExportText_Direct(ValueString, ValuePtr, ValuePtr, nullptr, PPF_None);
				InFunction(InProperty->GetCPPType(), ValueString, ++InId, PropertyParentId,  InPropertyNameBuilder);
			}
		}
	}

	static void IterateProperties(UStruct* InStruct, const void* InContainer, IterateFunction InFunction)
	{
		TStringBuilder<256> ParentNameBuilder;
		
		int32 Id = INDEX_NONE;
		for(TFieldIterator<FProperty> It(InStruct); It; ++It)
		{
			IteratePropertiesRecursive(*It, InContainer, It->GetName(), InFunction, Id, INDEX_NONE, ParentNameBuilder);
		}
	}

	static void TraceObjects()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FObjectPropertyTrace::TraceObjects);
		for(const TWeakObjectPtr<const UObject>& WeakObject : ObjectPropertyTrace::Objects)
		{
			if(const UObject* TracedObject = WeakObject.Get())
			{
				uint64 StartCycle = FPlatformTime::Cycles64();
				uint64 ObjectId = FObjectTrace::GetObjectId(TracedObject);

				UE_TRACE_LOG(Object, PropertiesStart2, ObjectProperties)
					<< PropertiesStart2.Cycle(StartCycle)
					<< PropertiesStart2.ObjectId(ObjectId)
					<< PropertiesStart2.RecordingTime(FObjectTrace::GetWorldElapsedTime(TracedObject->GetWorld()));

				IterateProperties(TracedObject->GetClass(), TracedObject, [StartCycle, ObjectId](const FString& InTypeString, const FString& InValue, int32 InId, int32 InParentId, FStringBuilderBase& InPropertyNameBuilder)
				{
					const FString NameString = InPropertyNameBuilder.ToString();

					// Compute parent id from property id 
					int32 ParentEndIndex;
					NameString.FindLastChar(TEXT('.'), ParentEndIndex);
					const FString ParentString = NameString.Left(ParentEndIndex);
					
					const uint32 TypeId = TraceStringId(InTypeString);
					const uint32 NameId = TraceStringId(NameString);
					const uint32 ParentId = TraceStringId(ParentString);
	
					UE_TRACE_LOG(Object, PropertyValue3, ObjectProperties)
						<< PropertyValue3.Cycle(StartCycle)
						<< PropertyValue3.ObjectId(ObjectId)
						<< PropertyValue3.Value(*InValue)
						<< PropertyValue3.ParentId(InParentId)
						<< PropertyValue3.TypeId(TypeId)
						<< PropertyValue3.NameId(NameId)
						<< PropertyValue3.ParentNameId(ParentId);
				});

				uint64 EndCycle = FPlatformTime::Cycles64();

				UE_TRACE_LOG(Object, PropertiesEnd, ObjectProperties)
					<< PropertiesEnd.Cycle(EndCycle)
					<< PropertiesEnd.ObjectId(ObjectId);
			}
		}
	}
}

void FObjectPropertyTrace::Init()
{
	check(!ObjectPropertyTrace::TickerHandle.IsValid());
	ObjectPropertyTrace::TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("ObjectPropertyTrace"), 0.0f, [](float InDelta)
	{
		if(UE_TRACE_CHANNELEXPR_IS_ENABLED(ObjectProperties))
		{
			ObjectPropertyTrace::TraceObjects();
		}

		return true;
	});
}

void FObjectPropertyTrace::Destroy()
{
	if (ObjectPropertyTrace::TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ObjectPropertyTrace::TickerHandle);
		ObjectPropertyTrace::TickerHandle.Reset();
	}
}

bool FObjectPropertyTrace::IsEnabled()
{
	return UE_TRACE_CHANNELEXPR_IS_ENABLED(ObjectProperties);
}

void FObjectPropertyTrace::ToggleObjectRegistration(const UObject* InObject)
{
	if(IsObjectRegistered(InObject))
	{
		UnregisterObject(InObject);
	}
	else
	{
		RegisterObject(InObject);
	}
}

void FObjectPropertyTrace::RegisterObject(const UObject* InObject)
{
	ObjectPropertyTrace::Objects.AddUnique(InObject);
}

void FObjectPropertyTrace::UnregisterObject(const UObject* InObject)
{
	ObjectPropertyTrace::Objects.Remove(InObject);
}

bool FObjectPropertyTrace::IsObjectRegistered(const UObject* InObject)
{
	return ObjectPropertyTrace::Objects.Contains(InObject);
}

#endif

#endif