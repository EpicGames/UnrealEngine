// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/ObjectReferencer.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectReferencer)

UObjectReferencer::UObjectReferencer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

