// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectTrace)

#if OBJECT_TRACE_ENABLED
#include "Misc/ScopeRWLock.h"
#include "SceneView.h"
#if WITH_EDITOR
#include "Editor.h"
#else
#include "UObject/Package.h"
#include "UObject/UObjectAnnotation.h"
#endif // WITH_EDITOR

UE_TRACE_MINIMAL_CHANNEL(ObjectChannel, "UObject and UWorld lifecycle (creation, destruction), type hierarchy, transforms, events, controller-pawn possession, and per-frame recording info. Required by Animation, ObjectProperties, and VisualLogger channels.")

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, Type)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, SuperId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Path)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, Object)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, ClassId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, OuterId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Path)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, Flags)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, ObjectLifetimeBegin2)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, ObjectTransform)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, TeleportType)
	UE_TRACE_MINIMAL_EVENT_FIELD(double[], Transform)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, ObjectLifetimeEnd2)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, ObjectEvent)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Event)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, PawnPossess)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, ControllerId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, PawnId)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, World)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, PIEInstanceId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, Type)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, NetMode)
	UE_TRACE_MINIMAL_EVENT_FIELD(bool, IsSimulating)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(Object, RecordingInfo)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, RecordingIndex)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, FrameIndex)
	UE_TRACE_MINIMAL_EVENT_FIELD(double, ElapsedTime)
UE_TRACE_MINIMAL_EVENT_END()

// Not in the minimal trace set of events since not required on shipping servers
UE_TRACE_EVENT_BEGIN(Object, View)
	UE_TRACE_EVENT_FIELD(uint64, PlayerId)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)

	UE_TRACE_EVENT_FIELD(double, PosX)
	UE_TRACE_EVENT_FIELD(double, PosY)
	UE_TRACE_EVENT_FIELD(double, PosZ)

	UE_TRACE_EVENT_FIELD(float, Pitch)
	UE_TRACE_EVENT_FIELD(float, Yaw)
	UE_TRACE_EVENT_FIELD(float, Roll)

	UE_TRACE_EVENT_FIELD(float, Fov)
	UE_TRACE_EVENT_FIELD(float, AspectRatio)
UE_TRACE_EVENT_END()

namespace UE::ObjectTrace
{

// duplicate ObjectInfoFlags must match FObjectInfo::Flag_
constexpr uint8 ObjectInfoFlag_Transient = 0x1;
constexpr uint8 ObjectInfoFlag_StructInstance = 0x2;

bool IsTracingDisabledForObject(const UObject* InObject)
{
	if (InObject == nullptr)
	{
		return true;
	}

	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(ObjectChannel))
	{
		return true;
	}

	if (InObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		return true;
	}

	if (CANNOT_TRACE_OBJECT(InObject->GetWorld()))
	{
		return true;
	}

	return false;
}

struct FObjectId
{
	static constexpr uint64 InvalidId = 0;

	uint64 ObjectId = InvalidId;

	bool operator==(const FObjectId& Other) const = default;

	friend uint32 GetTypeHash(const FObjectId Identifier)
	{
		return GetTypeHash(Identifier.ObjectId);
	}
};

}

// Object annotations used for tracing
struct FObjectIdAnnotation
{
	FObjectIdAnnotation()
		: Id(0)
	{}

	// Object ID
	uint64 Id;

	/** Determine if this annotation is default - required for annotations */
	FORCEINLINE bool IsDefault() const
	{
		return Id == 0;
	}

	bool operator == (const FObjectIdAnnotation& other) const
	{
		return Id == other.Id;
	}
};

int32 GetTypeHash(const FObjectIdAnnotation& Annotation)
{
	return GetTypeHash(Annotation.Id);
}

// Object annotations used for tracing
// FUObjectAnnotationSparse<FTracedObjectAnnotation, true> GObjectTracedAnnotations;

// GObjectTracedSet must only be accessed in the open, and must be protected by the GObjectTracedSetLock.
// Changing GObjectTracedSet in the closed is likely to cause a validation error.
FRWLock GObjectTracedSetLock;
TSet<UE::ObjectTrace::FObjectId> GObjectTracedSet;
FUObjectAnnotationSparseSearchable<FObjectIdAnnotation, true> GObjectIdAnnotations;

// Handle used to hook to world tick
static FDelegateHandle WorldTickStartHandle;

static void TickObjectTraceWorldSubsystem(UWorld* InWorld, ELevelTick InTickType, float InDeltaSeconds)
{
	if (InTickType == LEVELTICK_All)
	{
		if (!InWorld->IsPaused())
		{
			if (UObjectTraceWorldSubsystem* Subsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(InWorld))
			{
				Subsystem->FrameIndex++;
				Subsystem->ElapsedTime += InDeltaSeconds;

				UE_TRACE_MINIMAL_LOG(Object, RecordingInfo, ObjectChannel)
					<< RecordingInfo.WorldId(FObjectTrace::GetObjectId(InWorld))
					<< RecordingInfo.Cycle(FPlatformTime::Cycles64())
					<< RecordingInfo.ElapsedTime(Subsystem->ElapsedTime)
					<< RecordingInfo.FrameIndex(Subsystem->FrameIndex)
					<< RecordingInfo.RecordingIndex(Subsystem->RecordingIndex);
			}
		}
	}
}

// Returns true if the object ID was added to the traced set.
// Returns false if the object ID already existed in the set.
// If the object ID was already in the set, we don't need to log it again.
UE_AUTORTFM_ALWAYS_OPEN static bool AddObjectToTracedSet(const UE::ObjectTrace::FObjectId& ObjectId)
{
	{
		FRWScopeLock ScopeLock(GObjectTracedSetLock, SLT_ReadOnly);
		if (GObjectTracedSet.Contains(ObjectId))
		{
			// We've already traced this object ID, so it can be skipped.
			return false;
		}
	}

	// We haven't traced this object yet. Add its ID to the set.
	FRWScopeLock ScopeLock(GObjectTracedSetLock, SLT_Write);
	GObjectTracedSet.Add(ObjectId);
	return true;
}

void FObjectTrace::Init()
{
	WorldTickStartHandle = FWorldDelegates::OnWorldTickStart.AddStatic(&TickObjectTraceWorldSubsystem);
}

void FObjectTrace::Destroy()
{
	FWorldDelegates::OnWorldTickStart.Remove(WorldTickStartHandle);
}

void FObjectTrace::Reset()
{
	GObjectIdAnnotations.RemoveAllAnnotations();

	UE_AUTORTFM_OPEN
	{
		FRWScopeLock ScopeLock(GObjectTracedSetLock, SLT_Write);
		GObjectTracedSet.Empty();
	};
}

uint64 FObjectTrace::AllocateInstanceId()
{
	static int64 volatile CurrentId = 1;

	int64 NewID = 0;
	UE_AUTORTFM_OPEN
	{
		NewID = FPlatformAtomics::InterlockedIncrement(&CurrentId);
	};

	return NewID;
}

uint64 FObjectTrace::GetObjectId(const UObject* InObject)
{
	// An object ID uses a combination of its own and its outer's index
	// We do this to represent objects that get renamed into different outers 
	// as distinct traces (we don't attempt to link them).

	auto GetObjectIdInner = [](const UObject* InObjectInner)
	{
		FObjectIdAnnotation Annotation = GObjectIdAnnotations.GetAnnotation(InObjectInner);
		if (Annotation.Id == 0)
		{
			Annotation.Id = AllocateInstanceId();
			GObjectIdAnnotations.AddAnnotation(InObjectInner, MoveTemp(Annotation));
		}

		return Annotation.Id;
	};

	uint64 Id = 0;
	uint64 OuterId = 0;
	if (InObject)
	{
		Id = GetObjectIdInner(InObject);

		if (const UObject* Outer = InObject->GetOuter())
		{
			OuterId = GetObjectIdInner(Outer);
		}
	}

	return Id | (OuterId << 32);
}

UObject* FObjectTrace::GetObjectFromId(uint64 Id)
{
	FObjectIdAnnotation FindAnnotation;
	// Id used for annotation map doesn't include the parent id in the upper bits, so zero those first
	FindAnnotation.Id = Id & 0x00000000FFFFFFFFll;
	if (FindAnnotation.IsDefault())
	{
		return nullptr;
	}

	return GObjectIdAnnotations.Find(FindAnnotation);
}

void FObjectTrace::ResetWorldElapsedTime(const UWorld* World)
{
	if (UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(World))
	{
		WorldSubsystem->ElapsedTime = 0;
	}
}

double FObjectTrace::GetWorldElapsedTime(const UWorld* World)
{
	if (const UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(World))
	{
		return WorldSubsystem->ElapsedTime;
	}
	return 0;
}

double FObjectTrace::GetObjectWorldElapsedTime(const UObject* InObject)
{
	if (InObject != nullptr)
	{
		if (const UWorld* World = InObject->GetWorld())
		{
			if (const UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(World))
			{
				return WorldSubsystem->ElapsedTime;
			}
		}
	}

	return 0;
}

void FObjectTrace::SetWorldRecordingIndex(const UWorld* World, uint16 Index)
{
	if (UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(World))
	{
		WorldSubsystem->RecordingIndex = Index;
	}
}

uint16 FObjectTrace::GetWorldRecordingIndex(const UWorld* InWorld)
{
	if (const UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(InWorld))
	{
		return WorldSubsystem->RecordingIndex;
	}
	return 0;
}

uint16 FObjectTrace::GetObjectWorldRecordingIndex(const UObject* InObject)
{
	if (InObject != nullptr)
	{
		if (const UWorld* World = InObject->GetWorld())
		{
			if (const UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(World))
			{
				return WorldSubsystem->RecordingIndex;
			}
		}
	}

	return 0;
}

uint16 FObjectTrace::GetObjectWorldTickCounter(const UObject* InObject)
{
	if (InObject != nullptr)
	{
		if (const UWorld* World = InObject->GetWorld())
		{
			if (const UObjectTraceWorldSubsystem* WorldSubsystem = UWorld::GetSubsystem<UObjectTraceWorldSubsystem>(World))
			{
				return WorldSubsystem->FrameIndex;
			}
		}
	}

	return 0;
}

void FObjectTrace::OutputType(const UStruct* InType)
{
	const bool bChannelEnabled = UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(ObjectChannel);
	if (!bChannelEnabled || InType == nullptr)
	{
		return;
	}
	const uint64 ObjectId = GetObjectId(InType);
	if (!AddObjectToTracedSet(UE::ObjectTrace::FObjectId{ObjectId}))
	{
		return;
	}

	OutputType(InType->GetSuperStruct());

	const FString TypePathName = InType->GetPathName();
	TCHAR TypeName[FName::StringBufferSize];
	const uint32 TypeNameLength = InType->GetFName().ToString(TypeName);

	UE_TRACE_MINIMAL_LOG(Object, Type, ObjectChannel)
		<< Type.Id(ObjectId)
		<< Type.SuperId(GetObjectId(InType->GetSuperStruct()))
		<< Type.Name(TypeName, TypeNameLength)
		<< Type.Path(*TypePathName, TypePathName.Len());
}

void FObjectTrace::OutputView(const UObject* InPlayer, const FSceneView* InView)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InPlayer))
	{
		return;
	}

	const FIntRect& ViewRect = InView->CameraConstrainedViewRect;
	const float AspectRatio = static_cast<float>(ViewRect.Width()) / static_cast<float>(ViewRect.Height());

	const FMatrix ProjMatrix = InView->ViewMatrices.GetViewToClip();
	const float Fov = atan(1.0 / ProjMatrix.M[0][0]) * 2.0 * 180.0 / UE_DOUBLE_PI;

	UE_TRACE_LOG(Object, View, ObjectChannel)
		<< View.Cycle(FPlatformTime::Cycles64())
		<< View.PlayerId(GetObjectId(InPlayer))
		<< View.PosX(InView->ViewLocation.X)
		<< View.PosY(InView->ViewLocation.Y)
		<< View.PosZ(InView->ViewLocation.Z)
		<< View.Pitch(InView->ViewRotation.Pitch)
		<< View.Yaw(InView->ViewRotation.Yaw)
		<< View.Roll(InView->ViewRotation.Roll)
		<< View.Fov(Fov)
		<< View.AspectRatio(AspectRatio);
}

void OutputInstanceInternal(const UObject* InOuterObject, uint64 InInstanceId, uint64 InOuterId, UStruct* InType, const FString& InName, const FString& InPathName, uint8 InFlags)
{
	// Trace the object's class first
	TRACE_TYPE(InType);

	UE_TRACE_MINIMAL_LOG(Object, Object, ObjectChannel)
		<< Object.Id(InInstanceId)
		<< Object.ClassId(FObjectTrace::GetObjectId(InType))
		<< Object.OuterId(InOuterId)
		<< Object.Name(*InName, InName.Len())
		<< Object.Path(*InPathName, InPathName.Len())
		<< Object.Flags(InFlags);
		
	UE_TRACE_MINIMAL_LOG(Object, ObjectLifetimeBegin2, ObjectChannel)
		<< ObjectLifetimeBegin2.Cycle(FPlatformTime::Cycles64())
		<< ObjectLifetimeBegin2.RecordingTime(FObjectTrace::GetWorldElapsedTime(InOuterObject->GetWorld()))
		<< ObjectLifetimeBegin2.Id(InInstanceId);
}

void FObjectTrace::OutputInstance(const UObject* InOuterObject, uint64 InInstanceId, uint64 InOuterId, UStruct* InType, const FString& InName, const FString& InPathName)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InOuterObject))
	{
		return;
	}

	if (!AddObjectToTracedSet(UE::ObjectTrace::FObjectId{InInstanceId}))
	{
		return;
	}

	constexpr uint8 Flags = UE::ObjectTrace::ObjectInfoFlag_StructInstance;
	OutputInstanceInternal(InOuterObject, InInstanceId, InOuterId, InType, InName, InPathName, Flags);
}

bool FObjectTrace::OutputObject(const UObject* InObject, UObject* InUserDefinedOuter)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InObject))
	{
		return false;
	}

	const uint64 InstanceId = GetObjectId(InObject);
	if (!AddObjectToTracedSet(UE::ObjectTrace::FObjectId{ InstanceId }))
	{
		return false;
	}

	uint8 Flags = 0;
	if (InObject->HasAnyFlags(RF_Transient))
	{
		Flags |= UE::ObjectTrace::ObjectInfoFlag_Transient;
	}

	UObject* Outer = InUserDefinedOuter ? InUserDefinedOuter : InObject->GetOuter();	
	OutputObject(Outer);
	OutputInstanceInternal(InObject, InstanceId, GetObjectId(Outer), InObject->GetClass(), InObject->GetFName().ToString(), InObject->GetPathName(), Flags);
	return true;
}

void FObjectTrace::OutputObjectTransform(const UObject* InObject, const FTransform& InTransform, ETeleportType InTeleportType)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InObject))
	{
		return;
	}

	TRACE_OBJECT(InObject);

	UE_TRACE_MINIMAL_LOG(Object, ObjectTransform, ObjectChannel)
		<< ObjectTransform.Cycle(FPlatformTime::Cycles64())
		<< ObjectTransform.RecordingTime(FObjectTrace::GetWorldElapsedTime(InObject->GetWorld()))
		<< ObjectTransform.Id(GetObjectId(InObject))
		<< ObjectTransform.TeleportType(static_cast<uint8>(InTeleportType))
		<< ObjectTransform.Transform(reinterpret_cast<const double*>(&InTransform), sizeof(FTransform) / sizeof(double));
}

void FObjectTrace::OutputObjectEvent(const UObject* InObject, const TCHAR* InEvent)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InObject))
	{
		return;
	}

	TRACE_OBJECT(InObject);

	UE_TRACE_MINIMAL_LOG(Object, ObjectEvent, ObjectChannel)
		<< ObjectEvent.Cycle(FPlatformTime::Cycles64())
		<< ObjectEvent.Id(GetObjectId(InObject))
		<< ObjectEvent.Event(InEvent);
}

void FObjectTrace::OutputObjectLifetimeBegin(const UObject* InObject, UObject* InUserDefinedOuter)
{
	TRACE_OBJECT_WITH_OUTER(InObject, InUserDefinedOuter);
}

void FObjectTrace::OutputInstanceLifetimeEnd(const UObject* InOuterObject, uint64 InInstanceId)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InOuterObject))
	{
		return;
	}
	if (UWorld* World = InOuterObject->GetWorld())
	{
		UE_TRACE_MINIMAL_LOG(Object, ObjectLifetimeEnd2, ObjectChannel)
			<< ObjectLifetimeEnd2.Cycle(FPlatformTime::Cycles64())
			<< ObjectLifetimeEnd2.RecordingTime(FObjectTrace::GetWorldElapsedTime(World))
			<< ObjectLifetimeEnd2.Id(InInstanceId);
	}
}

void FObjectTrace::OutputObjectLifetimeEnd(const UObject* InObject)
{
	TRACE_OBJECT(InObject);
	OutputInstanceLifetimeEnd(InObject, GetObjectId(InObject));
}

void FObjectTrace::OutputPawnPossess(const UObject* InController, const UObject* InPawn)
{
	if (UE::ObjectTrace::IsTracingDisabledForObject(InController))
	{
		return;
	}

	TRACE_OBJECT(InController);
	if (InPawn)
	{
		TRACE_OBJECT(InPawn);
	}

	UE_TRACE_MINIMAL_LOG(Object, PawnPossess, ObjectChannel)
		<< PawnPossess.Cycle(FPlatformTime::Cycles64())
		<< PawnPossess.ControllerId(GetObjectId(InController))
		<< PawnPossess.PawnId(GetObjectId(InPawn));
}

void FObjectTrace::OutputWorld(const UWorld* InWorld)
{
	const bool bChannelEnabled = UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(ObjectChannel);
	if (!bChannelEnabled || InWorld == nullptr)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(InWorld))
	{
		return;
	}

#if WITH_EDITOR
	bool bIsSimulating = GEditor ? GEditor->bIsSimulatingInEditor : false;
#else
	bool bIsSimulating = false;
#endif

	UE_TRACE_MINIMAL_LOG(Object, World, ObjectChannel)
		<< World.Id(GetObjectId(InWorld))
		<< World.PIEInstanceId(InWorld->GetOutermost()->GetPIEInstanceID())
		<< World.Type(static_cast<uint8>(InWorld->WorldType))
		<< World.NetMode(static_cast<uint8>(InWorld->GetNetMode()))
		<< World.IsSimulating(bIsSimulating);

	// Trace object AFTER world info so we don't risk world info not being present in the trace
	TRACE_OBJECT(InWorld);
}

#endif // OBJECT_TRACE_ENABLED
