// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/OnlineBlueprintCallProxyBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineBlueprintCallProxyBase)

//////////////////////////////////////////////////////////////////////////
// UOnlineBlueprintCallProxyBase

UOnlineBlueprintCallProxyBase::UOnlineBlueprintCallProxyBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetFlags(RF_StrongRefOnFrame);
}

