// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/OnlineEngineInterface.h"
#include "UObject/Package.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineEngineInterface)

DEFINE_LOG_CATEGORY_STATIC(LogOnlineEngine, Log, All);

UOnlineEngineInterface* UOnlineEngineInterface::Singleton = nullptr;

UOnlineEngineInterface::UOnlineEngineInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UOnlineEngineInterface* UOnlineEngineInterface::Get()
{
	if (!Singleton)
	{
		FString OnlineEngineInterfaceClassName;
		GConfig->GetString(TEXT("/Script/Engine.OnlineEngineInterface"), TEXT("ClassName"), OnlineEngineInterfaceClassName, GEngineIni);

		bool bUseOnlineServices = FParse::Param(FCommandLine::Get(), TEXT("bUseOnlineServices"));
		if (bUseOnlineServices)
		{
			OnlineEngineInterfaceClassName = TEXT("/Script/OnlineSubsystemUtils.OnlineServicesEngineInterfaceImpl");
		}

		// To not break licensees using the config override, prefer it if it is present, and warn. Remove in 5.7
		if (GConfig->GetBool(TEXT("/Script/Engine.OnlineEngineInterface"), TEXT("bUseOnlineServicesV2"), bUseOnlineServices, GEngineIni))
		{
			const TCHAR* V1ClassName = TEXT("/Script/OnlineSubsystemUtils.OnlineEngineInterfaceImpl");
			const TCHAR* V2ClassName = TEXT("/Script/OnlineSubsystemUtils.OnlineServicesEngineInterfaceImpl");
			OnlineEngineInterfaceClassName = bUseOnlineServices ? V2ClassName : V1ClassName;
			UE_LOGF(LogOnlineEngine, Warning, "bUseOnlineServicesV2 is deprecated, please instead configure [/Script/Engine.OnlineEngineInterface]:ClassName=%ls", *OnlineEngineInterfaceClassName);
		}
		
		UClass* OnlineEngineInterfaceClass = nullptr;
		if (!OnlineEngineInterfaceClassName.IsEmpty())
		{
			OnlineEngineInterfaceClass = StaticLoadClass(UOnlineEngineInterface::StaticClass(), NULL, *OnlineEngineInterfaceClassName, NULL, LOAD_Quiet, NULL);
		}
		
		if (!OnlineEngineInterfaceClass)
		{
			// Default to the no op class if necessary
			OnlineEngineInterfaceClass = UOnlineEngineInterface::StaticClass();
		}

		Singleton = NewObject<UOnlineEngineInterface>(GetTransientPackage(), OnlineEngineInterfaceClass);
		Singleton->AddToRoot();
	}

	return Singleton;
}

