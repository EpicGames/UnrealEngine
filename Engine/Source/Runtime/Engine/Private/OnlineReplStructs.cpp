// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OnlineReplStructs.cpp: Unreal networking serialization helpers
=============================================================================*/

#include "GameFramework/OnlineReplStructs.h"
#include "UObject/CoreNet.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Dom/JsonValue.h"
#include "EngineLogs.h"
#include "Net/OnlineEngineInterface.h"
#include "Misc/AsciiSet.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceNull.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineReplStructs)

namespace
{
	static const FString InvalidUniqueNetIdStr = "INVALID";
}

/** Flags relevant to network serialization of a unique id */
enum class EUniqueIdEncodingFlags : uint8
{
	/** Default, nothing encoded, use normal FString serialization */
	NotEncoded = 0,
	/** Data is optimized based on some assumptions (even number of [0-9][a-f][A-F] that can be packed into nibbles) */
	IsEncoded = (1 << 0),
	/** This unique id is empty or invalid, nothing further to serialize */
	IsEmpty = (1 << 1),
	/** Original data is padded with a leading zero */
	IsPadded = (1 << 2),
	/** Remaining bits are used for encoding the type without requiring another byte */
	Reserved1 = (1 << 3),
	Reserved2 = (1 << 4),
	Reserved3 = (1 << 5),
	Reserved4 = (1 << 6),
	Reserved5 = (1 << 7),
	/** Helper masks */
	FlagsMask = (Reserved1 - 1),
	TypeMask = (MAX_uint8 ^ FlagsMask)
};
ENUM_CLASS_FLAGS(EUniqueIdEncodingFlags);

/** Use highest value for type for other (out of engine) oss type */
const uint8 TypeHash_Other = 31;
/** Use next highest value for V2 net id */
const uint8 TypeHash_V2 = 30;

FArchive& operator<<(FArchive& Ar, FUniqueNetIdRepl& UniqueNetId)
{
	if (!Ar.IsPersistent() || Ar.IsNetArchive())
	{
		bool bOutSuccess = false;
		UniqueNetId.NetSerialize(Ar, nullptr, bOutSuccess);
	}
	else
	{
		int32 Size = UniqueNetId.IsValid() ? UniqueNetId->GetSize() : 0;
		Ar << Size;

		if (Size > 0)
		{
			if (Ar.IsSaving())
			{
				check(UniqueNetId.IsValid());

				FName Type = UniqueNetId.IsValid() ? UniqueNetId->GetType() : NAME_None;
				Ar << Type;

				FString Contents = UniqueNetId->ToString();
				Ar << Contents;
			}
			else if (Ar.IsLoading())
			{
				FName Type;
				Ar << Type;

				FString Contents;
				Ar << Contents;	// that takes care about possible overflow

				UniqueNetId.UniqueIdFromString(Type, Contents);
			}
		}
		else if (Ar.IsLoading())
		{
			// @note: replicated a nullptr unique id
			UniqueNetId.SetUniqueNetId(nullptr);
		}
	}
	return Ar;
}

inline uint8 GetTypeHashFromEncoding(EUniqueIdEncodingFlags inFlags)
{
	uint8 TypeHash = static_cast<uint8>(inFlags & EUniqueIdEncodingFlags::TypeMask) >> 3;
	return (TypeHash < 32) ? TypeHash : 0;
}

/**
 * Possibly encode the unique net id in a smaller form
 *
 * Empty:
 *    <uint8 flags> noted it is encoded and empty
 * NonEmpty:
 * - Encoded - <uint8 flags/type> <uint8 encoded size> <encoded bytes>
 * - Encoded (out of engine oss type) - <uint8 flags/type> <serialized FName> <uint8 encoded size> <encoded bytes>
 * - Unencoded - <uint8 flags/type> <serialized FString>
 * - Unencoded (out of engine oss type) - <uint8 flags/type> <serialized FName> <serialized FString>
 */
void FUniqueNetIdRepl::MakeReplicationData()
{
	if (IsValid())
	{
		if (IsV1())
		{
			MakeReplicationDataV1();
		}
		else
		{
			MakeReplicationDataV2();
		}
	}
	else
	{
		EUniqueIdEncodingFlags EncodingFlags = (EUniqueIdEncodingFlags::IsEncoded | EUniqueIdEncodingFlags::IsEmpty);

		ReplicationBytes.Empty(sizeof(EncodingFlags));
		FMemoryWriter Writer(ReplicationBytes);
		Writer << EncodingFlags;
	}
}

void FUniqueNetIdRepl::MakeReplicationDataV1()
{
	FString Contents = GetUniqueNetId()->ToString();
	const int32 Length = Contents.Len();
	if (ensure(Length > 0))
	{
		// For now don't allow odd chars (HexToBytes adds a 0)
		const bool bEvenChars = (Length % 2) == 0;
		const int32 EncodedSize32 = ((Length * sizeof(ANSICHAR)) + 1) / 2;
		const bool bIsNumeric = Contents.IsNumeric() && !(Contents.StartsWith("+") || Contents.StartsWith("-"));
		bool bIsPadded = bIsNumeric && !bEvenChars;

		//UE_LOGF(LogNet, VeryVerbose, "bEvenChars: %d EncodedSize: %d bIsNumeric: %d bIsPadded: %d", bEvenChars, EncodedSize32, bIsNumeric, bIsPadded);

		EUniqueIdEncodingFlags EncodingFlags = (bIsNumeric || (bEvenChars && (EncodedSize32 < UINT8_MAX))) ? EUniqueIdEncodingFlags::IsEncoded : EUniqueIdEncodingFlags::NotEncoded;
		if (bIsPadded)
		{
			EncodingFlags |= EUniqueIdEncodingFlags::IsPadded;
		}

		if (EnumHasAllFlags(EncodingFlags, EUniqueIdEncodingFlags::IsEncoded) && !bIsNumeric)
		{
			const TCHAR* const ContentChar = *Contents;
			for (int32 i = 0; i < Length; ++i)
			{
				// Don't allow uppercase because HexToBytes loses case and we aren't encoding anything but all lowercase hex right now
				if (!FChar::IsHexDigit(ContentChar[i]) || FChar::IsUpper(ContentChar[i]))
				{
					EncodingFlags = EUniqueIdEncodingFlags::NotEncoded;
					break;
				}
			}
		}

		// Encode the unique id type
		FName Type = GetType();
		uint8 TypeHash = UOnlineEngineInterface::Get()->GetReplicationHashForSubsystem(Type);
		ensure(TypeHash < 32);
		if (TypeHash == 0 && Type != NAME_None)
		{
			TypeHash = TypeHash_Other;
		}
		EncodingFlags = static_cast<EUniqueIdEncodingFlags>((TypeHash << 3) | static_cast<uint8>(EncodingFlags));

		if (EnumHasAllFlags(EncodingFlags, EUniqueIdEncodingFlags::IsEncoded))
		{
			uint8 EncodedSize = static_cast<uint8>(EncodedSize32);
			const int32 TotalBytes = sizeof(EncodingFlags) + sizeof(EncodedSize) + EncodedSize; // no optimization for TypeHash_Other
			ReplicationBytes.Empty(TotalBytes);

			FMemoryWriter Writer(ReplicationBytes);
			Writer << EncodingFlags;
			if (TypeHash == TypeHash_Other)
			{
				FString TypeString = Type.ToString();
				Writer << TypeString;
			}
			Writer << EncodedSize;

			int32 HexStartOffset = Writer.Tell();
			ReplicationBytes.AddUninitialized(EncodedSize);
			int32 HexEncodeLength = HexToBytes(Contents, ReplicationBytes.GetData() + HexStartOffset);
			ensure(HexEncodeLength == EncodedSize32);
			//Writer.Seek(HexStartOffset + HexEncodeLength);
			//UE_LOGF(LogNet, VeryVerbose, "HexEncoded UniqueId, serializing %d bytes", ReplicationBytes.Num());
		}
		else
		{
			const int32 TotalBytes = sizeof(EncodingFlags) + sizeof(Length) + Length * sizeof(TCHAR); // no optimization for TypeHash_Other
			ReplicationBytes.Empty(TotalBytes);

			FMemoryWriter Writer(ReplicationBytes);
			Writer << EncodingFlags;
			if (TypeHash == TypeHash_Other)
			{
				FString TypeString = Type.ToString();
				Writer << TypeString;
			}
			Writer << Contents;
			//UE_LOGF(LogNet, VeryVerbose, "Normal UniqueId, serializing %d bytes", ReplicationBytes.Num());
		}
	}
	else
	{
		EUniqueIdEncodingFlags EncodingFlags = (EUniqueIdEncodingFlags::IsEncoded | EUniqueIdEncodingFlags::IsEmpty);

		ReplicationBytes.Empty(sizeof(EncodingFlags));
		FMemoryWriter Writer(ReplicationBytes);
		Writer << EncodingFlags;
	}
}

void FUniqueNetIdRepl::MakeReplicationDataV2()
{
	EUniqueIdEncodingFlags EncodingFlags = EUniqueIdEncodingFlags::IsEncoded;
	EncodingFlags |= static_cast<EUniqueIdEncodingFlags>(TypeHash_V2 << 3);

	UE::Online::FAccountId AccountId = GetV2();
	UE::Online::EOnlineServices OnlineServicesType = AccountId.GetOnlineServicesType();
	TArray<uint8> ReplicationData = UE::Online::FOnlineIdRegistryRegistry::Get().ToReplicationData(AccountId);
	checkSlow(!ReplicationData.IsEmpty());

	const int32 TotalBytes = sizeof(EncodingFlags) + sizeof(OnlineServicesType) + sizeof(TArray<uint8>::SizeType) + ReplicationData.Num();
	ReplicationBytes.Empty(TotalBytes);
	FMemoryWriter Writer(ReplicationBytes);
	Writer << EncodingFlags;
	Writer << OnlineServicesType;
	Writer << ReplicationData;
}

void FUniqueNetIdRepl::UniqueIdFromString(FName Type, const FString& Contents)
{
	// Don't need to distinguish OSS interfaces here with world because we just want the create function below
	FUniqueNetIdWrapper UniqueNetIdWrapper = UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(Contents, Type);
	SetUniqueNetId(UniqueNetIdWrapper.GetUniqueNetId());
}

bool FUniqueNetIdRepl::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = false;

	if (Ar.IsSaving())
	{
		if (ReplicationBytes.Num() == 0)
		{
			MakeReplicationData();
		}

		Ar.Serialize(ReplicationBytes.GetData(), ReplicationBytes.Num());
		bOutSuccess = (ReplicationBytes.Num() > 0);
		//UE_LOGF(LogNet, Warning, "UID Save: Bytes: %d Success: %d", ReplicationBytes.Num(), bOutSuccess);
	}
	else if (Ar.IsLoading())
	{
		// @note: start by assuming a replicated nullptr unique id
		SetUniqueNetId(nullptr);

		EUniqueIdEncodingFlags EncodingFlags = EUniqueIdEncodingFlags::NotEncoded;
		Ar << EncodingFlags;
		if (!Ar.IsError())
		{
			
			if (EnumHasAllFlags(EncodingFlags, EUniqueIdEncodingFlags::IsEncoded))
			{
				if (!EnumHasAllFlags(EncodingFlags, EUniqueIdEncodingFlags::IsEmpty))
				{
					const uint8 TypeHash = GetTypeHashFromEncoding(EncodingFlags);
					if (TypeHash == TypeHash_V2)
					{
						NetSerializeLoadV2(Ar, EncodingFlags, bOutSuccess);
					}
					else
					{
						NetSerializeLoadV1Encoded(Ar, EncodingFlags, bOutSuccess);
					}
				}
				else
				{
					// empty cleared out unique id
					bOutSuccess = true;
				}
			}
			else
			{
				NetSerializeLoadV1Unencoded(Ar, EncodingFlags, bOutSuccess);
			}
		}
		else
		{
			UE_LOGF(LogNet, Warning, "Error serializing unique id");
		}
	}

	return true;
}

void FUniqueNetIdRepl::NetSerializeLoadV1Encoded(FArchive& Ar, const EUniqueIdEncodingFlags EncodingFlags, bool& bOutSuccess)
{
	// Non empty and hex encoded
	uint8 TypeHash = GetTypeHashFromEncoding(EncodingFlags);
	if (TypeHash == 0)
	{
		// If no type was encoded, assume default
		TypeHash = UOnlineEngineInterface::Get()->GetReplicationHashForSubsystem(UOnlineEngineInterface::Get()->GetDefaultOnlineSubsystemName());
	}
	FName Type;
	bool bValidTypeHash = TypeHash != 0;
	if (TypeHash == TypeHash_Other)
	{
		FString TypeString;
		Ar << TypeString;
		Type = FName(*TypeString);
		if (Ar.IsError() || Type == NAME_None)
		{
			bValidTypeHash = false;
		}
	}
	else
	{
		Type = UOnlineEngineInterface::Get()->GetSubsystemFromReplicationHash(TypeHash);
	}

	if (bValidTypeHash)
	{
		// Get the size
		uint8 EncodedSize = 0;
		Ar << EncodedSize;
		if (!Ar.IsError())
		{
			if (EncodedSize > 0)
			{
				uint8* TempBytes = (uint8*)FMemory_Alloca(EncodedSize);
				Ar.Serialize(TempBytes, EncodedSize);
				if (!Ar.IsError())
				{
					FString Contents = BytesToHex(TempBytes, EncodedSize);
					if (Contents.Len() > 0)
					{
						if (Type != NAME_None)
						{
							// BytesToHex loses case
							Contents.ToLowerInline();
							if (EnumHasAllFlags(EncodingFlags, EUniqueIdEncodingFlags::IsPadded))
							{
								Contents.RightChopInline(1); // remove padded character
							}
							UniqueIdFromString(Type, Contents);
						}
						else
						{
							UE_LOGF(LogNet, Warning, "Error with unique id type");
						}
					}
				}
				else
				{
					UE_LOGF(LogNet, Warning, "Error with encoded unique id contents");
				}
			}
			else
			{
				UE_LOGF(LogNet, Warning, "Empty Encoding!");
			}

			bOutSuccess = (EncodedSize == 0) || IsValid();
		}
		else
		{
			UE_LOGF(LogNet, Warning, "Error with encoded unique id size");
		}
	}
	else
	{
		UE_LOGF(LogNet, Warning, "Error with encoded type hash");
	}
}

void FUniqueNetIdRepl::NetSerializeLoadV1Unencoded(FArchive& Ar, const EUniqueIdEncodingFlags EncodingFlags, bool& bOutSuccess)
{
	uint8 TypeHash = GetTypeHashFromEncoding(EncodingFlags);
	if (TypeHash == 0)
	{
		// If no type was encoded, assume default
		TypeHash = UOnlineEngineInterface::Get()->GetReplicationHashForSubsystem(UOnlineEngineInterface::Get()->GetDefaultOnlineSubsystemName());
	}
	FName Type;
	bool bValidTypeHash = TypeHash != 0;
	if (TypeHash == TypeHash_Other)
	{
		FString TypeString;
		Ar << TypeString;
		Type = FName(*TypeString);
		if (Ar.IsError() || Type == NAME_None)
		{
			bValidTypeHash = false;
		}
	}
	else
	{
		Type = UOnlineEngineInterface::Get()->GetSubsystemFromReplicationHash(TypeHash);
	}

	if (bValidTypeHash)
	{
		FString Contents;
		Ar << Contents;
		if (!Ar.IsError())
		{
			if (Type != NAME_None)
			{
				UniqueIdFromString(Type, Contents);
				bOutSuccess = !Contents.IsEmpty();
			}
			else
			{
				UE_LOGF(LogNet, Warning, "Error with unique id type");
			}
		}
		else
		{
			UE_LOGF(LogNet, Warning, "Error with unencoded unique id");
		}
	}
	else
	{
		UE_LOGF(LogNet, Warning, "Error with encoded type hash");
	}
}

void FUniqueNetIdRepl::NetSerializeLoadV2(FArchive& Ar, const EUniqueIdEncodingFlags EncodingFlags, bool& bOutSuccess)
{
	UE::Online::EOnlineServices OnlineServicesType;
	Ar << OnlineServicesType;
	TArray<uint8> ReplicationData;
	Ar << ReplicationData;

	const UE::Online::FAccountId AccountId = UE::Online::FOnlineIdRegistryRegistry::Get().ToAccountId(OnlineServicesType, ReplicationData);
	checkSlow(AccountId.IsValid());
	SetAccountId(AccountId);
}

bool FUniqueNetIdRepl::Serialize(FArchive& Ar)
{
	Ar << *this;
	return true;
}

bool FUniqueNetIdRepl::ShouldExportTextItemAsQuotedString(const FString& NetIdStr)
{
	// Logic derived from FPropertyHelpers::ReadToken. If that would not parse the string, we need to wrap in quotes.

	if (NetIdStr.IsEmpty())
	{
		return false;
	}

	constexpr FAsciiSet AlphaNumericChars = "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789";
	constexpr FAsciiSet ValidChars = AlphaNumericChars + '_' + '-' + '+';

	TCHAR FirstChar = NetIdStr[0];
	const bool bFirstCharValid = AlphaNumericChars.Test(FirstChar) || FirstChar > 255;
	if (!bFirstCharValid)
	{
		return true;
	}

	return !FAsciiSet::HasOnly(*NetIdStr, ValidChars);
}

bool FUniqueNetIdRepl::ExportTextItem(FString& ValueStr, FUniqueNetIdRepl const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (IsValid())
	{
		if (IsV1())
		{
			FName Type = GetType();
			const FString NetIdStr = ToString();
			if (Type == UOnlineEngineInterface::Get()->GetDefaultOnlineSubsystemName())
			{
				if (ShouldExportTextItemAsQuotedString(NetIdStr))
				{
					ValueStr += FString::Printf(TEXT("\"%s\""), *NetIdStr);
				}
				else
				{
					ValueStr += NetIdStr;
				}
			}
			else
			{
				if (ShouldExportTextItemAsQuotedString(NetIdStr))
				{
					ValueStr += FString::Printf(TEXT("%s:\"%s\""), *Type.ToString(), *NetIdStr);
				}
				else
				{
					ValueStr += FString::Printf(TEXT("%s:%s"), *Type.ToString(), *NetIdStr);
				}
				
			}
		}
		else
		{
			ValueStr += ToDebugString();
		}
	}
	else
	{
		ValueStr = InvalidUniqueNetIdStr;
	}
	
	return true;
}

bool FUniqueNetIdRepl::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	SetUniqueNetId(nullptr);

	bool bShouldWarn = true;
	if (Buffer)
	{
		// An empty string, BP empty "()", or the word invalid are just considered expected invalid FUniqueNetIdRepls. No need to warn about those.
		if (Buffer[0] == TEXT('\0'))
		{
			bShouldWarn = false;
		}
		else if (Buffer[0] == TEXT('(') && Buffer[1] == TEXT(')'))
		{
			bShouldWarn = false;
			Buffer += 2;
		}
		else
		{
			check(UOnlineEngineInterface::Get());

			FString Token;
			if (const TCHAR* NewBuffer1 = FPropertyHelpers::ReadToken(Buffer, Token))
			{
				Buffer = NewBuffer1;

				FName OnlineSubsystemName;
				FString IdContents;

				// Ids can be serialized with OSS prefix e.g.: PREFIX:48ac94b14ca949f3244f91f4a92afb86
				if (Buffer[0] == TEXT(':'))
				{
					Buffer++;

					OnlineSubsystemName = FName(*Token);
					if (const TCHAR* NewBuffer2 = FPropertyHelpers::ReadToken(Buffer, IdContents))
					{
						Buffer = NewBuffer2;
					}
				}
				else
				{
					IdContents = Token;
				}

				if (IdContents != InvalidUniqueNetIdStr)
				{
					// Log the subsystem name used once
					if (!UOnlineEngineInterface::Get()->IsLoaded(OnlineSubsystemName))
					{
						static bool bLogged = false;
						UE_CLOGF(!bLogged, LogNet, Log, "Attempted to ImportText to FUniqueNetIdRepl while OSS is not loaded. OSS:%ls Parent:%ls", *OnlineSubsystemName.ToString(), *GetPathNameSafe(Parent));
						bLogged = true;
					}

					UniqueIdFromString(OnlineSubsystemName, IdContents);
				}
				else
				{
					bShouldWarn = false;
				}				
			}
		}
	}

	if (bShouldWarn && !IsValid())
	{
#if !NO_LOGGING
		ErrorText->CategorizedLogf(LogNet.GetCategoryName(), ELogVerbosity::Warning, TEXT("Failed to import text to FUniqueNetIdRepl Parent:%s"), *GetPathNameSafe(Parent));
#endif
		return false;
	}

	return true;
}

TSharedRef<FJsonValue> FUniqueNetIdRepl::ToJson() const
{
	if (IsValid())
	{
		const FString JsonString = FString::Printf(TEXT("%s:%s"), *GetType().ToString(), *ToString());
		return MakeShareable(new FJsonValueString(JsonString));
	}
	else
	{
		return MakeShareable(new FJsonValueString(InvalidUniqueNetIdStr));
	}
}

void FUniqueNetIdRepl::FromJson(const FString& Json)
{
	SetUniqueNetId(nullptr);
	if (!Json.IsEmpty())
	{
		TArray<FString> Tokens;

		int32 NumTokens = Json.ParseIntoArray(Tokens, TEXT(":"));
		if (NumTokens == 2)
		{
			UniqueIdFromString(FName(*Tokens[0]), Tokens[1]);
		}
		else if (NumTokens == 1)
		{
			UniqueIdFromString(NAME_None, Tokens[0]);
		}
	}
}

void TestUniqueIdRepl(UWorld* InWorld)
{
#if !UE_BUILD_SHIPPING

#define CHECK_REPL_EQUALITY(IdOne, IdTwo, TheBool) \
	if (!IdOne.IsValid() || !IdTwo.IsValid() || (IdOne != IdTwo) || (*IdOne != *IdTwo.GetUniqueNetId())) \
	{ \
		UE_LOG(LogNet, Warning, TEXT(#IdOne) TEXT(" input %s != ") TEXT(#IdTwo) TEXT(" output %s"), *IdOne.ToString(), *IdTwo.ToString()); \
		TheBool = false; \
	} 

#define CHECK_REPL_VALIDITY(IdOne, TheBool) \
	if (!IdOne.IsValid()) \
	{ \
		UE_LOG(LogNet, Warning, TEXT(#IdOne) TEXT(" is not valid")); \
		TheBool = false; \
	} 

	bool bSetupSuccess = true;

	FUniqueNetIdPtr UserId = UOnlineEngineInterface::Get()->GetUniquePlayerIdWrapper(InWorld, 0).GetUniqueNetId();

	FUniqueNetIdRepl EmptyIdIn;
	if (EmptyIdIn.IsValid())
	{
		UE_LOGF(LogNet, Warning, "EmptyId is valid: %ls", *EmptyIdIn.ToString());
		bSetupSuccess = false;
	}

	FUniqueNetIdRepl ValidIdIn(UserId);
	if (!ValidIdIn.IsValid() || UserId != ValidIdIn.GetUniqueNetId() || *UserId != *ValidIdIn)
	{
		UE_LOGF(LogNet, Warning, "UserId input %ls != UserId output %ls", UserId.IsValid() ? *UserId->ToString() : *InvalidUniqueNetIdStr, *ValidIdIn.ToString());
		bSetupSuccess = false;
	}

	FUniqueNetIdRepl OddStringIdIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("abcde")));
	FUniqueNetIdRepl NonHexStringIdIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("thisisnothex")));
	FUniqueNetIdRepl UpperCaseStringIdIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("abcDEF")));

#if 1
#define WAYTOOLONG TEXT(\
		"deadbeefba5eba11deadbeefba5eba11 \
		deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11 \
		deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11 \
		deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11 \
		deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11 \
		deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11deadbeefba5eba11")
#else
#define WAYTOOLONG TEXT("deadbeef")
#endif

	FUniqueNetIdRepl WayTooLongForHexEncodingIdIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(WAYTOOLONG));

	CHECK_REPL_VALIDITY(OddStringIdIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(NonHexStringIdIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(UpperCaseStringIdIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(WayTooLongForHexEncodingIdIn, bSetupSuccess);

	static FName NAME_CustomOSS("MyCustomOSS");
	FUniqueNetIdRepl CustomOSSIdIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("a8d245fc-4b97-4150-a3cd-c2c91d8fc4b3"), NAME_CustomOSS));
	FUniqueNetIdRepl CustomOSSEncodedIdIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("0123456789abcdef"), NAME_CustomOSS));
	FUniqueNetIdRepl CustomOSSPlusPrefixIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("+123456"), NAME_CustomOSS));
	FUniqueNetIdRepl CustomOSSOddIntegerStringIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("123456789"), NAME_CustomOSS));
	FUniqueNetIdRepl CustomOSSEvenIntegerStringIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("1234567890"), NAME_CustomOSS));
	FUniqueNetIdRepl CustomOSSSeparatorsRequiringQuotedStringIn(UOnlineEngineInterface::Get()->CreateUniquePlayerIdWrapper(TEXT("1234_+_567|890"), NAME_CustomOSS));

	CHECK_REPL_VALIDITY(CustomOSSIdIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(CustomOSSEncodedIdIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(CustomOSSPlusPrefixIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(CustomOSSOddIntegerStringIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(CustomOSSEvenIntegerStringIn, bSetupSuccess);
	CHECK_REPL_VALIDITY(CustomOSSSeparatorsRequiringQuotedStringIn, bSetupSuccess);

	bool bRegularSerializationSuccess = true;
	bool bNetworkSerializationSuccess = true;
	bool bTextItemSerializationSuccess = true;
	if (bSetupSuccess)
	{
		// Regular Serialization (persistent/disk based using FString)
		{
			TArray<uint8> Buffer;
			Buffer.Empty();

			// Serialize In
			{
				FMemoryWriter TestUniqueIdWriter(Buffer, true);

				TestUniqueIdWriter << EmptyIdIn;
				TestUniqueIdWriter << ValidIdIn;
				TestUniqueIdWriter << OddStringIdIn;
				TestUniqueIdWriter << NonHexStringIdIn;
				TestUniqueIdWriter << UpperCaseStringIdIn;
				TestUniqueIdWriter << WayTooLongForHexEncodingIdIn;
				TestUniqueIdWriter << CustomOSSIdIn;
				TestUniqueIdWriter << CustomOSSEncodedIdIn;
				TestUniqueIdWriter << CustomOSSPlusPrefixIn;
				TestUniqueIdWriter << CustomOSSOddIntegerStringIn;
				TestUniqueIdWriter << CustomOSSEvenIntegerStringIn;
				TestUniqueIdWriter << CustomOSSSeparatorsRequiringQuotedStringIn;
			}

			FUniqueNetIdRepl EmptyIdOut;
			FUniqueNetIdRepl ValidIdOut;
			FUniqueNetIdRepl OddStringIdOut;
			FUniqueNetIdRepl NonHexStringIdOut;
			FUniqueNetIdRepl UpperCaseStringIdOut;
			FUniqueNetIdRepl WayTooLongForHexEncodingIdOut;
			FUniqueNetIdRepl CustomOSSIdOut;
			FUniqueNetIdRepl CustomOSSEncodedIdOut;
			FUniqueNetIdRepl CustomOSSPlusPrefixOut;
			FUniqueNetIdRepl CustomOSSOddIntegerStringOut;
			FUniqueNetIdRepl CustomOSSEvenIntegerStringOut;
			FUniqueNetIdRepl CustomOSSSeparatorsRequiringQuotedStringOut;

			// Serialize Out
			{
				FMemoryReader TestUniqueIdReader(Buffer, true);
				TestUniqueIdReader << EmptyIdOut;
				TestUniqueIdReader << ValidIdOut;
				TestUniqueIdReader << OddStringIdOut;
				TestUniqueIdReader << NonHexStringIdOut;
				TestUniqueIdReader << UpperCaseStringIdOut;
				TestUniqueIdReader << WayTooLongForHexEncodingIdOut;
				TestUniqueIdReader << CustomOSSIdOut;
				TestUniqueIdReader << CustomOSSEncodedIdOut;
				TestUniqueIdReader << CustomOSSPlusPrefixOut;
				TestUniqueIdReader << CustomOSSOddIntegerStringOut;
				TestUniqueIdReader << CustomOSSEvenIntegerStringOut;
				TestUniqueIdReader << CustomOSSSeparatorsRequiringQuotedStringOut;
			}

			if (EmptyIdOut.IsValid())
			{
				UE_LOGF(LogNet, Warning, "EmptyId %ls should have been invalid", *EmptyIdOut->ToDebugString());
				bRegularSerializationSuccess = false;
			}

			if (EmptyIdIn != EmptyIdOut)
			{
				UE_LOGF(LogNet, Warning, "EmptyId In/Out mismatch");
				bRegularSerializationSuccess = false;
			}

			CHECK_REPL_EQUALITY(ValidIdIn, ValidIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(OddStringIdIn, OddStringIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(NonHexStringIdIn, NonHexStringIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(UpperCaseStringIdIn, UpperCaseStringIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(WayTooLongForHexEncodingIdIn, WayTooLongForHexEncodingIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(CustomOSSIdIn, CustomOSSIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(CustomOSSEncodedIdIn, CustomOSSEncodedIdOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(CustomOSSPlusPrefixIn, CustomOSSPlusPrefixOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(CustomOSSOddIntegerStringIn, CustomOSSOddIntegerStringOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(CustomOSSEvenIntegerStringIn, CustomOSSEvenIntegerStringOut, bRegularSerializationSuccess);
			CHECK_REPL_EQUALITY(CustomOSSSeparatorsRequiringQuotedStringIn, CustomOSSSeparatorsRequiringQuotedStringOut, bRegularSerializationSuccess);
		}

		// Network serialization (network/transient using MakeReplicationData)
		{
			bool bOutSuccess = false;

			// Serialize In
			FNetBitWriter TestUniqueIdWriter(16 * 1024);
			uint8 EncodingFailures = 0;
			{
				EmptyIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				ValidIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				OddStringIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				NonHexStringIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				UpperCaseStringIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				WayTooLongForHexEncodingIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				CustomOSSIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				CustomOSSEncodedIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				CustomOSSPlusPrefixIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				CustomOSSOddIntegerStringIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				CustomOSSEvenIntegerStringIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
				CustomOSSSeparatorsRequiringQuotedStringIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
			}

			if (EncodingFailures > 0)
			{
				UE_LOGF(LogNet, Warning, "There were %d encoding failures", EncodingFailures);
				bNetworkSerializationSuccess = false;
			}

			if (bNetworkSerializationSuccess)
			{
				FUniqueNetIdRepl EmptyIdOut;
				FUniqueNetIdRepl ValidIdOut;
				FUniqueNetIdRepl OddStringIdOut;
				FUniqueNetIdRepl NonHexStringIdOut;
				FUniqueNetIdRepl UpperCaseStringIdOut;
				FUniqueNetIdRepl WayTooLongForHexEncodingIdOut;
				FUniqueNetIdRepl CustomOSSIdOut;
				FUniqueNetIdRepl CustomOSSEncodedIdOut;
				FUniqueNetIdRepl CustomOSSPlusPrefixOut;
				FUniqueNetIdRepl CustomOSSOddIntegerStringOut;
				FUniqueNetIdRepl CustomOSSEvenIntegerStringOut;
				FUniqueNetIdRepl CustomOSSSeparatorsRequiringQuotedStringOut;

				// Serialize Out
				uint8 DecodingFailures = 0;
				{
					FNetBitReader TestUniqueIdReader(nullptr, TestUniqueIdWriter.GetData(), TestUniqueIdWriter.GetNumBits());

					EmptyIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					ValidIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					OddStringIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					NonHexStringIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					UpperCaseStringIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					WayTooLongForHexEncodingIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					CustomOSSIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					CustomOSSEncodedIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					CustomOSSPlusPrefixOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					CustomOSSOddIntegerStringOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					CustomOSSEvenIntegerStringOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
					CustomOSSSeparatorsRequiringQuotedStringOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
					DecodingFailures += bOutSuccess ? 0 : 1;
				}

				if (DecodingFailures > 0)
				{
					UE_LOGF(LogNet, Warning, "There were %d decoding failures", DecodingFailures);
					bNetworkSerializationSuccess = false;
				}

				if (EmptyIdOut.IsValid())
				{
					UE_LOGF(LogNet, Warning, "EmptyId %ls should have been invalid", *EmptyIdOut->ToDebugString());
					bNetworkSerializationSuccess = false;
				}

				if (EmptyIdIn != EmptyIdOut)
				{
					UE_LOGF(LogNet, Warning, "EmptyId In/Out mismatch");
					bNetworkSerializationSuccess = false;
				}

				CHECK_REPL_EQUALITY(ValidIdIn, ValidIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(OddStringIdIn, OddStringIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(NonHexStringIdIn, NonHexStringIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(UpperCaseStringIdIn, UpperCaseStringIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(WayTooLongForHexEncodingIdIn, WayTooLongForHexEncodingIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSIdIn, CustomOSSIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSEncodedIdIn, CustomOSSEncodedIdOut, bNetworkSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSPlusPrefixIn, CustomOSSPlusPrefixOut, bRegularSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSOddIntegerStringIn, CustomOSSOddIntegerStringOut, bRegularSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSEvenIntegerStringIn, CustomOSSEvenIntegerStringOut, bRegularSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSSeparatorsRequiringQuotedStringIn, CustomOSSSeparatorsRequiringQuotedStringOut, bRegularSerializationSuccess);
			}
		}

		// TextItem serialization
		{
			bool bOutSuccess = false;
			const FUniqueNetIdRepl DefaultValue;

			// Serialize In
			FString EmptyIdTextItem;
			FString ValidIdTextItem;
			FString OddStringIdTextItem;
			FString NonHexStringIdTextItem;
			FString UpperCaseStringIdTextItem;
			FString WayTooLongForHexEncodingIdTextItem;
			FString CustomOSSIdTextItem;
			FString CustomOSSEncodedIdTextItem;
			FString CustomOSSPlusPrefixTextItem;
			FString CustomOSSOddIntegerStringTextItem;
			FString CustomOSSEvenIntegerStringTextItem;
			FString CustomOSSSeparatorsRequiringQuotedStringTextItem;
			uint8 ExportFailures = 0;
			{
				bOutSuccess = EmptyIdIn.ExportTextItem(EmptyIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = ValidIdIn.ExportTextItem(ValidIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = OddStringIdIn.ExportTextItem(OddStringIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = NonHexStringIdIn.ExportTextItem(NonHexStringIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = UpperCaseStringIdIn.ExportTextItem(UpperCaseStringIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = WayTooLongForHexEncodingIdIn.ExportTextItem(WayTooLongForHexEncodingIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = CustomOSSIdIn.ExportTextItem(CustomOSSIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = CustomOSSEncodedIdIn.ExportTextItem(CustomOSSEncodedIdTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = CustomOSSPlusPrefixIn.ExportTextItem(CustomOSSPlusPrefixTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = CustomOSSOddIntegerStringIn.ExportTextItem(CustomOSSOddIntegerStringTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = CustomOSSEvenIntegerStringIn.ExportTextItem(CustomOSSEvenIntegerStringTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
				bOutSuccess = CustomOSSSeparatorsRequiringQuotedStringIn.ExportTextItem(CustomOSSSeparatorsRequiringQuotedStringTextItem, DefaultValue, nullptr, 0, nullptr);
				ExportFailures += bOutSuccess ? 0 : 1;
			}

			if (ExportFailures > 0) //-V547 Expression 'ExportFailures > 0' is always false - ExportTextItem always returns true
			{
				UE_LOGF(LogNet, Warning, "There were %d export failures", ExportFailures);
				bTextItemSerializationSuccess = false;
			}

			if (bTextItemSerializationSuccess)
			{
				FOutputDeviceNull ErrorText;

				FUniqueNetIdRepl EmptyIdOut;
				FUniqueNetIdRepl ValidIdOut;
				FUniqueNetIdRepl OddStringIdOut;
				FUniqueNetIdRepl NonHexStringIdOut;
				FUniqueNetIdRepl UpperCaseStringIdOut;
				FUniqueNetIdRepl WayTooLongForHexEncodingIdOut;
				FUniqueNetIdRepl CustomOSSIdOut;
				FUniqueNetIdRepl CustomOSSEncodedIdOut;
				FUniqueNetIdRepl CustomOSSPlusPrefixOut;
				FUniqueNetIdRepl CustomOSSOddIntegerStringOut;
				FUniqueNetIdRepl CustomOSSEvenIntegerStringOut;
				FUniqueNetIdRepl CustomOSSSeparatorsRequiringQuotedStringOut;

				// Serialize Out
				uint8 ImportFailures = 0;
				const TCHAR* Buffer = nullptr;
				{
					Buffer = *EmptyIdTextItem;
					bOutSuccess = EmptyIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *ValidIdTextItem;
					bOutSuccess = ValidIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *OddStringIdTextItem;
					bOutSuccess = OddStringIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *NonHexStringIdTextItem;
					bOutSuccess = NonHexStringIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *UpperCaseStringIdTextItem;
					bOutSuccess = UpperCaseStringIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *WayTooLongForHexEncodingIdTextItem;
					bOutSuccess = WayTooLongForHexEncodingIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *CustomOSSIdTextItem;
					bOutSuccess = CustomOSSIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *CustomOSSEncodedIdTextItem;
					bOutSuccess = CustomOSSEncodedIdOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *CustomOSSPlusPrefixTextItem;
					bOutSuccess = CustomOSSPlusPrefixOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *CustomOSSOddIntegerStringTextItem;
					bOutSuccess = CustomOSSOddIntegerStringOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *CustomOSSEvenIntegerStringTextItem;
					bOutSuccess = CustomOSSEvenIntegerStringOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;

					Buffer = *CustomOSSSeparatorsRequiringQuotedStringTextItem;
					bOutSuccess = CustomOSSSeparatorsRequiringQuotedStringOut.ImportTextItem(Buffer, 0, nullptr, &ErrorText);
					ImportFailures += bOutSuccess ? 0 : 1;
				}

				if (ImportFailures > 0)
				{
					UE_LOGF(LogNet, Warning, "There were %d import failures", ImportFailures);
					bTextItemSerializationSuccess = false;
				}

				if (EmptyIdOut.IsValid())
				{
					UE_LOGF(LogNet, Warning, "EmptyId %ls should have been invalid", *EmptyIdOut->ToDebugString());
					bTextItemSerializationSuccess = false;
				}

				if (EmptyIdIn != EmptyIdOut)
				{
					UE_LOGF(LogNet, Warning, "EmptyId In/Out mismatch");
					bTextItemSerializationSuccess = false;
				}

				CHECK_REPL_EQUALITY(ValidIdIn, ValidIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(OddStringIdIn, OddStringIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(NonHexStringIdIn, NonHexStringIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(UpperCaseStringIdIn, UpperCaseStringIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(WayTooLongForHexEncodingIdIn, WayTooLongForHexEncodingIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSIdIn, CustomOSSIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSEncodedIdIn, CustomOSSEncodedIdOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSPlusPrefixIn, CustomOSSPlusPrefixOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSOddIntegerStringIn, CustomOSSOddIntegerStringOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSEvenIntegerStringIn, CustomOSSEvenIntegerStringOut, bTextItemSerializationSuccess);
				CHECK_REPL_EQUALITY(CustomOSSSeparatorsRequiringQuotedStringIn, CustomOSSSeparatorsRequiringQuotedStringOut, bTextItemSerializationSuccess);
			}
		}
	}

	bool bPlatformSerializationSuccess = true;
	FString NativePlatformService;
	if (bSetupSuccess && 
		GConfig->GetString(TEXT("OnlineSubsystem"), TEXT("NativePlatformService"), NativePlatformService, GEngineIni) && 
		!NativePlatformService.IsEmpty())
	{
		FUniqueNetIdPtr PlatformUserId = UOnlineEngineInterface::Get()->GetUniquePlayerIdWrapper(InWorld, 0, FName(*NativePlatformService)).GetUniqueNetId();

		FUniqueNetIdRepl ValidPlatformIdIn(PlatformUserId);
		if (!ValidPlatformIdIn.IsValid() || PlatformUserId != ValidPlatformIdIn.GetUniqueNetId() || *PlatformUserId != *ValidPlatformIdIn)
		{
			UE_LOGF(LogNet, Warning, "PlatformUserId input %ls != PlatformUserId output %ls", PlatformUserId.IsValid() ? *PlatformUserId->ToString() : *InvalidUniqueNetIdStr, *ValidPlatformIdIn.ToString());
			bPlatformSerializationSuccess = false;
		}

		if (bPlatformSerializationSuccess)
		{
			bool bOutSuccess = false;

			TArray<uint8> Buffer;
			Buffer.Empty();

			FMemoryWriter TestUniqueIdWriter(Buffer);

			// Serialize In
			uint8 EncodingFailures = 0;
			{
				ValidPlatformIdIn.NetSerialize(TestUniqueIdWriter, nullptr, bOutSuccess);
				EncodingFailures += bOutSuccess ? 0 : 1;
			}

			if (EncodingFailures > 0)
			{
				UE_LOGF(LogNet, Warning, "There were %d platform encoding failures", EncodingFailures);
				bPlatformSerializationSuccess = false;
			}

			FUniqueNetIdRepl ValidPlatformIdOut;

			// Serialize Out
			uint8 DecodingFailures = 0;
			{
				FMemoryReader TestUniqueIdReader(Buffer);

				ValidPlatformIdOut.NetSerialize(TestUniqueIdReader, nullptr, bOutSuccess);
				DecodingFailures += bOutSuccess ? 0 : 1;
			}

			if (DecodingFailures > 0)
			{
				UE_LOGF(LogNet, Warning, "There were %d platform decoding failures", DecodingFailures);
				bPlatformSerializationSuccess = false;
			}

			CHECK_REPL_EQUALITY(ValidPlatformIdIn, ValidPlatformIdOut, bPlatformSerializationSuccess);
		}
	}

	bool bJSONSerializationSuccess = true;
	if (bSetupSuccess)
	{
		// JSON Serialization
		FString OutString;
		TSharedRef<FJsonValue> JsonValue = ValidIdIn.ToJson();
		bJSONSerializationSuccess = JsonValue->TryGetString(OutString);
		if (bJSONSerializationSuccess)
		{
			FUniqueNetIdRepl NewIdOut;
			NewIdOut.FromJson(OutString);
			bJSONSerializationSuccess = NewIdOut.IsValid() && (ValidIdIn == NewIdOut);
		}
	}

	UE_LOGF(LogNet, Log, "TestUniqueIdRepl tests:");
	UE_LOGF(LogNet, Log, "	Setup: %ls", bSetupSuccess ? TEXT("PASS") : TEXT("FAIL"));
	UE_LOGF(LogNet, Log, "	Normal: %ls", bRegularSerializationSuccess ? (bSetupSuccess ? TEXT("PASS") : TEXT("SKIPPED")) : TEXT("FAIL"));
	UE_LOGF(LogNet, Log, "	Network: %ls", bNetworkSerializationSuccess ? (bSetupSuccess ? TEXT("PASS") : TEXT("SKIPPED")) : TEXT("FAIL"));
	UE_LOGF(LogNet, Log, "	TextItem: %ls", bTextItemSerializationSuccess ? (bSetupSuccess ? TEXT("PASS") : TEXT("SKIPPED")) : TEXT("FAIL"));
	UE_LOGF(LogNet, Log, "	Platform: %ls", bPlatformSerializationSuccess ? (bSetupSuccess ? TEXT("PASS") : TEXT("SKIPPED")) : TEXT("FAIL"));
	UE_LOGF(LogNet, Log, "	JSON: %ls", bJSONSerializationSuccess ? (bSetupSuccess ? TEXT("PASS") : TEXT("SKIPPED")) : TEXT("FAIL"));

#undef CHECK_REPL_VALIDITY
#undef CHECK_REPL_EQUALITY

#endif
}

