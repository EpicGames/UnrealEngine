// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OnlineSession.cpp: Online session related implementations
	(creating/joining/leaving/destroying sessions)
=============================================================================*/

#include "GameFramework/OnlineSession.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnlineSession)

UOnlineSession::UOnlineSession(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UOnlineSession::HandleDisconnect(UWorld *World, UNetDriver *NetDriver)
{
	// Let the engine cleanup this disconnect
	GEngine->HandleDisconnect(World, NetDriver);
}

