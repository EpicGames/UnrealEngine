// Copyright Epic Games, Inc. All Rights Reserved.

#include "OverriddenObjectsExternalPackageHelpers.h"

#include "Algo/RemoveIf.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/OverridableManager.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#if WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogExternalOverrides, Log, All);

FExternalObjectsOverrideHelper::FExternalObjectsOverrideHelper(UObject* ForOuter, UClass* InDesiredClass, FName InExternalsPropertyName)
	: Outer(ForOuter)
	, DesiredClass(InDesiredClass)
	, ExternalsPropertyName(InExternalsPropertyName)
{
	// will be enabled once all current usages are removed
	//ensureMsgf(Outer->GetPackage() != GetTransientPackage(), TEXT("Trying to build a FExternalObjectsOverrideHelper with an outer (%s) in the transient package, this is most likely an error and will not yield any result"), *Outer->GetFullName());
}

void FExternalObjectsOverrideHelper::GetOverridesFromExternalObjects()	
{
	if (bExternalAssetsScanned)
	{
		return;
	}

	GetOverridesFromExternalObjects(ExternalAssets);

	bExternalAssetsScanned = true;
}


void FExternalObjectsOverrideHelper::GetOverridesFromExternalObjects(TArray<FExternalAsset>& InExternalAssets, bool bOnlyDiskAssets /*= true */) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalObjectsOverrideHelper::GetOverridesFromExternalObjects);

	const UPackage* OuterObjectPackage = Outer->GetPackage();
	const FName OuterObjectPackageNameInMemory = OuterObjectPackage->GetFName();
	const FName OuterObjectPackageNameOnDisk = !OuterObjectPackage->GetLoadedPath().IsEmpty() ? OuterObjectPackage->GetLoadedPath().GetPackageFName() : OuterObjectPackageNameInMemory;
	const bool bIsInstanced = OuterObjectPackageNameInMemory != OuterObjectPackageNameOnDisk;

	FString OuterPath = Outer->GetPathName();
	FName Path;
		
	if (bOnlyDiskAssets)
	{	
		if (bIsInstanced)
		{
			// Replace the expected outer path to the one on disk for loading and filtering assets
			// Note at this point, OuterObjectPackageNameInMemory is not guaranteed to be a part of the path since 
			// externalization and instancing can happen anywhere so we reconstruct the outer path as it was saved 
			// on disk to be able to correctly filter the objects we need to load. 
			UObject* OuterMostObject = Outer->GetOutermostObject();
			OuterPath = FSoftObjectPath::ConstructFromAssetPathAndSubpath(FTopLevelAssetPath(OuterMostObject->GetPackage()->GetLoadedPath().GetPackageFName(), OuterMostObject->GetFName()), Outer->GetPathName(OuterMostObject)).ToString();
		}

		Path = OuterObjectPackageNameOnDisk;
	}
	else
	{
		Path = OuterObjectPackageNameInMemory;
	}
		
	const bool bIsTempPackage = FPackageName::IsTempPackage(Path.ToString());

	FLinkerLoad* OuterLinker = Outer->GetLinker();
	const FLinkerInstancingContext* OuterInstancingContext = &OuterLinker->GetInstancingContext();

	EAssetsWithOuterForPathsFlags OuterForPathsFlags = (EAssetsWithOuterForPathsFlags::RecursivePaths | ((bOnlyDiskAssets) ? EAssetsWithOuterForPathsFlags::IncludeOnlyOnDiskAsset : EAssetsWithOuterForPathsFlags::None) | EAssetsWithOuterForPathsFlags::ExactOuter | ((!bIsTempPackage) ? EAssetsWithOuterForPathsFlags::ScanPaths : EAssetsWithOuterForPathsFlags::None ));
	if (TArray<FAssetData> Assets = UAssetRegistryHelpers::GetAssetsWithOuterForPaths({Path}, *OuterPath, OuterForPathsFlags); !Assets.IsEmpty())
	{
		Assets.SetNum(Algo::RemoveIf(Assets, [](const FAssetData& InAssetData) { return InAssetData.IsRedirector(); }));
		
		for (const FAssetData& Asset : Assets)
		{
			FExternalAsset ExternalAsset;
				
			ExternalAsset.ObjectPackageName = Asset.PackageName;
			ExternalAsset.AssetName = Asset.AssetName;
			ExternalAsset.OverrideOperation = EOverriddenPropertyOperation::None;

			FString	ExternalOverridePropertyValue;
			if (Asset.GetTagValue(FName(TEXT("ExternalOverrideProperty")), ExternalOverridePropertyValue))
			{
				if (ExternalOverridePropertyValue != ExternalsPropertyName)
				{
					// disregard this external since it doesn't match the property we're loading externals for right now
					continue;
				}
			}
			else if (bConsiderExternalsWithoutPropertyTagsAsAdded)
			{
				// external packages objects that do not have this property set and are considered adds
				ExternalAsset.OverrideOperation = EOverriddenPropertyOperation::Add;
			}
				
			if (Asset.FindTag(FName(TEXT("OverrideAdded"))))
			{	
				ExternalAsset.OverrideOperation = EOverriddenPropertyOperation::Add;
			}				
			else if (Asset.FindTag(FName(TEXT("OverrideReplaced"))))
			{
				ExternalAsset.OverrideOperation = EOverriddenPropertyOperation::Replace;
			}
			else if (Asset.GetTagValue(FName(TEXT("OverrideRemoved")), ExternalAsset.RemovedObjectDefaultValue))
			{
				ExternalAsset.OverrideOperation = EOverriddenPropertyOperation::Remove;
				ExternalAsset.AssetObject = FSoftObjectPtr(FSoftObjectPath(ExternalAsset.RemovedObjectDefaultValue.ToString())).Get();
			}
			
			ExternalAsset.AssetPath = Asset.ToSoftObjectPath();

			if (bIsInstanced)
			{
				// remap asset path to instanced path
				OuterInstancingContext->FixupSoftObjectPath(ExternalAsset.AssetPath);
			}

			if (!ExternalAsset.AssetObject.IsValid())
			{
				ExternalAsset.AssetObject = TSoftObjectPtr<UObject>(ExternalAsset.AssetPath).Get();
			}
			InExternalAssets.Add(ExternalAsset);
		}
	}
}

void FExternalObjectsOverrideHelper::LoadExternalObjects(bool bFlushLoad)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalObjectsOverrideHelper::LoadExternalObjects);

	GetOverridesFromExternalObjects();

	const UPackage* OuterObjectPackage = Outer->GetPackage();
	const FName OuterObjectPackageNameInMemory = OuterObjectPackage->GetFName();
	const FName OuterObjectPackageNameOnDisk = !OuterObjectPackage->GetLoadedPath().IsEmpty() ? OuterObjectPackage->GetLoadedPath().GetPackageFName() : OuterObjectPackageNameInMemory;
	const bool bIsInstanced = OuterObjectPackageNameInMemory != OuterObjectPackageNameOnDisk;		
	FLinkerInstancingContext InstancingContext;
	const FLinkerInstancingContext* OuterInstancingContext = nullptr;

	PackagesAsyncLoadRequests.Reserve(ExternalAssets.Num());
	const uint32 LoadFlags = (Outer->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor) ? LOAD_PackageForPIE : LOAD_None) | LOAD_FindIfFail;

	for (FExternalAsset& ExternalAsset : ExternalAssets)
	{
		// No need to generate a load request if the object exists
		// This should always be the case since those are requested through dynamic imports
		if (ExternalAsset.AssetObject.IsValid())
		{
			continue;
		}

		// Before issuing a load, try to locate the export in the package and resolve it's resource there's a high likelyhood
		// this will succeed. 
		UE_LOGF(LogExternalOverrides, Verbose, "Attempting to locate the object resource in linker for %ls in package: %ls, for outer: %ls", *ExternalAsset.AssetName.ToString(), *ExternalAsset.ObjectPackageName.ToString(), *Outer->GetFullName());

		if (UPackage* Package = FindPackage(nullptr, *ExternalAsset.ObjectPackageName.ToString()))
		{
			FLinkerLoad* Linker = Package->GetLinker();

			auto FindIndicesFor = [Linker](FName Name) -> TArray<FPackageIndex>
			{
				TArray<FPackageIndex> Indices;

				for (FObjectExport& Export : Linker->ExportMap)
				{
					if (Export.ObjectName == Name)
					{
						Indices.Add(Export.ThisIndex);
					}
				}

				return Indices;
			};			

			// find possible exports matching the asset name (might be more than on we then need to
			// follow the outer chain back to validate which one is the export we want)
			TArray<FPackageIndex> Indices = FindIndicesFor(ExternalAsset.AssetName);

			// if we found possible exports
			if (Indices.Num())
			{
				// split the asset's outer chain into a list of FNames so that we can check each object individually
				TArray<FString> OuterChain;
				ExternalAsset.AssetPath.GetSubPathString().ParseIntoArray(OuterChain, TEXT("."));

				// for each export 
				for (FPackageIndex Index : Indices)
				{
					FPackageIndex OuterIndex = Index;
					const FObjectResource* Obj = nullptr;
					int32 OuterChainIndex = OuterChain.Num()-1;

					do 
					{
						Obj = &Linker->ImpExp(OuterIndex);
						OuterIndex = Obj->OuterIndex;
						OuterChainIndex --;
					} while (OuterChainIndex >= 0 && Obj->ObjectName.ToString() == OuterChain[OuterChainIndex+1] && !OuterIndex.IsNull());

					if (OuterChainIndex < 0)
					{
						// we walked up all the way, so we have the correct object
						UE_LOGF(LogExternalOverrides, Verbose, "Attempting to ResolveResource for %ls", *ExternalAsset.AssetName.ToString());

						if (UObject* Object = Package->GetLinker()->ResolveResource(Index))
						{
							ExternalAsset.AssetObject = Object;
						}

						break;
					}
				}
			}

			// reverify if we manage to find the object
			if (ExternalAsset.AssetObject.IsValid())
			{
				continue;
			}
		}
		
		UE_LOGF(LogExternalOverrides, Warning, "Force loading an external object outside of dynamic imports: %ls", *ExternalAsset.ObjectPackageName.ToString());

		FName InstancedName;
		FName ObjectPackageName = ExternalAsset.ObjectPackageName;
			
		if (bIsInstanced)
		{
			if (OuterInstancingContext)
			{
				InstancedName = OuterInstancingContext->RemapPackage(ObjectPackageName);
			}

			// Remap to the a instanced package if it wasn't remapped already by the outer instancing context
			if (InstancedName == ObjectPackageName || InstancedName.IsNone())
			{
				InstancedName = *FLinkerInstancingContext::GetInstancedPackageName(OuterObjectPackageNameInMemory.ToString(), ObjectPackageName.ToString());
			}

			checkf(!InstancedName.IsNone(), TEXT("Could find not the remapped package name for dynamic import package %s, in memory name %s"), *ObjectPackageName.ToString(), *OuterObjectPackageNameInMemory.ToString());

			InstancingContext.AddPackageMapping(ObjectPackageName, InstancedName);
		}
		
		FLoadPackageAsyncOptionalParams OptionalParams
		{
			.CustomPackageName = InstancedName,
			.PackagePriority = INT32_MAX,
			.InstancingContext = &InstancingContext,
			.LoadFlags = LoadFlags
		};

		const int32 RequestID = LoadPackageAsync(ObjectPackageName.ToString(), MoveTemp(OptionalParams));

		if (RequestID != INDEX_NONE )
		{
			PackagesAsyncLoadRequests.Add(RequestID);
		}
	}	
	
	if (bFlushLoad)
	{
		Flush();
	}
}

void FExternalObjectsOverrideHelper::Flush()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalObjectsOverrideHelper::Flush);

	if (PackagesAsyncLoadRequests.Num())
	{			
		FlushAsyncLoading(PackagesAsyncLoadRequests);
	}
	
	for (FExternalAsset& ExternalAsset : ExternalAssets)
	{		
		if (!ExternalAsset.AssetObject.IsValid())
		{
			// Locate the UPackage and the external assets in it
			if (UPackage* LoadedPackage = FindPackage(nullptr, *ExternalAsset.ObjectPackageName.ToString()))
			{
				ForEachObjectWithPackage(LoadedPackage, [&ExternalAsset, this](UObject* InPackageObject)
					{
						if (InPackageObject->IsA(DesiredClass) && InPackageObject->GetFName() == ExternalAsset.AssetName)
						{
							ExternalAsset.AssetObject = InPackageObject;
						}
						return true;
					}, EGetObjectsFlags::None);
			}
		}
	}
}

void FExternalObjectsOverrideHelper::SetConsiderExternalsWithoutPropertyTagsAsAdded(bool bValue)
{
	bConsiderExternalsWithoutPropertyTagsAsAdded = bValue;
}

void FExternalObjectsOverrideHelper::HandleRemove(UObject* RemovedObject, UObject* RemovedObjectArchetype)
{	
	if (!RemovedObjectArchetype)
	{
		// Removed objects lacking an archetypes are adds so we just remove the external file no need to keep a remove placeholder
		checkf(RemovedObject && !RemovedObject->HasAllFlags(RF_Transient), TEXT("FExternalObjectsOverrideHelper::HandleRemove called without a valid RemovedObject"));

		RemoveExternalObjectPackage(RemovedObject, true);
		return;
	}

	CreateExternalRemoveObjectPackage(RemovedObjectArchetype);
}

void FExternalObjectsOverrideHelper::HandleRename(UObject* Object)
{
	// Update all dependent external packages names
	ForEachObjectWithOuter(Object, [this](UObject* Inner)
	{
		if (Inner->IsPackageExternal())
		{
			if (Inner->IsA(DesiredClass) || Inner->IsA<UOverrideRemovePlaceholder>() )
			{
				UPackage* Package = Inner->GetPackage();
				UObject* InnerOuter = Inner->GetOuter();

				if (InnerOuter->IsA(DesiredClass))
				{
					const FString InnerPackagePathName = Package->GetPathName();
					const FString InnerName = *Inner->GetName();
					const FString OuterPackageName = GetPackageName(InnerOuter);
				
					const FString UpdatedPackageName = FPaths::Combine(OuterPackageName, InnerName);
					if (UpdatedPackageName != InnerPackagePathName)
					{
						Package->Rename(*UpdatedPackageName, nullptr, 0);
					}
				}
			}
		}
	});
}


void FExternalObjectsOverrideHelper::CreateExternalRemoveObjectPackage(UObject* RemovedObjectArchetype)
{	
	// Create package & dummy objects for removed objects + add the proper tag
	// 
	// Removed external objects are indicated by a package which contains 
	// a UOverrideRemovePlaceholder to preserve the remove override.
	// 
	// The package also contains an "OverrideRemoved" tag with the name of the 
	// remove object archetype
	
	bool bIsArchetype = Outer->HasAnyFlags(RF_ArchetypeObject);
				
	// determine package name	
	FString OuterPackageName = GetPackageName(Outer);
	FString RemovedObjectName = RemovedObjectArchetype->GetName() + TEXT("_removed");
	FString RemovedPackageName = OuterPackageName / RemovedObjectName;

	UE_LOGF(LogExternalOverrides, Verbose, "Creating external removed placholder object of achetype %ls in %ls", *RemovedObjectArchetype->GetName(), *Outer->GetName());

	// ensure package exists, this only works in memory... scan on disk too?
	UPackage* Package = FindPackage(nullptr, *RemovedPackageName);

	if (!Package)
	{
		Package = CreatePackage(*RemovedPackageName);
		if (!Outer->IsTemplate() || Outer->GetPackage()->HasAnyPackageFlags(PKG_ContainsMapData))
		{
			Package->SetPackageFlags(PKG_ContainsMapData);
		}
		// avoid reloading the package if one existing on disk in that location
		Package->MarkAsFullyLoaded();
	}			

	// ensure we have the correct placeholder
	FSoftObjectPath RemovedPath(RemovedObjectArchetype);
	TArray<UObject*> ObjectsToTrash;

	bool bNeedCreatePlaceHolder = true;

	if (Package)
	{
		ForEachObjectWithPackage(Package, [&bNeedCreatePlaceHolder, &RemovedPath, &ObjectsToTrash, this, Package](UObject* Object)
		{
			UOverrideRemovePlaceholder* Placeholder = Cast<UOverrideRemovePlaceholder>(Object);

			if (Placeholder && Placeholder->OverrideRemoved == RemovedPath && Placeholder->GetOuter() == Outer && Placeholder->OverrideProperty == ExternalsPropertyName)
			{
				bNeedCreatePlaceHolder = false;
			}
			else if (Object->GetExternalPackage() == Package ||
					 Object->GetOuter() == Package)
			{
				ObjectsToTrash.Add(Object);
			}

			return true;
		});
	}
		
	for (UObject* Object : ObjectsToTrash)
	{
		// thrash this object
		Package->SetDirtyFlag(true);
		Object->SetExternalPackage(nullptr);
		if (Object->GetOuter() == Package )
		{
			// To ensure a clean package for the remove placeholder we need to move that object out 
			// of the package if any is left. This should not normally happen so we log a warning 
			// help diagnose when it happens. 
			if (!Object->IsA<UOverrideRemovePlaceholder>())
			{
				UE_LOGF(LogExternalOverrides, Warning, "Outering %ls to Transient package so that we create a clean remove placeholer package", *Object->GetFullName());
			}
			Object->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		}
	}
	
	if (bNeedCreatePlaceHolder)
	{	
		UOverrideRemovePlaceholder* RemovedPlaceholder = NewObject<UOverrideRemovePlaceholder>(Outer, FName(RemovedObjectName));
		RemovedPlaceholder->SetExternalPackage(Package);
		RemovedPlaceholder->SetFlags(RF_Standalone | RF_Public| RF_Transactional);
		if (bIsArchetype)
		{
			RemovedPlaceholder->SetFlags(RF_ArchetypeObject);
		}
		RemovedPlaceholder->OverrideRemoved = RemovedPath;
		RemovedPlaceholder->OverrideProperty = ExternalsPropertyName.ToString();
		FAssetRegistryModule::AssetCreated(RemovedPlaceholder);

		Package->SetDirtyFlag(true); //only necessary if we're reusing a package with a different removed name 
	}
}

void FExternalObjectsOverrideHelper::HandleAdd(UObject* AddedObject)
{
	// done in rename but might be moved here eventually
}


void FExternalObjectsOverrideHelper::RemoveExternalObjectPackage(UObject* External, bool bCanDirtyPackage)
{
	if (External->IsPackageExternal())
	{
		checkf(!External->HasAllFlags(RF_Transient), TEXT("FExternalObjectsOverrideHelper::RemoveExternalObjectPackage called on an RF_Transient object"));
		RemoveExternalObjectPackage(External->GetPackage()->GetName(), External->GetFName(), External->IsA<UOverrideRemovePlaceholder>(), bCanDirtyPackage);
	}
}

void FExternalObjectsOverrideHelper::RemoveExternalObjectPackage(FString ExternalAssetPackageName, FName AssetName, bool bForAnExternalRemove, bool bCanDirtyPackage)
{
	UPackage* Package = FindPackage(nullptr, *ExternalAssetPackageName);
	bool bNeedCreateDummyObject = true;

	FName DummyObjectName = AssetName;
	UObject* DummyObjectOuter = GetOuterForRemovePlaceholder(Outer, Outer);

	bool bPackageWasNewlyCreated = false;
	// Create package instead of loading an object we don't need
	if (!Package)
	{
		Package = CreatePackage(*ExternalAssetPackageName);
		UPackage* OuterPackage = Outer->GetPackage();
		if (!Outer->IsTemplate() || OuterPackage->HasAnyPackageFlags(PKG_ContainsMapData)) 
		{
			Package->SetPackageFlags(PKG_ContainsMapData);
		}
	}
	else
	{
		bPackageWasNewlyCreated = Package->HasAllPackagesFlags(PKG_NewlyCreated); // newly created packages aren't on disk, so there's no need to create the dummy object in them, they'll just get GCed if we empty them, so we take not of this state for latter
		ResetLinkerExports(Package);
		
		TArray<UObject*> ObjectsToMoveOutOfPackage;

		// make sure package is "empty" (contains a appropriately named transient object to that it gets deleted)
		ForEachObjectWithPackage(Package, [&ObjectsToMoveOutOfPackage, DummyObjectName, DummyObjectOuter, &bNeedCreateDummyObject, Package](UObject* Object)
		{
			if (Object->GetName() == DummyObjectName && 
				Object->GetOuter() == DummyObjectOuter && 
				Object->HasAllFlags(RF_Standalone | RF_Public |RF_Transient|RF_Transactional))
			{
				bNeedCreateDummyObject = false;
			}
			else if (Object->GetExternalPackage() == Package ||
					 Object->GetOuter() == Package)
			{
				ObjectsToMoveOutOfPackage.Add(Object);
			}

			return true;
		});
		
		for (UObject* Object : ObjectsToMoveOutOfPackage)
		{
			Package->SetDirtyFlag(true);
			Object->SetExternalPackage(nullptr);

			if (Object->GetOuter() == Package || bForAnExternalRemove)
			{			
				// We need to move that object out of the package for the package to be removed from disk. 
				// It's expected that UOverrideRemovePlaceholder objects to be present and they need to be 
				// outered to transient (they're not necessary for normal override remove functionality so
				// we don't keep them)
				if (!bForAnExternalRemove)
				{
					UE_LOGF(LogExternalOverrides, Warning, "Outering %ls to Transient package so that we can delete it's package", *Object->GetFullName());
				}
				Object->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors); 
			}			
		}
	}	
				
	if (bNeedCreateDummyObject)
	{
		if(!bPackageWasNewlyCreated)
		{
			UObject* DummyObject = nullptr;

			// Objects is outered on the object that'll proxy it's save that it appears in the correct save contexts
			if (bForAnExternalRemove)
			{
				DummyObject = NewObject<UOverrideRemovePlaceholder>(DummyObjectOuter, FName(DummyObjectName));
			}
			else
			{	
				DummyObject = NewObject<UObject>(DummyObjectOuter, DesiredClass, FName(DummyObjectName));
			}
			DummyObject->SetExternalPackage(Package);
			DummyObject->SetFlags(RF_Standalone | RF_Public |RF_Transient|RF_Transactional);
		
			Package->SetDirtyFlag(true);
		}
		else
		{
			Package->SetDirtyFlag(false);
			Package->MarkAsGarbage();
		}
	}		
}

void FExternalObjectsOverrideHelper::CreateExternalObjectPackage(UObject* Object, bool bCanDirtyPackage)
{	
	if (Object->IsPackageExternal())
	{
		ensure(GetPackageName(Object) == Object->GetPackage()->GetName());
		return;
	}

	if (Object->HasAllFlags(RF_Transient))
	{
		return;
	}
	
	const bool bShouldDirty = bCanDirtyPackage;
	const bool bWasAsset = Object->IsAsset();

	Object->Modify(bShouldDirty);
	checkf(Object->GetOuter() == Outer, TEXT("FExternalObjectsOverrideHelper incorrectly setup for CreateExternalObjectPackage, expected object to be outered to %s, instead is outered to %s"), *Outer->GetFullName(), *Object->GetOuter()->GetFullName());
	checkf(FPropertyArrayExternalObjectHandler::Get().IsSavingExternal(Outer, Outer->GetClass()->FindPropertyByName(ExternalsPropertyName)), TEXT("Trying to create an external packager for Outer %s which doesn't use external packages"), *Outer->GetFullName());
	
	FString PackageName = GetPackageName(Object);
	UPackage* Package = CreatePackage(*PackageName);
	bool bForTemplate = Object->IsTemplate();

	if (ensureMsgf(Package, TEXT("CreatePackage returned null")))
	{
		Package->SetPackageFlags(PKG_EditorOnly | (bForTemplate ? 0 : PKG_ContainsMapData) | PKG_NewlyCreated);
		if (bShouldDirty)
		{
			Package->MarkPackageDirty();
		}
	}		

	Object->SetExternalPackage(Package);
	
	if (Object->IsAsset() && !bWasAsset)
	{
		FAssetRegistryModule::AssetCreated(Object);
	}

	if (bShouldDirty)
	{
		Object->MarkPackageDirty();
	}
}


void FExternalObjectsOverrideHelper::RemoveExternalPackages(bool bCanDirtyPackage)
{	
	TArray<FExternalAsset> CurrentExternalAssets;
	GetOverridesFromExternalObjects(CurrentExternalAssets, false);

	for (FExternalAsset& Asset : CurrentExternalAssets)
	{
		RemoveExternalObjectPackage(Asset.ObjectPackageName.ToString(), Asset.AssetName, Asset.OverrideOperation == EOverriddenPropertyOperation::Remove, bCanDirtyPackage);
	}
}

void FExternalObjectsOverrideHelper::CreateExternalPackages(bool bCanDirtyPackages)
{
	FOverridableObjectArrayOverrides Overrides;	
	InitializeObjectArrayOverridesFromObject(&Overrides, Outer->GetClass()->FindPropertyByName(ExternalsPropertyName), Outer);

	for (UObject* Added : Overrides.Added)
	{
		CreateExternalObjectPackage(Added, bCanDirtyPackages);
	}

	for (UObject* RemovedArchetype : Overrides.RemovedArchetypes)
	{
		CreateExternalRemoveObjectPackage(RemovedArchetype);
	}

	for (UObject* Object : Overrides.Objects)
	{
		CreateExternalObjectPackage(Object, bCanDirtyPackages);
	}
}

void FExternalObjectsOverrideHelper::InitializeObjectArrayOverridesFromExternalAssets(TNonNullPtr<FOverridableObjectArrayOverrides> ArrayOverrides, TArray<FExternalAsset>& FromExternalAssets, bool bLogInvalidAssets)
{		
	for (const FExternalAsset& ExternalAsset : FromExternalAssets)
	{
		switch (ExternalAsset.OverrideOperation)
		{
			case EOverriddenPropertyOperation::Add:
			{
				if (ExternalAsset.AssetObject.IsValid())
				{
					ArrayOverrides->Added.Add(ExternalAsset.AssetObject.Get());
				}
				else
				{
					if (bLogInvalidAssets)
					{
						UE_LOGF(LogExternalOverrides, Warning, "Could not load external asset %ls, it'll not be Added", *ExternalAsset.ObjectPackageName.ToString());
					}
				}
			}
			break;
			case EOverriddenPropertyOperation::Remove:
			{
				if (ExternalAsset.AssetObject.IsValid())
				{
					ArrayOverrides->RemovedArchetypes.Add(ExternalAsset.AssetObject.Get());
				}
				else
				{
					if (bLogInvalidAssets)
					{
						UE_LOGF(LogExternalOverrides, Warning, "Could not load external asset %ls, %ls will not be Removed", *ExternalAsset.ObjectPackageName.ToString(), *ExternalAsset.RemovedObjectDefaultValue.ToString());
					}
				}

			}
			break;
			default:
			{
				if (ExternalAsset.AssetObject.IsValid())
				{
					ArrayOverrides->Objects.Add(ExternalAsset.AssetObject.Get());
					ArrayOverrides->ObjectsOperation = ExternalAsset.OverrideOperation;
				}
				else
				{
					if (bLogInvalidAssets)
					{
						UE_LOGF(LogExternalOverrides, Warning, "Could not load external asset %ls, it's Modifications will be lost", *ExternalAsset.ObjectPackageName.ToString());
					}
				}
			}
		}
	}
}

void FExternalObjectsOverrideHelper::InitializeObjectArrayOverridesFromObject(TNonNullPtr<FOverridableObjectArrayOverrides> ArrayOverrides, TNonNullPtr<FProperty> Property, TNonNullPtr<UObject> Object)
{
	// Find the corresponding overrides
	if (const FOverriddenPropertySet* OverriddenSet = FOverridableManager::Get().GetOverriddenProperties(Object))
	{
		// Look the override node associated with the property
		FOverriddenPropertyNodeID OverriddenID(Property);

		const FOverriddenPropertyNode* Node = OverriddenSet->GetOverriddenPropertyNode(OverriddenID);
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FObjectProperty* InnerObjectProperty =  CastField<FObjectProperty>(ArrayProperty->GetInnerFieldByName(*ArrayProperty->GetName()));
				
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Outer));

		const int32 ArrayNum = ArrayHelper.Num();
		for (int i = 0; i < ArrayNum; ++i)
		{
			UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetElementPtr(i));
			ArrayOverrides->Objects.Add(CurrentObject);
		}
			
		if (Node)
		{			
			if(Node->GetOperation() == EOverriddenPropertyOperation::Replace)
			{
				ArrayOverrides->ObjectsOperation = EOverriddenPropertyOperation::Replace;
			}			

			FScriptArrayHelper DefaultsArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Outer->GetArchetype()));

			auto FindObjectPtr = [&InnerObjectProperty](const FOverriddenPropertyNodeID& ObjectToFind, FScriptArrayHelper& Helper) -> UObject*
			{
				const int32 ArrayNum = Helper.Num();

				for (int i = 0; i < ArrayNum; ++i)
				{
					if (UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(Helper.GetElementPtr(i)))
					{
						if (ObjectToFind == FOverriddenPropertyNodeID(CurrentObject))
						{
							return CurrentObject;
						}
					}
				}

				return nullptr;
			};

			for (const FOverriddenPropertyNode& SubNode : Node->GetSubPropertyNodes())
			{
				if (SubNode.GetOperation() == EOverriddenPropertyOperation::Remove)
				{
					UObject* FoundObject = FindObjectPtr(SubNode.GetNodeID(), DefaultsArrayHelper);
						
					if (FoundObject)
					{
						ArrayOverrides->RemovedArchetypes.Add(FoundObject);
					}
				}
				else if (SubNode.GetOperation() == EOverriddenPropertyOperation::Add)
				{					
					UObject* FoundObject = FindObjectPtr(SubNode.GetNodeID(), ArrayHelper); 
						
					if (FoundObject)
					{
						ArrayOverrides->Added.Add(FoundObject);
						ArrayOverrides->Objects.Remove(FoundObject);
					}
				}
			}
		}
	}	
}

void FExternalObjectsOverrideHelper::GetAssetRegistryTags(const UObject* ObjectInOuter, FAssetRegistryTagsContext Context)
{	
	// ensure we're in the correct outer
	checkf(ObjectInOuter->GetOuter() == Outer, TEXT("FExternalObjectsOverrideHelper incorrectly setup for GetAssetRegistryTags, expected object to be outered to %s, instead is outered to %s"), *Outer->GetFullName(), *ObjectInOuter->GetOuter()->GetFullName());
	
	if (ObjectInOuter->IsPackageExternal())
	{
		// Using entities as an example....
		// An entity is added to the parent if the archetype of its outer is 
		// different than the outer of its archetype
		// 
		// Example with entities......
		// 
		// PrefabA			=>	PrefabA'
		//		EntityA				EntityA'				Outer(Archetype(EntityA')) == Archetype(Outer(EntityA'))
		//							EntityC (added)			Outer(Archetype(EntityC)) != Archetype(Outer(EntityC))			=> Outer(entity) != Archetype(PrefabA')
		//							PrefabB' (addded)		Outer(Archetype(PrefabB')) != Archetype(Outer(PrefabA))			=> Outer(PrefabB) != Archetype(BlueprintA)
		//								EntityB'			Outer(Archetype(EntityB')) == Archetype(Outer(EntityB'))		=> Outer(EntityB) == Archetype(PrefabB')
		// PrefabB
		//		EntityB 
		//
	
		if (Outer->GetArchetype() != ObjectInOuter->GetArchetype()->GetOuter())
		{
			Context.AddTag(UObject::FAssetRegistryTag(TEXT("OverrideAdded"), TEXT("Added"), UObject::FAssetRegistryTag::TT_Alphabetical));
		}
		else if (Outer->IsA(DesiredClass))
		{
			if (FOverriddenPropertySet* ParentOwnedPropertyOverrideSet = FOverridableManager::Get().GetOverriddenProperties(Outer))
			{
				if (FProperty* ForProperty = DesiredClass->FindPropertyByName(ExternalsPropertyName))
				{
					FOverriddenPropertyNodeID OverriddenID(ForProperty);
					const FOverriddenPropertyNode* Node = ParentOwnedPropertyOverrideSet->GetOverriddenPropertyNode(OverriddenID);

					if (Node)
					{
						if (Node->GetOperation() ==  EOverriddenPropertyOperation::Replace)
						{
							Context.AddTag(UObject::FAssetRegistryTag(TEXT("OverrideReplaced"), TEXT("Replaced"), UObject::FAssetRegistryTag::TT_Alphabetical));
						}
					}
				}
			}
		}

		Context.AddTag(UObject::FAssetRegistryTag(TEXT("ExternalOverrideProperty"), *ExternalsPropertyName.ToString(), UObject::FAssetRegistryTag::TT_Alphabetical));
	}
}


/**
 * Recursive object path builder from linker export
 */
 enum EObjectPathState 
 {
	Empty,
	Package,
	Object,
	SubObject
 };


void BuildExportPath(const FLinkerLoad& Linker, FString& OutPathName, const FPackageIndex ResourceIndex, EObjectPathState& PathState)
{
	static const TCHAR StateDelimiter[] =
	 {
		TEXT('\0'),
		TEXT('.'),
		SUBOBJECT_DELIMITER_CHAR, 
		TEXT('.')
	 };

	 static const EObjectPathState NextPathState[] =
	 {
		EObjectPathState::Package,
		EObjectPathState::Object,
		EObjectPathState::SubObject,
		EObjectPathState::SubObject,
	 };


	const FObjectResource& Resource	= Linker.ImpExp(ResourceIndex);

	if (Resource.OuterIndex.IsNull())
	{
		if (!ResourceIndex.IsImport())
		{
			// If the export has no outer path, initialize the OuterPath with the Package's path on disk		
			OutPathName = Linker.LinkerRoot->GetName();
			PathState = EObjectPathState::Package;
		}		
	}
	else
	{		
		BuildExportPath(Linker, OutPathName, Resource.OuterIndex, PathState);
		PathState = NextPathState[PathState];
	}	
	
	OutPathName += StateDelimiter[PathState];
	OutPathName += Resource.ObjectName.ToString();
};

void BuildExportPath(const FLinkerLoad& Linker, FString& OutPathName, const FObjectExport& Resource)
{
	EObjectPathState State = EObjectPathState::Empty;
	BuildExportPath(Linker, OutPathName, Resource.ThisIndex, State);
}


/**
 * Given an Export from the package's export table, which is of class verse::entity and bears the flag RF_HasDynamicImports,
 * find the entities which have this Export as their outer, so they are added as Imports of this Export, which
 * automate their loading in the async loading queue.
 */
void FExternalObjectsOverrideHelper::ImportExternalOverrides(const FLinkerLoad& Linker, const FObjectExport& Export, TArray<FDynamicImport>& OutDynamicImports)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalObjectsOverrideHelper::ImportExternalOverrides);
	
	const FLinkerInstancingContext& InstancingContext = Linker.GetInstancingContext();

	const UPackage* Package = Linker.LinkerRoot;
	// Handle instancing
	const FName PackageNameInMemory = Package->GetFName();
	const FName PackageNameOnDisk = !Package->GetLoadedPath().IsEmpty() ? Package->GetLoadedPath().GetPackageFName() : PackageNameInMemory;	
	
	const bool bIsInstanced = PackageNameInMemory != PackageNameOnDisk;
	// Produce fully qualified pathname for this object, in the format:
	// 'Outermost.[Outer:]Name'
	// See: UObjectBaseUtility::GetPathName
	FString OuterPath;
	BuildExportPath(Linker, OuterPath, Export);

	UE_LOGF(LogExternalOverrides, Verbose, "Querying dynamic imports for package %ls (in memory: %ls), outer: %ls", *PackageNameOnDisk.ToString(), *PackageNameInMemory.ToString(), *OuterPath);

	if (bIsInstanced)
	{
		// Replace the expected outer path to the one on disk for loading and filtering assets
		OuterPath.ReplaceInline(*PackageNameInMemory.ToString(), *PackageNameOnDisk.ToString());
	}

	EAssetsWithOuterForPathsFlags OuterForPathsFlags = (EAssetsWithOuterForPathsFlags::RecursivePaths | EAssetsWithOuterForPathsFlags::IncludeOnlyOnDiskAsset | EAssetsWithOuterForPathsFlags::ExactOuter | EAssetsWithOuterForPathsFlags::ScanPaths);
	if (TArray<FAssetData> Assets = UAssetRegistryHelpers::GetAssetsWithOuterForPaths({ PackageNameOnDisk }, *OuterPath, OuterForPathsFlags); !Assets.IsEmpty())
	{
		Assets.SetNum(Algo::RemoveIf(Assets, [](const FAssetData& InAssetData) { return InAssetData.IsRedirector(); }));		
		
		if (Assets.Num() > 0)
		{
			OutDynamicImports.Reserve(Assets.Num());

			for (const FAssetData& Asset : Assets)
			{
				UE_LOGF(LogExternalOverrides, Verbose, "Adding dynamic import package %ls for %ls", *Asset.PackageName.ToString(), *OuterPath);

				FDynamicImport Import;
				Import.PackageName = Asset.PackageName;
				Import.ObjectName = Asset.AssetName;
				Import.ClassName = Asset.AssetClassPath.GetAssetName();
				Import.ClassPackage = Asset.AssetClassPath.GetPackageName();

				if (bIsInstanced)
				{
					Import.InstanceName = InstancingContext.RemapPackage(Asset.PackageName);

					// Remap to the instanced package if it wasn't remapped already by the outer instancing context
					if (Import.InstanceName == Asset.PackageName || Import.InstanceName.IsNone())
					{
						Import.InstanceName = *FLinkerInstancingContext::GetInstancedPackageName(PackageNameInMemory.ToString(), Asset.PackageName.ToString());
					}

					checkf(!Import.InstanceName.IsNone(), TEXT("Could find not the remapped package name for dynamic import package %s, in memory name %s"), *Asset.PackageName.ToString(), *PackageNameInMemory.ToString());
				}

				OutDynamicImports.Add(Import);
			}
		}
	}
}

#endif //#if WITH_EDITOR

UOverrideRemovePlaceholder::UOverrideRemovePlaceholder()
{
}

#if WITH_EDITORONLY_DATA
bool UOverrideRemovePlaceholder::IsEditorOnly() const
{
	return true;
}

bool UOverrideRemovePlaceholder::IsAsset() const
{
	if (!HasAllFlags(RF_ClassDefaultObject))
	{
		return true;
	}

	return Super::IsAsset();
}

void UOverrideRemovePlaceholder::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	//@todo_ow: Get owner entity and check if we're in it's overridden properties instead?
	if (!HasAllFlags(RF_ClassDefaultObject))
	{
		Context.AddTag(UObject::FAssetRegistryTag(TEXT("OverrideRemoved"), OverrideRemoved.ToString(), UObject::FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(UObject::FAssetRegistryTag(TEXT("ExternalOverrideProperty"), *OverrideProperty, UObject::FAssetRegistryTag::TT_Alphabetical));
	}
}
#endif //#if WITH_EDITORONLY_DATA
