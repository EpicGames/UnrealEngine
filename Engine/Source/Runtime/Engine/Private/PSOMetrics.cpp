// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOMetrics.cpp: 
=============================================================================*/

#include "PSOMetrics.h"
#include "HAL/CriticalSection.h"

namespace PSOMetrics
{
	FCriticalSection PSOCriticalSection;
	float DurationSum = 0;
	int Count = 0;
}

void AccumulatePSOMetrics(float CompilationDuration)
{
	FScopeLock ScopeLock(&PSOMetrics::PSOCriticalSection);
	
	PSOMetrics::DurationSum += CompilationDuration;
	++PSOMetrics::Count;
}

void GetPSOCompilationMetrics(float& DurationSum, int& Count)
{
	// we need this scope because this will be called from the GameThread
	FScopeLock ScopeLock(&PSOMetrics::PSOCriticalSection);

	DurationSum = PSOMetrics::DurationSum;
	Count = PSOMetrics::Count;

	DurationSum = 0;
	Count = 0;
}
