// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecache.cpp: 
=============================================================================*/

#include "PSOPrecache.h"
#include "Misc/App.h"
#include "HAL/IConsoleManager.h"
#include "ShaderCodeLibrary.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneInfo.h"
#include "PSOPrecacheValidation.h"
#include "PSOPrecacheComponent.h"
#include "UnrealEngine.h"

int32 GPSOPrecacheComponents = 1;
static FAutoConsoleVariableRef CVarPSOPrecacheComponents(
	TEXT("r.PSOPrecache.Components"),
	GPSOPrecacheComponents,
	TEXT("Precache all possible used PSOs by components during Postload (default 1 if PSOPrecaching is enabled)."),
	ECVF_ReadOnly
);

int32 GPSOPrecacheResources = 0;
static FAutoConsoleVariableRef CVarPSOPrecacheResources(
	TEXT("r.PSOPrecache.Resources"),
	GPSOPrecacheResources,
	TEXT("Precache all possible used PSOs by resources during Postload (default 0 if PSOPrecaching is enabled)."),
	ECVF_ReadOnly
);

int32 GPSOPrecacheProxyCreationStrategy = 1;
static FAutoConsoleVariableRef CVarPSOPrecacheProxyCreationStrategy(
	TEXT("r.PSOPrecache.ProxyCreationStrategy"), 
	GPSOPrecacheProxyCreationStrategy, 
	TEXT(" 0 : Always create the proxy even when PSOs are still compiling\n")
	TEXT(" 1 : Delay proxy creation when PSOs are still compiling (default)\n")
	TEXT(" 2 : Use fallback material when PSOs are still compiling"),
	ECVF_ReadOnly);

int32 GDeprecated_PSOProxyCreationWhenPSOReady = 1;
int32 GDeprecated_PSOProxyCreationDelayStrategy = 0;

static void TranslateOldProxyCreationCVarValueToNew()
{
	if (GDeprecated_PSOProxyCreationWhenPSOReady == 1)
	{
		if (GDeprecated_PSOProxyCreationDelayStrategy == 1)
		{
			CVarPSOPrecacheProxyCreationStrategy->Set((int32)EPSOPrecacheProxyCreationStrategy::UseFallbackMaterialUntilPSOPrecached);
		}
		else
		{
			CVarPSOPrecacheProxyCreationStrategy->Set((int32)EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached);
		}
	}
	else
	{
		CVarPSOPrecacheProxyCreationStrategy->Set((int32)EPSOPrecacheProxyCreationStrategy::AlwaysCreate);
	}

	UE_LOGF(LogEngine, Warning, "CVar 'r.PSOPrecache.ProxyCreationWhenPSOReady' and 'r.PSOPrecache.ProxyCreationDelayStrategy' are deprecated. Please use 'r.PSOPrecache.ProxyCreationStrategy'.");
}

static FAutoConsoleVariableRef CVarPSOProxyCreationWhenPSOReady(
	TEXT("r.PSOPrecache.ProxyCreationWhenPSOReady"),
	GDeprecated_PSOProxyCreationWhenPSOReady,
	TEXT("Delay the component proxy creation when the requested PSOs for precaching are still compiling.\n")
	TEXT(" 0: always create regardless of PSOs status (default)\n")
	TEXT(" 1: delay the creation of the render proxy depending on the specific strategy controlled by r.PSOPrecache.ProxyCreationDelayStrategy\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
    {
		TranslateOldProxyCreationCVarValueToNew();
    }),
	ECVF_ReadOnly
);

static FAutoConsoleVariableRef CVarPSOProxyCreationDelayStrategy(
	TEXT("r.PSOPrecache.ProxyCreationDelayStrategy"),
	GDeprecated_PSOProxyCreationDelayStrategy,
	TEXT("Control the component proxy creation strategy when the requested PSOs for precaching are still compiling. Ignored if r.PSOPrecache.ProxyCreationWhenPSOReady = 0.\n")
	TEXT(" 0: delay creation until PSOs are ready (default)\n")
	TEXT(" 1: create a proxy using the default material until PSOs are ready. Currently implemented for static and skinned meshes - Niagara components will delay creation instead"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
    {
		TranslateOldProxyCreationCVarValueToNew();
    }),
	ECVF_ReadOnly
);

static int32 GPSODrawnComponentBoostStrategy = 0;
static FAutoConsoleVariableRef CVarPSOComponentBoostStrategy(
	TEXT("r.PSOPrecache.DrawnComponentBoostStrategy"),
	GPSODrawnComponentBoostStrategy,
	TEXT("Increase priority of queued precache PSOs which are also required by the component for rendering.\n")
	TEXT("0 do not increase priority of drawn PSOs (default)\n")
	TEXT("1 if the component has been rendered then increase the priority of it's PSO precache requests. (this requires r.PSOPrecache.ProxyCreationStrategy > 0.)"),
	ECVF_ReadOnly
);

static bool bFallbackMaterialDelayStratCanIgnorePSOPrioBoost = false;
static FAutoConsoleVariableRef CVarDefaultMaterialDelayStratCanIgnorePSOPrioBoost(
	TEXT("r.PSOPrecache.FallbackMaterialDelayStratCanIgnorePSOPrioBoost"),
	bFallbackMaterialDelayStratCanIgnorePSOPrioBoost,
	TEXT("false: PSO precache requests are boosted during proxy create. (default) \n")
	TEXT("true: When r.PSOPrecache.ProxyCreationStrategy is set to UseFallbackMaterialUntilPSOPrecached, priority boost during proxy create will be ignored."));

int32 GPSOPrecacheMode = 0;
static FAutoConsoleVariableRef CVarPSOPrecacheMode(
	TEXT("r.PSOPrecache.Mode"),
	GPSOPrecacheMode,
	TEXT(" 0: Full PSO (default)\n")
	TEXT(" 1: Preload shaders\n"),
	ECVF_Default
);

CSV_DECLARE_CATEGORY_EXTERN(PSOPrecache);

EPSOPrecacheMode GetPSOPrecacheMode()
{
	switch (GPSOPrecacheMode)
	{
	case 1:
		return EPSOPrecacheMode::PreloadShader;
	case 0:
		[[fallthrough]];
	default:
		return EPSOPrecacheMode::PSO;
	}
}

bool IsPSOShaderPreloadingEnabled()
{
	return FApp::CanEverRender() && (GetPSOPrecacheMode() == EPSOPrecacheMode::PreloadShader) && !FShaderCodeLibrary::AreShaderMapsPreloadedAtLoadTime() && !GIsEditor && UMaterialInterface::IsDefaultMaterialInitialized();
}

bool IsComponentPSOPrecachingEnabled()
{
	return FApp::CanEverRender() && (PipelineStateCache::IsPSOPrecachingEnabled() || IsPSOShaderPreloadingEnabled()) && GPSOPrecacheComponents && !GIsEditor;
}

bool IsResourcePSOPrecachingEnabled()
{
	return FApp::CanEverRender() && (PipelineStateCache::IsPSOPrecachingEnabled() || IsPSOShaderPreloadingEnabled()) && GPSOPrecacheResources && !GIsEditor;
}

bool ShouldBoostPSOPrecachePriorityOnDraw()
{
	return FApp::CanEverRender() && PipelineStateCache::IsPSOPrecachingEnabled() && GPSODrawnComponentBoostStrategy && !GIsEditor;
}

EPSOPrecacheProxyCreationStrategy GetPSOPrecacheProxyCreationStrategy(int32 OverrideValue)
{
	if (OverrideValue >= 0 && OverrideValue <= 2)
	{
		return EPSOPrecacheProxyCreationStrategy(OverrideValue);
	}

	return EPSOPrecacheProxyCreationStrategy(GPSOPrecacheProxyCreationStrategy);
}

bool ProxyCreationWhenPSOReady()
{
	return FApp::CanEverRender() && (PipelineStateCache::IsPSOPrecachingEnabled() || IsPSOShaderPreloadingEnabled()) && GPSOPrecacheProxyCreationStrategy > 0 && !GIsEditor;
}

void BoostPrecachedPSORequestsOnDraw(const FPrimitiveSceneInfo* SceneInfo)
{
#if UE_WITH_PSO_PRECACHING 
	if (SceneInfo && SceneInfo->Proxy)
	{
		SceneInfo->Proxy->BoostPrecachedPSORequestsOnDraw();
	}
#endif
}

FPSOPrecacheVertexFactoryData::FPSOPrecacheVertexFactoryData(
	const FVertexFactoryType* InVertexFactoryType, const FVertexDeclarationElementList& ElementList) 
	: VertexFactoryType(InVertexFactoryType)
{
	CustomDefaultVertexDeclaration = PipelineStateCache::GetOrCreateVertexDeclaration(ElementList);
}

void AddMaterialInterfacePSOPrecacheParamsToList(const FMaterialInterfacePSOPrecacheParams& EntryToAdd, FMaterialInterfacePSOPrecacheParamsList& List)
{
	FMaterialInterfacePSOPrecacheParams* CurrentEntry = List.FindByPredicate([EntryToAdd](const FMaterialInterfacePSOPrecacheParams& Other)
		{
			return (Other.Priority			== EntryToAdd.Priority &&
					Other.MaterialInterface == EntryToAdd.MaterialInterface &&
					Other.PSOPrecacheParams == EntryToAdd.PSOPrecacheParams);
		});
	if (CurrentEntry)
	{
		for (const FPSOPrecacheVertexFactoryData& VFData : EntryToAdd.VertexFactoryDataList)
		{
			CurrentEntry->VertexFactoryDataList.AddUnique(VFData);
		}
	}
	else
	{
		List.Add(EntryToAdd);
	}
}

FGlobalPSOCollectorManager::FPSOCollectorData FGlobalPSOCollectorManager::PSOCollectors[FGlobalPSOCollectorManager::MaxPSOCollectorCount] = {};

int32 FGlobalPSOCollectorManager::GetIndex(const TCHAR* Name)
{
	#if PSO_PRECACHING_VALIDATE
	if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
	{
		for (int32 Index = 0; Index < PSOCollectorCount; ++Index)
		{
			if (FCString::Strcmp(PSOCollectors[Index].Name, Name) == 0)
			{
				return Index;
			}
		}
	}
	#endif

	return INDEX_NONE;
}

/**
 * Start the actual PSO precache tasks for all the initializers provided and return the request result array containing the graph event on which to wait for the PSOs to finish compiling (doesn't add not required PSOs to the graph event list)
 */
FPSOPrecacheRequestResultArray RequestPrecachePSOs(EPSOPrecacheType PSOPrecacheType, const FPSOPrecacheDataArray& PSOInitializers)
{
	FPSOPrecacheRequestResultArray Results;
	for (const FPSOPrecacheData& PrecacheData : PSOInitializers)
	{
		FPSOPrecacheRequestParams PSORequestParams;
		PSORequestParams.InMemoryMode = PrecacheData.InMemoryMode;
		// Make low priority when not required
		PSORequestParams.PrecachePriority = PrecacheData.bRequired ? PrecacheData.Priority : EPSOPrecachePriority::Low;
#if PSO_PRECACHING_VALIDATE
		PSORequestParams.ResourceName = PrecacheData.ResourceName;
		if (PrecacheData.PSOCollectorIndex != INDEX_NONE)
		{
			PSORequestParams.CollectorName = PSOPrecacheType == EPSOPrecacheType::Global ? FGlobalPSOCollectorManager::GetName(PrecacheData.PSOCollectorIndex) : FPSOCollectorCreateManager::GetName(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), PrecacheData.PSOCollectorIndex);
		}
		else
		{
			PSORequestParams.CollectorName = nullptr;
		}
#endif // PSO_PRECACHING_VALIDATE

		bool bUpdateStats = false;
		FPSOPrecacheRequestResult PSOPrecacheResult;
		switch (PrecacheData.Type)
		{
			case FPSOPrecacheData::EType::Graphics:
			{
			#if PSO_PRECACHING_VALIDATE
				bUpdateStats = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().AddStateToCache(PSOPrecacheType, PrecacheData.GraphicsPSOInitializer, PSOCollectorStats::GetPSOPrecacheHash, nullptr, PrecacheData.PSOCollectorIndex, PrecacheData.VertexFactoryType);
			#endif // PSO_PRECACHING_VALIDATE			
				
				PSOPrecacheResult = PipelineStateCache::PrecacheGraphicsPipelineState(PrecacheData.GraphicsPSOInitializer, PSORequestParams);
				break;
			}
			case FPSOPrecacheData::EType::Compute:
			{
			#if PSO_PRECACHING_VALIDATE
				bUpdateStats = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().AddStateToCache(PSOPrecacheType, PrecacheData.ComputeInitializer, PSOCollectorStats::GetPSOPrecacheHash, nullptr, PrecacheData.PSOCollectorIndex, nullptr);
			#endif // PSO_PRECACHING_VALIDATE
			
				PSOPrecacheResult = PipelineStateCache::PrecacheComputePipelineState(PrecacheData.ComputeInitializer, PSORequestParams);
				break;
			}
		}
		if (PSOPrecacheResult.IsValid() && PrecacheData.bRequired)
		{
			Results.AddUnique(PSOPrecacheResult);
		}
			
#if PSO_PRECACHING_VALIDATE
		if (bUpdateStats)
		{			
			// Only update stats when updated (first hit)
			PSOCollectorStats::UpdateGlobalStats(PSOPrecacheType, PrecacheData);
		}
#endif // PSO_PRECACHING_VALIDATE
	}

	return Results;
}

bool FPSOPrecacheComponentData::CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority NewPSOPrecachePriority)
{
#if UE_WITH_PSO_PRECACHING
	bool bPrecacheStillRunning = IsPSOPrecaching();

	ensure(!IsComponentPSOPrecachingEnabled() || bPSOPrecacheCalled);
	check(NewPSOPrecachePriority == EPSOPrecachePriority::High || NewPSOPrecachePriority == EPSOPrecachePriority::Highest);

	const bool bIgnoringBoostDueToFallbackMaterialStrat = bFallbackMaterialDelayStratCanIgnorePSOPrioBoost && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::UseFallbackMaterialUntilPSOPrecached && ShouldBoostPSOPrecachePriorityOnDraw();

	if (bPrecacheStillRunning && PSOPrecacheRequestPriority < NewPSOPrecachePriority && !bIgnoringBoostDueToFallbackMaterialStrat)
	{
		BoostPSOPriority(NewPSOPrecachePriority, MaterialPSOPrecacheRequestIDs);
		PSOPrecacheRequestPriority = NewPSOPrecachePriority;
	}
	return bPrecacheStillRunning;
#else
	return false;
#endif
}

#if UE_WITH_PSO_PRECACHING

class FPSOPrecacheStateCache
{
public:

	bool PrecachePSO(const FGraphicsPipelineStateInitializer& InInitializer)
	{	
		uint64 StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(InInitializer);

		auto StartPSOPrecache = [InInitializer, StatePrecachePSOHash]()
		{
			FGraphicsPipelineStateInitializer Initializer = InInitializer;
			Initializer.StatePrecachePSOHash = StatePrecachePSOHash;
			
			FPSOPrecacheRequestParams PSORequestParams;
			PSORequestParams.InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
			PSORequestParams.PrecachePriority = EPSOPrecachePriority::Highest;
			return PipelineStateCache::PrecacheGraphicsPipelineState(Initializer, PSORequestParams);
		};
		return PrecachePSOInternal(StatePrecachePSOHash, StartPSOPrecache);			
	}

	bool PrecachePSO(const FComputePipelineStateInitializer& InInitializer)
	{	
		uint64 ComputePSOHash = GetTypeHash(InInitializer.ComputeShader->GetHash());
		
		auto StartPSOPrecache = [InInitializer]()
		{			
			FPSOPrecacheRequestParams PSORequestParams;
			PSORequestParams.InMemoryMode = EPSOPrecacheInMemoryMode::Uncached;
			PSORequestParams.PrecachePriority = EPSOPrecachePriority::Highest;
			return PipelineStateCache::PrecacheComputePipelineState(InInitializer, PSORequestParams);
		};
		return PrecachePSOInternal(ComputePSOHash, StartPSOPrecache);			
	}

private:
	
	template<typename FStartPrecacheFunction>
	bool PrecachePSOInternal(uint64 PSOInitializerHash, FStartPrecacheFunction StartPrecacheFunction)
	{	
		FPrecacheStateAndRequestData ActiveRequestData;

		// Check if PSO is already cached with read only lock
		{
			FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);
			FPrecacheStateAndRequestData* StateAndRequestData = CachedPSOPrecacheResults.Find(PSOInitializerHash);
			if (StateAndRequestData)
			{
				ActiveRequestData = *StateAndRequestData;
			}
		}

		// Early out when done
		if (ActiveRequestData.bIsPrecached)
		{
			return true;
		}		

		// Not requested yet?
		if (!ActiveRequestData.RequestData.RequestID.IsValid())
		{
			// Start the PSO precache request and boost to highest priority (multiple thread could in theory do this which is 'fine')
			FPrecacheStateAndRequestData NewStateAndRequestData;
			NewStateAndRequestData.RequestData = StartPrecacheFunction();
			NewStateAndRequestData.bIsPrecached = !PipelineStateCache::IsPrecaching(NewStateAndRequestData.RequestData.RequestID);
			ActiveRequestData.bIsPrecached = NewStateAndRequestData.bIsPrecached;
			
			// Add with write lock
			{
				FRWScopeLock WriteLock(RWLock, SLT_Write);
			
				// check if it's already in the cache now that we have a write lock
				FPrecacheStateAndRequestData* StateAndRequestData = CachedPSOPrecacheResults.Find(PSOInitializerHash);
				if (StateAndRequestData)
				{
					check(StateAndRequestData->RequestData == NewStateAndRequestData.RequestData);
				}
				else
				{
					CachedPSOPrecacheResults.Add(PSOInitializerHash, NewStateAndRequestData);
				}
			}			
		}
		else 
		{
			// Check the current state again
			if (!PipelineStateCache::IsPrecaching(ActiveRequestData.RequestData.RequestID))
			{
				// Update the cache state with write lock 
				// Multiple thread can write this state after each other but that's fine because multiple threads can try
				// and precache the same PSO initializer and thus should get the same hash with StateAndRequestData
				FRWScopeLock WriteLock(RWLock, SLT_Write);
				FPrecacheStateAndRequestData* StateAndRequestData = CachedPSOPrecacheResults.Find(PSOInitializerHash);
				check(StateAndRequestData && StateAndRequestData->RequestData.RequestID == ActiveRequestData.RequestData.RequestID);
				StateAndRequestData->bIsPrecached = true;
				StateAndRequestData->RequestData.AsyncCompileEvent = nullptr;

				ActiveRequestData.bIsPrecached = true;
			}
		}

		return ActiveRequestData.bIsPrecached;				
	}

	struct FPrecacheStateAndRequestData
	{
		FPSOPrecacheRequestResult RequestData;
		bool bIsPrecached = false;
	};

	FRWLock RWLock;
	TMap <uint64, FPrecacheStateAndRequestData> CachedPSOPrecacheResults;
};

FPSOPrecacheStateCache GPSOPrecacheStateCache;

#endif // UE_WITH_PSO_PRECACHING


bool PrecachePSO(const FGraphicsPipelineStateInitializer& GraphicsPSO)
{
#if UE_WITH_PSO_PRECACHING
	return GPSOPrecacheStateCache.PrecachePSO(GraphicsPSO);
#else
	return true;
#endif
}

bool PrecachePSO(const FComputePipelineStateInitializer& ComputePSO)
{
#if UE_WITH_PSO_PRECACHING
	return GPSOPrecacheStateCache.PrecachePSO(ComputePSO);
#else
	return true;
#endif
}