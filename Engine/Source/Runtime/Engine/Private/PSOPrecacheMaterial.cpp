// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheMaterial.cpp: 
=============================================================================*/

#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheValidation.h"

#include "Misc/App.h"
#include "UObject/StrongObjectPtr.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "VertexFactory.h"
#include "SceneInterface.h"
#include "ShaderCodeArchive.h"
#include "ODSC/ODSCManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MetadataTrace.h"

int32 GPSOUseBackgroundThreadForCollection = 1;
static FAutoConsoleVariableRef CVarPSOUseBackgroundThreadForCollection(
	TEXT("r.PSOPrecache.UseBackgroundThreadForCollection"),
	GPSOUseBackgroundThreadForCollection,
	TEXT("Use background threads for PSO precache data collection on the mesh pass processors.\n"),
	ECVF_ReadOnly
);

bool GShaderPreloadFilterUniqueRequest = true;
static FAutoConsoleVariableRef CVarShaderPreloadFilterUniqueRequest(
	TEXT("r.PSOPrecache.ShaderPreloadFilterUniqueRequest"),
	GShaderPreloadFilterUniqueRequest,
	TEXT("Perf improvement (reduce contention on r/w lock). When kicking preload shaders job, only request one preload request per shaderIndex inside the same ShaderMapResource.\n"),
	ECVF_Default
);

int32 GPSOPrecacheKeepInMemoryForActiveMaterials = 0;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepInMemoryForActiveMaterials(
	TEXT("r.PSOPrecache.KeepInMemoryForActiveMaterials"),
	GPSOPrecacheKeepInMemoryForActiveMaterials,
	TEXT("Keep all precached PSOs in memory for loaded materials - if the material is released then all the PSOs unique for this material are also released.")
	TEXT("Mutually exlcusive with r.PSOPrecache.KeepInMemoryUntilUsed - if r.PSOPrecache.KeepInMemoryUntilUsed is set then caching will be ignored."),
	ECVF_ReadOnly);

FPSOCollectorCreateManager::FPSOCollectorData FPSOCollectorCreateManager::PSOCollectors[(int32)EShadingPath::Num][FPSOCollectorCreateManager::MaxPSOCollectorCount] = {};

int32 FPSOCollectorCreateManager::GetIndex(EShadingPath ShadingPath, const TCHAR* Name)
{
#if PSO_PRECACHING_VALIDATE
	if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
	{
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		for (int32 Index = 0; Index < PSOCollectorCount[ShadingPathIdx]; ++Index)
		{
			if (FCString::Strcmp(PSOCollectors[ShadingPathIdx][Index].Name, Name) == 0)
			{
				return Index;
			}
		}
	}
#endif

	return INDEX_NONE;
}

/**
 * Helper task used to release the strong object reference to the material interface on the game thread
 * The release has to happen on the gamethread and the material interface can't be GCd while the PSO
 * collection is happening because it touches the material resources
 */
class FMaterialInterfaceReleaseTask
{
public:
	explicit FMaterialInterfaceReleaseTask(TStrongObjectPtr<UMaterialInterface>* InMaterialInterface)
		: MaterialInterface(InMaterialInterface)
	{
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		check(IsInGameThread());
		delete MaterialInterface;
	}

public:

	TStrongObjectPtr<UMaterialInterface>* MaterialInterface;

	static ESubsequentsMode::Type	GetSubsequentsMode() { return ESubsequentsMode::FireAndForget; }
	ENamedThreads::Type				GetDesiredThread() { return ENamedThreads::GameThread; }
	FORCEINLINE TStatId				GetStatId() const { return TStatId(); }
};

/**
 * Helper task used to offload the PSO collection from the GameThread. The shader decompression
 * takes too long to run this on the GameThread and it isn't blocking anything crucial.
 * The graph event used to create this task is extended with the PSO compilation tasks itself so the user can optionally
 * wait or known when all PSOs are ready for rendering
 */
class FMaterialPSOPrecacheCollectionTask
{
public:
	explicit FMaterialPSOPrecacheCollectionTask(
		TStrongObjectPtr<UMaterialInterface> InMaterialInterface,
		const FMaterialPSOPrecacheParams& InPrecacheParams,
		FGraphEventRef& InCollectionGraphEvent,
		uint32 InRequestLifecycleID)
		: MaterialInterface(InMaterialInterface)
		, PrecacheParams(InPrecacheParams)
		, CollectionGraphEvent(InCollectionGraphEvent)
		, RequestLifecycleID(InRequestLifecycleID)
	{
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
	static FPSOPrecacheRequestResultArray CollectPrecacheDataAndStartCompiles(const FMaterialPSOPrecacheParams& PrecacheParams, uint32 LifecycleID);

	~FMaterialPSOPrecacheCollectionTask()
	{
		//check(MaterialInterface == nullptr);  // TODO: reinstate this or replace TStrongObjectPtr* with TStrongObjectPtr..
	}

public:

	TStrongObjectPtr<UMaterialInterface> MaterialInterface;
	FMaterialPSOPrecacheParams PrecacheParams;
	FGraphEventRef CollectionGraphEvent;
	uint32 RequestLifecycleID;	

	static ESubsequentsMode::Type	GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type				GetDesiredThread() { return ENamedThreads::AnyBackgroundThreadNormalTask; }
	FORCEINLINE TStatId				GetStatId() const { return TStatId(); }
};


class FShaderMapPreloadTask
{
public:
	explicit FShaderMapPreloadTask(
		TStrongObjectPtr<UMaterialInterface>* InMaterialInterface,
		FMaterialShaderMap* InMaterialShaderMap,
		FGraphEventRef InShaderPreloadEvents)
		: MaterialInterface(InMaterialInterface)
		, MaterialShaderMap(InMaterialShaderMap)
		, ShaderPreloadEvents(InShaderPreloadEvents)
	{
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

	static ESubsequentsMode::Type	GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type				GetDesiredThread() { return ENamedThreads::AnyBackgroundThreadNormalTask; }
	FORCEINLINE TStatId				GetStatId() const { return TStatId(); }

private:
	TStrongObjectPtr<UMaterialInterface>* MaterialInterface;
	FMaterialShaderMap* MaterialShaderMap;
	FGraphEventRef ShaderPreloadEvents;
};

class FShaderPreloadCollectionTask
{
public:
	explicit FShaderPreloadCollectionTask(
		TStrongObjectPtr<UMaterialInterface>* InMaterialInterface,
		const FMaterialPSOPrecacheParams& InPrecacheParams,
		FGraphEventRef InShaderPreloadEvents)
		: MaterialInterface(InMaterialInterface)
		, PrecacheParams(InPrecacheParams)
		, ShaderPreloadEvents(InShaderPreloadEvents)
	{
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

	static ESubsequentsMode::Type	GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type				GetDesiredThread() { return ENamedThreads::AnyBackgroundThreadNormalTask; }
	FORCEINLINE TStatId				GetStatId() const { return TStatId(); }

private:
	TStrongObjectPtr<UMaterialInterface>* MaterialInterface;
	FMaterialPSOPrecacheParams PrecacheParams;
	FGraphEventRef ShaderPreloadEvents;
	TOptional<uint32> RequestLifecycleID;
};


/**
 * Manages all the material PSO requests and cached which PSOs are still compiling for a certain material, vertex factory and precache param combination
 * Also caches all the request information used for detailed logging on PSO precache misses
 */
class FMaterialPSORequestManager
{
public:

	FMaterialPSOPrecacheRequestID PrecachePSOs(const FMaterialPSOPrecacheParams& Params, EPSOPrecachePriority Priority, FGraphEventArray& OutGraphEvents)
	{
		LLM_SCOPE(ELLMTag::PSO);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		FMaterialPSOPrecacheRequestID RequestID = INDEX_NONE;

		// only support medium and high priorities via the material PSO precaching system
		check(Priority == EPSOPrecachePriority::Medium || Priority == EPSOPrecachePriority::High);

		if (GetPSOPrecacheMode() == EPSOPrecacheMode::PreloadShader)
		{
			PreloadShaders(Params, OutGraphEvents);
		}
		else
		{
			// Fast check first with read lock if it's not requested or completely finished already
			{
				FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);
				FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
				if (FindResult != nullptr && FindResult->State == EState::Completed)
				{
					return RequestID;
				}
			}

			// Offload to background job task graph if threading is enabled
			// Don't use background thread in editor because shader maps and material resources could be destroyed while the task is running
			// If it's a perf problem at some point then FMaterialPSOPrecacheRequestID has to be used at material level in the correct places to wait for
			bool bUseBackgroundTask = GPSOUseBackgroundThreadForCollection && FApp::ShouldUseThreadingForPerformance() && !GIsEditor;

			FGraphEventRef CollectionGraphEvent;

			// Now try and add with write lock
			{
				FRWScopeLock WriteLock(RWLock, SLT_Write);

				FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
				if (FindResult != nullptr)
				{
					// Update the list of compiling PSOs and update the internal state
					bool bBoostPriority = (Priority == EPSOPrecachePriority::High && FindResult->Priority != Priority);
					CheckCompilingPSOs(*FindResult, bBoostPriority);
					if (FindResult->State != EState::Completed)
					{
						// If there is a collection graph event than task is used for collection and PSO compiles
						// The collection graph event is extended until all PSOs are compiled and caller only has to wait
						// for this event to finish
						if (FindResult->CollectionGraphEvent)
						{
							OutGraphEvents.Add(FindResult->CollectionGraphEvent);
						}
						else
						{
							for (FPSOPrecacheRequestResult& Result : FindResult->ActivePSOPrecacheRequests)
							{
								OutGraphEvents.Add(Result.AsyncCompileEvent);
							}
						}
						RequestID = FindResult->RequestID;
					}

					return RequestID;
				}
				else
				{
					// Add to array to get the new RequestID
					RequestID = MaterialPSORequests.Add(Params);

					// Add data to map
					FPrecacheData PrecacheData;
					PrecacheData.State = EState::Collecting;
					PrecacheData.RequestID = RequestID;
					PrecacheData.Priority = Priority;
					if (bUseBackgroundTask)
					{
						CollectionGraphEvent = FGraphEvent::CreateGraphEvent();
						PrecacheData.CollectionGraphEvent = CollectionGraphEvent;

						// Create task the clear mark fully complete in the cache when done
						uint32 RequestLifecycleID = LifecycleID;
						FFunctionGraphTask::CreateAndDispatchWhenReady(
							[this, Params, RequestLifecycleID, RequestID]
							{
								MarkCompilationComplete(Params, RequestLifecycleID, RequestID);
							},
							TStatId{}, CollectionGraphEvent
						);
					}
					MaterialPSORequestData.Add(Params, PrecacheData);
				}
			}

			if (bUseBackgroundTask)
			{
				// Make sure the material instance isn't garbage collected or destroyed yet (create TStrongObjectPtr which will be destroyed on the GT when the collection is done)
				TStrongObjectPtr<UMaterialInterface> MaterialInterface(Params.Material->GetMaterialInterface());

				FGraphEventArray Prereqs;
				// Create and kick off the PSO collection task.
				TGraphTask<FMaterialPSOPrecacheCollectionTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(MaterialInterface, Params, CollectionGraphEvent, LifecycleID);

				// Need to wait for collection task which will be extended during run with the actual async compile events.
				OutGraphEvents.Add(CollectionGraphEvent);
			}
			else
			{
				// Collect pso data. Note we don't explicitly collect and preload shaders here since we're not using background tasks
				// and doing so in separate phases wouldn't benefit anything.

				// Start the async compiles
				FPSOPrecacheRequestResultArray PrecacheResults = FMaterialPSOPrecacheCollectionTask::CollectPrecacheDataAndStartCompiles(Params, LifecycleID);
				
				// Add the graph events to wait for
				for (FPSOPrecacheRequestResult& Result : PrecacheResults)
				{
					check(Result.IsValid());

					// AsyncCompileEvent may be null if async compilation is disabled
					if (Result.AsyncCompileEvent)
					{
						OutGraphEvents.Add(Result.AsyncCompileEvent);
					}
				}
			}
		}
		return RequestID;
	}

	void PreloadShaders(const FMaterialPSOPrecacheParams& Params, FGraphEventArray& OutGraphEvents)
	{
		LLM_SCOPE(ELLMTag::PSO);

		if (!IsPSOShaderPreloadingEnabled())
		{
			return;
		}

		// Make sure the material instance isn't garbage collected or destroyed yet (create TStrongObjectPtr which will be destroyed on the GT when the collection is done)
		TStrongObjectPtr<UMaterialInterface>* MaterialInterface = new TStrongObjectPtr<UMaterialInterface>(Params.Material->GetMaterialInterface());
		FMaterialShaderMap* MaterialShaderMap = Params.Material->GetGameThreadShaderMap();

		FGraphEventRef ShadersPreloadedEvent = FGraphEvent::CreateGraphEvent();
		TGraphTask<FShaderPreloadCollectionTask>::CreateTask().ConstructAndDispatchWhenReady(MaterialInterface, Params, ShadersPreloadedEvent);

		FGraphEventArray Prereqs = { ShadersPreloadedEvent };
		TGraphTask<FMaterialInterfaceReleaseTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(MaterialInterface);

		OutGraphEvents.Add(ShadersPreloadedEvent);
	}

	void PreloadShaderMap(const FMaterial* Material, FGraphEventArray& OutGraphEvents)
	{
		LLM_SCOPE(ELLMTag::PSO);

		if (!IsPSOShaderPreloadingEnabled())
		{
			return;
		}

		// Make sure the material instance isn't garbage collected or destroyed yet (create TStrongObjectPtr which will be destroyed on the GT when the collection is done)
		TStrongObjectPtr<UMaterialInterface>* MaterialInterface = new TStrongObjectPtr<UMaterialInterface>(Material->GetMaterialInterface());
		FMaterialShaderMap* MaterialShaderMap = Material->GetGameThreadShaderMap();

		FGraphEventRef ShadersPreloadedEvent = FGraphEvent::CreateGraphEvent();
		TGraphTask<FShaderMapPreloadTask>::CreateTask().ConstructAndDispatchWhenReady(MaterialInterface, MaterialShaderMap, ShadersPreloadedEvent);
		FGraphEventArray Prereqs = { ShadersPreloadedEvent };
		TGraphTask<FMaterialInterfaceReleaseTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(MaterialInterface);

		OutGraphEvents.Add(ShadersPreloadedEvent);
	}

	void MarkCollectionComplete(const FMaterialPSOPrecacheParams& Params, const FPSOPrecacheDataArray& PrecacheData, const FPSOPrecacheRequestResultArray& PrecacheRequestResults, uint32 RequestLifecycleID)
	{
		LLM_SCOPE(ELLMTag::PSO);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		FRWScopeLock WriteLock(RWLock, SLT_Write);
		
		// Ignore requests not coming from current life cycle ID
		if (RequestLifecycleID != LifecycleID)
		{
			return;
		}

		FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
		check(FindResult->State == EState::Collecting);
		check(FindResult->ActivePSOPrecacheRequests.IsEmpty());
		for (const FPSOPrecacheRequestResult& RequestResult : PrecacheRequestResults)
		{
			if (RequestResult.AsyncCompileEvent)
			{
				FindResult->ActivePSOPrecacheRequests.Add(RequestResult);
			}
			else
			{
				FindResult->CachedPSOPrecacheRequests.Add(RequestResult.RequestID);
			}
		}
#if PSO_PRECACHING_TRACKING
		FindResult->PSOPrecachaData = PrecacheData;
#if PSO_PRECACHING_VALIDATE
		AddTrackedMaterialData(Params, PrecacheData);
#endif // PSO_PRECACHING_VALIDATE
#endif // PSO_PRECACHING_TRACKING

		// update the state
		FindResult->State = FindResult->ActivePSOPrecacheRequests.IsEmpty() ? EState::Completed : EState::Compiling;

		// Release the graph event when done
		if (FindResult->State == EState::Completed)
		{
			FindResult->CollectionGraphEvent = nullptr;
		}

		// Boost priority if requested already
		if (FindResult->Priority >= EPSOPrecachePriority::High)
		{
			CheckCompilingPSOs(*FindResult, true /*bBoostPriority*/);
		}
	}

	void ReleaseMaterialPrecacheData(FMaterialPSOPrecacheRequestID MaterialPSORequestID)
	{
		LLM_SCOPE(ELLMTag::PSO);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		check(MaterialPSORequestID != INDEX_NONE);

		FRWScopeLock WriteLock(RWLock, SLT_Write);
		if ((int32)MaterialPSORequestID >= MaterialPSORequests.Num())
		{
			return;
		}

		const FMaterialPSOPrecacheParams& Params = MaterialPSORequests[MaterialPSORequestID];
				
		if (GPSOPrecacheKeepInMemoryForActiveMaterials)
		{
			FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
			check(FindResult && FindResult->State != EState::Collecting); //< Shouldn't be collecting because material is kept alive then
			if (FindResult)
			{
				if (FindResult->State == EState::Compiling)
				{
					check(!FindResult->ActivePSOPrecacheRequests.IsEmpty());

					// Request release for all active and finished PSOs
					for (const FPSOPrecacheRequestResult& RequestResult : FindResult->ActivePSOPrecacheRequests)
					{
						PipelineStateCache::ReleasePrecachedPSO(RequestResult.RequestID);
					}
				}

				// Release cached PSOs
				for (const FPSOPrecacheRequestID& RequestID : FindResult->CachedPSOPrecacheRequests)
				{
					PipelineStateCache::ReleasePrecachedPSO(RequestID);
				}
			}
		}

		// Mark invalid & remove from from map (could reused IDs with free list)
		verify(MaterialPSORequestData.Remove(Params) == 1);
		MaterialPSORequests[MaterialPSORequestID] = FMaterialPSOPrecacheParams();
	}

	FGraphEventArray GetPrecacheCompletePrereq(FMaterialPSOPrecacheRequestID MaterialPSORequestID)
	{
		FGraphEventArray FoundEvents;
		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);

		if (MaterialPSORequests.IsValidIndex(MaterialPSORequestID))
		{
			const FMaterialPSOPrecacheParams& Params = MaterialPSORequests[MaterialPSORequestID];
			if (FPrecacheData* FindResult = MaterialPSORequestData.Find(Params))
			{				
				if (FindResult->State != EState::Completed)
				{
					if (!FindResult->CollectionGraphEvent.IsValid())
					{
						for (int32 Index = 0; Index < FindResult->ActivePSOPrecacheRequests.Num(); ++Index)
						{
							FoundEvents.Add(FindResult->ActivePSOPrecacheRequests[Index].AsyncCompileEvent);
						}
					}
					else
					{
						FoundEvents.Add(FindResult->CollectionGraphEvent);
					}
				}
			}
		}
		return FoundEvents;
	}


	void BoostPriority(EPSOPrecachePriority NewPri, FMaterialPSOPrecacheRequestID MaterialPSORequestID)
	{
		LLM_SCOPE(ELLMTag::PSO);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		check(MaterialPSORequestID != INDEX_NONE);

		{
			FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);

			if (MaterialPSORequestID >= (uint32)MaterialPSORequests.Num())
			{
				return;
			}

			const FMaterialPSOPrecacheParams& Params = MaterialPSORequests[MaterialPSORequestID];
			FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);

			// Only process if not boosted yet and not completed yet
			if (FindResult == nullptr || NewPri <= FindResult->Priority || FindResult->State == EState::Completed)
			{
				return;
			}
		}

		FRWScopeLock WriteLock(RWLock, SLT_Write);
		const FMaterialPSOPrecacheParams& Params = MaterialPSORequests[MaterialPSORequestID];
		FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
		check(FindResult);
		FindResult->Priority = NewPri;
		// Boost PSOs which are still compiling
		CheckCompilingPSOs(*FindResult, true /*bBoostPriority*/);
	}

	void ClearMaterialPSORequests()
	{
		LLM_SCOPE(ELLMTag::PSO);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		check(IsInGameThread());

		FRWScopeLock WriteLock(RWLock, SLT_Write);

		// Increment the life cycle ID - all current active collection tasks are 'not important' anymore and can either be skipped or ignored
		LifecycleID++;

		TSet<FMaterial*> Materials;
		for (auto Iter : MaterialPSORequestData)
		{
			// Collect all materials
			FMaterialPSOPrecacheParams& Params = Iter.Key;
			if (Params.Material)
			{
				Materials.Add(Params.Material);
			}	
			
			if (GPSOPrecacheKeepInMemoryForActiveMaterials)
			{
				// Request release for all active and finished PSOs
				FPrecacheData& PrecachData = Iter.Value;
				for (const FPSOPrecacheRequestResult& RequestResult : PrecachData.ActivePSOPrecacheRequests)
				{
					PipelineStateCache::ReleasePrecachedPSO(RequestResult.RequestID);
				}
				for (const FPSOPrecacheRequestID& RequestID : PrecachData.CachedPSOPrecacheRequests)
				{
					PipelineStateCache::ReleasePrecachedPSO(RequestID);
				}
			}
		}

		for (FMaterial* Material : Materials)
		{
			Material->ClearPrecachedPSORequestIDs();
		}

		// Clear the current cached pso requests so we gather the PSOs to compile again (usually called on cvar changes which could influence MDC and thus PSOs)			
		MaterialPSORequests.Empty(MaterialPSORequests.Num());
		MaterialPSORequestData.Empty(MaterialPSORequestData.Num());
	}

	uint32 GetLifecycleID() const { return LifecycleID; }

#if PSO_PRECACHING_TRACKING

	FMaterialPSOPrecacheParams GetMaterialPSOPrecacheParams(FMaterialPSOPrecacheRequestID MaterialPSORequestID)
	{
		check(MaterialPSORequestID != INDEX_NONE);

		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);
		return MaterialPSORequests[MaterialPSORequestID];
	}

	FPSOPrecacheDataArray GetMaterialPSOPrecacheData(FMaterialPSOPrecacheRequestID MaterialPSORequestID)
	{
		check(MaterialPSORequestID != INDEX_NONE);

		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);
		const FMaterialPSOPrecacheParams&  Params = MaterialPSORequests[MaterialPSORequestID];
		FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
		check(FindResult);

		// Should wait when still collecting?
		if (FindResult->State == EState::Collecting)
		{
			int i = 0;
		}

		return FindResult->PSOPrecachaData;
	}

	uint64 GetCompileTime(const TCHAR* InPassName, const TCHAR* InVertexFactoryName)
	{
		uint64 TotalCompileTime = 0;
#if PSO_PRECACHING_VALIDATE
		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);

		EShadingPath ShadingPath = GetFeatureLevelShadingPath(GMaxRHIFeatureLevel);
		for (auto Iter : TrackingMaterialPSORequestData)
		{
			uint64 PSOPrecacheHash = Iter.Key;
			const FTrackingMaterialPSOPrecacheParams& PrecacheParams = Iter.Value;				
			const TCHAR* PassName = FPSOCollectorCreateManager::GetName(ShadingPath, PrecacheParams.PSOCollectorIndex);
			const TCHAR* VertexFactoryName = PrecacheParams.VertexFactoryData.VertexFactoryType ? PrecacheParams.VertexFactoryData.VertexFactoryType->GetName() : TEXT("None");

			if ((InPassName == nullptr || FCString::Stricmp(InPassName, PassName) == 0) && (InVertexFactoryName == nullptr || FCString::Stricmp(InVertexFactoryName, VertexFactoryName) == 0))
			{
				uint64 PSOCompilationTime = PipelineStateCache::GetPrecachePSOCompilationTime(PrecacheParams.bGraphics, PSOPrecacheHash);
				TotalCompileTime += PSOCompilationTime;
			}
		}
#endif // PSO_PRECACHING_VALIDATE
		return TotalCompileTime;
	}

	void DumpPSOPrecacheMaterialStats()
	{		
#if PSO_PRECACHING_VALIDATE
		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);

		EShadingPath ShadingPath = GetFeatureLevelShadingPath(GMaxRHIFeatureLevel);

		struct FEntry
		{
			FString MaterialName;
			FString VertexFactoryName;
			FString PassName;
			uint32 CustomDefaultDeclarationPSOPrecacheHash;
			FPSOPrecacheParams PrecachePSOParams;
			bool bUsed;
			bool bGraphics;
			bool bRequired;
			uint64 PSOPrecacheHash;
			float PSOCompilationTime;
		};

		TArray<FPSOPrecacheParams> AllPSOPrecacheParams;
		TArray<FRHIVertexDeclaration*> AllCustomVertexDeclerations;
		TArray<FEntry> Entries;
		for (auto Iter : TrackingMaterialPSORequestData)
		{
			uint64 PSOPrecacheHash = Iter.Key;
			const FTrackingMaterialPSOPrecacheParams& PrecacheParams = Iter.Value;
				
			bool bUsed = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().IsUsed(PSOPrecacheHash);
			const TCHAR* PassName = FPSOCollectorCreateManager::GetName(ShadingPath, PrecacheParams.PSOCollectorIndex);
			uint64 PSOCompilationTime = PipelineStateCache::GetPrecachePSOCompilationTime(PrecacheParams.bGraphics, PSOPrecacheHash);
								
			FEntry& NewEntry = Entries.AddDefaulted_GetRef();
			NewEntry.MaterialName = PrecacheParams.MaterialName;
			NewEntry.VertexFactoryName = PrecacheParams.VertexFactoryData.VertexFactoryType ? PrecacheParams.VertexFactoryData.VertexFactoryType->GetName() : TEXT("None");
			NewEntry.PassName = PassName != nullptr ? PassName : TEXT("None");
			NewEntry.CustomDefaultDeclarationPSOPrecacheHash = PrecacheParams.VertexFactoryData.CustomDefaultVertexDeclaration != nullptr ? PrecacheParams.VertexFactoryData.CustomDefaultVertexDeclaration->GetPrecachePSOHash() : 0;
			NewEntry.PrecachePSOParams = PrecacheParams.PrecachePSOParams;
			NewEntry.bUsed = bUsed;
			NewEntry.bGraphics = PrecacheParams.bGraphics;
			NewEntry.bRequired = PrecacheParams.bRequired;
			NewEntry.PSOPrecacheHash = PSOPrecacheHash;
			NewEntry.PSOCompilationTime = FPlatformTime::ToMilliseconds64(PSOCompilationTime);

			AllPSOPrecacheParams.AddUnique(PrecacheParams.PrecachePSOParams);
			AllCustomVertexDeclerations.AddUnique(PrecacheParams.VertexFactoryData.CustomDefaultVertexDeclaration);
		}

		Algo::Sort(Entries, [](FEntry& LHS, FEntry& RHS)
		   {
			   if (LHS.MaterialName != RHS.MaterialName)
			   {
				   return LHS.MaterialName < RHS.MaterialName;
			   }
			   
			   if (LHS.VertexFactoryName != RHS.VertexFactoryName)
			   {
				   return LHS.VertexFactoryName < RHS.VertexFactoryName;
			   }
			   
			   if (LHS.PassName != RHS.PassName)
			   {
				   return LHS.PassName < RHS.PassName;
			   }
		
			   return LHS.PrecachePSOParams.Data < RHS.PrecachePSOParams.Data;
		   });

		{
			const FString Filename = FString::Printf(TEXT("%sPSOPrecacheMaterialStats-%s.csv"), *FPaths::ProfilingDir(), *FDateTime::Now().ToString());
			FArchive* CSVFile = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead);
			if (CSVFile != nullptr)
			{
				FString Header(TEXT("Material,VertexFactory,MeshPass,PrecachePSOParams,Used,Type,Required,PSOCompilationTime,CustomVertexDeclarationHash,PSOPrecacheHash\n"));
				CSVFile->Serialize(TCHAR_TO_ANSI(*Header), Header.Len());
				for (FEntry Entry : Entries)
				{
					FString Row = FString::Printf(TEXT("%s,%s,%s,%lld,%s,%s,%s,%.2f,%x,%llx\n"),
												  *Entry.MaterialName,
												  *Entry.VertexFactoryName,
												  *Entry.PassName,
												  Entry.PrecachePSOParams.Data,
												  Entry.bUsed ? TEXT("Used") : TEXT("NotUsed"),
												  Entry.bGraphics ? TEXT("Graphics") : TEXT("Compute"),
												  Entry.bRequired ? TEXT("Yes") : TEXT("No"),
												  Entry.PSOCompilationTime,
												  Entry.CustomDefaultDeclarationPSOPrecacheHash,
												  Entry.PSOPrecacheHash);
					CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
				}
			}
			delete CSVFile;
			CSVFile = nullptr;
		}

		{
			const FString Filename = FString::Printf(TEXT("%sPSOPrecacheCustomVertexDeclarations-%s.log"), *FPaths::ProfilingDir(), *FDateTime::Now().ToString());
			FArchive* CSVFile = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead);
			if (CSVFile != nullptr)
			{				
				for (FRHIVertexDeclaration* VertexDeclaration : AllCustomVertexDeclerations)
				{
					if (VertexDeclaration == nullptr)
					{
						continue;
					}

					TStringBuilder<0> StringBuilder;
					StringBuilder.Appendf(TEXT("VertexDeclaration: %x\n"), VertexDeclaration->GetPrecachePSOHash());
			
					FVertexDeclarationElementList Elements;
					if (VertexDeclaration->GetInitializer(Elements))
					{
						for (FVertexElement& VertexElememt : Elements)
						{
							StringBuilder.Appendf(TEXT("\tStreamIndex: %2d - Offset: %4d - Type: %2d - AttributeIndex: %2d - Stride: %4d - UseInstanceIndex: %d\n"),
												  VertexElememt.StreamIndex,
												  VertexElememt.Offset,
												  VertexElememt.Type.GetValue(),
												  VertexElememt.AttributeIndex,
												  VertexElememt.Stride,
												  VertexElememt.bUseInstanceIndex);
						}
					}
					StringBuilder.Append(TEXT("\n\n"));

					FString Row = StringBuilder.ToString();
					CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
				}
			}
			
			delete CSVFile;
			CSVFile = nullptr;
		}
#endif // PSO_PRECACHING_VALIDATE
	}

#endif // PSO_PRECACHING_TRACKING

private:

	// Request state
	enum class EState : uint8
	{
		Unknown,
		Collecting,
		Compiling,
		Completed,
	};

	struct FPrecacheData
	{
		FMaterialPSOPrecacheRequestID RequestID = INDEX_NONE;
		EState State = EState::Unknown;
		FGraphEventRef CollectionGraphEvent;
		FPSOPrecacheRequestResultArray ActivePSOPrecacheRequests;
		EPSOPrecachePriority Priority;
		TArray<FPSOPrecacheRequestID> CachedPSOPrecacheRequests;
#if PSO_PRECACHING_TRACKING
		FPSOPrecacheDataArray PSOPrecachaData;
#endif // PSO_PRECACHING_TRACKING
	};

	bool CheckCompilingPSOs(FPrecacheData& PrecacheData, bool bBoostPriority)
	{
		check(PrecacheData.State != EState::Unknown);

		// Check if compilation is done
		if (PrecacheData.State == EState::Compiling)
		{
			for (int32 Index = 0; Index < PrecacheData.ActivePSOPrecacheRequests.Num(); ++Index)
			{
				FPSOPrecacheRequestResult& RequestResult = PrecacheData.ActivePSOPrecacheRequests[Index];
				if (!PipelineStateCache::IsPrecaching(RequestResult.RequestID))
				{
					PrecacheData.CachedPSOPrecacheRequests.Add(RequestResult.RequestID);
					PrecacheData.ActivePSOPrecacheRequests.RemoveAtSwap(Index);
					Index--;
				}
				else if (bBoostPriority)
				{
					PipelineStateCache::BoostPrecachePriority(PrecacheData.Priority, RequestResult.RequestID);
				}
			}

			if (PrecacheData.ActivePSOPrecacheRequests.IsEmpty())
			{
				PrecacheData.State = EState::Completed;
				PrecacheData.CollectionGraphEvent = nullptr;
			}
		}

		// Not done yet?
		return (PrecacheData.State != EState::Completed);
	}	

	void MarkCompilationComplete(const FMaterialPSOPrecacheParams& Params, uint32 RequestLifecycleID, FMaterialPSOPrecacheRequestID RequestID)
	{
		FRWScopeLock WriteLock(RWLock, SLT_Write);
		FPrecacheData* FindResult = MaterialPSORequestData.Find(Params);
		if (FindResult && RequestLifecycleID == LifecycleID && RequestID == FindResult->RequestID)
		{
			check(!FindResult->CollectionGraphEvent|| FindResult->CollectionGraphEvent->IsCompleted());
			verifyf(!CheckCompilingPSOs(*FindResult, false /*bBoostPriority*/),
				TEXT("CheckCompilingPSOs should not be active: EState=%d, ActivePSOPrecacheRequests.Num()=%d"),
				(int32)FindResult->State,
				FindResult->ActivePSOPrecacheRequests.Num());
		}
	}

	FRWLock RWLock;
	TArray<FMaterialPSOPrecacheParams> MaterialPSORequests;	
	TMap<FMaterialPSOPrecacheParams, FPrecacheData> MaterialPSORequestData;
	uint32 LifecycleID = 0; //< ID to check current outstanding requests are still valid - incremented on re-precache of all current requests

#if PSO_PRECACHING_TRACKING && PSO_PRECACHING_VALIDATE

	void AddTrackedMaterialData(const FMaterialPSOPrecacheParams& InParams, const FPSOPrecacheDataArray& PrecacheDataArray)
	{
		for (const FPSOPrecacheData& PrecacheData : PrecacheDataArray)
		{
			// Track single entry for each PSO
			uint64 PSOPrecacheHash = 0; 
			if (PrecacheData.Type == FPSOPrecacheData::EType::Graphics)
			{
				PSOPrecacheHash = PSOCollectorStats::GetPSOPrecacheHash(PrecacheData.GraphicsPSOInitializer);
			}
			else
			{
				PSOPrecacheHash = PSOCollectorStats::GetPSOPrecacheHash(PrecacheData.ComputeInitializer);
			}

			if (!TrackingMaterialPSORequestData.Contains(PSOPrecacheHash))
			{				
				FTrackingMaterialPSOPrecacheParams TrackingParams;
				TrackingParams.MaterialName = InParams.Material->GetAssetName();
				TrackingParams.VertexFactoryData = InParams.VertexFactoryData;
				TrackingParams.PrecachePSOParams = InParams.PrecachePSOParams;
				TrackingParams.PSOCollectorIndex = PrecacheData.PSOCollectorIndex;
				TrackingParams.bGraphics = PrecacheData.Type == FPSOPrecacheData::EType::Graphics;
				TrackingParams.bRequired = PrecacheData.bRequired;

				TrackingMaterialPSORequestData.Add(PSOPrecacheHash, TrackingParams);
			}
		}
	}
	
	struct FTrackingMaterialPSOPrecacheParams
	{
		FString MaterialName;
		FPSOPrecacheVertexFactoryData VertexFactoryData;
		FPSOPrecacheParams PrecachePSOParams;
		int32 PSOCollectorIndex : 30;
		bool bGraphics : 1;
		bool bRequired : 1;
	};
	TMap<uint64, FTrackingMaterialPSOPrecacheParams> TrackingMaterialPSORequestData;
#endif // PSO_PRECACHING_TRACKING && PSO_PRECACHING_VALIDATE
};

// The global request manager - only used locally in a few global function to precache, release or boost PSO precache requests
FMaterialPSORequestManager GMaterialPSORequestManager;

FPSOPrecacheRequestResultArray FMaterialPSOPrecacheCollectionTask::CollectPrecacheDataAndStartCompiles(const FMaterialPSOPrecacheParams& PrecacheParams, uint32 LifecycleID)
{
	// Collect pso data
	FPSOPrecacheDataArray PSOPrecacheData;
	if (PrecacheParams.Material->GetGameThreadShaderMap())
	{
		PSOPrecacheData = PrecacheParams.Material->GetGameThreadShaderMap()->CollectPSOPrecacheData(PrecacheParams);
	}

	// Set the correct in memory mode for all the requested PSOs
	for (FPSOPrecacheData& PrecacheData : PSOPrecacheData)
	{
		// Also exclude default material from in memory?
		PrecacheData.InMemoryMode = GPSOPrecacheKeepInMemoryForActiveMaterials ? PrecacheData.InMemoryMode : EPSOPrecacheInMemoryMode::Uncached;
	}

	// Start the async compiles
	FPSOPrecacheRequestResultArray PrecacheResults = RequestPrecachePSOs(EPSOPrecacheType::MeshPass, PSOPrecacheData);

	// Mark collection complete
	GMaterialPSORequestManager.MarkCollectionComplete(PrecacheParams, PSOPrecacheData, PrecacheResults, LifecycleID);

	return MoveTemp(PrecacheResults);
}

void FMaterialPSOPrecacheCollectionTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	LLM_SCOPE(ELLMTag::PSO);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialPSOPrecacheCollectionTask);

#if WITH_ODSC
	FODSCSuspendForceRecompileScope ODSCSuspendForceRecompileScope;
#endif

	// Make sure task is still relevant
	if (RequestLifecycleID != GMaterialPSORequestManager.GetLifecycleID())
	{
		CollectionGraphEvent->Unlock();
		MaterialInterface.Reset();
		return;
	}

	FTaskTagScope ParallelGTScope(ETaskTag::EParallelGameThread);

	FPSOPrecacheRequestResultArray PrecacheResults = CollectPrecacheDataAndStartCompiles(PrecacheParams, RequestLifecycleID);

	// Won't touch the material interface anymore - PSO compile jobs take refs to all RHI resources while creating the task
	MaterialInterface.Reset();

	// Extend MyCompletionGraphEvent to wait for all the async compile events
	if (PrecacheResults.Num() > 0)
	{
		for (FPSOPrecacheRequestResult& Result : PrecacheResults)
		{
			check(Result.IsValid());
			CollectionGraphEvent->AddPrerequisites(Result.AsyncCompileEvent);
		}
	}

	CollectionGraphEvent->Unlock();
}

void FShaderMapPreloadTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	LLM_SCOPE(ELLMTag::PSO);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderMapPreloadTask);

	FTaskTagScope ParallelGTScope(ETaskTag::EParallelGameThread);

	if (MaterialShaderMap)
	{
		FGraphEventArray OutCompletionEvents;
		MaterialShaderMap->GetResource()->PreloadShaderMap(OutCompletionEvents);
		ShaderPreloadEvents->AddPrerequisites(OutCompletionEvents);
	}

	ShaderPreloadEvents->Unlock();
}

void FShaderPreloadCollectionTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	LLM_SCOPE(ELLMTag::PSO);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderPreloadCollectionTask);

	FTaskTagScope ParallelGTScope(ETaskTag::EParallelGameThread);

	TArray<TShaderRef<FShader>> UniqueShaderRequests;
	TMap<int32, TArray<int32>> UniqueShaderLibraryMap;

	FPSOPrecacheDataArray PSOPrecacheDataArray;
	if (PrecacheParams.Material->GetGameThreadShaderMap())
	{
		PSOPrecacheDataArray = PrecacheParams.Material->GetGameThreadShaderMap()->CollectPSOPrecacheData(PrecacheParams);
	}

	for (const FPSOPrecacheData& PrecacheData : PSOPrecacheDataArray)
	{
		// Gather unique shader requests
		for (const TShaderRef<FShader>& Shader : PrecacheData.ShaderPreloadData.Shaders)
		{
			TArray<int32>& ShaderLibraryIndexes = UniqueShaderLibraryMap.FindOrAdd(Shader.GetResource()->GetLibraryId());
			const int32 ShaderGroupIndex = Shader.GetResource()->GetLibraryShaderIndex(Shader->GetResourceIndex());

			if (ShaderLibraryIndexes.Find(ShaderGroupIndex) == INDEX_NONE)
			{
				ShaderLibraryIndexes.Add(ShaderGroupIndex);
				UniqueShaderRequests.Add(Shader);
			}
		}
	}

	for (TShaderRef<FShader>& Shader : UniqueShaderRequests)
	{
		// Preload shaders. This will issue IO requests if they haven't been
		// preloaded yet.
		FGraphEventArray OutCompletionEvents;
		Shader.GetResource()->PreloadShader(Shader->GetResourceIndex(), OutCompletionEvents);
		ShaderPreloadEvents->AddPrerequisites(OutCompletionEvents);
	}

	ShaderPreloadEvents->Unlock();
}

void PrecacheMaterialPSOs(const FMaterialInterfacePSOPrecacheParamsList& PSOPrecacheParamsList, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSOPrecacheRequestIDs, FGraphEventArray& OutGraphEvents)
{
	for (const FMaterialInterfacePSOPrecacheParams& MaterialPSOPrecacheParams : PSOPrecacheParamsList)
	{
		if (MaterialPSOPrecacheParams.MaterialInterface)
		{
			OutGraphEvents.Append(MaterialPSOPrecacheParams.MaterialInterface->PrecachePSOs(MaterialPSOPrecacheParams.VertexFactoryDataList, MaterialPSOPrecacheParams.PSOPrecacheParams, MaterialPSOPrecacheParams.Priority, OutMaterialPSOPrecacheRequestIDs));
		}
	}
}

void PreloadMaterialShaderMap(const FMaterial* Material, FGraphEventArray& OutGraphEvents)
{
	return GMaterialPSORequestManager.PreloadShaderMap(Material, OutGraphEvents);
}

FMaterialPSOPrecacheRequestID PrecacheMaterialPSOs(const FMaterialPSOPrecacheParams& MaterialPSOPrecacheParams, EPSOPrecachePriority Priority, FGraphEventArray& GraphEvents)
{
	return GMaterialPSORequestManager.PrecachePSOs(MaterialPSOPrecacheParams, Priority, GraphEvents);
}

void ReleasePSOPrecacheData(const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs)
{
	for (FMaterialPSOPrecacheRequestID RequestID : MaterialPSORequestIDs)
	{
		GMaterialPSORequestManager.ReleaseMaterialPrecacheData(RequestID);
	}
}

FGraphEventArray GetPrecacheRequestsCompletePrereqs(const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetPrecacheRequestsCompletePrereqs);

	FGraphEventArray Prereqs;
	for (FMaterialPSOPrecacheRequestID RequestID : MaterialPSORequestIDs)
	{
		Prereqs.Append(GMaterialPSORequestManager.GetPrecacheCompletePrereq(RequestID));
	}
	return Prereqs;
}

#if CSV_PROFILER_STATS
CSV_DECLARE_CATEGORY_MODULE_EXTERN(RHI_API, PSOPrecacheCompiling);

struct FMaterialPSOPrecacheStats
{
	TCsvPersistentCustomStat<int>* ProxiesWaitingForMaterialPSOs = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("ProxiesWaitingForMaterialPSOs"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));

	static FMaterialPSOPrecacheStats& GetStats()
	{
		static FMaterialPSOPrecacheStats PersistentStats;
		return PersistentStats;
	}

	static bool AreEnabled()
	{
		return FCsvProfiler::Get() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	}
};
#endif

void BoostPSOPriorityOnDraw(const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BoostPSOPriorityOnDraw);

	constexpr EPSOPrecachePriority NewPri = EPSOPrecachePriority::Highest;

	// if any of the requests are still in progress then we can consider this to be default material usage.
	FGraphEventArray AllRequests = GetPrecacheRequestsCompletePrereqs(MaterialPSORequestIDs);
	if(!AllRequests.IsEmpty())
	{ 
#if CSV_PROFILER_STATS
		if (FMaterialPSOPrecacheStats::AreEnabled())
		{
			FMaterialPSOPrecacheStats::GetStats().ProxiesWaitingForMaterialPSOs->Add(1);

			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[]
				{
					FMaterialPSOPrecacheStats::GetStats().ProxiesWaitingForMaterialPSOs->Add(-1);
				},
				TStatId{}, &AllRequests
			);
		}
#endif
 		BoostPSOPriority(NewPri, MaterialPSORequestIDs);
	}
}

void BoostPSOPriority(EPSOPrecachePriority NewPri, const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BoostPSOPriority);

	for (FMaterialPSOPrecacheRequestID RequestID : MaterialPSORequestIDs)
	{
		GMaterialPSORequestManager.BoostPriority(NewPri, RequestID);
	}
}

void ClearMaterialPSORequests()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ClearMaterialPSORequests);

	return GMaterialPSORequestManager.ClearMaterialPSORequests();
}

bool KeepPrecachedPSOsInMemoryForActiveMaterials()
{
	return GPSOPrecacheKeepInMemoryForActiveMaterials > 0;
}

#if PSO_PRECACHING_TRACKING

FMaterialPSOPrecacheParams GetMaterialPSOPrecacheParams(FMaterialPSOPrecacheRequestID RequestID)
{
	return GMaterialPSORequestManager.GetMaterialPSOPrecacheParams(RequestID);
}

FPSOPrecacheDataArray GetMaterialPSOPrecacheData(FMaterialPSOPrecacheRequestID RequestID)
{
	return GMaterialPSORequestManager.GetMaterialPSOPrecacheData(RequestID);
}

void DumpPSOPrecacheMaterialStats()
{		
	GMaterialPSORequestManager.DumpPSOPrecacheMaterialStats();
}

uint64 GetPSOPrecacheMaterialCompileTime(const TCHAR* InPassName, const TCHAR* InVertexFactoryName)
{
	return GMaterialPSORequestManager.GetCompileTime(InPassName, InVertexFactoryName);
}

static FAutoConsoleCommand GDumpPSOPrecacheMaterialStatsCmd(
	TEXT("r.PSOPrecache.DumpMaterialStats"),
	TEXT("Dumps all currently collected material based PSO precache data to csv file"),
	FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
{		
	DumpPSOPrecacheMaterialStats();
}));

#else

FMaterialPSOPrecacheParams GetMaterialPSOPrecacheParams(FMaterialPSOPrecacheRequestID RequestID)
{
	return FMaterialPSOPrecacheParams();
}

FPSOPrecacheDataArray GetMaterialPSOPrecacheData(FMaterialPSOPrecacheRequestID RequestID)
{
	return FPSOPrecacheDataArray();
}

void DumpPSOPrecacheMaterialStats()
{
	// Do nothing
}

uint64 GetPSOPrecacheMaterialCompileTime(const TCHAR* InPassName, const TCHAR* InVertexFactoryName)
{
	return 0;
}

#endif // PSO_PRECACHING_TRACKING
