// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheSettings.cpp: 
=============================================================================*/

#include "PSOPrecacheSettings.h"
#include "HAL/IConsoleManager.h"
#include "RHIShaderPlatform.h"
#include "RHIStrings.h"
#include "Engine/RendererSettings.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"

static int32 GPSOPrecacheGlobalShaders = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheGlobalShaders(
	TEXT("r.PSOPrecache.GlobalShaders"),
	GPSOPrecacheGlobalShaders,
	TEXT("Precache global shaders during startup (disable(0) - global compute shaders and registered global graphis PSOs(1) - all global shaders(2).\n") 
	TEXT("Note: r.PSOPrecache.GlobalShaders == 2 is only supported when r.PSOPrecache.Mode == 1 (Preload shaders)."),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheKeepGlobalGraphicPSOsInMemory = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepGlobalGraphicPSOsInMemory(
	TEXT("r.PSOPrecache.KeepGlobalGraphicPSOsInMemory"),
	GPSOPrecacheKeepGlobalGraphicPSOsInMemory,
	TEXT("Keep global graphic PSOs in memory when r.PSOPrecache.KeepInMemoryForActiveMaterials is enabled"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheKeepGlobalComputePSOsInMemory = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepGlobalComputePSOsInMemory(
	TEXT("r.PSOPrecache.KeepGlobalComputePSOsInMemory"),
	GPSOPrecacheKeepGlobalComputePSOsInMemory,
	TEXT("Keep global compute PSOs in memory when r.PSOPrecache.KeepInMemoryForActiveMaterials is enabled"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheLightMapPolicyMode = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheLightMapPolicyMode(
	TEXT("r.PSOPrecache.LightMapPolicyMode"),
	GPSOPrecacheLightMapPolicyMode,
	TEXT("Defines which light map policies should be checked during PSO precaching of the base pass.\n") \
	TEXT("-1: Use engine defaults.\n") \
	TEXT(" 0: All possible LMP will be checked.\n") \
	TEXT(" 1: Only LMP_NO_LIGHTMAP will be precached.\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheTranslucencyAllPass = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheTranslucencyAllPass(
	TEXT("r.PSOPrecache.TranslucencyAllPass"),
	GPSOPrecacheTranslucencyAllPass,
	TEXT("Precache PSOs for TranslucencyAll pass.\n") \
	TEXT("-1: Use engine defaults.\n") \
	TEXT(" 0: No PSOs are compiled for this pass.\n") \
	TEXT(" 1: PSOs are compiled for all primitives which render to a translucency pass.\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheTranslucencyStandardBeforeDOF = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheTranslucencyStandardBeforeDOF(
	TEXT("r.PSOPrecache.TranslucencyStandardBeforeDOF"),
	GPSOPrecacheTranslucencyStandardBeforeDOF,
	TEXT("Precache permutations for EMeshPass::TranslucencyStandard even when marked as AfterDOF - can be needed when AutoBeforeDOFTranslucencyBoundary is set on the view:\n") \
	TEXT("-1: Use engine defaults.\n") \
	TEXT(" 0: No PSOs are compiled for this use case.\n") \
	TEXT(" 1: PSOs are compiled for all primitives which render to a translucency standard and are marked for AfterDOF.\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheAlphaColorChannel = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheAlphaColorChannel(
	TEXT("r.PSOPrecache.PrecacheAlphaColorChannel"),
	GPSOPrecacheAlphaColorChannel,
	TEXT("Also Precache PSOs with scene color alpha channel enabled. Planar reflections and scene captures use this for compositing into a different scene later (-1 use engine default)") \
	TEXT("-1: Use engine defaults.\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheTranslucencyUnderWater = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheTranslucencyUnderWater(
	TEXT("r.PSOPrecache.PrecacheTranslucencyUnderWater"),
	GPSOPrecacheTranslucencyUnderWater,
	TEXT("Precache translucency under water pass permutations with different render target formats (-1 use engine default)"),
	ECVF_ReadOnly
);

static int32 GPSOPrecachePostProcessingMaterial = -1;
static FAutoConsoleVariableRef CVarPSOPrecachePostProcessingMaterial(
	TEXT("r.PSOPrecache.PostProcessingMaterial"), 
	GPSOPrecachePostProcessingMaterial, 
	TEXT("Precache all possible required PSOs for loaded PostProcessing Materials (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheTranslucencyLightingVolumeMaterial = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheTranslucencyLightingVolumeMaterial(
	TEXT("r.PSOPrecache.TranslucencyLightingVolumeMaterial"), 
	GPSOPrecacheTranslucencyLightingVolumeMaterial, 
	TEXT("Precache all possible required Translucency Lighting Volume PSOs for loaded LightMaterials (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheDitheredLODFadingOutMaskPass = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheDitheredLODFadingOutMaskPass(
	TEXT("r.PSOPrecache.DitheredLODFadingOutMaskPass"),
	GPSOPrecacheDitheredLODFadingOutMaskPass,
	TEXT("Precache PSOs for DitheredLODFadingOutMaskPass.\n") \
	TEXT("-1: Use engine defaults.\n") \
	TEXT(" 0: No PSOs are compiled for this pass.\n") \
	TEXT(" 1: PSOs are compiled for all primitives which render to depth pass.\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheProjectedShadows = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheProjectedShadows(
	TEXT("r.PSOPrecache.ProjectedShadows"),
	GPSOPrecacheProjectedShadows,
	TEXT("Also Precache PSOs with for projected shadows.") \
	TEXT("-1: Use engine defaults.\n") \
	TEXT(" 0: No PSOs are compiled for projected shadows.\n") \
	TEXT(" 1: PSOs are compiled for all primitives which render to depth pass when VSMs are disabled).\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheCustomDepth = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheCustomDepth(
	TEXT("r.PSOPrecache.CustomDepth"),
	GPSOPrecacheCustomDepth,
	TEXT("Also Precache PSOs with for custom depth pass.") \
	TEXT("-1: Use engine defaults.\n") \
	TEXT(" 0: No PSOs are compiled for this pass.\n") \
	TEXT(" 1: PSOs are compiled for all primitives which explicitly request custom depth rendering (PSOs are precached again on custom depth toggle on component).\n") \
	TEXT(" 2: PSOs are compiled for all primitives which also request regular depth rendering.\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheDeferredDecalMaterials = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheDeferredDecalMaterials(
	TEXT("r.PSOPrecache.DeferredDecalMaterials"), 
	GPSOPrecacheDeferredDecalMaterials, 
	TEXT("Precache all possible required PSOs for loaded DeferredDecalPass Materials (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheParticlePrecachingTime = -1;
static FAutoConsoleVariableRef CVarPSOPrecachingParticlePrecachingTime(
	TEXT("r.PSOPrecache.ParticlePrecachingTime"),
	GPSOPrecacheParticlePrecachingTime,
	TEXT("Controls when PSO precaching happens for particles systems:\n") \
	TEXT(" -1: Use engine defaults.\n")
	TEXT("	0: no precaching\n")
	TEXT("	1: precaching at component proxy creation time only\n")
	TEXT("	2: precaching at component loading time\n")
	TEXT("	3: precaching at asset loading time and component time"),
	ECVF_Default);

static int32 GPSOPrecacheWaterInfoDepthPass = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheWaterInfoDepthPass(
	TEXT("r.PSOPrecache.WaterInfoDepthPass"),
	GPSOPrecacheWaterInfoDepthPass,
	TEXT("Precaches PSOs for WaterInfoDepthPass (-1 use engine default)\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheLandscapeGrassWeightPass = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheWLandscapeGrassWeightPass(
	TEXT("r.PSOPrecache.LandscapeGrassWeightPass"),
	GPSOPrecacheLandscapeGrassWeightPass,
	TEXT("Precaches PSOs for LandscapeGrassWeightPass (-1 use engine default)\n"),
	ECVF_ReadOnly
);

static int32 GPSOPrecacheSlateMaterials = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheSlateMaterials(
	TEXT("r.PSOPrecache.SlateMaterials"), 
	GPSOPrecacheSlateMaterials, 
	TEXT("Precache all possible required PSOs for loaded Slate Materials (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheSkipSlateMaterialDrawOnPSOTooLate = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheSkipSlateMaterialDrawOnPSOTooLate(
	TEXT("r.PSOPrecache.SkipSlateMaterialDrawOnPSOTooLate"), 
	GPSOPrecacheSkipSlateMaterialDrawOnPSOTooLate, 
	TEXT("Skip draw of slate material batches when material based PSOs are still compiling and r.PSOPrecache.ProxyCreationStrategy != 0 (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheKeepSlatePSOsInMemoryForActiveMaterials = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheKeepSlatePSOsInMemoryForActiveMaterials(
	TEXT("r.PSOPrecache.KeepSlatePSOsInMemoryForActiveMaterials"), 
	GPSOPrecacheKeepSlatePSOsInMemoryForActiveMaterials, 
	TEXT("Keep slate PSOs in memory when r.PSOPrecache.KeepInMemoryForActiveMaterials is enabled (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheNaniteProgrammableRasterRequired = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheNaniteProgrammableRasterRequired(
	TEXT("r.PSOPrecache.NaniteProgrammableRasterRequired"), 
	GPSOPrecacheNaniteProgrammableRasterRequired, 
	TEXT("Make nanite raster programmable raster PSOs required before the component can be rendered with desired material or otherwise fallback to fixed function pipeline untill programmable raster PSOs are ready (-1 use engine default)"),
	ECVF_ReadOnly);

static int32 GPSOPrecacheNaniteLumenCardRequired = -1;
static FAutoConsoleVariableRef CVarPSOPrecacheNaniteLumenCardRequired(
	TEXT("r.PSOPrecache.NaniteLumenCardRequired"), 
	GPSOPrecacheNaniteLumenCardRequired, 
	TEXT("Make nanite lumen card PSOs required before the component can be rendered with desired material or otherwise mesh is not added to LumenScene until PSOs are compiled (-1 use engine default)"),
	ECVF_ReadOnly);

static void ApplyPSOPrecacheSettingsCVarOverrides(FPSOPrecacheSettings& PSOPrecacheSettings)
{
#define SET_CVAR_VALUE(Value, MemberName) \
	if (Value >= 0) \
	{ \
		PSOPrecacheSettings.MemberName = Value; \
	}
#define SET_CVAR_VALUE_BOOL(Value, MemberName) \
	if (Value >= 0) \
	{ \
		PSOPrecacheSettings.MemberName = Value > 0; \
	}

	// Reverse handling of cvar value to keep old naming working
	if (GPSOPrecacheLightMapPolicyMode >= 0)
	{
		PSOPrecacheSettings.bBasePassApplyLightMapPolicies = GPSOPrecacheLightMapPolicyMode == 0;
	}

	SET_CVAR_VALUE(GPSOPrecacheGlobalShaders, GlobalShaderMode);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheKeepGlobalGraphicPSOsInMemory, bKeepGlobalGraphicPSOsInMemory);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheKeepGlobalComputePSOsInMemory, bKeepGlobalComputePSOsInMemory);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheTranslucencyAllPass, bTranslucencyAll);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheTranslucencyStandardBeforeDOF, bTranslucencyStandardBeforeDOF);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheAlphaColorChannel, bBasePassColorWithAlphaChannel);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheTranslucencyUnderWater, bTranslucencyUnderWater);
	SET_CVAR_VALUE_BOOL(GPSOPrecachePostProcessingMaterial, bPostProcessingMaterial);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheTranslucencyLightingVolumeMaterial, bTranslucencyLightingVolume);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheDitheredLODFadingOutMaskPass, bDitheredLODFadingOutMask);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheProjectedShadows, bDepthProjectedShadowsState);
	SET_CVAR_VALUE(GPSOPrecacheCustomDepth, CustomDepthMode);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheDeferredDecalMaterials, bDeferredDecals);
	SET_CVAR_VALUE(GPSOPrecacheParticlePrecachingTime, ParticlePrecachingTime);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheWaterInfoDepthPass, bWaterInfoDepth);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheLandscapeGrassWeightPass, bLandscapeGrassWeight);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheSlateMaterials, bSlateMaterials);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheSkipSlateMaterialDrawOnPSOTooLate, bSkipSlateMaterialDrawOnPSOTooLate);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheKeepSlatePSOsInMemoryForActiveMaterials, bKeepSlatePSOsInMemoryForActiveMaterials);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheNaniteProgrammableRasterRequired, bNaniteProgrammableRasterRequired);
	SET_CVAR_VALUE_BOOL(GPSOPrecacheNaniteLumenCardRequired, bNaniteLumenCardRequired);

#undef SET_CVAR_VALUE
}

const FPSOPrecacheSettings& UPSOPrecacheSettingsManager::GetSettings()
{
	static FPSOPrecacheSettings PSOPrecacheSettings = []()
	{
		// Use default value is no specific setting can be found
		FPSOPrecacheSettings Settings;

		const UPSOPrecacheSettingsManager& Manager = *GetDefault<UPSOPrecacheSettingsManager>();

		// Try and find settings for current shader platform and lowest matching CPU thread count
		FString ShaderPlatformName = LexToString(GMaxRHIShaderPlatform, true);
		int32 NumThreads = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
		for (const FPSOPrecacheSettings& IterSettings : Manager.Settings)
		{
			if (IterSettings.ShaderPlatform == ShaderPlatformName && NumThreads <= IterSettings.MaxCPUThreadCount && IterSettings.MaxCPUThreadCount <= Settings.MaxCPUThreadCount)
			{
				Settings = IterSettings;
			}
		}

		ApplyPSOPrecacheSettingsCVarOverrides(Settings);

		return Settings;
	}();
	return PSOPrecacheSettings;
}

TStrongObjectPtr<UMaterial> UPSOPrecacheSettingsManager::FallbackMaterial;

void UPSOPrecacheSettingsManager::SetupFallbackMaterial()
{
#if UE_WITH_PSO_PRECACHING
	UMaterial* Material = UMaterial::GetDefaultMaterial(MD_Surface);

	const URendererSettings* RendererSettings = GetDefault<URendererSettings>();
	check(RendererSettings);
	if (RendererSettings->PSOPrecacheFallbackMaterial.IsValid())
	{
		if (UMaterial* SettingsMaterial = Cast<UMaterial>(RendererSettings->PSOPrecacheFallbackMaterial.TryLoad()))
		{
			Material = SettingsMaterial;
		}
	}

	UMaterialInterface::PrecacheFallbackMaterialPSOs(Material);

	FallbackMaterial = TStrongObjectPtr(Material);
#endif // UE_WITH_PSO_PRECACHING
}

UMaterial* UPSOPrecacheSettingsManager::GetFallbackMaterial()
{
	return FallbackMaterial.Get();
}
