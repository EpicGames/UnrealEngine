// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheValidation.cpp: 
=============================================================================*/

#include "PSOPrecacheValidation.h"
#include "Materials/MaterialRenderProxy.h"
#include "PrimitiveSceneProxy.h"
#include "UnrealEngine.h"
#include "Materials/Material.h"
#include "PSOPrecacheMaterial.h"
#include "HAL/FileManager.h"

#if PSO_PRECACHING_VALIDATE && UE_WITH_PSO_PRECACHING

// CSV category for PSOPrecache
CSV_DEFINE_CATEGORY(PSOPrecache, true);

/**
* Different IHVs and drivers can have different opinions on what subset of a PSO
* matters for caching. We track multiple PSO subsets, ranging from shaders-only
* to the full PSO created by the RHI. Generally, a precaching miss on a minimal
* state has worse consequences than a miss on the full state.
*/
DECLARE_STATS_GROUP(TEXT("PSOPrecache"), STATGROUP_PSOPrecache, STATCAT_Advanced);

// Stats tracking shaders-only PSOs (everything except shaders is ignored).
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Shaders-only PSO Precache Miss Count"), STAT_ShadersOnlyPSOPrecacheMissCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Shaders-only PSO Precache Miss (Untracked) Count"), STAT_ShadersOnlyPSOPrecacheUntrackedCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Shaders-only PSO Precache Hit Count"), STAT_ShadersOnlyPSOPrecacheHitCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Shaders-only PSO Precache Used Count"), STAT_ShadersOnlyPSOPrecacheUsedCount, STATGROUP_PSOPrecache);

// Stats tracking minimal PSOs (render target state is ignored).
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Minimal PSO Precache Miss Count"), STAT_MinimalPSOPrecacheMissCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Minimal PSO Precache Miss (Untracked) Count"), STAT_MinimalPSOPrecacheUntrackedCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Minimal PSO Precache Hit Count"), STAT_MinimalPSOPrecacheHitCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Minimal PSO Precache Used Count"), STAT_MinimalPSOPrecacheUsedCount, STATGROUP_PSOPrecache);

// Stats tracking full PSOs (the ones used by the RHI).
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Full PSO Precache Miss Count"), STAT_FullPSOPrecacheMissCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Full PSO Precache Miss (Untracked) Count"), STAT_FullPSOPrecacheUntrackedCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Full PSO Precache Hit Count"), STAT_FullPSOPrecacheHitCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Full PSO Precache Used Count"), STAT_FullPSOPrecacheUsedCount, STATGROUP_PSOPrecache);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Full PSO Precache Too Late Count"), STAT_FullPSOPrecacheTooLateCount, STATGROUP_PSOPrecache);

int32 GPSOPrecachingValidationMode = 0;
static FAutoConsoleVariableRef CVarValidatePSOPrecaching(
	TEXT("r.PSOPrecache.Validation"),
	GPSOPrecachingValidationMode,
	TEXT("Validate whether runtime PSOs are correctly precached and optionally track information per pass type, vertex factory and cache hit state.\n")
	TEXT(" 0: disabled (default)\n")
	TEXT(" 1: lightweight tracking (counter stats)\n")
	TEXT(" 2: full tracking (counter stats by vertex factory, mesh pass type) and detailed logging on PSO misses in development builds\n"),
	ECVF_ReadOnly
);

FString GPSOPrecachingBreakOnMaterialName;
static FAutoConsoleVariableRef CVarPSOPrecachingBreakOnMaterialName(
	TEXT("r.PSOPrecache.BreakOnMaterialName"),
	GPSOPrecachingBreakOnMaterialName,
	TEXT("Debug break when PSO precaching the requested material.\n")
	TEXT("Use in combination with r.PSOPrecache.BreakOnPassName to break only on a certain PSO Collector Pass name.\n"),
	ECVF_ReadOnly
);

FString GPSOPrecachingBreakOnPassName;
static FAutoConsoleVariableRef CVarPSOPrecachingBreakOnPSOCollectorIndex(
	TEXT("r.PSOPrecache.BreakOnPassName"),
	GPSOPrecachingBreakOnPassName,
	TEXT("Debug break when PSO precaching the material set with r.PSOPrecache.BreakOnMaterialName and the given PSO Collector Pass name.\n"),
	ECVF_ReadOnly
);

FString GPSOPrecachingBreakOnShaderHash;
static FAutoConsoleVariableRef CVarPSOPrecachingBreakOnShaderHash(
	TEXT("r.PSOPrecache.BreakOnShaderHash"),
	GPSOPrecachingBreakOnShaderHash,
	TEXT("Debug break when PSO precaching the requested shader hash.\n"),
	ECVF_ReadOnly
);

FString GPSOPrecachingBreakValidationLogOnShaderHash;
static FAutoConsoleVariableRef CVarPSOPrecachingBreakValidationLogOnShaderHash(
	TEXT("r.PSOPrecache.BreakValidationLogOnShaderHash"),
	GPSOPrecachingBreakValidationLogOnShaderHash,
	TEXT("Debug break when PSO miss info logs the requested shader hash.\n"),
	ECVF_ReadOnly
);

void ConditionalBreakOnPSOPrecacheMaterial(const FMaterial& Material, int32 PSOCollectorIndex)
{
	if (!GPSOPrecachingBreakOnMaterialName.IsEmpty())
	{
		int PSOCollectorIndexToDebug = FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), *GPSOPrecachingBreakOnPassName);
		FString MaterialName = Material.GetAssetName();
		if (MaterialName == GPSOPrecachingBreakOnMaterialName && PSOCollectorIndex == PSOCollectorIndexToDebug)
		{
			UE_DEBUG_BREAK();
		}
	}
}

void ConditionalBreakOnPSOPrecacheShader(const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer)
{
	if (!GPSOPrecachingBreakOnShaderHash.IsEmpty())
	{
		FShaderHash ShaderHash;
		ShaderHash.FromString(GPSOPrecachingBreakOnShaderHash);
		const auto MatchShaderHash = [&](FRHIShader* RHIShader)
		{
			return RHIShader ? RHIShader->GetHash() == ShaderHash : false;
		};

		if (MatchShaderHash(GraphicsPSOInitializer.BoundShaderState.GetVertexShader())
			|| MatchShaderHash(GraphicsPSOInitializer.BoundShaderState.GetPixelShader())
			|| MatchShaderHash(GraphicsPSOInitializer.BoundShaderState.GetGeometryShader())
			|| MatchShaderHash(GraphicsPSOInitializer.BoundShaderState.GetMeshShader())
			|| MatchShaderHash(GraphicsPSOInitializer.BoundShaderState.GetAmplificationShader()))
		{
			UE_DEBUG_BREAK();
		}
	}
}

void ConditionalBreakOnPSOPrecacheShader(const FComputePipelineStateInitializer& ComputeInitializer)
{
	if (!GPSOPrecachingBreakOnShaderHash.IsEmpty() && ComputeInitializer.ComputeShader)
	{
		FShaderHash ShaderHash;
		ShaderHash.FromString(GPSOPrecachingBreakOnShaderHash);
		if (ComputeInitializer.ComputeShader->GetHash() == ShaderHash)
		{
			UE_DEBUG_BREAK();
		}
	}
}

void ConditionalBreakOnLogPSOMissInfoShader(FRHIShader* RHIShader)
{
	if (!GPSOPrecachingBreakValidationLogOnShaderHash.IsEmpty())
	{
		FShaderHash ShaderHash;
		ShaderHash.FromString(GPSOPrecachingBreakValidationLogOnShaderHash);
		const auto MatchShaderHash = [&](FRHIShader* RHIShader)
		{
			return RHIShader ? RHIShader->GetHash() == ShaderHash : false;
		};

		if (MatchShaderHash(RHIShader))
		{
			UE_DEBUG_BREAK();
		}
	}
}

PSOCollectorStats::EPSOPrecacheValidationMode PSOCollectorStats::GetPrecachingValidationMode()
{
	switch (GPSOPrecachingValidationMode)
	{
	case 1:
		return EPSOPrecacheValidationMode::Lightweight;
	case 2:
		return EPSOPrecacheValidationMode::Full;
	case 0:
		// Passthrough.
	default:
		return EPSOPrecacheValidationMode::Disabled;
	}
}

bool PSOCollectorStats::IsPrecachingValidationEnabled()
{
	return (GetPSOPrecacheMode() == EPSOPrecacheMode::PSO) && PipelineStateCache::IsPSOPrecachingEnabled() && GetPrecachingValidationMode() != EPSOPrecacheValidationMode::Disabled;
}

bool PSOCollectorStats::IsFullPrecachingValidationEnabled()
{
	return (GetPSOPrecacheMode() == EPSOPrecacheMode::PSO) && PipelineStateCache::IsPSOPrecachingEnabled() && GetPrecachingValidationMode() == EPSOPrecacheValidationMode::Full;
}

void PSOCollectorStats::FPrecacheUsageData::Empty()
{
	if (!StatFName.IsNone())
	{
		SET_DWORD_STAT_FName(StatFName, 0)
	}

	FPlatformAtomics::InterlockedExchange(&Count, 0);

	if (IsFullPrecachingValidationEnabled())
	{
		FScopeLock Lock(&StatsLock);

		FMemory::Memzero(PerMeshPassCount, FPSOCollectorCreateManager::MaxPSOCollectorCount * sizeof(uint32));
		UntrackedMeshPassCount = 0;		
		PerVertexFactoryCount.Empty();
		FMemory::Memzero(PerGlobalTypeCount, FGlobalPSOCollectorManager::MaxPSOCollectorCount * sizeof(uint32));
		UntrackedGlobalTypeCount = 0;
	}
}

void PSOCollectorStats::FPrecacheUsageData::UpdateStats(EPSOPrecacheType PrecacheType, int32 PSOCollectorIndex, const FVertexFactoryType* VFType)
{
	if (!StatFName.IsNone())
	{
		INC_DWORD_STAT_FName(StatFName);
	}

	FPlatformAtomics::InterlockedIncrement(&Count);

	if (ShouldRecordFullStats(PrecacheType, PSOCollectorIndex, VFType))
	{
		FScopeLock Lock(&StatsLock);

		if (PrecacheType == EPSOPrecacheType::MeshPass)
		{
			if (PSOCollectorIndex != INDEX_NONE && PSOCollectorIndex < FPSOCollectorCreateManager::MaxPSOCollectorCount)
			{
				PerMeshPassCount[PSOCollectorIndex]++;
			}
			else
			{
				UntrackedMeshPassCount++;
			}

			if (VFType != nullptr)
			{
				uint32* Value = PerVertexFactoryCount.FindOrAdd(VFType, 0);
				(*Value)++;
			}
		}
		else
		{
			if (PSOCollectorIndex != INDEX_NONE && PSOCollectorIndex < FGlobalPSOCollectorManager::MaxPSOCollectorCount)
			{
				PerGlobalTypeCount[PSOCollectorIndex]++;
			}
			else
			{
				UntrackedGlobalTypeCount++;
			}
		}
	}
}

uint64 PSOCollectorStats::FPrecacheUsageData::GetTotalMeshPassCount()
{
	FScopeLock Lock(&StatsLock);
	uint64 TotalCount = UntrackedMeshPassCount;
	for (int CollectorIndex = 0; CollectorIndex < FPSOCollectorCreateManager::MaxPSOCollectorCount; ++CollectorIndex)
	{
		TotalCount += PerMeshPassCount[CollectorIndex];
	}
	return TotalCount;
}

uint64 PSOCollectorStats::FPrecacheUsageData::GetTotalGlobalCount()
{
	FScopeLock Lock(&StatsLock);
	uint64 TotalCount = UntrackedGlobalTypeCount;
	for (int CollectorIndex = 0; CollectorIndex < FGlobalPSOCollectorManager::MaxPSOCollectorCount; ++CollectorIndex)
	{
		TotalCount += PerGlobalTypeCount[CollectorIndex];
	}
	return TotalCount;
}

bool PSOCollectorStats::FPrecacheStatsCollector::IsStateTracked(EPSOPrecacheType PrecacheType, int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType) 
{
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	// mark everything tracked in shipping and test because collected index and VF type might be missing from MDCs at draw time so can't be validated
	// allows for better tracking of TooLate state
	return true;
#else
	bool bTracked = PSOCollectorIndex != INDEX_NONE;
	if (PrecacheType == EPSOPrecacheType::MeshPass)
	{
		if (PSOCollectorIndex != INDEX_NONE && PSOCollectorIndex < FPSOCollectorCreateManager::MaxPSOCollectorCount)
		{
			const EShadingPath ShadingPath = GetFeatureLevelShadingPath(GMaxRHIFeatureLevel);
			bool bCollectPSOs = FPSOCollectorCreateManager::GetCreateFunction(ShadingPath, PSOCollectorIndex) != nullptr;
			bTracked = bCollectPSOs && (VertexFactoryType == nullptr || VertexFactoryType->SupportsPSOPrecaching());
		}
	}
	else
	{
		bTracked = bTracked && PSOCollectorIndex < FGlobalPSOCollectorManager::MaxPSOCollectorCount;
	}
	return bTracked;
#endif
}

bool PSOCollectorStats::FPrecacheStatsCollector::UpdatePrecacheStats(EPSOPrecacheType PrecacheType, uint64 PrecacheHash, int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType, bool bTracked, EPSOPrecacheResult& InOutPrecacheResult)
{
	EPSOPrecacheResult OutResult = EPSOPrecacheResult::Unknown;

	// Only update stats once per state.
	bool bUpdateStats = false;
	bool bWasPrecached = false;
	{
		FScopeLock Lock(&StateMapLock);
		FShaderStateUsage* Value = HashedStateMap.FindOrAdd(PrecacheHash, FShaderStateUsage());
		if (!Value->bUsed)
		{
			Value->bUsed = true;

			bUpdateStats = true;
			bWasPrecached = Value->bPrecached;
		}
	}	

	// Make sure something was precached already
	bUpdateStats = bUpdateStats && Stats.PrecacheData.Count > 0;
	if (bUpdateStats)
	{
		Stats.UsageData.UpdateStats(PrecacheType, PSOCollectorIndex, VertexFactoryType);

		if (!bTracked)
		{
			Stats.UntrackedData.UpdateStats(PrecacheType, PSOCollectorIndex, VertexFactoryType);
			OutResult = EPSOPrecacheResult::Untracked;
		}
		else if (!bWasPrecached)
		{
			Stats.MissData.UpdateStats(PrecacheType, PSOCollectorIndex, VertexFactoryType);
			OutResult = EPSOPrecacheResult::Missed;
		}
		else if (InOutPrecacheResult == EPSOPrecacheResult::Active)
		{
			Stats.TooLateData.UpdateStats(PrecacheType, PSOCollectorIndex, VertexFactoryType);
			OutResult = EPSOPrecacheResult::TooLate;
		}
		else
		{
			Stats.HitData.UpdateStats(PrecacheType, PSOCollectorIndex, VertexFactoryType);
			OutResult = EPSOPrecacheResult::Complete;
		}
	}

	InOutPrecacheResult = OutResult;
	return bUpdateStats;
}

bool PSOCollectorStats::FPrecacheStatsCollector::IsPrecached(uint64 PrecacheStateHash)
{
	FScopeLock Lock(&StateMapLock);
	FShaderStateUsage* Value = HashedStateMap.Find(PrecacheStateHash);
	return (Value && Value->bPrecached);
}

bool PSOCollectorStats::FPrecacheStatsCollector::IsUsed(uint64 PrecacheStateHash)
{
	FScopeLock Lock(&StateMapLock);
	FShaderStateUsage* Value = HashedStateMap.Find(PrecacheStateHash);
	return (Value && Value->bUsed);
}

#if PSO_PRECACHING_TRACKING

bool PSOCollectorStats::FPrecacheStatsCollector::GetPrecacheData(uint64 PrecacheStateHash, FString& OutMaterialName, int32& OutPSOCollectorIndex, const FVertexFactoryType*& OutVertexFactoryType)
{
	FScopeLock Lock(&StateMapLock);
	FShaderStateUsage* Value = HashedStateMap.Find(PrecacheStateHash);
	if (Value && Value->bPrecached)
	{
		OutMaterialName = Value->MaterialName;
		OutPSOCollectorIndex = Value->PSOCollectorIndex;
		OutVertexFactoryType = Value->VertexFactoryType;
		return true;
	}

	return false;
}

#endif // PSO_PRECACHING_TRACKING

static PSOCollectorStats::FPrecacheStatsCollector FullPSOPrecacheStatsCollector(
	GET_STATFNAME(STAT_FullPSOPrecacheUntrackedCount), GET_STATFNAME(STAT_FullPSOPrecacheMissCount), GET_STATFNAME(STAT_FullPSOPrecacheHitCount), GET_STATFNAME(STAT_FullPSOPrecacheUsedCount), GET_STATFNAME(STAT_FullPSOPrecacheTooLateCount));

static PSOCollectorStats::FPrecacheStatsCollector ShadersOnlyPSOPrecacheStatsCollector(
	GET_STATFNAME(STAT_ShadersOnlyPSOPrecacheUntrackedCount), GET_STATFNAME(STAT_ShadersOnlyPSOPrecacheMissCount), GET_STATFNAME(STAT_ShadersOnlyPSOPrecacheHitCount), GET_STATFNAME(STAT_ShadersOnlyPSOPrecacheUsedCount));

static PSOCollectorStats::FPrecacheStatsCollector MinimalPSOPrecacheStatsCollector(
	GET_STATFNAME(STAT_MinimalPSOPrecacheUntrackedCount), GET_STATFNAME(STAT_MinimalPSOPrecacheMissCount), GET_STATFNAME(STAT_MinimalPSOPrecacheHitCount), GET_STATFNAME(STAT_MinimalPSOPrecacheUsedCount));

PSOCollectorStats::FPrecacheStatsCollector& PSOCollectorStats::GetShadersOnlyPSOPrecacheStatsCollector()
{
	return ShadersOnlyPSOPrecacheStatsCollector;
}

PSOCollectorStats::FPrecacheStatsCollector& PSOCollectorStats::GetMinimalPSOPrecacheStatsCollector()
{
	return MinimalPSOPrecacheStatsCollector;
}

PSOCollectorStats::FPrecacheStatsCollector& PSOCollectorStats::GetFullPSOPrecacheStatsCollector()
{
	return FullPSOPrecacheStatsCollector;
}

CSV_DECLARE_CATEGORY_MODULE_EXTERN(RHI_API, PSOPrecacheCompiling);

struct FGlobalPSOPrecacheStats
{	
#if CSV_PROFILER_STATS

	TCsvPersistentCustomStat<int>* TotalGlobalGraphicPSOPrecacheRequests 	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("TotalGlobalGraphicPSOPrecacheRequests"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	TCsvPersistentCustomStat<int>* TotalGlobalComputePSOPrecacheRequests 	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("TotalGlobalComputePSOPrecacheRequests"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
		
	TCsvPersistentCustomStat<int>* TotalMeshPassGraphicPSOPrecacheRequests 	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("TotalMeshPassGraphicPSOPrecacheRequests"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	TCsvPersistentCustomStat<int>* TotalMeshPassComputePSOPrecacheRequests 	= FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("TotalMeshPassComputePSOPrecacheRequests"), CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	
	static bool CSVStatsEnabled()
	{
		return FCsvProfiler::Get() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(PSOPrecacheCompiling));
	}

#endif // CSV_PROFILER_STATS

private:
	struct FGlobalPrecacheData
	{
		uint64 PSOPrecacheHash;
		FPSOPrecacheData::EType Type;
	};
	TMap < int32, TArray<FGlobalPrecacheData> > GlobalPrecacheDataPerCollector;
	FCriticalSection GlobalPrecacheDataPerCollectorCS;

public:

	void AddGlobalPrecacheData(const FPSOPrecacheData& PrecacheData)
	{
		uint64 PSOPrecacheHash = 0; 
		if (PrecacheData.Type == FPSOPrecacheData::EType::Graphics)
		{
			PSOPrecacheHash = PSOCollectorStats::GetPSOPrecacheHash(PrecacheData.GraphicsPSOInitializer);
		}
		else
		{
			PSOPrecacheHash = PSOCollectorStats::GetPSOPrecacheHash(PrecacheData.ComputeInitializer);
		}

		FScopeLock ScopeLock(&GlobalPrecacheDataPerCollectorCS);
		GlobalPrecacheDataPerCollector.FindOrAdd(PrecacheData.PSOCollectorIndex).Add({PSOPrecacheHash, PrecacheData.Type});
	}

	uint64 GetGlobalCompileTime(const TCHAR* InGlobalPSOCollectorName)
	{
		FScopeLock ScopeLock(&GlobalPrecacheDataPerCollectorCS);

		auto GetTotalCompileTime = [](const TArray<FGlobalPrecacheData>& GlobalPrecacheDatas)
		{
			uint64 TotalCompileTime = 0;
			for (const FGlobalPrecacheData& GlobalPrecacheData : GlobalPrecacheDatas)
			{
				uint64 PSOCompilationTime = PipelineStateCache::GetPrecachePSOCompilationTime(GlobalPrecacheData.Type == FPSOPrecacheData::EType::Graphics, GlobalPrecacheData.PSOPrecacheHash);
				TotalCompileTime += PSOCompilationTime;
			}
			return TotalCompileTime;
		};

		uint64 TotalCompileTime = 0;
		if (InGlobalPSOCollectorName)
		{
			const int32 PSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(InGlobalPSOCollectorName);
			check(PSOCollectorIndex >= 0);
			TArray<FGlobalPrecacheData>* FindRes = GlobalPrecacheDataPerCollector.Find(PSOCollectorIndex);
			if (FindRes)
			{
				TotalCompileTime = GetTotalCompileTime(*FindRes);
			}
		}
		else
		{
			for (auto Iter : GlobalPrecacheDataPerCollector)
			{
				TotalCompileTime += GetTotalCompileTime(Iter.Value);
			}
		}
		return TotalCompileTime;
	}

	static FGlobalPSOPrecacheStats& GetStats()
	{
		static FGlobalPSOPrecacheStats PersistentStats;
		return PersistentStats;
	}
};

uint64 PSOCollectorStats::GetPSOPrecacheHash(const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer)
{
	if (GraphicsPSOInitializer.StatePrecachePSOHash == 0)
	{
		ensureMsgf(false, TEXT("StatePrecachePSOHash should not be zero"));
		return 0;
	}
	return RHIComputePrecachePSOHash(GraphicsPSOInitializer);
}

uint64 PSOCollectorStats::GetPSOPrecacheHash(const FComputePipelineStateInitializer& ComputeInitializer)
{
	return ComputeInitializer.ComputeShader ? GetTypeHash(ComputeInitializer.ComputeShader->GetHash()) : 0;
}

FGraphicsPipelineStateInitializer PSOCollectorStats::GetShadersOnlyInitializer(const FGraphicsPipelineStateInitializer& Initializer)
{
	FGraphicsPipelineStateInitializer ShadersOnlyInitializer;
	ShadersOnlyInitializer.PrimitiveType = PT_TriangleList;
	ShadersOnlyInitializer.BoundShaderState = Initializer.BoundShaderState;
	ShadersOnlyInitializer.BoundShaderState.VertexDeclarationRHI = nullptr;
	ShadersOnlyInitializer.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(ShadersOnlyInitializer);

	return ShadersOnlyInitializer;
}

void PSOCollectorStats::CheckFullPipelineStateInCache(
	const FGraphicsPipelineStateInitializer& Initializer,
	EPSOPrecacheResult PSOPrecacheResult,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FVertexFactoryType* VFType,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	int32 PSOCollectorIndex)
{
	const FMaterial* Material = MaterialRenderProxy ? MaterialRenderProxy->GetMaterialNoFallback(GMaxRHIFeatureLevel) : nullptr;
	CheckFullPipelineStateInCache(Initializer, PSOPrecacheResult, Material, VFType, PrimitiveSceneProxy, PSOCollectorIndex);
}

void PSOCollectorStats::CheckFullPipelineStateInCache(
	const FGraphicsPipelineStateInitializer& Initializer,
	EPSOPrecacheResult PSOPrecacheResult,
	const FMaterial* Material,
	const FVertexFactoryType* VFType,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	int32 PSOCollectorIndex)
{
	bool bCheckMinimalPSOPrecached = true;
	uint64 PrecacheStateHash = 0;
	if (Initializer.StatePrecachePSOHash == 0)
	{
		FGraphicsPipelineStateInitializer LocalInitializer = Initializer;
		LocalInitializer.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(LocalInitializer);
		PSOPrecacheResult = PipelineStateCache::CheckPipelineStateInCache(LocalInitializer);
		PrecacheStateHash = PSOCollectorStats::GetPSOPrecacheHash(LocalInitializer);
		bCheckMinimalPSOPrecached = false;
	}		
	else
	{
		PrecacheStateHash = PSOCollectorStats::GetPSOPrecacheHash(Initializer);
	}
	
	bool bStatUpdated = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().CheckStateInCacheByHash(EPSOPrecacheType::MeshPass, PrecacheStateHash, PSOCollectorIndex, VFType, PSOPrecacheResult);
	if (IsFullPrecachingValidationEnabled() && bStatUpdated && (PSOPrecacheResult == EPSOPrecacheResult::Missed || PSOPrecacheResult == EPSOPrecacheResult::TooLate))
	{
		// only report here if it's not missing with minimal PSO initializer
		bool bMinimalPSOPrecached = bCheckMinimalPSOPrecached ? PSOCollectorStats::GetMinimalPSOPrecacheStatsCollector().IsPrecached(Initializer.StatePrecachePSOHash) : true;
		if (bMinimalPSOPrecached)
		{
			LogPSOMissInfo(Initializer, EPSOPrecacheMissType::FullPSO, PSOPrecacheResult, Material, VFType, PrimitiveSceneProxy, EPSOPrecacheType::MeshPass, PSOCollectorIndex, 0);
		}
	}

	if (bStatUpdated)
	{
		UpdateCSVStats(PSOPrecacheResult);
	}
}

void PSOCollectorStats::CheckComputePipelineStateInCache(
	const FComputePipelineStateInitializer& ComputeInitializer,
	EPSOPrecacheResult PSOPrecacheResult,
	const FMaterialRenderProxy* MaterialRenderProxy,
	int32 PSOCollectorIndex)
{
	bool bStatsUpdated = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().CheckStateInCache(EPSOPrecacheType::MeshPass, ComputeInitializer, PSOCollectorStats::GetPSOPrecacheHash, PSOCollectorIndex, nullptr, PSOPrecacheResult);
	if (IsFullPrecachingValidationEnabled() && bStatsUpdated && (PSOPrecacheResult == EPSOPrecacheResult::Missed || PSOPrecacheResult == EPSOPrecacheResult::TooLate))
	{
		const FMaterial* Material = MaterialRenderProxy ? MaterialRenderProxy->GetMaterialNoFallback(GMaxRHIFeatureLevel) : nullptr;
		LogPSOMissInfo(*ComputeInitializer.ComputeShader, PSOPrecacheResult, Material, EPSOPrecacheType::MeshPass, PSOCollectorIndex);
	}

	if (bStatsUpdated)
	{
		UpdateCSVStats(PSOPrecacheResult);
	}
}

void PSOCollectorStats::CheckGlobalComputePipelineStateInCache(const FComputePipelineStateInitializer& ComputeInitializer)
{
	static const int32 GlobalComputePSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(TEXT("GlobalComputePSOCollector"));
	EPSOPrecacheResult PSOPrecacheResult = PipelineStateCache::CheckPipelineStateInCache(ComputeInitializer.ComputeShader);

	bool bStatsUpdated = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().CheckStateInCache(EPSOPrecacheType::Global, ComputeInitializer, PSOCollectorStats::GetPSOPrecacheHash, GlobalComputePSOCollectorIndex, nullptr, PSOPrecacheResult);
	if (IsFullPrecachingValidationEnabled() && bStatsUpdated && PSOPrecacheResult == EPSOPrecacheResult::Missed)
	{
		LogPSOMissInfo(*ComputeInitializer.ComputeShader, PSOPrecacheResult, nullptr, EPSOPrecacheType::Global, GlobalComputePSOCollectorIndex);
	}

	if (bStatsUpdated)
	{
		UpdateCSVStats(PSOPrecacheResult);
	}
}

void PSOCollectorStats::CheckGlobalGraphicsPipelineStateInCache(
	const FGraphicsPipelineStateInitializer& Initializer, int32 GlobalPSOCollectorIndex)
{
	// Assume no state hash has been computed yet for global graphics PSOs
	check(Initializer.StatePrecachePSOHash == 0);
	
	FGraphicsPipelineStateInitializer LocalInitializer = Initializer;
	LocalInitializer.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(LocalInitializer);
	EPSOPrecacheResult PSOPrecacheResult = PipelineStateCache::CheckPipelineStateInCache(LocalInitializer);
	uint64 PrecacheStateHash = PSOCollectorStats::GetPSOPrecacheHash(LocalInitializer);
	
	bool bStatUpdated = PSOCollectorStats::GetFullPSOPrecacheStatsCollector().CheckStateInCacheByHash(EPSOPrecacheType::Global, PrecacheStateHash, GlobalPSOCollectorIndex, nullptr, PSOPrecacheResult);
	if (IsFullPrecachingValidationEnabled() && bStatUpdated && (PSOPrecacheResult == EPSOPrecacheResult::Missed || PSOPrecacheResult == EPSOPrecacheResult::TooLate))
	{
		LogPSOMissInfo(Initializer, EPSOPrecacheMissType::FullPSO, PSOPrecacheResult, nullptr, nullptr, nullptr, EPSOPrecacheType::Global, GlobalPSOCollectorIndex, 0);
	}

	if (bStatUpdated)
	{
		UpdateCSVStats(PSOPrecacheResult);
	}
}

void PSOCollectorStats::UpdateCSVStats(EPSOPrecacheResult PSOPrecacheResult)
{
	switch (PSOPrecacheResult)
	{
	case EPSOPrecacheResult::Complete:
	{
		CSV_CUSTOM_STAT(PSOPrecache, Hit, 1, ECsvCustomStatOp::Accumulate);
		break;
	}
	case EPSOPrecacheResult::Missed:
	{
		CSV_CUSTOM_STAT(PSOPrecache, Miss, 1, ECsvCustomStatOp::Accumulate);
		break;
	}
	case EPSOPrecacheResult::Active:
	case EPSOPrecacheResult::TooLate:
	{
		CSV_CUSTOM_STAT(PSOPrecache, TooLate, 1, ECsvCustomStatOp::Accumulate);
		break;
	}
	case EPSOPrecacheResult::Untracked:
	{
		CSV_CUSTOM_STAT(PSOPrecache, Untracked, 1, ECsvCustomStatOp::Accumulate);
		break;
	}
	default:
	{
		check(false);
	}
	}
}

void PSOCollectorStats::UpdateGlobalStats(EPSOPrecacheType PSOPrecacheType, const FPSOPrecacheData& PrecacheData)
{	
	FGlobalPSOPrecacheStats& GlobalPSOPrecacheStats = FGlobalPSOPrecacheStats::GetStats();

	// Track all global requests so they can be used during stat dumping
	if (PSOPrecacheType == EPSOPrecacheType::Global)
	{
		GlobalPSOPrecacheStats.AddGlobalPrecacheData(PrecacheData);
	}

#if CSV_PROFILER_STATS
	if (PrecacheData.Type == FPSOPrecacheData::EType::Compute)
	{
		CSV_CUSTOM_STAT(PSOPrecache, PrecachedCompute, 1, ECsvCustomStatOp::Accumulate);
	}
	else
	{
		CSV_CUSTOM_STAT(PSOPrecache, PrecachedGraphics, 1, ECsvCustomStatOp::Accumulate);
	}

	// Update total requested stats for CSV profiler
	if (FGlobalPSOPrecacheStats::CSVStatsEnabled())
	{
		if (PrecacheData.Type == FPSOPrecacheData::EType::Compute)
		{
			if (PSOPrecacheType == EPSOPrecacheType::Global)
			{
				GlobalPSOPrecacheStats.TotalGlobalComputePSOPrecacheRequests->Add(1);
			}
			else
			{
				GlobalPSOPrecacheStats.TotalMeshPassComputePSOPrecacheRequests->Add(1);
			}
		}
		else
		{
			if (PSOPrecacheType == EPSOPrecacheType::Global)
			{
				GlobalPSOPrecacheStats.TotalGlobalGraphicPSOPrecacheRequests->Add(1);
			}
			else
			{
				GlobalPSOPrecacheStats.TotalMeshPassGraphicPSOPrecacheRequests->Add(1);
			}
		}
	}
#endif // CSV_PROFILER_STATS
}


//////////////////////////////////////////////////////////////////////////

using PSOMissStringBuilder = TStringBuilder<2048>;

static const TCHAR* GetPSOMissTypeName(EPSOPrecacheMissType Type)
{
	const TCHAR* ResultString = nullptr;
	switch (Type)
	{
	case EPSOPrecacheMissType::ShadersOnly:		ResultString = TEXT("ShadersOnly"); break;
	case EPSOPrecacheMissType::MinimalPSOState:	ResultString = TEXT("MinimalPSOState"); break;
	case EPSOPrecacheMissType::FullPSO:			ResultString = TEXT("FullPSO"); break;
	}
	return ResultString;
}

static void AppendCSVProfilerInfo(PSOMissStringBuilder& StringBuilder)
{
	#if CSV_PROFILER_STATS
	if (FCsvProfiler::Get() && FCsvProfiler::Get()->IsCapturing_Renderthread())
	{
		StringBuilder.Appendf(TEXT("\nFrame:\t\t\t\t\t%d"), FCsvProfiler::Get()->GetCaptureFrameNumberRT());
	}
	#endif
}

static void LogGeneralPSOMissInfo(
	const FGraphicsPipelineStateInitializer& Initializer,
	const FMaterial* Material,
	const FVertexFactoryType* VFType,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	EPSOPrecacheType PSOPrecacheType, 
	int32 PSOCollectorIndex,
	EPSOPrecacheMissType MissType,
	EPSOPrecacheResult PrecacheResult,
	PSOMissStringBuilder& StringBuilder)
{
	StringBuilder.Appendf(TEXT("\n\nPSO PRECACHING MISS:"));
	AppendCSVProfilerInfo(StringBuilder);
	StringBuilder.Appendf(TEXT("\n\tType:\t\t\t\t\t%s"), GetPSOMissTypeName(MissType));
	StringBuilder.Appendf(TEXT("\n\tPSOPrecachingState:\t\t%s"), LexToString(PrecacheResult));
	StringBuilder.Appendf(TEXT("\n\tMaterial:\t\t\t\t%s"), Material ? *Material->GetAssetName() : TEXT("Unknown"));
	StringBuilder.Appendf(TEXT("\n\tVertexFactoryType:\t\t%s"), VFType ? VFType->GetName() : TEXT("None"));
#if MESH_DRAW_COMMAND_STATS
	StringBuilder.Appendf(TEXT("\n\tMDCStatsCategory:\t\t%s"), PrimitiveSceneProxy ? *PrimitiveSceneProxy->GetMeshDrawCommandStatsCategory().ToString() : TEXT("Unknown"));
#endif // MESH_DRAW_COMMAND_STATS
	if (PSOPrecacheType == EPSOPrecacheType::MeshPass)
	{
		StringBuilder.Appendf(TEXT("\n\tMeshPassName:\t\t\t%s"), FPSOCollectorCreateManager::GetName(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), PSOCollectorIndex));
	}
	else
	{
		StringBuilder.Appendf(TEXT("\n\tGlobalTypeName:\t\t\t%s"), FGlobalPSOCollectorManager::GetName(PSOCollectorIndex));
	}
	StringBuilder.Appendf(TEXT("\n\tShader Hashes:"));
	const auto LogShaderInfo = [&](const TCHAR* ShaderTypeName, FRHIShader* RHIShader)
		{
			if (RHIShader)
			{
				StringBuilder.Appendf(TEXT("\n\t\t%s:\t\t%s"), ShaderTypeName, *(RHIShader->GetHash().ToString()));

				ConditionalBreakOnLogPSOMissInfoShader(RHIShader);
			}
		};


	LogShaderInfo(TEXT("VertexShader"), Initializer.BoundShaderState.GetVertexShader());
	LogShaderInfo(TEXT("PixelShader"), Initializer.BoundShaderState.GetPixelShader());
	LogShaderInfo(TEXT("GeometryShader"), Initializer.BoundShaderState.GetGeometryShader());
	LogShaderInfo(TEXT("MeshShader"), Initializer.BoundShaderState.GetMeshShader());
	LogShaderInfo(TEXT("AmplificationShader"), Initializer.BoundShaderState.GetAmplificationShader());
}

#if PSO_PRECACHING_TRACKING

static void LogMaterialPSOPrecacheRequestData(const FMaterial& Material, const FVertexFactoryType* VFType, PSOMissStringBuilder& StringBuilder)
{
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs = Material.GetMaterialPSOPrecacheRequestIDs();
	if (MaterialPSOPrecacheRequestIDs.Num() > 0)
	{
		for (FMaterialPSOPrecacheRequestID RequestID : MaterialPSOPrecacheRequestIDs)
		{
			FMaterialPSOPrecacheParams MaterialPSOPrecacheParams = GetMaterialPSOPrecacheParams(RequestID);
			check(MaterialPSOPrecacheParams.Material == &Material);
			const TCHAR* CachedVFTypeName = MaterialPSOPrecacheParams.VertexFactoryData.VertexFactoryType ? MaterialPSOPrecacheParams.VertexFactoryData.VertexFactoryType->GetName() : TEXT("None");
			StringBuilder.Appendf(TEXT("\n\t\tPrecached with:\t\t%s (PSOPrecacheParamData: %d)"), CachedVFTypeName, MaterialPSOPrecacheParams.PrecachePSOParams.Data);
			if (MaterialPSOPrecacheParams.VertexFactoryData.VertexFactoryType == VFType)
			{
				// TODO add more info on diff of PSOPrecacheParams which could cause the miss
				// UE_DEBUG_BREAK();
			}
		}
	}
	else
	{
		StringBuilder.Appendf(TEXT("\n\t\tMaterial not precached yet"));
	}
}

// Compare other values
static void CompareStateAndLogChanges(const TCHAR* StateName, uint32 LHSState, uint32 RHSState, PSOMissStringBuilder& StringBuilder)
{
	if (LHSState != RHSState)
	{
		StringBuilder.Appendf(TEXT("\n\t\t\t\t* %s different:"), StateName);
		StringBuilder.Appendf(TEXT("\n\t\t\t\t\tPrecached:\t%d"), LHSState);
		StringBuilder.Appendf(TEXT("\n\t\t\t\t\tRequested:\t%d"), RHSState);
	}
}

// Compare other values
static void CompareStateAndLogFloatChanges(const TCHAR* StateName, float LHSState, float RHSState, PSOMissStringBuilder& StringBuilder)
{
	if (LHSState != RHSState)
	{
		StringBuilder.Appendf(TEXT("\n\t\t\t\t* %s different:"), StateName);
		StringBuilder.Appendf(TEXT("\n\t\t\t\t\tPrecached:\t%f"), LHSState);
		StringBuilder.Appendf(TEXT("\n\t\t\t\t\tRequested:\t%f"), RHSState);
	}
}

static void LogVertexElement(const FVertexElement& VertexElement, PSOMissStringBuilder& StringBuilder)
{
	StringBuilder.Appendf(TEXT("\n\t\t\t\t\tStreamIndex:\t\t%d"), VertexElement.StreamIndex);
	StringBuilder.Appendf(TEXT("\n\t\t\t\t\tOffset:\t\t\t\t%d"), VertexElement.Offset);
	StringBuilder.Appendf(TEXT("\n\t\t\t\t\tType:\t\t\t\t%d"), VertexElement.Type.GetIntValue());
	StringBuilder.Appendf(TEXT("\n\t\t\t\t\tAttributeIndex:\t\t%d"), VertexElement.AttributeIndex);
	StringBuilder.Appendf(TEXT("\n\t\t\t\t\tStride:\t\t\t\t%d"), VertexElement.Stride);
	StringBuilder.Appendf(TEXT("\n\t\t\t\t\tbUseInstanceIndex:\t%d"), VertexElement.bUseInstanceIndex);
}

static void CompareVertexDeclarationAndLogChanges(FRHIVertexDeclaration* PrecachedDeclaration, FRHIVertexDeclaration* RequestedDeclaration, PSOMissStringBuilder& StringBuilder)
{
	if (PrecachedDeclaration == nullptr)
	{
		StringBuilder.Appendf(TEXT("\n\t\t\t- Missing vertex declaration on precached PSO"));
	}
	else if (RequestedDeclaration == nullptr)
	{
		StringBuilder.Appendf(TEXT("\n\t\t\t- Missing vertex declaration on requested PSO"));
	}
	else if (PrecachedDeclaration->GetPrecachePSOHash() != RequestedDeclaration->GetPrecachePSOHash())
	{
		FVertexDeclarationElementList PrecachedDeclarationInitializer, RequestedDeclarationInitializer;
		PrecachedDeclaration->GetInitializer(PrecachedDeclarationInitializer);
		RequestedDeclaration->GetInitializer(RequestedDeclarationInitializer);

		StringBuilder.Appendf(TEXT("\n\t\t\t- VertexDeclaration different:"));
		TCHAR VertexElementIndexBuffer[64] = { 0 };
		for (int32 VertexElementIndex = 0; VertexElementIndex < MaxVertexElementCount; ++VertexElementIndex)
		{
			FCString::Sprintf(VertexElementIndexBuffer, TEXT("VertexElement %d"), VertexElementIndex);
			if (VertexElementIndex < RequestedDeclarationInitializer.Num() && VertexElementIndex < PrecachedDeclarationInitializer.Num() &&
				PrecachedDeclarationInitializer[VertexElementIndex] != RequestedDeclarationInitializer[VertexElementIndex])
			{				
				StringBuilder.Appendf(TEXT("\n\t\t\t  - %s different:"), VertexElementIndexBuffer);

				CompareStateAndLogChanges(TEXT("StreamIndex"), uint32(PrecachedDeclarationInitializer[VertexElementIndex].StreamIndex), uint32(RequestedDeclarationInitializer[VertexElementIndex].StreamIndex), StringBuilder);
				CompareStateAndLogChanges(TEXT("Offset"), uint32(PrecachedDeclarationInitializer[VertexElementIndex].Offset), uint32(RequestedDeclarationInitializer[VertexElementIndex].Offset), StringBuilder);
				CompareStateAndLogChanges(TEXT("Type"), uint32(PrecachedDeclarationInitializer[VertexElementIndex].Type.GetValue()), uint32(RequestedDeclarationInitializer[VertexElementIndex].Type.GetValue()), StringBuilder);
				CompareStateAndLogChanges(TEXT("AttributeIndex"), uint32(PrecachedDeclarationInitializer[VertexElementIndex].AttributeIndex), uint32(RequestedDeclarationInitializer[VertexElementIndex].AttributeIndex), StringBuilder);
				CompareStateAndLogChanges(TEXT("Stride"), uint32(PrecachedDeclarationInitializer[VertexElementIndex].Stride), uint32(RequestedDeclarationInitializer[VertexElementIndex].Stride), StringBuilder);
				CompareStateAndLogChanges(TEXT("bUseInstanceIndex"), uint32(PrecachedDeclarationInitializer[VertexElementIndex].bUseInstanceIndex), uint32(RequestedDeclarationInitializer[VertexElementIndex].bUseInstanceIndex), StringBuilder);
			}
			else if (VertexElementIndex < RequestedDeclarationInitializer.Num() && VertexElementIndex >= PrecachedDeclarationInitializer.Num())
			{
				StringBuilder.Appendf(TEXT("\n\t\t\t  - %s requested but not precached:"), VertexElementIndexBuffer);
				LogVertexElement(RequestedDeclarationInitializer[VertexElementIndex], StringBuilder);
			}
			else if (VertexElementIndex < PrecachedDeclarationInitializer.Num() && VertexElementIndex >= RequestedDeclarationInitializer.Num())
			{
				StringBuilder.Appendf(TEXT("\n\t\t\t  - %s precached but not requested:"), VertexElementIndexBuffer);
				LogVertexElement(PrecachedDeclarationInitializer[VertexElementIndex], StringBuilder);
			}
		}
	}
}

static void CompareRHIBlendStateAndLogChanges(FRHIBlendState* LHSState, FRHIBlendState* RHSState, PSOMissStringBuilder& StringBuilder)
{
	if (!MatchRHIState<FRHIBlendState, FBlendStateInitializerRHI>(LHSState, RHSState))
	{
		FBlendStateInitializerRHI LHSStateInitializer, RHSStateInitializer;
		if (LHSState)
		{
			LHSState->GetInitializer(LHSStateInitializer);
		}
		if (RHSState)
		{
			RHSState->GetInitializer(RHSStateInitializer);
		}

		StringBuilder.Appendf(TEXT("\n\t\t\t- BlendState different:"));
		CompareStateAndLogChanges(TEXT("UseIndependentRenderTargetBlendStates"), LHSStateInitializer.bUseIndependentRenderTargetBlendStates, RHSStateInitializer.bUseIndependentRenderTargetBlendStates, StringBuilder);
		CompareStateAndLogChanges(TEXT("UseAlphaToCoverage"), LHSStateInitializer.bUseAlphaToCoverage, RHSStateInitializer.bUseAlphaToCoverage, StringBuilder);

		TCHAR ColorBlendOpBuffer[64] = { 0 };
		TCHAR ColorSrcBlendBuffer[64] = { 0 };
		TCHAR ColorDestBlendBuffer[64] = { 0 };
		TCHAR AlphaBlendOpBuffer[64] = { 0 };
		TCHAR AlphaSrcBlendBuffer[64] = { 0 };
		TCHAR AlphaDestBlendBuffer[64] = { 0 };
		TCHAR ColorWriteMaskBuffer[64] = { 0 };
		for (int32 RTIndex = 0; RTIndex < MaxSimultaneousRenderTargets; ++RTIndex)
		{
			FCString::Sprintf(ColorBlendOpBuffer, TEXT("ColorBlendOp %d"), RTIndex);
			FCString::Sprintf(ColorSrcBlendBuffer, TEXT("ColorSrcBlend %d"), RTIndex);
			FCString::Sprintf(ColorDestBlendBuffer, TEXT("ColorDestBlend %d"), RTIndex);
			FCString::Sprintf(AlphaBlendOpBuffer, TEXT("AlphaBlendOp %d"), RTIndex);
			FCString::Sprintf(AlphaSrcBlendBuffer, TEXT("AlphaSrcBlend %d"), RTIndex);
			FCString::Sprintf(AlphaDestBlendBuffer, TEXT("AlphaDestBlend %d"), RTIndex);
			FCString::Sprintf(ColorWriteMaskBuffer, TEXT("ColorWriteMask %d"), RTIndex);

			CompareStateAndLogChanges(ColorBlendOpBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].ColorBlendOp.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].ColorBlendOp.GetValue()), StringBuilder);
			CompareStateAndLogChanges(ColorSrcBlendBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].ColorSrcBlend.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].ColorSrcBlend.GetValue()), StringBuilder);
			CompareStateAndLogChanges(ColorDestBlendBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].ColorDestBlend.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].ColorDestBlend.GetValue()), StringBuilder);
			CompareStateAndLogChanges(AlphaBlendOpBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].AlphaBlendOp.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].AlphaBlendOp.GetValue()), StringBuilder);
			CompareStateAndLogChanges(AlphaSrcBlendBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].AlphaSrcBlend.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].AlphaSrcBlend.GetValue()), StringBuilder);
			CompareStateAndLogChanges(AlphaDestBlendBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].AlphaDestBlend.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].AlphaDestBlend.GetValue()), StringBuilder);
			CompareStateAndLogChanges(ColorWriteMaskBuffer, uint32(LHSStateInitializer.RenderTargets[RTIndex].ColorWriteMask.GetValue()), uint32(RHSStateInitializer.RenderTargets[RTIndex].ColorWriteMask.GetValue()), StringBuilder);
		}
	}
}

static void CompareRHIDepthStencilStateAndLogChanges(FRHIDepthStencilState* LHSState, FRHIDepthStencilState* RHSState, PSOMissStringBuilder& StringBuilder)
{
	if (!MatchRHIState<FRHIDepthStencilState, FDepthStencilStateInitializerRHI>(LHSState, RHSState))
	{
		FDepthStencilStateInitializerRHI LHSStateInitializer, RHSStateInitializer;
		if (LHSState)
		{
			LHSState->GetInitializer(LHSStateInitializer);
		}
		if (RHSState)
		{
			RHSState->GetInitializer(RHSStateInitializer);
		}

		StringBuilder.Appendf(TEXT("\n\t\t\t- DepthStencilState different:"));
		CompareStateAndLogChanges(TEXT("bEnableDepthWrite"), LHSStateInitializer.bEnableDepthWrite, RHSStateInitializer.bEnableDepthWrite, StringBuilder);
		CompareStateAndLogChanges(TEXT("DepthTest"), LHSStateInitializer.DepthTest, RHSStateInitializer.DepthTest, StringBuilder);
		CompareStateAndLogChanges(TEXT("bEnableFrontFaceStencil"), LHSStateInitializer.bEnableFrontFaceStencil, RHSStateInitializer.bEnableFrontFaceStencil, StringBuilder);
		CompareStateAndLogChanges(TEXT("FrontFaceStencilTest"), LHSStateInitializer.FrontFaceStencilTest, RHSStateInitializer.FrontFaceStencilTest, StringBuilder);
		CompareStateAndLogChanges(TEXT("FrontFaceStencilFailStencilOp"), LHSStateInitializer.FrontFaceStencilFailStencilOp, RHSStateInitializer.FrontFaceStencilFailStencilOp, StringBuilder);
		CompareStateAndLogChanges(TEXT("FrontFaceDepthFailStencilOp"), LHSStateInitializer.FrontFaceDepthFailStencilOp, RHSStateInitializer.FrontFaceDepthFailStencilOp, StringBuilder);
		CompareStateAndLogChanges(TEXT("FrontFacePassStencilOp"), LHSStateInitializer.FrontFacePassStencilOp, RHSStateInitializer.FrontFacePassStencilOp, StringBuilder);
		CompareStateAndLogChanges(TEXT("bEnableBackFaceStencil"), LHSStateInitializer.bEnableBackFaceStencil, RHSStateInitializer.bEnableBackFaceStencil, StringBuilder);
		CompareStateAndLogChanges(TEXT("BackFaceStencilTest"), LHSStateInitializer.BackFaceStencilTest, RHSStateInitializer.BackFaceStencilTest, StringBuilder);
		CompareStateAndLogChanges(TEXT("BackFaceStencilFailStencilOp"), LHSStateInitializer.BackFaceStencilFailStencilOp, RHSStateInitializer.BackFaceStencilFailStencilOp, StringBuilder);
		CompareStateAndLogChanges(TEXT("BackFaceDepthFailStencilOp"), LHSStateInitializer.BackFaceDepthFailStencilOp, RHSStateInitializer.BackFaceDepthFailStencilOp, StringBuilder);
		CompareStateAndLogChanges(TEXT("BackFacePassStencilOp"), LHSStateInitializer.BackFacePassStencilOp, RHSStateInitializer.BackFacePassStencilOp, StringBuilder);
		CompareStateAndLogChanges(TEXT("StencilReadMask"), LHSStateInitializer.StencilReadMask, RHSStateInitializer.StencilReadMask, StringBuilder);
		CompareStateAndLogChanges(TEXT("StencilWriteMask"), LHSStateInitializer.StencilWriteMask, RHSStateInitializer.StencilWriteMask, StringBuilder);
	}
}

static void CompareRHIRasterizerStateAndLogChanges(FRHIRasterizerState* LHSState, FRHIRasterizerState* RHSState, PSOMissStringBuilder& StringBuilder)
{
	if (!MatchRHIState<FRHIRasterizerState, FRasterizerStateInitializerRHI>(LHSState, RHSState))
	{
		FRasterizerStateInitializerRHI LHSStateInitializer, RHSStateInitializer;
		if (LHSState)
		{
			LHSState->GetInitializer(LHSStateInitializer);
		}
		if (RHSState)
		{
			RHSState->GetInitializer(RHSStateInitializer);
		}

		StringBuilder.Appendf(TEXT("\n\t\t\t- RasterizerState different:"));
		CompareStateAndLogChanges(TEXT("FillMode"), LHSStateInitializer.FillMode, RHSStateInitializer.FillMode, StringBuilder);
		CompareStateAndLogChanges(TEXT("CullMode"), LHSStateInitializer.CullMode, RHSStateInitializer.CullMode, StringBuilder);
		CompareStateAndLogFloatChanges(TEXT("DepthBias"), LHSStateInitializer.DepthBias, RHSStateInitializer.DepthBias, StringBuilder);
		CompareStateAndLogFloatChanges(TEXT("SlopeScaleDepthBias"), LHSStateInitializer.SlopeScaleDepthBias, RHSStateInitializer.SlopeScaleDepthBias, StringBuilder);
		CompareStateAndLogChanges(TEXT("DepthClipMode"), uint32(LHSStateInitializer.DepthClipMode), uint32(RHSStateInitializer.DepthClipMode), StringBuilder);
		CompareStateAndLogChanges(TEXT("bAllowMSAA"), LHSStateInitializer.bAllowMSAA, RHSStateInitializer.bAllowMSAA, StringBuilder);
	}
}

static bool IsUsingDefaultMaterial(
	const FMaterial& Material,
	const FVertexFactoryType* VFType,
	int32 PSOCollectorIndex)
{
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs = Material.GetMaterialPSOPrecacheRequestIDs();
	for (FMaterialPSOPrecacheRequestID RequestID : MaterialPSOPrecacheRequestIDs)
	{
		// Try and find the precached PSOs with the same shaders
		FPSOPrecacheDataArray PSOPrecacheDataArray = GetMaterialPSOPrecacheData(RequestID);
		for (FPSOPrecacheData& PSOPrecacheData : PSOPrecacheDataArray)
		{
			if (PSOPrecacheData.PSOCollectorIndex == PSOCollectorIndex && PSOPrecacheData.VertexFactoryType == VFType && PSOPrecacheData.bDefaultMaterial != 0)
			{
				return true;
			}
		}
	}

	return false;
}

static const FMaterial* GetEffectiveMaterial(
	const FMaterial* Material,
	const FVertexFactoryType* VFType,
	int32 PSOCollectorIndex,
	PSOMissStringBuilder& StringBuilder)
{	
	const FMaterial* EffectiveMaterial = Material;
	if (Material)
	{
		// If it's using the default material for this pass, then retrieve that material to collect PSO precache data to find what's different
		bool bUsingDefaultMaterial = IsUsingDefaultMaterial(*Material, VFType, PSOCollectorIndex);
		if (bUsingDefaultMaterial)
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			EffectiveMaterial = UMaterial::GetDefaultMaterial(Material->GetMaterialDomain())->GetMaterialResource(Material->GetShaderPlatform(), ActiveQualityLevel);

			StringBuilder.Appendf(TEXT("\n\tUsing Default Material"));
		}
	}
	return EffectiveMaterial;
}

static FPSOPrecacheDataArray FindShaderMatchingPrecacheData(
	const FGraphicsPipelineStateInitializer& Initializer,
	const FMaterial* Material)
{
	const auto MatchShader = [&](FRHIShader* LHS, FRHIShader* RHS)
		{
			if (LHS)
			{
				return RHS && LHS->GetHash() == RHS->GetHash();
			}
			return RHS == nullptr;
		};

	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs = Material->GetMaterialPSOPrecacheRequestIDs();

	FPSOPrecacheDataArray MatchingPSOPrecacheData;
	TArray<uint64> TrackedMinimalPSOData;
	for (FMaterialPSOPrecacheRequestID RequestID : MaterialPSOPrecacheRequestIDs)
	{
		// Try and find the precached PSOs with the same shaders
		FPSOPrecacheDataArray PSOPrecacheDataArray = GetMaterialPSOPrecacheData(RequestID);
		for (FPSOPrecacheData& PSOPrecacheData : PSOPrecacheDataArray)
		{
			if (PSOPrecacheData.Type == FPSOPrecacheData::EType::Graphics)
			{
				// Already logged data for this PSO (can be precached multiple times for the same material)
				if (TrackedMinimalPSOData.Contains(PSOPrecacheData.GraphicsPSOInitializer.StatePrecachePSOHash))
				{
					continue;
				}

				bool bMatchShader = MatchShader(Initializer.BoundShaderState.GetVertexShader(), PSOPrecacheData.GraphicsPSOInitializer.BoundShaderState.GetVertexShader());
				bMatchShader = bMatchShader && MatchShader(Initializer.BoundShaderState.GetPixelShader(), PSOPrecacheData.GraphicsPSOInitializer.BoundShaderState.GetPixelShader());
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
				bMatchShader = bMatchShader && MatchShader(Initializer.BoundShaderState.GetGeometryShader(), PSOPrecacheData.GraphicsPSOInitializer.BoundShaderState.GetGeometryShader());
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
				if (bMatchShader)
				{
					MatchingPSOPrecacheData.Add(PSOPrecacheData);
					TrackedMinimalPSOData.Add(PSOPrecacheData.GraphicsPSOInitializer.StatePrecachePSOHash);
				}
			}
		}
	}

	return MatchingPSOPrecacheData;
}

void LogMinimalPSOStateMissInfo(
	const FGraphicsPipelineStateInitializer& Initializer,
	const FMaterial& Material,
	const FVertexFactoryType* VFType,
	int32 PSOCollectorIndex,
	uint64 ShadersOnlyPSOInitializerHash,
	PSOMissStringBuilder& StringBuilder)
{
	FString PrecachedMaterialName;
	int32 PrecachedPSOCollectorIndex;
	const FVertexFactoryType* PrecachedVertexFactoryType;

	// only report here if it's not missing with shader only state
	bool bShaderOnlyPrecached = PSOCollectorStats::GetShadersOnlyPSOPrecacheStatsCollector().GetPrecacheData(ShadersOnlyPSOInitializerHash, PrecachedMaterialName, PrecachedPSOCollectorIndex, PrecachedVertexFactoryType);
	check(bShaderOnlyPrecached);
		
	// If it's using the default material for this pass, then retrieve that material to collect PSO precache data to find what's different
	const FMaterial* EffectiveMaterial = GetEffectiveMaterial(&Material, VFType, PSOCollectorIndex, StringBuilder);

	StringBuilder.Appendf(TEXT("\n\n\tShadersOnly precache information:"));
	StringBuilder.Appendf(TEXT("\n\tMaterial:\t\t\t\t%s"), *PrecachedMaterialName);
	StringBuilder.Appendf(TEXT("\n\tVertexFactoryType:\t\t%s"), PrecachedVertexFactoryType ? PrecachedVertexFactoryType->GetName() : TEXT("None"));
	StringBuilder.Appendf(TEXT("\n\tPassName:\t\t\t\t%s"), FPSOCollectorCreateManager::GetName(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), PrecachedPSOCollectorIndex));

	StringBuilder.Appendf(TEXT("\n\n\tMissed Info:"));

	// Collect the PSO precache data matching the shader hashes and should at least find one
	FPSOPrecacheDataArray MatchingPSOPrecacheData = FindShaderMatchingPrecacheData(Initializer, EffectiveMaterial);
	if (MatchingPSOPrecacheData.Num() > 0)
	{
		for (FPSOPrecacheData& PSOPrecacheData : MatchingPSOPrecacheData)
		{
			StringBuilder.Appendf(TEXT("\n\t\t- Found PSO With same shaders & different state:"));
			StringBuilder.Appendf(TEXT("\n\t\t\tVertexFactoryType:\t\t%s"), PSOPrecacheData.VertexFactoryType ? PSOPrecacheData.VertexFactoryType->GetName() : TEXT("None"));
			StringBuilder.Appendf(TEXT("\n\t\t\tPassName:\t\t\t\t%s"), FPSOCollectorCreateManager::GetName(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), PSOPrecacheData.PSOCollectorIndex));
			StringBuilder.Appendf(TEXT("\n\t\t  Differences:"));

			CompareVertexDeclarationAndLogChanges(PSOPrecacheData.GraphicsPSOInitializer.BoundShaderState.VertexDeclarationRHI, Initializer.BoundShaderState.VertexDeclarationRHI, StringBuilder);

			CompareRHIBlendStateAndLogChanges(PSOPrecacheData.GraphicsPSOInitializer.BlendState, Initializer.BlendState, StringBuilder);
			CompareRHIDepthStencilStateAndLogChanges(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilState, Initializer.DepthStencilState, StringBuilder);
			CompareRHIRasterizerStateAndLogChanges(PSOPrecacheData.GraphicsPSOInitializer.RasterizerState, Initializer.RasterizerState, StringBuilder);

			CompareStateAndLogChanges(TEXT("DepthBounds"), uint32(PSOPrecacheData.GraphicsPSOInitializer.bDepthBounds), uint32(Initializer.bDepthBounds), StringBuilder);
			CompareStateAndLogChanges(TEXT("DrawShadingRate"), uint32(PSOPrecacheData.GraphicsPSOInitializer.ShadingRate), uint32(Initializer.ShadingRate), StringBuilder);
			CompareStateAndLogChanges(TEXT("PrimitiveType"), uint32(PSOPrecacheData.GraphicsPSOInitializer.PrimitiveType), uint32(Initializer.PrimitiveType), StringBuilder);

			// Ignored for now since it doesn't add to any real PSO and can give different state
			//CompareStateAndLogChanges(TEXT("AllowVariableRateShading"), uint32(PSOPrecacheData.GraphicsPSOInitializer.bAllowVariableRateShading), uint32(Initializer.bAllowVariableRateShading), StringBuilder);			
		}
	}
	else
	{
		StringBuilder.Appendf(TEXT("\n\t\tFound no PSOs with same shaders for this material."));
		LogMaterialPSOPrecacheRequestData(Material, VFType, StringBuilder);
	}
}

void LogFullPSOStateMissInfo(
	const FGraphicsPipelineStateInitializer& Initializer,
	const FMaterial* Material,
	const FVertexFactoryType* VFType,
	int32 PSOCollectorIndex,
	PSOMissStringBuilder& StringBuilder)
{
	const FMaterial* EffectiveMaterial = GetEffectiveMaterial(Material, VFType, PSOCollectorIndex, StringBuilder);
	if (EffectiveMaterial)
	{
		StringBuilder.Appendf(TEXT("\n\n\tMissed Info:"));

		// Collect the PSO precache data matching the shader hashes and should at least find one
		FPSOPrecacheDataArray MatchingPSOPrecacheData = FindShaderMatchingPrecacheData(Initializer, EffectiveMaterial);
		if (MatchingPSOPrecacheData.Num() > 0)
		{
			for (FPSOPrecacheData& PSOPrecacheData : MatchingPSOPrecacheData)
			{
				if (PSOPrecacheData.GraphicsPSOInitializer.StatePrecachePSOHash == Initializer.StatePrecachePSOHash)
				{
					StringBuilder.Appendf(TEXT("\n\t\tFound PSO With same state PSO hash & different render target data:"));
					StringBuilder.Appendf(TEXT("\n\t\tDifferences:"));

					CompareStateAndLogChanges(TEXT("RenderTargetsEnabled"), uint32(PSOPrecacheData.GraphicsPSOInitializer.RenderTargetsEnabled), uint32(Initializer.RenderTargetsEnabled), StringBuilder);
					TCHAR RTFormatBuffer[64] = { 0 };
					TCHAR RTFlagsBuffer[64] = { 0 };
					for (int32 RTIndex = 0; RTIndex < MaxSimultaneousRenderTargets; ++RTIndex)
					{
						FCString::Sprintf(RTFormatBuffer, TEXT("RenderTargetFormat %d"), RTIndex);
						FCString::Sprintf(RTFlagsBuffer, TEXT("RenderTargetFlags %d"), RTIndex);
						CompareStateAndLogChanges(RTFormatBuffer, uint32(PSOPrecacheData.GraphicsPSOInitializer.RenderTargetFormats[RTIndex]), uint32(Initializer.RenderTargetFormats[RTIndex]), StringBuilder);
						CompareStateAndLogChanges(RTFlagsBuffer, uint32(PSOPrecacheData.GraphicsPSOInitializer.RenderTargetFlags[RTIndex]), uint32(Initializer.RenderTargetFlags[RTIndex]), StringBuilder);
					}
					CompareStateAndLogChanges(TEXT("DepthStencilTargetFormat"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilTargetFormat), uint32(Initializer.DepthStencilTargetFormat), StringBuilder);
					CompareStateAndLogChanges(TEXT("DepthStencilTargetFlag"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilTargetFlag), uint32(Initializer.DepthStencilTargetFlag), StringBuilder);
					CompareStateAndLogChanges(TEXT("DepthTargetLoadAction"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthTargetLoadAction), uint32(Initializer.DepthTargetLoadAction), StringBuilder);
					CompareStateAndLogChanges(TEXT("DepthTargetStoreAction"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthTargetStoreAction), uint32(Initializer.DepthTargetStoreAction), StringBuilder);
					CompareStateAndLogChanges(TEXT("StencilTargetLoadAction"), uint32(PSOPrecacheData.GraphicsPSOInitializer.StencilTargetLoadAction), uint32(Initializer.StencilTargetLoadAction), StringBuilder);
					CompareStateAndLogChanges(TEXT("StencilTargetStoreAction"), uint32(PSOPrecacheData.GraphicsPSOInitializer.StencilTargetStoreAction), uint32(Initializer.StencilTargetStoreAction), StringBuilder);
					if (PSOPrecacheData.GraphicsPSOInitializer.DepthStencilAccess != Initializer.DepthStencilAccess)
					{
						CompareStateAndLogChanges(TEXT("DepthWrite"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilAccess.IsDepthWrite()), uint32(Initializer.DepthStencilAccess.IsDepthWrite()), StringBuilder);
						CompareStateAndLogChanges(TEXT("DepthRead"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilAccess.IsDepthRead()), uint32(Initializer.DepthStencilAccess.IsDepthRead()), StringBuilder);
						CompareStateAndLogChanges(TEXT("StencilWrite"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilAccess.IsStencilWrite()), uint32(Initializer.DepthStencilAccess.IsStencilWrite()), StringBuilder);
						CompareStateAndLogChanges(TEXT("StencilRead"), uint32(PSOPrecacheData.GraphicsPSOInitializer.DepthStencilAccess.IsStencilRead()), uint32(Initializer.DepthStencilAccess.IsStencilRead()), StringBuilder);
					}
					CompareStateAndLogChanges(TEXT("NumSamples"), uint32(PSOPrecacheData.GraphicsPSOInitializer.NumSamples), uint32(Initializer.NumSamples), StringBuilder);
					CompareStateAndLogChanges(TEXT("SubpassHint"), uint32(PSOPrecacheData.GraphicsPSOInitializer.SubpassHint), uint32(Initializer.SubpassHint), StringBuilder);
					CompareStateAndLogChanges(TEXT("SubpassIndex"), uint32(PSOPrecacheData.GraphicsPSOInitializer.SubpassIndex), uint32(Initializer.SubpassIndex), StringBuilder);
					CompareStateAndLogChanges(TEXT("MultiViewCount"), uint32(PSOPrecacheData.GraphicsPSOInitializer.MultiViewCount), uint32(Initializer.MultiViewCount), StringBuilder);
					CompareStateAndLogChanges(TEXT("HasFragmentDensityAttachment"), uint32(PSOPrecacheData.GraphicsPSOInitializer.bHasFragmentDensityAttachment), uint32(Initializer.bHasFragmentDensityAttachment), StringBuilder);
					CompareStateAndLogChanges(TEXT("ConservativeRasterization"), uint32(PSOPrecacheData.GraphicsPSOInitializer.ConservativeRasterization), uint32(Initializer.ConservativeRasterization), StringBuilder);										
				}
			}
		}
	}
	else
	{
		StringBuilder.Appendf(TEXT("\n\n\tNo material found so no extra information on miss"));
	}
}

#endif // PSO_PRECACHING_TRACKING

void LogPSOMissInfo(
	const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer,
	EPSOPrecacheMissType MissType,
	EPSOPrecacheResult Result,
	const FMaterial* Material,
	const FVertexFactoryType* VFType,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	EPSOPrecacheType PSOPrecacheType, 
	int32 PSOCollectorIndex,
	uint64 ShadersOnlyPSOInitializerHash)
{
	PSOMissStringBuilder StringBuilder;
	LogGeneralPSOMissInfo(GraphicsPSOInitializer, Material, VFType, PrimitiveSceneProxy, PSOPrecacheType, PSOCollectorIndex, MissType, Result, StringBuilder);

#if PSO_PRECACHING_TRACKING
	if (Result == EPSOPrecacheResult::Untracked)
	{
		check(MissType == EPSOPrecacheMissType::ShadersOnly);
		StringBuilder << TEXT("\n\n\tUntracked Info:");
		if (!VFType->SupportsPSOPrecaching())
		{
			StringBuilder << TEXT("\n\t\t- VertexFactory doesn't support PSO precaching.");
		}
		if (!PSOCollectorStats::FPrecacheStatsCollector::IsStateTracked(PSOPrecacheType, PSOCollectorIndex, nullptr))
		{
			StringBuilder << TEXT("\n\t\t- MeshPassProcessor doesn't support PSO precaching.");
		}
	}
	else if (Result == EPSOPrecacheResult::Missed)
	{
		switch (MissType)
		{
		case EPSOPrecacheMissType::ShadersOnly:
		{
			check(Material);
			StringBuilder << TEXT("\n\n\tMissed Info:");
			LogMaterialPSOPrecacheRequestData(*Material, VFType, StringBuilder);
			break;
		}
		case EPSOPrecacheMissType::MinimalPSOState:
		{	
			check(Material);
			LogMinimalPSOStateMissInfo(GraphicsPSOInitializer, *Material, VFType, PSOCollectorIndex, ShadersOnlyPSOInitializerHash, StringBuilder);
			break;
		}
		case EPSOPrecacheMissType::FullPSO:
		{
			LogFullPSOStateMissInfo(GraphicsPSOInitializer, Material, VFType, PSOCollectorIndex, StringBuilder);
			break;
		}
		}
	}
	else if (Result == EPSOPrecacheResult::TooLate)
	{
		check(MissType == EPSOPrecacheMissType::FullPSO);
		
		// Nothing extra log yet
	}
	else
	{
		// Should have get any other results on shaders only
		check(false);
	}
#endif // PSO_PRECACHING_TRACKING

	UE_LOGF(LogEngine, Log, "%ls\n", StringBuilder.ToString());
}

void LogPSOMissInfo(
	const FRHIComputeShader& ComputeShader,
	EPSOPrecacheResult PrecacheResult,
	const FMaterial* Material,
	EPSOPrecacheType PSOPrecacheType,
	int32 PSOCollectorIndex)
{
	PSOMissStringBuilder StringBuilder;
	StringBuilder.Appendf(TEXT("\n\nPSO PRECACHING MISS:"));
	AppendCSVProfilerInfo(StringBuilder);
	StringBuilder.Appendf(TEXT("\n\tType:\t\t\t\t\t%s"), TEXT("Compute"));
	StringBuilder.Appendf(TEXT("\n\tPSOPrecachingState:\t\t%s"), LexToString(PrecacheResult));
	if (PSOPrecacheType == EPSOPrecacheType::MeshPass)
	{
		StringBuilder.Appendf(TEXT("\n\tMaterial:\t\t\t\t%s"), Material ? *Material->GetAssetName() : TEXT("Unknown"));
		StringBuilder.Appendf(TEXT("\n\tPassName:\t\t\t\t%s"), FPSOCollectorCreateManager::GetName(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), PSOCollectorIndex));
	}
	StringBuilder.Appendf(TEXT("\n\tCompute Shader Hash:\t%s"), *(ComputeShader.GetHash().ToString()));

	// Not sure yet if this is interesting data or not
	/*
#if PSO_PRECACHING_TRACKING
	if (Material)
	{
		StringBuilder << TEXT("\n\n\tMissed Info:");
		LogMaterialPSOPrecacheRequestData(*Material, nullptr, StringBuilder);
	}
#endif // PSO_PRECACHING_TRACKING
	*/

	UE_LOGF(LogEngine, Log, "%ls\n", StringBuilder.ToString());
}

namespace PSOCollectorStats
{
	enum class ETrackingType
	{
		Missed,
		Precached,
		Used,
		TooLate,
		Untracked,
		Num,
	};

	enum class EStatsType
	{
		ShaderOnly,
		Minimal,
		Full,
		Num,
	};

	struct FPerTypeStats
	{
		FPerTypeStats()
		{
			FMemory::Memzero(TrackedStats, (int32)EStatsType::Num * (int32)ETrackingType::Num * sizeof(int32));
		}

		int32 GetStat(EStatsType StatsType, ETrackingType TrackingType) const
		{
			return TrackedStats[(int32)StatsType][(int32)TrackingType];
		}
		
		void SetStat(EStatsType StatsType, ETrackingType TrackingType, int32 Value)
		{
			TrackedStats[(int32)StatsType][(int32)TrackingType] = Value;
		}

		template<typename CountGetterCallback>
		void AddStatsPerTrackingType(EStatsType StatsType, PSOCollectorStats::FPrecacheStats& Stats, CountGetterCallback Callback)
		{
			SetStat(StatsType, ETrackingType::Missed, Callback(Stats.MissData));
			SetStat(StatsType, ETrackingType::Precached, Callback(Stats.PrecacheData));
			SetStat(StatsType, ETrackingType::Used, Callback(Stats.UsageData));
			SetStat(StatsType, ETrackingType::TooLate, Callback(Stats.TooLateData));
			SetStat(StatsType, ETrackingType::Untracked, Callback(Stats.UntrackedData));
		}

		template<typename CountGetterCallback>
		void AddStatsPerType(CountGetterCallback Callback)
		{
			AddStatsPerTrackingType(EStatsType::Full, FullPSOPrecacheStatsCollector.GetStats(), Callback);
			AddStatsPerTrackingType(EStatsType::Minimal, MinimalPSOPrecacheStatsCollector.GetStats(), Callback);
			AddStatsPerTrackingType(EStatsType::ShaderOnly, ShadersOnlyPSOPrecacheStatsCollector.GetStats(), Callback);
		}

		FString Name;
		uint64 TotalCompileTime = 0;
		int32 TrackedStats[(int32)EStatsType::Num][(int32)ETrackingType::Num];
	};
}

void PSOCollectorStats::DumpPSOPrecacheValidationStats()
{
	const FString Filename = FString::Printf(TEXT("%sPSOPrecacheStats-%s.csv"), *FPaths::ProfilingDir(), *FDateTime::Now().ToString());
	FArchive* CSVFile = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead);
	if (CSVFile == nullptr)
	{
		return;
	}
		
	auto AddStatsToCSVFile = [CSVFile](const TCHAR* TypeName, bool bSort, TArray<FPerTypeStats>& TypeStats)
	{		
		if (bSort)
		{
			Algo::Sort(TypeStats, [](FPerTypeStats& LHS, FPerTypeStats& RHS)
				{
					return LHS.GetStat(EStatsType::Full, ETrackingType::Precached) > RHS.GetStat(EStatsType::Full, ETrackingType::Precached);
				});
		}

		FString Header = FString::Printf(TEXT("%s,Total Time (sec),Average Time (msec),Full Precached,Full Used,Full Usage,Full Missed,Full TooLate,Full Untracked,ShaderOnly Precached,ShaderOnly Used,ShaderOnly Usage,ShaderOnly Missed,Minimal Precached,Minimal Used,Minimal Usage,Minimal Missed\n"), TypeName);		
		CSVFile->Serialize(TCHAR_TO_ANSI(*Header), Header.Len());
				
		for (auto Stats : TypeStats)
		{
			auto AddStats = [CSVFile, Stats](EStatsType StatsType)
			{				
				int32 Precached = Stats.GetStat(StatsType, ETrackingType::Precached);
				int32 Used = Stats.GetStat(StatsType, ETrackingType::Used);
				float Usage = Precached > 0 ? float(Used) / Precached : 0.0f;
				int32 Missed = Stats.GetStat(StatsType, ETrackingType::Missed);

				FString Row = FString::Printf(TEXT(",%d,%d,%0.2f,%d"), Precached, Used, Usage, Missed);
				CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
			};
			
			int32 FullPrecached = Stats.GetStat(EStatsType::Full, ETrackingType::Precached);
			int32 FullUsed = Stats.GetStat(EStatsType::Full, ETrackingType::Used);
			int32 FullMissed = Stats.GetStat(EStatsType::Full, ETrackingType::Missed);

			if (FullPrecached != 0 || FullUsed != 0 || FullMissed != 0)
			{
				FString NameEntry = FString::Printf(TEXT("%s"), *Stats.Name);
				CSVFile->Serialize(TCHAR_TO_ANSI(*NameEntry), NameEntry.Len());

				float TotalTimeInSeconds = FPlatformTime::ToSeconds64(Stats.TotalCompileTime);
				float AverageTimeInMsec = FullPrecached > 0.0f ? TotalTimeInSeconds / FullPrecached * 1000.0f : 0.0f;
				FString CompileTimeData = FString::Printf(TEXT(",%3.2f,%3.2f"), TotalTimeInSeconds, AverageTimeInMsec);
				CSVFile->Serialize(TCHAR_TO_ANSI(*CompileTimeData), CompileTimeData.Len());

				AddStats(EStatsType::Full);
				
				FString Row = FString::Printf(TEXT(",%d,%d"), Stats.GetStat(EStatsType::Full, ETrackingType::TooLate), Stats.GetStat(EStatsType::Full, ETrackingType::Untracked));
				CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());

				AddStats(EStatsType::ShaderOnly);
				AddStats(EStatsType::Minimal);
				
				FString NewLine(TEXT("\n"));
				CSVFile->Serialize(TCHAR_TO_ANSI(*NewLine), NewLine.Len());
			}
		}
				
		const TCHAR* Footer = TEXT("\n\n\n");
		CSVFile->Serialize(TCHAR_TO_ANSI(Footer), FPlatformString::Strlen(Footer));
	};

	const PSOCollectorStats::FPrecacheStats& FullPSOPrecacheStats = FullPSOPrecacheStatsCollector.GetStats();
	const PSOCollectorStats::FPrecacheStats& ShaderOnlyPSOPrecacheStats = ShadersOnlyPSOPrecacheStatsCollector.GetStats();
	const PSOCollectorStats::FPrecacheStats& MinimalPSOPrecacheStats = MinimalPSOPrecacheStatsCollector.GetStats();

	FGlobalPSOPrecacheStats& GlobalPSOPrecacheStats = FGlobalPSOPrecacheStats::GetStats();

	// Totals
	{
		TArray<FPerTypeStats> TotalStats;
		
		auto TotalGetter = [](FPrecacheUsageData& UsageData) { return UsageData.GetTotalCount(); };

		FPerTypeStats TotalPerTypeStats;
		TotalPerTypeStats.Name = TEXT("Total");
		TotalPerTypeStats.TotalCompileTime = GetPSOPrecacheMaterialCompileTime(nullptr, nullptr) + GlobalPSOPrecacheStats.GetGlobalCompileTime(nullptr);
		TotalPerTypeStats.AddStatsPerType(TotalGetter);
		TotalStats.Add(TotalPerTypeStats);

		auto TotalGlobalGetter = [](FPrecacheUsageData& UsageData) { return UsageData.GetTotalGlobalCount(); };

		FPerTypeStats GlobalStats;
		GlobalStats.Name = TEXT("Global");
		GlobalStats.TotalCompileTime = GlobalPSOPrecacheStats.GetGlobalCompileTime(nullptr);
		GlobalStats.AddStatsPerType(TotalGlobalGetter);
		TotalStats.Add(GlobalStats);

		auto TotalMeshPassGetter = [](FPrecacheUsageData& UsageData) { return UsageData.GetTotalMeshPassCount(); };

		FPerTypeStats MeshPassStats;
		MeshPassStats.Name = TEXT("MeshPass");
		MeshPassStats.TotalCompileTime = GetPSOPrecacheMaterialCompileTime(nullptr, nullptr);
		MeshPassStats.AddStatsPerType(TotalMeshPassGetter);
		TotalStats.Add(MeshPassStats);

		AddStatsToCSVFile(TEXT("Totals"), false, TotalStats);
	}
	
	// Collect Global stats
	{	
		TArray<FPerTypeStats> GlobalTypeStats;

		for (int32 PSOCollectorIndex = 0; PSOCollectorIndex < FGlobalPSOCollectorManager::GetPSOCollectorCount(); ++PSOCollectorIndex)
		{
			FPerTypeStats PerTypeStats;
			PerTypeStats.Name = FGlobalPSOCollectorManager::GetName(PSOCollectorIndex);
			PerTypeStats.TotalCompileTime = GlobalPSOPrecacheStats.GetGlobalCompileTime(*PerTypeStats.Name);

			PerTypeStats.SetStat(EStatsType::Full, ETrackingType::Missed, FullPSOPrecacheStats.MissData.PerGlobalTypeCount[PSOCollectorIndex]);
			PerTypeStats.SetStat(EStatsType::Full, ETrackingType::Precached, FullPSOPrecacheStats.PrecacheData.PerGlobalTypeCount[PSOCollectorIndex]);
			PerTypeStats.SetStat(EStatsType::Full, ETrackingType::Used, FullPSOPrecacheStats.UsageData.PerGlobalTypeCount[PSOCollectorIndex]);			
			PerTypeStats.SetStat(EStatsType::Full, ETrackingType::TooLate, FullPSOPrecacheStats.TooLateData.PerGlobalTypeCount[PSOCollectorIndex]);
			PerTypeStats.SetStat(EStatsType::Full, ETrackingType::Untracked, FullPSOPrecacheStats.UntrackedData.PerGlobalTypeCount[PSOCollectorIndex]);
			GlobalTypeStats.Add(PerTypeStats);
		}
		
		AddStatsToCSVFile(TEXT("GlobalType"), true, GlobalTypeStats);
	}
	
	// Collect MeshPass stats
	{	
		TArray<FPerTypeStats> PerMeshPassStats;

		EShadingPath ShadingPath = GetFeatureLevelShadingPath(GMaxRHIFeatureLevel);
		for (int32 PSOCollectorIndex = 0; PSOCollectorIndex < FPSOCollectorCreateManager::GetPSOCollectorCount(ShadingPath); ++PSOCollectorIndex)
		{
			FPerTypeStats MeshPassStats;
			MeshPassStats.Name = FPSOCollectorCreateManager::GetName(ShadingPath, PSOCollectorIndex);
			MeshPassStats.TotalCompileTime = GetPSOPrecacheMaterialCompileTime(*MeshPassStats.Name, nullptr);
						
			MeshPassStats.SetStat(EStatsType::Full, ETrackingType::Missed, FullPSOPrecacheStats.MissData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::Full, ETrackingType::Precached, FullPSOPrecacheStats.PrecacheData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::Full, ETrackingType::Used, FullPSOPrecacheStats.UsageData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::Full, ETrackingType::TooLate, FullPSOPrecacheStats.TooLateData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::Full, ETrackingType::Untracked, FullPSOPrecacheStats.UntrackedData.PerMeshPassCount[PSOCollectorIndex]);

			MeshPassStats.SetStat(EStatsType::ShaderOnly, ETrackingType::Missed, ShaderOnlyPSOPrecacheStats.MissData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::ShaderOnly, ETrackingType::Precached, ShaderOnlyPSOPrecacheStats.PrecacheData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::ShaderOnly, ETrackingType::Used, ShaderOnlyPSOPrecacheStats.UsageData.PerMeshPassCount[PSOCollectorIndex]);

			MeshPassStats.SetStat(EStatsType::Minimal, ETrackingType::Missed, MinimalPSOPrecacheStats.MissData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::Minimal, ETrackingType::Precached, MinimalPSOPrecacheStats.PrecacheData.PerMeshPassCount[PSOCollectorIndex]);
			MeshPassStats.SetStat(EStatsType::Minimal, ETrackingType::Used, MinimalPSOPrecacheStats.UsageData.PerMeshPassCount[PSOCollectorIndex]);

			PerMeshPassStats.Add(MeshPassStats);
		}
				
		AddStatsToCSVFile(TEXT("PassName"), true, PerMeshPassStats);
	}

	// Collect VF stats
	{
		TMap<FString, FPerTypeStats> VertexFactoryStatMap;
		auto AddVertexFactoryStats = [&VertexFactoryStatMap](EStatsType StatsType, ETrackingType TrackingType, PSOCollectorStats::VertexFactoryCountTableType PerVertexFactoryCounts)
		{
			for (auto Iter : PerVertexFactoryCounts)
			{
				FString VertexFactoryTypeName = Iter.Key->GetName();
				FPerTypeStats& Stats = VertexFactoryStatMap.FindOrAdd(VertexFactoryTypeName);
				Stats.Name = VertexFactoryTypeName;
				int32 CurrentValue = Stats.GetStat(StatsType, TrackingType);
				Stats.SetStat(StatsType, TrackingType, CurrentValue + Iter.Value);
			}
		};
		AddVertexFactoryStats(EStatsType::Full, ETrackingType::Precached, FullPSOPrecacheStats.PrecacheData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::Full, ETrackingType::Used, FullPSOPrecacheStats.UsageData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::Full, ETrackingType::Missed, FullPSOPrecacheStats.MissData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::Full, ETrackingType::TooLate, FullPSOPrecacheStats.TooLateData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::Full, ETrackingType::Untracked, FullPSOPrecacheStats.UntrackedData.PerVertexFactoryCount);
		
		AddVertexFactoryStats(EStatsType::ShaderOnly, ETrackingType::Precached, ShaderOnlyPSOPrecacheStats.PrecacheData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::ShaderOnly, ETrackingType::Used, ShaderOnlyPSOPrecacheStats.UsageData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::ShaderOnly, ETrackingType::Missed, ShaderOnlyPSOPrecacheStats.MissData.PerVertexFactoryCount);
		
		AddVertexFactoryStats(EStatsType::Minimal, ETrackingType::Precached, MinimalPSOPrecacheStats.PrecacheData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::Minimal, ETrackingType::Used, MinimalPSOPrecacheStats.UsageData.PerVertexFactoryCount);
		AddVertexFactoryStats(EStatsType::Minimal, ETrackingType::Missed, MinimalPSOPrecacheStats.MissData.PerVertexFactoryCount);

		TArray<FPerTypeStats> VertexFactoryStats;
		for (auto StatEntry : VertexFactoryStatMap)
		{
			FPerTypeStats& Stat = StatEntry.Value;
			Stat.TotalCompileTime = GetPSOPrecacheMaterialCompileTime(nullptr, *Stat.Name);
			VertexFactoryStats.Add(Stat);
		}

		AddStatsToCSVFile(TEXT("VertexFactoryType"), true, VertexFactoryStats);
	}

	delete CSVFile;
	CSVFile = nullptr;
}

static FAutoConsoleCommand GDumpPSOPrecacheValidationStatsCmd(
	TEXT("r.PSOPrecache.DumpStats"),
	TEXT("Dumps all currently collected PSO precache validation stats to csv file"),
	FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
{		
	PSOCollectorStats::DumpPSOPrecacheValidationStats();
}));

#endif // PSO_PRECACHING_VALIDATE && UE_WITH_PSO_PRECACHING
