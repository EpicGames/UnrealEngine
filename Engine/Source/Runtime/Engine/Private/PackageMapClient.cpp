// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/PackageMapClient.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "UObject/Package.h"
#include "EngineStats.h"
#include "Engine/Level.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "UObject/UObjectIterator.h"
#include "Engine/NetConnection.h"
#include "Net/DataBunch.h"
#include "Net/NetworkProfiler.h"
#include "Engine/ActorChannel.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "GameFramework/GameStateBase.h"
#include "HAL/LowLevelMemStats.h"
#include "Net/Core/Misc/GuidReferences.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Serialization/MemoryReader.h"
#include "Net/NetworkGranularMemoryLogging.h"
#include "Misc/CommandLine.h"
#include "Net/NetSubObjectRegistryGetter.h"

#include "Iris/ReplicationSystem/NetTokenStore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackageMapClient)

#if WITH_EDITOR
#include "UObject/ObjectRedirector.h"
#endif // WITH_EDITOR

DECLARE_LLM_MEMORY_STAT(TEXT("GuidCache"), STAT_GuidCacheLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(GuidCache, NAME_None, TEXT("Networking"), GET_STATFNAME(STAT_GuidCacheLLM), GET_STATFNAME(STAT_NetworkingSummaryLLM));

// ( OutPacketId == GUID_PACKET_NOT_ACKED ) == NAK'd		(this GUID is not acked, and is not pending either, so sort of waiting)
// ( OutPacketId == GUID_PACKET_ACKED )		== FULLY ACK'd	(this GUID is fully acked, and we no longer need to send full path)
// ( OutPacketId > GUID_PACKET_ACKED )		== PENDING		(this GUID needs to be acked, it has been recently reference, and path was sent)

static const int GUID_PACKET_NOT_ACKED	= -2;		
static const int GUID_PACKET_ACKED		= -1;		

CSV_DEFINE_CATEGORY(PackageMap, true);

#if UE_WITH_REMOTE_OBJECT_HANDLE
DEFINE_LOG_CATEGORY_STATIC(LogNetPackageMapRemoteObject, Warning, All);
#endif

/**
 * Don't allow infinite recursion of InternalLoadObject - an attacker could
 * send malicious packets that cause a stack overflow on the server.
 */
static const int INTERNAL_LOAD_OBJECT_RECURSION_LIMIT = 16;

extern FAutoConsoleVariableRef CVarEnableMultiplayerWorldOriginRebasing;

namespace UE
{
	namespace Net
	{
		extern int32 FilterGuidRemapping;

		int32 MaxSerializedNetGuids = 2048;
		static FAutoConsoleVariableRef CVarMaxSerializedNetGuids(TEXT("net.MaxSerializedNetGuids"), MaxSerializedNetGuids, TEXT("Maximum number of network guids we would expect to receive in a bunch"));

		int32 MaxSerializedReplayNetGuids = 32 * 1024;
		static FAutoConsoleVariableRef CVarMaxSerializedReplayNetGuids(TEXT("net.MaxSerializedReplayNetGuids"), MaxSerializedReplayNetGuids, TEXT("Maximum number of network guids we would expect to receive in replay export data."));

		int32 MaxSerializedNetExportGroups = 64 * 1024;
		static FAutoConsoleVariableRef CVarMaxSerializedNetExportGroups(TEXT("net.MaxSerializedNetExportGroups"), MaxSerializedNetExportGroups, TEXT("Maximum number of network export groups we would expect to receive in a bunch"));

		int32 MaxSerializedNetExportsPerGroup = 128 * 1024;	// Gameplay tags will be exported into a single large group for replays
		static FAutoConsoleVariableRef CVarMaxSerializedNetExportsPerGroup(TEXT("net.MaxSerializedNetExportsPerGroup"), MaxSerializedNetExportsPerGroup, TEXT("Maximum number of network exports in each group we would expect to receive in a bunch"));

		static bool ObjectLevelHasFinishedLoading(UObject* Object, UNetDriver* Driver)
		{
			if (Object != nullptr && Driver != nullptr && Driver->GetWorld() != nullptr)
			{
				// get the level for the object
				AActor* Actor = Cast<AActor>(Object);
				ULevel* Level = Actor ? Actor->GetLevel() : Object->GetTypedOuter<ULevel>();

				if (Level != nullptr && Level != Driver->GetWorld()->PersistentLevel)
				{
					return Level->bIsVisible;
				}
			}

			return true;
		}
	};
};

namespace UE::Net::Private
{
	void FRefCountedNetGUIDArray::Add(FNetworkGUID NetGUID)
	{
		const int32 FoundIndex = NetGUIDs.IndexOfByKey(NetGUID);

		if (RefCounts.IsValidIndex(FoundIndex))
		{
			++RefCounts[FoundIndex];
		}
		else
		{
			NetGUIDs.Add(NetGUID);
			RefCounts.Add(1);

			ensureMsgf(NetGUIDs.Num() == RefCounts.Num(), TEXT("FRefCountedNetGUIDArray::Add: arrays out of sync"));
		}
	}

	void FRefCountedNetGUIDArray::RemoveSwap(FNetworkGUID NetGUID)
	{
		const int32 FoundIndex = NetGUIDs.IndexOfByKey(NetGUID);

		if (RefCounts.IsValidIndex(FoundIndex))
		{
			--RefCounts[FoundIndex];

			ensureMsgf(RefCounts[FoundIndex] >= 0, TEXT("FRefCountedNetGUIDArray::RemoveSwap: invalid RefCount %d at index %d"), RefCounts[FoundIndex], FoundIndex);

			if (RefCounts[FoundIndex] == 0)
			{
				NetGUIDs.RemoveAtSwap(FoundIndex);
				RefCounts.RemoveAtSwap(FoundIndex);
			}
		}
	}
}

static TAutoConsoleVariable<int32> CVarAllowAsyncLoading(
	TEXT("net.AllowAsyncLoading"),
	0,
	TEXT("Allow async loading of unloaded assets referenced in packets."
		" If false the client will hitch and immediately load the asset,"
		" if true the packet will be delayed while the asset is async loaded."
		" net.DelayUnmappedRPCs can be enabled to delay RPCs relying on async loading assets.")
);

static TAutoConsoleVariable<int32> CVarIgnoreNetworkChecksumMismatch(
	TEXT("net.IgnoreNetworkChecksumMismatch"),
	0,
	TEXT("If true, the integrity checksum on packagemap objects will be ignored, which can cause issues with out of sync data")
);

static TAutoConsoleVariable<int32> CVarReservedNetGuidSize(
	TEXT("net.ReservedNetGuidSize"),
	512,
	TEXT("Reserved size in bytes for NetGUID serialization, used as a placeholder for later serialization")
);

static float GGuidCacheTrackAsyncLoadingGUIDThreshold = 0.f;
static FAutoConsoleVariableRef CVarTrackAsyncLoadingGUIDTreshold(
	TEXT("net.TrackAsyncLoadingGUIDThreshold"),
	GGuidCacheTrackAsyncLoadingGUIDThreshold,
	TEXT("When > 0, any objects that take longer than the threshold to async load will be tracked."
		" Threshold in seconds, @see FNetGUIDCache::ConsumeDelinquencyAnalytics. Used for Debugging and Analytics")
);

static float GGuidCacheTrackAsyncLoadingGUIDThresholdOwner = 0.f;
static FAutoConsoleVariableRef CVarTrackAsyncLoadingGUIDThresholdOwner(
	TEXT("net.TrackAsyncLoadingGUIDThresholdOwner"),
	GGuidCacheTrackAsyncLoadingGUIDThresholdOwner,
	TEXT("When > 0, if the Net Connection's owning Controller or Pawn is waiting on Async Loads for longer than this"
		" threshold, we will fire a CSV Event to track it. Used for Debugging and Profiling")
);

static float GPackageMapTrackQueuedActorThreshold = 0.f;
static FAutoConsoleVariableRef CVarTrackQueuedActorTreshold(
	TEXT("net.TrackQueuedActorThreshold"),
	GPackageMapTrackQueuedActorThreshold,
	TEXT("When > 0, any actors that spend longer than the threshold with queued bunches will be tracked."
		" Threshold in seconds, @see UPackageMap::ConsumeDelinquencyAnalytics. Used for Debugging and Analytics")
);

static float GPackageMapTrackQueuedActorThresholdOwner = 0.f;
static FAutoConsoleVariableRef CVarTrackQueuedActorOwnerThreshold(
	TEXT("net.TrackQueuedActorThresholdOwner"),
	GPackageMapTrackQueuedActorThresholdOwner,
	TEXT("When > 0, if the Net Connection's owning Controller or Pawn has Queued Bunches for longer than this"
		" threshold, we will fire a CSV Event to track it. Used for Debugging and Profiling")
);

static int32 GDelinquencyNumberOfTopOffendersToTrack = 10;
static FAutoConsoleVariableRef CVarDelinquencyNumberOfTopOffendersToTrack(
	TEXT("net.DelinquencyNumberOfTopOffendersToTrack"),
	GDelinquencyNumberOfTopOffendersToTrack,
	TEXT("When > 0 , this will be the number of 'TopOffenders' that are tracked by the PackageMap and GuidCache for"
		" Queued Actors and Async Loads respectively."
		" net.TrackAsyncLoadingGUIDThreshold / net.TrackQueuedActorThreshold still dictate whether or not any of these"
		" items are tracked.")
);

static bool GbAllowClientRemapCacheObject = false;
static FAutoConsoleVariableRef CVarAllowClientRemapCacheObject(
	TEXT("net.AllowClientRemapCacheObject"),
	GbAllowClientRemapCacheObject,
	TEXT("When enabled, we will allow clients to remap read only cache objects and keep the same NetGUID.")
);

static bool GbQuantizeActorScaleOnSpawn = false;
static FAutoConsoleVariableRef CVarQuantizeActorScaleOnSpawn(
	TEXT("net.QuantizeActorScaleOnSpawn"),
	GbQuantizeActorScaleOnSpawn,
	TEXT("When enabled, we will quantize Scale for newly spawned actors to a single decimal of precision.")
);

static bool GbQuantizeActorLocationOnSpawn = true;
static FAutoConsoleVariableRef CVarQuantizeActorLocationOnSpawn(
	TEXT("net.QuantizeActorLocationOnSpawn"),
	GbQuantizeActorLocationOnSpawn,
	TEXT("When enabled, we will quantize Location for newly spawned actors to a single decimal of precision.")
);

static bool GbQuantizeActorVelocityOnSpawn = true;
static FAutoConsoleVariableRef CVarQuantizeActorVelocityOnSpawn(
	TEXT("net.QuantizeActorVelocityOnSpawn"),
	GbQuantizeActorVelocityOnSpawn,
	TEXT("When enabled, we will quantize Velocity for newly spawned actors to a single decimal of precision.")
);

static bool GbQuantizeActorRotationOnSpawn = true;
static FAutoConsoleVariableRef CVarQuantizeActorRotationOnSpawn(
	TEXT("net.QuantizeActorRotationOnSpawn"),
	GbQuantizeActorRotationOnSpawn,
	TEXT("When enabled, we will quantize Rotation for newly spawned actors to a single decimal of precision.")
);

static bool GbNetCheckNoLoadPackages = true;
static FAutoConsoleVariableRef CVarNetCheckNoLoadPackages(
	TEXT("net.CheckNoLoadPackages"),
	GbNetCheckNoLoadPackages,
	TEXT("If enabled, check the no load flag in GetObjectFromNetGUID before forcing a sync load on packages that are not marked IsFullyLoaded")
);

static bool bNetReportSyncLoads = false;
static FAutoConsoleVariableRef CVarNetReportSyncLoads(
	TEXT("net.ReportSyncLoads"),
	bNetReportSyncLoads,
	TEXT("If enabled, the engine will track objects loaded by the networking system and broadcast FNetDelegates::OnSyncLoadDetected to report them."
		"By default they are logged to the LogNetSyncLoads category.")
);

void BroadcastNetFailure(UNetDriver* Driver, ENetworkFailure::Type FailureType, const FString& ErrorStr)
{
	UWorld* World = Driver->GetWorld();

	TWeakObjectPtr<UWorld> WeakWorld(World);
	TWeakObjectPtr<UNetDriver> WeakDriver(Driver);

	auto BroadcastFailureNextFrame = [WeakWorld, WeakDriver, FailureType, ErrorStr]()
	{
		UWorld* LambdaWorld = nullptr;
		UNetDriver* NetDriver = nullptr;
		if (WeakWorld.IsValid())
		{
			LambdaWorld = WeakWorld.Get();
		}

		if (WeakDriver.IsValid())
		{
			NetDriver = WeakDriver.Get();
		}

		GEngine->BroadcastNetworkFailure(LambdaWorld, NetDriver, FailureType, ErrorStr);
	};

	if (World)
	{
		FTimerManager& TM = World->GetTimerManager();
		TM.SetTimerForNextTick(FTimerDelegate::CreateLambda(BroadcastFailureNextFrame));
	}
	else
	{
		BroadcastFailureNextFrame();
	}
}

/*-----------------------------------------------------------------------------
	UPackageMapClient implementation.
-----------------------------------------------------------------------------*/
UPackageMapClient::UPackageMapClient(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Connection(nullptr)
	, DelinquentQueuedActors(GDelinquencyNumberOfTopOffendersToTrack > 0 ? GDelinquencyNumberOfTopOffendersToTrack : 0)
{
}

void UPackageMapClient::Initialize(UNetConnection * InConnection, TSharedPtr<FNetGUIDCache> InNetGUIDCache)
{
	Connection = InConnection;
	GuidCache = InNetGUIDCache;
	ExportNetGUIDCount = 0;
	OverrideAckState = &AckState;

	// Initialize NetTokenResolveContext
	NetTokenResolveContext.NetTokenStore = InConnection->GetDriver()->GetNetTokenStore();
	if (NetTokenResolveContext.NetTokenStore)
	{
		NetTokenResolveContext.RemoteNetTokenStoreState = NetTokenResolveContext.NetTokenStore->GetRemoteNetTokenStoreState(InConnection->GetConnectionHandle().GetParentConnectionId());
	}
}

/**
 *	This is the meat of the PackageMap class which serializes a reference to Object.
 */
bool UPackageMapClient::SerializeObject( FArchive& Ar, UClass* Class, UObject*& Object, FNetworkGUID *OutNetGUID)
{
	SCOPE_CYCLE_COUNTER(STAT_PackageMap_SerializeObjectTime);
	
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static IConsoleVariable* DebugObjectCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.PackageMap.DebugObject"));
	static IConsoleVariable* DebugAllObjectsCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.PackageMap.DebugAll"));
	if (Object &&
		((DebugObjectCvar && !DebugObjectCvar->GetString().IsEmpty() && Object->GetName().Contains(DebugObjectCvar->GetString())) ||
		(DebugAllObjectsCvar && DebugAllObjectsCvar->GetInt() != 0)))
	{
		UE_LOGF(LogNetPackageMap, Log, "Serialized Object %ls", *Object->GetName());
	}
#endif

	if (Ar.IsSaving())
	{
		// If pending kill, just serialize as NULL.
		// TWeakObjectPtrs of PendingKill objects will behave strangely with TSets and TMaps
		//	PendingKill objects will collide with each other and with NULL objects in those data structures.
		if (Object && !IsValid(Object))
		{
			UObject* NullObj = NULL;
			return SerializeObject( Ar, Class, NullObj, OutNetGUID);
		}

		FNetworkGUID NetGUID = GuidCache->GetOrAssignNetGUID( Object );

		// Write out NetGUID to caller if necessary
		if (OutNetGUID)
		{
			*OutNetGUID = NetGUID;
		}

		// Write object NetGUID to the given FArchive
		InternalWriteObject( Ar, NetGUID, Object, TEXT( "" ), NULL );

		// If we need to export this GUID (its new or hasnt been ACKd, do so here)
		if (!NetGUID.IsDefault() && Object && ShouldSendFullPath(Object, NetGUID))
		{
			check(IsNetGUIDAuthority());
			if ( !ExportNetGUID( NetGUID, Object, TEXT(""), NULL ) )
			{
				UE_LOGF( LogNetPackageMap, Verbose, "Failed to export in ::SerializeObject %ls", *Object->GetName() );
			}
		}

		return true;
	}
	else if (Ar.IsLoading())
	{
		FNetworkGUID NetGUID;
		double LoadTime = 0.0;
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			FScopedDurationTimer NetSerializeTime(LoadTime);
#endif

			// ----------------	
			// Read NetGUID from stream and resolve object
			// ----------------	
			NetGUID = InternalLoadObject(Ar, Object, 0);

			// Write out NetGUID to caller if necessary
			if (OutNetGUID)
			{
				*OutNetGUID = NetGUID;
			}

#if 0		// Enable this code to force any actor with missing/broken content to not load in replays
			if (NetGUID.IsValid() && Connection->IsInternalAck() && GuidCache->IsGUIDBroken(NetGUID, true))
			{
				Ar.SetError();
				UE_LOGF(LogNetPackageMap, Warning, "UPackageMapClient::SerializeObject: InternalAck GUID broken.");
				return false;
			}
#endif

			// ----------------	
			// Final Checks/verification
			// ----------------	

			// NULL if we haven't finished loading the objects level yet
			if (!ObjectLevelHasFinishedLoading(Object))
			{
				UE_LOGF(LogNetPackageMap, Warning, "Using None instead of replicated reference to %ls because the level it's in has not been made visible", *Object->GetFullName());
				Object = NULL;
			}

			// Check that we got the right class
			if (Object && !(Class->HasAnyClassFlags(CLASS_Interface) ? Object->GetClass()->ImplementsInterface(Class) : Object->IsA(Class)))
			{
				UE_LOGF(LogNetPackageMap, Warning, "Forged object: got %ls, expecting %ls", *Object->GetFullName(), *Class->GetFullName());
				Object = NULL;
			}

			if ( NetGUID.IsValid() && bShouldTrackUnmappedGuids && !GuidCache->IsGUIDBroken( NetGUID, false ) )
			{
				if ( Object == nullptr )
				{
					TrackedUnmappedNetGuids.Add( NetGUID );
				}
				else if ( NetGUID.IsDynamic() )
				{
					TrackedMappedDynamicNetGuids.Add( NetGUID );
				}
			}

			if (bNetReportSyncLoads && NetGUID.IsValid())
			{
				// Track the GUID of anything in the outer chain that was sync loaded, to catch packages.
				for (FNetworkGUID CurrentGUID = NetGUID; CurrentGUID.IsValid(); CurrentGUID = GuidCache->GetOuterNetGUID(CurrentGUID))
				{
					if (GuidCache->WasGUIDSyncLoaded(CurrentGUID))
					{
						TrackedSyncLoadedGUIDs.Add(CurrentGUID);
					}
				}
			}

			UE_LOGF(LogNetPackageMap, VeryVerbose, "UPackageMapClient::SerializeObject Serialized Object %ls as <%ls>", Object ? *Object->GetPathName() : TEXT("NULL"), *NetGUID.ToString());
		}
		
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		static IConsoleVariable* LongLoadThreshholdCVAR = IConsoleManager::Get().FindConsoleVariable(TEXT("net.PackageMap.LongLoadThreshhold"));		
		if (LongLoadThreshholdCVAR && ((float)LoadTime > LongLoadThreshholdCVAR->GetFloat()))
		{
			UE_LOGF(LogNetPackageMap, Warning, "Long net serialize: %fms, Serialized Object %ls", (float)LoadTime * 1000.0f, *GetNameSafe(Object));
		}
#endif

		// reference is mapped if it was not NULL (or was explicitly null)
		return (Object != NULL || !NetGUID.IsValid());
	}

	return true;
}

/**
 *	Slimmed down version of SerializeObject, that writes an object reference given a NetGUID and name
 *	(e.g, it does not require the actor to actually exist anymore to serialize the reference).
 *	This must be kept in sync with UPackageMapClient::SerializeObject.
 */
bool UPackageMapClient::WriteObject( FArchive& Ar, UObject* ObjOuter, FNetworkGUID NetGUID, FString ObjName )
{
	Ar << NetGUID;
	NET_CHECKSUM(Ar);

	UE_LOGF(LogNetPackageMap, Log, "WroteObject %ls NetGUID <%ls>", *ObjName, *NetGUID.ToString() );

	if (NetGUID.IsStatic() && !NetGUID.IsDefault() && !NetGUIDHasBeenAckd(NetGUID))
	{
		if ( !ExportNetGUID( NetGUID, NULL, ObjName, ObjOuter ) )
		{
			UE_LOGF( LogNetPackageMap, Warning, "Failed to export in ::WriteObject %ls", *ObjName );
		}
	}

	return true;
}

bool UPackageMapClient::IsUsingRemoteObjectReferences() const
{
	if (Connection)
	{
		if (UNetDriver* Driver = Connection->Driver.Get())
		{
			return Driver->IsUsingRemoteObjectReferences();
		}
	}
	return false;
}

/**
 *	Standard method of serializing a new actor.
 *		For static actors, this will just be a single call to SerializeObject, since they can be referenced by their path name.
 *		For dynamic actors, first the actor's reference is serialized but will not resolve on clients since they haven't spawned the actor yet.
 *		The actor archetype is then serialized along with the starting location, rotation, and velocity.
 *		After reading this information, the client spawns this actor in the NetDriver's World and assigns it the NetGUID it read at the top of the function.
 *
 *		returns true if a new actor was spawned. false means an existing actor was found for the netguid.
 */
bool UPackageMapClient::SerializeNewActor(FArchive& Ar, class UActorChannel *Channel, class AActor*& Actor)
{
	LLM_SCOPE(ELLMTag::EngineMisc);

	UE_LOGF( LogNetPackageMap, VeryVerbose, "SerializeNewActor START" );

	uint8 bIsClosingChannel = 0;

	if (Ar.IsLoading() )
	{
		FInBunch* InBunch = (FInBunch*)&Ar;
		bIsClosingChannel = InBunch->bClose;		// This is so we can determine that this channel was opened/closed for destruction
		UE_LOG(LogNetPackageMap, Log, TEXT("UPackageMapClient::SerializeNewActor BitPos: %" INT64_FMT), InBunch->GetPosBits() );

		ResetTrackedSyncLoadedGuids();
	}

	NET_CHECKSUM(Ar);

	FNetworkGUID NetGUID;
	UObject *NewObj = Actor;
	SerializeObject(Ar, AActor::StaticClass(), NewObj, &NetGUID);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogNetPackageMapRemoteObject, Log, "UPackageMapClient::SerializeNewActor: Actor=%ls NetGUID=%ls",
		   *GetNameSafe(NewObj), *NetGUID.ToString());
#endif

	if ( Ar.IsError() )
	{
		UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::SerializeNewActor: Ar.IsError after SerializeObject 1" );
		return false;
	}

	if (UE::Net::FilterGuidRemapping == 0)
	{
		if ( GuidCache.IsValid() )
		{
			if (ensureMsgf(NetGUID.IsValid(), TEXT("Channel tried to add an invalid GUID to the import list: %s"), *Channel->Describe()))
			{
				LLM_SCOPE_BYTAG(GuidCache);
				GuidCache->ImportedNetGuids.Add( NetGUID );
			}
		}
	}

	Channel->ActorNetGUID = NetGUID;

	Actor = Cast<AActor>(NewObj);

	// When we return an actor, we don't necessarily always spawn it (we might have found it already in memory)
	// The calling code may want to know, so this is why we distinguish
	bool bActorWasSpawned = false;

	if ( Ar.AtEnd() && NetGUID.IsDynamic() )
	{
		// This must be a destruction info coming through or something is wrong
		// If so, we should be closing the channel
		// This can happen when dormant actors that don't have channels get destroyed
		// Not finding the actor can happen if the client streamed in this level after a dynamic actor has been spawned and deleted on the server side
		if ( bIsClosingChannel == 0 )
		{
			UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::SerializeNewActor: bIsClosingChannel == 0 : %ls [%ls]", *GetNameSafe(Actor), *NetGUID.ToString() );
			Ar.SetError();
			return false;
		}

		UE_LOGF( LogNetPackageMap, Log, "UPackageMapClient::SerializeNewActor:  Skipping full read because we are deleting dynamic actor: %ls", *GetNameSafe(Actor) );
		return false;		// This doesn't mean an error. This just simply means we didn't spawn an actor.
	}

	if (UE::Net::FilterGuidRemapping != 0)
	{
		// Do not mark guid as imported until we know we aren't deleting it
		if ( GuidCache.IsValid() )
		{
			if (ensureMsgf(NetGUID.IsValid(), TEXT("Channel tried to add an invalid GUID to the import list: %s"), *Channel->Describe()))
			{
				LLM_SCOPE_BYTAG(GuidCache);
				GuidCache->ImportedNetGuids.Add(NetGUID);
			}
		}
	}

	if ( NetGUID.IsDynamic() )
	{
		UObject* Archetype = nullptr;
		UObject* ActorLevel = nullptr;
		FVector Location = FVector::ZeroVector;
		FVector Scale = FVector::OneVector;
		FVector Velocity = FVector::ZeroVector;
		FRotator Rotation = FRotator::ZeroRotator;
		bool SerSuccess = false;

		if (Ar.IsSaving())
		{
			// ChildActor's need to be spawned from the ChildActorTemplate otherwise any non-replicated 
			// customized properties will be incorrect on the Client.
			if (UChildActorComponent* CAC = Actor->GetParentComponent())
			{
				Archetype = CAC->GetSpawnableChildActorTemplate();
			}
			if (Archetype == nullptr)
			{
				Archetype = Actor->GetArchetype();
			}

			// If enabled, send the actor's level to the client. If left null, the client will spawn the actor in the persistent level.
			if (UE::Net::Private::SerializeNewActorOverrideLevel)
			{
				ActorLevel = Actor->GetLevel();
			}

			check( Archetype != nullptr );
			check( Actor->NeedsLoadForClient() );			// We have no business sending this unless the client can load
			check( Archetype->NeedsLoadForClient() );		// We have no business sending this unless the client can load

			const USceneComponent* RootComponent = Actor->GetRootComponent();

			if (RootComponent)
			{
				Location = FRepMovement::RebaseOntoZeroOrigin(Actor->GetActorLocation(), Actor);
				Rotation = Actor->GetActorRotation();
				Scale = Actor->GetActorScale();

				if (USceneComponent* AttachParent = RootComponent->GetAttachParent())
				{
					// If this actor is attached, when the scale is serialized on the client, the attach parent property won't be set yet.
					// USceneComponent::SetWorldScale3D (which got called by AActor::SetActorScale3D, which we used to do but no longer).
					// would perform this transformation so that what is sent is relative to the parent. If we don't do this, we will
					// apply the world scale on the client, which will then get applied a second time when the attach parent property is received.
					FTransform ParentToWorld = AttachParent->GetSocketTransform(RootComponent->GetAttachSocketName());
					Scale = Scale * ParentToWorld.GetSafeScaleReciprocal(ParentToWorld.GetScale3D());
				}
				Velocity = Actor->GetVelocity();
			}
		}

		FNetworkGUID ArchetypeNetGUID;
		SerializeObject(Ar, UObject::StaticClass(), Archetype, &ArchetypeNetGUID);

		if (Ar.IsSaving() || (Connection && (Connection->GetNetworkCustomVersion(FEngineNetworkCustomVersion::Guid) >= FEngineNetworkCustomVersion::NewActorOverrideLevel)))
		{
			SerializeObject(Ar, ULevel::StaticClass(), ActorLevel);
		}

#if WITH_EDITOR
		UObjectRedirector* ArchetypeRedirector = Cast<UObjectRedirector>(Archetype);
		if (ArchetypeRedirector)
		{
			// Redirectors not supported
			Archetype = nullptr;
		}
#endif // WITH_EDITOR

		if ( ArchetypeNetGUID.IsValid() && Archetype == NULL )
		{
			const FNetGuidCacheObject* ExistingCacheObjectPtr = GuidCache->ObjectLookup.Find( ArchetypeNetGUID );

			if ( ExistingCacheObjectPtr != NULL )
			{
				UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::SerializeNewActor. Unresolved Archetype GUID. Path: %ls, NetGUID: %ls.", *ExistingCacheObjectPtr->PathName.ToString(), *ArchetypeNetGUID.ToString() );
			}
			else
			{
				UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::SerializeNewActor. Unresolved Archetype GUID. Guid not registered! NetGUID: %ls.", *ArchetypeNetGUID.ToString() );
			}
		}

		// SerializeCompressedInitial
		// only serialize the components that need to be serialized otherwise default them
		bool bSerializeLocation = false;
		bool bSerializeRotation = false;
		bool bSerializeScale = false;
		bool bSerializeVelocity = false;

		{			
			// Server is serializing an object to be sent to a client
			if (Ar.IsSaving())
			{
				// We use 0.01f for comparing when using quantization, because we will only send a single point of precision anyway.
				// We could probably get away with 0.1f, but that may introduce edge cases for rounding.
				static constexpr float Epsilon_Quantized = 0.01f;
				
				// We use KINDA_SMALL_NUMBER for comparing when not using quantization, because that's the default for FVector::Equals.
				static constexpr float Epsilon = UE_KINDA_SMALL_NUMBER;

				bSerializeLocation = !Location.Equals(FVector::ZeroVector, GbQuantizeActorLocationOnSpawn ? Epsilon_Quantized : Epsilon);
				bSerializeVelocity = !Velocity.Equals(FVector::ZeroVector, GbQuantizeActorVelocityOnSpawn ? Epsilon_Quantized : Epsilon);
				bSerializeScale = !Scale.Equals(FVector::OneVector, GbQuantizeActorScaleOnSpawn ? Epsilon_Quantized : Epsilon);

				// We use 0.001f for Rotation comparison to keep consistency with old behavior.
				bSerializeRotation = !Rotation.IsNearlyZero(0.001f);
				
			}

			auto ConditionallySerializeQuantizedVector = [this, &Ar, &SerSuccess](
				FVector& InOutValue,
				const FVector& DefaultValue,
				bool bShouldQuantize,
				bool& bWasSerialized)
			{
				Ar.SerializeBits(&bWasSerialized, 1);
				if (bWasSerialized)
				{
					if (Ar.IsLoading() && Ar.EngineNetVer() < FEngineNetworkCustomVersion::OptionallyQuantizeSpawnInfo)
					{
						bShouldQuantize = true;
					}
					else
					{
						Ar.SerializeBits(&bShouldQuantize, 1);
					}

					if (bShouldQuantize)
					{
						FVector_NetQuantize10 Temp = InOutValue;
						Temp.NetSerialize(Ar, this, SerSuccess);
						InOutValue = Temp;
					}
					else
					{
						Ar << InOutValue;
					}
				}
				else
				{
					InOutValue = DefaultValue;
				}
			};

			ConditionallySerializeQuantizedVector(Location, FVector::ZeroVector, GbQuantizeActorLocationOnSpawn, bSerializeLocation);

			Ar.SerializeBits(&bSerializeRotation, 1);
			if (bSerializeRotation)
			{
				if (GbQuantizeActorRotationOnSpawn)
				{
					Rotation.NetSerialize(Ar, this, SerSuccess);
				} 
				else
				{
					Ar << Rotation;
				}
			}
			else
			{
				Rotation = FRotator::ZeroRotator;
			}

			ConditionallySerializeQuantizedVector(Scale, FVector::OneVector, GbQuantizeActorScaleOnSpawn, bSerializeScale);
			ConditionallySerializeQuantizedVector(Velocity, FVector::ZeroVector, GbQuantizeActorVelocityOnSpawn, bSerializeVelocity);
		}

		if ( Ar.IsLoading() )
		{
			// Spawn actor if necessary (we may have already found it if it was dormant)
			if ( Actor == NULL )
			{
				if ( Archetype )
				{
					// For streaming levels, it's possible that the owning level has been made not-visible but is still loaded.
					// In that case, the level will still be found but the owning world will be invalid.
					// If that happens, wait to spawn the Actor until the next time the level is streamed in.
					// At that point, the Server should resend any dynamic Actors.
					ULevel* SpawnLevel = Cast<ULevel>(ActorLevel);
					if (SpawnLevel == nullptr || SpawnLevel->GetWorld() != nullptr)
					{
						FActorSpawnParameters SpawnInfo;
						SpawnInfo.Template = Cast<AActor>(Archetype);
						SpawnInfo.OverrideLevel = SpawnLevel;
						SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
						SpawnInfo.bRemoteOwned = true;
						SpawnInfo.bNoFail = true;

						UWorld* World = Connection->Driver->GetWorld();
						FVector SpawnLocation = FRepMovement::RebaseOntoLocalOrigin(Location, World->OriginLocation);
						Actor = World->SpawnActorAbsolute(Archetype->GetClass(), FTransform(Rotation, SpawnLocation), SpawnInfo);
						if (Actor)
						{
							// Velocity was serialized by the server
							if (bSerializeVelocity)
							{
								Actor->PostNetReceiveVelocity(Velocity);
							}

							// Scale was serialized by the server
							if (bSerializeScale)
							{
								Actor->SetActorRelativeScale3D(Scale);
							}

							GuidCache->RegisterNetGUID_Client(NetGUID, Actor);
							bActorWasSpawned = true;
						}
						else
						{
							UE_LOGF(LogNetPackageMap, Warning, "SerializeNewActor: Failed to spawn actor for NetGUID: %ls, Channel: %d", *NetGUID.ToString(), Channel->ChIndex);
						}
					}
					else
					{
						UE_LOGF(LogNetPackageMap, Log, "SerializeNewActor: Actor level has invalid world (may be streamed out). NetGUID: %ls, Channel: %d", *NetGUID.ToString(), Channel->ChIndex);
					}
				}
				else
				{
					UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::SerializeNewActor Unable to read Archetype for NetGUID %ls / %ls", *NetGUID.ToString(), *ArchetypeNetGUID.ToString() );
				}
			}
		}
	}
	else if ( Ar.IsLoading() && Actor == NULL )
	{
		// Do not log a warning during replay, since this is a valid case
		UE_CLOGF(!Connection->IsReplay(), LogNetPackageMap, Log, "SerializeNewActor: Failed to find static actor: FullNetGuidPath: %ls, Channel: %d", *GuidCache->FullNetGUIDPath(NetGUID), Channel->ChIndex);

		if (UE::Net::FilterGuidRemapping != 0)
		{
			// Do not attempt to resolve this missing actor
			if ( GuidCache.IsValid() )
			{
				GuidCache->ImportedNetGuids.Remove( NetGUID );
			}
		}
	}

	if (Ar.IsLoading())
	{
		ReportSyncLoadsForActorSpawn(Actor);
	}

	UE_LOGF( LogNetPackageMap, Log, "SerializeNewActor END: Finished Serializing. Actor: %ls, FullNetGUIDPath: %ls, Channel: %d, IsLoading: %i, IsDynamic: %i", Actor ? *Actor->GetName() : TEXT("NULL"), *GuidCache->FullNetGUIDPath( NetGUID ), Channel->ChIndex, (int)Ar.IsLoading(), (int)NetGUID.IsDynamic() );

	return bActorWasSpawned;
}

//--------------------------------------------------------------------
//
//	Writing
//
//--------------------------------------------------------------------

struct FExportFlags
{
	union
	{
		struct
		{
			uint8 bHasPath				: 1;
			uint8 bNoLoad				: 1;
			uint8 bHasNetworkChecksum	: 1;
		};

		uint8	Value;
	};

	FExportFlags()
	{
		Value = 0;
	}
};

bool FNetGUIDCache::CanClientLoadObject( const UObject* Object, const FNetworkGUID& NetGUID ) const
{
	if ( !NetGUID.IsValid() || NetGUID.IsDynamic() )
	{
		return false;		// We should never tell the client to load dynamic objects (actors or objects created during play for example)
	}

	// PackageMapClient can't load maps, we must wait for the client to load the map when ready
	// These guids are special guids, where the guid and all child guids resolve once the map has been loaded
	if (Object)
	{
		if (Object->GetPackage()->ContainsMap())
		{
			return false;
		}

#if WITH_EDITOR
		// For objects using external package, we need to do the test on the package of their outer most object
		// (this is currently only possible in Editor)
		UObject* OutermostObject = Object->GetOutermostObject();
		if (OutermostObject && OutermostObject->GetPackage()->ContainsMap())
		{
			return false;
		}
#endif
	}

	// If the object is null, we can't check whether the outermost contains a map anymore, so
	// see if there is already a cache entry for the GUID and if so, use its existing NoLoad value.
	// Fixes an edge case where if a GUID is being exported for a map object after the object is
	// destroyed due to latency/timing issues, this function could return true and ultimately
	// cause the server to try to re-load map objects.
	if ( Object == nullptr && IsGUIDNoLoad(NetGUID) )
	{
		return false;
	}

	// We can load everything else
	return true;
}

/** Writes an object NetGUID given the NetGUID and either the object itself, or FString full name of the object. Appends full name/path if necessary */
void UPackageMapClient::InternalWriteObject(FArchive & Ar, FNetworkGUID NetGUID, UObject* Object, FString ObjectPathName, UObject* ObjectOuter)
{
	check(Ar.IsSaving());

	const bool bNoLoad = !GuidCache->CanClientLoadObject(Object, NetGUID);

	if (GuidCache->ShouldAsyncLoad() && IsNetGUIDAuthority() && !GuidCache->IsExportingNetGUIDBunch && !bNoLoad)
	{
		// These are guids that must exist on the client in a package
		// The client needs to know about these so it can determine if it has finished loading them
		// and pause the network stream for that channel if it hasn't
		MustBeMappedGuidsInLastBunch.AddUnique(NetGUID);
	}

	Ar << NetGUID;
	NET_CHECKSUM(Ar);

	if (!NetGUID.IsValid())
	{
		// We're done writing
		return;
	}

	// Write export flags
	//   note: Default NetGUID is implied to always send path
	FExportFlags ExportFlags;

	ExportFlags.bHasNetworkChecksum = (GuidCache->NetworkChecksumMode != FNetGUIDCache::ENetworkChecksumMode::None) ? 1 : 0;

	if (NetGUID.IsDefault())
	{
		// Only the client sends default guids
		check(!IsNetGUIDAuthority());
		ExportFlags.bHasPath = 1;

		Ar << ExportFlags.Value;
	}
	else if (GuidCache->IsExportingNetGUIDBunch)
	{
		// Only the server should be exporting guids
		check(IsNetGUIDAuthority());

		if (Object != nullptr)
		{
			ExportFlags.bHasPath = ShouldSendFullPath(Object, NetGUID) ? 1 : 0;
		}
		else
		{
			ExportFlags.bHasPath = ObjectPathName.IsEmpty() ? 0 : 1;
		}

		ExportFlags.bNoLoad	= bNoLoad ? 1 : 0;

		Ar << ExportFlags.Value;
	}

	if (ExportFlags.bHasPath)
	{
		if (Object != nullptr)
		{
			// If the object isn't nullptr, expect an empty path name, then fill it out with the actual info
			check(ObjectOuter == nullptr);
			check(ObjectPathName.IsEmpty());
			ObjectPathName = Object->GetName();
			ObjectOuter = Object->GetOuter();
		}
		else
		{
			// If we don't have an object, expect an already filled out path name
			checkf(ObjectOuter != nullptr, TEXT("ObjectOuter is null. NetGuid: %s. Object: %s. ObjectPathName: %s"), *NetGUID.ToString(), *GetPathNameSafe(Object), *ObjectPathName);
			checkf(!ObjectPathName.IsEmpty(), TEXT("ObjectPathName is empty. NetGuid: %s. Object: %s"), *NetGUID.ToString(), *GetPathNameSafe(Object));
		}

		const bool bIsPackage = (NetGUID.IsStatic() && Object != nullptr && Object->GetOuter() == nullptr);

		check(bIsPackage == (Cast<UPackage>(Object) != nullptr));		// Make sure it really is a package

		// Serialize reference to outer. This is basically a form of compression.
		FNetworkGUID OuterNetGUID = GuidCache->GetOrAssignNetGUID(ObjectOuter);

		InternalWriteObject(Ar, OuterNetGUID, ObjectOuter, TEXT( "" ), nullptr);

		// Look for renamed startup actors
		if (Connection->Driver)
		{
			const FName SearchPath = FName(*ObjectPathName);
			const FName RenamedPath = Connection->Driver->RenamedStartupActors.FindRef(SearchPath);
			if (!RenamedPath.IsNone())
			{
				ObjectPathName = RenamedPath.ToString();
			}
		}

#if WITH_EDITOR
		FString TempObjectName = ObjectPathName;
#endif

		GEngine->NetworkRemapPath(Connection, ObjectPathName, false);

#if WITH_EDITOR
		ensureMsgf(!ObjectPathName.IsEmpty(), TEXT("NetworkRemapPath found PathName: %s to be an invalid name for %s. This object will not replicate!"), *TempObjectName, *GetPathNameSafe(Object));
#endif

		// Serialize Name of object
		Ar << ObjectPathName;

		uint32 NetworkChecksum = 0;

		if ( ExportFlags.bHasNetworkChecksum )
		{
			NetworkChecksum = GuidCache->GetNetworkChecksum(Object);
			Ar << NetworkChecksum;
		}

		if (FNetGuidCacheObject* CacheObject = GuidCache->ObjectLookup.Find(NetGUID))
		{
			if (CacheObject->PathName.IsNone())
			{
				CacheObject->PathName = FName(*ObjectPathName);
			}

			CacheObject->OuterGUID			= OuterNetGUID;
			CacheObject->bNoLoad			= ExportFlags.bNoLoad;
			CacheObject->bIgnoreWhenMissing = ExportFlags.bNoLoad;
			CacheObject->NetworkChecksum	= NetworkChecksum;
		}

		if (GuidCache->IsExportingNetGUIDBunch)
		{
			CurrentExportNetGUIDs.Add(NetGUID);

			int32& ExportCount = NetGUIDExportCountMap.FindOrAdd(NetGUID);
			ExportCount++;
		}
	}
}

//--------------------------------------------------------------------
//
//	Loading
//
//--------------------------------------------------------------------

static void SanityCheckExport( 
	const FNetGUIDCache *	GuidCache,
	const UObject *			Object, 
	const FNetworkGUID &	NetGUID, 
	const FString &			ExpectedPathName, 
	const UObject *			ExpectedOuter, 
	const FNetworkGUID &	ExpectedOuterGUID,
	const FExportFlags &	ExportFlags )
{
	
	check( GuidCache != NULL );
	check( Object != NULL );

	const FNetGuidCacheObject* CacheObject = GuidCache->ObjectLookup.Find( NetGUID );

	if ( CacheObject != NULL )
	{
		if ( CacheObject->OuterGUID != ExpectedOuterGUID )
		{
			UE_LOGF( LogNetPackageMap, Warning, "SanityCheckExport: CacheObject->OuterGUID != ExpectedOuterGUID. NetGUID: %ls, Object: %ls, Expected: %ls", *NetGUID.ToString(), *Object->GetPathName(), *ExpectedPathName );
		}
	}
	else
	{
		UE_LOGF( LogNetPackageMap, Warning, "SanityCheckExport: CacheObject == NULL. NetGUID: %ls, Object: %ls, Expected: %ls", *NetGUID.ToString(), *Object->GetPathName(), *ExpectedPathName );
	}

	if ( Object->GetName() != ExpectedPathName )
	{
		UE_LOGF( LogNetPackageMap, Warning, "SanityCheckExport: Name mismatch. NetGUID: %ls, Object: %ls, Expected: %ls", *NetGUID.ToString(), *Object->GetPathName(), *ExpectedPathName );
	}

	if ( Object->GetOuter() != ExpectedOuter )
	{
		const FString CurrentOuterName	= Object->GetOuter() != NULL ? *Object->GetOuter()->GetName() : TEXT( "NULL" );
		const FString ExpectedOuterName = ExpectedOuter != NULL ? *ExpectedOuter->GetName() : TEXT( "NULL" );
		UE_LOGF( LogNetPackageMap, Warning, "SanityCheckExport: Outer mismatch. Object: %ls, NetGUID: %ls, Current: %ls, Expected: %ls", *Object->GetPathName(), *NetGUID.ToString(), *CurrentOuterName, *ExpectedOuterName );
	}

	const bool bIsPackage = ( NetGUID.IsStatic() && Object->GetOuter() == NULL );
	const UPackage* Package = Cast< const UPackage >( Object );

	if ( bIsPackage != ( Package != NULL ) )
	{
		UE_LOGF( LogNetPackageMap, Warning, "SanityCheckExport: Package type mismatch. Object:%ls, NetGUID: %ls", *Object->GetPathName(), *NetGUID.ToString() );
	}
}

/** Loads a UObject from an FArchive stream. Reads object path if there, and tries to load object if its not already loaded */
FNetworkGUID UPackageMapClient::InternalLoadObject( FArchive & Ar, UObject *& Object, const int32 InternalLoadObjectRecursionCount )
{
	if ( InternalLoadObjectRecursionCount > INTERNAL_LOAD_OBJECT_RECURSION_LIMIT ) 
	{
		UE_LOGF( LogNetPackageMap, Warning, "InternalLoadObject: Hit recursion limit." );
		Ar.SetError(); 
		Object = NULL;
		return FNetworkGUID(); 
	} 

	// ----------------	
	// Read the NetGUID
	// ----------------	
	FNetworkGUID NetGUID;
	Ar << NetGUID;
	NET_CHECKSUM_OR_END( Ar );

	if ( Ar.IsError() )
	{
		Object = NULL;
		return NetGUID;
	}

	if ( !NetGUID.IsValid() )
	{
		Object = NULL;
		return NetGUID;
	}

	// ----------------	
	// Try to resolve NetGUID
	// ----------------	
	if ( NetGUID.IsValid() && !NetGUID.IsDefault() )
	{
		Object = GetObjectFromNetGUID( NetGUID, GuidCache->IsExportingNetGUIDBunch );

#if UE_WITH_REMOTE_OBJECT_HANDLE
		UE_LOGF(LogNetPackageMapRemoteObject, Log, "Looking up NetGUID %ls in the local cache and found '%ls'", *NetGUID.ToString(), *GetNameSafe(Object));

		if (Object == nullptr)
		{
			FRemoteObjectId RemoteObjectId = FNetGUIDCache::GetRemoteObjectIdFromNetworkGUID(NetGUID);
			if (UE::RemoteObject::Handle::GetResidence(RemoteObjectId) != EResidence::Local)
			{
				if (AutoRTFM::IsTransactional())
				{
					UE_LOGF(LogNetPackageMapRemoteObject, Log, "Dereferencing a remote NetGUID %ls (%ls).", *NetGUID.ToString(), *RemoteObjectId.ToString());
					TObjectPtr<UObject> RemoteObjectPtr = TObjectPtr<UObject>(FObjectPtr(RemoteObjectId));
					Object = RemoteObjectPtr.Get();
				}
				else
				{
					UE_LOGF(LogNetPackageMapRemoteObject, Log, "Dereferencing a remote NetGUID %ls (%ls) that is outside a transaction.", *NetGUID.ToString(), *RemoteObjectId.ToString());
				}
			}
		}
#endif

		UE_LOGF(LogNetPackageMap, VeryVerbose, "InternalLoadObject loaded %ls from NetGUID <%ls>", Object ? *Object->GetFullName() : TEXT( "NULL" ), *NetGUID.ToString() );
	}

	// ----------------	
	// Read the full if its there
	// ----------------	
	FExportFlags ExportFlags;

	if ( NetGUID.IsDefault() || GuidCache->IsExportingNetGUIDBunch )
	{
		Ar << ExportFlags.Value;

		if ( Ar.IsError() )
		{
			Object = NULL;
			return NetGUID;
		}
	}

	if ( GuidCache->IsExportingNetGUIDBunch )
	{
		if (ensureMsgf(NetGUID.IsValid(), TEXT("InternalLoadObject tried to add an invalid GUID to the import list, Object: %s"), *GetFullNameSafe(Object)))
		{
			GuidCache->ImportedNetGuids.Add( NetGUID );
		}
	}

	if ( ExportFlags.bHasPath )
	{
		UObject* ObjOuter = NULL;

		FNetworkGUID OuterGUID = InternalLoadObject( Ar, ObjOuter, InternalLoadObjectRecursionCount + 1 );

		FString ObjectName;
		uint32	NetworkChecksum = 0;

		Ar << ObjectName;

		if ( ExportFlags.bHasNetworkChecksum )
		{
			Ar << NetworkChecksum;

			UE_LOGF(LogNetPackageMap, Verbose, "%ls has network checksum %u", *ObjectName, NetworkChecksum);
		}

		const bool bIsPackage = NetGUID.IsStatic() && !OuterGUID.IsValid();

		if ( Ar.IsError() )
		{
			UE_LOGF( LogNetPackageMap, Error, "InternalLoadObject: Failed to load path name" );
			Object = NULL;
			return NetGUID;
		}

#if WITH_EDITOR
		FString TempObjectName = ObjectName;
#endif

		// Remap name for PIE
		GEngine->NetworkRemapPath( Connection, ObjectName, true );

#if WITH_EDITOR
		ensureMsgf(!ObjectName.IsEmpty(), TEXT("NetworkRemapPath found %s to be an invalid name. This object will not be binded and replicated!"), *TempObjectName);
#endif

		if (NetGUID.IsDefault())
		{
			// This should be from the client
			// If we get here, we want to go ahead and assign a network guid, 
			// then export that to the client at the next available opportunity
			check(IsNetGUIDAuthority());

			// If the object is not a package and we couldn't find the outer, we have to bail out, since the
			// relative path name is meaningless. This may happen if the outer has been garbage collected.
			if (!bIsPackage && OuterGUID.IsValid() && ObjOuter == nullptr)
			{
				UE_LOGF( LogNetPackageMap, Log, "InternalLoadObject: couldn't find outer for non-package object. GUID: %ls, ObjectName: %ls", *NetGUID.ToString(), *ObjectName );
				Object = nullptr;
				return NetGUID;
			}

			Object = StaticFindObject(UObject::StaticClass(), ObjOuter, *ObjectName, EFindObjectFlags::None);

			// Try to load package if it wasn't found. Note load package fails if the package is already loaded.
			if (Object == nullptr && bIsPackage)
			{
				FPackagePath Path = FPackagePath::FromPackageNameChecked(ObjectName);
				Object = LoadPackage(nullptr, Path, LOAD_None);
			}

			if ( Object == NULL )
			{
				UE_LOGF( LogNetPackageMap, Warning, "UPackageMapClient::InternalLoadObject: Unable to resolve default guid from client: ObjectName: %ls, ObjOuter: %ls ", *ObjectName, ObjOuter != NULL ? *ObjOuter->GetPathName() : TEXT( "NULL" ) );
				return NetGUID;
			}

			if (!IsValid(Object))
			{
				UE_LOGF( LogNetPackageMap, Warning, "UPackageMapClient::InternalLoadObject: Received reference to invalid object from client: ObjectName: %ls, ObjOuter: %ls ", *ObjectName, ObjOuter != NULL ? *ObjOuter->GetPathName() : TEXT( "NULL" ) );
				Object = NULL;
				return NetGUID;
			}

			if ( NetworkChecksum != 0 && GuidCache->NetworkChecksumMode == FNetGUIDCache::ENetworkChecksumMode::SaveAndUse && !CVarIgnoreNetworkChecksumMismatch.GetValueOnAnyThread() )
			{
				const uint32 CompareNetworkChecksum = GuidCache->GetNetworkChecksum( Object );

				if (CompareNetworkChecksum != NetworkChecksum )
				{
					FString ErrorStr = FString::Printf(TEXT("UPackageMapClient::InternalLoadObject: Default object package network checksum mismatch! ObjectName: %s, ObjOuter: %s, GUID1: %u, GUID2: %u "), *ObjectName, ObjOuter != NULL ? *ObjOuter->GetPathName() : TEXT("NULL"), CompareNetworkChecksum, NetworkChecksum);
					UE_LOGF( LogNetPackageMap, Error, "%ls", *ErrorStr);
					Object = NULL;

					BroadcastNetFailure(GuidCache->Driver, ENetworkFailure::NetChecksumMismatch, ErrorStr);
					return NetGUID;
				}
			}

			if ( bIsPackage )
			{
				UPackage * Package = Cast< UPackage >( Object );

				if ( Package == NULL )
				{
					UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::InternalLoadObject: Default object not a package from client: ObjectName: %ls, ObjOuter: %ls ", *ObjectName, ObjOuter != NULL ? *ObjOuter->GetPathName() : TEXT( "NULL" ) );
					Object = NULL;
					return NetGUID;
				}
			}

			// Assign the guid to the object
			NetGUID = GuidCache->GetOrAssignNetGUID( Object );

			// Let this client know what guid we assigned
			HandleUnAssignedObject( Object );

			return NetGUID;
		}
		else if ( Object != nullptr )
		{
			// If we already have the object, just do some sanity checking and return
			SanityCheckExport( GuidCache.Get(), Object, NetGUID, ObjectName, ObjOuter, OuterGUID, ExportFlags );
			return NetGUID;
		}

		// If we are the server, we should have found the object by now
		if ( IsNetGUIDAuthority() )
		{
			UE_LOGF( LogNetPackageMap, Warning, "UPackageMapClient::InternalLoadObject: Server could not resolve non default guid from client. ObjectName: %ls, ObjOuter: %ls ", *ObjectName, ObjOuter != NULL ? *ObjOuter->GetPathName() : TEXT( "NULL" ) );
			return NetGUID;
		}

		//
		// At this point, only the client gets this far
		//

		const bool bIgnoreWhenMissing = ExportFlags.bNoLoad;

		// Register this path and outer guid combo with the net guid
		GuidCache->RegisterNetGUIDFromPath_Client( NetGUID, ObjectName, OuterGUID, NetworkChecksum, ExportFlags.bNoLoad, bIgnoreWhenMissing );

		// Try again now that we've registered the path
		Object = GuidCache->GetObjectFromNetGUID( NetGUID, GuidCache->IsExportingNetGUIDBunch );

		if ( Object == NULL && !GuidCache->ShouldIgnoreWhenMissing( NetGUID ) )
		{
			UE_LOGF( LogNetPackageMap, Warning, "InternalLoadObject: Unable to resolve object from path. Path: %ls, Outer: %ls, NetGUID: %ls", *ObjectName, ObjOuter ? *ObjOuter->GetPathName() : TEXT( "NULL" ), *NetGUID.ToString() );
		}
	}
	else if ( Object == NULL && !GuidCache->ShouldIgnoreWhenMissing( NetGUID ) )
	{
		UE_LOGF( LogNetPackageMap, Warning, "InternalLoadObject: Unable to resolve object. FullNetGUIDPath: %ls", *GuidCache->FullNetGUIDPath( NetGUID ) );
	}

	return NetGUID;
}

UObject* UPackageMapClient::ResolvePathAndAssignNetGUID( const FNetworkGUID& NetGUID, const FString& PathName )
{
	check( 0 );
	return NULL;
}

//--------------------------------------------------------------------
//
//	Network - NetGUID Bunches (Export Table)
//
//	These functions deal with exporting new NetGUIDs in separate, discrete bunches.
//	These bunches are appended to normal 'content' bunches. You can think of it as an
//	export table that is prepended to bunches.
//
//--------------------------------------------------------------------

bool UPackageMapClient::ExportNetGUIDForReplay(FNetworkGUID& NetGUID, UObject* Object, FString& PathName, UObject* ObjOuter)
{
	int32 const * const FoundExpectedPacketIdPtr = OverrideAckState->NetGUIDAckStatus.Find(NetGUID);
	const int32 ExpectedPacketId = FoundExpectedPacketIdPtr ? *FoundExpectedPacketIdPtr : OverrideAckState->NetGUIDAckStatus.Emplace(NetGUID, GUID_PACKET_NOT_ACKED);

	if (GUID_PACKET_ACKED != ExpectedPacketId)
	{
		TGuardValue<bool> ExportingGUID(GuidCache->IsExportingNetGUIDBunch, true);

		const int32 MaxReservedSize(CVarReservedNetGuidSize.GetValueOnAnyThread());

		TArray<uint8>& GUIDMemory = ExportGUIDArchives.Emplace_GetRef();
		GUIDMemory.Reserve(MaxReservedSize);

		FNetGUIDCache::ENetworkChecksumMode RestoreMode = GuidCache->NetworkChecksumMode;

		GuidCache->SetNetworkChecksumMode(FNetGUIDCache::ENetworkChecksumMode::None);

		FMemoryWriter Writer(GUIDMemory);
		InternalWriteObject(Writer, NetGUID, Object, PathName, ObjOuter);

		GuidCache->SetNetworkChecksumMode(RestoreMode);

		check(!Writer.IsError());
		ensureMsgf(GUIDMemory.Num() <= MaxReservedSize, TEXT("ExportNetGUIDForReplay exceeded CVarReservedNetGuidSize. Max=%d Count=%d"), MaxReservedSize, GUIDMemory.Num());

		GUIDMemory.Shrink();

		// It's possible InternalWriteObject has modified the NetGUIDAckStatus, so
		// do a quick sanity check to make sure the ID wasn't removed before updating the status.
		int32* NewPacketIdPtr = OverrideAckState->NetGUIDAckStatus.Find(NetGUID);
		if (ensureMsgf(NewPacketIdPtr != nullptr, TEXT("ExportNetGUIDForReplay PacketID was removed for %s %s"), *NetGUID.ToString(), *GetPathNameSafe(Object)))
		{
			*NewPacketIdPtr = GUID_PACKET_ACKED;
		}
	}

	CurrentExportNetGUIDs.Empty();
	ExportNetGUIDCount = 0;
	return true;
}

/** Exports the NetGUID and paths needed to the CurrentExportBunch */
bool UPackageMapClient::ExportNetGUID( FNetworkGUID NetGUID, UObject* Object, FString PathName, UObject* ObjOuter )
{
	check( NetGUID.IsValid() );
	check( ( Object == NULL ) == !PathName.IsEmpty() );
	check( !NetGUID.IsDefault() );
	check( Object == NULL || ShouldSendFullPath( Object, NetGUID ) );

	if (Connection->IsInternalAck())
	{
		return ExportNetGUIDForReplay(NetGUID, Object, PathName, ObjOuter);
	}

	// Two passes are used to export this net guid:
	// 1. Attempt to append this net guid to current bunch
	// 2. If step 1 fails, append to fresh new bunch
	for ( int32 NumTries = 0; NumTries < 2; NumTries++ )
	{
		if ( !CurrentExportBunch )
		{
			check( ExportNetGUIDCount == 0 );

			CurrentExportBunch = new FOutBunch(this, Connection->GetMaxSingleBunchSizeBits());

#if UE_NET_TRACE_ENABLED
			// Only enable this if we are doing verbose tracing
			// We leave it to the bunch to destroy the trace collector
			SetTraceCollector(*CurrentExportBunch, UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Verbose));
#endif

			CurrentExportBunch->SetAllowResize(false);
			CurrentExportBunch->SetAllowOverflow(true);
			CurrentExportBunch->bHasPackageMapExports = true;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			CurrentExportBunch->DebugString = TEXT("NetGUIDs");
#endif

			UE_NET_TRACE_SCOPE(NetGUIDExportBuncheader, *CurrentExportBunch, GetTraceCollector(*CurrentExportBunch), ENetTraceVerbosity::Verbose);

			CurrentExportBunch->WriteBit( 0 );		// To signify this is NOT a rep layout export

			ExportNetGUIDCount = 0;
			*CurrentExportBunch << ExportNetGUIDCount;
			NET_CHECKSUM( *CurrentExportBunch );
		}

		if ( CurrentExportNetGUIDs.Num() != 0 )
		{
			UE_LOGF( LogNetPackageMap, Fatal, "ExportNetGUID - CurrentExportNetGUIDs.Num() != 0 (%ls).", Object ? *Object->GetName() : *PathName );
			return false;
		}

		UE_NET_TRACE_OBJECT_SCOPE(NetGUID, *CurrentExportBunch, GetTraceCollector(*CurrentExportBunch), ENetTraceVerbosity::Verbose);

		// Push our current state in case we overflow with this export and have to pop it off.
		FBitWriterMark LastExportMark;
		LastExportMark.Init( *CurrentExportBunch );

		GuidCache->IsExportingNetGUIDBunch = true;

		InternalWriteObject( *CurrentExportBunch, NetGUID, Object, PathName, ObjOuter );

		GuidCache->IsExportingNetGUIDBunch = false;

		if ( CurrentExportNetGUIDs.Num() == 0 )
		{
			// Some how we failed to export this GUID 
			// This means no path names were written, which means we possibly are incorrectly not writing paths out, or we shouldn't be here in the first place
			UE_LOGF( LogNetPackageMap, Warning, "ExportNetGUID - InternalWriteObject no GUID's were exported: %ls ", Object ? *Object->GetName() : *PathName );
			LastExportMark.Pop( *CurrentExportBunch );
			return false;
		}
	
		if ( !CurrentExportBunch->IsError() )
		{
			// Success, append these exported guid's to the list going out on this bunch
			CurrentExportBunch->ExportNetGUIDs.Append( CurrentExportNetGUIDs.Array() );
			CurrentExportNetGUIDs.Empty();		// Done with this
			ExportNetGUIDCount++;
			return true;
		}

		// Overflowed, wrap up the currently pending bunch, and start a new one
		LastExportMark.Pop( *CurrentExportBunch );

		// Make sure we reset this so it doesn't persist into the next batch
		CurrentExportNetGUIDs.Empty();

		if ( ExportNetGUIDCount == 0 || NumTries == 1 )
		{
			// This means we couldn't serialize this NetGUID into a single bunch. The path could be ridiculously big (> ~512 bytes) or something else is very wrong
			UE_LOGF( LogNetPackageMap, Fatal, "ExportNetGUID - Failed to serialize NetGUID into single bunch. (%ls)", Object ? *Object->GetName() : *PathName  );
			return false;
		}

		for ( auto It = CurrentExportNetGUIDs.CreateIterator(); It; ++It )
		{
			int32& Count = NetGUIDExportCountMap.FindOrAdd( *It );
			Count--;
		}

		// Export current bunch, create a new one, and try again.
		ExportNetGUIDHeader();
	}

	check( 0 );		// Shouldn't get here

	return false;
}

static void PatchHeaderCount( FBitWriter& Writer, bool bHasRepLayoutExport, uint32 NewCount )
{
	FBitWriterMark Reset;
	FBitWriterMark Restore( Writer );
	Reset.PopWithoutClear( Writer );
	Writer.WriteBit( bHasRepLayoutExport ? 1 : 0 );
	Writer << NewCount;
	Restore.PopWithoutClear( Writer );
}

/** Called when an export bunch is finished. It writes how many NetGUIDs are contained in the bunch and finalizes the book keeping so we know what NetGUIDs are in the bunch */
void UPackageMapClient::ExportNetGUIDHeader()
{
	check(CurrentExportBunch);

	UE_LOG(LogNetPackageMap, Log, TEXT("	UPackageMapClient::ExportNetGUID. Bytes: %" INT64_FMT " Bits: %" INT64_FMT " ExportNetGUIDCount: %d"), CurrentExportBunch->GetNumBytes(), CurrentExportBunch->GetNumBits(), ExportNetGUIDCount);

	// Rewrite how many NetGUIDs were exported.
	PatchHeaderCount( *CurrentExportBunch, false, ExportNetGUIDCount );

	// If we've written new NetGUIDs to the 'bunch' set (current+1)
	if (UE_LOG_ACTIVE(LogNetPackageMap,Verbose))
	{
		UE_LOGF(LogNetPackageMap, Verbose, "ExportNetGUIDHeader: ");
		for (auto It = CurrentExportBunch->ExportNetGUIDs.CreateIterator(); It; ++It)
		{
			UE_LOGF(LogNetPackageMap, Verbose, "  NetGUID: %ls", *It->ToString());
		}
	}

	// CurrentExportBunch *should* always have NetGUIDs to export. If it doesn't warn. This is a bug.
	if ( CurrentExportBunch->ExportNetGUIDs.Num() != 0 )	
	{
		ExportBunches.Add( CurrentExportBunch );
	}
	else
	{
		UE_LOGF(LogNetPackageMap, Warning, "Attempted to export a NetGUID Bunch with no NetGUIDs!");
	}

	CSV_CUSTOM_STAT(PackageMap, NetGuidExports, ExportNetGUIDCount, ECsvCustomStatOp::Accumulate);
	
	CurrentExportBunch = NULL;
	ExportNetGUIDCount = 0;
}

void UPackageMapClient::ReceiveNetGUIDBunch( FInBunch &InBunch )
{
	check( InBunch.bHasPackageMapExports );

	const int64 StartingBitPos = InBunch.GetPosBits();
	const bool bHasRepLayoutExport = InBunch.ReadBit() == 1 ? true : false;

	if ( bHasRepLayoutExport )
	{
		// We need to keep this around to ensure we don't break backwards compatability.
		ReceiveNetFieldExportsCompat( InBunch );
		return;
	}

	TGuardValue<bool> IsExportingGuard(GuidCache->IsExportingNetGUIDBunch, true);

	int32 NumGUIDsInBunch = 0;
	InBunch << NumGUIDsInBunch;

	if ( NumGUIDsInBunch > UE::Net::MaxSerializedNetGuids )
	{
		UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetGUIDBunch: NumGUIDsInBunch > MaxSerializedNetGuids (%d / %d)", NumGUIDsInBunch, UE::Net::MaxSerializedNetGuids);
		InBunch.SetError();
		return;
	}

	NET_CHECKSUM(InBunch);

	UE_LOGF(LogNetPackageMap, Log, "UPackageMapClient::ReceiveNetGUIDBunch %d NetGUIDs. PacketId %d. ChSequence %d. ChIndex %d", NumGUIDsInBunch, InBunch.PacketId, InBunch.ChSequence, InBunch.ChIndex );

	UE_NET_TRACE(NetGUIDExportBuncheader, Connection->GetInTraceCollector(), StartingBitPos, InBunch.GetPosBits(), ENetTraceVerbosity::Verbose);

	int32 NumGUIDsRead = 0;
	while( NumGUIDsRead < NumGUIDsInBunch )
	{
		UE_NET_TRACE_NAMED_OBJECT_SCOPE(ObjectScope, FNetworkGUID(), InBunch, Connection->GetInTraceCollector(), ENetTraceVerbosity::Verbose);

		UObject* Obj = NULL;
		const FNetworkGUID LoadedGUID = InternalLoadObject( InBunch, Obj, 0 );

		UE_NET_TRACE_SET_SCOPE_OBJECTID(ObjectScope, LoadedGUID);

		if ( InBunch.IsError() )
		{
			UE_LOGF( LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetGUIDBunch: InBunch.IsError() after InternalLoadObject" );
			return;
		}
		NumGUIDsRead++;
	}

	UE_LOG(LogNetPackageMap, Log, TEXT("UPackageMapClient::ReceiveNetGUIDBunch end. BitPos: %" INT64_FMT), InBunch.GetPosBits());
}

TSharedPtr<FNetFieldExportGroup> UPackageMapClient::GetNetFieldExportGroup(const FString& PathName)
{
	return GuidCache->NetFieldExportGroupMap.FindRef(PathName);
}

void UPackageMapClient::AddNetFieldExportGroup(const FString& PathName, TSharedPtr< FNetFieldExportGroup > NewNetFieldExportGroup)
{
	check(!GuidCache->NetFieldExportGroupMap.Contains(NewNetFieldExportGroup->PathName));

	NewNetFieldExportGroup->PathNameIndex = ++GuidCache->UniqueNetFieldExportGroupPathIndex;

	check(!GuidCache->NetFieldExportGroupPathToIndex.Contains(NewNetFieldExportGroup->PathName));
	check(!GuidCache->NetFieldExportGroupIndexToGroup.Contains(NewNetFieldExportGroup->PathNameIndex));

	GuidCache->NetFieldExportGroupPathToIndex.Add(NewNetFieldExportGroup->PathName, NewNetFieldExportGroup->PathNameIndex);
	GuidCache->NetFieldExportGroupIndexToGroup.Add(NewNetFieldExportGroup->PathNameIndex, NewNetFieldExportGroup.Get());
	GuidCache->NetFieldExportGroupMap.Add(NewNetFieldExportGroup->PathName, NewNetFieldExportGroup);
}

void UPackageMapClient::TrackNetFieldExport(FNetFieldExportGroup* NetFieldExportGroup, const int32 NetFieldExportHandle)
{
	check(Connection->IsInternalAck());
	check(NetFieldExportGroup);

	checkf(NetFieldExportGroup->NetFieldExports.IsValidIndex(NetFieldExportHandle),
		TEXT("Invalid NetFieldExportHandle. GroupPath = %s, NumExports = %d, ExportHandle = %d"),
		*(NetFieldExportGroup->PathName), NetFieldExportGroup->NetFieldExports.Num(), NetFieldExportHandle);

	checkf(NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].Handle == NetFieldExportHandle,
		TEXT("NetFieldExportHandle Mismatch. GroupPath = %s, NumExports = %d, ExportHandle = %d, Expected Handle = %d"),
		*(NetFieldExportGroup->PathName), NetFieldExportGroup->NetFieldExports.Num(), NetFieldExportHandle, NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].Handle);


	NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].bExported = true;

	const uint64 CmdHandle = ((uint64)NetFieldExportGroup->PathNameIndex) << 32 | (uint64)NetFieldExportHandle;

	// If this cmd hasn't been confirmed as exported, we need to export it for this bunch
	if (!OverrideAckState->NetFieldExportAcked.Contains(CmdHandle))
	{
		NetFieldExports.Add(CmdHandle);		// NOTE - This is a set, so it will only add once
	}
}

TSharedPtr< FNetFieldExportGroup > UPackageMapClient::GetNetFieldExportGroupChecked(const FString& PathName) const
{
	return GuidCache->NetFieldExportGroupMap.FindChecked(PathName);
}

void UPackageMapClient::SerializeNetFieldExportGroupMap(FArchive& Ar, bool bClearPendingExports)
{
	if (Ar.IsSaving())
	{
		if (bClearPendingExports)
		{
			NetFieldExports.Empty();
		}

		// Save the number of layouts
		uint32 NumNetFieldExportGroups = GuidCache->NetFieldExportGroupMap.Num();
		Ar << NumNetFieldExportGroups;

		// Save each layout
		for (auto It = GuidCache->NetFieldExportGroupMap.CreateIterator(); It; ++It)
		{
			// Save out the export group
			Ar << *It.Value().Get();
		}
	}
	else
	{
		// Clear all of our mappings, since we're starting over
		GuidCache->NetFieldExportGroupMap.Reset();
		GuidCache->NetFieldExportGroupPathToIndex.Reset();
		GuidCache->NetFieldExportGroupIndexToGroup.Reset();

		// Read the number of export groups
		uint32 NumNetFieldExportGroups = 0;
		Ar << NumNetFieldExportGroups;

		if (Ar.IsError())
		{
			UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::SerializeNetFieldExportGroupMap - Archive error while reading NumNetFieldExportGroups");
			return;
		}

		if (NumNetFieldExportGroups > (uint32)UE::Net::MaxSerializedNetExportGroups)
		{
			Ar.SetError();
			UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::SerializeNetFieldExportGroupMap - NumNetFieldExportGroups exceeds MaxSerializedNetExportGroups (%u / %d)", NumNetFieldExportGroups, UE::Net::MaxSerializedNetExportGroups);
			return;
		}

		// Read each export group
		for (uint32 i = 0; i < NumNetFieldExportGroups; ++i)
		{
			TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = MakeShared<FNetFieldExportGroup>();

			// Read in the export group
			Ar << *NetFieldExportGroup.Get();

			if (Ar.IsError())
			{
				UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::SerializeNetFieldExportGroupMap - Archive error while loading FNetFieldExportGroup, Index: %u", i);
				return;
			}

			GEngine->NetworkRemapPath(Connection, NetFieldExportGroup->PathName, true);

			// Assign index to path name
			GuidCache->NetFieldExportGroupPathToIndex.Add( NetFieldExportGroup->PathName, NetFieldExportGroup->PathNameIndex );
			GuidCache->NetFieldExportGroupIndexToGroup.Add( NetFieldExportGroup->PathNameIndex, NetFieldExportGroup.Get() );

			// Add the export group to the map
			GuidCache->NetFieldExportGroupMap.Add( NetFieldExportGroup->PathName, NetFieldExportGroup );
		}
	}
}

void UPackageMapClient::AppendExportData(FArchive& Archive)
{
	check(Connection->IsInternalAck());

	AppendNetFieldExports(Archive);
	AppendNetExportGUIDs(Archive);
}

void UPackageMapClient::ReceiveExportData(FArchive& Archive)
{
	check(Connection->IsInternalAck());

	ReceiveNetFieldExports(Archive);
	ReceiveNetExportGUIDs(Archive);
}

void UPackageMapClient::SerializeNetFieldExportDelta(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		TSet<uint64> DeltaNetFieldExports;
		
		for ( auto It = GuidCache->NetFieldExportGroupMap.CreateIterator(); It; ++It )
		{
			// Save out the export group
			TSharedPtr<FNetFieldExportGroup> ExportGroup = It.Value();
			if (ExportGroup.IsValid())
			{
				for ( int32 i = 0; i < ExportGroup->NetFieldExports.Num(); i++ )
				{
					if (ExportGroup->NetFieldExports[i].bExported && ExportGroup->NetFieldExports[i].bDirtyForReplay)
					{
						check(ExportGroup->PathNameIndex != 0);

						const uint64 CmdHandle = ((uint64)ExportGroup->PathNameIndex) << 32 | (uint64)i;

						check(i == ExportGroup->NetFieldExports[i].Handle);

						DeltaNetFieldExports.Add(CmdHandle);

						ExportGroup->NetFieldExports[i].bDirtyForReplay = false;
					}
				}
			}
		}

		AppendNetFieldExportsInternal(Ar, DeltaNetFieldExports, EAppendNetExportFlags::ForceExportDirtyGroups);

		NetFieldExports.Empty();
	}
	else
	{
		ReceiveNetFieldExports(Ar);
	}
}

void UPackageMapClient::AppendNetFieldExports(FArchive& Archive)
{
	AppendNetFieldExportsInternal(Archive, NetFieldExports, EAppendNetExportFlags::None);
	NetFieldExports.Empty();
}

void UPackageMapClient::AppendNetFieldExportsInternal(FArchive& Archive, const TSet<uint64>& InNetFieldExports, EAppendNetExportFlags Flags)
{
	check(Connection->IsInternalAck());

	uint32 NetFieldCount = InNetFieldExports.Num();
	Archive.SerializeIntPacked(NetFieldCount);

	if (0 == NetFieldCount)
	{
		return;
	}

	TArray< uint32, TInlineAllocator<64> > ExportedPathInThisBunchAlready;
	ExportedPathInThisBunchAlready.Reserve(NetFieldCount);

	for (const uint64 FieldExport : InNetFieldExports)
	{
		// Parse the path name index and cmd index out of the uint64
		uint32 PathNameIndex = FieldExport >> 32;
		uint32 NetFieldExportHandle = FieldExport & (((uint64)1 << 32) - 1);

		check(PathNameIndex != 0);

		FNetFieldExportGroup* NetFieldExportGroup = GuidCache->NetFieldExportGroupIndexToGroup.FindChecked(PathNameIndex);
		const FString& PathName = NetFieldExportGroup->PathName;

		check(NetFieldExportHandle == NetFieldExportGroup->NetFieldExports[NetFieldExportHandle].Handle);

		// Export the path if we need to
		// TODO: CVar this change so we can && this (old behavior) or || this (new behavior)
		const bool bForceExportDirty = EnumHasAnyFlags(Flags, EAppendNetExportFlags::ForceExportDirtyGroups) || NetFieldExportGroup->bDirtyForReplay;

		uint32 NeedsExport = ((bForceExportDirty || !OverrideAckState->NetFieldExportGroupPathAcked.Contains(PathNameIndex)) && !ExportedPathInThisBunchAlready.Contains(PathNameIndex)) ? 1 : 0;

		Archive.SerializeIntPacked(PathNameIndex);
		Archive.SerializeIntPacked(NeedsExport);

		if (NeedsExport)
		{
			uint32 NumExports = NetFieldExportGroup->NetFieldExports.Num();

			Archive << const_cast<FString&>(PathName);
			Archive.SerializeIntPacked(NumExports);

			ExportedPathInThisBunchAlready.Add(PathNameIndex);

			if (bForceExportDirty)
			{
				NetFieldExportGroup->bDirtyForReplay = false;
			}
		}

		Archive << NetFieldExportGroup->NetFieldExports[NetFieldExportHandle];

		OverrideAckState->NetFieldExportGroupPathAcked.Add( PathNameIndex );
		OverrideAckState->NetFieldExportAcked.Add( FieldExport );
	}
}

void UPackageMapClient::ReceiveNetFieldExportsCompat(FInBunch &InBunch)
{
	if (!Connection->IsInternalAck())
	{
		UE_LOGF(LogNetPackageMap, Error, "ReceiveNetFieldExportsCompat: connection is not a replay connection.");
		InBunch.SetError();
		return;
	}

	// Read number of net field exports
	uint32 NumExportGroups = 0;
	InBunch << NumExportGroups;

	if (NumExportGroups > (uint32)UE::Net::MaxSerializedNetExportGroups)
	{
		UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetFieldExportsCompat - NumExportGroups exceeds MaxSerializedNetExportGroups (%u / %d)", NumExportGroups, UE::Net::MaxSerializedNetExportGroups);
		InBunch.SetError();
		return;
	}

	for (int32 i = 0; i < (int32)NumExportGroups; i++)
	{
		// Read the index that represents the name in the NetFieldExportGroupIndexToPath map
		uint32 PathNameIndex;
		InBunch.SerializeIntPacked(PathNameIndex);

		if (InBunch.IsError())
		{
			UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetFieldExportsCompat - Error serializing export path index.");
			return;
		}

		int32 NumExportsInGroup = 0;

		FNetFieldExportGroup* NetFieldExportGroup = nullptr;

		// See if the path name was exported (we'll expect it if we haven't seen this index before)
		if (InBunch.ReadBit() == 1)
		{
			FString PathName;
			InBunch << PathName;

			if (InBunch.IsError())
			{
				UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetFieldExportsCompat - Error serializing export path.");
				return;
			}

			InBunch << NumExportsInGroup;

			if (NumExportsInGroup > UE::Net::MaxSerializedNetExportsPerGroup)
			{
				UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetFieldExportsCompat - NumExportsInGroup exceeds MaxSerializedNetExportsPerGroup (%d / %d)", NumExportsInGroup, UE::Net::MaxSerializedNetExportsPerGroup);
				InBunch.SetError();
				return;
			}

			GEngine->NetworkRemapPath(Connection, PathName, true);

			NetFieldExportGroup = GuidCache->NetFieldExportGroupMap.FindRef(PathName).Get();
			if (!NetFieldExportGroup)
			{
				TSharedPtr<FNetFieldExportGroup> NewNetFieldExportGroup(new FNetFieldExportGroup());
				NetFieldExportGroup = NewNetFieldExportGroup.Get();

				NetFieldExportGroup->PathName = PathName;
				NetFieldExportGroup->PathNameIndex = PathNameIndex;

				NetFieldExportGroup->NetFieldExports.SetNum(NumExportsInGroup);

				GuidCache->NetFieldExportGroupMap.Add(PathName, NewNetFieldExportGroup);
			}

			GuidCache->NetFieldExportGroupPathToIndex.Add(PathName, PathNameIndex);
			GuidCache->NetFieldExportGroupIndexToGroup.Add(PathNameIndex, NetFieldExportGroup);
		}
		else
		{
			NetFieldExportGroup = GuidCache->NetFieldExportGroupIndexToGroup.FindChecked(PathNameIndex);
		}


		FNetFieldExport NetFieldExport;

		// Read the cmd
		InBunch << NetFieldExport;

		if (InBunch.IsError())
		{
			return;
		}

		TArray<FNetFieldExport>& NetFieldExportsRef = NetFieldExportGroup->NetFieldExports;

		if (NetFieldExportsRef.IsValidIndex((int32)NetFieldExport.Handle))
		{
			// Assign it to the correct slot (NetFieldExport.Handle is just the index into the array)
			NetFieldExportGroup->NetFieldExports[NetFieldExport.Handle] = NetFieldExport;
		}
		else
		{
			UE_LOGF(LogNetPackageMap, Error, "ReceiveNetFieldExports: Invalid NetFieldExport Handle '%i', Max '%i'.",
				NetFieldExport.Handle, NetFieldExportsRef.Num());

			InBunch.SetError();
			return;
		}
	}
}

void UPackageMapClient::ReceiveNetFieldExports(FArchive& Archive)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ReceiveNetFieldExports time"), STAT_ReceiveNetFieldExportsTime, STATGROUP_Net);

	check(Connection->IsInternalAck());

	// Read number of net field exports
	uint32 NumNetExports = 0;
	Archive.SerializeIntPacked(NumNetExports);

	if (NumNetExports > (uint32)UE::Net::MaxSerializedNetExportGroups)
	{
		UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetFieldExports - NumNetExports exceeds MaxSerializedNetExportGroups (%u / %d)", NumNetExports, UE::Net::MaxSerializedNetExportGroups);
		Archive.SetError();
		return;
	}

	for (int32 i = 0; i < (int32)NumNetExports; i++)
	{
		uint32 PathNameIndex = 0;
		uint32 WasExported = 0;

		Archive.SerializeIntPacked(PathNameIndex);
		Archive.SerializeIntPacked(WasExported);
		
		FNetFieldExportGroup* NetFieldExportGroup = nullptr;
		if (!!WasExported)
		{
			FString PathName;
			uint32 NumExportsInGroup = 0;

			Archive << PathName;
			Archive.SerializeIntPacked(NumExportsInGroup);

			if (NumExportsInGroup > (uint32)UE::Net::MaxSerializedNetExportsPerGroup)
			{
				UE_LOGF(LogNetPackageMap, Warning, "UPackageMapClient::ReceiveNetFieldExports - NumExportsInGroup exceeds MaxSerializedNetExportsPerGroup (%u / %d)", NumExportsInGroup, UE::Net::MaxSerializedNetExportsPerGroup);
				Archive.SetError();
				return;
			}

			GEngine->NetworkRemapPath(Connection, PathName, true);

			NetFieldExportGroup = GuidCache->NetFieldExportGroupMap.FindRef(PathName).Get();
			if (!NetFieldExportGroup)
			{
				TSharedPtr<FNetFieldExportGroup> NewNetFieldExportGroup(new FNetFieldExportGroup());
				NetFieldExportGroup = NewNetFieldExportGroup.Get();

				NetFieldExportGroup->PathName = PathName;
				NetFieldExportGroup->PathNameIndex = PathNameIndex;
				NetFieldExportGroup->NetFieldExports.SetNum(NumExportsInGroup);

				GuidCache->NetFieldExportGroupMap.Add(PathName, NewNetFieldExportGroup);
			}
			else if (NumExportsInGroup > (uint32)NetFieldExportGroup->NetFieldExports.Num())
			{
				// Allow our exports to dynamically grow as more are encountered
				NetFieldExportGroup->NetFieldExports.SetNum(NumExportsInGroup);
			}

			GuidCache->NetFieldExportGroupPathToIndex.Add(PathName, PathNameIndex);
			GuidCache->NetFieldExportGroupIndexToGroup.Add(PathNameIndex, NetFieldExportGroup);
		}
		else
		{
			FNetFieldExportGroup** FoundNetFieldExport = GuidCache->NetFieldExportGroupIndexToGroup.Find(PathNameIndex);
			NetFieldExportGroup = FoundNetFieldExport ? *FoundNetFieldExport : nullptr;
		}

		FNetFieldExport Export;
		Archive << Export;

		if (NetFieldExportGroup)
		{
			TArray<FNetFieldExport>& Exports = NetFieldExportGroup->NetFieldExports;
			if (Exports.IsValidIndex(Export.Handle))
			{
				// preserve compatibility flag
				Export.bIncompatible = Exports[Export.Handle].bIncompatible;
				Exports[Export.Handle] = Export;
			}
			else
			{
				UE_LOGF(LogNetPackageMap, Error, "ReceiveNetFieldExports: Invalid NetFieldExportHandle '%i', Max '%i'", Export.Handle, Exports.Num());
			}
		}
		else
		{
			UE_LOGF(LogNetPackageMap, Error, "ReceiveNetFieldExports: Unable to find NetFieldExportGroup for export. Export.Handle=%i, Export.Name=%ls, PathNameIndex=%lu, WasExported=%d, Archive.IsError()=%d",
				Export.Handle, *Export.ExportName.ToString(), PathNameIndex, !!WasExported, !!Archive.IsError());
		}
	}
}

void UPackageMapClient::AppendNetExportGUIDs(FArchive& Archive)
{
	check(Connection->IsInternalAck());

	uint32 NumGUIDs = ExportGUIDArchives.Num();
	Archive.SerializeIntPacked(NumGUIDs);

	for (TArray<uint8>& GUIDData : ExportGUIDArchives)
	{
		Archive << GUIDData;
	}

	ExportGUIDArchives.Empty()	;
}

void UPackageMapClient::ReceiveNetExportGUIDs(FArchive& Archive)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ReceiveNetExportGUIDs time"), STAT_ReceiveNetExportGUIDsTime, STATGROUP_Net);

	check(Connection->IsInternalAck());
	TGuardValue<bool> IsExportingGuard(GuidCache->IsExportingNetGUIDBunch, true);

	uint32 NumGUIDs = 0;
	Archive.SerializeIntPacked(NumGUIDs);

	if (Archive.IsError())
	{
		return;
	}

	if (NumGUIDs > (uint32)UE::Net::MaxSerializedReplayNetGuids)
	{
		UE_LOGF(LogNetPackageMap, Error, "UPackageMapClient::ReceiveNetExportGUIDs: NumGUIDs > MaxSerializedReplayNetGuids (%u / %d)", NumGUIDs, UE::Net::MaxSerializedReplayNetGuids);
		Archive.SetError();
		return;
	}

	if (bIgnoreReceivedExportGUIDs)
	{
		// Array Serialization works by first serializing an int32 count, and then serializing
		// each member in the array.
		// For arrays whose elements are only 1 byte, the array memory will just be dumped into the archive.
		// Note, this is hacky and depends on the above (which are simple implementation details),
		// but those details are likely not to change anytime soon, given how fundamental they are.

		int32 Count = 0;
		for (uint32 i = 0; i < NumGUIDs; i++)
		{
			Archive << Count;
			Archive.Seek(Count + Archive.Tell());
		}
	}
	else
	{
		TArray<uint8> GUIDData;
		for (uint32 i = 0; i < NumGUIDs; i++)
		{
			Archive << GUIDData;
			FMemoryReader Reader(GUIDData);
			UObject* Object = nullptr;
			InternalLoadObject(Reader, Object, 0);
		}
	}
}

void UPackageMapClient::AppendExportBunches(TArray<FOutBunch *>& OutgoingBunches)
{
	check(!Connection->IsInternalAck());
	check(NetFieldExports.Num() == 0);

	// Finish current in progress bunch if necessary
	if (ExportNetGUIDCount > 0)
	{
		ExportNetGUIDHeader();
	}

	// Let the profiler know about exported GUID bunches
	for (const FOutBunch* ExportBunch : ExportBunches )
	{
		if (ExportBunch != nullptr)
		{
			NETWORK_PROFILER(GNetworkProfiler.TrackExportBunch(ExportBunch->GetNumBits(), Connection));
		}
	}

	// Append the bunches we've made to the passed in list reference
	if (ExportBunches.Num() > 0)
	{
		if (UE_LOG_ACTIVE(LogNetPackageMap,Verbose))
		{
			UE_LOGF(LogNetPackageMap, Verbose, "AppendExportBunches. ExportBunches: %d, ExportNetGUIDCount: %d", ExportBunches.Num(), ExportNetGUIDCount);
			for (auto It=ExportBunches.CreateIterator(); It; ++It)
			{
				UE_LOG(LogNetPackageMap, Verbose, TEXT("   BunchIndex: %d, ExportNetGUIDs: %d, NumBytes: %" INT64_FMT ", NumBits: %" INT64_FMT), It.GetIndex(), (*It)->ExportNetGUIDs.Num(), (*It)->GetNumBytes(), (*It)->GetNumBits() );
			}
		}

		OutgoingBunches.Append(ExportBunches);
		ExportBunches.Empty();
	}
}

/**
 * Takes an outgoing custom exports bunch and splits it into multiple bunches if it will be too large to fit in a single Packet (specified by MaxBunchSizeInBits).
 * Notes:
 *  - The passed-in Bunch argument can be invalidated (if a split occurs).
 *  - We are not setting bPartialInitial or bPartialFinal on the first/last bunches because this code is intended to be used before such flags get
 * set on the final bundles of outgoing bunches.
 * - returns true of the Orignal bunch was split, false if not.
 */
bool SplitAndAppendCustomExportsBunch(TArray<FOutBunch*>& AdditionalRequiredBunches, FOutBunch*& OriginalBunch, int32 MaxBunchSizeInBits)
{
	// Always land on a byte boundary (except for the final bunch)
	MaxBunchSizeInBits = (MaxBunchSizeInBits & ~0x07);

	TArray<FOutBunch*, TInlineAllocator<8>> OutBunches;

	//-----------------------------------------------------
	// Possibly split large bunch into list of smaller partial bunches
	//-----------------------------------------------------
	if(OriginalBunch->GetNumBits() > MaxBunchSizeInBits)
	{
		uint8* Data = OriginalBunch->GetData();
		int64 BitsLeft = OriginalBunch->GetNumBits();

		while(BitsLeft > 0)
		{
			FOutBunch* PartialBunch = OriginalBunch->PackageMap ? new FOutBunch(OriginalBunch->PackageMap, MaxBunchSizeInBits) : new FOutBunch(OriginalBunch->Channel, false);
			int64 BitsThisBunch = FMath::Min<int64>(BitsLeft, MaxBunchSizeInBits);
			PartialBunch->SerializeBits(Data, BitsThisBunch);
			PartialBunch->bPartial = true;

#if UE_NET_TRACE_ENABLED
			// Attach tracecollector of split bunch to first partial bunch
			SetTraceCollector(*PartialBunch, GetTraceCollector(*OriginalBunch));
			SetTraceCollector(*OriginalBunch, nullptr);
#endif

			OutBunches.Add(PartialBunch);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			PartialBunch->DebugString = FString::Printf(TEXT("Partial[%d]: %s"), OutBunches.Num(), *OriginalBunch->GetDebugString());
#endif
		
			BitsLeft -= BitsThisBunch;
			Data += (BitsThisBunch >> 3);

			UE_LOG(LogNetPackageMap, Verbose, TEXT("	Creating partial bunch %s. bitsThisBunch: %" INT64_FMT " bitsLeft: %" INT64_FMT), *OriginalBunch->GetDebugString(), BitsThisBunch, BitsLeft);
			
			ensure(BitsLeft == 0 || BitsThisBunch % 8 == 0); // Byte aligned or it was the last bunch
		}

		// The last bunch has the packagemap extension data
		FOutBunch* LastPartialBunch = OutBunches.Last();
		if (ensure(LastPartialBunch))
		{
			LastPartialBunch->bPartialCustomExportsFinal = OriginalBunch->bPartialCustomExportsFinal;
			LastPartialBunch->NetTokensPendingExport = OriginalBunch->NetTokensPendingExport;
		}

		// The incoming Bunch is no longer valid
		delete OriginalBunch;
		OriginalBunch = nullptr;
	}
	else
	{
		OutBunches.Add(OriginalBunch);
	}

	AdditionalRequiredBunches.Append(OutBunches);
	return OutBunches.Num() > 1;
}

TArray<FOutBunch*> UPackageMapClient::GetAdditionalRequiredBunches(const FOutBunch& OutgoingBunch, EChannelGetAdditionalRequiredBunchesFlags Flags)
{
	TArray<FOutBunch*> AdditionalBunches;

	// Fastpath does not want to append NetGUID exports.
	if (!EnumHasAnyFlags(Flags, EChannelGetAdditionalRequiredBunchesFlags::SkipNetGUIDExports))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AppendExportBunches(AdditionalBunches);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (FOutBunch* CustomExportsBunch = CreateCustomExportsBunch(OutgoingBunch))
	{
		ensureMsgf(CustomExportsBunch->bPartialCustomExportsFinal, TEXT("The Custom Exports Bunch must have bPartialCustomExportsFinal set."));

		// This function will append the bunch to the AdditionalBunches, splitting into multiple bunches if necessary. 
		// If the orignal bunch it split it will be destroyed
		SplitAndAppendCustomExportsBunch(AdditionalBunches, CustomExportsBunch, Connection->GetMaxSingleBunchSizeBits());
	}

	return AdditionalBunches;
}

FOutBunch* UPackageMapClient::CreateCustomExportsBunch(const FOutBunch& OutgoingBunch)
{
	FOutBunch* ExportsBunch = nullptr;

	// NetTokenExports is still an experimental feature that depends on Iris code.
	// NetTokens - begin
	using namespace UE::Net;
	TArray<UE::Net::FNetToken, TInlineAllocator<4>> NetTokensExportedInThisBunch;

	if (OutgoingBunch.NetTokensPendingExport.Num() > 0)
	{
		// See if we have something to export
		const bool bIsNetTokenAuthority = NetTokenResolveContext.NetTokenStore->IsAuthority();
		for (const UE::Net::FNetToken& NetToken : OutgoingBunch.NetTokensPendingExport)
		{
			// We do not need to export tokens assigned by authority unless we are the authority or if they are not already exported in this bunch.
			const bool bShouldExportToken = bIsNetTokenAuthority || (NetToken.IsAssignedByAuthority() == bIsNetTokenAuthority);
			if (bShouldExportToken && !(AcknowledgedExportedNetTokens.Contains(NetToken) || NetTokensExportedInThisBunch.Contains(NetToken)))
			{
				NetTokensExportedInThisBunch.Add(NetToken);
			}
		}
	}

	if (NetTokensExportedInThisBunch.IsEmpty())
	{
		return ExportsBunch;
	}

	// Need to create an exports bunch that is resizeable (can grow indefinitely)
	// At a higher level, this (potentially mega-)bunch will be split into partial bunches
	bool bHasAnyExports = false;
	ExportsBunch = new FOutBunch(this);
	ExportsBunch->bPartialCustomExportsFinal = true;
	ExportsBunch->SetAllowResize(true);
	ExportsBunch->SetDebugString(FString::Printf(TEXT("%s ExportsBunch"), *OutgoingBunch.GetDebugString()));

#if UE_NET_TRACE_ENABLED
	// Only enable this if we are doing verbose tracing
	// We leave it to the bunch to destroy the trace collector
	SetTraceCollector(*ExportsBunch, UE_NET_TRACE_CREATE_COLLECTOR(ENetTraceVerbosity::Verbose));
#endif
	UE_NET_TRACE_SCOPE(CustomExportsBunch, *ExportsBunch, GetTraceCollector(*ExportsBunch), ENetTraceVerbosity::Verbose);

	// NetToken exports
	{
		UE_NET_TRACE_SCOPE(NetTokenExports, *ExportsBunch, GetTraceCollector(*ExportsBunch), ENetTraceVerbosity::Verbose);
		for (FNetToken NetToken : NetTokensExportedInThisBunch)
		{
			bool bWroteExport = true; 
			ExportsBunch->SerializeBits(&bWroteExport, 1);

			UE_NET_TRACE_DYNAMIC_NAME_SCOPE(*NetToken.ToString(), *ExportsBunch, GetTraceCollector(*ExportsBunch), ENetTraceVerbosity::VeryVerbose);

			// Write token 
			NetTokenResolveContext.NetTokenStore->WriteNetToken(*ExportsBunch, NetToken);
			NetTokenResolveContext.NetTokenStore->WriteTokenData(*ExportsBunch, NetToken, this);

			// Track what exports we exported until we know what packet we actually did commit to.
			ExportsBunch->NetTokensPendingExport.Add(NetToken);
		}
		bool bNetTokenExportStopBit = false; 
		ExportsBunch->SerializeBits(&bNetTokenExportStopBit, 1);
	}

	check(!ExportsBunch->IsError());

	return ExportsBunch;
}

/** Receive Exports written by CreateCustomExportsBunch */
void UPackageMapClient::ReceiveCustomExportsBunch(FInBunch &InBunch)
{
	check(InBunch.bPartialCustomExportsFinal);

	// NetTokenExports is still an experimental feature that depends on Iris code.
	using namespace UE::Net;

	// Read NetTokenExports
	bool bHasExportsToRead = false; 
	InBunch.SerializeBits(&bHasExportsToRead, 1);

	UE::Net::FNetTokenStore* NetTokenStore = NetTokenResolveContext.NetTokenStore;
	UE::Net::FNetTokenStoreState* RemoteNetTokenStoreState = NetTokenStore->GetRemoteNetTokenStoreState(Connection->GetConnectionHandle().GetParentConnectionId());

	while (bHasExportsToRead && !InBunch.IsError())
	{
		FNetToken ImportedNetToken = NetTokenStore->ReadNetToken(InBunch);

		NetTokenStore->ReadTokenData(InBunch, ImportedNetToken, *RemoteNetTokenStoreState, this);

		UE_LOGF(LogNetToken, VeryVerbose, "%s: Imported token %ls", __func__, *ImportedNetToken.ToString());

		// Continuation bit
		InBunch.SerializeBits(&bHasExportsToRead, 1);
	}
}

int32 UPackageMapClient::GetNumExportBunches() const
{
	return ExportBunches.Num();
}

void UPackageMapClient::SyncPackageMapExportAckStatus( const UPackageMapClient* Source )
{
	AckState = Source->AckState;
}

void UPackageMapClient::SavePackageMapExportAckStatus( FPackageMapAckState& OutState )
{
	OutState = AckState;
}

void UPackageMapClient::RestorePackageMapExportAckStatus( const FPackageMapAckState& InState )
{
	AckState = InState;
}

void UPackageMapClient::OverridePackageMapExportAckStatus( FPackageMapAckState* NewState )
{
	OverrideAckState = NewState ? NewState : &AckState;
}

void UPackageMapClient::ResetAckState()
{
	AckState.Reset();
	PendingAckGUIDs.Empty();
}

//--------------------------------------------------------------------
//
//	Network - ACKing
//
//--------------------------------------------------------------------

/**
 *	Called when a bunch is committed to the connection's Out buffer.
 *	ExportNetGUIDs is the list of GUIDs stored on the bunch that we use to update the expected sequence for those exported GUIDs
 */
void UPackageMapClient::NotifyBunchCommit(const int32 OutPacketId, const FOutBunch* OutBunch)
{
	// Mark all of the net field exports in this bunch as ack'd
	// NOTE - This only currently works with reliable connections (i.e. InternalAck)
	// For this to work with normal connections, we'll need to do real ack logic here
	for (int32 i = 0; i < OutBunch->NetFieldExports.Num(); i++)
	{
		OverrideAckState->NetFieldExportGroupPathAcked.Add(OutBunch->NetFieldExports[i] >> 32);
		OverrideAckState->NetFieldExportAcked.Add(OutBunch->NetFieldExports[i]);
	}

	// Register exported NetTokens in the PendingAckMap for this OutPackedId
	if (OutBunch->bPartialCustomExportsFinal)
	{
		using namespace UE::Net;
		for (const UE::Net::FNetToken& NetToken : OutBunch->NetTokensPendingExport)
		{
			NetTokenPendingAckMap.Add(OutPacketId, NetToken);
		}
	}

	const TArray<FNetworkGUID>& ExportNetGUIDs = OutBunch->ExportNetGUIDs;

	if (ExportNetGUIDs.Num() == 0)
	{
		return;		// Nothing to do
	}

	check(OutPacketId > GUID_PACKET_ACKED);	// Assumptions break if this isn't true ( We assume ( OutPacketId > GUID_PACKET_ACKED ) == PENDING )

	for (int32 i = 0; i < ExportNetGUIDs.Num(); i++)
	{
		if (!OverrideAckState->NetGUIDAckStatus.Contains(ExportNetGUIDs[i]))
		{
			OverrideAckState->NetGUIDAckStatus.Add(ExportNetGUIDs[i], GUID_PACKET_NOT_ACKED);
		}

		int32& ExpectedPacketIdRef = OverrideAckState->NetGUIDAckStatus.FindChecked(ExportNetGUIDs[i]);

		// Only update expected sequence if this guid was previously nak'd
		// If we always update to the latest packet id, we risk prolonging the ack for no good reason
		// (GUID information doesn't change, so updating to the latest expected sequence is unnecessary)
		if (ExpectedPacketIdRef == GUID_PACKET_NOT_ACKED)
		{
			if (Connection->IsInternalAck())
			{
				// Auto ack now if the connection is 100% reliable
				ExpectedPacketIdRef = GUID_PACKET_ACKED;
				continue;
			}

			ExpectedPacketIdRef = OutPacketId;
			check( !PendingAckGUIDs.Contains(ExportNetGUIDs[i]));	// If we hit this assert, this means the lists are out of sync
			PendingAckGUIDs.AddUnique(ExportNetGUIDs[i]);
		}
	}
}

/**
 *	Called by the PackageMap's UConnection after a receiving an ack
 *	Updates the respective GUIDs that were acked by this packet
 */
void UPackageMapClient::ReceivedAck( const int32 AckPacketId )
{
	for ( int32 Index = PendingAckGUIDs.Num() - 1; Index >= 0; Index-- )
	{
		int32& ExpectedPacketIdRef = OverrideAckState->NetGUIDAckStatus.FindChecked( PendingAckGUIDs[Index] );

		check( ExpectedPacketIdRef > GUID_PACKET_ACKED );		// Make sure we really are pending, since we're on the list

		if ( ExpectedPacketIdRef > GUID_PACKET_ACKED && ExpectedPacketIdRef <= AckPacketId )
		{
			ExpectedPacketIdRef = GUID_PACKET_ACKED;	// Fully acked
			PendingAckGUIDs.RemoveAt( Index );			// Remove from pending list, since we're now acked
		}
	}

	// Acknowledge NetTokenExports
	for (TMultiMap<int32, UE::Net::FNetToken>::TConstKeyIterator It = NetTokenPendingAckMap.CreateConstKeyIterator(AckPacketId); It; ++It)
	{
		UE_LOGF(LogNetToken, VeryVerbose, "%s: Acked token %ls", __func__, *(It.Value().ToString()));
		AcknowledgedExportedNetTokens.Add(It.Value());
	}
	NetTokenPendingAckMap.Remove(AckPacketId);
}

/**
 *	Handles a NACK for given packet id. If this packet ID contained a NetGUID reference, we redirty the NetGUID by setting
 *	its entry in NetGUIDAckStatus to GUID_PACKET_NOT_ACKED.
 */
void UPackageMapClient::ReceivedNak( const int32 NakPacketId )
{
	for ( int32 i = PendingAckGUIDs.Num() - 1; i >= 0; i-- )
	{
		int32& ExpectedPacketIdRef = OverrideAckState->NetGUIDAckStatus.FindChecked( PendingAckGUIDs[i] );

		check( ExpectedPacketIdRef > GUID_PACKET_ACKED );		// Make sure we aren't acked, since we're on the list

		if ( ExpectedPacketIdRef == NakPacketId )
		{
			ExpectedPacketIdRef = GUID_PACKET_NOT_ACKED;
			// Remove from pending list since we're no longer pending
			// If we send another reference to this GUID, it will get added back to this list to hopefully get acked next time
			PendingAckGUIDs.RemoveAt( i );	
		}
	}

	// Just remove our entry
	NetTokenPendingAckMap.Remove(NakPacketId);
}

/**
 *	Returns true if this PackageMap's connection has ACK'd the given NetGUID.
 */
bool UPackageMapClient::NetGUIDHasBeenAckd(FNetworkGUID NetGUID)
{
	if (!NetGUID.IsValid())
	{
		// Invalid NetGUID == NULL obect, so is ack'd by default
		return true;
	}

	if (NetGUID.IsDefault())
	{
		// Default NetGUID is 'unassigned' but valid. It is never Ack'd
		return false;
	}

	if (!IsNetGUIDAuthority())
	{
		// We arent the ones assigning NetGUIDs, so yes this is fully ackd
		return true;
	}

	// If brand new, add it to map with GUID_PACKET_NOT_ACKED
	if ( !OverrideAckState->NetGUIDAckStatus.Contains( NetGUID ) )
	{
		OverrideAckState->NetGUIDAckStatus.Add( NetGUID, GUID_PACKET_NOT_ACKED );
	}

	int32& AckPacketId = OverrideAckState->NetGUIDAckStatus.FindChecked( NetGUID );

	if ( AckPacketId == GUID_PACKET_ACKED )
	{
		// This GUID has been fully Ackd
		UE_LOGF( LogNetPackageMap, Verbose, "NetGUID <%ls> is fully ACKd (AckPacketId: %d <= Connection->OutAckPacketIdL %d) ", *NetGUID.ToString(), AckPacketId, Connection->OutAckPacketId );
		return true;
	}
	else if ( AckPacketId == GUID_PACKET_NOT_ACKED )
	{
		
	}

	return false;
}

/** Immediately export an Object's NetGUID. This will */
void UPackageMapClient::HandleUnAssignedObject(UObject* Obj)
{
	check( Obj != NULL );

	FNetworkGUID NetGUID = GuidCache->GetOrAssignNetGUID( Obj );

	if ( !NetGUID.IsDefault() && ShouldSendFullPath( Obj, NetGUID ) )
	{
		if ( !ExportNetGUID( NetGUID, Obj, TEXT( "" ), NULL ) )
		{
			UE_LOGF( LogNetPackageMap, Verbose, "Failed to export in ::HandleUnAssignedObject %ls", Obj ? *Obj->GetName() : TEXT("NULL") );
		}
	}
}

//--------------------------------------------------------------------
//
//	Misc
//
//--------------------------------------------------------------------

/** Do we need to include the full path of this object for the client to resolve it? */
bool UPackageMapClient::ShouldSendFullPath( const UObject* Object, const FNetworkGUID &NetGUID )
{
	if ( !Connection )
	{
		return false;
	}

	// NetGUID is already exported
	if ( CurrentExportBunch != NULL && CurrentExportBunch->ExportNetGUIDs.Contains( NetGUID ) )
	{
		return false;
	}

	if ( !NetGUID.IsValid() )
	{
		return false;
	}

	if ( !Object->IsNameStableForNetworking() )
	{
		checkf( !NetGUID.IsDefault(), TEXT("Non-stably named object %s has a default NetGUID. %s"), *GetFullNameSafe(Object), *Connection->Describe() );
		checkf( NetGUID.IsDynamic(), TEXT("Non-stably named object %s has static NetGUID [%s]. %s"), *GetFullNameSafe(Object), *NetGUID.ToString(), *Connection->Describe() );
		return false;		// We only export objects that have stable names
	}

	if ( NetGUID.IsDefault() )
	{
		checkf( !IsNetGUIDAuthority(), TEXT("A default NetGUID for object %s is being exported on the server. %s"), *GetFullNameSafe(Object), *Connection->Describe() );
		checkf( Object->IsNameStableForNetworking(), TEXT("A default NetGUID is being exported for non-stably named object %s. %s"), *GetFullNameSafe(Object), *Connection->Describe() );
		return true;
	}

	return !NetGUIDHasBeenAckd( NetGUID );
}

void UPackageMapClient::ReportSyncLoadsForProperty(const FProperty* Property, const UObject* Object)
{
	if (bNetReportSyncLoads)
	{
		for (FNetworkGUID SyncLoadedGUID : TrackedSyncLoadedGUIDs)
		{
			const UObject* LoadedObject = GetObjectFromNetGUID(SyncLoadedGUID, false);
		
			FNetSyncLoadReport Report;
			Report.Type = ENetSyncLoadType::PropertyReference;
			Report.NetDriver = Connection ? Connection->Driver.Get() : nullptr;
			Report.OwningObject = Object;
			Report.Property = Property;
			Report.LoadedObject = LoadedObject;
			FNetDelegates::OnSyncLoadDetected.Broadcast(Report);

			// Remove the GUID from cache tracking so we don't log duplicates.
			GuidCache->ClearSyncLoadedGUID(SyncLoadedGUID);
		}
	}

	ResetTrackedSyncLoadedGuids();
}

void UPackageMapClient::ReportSyncLoadsForActorSpawn(const AActor* Actor)
{
	if (bNetReportSyncLoads)
	{
		for (FNetworkGUID SyncLoadedGUID : TrackedSyncLoadedGUIDs)
		{
			const UObject* LoadedObject = GetObjectFromNetGUID(SyncLoadedGUID, false);
		
			FNetSyncLoadReport Report;
			Report.Type = ENetSyncLoadType::ActorSpawn;
			Report.NetDriver = Connection ? Connection->Driver.Get() : nullptr;
			Report.OwningObject = Actor;
			Report.LoadedObject = LoadedObject;
			FNetDelegates::OnSyncLoadDetected.Broadcast(Report);

			// Remove the GUID from cache tracking so we don't log duplicates.
			GuidCache->ClearSyncLoadedGUID(SyncLoadedGUID);
		}
	}

	ResetTrackedSyncLoadedGuids();
}

/**
 *	Prints debug info about this package map's state
 */
void UPackageMapClient::LogDebugInfo( FOutputDevice & Ar )
{
	for ( auto It = GuidCache->NetGUIDLookup.CreateIterator(); It; ++It )
	{
		FNetworkGUID NetGUID = It.Value();

		FString Status = "Unused";
		if ( OverrideAckState->NetGUIDAckStatus.Contains( NetGUID ) )
		{
			const int32 PacketId = OverrideAckState->NetGUIDAckStatus.FindRef(NetGUID);
			if ( PacketId == GUID_PACKET_NOT_ACKED )
			{
				Status = TEXT("UnAckd");
			}
			else if ( PacketId == GUID_PACKET_ACKED )
			{
				Status = TEXT("Ackd");
			}
			else
			{
				Status = TEXT("Pending");
			}
		}

		UObject *Obj = It.Key().Get();
		FString Str = FString::Printf(TEXT("%s [%s] [%s] - %s"), *NetGUID.ToString(), *Status, NetGUID.IsDynamic() ? TEXT("Dynamic") : TEXT("Static") , Obj ? *Obj->GetFullName() : TEXT("NULL"));
		Ar.Logf(TEXT("%s"), *Str);
		UE_LOGF(LogNetPackageMap, Log, "%ls", *Str);
	}
}

/**
 *	Returns true if Object's outer level has completely finished loading.
 */
bool UPackageMapClient::ObjectLevelHasFinishedLoading(UObject* Object) const
{
	return UE::Net::ObjectLevelHasFinishedLoading(Object, Connection != nullptr ? Connection->Driver : nullptr);
}

/**
 * Return false if our connection is the netdriver's server connection
 *  This is ugly but probably better than adding a shadow variable that has to be
 *  set/cleared at the net driver level.
 */
bool UPackageMapClient::IsNetGUIDAuthority() const
{
	return GuidCache->IsNetGUIDAuthority();
}

/**	
 *	Returns stats for NetGUID usage
 */
void UPackageMapClient::GetNetGUIDStats(int32 &AckCount, int32 &UnAckCount, int32 &PendingCount)
{
	AckCount = UnAckCount = PendingCount = 0;
	for ( auto It = OverrideAckState->NetGUIDAckStatus.CreateIterator(); It; ++It )
	{
		// Sanity check that we're in sync
		check( ( It.Value() > GUID_PACKET_ACKED ) == PendingAckGUIDs.Contains( It.Key() ) );

		if ( It.Value() == GUID_PACKET_NOT_ACKED )
		{
			UnAckCount++;
		}
		else if ( It.Value() == GUID_PACKET_ACKED )
		{
			AckCount++;
		}
		else
		{
			PendingCount++;
		}
	}

	// Sanity check that we're in sync
	check( PendingAckGUIDs.Num() == PendingCount );
}

void UPackageMapClient::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	return Super::AddReferencedObjects(InThis, Collector);
}

void UPackageMapClient::NotifyStreamingLevelUnload(UObject* UnloadedLevel)
{
}

bool UPackageMapClient::PrintExportBatch()
{
	if ( ExportNetGUIDCount <= 0 && CurrentExportBunch == NULL )
	{
		return false;
	}

	// Print the whole thing for reference
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (FNetGUIDCache::IsHistoryEnabled())
	{
		for (auto It = GuidCache->History.CreateIterator(); It; ++It)
		{
			FString Str = It.Value();
			FNetworkGUID NetGUID = It.Key();
			UE_LOGF(LogNetPackageMap, Warning, "<%ls> - %ls", *NetGUID.ToString(), *Str);
		}
	}
#endif

	UE_LOGF(LogNetPackageMap, Warning, "\n\n");
	if ( CurrentExportBunch != NULL )
	{
		for (auto It = CurrentExportBunch->ExportNetGUIDs.CreateIterator(); It; ++It)
		{
			UE_LOGF(LogNetPackageMap, Warning, "  CurrentExportBunch->ExportNetGUIDs: %ls", *It->ToString());
		}
	}

	UE_LOGF(LogNetPackageMap, Warning, "\n");
	for (auto It = CurrentExportNetGUIDs.CreateIterator(); It; ++It)
	{
		UE_LOGF(LogNetPackageMap, Warning, "  CurrentExportNetGUIDs: %ls", *It->ToString());
	}

	return true;
}

UObject* UPackageMapClient::GetObjectFromNetGUID( const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped )
{
	return GuidCache->GetObjectFromNetGUID( NetGUID, bIgnoreMustBeMapped );
}

FNetworkGUID UPackageMapClient::GetNetGUIDFromObject(const UObject* InObject) const
{
	return GuidCache->GetNetGUID(InObject);
}

bool UPackageMapClient::IsGUIDPending(const FNetworkGUID& NetGUID) const
{
	FNetworkGUID NetGUIDToSearch = NetGUID;

	// Check Outer chain
	while (NetGUIDToSearch.IsValid())
	{
		if (CurrentQueuedBunchNetGUIDs.Contains(NetGUIDToSearch))
		{
			return true;
		}

		const FNetGuidCacheObject* CacheObjectPtr = GuidCache->ObjectLookup.Find(NetGUIDToSearch);

		if (!CacheObjectPtr)
		{
			return false;
		}

		if (CacheObjectPtr->bIsPending)
		{
			return true;
		}

		NetGUIDToSearch = CacheObjectPtr->OuterGUID;
	}

	return false;
}

void UPackageMapClient::SetHasQueuedBunches(const FNetworkGUID& NetGUID, bool bHasQueuedBunches)
{
	if (bHasQueuedBunches)
	{
		if (GPackageMapTrackQueuedActorThreshold > 0.f)
		{
			if (UNetDriver const * const NetDriver = ((Connection) ? Connection->GetDriver() : nullptr))
			{
				CurrentQueuedBunchNetGUIDs.Emplace(NetGUID, NetDriver->GetElapsedTime());
			}
		}

		DelinquentQueuedActors.MaxConcurrentQueuedActors = FMath::Max<uint32>(DelinquentQueuedActors.MaxConcurrentQueuedActors, CurrentQueuedBunchNetGUIDs.Num());
	}
	else
	{
		double StartTime = 0.f;

		const bool bNormalQueueEnabled = GPackageMapTrackQueuedActorThreshold > 0.f;
		
#if CSV_PROFILER_STATS		
		const bool bOwnerQueueEnabled = GPackageMapTrackQueuedActorThresholdOwner > 0.f;
#else
		constexpr bool bOwnerQueueEnabled = false;
#endif		

		// We try to remove the value regardless of whether or not the CVar is on.
		// That way if it's toggled on and off, we don't end up wasting resources.
		// If it is disabled with entries in DelinquentQueuedActors, it will be up
		// to clients to clear out the map by calling ConsumeDelinquencyAnalytics.
		if (CurrentQueuedBunchNetGUIDs.RemoveAndCopyValue(NetGUID, StartTime) &&
			(bNormalQueueEnabled || bOwnerQueueEnabled) &&
			GuidCache)
		{
			if (UNetDriver const * const NetDriver = ((Connection) ? Connection->GetDriver() : nullptr))
			{
				const double QueuedTime = NetDriver->GetElapsedTime() - StartTime;
				const bool bAboveNormalQueuedTime = bNormalQueueEnabled && QueuedTime > GPackageMapTrackQueuedActorThreshold;
				const bool bAboveOwnerQueuedTime = bOwnerQueueEnabled && QueuedTime > GPackageMapTrackQueuedActorThresholdOwner;

				if (bAboveNormalQueuedTime || bAboveOwnerQueuedTime)
				{
					if (FNetGuidCacheObject const * const CacheObject = GuidCache->ObjectLookup.Find(NetGUID))
					{
						if (UObject const * const Object = CacheObject->Object.Get())
						{
							const FName ObjectClass = Object->GetClass()->GetFName();

							if (bAboveNormalQueuedTime)
							{
								DelinquentQueuedActors.DelinquentQueuedActors.Emplace(ObjectClass, QueuedTime);
							}

#if CSV_PROFILER_STATS
							if (bAboveOwnerQueuedTime && GuidCache->IsTrackingOwnerOrPawn())
							{
								CSV_EVENT(PackageMap, TEXT("Owner Net Stall Queued Actor (QueueTime=%.2f)"), QueuedTime);
							}
#endif
						}
					}
				}
			}
		}
	}
}

void UPackageMapClient::Serialize(FArchive& Ar)
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "UPackageMapClient::Serialize");

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("Super", Super::Serialize(Ar));

	if (Ar.IsCountingMemory())
	{
		// TODO: We don't currently track:
		//		Working Bunches.

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetGUIDExportCountMap", NetGUIDExportCountMap.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ExportGUIDArchives",
			ExportGUIDArchives.CountBytes(Ar);
			for (const TArray<uint8>& Archive : ExportGUIDArchives)
			{
				Archive.CountBytes(Ar);
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CurrentExportNetGUIDS", CurrentExportNetGUIDs.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("CurrentQueuedBunchNetGUIDs", CurrentQueuedBunchNetGUIDs.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingAckGUIDs", PendingAckGUIDs.CountBytes(Ar));

		// Don't use the override here, as that's not technically owned by us.
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("AckState", AckState.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ExportBunches",
			ExportBunches.CountBytes(Ar);
			for (FOutBunch const * const ExportBunch : ExportBunches)
			{
				if (ExportBunch)
				{
					ExportBunch->CountMemory(Ar);
				}
			}
		);

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("MustBeMappedGuidsInLastBunch", MustBeMappedGuidsInLastBunch.CountBytes(Ar));
		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetFieldExports", NetFieldExports.CountBytes(Ar));

		GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DelinquentQueuedActors", DelinquentQueuedActors.CountBytes(Ar));

		// Don't count the GUID Cache here. Instead, we'll let the UNetDriver count it as
		// that's the class that constructs it.
	}
}

const TArray<FNetworkGUID>* FNetGUIDCache::FindUnmappedStablyNamedGuidsWithOuter(FNetworkGUID OuterGUID) const
{
	using namespace UE::Net::Private;

	const FRefCountedNetGUIDArray* Found = UnmappedStablyNamedGuids_OuterToInner.Find(OuterGUID);
	if (Found)
	{
		return &Found->GetNetGUIDs();
	}

	return nullptr;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
FNetworkGUID FNetGUIDCache::GetNetworkGUIDFromRemoteObjectId(const FRemoteObjectId RemoteObjectId, const bool bIsStatic)
{
	// Ignore the first bit of FRemoteObjectId which is reserved for the NetGUID static flag.
	return FNetworkGUID::CreateFromIndex(RemoteObjectId.GetIdNumber() >> 1, bIsStatic);
}

FRemoteObjectId FNetGUIDCache::GetRemoteObjectIdFromNetworkGUID(const FNetworkGUID NetGuid)
{
	// The NetGUID ObjectId is a 64-bit value with the first bit as the static flag. Since this value is ignored by
	// FRemoteObjectId (it's a reserved bit) the value is masked away.
	return FRemoteObjectId::CreateFromInt(NetGuid.ObjectId & ~1).GetLocalized();
}
#endif

void UPackageMapClient::AddUnmappedNetGUIDReference(FNetworkGUID UnmappedGUID)
{
	using namespace UE::Net::Private;

	if (bRemapStableSubobjects && GuidCache)
	{
		// For any new unmapped guids that represent stably-named inner objects, keep track of them
		// so that when the NetDriver updates unmapped objects, if an outer GUID is imported, we can also import its
		// stably-named inners. These are usually subobjects created in the constructor and don't get imported via any other path.
		const FNetGuidCacheObject* CacheObject = GuidCache->GetCacheObject(UnmappedGUID);
		if (CacheObject && CacheObject->OuterGUID.IsValid() && !CacheObject->PathName.IsNone())
		{
			FRefCountedNetGUIDArray& Inners = GuidCache->UnmappedStablyNamedGuids_OuterToInner.FindOrAdd(CacheObject->OuterGUID);
			Inners.Add(UnmappedGUID);

			UE_LOGF(LogNetPackageMap, VeryVerbose, "Adding unmapped stably-named inner object NetGUID to tracking map: %ls. With outer: %ls", *GuidCache->Describe(UnmappedGUID), *GuidCache->Describe(CacheObject->OuterGUID));
		}
	}
}

void UPackageMapClient::RemoveUnmappedNetGUIDReference(FNetworkGUID NetGUID)
{
	using namespace UE::Net::Private;

	if (bRemapStableSubobjects && GuidCache)
	{
		// When a GUID reference is no longer tracked, if we were tracking it as a stably-named inner object,
		// do the bookkeeping here. Decrement the refcount and remove it when there are no more references.
		const FNetGuidCacheObject* CacheObject = GuidCache->GetCacheObject(NetGUID);
		if (CacheObject)
		{
			FRefCountedNetGUIDArray* FoundInners = GuidCache->UnmappedStablyNamedGuids_OuterToInner.Find(CacheObject->OuterGUID);
			if (FoundInners)
			{
				UE_LOGF(LogNetPackageMap, VeryVerbose, "Removing stably-named inner GUID from tracking map: %ls. With outer: %ls", *GuidCache->Describe(NetGUID), *GuidCache->Describe(CacheObject->OuterGUID));
				
				FoundInners->RemoveSwap(NetGUID);

				if (FoundInners->GetNetGUIDs().Num() == 0)
				{
					GuidCache->RemoveUnmappedStablyNamedGuidsWithOuter(CacheObject->OuterGUID);
				}
			}
		}
	}
}

//----------------------------------------------------------------------------------------
//	FNetGUIDCache
//----------------------------------------------------------------------------------------

FNetGUIDCache::FNetGUIDCache(UNetDriver* InDriver, uint64 NetworkGuidSeed) 
	: Driver(InDriver)
	, NetworkChecksumMode(ENetworkChecksumMode::SaveAndUse)
	, AsyncLoadMode(EAsyncLoadMode::UseCVar)
	, IsExportingNetGUIDBunch(false)
	, DelinquentAsyncLoads(GDelinquencyNumberOfTopOffendersToTrack > 0 ? GDelinquencyNumberOfTopOffendersToTrack : 0)
{
	UniqueNetFieldExportGroupPathIndex = 0;

#if !UE_BUILD_SHIPPING
	FParse::Value(FCommandLine::Get(), TEXT("NetworkGuidSeed="), NetworkGuidSeed);
#endif

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UniqueNetIDs[0] = UniqueNetIDs[1] = (int32)NetworkGuidSeed;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	NetworkGuidIndex[0] = NetworkGuidIndex[1] = NetworkGuidSeed;
}

class FArchiveCountMemGUID : public FArchive
{
public:
	FArchiveCountMemGUID() : Mem( 0 ) { ArIsCountingMemory = true; }
	void CountBytes( SIZE_T InNum, SIZE_T InMax ) { Mem += InMax; }
	SIZE_T Mem;
};

void FNetGUIDCache::CleanReferences()
{
	const double Time = FPlatformTime::Seconds();

	TMap<TWeakObjectPtr<UObject>, FNetworkGUID> StaticObjectGuids;

	// Mark all static or non valid dynamic guids to timeout after NETWORK_GUID_TIMEOUT seconds
	// We want to leave them around for a certain amount of time to allow in-flight references to these guids to continue to resolve
	for (auto It = ObjectLookup.CreateIterator(); It; ++It)
	{
		const FNetworkGUID& Guid = It.Key();
		FNetGuidCacheObject& CacheObject = It.Value();

		if (CacheObject.ReadOnlyTimestamp != 0)
		{
			// If this guid was suppose to time out, check to see if it has, otherwise ignore it
			const double NETWORK_GUID_TIMEOUT = 90;

			if (Time - CacheObject.ReadOnlyTimestamp > NETWORK_GUID_TIMEOUT)
			{
				It.RemoveCurrent();
			}
		}
		else if (!CacheObject.Object.IsValid())
		{
			// We will leave this guid around for NETWORK_GUID_TIMEOUT seconds to make sure any in-flight guids can be resolved
			CacheObject.ReadOnlyTimestamp = Time;
		}

		// Static GUIDs may refer to things on disk that don't get unloaded during travel (Packages, Sublevels, Compiled Blueprints, etc.),
		// especially if we're traveling to the same map.
		// The server will forcibly assign a new GUID to certain static objects, but there may be existing requests in flight
		// already with the old GUID.
		// So, we'll do a quick sanity check to make sure everything is fixed up.
		// (Note, even if we get the new GUID later and register it, at worst we'll end up with 2 entries in the ObjectLookup
		// with different GUIDs, but they'll both point at the same WeakObject, and we can clean them up later).
		else if (Guid.IsStatic())
		{
			FNetworkGUID& FoundGuid = StaticObjectGuids.FindOrAdd(CacheObject.Object);

			// We haven't seen this static object before, so just track it.
			if (!FoundGuid.IsValid())
			{
				FoundGuid = Guid;
			}

			// We've seen this static object before, but we're seeing it again with a higher guid.
			// That means this is our newly assigned GUID and we can safely time out the old one.
			else if (FoundGuid < Guid)
			{
				ObjectLookup[FoundGuid].ReadOnlyTimestamp = Time;
				FoundGuid = Guid;
			}
			// We've seen this static object before, but we're seeing it again with a lower guid.
			// This means this is an older assignment and we can time out this cache object.
			else
			{
				CacheObject.ReadOnlyTimestamp = Time;
			}
		}
	}

	for (auto It = NetGUIDLookup.CreateIterator(); It; ++It)
	{
		if (!It.Key().IsValid() || !ObjectLookup.Contains(It.Value()))
		{
			It.RemoveCurrent();
		}
	}

	// Sanity check
	// (make sure look-ups are reciprocal)
	for (auto It = ObjectLookup.CreateIterator(); It; ++It)
	{
		check(!It.Key().IsDefault());
		check(It.Key().IsStatic() != It.Key().IsDynamic());

		checkf(!It.Value().Object.IsValid() || NetGUIDLookup.FindRef(It.Value().Object) == It.Key() || It.Value().ReadOnlyTimestamp != 0, TEXT("Failed to validate ObjectLookup map in UPackageMap. Object '%s' was not in the NetGUIDLookup map with with value '%s'." ), *It.Value().Object.Get()->GetPathName(), *It.Key().ToString());
	}

#if !UE_BUILD_SHIPPING || !UE_BUILD_TEST
	for (auto It = NetGUIDLookup.CreateIterator(); It; ++It)
	{
		check(It.Key().IsValid());
		checkf(ObjectLookup.FindRef(It.Value() ).Object == It.Key(), TEXT("Failed to validate NetGUIDLookup map in UPackageMap. GUID '%s' was not in the ObjectLookup map with with object '%s'."), *It.Value().ToString(), *It.Key().Get()->GetPathName());
	}
#endif

	FArchiveCountMemGUID CountBytesAr;

	ObjectLookup.CountBytes( CountBytesAr );
	NetGUIDLookup.CountBytes( CountBytesAr );

	UE_LOG(LogNetPackageMap, Log, TEXT("FNetGUIDCache::CleanReferences: ObjectLookup: %i, NetGUIDLookup: %i, Mem: %" SIZE_T_FMT " kB"), ObjectLookup.Num(), NetGUIDLookup.Num(), (CountBytesAr.Mem / 1024));
}

bool FNetGUIDCache::SupportsObject( const UObject* Object, const TWeakObjectPtr<UObject>* WeakObjectPtr ) const
{
	// NULL is always supported
	if ( !Object )
	{
		return true;
	}

	// Construct WeakPtr once: either use the passed in one or create a new one.
	const TWeakObjectPtr<UObject>& WeakObject = WeakObjectPtr ? *WeakObjectPtr : MakeWeakObjectPtr<UObject>(const_cast<UObject*>(Object));

	// If we already gave it a NetGUID, its supported.
	// This should happen for dynamic subobjects.
	FNetworkGUID NetGUID = NetGUIDLookup.FindRef( WeakObject );

	if ( NetGUID.IsValid() )
	{
		return true;
	}

#if WITH_EDITOR
	const UPackage* ObjectPackage = Object->GetPackage();
	if (ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		const int32 DriverPIEInstanceID = Driver->GetWorld() ? Driver->GetWorld()->GetPackage()->GetPIEInstanceID() : INDEX_NONE;
		const int32 ObjectPIEInstanceID = ObjectPackage->GetPIEInstanceID();

		if (!ensureAlwaysMsgf(DriverPIEInstanceID == ObjectPIEInstanceID, TEXT("FNetGUIDCache::SupportsObject: Object %s is not supported since its PIE InstanceID: %d differs from the one of the NetDriver's world PIE InstanceID: %d, it will replicate as an invalid reference."), *GetPathNameSafe(Object), ObjectPIEInstanceID, DriverPIEInstanceID))
		{
			// Don't replicate references to objects owned by other PIE instances.
			return false;
		}
	}
#endif

	if ( Object->IsFullNameStableForNetworking() )
	{
		// If object is fully net addressable, it's definitely supported
		return true;
	}

	if ( Object->IsSupportedForNetworking() )
	{
		// This means the server will explicitly tell the client to spawn and assign the id for this object
		return true;
	}

	// Do not display warning when recording replays 
	// IMPORTANT : this is a workaround until until UE-169326 is properly fixed
	bool bDisplayWarning = true;
	if (UWorld* World = Object->GetWorld())
	{
		bDisplayWarning = !World->IsRecordingClientReplay();
	}
	if (bDisplayWarning)
	{
		UE_LOGF(LogNetPackageMap, Warning, "FNetGUIDCache::SupportsObject: %ls NOT Supported.", *Object->GetFullName());
		//UE_LOGF( LogNetPackageMap, Warning, "   %ls", *DebugContextString );
	}
	return false;
}

/**
 *	Dynamic objects are actors or sub-objects that were spawned in the world at run time, and therefor cannot be
 *	referenced with a path name to the client.
 */
bool FNetGUIDCache::IsDynamicObject( const UObject* Object )
{
	check( Object != NULL );
	check( Object->IsSupportedForNetworking() );

	// Any non net addressable object is dynamic
	return !Object->IsFullNameStableForNetworking();
}

bool FNetGUIDCache::IsNetGUIDAuthority() const
{
	return Driver == NULL || Driver->IsServer();
}

/** Gets or assigns a new NetGUID to this object. Returns whether the object is fully mapped or not */
FNetworkGUID FNetGUIDCache::GetOrAssignNetGUID(UObject* Object, const TWeakObjectPtr<UObject>* WeakObjectPtr)
{
	// Construct WeakPtr once: either use the passed in one or create a new one.
	const TWeakObjectPtr<UObject>& WeakObject = WeakObjectPtr ? *WeakObjectPtr : MakeWeakObjectPtr(const_cast<UObject*>(Object));

	if (!Object || !SupportsObject(Object, &WeakObject))
	{
		// Null of unsupported object, leave as default NetGUID and just return mapped=true
		UE_LOGF(LogNetPackageMap, Verbose, "GetOrAssignNetGUID: Object is not supported. Object %ls", *GetPathNameSafe(Object));
		return FNetworkGUID();
	}

	// ----------------
	// Assign NetGUID if necessary
	// ----------------
	
	const bool bIsNetGUIDAuthority = IsNetGUIDAuthority();
	FNetworkGUID NetGUID = NetGUIDLookup.FindRef(WeakObject);

	if (NetGUID.IsValid())
	{
		FNetGuidCacheObject* CacheObject = ObjectLookup.Find(NetGUID);

		// Check to see if this guid is read only
		// If so, we should ignore this entry, and create a new one (or send default as client)
		const bool bReadOnly = CacheObject != nullptr && CacheObject->ReadOnlyTimestamp > 0;

		if (bReadOnly)
		{
			// Reset this object's guid, we will re-assign below (or send default as a client)
			UE_LOGF(LogNetPackageMap, Warning, "GetOrAssignNetGUID: Attempt to reassign read-only guid. FullNetGUIDPath: %ls", *FullNetGUIDPath(NetGUID));

			const bool bAllowClientRemap = !bIsNetGUIDAuthority && GbAllowClientRemapCacheObject;
			if (bAllowClientRemap)
			{
				CacheObject->ReadOnlyTimestamp = 0;
				return NetGUID;
			}
			else
			{
				NetGUIDLookup.Remove(WeakObject);
			}
		}
		else
		{
			return NetGUID;
		}
	}

	if (!bIsNetGUIDAuthority)
	{
		// We cannot make or assign new NetGUIDs
		// Generate a default GUID, which signifies we write the full path
		// The server should detect this, and assign a full-time guid, and send that back to us
		UE_LOGF(LogNetPackageMap, Verbose, "GetOrAssignNetGUID: NetGUIDLookup did not contain object on client, returning default. Object %ls", *Object->GetPathName());
		return FNetworkGUID::GetDefault();
	}

	return AssignNewNetGUID_Server(Object);
}

FNetworkGUID FNetGUIDCache::GetNetGUID(const UObject* Object) const
{
	TWeakObjectPtr<UObject> WeakObj(const_cast<UObject*>(Object));

	if ( !Object || !SupportsObject( Object, &WeakObj ) )
	{
		// Null of unsupported object, leave as default NetGUID and just return mapped=true
		return FNetworkGUID();
	}

	FNetworkGUID NetGUID = NetGUIDLookup.FindRef( WeakObj );
	return NetGUID;
}

/**
 *	Generate a new NetGUID for this object and assign it.
 */
FNetworkGUID FNetGUIDCache::AssignNewNetGUID_Server( UObject* Object )
{
	check( IsNetGUIDAuthority() );

	// Generate new NetGUID and assign it
	const int32 IsStatic = IsDynamicObject( Object ) ? 0 : 1;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	const FRemoteObjectId RemoteObjectId = TObjectPtr<UObject>(Object).GetRemoteId();
	const FNetworkGUID NewNetGuid = GetNetworkGUIDFromRemoteObjectId(RemoteObjectId, IsStatic != 0);

	UE_LOGF(LogNetPackageMapRemoteObject, Log, "Assigning object %ls to use NetGUID %ls based on remote object id %ls",
		   *Object->GetName(),
		   *NewNetGuid.ToString(),
		   *RemoteObjectId.ToString());

#else
	const FNetworkGUID NewNetGuid = FNetworkGUID::CreateFromIndex(++NetworkGuidIndex[IsStatic], IsStatic != 0);
#endif

	RegisterNetGUID_Server( NewNetGuid, Object );

	UE_NET_TRACE_ASSIGNED_GUID(Driver->GetNetTraceId(), NewNetGuid, Object->GetClass()->GetFName(), 0);

	return NewNetGuid;
}

FNetworkGUID FNetGUIDCache::AssignNewNetGUIDFromPath_Server( const FString& PathName, UObject* ObjOuter, UClass* ObjClass )
{
	if ( !IsNetGUIDAuthority() )
	{
		return FNetworkGUID::GetDefault();
	}

	FNetworkGUID OuterGUID = GetOrAssignNetGUID( ObjOuter );

	// Generate new NetGUID and assign it
#if UE_WITH_REMOTE_OBJECT_HANDLE
	const FRemoteObjectId RemoteObjectId = TObjectPtr<UObject>(ObjOuter).GetRemoteId();
	const FNetworkGUID NewNetGuid = FNetworkGUID::CreateFromIndex(RemoteObjectId.GetIdNumber(), true);

	UE_LOGF(LogNetPackageMapRemoteObject, Log, "Assigning object %ls from path %ls to use NetGUID %ls based on remote object id %ls",
		   *ObjOuter->GetName(),
		   *PathName,
		   *NewNetGuid.ToString(),
		   *RemoteObjectId.ToString());
#else
	const FNetworkGUID NewNetGuid = FNetworkGUID::CreateFromIndex(++NetworkGuidIndex[1], true);
#endif

	uint32 NetworkChecksum = GetClassNetworkChecksum( ObjClass );

	RegisterNetGUIDFromPath_Server( NewNetGuid, PathName, OuterGUID, NetworkChecksum, true, true );

	return NewNetGuid;
}

void FNetGUIDCache::RegisterNetGUID_Internal( const FNetworkGUID& NetGUID, const FNetGuidCacheObject& CacheObject )
{
	LLM_SCOPE_BYTAG(GuidCache);

	// We're pretty strict in this function, we expect everything to have been handled before we get here
	check( !ObjectLookup.Contains( NetGUID ) );

	ObjectLookup.Add( NetGUID, CacheObject );

	if ( CacheObject.Object != NULL )
	{
		check( !NetGUIDLookup.Contains( CacheObject.Object ) );

		// If we have an object, associate it with this guid now
		NetGUIDLookup.Add( CacheObject.Object, NetGUID );

		UE_NET_TRACE_ASSIGNED_GUID(Driver->GetNetTraceId(), NetGUID, CacheObject.Object->GetClass()->GetFName(), IsNetGUIDAuthority() ? 0U : 1U);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (IsHistoryEnabled())
		{
			History.Add(NetGUID, CacheObject.Object->GetPathName());
		}
#endif
	}
	else
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (IsHistoryEnabled())
		{
			History.Add(NetGUID, CacheObject.PathName.ToString());
		}
#endif
	}
}

/**
 *	Associates a net guid directly with an object
 *  This function is only called on server
 */
void FNetGUIDCache::RegisterNetGUID_Server( const FNetworkGUID& NetGUID, UObject* Object )
{
	check( IsValid(Object) );
	check( IsNetGUIDAuthority() );				// Only the server should call this
	check( !NetGUID.IsDefault() );
	check( !ObjectLookup.Contains( NetGUID ) );	// Server should never add twice

	FNetGuidCacheObject CacheObject;

	CacheObject.Object				= MakeWeakObjectPtr(const_cast<UObject*>(Object));
	CacheObject.OuterGUID			= GetOrAssignNetGUID( Object->GetOuter() );
	CacheObject.PathName			= Object->GetFName();
	CacheObject.NetworkChecksum		= GetNetworkChecksum( Object );
	CacheObject.bNoLoad				= !CanClientLoadObject( Object, NetGUID );
	CacheObject.bIgnoreWhenMissing	= CacheObject.bNoLoad;

	RegisterNetGUID_Internal( NetGUID, CacheObject );
}

/**
 *	Associates a net guid directly with an object
 *  This function is only called on clients with dynamic guids
 */
void FNetGUIDCache::RegisterNetGUID_Client( const FNetworkGUID& NetGUID, const UObject* Object )
{
	check( !IsNetGUIDAuthority() );			// Only clients should be here
	check( !Object || IsValid(Object) );
	check( !NetGUID.IsDefault() );
	check( NetGUID.IsDynamic() );	// Clients should only assign dynamic guids through here (static guids go through RegisterNetGUIDFromPath_Client)

	UE_LOGF( LogNetPackageMap, Log, "RegisterNetGUID_Client: NetGUID: %ls, Object: %ls", *NetGUID.ToString(), Object ? *Object->GetName() : TEXT( "NULL" ) );
	
	//
	// If we have an existing entry, make sure things match up properly
	// We also completely disassociate anything so that RegisterNetGUID_Internal can be fairly strict
	//

	const FNetGuidCacheObject* ExistingCacheObjectPtr = ObjectLookup.Find( NetGUID );

	if ( ExistingCacheObjectPtr )
	{
		if ( ExistingCacheObjectPtr->PathName != NAME_None )
		{
			UE_LOGF( LogNetPackageMap, Warning, "RegisterNetGUID_Client: Guid with pathname. FullNetGUIDPath: %ls", *FullNetGUIDPath( NetGUID ) );
		}

		// If this net guid was found but the old object is NULL, this can happen due to:
		//	1. Actor channel was closed locally (but we don't remove the net guid entry, since we can't know for sure if it will be referenced again)
		//		a. Then when we re-create a channel, and assign this actor, we will find the old guid entry here
		//	2. Dynamic object was locally GC'd, but then exported again from the server
		//
		// If this net guid was found and the objects match, we don't care. This can happen due to:
		//	1. Same thing above can happen, but if we for some reason didn't destroy the actor/object we will see this case
		//
		// If the object pointers are different, this can be a problem, 
		//	since this should only be possible if something gets out of sync during the net guid exchange code

		const UObject* OldObject = ExistingCacheObjectPtr->Object.Get();

		if ( OldObject != NULL && OldObject != Object )
		{
			UE_LOGF( LogNetPackageMap, Warning, "RegisterNetGUID_Client: Reassigning NetGUID <%ls> to %ls (was assigned to object %ls)", *NetGUID.ToString(), Object ? *Object->GetPathName() : TEXT( "NULL" ), OldObject ? *OldObject->GetPathName() : TEXT( "NULL" ) );
		}
		else
		{
			UE_LOGF( LogNetPackageMap, Verbose, "RegisterNetGUID_Client: Reassigning NetGUID <%ls> to %ls (was assigned to object %ls)", *NetGUID.ToString(), Object ? *Object->GetPathName() : TEXT( "NULL" ), OldObject ? *OldObject->GetPathName() : TEXT( "NULL" ) );
		}

		NetGUIDLookup.Remove( ExistingCacheObjectPtr->Object );
		ObjectLookup.Remove( NetGUID );
	}

	const FNetworkGUID* ExistingNetworkGUIDPtr = NetGUIDLookup.Find( MakeWeakObjectPtr( const_cast<UObject*>( Object ) ) );

	if ( ExistingNetworkGUIDPtr )
	{
		// This shouldn't happen on dynamic guids
		UE_LOGF( LogNetPackageMap, Warning, "Changing NetGUID on object %ls from <%ls:%ls> to <%ls:%ls>", Object ? *Object->GetPathName() : TEXT( "NULL" ), *ExistingNetworkGUIDPtr->ToString(), ExistingNetworkGUIDPtr->IsDynamic() ? TEXT("TRUE") : TEXT("FALSE"), *NetGUID.ToString(), NetGUID.IsDynamic() ? TEXT("TRUE") : TEXT("FALSE") );
		ObjectLookup.Remove( *ExistingNetworkGUIDPtr );
		NetGUIDLookup.Remove( MakeWeakObjectPtr( const_cast<UObject*>( Object ) ) );
	}

	FNetGuidCacheObject CacheObject;

	CacheObject.Object = MakeWeakObjectPtr(const_cast<UObject*>(Object));

	RegisterNetGUID_Internal( NetGUID, CacheObject );
}

/**
 *	Associates a net guid with a path, that can be loaded or found later
 *  This function is only called on the client
 */
void FNetGUIDCache::RegisterNetGUIDFromPath_Client( const FNetworkGUID& NetGUID, const FString& PathName, const FNetworkGUID& OuterGUID, const uint32 NetworkChecksum, const bool bNoLoad, const bool bIgnoreWhenMissing )
{
	check( !IsNetGUIDAuthority() );		// Server never calls this locally
	check( !NetGUID.IsDefault() );

	UE_LOGF( LogNetPackageMap, Log, "RegisterNetGUIDFromPath_Client: NetGUID: %ls, PathName: %ls, OuterGUID: %ls", *NetGUID.ToString(), *PathName, *OuterGUID.ToString() );

	const FNetGuidCacheObject* ExistingCacheObjectPtr = ObjectLookup.Find( NetGUID );

	// If we find this guid, make sure nothing changes
	if ( ExistingCacheObjectPtr != NULL )
	{
		FString ErrorStr;
		bool bPathnameMismatch = false;
		bool bOuterMismatch = false;
		bool bNetGuidMismatch = false;

		if ( ExistingCacheObjectPtr->PathName.ToString() != PathName )
		{
			UE_LOGF( LogNetPackageMap, Warning, "FNetGUIDCache::RegisterNetGUIDFromPath_Client: Path mismatch. Path: %ls, Expected: %ls, NetGUID: %ls", *PathName, *ExistingCacheObjectPtr->PathName.ToString(), *NetGUID.ToString() );
			
			ErrorStr = FString::Printf(TEXT("Path mismatch. Path: %s, Expected: %s, NetGUID: %s"), *PathName, *ExistingCacheObjectPtr->PathName.ToString(), *NetGUID.ToString());
			bPathnameMismatch = true;
		}

		if ( ExistingCacheObjectPtr->OuterGUID != OuterGUID )
		{
			UE_LOGF( LogNetPackageMap, Warning, "FNetGUIDCache::RegisterNetGUIDFromPath_Client: Outer mismatch. Path: %ls, Outer: %ls, Expected: %ls, NetGUID: %ls", *PathName, *OuterGUID.ToString(), *ExistingCacheObjectPtr->OuterGUID.ToString(), *NetGUID.ToString() );
			ErrorStr = FString::Printf(TEXT("Outer mismatch. Path: %s, Outer: %s, Expected: %s, NetGUID: %s"), *PathName, *OuterGUID.ToString(), *ExistingCacheObjectPtr->OuterGUID.ToString(), *NetGUID.ToString());
			bOuterMismatch = true;
		}

		if ( ExistingCacheObjectPtr->Object != NULL )
		{
			FNetworkGUID CurrentNetGUID = NetGUIDLookup.FindRef( ExistingCacheObjectPtr->Object );

			if ( CurrentNetGUID != NetGUID )
			{
				UE_LOGF( LogNetPackageMap, Warning, "FNetGUIDCache::RegisterNetGUIDFromPath_Client: Netguid mismatch. Path: %ls, NetGUID: %ls, Expected: %ls", *PathName, *NetGUID.ToString(), *CurrentNetGUID.ToString() );
				ErrorStr = FString::Printf(TEXT("Netguid mismatch. Path: %s, NetGUID: %s, Expected: %s"), *PathName, *NetGUID.ToString(), *CurrentNetGUID.ToString());
				bNetGuidMismatch = true;
			}
		}

		if (bPathnameMismatch || bOuterMismatch || bNetGuidMismatch)
		{
			BroadcastNetFailure(Driver, ENetworkFailure::NetGuidMismatch, ErrorStr);
		}

		return;
	}

	// Register a new guid with this path
	FNetGuidCacheObject CacheObject;

	CacheObject.PathName			= FName( *PathName );
	CacheObject.OuterGUID			= OuterGUID;
	CacheObject.NetworkChecksum		= NetworkChecksum;
	CacheObject.bNoLoad				= bNoLoad;
	CacheObject.bIgnoreWhenMissing	= bIgnoreWhenMissing;

	RegisterNetGUID_Internal( NetGUID, CacheObject );
}

/**
*	Associates a net guid with a path, that can be loaded or found later
*  This function is only called on the server
*/
void FNetGUIDCache::RegisterNetGUIDFromPath_Server( const FNetworkGUID& NetGUID, const FString& PathName, const FNetworkGUID& OuterGUID, const uint32 NetworkChecksum, const bool bNoLoad, const bool bIgnoreWhenMissing )
{
	check( IsNetGUIDAuthority() );		// Server never calls this locally
	check( !NetGUID.IsDefault() );

	UE_LOGF( LogNetPackageMap, Log, "RegisterNetGUIDFromPath_Server: NetGUID: %ls, PathName: %ls, OuterGUID: %ls", *NetGUID.ToString(), *PathName, *OuterGUID.ToString() );

	const FNetGuidCacheObject* ExistingCacheObjectPtr = ObjectLookup.Find( NetGUID );

	// If we find this guid, make sure nothing changes
	if ( ExistingCacheObjectPtr != nullptr )
	{
		FString ErrorStr;
		bool bPathnameMismatch = false;
		bool bOuterMismatch = false;

		if ( ExistingCacheObjectPtr->PathName.ToString() != PathName )
		{
			UE_LOGF( LogNetPackageMap, Warning, "FNetGUIDCache::RegisterNetGUIDFromPath_Server: Path mismatch. Path: %ls, Expected: %ls, NetGUID: %ls", *PathName, *ExistingCacheObjectPtr->PathName.ToString(), *NetGUID.ToString() );

			ErrorStr = FString::Printf(TEXT("Path mismatch. Path: %s, Expected: %s, NetGUID: %s"), *PathName, *ExistingCacheObjectPtr->PathName.ToString(), *NetGUID.ToString());
			bPathnameMismatch = true;
		}

		if ( ExistingCacheObjectPtr->OuterGUID != OuterGUID )
		{
			UE_LOGF( LogNetPackageMap, Warning, "FNetGUIDCache::RegisterNetGUIDFromPath_Server: Outer mismatch. Path: %ls, Outer: %ls, Expected: %ls, NetGUID: %ls", *PathName, *OuterGUID.ToString(), *ExistingCacheObjectPtr->OuterGUID.ToString(), *NetGUID.ToString() );
			ErrorStr = FString::Printf(TEXT("Outer mismatch. Path: %s, Outer: %s, Expected: %s, NetGUID: %s"), *PathName, *OuterGUID.ToString(), *ExistingCacheObjectPtr->OuterGUID.ToString(), *NetGUID.ToString());
			bOuterMismatch = true;
		}

		if (bPathnameMismatch || bOuterMismatch)
		{
			BroadcastNetFailure(Driver, ENetworkFailure::NetGuidMismatch, ErrorStr);
		}

		return;
	}

	// Register a new guid with this path
	FNetGuidCacheObject CacheObject;

	CacheObject.PathName			= FName( *PathName );
	CacheObject.OuterGUID			= OuterGUID;
	CacheObject.NetworkChecksum		= NetworkChecksum;
	CacheObject.bNoLoad				= bNoLoad;
	CacheObject.bIgnoreWhenMissing	= bIgnoreWhenMissing;

	RegisterNetGUID_Internal( NetGUID, CacheObject );
}

void FNetGUIDCache::ValidateAsyncLoadingPackage(FNetGuidCacheObject& CacheObject, const FNetworkGUID NetGUID)
{
	// With level streaming support we may end up trying to load the same package with a different
	// NetGUID during replay fast-forwarding. This is because if a package was unloaded, and later
	// re-loaded, it will likely be assigned a new NetGUID (since the TWeakObjectPtr to the old package
	// in the cache object would have gone stale). During replay fast-forward, it's possible
	// to see the new NetGUID before the previous one has finished loading, so here we fix up
	// PendingAsyncPackages to refer to the new NewGUID. Also keep track of all the GUIDs referring
	// to the same package so their CacheObjects can be properly updated later.
	FPendingAsyncLoadRequest& PendingLoadRequest = PendingAsyncLoadRequests[CacheObject.PathName];

	PendingLoadRequest.Merge(NetGUID);
	CacheObject.bIsPending = true;

	if (PendingLoadRequest.NetGUIDs.Last() != NetGUID)
	{
		UE_LOGF(LogNetPackageMap, Log, "ValidateAsyncLoadingPackage: Already async loading package with a different NetGUID. Path: %ls, original NetGUID: %ls, new NetGUID: %ls",
			*CacheObject.PathName.ToString(), *PendingLoadRequest.NetGUIDs.Last().ToString(), *NetGUID.ToString());

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PendingAsyncPackages[CacheObject.PathName] = NetGUID;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		UE_LOGF(LogNetPackageMap, Log, "ValidateAsyncLoadingPackage: Already async loading package. Path: %ls, NetGUID: %ls", *CacheObject.PathName.ToString(), *NetGUID.ToString());
	}
	
#if CSV_PROFILER_STATS
	PendingLoadRequest.bWasRequestedByOwnerOrPawn |= IsTrackingOwnerOrPawn();
#endif
}

void FNetGUIDCache::StartAsyncLoadingPackage(FNetGuidCacheObject& CacheObject, const FNetworkGUID NetGUID, const bool bWasAlreadyAsyncLoading)
{
	LLM_SCOPE_BYTAG(GuidCache);

	// Something else is already async loading this package, calling load again will add our callback to the existing load request
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PendingAsyncPackages.Add(CacheObject.PathName, NetGUID);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FPendingAsyncLoadRequest LoadRequest(NetGUID, Driver->GetElapsedTime());
	
#if CSV_PROFILER_STATS
	LoadRequest.bWasRequestedByOwnerOrPawn = IsTrackingOwnerOrPawn();
#endif

	CacheObject.bIsPending = true;

	FPendingAsyncLoadRequest* ExistingRequest = PendingAsyncLoadRequests.Find(CacheObject.PathName);
	if (ExistingRequest)
	{
		// Same package name but a possibly different net GUID. Note down the GUID and wait for the async load completion callback
		ExistingRequest->Merge(LoadRequest);
		return;
	}

	PendingAsyncLoadRequests.Emplace(CacheObject.PathName, MoveTemp(LoadRequest));

	DelinquentAsyncLoads.MaxConcurrentAsyncLoads = FMath::Max<uint32>(DelinquentAsyncLoads.MaxConcurrentAsyncLoads, PendingAsyncLoadRequests.Num());

	FLoadPackageAsyncDelegate LoadPackageCompleteDelegate = FLoadPackageAsyncDelegate::CreateWeakLambda(Driver, [NetDriver = Driver](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
	{
		if (NetDriver->GetNetGuidCache().IsValid())
		{
			NetDriver->GetNetGuidCache()->AsyncPackageCallback(PackageName, Package, Result);
		}
	});

	LoadPackageAsync(CacheObject.PathName.ToString(), LoadPackageCompleteDelegate);
}

void FNetGUIDCache::AsyncPackageCallback(const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
{
	LLM_SCOPE_BYTAG(GuidCache);

	check(Package == nullptr || Package->IsFullyLoaded());

	if (FPendingAsyncLoadRequest const * const PendingLoadRequest = PendingAsyncLoadRequests.Find(PackageName))
	{
		const bool bIsBroken = (Package == nullptr);

		for (FNetworkGUID NetGUIDToProcess : PendingLoadRequest->NetGUIDs)
		{
			if (FNetGuidCacheObject* CacheObject = ObjectLookup.Find(NetGUIDToProcess))
			{
				if (!CacheObject->bIsPending)
				{
					UE_LOGF(LogNetPackageMap, Error, "AsyncPackageCallback: Package wasn't pending. Path: %ls, NetGUID: %ls", *PackageName.ToString(), *NetGUIDToProcess.ToString());
				}

				CacheObject->bIsPending = false;

				if (bIsBroken)
				{
					CacheObject->bIsBroken = true;
					UE_LOGF(LogNetPackageMap, Error, "AsyncPackageCallback: Package FAILED to load. Path: %ls, NetGUID: %ls", *PackageName.ToString(), *NetGUIDToProcess.ToString());
				}

				if (UObject* Object = CacheObject->Object.Get())
				{
					UpdateQueuedBunchObjectReference(NetGUIDToProcess, Object);

					if (UWorld* World = Object->GetWorld())
					{
						if (AGameStateBase* GS = World->GetGameState())
						{
							GS->AsyncPackageLoaded(Object);
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogNetPackageMap, Error, "AsyncPackageCallback: Could not find net guid. Path: %ls, NetGUID: %ls", *PackageName.ToString(), *NetGUIDToProcess.ToString());
			}
		}

		// This won't be the exact amount of time that we spent loading the package, but should
		// give us a close enough estimate (within a frame time).
		const double LoadTime = (Driver->GetElapsedTime() - PendingLoadRequest->RequestStartTime);
		if (GGuidCacheTrackAsyncLoadingGUIDThreshold > 0.f &&
			LoadTime >= GGuidCacheTrackAsyncLoadingGUIDThreshold)
		{
			DelinquentAsyncLoads.DelinquentAsyncLoads.Emplace(PackageName, LoadTime);
		}

#if CSV_PROFILER_STATS
		if (PendingLoadRequest->bWasRequestedByOwnerOrPawn &&
			GGuidCacheTrackAsyncLoadingGUIDThresholdOwner > 0.f &&
			LoadTime >= GGuidCacheTrackAsyncLoadingGUIDThresholdOwner &&
			Driver->ServerConnection)
		{
			CSV_EVENT(PackageMap, TEXT("Owner Net Stall Async Load (Package=%s|LoadTime=%.2f)"), *PackageName.ToString(), LoadTime);
		}
#endif

		PendingAsyncLoadRequests.Remove(PackageName);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PendingAsyncPackages.Remove(PackageName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		UE_LOGF(LogNetPackageMap, Error, "AsyncPackageCallback: Could not find package. Path: %ls", *PackageName.ToString());
	}
}

UObject* FNetGUIDCache::GetObjectFromNetGUID( const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped )
{
	LLM_SCOPE_BYTAG(GuidCache);

	if ( !ensure( NetGUID.IsValid() ) )
	{
		return NULL;
	}

	if ( !ensure( !NetGUID.IsDefault() ) )
	{
		return NULL;
	}

	FNetGuidCacheObject * CacheObjectPtr = ObjectLookup.Find( NetGUID );

	if ( CacheObjectPtr == NULL )
	{
		// This net guid has never been registered
		return NULL;
	}

	UObject* Object = nullptr;

	if (CacheObjectPtr->Object.IsValid()
#if UE_WITH_REMOTE_OBJECT_HANDLE
		&& (UE::RemoteObject::Handle::GetResidence(FWeakObjectPtr(CacheObjectPtr->Object).GetRemoteId()) != EResidence::Remote)
#endif
		)
	{
		Object = CacheObjectPtr->Object.Get();

		// Either the name should match, or this is dynamic, or we're on the server
		checkf( Object->GetFName() == CacheObjectPtr->PathName || NetGUID.IsDynamic() || IsNetGUIDAuthority(),
		        TEXT("ObjectName: '%s', CacheObjectPathName: '%s'"), *Object->GetName(), *CacheObjectPtr->PathName.ToString());
		return Object;
	}

	if ( CacheObjectPtr->bIsBroken )
	{
		// This object is broken, we know it won't load
		// At this stage, any warnings should have already been logged, so we just need to ignore from this point forward
		return NULL;
	}

	if ( CacheObjectPtr->bIsPending )
	{
		// We're not done loading yet (and no error has been reported yet)
		return NULL;
	}

	if ( CacheObjectPtr->PathName == NAME_None )
	{
		// If we don't have a path, assume this is a non stably named guid
		check( NetGUID.IsDynamic() );
		return NULL;
	}

	// First, resolve the outer
	UObject* ObjOuter = NULL;

	if ( CacheObjectPtr->OuterGUID.IsValid() )
	{
		// If we get here, we depend on an outer to fully load, don't go further until we know we have a fully loaded outer
		FNetGuidCacheObject * OuterCacheObject = ObjectLookup.Find( CacheObjectPtr->OuterGUID );

		if ( OuterCacheObject == NULL )
		{
			// Shouldn't be possible, but just in case...
			if ( CacheObjectPtr->OuterGUID.IsStatic() )
			{
				UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Static outer not registered. FullNetGUIDPath: %ls", *FullNetGUIDPath( NetGUID ) );
				CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming
			}
			return NULL;
		}

		// If outer is broken, we will never load, set ourselves to broken as well and bail
		if ( OuterCacheObject->bIsBroken )
		{
			UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Outer is broken. FullNetGUIDPath: %ls", *FullNetGUIDPath( NetGUID ) );
			CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming
			return NULL;
		}

		// Try to resolve the outer
		ObjOuter = GetObjectFromNetGUID( CacheObjectPtr->OuterGUID, bIgnoreMustBeMapped );

		// If we can't resolve the outer
		if ( ObjOuter == NULL )
		{
			// If the outer is missing, warn unless told to ignore
			if ( !ShouldIgnoreWhenMissing( CacheObjectPtr->OuterGUID ) )
			{
				UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Failed to find outer. FullNetGUIDPath: %ls", *FullNetGUIDPath( NetGUID ) );
			}

			return NULL;
		}
	}

	const uint32 TreatAsLoadedFlags = EPackageFlags::PKG_CompiledIn | EPackageFlags::PKG_PlayInEditor;

	// At this point, we either have an outer, or we are a package
	check( !CacheObjectPtr->bIsPending );

	if (!ensure(ObjOuter == nullptr || ObjOuter->GetPackage()->IsFullyLoaded() || ObjOuter->GetPackage()->HasAnyPackageFlags(TreatAsLoadedFlags)))
	{
		UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Outer is null or package is not fully loaded.  FullNetGUIDPath: %ls Outer: %ls", *FullNetGUIDPath( NetGUID ), *GetFullNameSafe(ObjOuter) );
	}

	// See if this object is in memory
	Object = FindObjectFast<UObject>(ObjOuter, CacheObjectPtr->PathName);
#if WITH_EDITOR
	// Object must be null if the package is a dynamic PIE package with pending external objects still loading, as it would normally while object is async loading
	if (Object && Object->GetPackage()->IsDynamicPIEPackagePending())
	{
		Object = NULL;
	}
#endif

	// Assume this is a package if the outer is invalid and this is a static guid
	const bool bIsPackage = NetGUID.IsStatic() && !CacheObjectPtr->OuterGUID.IsValid();
	const bool bIsNetGUIDAuthority = IsNetGUIDAuthority();

	if ( Object == NULL && !CacheObjectPtr->bNoLoad )
	{
		if (bIsNetGUIDAuthority)
		{
			// Log when the server needs to re-load an object, it's probably due to a GC after initially loading as default guid
			UE_LOGF(LogNetPackageMap, Warning, "GetObjectFromNetGUID: Server re-loading object (might have been GC'd). FullNetGUIDPath: %ls", *FullNetGUIDPath(NetGUID));
		}

		if ( bIsPackage )
		{
			// Async load the package if:
			//	1. We are actually a package
			//	2. We aren't already pending
			//	3. We're actually suppose to load (levels don't load here for example)
			//		(Refer to CanClientLoadObject, which is where we protect clients from trying to load levels)

			if ( ShouldAsyncLoad() )
			{
				if (!PendingAsyncLoadRequests.Contains(CacheObjectPtr->PathName))
				{
					StartAsyncLoadingPackage(*CacheObjectPtr, NetGUID, false);
					UE_LOGF(LogNetPackageMap, Log, "GetObjectFromNetGUID: Async loading package. Path: %ls, NetGUID: %ls", *CacheObjectPtr->PathName.ToString(), *NetGUID.ToString());
				}
				else
				{
					ValidateAsyncLoadingPackage(*CacheObjectPtr, NetGUID);
				}

				// There is nothing else to do except wait on the delegate to tell us this package is done loading
				return NULL;
			}
			else
			{
				// Async loading disabled
				FPackagePath Path = FPackagePath::FromPackageNameChecked(CacheObjectPtr->PathName);
				Object = LoadPackage(nullptr, Path, LOAD_None);
				SyncLoadedGUIDs.AddUnique(NetGUID);
			}
		}
		else
		{
			// If we have a package, but for some reason didn't find the object then do a blocking load as a last attempt
			// This can happen for a few reasons:
			//	1. The object was GC'd, but the package wasn't, so we need to reload
			//	2. Someone else started async loading the outer package, and it's not fully loaded yet
			Object = StaticLoadObject( UObject::StaticClass(), ObjOuter, *CacheObjectPtr->PathName.ToString(), NULL, LOAD_NoWarn );

			if ( ShouldAsyncLoad() )
			{
				UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Forced blocking load. Path: %ls, NetGUID: %ls", *CacheObjectPtr->PathName.ToString(), *NetGUID.ToString() );
			}
		}
	}

	if ( Object == NULL )
	{
		if ( !CacheObjectPtr->bIgnoreWhenMissing )
		{
			CacheObjectPtr->bIsBroken = 1;	// Set this to 1 so that we don't keep spamming
			UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Failed to resolve path. FullNetGUIDPath: %ls", *FullNetGUIDPath( NetGUID ) );
		}

		return NULL;
	}

	if ( bIsPackage )
	{
		UPackage * Package = Cast< UPackage >( Object );

		if ( Package == NULL )
		{
			// This isn't really a package but it should be
			CacheObjectPtr->bIsBroken = true;
			UE_LOGF( LogNetPackageMap, Error, "GetObjectFromNetGUID: Object is not a package but should be! Path: %ls, NetGUID: %ls", *CacheObjectPtr->PathName.ToString(), *NetGUID.ToString() );
			return NULL;
		}

		if (!Package->IsFullyLoaded() 
			&& !Package->HasAnyPackageFlags( TreatAsLoadedFlags )) //TODO: dependencies of CompiledIn could still be loaded asynchronously. Are they necessary at this point??
		{
			if (ShouldAsyncLoad() && Package->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
			{
				// Something else is already async loading this package, calling load again will add our callback to the existing load request
				StartAsyncLoadingPackage(*CacheObjectPtr, NetGUID, true);
				UE_LOGF(LogNetPackageMap, Log, "GetObjectFromNetGUID: Listening to existing async load. Path: %ls, NetGUID: %ls", *CacheObjectPtr->PathName.ToString(), *NetGUID.ToString());

				// We don't want to hook up this package into the cache yet or return it, because it's only partially loaded.
				return nullptr;
			}
			else if (!GbNetCheckNoLoadPackages || !CacheObjectPtr->bNoLoad)
			{
				// If package isn't fully loaded, load it now
				UE_LOGF(LogNetPackageMap, Log, "GetObjectFromNetGUID: Blocking load of %ls, NetGUID: %ls", *CacheObjectPtr->PathName.ToString(), *NetGUID.ToString());
				Object = LoadPackage(NULL, *CacheObjectPtr->PathName.ToString(), LOAD_None);
				SyncLoadedGUIDs.AddUnique(NetGUID);
			}
			else
			{
				// Not fully loaded but we should not be loading it directly.
				return nullptr;
			}
		}
	}

	if ( CacheObjectPtr->NetworkChecksum != 0 && !CVarIgnoreNetworkChecksumMismatch.GetValueOnAnyThread() )
	{
		const uint32 NetworkChecksum = GetNetworkChecksum( Object );

		if (CacheObjectPtr->NetworkChecksum != NetworkChecksum )
		{
			if ( NetworkChecksumMode == ENetworkChecksumMode::SaveAndUse )
			{
				FString ErrorStr = FString::Printf(TEXT("GetObjectFromNetGUID: Network checksum mismatch. FullNetGUIDPath: %s, %u, %u"), *FullNetGUIDPath(NetGUID), CacheObjectPtr->NetworkChecksum, NetworkChecksum);
				UE_LOGF( LogNetPackageMap, Warning, "%ls", *ErrorStr );

				CacheObjectPtr->bIsBroken = true;

				BroadcastNetFailure(Driver, ENetworkFailure::NetChecksumMismatch, ErrorStr);
				return NULL;
			}
			else
			{
				UE_LOGF( LogNetPackageMap, Verbose, "GetObjectFromNetGUID: Network checksum mismatch. FullNetGUIDPath: %ls, %u, %u", *FullNetGUIDPath( NetGUID ), CacheObjectPtr->NetworkChecksum, NetworkChecksum );
			}
		}
	}

	if ( Object && !UE::Net::ObjectLevelHasFinishedLoading( Object, Driver ) )
	{
		UE_LOGF(LogNetPackageMap, Verbose, "GetObjectFromNetGUID: Forcing object to NULL since level is not loaded yet. Object: %ls", *GetFullNameSafe(Object));
		return NULL;
	}

	// Assign the resolved object to this guid
	CacheObjectPtr->Object = Object;		

	// Assign the guid to the object 
	// We don't want to assign this guid to the object if this guid is timing out
	// But we'll have to if there is no other guid yet
	const bool bAllowClientRemap = !bIsNetGUIDAuthority && GbAllowClientRemapCacheObject;
	const bool bIsNotReadOnlyOrAllowRemap = (CacheObjectPtr->ReadOnlyTimestamp == 0 || bAllowClientRemap);

	if (bIsNotReadOnlyOrAllowRemap || !NetGUIDLookup.Contains(Object))
	{
		if (CacheObjectPtr->ReadOnlyTimestamp > 0)
		{
			UE_LOGF( LogNetPackageMap, Warning, "GetObjectFromNetGUID: Attempt to reassign read-only guid. FullNetGUIDPath: %ls", *FullNetGUIDPath( NetGUID ) );

			if (bAllowClientRemap)
			{
				CacheObjectPtr->ReadOnlyTimestamp = 0;
			}
		}

		NetGUIDLookup.Add( Object, NetGUID );

		if (Object)
		{
			UE_NET_TRACE_ASSIGNED_GUID(Driver->GetNetTraceId(), NetGUID, Object->GetClass()->GetFName(), IsNetGUIDAuthority() ? 0U : 1U);
		}
	}

	// Update our QueuedObjectReference if one exists.
	UpdateQueuedBunchObjectReference(NetGUID, Object);

	return Object;
}

bool FNetGUIDCache::ShouldIgnoreWhenMissing( const FNetworkGUID& NetGUID ) const
{
	if (NetGUID.IsDynamic())
	{
		return true;		// Ignore missing dynamic guids (even on server because client may send RPC on/with object it doesn't know server destroyed)
	}

	if ( IsNetGUIDAuthority() )
	{
		return false;		// Server never ignores when missing, always warns
	}

	const FNetGuidCacheObject* CacheObject = ObjectLookup.Find( NetGUID );

	if ( CacheObject == NULL )
	{
		return false;		// If we haven't been told about this static guid before, we need to warn
	}

	const FNetGuidCacheObject* OutermostCacheObject = CacheObject;

	while ( OutermostCacheObject != NULL && OutermostCacheObject->OuterGUID.IsValid() )
	{
		OutermostCacheObject = ObjectLookup.Find( OutermostCacheObject->OuterGUID );
	}

	if ( OutermostCacheObject != NULL )
	{
		// If our outer package is not fully loaded, then don't warn, assume it will eventually come in
		if ( OutermostCacheObject->bIsPending )
		{
			// Outer is pending, don't warn
			return true;
		}

		if ( OutermostCacheObject->Object != NULL )
		{
#if WITH_EDITOR
			// Ignore if the package is a dynamic PIE package with pending external objects still loading
			if ( OutermostCacheObject->Object->GetPackage()->IsDynamicPIEPackagePending() )
			{
				return true;
			}
#endif
			// Sometimes, other systems async load packages, which we don't track, but still must be aware of
			if ( !OutermostCacheObject->Object->GetPackage()->IsFullyLoaded() )
			{
				return true;
			}
		}
	}

	// Ignore warnings when we explicitly are told to
	return CacheObject->bIgnoreWhenMissing;
}

bool FNetGUIDCache::IsGUIDRegistered( const FNetworkGUID& NetGUID ) const
{
	if ( !NetGUID.IsValid() )
	{
		return false;
	}

	if ( NetGUID.IsDefault() )
	{
		return false;
	}

	return ObjectLookup.Contains( NetGUID );
}

FNetGuidCacheObject const * const FNetGUIDCache::GetCacheObject(const FNetworkGUID& NetGUID) const
{
	return (!NetGUID.IsValid() || NetGUID.IsDefault()) ? nullptr : ObjectLookup.Find(NetGUID);
}

bool FNetGUIDCache::IsGUIDLoaded(const FNetworkGUID& NetGUID) const
{
	FNetGuidCacheObject const * const CacheObjectPtr = GetCacheObject(NetGUID);
	return CacheObjectPtr && CacheObjectPtr->Object != nullptr;
}

bool FNetGUIDCache::IsGUIDBroken(const FNetworkGUID& NetGUID, const bool bMustBeRegistered) const
{
	if (!NetGUID.IsValid())
	{
		return false;
	}

	if (NetGUID.IsDefault())
	{
		return false;
	}

	if (FNetGuidCacheObject const * const CacheObjectPtr = ObjectLookup.Find(NetGUID))
	{
		return CacheObjectPtr->bIsBroken;
	}

	return bMustBeRegistered;
}

bool FNetGUIDCache::IsGUIDNoLoad(const FNetworkGUID& NetGUID) const
{
	FNetGuidCacheObject const * const CacheObjectPtr = GetCacheObject(NetGUID);
	return CacheObjectPtr && CacheObjectPtr->bNoLoad;
}

bool FNetGUIDCache::IsGUIDPending(const FNetworkGUID& NetGUID) const
{
	FNetGuidCacheObject const * const CacheObjectPtr = GetCacheObject(NetGUID);
	return CacheObjectPtr && CacheObjectPtr->bIsPending;
}

FNetworkGUID FNetGUIDCache::GetOuterNetGUID( const FNetworkGUID& NetGUID ) const
{
	FNetworkGUID OuterGUID;

	if (FNetGuidCacheObject const * const CacheObjectPtr = GetCacheObject(NetGUID))
	{
		OuterGUID = CacheObjectPtr->OuterGUID;
	}

	return OuterGUID;
}

FString FNetGUIDCache::FullNetGUIDPath( const FNetworkGUID& NetGUID ) const
{
	FString FullPath;

	GenerateFullNetGUIDPath_r( NetGUID, FullPath );

	return FullPath;
}

FString	FNetGUIDCache::Describe(const FNetworkGUID& NetGUID) const
{
	FString Desc = FString::Printf(TEXT("NetworkGUID [%s]"), *NetGUID.ToString());

	if (FNetGuidCacheObject const* const CacheObjectPtr = GetCacheObject(NetGUID))
	{
		Desc += FString::Printf(TEXT(" NoLoad [%d] Pending [%d] Broken [%d] Outer [%s] FullPath [%s] Object [%s]"), !!CacheObjectPtr->bNoLoad, !!CacheObjectPtr->bIsPending, !!CacheObjectPtr->bIsBroken, *CacheObjectPtr->OuterGUID.ToString(), *FullNetGUIDPath(NetGUID), *GetFullNameSafe(CacheObjectPtr->Object.Get()));
	}
	else
	{
		Desc += TEXT(" Unregistered");
	}

	return Desc;
}

bool FNetGUIDCache::RemoveNetGUID(const UObject* Object)
{
	FNetworkGUID* NetGUIDPtr = NetGUIDLookup.Find(Object);
	if (NetGUIDPtr)
	{
		FNetworkGUID NetGUID = *NetGUIDPtr;

		ensure(ObjectLookup.Contains(NetGUID));
		ObjectLookup.Remove(NetGUID);
		NetGUIDLookup.Remove(Object);

		return true;
	}

	return false;
}

void FNetGUIDCache::RemoveActorNetGUIDs(const AActor* Actor)
{
	using namespace UE::Net;

	auto RemoveSubObjectsFromGuidCache = [this](const FSubObjectRegistry* SubObjectRegistry)
	{
		if (SubObjectRegistry)
		{
			for (const FSubObjectRegistry::FEntry& SubObjectInfo : SubObjectRegistry->GetRegistryList())
			{
				RemoveNetGUID(SubObjectInfo.GetSubObject());
			}
		}
	};

	RemoveNetGUID(Actor);
	RemoveSubObjectsFromGuidCache(&FSubObjectRegistryGetter::GetSubObjects(Actor));

	for (UActorComponent* Component : Actor->GetComponents())
	{
		RemoveNetGUID(Component);
		RemoveSubObjectsFromGuidCache(FSubObjectRegistryGetter::GetSubObjectsOfActorComponent(Actor, Component));
	}
}

void FNetGUIDCache::GenerateFullNetGUIDPath_r( const FNetworkGUID& NetGUID, FString& FullPath ) const
{
	if ( !NetGUID.IsValid() )
	{
		// This is the end of the outer chain, we're done
		return;
	}

	const FNetGuidCacheObject* CacheObject = ObjectLookup.Find( NetGUID );

	if ( CacheObject == nullptr )
	{
		// Doh, this shouldn't be possible, but if this happens, we can't continue
		// So warn, and return
		FullPath += FString::Printf( TEXT( "[%s]NOT_IN_CACHE" ), *NetGUID.ToString() );
		return;
	}

	GenerateFullNetGUIDPath_r( CacheObject->OuterGUID, FullPath );

	if ( !FullPath.IsEmpty() )
	{
		FullPath += TEXT( "." );
	}

	// Prefer the object name first, since non stable named objects don't store the path
	if ( CacheObject->Object.IsValid() )
	{
		// Sanity check that the names match if the path was stored
		if ( !CacheObject->PathName.IsNone() && CacheObject->Object->GetFName() != CacheObject->PathName )
		{
			UE_LOGF( LogNetPackageMap, Warning, "GenerateFullNetGUIDPath_r: Name mismatch! %ls != %ls", *CacheObject->PathName.ToString(), *CacheObject->Object->GetName() );	
		}

		FullPath += FString::Printf( TEXT( "[%s]%s" ), *NetGUID.ToString(), *CacheObject->Object->GetName() );
	}
	else
	{
		if (CacheObject->PathName.IsNone())
		{
			// This can happen when a non stably named object is NULL
			FullPath += FString::Printf( TEXT( "[%s]EMPTY" ), *NetGUID.ToString() );
		}
		else
		{
			FullPath += FString::Printf( TEXT( "[%s]%s" ), *NetGUID.ToString(), *CacheObject->PathName.ToString() );			
		}
	}
}

uint32 FNetGUIDCache::GetClassNetworkChecksum( UClass* Class )
{
	return Driver->NetCache->GetClassNetCache( Class )->GetClassChecksum();
}

uint32 FNetGUIDCache::GetNetworkChecksum( UObject* Obj )
{
	if ( Obj == NULL )
	{
		return 0;
	}

	// If Obj is already a class, we can use that directly
	UClass* Class = Cast< UClass >( Obj );

	return ( Class != NULL ) ? GetClassNetworkChecksum( Class ) : GetClassNetworkChecksum( Obj->GetClass() );
}

void FNetGUIDCache::SetNetworkChecksumMode( const ENetworkChecksumMode NewMode )
{
	NetworkChecksumMode = NewMode;
}

void FNetGUIDCache::SetAsyncLoadMode( const EAsyncLoadMode NewMode )
{
	AsyncLoadMode = NewMode;
}

bool FNetGUIDCache::ShouldAsyncLoad() const
{
	switch ( AsyncLoadMode )
	{
		case EAsyncLoadMode::UseCVar:		return CVarAllowAsyncLoading.GetValueOnAnyThread() > 0;
		case EAsyncLoadMode::ForceDisable:	return false;
		case EAsyncLoadMode::ForceEnable:	return true;
		default: ensureMsgf( false, TEXT( "Invalid AsyncLoadMode: %i" ), (int32)AsyncLoadMode ); return false;
	}
}

void FNetGUIDCache::ResetCacheForDemo()
{
	ObjectLookup.Reset();
	NetGUIDLookup.Reset();

	NetFieldExportGroupMap.Reset();
	NetFieldExportGroupIndexToGroup.Reset();
	NetFieldExportGroupPathToIndex.Reset();
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static int32 bIsNetGuidCacheHistoryEnabled = 0;
static FAutoConsoleVariableRef CVarIsNetGuidCacheHistoryEnabled(
	TEXT("Net.NetGuidCacheHistoryEnabled"),
	bIsNetGuidCacheHistoryEnabled,
	TEXT("When enabled, allows logging of NetGUIDCache History. Warning, this can eat up a lot of memory, and won't free itself until the Cache is destroyed.")
);

const bool FNetGUIDCache::IsHistoryEnabled()
{
	return !!bIsNetGuidCacheHistoryEnabled;
}
#endif

void FNetGUIDCache::CountBytes(FArchive& Ar) const
{
	GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Ar, "FNetGuidCache::CountBytes");
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ObjectLookup", ObjectLookup.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetGUIDLookup", NetGUIDLookup.CountBytes(Ar));
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("ImportedNetGuids", ImportedNetGuids.CountBytes(Ar));
	
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingOuterNetGuids",
		PendingOuterNetGuids.CountBytes(Ar);

		for (const auto& PendingOuterNetGuidPar : PendingOuterNetGuids)
		{
			PendingOuterNetGuidPar.Value.CountBytes(Ar);
		}
	);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("PendingAsyncPackages", PendingAsyncPackages.CountBytes(Ar));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetFieldExportGroupMap",
		NetFieldExportGroupMap.CountBytes(Ar);

		for (const auto& NetfieldExportGroupPair : NetFieldExportGroupMap)
		{
			NetfieldExportGroupPair.Key.CountBytes(Ar);
			if (FNetFieldExportGroup const * const NetfieldExportGroup = NetfieldExportGroupPair.Value.Get())
			{
				Ar.CountBytes(sizeof(FNetFieldExportGroup), sizeof(FNetFieldExportGroup));
				NetfieldExportGroup->CountBytes(Ar);
			}
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetFieldExportGroupPathToIndex",
		NetFieldExportGroupPathToIndex.CountBytes(Ar);

		for (const auto& NetFieldExportGroupPathToIndexPair : NetFieldExportGroupPathToIndex)
		{
			NetFieldExportGroupPathToIndexPair.Key.CountBytes(Ar);
		}
	);

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("NetFieldExportGroupIndexToGroup", NetFieldExportGroupIndexToGroup.CountBytes(Ar));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("History",
		History.CountBytes(Ar);
		for (const auto& HistoryPairs : History)
		{
			HistoryPairs.Value.CountBytes(Ar);
		}
	);
#endif

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("DelinquentAsyncLoads", DelinquentAsyncLoads.CountBytes(Ar));

	GRANULAR_NETWORK_MEMORY_TRACKING_TRACK("QueuedBunchObjectReferences",

		QueuedBunchObjectReferences.CountBytes(Ar);
		for (const auto& QueuedBunchObjectReferencePair : QueuedBunchObjectReferences)
		{
			if (QueuedBunchObjectReferencePair.Value.IsValid())
			{
				Ar.CountBytes(sizeof(FQueuedBunchObjectReference), sizeof(FQueuedBunchObjectReference));
			}
		}
	);
}

void FNetFieldExport::CountBytes(FArchive& Ar) const
{

}

FArchive& operator<<(FArchive& Ar, FNetFieldExport& C)
{
	uint8 Flags = 0;
	Flags |= C.bExported	? (1 << 0) : 0;
	Flags |= C.bExportBlob	? (1 << 1) : 0;
	Ar << Flags;
	if (Ar.IsLoading())
	{
		C.bExported =	Flags & (1 << 0);
		C.bExportBlob = Flags & (1 << 1);
	}

	if (C.bExported)
	{
		Ar.SerializeIntPacked(C.Handle);
		Ar << C.CompatibleChecksum;

		if (Ar.IsLoading() && Ar.EngineNetVer() < FEngineNetworkCustomVersion::NetExportSerialization)
		{
			FName TempName;
			FString TempType;

			Ar << TempName;
			Ar << TempType;

			C.ExportName = TempName;
		}
		else
		{
			if (Ar.IsLoading() && Ar.EngineNetVer() < FEngineNetworkCustomVersion::NetExportSerializeFix)
			{
				Ar << C.ExportName;
			}
			else
			{
				UPackageMap::StaticSerializeName(Ar, C.ExportName);
			}
		}

		if (C.bExportBlob)
		{
			SafeNetSerializeTArray_Default<4096>(Ar,C.Blob);
		}
	}

	return Ar;
}

void FNetFieldExportGroup::CountBytes(FArchive& Ar) const
{
	PathName.CountBytes(Ar);
	NetFieldExports.CountBytes(Ar);
	for (const FNetFieldExport& NetFieldExport : NetFieldExports)
	{
		NetFieldExport.CountBytes(Ar);
	}
}

FArchive& operator<<(FArchive& Ar, FNetFieldExportGroup& C)
{
	Ar << C.PathName;

	Ar.SerializeIntPacked(C.PathNameIndex);

	uint32 NumNetFieldExports = C.NetFieldExports.Num();
	Ar.SerializeIntPacked(NumNetFieldExports);

	if (Ar.IsLoading())
	{
		if (NumNetFieldExports > (uint32)UE::Net::MaxSerializedNetExportsPerGroup)
		{
			UE_LOGF(LogNetPackageMap, Error, "FNetFieldExportGroup - NumNetFieldExports exceeds MaxSerializedNetExportsPerGroup (%d / %d)", NumNetFieldExports, UE::Net::MaxSerializedNetExportsPerGroup);
			Ar.SetError();
			return Ar;
		}

		C.NetFieldExports.AddDefaulted((int32)NumNetFieldExports);
	}

	for (int32 i = 0; i < C.NetFieldExports.Num(); i++)
	{
		Ar << C.NetFieldExports[i];
	}

	return Ar;
}

void FPackageMapAckState::CountBytes(FArchive& Ar) const
{
	NetGUIDAckStatus.CountBytes(Ar);
	NetFieldExportGroupPathAcked.CountBytes(Ar);
	NetFieldExportAcked.CountBytes(Ar);
}

int32 UPackageMapClient::GetNumQueuedBunchNetGUIDs() const
{
	return CurrentQueuedBunchNetGUIDs.Num();
}

void FNetGUIDCache::ConsumeAsyncLoadDelinquencyAnalytics(FNetAsyncLoadDelinquencyAnalytics& Out)
{
	Out = MoveTemp(DelinquentAsyncLoads);
	DelinquentAsyncLoads.MaxConcurrentAsyncLoads = PendingAsyncLoadRequests.Num();
}

const FNetAsyncLoadDelinquencyAnalytics& FNetGUIDCache::GetAsyncLoadDelinquencyAnalytics() const
{
	return DelinquentAsyncLoads;
}

void FNetGUIDCache::ResetAsyncLoadDelinquencyAnalytics()
{
	DelinquentAsyncLoads.Reset();
}

void UPackageMapClient::ConsumeQueuedActorDelinquencyAnalytics(FNetQueuedActorDelinquencyAnalytics& Out)
{
	Out = MoveTemp(DelinquentQueuedActors);
	DelinquentQueuedActors.MaxConcurrentQueuedActors = CurrentQueuedBunchNetGUIDs.Num();
}

const FNetQueuedActorDelinquencyAnalytics& UPackageMapClient::GetQueuedActorDelinquencyAnalytics() const
{
	return DelinquentQueuedActors;
}

void UPackageMapClient::ResetQueuedActorDelinquencyAnalytics()
{
	DelinquentQueuedActors.Reset();
}

void FNetGUIDCache::CollectReferences(class FReferenceCollector& ReferenceCollector)
{
	for (auto It = QueuedBunchObjectReferences.CreateIterator(); It; ++It)
	{
		TSharedPtr<FQueuedBunchObjectReference> SharedQueuedObjectReference = It.Value().Pin();
		FQueuedBunchObjectReference* ObjectRefPtr = SharedQueuedObjectReference.Get();
		if (ObjectRefPtr)
		{
			// Don't bother adding the reference if we don't have an object.
			if (ObjectRefPtr->Object)
			{
				// AddReferencedObject will set our reference to nullptr if the object is pending kill.
				ReferenceCollector.AddReferencedObject(ObjectRefPtr->Object, Driver);

				if (!ObjectRefPtr->Object)
				{
					UE_LOGF(LogNetPackageMap, Warning, "FNetGUIDCache::CollectReferences: QueuedBunchObjectReference was killed by GC. NetGUID=%ls", *It.Key().ToString());
				}
			}
		}
		else
		{
			// If our weak pointer doesn't resolve, we're clearly the last referencer.
			// No need to keep it around.
			It.RemoveCurrent();
		}
	}

	QueuedBunchObjectReferences.Compact();
}

TSharedRef<FQueuedBunchObjectReference> FNetGUIDCache::TrackQueuedBunchObjectReference(const FNetworkGUID InNetGUID, UObject* InObject)
{
	if (TWeakPtr<FQueuedBunchObjectReference>* ExistingRef = QueuedBunchObjectReferences.Find(InNetGUID))
	{
		TSharedPtr<FQueuedBunchObjectReference> ExistingRefShared = ExistingRef->Pin();
		if (ExistingRefShared.IsValid())
		{
			return ExistingRefShared.ToSharedRef();
		}
	}

	// Using MakeShareable instead of MakeShared to allow private constructor.
	TSharedRef<FQueuedBunchObjectReference> NewRef = MakeShareable<FQueuedBunchObjectReference>(new FQueuedBunchObjectReference(InNetGUID, InObject));
	QueuedBunchObjectReferences.Add(InNetGUID, NewRef);
	return NewRef;
}

void FNetGUIDCache::UpdateQueuedBunchObjectReference(const FNetworkGUID NetGUID, UObject* NewObject)
{
	if (TWeakPtr<FQueuedBunchObjectReference>* WeakObjectReference = QueuedBunchObjectReferences.Find(NetGUID))
	{
		TSharedPtr<FQueuedBunchObjectReference> SharedObjectReference = WeakObjectReference->Pin();
		if (FQueuedBunchObjectReference* ObjectReference = SharedObjectReference.Get())
		{
			ObjectReference->Object = NewObject;
		}
	}
}

void FNetGUIDCache::ReportSyncLoadedGUIDs()
{
	if (bNetReportSyncLoads)
	{
		// Log all sync loaded packages that weren't logged previously for being associated with specific properties
		for (FNetworkGUID SyncLoadedGUID : SyncLoadedGUIDs)
		{	
			const UObject* LoadedObject = GetObjectFromNetGUID(SyncLoadedGUID, false);

			FNetSyncLoadReport Report;
			Report.Type = ENetSyncLoadType::Unknown;
			Report.NetDriver = Driver;
			Report.LoadedObject = LoadedObject;
			FNetDelegates::OnSyncLoadDetected.Broadcast(Report);
		}
	}
	SyncLoadedGUIDs.Reset();
}

//------------------------------------------------------
// Debug command to see how many times we've exported each NetGUID
// Used for measuring inefficiencies. Some duplication is unavoidable since we cannot garuntee atomicicity across multiple channels.
// (for example if you have 100 actor channels of the same actor class go out at once, each will have to export the actor's class path in 
// order to be safey resolved... until the NetGUID is ACKd and then new actor channels will not have to export it again).
//
//------------------------------------------------------

static void	ListNetGUIDExports()
{
	struct FCompareNetGUIDCount
	{
		FORCEINLINE bool operator()( const int32& A, const int32& B ) const { return A > B; }
	};

	for (TObjectIterator<UPackageMapClient> PmIt; PmIt; ++PmIt)
	{
		UPackageMapClient *PackageMap = *PmIt;

		
		PackageMap->NetGUIDExportCountMap.ValueSort(FCompareNetGUIDCount());


		UE_LOGF(LogNetPackageMap, Warning, "-----------------------");
		for (auto It = PackageMap->NetGUIDExportCountMap.CreateIterator(); It; ++It)
		{
			UE_LOGF(LogNetPackageMap, Warning, "NetGUID <%ls> - %d", *(It.Key().ToString()), It.Value() );	
		}
		UE_LOGF(LogNetPackageMap, Warning, "-----------------------");
	}			
}

FAutoConsoleCommand	ListNetGUIDExportsCommand(
	TEXT("net.ListNetGUIDExports"), 
	TEXT( "Lists open actor channels" ), 
	FConsoleCommandDelegate::CreateStatic(ListNetGUIDExports)
	);

// ----------------------------------------------------------------

#if CSV_PROFILER_STATS
bool FNetGUIDCache::IsTrackingOwnerOrPawn() const
{
	return TrackingOwnerOrPawnHelper && TrackingOwnerOrPawnHelper->IsOwnerOrPawn();
}

FNetGUIDCache::FIsOwnerOrPawnHelper::FIsOwnerOrPawnHelper(
	FNetGUIDCache* InGuidCache,
	const AActor* InConnectionActor,
	const AActor* InChannelActor)

	: GuidCache(InGuidCache)
	, ConnectionActor(InConnectionActor)
	, ChannelActor(InChannelActor)
{
	if (GuidCache)
	{
		GuidCache->TrackingOwnerOrPawnHelper = this;
	}
}

FNetGUIDCache::FIsOwnerOrPawnHelper::~FIsOwnerOrPawnHelper()
{
	if (GuidCache)
	{
		GuidCache->TrackingOwnerOrPawnHelper = nullptr;
	}
}

bool FNetGUIDCache::FIsOwnerOrPawnHelper::IsOwnerOrPawn() const
{
	if (CachedResult == INDEX_NONE)
	{
		bool bIsOwnerOrPawn = false;
		if (ChannelActor && ConnectionActor)
		{
			bIsOwnerOrPawn = (ChannelActor == ConnectionActor);
			if (!bIsOwnerOrPawn)
			{
				const AController* Controller = Cast<AController>(ConnectionActor);
				bIsOwnerOrPawn = Controller ? (Controller->GetPawn() == ChannelActor) : false;
			}
		}

		CachedResult = bIsOwnerOrPawn ? 1 : 0;
	}

	return !!CachedResult;
}
#endif

void FNetGUIDCache::ResetReplayDirtyTracking()
{
	// reset guids
	for (auto It = ObjectLookup.CreateIterator(); It; ++It)
	{
		FNetGuidCacheObject& CacheObject = It.Value();
		CacheObject.bDirtyForReplay = true;
	}

	// reset net export groups
	for (auto It = NetFieldExportGroupMap.CreateIterator(); It; ++It)
	{
		TSharedPtr<FNetFieldExportGroup>& ExportGroup = It.Value();
		if (ExportGroup.IsValid())
		{
			// reset the exports in each group
			for (FNetFieldExport& Export : ExportGroup->NetFieldExports)
			{
				Export.bDirtyForReplay = true;
			}

			ExportGroup->bDirtyForReplay = true;
		}
	}
}