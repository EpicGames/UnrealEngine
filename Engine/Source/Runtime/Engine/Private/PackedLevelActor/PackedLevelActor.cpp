// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackedLevelActor/PackedLevelActor.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "WorldPartition/PackedLevelActor/PackedLevelActorDesc.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstancePrivate.h"
#include "UObject/Package.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackedLevelActor)

APackedLevelActor::APackedLevelActor()
	: Super()
#if WITH_EDITOR
	, bChildChanged(false)
	, bLoadForPacking(false)
#endif
{
#if WITH_EDITORONLY_DATA
	// Packed Level Instances don't support level streaming or sub actors
	SetDesiredRuntimeBehavior(ELevelInstanceRuntimeBehavior::None);
#endif
}

void APackedLevelActor::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	// We want to make sure we serialize that property so we can compare to the CDO
	if (!Ar.IsFilterEditorOnly() && Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::PackedLevelInstanceVersion)
	{
		Ar << PackedVersion;
	}
#endif
}

bool APackedLevelActor::IsLoadingEnabled() const
{
#if WITH_EDITOR
	return HasChildEdit() || IsLoaded() || ShouldLoadForPacking();
#else
	return false;
#endif
}

#if WITH_EDITOR

void APackedLevelActor::PostLoad()
{
	Super::PostLoad();

	// Non CDO: Set the Guid to something different then the default value so that we actually run the construction script on actors that haven't been resaved against their latest BP
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) && GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::PackedLevelInstanceVersion)
	{
		const static FGuid NoVersionGUID(0x50817615, 0x74A547A3, 0x9295D655, 0x8A852C0F);
		PackedVersion = NoVersionGUID;
	}
}

void APackedLevelActor::RerunConstructionScripts()
{
	bool bShouldRerunConstructionScript = true;

	if (GetWorld() && GetWorld()->IsGameWorld())
	{
		bShouldRerunConstructionScript = false;

		// Only rerun if version mismatchs
		if (PackedVersion != GetClass()->GetDefaultObject<APackedLevelActor>()->PackedVersion)
		{
			bShouldRerunConstructionScript = true;
			UE_LOGF(LogLevelInstance, Verbose, "RerunConstructionScript was executed on %ls (%ls) because its version (%ls) doesn't match latest version (%ls). Resaving this actor will fix this",
				*GetPathName(),
				*GetPackage()->GetPathName(),
				*PackedVersion.ToString(),
				*GetClass()->GetDefaultObject<APackedLevelActor>()->PackedVersion.ToString());
		}
	}

	if(bShouldRerunConstructionScript)
	{
		// Set bEditableWhenInherited to false to disable editing of properties on components.
		// This was enabled in 5.4, but is not properly handled, so for now we are disabling it.
		// @todo_ow: See https://jira.it.epicgames.com/browse/UE-216035
		TArray<UActorComponent*> PackedComponents;
		GetPackedComponents(PackedComponents);
		for (UActorComponent* PackedComponent : PackedComponents)
		{
			PackedComponent->bEditableWhenInherited = false;
		}
		Super::RerunConstructionScripts();
		PackedVersion = GetClass()->GetDefaultObject<APackedLevelActor>()->PackedVersion;
	}
}

EWorldPartitionActorFilterType APackedLevelActor::GetDetailsFilterTypes() const
{
	if (IsEditing())
	{
		return Super::GetDetailsFilterTypes();
	}

	return IsRootBlueprintTemplate() ? EWorldPartitionActorFilterType::All : EWorldPartitionActorFilterType::None;
}

EWorldPartitionActorFilterType APackedLevelActor::GetLoadingFilterTypes() const
{
	if (IsEditing())
	{
		return Super::GetLoadingFilterTypes();
	}

	return ShouldLoadForPacking() ? EWorldPartitionActorFilterType::All : EWorldPartitionActorFilterType::None;
}

bool APackedLevelActor::IsRootBlueprintTemplate() const
{
	return IsTemplate() && IsRootBlueprint(GetClass());
}

bool APackedLevelActor::IsRootBlueprint(UClass* InClass)
{
	return InClass && InClass->ClassGeneratedBy != nullptr && InClass->GetSuperClass()->IsNative();
}

UBlueprint* APackedLevelActor::GetRootBlueprint() const
{
	UClass* Class = GetClass();
	while (Class->GetSuperClass() && !Class->GetSuperClass()->IsNative())
	{
		Class = Class->GetSuperClass();
	}

	return Cast<UBlueprint>(Class->ClassGeneratedBy);
}

void APackedLevelActor::OnFilterChanged()
{
	Super::OnFilterChanged();

	if (IsRootBlueprintTemplate())
	{
		// Reflect child changes
		TSharedPtr<FPackedLevelActorBuilder> Builder = FPackedLevelActorBuilder::CreateDefaultBuilder();
		Builder->UpdateBlueprint(GetRootBlueprint(), false);
	}
}

bool APackedLevelActor::ShouldLoadForPacking() const
{
	if (bLoadForPacking)
	{
		return true;
	}

	bool bAncestorLoadForPacking = false;
	if (ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem())
	{
		LevelInstanceSubsystem->ForEachLevelInstanceAncestors(this, [&bAncestorLoadForPacking](const ILevelInstanceInterface* LevelInstanceInterface)
		{
			if(const APackedLevelActor* PackedAncestor = Cast<APackedLevelActor>(LevelInstanceInterface))
			{
				if (PackedAncestor->bLoadForPacking)
				{
					bAncestorLoadForPacking = true;
					return false;
				}
			}
			return true;
		});
	}

	return bAncestorLoadForPacking;
}

TUniquePtr<class FWorldPartitionActorDesc> APackedLevelActor::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FPackedLevelActorDesc());
}

FName APackedLevelActor::GetPackedComponentTag()
{
	static FName PackedComponentTag("PackedComponent");
	return PackedComponentTag;
}

void APackedLevelActor::UpdateLevelInstanceFromWorldAsset()
{
	Super::UpdateLevelInstanceFromWorldAsset();

	if (!GetRootBlueprint())
	{
		if (IsWorldAssetValid())
		{
			TSharedPtr<FPackedLevelActorBuilder> Builder = FPackedLevelActorBuilder::CreateDefaultBuilder();
			Builder->PackActor(this, GetWorldAsset());
		}
		else
		{
			DestroyPackedComponents();
		}
	}
}

void APackedLevelActor::OnEditChild()
{
	Super::OnEditChild();

	check(GetLevelInstanceSubsystem()->GetLevelInstanceLevel(this) != nullptr);
	MarkComponentsRenderStateDirty();
}

void APackedLevelActor::OnCommitChild(bool bChanged)
{
	Super::OnCommitChild(bChanged);

	ULevelInstanceSubsystem* LevelInstanceSubsystem = GetLevelInstanceSubsystem();
	check(GetLevelInstanceSubsystem()->GetLevelInstanceLevel(this));
	bChildChanged |= bChanged;
	if (!HasChildEdit())
	{
		UnloadLevelInstance();

		// When child edit state changes we need to dirty render state so that actor is no longer hidden
		MarkComponentsRenderStateDirty();

		if (bChildChanged)
		{
			bChildChanged = false;

			// Reflect child changes
			TSharedPtr<FPackedLevelActorBuilder> Builder = FPackedLevelActorBuilder::CreateDefaultBuilder();

			if (UBlueprint* BlueprintToPack = GetRootBlueprint())
			{
				Builder->UpdateBlueprint(BlueprintToPack);
				// return here because Actor might have been reinstanced
				// @todo_ow: We should only return if actor was reinstanced
				return;
			}
			else
			{
				Builder->PackActor(this, GetWorldAsset());
			}
		}
	}
}

void APackedLevelActor::OnEdit()
{
	Super::OnEdit();
	MarkComponentsRenderStateDirty();
}

void APackedLevelActor::OnCommit(bool bChanged)
{
	Super::OnCommit(bChanged);

	// When edit state changes we need to dirty render state so that actor is no longer hidden
	MarkComponentsRenderStateDirty();

	if (bChanged)
	{
		if (UBlueprint* BlueprintToPack = GetRootBlueprint())
		{
			TSharedPtr<FPackedLevelActorBuilder> Builder = FPackedLevelActorBuilder::CreateDefaultBuilder();
			Builder->UpdateBlueprint(BlueprintToPack);
			// return here because Actor might have been reinstanced
			// @todo_ow: We should only return if actor was reinstanced
			return;
		}
	}
}

bool APackedLevelActor::IsHiddenEd() const
{
	if (UWorld* World = GetWorld(); World && World->IsGameWorld())
	{
		return Super::IsHiddenEd();
	}
	return Super::IsHiddenEd() || IsEditing() || HasChildEdit() || ShouldLoadForPacking();
}

bool APackedLevelActor::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	// Disallow change of the World on Packed Level Actors as it is set once when creating the blueprint.
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(APackedLevelActor, WorldAsset))
	{
		return false;
	}

	return true;
}

void APackedLevelActor::GetPackedComponents(TArray<UActorComponent*>& OutPackedComponents) const
{
	const TSet<UActorComponent*>& Components = GetComponents();
	OutPackedComponents.Reserve(Components.Num());
		
	for (UActorComponent* Component : Components)
	{
		if (Component && Component->ComponentHasTag(GetPackedComponentTag()))
		{
			OutPackedComponents.Add(Component);
		}
	}
}

void APackedLevelActor::DestroyPackedComponents()
{
	Modify();
	TArray<UActorComponent*> PackedComponents;
	GetPackedComponents(PackedComponents);
	for (UActorComponent* PackedComponent : PackedComponents)
	{
		PackedComponent->Modify();
		PackedComponent->DestroyComponent();
	}
}
#endif
