// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackedLevelActor/PackedLevelActorBuilder.h"

#if WITH_EDITOR

#include "Engine/Level.h"
#include "PackedLevelActor/PackedLevelActor.h"
#include "IAssetTools.h"
#include "PackedLevelActor/PackedLevelActorISMBuilder.h"
#include "PackedLevelActor/PackedLevelActorRecursiveBuilder.h"

#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "Engine/SCS_Node.h"

#include "Misc/Paths.h"
#include "Logging/MessageLog.h"

#include "GameFramework/WorldSettings.h"
#include "Engine/LevelBounds.h"
#include "Engine/LevelStreaming.h"
#include "FileHelpers.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "AssetToolsModule.h"
#include "Factories/BlueprintFactory.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "EditorDirectories.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "Serialization/ArchiveCrc32.h"

#include "Components/InstancedStaticMeshComponent.h"


#define LOCTEXT_NAMESPACE "FPackedLevelActorBuilder"

void FPackedLevelActorBuilderContext::FindOrAddCluster(FPackedLevelActorBuilderClusterID&& InClusterID, UActorComponent* InComponent)
{
	TArray<UActorComponent*>& ClusterComponents = Clusters.FindOrAdd(MoveTemp(InClusterID));
	if (InComponent)
	{
		ClusterComponents.Add(InComponent);
		PerActorClusteredComponents.FindChecked(InComponent->GetOwner()).Add(InComponent);
	}
}

void FPackedLevelActorBuilderContext::DiscardActor(AActor* InActor)
{
	ActorDiscards.Add(InActor);
}

FPackedLevelActorBuilder::FPackedLevelActorBuilder()
	: PreviewScene(FPreviewScene::ConstructionValues().SetTransactional(false))
{
}

const FString& FPackedLevelActorBuilder::GetPackedBPPrefix()
{
	static FString BPPrefix = "BPP_";
	return BPPrefix;
}

UBlueprint* FPackedLevelActorBuilder::CreatePackedLevelActorBlueprintWithDialog(TSoftObjectPtr<UBlueprint> InBlueprintAsset, TSoftObjectPtr<UWorld> InWorldAsset, bool bInCompile)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = FPaths::GetPath(InBlueprintAsset.GetLongPackageName());
	SaveAssetDialogConfig.DefaultAssetName = InBlueprintAsset.GetAssetName();
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (!SaveObjectPath.IsEmpty())
	{
		TSoftObjectPtr<UBlueprint> ExistingBPAsset = TSoftObjectPtr<UBlueprint>(FSoftObjectPath(SaveObjectPath));

		if (UBlueprint* BP = ExistingBPAsset.LoadSynchronous())
		{
			return BP;
		}
				
		return CreatePackedLevelActorBlueprint(ExistingBPAsset, InWorldAsset, bInCompile);
	}

	return nullptr;
}

UBlueprint* FPackedLevelActorBuilder::CreatePackedLevelActorBlueprint(TSoftObjectPtr<UBlueprint> InBlueprintAsset, TSoftObjectPtr<UWorld> InWorldAsset, bool bInCompile)
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();

	UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
	BlueprintFactory->ParentClass = APackedLevelActor::StaticClass();
	BlueprintFactory->bSkipClassPicker = true;

	FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(BlueprintFactory);
	if (BlueprintFactory->ConfigureProperties())
	{
		FString PackageDir = FPaths::GetPath(InBlueprintAsset.GetLongPackageName());
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, PackageDir);

		if (UBlueprint* NewBP = Cast<UBlueprint>(AssetTools.CreateAsset(InBlueprintAsset.GetAssetName(), PackageDir, UBlueprint::StaticClass(), BlueprintFactory, FName("Create LevelInstance Blueprint"))))
		{
			APackedLevelActor* CDO = CastChecked<APackedLevelActor>(NewBP->GeneratedClass->GetDefaultObject());
			CDO->SetWorldAsset(InWorldAsset);

			if (bInCompile)
			{
				FKismetEditorUtilities::CompileBlueprint(NewBP, EBlueprintCompileOptions::None);
			}

			AssetTools.SyncBrowserToAssets(TArray<UObject*>{ NewBP });

			return NewBP;
		}
	}

	return nullptr;
}

TSharedPtr<FPackedLevelActorBuilder> FPackedLevelActorBuilder::CreateDefaultBuilder()
{
	TSharedPtr<FPackedLevelActorBuilder> Builder = MakeShared<FPackedLevelActorBuilder>();

	// Class Discards are used to validate the packing result.
	// Components or Actor classes in this set will not generate warnings
	// Avoid dependency (find class)
	UClass* ChaosDebugClass = FindObject<UClass>(nullptr, TEXT("/Script/ChaosSolverEngine.ChaosDebugDrawComponent"));
	if (ChaosDebugClass)
	{
		Builder->ClassDiscards.Add(ChaosDebugClass);
	}

	// Root Components that are SceneComponents (not child class of)
	Builder->ClassDiscards.Add(USceneComponent::StaticClass());
	
	Builder->AddBuilder<FPackedLevelActorRecursiveBuilder>();
	Builder->AddBuilder<FPackedLevelActorISMBuilder>();

	return Builder;
}

void FPackedLevelActorBuilder::ClusterActor(FPackedLevelActorBuilderContext& InContext, AActor* InActor)
{
	if (!InContext.ActorDiscards.Contains(InActor))
	{
		InContext.PerActorClusteredComponents.FindOrAdd(InActor);

		for (const auto& Pair : Builders)
		{
			Pair.Value->GetPackClusters(InContext, InActor);
		}
	}
}

bool FPackedLevelActorBuilder::PackActor(APackedLevelActor* InPackedLevelActor)
{
	return PackActor(InPackedLevelActor, InPackedLevelActor);
}

bool FPackedLevelActorBuilder::PackActor(APackedLevelActor* InPackedLevelActor, ILevelInstanceInterface* InLevelInstanceToPack)
{
	FPackedLevelActorBuilderContext Context(InPackedLevelActor, InLevelInstanceToPack, ClassDiscards);
	return PackActor(Context);
}

bool FPackedLevelActorBuilder::PackActor(FPackedLevelActorBuilderContext& InContext)
{
	FMessageLog Log("PackedLevelActor");
	Log.Info(FText::Format(LOCTEXT("PackingStarted", "Packing of '{0}' started..."), FText::FromString(InContext.GetPackedLevelActor()->GetWorldAssetPackage())));
		
	InContext.GetPackedLevelActor()->DestroyPackedComponents();

	ULevelInstanceSubsystem* LevelInstanceSubystem = InContext.GetPackedLevelActor()->GetLevelInstanceSubsystem();
	check(LevelInstanceSubystem);
	
	ULevel* SourceLevel = LevelInstanceSubystem->GetLevelInstanceLevel(InContext.GetLevelInstanceToPack());
	if (!SourceLevel)
	{
		Log.Error(FText::Format(LOCTEXT("FailedPackingNoLevel", "Packing of '{0}' failed"), FText::FromString(InContext.GetPackedLevelActor()->GetWorldAssetPackage())));
		return false;
	}

	ULevelStreaming* SourceLevelStreaming = ULevelStreaming::FindStreamingLevel(SourceLevel);
	AWorldSettings* WorldSettings = SourceLevel->GetWorldSettings();
	
	// Build relative transform without rotation because pivots don't support rotation
	FTransform CurrentPivotTransform(SourceLevelStreaming->LevelTransform.GetRelativeTransform(InContext.GetPackedLevelActor()->GetActorTransform()).GetTranslation());
	FTransform NewPivotTransform(WorldSettings->LevelInstancePivotOffset);
	FTransform RelativePivotTransform(NewPivotTransform.GetRelativeTransform(CurrentPivotTransform));
		
	InContext.SetRelativePivotTransform(RelativePivotTransform);	
	ClusterActor(InContext, CastChecked<AActor>(InContext.GetLevelInstanceToPack()));

	TArray<uint32> HashArray;
	for (const auto& Pair : InContext.GetClusters())
	{
		TUniquePtr<IPackedLevelActorBuilder>& Builder = Builders.FindChecked(Pair.Key.GetBuilderID());
		HashArray.Add(Builder->PackActors(InContext, Pair.Key, Pair.Value));
	}
	HashArray.Sort();
	FArchiveCrc32 Ar;

	// Change this Guid if existing hashes need to be invalidated (code changes that aren't part of the computed hash)
	static FGuid CodeVersionGuid = FGuid(0xE023D897, 0x4F404333, 0xAD604BB9, 0x8CC6B959);
	Ar << CodeVersionGuid;
	Ar << HashArray;
	InContext.GetPackedLevelActor()->SetPackedHash(Ar.GetCrc());
	InContext.Report(Log);
	return true;
}

bool FPackedLevelActorBuilderContext::ShouldPackComponent(UActorComponent* ActorComponent) const
{
	return ActorComponent && !ActorComponent->IsVisualizationComponent();
}

void FPackedLevelActorBuilderContext::Report(FMessageLog& Log) const
{
	TSet<UActorComponent*> NotClusteredComponents;
	uint32 TotalWarningCount = 0;

	for (const auto& Pair : PerActorClusteredComponents)
	{
		AActor* Actor = Pair.Key;
		const TSet<UActorComponent*>& ClusteredComponents = Pair.Value;

		if (ActorDiscards.Contains(Actor))
		{
			Log.Info(FText::Format(LOCTEXT("ActorDiscard", "Actor '{0}' ignored (Actor Discard)"), FText::FromString(Actor->GetPathName())));
			continue;
		}

		if (Actor->GetClass()->HasAnyClassFlags(CLASS_Transient))
		{
			Log.Info(FText::Format(LOCTEXT("ActorTransientClassDiscard", "Actor '{0}' of type '{1}' ignored (Transient Class Discard)"), FText::FromString(Actor->GetPathName()), FText::FromString(Actor->GetClass()->GetPathName())));
			continue;
		}

		// Class must match (not a child)
		if (ClassDiscards.Contains(Actor->GetClass()))
		{
			Log.Info(FText::Format(LOCTEXT("ActorClassDiscard", "Actor '{0}' of type '{1}' ignored (Class Discard)"), FText::FromString(Actor->GetPathName()), FText::FromString(Actor->GetClass()->GetPathName())));
			continue;
		}

		NotClusteredComponents = Actor->GetComponents().Difference(ClusteredComponents);
		uint32 WarningCount = 0;
		for (UActorComponent* Component : NotClusteredComponents)
		{
			if (!ShouldPackComponent(Component))
			{
				continue;
			}

			if (ClassDiscards.Contains(Component->GetClass()))
			{
				Log.Info(FText::Format(LOCTEXT("ComponentClassDiscard", "Component '{0}' of type '{1}' ignored (Class Discard)"), FText::FromString(Component->GetPathName()), FText::FromString(Component->GetClass()->GetPathName())));
				continue;
			}

			if (Actor->GetClass()->HasAnyClassFlags(CLASS_Transient))
			{
				Log.Info(FText::Format(LOCTEXT("ComopnentTransientClassDiscard", "Component '{0}' of type '{1}' ignored (Transient Class Discard)"), FText::FromString(Component->GetPathName()), FText::FromString(Component->GetClass()->GetPathName())));
				continue;
			}

			WarningCount++;

			Log.Warning(FText::Format(LOCTEXT("ComponentNotPacked", "Component '{0}' was not packed"), FText::FromString(Component->GetPathName())));
		}

		if (WarningCount)
		{
			Log.Warning(FText::Format(LOCTEXT("ActorNotPacked", "Actor '{0}' was not packed completely ({1} warning(s))"), FText::FromString(Actor->GetPathName()), FText::AsNumber(WarningCount)));
		}
		else
		{
			Log.Info(FText::Format(LOCTEXT("ActorPacked", "Actor '{0}' packed successfully"), FText::FromString(Actor->GetPathName())));
		}
		TotalWarningCount += WarningCount;
	}

	if (TotalWarningCount)
	{
		Log.Warning(LOCTEXT("WarningsReported", "Warnings have been reported. Consider using a regular ALevelInstance instead."));
		Log.Open();
	}
	Log.Info(FText::Format(LOCTEXT("PackCompleted", "Packing '{0}' completed with {1} warning(s)"), FText::FromString(PackedLevelActor->GetWorldAssetPackage()), FText::AsNumber(TotalWarningCount)));
}

ALevelInstance* FPackedLevelActorBuilder::CreateTransientLevelInstanceForPacking(TSoftObjectPtr<UWorld> InWorldAsset, const FVector& InLocation, const FRotator& InRotator, const FWorldPartitionActorFilter& InFilter)
{
	// Create Temp Actor for Packing
	FActorSpawnParameters SpawnParams;
	SpawnParams.bCreateActorPackage = false;
	SpawnParams.bHideFromSceneOutliner = true;
	SpawnParams.bNoFail = true;
	SpawnParams.ObjectFlags |= RF_Transient;

	UWorld* World = PreviewScene.GetWorld();
	SpawnParams.OverrideLevel = World->PersistentLevel;

	// Provide the same ActorGuid all the time so that sorting of Component by ObjectPath are deterministic (used in FPackedLevelActorISMBuilder::PackActors)
	// It is safe because CreateTransientLevelInstanceForPacking is always called to Create a Level Instance on the stack, pack it and destroy it so no Guid conflicts can happen.
	const FGuid PackingActorGuid(0x626EAF37, 0x1F7E47B9, 0x8C4788DC, 0x85F52822);
	SpawnParams.OverrideActorGuid = PackingActorGuid;
	APackedLevelActor* LevelInstance = World->SpawnActor<APackedLevelActor>(InLocation, InRotator, SpawnParams);
	LevelInstance->SetFilter(InFilter, false);
	LevelInstance->SetShouldLoadForPacking(true);
	LevelInstance->SetWorldAsset(InWorldAsset);

	// Wait for load
	LevelInstance->GetLevelInstanceSubsystem()->BlockLoadLevelInstance(LevelInstance);

	return LevelInstance;
}

bool FPackedLevelActorBuilder::PackActor(APackedLevelActor* InActor, TSoftObjectPtr<UWorld> InWorldAsset)
{
	ALevelInstance* TransientLevelInstance = CreateTransientLevelInstanceForPacking(InWorldAsset, InActor->GetActorLocation(), InActor->GetActorRotation(), InActor->GetFilter());
	ON_SCOPE_EXIT
	{
		TransientLevelInstance->GetWorld()->DestroyActor(TransientLevelInstance);
	};
	
	return PackActor(InActor, TransientLevelInstance);
}

void FPackedLevelActorBuilder::UpdateBlueprint(UBlueprint* Blueprint, bool bCheckoutAndSave)
{
	APackedLevelActor* CDO = CastChecked<APackedLevelActor>(Blueprint->GeneratedClass->GetDefaultObject());
	check(CDO);

	CreateOrUpdateBlueprint(CDO->GetWorldAsset(), Blueprint, bCheckoutAndSave, /*bPromptForSave=*/false);
}

bool FPackedLevelActorBuilder::CreateOrUpdateBlueprint(TSoftObjectPtr<UWorld> InWorldAsset, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave)
{
	bool bResult = true;
	
	// Transfer filter from existing BP CDO to Transient Level Instance
	FWorldPartitionActorFilter Filter;
	if (!InBlueprintAsset.IsNull())
	{
		if (UBlueprint* BP = InBlueprintAsset.LoadSynchronous())
		{
			Filter = CastChecked<APackedLevelActor>(BP->GeneratedClass->GetDefaultObject())->GetFilter();
		}
	}

	ALevelInstance* TransientLevelInstance = CreateTransientLevelInstanceForPacking(InWorldAsset, FVector::ZeroVector, FRotator::ZeroRotator, Filter);
	
	bResult = CreateOrUpdateBlueprintFromUnpacked(TransientLevelInstance, InBlueprintAsset, bCheckoutAndSave, bPromptForSave);

	TransientLevelInstance->GetWorld()->DestroyActor(TransientLevelInstance);

	return bResult;
}

bool FPackedLevelActorBuilder::CreateOrUpdateBlueprint(ILevelInstanceInterface* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave)
{
	if (APackedLevelActor* PackedLevelActor = Cast<APackedLevelActor>(InLevelInstance))
	{
		return CreateOrUpdateBlueprintFromPacked(PackedLevelActor, InBlueprintAsset, bCheckoutAndSave, bPromptForSave);
	}
	
	return CreateOrUpdateBlueprintFromUnpacked(InLevelInstance, InBlueprintAsset, bCheckoutAndSave, bPromptForSave);
}

bool FPackedLevelActorBuilder::CreateOrUpdateBlueprintFromUnpacked(ILevelInstanceInterface* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave)
{
	bool bResult = true;
	AActor* LevelInstanceActor = CastChecked<AActor>(InLevelInstance);

	// Create Temp Actor for Packing
	FActorSpawnParameters SpawnParams;
	SpawnParams.bCreateActorPackage = false;
	SpawnParams.bHideFromSceneOutliner = true;
	SpawnParams.bNoFail = true;
	SpawnParams.ObjectFlags |= RF_Transient;
	SpawnParams.ObjectFlags &= ~RF_Transactional;
	
	UWorld* World = PreviewScene.GetWorld();
	SpawnParams.OverrideLevel = World->PersistentLevel;

	APackedLevelActor* PackedLevelActor = World->SpawnActor<APackedLevelActor>(LevelInstanceActor->GetActorLocation(), LevelInstanceActor->GetActorRotation(), SpawnParams);
	PackedLevelActor->SetFilter(InLevelInstance->GetFilter(), false);
	PackedLevelActor->SetWorldAsset(InLevelInstance->GetWorldAsset());
	ON_SCOPE_EXIT
	{
		LevelInstanceActor->GetWorld()->DestroyActor(PackedLevelActor);
	};

	if (!PackActor(PackedLevelActor, InLevelInstance))
	{
		return false;
	}

	bResult &= CreateOrUpdateBlueprintFromPacked(PackedLevelActor, InBlueprintAsset, bCheckoutAndSave, bPromptForSave);

	return bResult;
}

bool FPackedLevelActorBuilder::CreateOrUpdateBlueprintFromPacked(APackedLevelActor* InActor, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptToSave)
{
	if (GEditor->Trans)
	{
		GEditor->Trans->Reset(LOCTEXT("BlueprintPacking", "Blueprint Packing"));
	}

	UBlueprint* BP = nullptr;
	if (!InBlueprintAsset.IsNull())
	{
		BP = InBlueprintAsset.LoadSynchronous();
	}

	if (!BP)
	{
		int32 LastSlashIndex = 0;
		FString LongPackageName = InActor->GetWorldAsset().GetLongPackageName();
		LongPackageName.FindLastChar('/', LastSlashIndex);

		FString PackagePath = LongPackageName.Mid(0, LastSlashIndex == INDEX_NONE ? MAX_int32 : LastSlashIndex);
		FString AssetName = GetPackedBPPrefix() + InActor->GetWorldAsset().GetAssetName();
		const bool bCompile = false;

		FString AssetPath = PackagePath + AssetName + "." + AssetName;
		BP = CreatePackedLevelActorBlueprintWithDialog(TSoftObjectPtr<UBlueprint>(FSoftObjectPath(AssetPath)), InActor->GetWorldAsset(), bCompile);
	}

	if (!BP || !BP->SimpleConstructionScript)
	{
		return false;
	}
	
	// No Need to update the Blueprint because packing result is the same
	FMessageLog Log("PackedLevelActor");
	uint32 PackedHash = CastChecked<APackedLevelActor>(BP->GeneratedClass->GetDefaultObject())->GetPackedHash();
	if (PackedHash != 0 && PackedHash == InActor->GetPackedHash())
	{
		Log.Info(FText::Format(LOCTEXT("PackedBlueprintUpToDate", "Packed Blueprint '{0}' already up to date"), FText::FromString(InBlueprintAsset.ToString())));
		return true;
	}

	BP->Modify();

	TArray<USCS_Node*> AllNodes = BP->SimpleConstructionScript->GetAllNodes();
	for (USCS_Node* Node : AllNodes)
	{
		BP->SimpleConstructionScript->RemoveNodeAndPromoteChildren(Node);
	}
	FKismetEditorUtilities::GenerateBlueprintSkeleton(BP,true);
		
	// Avoid running construction script while dragging an instance of that BP for performance reasons
	BP->bRunConstructionScriptOnDrag = false;

	// New Guid everytime we pack a BP so that we can avoid running construction scripts in PIE for Instances that have a matching version
	FGuid NewVersion = FGuid::NewGuid();
	APackedLevelActor* CDO = CastChecked<APackedLevelActor>(BP->GeneratedClass->GetDefaultObject());
			
	// Prep AddComponentsToBlueprintParam
	FKismetEditorUtilities::FAddComponentsToBlueprintParams AddCompToBPParams;
	AddCompToBPParams.HarvestMode = FKismetEditorUtilities::EAddComponentToBPHarvestMode::None;
	AddCompToBPParams.bKeepMobility = true;

	// Add Components
	TArray<UActorComponent*> PackedComponents;
	InActor->GetPackedComponents(PackedComponents);
	
	for (UActorComponent* PackedComponent : PackedComponents)
	{
		// To avoid any custom serialization on PerInstance data we make PerInstance data inherited always
		if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(PackedComponent))
		{
			ISMComponent->bInheritPerInstanceData = true;
		}
		// Set bEditableWhenInherited to false to disable editing of properties on components.
		// This was enabled in 5.4, but is not properly handled, so for now we are disabling it.
		// @todo_ow: See https://jira.it.epicgames.com/browse/UE-216035
		PackedComponent->bEditableWhenInherited = false;
	}

	FKismetEditorUtilities::AddComponentsToBlueprint(BP, PackedComponents, AddCompToBPParams);
	// If we are packing the actors BP then destroy packed components as they are now part of the BPs construction script
	if (UBlueprint* GeneratedBy = Cast<UBlueprint>(InActor->GetClass()->ClassGeneratedBy))
	{
		InActor->DestroyPackedComponents();
	}

	// Propagate properties before BP Compilation so that they are considered default (no delta)
	TArray<UObject*> ObjectsOfClass;
	GetObjectsOfClass(BP->GeneratedClass, ObjectsOfClass, true, RF_NoFlags, EInternalObjectFlags::Garbage);
	for (UObject* ObjectOfClass : ObjectsOfClass)
	{
		const bool bWasDirty = ObjectOfClass->GetPackage()->IsDirty();

		APackedLevelActor* PackedLevelActor = CastChecked<APackedLevelActor>(ObjectOfClass);

		PackedLevelActor->Modify(false);
		PackedLevelActor->SetWorldAsset(InActor->GetWorldAsset());
		PackedLevelActor->SetFilter(InActor->GetFilter(), false);
		PackedLevelActor->SetPackedVersion(NewVersion);
		PackedLevelActor->SetPackedHash(InActor->GetPackedHash());
		// match root component mobility to source actor
		USceneComponent* Root = PackedLevelActor->GetRootComponent();
		Root->SetMobility(InActor->GetRootComponent()->Mobility);

		if (!bWasDirty && ObjectOfClass->GetPackage()->IsDirty())
		{
			ObjectOfClass->GetPackage()->SetDirtyFlag(false);
		}
	}
	
			
	// Synchronous compile
	FKismetEditorUtilities::CompileBlueprint(BP, EBlueprintCompileOptions::SkipGarbageCollection);
	Log.Info(FText::Format(LOCTEXT("PackedBlueprintUpdated", "Updated Packed Blueprint '{0}'"), FText::FromString(InBlueprintAsset.ToString())));

	if (bCheckoutAndSave)
	{
		const bool bCheckDirty = false;
		TArray<UPackage*> OutFailedPackages;
		FEditorFileUtils::PromptForCheckoutAndSave({ BP->GetPackage() }, bCheckDirty, bPromptToSave, &OutFailedPackages);

		return !OutFailedPackages.Num();
	}

	return true;
}
#endif

#undef LOCTEXT_NAMESPACE
