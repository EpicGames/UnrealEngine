// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackedLevelActor/PackedLevelActorISMBuilder.h"

#if WITH_EDITOR

#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "PackedLevelActor/PackedLevelActor.h"


#include "Components/StaticMeshComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include "ISMPartition/ISMComponentBatcher.h"
#include "Templates/TypeHash.h"
#include "Misc/Crc.h"

FPackedLevelActorBuilderID FPackedLevelActorISMBuilder::BuilderID = 'ISMP';

FPackedLevelActorBuilderID FPackedLevelActorISMBuilder::GetID() const
{
	return BuilderID;
}

void FPackedLevelActorISMBuilder::GetPackClusters(FPackedLevelActorBuilderContext& InContext, AActor* InActor) const
{
	// Skip PackedLevelActors that are loaded for Packing as their Level Actors will get packed
	if (APackedLevelActor* PackedLevelActor = Cast<APackedLevelActor>(InActor))
	{
		if (PackedLevelActor->ShouldLoadForPacking())
		{
			return;
		}
	}

	TArray<UStaticMeshComponent*> StaticMeshComponents;
	InActor->GetComponents(StaticMeshComponents);

	for(UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		if (InContext.ShouldPackComponent(StaticMeshComponent))
		{
			FPackedLevelActorBuilderClusterID ClusterID(MakeUnique<FPackedLevelActorISMBuilderCluster>(GetID(), StaticMeshComponent));

			InContext.FindOrAddCluster(MoveTemp(ClusterID), StaticMeshComponent);
		}
	}
}

uint32 FPackedLevelActorISMBuilder::PackActors(FPackedLevelActorBuilderContext& InContext, const FPackedLevelActorBuilderClusterID& InClusterID, const TArray<UActorComponent*>& InComponents) const
{
	check(InClusterID.GetBuilderID() == GetID());
	APackedLevelActor* PackingActor = InContext.GetPackedLevelActor();
	FTransform ActorTransform = PackingActor->GetActorTransform();

	FPackedLevelActorISMBuilderCluster* ISMCluster = (FPackedLevelActorISMBuilderCluster*)InClusterID.GetData();
	check(ISMCluster);

	FISMComponentBatcher ISMComponentBatcher;
	TArray<UActorComponent*> Components(InComponents);
	
	// Sort by path name to generate stable hash
	Components.Sort([](const UActorComponent& LHS, const UActorComponent& RHS)
	{
		return LHS.GetPathName() < RHS.GetPathName();
	});
	
	ISMComponentBatcher.Append(Components, [&ActorTransform, &InContext](const FTransform& InTransform) { return InTransform.GetRelativeTransform(ActorTransform) * InContext.GetRelativePivotTransform() * ActorTransform; });

	TSubclassOf<UInstancedStaticMeshComponent> ComponentClass = UInstancedStaticMeshComponent::StaticClass();
	if (ISMComponentBatcher.GetNumInstances() > 1 && ISMCluster->ISMDescriptor.StaticMesh && !ISMCluster->ISMDescriptor.StaticMesh->IsNaniteEnabled())
	{
		// Use HISM for non-nanite when there is more than one transform (no use in using HISM cpu occlusion for a single instance)
		ComponentClass = UHierarchicalInstancedStaticMeshComponent::StaticClass();
	}

	UInstancedStaticMeshComponent* PackComponent = PackingActor->AddPackedComponent<UInstancedStaticMeshComponent>(ComponentClass);
	PackComponent->AttachToComponent(PackingActor->GetRootComponent(), FAttachmentTransformRules::SnapToTargetIncludingScale);

	// Initialize the ISM properties using the ISM descriptor
	ISMCluster->ISMDescriptor.InitComponent(PackComponent);

	// Initialize the ISM instances using the ISM batcher
	ISMComponentBatcher.InitComponent(PackComponent);

	PackComponent->RegisterComponent();

	uint32 Hash = FCrc::StrCrc32(*ComponentClass->GetPathName(), 0);
	Hash = HashCombine(Hash, HashCombine(InClusterID.GetHash(), ISMComponentBatcher.GetHash()));
	return Hash;
}

FPackedLevelActorISMBuilderCluster::FPackedLevelActorISMBuilderCluster(FPackedLevelActorBuilderID InBuilderID, UStaticMeshComponent* InComponent)
	: FPackedLevelActorBuilderCluster(InBuilderID)
{
	ISMDescriptor.InitFrom(InComponent, /** bInitBodyInstance= */ false);
	// ComponentClass will be determined based on StaticMesh nanite settings and number of instances and we don't want it to be part of the cluster hash
	ISMDescriptor.ComponentClass = nullptr;
	// Component descriptor should be considered hidden if original actor owner was.
	ISMDescriptor.bHiddenInGame |= InComponent->GetOwner()->IsHidden();
	ISMDescriptor.BodyInstance.CopyRuntimeBodyInstancePropertiesFrom(&InComponent->BodyInstance);
	ISMDescriptor.ComputeHash();
}

uint32 FPackedLevelActorISMBuilderCluster::ComputeHash() const
{
	return HashCombine(FPackedLevelActorBuilderCluster::ComputeHash(), ISMDescriptor.Hash);
}

bool FPackedLevelActorISMBuilderCluster::Equals(const FPackedLevelActorBuilderCluster& InOther) const
{
	if (!FPackedLevelActorBuilderCluster::Equals(InOther))
	{
		return false;
	}

	const FPackedLevelActorISMBuilderCluster& ISMOther = (const FPackedLevelActorISMBuilderCluster&)InOther;
	return ISMDescriptor == ISMOther.ISMDescriptor;
}

#endif
