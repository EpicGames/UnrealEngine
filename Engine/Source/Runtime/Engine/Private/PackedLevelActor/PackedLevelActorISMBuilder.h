// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PackedLevelActor/PackedLevelActorTypes.h"
#include "PackedLevelActor/IPackedLevelActorBuilder.h"
#include "ISMPartition/ISMComponentDescriptor.h"

class UStaticMeshComponent;

class FPackedLevelActorISMBuilder : public IPackedLevelActorBuilder
{
public:
	FPackedLevelActorISMBuilder(FPackedLevelActorBuilder& InOwner)
		: IPackedLevelActorBuilder(InOwner){}

	static FPackedLevelActorBuilderID BuilderID;

	virtual FPackedLevelActorBuilderID GetID() const override;
	virtual void GetPackClusters(FPackedLevelActorBuilderContext& InContext, AActor* InActor) const override;
	virtual uint32 PackActors(FPackedLevelActorBuilderContext& InContext, const FPackedLevelActorBuilderClusterID& InClusterID, const TArray<UActorComponent*>& InComponents) const override;
};

class FPackedLevelActorISMBuilderCluster : public FPackedLevelActorBuilderCluster
{
public:
	FPackedLevelActorISMBuilderCluster(FPackedLevelActorBuilderID InBuilderID, UStaticMeshComponent* InComponent);

	virtual bool Equals(const FPackedLevelActorBuilderCluster& InOther) const override;
	virtual uint32 ComputeHash() const override;

	FISMComponentDescriptor ISMDescriptor;
};

#endif