// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackedLevelActor/PackedLevelActorRecursiveBuilder.h"

#if WITH_EDITOR

#include "LevelInstance/LevelInstanceInterface.h"
#include "PackedLevelActor/PackedLevelActorBuilder.h"
#include "PackedLevelActor/PackedLevelActor.h"

#include "LevelInstance/LevelInstanceSubsystem.h"

#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "Engine/Brush.h"
#include "GameFramework/WorldSettings.h"

FPackedLevelActorBuilderID FPackedLevelActorRecursiveBuilder::BuilderID = 'RECP';

FPackedLevelActorBuilderID FPackedLevelActorRecursiveBuilder::GetID() const
{
	return BuilderID;
}

void FPackedLevelActorRecursiveBuilder::GetPackClusters(FPackedLevelActorBuilderContext& InContext, AActor* InActor) const
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActor))
	{
		// This Actor can be safely discarded without warning because it is a container
		InContext.DiscardActor(InActor);

		ULevelInstanceSubsystem* LevelInstanceSubsystem = LevelInstance->GetLevelInstanceSubsystem();
		check(LevelInstanceSubsystem);
		if (ULevel* Level = LevelInstanceSubsystem->GetLevelInstanceLevel(LevelInstance))
		{
			for (AActor* LevelActor : Level->Actors)
			{
				if (LevelActor)
				{
					if (LevelActor == Level->GetDefaultBrush() || 
						LevelActor == Level->GetWorldSettings() ||
						LevelActor->IsMainWorldOnly())
					{
						InContext.DiscardActor(LevelActor);
					}
					else
					{
						Owner.ClusterActor(InContext, LevelActor);
					}
				}
			}
		}
	}
}

#endif
