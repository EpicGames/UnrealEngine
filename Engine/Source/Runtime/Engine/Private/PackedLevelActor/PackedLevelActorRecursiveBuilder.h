// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PackedLevelActor/PackedLevelActorTypes.h"
#include "PackedLevelActor/IPackedLevelActorBuilder.h"

class ILevelInstanceInterface;

class FPackedLevelActorRecursiveBuilder : public IPackedLevelActorBuilder
{
public:
	FPackedLevelActorRecursiveBuilder(FPackedLevelActorBuilder& InOwner)
		: IPackedLevelActorBuilder(InOwner) {}

	static FPackedLevelActorBuilderID BuilderID;

	virtual FPackedLevelActorBuilderID GetID() const override;
	virtual void GetPackClusters(FPackedLevelActorBuilderContext& InContext, AActor* InActor) const override;
	virtual uint32 PackActors(FPackedLevelActorBuilderContext& InContext, const FPackedLevelActorBuilderClusterID& InClusterID, const TArray<UActorComponent*>& InComponents) const override { return 0; }
};

#endif