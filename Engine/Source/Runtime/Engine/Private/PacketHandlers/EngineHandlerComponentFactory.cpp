// Copyright Epic Games, Inc. All Rights Reserved.

#include "PacketHandlers/EngineHandlerComponentFactory.h"
#include "PacketHandlers/StatelessConnectHandlerComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineHandlerComponentFactory)


/**
 * UEngineHandlerComponentFactor
 */
UEngineHandlerComponentFactory::UEngineHandlerComponentFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<HandlerComponent> UEngineHandlerComponentFactory::CreateComponentInstance(FString& Options)
{
	if (Options == TEXT("StatelessConnectHandlerComponent"))
	{
		return MakeShareable(new StatelessConnectHandlerComponent);
	}

	return nullptr;
}

