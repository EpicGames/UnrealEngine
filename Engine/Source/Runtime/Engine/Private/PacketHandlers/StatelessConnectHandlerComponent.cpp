// Copyright Epic Games, Inc. All Rights Reserved.

#include "PacketHandlers/StatelessConnectHandlerComponent.h"
#include "Serialization/MemoryWriter.h"
#include "EngineStats.h"
#include "Misc/SecureHash.h"
#include "Engine/NetConnection.h"
#include "Net/Core/Misc/PacketAudit.h"
#include "Misc/ConfigCacheIni.h"
#include "Stats/StatsTrace.h"
#include <limits> // IWYU pragma: keep


DEFINE_LOG_CATEGORY(LogHandshake);


/**
 * Purpose:
 *
 * UDP connections are vulnerable to various types of DoS attacks, particularly spoofing the IP address in UDP packets,
 * and to protect against this a handshake is needed to verify that the IP is really owned by the client.
 *
 * This handshake can be implemented in two ways:
 *	Stateful:	Here the server stores connection state information (i.e. maintains a UNetConnection) while the handshake is underway,
 *				allowing spoofed packets to allocate server memory space, prior to handshake verification.
 *
 *	Stateless:	Here the server does not store any connection state information, until the handshake completes,
 *				preventing spoofed packets from allocating server memory space until after the handshake.
 *
 * Stateful handshakes are vulnerable to DoS attacks through server memory usage, whereas stateless handshakes are not,
 * so this implementation uses stateless handshakes.
 *
 *
 *
 * Handshake Process/Protocol:
 * --------------------------
 *
 * The protocol for the handshake involves the client sending an initial packet to the server,
 * and the server responding with a unique 'Cookie' value, which the client has to respond with.
 *
 * Client - Initial Connect:
 *
 * [?:MagicHeader][2:SessionID][3:ClientID][HandshakeBit][RestartHandshakeBit]
 * [8:MinVersion][8:CurVersion][8:HandshakePacketType][8:SentPacketCount][32:NetworkVersion]
 * [16:NetworkFeatures][SecretIdBit][28:PacketSizeFiller][AlignPad][?:RandomData]
 *													--->
 *															Server - Stateless Handshake Challenge:
 *
 *															[?:MagicHeader][2:SessionID][3:ClientID][HandshakeBit][RestartHandshakeBit]
 *															[8:MinVersion][8:CurVersion][8:HandshakePacketType][8:SentPacketCount][32:NetworkVersion]
 *															[16:NetworkFeatures][SecretIdBit][8:Timestamp][20:Cookie][AlignPad][?:RandomData]
 *													<---
 * Client - Stateless Challenge Response:
 *
 * [?:MagicHeader][2:SessionID][3:ClientID][HandshakeBit][RestartHandshakeBit]
 * [8:MinVersion][8:CurVersion][8:HandshakePacketType][8:SentPacketCount][32:NetworkVersion]
 * [16:NetworkFeatures][SecretIdBit][8:Timestamp][20:Cookie][AlignPad][?:RandomData]
 *													--->
 *															Server:
 *															Ignore, or create UNetConnection.
 *
 *															Server - Stateless Handshake Ack:
 *
 *															[?:MagicHeader][2:SessionID][3:ClientID][HandshakeBit][RestartHandshakeBit]
 *															[8:MinVersion][8:CurVersion][8:HandshakePacketType][8:SentPacketCount][32:NetworkVersion]
 *															[16:NetworkFeatures][SecretIdBit][8:Timestamp][20:Cookie][AlignPad][?:RandomData]
 *													<---
 * Client:
 * Handshake Complete.
 *
 *
 * Restart Handshake Process/Protocol:
 * ----------------------------------
 * The Restart Handshake process is triggered by receiving a (possibly spoofed) non-handshake packet from an unknown IP,
 * so the protocol has been crafted so the server sends only a minimal (1 byte) response, to minimize DRDoS reflection amplification.
 *
 *															Server - Restart Handshake Request:
 *
 *															[?:MagicHeader][2:SessionID][3:ClientID][HandshakeBit][RestartHandshakeBit]
 *															[8:HandshakePacketType][8:SentPacketCount][32:NetworkVersion][16:NetworkFeatures]
 *															[AlignPad][?:RandomData]
 *													<--
 * Client -  Initial Connect (as above)
 *													-->
 *															Server -  Stateless Handshake Challenge (as above)
 *													<--
 * Client - Stateless Challenge Response + Original Cookie:
 *
 * [?:MagicHeader][2:SessionID][3:ClientID][HandshakeBit][RestartHandshakeBit]
 * [8:MinVersion][8:CurVersion][8:HandshakePacketType][8:SentPacketCount][32:NetworkVersion]
 * [16:NetworkFeatures][SecretIdBit][8:Timestamp][20:Cookie][20:OriginalCookie]
 * [AlignPad][?:RandomData]
 *													-->
 *															Server:
 *															Ignore, or restore UNetConnection.
 *
 *															Server - Stateless Handshake Ack (as above)
 *													<--
 * Client:
 * Handshake Complete. Connection restored.
 *
 *
 *
 *	- MagicHeader:			An optional static/predefined header, between 0-32 bits in size. Serves no purpose for the handshake code.
 *	- SessionID:			Session id incremented serverside every non-seamless server travel, to prevent non-ephemeral old/new-session crosstalk.
 *	- ClientID:				Connection id incremented clientside every connection per-NetDriver, to prevent non-ephemeral old/new-connection crosstalk.
 *	- HandshakeBit:			Bit signifying whether a packet is a handshake packet. Applied to all game packets.
 *	- SecretIdBit:			For handshake packets, specifies which HandshakeSecret array was used to generate Cookie.
 *	- RestartHandshakeBit:  Sent by the server when it detects normal game traffic from an unknown IP/port combination.
 *	- MinVersion:			The minimum handshake protocol version supported by the remote side
 *	- CurVersion:			The currently active protocol version used by the remote side (determines received packet format)
 *	- HandshakePacketType:	Number indicating the type of handshake packet, based on EHandshakePacketType
 *	- SentPacketCount:		The number of handshake packets sent - for packet-analysis/debugging purposes
 *	- NetworkVersion:		The Network CL version, according to FNetworkVersion::GetLocalNetworkVersion
 *	- NetworkFeatures		The runtime Network Features, according to UNetDriver::GetNetworkRuntimeFeatures
 *	- Timestamp:			Server timestamp, from the moment the handshake challenge was sent.
 *	- Cookie:				Cookie generated by the server, which the client must reply with.
 *	- AlignPad:				Handshake packets and PacketHandler's in general, require complex padding of packets. See ParseHandshakePacket.
 *	- RandomData:			Data of random length/content appended to handshake packets, to work around potential faulty ISP packet filtering
 *
 *	- PacketSizeFiller:		Pads the client packet with blank information, so that the initial client packet,
 *							is the same size as the server response packet.
 *
 *							The server will ignore initial packets below/above this length. This prevents hijacking of game servers,
 *							for use in 'DRDoS' reflection amplification attacks.
 *
 *
 *
 * Game Protocol Changes:
 *
 * Every packet (game and handshake) starts with the MagicHeader bits (if set), then 2 SessionID bits and 3 ClientID bits,
 * and finally HandshakeBit (which is 0 for game packets, going through normal PacketHandler/NetConnection protocol processing,
 * and 1 for handshake packets, going through the separate protocol documented above).
 *
 *
 *
 * HandshakeSecret/Cookie:
 *
 * The Cookie value is used to uniquely identify and perform a handshake with a connecting client,
 * but only the server can generate and recognize valid cookies, and the server must do this without storing any connection state data.
 *
 * To do this, the server stores 2 large random HandshakeSecret values, that only the server knows,
 * and combines that with data unique to the client connection (IP and Port), plus a server Timestamp, as part of generating the cookie.
 *
 * This data is then combined using a special HMAC hashing function, used specifically for authentication, to generate the cookie:
 *	Cookie = HMAC(HandshakeSecret, Timestamp + Client IP + Client Port)
 *
 * When the client responds to the handshake challenge, sending back TimeStamp and the Cookie,
 * the server will be able to collect all the remaining information it needs from the client packet (Client IP, Client Port),
 * plus the HandshakeSecret, to be able to regenerate the Cookie from scratch, and verify that the regenerated cookie,
 * is the same as the one the client sent back.
 *
 * No connection state data needs to be stored in order to do this, so this allows a stateless handshake.
 *
 *
 * In addition, HandshakeSecret updates every 15 + Rand(0,5) seconds (with previous value being stored/accepted for same amount of time)
 * in order to limit packet replay attacks, where a valid cookie can be reused multiple times.
 *
 * Checks on the handshake Timestamp, especially when combined with 5 second variance above, compliment this in limiting replay attacks.
 *
 *
 *
 * IP/Port Switching:
 *
 * Rarely, some routers have a bug where they suddenly change the port they send traffic from. The consequence of this is the server starts
 * receiving traffic from a new IP/port combination from an already connected player. When this happens, it tells the client via the
 * RestartHandshakeBit to restart the handshake process.
 *
 * The client carries on with the handshake as normal, but when completing the handshake, the client also sends the cookie it previously connected with.
 * The server looks up the NetConnection associated with that cookie, and then updates the address for the connection.
 *
 *
 *
 * SessionID/ClientID and non-ephemeral sockets
 *
 * The packet protocol has a reliance on IP packet ephemeral ports (the random client-specified source port) to differentiate client connections,
 * but not all socket subsystems provide something which serves this role - some socket subsystems only provide a static address without a port,
 * where the address remains indistinguishable between old/new client connections, e.g. when performing a non-seamless travel between levels.
 *
 * The SessionID value solves this for the case of non-seamless travel, by specifying an incrementing server-authoritative ID for the game session
 * (which changes upon non-seamless travel).
 *
 * The ClientID solves this for the case of clients reconnecting to a server they are currently connected to,
 * or which their old connection is pending timeout from (e.g. after a crash or other fault requiring a reconnect),
 * by specifying an incrementing client-authoritative ID for the connection (per-NetDriver - so e.g. Game and Beacon drivers increment separately).
 *
 *
 * This is not a complete solution, however - multiple clients from the same address will not work, presently.
 * ClientID has enough bits to implement this in the future, while keeping net compatibility, but is non-trivial and not guaranteed to be added.
 *
 * If this is to be added though, it will require adjustments to the serverside NetDriver receive code, to set the ClientID as the address port,
 * and will require adjustments to the clientside setting of ClientID, to perform inter-process communication when picking the ClientID,
 * so that the value is unique for every NetConnection across every game process, connecting to the same server address + port.
 *
 *
 * Also, if the server and client are using the same *Engine.ini file, the last process to close will clobber the
 * GlobalNetTravelCount/CachedClientID increments from the other process, which are used for SessionID/ClientID.
 */


/**
 * Debug Defines
 */

// Enables packetloss testing, which should be tested by connecting/reconnecting to a server a couple dozen times.
// Every such connection attempt should eventually succeed/recover automatically - if any fail, something's broken.
#define PACKETLOSS_TEST 0


/**
 * Defines
 */

#define BASE_PACKET_SIZE_BITS					82
#define HANDSHAKE_PACKET_SIZE_BITS				(BASE_PACKET_SIZE_BITS + 225)
#define RESTART_HANDSHAKE_PACKET_SIZE_BITS		BASE_PACKET_SIZE_BITS
#define RESTART_RESPONSE_SIZE_BITS				(BASE_PACKET_SIZE_BITS + 385)
#define VERSION_UPGRADE_SIZE_BITS				BASE_PACKET_SIZE_BITS

// The number of seconds between secret value updates, and the random variance applied to this
#define SECRET_UPDATE_TIME			15.f
#define SECRET_UPDATE_TIME_VARIANCE	5.f

// The maximum allowed lifetime (in seconds) of any one handshake cookie
#define MAX_COOKIE_LIFETIME			((SECRET_UPDATE_TIME + SECRET_UPDATE_TIME_VARIANCE) * (float)SECRET_COUNT)

// The minimum amount of possible time a cookie may exist (for calculating when the clientside should timeout a challenge response)
#define MIN_COOKIE_LIFETIME			SECRET_UPDATE_TIME


/**
 * CVars
 */

TAutoConsoleVariable<FString> CVarNetMagicHeader(
	TEXT("net.MagicHeader"),
	TEXT(""),
	TEXT("String representing binary bits which are prepended to every packet sent by the game. Max length: 32 bits."));


namespace UE::Net
{
	static float HandshakeResendInterval = 1.f;

	static FAutoConsoleVariableRef CVarNetHandshakeResendInterval(
		TEXT("net.HandshakeResendInterval"),
		HandshakeResendInterval,
		TEXT("The delay between resending handshake packets which we have not received a response for."));

	/** The minimum supported stateless handshake protocol version */
#ifdef HANDSHAKE_MIN_VERSION_OVERRIDE
	static int32 MinSupportedHandshakeVersion = HANDSHAKE_MIN_VERSION_OVERRIDE;
#else
	static int32 MinSupportedHandshakeVersion = static_cast<uint8>(EHandshakeVersion::SessionClientId);
#endif

	/** The current compile-time handshake version */
#ifdef HANDSHAKE_VERSION_OVERRIDE
	static int32 CurrentHandshakeVersion = HANDSHAKE_VERSION_OVERRIDE;
#else
	static int32 CurrentHandshakeVersion = static_cast<uint8>(EHandshakeVersion::Latest);
#endif

	static FAutoConsoleVariableRef CVarNetMinHandshakeVersion(
		TEXT("net.MinHandshakeVersion"),
		MinSupportedHandshakeVersion,
		TEXT("The minimum supported stateless handshake protocol version (numeric)."));

	static FAutoConsoleVariableRef CVarNetCurrentHandshakeVersion(
		TEXT("net.CurrentHandshakeVersion"),
		CurrentHandshakeVersion,
		TEXT("The current supported stateless handshake protocol version (numeric)"));

	static TAutoConsoleVariable<int32> CVarNetDoHandshakeVersionFallback(
		TEXT("net.DoHandshakeVersionFallback"),
		0,
		TEXT("Whether or not to (clientside) perform randomized falling-back to previous versions of the handshake protocol, upon failure."));

	static int32 GHandshakeEnforceNetworkCLVersion = 0;

	static FAutoConsoleVariableRef CVarNetHandshakeEnforceNetworkCLVersion(
		TEXT("net.HandshakeEnforceNetworkCLVersion"),
		GHandshakeEnforceNetworkCLVersion,
		TEXT("Whether or not the stateless handshake should enforce the Network CL version, instead of the higher level netcode."));

	static int32 GVerifyNetSessionID = 1;

	static FAutoConsoleVariableRef CVarNetVerifyNetSessionID(
		TEXT("net.VerifyNetSessionID"),
		GVerifyNetSessionID,
		TEXT("Whether or not verification of the packet SessionID value is performed."));

	static int32 GVerifyNetClientID = 1;

	static FAutoConsoleVariableRef CVarNetVerifyNetClientID(
		TEXT("net.VerifyNetClientID"),
		GVerifyNetClientID,
		TEXT("Whether or not verification of the packet ClientID value is performed."));

	static int32 GVerifyMagicHeader = 0;

	static FAutoConsoleVariableRef CVarNetVerifyMagicHeader(
		TEXT("net.VerifyMagicHeader"),
		GVerifyMagicHeader,
		TEXT("Whether or not verification of the magic header is performed, prior to processing a packet. ")
		TEXT("Disable if transitioning to a new magic header, while wishing to continue supporting the old header for a time."));




	/** The base amount of random data to add to handshake packets */
	static constexpr int32 BaseRandomDataLengthBytes		= 16;

	/** The amount by which the length of random data should randomly vary */
	static constexpr int32 RandomDataLengthVarianceBytes	= 8;

	/** HANDSHAKE_PACKET_SIZE_BITS for EHandshakeVersion::Original */
	static constexpr int32 OriginalHandshakePacketSizeBits = 227;

	/** RESTART_HANDSHAKE_PACKET_SIZE_BITS for EHandshakeVersion::Original */
	static constexpr int32 OriginalRestartHandshakePacketSizeBits = 2;

	/** RESTART_RESPONSE_SIZE_BITS for EHandshakeVersion::Original */
	static constexpr int32 OriginalRestartResponseSizeBits = 387;

	/** HANDSHAKE_PACKET_SIZE_BITS for EHandshakeVersion::Randomized */
	static constexpr int32 VerRandomizedHandshakePacketSizeBits = 259;

	/** RESTART_HANDSHAKE_PACKET_SIZE_BITS for EHandshakeVersion::Randomized */
	static constexpr int32 VerRandomizedRestartHandshakePacketSizeBits = 34;

	/** RESTART_RESPONSE_SIZE_BITS for EHandshakeVersion::Randomized */
	static constexpr int32 VerRandomizedRestartResponseSizeBits = 419;

	/** Initial value of LastClientSendTimestamp, indicating the client hasn't sent anything */
	static constexpr double UninitializedLastClientSendTimestamp = -1.0;

	const TCHAR* LexToString(EHandshakePacketType PacketType)
	{
		static_assert(EHandshakePacketType::Last == EHandshakePacketType::VersionUpgrade &&
						static_cast<uint8>(EHandshakePacketType::VersionUpgrade) == 6, "Add new EHandshakePacketType entries to LexToString.");

		switch (PacketType)
		{
		case EHandshakePacketType::InitialPacket:		return TEXT("InitialPacket");
		case EHandshakePacketType::Challenge:			return TEXT("Challenge");
		case EHandshakePacketType::Response:			return TEXT("Response");
		case EHandshakePacketType::Ack:					return TEXT("Ack");
		case EHandshakePacketType::RestartHandshake:	return TEXT("RestartHandshake");
		case EHandshakePacketType::RestartResponse:		return TEXT("RestartResponse");
		case EHandshakePacketType::VersionUpgrade:		return TEXT("VersionUpgrade");
		}

		return TEXT("Unknown");
	}


	/**
	 * FStatelessHandshakeFailureInfo
	 */

	FStatelessHandshakeFailureInfo::FStatelessHandshakeFailureInfo()
		: RemoteNetworkFeatures(EEngineNetworkRuntimeFeatures::None)
	{
	}
}


/**
 * StatelessConnectHandlerComponent
 */

StatelessConnectHandlerComponent::FCommonSendToClientParams::FCommonSendToClientParams(const TSharedPtr<const FInternetAddr>& InClientAddress,
																						EHandshakeVersion InHandshakeVersion, uint32 InClientID)
	: ClientAddress(InClientAddress)
	, HandshakeVersion(InHandshakeVersion)
	, ClientID(InClientID)
{
}

StatelessConnectHandlerComponent::FParsedHandshakeData::FParsedHandshakeData()
	: RemoteNetworkFeatures(EEngineNetworkRuntimeFeatures::None)
{
}

StatelessConnectHandlerComponent::StatelessConnectHandlerComponent()
	: HandlerComponent(FName("StatelessConnectHandlerComponent"))
	, Driver(nullptr)
	, HandshakeSecret()
	, ActiveSecret(255)
	, LastSecretUpdateTimestamp(0.0)
	, LastChallengeSuccessAddress(nullptr)
	, LastServerSequence(0)
	, LastClientSequence(0)
	, MinClientHandshakeVersion(static_cast<EHandshakeVersion>(UE::Net::CurrentHandshakeVersion))
	, LastClientSendTimestamp(UE::Net::UninitializedLastClientSendTimestamp)
	, LastChallengeTimestamp(0.0)
	, LastRestartPacketTimestamp(0.0)
	, LastSecretId(0)
	, LastTimestamp(0.0)
	, LastCookie()
	, bRestartedHandshake(false)
	, AuthorisedCookie()
	, MagicHeader()
	, LastRemoteHandshakeVersion(static_cast<EHandshakeVersion>(UE::Net::CurrentHandshakeVersion))
{
	SetActive(true);

	bRequiresHandshake = true;

	FString MagicHeaderStr = CVarNetMagicHeader.GetValueOnAnyThread();

	if (!MagicHeaderStr.IsEmpty())
	{
		int32 HeaderStrLen = MagicHeaderStr.Len();

		if (HeaderStrLen <= 32)
		{
			bool bValidBinaryStr = true;

			for (int32 i=0; i<MagicHeaderStr.Len() && bValidBinaryStr; i++)
			{
				const TCHAR& CurChar = MagicHeaderStr[i];

				bValidBinaryStr = CurChar == '0' || CurChar == '1';

				MagicHeader.Add(CurChar != '0');
			}

			if (bValidBinaryStr)
			{
				MagicHeaderUint = *static_cast<uint32*>(MagicHeader.GetData());
			}
			else
			{
				UE_LOGF(LogHandshake, Error, "CVar net.MagicHeader must be a binary string, containing only 1's and 0's, e.g.: 00010101. Current string: %ls", *MagicHeaderStr);

				MagicHeader.Empty();
			}
		}
		else
		{
			UE_LOGF(LogHandshake, Error, "CVar net.MagicHeader is too long (%i), maximum size is 32 bits: %ls", MagicHeaderStr.Len(), *MagicHeaderStr);
		}
	}
}

void StatelessConnectHandlerComponent::CountBytes(FArchive& Ar) const
{
	HandlerComponent::CountBytes(Ar);

	const SIZE_T SizeOfThis = sizeof(*this) - sizeof(HandlerComponent);

	for (int32 i = 0; i < SECRET_COUNT; ++i)
	{
		HandshakeSecret[i].CountBytes(Ar);
	}
}

void StatelessConnectHandlerComponent::NotifyHandshakeBegin()
{
	using namespace UE::Net;

	SendInitialPacket(static_cast<EHandshakeVersion>(CurrentHandshakeVersion));
}

void StatelessConnectHandlerComponent::SendInitialPacket(EHandshakeVersion HandshakeVersion)
{
	using namespace UE::Net;

	if (Handler->Mode == UE::Handler::Mode::Client)
	{
		UNetConnection* ServerConn = (Driver != nullptr ? ToRawPtr(Driver->ServerConnection) : nullptr);

		if (ServerConn != nullptr)
		{
			const int32 AdjustedSize = GetAdjustedSizeBits(HANDSHAKE_PACKET_SIZE_BITS, HandshakeVersion);
			FBitWriter InitialPacket(AdjustedSize + (BaseRandomDataLengthBytes * 8) + 1 /* Termination bit */);

			BeginHandshakePacket(InitialPacket, EHandshakePacketType::InitialPacket, HandshakeVersion, SentHandshakePacketCount, CachedClientID,
									(bRestartedHandshake ? EHandshakePacketModifier::RestartHandshake : EHandshakePacketModifier::None));

			uint8 SecretIdPad = 0;
			uint8 PacketSizeFiller[28];

			InitialPacket.WriteBit(SecretIdPad);

			FMemory::Memzero(PacketSizeFiller, UE_ARRAY_COUNT(PacketSizeFiller));
			InitialPacket.Serialize(PacketSizeFiller, UE_ARRAY_COUNT(PacketSizeFiller));

			SendToServer(HandshakeVersion, EHandshakePacketType::InitialPacket, InitialPacket);
		}
		else
		{
			UE_LOGF(LogHandshake, Error, "Tried to send handshake connect packet without a server connection.");
		}
	}
}

void StatelessConnectHandlerComponent::SendConnectChallenge(FCommonSendToClientParams CommonParams, uint8 ClientSentHandshakePacketCount)
{
	using namespace UE::Net;

	if (Driver != nullptr)
	{
		const int32 AdjustedSize = GetAdjustedSizeBits(HANDSHAKE_PACKET_SIZE_BITS, CommonParams.HandshakeVersion);
		FBitWriter ChallengePacket(AdjustedSize + (BaseRandomDataLengthBytes * 8) + 1 /* Termination bit */);

		BeginHandshakePacket(ChallengePacket, EHandshakePacketType::Challenge, CommonParams.HandshakeVersion, ClientSentHandshakePacketCount,
								CommonParams.ClientID);

		double Timestamp = Driver->GetElapsedTime();
		uint8 Cookie[COOKIE_BYTE_SIZE];

		GenerateCookie(CommonParams.ClientAddress, ActiveSecret, Timestamp, Cookie);

		ChallengePacket.WriteBit(ActiveSecret);

		ChallengePacket << Timestamp;

		ChallengePacket.Serialize(Cookie, UE_ARRAY_COUNT(Cookie));

#if !UE_BUILD_SHIPPING
		FDDoSDetection* DDoS = Handler->GetDDoS();

		UE_CLOGF((DDoS == nullptr || !DDoS->CheckLogRestrictions()), LogHandshake, Log,
				"SendConnectChallenge. Timestamp: %f, Cookie: %ls", Timestamp, *FString::FromBlob(Cookie, UE_ARRAY_COUNT(Cookie)));
#endif

		SendToClient(CommonParams, EHandshakePacketType::Challenge, ChallengePacket);
	}
}

void StatelessConnectHandlerComponent::SendChallengeResponse(EHandshakeVersion HandshakeVersion, uint8 InSecretId, double InTimestamp,
																uint8 InCookie[COOKIE_BYTE_SIZE])
{
	using namespace UE::Net;

	UNetConnection* ServerConn = (Driver != nullptr ? ToRawPtr(Driver->ServerConnection) : nullptr);

	if (ServerConn != nullptr)
	{
		const int32 AdjustedSize = GetAdjustedSizeBits((bRestartedHandshake ? RESTART_RESPONSE_SIZE_BITS : HANDSHAKE_PACKET_SIZE_BITS),
														HandshakeVersion);
		FBitWriter ResponsePacket(AdjustedSize + (BaseRandomDataLengthBytes * 8) + 1 /* Termination bit */);
		EHandshakePacketType HandshakePacketType = bRestartedHandshake ? EHandshakePacketType::RestartResponse : EHandshakePacketType::Response;

		BeginHandshakePacket(ResponsePacket, HandshakePacketType, HandshakeVersion, SentHandshakePacketCount, CachedClientID,
								(bRestartedHandshake ? EHandshakePacketModifier::RestartHandshake : EHandshakePacketModifier::None));

		ResponsePacket.WriteBit(InSecretId);

		ResponsePacket << InTimestamp;
		ResponsePacket.Serialize(InCookie, COOKIE_BYTE_SIZE);

		if (bRestartedHandshake)
		{
			ResponsePacket.Serialize(AuthorisedCookie, COOKIE_BYTE_SIZE);
		}

#if !UE_BUILD_SHIPPING
		UE_LOGF(LogHandshake, Log, "SendChallengeResponse. Timestamp: %f, Cookie: %ls", InTimestamp,
				*FString::FromBlob(InCookie, COOKIE_BYTE_SIZE));
#endif

		SendToServer(HandshakeVersion, HandshakePacketType, ResponsePacket);


		int16* CurSequence = (int16*)InCookie;

		LastSecretId = InSecretId;
		LastTimestamp = InTimestamp;
		LastServerSequence = *CurSequence & (MAX_PACKETID - 1);
		LastClientSequence = *(CurSequence + 1) & (MAX_PACKETID - 1);
		LastRemoteHandshakeVersion = HandshakeVersion;

		FMemory::Memcpy(LastCookie, InCookie, UE_ARRAY_COUNT(LastCookie));
	}
	else
	{
		UE_LOGF(LogHandshake, Error, "Tried to send handshake response packet without a server connection.");
	}
}

void StatelessConnectHandlerComponent::SendChallengeAck(FCommonSendToClientParams CommonParams, uint8 ClientSentHandshakePacketCount,
														uint8 InCookie[COOKIE_BYTE_SIZE])
{
	using namespace UE::Net;

	if (Driver != nullptr)
	{
		const int32 AdjustedSize = GetAdjustedSizeBits(HANDSHAKE_PACKET_SIZE_BITS, CommonParams.HandshakeVersion);
		FBitWriter AckPacket(AdjustedSize + (BaseRandomDataLengthBytes * 8) + 1 /* Termination bit */);

		BeginHandshakePacket(AckPacket, EHandshakePacketType::Ack, CommonParams.HandshakeVersion, ClientSentHandshakePacketCount,
								CommonParams.ClientID);


		double Timestamp  = -1.0;
		uint8 ActiveSecret_Unused = 1;

		AckPacket.WriteBit(ActiveSecret_Unused);

		AckPacket << Timestamp;
		AckPacket.Serialize(InCookie, COOKIE_BYTE_SIZE);

#if !UE_BUILD_SHIPPING
		UE_LOGF(LogHandshake, Log, "SendChallengeAck. InCookie: %ls", *FString::FromBlob(InCookie, COOKIE_BYTE_SIZE));
#endif

		SendToClient(CommonParams, EHandshakePacketType::Ack, AckPacket);
	}
}

void StatelessConnectHandlerComponent::SendRestartHandshakeRequest(FCommonSendToClientParams CommonParams)
{
	using namespace UE::Net;

	if (Driver != nullptr)
	{
		const int32 AdjustedSize = GetAdjustedSizeBits(RESTART_HANDSHAKE_PACKET_SIZE_BITS, CommonParams.HandshakeVersion);
		FBitWriter RestartPacket(AdjustedSize + (BaseRandomDataLengthBytes * 8) + 1 /* Termination bit */);

		BeginHandshakePacket(RestartPacket, EHandshakePacketType::RestartHandshake, CommonParams.HandshakeVersion, SentHandshakePacketCount,
								CommonParams.ClientID, EHandshakePacketModifier::RestartHandshake);

#if !UE_BUILD_SHIPPING
		FDDoSDetection* DDoS = Handler->GetDDoS();

		UE_CLOGF((DDoS == nullptr || !DDoS->CheckLogRestrictions()), LogHandshake, Verbose, "SendRestartHandshakeRequest.");
#endif

		SendToClient(CommonParams, EHandshakePacketType::RestartHandshake, RestartPacket);
	}
}

void StatelessConnectHandlerComponent::SendVersionUpgradeMessage(FCommonSendToClientParams CommonParams)
{
	using namespace UE::Net;

	if (Driver != nullptr)
	{
		const int32 AdjustedSize = GetAdjustedSizeBits(VERSION_UPGRADE_SIZE_BITS, CommonParams.HandshakeVersion);
		FBitWriter UpgradePacket(AdjustedSize + (BaseRandomDataLengthBytes * 8) + 1 /* Termination bit */);

		BeginHandshakePacket(UpgradePacket, EHandshakePacketType::VersionUpgrade, CommonParams.HandshakeVersion, SentHandshakePacketCount,
								CommonParams.ClientID);

		SendToClient(CommonParams, EHandshakePacketType::VersionUpgrade, UpgradePacket);
	}
}

void StatelessConnectHandlerComponent::BeginHandshakePacket(FBitWriter& HandshakePacket, EHandshakePacketType HandshakePacketType,
															EHandshakeVersion HandshakeVersion, uint8 SentHandshakePacketCount_LocalOrRemote,
															uint32 ClientID,
															EHandshakePacketModifier HandshakePacketModifier/*=EHandshakePacketModifier::None*/)
{
	using namespace UE::Net;

	uint8 bHandshakePacket = 1;
	uint8 bRestartHandshake = EnumHasAnyFlags(HandshakePacketModifier, EHandshakePacketModifier::RestartHandshake) ? 1 : 0;

	if (MagicHeader.Num() > 0)
	{
		HandshakePacket.SerializeBits(MagicHeader.GetData(), MagicHeader.Num());
	}

	if (HandshakeVersion >= EHandshakeVersion::SessionClientId)
	{
		HandshakePacket.SerializeBits(&CachedGlobalNetTravelCount, SessionIDSizeBits);
		HandshakePacket.SerializeBits(&ClientID, ClientIDSizeBits);
	}

	HandshakePacket.WriteBit(bHandshakePacket);
	HandshakePacket.WriteBit(bRestartHandshake);

	if (HandshakeVersion >= EHandshakeVersion::Randomized)
	{
		uint8 MinVersion = MinSupportedHandshakeVersion;
		uint8 CurVersion = static_cast<uint8>(HandshakeVersion);
		uint8 HandshakePacketTypeUint = static_cast<uint8>(HandshakePacketType);

		HandshakePacket << MinVersion;
		HandshakePacket << CurVersion;
		HandshakePacket << HandshakePacketTypeUint;
		HandshakePacket << SentHandshakePacketCount_LocalOrRemote;
	}

	if (HandshakeVersion >= EHandshakeVersion::NetCLVersion)
	{
		uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();
		EEngineNetworkRuntimeFeatures LocalNetworkFeatures = (Driver != nullptr ? Driver->GetNetworkRuntimeFeatures() :
																EEngineNetworkRuntimeFeatures::None);

		static_assert(sizeof(EEngineNetworkRuntimeFeatures) == 2, "If EEngineNetworkRuntimeFeatures size changes, adjust BASE_PACKET_SIZE_BITS");

		HandshakePacket << LocalNetworkVersion;
		HandshakePacket << LocalNetworkFeatures;
	}
}

void StatelessConnectHandlerComponent::SendToServer(EHandshakeVersion HandshakeVersion, EHandshakePacketType PacketType, FBitWriter& Packet)
{
	if (UNetConnection* ServerConn = (Driver != nullptr ? Driver->ServerConnection.Get() : nullptr))
	{
		CapHandshakePacket(Packet, HandshakeVersion);


		// Disable PacketHandler parsing, and send the raw packet
		Handler->SetRawSend(true);

#if !UE_BUILD_SHIPPING && PACKETLOSS_TEST
		bool bRandFail = FMath::RandBool();

		if (bRandFail)
		{
			UE_LOGF(LogHandshake, Log, "Triggering random '%ls' packet fail.", ToCStr(LexToString(PacketType)));
		}

		if (!bRandFail)
#endif
		{
			if (Driver->IsNetResourceValid())
			{
				FOutPacketTraits Traits;

				Driver->ServerConnection->LowLevelSend(Packet.GetData(), Packet.GetNumBits(), Traits);
				LastClientSendTimestamp = Driver->GetElapsedTime();
			}
			else
			{
				// ignore resent interval, send initial packet on "next frame" as the socket will be ready by then
				LastClientSendTimestamp = Driver->GetElapsedTime() - UE::Net::HandshakeResendInterval + 0.01f;
			}
		}

		Handler->SetRawSend(false);
	}
}

void StatelessConnectHandlerComponent::SendToClient(FCommonSendToClientParams CommonParams, EHandshakePacketType PacketType, FBitWriter& Packet)
{
	if (Driver != nullptr)
	{
		CapHandshakePacket(Packet, CommonParams.HandshakeVersion);

		
		// Disable PacketHandler parsing, and send the raw packet
		PacketHandler* ConnectionlessHandler = Driver->ConnectionlessHandler.Get();

		if (ConnectionlessHandler != nullptr)
		{
			ConnectionlessHandler->SetRawSend(true);
		}

#if !UE_BUILD_SHIPPING && PACKETLOSS_TEST
		bool bRandFail = FMath::RandBool();

		if (bRandFail)
		{
			UE_LOGF(LogHandshake, Log, "Triggering random '%ls' packet fail.", ToCStr(LexToString(PacketType)));
		}

		if (!bRandFail)
#endif
		{
			if (Driver->IsNetResourceValid())
			{
				FOutPacketTraits Traits;

				Driver->LowLevelSend(CommonParams.ClientAddress, Packet.GetData(), Packet.GetNumBits(), Traits);
			}
		}


		if (ConnectionlessHandler != nullptr)
		{
			ConnectionlessHandler->SetRawSend(false);
		}
	}
}

void StatelessConnectHandlerComponent::CapHandshakePacket(FBitWriter& HandshakePacket, EHandshakeVersion HandshakeVersion)
{
	using namespace UE::Net;

	uint32 NumBits = HandshakePacket.GetNumBits() - GetAdjustedSizeBits(0, HandshakeVersion);

#if !UE_BUILD_SHIPPING
	if (HandshakeVersion == EHandshakeVersion::Original)
	{
		check(NumBits == OriginalHandshakePacketSizeBits || NumBits == OriginalRestartHandshakePacketSizeBits ||
				NumBits == OriginalRestartResponseSizeBits);
	}
	else if (HandshakeVersion == EHandshakeVersion::Randomized)
	{
		check(NumBits == VerRandomizedHandshakePacketSizeBits || NumBits == VerRandomizedRestartHandshakePacketSizeBits ||
				NumBits == VerRandomizedRestartResponseSizeBits);
	}
	else
	{
		check(NumBits == HANDSHAKE_PACKET_SIZE_BITS || NumBits == RESTART_HANDSHAKE_PACKET_SIZE_BITS || NumBits == RESTART_RESPONSE_SIZE_BITS ||
				NumBits == VERSION_UPGRADE_SIZE_BITS);
	}
#endif

	FPacketAudit::AddStage(TEXT("PostPacketHandler"), HandshakePacket);

	if (HandshakeVersion >= EHandshakeVersion::Randomized)
	{
		int32 RandomDataLengthBytes = BaseRandomDataLengthBytes - FMath::RandRange(0, RandomDataLengthVarianceBytes);

		// In versions that must stay compatible with the original protocol, make sure there isn't a size collision with the original restart response
		if (HandshakeVersion < EHandshakeVersion::SessionClientId)
		{
			if (NumBits + (RandomDataLengthBytes * 8) == OriginalRestartResponseSizeBits)
			{
				RandomDataLengthBytes = FMath::Max(0, RandomDataLengthBytes - 1);
			}
		}

		for (int32 RandIdx=0; RandIdx<RandomDataLengthBytes; RandIdx++)
		{
			uint8 RandVal = FMath::Rand() % 255;

			HandshakePacket << RandVal;
		}
	}

	// Add a termination bit, the same as the UNetConnection code does
	HandshakePacket.WriteBit(1);

	SentHandshakePacketCount++;
}

void StatelessConnectHandlerComponent::SetDriver(UNetDriver* InDriver)
{
	Driver = InDriver;

	if (Handler->Mode == UE::Handler::Mode::Server)
	{
		StatelessConnectHandlerComponent* StatelessComponent = Driver->StatelessConnectComponent.Pin().Get();

		if (StatelessComponent != nullptr)
		{
			if (StatelessComponent == this)
			{
				UpdateSecret();
			}
			else
			{
				InitFromConnectionless(StatelessComponent);
			}
		}

		CachedGlobalNetTravelCount = Driver->GetCachedGlobalNetTravelCount() & ((1 << SessionIDSizeBits) - 1);
	}
	else //if (Handler->Mode == Handler::Mode::Client)
	{
		// Use basic GConfig until NetDriver per-NetDriverDefinitionName per-object-config refactor
		TStringBuilder<256> ConfigSection;
		const FString NetDriverDef = Driver->GetNetDriverDefinition().ToString();

		ConfigSection.Append(ToCStr(NetDriverDef));
		ConfigSection.Append(TEXT(" StatelessConnectHandlerComponent"));

		const TCHAR* ConfigKey = TEXT("CachedClientID");
		int32 ConfigCachedClientID = 0;

		GConfig->GetInt(ConfigSection.ToString(), ConfigKey, ConfigCachedClientID, GEngineIni);

		CachedClientID = FMath::Max(ConfigCachedClientID, 0) + 1;

		if (CachedClientID > static_cast<uint32>(std::numeric_limits<int32>::max()))
		{
			CachedClientID = 0;
		}

		GConfig->SetInt(ConfigSection.ToString(), ConfigKey, CachedClientID, GEngineIni);
		GConfig->Flush(false, GEngineIni);

		CachedClientID = CachedClientID & ((1 << ClientIDSizeBits) - 1);

		UE_LOGF(LogHandshake, Log, "Stateless Handshake: NetDriverDefinition '%ls' CachedClientID: %u", ToCStr(NetDriverDef),
				CachedClientID);
	}
}

void StatelessConnectHandlerComponent::Initialize()
{
	using namespace UE::Net;

	// On the server, initializes immediately. Clientside doesn't initialize until handshake completes.
	if (Handler->Mode == UE::Handler::Mode::Server)
	{
		Initialized();

#if !UE_BUILD_SHIPPING
		const EHandshakeVersion CurHandshakeVersion = static_cast<EHandshakeVersion>(CurrentHandshakeVersion);
		const EHandshakeVersion MinHandshakeVersion = static_cast<EHandshakeVersion>(MinSupportedHandshakeVersion);

		if (CurHandshakeVersion > EHandshakeVersion::Latest)
		{
			UE_LOGF(LogHandshake, Error, "net.CurrentHandshakeVersion value '%i' is invalid. Maximum value: %i", CurrentHandshakeVersion,
					static_cast<int32>(EHandshakeVersion::Latest));
		}
		else
		{
			EHandshakeVersion CompatibleRangeStart = EHandshakeVersion::Original;
			EHandshakeVersion CompatibleRangeEnd = EHandshakeVersion::Latest;

			for (EHandshakeVersion CurBreakVersion : HandshakeCompatibilityBreaks)
			{
				if (CurBreakVersion <= CurHandshakeVersion)
				{
					CompatibleRangeStart = CurBreakVersion;
				}
				else if (CurBreakVersion > CurHandshakeVersion)
				{
					CompatibleRangeEnd = static_cast<EHandshakeVersion>(static_cast<uint8>(CurBreakVersion) - 1);
					break;
				}
			}

			if (MinHandshakeVersion < CompatibleRangeStart || MinHandshakeVersion > CompatibleRangeEnd)
			{
				UE_LOGF(LogHandshake, Error, "net.MinHandshakeVersion value '%i' is invalid relative to net.CurrentHandshakeVersion value '%i'. "
					"Minimum value: %u, Maximum value: %u", MinSupportedHandshakeVersion, CurrentHandshakeVersion,
					static_cast<uint8>(CompatibleRangeStart), static_cast<uint8>(CompatibleRangeEnd));
			}
		}
#endif
	}
}

void StatelessConnectHandlerComponent::InitFromConnectionless(StatelessConnectHandlerComponent* InConnectionlessHandler)
{
	// Store the cookie/address used for the handshake, to enable server ack-retries
	LastChallengeSuccessAddress = InConnectionlessHandler->LastChallengeSuccessAddress;
	LastRemoteHandshakeVersion = InConnectionlessHandler->LastRemoteHandshakeVersion;
	CachedClientID = InConnectionlessHandler->CachedClientID;

	FMemory::Memcpy(AuthorisedCookie, InConnectionlessHandler->AuthorisedCookie, UE_ARRAY_COUNT(AuthorisedCookie));

	LastInitTimestamp = (Driver != nullptr ? Driver->GetElapsedTime() : 0.0);
}

void StatelessConnectHandlerComponent::SetHandshakeFailureCallback(FHandshakeFailureFunc&& InHandshakeFailureFunc)
{
	HandshakeFailureCallback = MoveTemp(InHandshakeFailureFunc);
}

bool StatelessConnectHandlerComponent::SupportsParallelConnectionOutgoing() const
{
	return true;
}

void StatelessConnectHandlerComponent::Incoming(FBitReader& Packet)
{
	using namespace UE::Net;

	if (MagicHeader.Num() > 0)
	{
		// Don't bother with the expense of verifying the magic header here.
		uint32 ReadMagic = 0;
		Packet.SerializeBits(&ReadMagic, MagicHeader.Num());
	}

	bool bHasValidSessionID = true;
	bool bHasValidClientID = true;
	uint8 SessionID = 0;
	uint8 ClientID = 0;

	if (LastRemoteHandshakeVersion >= EHandshakeVersion::SessionClientId)
	{
		Packet.SerializeBits(&SessionID, SessionIDSizeBits);
		Packet.SerializeBits(&ClientID, ClientIDSizeBits);

		bHasValidSessionID = GVerifyNetSessionID == 0 || (SessionID == CachedGlobalNetTravelCount && !Packet.IsError());
		bHasValidClientID = GVerifyNetClientID == 0 || (ClientID == CachedClientID && !Packet.IsError());
	}

	bool bHandshakePacket = !!Packet.ReadBit() && !Packet.IsError();

	if (bHandshakePacket)
	{
		FParsedHandshakeData HandshakeData;

		bHandshakePacket = ParseHandshakePacket(Packet, HandshakeData);

		if (bHandshakePacket)
		{
			const bool bIsChallengePacket = HandshakeData.HandshakePacketType == EHandshakePacketType::Challenge && HandshakeData.Timestamp > 0.0;
			const bool bIsInitialChallengePacket = bIsChallengePacket && State != UE::Handler::Component::State::Initialized;
			const bool bIsUpgradePacket = HandshakeData.HandshakePacketType == EHandshakePacketType::VersionUpgrade;

			if (Handler->Mode == UE::Handler::Mode::Client && bHasValidClientID && (bHasValidSessionID || bIsInitialChallengePacket || bIsUpgradePacket))
			{
				if (State == UE::Handler::Component::State::UnInitialized || State == UE::Handler::Component::State::InitializedOnLocal)
				{
					if (HandshakeData.bRestartHandshake)
					{
#if !UE_BUILD_SHIPPING
						UE_LOGF(LogHandshake, Log, "Ignoring restart handshake request, while already restarted.");
#endif
					}
					// Receiving challenge
					else if (bIsChallengePacket)
					{
#if !UE_BUILD_SHIPPING
						UE_LOGF(LogHandshake, Log, "Cached server SessionID: %u", SessionID);
#endif

						CachedGlobalNetTravelCount = SessionID;

						LastChallengeTimestamp = (Driver != nullptr ? Driver->GetElapsedTime() : 0.0);

						SendChallengeResponse(HandshakeData.RemoteCurVersion, HandshakeData.SecretId, HandshakeData.Timestamp, HandshakeData.Cookie);

						// Utilize this state as an intermediary, indicating that the challenge response has been sent
						SetState(UE::Handler::Component::State::InitializedOnLocal);
					}
					// Receiving challenge ack, verify the timestamp is < 0.0f
					else if (HandshakeData.HandshakePacketType == EHandshakePacketType::Ack && HandshakeData.Timestamp < 0.0)
					{
						if (!bRestartedHandshake)
						{
							UNetConnection* ServerConn = (Driver != nullptr ? ToRawPtr(Driver->ServerConnection) : nullptr);

							// Extract the initial packet sequence from the random Cookie data
							if (ensure(ServerConn != nullptr))
							{
								int16* CurSequence = (int16*)HandshakeData.Cookie;

								int32 ServerSequence = *CurSequence & (MAX_PACKETID - 1);
								int32 ClientSequence = *(CurSequence + 1) & (MAX_PACKETID - 1);

								ServerConn->InitSequence(ServerSequence, ClientSequence);
							}

							// Save the final authorized cookie
							FMemory::Memcpy(AuthorisedCookie, HandshakeData.Cookie, UE_ARRAY_COUNT(AuthorisedCookie));
						}

						// Now finish initializing the handler - flushing the queued packet buffer in the process.
						SetState(UE::Handler::Component::State::Initialized);
						Initialized();

						bRestartedHandshake = false;

						// Reset packet count clientside, due to how it affects protocol version fallback selection
						SentHandshakePacketCount = 0;
					}
					else if (bIsUpgradePacket)
					{
						const uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();
						TStringBuilder<128> LocalNetFeaturesDescription;
						TStringBuilder<128> RemoteNetFeaturesDescription;
						EEngineNetworkRuntimeFeatures LocalNetworkFeatures = (Driver != nullptr ? Driver->GetNetworkRuntimeFeatures() :
																				EEngineNetworkRuntimeFeatures::None);
						FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(LocalNetworkFeatures, LocalNetFeaturesDescription);
						FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(HandshakeData.RemoteNetworkFeatures, RemoteNetFeaturesDescription);

						UE_LOGF(LogHandshake, Log, "Server is running an incompatible version of the game, and has rejected the connection. "
							"Server version '%u', Local version '%u', Server features '%ls', Local features '%ls'.",
							HandshakeData.RemoteNetworkVersion, LocalNetworkVersion, ToCStr(RemoteNetFeaturesDescription.ToString()),
							ToCStr(LocalNetFeaturesDescription.ToString()));

						if (HandshakeFailureCallback)
						{
							FStatelessHandshakeFailureInfo FailureInfo;

							FailureInfo.FailureReason = EHandshakeFailureReason::WrongVersion;
							FailureInfo.RemoteNetworkVersion = HandshakeData.RemoteNetworkVersion;
							FailureInfo.RemoteNetworkFeatures = HandshakeData.RemoteNetworkFeatures;

							HandshakeFailureCallback(FailureInfo);
						}
					}
				}
				else if (HandshakeData.bRestartHandshake)
				{
					uint8 ZeroCookie[COOKIE_BYTE_SIZE] = {0};
					bool bValidAuthCookie = FMemory::Memcmp(AuthorisedCookie, ZeroCookie, COOKIE_BYTE_SIZE) != 0;

					// The server has requested us to restart the handshake process - this is because
					// it has received traffic from us on a different address than before.
					if (ensure(bValidAuthCookie))
					{
						bool bPassedDelayCheck = false;
						bool bPassedDualIPCheck = false;
						double CurrentTime = FPlatformTime::Seconds();;

						if (!bRestartedHandshake)
						{
							UNetConnection* ServerConn = (Driver != nullptr ? ToRawPtr(Driver->ServerConnection) : nullptr);
							double LastNetConnPacketTime = (ServerConn != nullptr ? ServerConn->LastReceiveRealtime : 0.0);
							const double DriverElapsedTime = (Driver != nullptr ? Driver->GetElapsedTime() : 0.0);

							// The server may send multiple restart handshake packets, so have a 10 second delay between accepting them
							bPassedDelayCheck = (DriverElapsedTime - LastClientSendTimestamp) > 10.0;

							// Some clients end up sending packets duplicated over multiple IP's, triggering the restart handshake.
							// Detect this by checking if any restart handshake requests have been received in roughly the last second
							// (Dual IP situations will make the server send them constantly) - and override the checks as a failsafe,
							// if no NetConnection packets have been received in the last second.
							double LastRestartPacketTimeDiff = CurrentTime - LastRestartPacketTimestamp;
							double LastNetConnPacketTimeDiff = CurrentTime - LastNetConnPacketTime;

							bPassedDualIPCheck = LastRestartPacketTimestamp == 0.0 ||
													LastRestartPacketTimeDiff > 1.1 ||
													LastNetConnPacketTimeDiff > 1.0;
						}

						LastRestartPacketTimestamp = CurrentTime;

						auto WithinHandshakeLogLimit = [&Driver = Driver]() -> bool
							{
								const double LogCountPeriod = 30.0;
								const int8 MaxLogCount = 3;

								static double LastLogStartTime = 0.0;
								static int32 LogCounter = 0;

								double CurTimeApprox = Driver->GetElapsedTime();
								bool bWithinLimit = false;

								if ((CurTimeApprox - LastLogStartTime) > LogCountPeriod)
								{
									LastLogStartTime = CurTimeApprox;
									LogCounter = 1;
									bWithinLimit = true;
								}
								else if (LogCounter < MaxLogCount)
								{
									LogCounter++;
									bWithinLimit = true;
								}

								return bWithinLimit;
							};

						if (!bRestartedHandshake && bPassedDelayCheck && bPassedDualIPCheck)
						{
							UE_LOGF(LogHandshake, Log, "Beginning restart handshake process.");

							bRestartedHandshake = true;

							SetState(UE::Handler::Component::State::UnInitialized);
							SendInitialPacket(LastRemoteHandshakeVersion);
						}
						else if (WithinHandshakeLogLimit())
						{
							if (bRestartedHandshake)
							{
								UE_LOGF(LogHandshake, Log, "Ignoring restart handshake request, while already restarted (this is normal).");
							}
#if !UE_BUILD_SHIPPING
							else if (!bPassedDelayCheck)
							{
								UE_LOGF(LogHandshake, Log, "Ignoring restart handshake request, due to < 10 seconds since last handshake.");
							}
							else // if (!bPassedDualIPCheck)
							{
								UE_LOGF(LogHandshake, Log, "Ignoring restart handshake request, due to recent NetConnection packets.");
							}
#endif
						}
					}
					else
					{
						UE_LOGF(LogHandshake, Log, "Server sent restart handshake request, when we don't have an authorised cookie.");

						Packet.SetError();
					}
				}
				else
				{
					// Ignore, could be a dupe/out-of-order challenge packet
				}
			}
			else if (Handler->Mode == UE::Handler::Mode::Server && bHasValidSessionID && bHasValidClientID)
			{
				if (LastChallengeSuccessAddress.IsValid())
				{
					// The server should not be receiving handshake packets at this stage - resend the ack in case it was lost.
					// In this codepath, this component is linked to a UNetConnection, and the Last* values below, cache the handshake info.
#if !UE_BUILD_SHIPPING
					UE_LOGF(LogHandshake, Log, "Received unexpected post-connect handshake packet - resending ack for LastChallengeSuccessAddress %ls and LastCookie %ls.",
							*LastChallengeSuccessAddress->ToString(true), *FString::FromBlob(AuthorisedCookie, COOKIE_BYTE_SIZE));
#endif

					SendChallengeAck(FCommonSendToClientParams(LastChallengeSuccessAddress, LastRemoteHandshakeVersion, CachedClientID), 0,
										AuthorisedCookie);
				}
			}
			else if (!bHasValidSessionID || !bHasValidClientID)
			{
#if !UE_BUILD_SHIPPING
				UE_CLOGF(TrackValidationLogs(), LogHandshake, Log,
						"Incoming: Rejecting handshake packet with invalid session id (%u vs %u) or connection id (%u vs %u).",
						SessionID, CachedGlobalNetTravelCount, ClientID, CachedClientID);
#endif

				// Ignore, don't trigger disconnect
				Packet.SetAtEnd();
			}
		}
		else
		{
			Packet.SetError();

#if !UE_BUILD_SHIPPING
			UE_LOGF(LogHandshake, Log, "Incoming: Error reading handshake packet.");
#endif
		}
	}
#if !UE_BUILD_SHIPPING
	else if (Packet.IsError())
	{
		UE_LOGF(LogHandshake, Log, "Incoming: Error reading session id, connection id and handshake bit from packet.");
	}
#endif
	else if (!bHasValidSessionID || !bHasValidClientID)
	{
#if !UE_BUILD_SHIPPING
		UE_CLOGF(TrackValidationLogs(), LogHandshake, Log,
				"Incoming: Rejecting game packet with invalid session id (%u vs %u) or connection id (%u vs %u).",
				SessionID, CachedGlobalNetTravelCount, ClientID, CachedClientID);
#endif

		// Ignore, don't trigger disconnect
		Packet.SetAtEnd();
	}
	else
	{
		// Servers should wipe LastChallengeSuccessAddress shortly after the first non-handshake packet is received by the client,
		// in order to disable challenge ack resending
		if (LastInitTimestamp != 0.0 && LastChallengeSuccessAddress.IsValid() && Handler->Mode == UE::Handler::Mode::Server)
		{
			// Restart handshakes require extra time before disabling challenge ack resends, as NetConnection packets will already be in flight
			const double RestartHandshakeAckResendWindow = 10.0;
			double CurTime = Driver != nullptr ? Driver->GetElapsedTime() : 0.0;

			if (CurTime - LastInitTimestamp >= RestartHandshakeAckResendWindow)
			{
				LastChallengeSuccessAddress.Reset();
				LastInitTimestamp = 0.0;
			}
		}
	}
}

void StatelessConnectHandlerComponent::Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
{
	// All UNetConnection packets must specify a zero bHandshakePacket value
	const int32 AdjustedSize = GetAdjustedSizeBits(Packet.GetNumBits(), LastRemoteHandshakeVersion);

	// Try to reuse memory from previous Outgoing call if there is enough space
	FBitWriter NewPacket = MoveTemp(OldPacket);
	if (NewPacket.GetMaxBits() < AdjustedSize + 1)
	{
		// Otherwise allocate at least MAX_PACKET_SIZE bytes, just like PacketHandler does, so packet memory can be reused later
		NewPacket = FBitWriter(FMath::Max(AdjustedSize + 1, MAX_PACKET_SIZE * 8), true);
	}
	else
	{
		NewPacket.Reset();
	}

	uint8 bHandshakePacket = 0;

	if (MagicHeader.Num() > 0)
	{
		NewPacket.SerializeBits(MagicHeader.GetData(), MagicHeader.Num());
	}

	if (LastRemoteHandshakeVersion >= EHandshakeVersion::SessionClientId)
	{
		NewPacket.SerializeBits(&CachedGlobalNetTravelCount, SessionIDSizeBits);
		NewPacket.SerializeBits(&CachedClientID, ClientIDSizeBits);
	}

	NewPacket.WriteBit(bHandshakePacket);
	NewPacket.SerializeBits(Packet.GetData(), Packet.GetNumBits());

	// Remember bit writer memory of old packet, so it can be re-used next time
	OldPacket = MoveTemp(Packet);

	Packet = MoveTemp(NewPacket);
}

void StatelessConnectHandlerComponent::IncomingConnectionless(FIncomingPacketRef PacketRef)
{
	using namespace UE::Net;

	FBitReader& Packet = PacketRef.Packet;
	const TSharedPtr<const FInternetAddr> Address = PacketRef.Address;

	if (MagicHeader.Num() > 0)
	{
		uint32 ReadMagic = 0;

		Packet.SerializeBits(&ReadMagic, MagicHeader.Num());

		if (GVerifyMagicHeader && ReadMagic != MagicHeaderUint)
		{
#if !UE_BUILD_SHIPPING
			UE_CLOGF(TrackValidationLogs(), LogNet, Log, "Rejecting packet with invalid magic header '%08X' vs '%08X' (%i bits)",
					ReadMagic, MagicHeaderUint, MagicHeader.Num());
#endif

			Packet.SetError();

			return;
		}
	}


	bool bHasValidSessionID = true;
	uint8 SessionID = 0;
	uint8 ClientID = 0;

	if (CurrentHandshakeVersion >= static_cast<uint8>(EHandshakeVersion::SessionClientId))
	{
		Packet.SerializeBits(&SessionID, SessionIDSizeBits);
		Packet.SerializeBits(&ClientID, ClientIDSizeBits);

		bHasValidSessionID = GVerifyNetSessionID == 0 || (SessionID == CachedGlobalNetTravelCount && !Packet.IsError());

		// No ClientID validation until connected
	}

	const bool bHandshakePacket = !!Packet.ReadBit() && !Packet.IsError();

	LastChallengeSuccessAddress = nullptr;

	if (!bHandshakePacket)
	{
		if (Packet.IsError())
		{
#if !UE_BUILD_SHIPPING
			UE_LOGF(LogHandshake, Log, "IncomingConnectionless: Error reading session id, connection id and handshake bit from packet.");
#endif
		}
		else if (bHasValidSessionID)
		{
			// Late packets from recently disconnected clients may incorrectly trigger this code path, so detect and exclude those packets
			if (!Packet.IsError() && !PacketRef.Traits.bFromRecentlyDisconnected)
			{
				// The packet was fine but not a handshake packet - an existing client might suddenly be communicating on a different address.
				// If we get them to resend their cookie, we can update the connection's info with their new address.
				SendRestartHandshakeRequest(FCommonSendToClientParams(Address, static_cast<EHandshakeVersion>(MinSupportedHandshakeVersion), ClientID));
			}
		}

		return;
	}

	FParsedHandshakeData HandshakeData;
	const bool bValidHandshakePacket = ParseHandshakePacket(Packet, HandshakeData);

	if (UNLIKELY(!bValidHandshakePacket))
	{
		Packet.SetError();

		FDDoSDetection* DDoS = Handler->GetDDoS();

		if (DDoS != nullptr)
		{
			DDoS->IncBadPacketCounter();
		}

#if !UE_BUILD_SHIPPING
		UE_CLOGF(DDoS == nullptr || !DDoS->CheckLogRestrictions(), LogHandshake, Log,
			"IncomingConnectionless: Error reading handshake packet.");
#endif
		return;
	}
	
	const bool bIsServer = Handler->Mode == UE::Handler::Mode::Server;
	if (UNLIKELY(!bIsServer))
	{
		// Only server can negotiate handshake requests here
		return;
	}

	EHandshakeVersion TargetVersion = EHandshakeVersion::Latest;
	const bool bValidVersion = CheckVersion(HandshakeData, TargetVersion);
	const bool bInitialConnect = HandshakeData.HandshakePacketType == EHandshakePacketType::InitialPacket && HandshakeData.Timestamp == 0.0;
	const double ElapsedTime = Driver ? Driver->GetElapsedTime() : 0.0;

	const bool bIsValidRequest = bValidVersion && (bHasValidSessionID || bInitialConnect);

	// Handle invalid requests
	if (!bIsValidRequest)
	{
		// Connection has an invalid Session ID
		if (bValidVersion && !bHasValidSessionID)
		{
#if !UE_BUILD_SHIPPING
			UE_LOGF(LogHandshake, Log, "IncomingConnectionless: Rejecting packet with invalid session id: %u vs %u.", SessionID, CachedGlobalNetTravelCount);
#endif
			Packet.SetError();
		}
		// Connection is using an incompatible version, tell it to upgrade
		else if (!bValidVersion && bInitialConnect && HandshakeData.RemoteCurVersion >= EHandshakeVersion::NetCLUpgradeMessage)
		{
			// Limit of 512 upgrade message packets, over 5 minutes
			const uint32 NumUpgradeMessagesPerPeriod = 512;
			const double UpgradeMessagePeriod = 300;

			if (ElapsedTime - LastUpgradeMessagePeriodStart >= UpgradeMessagePeriod)
			{
				LastUpgradeMessagePeriodStart = ElapsedTime;
				UpgradeMessageCounter = 0;
			}
			else
			{
				UpgradeMessageCounter++;
			}

			if (UpgradeMessageCounter < NumUpgradeMessagesPerPeriod)
			{
				SendVersionUpgradeMessage(FCommonSendToClientParams(Address, TargetVersion, ClientID));
			}
		}

		return;
	}

	if (bInitialConnect)
	{
		SendConnectChallenge(FCommonSendToClientParams(Address, TargetVersion, ClientID), HandshakeData.RemoteSentHandshakePacketCount);
	}
	else
	{
		// Challenge response
		// NOTE: Allow CookieDelta to be 0.0, as it is possible for a server to send a challenge and receive a response,
		//			during the same tick
		bool bChallengeSuccess = false;
		const double CookieDelta = ElapsedTime - HandshakeData.Timestamp;
		const double SecretDelta = HandshakeData.Timestamp - LastSecretUpdateTimestamp;
		const bool bValidCookieLifetime = CookieDelta >= 0.0 && (MAX_COOKIE_LIFETIME - CookieDelta) > 0.0;
		const bool bValidSecretIdTimestamp = (HandshakeData.SecretId == ActiveSecret) ? (SecretDelta >= 0.0) : (SecretDelta <= 0.0);

		if (bValidCookieLifetime && bValidSecretIdTimestamp)
		{
			// Regenerate the cookie from the packet info, and see if the received cookie matches the regenerated one
			uint8 RegenCookie[COOKIE_BYTE_SIZE];

			GenerateCookie(Address, HandshakeData.SecretId, HandshakeData.Timestamp, RegenCookie);

			bChallengeSuccess = FMemory::Memcmp(HandshakeData.Cookie, RegenCookie, COOKIE_BYTE_SIZE) == 0;

			if (bChallengeSuccess)
			{
				if (HandshakeData.bRestartHandshake)
				{
					FMemory::Memcpy(AuthorisedCookie, HandshakeData.OrigCookie, UE_ARRAY_COUNT(AuthorisedCookie));
				}
				else
				{
					int16* CurSequence = (int16*)HandshakeData.Cookie;

					LastServerSequence = *CurSequence & (MAX_PACKETID - 1);
					LastClientSequence = *(CurSequence + 1) & (MAX_PACKETID - 1);

					FMemory::Memcpy(AuthorisedCookie, HandshakeData.Cookie, UE_ARRAY_COUNT(AuthorisedCookie));
				}

				bRestartedHandshake = HandshakeData.bRestartHandshake;
				LastChallengeSuccessAddress = Address->Clone();
				LastRemoteHandshakeVersion = TargetVersion;
				CachedClientID = ClientID;

				if (TargetVersion < MinClientHandshakeVersion && static_cast<uint8>(TargetVersion) >= MinSupportedHandshakeVersion)
				{
					MinClientHandshakeVersion = TargetVersion;
				}

				// Now ack the challenge response - the cookie is stored in AuthorisedCookie, to enable retries
				SendChallengeAck(FCommonSendToClientParams(Address, TargetVersion, ClientID), HandshakeData.RemoteSentHandshakePacketCount, AuthorisedCookie);
			}
		}
	}
}

bool StatelessConnectHandlerComponent::ParseHandshakePacket(FBitReader& Packet, FParsedHandshakeData& OutResult) const
{
	using namespace UE::Net;

	// Ensure original packet sizes don't overlap with size range of new packet format - so that we can detect original version, based on size
	{
		static constexpr int32 MinRandomBits = (BaseRandomDataLengthBytes - RandomDataLengthVarianceBytes) * 8;
		static constexpr int32 MaxRandomBits = BaseRandomDataLengthBytes * 8;
		static constexpr int32 MinHandshakePacketVariance = HANDSHAKE_PACKET_SIZE_BITS + MinRandomBits;
		static constexpr int32 MaxHandshakePacketVariance = HANDSHAKE_PACKET_SIZE_BITS + MaxRandomBits;
		static constexpr int32 MinRestartHandshakePacketVariance = RESTART_HANDSHAKE_PACKET_SIZE_BITS + MinRandomBits;
		static constexpr int32 MaxRestartHandshakePacketVariance = RESTART_HANDSHAKE_PACKET_SIZE_BITS + MaxRandomBits;
		static constexpr int32 MinRestartResponsePacketVariance = RESTART_RESPONSE_SIZE_BITS + MinRandomBits;
		static constexpr int32 MaxRestartResponsePacketVariance = RESTART_RESPONSE_SIZE_BITS + MaxRandomBits;
		static constexpr int32 MinVersionUpgradePacketVariance = VERSION_UPGRADE_SIZE_BITS + MinRandomBits;
		static constexpr int32 MaxVersionUpgradePacketVariance = VERSION_UPGRADE_SIZE_BITS + MaxRandomBits;

		static_assert(OriginalHandshakePacketSizeBits < MinHandshakePacketVariance || OriginalHandshakePacketSizeBits > MaxHandshakePacketVariance);
		static_assert(OriginalHandshakePacketSizeBits < MinRestartHandshakePacketVariance ||
						OriginalHandshakePacketSizeBits > MaxRestartHandshakePacketVariance);
		static_assert(OriginalHandshakePacketSizeBits < MinRestartResponsePacketVariance ||
						OriginalHandshakePacketSizeBits > MaxRestartResponsePacketVariance);
		static_assert(OriginalHandshakePacketSizeBits < MinVersionUpgradePacketVariance ||
						OriginalHandshakePacketSizeBits > MaxVersionUpgradePacketVariance);
		static_assert(OriginalRestartHandshakePacketSizeBits < MinHandshakePacketVariance ||
						OriginalRestartHandshakePacketSizeBits > MaxHandshakePacketVariance);
		static_assert(OriginalRestartHandshakePacketSizeBits < MinRestartHandshakePacketVariance ||
						OriginalRestartHandshakePacketSizeBits > MaxRestartHandshakePacketVariance);
		static_assert(OriginalRestartHandshakePacketSizeBits < MinRestartResponsePacketVariance ||
						OriginalRestartHandshakePacketSizeBits > MaxRestartResponsePacketVariance);
		static_assert(OriginalRestartHandshakePacketSizeBits < MinVersionUpgradePacketVariance ||
						OriginalRestartHandshakePacketSizeBits > MaxVersionUpgradePacketVariance);
	};

	bool bValidPacket = false;
	const int32 BitsLeft = Packet.GetBitsLeft();

	if (MinSupportedHandshakeVersion == static_cast<uint8>(EHandshakeVersion::Original))
	{
		const bool bOriginalVersion = BitsLeft == (OriginalHandshakePacketSizeBits - 1) || BitsLeft == (OriginalRestartHandshakePacketSizeBits - 1) ||
										BitsLeft == (OriginalRestartResponseSizeBits - 1);

		if (bOriginalVersion)
		{
			return ParseHandshakePacketOriginal(Packet, OutResult);
		}
	}


	// Remaining bits, excluding packet sizes from different protocol versions (NOTE: Current code assumes packet size defines only increase)
	const int32 MinBitsLeftExclHandshake = BitsLeft - (HANDSHAKE_PACKET_SIZE_BITS - 1);
	const int32 MaxBitsLeftExclHandshake = BitsLeft - (VerRandomizedHandshakePacketSizeBits - 1);
	const int32 MinBitsLeftExclRestartHandshake = BitsLeft - (RESTART_HANDSHAKE_PACKET_SIZE_BITS - 1);
	const int32 MaxBitsLeftExclRestartHandshake = BitsLeft - (VerRandomizedRestartHandshakePacketSizeBits - 1);
	const int32 MinBitsLeftExclRestartResponse = BitsLeft - (RESTART_RESPONSE_SIZE_BITS - 1);
	const int32 MaxBitsLeftExclRestartResponse = BitsLeft - (VerRandomizedRestartResponseSizeBits - 1);
	const int32 MinBitsLeftExclVersionUpgrade = BitsLeft - (VERSION_UPGRADE_SIZE_BITS - 1);
	const int32 MaxBitsLeftExclVersionUpgrade = BitsLeft - (VERSION_UPGRADE_SIZE_BITS - 1);	// To be updated if size or BASE_PACKET_SIZE_BITS changes
	const int32 MinRandomBits = (BaseRandomDataLengthBytes - RandomDataLengthVarianceBytes) * 8;
	const int32 MaxRandomBits = BaseRandomDataLengthBytes * 8;
	const bool bMaybeHandshakePacket = MaxBitsLeftExclHandshake >= MinRandomBits && MinBitsLeftExclHandshake <= MaxRandomBits;
	const bool bMaybeRestartHandshakePacket = MaxBitsLeftExclRestartHandshake >= MinRandomBits && MinBitsLeftExclRestartHandshake <= MaxRandomBits;
	const bool bMaybeRestartResponsePacket = MaxBitsLeftExclRestartResponse >= MinRandomBits && MinBitsLeftExclRestartResponse <= MaxRandomBits;
	const bool bMaybeVersionUpgradePacket = MaxBitsLeftExclVersionUpgrade >= MinRandomBits && MinBitsLeftExclVersionUpgrade <= MaxRandomBits;

	static_assert(BASE_PACKET_SIZE_BITS == 82 && VERSION_UPGRADE_SIZE_BITS == 82, "MaxBitsLeftExclVersionUpgrade needs to be updated."); // -V501

	OutResult.bRestartHandshake = !!Packet.ReadBit();

	uint8 RemoteMinVersion = 0;
	uint8 RemoteCurVersion = 0;
	uint8 HandshakePacketType = 0;
	EHandshakePacketType& HandshakePacketTypeEnum = OutResult.HandshakePacketType;

	Packet << RemoteMinVersion;
	Packet << RemoteCurVersion;
	Packet << HandshakePacketType;
	Packet << OutResult.RemoteSentHandshakePacketCount;

	OutResult.RemoteMinVersion = static_cast<EHandshakeVersion>(RemoteMinVersion);
	OutResult.RemoteCurVersion = static_cast<EHandshakeVersion>(RemoteCurVersion);
	HandshakePacketTypeEnum = static_cast<EHandshakePacketType>(HandshakePacketType);

	if (OutResult.RemoteCurVersion >= EHandshakeVersion::NetCLVersion)
	{
		Packet << OutResult.RemoteNetworkVersion;
		Packet << OutResult.RemoteNetworkFeatures;
	}

	// Only accept handshake packets of roughly the right size
	const bool bHandshakePacket = bMaybeHandshakePacket && (HandshakePacketTypeEnum == EHandshakePacketType::InitialPacket ||
		HandshakePacketTypeEnum == EHandshakePacketType::Challenge || HandshakePacketTypeEnum == EHandshakePacketType::Response ||
		HandshakePacketTypeEnum == EHandshakePacketType::Ack);

	const bool bRestartHandshakePacket = bMaybeRestartHandshakePacket && HandshakePacketTypeEnum == EHandshakePacketType::RestartHandshake;
	const bool bRestartResponsePacket = bMaybeRestartResponsePacket && HandshakePacketTypeEnum == EHandshakePacketType::RestartResponse;
	const bool bVersionUpgradePacket = bMaybeVersionUpgradePacket && HandshakePacketTypeEnum == EHandshakePacketType::VersionUpgrade;

	// Only accept handshake packets of precisely the right size
	if (bHandshakePacket || bRestartResponsePacket)
	{
		OutResult.SecretId = Packet.ReadBit();

		Packet << OutResult.Timestamp;

		Packet.Serialize(OutResult.Cookie, COOKIE_BYTE_SIZE);

		if (bRestartResponsePacket)
		{
			Packet.Serialize(OutResult.OrigCookie, COOKIE_BYTE_SIZE);
		}

		bValidPacket = !Packet.IsError();
	}
	else if (bRestartHandshakePacket)
	{
		bValidPacket = !Packet.IsError() && OutResult.bRestartHandshake && Handler->Mode == UE::Handler::Mode::Client;
	}
	else if (bVersionUpgradePacket)
	{
		bValidPacket = !Packet.IsError() && !OutResult.bRestartHandshake && Handler->Mode == UE::Handler::Mode::Client;
	}

	if (bValidPacket)
	{
		Packet.SetAtEnd();
	}

	return bValidPacket;
}

bool StatelessConnectHandlerComponent::ParseHandshakePacketOriginal(FBitReader& Packet, FParsedHandshakeData& OutResult) const
{
	using namespace UE::Net;

	bool bValidPacket = false;
	uint32 BitsLeft = Packet.GetBitsLeft();
	bool bHandshakePacketSize = BitsLeft == (OriginalHandshakePacketSizeBits - 1);
	bool bRestartResponsePacketSize = BitsLeft == (OriginalRestartResponseSizeBits - 1);

	OutResult.RemoteMinVersion = EHandshakeVersion::Original;
	OutResult.RemoteCurVersion = EHandshakeVersion::Original;

	// Only accept handshake packets of precisely the right size
	if (bHandshakePacketSize || bRestartResponsePacketSize)
	{
		OutResult.bRestartHandshake = !!Packet.ReadBit();
		OutResult.SecretId = Packet.ReadBit();

		Packet << OutResult.Timestamp;

		Packet.Serialize(OutResult.Cookie, COOKIE_BYTE_SIZE);

		if (bRestartResponsePacketSize)
		{
			OutResult.HandshakePacketType = EHandshakePacketType::RestartResponse;
			Packet.Serialize(OutResult.OrigCookie, COOKIE_BYTE_SIZE);
		}
		else if (OutResult.Timestamp > 0.0)
		{
			if (Handler->Mode == UE::Handler::Mode::Client)
			{
				OutResult.HandshakePacketType = EHandshakePacketType::Challenge;
			}
			else
			{
				OutResult.HandshakePacketType = EHandshakePacketType::Response;
			}
		}
		else if (OutResult.Timestamp < 0.0)
		{
			OutResult.HandshakePacketType = EHandshakePacketType::Ack;
		}
		else
		{
			OutResult.HandshakePacketType = EHandshakePacketType::InitialPacket;
		}

		bValidPacket = !Packet.IsError();
	}
	else if (BitsLeft == (OriginalRestartHandshakePacketSizeBits - 1))
	{
		OutResult.HandshakePacketType = EHandshakePacketType::RestartHandshake;
		OutResult.bRestartHandshake = !!Packet.ReadBit();
		bValidPacket = !Packet.IsError() && OutResult.bRestartHandshake && Handler->Mode == UE::Handler::Mode::Client;
	}

	return bValidPacket;
}

bool StatelessConnectHandlerComponent::CheckVersion(const FParsedHandshakeData& HandshakeData, EHandshakeVersion& OutTargetVersion) const
{
	using namespace UE::Net;

	bool bValidHandshakeVersion = false;
	const uint8 RemoteMinVersionUint8 = static_cast<uint8>(HandshakeData.RemoteMinVersion);
	EHandshakeVersion LocalMaxVersion = EHandshakeVersion::Latest;
	bool bHasMaxVersion = false;

	OutTargetVersion = static_cast<EHandshakeVersion>(CurrentHandshakeVersion);

	for (EHandshakeVersion CurBreakVersion : HandshakeCompatibilityBreaks)
	{
		if (CurBreakVersion > OutTargetVersion)
		{
			// A maximum version is only enforced if we're aware of a net compatibility breakage in a higher version
			LocalMaxVersion = static_cast<EHandshakeVersion>(static_cast<uint8>(CurBreakVersion) - 1);
			bHasMaxVersion = true;

			break;
		}
	}

	if (RemoteMinVersionUint8 >= MinSupportedHandshakeVersion && HandshakeData.RemoteMinVersion <= OutTargetVersion
		&& HandshakeData.RemoteMinVersion <= HandshakeData.RemoteCurVersion &&
		(!bHasMaxVersion || HandshakeData.RemoteCurVersion <= LocalMaxVersion))
	{
		if (HandshakeData.RemoteCurVersion <= OutTargetVersion)
		{
			OutTargetVersion = HandshakeData.RemoteCurVersion;
		}

		bValidHandshakeVersion = true;
	}

	const EEngineNetworkRuntimeFeatures LocalNetworkFeatures = Driver ? Driver->GetNetworkRuntimeFeatures() : EEngineNetworkRuntimeFeatures::None;
	bool bUEBuildShipping = UE_BUILD_SHIPPING;
	const bool bCheckNetVersion = !bUEBuildShipping || GHandshakeEnforceNetworkCLVersion;
	bool bValidNetCLVersion = true;
	bool bIsNetFeatureCompatible = true;

	if (HandshakeData.RemoteCurVersion >= EHandshakeVersion::NetCLVersion)
	{
		if (bCheckNetVersion)
		{
			const uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();
			bValidNetCLVersion = FNetworkVersion::IsNetworkCompatible(LocalNetworkVersion, HandshakeData.RemoteNetworkVersion);
		}

		// Network Runtime Features are always validated since the client can try to upgrade his NetDriver and become compatible
		bIsNetFeatureCompatible = FNetworkVersion::AreNetworkRuntimeFeaturesCompatible(LocalNetworkFeatures, HandshakeData.RemoteNetworkFeatures);
	}

#if !UE_BUILD_SHIPPING
	if (!bValidHandshakeVersion || !bValidNetCLVersion || !bIsNetFeatureCompatible)
	{
		FDDoSDetection* DDoS = Handler->GetDDoS();
		const uint32 LocalNetworkVersion = FNetworkVersion::GetLocalNetworkVersion();
		TStringBuilder<128> LocalNetFeaturesDescription;
		TStringBuilder<128> RemoteNetFeaturesDescription;

		FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(LocalNetworkFeatures, LocalNetFeaturesDescription);
		FNetworkVersion::DescribeNetworkRuntimeFeaturesBitset(HandshakeData.RemoteNetworkFeatures, RemoteNetFeaturesDescription);

		UE_CLOGF((DDoS == nullptr || !DDoS->CheckLogRestrictions()), LogHandshake, Log,
				"CheckVersion: Incompatible version. bValidHandshakeVersion: %i, bValidNetVersion: %i, " "GHandshakeEnforceNetworkCLVersion: %i, RemoteMinVersion: %u, RemoteCurVersion: %u, MinSupportedHandshakeVersion: %i, " "CurrentHandshakeVersion: %i, RemoteNetworkVersion: %u, LocalNetworkVersion: %u, RemoteNetworkFeatures: %ls, " "LocalNetworkFeatures: %ls",
				(int32)bValidHandshakeVersion, (int32)bValidNetCLVersion, GHandshakeEnforceNetworkCLVersion, RemoteMinVersionUint8,
				static_cast<uint8>(HandshakeData.RemoteCurVersion), MinSupportedHandshakeVersion, CurrentHandshakeVersion,
				HandshakeData.RemoteNetworkVersion, LocalNetworkVersion, RemoteNetFeaturesDescription.ToString(), 
				LocalNetFeaturesDescription.ToString());
	}
#endif

	const bool bPassedNetVersionConditions = bValidNetCLVersion || !GHandshakeEnforceNetworkCLVersion;

	return bValidHandshakeVersion && bPassedNetVersionConditions && bIsNetFeatureCompatible;
}

void StatelessConnectHandlerComponent::GenerateCookie(const TSharedPtr<const FInternetAddr>& ClientAddress, uint8 SecretId, double Timestamp,
														uint8 (&OutCookie)[20]) const
{
	TArray<uint8> CookieData;
	FMemoryWriter CookieArc(CookieData);
	FString ClientAddressString(ClientAddress->ToString(true));

	CookieArc << Timestamp;
	CookieArc << ClientAddressString;

	FSHA1::HMACBuffer(HandshakeSecret[!!SecretId].GetData(), SECRET_BYTE_SIZE, CookieData.GetData(), CookieData.Num(), OutCookie);
}

void StatelessConnectHandlerComponent::UpdateSecret()
{
	LastSecretUpdateTimestamp = Driver != nullptr ? Driver->GetElapsedTime() : 0.0;

	// On first update, update both secrets
	if (ActiveSecret == 255)
	{
		// NOTE: The size of this may be excessive.
		HandshakeSecret[0].AddUninitialized(SECRET_BYTE_SIZE);
		HandshakeSecret[1].AddUninitialized(SECRET_BYTE_SIZE);

		TArray<uint8>& CurArray = HandshakeSecret[1];

		for (int32 i=0; i<SECRET_BYTE_SIZE; i++)
		{
			CurArray[i] = FMath::Rand() % 255;
		}

		ActiveSecret = 0;
	}
	else
	{
		ActiveSecret = (uint8)!ActiveSecret;
	}

	TArray<uint8>& CurArray = HandshakeSecret[ActiveSecret];

	for (int32 i=0; i<SECRET_BYTE_SIZE; i++)
	{
		CurArray[i] = FMath::Rand() % 255;
	}
}

int32 StatelessConnectHandlerComponent::GetReservedPacketBits() const
{
	// Count all base bit additions which affect NetConnection packets, regardless of handshake protocol version - as this is called upon construction
	int32 ReturnVal = MagicHeader.Num() + SessionIDSizeBits + ClientIDSizeBits + 1 /* bHandshakePacket */;

#if !UE_BUILD_SHIPPING
	SET_DWORD_STAT(STAT_PacketReservedHandshake, ReturnVal);
#endif

	return ReturnVal;
}

int32 StatelessConnectHandlerComponent::GetAdjustedSizeBits(int32 InSizeBits, EHandshakeVersion HandshakeVersion) const
{
	int32 ReturnVal = MagicHeader.Num() + InSizeBits;

	if (HandshakeVersion >= EHandshakeVersion::SessionClientId)
	{
		ReturnVal += SessionIDSizeBits + ClientIDSizeBits;
	}

	return ReturnVal;
}

void StatelessConnectHandlerComponent::Tick(float DeltaTime)
{
	using namespace UE::Net;

	if (Handler->Mode == UE::Handler::Mode::Client)
	{
		if (State != UE::Handler::Component::State::Initialized && LastClientSendTimestamp != UninitializedLastClientSendTimestamp)
		{
			ensureMsgf(Driver, TEXT("StatelessConnectHandlerComponent::Tick: Driver is null."));
			const double CurrentTime = Driver ? Driver->GetElapsedTime() : 0.0;

			const double LastSendTimeDiff = CurrentTime - LastClientSendTimestamp;

			if (LastSendTimeDiff > UE::Net::HandshakeResendInterval)
			{
				const bool bRestartChallenge = Driver != nullptr && ((Driver->GetElapsedTime() - LastChallengeTimestamp) > MIN_COOKIE_LIFETIME);

				if (bRestartChallenge)
				{
					SetState(UE::Handler::Component::State::UnInitialized);
				}

				if (State == UE::Handler::Component::State::UnInitialized)
				{
					UE_LOGF(LogHandshake, Verbose, "Initial handshake packet timeout - resending.");

					EHandshakeVersion ResendVersion = static_cast<EHandshakeVersion>(CurrentHandshakeVersion);

					// In case the server doesn't support the current handshake version, randomly switch between supported versions - if enabled
					// (we don't know if the server supports the minimum version either, so pick from the full range).
					// It's better for devs to explicitly hotfix the 'net.MinHandshakeVersion' value, instead of relying upon this fallback.
					if (!!CVarNetDoHandshakeVersionFallback.GetValueOnAnyThread() && FMath::RandBool())
					{
						// Decrement the minimum version, based on the number of handshake packets sent - to select for higher supported versions
						const int32 MinVersion = FMath::Max(MinSupportedHandshakeVersion, CurrentHandshakeVersion - SentHandshakePacketCount);

						if (MinVersion != CurrentHandshakeVersion)
						{
							ResendVersion = static_cast<EHandshakeVersion>(FMath::RandRange(MinVersion, CurrentHandshakeVersion));
						}
					}

					SendInitialPacket(ResendVersion);
				}
				else if (State == UE::Handler::Component::State::InitializedOnLocal && LastTimestamp != 0.0)
				{
					UE_LOGF(LogHandshake, Verbose, "Challenge response packet timeout - resending.");

					SendChallengeResponse(LastRemoteHandshakeVersion, LastSecretId, LastTimestamp, LastCookie);
				}
			}
		}
	}
	else // if (Handler->Mode == Handler::Mode::Server)
	{
		const bool bConnectionlessHandler = Driver != nullptr && Driver->StatelessConnectComponent.HasSameObject(this);

		if (bConnectionlessHandler)
		{
			static float CurVariance = FMath::FRandRange(0.f, SECRET_UPDATE_TIME_VARIANCE);

			// Update the secret value periodically, to reduce replay attacks. Also adds a bit of randomness to the timing of this,
			// so that handshake Timestamp checking as an added method of reducing replay attacks, is more effective.
			if (((Driver->GetElapsedTime() - LastSecretUpdateTimestamp) - (SECRET_UPDATE_TIME + CurVariance)) > 0.0)
			{
				CurVariance = FMath::FRandRange(0.f, SECRET_UPDATE_TIME_VARIANCE);

				UpdateSecret();
			}
		}
	}
}

#if !UE_BUILD_SHIPPING
bool StatelessConnectHandlerComponent::TrackValidationLogs()
{
	const uint32 NumValidationLogsPerPeriod = 10;
	const double ValidationLogPeriod = 30.0;
	const double ElapsedTime = Driver->GetElapsedTime();

	if (ElapsedTime - LastValidationLogPeriodStart >= ValidationLogPeriod)
	{
		LastValidationLogPeriodStart = ElapsedTime;
		ValidationLogCounter = 0;
	}
	else
	{
		ValidationLogCounter++;
	}

	return ValidationLogCounter < NumValidationLogsPerPeriod;
}
#endif

