// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PainCausingVolume.h"
#include "TimerManager.h"
#include "Engine/DamageEvents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PainCausingVolume)

APainCausingVolume::APainCausingVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bPainCausing = true;
	DamageType = UDamageType::StaticClass();
	DamagePerSec = 1.0f;
	bEntryPain = true;
	PainInterval = 1.0f;
}

void APainCausingVolume::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	BACKUP_bPainCausing	= bPainCausing;
}

void APainCausingVolume::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	
	GetWorldTimerManager().ClearTimer(TimerHandle_PainTimer);
}

void APainCausingVolume::Reset()
{
	bPainCausing = BACKUP_bPainCausing;
	ForceNetUpdate();
}

void APainCausingVolume::ActorEnteredVolume(AActor* Other)
{
	Super::ActorEnteredVolume(Other);
	if ( bPainCausing && bEntryPain && Other->CanBeDamaged() )
	{
		CausePainTo(Other);
	}

	// Start timer if none is active
	if (!GetWorldTimerManager().IsTimerActive(TimerHandle_PainTimer))
	{
		GetWorldTimerManager().SetTimer(TimerHandle_PainTimer, this, &APainCausingVolume::PainTimer, PainInterval, true);
	}
}

void APainCausingVolume::PainTimer()
{
	if (bPainCausing)
	{
		TSet<AActor*> TouchingActors;
		GetOverlappingActors(TouchingActors);

		for (AActor* const A : TouchingActors)
		{
			if (IsValid(A) && A->CanBeDamaged() && A->GetPhysicsVolume() == this)
			{
				CausePainTo(A);
			}
		}

		// Stop timer if nothing is overlapping us
		if (TouchingActors.Num() == 0)
		{
			GetWorldTimerManager().ClearTimer(TimerHandle_PainTimer);
		}
	}
}

void APainCausingVolume::CausePainTo(AActor* Other)
{
	if (DamagePerSec > 0.f)
	{
		TSubclassOf<UDamageType> DmgTypeClass = DamageType ? *DamageType : UDamageType::StaticClass();
		Other->TakeDamage(DamagePerSec*PainInterval, FDamageEvent(DmgTypeClass), DamageInstigator, this);
	}
}


