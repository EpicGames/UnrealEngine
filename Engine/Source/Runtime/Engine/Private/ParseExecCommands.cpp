// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParseExecCommands.h"
#include "Containers/UnrealString.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Engine/Engine.h"

namespace ParseExecCommands
{
	TArray<FString> ParseExecCmds(const FString& Line)
	{
		// Read the command array, ignoring any commas in single-quotes. 
				// Convert any single-quotes to double-quotes and skip leading whitespace
				// This allows passing of strings, e:g -execcmds="exampleCvar '0,1,2,3'"
		TArray<FString> CommandArray;
		FString CurrentCommand;
		bool bInQuotes = false;
		bool bSkippingWhitespace = true;
		for (int i = 0; i < Line.Len(); i++)
		{
			TCHAR CurrentChar = Line[i];
			if (CurrentChar == '\'')
			{
				bInQuotes = !bInQuotes;
				CurrentCommand += "\"";
			}
			else if (CurrentChar == ',' && !bInQuotes)
			{
				if (CurrentCommand.Len() > 0)
				{
					CommandArray.Add(CurrentCommand);
					CurrentCommand = "";
				}
				bSkippingWhitespace = true;
			}
			else
			{
				if (bSkippingWhitespace)
				{
					bSkippingWhitespace = FChar::IsWhitespace(CurrentChar);
				}
				if (!bSkippingWhitespace)
				{
					CurrentCommand += CurrentChar;
				}
			}
		}
		if (CurrentCommand.Len() > 0)
		{
			CommandArray.Add(CurrentCommand);
		}

		return CommandArray;
	}

	TArray<FString> ParseExecCmdsFromCommandLine(const FString& InKey)
	{
		TArray<FString> Result;

		FString Line;
		const TCHAR* CommandLine = FCommandLine::Get();
		while (CommandLine && FParse::Value(CommandLine, *(InKey + TEXT("=")), Line, /*bShouldStopOnSeparator*/false, &CommandLine))
		{
			Result.Append(ParseExecCmds(Line));
		}

		return Result;
	}

	void QueueDeferredCommands(const TArray<FString>& CommandArray)
	{
		for (const FString& Command : CommandArray)
		{
			GEngine->DeferredCommands.Add(Command);
		}
	}

	void QueueDeferredCommands(const FString& Line)
	{
		QueueDeferredCommands(ParseExecCmds(Line));
	}
}
