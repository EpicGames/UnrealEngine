// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParticleSystemSceneProxy.h"
#include "InGamePerformanceTracker.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "ParticleHelper.h"
#include "Particles/ParticleSystemComponent.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "RenderCore.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "UnrealEngine.h"
#include "PSOPrecacheSettings.h"

#if WITH_EDITOR
#include "ParticleEmitterInstances.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "UObject/UObjectIterator.h"
#endif

DECLARE_CYCLE_STAT(TEXT("ParticleSystemSceneProxy Create GT"), STAT_FParticleSystemSceneProxy_Create, STATGROUP_Particles);
DECLARE_CYCLE_STAT(TEXT("ParticleSystemSceneProxy GetMeshElements RT"), STAT_FParticleSystemSceneProxy_GetMeshElements, STATGROUP_Particles);

/** 
 * Whether to track particle rendering stats.  
 * Enable with the TRACKPARTICLERENDERINGSTATS command. 
 */
bool GTrackParticleRenderingStats = false;

/** Whether to do LOD calculation on GameThread in game */
extern bool GbEnableGameThreadLODCalculation;

///////////////////////////////////////////////////////////////////////////////
//	FParticleSystemSceneProxyDesc
///////////////////////////////////////////////////////////////////////////////

FParticleSystemSceneProxyDesc::FParticleSystemSceneProxyDesc()
{
	SystemAsset = nullptr;
	DynamicData = nullptr;
	VisualizeLODIndex = 0;
	LODMethod = 0;
	bCanBeOccluded = true;
	bManagingSignificance = false;
	bAlwaysHasVelocity = false;
}

FParticleSystemSceneProxyDesc::FParticleSystemSceneProxyDesc(UParticleSystemComponent& Component, FParticleDynamicData* InDynamicData, bool InbCanBeOccluded)
:	FPrimitiveSceneProxyDesc(&Component)
{
	SystemAsset = Component.Template;
	DynamicData = InDynamicData;

	if (Component.GetCurrentLODIndex() >= 0 && Component.GetCurrentLODIndex() < Component.CachedViewRelevanceFlags.Num())
	{
		MaterialRelevance = Component.CachedViewRelevanceFlags[Component.GetCurrentLODIndex()];
	}
	else if (Component.GetCurrentLODIndex() == -1 && Component.CachedViewRelevanceFlags.Num() >= 1)
	{
		MaterialRelevance = Component.CachedViewRelevanceFlags[0];
	}

	VisualizeLODIndex = Component.GetCurrentLODIndex();
	LODMethod = Component.LODMethod;
	bCanBeOccluded = InbCanBeOccluded;
	bManagingSignificance = Component.ShouldManageSignificance();
	bAlwaysHasVelocity = SystemAsset && SystemAsset->DoesAnyEmitterHaveMotionBlur(Component.GetCurrentLODIndex());
}

void FParticleSystemSceneProxyDesc::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	if (Component)
	{
		FPrimitiveSceneProxyDesc::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
	}
	else
	{
		// TODO
	}
}

///////////////////////////////////////////////////////////////////////////////
//	ParticleSystemSceneProxy
///////////////////////////////////////////////////////////////////////////////

FParticleSystemSceneProxy::FParticleSystemSceneProxy(const FParticleSystemSceneProxyDesc& Desc)
	: FPrimitiveSceneProxy(Desc, Desc.SystemAsset ? Desc.SystemAsset->GetFName() : NAME_None)
	, bCastShadow(Desc.CastShadow)
	, bManagingSignificance(Desc.bManagingSignificance)
	, bCanBeOccluded(Desc.bCanBeOccluded)
	, bHasCustomOcclusionBounds(false)
	, FeatureLevel(GetScene().GetFeatureLevel())
	, MaterialRelevance(Desc.MaterialRelevance)
	, DynamicData(Desc.DynamicData)
	, LastDynamicData(NULL)
	, DeselectedWireframeMaterialInstance(new FColoredMaterialRenderProxy(
		GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : NULL,
		GetSelectionColor(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f),false,false)
		))
	, PendingLODDistance(0.0f)
	, VisualizeLODIndex(Desc.VisualizeLODIndex)
	, LastFramePreRendered(-1)
	, FirstFreeMeshBatch(0)
{
	SetWireframeColor(FLinearColor(3.0f, 0.0f, 0.0f));

	LODMethod = Desc.LODMethod;

	// Particle systems intrinsically always have motion, but is this motion relevant to systems external to particle systems?
	bAlwaysHasVelocity = Desc.bAlwaysHasVelocity;

	if (bCanBeOccluded && Desc.SystemAsset && (Desc.SystemAsset->OcclusionBoundsMethod == EPSOBM_CustomBounds))
	{
		OcclusionBounds = FBoxSphereBounds(Desc.SystemAsset->CustomOcclusionBounds);
		bHasCustomOcclusionBounds = true;
	}
}

SIZE_T FParticleSystemSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FParticleSystemSceneProxy::~FParticleSystemSceneProxy()
{
	ReleaseRenderThreadResources();

	delete DynamicData;
	DynamicData = NULL;

	delete DeselectedWireframeMaterialInstance;
	DeselectedWireframeMaterialInstance = nullptr;
}

FMeshBatch* FParticleSystemSceneProxy::GetPooledMeshBatch()
{
	FMeshBatch* Batch = NULL;
	if (FirstFreeMeshBatch < MeshBatchPool.Num())
	{
		Batch = &MeshBatchPool[FirstFreeMeshBatch];
	}
	else
	{
		Batch = new FMeshBatch();
		MeshBatchPool.Add(Batch);
	}
	FirstFreeMeshBatch++;
	return Batch;
}

// FPrimitiveSceneProxy interface.

void FParticleSystemSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	FInGameScopedCycleCounter InGameCycleCounter(GetScene().GetWorld(), EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::RenderThread, bManagingSignificance);

	SCOPE_CYCLE_COUNTER(STAT_FParticleSystemSceneProxy_GetMeshElements);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_RT);
	PARTICLE_PERF_STAT_CYCLES_RT(PerfStatContext, GetDynamicMeshElements);

	if ((GIsEditor == true) || (GbEnableGameThreadLODCalculation == false))
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				//@todo parallelrendering - get rid of this legacy feedback to the game thread!  
				const_cast<FParticleSystemSceneProxy*>(this)->DetermineLODDistance(View, ViewFamily.FrameNumber);
			}
		}
	}

	if (ViewFamily.EngineShowFlags.Particles)
	{
		SCOPE_CYCLE_COUNTER(STAT_ParticleRenderingTime);
		FScopeCycleCounter Context(GetStatId());

		const double StartTime = GTrackParticleRenderingStats ? FPlatformTime::Seconds() : 0;
		int32 NumDraws = 0;

		if (DynamicData != NULL)
		{
			for (int32 Index = 0; Index < DynamicData->DynamicEmitterDataArray.Num(); Index++)
			{
				FDynamicEmitterDataBase* Data =	DynamicData->DynamicEmitterDataArray[Index];
				if ((Data == NULL) || (Data->bValid != true))
				{
					continue;
				}
				FScopeCycleCounter AdditionalScope(Data->StatID);

				//hold on to the emitter index in case we need to access any of its properties
				DynamicData->EmitterIndex = Index;

				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if (VisibilityMap & (1 << ViewIndex))
					{
						const FSceneView* View = Views[ViewIndex];
						Data->GetDynamicMeshElementsEmitter(this, View, ViewFamily, ViewIndex, Collector);
						NumDraws++;
					}
				}
			}
		}

		INC_DWORD_STAT_BY(STAT_ParticleDrawCalls, NumDraws);

		if (ViewFamily.EngineShowFlags.Particles)
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
					if (HasCustomOcclusionBounds())
					{
						RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetCustomOcclusionBounds(), IsSelected());
					}
				}
			}
		}
	}
}

void FParticleSystemSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	CreateRenderThreadResourcesForEmitterData();
}

void FParticleSystemSceneProxy::ReleaseRenderThreadResources()
{
	ReleaseRenderThreadResourcesForEmitterData();
	WorldSpacePrimitiveUniformBuffer.ReleaseResource();
}

void FParticleSystemSceneProxy::CreateRenderThreadResourcesForEmitterData()
{
	if (DynamicData != nullptr)
	{
		for (int32 Index = 0; Index < DynamicData->DynamicEmitterDataArray.Num(); Index++)
		{
			FDynamicEmitterDataBase* Data =	DynamicData->DynamicEmitterDataArray[Index];
			if (Data != NULL)
			{
				FScopeCycleCounter AdditionalScope(Data->StatID);
				Data->UpdateRenderThreadResourcesEmitter(this);
			}
		}
	}
}

void FParticleSystemSceneProxy::ReleaseRenderThreadResourcesForEmitterData()
{
	if (DynamicData)
	{
		for (int32 Index = 0; Index < DynamicData->DynamicEmitterDataArray.Num(); Index++)
		{
			FDynamicEmitterDataBase* Data =	DynamicData->DynamicEmitterDataArray[Index];
			if (Data != NULL)
			{
				FScopeCycleCounter AdditionalScope(Data->StatID);
				Data->ReleaseRenderThreadResources(this);
			}
		}
	}
}

void FParticleSystemSceneProxy::UpdateData(FParticleDynamicData* NewDynamicData)
{
	FParticleSystemSceneProxy* Proxy = this;
	ENQUEUE_RENDER_COMMAND(ParticleUpdateDataCommand)(
		[Proxy, NewDynamicData](FRHICommandListImmediate& RHICmdList)
		{
		#if WITH_PARTICLE_PERF_STATS
			Proxy->PerfStatContext = NewDynamicData ? NewDynamicData->PerfStatContext : FParticlePerfStatsContext();
		#endif
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ParticleUpdate);
			SCOPE_CYCLE_COUNTER(STAT_ParticleUpdateRTTime);
			STAT(FScopeCycleCounter Context(Proxy->GetStatId());)
			PARTICLE_PERF_STAT_CYCLES_WITH_COUNT_RT(Proxy->PerfStatContext, RenderUpdate, 1);

			Proxy->UpdateData_RenderThread(NewDynamicData);
		}
	);
}

void FParticleSystemSceneProxy::UpdateData_RenderThread(FParticleDynamicData* NewDynamicData)
{
	FInGameScopedCycleCounter InGameCycleCounter(GetScene().GetWorld(), EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::RenderThread, bManagingSignificance);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_RT);

	ReleaseRenderThreadResourcesForEmitterData();
	if (DynamicData != NewDynamicData)
	{
		delete DynamicData;
	}
	DynamicData = NewDynamicData;
	CreateRenderThreadResourcesForEmitterData();
}

void FParticleSystemSceneProxy::DetermineLODDistance(const FSceneView* View, int32 FrameNumber)
{
	if (LODMethod == PARTICLESYSTEMLODMETHOD_Automatic)
	{
		// Default to the highest LOD level
		FVector	CameraPosition		= View->ViewMatrices.GetViewOrigin();
		FVector	ComponentPosition	= GetLocalToWorld().GetOrigin();
		FVector	DistDiff			= ComponentPosition - CameraPosition;
		float	Distance			= DistDiff.Size() * View->LODDistanceFactor;

		if (FrameNumber != LastFramePreRendered)
		{
			// First time in the frame - then just set it...
			PendingLODDistance = Distance;
			LastFramePreRendered = FrameNumber;
		}
		else if (Distance < PendingLODDistance)
		{
			// Not first time in the frame, then we compare and set if closer
			PendingLODDistance = Distance;
		}
	}
}

int32 GEnableMacroUVDebugSpam = 1;
static FAutoConsoleVariableRef EnableMacroUVDebugSpam(
	TEXT("r.EnableDebugSpam_GetObjectPositionAndScale"),
	GEnableMacroUVDebugSpam,
	TEXT("Enables or disables debug log spam for a bug in FParticleSystemSceneProxy::GetObjectPositionAndScale()")
	);

/** Object position in post projection space. */
void FParticleSystemSceneProxy::GetObjectPositionAndScale(const FSceneView& View, FVector2D& ObjectNDCPosition, FVector2D& ObjectMacroUVScales) const
{
	const FVector4 ObjectPostProjectionPositionWithW = View.ViewMatrices.GetWorldToClip().TransformPosition(DynamicData->SystemPositionForMacroUVs);
	ObjectNDCPosition = FVector2D(ObjectPostProjectionPositionWithW / FMath::Max(ObjectPostProjectionPositionWithW.W, 0.00001f));
	
	float MacroUVRadius = DynamicData->SystemRadiusForMacroUVs;
	FVector MacroUVPosition = DynamicData->SystemPositionForMacroUVs;
   
	uint32 Index = DynamicData->EmitterIndex;
	const FMacroUVOverride& MacroUVOverride = DynamicData->DynamicEmitterDataArray[Index]->GetMacroUVOverride();
	if (MacroUVOverride.bOverride)
	{
		MacroUVRadius = MacroUVOverride.Radius;
		MacroUVPosition = GetLocalToWorld().TransformVector((FVector)MacroUVOverride.Position);

#if !(UE_BUILD_SHIPPING)
		if (MacroUVPosition.ContainsNaN())
		{
			UE_LOGF(LogParticles, Error, "MacroUVPosition.ContainsNaN()");
		}
#endif
	}

	ObjectMacroUVScales = FVector2D(0,0);
	if (MacroUVRadius > 0.0f)
	{
		// Need to determine the scales required to transform positions into UV's for the ParticleMacroUVs material node
		// Determine screenspace extents by transforming the object position + appropriate camera vector * radius
		const FVector4 RightPostProjectionPosition = View.ViewMatrices.GetWorldToClip().TransformPosition(MacroUVPosition + MacroUVRadius * View.ViewMatrices.GetTranslatedViewMatrix().GetColumn(0));
		const FVector4 UpPostProjectionPosition = View.ViewMatrices.GetWorldToClip().TransformPosition(MacroUVPosition + MacroUVRadius * View.ViewMatrices.GetTranslatedViewMatrix().GetColumn(1));
		//checkSlow(RightPostProjectionPosition.X - ObjectPostProjectionPositionWithW.X >= 0.0f && UpPostProjectionPosition.Y - ObjectPostProjectionPositionWithW.Y >= 0.0f);

		// Scales to transform the view space positions corresponding to SystemPositionForMacroUVs +- SystemRadiusForMacroUVs into [0, 1] in xy
		// Scales to transform the screen space positions corresponding to SystemPositionForMacroUVs +- SystemRadiusForMacroUVs into [0, 1] in zw

		const FVector4::FReal RightNDCPosX = RightPostProjectionPosition.X / RightPostProjectionPosition.W;
		const FVector4::FReal UpNDCPosY = UpPostProjectionPosition.Y / UpPostProjectionPosition.W;
		FVector4::FReal DX = FMath::Min(RightNDCPosX - ObjectNDCPosition.X, WORLD_MAX);
		FVector4::FReal DY = FMath::Min(UpNDCPosY - ObjectNDCPosition.Y, WORLD_MAX);
		if (DX != 0 && DY != 0 && !FMath::IsNaN(DX) && FMath::IsFinite(DX) && !FMath::IsNaN(DY) && FMath::IsFinite(DY))
		{
			ObjectMacroUVScales = FVector2D(1.0f / DX, -1.0f / DY);
		}
		else
		{
			//Spam the logs to track down infrequent / hard to repro bug.
			if (GEnableMacroUVDebugSpam != 0)
			{
				UE_LOGF(LogParticles, Error, "Bad values in FParticleSystemSceneProxy::GetObjectPositionAndScale");
				UE_LOGF(LogParticles, Error, "SystemPositionForMacroUVs: {%.6f, %.6f, %.6f}", DynamicData->SystemPositionForMacroUVs.X, DynamicData->SystemPositionForMacroUVs.Y, DynamicData->SystemPositionForMacroUVs.Z);
				UE_LOGF(LogParticles, Error, "ObjectPostProjectionPositionWithW: {%.6f, %.6f, %.6f, %.6f}", ObjectPostProjectionPositionWithW.X, ObjectPostProjectionPositionWithW.Y, ObjectPostProjectionPositionWithW.Z, ObjectPostProjectionPositionWithW.W);
				UE_LOGF(LogParticles, Error, "RightPostProjectionPosition: {%.6f, %.6f, %.6f, %.6f}", RightPostProjectionPosition.X, RightPostProjectionPosition.Y, RightPostProjectionPosition.Z, RightPostProjectionPosition.W);
				UE_LOGF(LogParticles, Error, "UpPostProjectionPosition: {%.6f, %.6f, %.6f, %.6f}", UpPostProjectionPosition.X, UpPostProjectionPosition.Y, UpPostProjectionPosition.Z, UpPostProjectionPosition.W);
				UE_LOGF(LogParticles, Error, "ObjectNDCPosition: {%.6f, %.6f}", ObjectNDCPosition.X, ObjectNDCPosition.Y);
				UE_LOGF(LogParticles, Error, "RightNDCPosX: %.6f", RightNDCPosX);
				UE_LOGF(LogParticles, Error, "UpNDCPosY: %.6f", UpNDCPosY);
				UE_LOGF(LogParticles, Error, "MacroUVPosition: {%.6f, %.6f, %.6f}", MacroUVPosition.X, MacroUVPosition.Y, MacroUVPosition.Z);
				UE_LOGF(LogParticles, Error, "MacroUVRadius: %.6f", MacroUVRadius);
				UE_LOGF(LogParticles, Error, "DX: %.6f", DX);
				UE_LOGF(LogParticles, Error, "DY: %.6f", DY);
				FVector4 View0 = View.ViewMatrices.GetWorldToView().GetColumn(0);
				FVector4 View1 = View.ViewMatrices.GetWorldToView().GetColumn(1);
				FVector4 View2 = View.ViewMatrices.GetWorldToView().GetColumn(2);
				FVector4 View3 = View.ViewMatrices.GetWorldToView().GetColumn(3);
				UE_LOGF(LogParticles, Error, "View0: {%.6f, %.6f, %.6f, %.6f}", View0.X, View0.Y, View0.Z, View0.W);
				UE_LOGF(LogParticles, Error, "View1: {%.6f, %.6f, %.6f, %.6f}", View1.X, View1.Y, View1.Z, View1.W);
				UE_LOGF(LogParticles, Error, "View2: {%.6f, %.6f, %.6f, %.6f}", View2.X, View2.Y, View2.Z, View2.W);
				UE_LOGF(LogParticles, Error, "View3: {%.6f, %.6f, %.6f, %.6f}", View3.X, View3.Y, View3.Z, View3.W);
				FVector4 ViewProj0 = View.ViewMatrices.GetWorldToClip().GetColumn(0);
				FVector4 ViewProj1 = View.ViewMatrices.GetWorldToClip().GetColumn(1);
				FVector4 ViewProj2 = View.ViewMatrices.GetWorldToClip().GetColumn(2);
				FVector4 ViewProj3 = View.ViewMatrices.GetWorldToClip().GetColumn(3);
				UE_LOGF(LogParticles, Error, "ViewProj0: {%.6f, %.6f, %.6f, %.6f}", ViewProj0.X, ViewProj0.Y, ViewProj0.Z, ViewProj0.W);
				UE_LOGF(LogParticles, Error, "ViewProj1: {%.6f, %.6f, %.6f, %.6f}", ViewProj1.X, ViewProj1.Y, ViewProj1.Z, ViewProj1.W);
				UE_LOGF(LogParticles, Error, "ViewProj2: {%.6f, %.6f, %.6f, %.6f}", ViewProj2.X, ViewProj2.Y, ViewProj2.Z, ViewProj2.W);
				UE_LOGF(LogParticles, Error, "ViewProj3: {%.6f, %.6f, %.6f, %.6f}", ViewProj3.X, ViewProj3.Y, ViewProj3.Z, ViewProj3.W);
			}
		}
	}
}

/**
* @return Relevance for rendering the particle system primitive component in the given View
*/
FPrimitiveViewRelevance FParticleSystemSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.Particles;
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
	Result.bDynamicRelevance = true;
	Result.bHasSimpleLights = true;
	if (!View->Family->EngineShowFlags.Wireframe && View->Family->EngineShowFlags.Materials)
	{
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
	}
	if (View->Family->EngineShowFlags.Bounds || View->Family->EngineShowFlags.VectorFields)
	{
		Result.bOpaque = true;
	}
	// see if any of the emitters use dynamic vertex data
	if (DynamicData == NULL)
	{
		// In order to get the LOD distances to update,
		// we need to force a call to DrawDynamicElements...
		Result.bOpaque = true;
	}

	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

	return Result;
}

void FParticleSystemSceneProxy::OnTransformChanged(FRHICommandListBase& RHICmdList)
{
	WorldSpacePrimitiveUniformBuffer.ReleaseResource();
	WorldSpaceUBHash = 0;
}

void FParticleSystemSceneProxy::UpdateWorldSpacePrimitiveUniformBuffer(FRHICommandListBase& RHICmdList) const
{
	// Hash custom floats because we need to invalidate this UB if they don't match otherwise updates to the buffer won't work
	uint32 NewWorldSpaceUBHash = 0;
	const FCustomPrimitiveData* LocalCustomPrimitiveData = GetCustomPrimitiveData();
	if (LocalCustomPrimitiveData && LocalCustomPrimitiveData->Data.Num())
	{
		NewWorldSpaceUBHash = FCrc::MemCrc32(LocalCustomPrimitiveData->Data.GetData(), LocalCustomPrimitiveData->Data.Num() * LocalCustomPrimitiveData->Data.GetTypeSize());
	}

	UE::TScopeLock Lock(WorldSpacePrimitiveUniformBufferMutex);

	const bool bNeedsInit = !WorldSpacePrimitiveUniformBuffer.IsInitialized();

	if (bNeedsInit || (WorldSpaceUBHash != NewWorldSpaceUBHash))
	{
		WorldSpaceUBHash = NewWorldSpaceUBHash;
		WorldSpacePrimitiveUniformBuffer.SetContents(
			RHICmdList,
			FPrimitiveUniformShaderParametersBuilder{}
			.Defaults()
				.LocalToWorld(FMatrix::Identity)
				.ActorWorldPosition(GetActorPosition())
				.WorldBounds(GetBounds())
				.LocalBounds(GetLocalBounds())
				.ReceivesDecals(ReceivesDecals())
				.OutputVelocity(AlwaysHasVelocity())
				.LightingChannelMask(GetLightingChannelMask())
				.UseSingleSampleShadowFromStationaryLights(UseSingleSampleShadowFromStationaryLights())
				.UseVolumetricLightmap(GetScene().HasPrecomputedVolumetricLightmap_RenderThread())
				.CustomPrimitiveData(GetCustomPrimitiveData())
				.HasPixelAnimation(AnyMaterialHasPixelAnimation())
				.IsFirstPerson(IsFirstPerson())
			.Build()
		);
	}

	if ( bNeedsInit)
	{
		WorldSpacePrimitiveUniformBuffer.InitResource(RHICmdList);
	}
}

void FParticleSystemSceneProxy::GatherSimpleLights(const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const
{
	FInGameScopedCycleCounter InGameCycleCounter(GetScene().GetWorld(), EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::RenderThread, bManagingSignificance);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_RT);
	if (DynamicData != NULL)
	{
		FScopeCycleCounter Context(GetStatId());
		for (int32 EmitterIndex = 0; EmitterIndex < DynamicData->DynamicEmitterDataArray.Num(); EmitterIndex++)
		{
			const FDynamicEmitterDataBase* DynamicEmitterData = DynamicData->DynamicEmitterDataArray[EmitterIndex];
			if (DynamicEmitterData)
			{
				FScopeCycleCounter AdditionalScope(DynamicEmitterData->StatID);
				DynamicEmitterData->GatherSimpleLights(this, ViewFamily, OutParticleLights);
			}
		}
	}
}

FPrimitiveSceneProxy* UParticleSystemComponent::CreateSceneProxy()
{
	SCOPE_CYCLE_COUNTER(STAT_FParticleSystemSceneProxy_Create);
	SCOPE_CYCLE_COUNTER(STAT_ParticlesOverview_GT);

	FParticleSystemSceneProxy* NewProxy = NULL;

	//@fixme EmitterInstances.Num() check should be here to avoid proxies for dead emitters but there are some edge cases where it happens for emitters that have just activated...
	//@fixme Get non-instanced path working in ES!
	if ((IsActive() == true)/** && (EmitterInstances.Num() > 0)*/ && Template)
	{
#if UE_WITH_PSO_PRECACHING
		if (!PSOPrecacheComponentData.IsPSOPrecacheCalled())
		{
			if (UPSOPrecacheSettingsManager::GetSettings().ParticlePrecachingTime == 1)
			{
				Template->PrecacheAssetPSOs();
			}
			PrecacheAssetPSOs(Template);
		}

		if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
		{
			UE_LOGF(LogParticles, Verbose, "Skipping CreateSceneProxy for UParticleSystemComponent %ls (UParticleSystem PSOs are still compiling)", *GetFullName());
			return nullptr;
		}
#endif // UE_WITH_PSO_PRECACHING

		FInGameScopedCycleCounter InGameCycleCounter(GetWorld(), EInGamePerfTrackers::VFXSignificance, EInGamePerfTrackerThreads::GameThread, bIsManagingSignificance);

		UE_LOGF(LogParticles,Verbose,
			"CreateSceneProxy @ %fs %ls bIsActive=%d", GetWorld()->TimeSeconds,
			Template != NULL ? *Template->GetName() : TEXT("NULL"), IsActive());

		if (EmitterInstances.Num() > 0)
		{
			CacheViewRelevanceFlags(Template);
		}

		// Create the dynamic data for rendering this particle system.
		bParallelRenderThreadUpdate = true;
		FParticleDynamicData* ParticleDynamicData = CreateDynamicData(GetScene()->GetFeatureLevel());
		bParallelRenderThreadUpdate = false;

		if (CanBeOccluded())
		{
			Template->CustomOcclusionBounds.IsValid = true;
			NewProxy = ::new FParticleSystemSceneProxy(FParticleSystemSceneProxyDesc(*this,ParticleDynamicData, true));
		}
		else
		{
			NewProxy = ::new FParticleSystemSceneProxy(FParticleSystemSceneProxyDesc(*this,ParticleDynamicData, false));
		}
		check (NewProxy);
	}
	
	// 
	return NewProxy;
}

#if WITH_EDITOR
void DrawParticleSystemHelpers(UParticleSystemComponent* InPSysComp, const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	if (InPSysComp != NULL)
	{
		for (int32 EmitterIdx = 0; EmitterIdx < InPSysComp->EmitterInstances.Num(); EmitterIdx++)
		{
			FParticleEmitterInstance* EmitterInst = InPSysComp->EmitterInstances[EmitterIdx];
			if (EmitterInst && EmitterInst->SpriteTemplate)
			{
				UParticleLODLevel* LODLevel = EmitterInst->SpriteTemplate->GetCurrentLODLevel(EmitterInst);
				for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
				{
					UParticleModule* Module = LODLevel->Modules[ModuleIdx];
					if (Module && Module->bSupported3DDrawMode && Module->b3DDrawMode)
					{
						Module->Render3DPreview({ *EmitterInst, View, PDI });
					}
				}
			}
		}
	}
}

ENGINE_API void DrawParticleSystemHelpers(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	for (TObjectIterator<AActor> It; It; ++It)
	{
		for (UActorComponent* Component : It->GetComponents())
		{
			if (UParticleSystemComponent* PSC = Cast<UParticleSystemComponent>(Component))
			{
				DrawParticleSystemHelpers(PSC, View, PDI);
			}
		}
	}
}
#endif	//#if WITH_EDITOR
