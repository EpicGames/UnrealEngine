// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/Emitter.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialRelevance.h"
#include "Particles/ParticleSystemComponent.h"
#include "Net/UnrealNetwork.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Particles/ParticleSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Emitter)

#define LOCTEXT_NAMESPACE "Emitter"

/*-----------------------------------------------------------------------------
AEmitter implementation.
-----------------------------------------------------------------------------*/
AEmitter::AEmitter(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ParticleSystemComponent = CreateDefaultSubobject<UParticleSystemComponent>(TEXT("ParticleSystemComponent0"));
	ParticleSystemComponent->SecondsBeforeInactive = 1;
	RootComponent = ParticleSystemComponent;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
			FName ID_Effects;
			FText NAME_Effects;
			FConstructorStatics()
				: SpriteTextureObject(TEXT("/Engine/EditorResources/S_Emitter"))
				, ID_Effects(TEXT("Effects"))
				, NAME_Effects(NSLOCTEXT("SpriteCategory", "Effects", "Effects"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
			SpriteComponent->SetRelativeScale3D_Direct(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->bHiddenInGame = true;
			SpriteComponent->bIsScreenSizeScaled = true;
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
			SpriteComponent->SetupAttachment(ParticleSystemComponent);
			SpriteComponent->bReceivesDecals = false;
		}

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(0, 255, 128);

			ArrowComponent->ArrowSize = 1.5f;
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->bIsScreenSizeScaled = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
			ArrowComponent->SetupAttachment(ParticleSystemComponent);
			ArrowComponent->SetUsingAbsoluteScale(true);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void AEmitter::PostActorCreated()
{
	Super::PostActorCreated();

	if (ParticleSystemComponent && bPostUpdateTickGroup)
	{
		ParticleSystemComponent->SetTickGroup(TG_PostUpdateWork);
	}
}

void AEmitter::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(AEmitter, bCurrentlyActive);
}

void AEmitter::SetTemplate(UParticleSystem* NewTemplate)
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->SetTemplate(NewTemplate);
		if (bPostUpdateTickGroup)
		{
			ParticleSystemComponent->SetTickGroup(TG_PostUpdateWork);
		}
	}
}

void AEmitter::AutoPopulateInstanceProperties()
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->AutoPopulateInstanceProperties();
	}
}

#if WITH_EDITOR
void AEmitter::CheckForErrors()
{
	Super::CheckForErrors();

	// Emitters placed in a level should have a non-NULL ParticleSystemComponent.
	if (!ParticleSystemComponent && GetLevel())
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_ParticleSystemComponentNull", "Emitter actor has NULL ParticleSystemComponent property - please delete")))
			->AddToken(FMapErrorToken::Create(FMapErrors::ParticleSystemComponentNull));
	}

}
#endif


FString AEmitter::GetDetailedInfoInternal() const
{
	FString Result;

	if (ParticleSystemComponent)
	{
		Result = ParticleSystemComponent->GetDetailedInfoInternal();
	}
	else
	{
		Result = TEXT("No_ParticleSystemComponent");
	}

	return Result;
}


#if WITH_EDITOR
void AEmitter::ResetInLevel()
{
	if (ParticleSystemComponent)
	{
		// Force a recache of the view relevance
		ParticleSystemComponent->ResetParticles();
		ParticleSystemComponent->ActivateSystem(true);
		ParticleSystemComponent->bIsViewRelevanceDirty = true;
		ParticleSystemComponent->CachedViewRelevanceFlags.Empty();
		ParticleSystemComponent->ConditionalCacheViewRelevanceFlags();
		ParticleSystemComponent->ReregisterComponent();
	}
}
#endif

void AEmitter::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	// Let them die quickly on a dedicated server
	if (IsNetMode(NM_DedicatedServer) && (GetRemoteRole() == ROLE_None || bNetTemporary))
	{
		SetLifeSpan(0.2f);
	}

	// Set Notification Delegate
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->OnSystemFinished.AddUniqueDynamic(this, &AEmitter::OnParticleSystemFinished);
		bCurrentlyActive = ParticleSystemComponent->bAutoActivate;
	}

	if (ParticleSystemComponent && bPostUpdateTickGroup)
	{
		ParticleSystemComponent->SetTickGroup(TG_PostUpdateWork);
	}
}

void AEmitter::OnRep_bCurrentlyActive()
{
	ParticleSystemComponent->SetActive(bCurrentlyActive);
}

void AEmitter::OnParticleSystemFinished(UParticleSystemComponent* FinishedComponent)
{
	if (bDestroyOnSystemFinish)
	{
		SetLifeSpan(0.0001f);
	}
	bCurrentlyActive = false;
}

void AEmitter::Activate()
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->ActivateSystem(false);
	}
	bCurrentlyActive = true;
}

void AEmitter::Deactivate()
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->DeactivateSystem();
	}
	bCurrentlyActive = false;
}

void AEmitter::ToggleActive()
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->ToggleActive();
		bCurrentlyActive = ParticleSystemComponent->IsActive();
	}
}

bool AEmitter::IsActive() const
{
	if (ParticleSystemComponent)
	{
		// @todo: I'm not updating bCurrentlyActive flag here. 
		// Technically I don't have to, and it seems bCurrentlyActive flag can be easily broken if we modify Component directly.
		return ParticleSystemComponent->IsActive();
	}

	return false;
}

void AEmitter::SetFloatParameter(FName ParameterName, float Param)
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->SetFloatParameter(ParameterName, Param);
	}
}

void AEmitter::SetVectorParameter(FName ParameterName, FVector Param)
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->SetVectorParameter(ParameterName, Param);
	}
}

void AEmitter::SetColorParameter(FName ParameterName, FLinearColor Param)
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->SetColorParameter(ParameterName, Param);
	}
}

void AEmitter::SetActorParameter(FName ParameterName, AActor* Param)
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->SetActorParameter(ParameterName, Param);
	}
}

void AEmitter::SetMaterialParameter(FName ParameterName, UMaterialInterface* Param)
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->SetMaterialParameter(ParameterName, Param);
	}
}

#if WITH_EDITOR

bool AEmitter::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (ParticleSystemComponent->Template)
	{
		Objects.Add(ParticleSystemComponent->Template);
	}
	return true;
}
#endif

#undef LOCTEXT_NAMESPACE

