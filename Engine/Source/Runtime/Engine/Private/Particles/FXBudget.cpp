// Copyright Epic Games, Inc. All Rights Reserved.


#include "Particles/FXBudget.h"
#include "Particles/ParticlePerfStatsManager.h"
#include "InGamePerformanceTracker.h"

#if WITH_GLOBAL_RUNTIME_FX_BUDGET

static float GFXBudget_GameThread = 2.0f;
static FAutoConsoleVariableRef CVarFXBudget_GameThread(
	TEXT("fx.Budget.GameThread"),
	GFXBudget_GameThread,
	TEXT("Budget (in ms) for all combined FX work that runs only on the gamethread. As this budget is approached or exceeded, various FX systems will attempt to scale down more and mroe agressively to remain in budget."),
	ECVF_Default
);

static float GFXBudget_GameThreadConcurrent = 2.0f;
static FAutoConsoleVariableRef CVarFXBudget_GameThreadConcurrent(
	TEXT("fx.Budget.GameThreadConcurrent"),
	GFXBudget_GameThreadConcurrent,
	TEXT("Budget (in ms) for all combined FX work that runs on the gamethread or on a concurrent task spawned from the game thread. As this budget is approached or exceeded, various FX systems will attempt to scale down more and mroe agressively to remain in budget."),
	ECVF_Default
);

static float GFXBudget_RenderThread = 2.0f;
static FAutoConsoleVariableRef CVarFXBudget_RenderThread(
	TEXT("fx.Budget.RenderThread"),
	GFXBudget_RenderThread,
	TEXT("Budget (in ms) for all combined FX work that runs on the Render Thread.  As this budget is approached or exceeded, various FX systems will attempt to scale down more and mroe agressively to remain in budget."),
	ECVF_Default
);

static int32 GFXBudget_HistorySize = 60;
static FAutoConsoleVariableRef CVarFXBudget_HistoryFrames(
	TEXT("fx.Budget.HistoryFrames"),
	GFXBudget_HistorySize,
	TEXT("Number of frames the global FX budget tracking will hold to work out it's average frame time."),
	ECVF_Default | ECVF_ReadOnly
);



static float GFXTimeOverride_GameThread = -1.0f;
static FAutoConsoleVariableRef CVarFXGameThreadTimeOverride(
	TEXT("fx.Budget.Debug.GameThreadTimeOverride"),
	GFXTimeOverride_GameThread,
	TEXT("When >= 0.0 overrides the reported time for FX on the GameThread. Useful for observing/debugging the impact on other systems."),
	ECVF_Default
);

static float GFXTimeOverride_GameThreadConcurrent = -1.0f;
static FAutoConsoleVariableRef CVarFXGameThreadConcurrentTimeOverride(
	TEXT("fx.Budget.Debug.GameThreadConcurrentTimeOverride"),
	GFXTimeOverride_GameThreadConcurrent,
	TEXT("When >= 0.0 overrides the reported time for FX on the GameThreadConcurrent. Useful for observing/debugging the impact on other systems."),
	ECVF_Default
);

static float GFXTimeOverride_RenderThread = -1.0f;
static FAutoConsoleVariableRef CVarFXRenderThreadTimeOverride(
	TEXT("fx.Budget.Debug.RenderThreadTimeOverride"),
	GFXTimeOverride_RenderThread,
	TEXT("When >= 0.0 overrides the reported time for FX on the RenderThread. Useful for observing/debugging the impact on other systems."),
	ECVF_Default
);


static float GFXBudget_AdjustedUsageDecayRate = 0.1f;
static FAutoConsoleVariableRef CVarFXBudget_AdjustedUsageDecayRate(
	TEXT("fx.Budget.AdjustedUsageDecayRate"),
	GFXBudget_AdjustedUsageDecayRate,
	TEXT("Rate at which the FX budget adjusted usage value is allowed to decay. This helps prevent FX flipping off/on if the usage oscilates over the cull threshold as the FX are culled/enabled."),
	ECVF_Default
);

static float GFXBudget_AdjustedUsageMax = 2.0f;
static FAutoConsoleVariableRef CVarFXBudget_AdjustedUsageMax(
	TEXT("fx.Budget.AdjustedUsageMax"),
	GFXBudget_AdjustedUsageMax,
	TEXT("Max value for FX Budget adjusted usage. Prevents one very long frame from keeping the usage above 1.0 for long periods under budget."),
	ECVF_Default
);

#if WITH_EDITOR
static bool GFXBudget_EnabledInEditor=false;
static FAutoConsoleVariableRef CVarFXBudget_EnabledInEditor(
	TEXT("fx.Budget.EnabledInEditor"),
	GFXBudget_EnabledInEditor,
	TEXT("Controls whether we track global FX budgets in editor builds."),
	FConsoleVariableDelegate::CreateStatic(FFXBudget::OnEnabledCVarChanged),
	ECVF_Default
);
#endif

static FAutoConsoleVariableRef CVarFXBudget_Enabled(
	TEXT("fx.Budget.Enabled"),
	FFXBudget::bEnabled,
	TEXT("Controls whether we track global FX budgets."),
	FConsoleVariableDelegate::CreateStatic(FFXBudget::OnEnabledCVarChanged),
	ECVF_Default
);

class FParticlePerfStatsListener_FXBudget : public FParticlePerfStatsListener
{
public:
	
	FParticlePerfStatsListener_FXBudget()
	: GTHistory(GFXBudget_HistorySize)
	, GTConcurrentHistory(GFXBudget_HistorySize)
	, RTHistory(GFXBudget_HistorySize)
	{

	}

	virtual bool NeedsWorldStats()const override{ return true; }
	virtual bool NeedsSystemStats()const override { return false; }
	virtual bool NeedsComponentStats()const override { return false; }

	virtual void Begin() override {}
	virtual void End() override
	{
		FFXBudget::SetWorstAdjustedUsage(0.0f);
	}

	virtual bool Tick() override
	{
		if (FFXBudget::Enabled())
		{
			FParticlePerfStatsManager::ForAllWorldStats(
				[&](TWeakObjectPtr<const UWorld>& WeakWorld, TUniquePtr<FParticlePerfStats>& Stats)
				{
					GTHistory.AddCycles(Stats->GetGameThreadStats().GetTotalCycles_GTOnly());
					GTConcurrentHistory.AddCycles(Stats->GetGameThreadStats().GetTotalCycles());
				}
			);
		
			GTHistory.NextFrame();
			GTConcurrentHistory.NextFrame();

			if (GFXTimeOverride_GameThread >= 0.0f)
			{
				AverageTimesMs.GT = GFXTimeOverride_GameThread;
			}
			else if (GFXBudget_GameThread > 0.0f)
			{
				AverageTimesMs.GT = FPlatformTime::ToMilliseconds64(GTHistory.GetAverageCycles());
			}
			else
			{
				AverageTimesMs.GT = 0.0f;
			}

			if (GFXTimeOverride_GameThreadConcurrent >= 0.0f)
			{
				AverageTimesMs.GTConcurrent = GFXTimeOverride_GameThreadConcurrent;
			}
			else if (GFXBudget_GameThreadConcurrent > 0.0f)
			{
				AverageTimesMs.GTConcurrent = FPlatformTime::ToMilliseconds64(GTConcurrentHistory.GetAverageCycles());
			}
			else
			{
				AverageTimesMs.GTConcurrent = 0.0f;
			}

			//Update AdjustedUsage
			uint64 CurrentCylces = FPlatformTime::Cycles64();
			float Dt = FPlatformTime::ToSeconds64(CurrentCylces - PrevTickCycles);
			PrevTickCycles = CurrentCylces;
			FFXTimeData TargetUsage = GetUsage();
			float AllowedChange = (GFXBudget_AdjustedUsageDecayRate * Dt);
			auto UpdateUsage = [&](float Target, float& Current)
			{
				//We only ever move to lower usage slowly. Any higher usage is immediately applied.
				float NewVal = Current - AllowedChange;
				if (Target > NewVal)
				{
					NewVal = Target;
				}
				Current = FMath::Min(GFXBudget_AdjustedUsageMax, NewVal);
			};
			UpdateUsage(TargetUsage.GT, AdjustedUsage.GT);
			UpdateUsage(TargetUsage.GTConcurrent, AdjustedUsage.GTConcurrent);
			UpdateUsage(TargetUsage.RT, AdjustedUsage.RT);

			FFXBudget::SetWorstAdjustedUsage(FMath::Max3(AdjustedUsage.GT, AdjustedUsage.GTConcurrent, AdjustedUsage.RT));
		}

		#if WITH_PARTICLE_PERF_CSV_STATS
		if (FCsvProfiler* CSVProfiler = FCsvProfiler::Get())
		{
			if (CSVProfiler->IsCapturing() && FParticlePerfStats::GetCSVStatsEnabled())
			{
				static FName GTUsageStat("Budget/GT");
				static FName GTCNCUsageStat("Budget/GTCNC");
				static FName RTUsageStat("Budget/RT");
				static FName AdjustedUsageStat("Budget/Adjusted");

				const FFXTimeData& Usage = FFXBudget::GetUsage();
				float WorstAdjusted = FFXBudget::GetWorstAdjustedUsage();

				CSVProfiler->RecordCustomStat(GTUsageStat, CSV_CATEGORY_INDEX(Particles), Usage.GT, ECsvCustomStatOp::Set);
				CSVProfiler->RecordCustomStat(GTCNCUsageStat, CSV_CATEGORY_INDEX(Particles), Usage.GTConcurrent, ECsvCustomStatOp::Set);
				CSVProfiler->RecordCustomStat(RTUsageStat, CSV_CATEGORY_INDEX(Particles), Usage.RT, ECsvCustomStatOp::Set);
				CSVProfiler->RecordCustomStat(AdjustedUsageStat, CSV_CATEGORY_INDEX(Particles), WorstAdjusted, ECsvCustomStatOp::Set);
			}
		}
		#endif//WITH_PARTICLE_PERF_CSV_STATS

		return true; 
	}

	virtual void TickRT() override
	{
		FParticlePerfStatsManager::ForAllWorldStats(
			[&](TWeakObjectPtr<const UWorld>& WeakWorld, TUniquePtr<FParticlePerfStats>& Stats)
			{
				RTHistory.AddCycles(Stats->GetRenderThreadStats().GetTotalCycles());
			}
		);

		RTHistory.NextFrame();

		if (GFXTimeOverride_RenderThread >= 0.0f)
		{
			AverageTimesMs.RT = GFXTimeOverride_RenderThread;
		}
		else if (GFXBudget_RenderThread > 0.0f)
		{
			AverageTimesMs.RT = FPlatformTime::ToMilliseconds64(RTHistory.GetAverageCycles());
		}
		else
		{
			AverageTimesMs.RT = 0.0f;
		}
	}

	/** Returns the current global time spent on FX. */
	FORCEINLINE FFXTimeData GetTime()const { return AverageTimesMs; }

	FFXTimeData GetBudget()const	{ return FFXTimeData(GFXBudget_GameThread, GFXBudget_GameThreadConcurrent, GFXBudget_RenderThread);	}

	FFXTimeData GetUsage()
	{
		FFXTimeData Time = GetTime();
		FFXTimeData Usage;

		if (GFXBudget_GameThread > 0.0f)
		{
			Usage.GT = Time.GT / GFXBudget_GameThread;
		}

		if (GFXBudget_GameThreadConcurrent > 0.0f)
		{
			Usage.GTConcurrent = Time.GTConcurrent / GFXBudget_GameThreadConcurrent;
		}

		if (GFXBudget_RenderThread > 0.0f)
		{
			Usage.RT = Time.RT / GFXBudget_RenderThread;
		}

		return Usage;
	}

	FFXTimeData GetAdjustedUsage()const { return AdjustedUsage; }

	FInGameCycleHistory GTHistory;
	FInGameCycleHistory GTConcurrentHistory;
	FInGameCycleHistory RTHistory;

	FFXTimeData AverageTimesMs;

	/** Adjusted time/budget data. Once usage goes up it decays at a set rate to avoid FX flipping off/on. */
	FFXTimeData AdjustedUsage;

	/** Track the previous tick time so we can derive a delta time for updating AdjustedUsage. */
	uint64 PrevTickCycles;
};

//////////////////////////////////////////////////////////////////////////

TSharedPtr<FParticlePerfStatsListener_FXBudget, ESPMode::ThreadSafe> FFXBudget::StatsListener;
bool FFXBudget::bEnabled = false;
FFXTimeData FFXBudget::AdjustedUsage;
float FFXBudget::WorstAdjustedUsage = 0.0f;

FFXTimeData FFXBudget::GetTime()
{
	if (Enabled() && StatsListener.IsValid())
	{
		return StatsListener->GetTime();
	}
	else
	{
		return FFXTimeData();
	}
}

FFXTimeData FFXBudget::GetBudget()
{
	return FFXTimeData(GFXBudget_GameThread, GFXBudget_GameThreadConcurrent, GFXBudget_RenderThread);
}

FFXTimeData FFXBudget::GetUsage()
{
	if (Enabled() && StatsListener.IsValid())
	{
		return StatsListener->GetUsage();
	}
	else
	{
		return FFXTimeData();
	}
}

FFXTimeData FFXBudget::GetAdjustedUsage()
{
	if (Enabled() && StatsListener.IsValid())
	{
		return StatsListener->GetAdjustedUsage();
	}
	else
	{
		return FFXTimeData();
	}
}

void FFXBudget::Reset()
{
	if (StatsListener.IsValid())
	{
		FParticlePerfStatsManager::RemoveListener(StatsListener);
		StatsListener = MakeShared<FParticlePerfStatsListener_FXBudget, ESPMode::ThreadSafe>();
		FParticlePerfStatsManager::AddListener(StatsListener);
	}
}

void FFXBudget::OnEnabledCVarChanged(IConsoleVariable* CVar)
{
	OnEnabledChangedInternal();
}

void FFXBudget::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

void FFXBudget::OnEnabledChangedInternal()
{
#if WITH_EDITOR
	//Don't allow budgeting to be enabled in the editor if the editor specific CVar is false.
	SetEnabled(GFXBudget_EnabledInEditor && CVarFXBudget_Enabled->GetBool());
#endif

	if (bEnabled)
	{
		if (!StatsListener.IsValid())
		{
			StatsListener = MakeShared<FParticlePerfStatsListener_FXBudget, ESPMode::ThreadSafe>();
			FParticlePerfStatsManager::AddListener(StatsListener);
		}
	}
	else
	{
		//Destroy the listener if we disable at runtime.
		if (StatsListener.IsValid())
		{
			FParticlePerfStatsManager::RemoveListener(StatsListener);
			StatsListener.Reset();
		}
	}
}

#endif //WITH_GLOBAL_RUNTIME_FX_BUDGET
