// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FXSystem.cpp: Implementation of the effects system.
=============================================================================*/

#include "FXSystem.h"
#include "EngineModule.h"
#include "Particles/FXSystemPrivate.h"
#include "Particles/FXSystemSet.h"
#include "GPUSort.h"
#include "Particles/ParticleCurveTexture.h"
#include "Particles/ParticleSortingGPU.h"
#include "SceneView.h"
#include "VectorField/VectorField.h"
#include "Components/VectorFieldComponent.h"
#include "SceneInterface.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "FXRenderingUtils.h"
#include "Containers/StridedView.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphEvent.h"
#include "RenderGraphUtils.h"

TMap<FName, FCreateCustomFXSystemDelegate> FFXSystemInterface::CreateCustomFXDelegates;

bool IsParticleCollisionModeSupported(EShaderPlatform InPlatform, EParticleCollisionShaderMode InCollisionShaderMode)
{
	switch (InCollisionShaderMode)
	{
	case PCM_None:
		return true;
	case PCM_DepthBuffer:
		return IsFeatureLevelSupported(InPlatform, ERHIFeatureLevel::SM5);
	case PCM_DistanceField:
		return FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields(InPlatform);
	}
	check(0);
	return IsFeatureLevelSupported(InPlatform, ERHIFeatureLevel::SM5);
}

void FFXSystemInterface::FFXInterfaceDeletor::operator()(FFXSystemInterface* ObjectToDelete) const
{
	ENQUEUE_RENDER_COMMAND(FFXInterfaceDeletor)
	(
		[ObjectToDelete](FRHICommandListImmediate&)
		{
			delete ObjectToDelete;
		}
	);
}

/*-----------------------------------------------------------------------------
	External FX system interface.
-----------------------------------------------------------------------------*/

TSharedRef<FFXSystemInterface> FFXSystemInterface::Create(ERHIFeatureLevel::Type InFeatureLevel, FSceneInterface* Scene)
{
	check(Scene);
	EShaderPlatform InShaderPlatform = Scene->GetShaderPlatform();
	// The FGPUSortManager is currently only being used by FFXSystemInterface implementations.
	// Because of this, the lifetime management of the GPUSortManager only consists of a ref counter incremented initially 
	// in the gamethread (in this function) and decremented on the renderthread (when the system interfaces are deleted).
	TRefCountPtr<FGPUSortManager> GPUSortManager = new FGPUSortManager(InFeatureLevel);

	if (CreateCustomFXDelegates.Num())
	{
		TSharedRef<FFXSystemSet> Set = MakeShareable(new FFXSystemSet(GPUSortManager), FFXInterfaceDeletor());
		Scene->SetFXSystem(&Set.Get());
		Set->SetSceneInterface(Scene);
		Set->SetScene((FScene*)Scene);

		Set->FXSystems.Emplace(MakeShareable(new FFXSystem(InFeatureLevel, InShaderPlatform, GPUSortManager), FFXInterfaceDeletor()));

		for (TMap<FName, FCreateCustomFXSystemDelegate>::TConstIterator Ite(CreateCustomFXDelegates); Ite; ++Ite)
		{
			FFXSystemInterface* CustomFX = Ite.Value().Execute(InFeatureLevel, InShaderPlatform, GPUSortManager);
			if (CustomFX)
			{
				CustomFX->SetSceneInterface(Scene);
				CustomFX->SetScene((FScene*)Scene);
				Set->FXSystems.Emplace(MakeShareable(CustomFX, FFXInterfaceDeletor()));
			}
		}
		return Set;
	}
	else
	{
		TSharedRef<FFXSystemInterface> Ret = MakeShareable(new FFXSystem(InFeatureLevel, InShaderPlatform, GPUSortManager), FFXInterfaceDeletor());
		Scene->SetFXSystem(&Ret.Get());
		Ret->SetSceneInterface(Scene);
		Ret->SetScene((FScene*)Scene);
		return Ret;
	}

}
void FFXSystemInterface::QueueDestroyGPUSimulation(FFXSystemInterface* FXSystem)
{
	check(FXSystem);
	ENQUEUE_RENDER_COMMAND(FDestroyGPUSimulationCommand)(
		[FXSystem](FRHICommandList& RHICmdList)
	{
		FXSystem->DestroyGPUSimulation();
	});
}

void FFXSystemInterface::MarkPendingKill(FFXSystemInterface* FXSystem)
{
	check(FXSystem);
	FXSystem->OnMarkPendingKill();
}

void FFXSystemInterface::Destroy( FFXSystemInterface* FXSystem )
{
	MarkPendingKill(FXSystem);
}

void FFXSystemInterface::RegisterCustomFXSystem(const FName& InterfaceName, const FCreateCustomFXSystemDelegate& InCreateDelegate)
{
	CreateCustomFXDelegates.Add(InterfaceName, InCreateDelegate);
}

void FFXSystemInterface::UnregisterCustomFXSystem(const FName& InterfaceName)
{
	CreateCustomFXDelegates.Remove(InterfaceName);
}

/*------------------------------------------------------------------------------
	FX system console variables.
------------------------------------------------------------------------------*/

namespace FXConsoleVariables
{
	int32 VisualizeGPUSimulation = 0;
	int32 bAllowGPUSorting = true;
	int32 bAllowCulling = true;
	int32 bFreezeGPUSimulation = false;
	int32 bFreezeParticleSimulation = false;
	int32 bAllowAsyncTick = !WITH_EDITOR;
	float ParticleSlackGPU = 0.02f;
	int32 MaxParticleTilePreAllocation = 100;
	int32 MaxCPUParticlesPerEmitter = 1000;
	int32 MaxGPUParticlesSpawnedPerFrame = 1024 * 1024;
	int32 GPUSpawnWarningThreshold = 20000;
	float GPUCollisionDepthBounds = 500.0f;
	TAutoConsoleVariable<int32> TestGPUSort(TEXT("FX.TestGPUSort"),0,TEXT("Test GPU sort. 1: Small, 2: Large, 3: Exhaustive, 4: Random"),ECVF_Cheat);

	/** Register references to flags. */
	FAutoConsoleVariableRef CVarVisualizeGPUSimulation(
		TEXT("FX.VisualizeGPUSimulation"),
		VisualizeGPUSimulation,
		TEXT("Visualize the current state of GPU simulation.\n")
		TEXT("0 = off\n")
		TEXT("1 = visualize particle state\n")
		TEXT("2 = visualize curve texture"),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarAllowGPUSorting(
		TEXT("FX.AllowGPUSorting"),
		bAllowGPUSorting,
		TEXT("Allow particles to be sorted on the GPU."),
		ECVF_ReadOnly | ECVF_Preview
		);
	FAutoConsoleVariableRef CVarFreezeGPUSimulation(
		TEXT("FX.FreezeGPUSimulation"),
		bFreezeGPUSimulation,
		TEXT("Freeze particles simulated on the GPU."),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarFreezeParticleSimulation(
		TEXT("FX.FreezeParticleSimulation"),
		bFreezeParticleSimulation,
		TEXT("Freeze particle simulation."),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarAllowAsyncTick(
		TEXT("FX.AllowAsyncTick"),
		bAllowAsyncTick,
		TEXT("allow parallel ticking of particle systems."),
		ECVF_Default
		);
	FAutoConsoleVariableRef CVarParticleSlackGPU(
		TEXT("FX.ParticleSlackGPU"),
		ParticleSlackGPU,
		TEXT("Amount of slack to allocate for GPU particles to prevent tile churn as percentage of total particles."),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarMaxParticleTilePreAllocation(
		TEXT("FX.MaxParticleTilePreAllocation"),
		MaxParticleTilePreAllocation,
		TEXT("Maximum tile preallocation for GPU particles."),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarMaxCPUParticlesPerEmitter(
		TEXT("FX.MaxCPUParticlesPerEmitter"),
		MaxCPUParticlesPerEmitter,
		TEXT("Maximum number of CPU particles allowed per-emitter.")
		);
	FAutoConsoleVariableRef CVarMaxGPUParticlesSpawnedPerFrame(
		TEXT("FX.MaxGPUParticlesSpawnedPerFrame"),
		MaxGPUParticlesSpawnedPerFrame,
		TEXT("Maximum number of GPU particles allowed to spawn per-frame per-emitter.")
		);
	FAutoConsoleVariableRef CVarGPUSpawnWarningThreshold(
		TEXT("FX.GPUSpawnWarningThreshold"),
		GPUSpawnWarningThreshold,
		TEXT("Warning threshold for spawning of GPU particles."),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarGPUCollisionDepthBounds(
		TEXT("FX.GPUCollisionDepthBounds"),
		GPUCollisionDepthBounds,
		TEXT("Limits the depth bounds when searching for a collision plane."),
		ECVF_Cheat
		);
	FAutoConsoleVariableRef CVarAllowCulling(
		TEXT("FX.AllowCulling"),
		bAllowCulling,
		TEXT("Allow emitters to be culled."),
		ECVF_Cheat
		);
	int32 bAllowGPUParticles = true;
	FAutoConsoleVariableRef CVarAllowGPUParticles(
		TEXT("FX.AllowGPUParticles"),
		bAllowGPUParticles,
		TEXT("If true, allow the usage of GPU particles.")
	);
}

/*------------------------------------------------------------------------------
	FX system.
------------------------------------------------------------------------------*/

FFXSystem::FFXSystem(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, FGPUSortManager* InGPUSortManager)
	: ParticleSimulationResources(NULL)
	, FeatureLevel(InFeatureLevel)
	, GPUSortManager(nullptr)
#if WITH_EDITOR
	, bSuspended(false)
#endif // #if WITH_EDITOR
{
	// FXSystem GPU sorting is disabled for mobile, see FParticleSortKeyGenCS
	if (FeatureLevel >= ERHIFeatureLevel::SM5)
	{
		GPUSortManager = InGPUSortManager;
	}
	
	InitGPUSimulation();

	// Register the callback in the GPUSortManager. 
	// The callback is used to generate the initial keys and values for the GPU sort tasks, 
	// the values being the sorted particle indices used by the GPU particles.
	// The registration also involves defining the list of flags possibly used in GPUSortManager::AddTask()
	if (GPUSortManager)
	{
		GPUSortManager->Register(FGPUSortKeyGenDelegate::CreateLambda([this](FRHICommandListImmediate& RHICmdList, int32 BatchId, int32 NumElementsInBatch, EGPUSortFlags Flags, FRHIUnorderedAccessView* KeysUAV, FRHIUnorderedAccessView* ValuesUAV)
		{ 
			GenerateSortKeys(RHICmdList, BatchId, NumElementsInBatch, Flags, KeysUAV, ValuesUAV);
		}), 
		EGPUSortFlags::LowPrecisionKeys | EGPUSortFlags::KeyGenAfterPostRenderOpaque | EGPUSortFlags::AnySortLocation | EGPUSortFlags::ValuesAsG16R16F,
		Name);
	}
}

FFXSystem::~FFXSystem()
{
	ensure(GPUSimulations.Num() == 0);
	ensure(VectorFields.Num() == 0);
}

const FName FFXSystem::Name(TEXT("FFXSystem"));

FFXSystemInterface* FFXSystem::GetInterface(const FName& InName)
{
	return InName == Name ? this : nullptr;
}

void FFXSystem::Tick(UWorld* World, float DeltaSeconds)
{
	if (RHISupportsGPUParticles())
	{
		// Test GPU sorting if requested.
		if (FXConsoleVariables::TestGPUSort.GetValueOnGameThread() != 0)
		{
			TestGPUSort((EGPUSortTest)FXConsoleVariables::TestGPUSort.GetValueOnGameThread(), GetFeatureLevel());
			// Reset CVar
			static IConsoleVariable* CVarTestGPUSort = IConsoleManager::Get().FindConsoleVariable(TEXT("FX.TestGPUSort"));

			// todo: bad use of console variables, this should be a console command 
			CVarTestGPUSort->Set(0, ECVF_SetByCode);
		}

		// Before ticking GPU particles, ensure any pending curves have been
		// uploaded.
		GParticleCurveTexture.SubmitPendingCurves();
	}
}

void FFXSystem::OnMarkPendingKill()
{
	FFXSystemInterface::OnMarkPendingKill();

	// Enqueue command to destroy simulations as destruction order can matter for Cascade
	ENQUEUE_RENDER_COMMAND(FOnMarkPendingKill)(
		[this](FRHICommandListImmediate&)
		{
			for (FVectorFieldInstance* VectorFieldInstance : VectorFields)
			{
				if (VectorFieldInstance)
				{
					delete VectorFieldInstance;
				}
			}
			VectorFields.Empty();

			DestroyGPUSimulation();
		}
	);
}

#if WITH_EDITOR
void FFXSystem::Suspend()
{
	if (!bSuspended && RHISupportsGPUParticles())
	{
		ReleaseGPUResources();
		bSuspended = true;
	}
}

void FFXSystem::Resume()
{
	if (bSuspended && RHISupportsGPUParticles())
	{
		bSuspended = false;
		InitGPUResources();
	}
}
#endif // #if WITH_EDITOR

/*------------------------------------------------------------------------------
	Vector field instances.
------------------------------------------------------------------------------*/

void FFXSystem::AddVectorField( UVectorFieldComponent* VectorFieldComponent )
{
	if (RHISupportsGPUParticles())
	{
		check( VectorFieldComponent->VectorFieldInstance == NULL );
		checkSlow( VectorFieldComponent->FXSystem && VectorFieldComponent->FXSystem->GetInterface(Name) == this );

		if ( VectorFieldComponent->VectorField && !IsPendingKill() )
		{
			FVectorFieldInstance* Instance = new FVectorFieldInstance();
			VectorFieldComponent->VectorField->InitInstance(Instance, /*bPreviewInstance=*/ false);
			VectorFieldComponent->VectorFieldInstance = Instance;
			Instance->WorldBounds = VectorFieldComponent->Bounds.GetBox();
			Instance->Intensity = VectorFieldComponent->Intensity;
			Instance->Tightness = VectorFieldComponent->Tightness;

			FFXSystem* FXSystem = this;
			FMatrix ComponentToWorld = VectorFieldComponent->GetComponentTransform().ToMatrixWithScale();
			ENQUEUE_RENDER_COMMAND(FAddVectorFieldCommand)(
				[FXSystem, Instance, ComponentToWorld](FRHICommandListImmediate& RHICmdList)
				{
					Instance->UpdateTransforms( ComponentToWorld );
					Instance->Index = FXSystem->VectorFields.AddUninitialized().Index;
					FXSystem->VectorFields[ Instance->Index ] = Instance;
				});
		}
	}
}

void FFXSystem::RemoveVectorField( UVectorFieldComponent* VectorFieldComponent )
{
	if (RHISupportsGPUParticles())
	{
		checkSlow(VectorFieldComponent->FXSystem && VectorFieldComponent->FXSystem->GetInterface(Name) == this);

		FVectorFieldInstance* Instance = VectorFieldComponent->VectorFieldInstance;
		VectorFieldComponent->VectorFieldInstance = NULL;

		// If pending kill the VectorFieldInstance will be freed in the destructor.
		if ( Instance  && !IsPendingKill() )
		{
			FFXSystem* FXSystem = this;
			ENQUEUE_RENDER_COMMAND(FRemoveVectorFieldCommand)(
				[FXSystem, Instance](FRHICommandListImmediate& RHICmdList)
				{
					if ( Instance->Index != INDEX_NONE )
					{
						FXSystem->VectorFields.RemoveAt( Instance->Index );
						delete Instance;
					}
				});
		}
	}
}

void FFXSystem::UpdateVectorField( UVectorFieldComponent* VectorFieldComponent )
{
	if (RHISupportsGPUParticles())
	{
		checkSlow(VectorFieldComponent->FXSystem && VectorFieldComponent->FXSystem->GetInterface(Name) == this);

		FVectorFieldInstance* Instance = VectorFieldComponent->VectorFieldInstance;

		if ( Instance && !IsPendingKill() )
		{
			struct FUpdateVectorFieldParams
			{
				FBox Bounds;
				FMatrix ComponentToWorld;
				float Intensity;
				float Tightness;
			};

			FUpdateVectorFieldParams UpdateParams;
			UpdateParams.Bounds = VectorFieldComponent->Bounds.GetBox();
			UpdateParams.ComponentToWorld = VectorFieldComponent->GetComponentTransform().ToMatrixWithScale();
			UpdateParams.Intensity = VectorFieldComponent->Intensity;
			UpdateParams.Tightness = VectorFieldComponent->Tightness;

			FFXSystem* FXSystem = this;
			ENQUEUE_RENDER_COMMAND(FUpdateVectorFieldCommand)(
				[FXSystem, Instance, UpdateParams](FRHICommandListImmediate& RHICmdList)
				{
					Instance->WorldBounds = UpdateParams.Bounds;
					Instance->Intensity = UpdateParams.Intensity;
					Instance->Tightness = UpdateParams.Tightness;
					Instance->UpdateTransforms( UpdateParams.ComponentToWorld );
				});
		}
	}
}

/*-----------------------------------------------------------------------------
	Render related functionality.
-----------------------------------------------------------------------------*/

void FFXSystem::DrawDebug( FCanvas* Canvas )
{
	if (FXConsoleVariables::VisualizeGPUSimulation > 0
		&& RHISupportsGPUParticles())
	{
		VisualizeGPUParticles(Canvas);
	}
}

DECLARE_GPU_DRAWCALL_STAT(FXSystemPreInitViews);

void FFXSystem::PreInitViews(FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*>& ViewFamilies, const FSceneViewFamily* CurrentFamily)
{
	if (RHISupportsGPUParticles())
	{
		// Note: This can not be put into a GraphBuilder pass directly, the internals need to be refactored in order to that.
		// This is because the data modified in here will be used in GDME pass which must have the most up to date information
		AdvanceGPUParticleFrame(GraphBuilder.RHICmdList, bAllowGPUParticleUpdate);
	}
}

void FFXSystem::PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate)
{
	// nothing to do here
}

bool FFXSystem::UsesGlobalDistanceField() const
{
	if (RHISupportsGPUParticles())
	{
		return UsesGlobalDistanceFieldInternal();
	}

	return false;
}

bool FFXSystem::UsesDepthBuffer() const
{
	if (RHISupportsGPUParticles())
	{
		return UsesDepthBufferInternal();
	}

	return false;
}

bool FFXSystem::RequiresEarlyViewUniformBuffer() const
{
	if (RHISupportsGPUParticles())
	{
		return RequiresEarlyViewUniformBufferInternal();
	}

	return false;
}

bool FFXSystem::RequiresRayTracingScene() const
{
	if (RHISupportsGPUParticles())
	{
		return RequiresRayTracingSceneInternal();
	}

	return false;
}

DECLARE_GPU_DRAWCALL_STAT(FXSystemPreRender);
DECLARE_GPU_DRAWCALL_STAT(FXSystemPostRenderOpaque);

void FFXSystem::PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleSceneUpdate)
{
	bAllowGPUParticleSceneUpdate = bAllowGPUParticleSceneUpdate && Views.Num() > 0 && Views[0].AllowGPUParticleUpdate();

	if (RHISupportsGPUParticles() && bAllowGPUParticleSceneUpdate)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FXSystemPreRender, "FXSystemPreRender");
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, FXSystem);

		TUniformBufferRef<FViewUniformShaderParameters> ViewUniformBuffer = Views[0].ViewUniformBuffer;
		const FGlobalDistanceFieldParameterData* GlobalDistanceFieldParameterData = GetRendererModule().GetGlobalDistanceFieldParameterData(Views[0]);

		AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FFXSystem::PreRender"),
			[this, ViewUniformBuffer, GlobalDistanceFieldParameterData](FRHICommandListImmediate& RHICmdList)
			{
				//SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_FXSystem_PostRenderOpaque);

				SCOPED_DRAW_EVENT(RHICmdList, GPUParticles_PreRender);
				UpdateMultiGPUResources(RHICmdList);

				SimulateGPUParticles(RHICmdList, EParticleSimulatePhase::Main, {}, nullptr);

				if (IsParticleCollisionModeSupported(GetShaderPlatform(), PCM_DistanceField))
				{
					SimulateGPUParticles(RHICmdList, EParticleSimulatePhase::CollisionDistanceField, ViewUniformBuffer, GlobalDistanceFieldParameterData);
				}
			}
		);
    }
}

void FFXSystem::PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate)
{
	bAllowGPUParticleUpdate = bAllowGPUParticleUpdate && Views.Num() > 0 && Views[0].AllowGPUParticleUpdate();

	if (RHISupportsGPUParticles() && IsParticleCollisionModeSupported(GetShaderPlatform(), PCM_DepthBuffer) && bAllowGPUParticleUpdate)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FXSystemPostRenderOpaque, "FXSystemPostRenderOpaque");
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, FXSystem);

		TUniformBufferRef<FViewUniformShaderParameters> ViewUniformBuffer = Views[0].ViewUniformBuffer;

		AddPass(GraphBuilder, RDG_EVENT_NAME("FFXSystem::PostRenderOpaque"), 
			[this, ViewUniformBuffer](FRHICommandListImmediate& RHICmdList)
			{
				SCOPED_DRAW_EVENT(RHICmdList, GPUParticles_PostRenderOpaque);
				SimulateGPUParticles(RHICmdList, EParticleSimulatePhase::CollisionDepthBuffer, ViewUniformBuffer, nullptr);
			}
		);
	}
}

FGPUSortManager* FFXSystem::GetGPUSortManager() const
{
	return GPUSortManager;
}

