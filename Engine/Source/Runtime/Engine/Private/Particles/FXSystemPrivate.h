// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FXSystemPrivate.h: Internal effects system interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "FXSystem.h"
#include "VectorField.h"
#include "ParticleSortingGPU.h"
#include "GPUSortManager.h"
#include "ParticleVertexFactory.h"

class FCanvas;
class FGlobalDistanceFieldParameterData;
class FParticleSimulationGPU;
class FParticleSimulationResources;
class FViewUniformShaderParameters;
class IParticleEmitterInstanceOwner;
class UParticleSystemComponent;
class UVectorFieldComponent;
struct FGPUSpriteEmitterInfo;
struct FParticleEmitterInstance;

/*-----------------------------------------------------------------------------
	Forward declarations.
-----------------------------------------------------------------------------*/

/** An individual particle simulation taking place on the GPU. */
class FParticleSimulationGPU;
/** Resources used for particle simulation. */
class FParticleSimulationResources;

namespace EParticleSimulatePhase
{
	enum Type
	{
		/** The main simulation pass is for standard particles. */
		Main,
		CollisionDistanceField,
		/** The collision pass is used by these that collide against the scene depth buffer. */
		CollisionDepthBuffer,

		/**********************************************************************/

		/** The first simulation phase that is run each frame. */
		First = Main,
		/** The final simulation phase that is run each frame. */
		Last = CollisionDepthBuffer
	};
};

enum EParticleCollisionShaderMode
{
	PCM_None,
	PCM_DepthBuffer,
	PCM_DistanceField
};

/** Helper function to determine whether the given particle collision shader mode is supported on the given shader platform */
extern bool IsParticleCollisionModeSupported(EShaderPlatform InPlatform, EParticleCollisionShaderMode InCollisionShaderMode);

inline EParticleSimulatePhase::Type GetLastParticleSimulationPhase(EShaderPlatform InPlatform)
{
	return (IsParticleCollisionModeSupported(InPlatform, PCM_DepthBuffer) ? EParticleSimulatePhase::Last : EParticleSimulatePhase::Main);
}

/*-----------------------------------------------------------------------------
Injecting particles in to the GPU for simulation.
-----------------------------------------------------------------------------*/

/**
* Data passed to the GPU to inject a new particle in to the simulation.
*/
struct FNewParticle
{
	/** The initial position of the particle. */
	FVector3f Position;
	/** The relative time of the particle. */
	float RelativeTime;
	/** The initial velocity of the particle. */
	FVector3f Velocity;
	/** The time scale for the particle. */
	float TimeScale;
	/** Initial size of the particle. */
	FVector2f Size;
	/** Initial rotation of the particle. */
	float Rotation;
	/** Relative rotation rate of the particle. */
	float RelativeRotationRate;
	/** Coefficient of drag. */
	float DragCoefficient;
	/** Per-particle vector field scale. */
	float VectorFieldScale;
	/** Resilience for collision. */
	union
	{
		float Resilience;
		int32 AllocatedTileIndex;
	} ResilienceAndTileIndex;
	/** Random selection of orbit attributes. */
	float RandomOrbit;
	/** The offset at which to inject the new particle. */
	FVector3f Offset;
};


/**
 * Vertex factory for render sprites from GPU simulated particles.
 */
class FGPUSpriteVertexFactory : public FParticleVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE(FGPUSpriteVertexFactory);

public:
	FGPUSpriteVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FParticleVertexFactoryBase(InFeatureLevel)
	{
	}

	/** Emitter uniform buffer. */
	FRHIUniformBuffer* EmitterUniformBuffer;
	/** Emitter uniform buffer for dynamic parameters. */
	FUniformBufferRHIRef EmitterDynamicUniformBuffer;
	/** Buffer containing unsorted particle indices. */
	FRHIShaderResourceView* UnsortedParticleIndicesSRV;
	/** Texture containing positions for all particles. */
	FRHITexture* PositionTextureRHI;
	/** Texture containing velocities for all particles. */
	FRHITexture* VelocityTextureRHI;
	/** Texture containint attributes for all particles. */
	FRHITexture* AttributesTextureRHI;
	/** LWC tile offset, will be 0,0,0 for localspace emitters. */
	FVector3f LWCTile;
	/** Tile page offset factors associated with the GPU particle simulation resources. */
	FVector3f TilePageScale;

	FGPUSpriteVertexFactory()
		: FParticleVertexFactoryBase(PVFT_MAX, ERHIFeatureLevel::Num)
		, UnsortedParticleIndicesSRV(0)
		, PositionTextureRHI(nullptr)
		, VelocityTextureRHI(nullptr)
		, AttributesTextureRHI(nullptr)
		, LWCTile(FVector3f::ZeroVector)
		, TilePageScale(FVector3f::OneVector)
	{}

	/**
	 * Constructs render resources for this vertex factory.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual bool RendersPrimitivesAsCameraFacingSprites() const override { return true; }

	/**
	 * Set the source vertex buffer that contains particle indices.
	 */
	void SetUnsortedParticleIndicesSRV(FRHIShaderResourceView* VertexBuffer)
	{
		UnsortedParticleIndicesSRV = VertexBuffer;
	}

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory?
	 */
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	 */
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	 * Get vertex elements used when during PSO precaching materials using this vertex factory type
	 */
	static void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
};

/*-----------------------------------------------------------------------------
	FX system declaration.
-----------------------------------------------------------------------------*/

/**
 * FX system.
 */
class FFXSystem : public FFXSystemInterface
{
public:

	/** Default constructoer. */
	FFXSystem(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InShaderPlatform, FGPUSortManager* InGPUSortManager);

	/** Destructor. */
	virtual ~FFXSystem();

	static const FName Name;
	virtual FFXSystemInterface* GetInterface(const FName& InName) override;

	// Begin FFXSystemInterface.
	virtual void Tick(UWorld* World, float DeltaSeconds) override;
	virtual void OnMarkPendingKill() override;
#if WITH_EDITOR
	virtual void Suspend() override;
	virtual void Resume() override;
#endif // #if WITH_EDITOR
	virtual void DrawDebug(FCanvas* Canvas) override;
	virtual void AddVectorField(UVectorFieldComponent* VectorFieldComponent) override;
	virtual void RemoveVectorField(UVectorFieldComponent* VectorFieldComponent) override;
	virtual void UpdateVectorField(UVectorFieldComponent* VectorFieldComponent) override;
	FParticleEmitterInstance* CreateGPUSpriteEmitterInstance(IParticleEmitterInstanceOwner& InComponent, FGPUSpriteEmitterInfo& EmitterInfo);
	virtual void PreInitViews(class FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*>& ViewFamilies, const FSceneViewFamily* CurrentFamily) override;
	virtual void PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate) override;
	virtual bool UsesGlobalDistanceField() const override;
	virtual bool UsesDepthBuffer() const override;
	virtual bool RequiresEarlyViewUniformBuffer() const override;
	virtual bool RequiresRayTracingScene() const override;
	virtual void PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate) override;
	virtual void PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate) override;
	// End FFXSystemInterface.

	/*--------------------------------------------------------------------------
		Internal interface for GPU simulation.
	--------------------------------------------------------------------------*/
	/**
	 * Retrieve feature level that this FXSystem was created for
	 */
	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	/**
	 * Retrieve shaderplatform that this FXSystem was created for
	 */
	EShaderPlatform GetShaderPlatform() const { return GShaderPlatformForFeatureLevel[FeatureLevel]; }

	/**
	 * Add a new GPU simulation to the system.
	 * @param Simulation The GPU simulation to add.
	 */
	void AddGPUSimulation(FParticleSimulationGPU* Simulation);

	/**
	 * Remove an existing GPU simulation to the system.
	 * @param Simulation The GPU simulation to remove.
	 */
	void RemoveGPUSimulation(FParticleSimulationGPU* Simulation);

	/**
	 * Notifies the system that the SimulationPhase of a particular simulation has
	 * changed.
	 */
	void OnSimulationPhaseChanged(const FParticleSimulationGPU* Simulation, EParticleSimulatePhase::Type PrevPhase);

	/**
	 * Retrieve GPU particle rendering resources.
	 */
	FParticleSimulationResources* GetParticleSimulationResources()
	{
		return ParticleSimulationResources;
	}

	/** 
	 * Register work for GPU sorting (using the GPUSortManager). 
	 * The initial keys and values are generated in the GenerateSortKeys() callback.
	 *
	 * @param Simulation The simulation to be sorted.
	 * @param ViewOrigin The origin of the view from which to sort.
	 * @param bIsTranslucent Whether this is for sorting translucent particles or opaque particles, affect when the data is required for the rendering.
	 * @param OutInfo The bindings for this GPU sort task, if success. 
	 * @returns true if the work was registered, or false it GPU sorting is not available or impossible.
	 */
	bool AddSortedGPUSimulation(FRHICommandListBase& RHICmdList, FParticleSimulationGPU* Simulation, const FVector& ViewOrigin, bool bIsTranslucent, FGPUSortManager::FAllocationInfo& OutInfo);

	/** Get the shared SortManager, used in the rendering loop to call FGPUSortManager::OnPreRender() and FGPUSortManager::OnPostRenderOpaque() */
	virtual FGPUSortManager* GetGPUSortManager() const override;

	virtual void SetSceneTexturesUniformBuffer(const TUniformBufferRef<FSceneTextureUniformParameters>& InSceneTexturesUniformParams) override { SceneTexturesUniformParams = InSceneTexturesUniformParams; }

private:

	/**
	 * Generate all the initial keys and values for a GPUSortManager sort batch.
	 * Sort batches are created when GPU sort tasks are registered in AddSortedGPUSimulation().
	 * Each sort task defines constraints about when the initial sort data can generated and
	 * and when the sorted results are needed (see EGPUSortFlags for details).
	 * Currently, for Cascade, all the sort tasks have the EGPUSortFlags::KeyGenAfterPostRenderOpaque flag
	 * and so the callback registered in GPUSortManager->Register() only has the EGPUSortFlags::KeyGenAfterPostRenderOpaque usage.
	 * This garanties that GenerateSortKeys() only gets called after PostRenderOpaque(), which is a constraint required because
	 * cascade renders the GPU emitters after they have been ticked in PostRenderOpaque.
	 * Note that this callback must only initialize the content for the elements that relates to the tasks it has registered in this batch.
	 *
	 * @param RHICmdList The command list used to initiate the keys and values on GPU.
	 * @param BatchId The GPUSortManager batch id (regrouping several similar sort tasks).
	 * @param NumElementsInBatch The number of elements grouped in the batch (each element maps to a sort task)
	 * @param Flags Details about the key precision (see EGPUSortFlags::AnyKeyPrecision) and the keygen location (see EGPUSortFlags::AnyKeyGenLocation).
	 * @param KeysUAV The UAV that holds all the initial keys used to sort the values (being the particle indices here). 
	 * @param ValuesUAV The UAV that holds the initial values (particle indices) to be sorted accordingly to the keys.
	 */
	void GenerateSortKeys(FRHICommandListImmediate& RHICmdList, int32 BatchId, int32 NumElementsInBatch, EGPUSortFlags Flags, FRHIUnorderedAccessView* KeysUAV, FRHIUnorderedAccessView* ValuesUAV);

	/*--------------------------------------------------------------------------
		Private interface for GPU simulations.
	--------------------------------------------------------------------------*/

	/**
	 * Initializes GPU simulation for this system.
	 */
	void InitGPUSimulation();

	/**
	 * Destroys any resources allocated for GPU simulation for this system.
	 */
	virtual void DestroyGPUSimulation();

	/**
	 * Initializes GPU resources.
	 */
	void InitGPUResources();

	/**
	 * Releases GPU resources.
	 */
	void ReleaseGPUResources();

	/**
	 * Prepares GPU particles for simulation and rendering in the next frame.
	 */
	void AdvanceGPUParticleFrame(FRHICommandListImmediate& RHICmdList, bool bAllowGPUParticleUpdate);

	bool UsesGlobalDistanceFieldInternal() const;
	bool UsesDepthBufferInternal() const;
	bool RequiresEarlyViewUniformBufferInternal() const;
	bool RequiresRayTracingSceneInternal() const;

	/**
	* Updates resources used in a multi-GPU context
	*/
	void UpdateMultiGPUResources(FRHICommandListImmediate& RHICmdList);

	/**
	 * Update particles simulated on the GPU.
	 * @param Phase				Which emitters are being simulated.
	 * @param CollisionView		View to be used for collision checks.
	 */
	void SimulateGPUParticles(
		FRHICommandListImmediate& RHICmdList,
		EParticleSimulatePhase::Type Phase,
		const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
		const FGlobalDistanceFieldParameterData* GlobalDistanceFieldParameterData
		);

	/**
	 * Visualizes the current state of GPU particles.
	 * @param Canvas The canvas on which to draw the visualization.
	 */
	void VisualizeGPUParticles(FCanvas* Canvas);

private:

	template<typename TVectorFieldUniformParametersType>
	void SimulateGPUParticles_Internal(
		FRHICommandListImmediate& RHICmdList,
		EParticleSimulatePhase::Type Phase,
		const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
		const FGlobalDistanceFieldParameterData* GlobalDistanceFieldParameterData,
		FRHITexture* SceneDepthTexture,
		FRHITexture* GBufferATexture
	);

	/*-------------------------------------------------------------------------
		GPU simulation state.
	-------------------------------------------------------------------------*/

	/** List of all vector field instances. */
	FVectorFieldInstanceList VectorFields;
	/** List of all active GPU simulations. */
	TSparseArray<FParticleSimulationGPU*> GPUSimulations;
	/** Number of simulations of each type. */
	int32 NumGPUSimulations[EParticleSimulatePhase::Last + 1] = {};
	/** Particle render resources. */
	FParticleSimulationResources* ParticleSimulationResources;
	/** Feature level of this effects system */
	ERHIFeatureLevel::Type FeatureLevel;

	/** The shared GPUSortManager, used to register GPU sort tasks in order to generate sorted particle indices per emitter. */
	TRefCountPtr<FGPUSortManager> GPUSortManager;
	/** All sort tasks registered in AddSortedGPUSimulation(). Holds all the data required in GenerateSortKeys(). */
	TArray<FParticleSimulationSortInfo> SimulationsToSort;

	/** Previous frame new particles for multi-gpu simulation*/
	TArray<FNewParticle> LastFrameNewParticles;

	UE::FMutex AddSortedGPUSimulationMutex;

#if WITH_EDITOR
	/** true if the system has been suspended. */
	bool bSuspended;
#endif // #if WITH_EDITOR

#if WITH_MGPU
	EParticleSimulatePhase::Type PhaseToWaitForResourceTransfer = EParticleSimulatePhase::First;
	EParticleSimulatePhase::Type PhaseToBroadcastResourceTransfer = EParticleSimulatePhase::First;
#endif

	TUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformParams;
};

