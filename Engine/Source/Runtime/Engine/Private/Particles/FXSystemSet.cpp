// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FXSystemSet.cpp: Internal redirector to several fx systems.
=============================================================================*/

#include "FXSystemSet.h"
#include "FXRenderingUtils.h"
#include "GPUSortManager.h"
#include "Containers/StridedView.h"

FFXSystemSet::FFXSystemSet(FGPUSortManager* InGPUSortManager)
	: GPUSortManager(InGPUSortManager)
{
}

FFXSystemInterface* FFXSystemSet::GetInterface(const FName& InName)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		if (FFXSystemInterface* FXSystemInterface = FXSystem->GetInterface(InName))
		{
			return FXSystemInterface;
		}
	}
	return nullptr;
}

void FFXSystemSet::Tick(UWorld* World, float DeltaSeconds)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->Tick(World, DeltaSeconds);
	}
}

#if WITH_EDITOR

void FFXSystemSet::Suspend()
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->Suspend();
	}
}

void FFXSystemSet::Resume()
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->Resume();
	}
}

#endif // #if WITH_EDITOR

void FFXSystemSet::DrawDebug(FCanvas* Canvas)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->DrawDebug(Canvas);
	}
}

bool FFXSystemSet::ShouldDebugDraw_RenderThread() const
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		if (FXSystem->ShouldDebugDraw_RenderThread())
		{
			return true;
		}
	}
	return false;
}

void FFXSystemSet::DrawDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FScreenPassRenderTarget& Output)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->DrawDebug_RenderThread(GraphBuilder, View, Output);
	}
}

void FFXSystemSet::DrawSceneDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->DrawSceneDebug_RenderThread(GraphBuilder, View, SceneColor, SceneDepth);
	}
}

void FFXSystemSet::AddVectorField(UVectorFieldComponent* VectorFieldComponent)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->AddVectorField(VectorFieldComponent);
	}
}

void FFXSystemSet::RemoveVectorField(UVectorFieldComponent* VectorFieldComponent)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->RemoveVectorField(VectorFieldComponent);
	}
}

void FFXSystemSet::UpdateVectorField(UVectorFieldComponent* VectorFieldComponent)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->UpdateVectorField(VectorFieldComponent);
	}
}

void FFXSystemSet::PreInitViews(FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*>& ViewFamilies, const FSceneViewFamily* CurrentFamily)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->PreInitViews(GraphBuilder, bAllowGPUParticleUpdate, ViewFamilies, CurrentFamily);
	}
}

void FFXSystemSet::PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->PostInitViews(GraphBuilder, Views, bAllowGPUParticleUpdate);
	}
}

bool FFXSystemSet::UsesGlobalDistanceField() const
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		if (FXSystem->UsesGlobalDistanceField())
		{
			return true;
		}
	}
	return false;
}

bool FFXSystemSet::UsesDepthBuffer() const
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		if (FXSystem->UsesDepthBuffer())
		{
			return true;
		}
	}
	return false;
}

bool FFXSystemSet::RequiresEarlyViewUniformBuffer() const
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		if (FXSystem->RequiresEarlyViewUniformBuffer())
		{
			return true;
		}
	}
	return false;
}

bool FFXSystemSet::RequiresRayTracingScene() const
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		if (FXSystem->RequiresRayTracingScene())
		{
			return true;
		}
	}
	return false;
}

void FFXSystemSet::PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleSceneUpdate)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->PreRender(GraphBuilder, Views, SceneUniformBuffer, bAllowGPUParticleSceneUpdate);
	}
}

void FFXSystemSet::SetSceneTexturesUniformBuffer(const TUniformBufferRef<FSceneTextureUniformParameters>& InSceneTexturesUniformParams)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->SetSceneTexturesUniformBuffer(InSceneTexturesUniformParams);
	}
}

void FFXSystemSet::PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleSceneUpdate)
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->PostRenderOpaque(GraphBuilder, Views, SceneUniformBuffer, bAllowGPUParticleSceneUpdate);
	}
}

void FFXSystemSet::OnMarkPendingKill()
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->OnMarkPendingKill();
	}

	FFXSystemInterface::OnMarkPendingKill();
}

void FFXSystemSet::DestroyGPUSimulation()
{
	for (const TSharedRef<FFXSystemInterface>& FXSystem : FXSystems)
	{
		FXSystem->DestroyGPUSimulation();
	}
}

FFXSystemSet::~FFXSystemSet()
{
	FXSystems.Empty();
}

FGPUSortManager* FFXSystemSet::GetGPUSortManager() const
{
	return GPUSortManager;
}
