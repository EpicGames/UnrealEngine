// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FXSystemSet.h: Internal redirector to several fx systems.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "FXSystem.h"
#include "Templates/RefCounting.h"
#include "RenderGraphDefinitions.h"

class FGPUSortManager;

/**
 * FX system.
 */
class FFXSystemSet : public FFXSystemInterface
{
public:

	FFXSystemSet(FGPUSortManager* InGPUSortManager);

	TArray<TSharedRef<FFXSystemInterface>> FXSystems;

	virtual FFXSystemInterface* GetInterface(const FName& InName) override;
	virtual void Tick(UWorld* World, float DeltaSeconds) override;

#if WITH_EDITOR
	virtual void Suspend() override;
	virtual void Resume() override;
#endif // #if WITH_EDITOR

	virtual void DrawDebug(FCanvas* Canvas) override;
	virtual bool ShouldDebugDraw_RenderThread() const override;
	virtual void DrawDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FScreenPassRenderTarget& Output) override;
	virtual void DrawSceneDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth) override;
	virtual void AddVectorField(UVectorFieldComponent* VectorFieldComponent) override;
	virtual void RemoveVectorField(UVectorFieldComponent* VectorFieldComponent) override;
	virtual void UpdateVectorField(UVectorFieldComponent* VectorFieldComponent) override;
	virtual void PreInitViews(FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*>& ViewFamilies, const FSceneViewFamily* CurrentFamily) override;
	virtual void PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate) override;
	virtual bool UsesGlobalDistanceField() const override;
	virtual bool UsesDepthBuffer() const override;
	virtual bool RequiresEarlyViewUniformBuffer() const override;
	virtual bool RequiresRayTracingScene() const override;
	virtual void PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleSceneUpdate) override;
	virtual void PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleSceneUpdate) override;

	virtual void OnMarkPendingKill() override;
	virtual void DestroyGPUSimulation() override;

	/** Get the shared SortManager, used in the rendering loop to call FGPUSortManager::OnPreRender() and FGPUSortManager::OnPostRenderOpaque() */
	virtual FGPUSortManager* GetGPUSortManager() const override;

	virtual void SetSceneTexturesUniformBuffer(const TUniformBufferRef<FSceneTextureUniformParameters>& InSceneTexturesUniformParams) override;

protected:

	/** By making the destructor protected, an instance must be destroyed via FFXSystemInterface::Destroy. */
	virtual ~FFXSystemSet();

	// We keep a reference to the GPUSortManager only for GetGPUSortManager().
	TRefCountPtr<FGPUSortManager> GPUSortManager;
};
