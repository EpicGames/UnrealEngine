// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FParticleBeam2EmitterInstance.cpp: 
	Particle beam emitter instance implementation.
=============================================================================*/

#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/Beam/ParticleModuleBeamBase.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/Beam/ParticleModuleBeamModifier.h"
#include "Particles/Beam/ParticleModuleBeamNoise.h"
#include "Particles/Beam/ParticleModuleBeamSource.h"
#include "Particles/Beam/ParticleModuleBeamTarget.h"
#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/ParticleModule.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/ParticleSystem.h"
#include "Particles/TypeData/ParticleModuleTypeDataBeam2.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Stats/StatsTrace.h"

/** Beam particle stat objects */


DEFINE_STAT(STAT_BeamParticles);
DEFINE_STAT(STAT_BeamParticlesRenderCalls);
DEFINE_STAT(STAT_BeamParticlesSpawned);
DEFINE_STAT(STAT_BeamParticlesUpdateCalls);
DEFINE_STAT(STAT_BeamParticlesUpdated);
DEFINE_STAT(STAT_BeamParticlesKilled);
DEFINE_STAT(STAT_BeamParticlesTrianglesRendered);

DEFINE_STAT(STAT_BeamSpawnTime);
DEFINE_STAT(STAT_BeamFillVertexTime);
DEFINE_STAT(STAT_BeamFillIndexTime);
DEFINE_STAT(STAT_BeamRenderingTime);
DEFINE_STAT(STAT_BeamTickTime);

DECLARE_CYCLE_STAT(TEXT("BeamEmitterInstance Init"), STAT_BeamEmitterInstance_Init, STATGROUP_Particles);

/*-----------------------------------------------------------------------------
	ParticleBeam2EmitterInstance
-----------------------------------------------------------------------------*/
/**
 *	Structure for beam emitter instances
 */
/** Constructor	*/
FParticleBeam2EmitterInstance::FParticleBeam2EmitterInstance(IParticleEmitterInstanceOwner& InComponent) :
	FParticleEmitterInstance(InComponent)
	, BeamTypeData(NULL)
	, BeamModule_Source(NULL)
	, BeamModule_Target(NULL)
	, BeamModule_Noise(NULL)
	, BeamModule_SourceModifier(NULL)
	, BeamModule_SourceModifier_Offset(-1)
	, BeamModule_TargetModifier(NULL)
	, BeamModule_TargetModifier_Offset(-1)
	, FirstEmission(true)
	, TickCount(0)
	, ForceSpawnCount(0)
	, BeamMethod(0)
	, BeamCount(0)
	, SourceActor(NULL)
	, SourceEmitter(NULL)
	, TargetActor(NULL)
	, TargetEmitter(NULL)
	, VertexCount(0)
	, TriangleCount(0)
{
	TextureTiles.Empty();
	UserSetSourceArray.Empty();
	UserSetSourceTangentArray.Empty();
	UserSetSourceStrengthArray.Empty();
	DistanceArray.Empty();
	TargetPointArray.Empty();
	TargetTangentArray.Empty();
	UserSetTargetStrengthArray.Empty();
	TargetPointSourceNames.Empty();
	UserSetTargetArray.Empty();
	UserSetTargetTangentArray.Empty();
	BeamTrianglesPerSheet.Empty();

	bIsBeam = true;
}

/** Destructor	*/
FParticleBeam2EmitterInstance::~FParticleBeam2EmitterInstance()
{
	TextureTiles.Empty();
	UserSetSourceArray.Empty();
	UserSetSourceTangentArray.Empty();
	UserSetSourceStrengthArray.Empty();
	DistanceArray.Empty();
	TargetPointArray.Empty();
	TargetTangentArray.Empty();
	UserSetTargetStrengthArray.Empty();
	TargetPointSourceNames.Empty();
	UserSetTargetArray.Empty();
	UserSetTargetTangentArray.Empty();
	BeamTrianglesPerSheet.Empty();
}

/**
 *	Set the end point position
 *
 *	@param	NewEndPoint
 */
void FParticleBeam2EmitterInstance::SetBeamEndPoint(FVector NewEndPoint)
{
	if (UserSetTargetArray.Num() < 1)
	{
		UserSetTargetArray.AddUninitialized(1);

	}
	UserSetTargetArray[0] = NewEndPoint;
}

/**
 *	Set the source point
 *
 *	@param	NewSourcePoint
 *	@param	SourceIndex			The index of the source being set
 */
void FParticleBeam2EmitterInstance::SetBeamSourcePoint(FVector NewSourcePoint,int32 SourceIndex)
{
	if (SourceIndex < 0)
		return;

	if (UserSetSourceArray.Num() < (SourceIndex + 1))
	{
		UserSetSourceArray.AddUninitialized((SourceIndex + 1) - UserSetSourceArray.Num());
	}
	UserSetSourceArray[SourceIndex] = NewSourcePoint;
}

/**
 *	Set the source tangent
 *
 *	@param	NewTangentPoint		The tangent value to set it to
 *	@param	SourceIndex			The index of the source being set
 */
void FParticleBeam2EmitterInstance::SetBeamSourceTangent(FVector NewTangentPoint,int32 SourceIndex)
{
	if (SourceIndex < 0)
		return;

	if (UserSetSourceTangentArray.Num() < (SourceIndex + 1))		
	{
		UserSetSourceTangentArray.AddUninitialized((SourceIndex + 1) - UserSetSourceTangentArray.Num());
	}
	UserSetSourceTangentArray[SourceIndex] = NewTangentPoint;
}

/**
 *	Set the source strength
 *
 *	@param	NewSourceStrength	The source strenght to set it to
 *	@param	SourceIndex			The index of the source being set
 */
void FParticleBeam2EmitterInstance::SetBeamSourceStrength(float NewSourceStrength,int32 SourceIndex)
{
	if (SourceIndex < 0)
		return;

	if (UserSetSourceStrengthArray.Num() < (SourceIndex + 1))
	{
		UserSetSourceStrengthArray.AddUninitialized((SourceIndex + 1) - UserSetSourceStrengthArray.Num());
	}
	UserSetSourceStrengthArray[SourceIndex] = NewSourceStrength;
}

/**
 *	Set the target point
 *
 *	@param	NewTargetPoint		The target point to set it to
 *	@param	TargetIndex			The index of the target being set
 */
void FParticleBeam2EmitterInstance::SetBeamTargetPoint(FVector NewTargetPoint,int32 TargetIndex)
{
	if (TargetIndex < 0)
		return;

	if (UserSetTargetArray.Num() < (TargetIndex + 1))
	{
		UserSetTargetArray.AddUninitialized((TargetIndex + 1) - UserSetTargetArray.Num());
	}
	UserSetTargetArray[TargetIndex] = NewTargetPoint;
}

/**
 *	Set the target tangent
 *
 *	@param	NewTangentPoint		The tangent to set it to
 *	@param	TargetIndex			The index of the target being set
 */
void FParticleBeam2EmitterInstance::SetBeamTargetTangent(FVector NewTangentPoint,int32 TargetIndex)
{
	if (TargetIndex < 0)
		return;

	if (UserSetTargetTangentArray.Num() < (TargetIndex + 1))
	{
		UserSetTargetTangentArray.AddUninitialized((TargetIndex + 1) - UserSetTargetTangentArray.Num());
	}
	UserSetTargetTangentArray[TargetIndex] = NewTangentPoint;
}

/**
 *	Set the target strength
 *
 *	@param	NewTargetStrength	The strength to set it ot
 *	@param	TargetIndex			The index of the target being set
 */
void FParticleBeam2EmitterInstance::SetBeamTargetStrength(float NewTargetStrength,int32 TargetIndex)
{
	if (TargetIndex < 0)
		return;

	if (UserSetTargetStrengthArray.Num() < (TargetIndex + 1))
	{
		UserSetTargetStrengthArray.AddUninitialized((TargetIndex + 1) - UserSetTargetStrengthArray.Num());
	}
	UserSetTargetStrengthArray[TargetIndex] = NewTargetStrength;
}

/**
*	Get the end point position
*
*	@param	OutEndPoint	Current end point position
*
*	@return	true		End point is set - OutEndPoint is valid
*			false		End point is not set - OutEndPoint is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamEndPoint(FVector& OutEndPoint) const
{
	if (UserSetTargetArray.Num() < 1)
	{
		return false;
	}
	OutEndPoint = UserSetTargetArray[0];
	return true;
}

/**
*	Get the source point at the given index
*
*	@param	SourceIndex			The index of the source point
*	@param	OutSourcePoint		Value of of source point
*
*	@return	true		SourceIndex is valid - OutSourcePoint is valid
*			false		SourceIndex is invalid - OutSourcePoint is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamSourcePoint(int32 SourceIndex, FVector& OutSourcePoint) const
{  
	if (SourceIndex < 0 || (UserSetSourceArray.Num() < (SourceIndex + 1)))
	{
		return false;
	}

	OutSourcePoint = UserSetSourceArray[SourceIndex];
	return true;
}

/**
*	Get the source tangent at the given index
*
*	@param	SourceIndex			The index of the source point
*	@param	OutSourceTangent	Value of of source tangent
*
*	@return	true		SourceIndex is valid - OutSourceTangent is valid
*			false		SourceIndex is invalid - OutSourceTangent is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamSourceTangent(int32 SourceIndex, FVector& OutSourcePoint) const
{
	if (SourceIndex < 0 || (UserSetSourceTangentArray.Num() < (SourceIndex + 1)))
	{
		return false;
	}

	OutSourcePoint = UserSetSourceTangentArray[SourceIndex];
	return true;
}

/**
*	Get the source strength at the given index
*
*	@param	SourceIndex			The index of the source point
*	@param	OutSourceStrength	Value of of source strength
*
*	@return	true		SourceIndex is valid - OutSourceStrength is valid
*			false		SourceIndex is invalid - OutSourceStrength is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamSourceStrength(int32 SourceIndex, float& OutSourceStrength) const
{
	if (SourceIndex < 0 || (UserSetSourceStrengthArray.Num() < (SourceIndex + 1)))
	{
		return false;
	}

	OutSourceStrength = UserSetSourceStrengthArray[SourceIndex];
	return true;
}

/**
*	Get the target point at the given index
*
*	@param	TargetIndex			The index of the target point
*	@param	OutTargetPoint		Value of of target point
*
*	@return	true		TargetIndex is valid - OutTargetPoint is valid
*			false		TargetIndex is invalid - OutTargetPoint is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamTargetPoint(int32 TargetIndex, FVector& OutTargetPoint) const
{
	if (TargetIndex < 0 || (UserSetTargetArray.Num() < (TargetIndex + 1)))
	{
		return false;
	}

	OutTargetPoint = UserSetTargetArray[TargetIndex];
	return true;
}

/**
*	Get the target tangent at the given index
*
*	@param	TargetIndex			The index of the target point
*	@param	OutTangentPoint		Value of of target tangent
*
*	@return	true		TargetIndex is valid - OutTangentPoint is valid
*			false		TargetIndex is invalid - OutTangentPoint is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamTargetTangent(int32 TargetIndex, FVector& OutTangentPoint) const
{
	if (TargetIndex < 0 || (UserSetTargetTangentArray.Num() < (TargetIndex + 1)))
	{
		return false;
	}

	OutTangentPoint = UserSetTargetTangentArray[TargetIndex];
	return true;
}

/**
*	Get the target strength at the given index
*
*	@param	TargetIndex			The index of the target point
*	@param	OutTargetStrength	Value of of target strength
*
*	@return	true		TargetIndex is valid - OutTargetStrength is valid
*			false		TargetIndex is invalid - OutTargetStrength is invalid
*/
bool FParticleBeam2EmitterInstance::GetBeamTargetStrength(int32 TargetIndex, float& OutTargetStrength) const
{
	if (TargetIndex < 0 || (UserSetTargetStrengthArray.Num() < (TargetIndex + 1)))
	{
		return false;
	}

	OutTargetStrength = UserSetTargetStrengthArray[TargetIndex];
	return true;
}

void FParticleBeam2EmitterInstance::ApplyWorldOffset(FVector InOffset, bool bWorldShift)
{
	FParticleEmitterInstance::ApplyWorldOffset(InOffset, bWorldShift);
	
	for (auto It = UserSetSourceArray.CreateIterator(); It; ++It)
	{
		(*It)+= InOffset;
	}

	for (auto It = TargetPointArray.CreateIterator(); It; ++It)
	{
		(*It)+= InOffset;
	}
}


void FParticleBeam2EmitterInstance::InitParameters(UParticleEmitter* InTemplate)
{
	FParticleEmitterInstance::InitParameters(InTemplate);

	UParticleLODLevel* LODLevel	= InTemplate->GetLODLevel(0);
	check(LODLevel);
	BeamTypeData = CastChecked<UParticleModuleTypeDataBeam2>(LODLevel->TypeDataModule);
	check(BeamTypeData);

	//@todo. Determine if we need to support local space.
	if (LODLevel->RequiredModule->bUseLocalSpace)
	{
		LODLevel->RequiredModule->bUseLocalSpace	= false;
	}

	BeamModule_Source			= NULL;
	BeamModule_Target			= NULL;
	BeamModule_Noise			= NULL;
	BeamModule_SourceModifier	= NULL;
	BeamModule_TargetModifier	= NULL;

	// Always have at least one beam
	if (BeamTypeData->MaxBeamCount == 0)
	{
		BeamTypeData->MaxBeamCount	= 1;
	}

	BeamCount					= BeamTypeData->MaxBeamCount;
	FirstEmission				= true;
	TickCount					= 0;
	ForceSpawnCount				= 0;

	BeamMethod					= BeamTypeData->BeamMethod;

	TextureTiles.Empty();
	TextureTiles.Add(BeamTypeData->TextureTile);

	UserSetSourceArray.Empty();
	UserSetSourceTangentArray.Empty();
	UserSetSourceStrengthArray.Empty();
	DistanceArray.Empty();
	TargetPointArray.Empty();
	TargetPointSourceNames.Empty();
	UserSetTargetArray.Empty();
	UserSetTargetTangentArray.Empty();
	UserSetTargetStrengthArray.Empty();

	// Resolve any actors...
	ResolveSource();
	ResolveTarget();
}

/**
 *	Initialize the instance
 */
void FParticleBeam2EmitterInstance::Init()
{
	SCOPE_CYCLE_COUNTER(STAT_BeamEmitterInstance_Init);

	// Setup the modules prior to initializing...
	check(SpriteTemplate);
	UParticleLODLevel* LODLevel = SpriteTemplate->GetLODLevel(0);
	check(LODLevel);
	BeamTypeData = CastChecked<UParticleModuleTypeDataBeam2>(LODLevel->TypeDataModule);

	BeamModule_Source = BeamTypeData->LOD_BeamModule_Source[0];
	BeamModule_Target = BeamTypeData->LOD_BeamModule_Target[0];
	BeamModule_Noise = BeamTypeData->LOD_BeamModule_Noise[0];
	BeamModule_SourceModifier = BeamTypeData->LOD_BeamModule_SourceModifier[0];
	BeamModule_TargetModifier = BeamTypeData->LOD_BeamModule_TargetModifier[0];

	FParticleEmitterInstance::Init();

	SetupBeamModifierModulesOffsets();
}

/**
 *	Tick the instance.
 *
 *	@param	DeltaTime			The time slice to use
 *	@param	bSuppressSpawning	If true, do not spawn during Tick
 */
void FParticleBeam2EmitterInstance::Tick(float DeltaTime, bool bSuppressSpawning)
{
	SCOPE_CYCLE_COUNTER(STAT_BeamTickTime);
	{
		UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
		check(LODLevel);	// TTP #33141

		// Handle EmitterTime setup, looping, etc.
		float EmitterDelay = Tick_EmitterTimeSetup(DeltaTime, LODLevel);

		if (bEnabled)
		{
			// Kill before the spawn... Otherwise, we can get 'flashing'
			KillParticles();

			// If not suppressing spawning...
			if (!bHaltSpawning && !bHaltSpawningExternal && !bSuppressSpawning && (EmitterTime >= 0.0f))
			{
				if ((LODLevel->RequiredModule->EmitterLoops == 0) ||
					(LoopCount < LODLevel->RequiredModule->EmitterLoops) ||
					(SecondsSinceCreation < (EmitterDuration * LODLevel->RequiredModule->EmitterLoops)))
				{
					// For beams, we probably want to ignore the SpawnRate distribution,
					// and focus strictly on the BurstList...
					float SpawnRate = 0.0f;
					// Figure out spawn rate for this tick.
					SpawnRate = LODLevel->SpawnModule->Rate.GetValue(EmitterTime, Component.GetDistributionData());
					// Take Bursts into account as well...
					int32		Burst = 0;
					float	BurstTime = GetCurrentBurstRateOffset(DeltaTime, Burst);
					SpawnRate += BurstTime;

					// Spawn new particles...

					//@todo. Fix the issue of 'blanking' beams when the count drops...
					// This is a temporary hack!
					const float InvDeltaTime = (DeltaTime > 0.0f) ? 1.0f / DeltaTime : 0.0f;
					if ((ActiveParticles < BeamCount) && (SpawnRate <= 0.0f))
					{
						// Force the spawn of a single beam...
						SpawnRate = 1.0f * InvDeltaTime;
					}

					// Force beams if the emitter is marked "AlwaysOn"
					if ((ActiveParticles < BeamCount) && BeamTypeData->bAlwaysOn)
					{
						Burst = BeamCount;
						if (DeltaTime > UE_KINDA_SMALL_NUMBER)
						{
							BurstTime = Burst * InvDeltaTime;
							SpawnRate += BurstTime;
						}
					}

					if (SpawnRate > 0.f)
					{
						SpawnFraction = SpawnBeamParticles(SpawnFraction, SpawnRate, DeltaTime, Burst, BurstTime);
					}
				}
			}
			else if (bFakeBurstsWhenSpawningSupressed)
			{
				FakeBursts();
			}

			// Reset particle data
			ResetParticleParameters(DeltaTime);

			// Not really necessary as beams do not LOD at the moment, but for consistency...
			CurrentMaterial = LODLevel->RequiredModule->Material;

			Tick_ModuleUpdate(DeltaTime, LODLevel);
			Tick_ModulePostUpdate(DeltaTime, LODLevel);

			// Calculate bounding box and simulate velocity.
			UpdateBoundingBox(DeltaTime);

			if (!bSuppressSpawning)
			{
				// Ensure that we flip the 'FirstEmission' flag
				FirstEmission = false;
			}

			// Invalidate the contents of the vertex/index buffer.
			IsRenderDataDirty = 1;

			// Bump the tick count
			TickCount++;
		}
		else
		{
			FakeBursts();
		}
		// 'Reset' the emitter time so that the delay functions correctly
		EmitterTime += CurrentDelay;
		
		// Reset particles position offset
		PositionOffsetThisTick = FVector::ZeroVector;
	}
	INC_DWORD_STAT(STAT_BeamParticlesUpdateCalls);
}

/**
 *	Tick sub-function that handles module post updates
 *
 *	@param	DeltaTime			The current time slice
 *	@param	CurrentLODLevel		The current LOD level for the instance
 */
void FParticleBeam2EmitterInstance::Tick_ModulePostUpdate(float DeltaTime, UParticleLODLevel* InCurrentLODLevel)
{
	if (InCurrentLODLevel->TypeDataModule)
	{
		// The order of the update here is VERY important
		if (BeamModule_Source && BeamModule_Source->bEnabled)
		{
			BeamModule_Source->Update({*this, (int)GetModuleDataOffset(BeamModule_Source), DeltaTime});
		}
		if (BeamModule_SourceModifier && BeamModule_SourceModifier->bEnabled)
		{
			int32 TempOffset = BeamModule_SourceModifier_Offset;
			BeamModule_SourceModifier->Update({*this, TempOffset, DeltaTime});
		}
		if (BeamModule_Target && BeamModule_Target->bEnabled)
		{
			BeamModule_Target->Update({*this, (int)GetModuleDataOffset(BeamModule_Target), DeltaTime});
		}
		if (BeamModule_TargetModifier && BeamModule_TargetModifier->bEnabled)
		{
			int32 TempOffset = BeamModule_TargetModifier_Offset;
			BeamModule_TargetModifier->Update({*this, TempOffset, DeltaTime});
		}
		if (BeamModule_Noise && BeamModule_Noise->bEnabled)
		{
			BeamModule_Noise->Update({*this, (int)GetModuleDataOffset(BeamModule_Noise), DeltaTime});
		}

		FParticleEmitterInstance::Tick_ModulePostUpdate(DeltaTime, InCurrentLODLevel);
	}
}

/**
 *	Set the LOD to the given index
 *
 *	@param	InLODIndex			The index of the LOD to set as current
 *	@param	bInFullyProcess		If true, process burst lists, etc.
 */
void FParticleBeam2EmitterInstance::SetCurrentLODIndex(int32 InLODIndex, bool bInFullyProcess)
{
	bool bDifferent = (InLODIndex != CurrentLODLevelIndex);
	FParticleEmitterInstance::SetCurrentLODIndex(InLODIndex, bInFullyProcess);

	// Setup the beam modules!
	BeamModule_Source = BeamTypeData->LOD_BeamModule_Source[CurrentLODLevelIndex];
	BeamModule_Target = BeamTypeData->LOD_BeamModule_Target[CurrentLODLevelIndex];
	BeamModule_Noise = BeamTypeData->LOD_BeamModule_Noise[CurrentLODLevelIndex];
	BeamModule_SourceModifier = BeamTypeData->LOD_BeamModule_SourceModifier[CurrentLODLevelIndex];
	BeamModule_TargetModifier= BeamTypeData->LOD_BeamModule_TargetModifier[CurrentLODLevelIndex];
}

/**
 *	Update the bounding box for the emitter
 *
 *	@param	DeltaTime		The time slice to use
 */
void FParticleBeam2EmitterInstance::UpdateBoundingBox(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleUpdateBounds);
	{
		const UParticleSystem* Template = Component.GetTemplate();
		bool bUpdateBox = ((Component.GetIsWarmingUp() == false) && (Template != NULL) && (Template->bUseFixedRelativeBoundingBox == false));
		float MaxSizeScale	= 1.0f;
		if (bUpdateBox)
		{
			ParticleBoundingBox.Init();

			//@todo. Currently, we don't support UseLocalSpace for beams
			//if (Template.UseLocalSpace == false) 
			{
				ParticleBoundingBox += Component.GetComponentTransform().GetLocation();
			}
		}

		FVector	NoiseMin(0.0f);
		FVector NoiseMax(0.0f);
		// Noise points have to be taken into account...
		if (BeamModule_Noise)
		{
			BeamModule_Noise->GetNoiseRange(NoiseMin, NoiseMax);
		}

		// Take scale into account as well
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		bool bSkipDoubleSpawnUpdate = !SpriteTemplate->bUseLegacySpawningBehavior;
		// Take each particle into account
		for (int32 i=0; i<ActiveParticles; i++)
		{
			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);

			int32						CurrentOffset		= TypeDataOffset;
			FBeam2TypeDataPayload*	BeamData			= NULL;
			FVector*				InterpolatedPoints	= NULL;
			float*					NoiseRate			= NULL;
			float*					NoiseDelta			= NULL;
			FVector*				TargetNoisePoints	= NULL;
			FVector*				NextNoisePoints		= NULL;
			float*					TaperValues			= NULL;
			float*					NoiseDistanceScale	= NULL;
			FBeamParticleModifierPayloadData* SourceModifier = NULL;
			FBeamParticleModifierPayloadData* TargetModifier = NULL;

			bool bJustSpawned = (Particle->Flags & STATE_Particle_JustSpawned) != 0;
			Particle->Flags &= ~STATE_Particle_JustSpawned;

			BeamTypeData->GetDataPointers(this, (const uint8*)Particle, CurrentOffset, 
				BeamData, InterpolatedPoints, NoiseRate, NoiseDelta, TargetNoisePoints, 
				NextNoisePoints, TaperValues, NoiseDistanceScale,
				SourceModifier, TargetModifier);

			// Do linear integrator and update bounding box
			Particle->OldLocation = Particle->Location;

			bool bSkipUpdate = bJustSpawned && bSkipDoubleSpawnUpdate;

			Particle->Location	+= bSkipUpdate ? FVector::ZeroVector : DeltaTime * (FVector)Particle->Velocity;
			Particle->Rotation	+= bSkipUpdate ? 0.0f : DeltaTime * Particle->RotationRate;
			Particle->OldLocation += PositionOffsetThisTick;
			FVector Size = (FVector)Particle->Size * Scale;
			if (bUpdateBox)
			{
				ParticleBoundingBox += Particle->Location;
				ParticleBoundingBox += Particle->Location + NoiseMin;
				ParticleBoundingBox += Particle->Location + NoiseMax;
				ParticleBoundingBox += (FVector)BeamData->SourcePoint;
				ParticleBoundingBox += (FVector)BeamData->SourcePoint + NoiseMin;
				ParticleBoundingBox += (FVector)BeamData->SourcePoint + NoiseMax;
				ParticleBoundingBox += (FVector)BeamData->TargetPoint;
				ParticleBoundingBox += (FVector)BeamData->TargetPoint + NoiseMin;
				ParticleBoundingBox += (FVector)BeamData->TargetPoint + NoiseMax;
			}

			// Do angular integrator, and wrap result to within +/- 2 PI
			Particle->Rotation	 = FMath::Fmod(Particle->Rotation, 2.f*(float)UE_PI);
			MaxSizeScale		 = FMath::Max<float>(MaxSizeScale, Size.GetAbsMax()); //@todo particles: this does a whole lot of compares that can be avoided using SSE/ Altivec.
		}
		if (bUpdateBox)
		{
			ParticleBoundingBox = ParticleBoundingBox.ExpandBy(MaxSizeScale);
		}
	}
}

/**
 * Force the bounding box to be updated.
 */
void FParticleBeam2EmitterInstance::ForceUpdateBoundingBox()
{
	{
		float MaxSizeScale	= 1.0f;
		ParticleBoundingBox.Init();
		ParticleBoundingBox += Component.GetComponentTransform().GetLocation();

		FVector	NoiseMin(0.0f);
		FVector NoiseMax(0.0f);
		// Noise points have to be taken into account...
		if (BeamModule_Noise)
		{
			BeamModule_Noise->GetNoiseRange(NoiseMin, NoiseMax);
		}

		// Take scale into account as well
		FVector Scale = Component.GetComponentTransform().GetScale3D();

		// Take each particle into account
		for (int32 i=0; i<ActiveParticles; i++)
		{
			DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);

			int32						CurrentOffset		= TypeDataOffset;
			FBeam2TypeDataPayload*	BeamData			= NULL;
			FVector*				InterpolatedPoints	= NULL;
			float*					NoiseRate			= NULL;
			float*					NoiseDelta			= NULL;
			FVector*				TargetNoisePoints	= NULL;
			FVector*				NextNoisePoints		= NULL;
			float*					TaperValues			= NULL;
			float*					NoiseDistanceScale	= NULL;
			FBeamParticleModifierPayloadData* SourceModifier = NULL;
			FBeamParticleModifierPayloadData* TargetModifier = NULL;

			BeamTypeData->GetDataPointers(this, (const uint8*)Particle, CurrentOffset, 
				BeamData, InterpolatedPoints, NoiseRate, NoiseDelta, TargetNoisePoints, 
				NextNoisePoints, TaperValues, NoiseDistanceScale,
				SourceModifier, TargetModifier);

			FVector Size = (FVector)Particle->Size * Scale;

			ParticleBoundingBox += Particle->Location;
			ParticleBoundingBox += Particle->Location + NoiseMin;
			ParticleBoundingBox += Particle->Location + NoiseMax;
			ParticleBoundingBox += (FVector)BeamData->SourcePoint;
			ParticleBoundingBox += (FVector)BeamData->SourcePoint + NoiseMin;
			ParticleBoundingBox += (FVector)BeamData->SourcePoint + NoiseMax;
			ParticleBoundingBox += (FVector)BeamData->TargetPoint;
			ParticleBoundingBox += (FVector)BeamData->TargetPoint + NoiseMin;
			ParticleBoundingBox += (FVector)BeamData->TargetPoint + NoiseMax;

			MaxSizeScale = FMath::Max<FVector::FReal>(MaxSizeScale, Size.GetAbsMax()); //@todo particles: this does a whole lot of compares that can be avoided using SSE/ Altivec.
		}

		ParticleBoundingBox = ParticleBoundingBox.ExpandBy(MaxSizeScale);
	}
}

/**
 *	Retrieved the per-particle bytes that this emitter type requires.
 *
 *	@return	uint32	The number of required bytes for particles in the instance
 */
uint32 FParticleBeam2EmitterInstance::RequiredBytes()
{
	uint32 uiBytes = FParticleEmitterInstance::RequiredBytes();

	// Flag bits indicating particle 
	uiBytes += sizeof(int32);

	return uiBytes;
}

/**
 *	Spawn particles for this instance
 *
 *	@param	OldLeftover		The leftover time from the last spawn
 *	@param	Rate			The rate at which particles should be spawned
 *	@param	DeltaTime		The time slice to spawn over
 *	@param	Burst			The number of burst particle
 *	@param	BurstTime		The burst time addition (faked time slice)
 *
 *	@return	float			The leftover fraction of spawning
 */
float FParticleBeam2EmitterInstance::SpawnBeamParticles(float OldLeftover, float Rate, float DeltaTime, int32 Burst, float BurstTime)
{
	SCOPE_CYCLE_COUNTER(STAT_BeamSpawnTime);

	float SafetyLeftover = OldLeftover;
	float	NewLeftover = OldLeftover + DeltaTime * Rate;

	// Ensure continous spawning... lots of fiddling.
	int32		Number		= FMath::FloorToInt(NewLeftover);
	float	Increment	= 1.f / Rate;
	float	StartTime	= DeltaTime + OldLeftover * Increment - Increment;
	NewLeftover			= NewLeftover - Number;

	// Always match the burst at a minimum
	if (Number < Burst)
	{
		Number = Burst;
	}

	// Account for burst time simulation
	if (BurstTime > UE_KINDA_SMALL_NUMBER)
	{
		NewLeftover -= BurstTime / Burst;
		NewLeftover	= FMath::Clamp<float>(NewLeftover, 0, NewLeftover);
	}

	// Force a beam
	bool bNoLivingParticles = false;
	if (ActiveParticles == 0)
	{
		bNoLivingParticles = true;
		if (Number == 0)
			Number = 1;
	}

	// Don't allow more than BeamCount beams...
	if (Number + ActiveParticles > BeamCount)
	{
		Number	= BeamCount - ActiveParticles;
	}

	// Handle growing arrays.
	bool bProcessSpawn = true;
	int32 NewCount = ActiveParticles + Number;
	if (NewCount >= MaxActiveParticles)
	{
		if (DeltaTime < 0.25f)
		{
			bProcessSpawn = Resize(NewCount + FMath::TruncToInt(FMath::Sqrt((float)NewCount)) + 1);
		}
		else
		{
			bProcessSpawn = Resize((NewCount + FMath::TruncToInt(FMath::Sqrt((float)NewCount)) + 1), false);
		}
	}

	if (bProcessSpawn == true)
	{
		UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
		check(LODLevel);

		// Spawn particles.
		SpawnParticles( Number, StartTime, Increment, Location, FVector::ZeroVector, NULL );

		if (ForceSpawnCount > 0)
		{
			ForceSpawnCount = 0;
		}

		INC_DWORD_STAT_BY(STAT_BeamParticles, ActiveParticles);

		return NewLeftover;
	}

	INC_DWORD_STAT_BY(STAT_BeamParticles, ActiveParticles);

	return SafetyLeftover;
}

/**
 *	Handle any post-spawning actions required by the instance
 *
 *	@param	Particle					The particle that was spawned
 *	@param	InterpolationPercentage		The percentage of the time slice it was spawned at
 *	@param	SpawnTIme					The time it was spawned at
 */
void FParticleBeam2EmitterInstance::PostSpawn(FBaseParticle* Particle, float InterpolationPercentage, float SpawnTime)
{
	// The order of the Spawn here is VERY important as the modules may(will) depend on it occuring as such.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if (BeamModule_Source && BeamModule_Source->bEnabled)
	{
		BeamModule_Source->Spawn({*this, (int32)GetModuleDataOffset(BeamModule_Source), SpawnTime, Particle});
	}
	if (BeamModule_SourceModifier && BeamModule_SourceModifier->bEnabled)
	{
		int32 ModifierOffset = BeamModule_SourceModifier_Offset;
		BeamModule_SourceModifier->Spawn({*this, ModifierOffset, SpawnTime, Particle});
	}
	if (BeamModule_Target && BeamModule_Target->bEnabled)
	{
		BeamModule_Target->Spawn({*this, (int32)GetModuleDataOffset(BeamModule_Target), SpawnTime, Particle});
	}
	if (BeamModule_TargetModifier && BeamModule_TargetModifier->bEnabled)
	{
		int32 ModifierOffset = BeamModule_TargetModifier_Offset;
		BeamModule_TargetModifier->Spawn({*this, ModifierOffset, SpawnTime, Particle});
	}
	if (BeamModule_Noise && BeamModule_Noise->bEnabled)
	{
		BeamModule_Noise->Spawn({*this, (int32)GetModuleDataOffset(BeamModule_Noise), SpawnTime, Particle});
	}
	if (LODLevel->TypeDataModule)
	{
		//@todo. Need to track TypeData offset into payload!
		LODLevel->TypeDataModule->Spawn({*this, TypeDataOffset, SpawnTime, Particle});
	}

	FParticleEmitterInstance::PostSpawn( Particle, InterpolationPercentage, SpawnTime );
}

/**
 *	Kill off any dead particles. (Remove them from the active array)
 */
void FParticleBeam2EmitterInstance::KillParticles()
{
	if (ActiveParticles > 0)
	{
		UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
		check(LODLevel);
		FParticleEventInstancePayload* EventPayload = NULL;
		if (LODLevel->EventGenerator)
		{
			EventPayload = (FParticleEventInstancePayload*)GetModuleInstanceData(LODLevel->EventGenerator);
			if (EventPayload && (EventPayload->bDeathEventsPresent == false))
			{
				EventPayload = NULL;
			}
		}

		// Loop over the active particles... If their RelativeTime is > 1.0f (indicating they are dead),
		// move them to the 'end' of the active particle list.
		for (int32 i=ActiveParticles-1; i>=0; i--)
		{
			const int32	CurrentIndex	= ParticleIndices[i];
			const uint8* ParticleBase	= ParticleData + CurrentIndex * ParticleStride;
			FBaseParticle& Particle		= *((FBaseParticle*) ParticleBase);
			if (Particle.RelativeTime > 1.0f)
			{
				if (EventPayload)
				{
					LODLevel->EventGenerator->HandleParticleKilled(this, EventPayload, &Particle);
				}
				ParticleIndices[i]	= ParticleIndices[ActiveParticles-1];
				ParticleIndices[ActiveParticles-1]	= CurrentIndex;
				ActiveParticles--;

				INC_DWORD_STAT(STAT_BeamParticlesKilled);
			}
		}
	}
}

/**
 *	Setup the offsets to the BeamModifier modules...
 *	This must be done after the base Init call as that inserts modules into the offset map.
 */
void FParticleBeam2EmitterInstance::SetupBeamModifierModulesOffsets()
{
	check(BeamTypeData);

	if (BeamTypeData->LOD_BeamModule_SourceModifier.Num() > 0)
	{
		uint32* Offset = SpriteTemplate->ModuleOffsetMap.Find(BeamTypeData->LOD_BeamModule_SourceModifier[0]);
		if (Offset != NULL)
		{
			BeamModule_SourceModifier_Offset = (int32)(*Offset);
		}
	}

	if (BeamTypeData->LOD_BeamModule_TargetModifier.Num() > 0)
	{
		uint32* Offset = SpriteTemplate->ModuleOffsetMap.Find(BeamTypeData->LOD_BeamModule_TargetModifier[0]);
		if (Offset != NULL)
		{
			BeamModule_TargetModifier_Offset = (int32)(*Offset);
		}
	}
}

/**
 *	Resolve the source for the beam
 */
void FParticleBeam2EmitterInstance::ResolveSource()
{
	check(IsInGameThread());
	if (BeamModule_Source)
	{
		if (BeamModule_Source->SourceName != NAME_None)
		{
			switch (BeamModule_Source->SourceMethod)
			{
			case PEB2STM_Actor:
				{
					TArrayView<const FParticleSysParam> InstanceParameters = Component.GetInstanceParameters();
					FParticleSysParam Param;
					for (int32 i = 0; i < InstanceParameters.Num(); i++)
					{
						Param = InstanceParameters[i];
						if (Param.Name == BeamModule_Source->SourceName)
						{
							SourceActor = Param.Actor;
							break;
						}
					}
				}
				break;
			case PEB2STM_Emitter:
			case PEB2STM_Particle:
				if (SourceEmitter == NULL)
				{
					TArrayView<FParticleEmitterInstance*> EmitterInstances = Component.GetEmitterInstances();
					for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
					{
						FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
						if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == BeamModule_Source->SourceName))
						{
							SourceEmitter = pkEmitInst;
							break;
						}
					}
				}
				break;
			}
		}
	}
}

/**
 *	Resolve the target for the beam
 */
void FParticleBeam2EmitterInstance::ResolveTarget()
{
	check(IsInGameThread());
	if (BeamModule_Target)
	{
		if (BeamModule_Target->TargetName != NAME_None)
		{
			switch (BeamModule_Target->TargetMethod)
			{
			case PEB2STM_Actor:
				{
					TArrayView<const FParticleSysParam> InstanceParameters = Component.GetInstanceParameters();
					FParticleSysParam Param;
					for (int32 i = 0; i < InstanceParameters.Num(); i++)
					{
						Param = InstanceParameters[i];
						if (Param.Name == BeamModule_Target->TargetName)
						{
							TargetActor = Param.Actor;
							break;
						}
					}
				}
				break;
			case PEB2STM_Emitter:
			case PEB2STM_Particle:
				if (TargetEmitter == NULL)
				{
					TArrayView<FParticleEmitterInstance*> EmitterInstances = Component.GetEmitterInstances();
					for (int32 ii = 0; ii < EmitterInstances.Num(); ii++)
					{
						FParticleEmitterInstance* pkEmitInst = EmitterInstances[ii];
						if (pkEmitInst && (pkEmitInst->SpriteTemplate->EmitterName == BeamModule_Target->TargetName))
						{
							TargetEmitter = pkEmitInst;
							break;
						}
					}
				}
				break;
			}
		}
	}
}

/**
 *	Determine the vertex and triangle counts for the emitter
 */
void FParticleBeam2EmitterInstance::DetermineVertexAndTriangleCount()
{
	// Need to determine # tris per beam...
	int32 VerticesToRender = 0;
	int32 EmitterTrianglesToRender = 0;

	check(BeamTypeData);
	int32 Sheets = BeamTypeData->Sheets ? BeamTypeData->Sheets : 1;

	BeamTrianglesPerSheet.Empty(ActiveParticles);
	BeamTrianglesPerSheet.AddZeroed(ActiveParticles);
	for (int32 i = 0; i < ActiveParticles; i++)
	{
		DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[i]);

		int32						CurrentOffset		= TypeDataOffset;
		FBeam2TypeDataPayload*	BeamData			= NULL;
		FVector*				InterpolatedPoints	= NULL;
		float*					NoiseRate			= NULL;
		float*					NoiseDelta			= NULL;
		FVector*				TargetNoisePoints	= NULL;
		FVector*				NextNoisePoints		= NULL;
		float*					TaperValues			= NULL;
		float*					NoiseDistanceScale	= NULL;
		FBeamParticleModifierPayloadData* SourceModifier = NULL;
		FBeamParticleModifierPayloadData* TargetModifier = NULL;

		BeamTypeData->GetDataPointers(this, (const uint8*)Particle, CurrentOffset, 
			BeamData, InterpolatedPoints, NoiseRate, NoiseDelta, TargetNoisePoints, 
			NextNoisePoints, TaperValues, NoiseDistanceScale,
			SourceModifier, TargetModifier);

		BeamTrianglesPerSheet[i] = BeamData->TriangleCount;

		// Take sheets into account
		int32 LocalTriangles = 0;
		if (BeamData->TriangleCount > 0)
		{
			if (VerticesToRender > 0)
			{
				LocalTriangles += 4;//Degenerate tris linking from previous beam.
			}

			// Stored triangle count is per sheet...
			LocalTriangles	+= BeamData->TriangleCount * Sheets;
			VerticesToRender += (BeamData->TriangleCount + 2) * Sheets;
			// 4 Degenerates Per Sheet (except for last one)
			LocalTriangles	+= (Sheets - 1) * 4;
			EmitterTrianglesToRender += LocalTriangles;
		}
	}

	VertexCount = VerticesToRender;
	TriangleCount = EmitterTrianglesToRender;
}



/**
 *	Retrieves the dynamic data for the emitter
 *	
 *	@param	bSelected					Whether the emitter is selected in the editor
 *  @param InFeatureLevel				The relevant shader feature level.
 *
 *	@return	FDynamicEmitterDataBase*	The dynamic data, or NULL if it shouldn't be rendered
 */
FDynamicEmitterDataBase* FParticleBeam2EmitterInstance::GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleBeam2EmitterInstance_GetDynamicData);

	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if (IsDynamicDataRequired(LODLevel) == false || !bEnabled)
	{
		return NULL;
	}

	//@todo.SAS. Have this call the UpdateDynamicData function to reduce duplicate code!!!
	//@SAS. This removes the need for the assertion in the actual render call...
	if ((ActiveParticles > FDynamicBeam2EmitterData::MaxBeams) ||	// TTP #33330 - Max of 2048 beams from a single emitter
		(ParticleStride >
			((FDynamicBeam2EmitterData::MaxInterpolationPoints + 2) * (sizeof(FVector) + sizeof(float))) + 
			(FDynamicBeam2EmitterData::MaxNoiseFrequency * (sizeof(FVector) + sizeof(FVector) + sizeof(float) + sizeof(float)))
		)	// TTP #33330 - Max of 10k per beam (includes interpolation points, noise, etc.)
		)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (Component.HasWorld())
		{
			FString ErrorMessage = 
				FString::Printf(TEXT("BeamEmitter with too much data: %s"),
					Component.GetTemplate() ? *Component.GetTemplate()->GetName() : TEXT("No template"));
			FColor ErrorColor(255,0,0);
			GEngine->AddOnScreenDebugMessage((uint64)((PTRINT)this), 5.0f, ErrorColor,ErrorMessage);
			UE_LOGF(LogParticles, Log, "%ls", *ErrorMessage);
		}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		return NULL;
	}

	// Allocate the dynamic data
	FDynamicBeam2EmitterData* NewEmitterData = new FDynamicBeam2EmitterData(LODLevel->RequiredModule);
	{
		INC_DWORD_STAT(STAT_DynamicEmitterCount);
		INC_DWORD_STAT(STAT_DynamicBeamCount);
		INC_DWORD_STAT_BY(STAT_DynamicEmitterMem, sizeof(FDynamicBeam2EmitterData));
	}

	// Now fill in the source data
	if( !FillReplayData( NewEmitterData->Source ) )
	{
		delete NewEmitterData;
		return NULL;
	}

	// Setup dynamic render data.  Only call this AFTER filling in source data for the emitter.
	NewEmitterData->Init( bSelected );

	return NewEmitterData;
}

/**
 *	Retrieves replay data for the emitter
 *
 *	@return	The replay data, or NULL on failure
 */
FDynamicEmitterReplayDataBase* FParticleBeam2EmitterInstance::GetReplayData()
{
	FDynamicEmitterReplayDataBase* NewEmitterReplayData = new FDynamicBeam2EmitterReplayData();
	check( NewEmitterReplayData != NULL );

	if( !FillReplayData( *NewEmitterReplayData ) )
	{
		delete NewEmitterReplayData;
		return NULL;
	}

	return NewEmitterReplayData;
}

/**
 *	Retrieve the allocated size of this instance.
 *
 *	@param	OutNum			The size of this instance
 *	@param	OutMax			The maximum size of this instance
 */
void FParticleBeam2EmitterInstance::GetAllocatedSize(int32& OutNum, int32& OutMax)
{
	int32 Size = sizeof(FParticleBeam2EmitterInstance);
	int32 ActiveParticleDataSize = (ParticleData != NULL) ? (ActiveParticles * ParticleStride) : 0;
	int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
	int32 ActiveParticleIndexSize = 0;
	int32 MaxActiveParticleIndexSize = 0;

	OutNum = ActiveParticleDataSize + ActiveParticleIndexSize + Size;
	OutMax = MaxActiveParticleDataSize + MaxActiveParticleIndexSize + Size;
}

/**
 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
 *
 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
 * @return	Size of resource as to be displayed to artists/ LDs in the Editor.
 */
void FParticleBeam2EmitterInstance::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal || (Component.GetSceneProxy()))
	{
		int32 MaxActiveParticleDataSize = (ParticleData != NULL) ? (MaxActiveParticles * ParticleStride) : 0;
		int32 MaxActiveParticleIndexSize = 0;
		// Take dynamic data into account as well
		CumulativeResourceSize.AddUnknownMemoryBytes(sizeof(FDynamicBeam2EmitterData));
		CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticleDataSize);		// Copy of the particle data on the render thread
		if (DynamicParameterDataOffset == 0)
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleBeamTrailVertex));	// The vertex data array
		}
		else
		{
			CumulativeResourceSize.AddUnknownMemoryBytes(MaxActiveParticles * sizeof(FParticleBeamTrailVertexDynamicParameter));
		}
	}
}

/**
 * Captures dynamic replay data for this particle system.
 *
 * @param	OutData		[Out] Data will be copied here
 *
 * @return Returns true if successful
 */
bool FParticleBeam2EmitterInstance::FillReplayData( FDynamicEmitterReplayDataBase& OutData )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleBeam2EmitterInstance_FillReplayData);

	if (ActiveParticles <= 0)
	{
		return false;
	}
	// Call parent implementation first to fill in common particle source data
	if( !FParticleEmitterInstance::FillReplayData( OutData ) )
	{
		return false;
	}

	// If the template is disabled, don't return data.
	UParticleLODLevel* LODLevel = SpriteTemplate->GetCurrentLODLevel(this);
	if ((LODLevel == NULL) || (LODLevel->bEnabled == false))
	{
		return false;
	}

	OutData.eEmitterType = DET_Beam2;

	FDynamicBeam2EmitterReplayData* NewReplayData =
		static_cast< FDynamicBeam2EmitterReplayData* >( &OutData );

	NewReplayData->MaterialInterface = GetCurrentMaterial();

	// We never want local space for beams
	NewReplayData->bUseLocalSpace = false;

	// Never use axis lock for beams
	NewReplayData->bLockAxis = false;

	DetermineVertexAndTriangleCount();

	NewReplayData->UpVectorStepSize = BeamTypeData->UpVectorStepSize;
	NewReplayData->TrianglesPerSheet.Empty(BeamTrianglesPerSheet.Num());
	NewReplayData->TrianglesPerSheet.AddZeroed(BeamTrianglesPerSheet.Num());
	for (int32 BeamIndex = 0; BeamIndex < BeamTrianglesPerSheet.Num(); BeamIndex++)
	{
		NewReplayData->TrianglesPerSheet[BeamIndex] = BeamTrianglesPerSheet[BeamIndex];
	}


	int32 IgnoredTaperCount = 0;
	BeamTypeData->GetDataPointerOffsets(this, NULL, TypeDataOffset, 
		NewReplayData->BeamDataOffset, NewReplayData->InterpolatedPointsOffset,
		NewReplayData->NoiseRateOffset, NewReplayData->NoiseDeltaTimeOffset,
		NewReplayData->TargetNoisePointsOffset, NewReplayData->NextNoisePointsOffset, 
		IgnoredTaperCount, NewReplayData->TaperValuesOffset,
		NewReplayData->NoiseDistanceScaleOffset);

	NewReplayData->VertexCount = VertexCount;


	if (BeamModule_Source)
	{
		NewReplayData->bUseSource = true;
	}
	else
	{
		NewReplayData->bUseSource = false;
	}

	if (BeamModule_Target)
	{
		NewReplayData->bUseTarget = true;
	}
	else
	{
		NewReplayData->bUseTarget = false;
	}

	if (BeamModule_Noise)
	{
		NewReplayData->bLowFreqNoise_Enabled = BeamModule_Noise->bLowFreq_Enabled;
		NewReplayData->bHighFreqNoise_Enabled = false;
		NewReplayData->bSmoothNoise_Enabled = BeamModule_Noise->bSmooth;

	}
	else
	{
		NewReplayData->bLowFreqNoise_Enabled = false;
		NewReplayData->bHighFreqNoise_Enabled = false;
		NewReplayData->bSmoothNoise_Enabled = false;
	}
	NewReplayData->Sheets = (BeamTypeData->Sheets > 0) ? BeamTypeData->Sheets : 1;
	NewReplayData->Sheets = FMath::Max(NewReplayData->Sheets, 1);

	NewReplayData->TextureTile = BeamTypeData->TextureTile;
	NewReplayData->TextureTileDistance = BeamTypeData->TextureTileDistance;
	NewReplayData->TaperMethod = BeamTypeData->TaperMethod;
	NewReplayData->InterpolationPoints = BeamTypeData->InterpolationPoints;

	NewReplayData->NoiseTessellation	= 0;
	NewReplayData->Frequency			= 1;
	NewReplayData->NoiseRangeScale		= 1.0f;
	NewReplayData->NoiseTangentStrength= 1.0f;

	int32 TessFactor = 1;
	if ((BeamModule_Noise == NULL) || (BeamModule_Noise->bLowFreq_Enabled == false))
	{
		TessFactor	= BeamTypeData->InterpolationPoints ? BeamTypeData->InterpolationPoints : 1;
	}
	else
	{
		NewReplayData->Frequency			= (BeamModule_Noise->Frequency > 0) ? BeamModule_Noise->Frequency : 1;
		NewReplayData->Frequency = FMath::Max(NewReplayData->Frequency, 1);
		NewReplayData->NoiseTessellation	= (BeamModule_Noise->NoiseTessellation > 0) ? BeamModule_Noise->NoiseTessellation : 1;
		NewReplayData->NoiseTangentStrength= BeamModule_Noise->NoiseTangentStrength.GetValue(EmitterTime);
		if (BeamModule_Noise->bNRScaleEmitterTime)
		{
			NewReplayData->NoiseRangeScale = BeamModule_Noise->NoiseRangeScale.GetValue(EmitterTime, Component.GetDistributionData());
		}
		else
		{	//-V523 Remove when todo will be implemented
			//@todo.SAS. Need to address this!!!!
			//					check(0 && TEXT("NoiseRangeScale - No way to get per-particle setting at this time."));
			//					NewReplayData->NoiseRangeScale	= BeamModule_Noise->NoiseRangeScale.GetValue(Particle->RelativeTime, Component);
			NewReplayData->NoiseRangeScale = BeamModule_Noise->NoiseRangeScale.GetValue(EmitterTime, Component.GetDistributionData());
		}
		NewReplayData->NoiseSpeed = (FVector3f)BeamModule_Noise->NoiseSpeed.GetValue(EmitterTime);
		NewReplayData->NoiseLockTime = BeamModule_Noise->NoiseLockTime;
		NewReplayData->NoiseLockRadius = BeamModule_Noise->NoiseLockRadius;
		NewReplayData->bTargetNoise = BeamModule_Noise->bTargetNoise;
		NewReplayData->NoiseTension = BeamModule_Noise->NoiseTension;
	}

	int32 MaxSegments	= ((TessFactor * NewReplayData->Frequency) + 1 + 1);		// Tessellation * Frequency + FinalSegment + FirstEdge;

	// Determine the index count
	NewReplayData->IndexCount	= 0;
	for (int32 Beam = 0; Beam < ActiveParticles; Beam++)
	{
		DECLARE_PARTICLE_PTR(Particle, ParticleData + ParticleStride * ParticleIndices[Beam]);

		int32						CurrentOffset		= TypeDataOffset;
		FBeam2TypeDataPayload*	BeamData			= NULL;
		FVector*				InterpolatedPoints	= NULL;
		float*					NoiseRate			= NULL;
		float*					NoiseDelta			= NULL;
		FVector*				TargetNoisePoints	= NULL;
		FVector*				NextNoisePoints		= NULL;
		float*					TaperValues			= NULL;
		float*					NoiseDistanceScale	= NULL;
		FBeamParticleModifierPayloadData* SourceModifier = NULL;
		FBeamParticleModifierPayloadData* TargetModifier = NULL;

		BeamTypeData->GetDataPointers(this, (const uint8*)Particle, CurrentOffset, BeamData, 
			InterpolatedPoints, NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, 
			TaperValues, NoiseDistanceScale, SourceModifier, TargetModifier);

		if (BeamData->TriangleCount > 0)
		{
			if (NewReplayData->IndexCount == 0)
			{
				NewReplayData->IndexCount = 2;
			}
			NewReplayData->IndexCount	+= BeamData->TriangleCount * NewReplayData->Sheets;	// 1 index per triangle in the strip PER SHEET
			NewReplayData->IndexCount	+= ((NewReplayData->Sheets - 1) * 4);					// 4 extra indices per stitch (degenerates)
			if (Beam > 0)
			{
				NewReplayData->IndexCount	+= 4;	// 4 extra indices per beam (degenerates)
			}
		}
	}

	if (NewReplayData->IndexCount > 15000)
	{
		NewReplayData->IndexStride	= sizeof(uint32);
	}
	else
	{
		NewReplayData->IndexStride	= sizeof(uint16);
	}


	//@todo. SORTING IS A DIFFERENT ISSUE NOW! 
	//		 GParticleView isn't going to be valid anymore?
	uint8* PData = NewReplayData->DataContainer.ParticleData;
	for (int32 i = 0; i < NewReplayData->ActiveParticleCount; i++)
	{
		DECLARE_PARTICLE(Particle, ParticleData + ParticleStride * ParticleIndices[i]);
		FMemory::Memcpy(PData, &Particle, ParticleStride);
		PData += ParticleStride;
	}

	// Set the debug rendering flags...
	NewReplayData->bRenderGeometry = BeamTypeData->RenderGeometry;
	NewReplayData->bRenderDirectLine = BeamTypeData->RenderDirectLine;
	NewReplayData->bRenderLines = BeamTypeData->RenderLines;
	NewReplayData->bRenderTessellation = BeamTypeData->RenderTessellation;


	return true;
}

UMaterialInterface* FParticleBeam2EmitterInstance::GetCurrentMaterial()
{
	UMaterialInterface* RenderMaterial = CurrentMaterial;
	if ((RenderMaterial == NULL) || (RenderMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_BeamTrails) == false))
	{
		RenderMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}
	CurrentMaterial = RenderMaterial;
	return RenderMaterial;
}
