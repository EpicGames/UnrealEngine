// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleBeamModules.cpp: Particle module implementations for beams.
=============================================================================*/

#include "Particles/ParticleSystem.h"
#include "Particles/ParticleModule.h"
#include "ParticleEmitterInstances.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/ParticleSystemComponent.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Particles/Beam/ParticleModuleBeamBase.h"
#include "Particles/Beam/ParticleModuleBeamModifier.h"
#include "Particles/Beam/ParticleModuleBeamNoise.h"
#include "Particles/Beam/ParticleModuleBeamSource.h"
#include "Particles/Beam/ParticleModuleBeamTarget.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataBeam2.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Engine/InterpCurveEdSetup.h"
#include "Engine/World.h"
#include "ParticleBeamTrailVertexFactory.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	Abstract base modules used for categorization.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	UParticleModuleTypeDataBeam2 implementation.
-----------------------------------------------------------------------------*/
UParticleModuleTypeDataBeam2::UParticleModuleTypeDataBeam2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	BeamMethod = PEB2M_Target;
	TextureTile = 1;
	TextureTileDistance = 0.0f;
	Sheets = 1;
	Speed = 10.0f;
	InterpolationPoints = 0;
	bAlwaysOn = false;
	BranchParentName = ConstructorStatics.NAME_None;
	TaperMethod = PEBTM_None;
	RenderGeometry = true;
	RenderDirectLine = false;
	RenderLines = false;
	RenderTessellation = false;
}

void UParticleModuleTypeDataBeam2::InitializeDefaults()
{
	if (!Distance.IsCreated())
	{
		UDistributionFloatConstant* DistributionDistance = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionDistance"));
		DistributionDistance->Constant = 25.0f;
		Distance.Distribution = DistributionDistance;
	}

	if (!TaperFactor.IsCreated())
	{
		UDistributionFloatConstant* DistributionTaperFactor = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionTaperFactor"));
		DistributionTaperFactor->Constant = 1.0f;
		TaperFactor.Distribution = DistributionTaperFactor;
	}

	if (!TaperScale.IsCreated())
	{
		UDistributionFloatConstant* DistributionTaperScale = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionTaperScale"));
		DistributionTaperScale->Constant = 1.0f;
		TaperScale.Distribution = DistributionTaperScale;
	}
}

void UParticleModuleTypeDataBeam2::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleTypeDataBeam2::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	check(Owner->bIsBeam);

	// Setup the particle data points with the SPAWN_INIT macro
	SPAWN_INIT;

	FBeam2TypeDataPayload*	BeamData			= NULL;
	FVector*				InterpolatedPoints	= NULL;
	float*					NoiseRate			= NULL;
	float*					NoiseDelta			= NULL;
	FVector*				TargetNoisePoints	= NULL;
	FVector*				NextNoisePoints		= NULL;
	float*					TaperValues			= NULL;
	float*					NoiseDistanceScale	= NULL;
	FBeamParticleModifierPayloadData* SourceModifier = NULL;
	FBeamParticleModifierPayloadData* TargetModifier = NULL;

	// Get the pointers to the data, but save the module offset that is passed in.
	int32 TempOffset = (int32)CurrentOffset;
	GetDataPointers(Owner, (uint8*)ParticleBase, TempOffset, BeamData, InterpolatedPoints, 
		NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, TaperValues,
		NoiseDistanceScale, SourceModifier, TargetModifier);

	CurrentOffset	= TempOffset;

	// If there is no Source module, use the emitter position as the source point
	if (BeamInst->BeamModule_Source == NULL)
	{
		BeamData->SourcePoint	= Owner->Component.GetComponentTransform().GetLocation();
		BeamData->SourceTangent = Context.GetTransform().GetScaledAxis( EAxis::X );
		BeamData->SourceStrength = 1.0f;
	}

	// If the beam is set for distance, or there is no target module, determine the target point
	if ((BeamInst->BeamModule_Target == NULL) && (BeamInst->BeamMethod == PEB2M_Distance))
	{
		// Set the particle target based on the distance
		double	TotalDistance	= Distance.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		// Always use the X-axis of the component as the direction
		FVector	Direction		= Context.GetTransform().GetScaledAxis( EAxis::X );
		Direction.Normalize();
		// Calculate the final target point
		BeamData->TargetPoint	= BeamData->SourcePoint + Direction * TotalDistance;
		BeamData->TargetTangent = -Direction;
		BeamData->TargetStrength = 1.0f;
	}

	// Modify the source and target positions, if modifiers are present
	if (SourceModifier != NULL)
	{
		// Position
		SourceModifier->UpdatePosition(BeamData->SourcePoint);
		// Tangent...
		SourceModifier->UpdateTangent(BeamData->SourceTangent, 
			BeamInst->BeamModule_SourceModifier ? BeamInst->BeamModule_SourceModifier->bAbsoluteTangent : false);
		// Strength
		SourceModifier->UpdateStrength(BeamData->SourceStrength);
	}

	if (TargetModifier != NULL)
	{
		// Position
		TargetModifier->UpdatePosition(BeamData->TargetPoint);
		// Tangent...
		TargetModifier->UpdateTangent(BeamData->TargetTangent,
			BeamInst->BeamModule_TargetModifier ? BeamInst->BeamModule_TargetModifier->bAbsoluteTangent : false);
		// Strength
		TargetModifier->UpdateStrength(BeamData->TargetStrength);
	}

	// If we are tapering, determine the taper points
	if (TaperMethod != PEBTM_None)
	{
		int32	TaperCount	= 2;
		int32	TotalSteps	= 2;

		if (BeamInst->BeamModule_Noise && BeamInst->BeamModule_Noise->bLowFreq_Enabled)
		{
			//@todo. determine the required number of taper points...
			// (Depends on interaction of interpolation points and noise freq.)
			int32 Freq = BEAM2_TYPEDATA_FREQUENCY(BeamData->Lock_Max_NumNoisePoints);
			TaperCount = (Freq + 1) * 
				(BeamInst->BeamModule_Noise->NoiseTessellation ? BeamInst->BeamModule_Noise->NoiseTessellation : 1);
		}
		else
		{
			// The taper count is simply the number of interpolation points + 1.
			TaperCount	 = InterpolationPoints ? (InterpolationPoints + 1) : 2;
		}

		// Taper the beam for the FULL length, regardless of position
		// If the mode is set to partial, it will be handled in the GetData function
		float Increment	= 1.0f / (float)(TaperCount - 1);
		float CurrStep;
		for (int32 TaperIndex = 0; TaperIndex < TaperCount; TaperIndex++)
		{
			CurrStep	= TaperIndex * Increment;
			TaperValues[TaperIndex] = TaperFactor.GetValue(CurrStep, Context.GetDistributionData()) * TaperScale.GetValue(CurrStep, Context.GetDistributionData());
		}
	}
}


void UParticleModuleTypeDataBeam2::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	check(Owner->bIsBeam);
	UParticleModuleBeamNoise*		BeamNoise	= BeamInst->BeamModule_Noise;
	UParticleModuleBeamSource*		BeamSource	= BeamInst->BeamModule_Source;
	UParticleModuleBeamTarget*		BeamTarget	= BeamInst->BeamModule_Target;
	UParticleModuleBeamModifier*	SourceMod	= BeamInst->BeamModule_SourceModifier;
	UParticleModuleBeamModifier*	TargetMod	= BeamInst->BeamModule_TargetModifier;

	// If we are targeting, set the lock radius
	float	LockRadius	= 1.0f;
	if (BeamTarget)
	{
		LockRadius = BeamTarget->LockRadius;
	}

	bool bSourceTangentAbsolute = BeamInst->BeamModule_SourceModifier ? BeamInst->BeamModule_SourceModifier->bAbsoluteTangent : false;
	bool bTargetTangentAbsolute = BeamInst->BeamModule_TargetModifier ? BeamInst->BeamModule_TargetModifier->bAbsoluteTangent : false;

	// For each particle, run the Update loop
	BEGIN_UPDATE_LOOP;
	{
		FBeam2TypeDataPayload*	BeamData			= NULL;
		FVector*				InterpolatedPoints	= NULL;
		float*					NoiseRate			= NULL;
		float*					NoiseDelta			= NULL;
		FVector*				TargetNoisePoints	= NULL;
		FVector*				NextNoisePoints		= NULL;
		float*					TaperValues			= NULL;
		float*					NoiseDistanceScale	= NULL;
		FBeamParticleModifierPayloadData* SourceModifier = NULL;
		FBeamParticleModifierPayloadData* TargetModifier = NULL;

		// Setup the pointers for the payload data
		int32 TempOffset = (int32)CurrentOffset;
		GetDataPointers(Owner, ParticleBase, TempOffset, BeamData, InterpolatedPoints, 
			NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, TaperValues,
			NoiseDistanceScale, SourceModifier, TargetModifier);

		// If there is no Source module, use the emitter location
		if (BeamInst->BeamModule_Source == NULL)
		{
			BeamData->SourcePoint	= Owner->Component.GetComponentTransform().GetLocation();
			BeamData->SourceTangent = Context.GetTransform().GetScaledAxis( EAxis::X );
		}

		// If the method is set for distance, or there is no target, determine the target point
		if ((BeamInst->BeamModule_Target == NULL) && (BeamInst->BeamMethod == PEB2M_Distance))
		{
			// Set the particle target based on the distance
			double	TotalDistance	= Distance.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			FVector	Direction		= Context.GetTransform().GetScaledAxis( EAxis::X );
			Direction.Normalize();
			BeamData->TargetPoint	= BeamData->SourcePoint + Direction * TotalDistance;
			BeamData->TargetTangent = -Direction;
		}

		// Modify the source and target positions, if modifiers are present
		if (SourceModifier != NULL)
		{
			// Position
			SourceModifier->UpdatePosition(BeamData->SourcePoint);
			// Tangent...
			SourceModifier->UpdateTangent(BeamData->SourceTangent, bSourceTangentAbsolute);
			// Strength
			SourceModifier->UpdateStrength(BeamData->SourceStrength);
		}

		if (TargetModifier != NULL)
		{
			// Position
			TargetModifier->UpdatePosition(BeamData->TargetPoint);
			// Tangent...
			TargetModifier->UpdateTangent(BeamData->TargetTangent, bTargetTangentAbsolute);
			// Strength
			TargetModifier->UpdateStrength(BeamData->TargetStrength);
		}

		// The number of interpolated points (for 'bendy' beams)
		int32		InterpolationCount	= InterpolationPoints ? InterpolationPoints : 1;
		bool	bLowFreqNoise		= (BeamNoise && BeamNoise->bLowFreq_Enabled) ? true : false;


		// Determine the current location of the particle
		//
		// If 'growing' the beam, determine the position along the source->target line
		// Otherwise, pop it right to the target
		if ((Speed != 0.0f) && (!BEAM2_TYPEDATA_LOCKED(BeamData->Lock_Max_NumNoisePoints)))
		{
			// If the beam isn't locked, then move it towards the target...
			if (Particle.Location != BeamData->TargetPoint)
			{
				// Determine the direction of travel
				FVector Direction	= BeamData->TargetPoint - Particle.Location;
				Direction.Normalize();
				// Setup the offset and the current distance travelled
				const FVector BeamOffset		= Direction * Speed * Context.DeltaTime;
				FVector	Sum			= Particle.Location + BeamOffset;
				if ((FMath::Abs(Sum.X - BeamData->TargetPoint.X) < LockRadius) && 
					(FMath::Abs(Sum.Y - BeamData->TargetPoint.Y) < LockRadius) &&
					(FMath::Abs(Sum.Z - BeamData->TargetPoint.Z) < LockRadius))
				{
					// We are within the lock radius, so lock the beam
					Particle.Location	= BeamData->TargetPoint;
					BEAM2_TYPEDATA_SETLOCKED(BeamData->Lock_Max_NumNoisePoints, 1);
				}
				else
				{
					// Otherwise, just set the location
					Particle.Location	= Sum;
				}
			}
		}
		else
		{
			// Pop right to the target and set the beam as locked
			Particle.Location = BeamData->TargetPoint;
			BEAM2_TYPEDATA_SETLOCKED(BeamData->Lock_Max_NumNoisePoints, 1);
		}

		// Determine the step size, count, and travelled ratio
		BeamData->Direction		= BeamData->TargetPoint - BeamData->SourcePoint;
		double	FullMagnitude	= FMath::Max(BeamData->Direction.Size(), 0.001);
		BeamData->Direction.Normalize();

		int32 InterpSteps = 0;

		if (bLowFreqNoise == false)
		{
			// No noise branch...
			if (BEAM2_TYPEDATA_LOCKED(BeamData->Lock_Max_NumNoisePoints))
			{
				// If the beam is locked to the target, the steps are the interpolation count
				BeamData->StepSize		= FullMagnitude / double(InterpolationCount);
				BeamData->Steps			= InterpolationCount;
				BeamData->TravelRatio	= 0.0f;
			}
			else
			{
				// Determine the number of steps we have traveled
				FVector	TrueDistance	= Particle.Location - BeamData->SourcePoint;
				double	TrueMagnitude	= TrueDistance.Size();
				if (TrueMagnitude > FullMagnitude)
				{
					// Lock to the target if we are over-shooting and determine the steps and step size
					Particle.Location	= BeamData->TargetPoint;
					TrueDistance		= Particle.Location - BeamData->SourcePoint;
					TrueMagnitude		= TrueDistance.Size();
					BEAM2_TYPEDATA_SETLOCKED(BeamData->Lock_Max_NumNoisePoints, true);
					BeamData->StepSize		= FullMagnitude / double(InterpolationCount);
					BeamData->Steps			= InterpolationCount;
					BeamData->TravelRatio	= 0.0f;
				}
				else
				{
					// Determine the steps and step size
					BeamData->StepSize		= FullMagnitude / double(InterpolationCount);
					BeamData->TravelRatio	= TrueMagnitude / FullMagnitude;
					BeamData->Steps			= FMath::FloorToInt(BeamData->TravelRatio * InterpolationCount);
					// Readjust the travel ratio
					BeamData->TravelRatio	= (TrueMagnitude - (BeamData->StepSize * double(BeamData->Steps))) / BeamData->StepSize;
				}
			}
			InterpSteps = BeamData->Steps;
		}
		else
		{
			// Noise branch...
			InterpSteps = InterpolationCount;

			// Grab the frequency for this beam (particle)
			int32 Freq = BEAM2_TYPEDATA_FREQUENCY(BeamData->Lock_Max_NumNoisePoints);

			if (BEAM2_TYPEDATA_LOCKED(BeamData->Lock_Max_NumNoisePoints))
			{
				// Locked to the target
				if (BeamNoise->FrequencyDistance > 0.0f)
				{
					// The noise points are based on distance...
					// Determine the number of points to drop.
					int32 Count = FMath::TruncToInt(FullMagnitude / BeamNoise->FrequencyDistance);
					Count = FMath::Min<int32>(Count, Freq);
					BeamData->StepSize		= FullMagnitude / double(Count + 1);
					BeamData->Steps			= Count;
					BeamData->TravelRatio	= 0.0f;
					if (NoiseDistanceScale != NULL)
					{
						float Delta = (float)Count / (float)(Freq);
						*NoiseDistanceScale = BeamNoise->NoiseScale.GetValue(Delta);
					}
				}
				else
				{
					// If locked, just use the noise frequency to determine steps
					BeamData->StepSize		= FullMagnitude / double(Freq + 1);
					BeamData->Steps			= Freq;
					BeamData->TravelRatio	= 0.0f;
					if (NoiseDistanceScale != NULL)
					{
						*NoiseDistanceScale = 1.0f;
					}
				}
			}
			else
			{
				// Determine that actual distance traveled, and its magnitude
				FVector	TrueDistance	= Particle.Location - BeamData->SourcePoint;
				double	TrueMagnitude	= TrueDistance.Size();

				if (BeamNoise->FrequencyDistance > 0.0f)
				{
					int32 Count = FMath::TruncToInt(FullMagnitude / BeamNoise->FrequencyDistance);
					Count = FMath::Min<int32>(Count, Freq);
					BeamData->StepSize		= FullMagnitude / double(Count + 1);
					// Determine the partial trail amount and the steps taken
					BeamData->TravelRatio	= TrueMagnitude / FullMagnitude;
					BeamData->Steps			= FMath::FloorToInt(BeamData->TravelRatio * (Count + 1));
					// Lock the steps to the frequency
					if (BeamData->Steps > Count)
					{
						BeamData->Steps = Count;
					}
					// Readjust the travel ratio
					if (BeamData->Steps == Count)
					{
						BeamData->TravelRatio	= 
							(TrueMagnitude - (BeamData->StepSize * double(BeamData->Steps))) /
							(FullMagnitude - (BeamData->StepSize * double(BeamData->Steps)));
					}
					else
					{
						BeamData->TravelRatio	= (TrueMagnitude - (BeamData->StepSize * double(BeamData->Steps))) / BeamData->StepSize;
					}

					if (NoiseDistanceScale != NULL)
					{
						float Delta = (float)Count / (float)(Freq);
						*NoiseDistanceScale = BeamNoise->NoiseScale.GetValue(Delta);
					}
				}
				else
				{
					// If we are not doing noisy interpolation
					// Determine the step size for the full beam
					BeamData->StepSize		= FullMagnitude / double(Freq + 1);
					// Determine the partial trail amount and the steps taken
					BeamData->TravelRatio	= TrueMagnitude / FullMagnitude;
					BeamData->Steps			= FMath::FloorToInt(BeamData->TravelRatio * (Freq + 1));
					// Lock the steps to the frequency
					if (BeamData->Steps > Freq)
					{
						BeamData->Steps = Freq;
					}
					// Readjust the travel ratio
					if (BeamData->Steps == Freq)
					{
						BeamData->TravelRatio	= 
							(TrueMagnitude - (BeamData->StepSize * double(BeamData->Steps))) /
							(FullMagnitude - (BeamData->StepSize * double(BeamData->Steps)));
					}
					else
					{
						BeamData->TravelRatio	= (TrueMagnitude - (BeamData->StepSize * double(BeamData->Steps))) / BeamData->StepSize;
					}
				}

				if (NoiseDistanceScale != NULL)
				{
					*NoiseDistanceScale = 1.0f;
				}
			}
		}

		// Form the interpolated points
		//@todo. Handle interpolate & noise case!
		if (InterpolationPoints > 0)
		{
			BeamData->InterpolationSteps = InterpSteps;

			// Use the tangents
			FVector	SourcePosition;
			FVector	SourceTangent;
			FVector	TargetPosition;
			FVector	TargetTangent;

			double	InvTess	= 1.0 / double(InterpolationPoints);

			SourcePosition	 = BeamData->SourcePoint;
			SourceTangent	 = BeamData->SourceTangent;
#if defined(_BEAM2_TYPEDATA_NORMAL_TANGENTS_)
			SourceTangent.Normalize();
#endif	//#if defined(_BEAM2_TYPEDATA_NORMAL_TANGENTS_)
			if (SourceTangent.IsNearlyZero())
			{
				SourceTangent	= Context.GetTransform().GetScaledAxis( EAxis::X );
			}
			SourceTangent	*= BeamData->SourceStrength;

			TargetPosition	 = BeamData->TargetPoint;
			TargetTangent	 = BeamData->TargetTangent;
#if defined(_BEAM2_TYPEDATA_NORMAL_TANGENTS_)
			TargetTangent.Normalize();
#endif	//#if defined(_BEAM2_TYPEDATA_NORMAL_TANGENTS_)
			if (TargetTangent.IsNearlyZero())
			{
				TargetTangent	= Context.GetTransform().GetScaledAxis( EAxis::X );
			}
			TargetTangent	*= BeamData->TargetStrength;

			// Determine the interpolated points along the beam
			FVector	LastPosition	= SourcePosition;
			int32		ii;
			for (ii = 0; ii < InterpSteps; ii++)
			{
				InterpolatedPoints[ii] = FMath::CubicInterp(
					SourcePosition, SourceTangent,
					TargetPosition, TargetTangent,
					InvTess * double(ii + 1));
				LastPosition		= InterpolatedPoints[ii];
			}

			BeamData->TriangleCount	= BeamData->Steps * 2;
			if (BeamData->TravelRatio > UE_KINDA_SMALL_NUMBER)
			{
//070305.SAS.		BeamData->TriangleCount	+= 2;
			}

			// Grab the remaining steps...
			for (; ii < InterpSteps; ii++)
			{
				InterpolatedPoints[ii] = FMath::CubicInterp(
					SourcePosition, SourceTangent,
					TargetPosition, TargetTangent,
					InvTess * double(ii + 1));
			}

			if (bLowFreqNoise == true)
			{
				// Noisy interpolated beams!!!
				int32	NoiseTess = BeamNoise->NoiseTessellation ? BeamNoise->NoiseTessellation : 1;
				// Determine the triangle count
				BeamData->TriangleCount	 = BeamData->Steps * NoiseTess * 2;

					// If it is locked, there is an additional segment
				if (BEAM2_TYPEDATA_LOCKED(BeamData->Lock_Max_NumNoisePoints))
				{
					// The final segment of the beam
					BeamData->TriangleCount	+= NoiseTess * 2;
				}
				else
				if (BeamData->TravelRatio > UE_KINDA_SMALL_NUMBER)
				{
					//@todo.SAS. Fix this!
					// When the data fills in (vertices), it is incorrect.
					BeamData->TriangleCount	+= FMath::FloorToInt(BeamData->TravelRatio * NoiseTess) * 2;
				}
			}
		}
		else
		{
			BeamData->InterpolationSteps = 0;
			if (bLowFreqNoise == false)
			{
				// Straight-line - 2 triangles
				BeamData->TriangleCount	= 2;
			}
			else
			{
				// Determine the triangle count
				int32	NoiseTess = BeamNoise->NoiseTessellation ? BeamNoise->NoiseTessellation : 1;
				BeamData->TriangleCount	 = BeamData->Steps * NoiseTess * 2;

				// If it is locked, there is an additional segment
				if (BEAM2_TYPEDATA_LOCKED(BeamData->Lock_Max_NumNoisePoints))
				{
					// The final segment of the beam
					BeamData->TriangleCount	+= NoiseTess * 2;
				}
				else
				if (BeamData->TravelRatio > UE_KINDA_SMALL_NUMBER)
				{
					//@todo.SAS. Fix this!
					// When the data fills in (vertices), it is incorrect.
//					BeamData->TriangleCount	+= FMath::FloorToInt(BeamData->TravelRatio * NoiseTess) * 2;
				}
			}
		}
	}
	END_UPDATE_LOOP;
}


uint32 UParticleModuleTypeDataBeam2::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	int32	Size		= 0;
	int32	TaperCount	= 2;

	// Every beam requires the Beam2PayloadData
	Size	+= sizeof(FBeam2TypeDataPayload);		// Beam2 payload data

	// Store the interpolated points for each beam.
	if (InterpolationPoints >= 0)
	{
		Size		+= sizeof(FVector) * InterpolationPoints;
		TaperCount	 = InterpolationPoints ? (InterpolationPoints + 1) : 2;
	}

	// Grab pointer to highest LOD noise module to look for options
	UParticleModuleBeamNoise* BeamNoise = (LOD_BeamModule_Noise.Num() > 0) ? LOD_BeamModule_Noise[0] : nullptr;
	if (BeamNoise)
	{
		if (BeamNoise->bLowFreq_Enabled)
		{
			// This is ok as it will be the maximum number of points required...
			int32	Frequency	= BeamNoise->Frequency + 1;

			// For locking noise
//			if (NoiseLockTime > 0.0f)
			{
				Size	+= sizeof(float);				// Particle noise update time
				Size	+= sizeof(float);				// Delta time
			}
			Size	+= sizeof(FVector) * Frequency;		// The noise point positions
			if (BeamNoise->bSmooth)
			{
				Size	+= sizeof(FVector) * Frequency;	// The current noise point positions
			}

			//@todo. determine the required number of taper points...
			// (Depends on interaction of interpolation points and noise freq.
			TaperCount = (Frequency + 1) * (BeamNoise->NoiseTessellation ? BeamNoise->NoiseTessellation : 1);

			if (BeamNoise->bApplyNoiseScale)
			{
				Size	+= sizeof(float);				// Noise point scale
			}
		}
	}

	// If tapering, we need to store the taper sizes as well
	if (TaperMethod != PEBTM_None)
	{
		Size	+= sizeof(float) * TaperCount;
	}

	return Size;
}

#if WITH_EDITOR
void UParticleModuleTypeDataBeam2::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		// Make sure that 0 <= beam count <= FDynamicBeam2EmitterData::MaxBeams.
		if (PropertyThatChanged->GetFName() == FName("MaxBeamCount"))
		{
			MaxBeamCount = FMath::Clamp<int32>(MaxBeamCount, 0, FDynamicBeam2EmitterData::MaxBeams);
		}
		// Make sure that the interpolation count is > 0.
		if (PropertyThatChanged->GetFName() == FName("InterpolationPoints"))
		{
			// Clamp the interpolation points to FDynamicBeam2EmitterData::MaxInterpolationPoints...
			InterpolationPoints = FMath::Clamp<int32>(InterpolationPoints, 0, FDynamicBeam2EmitterData::MaxInterpolationPoints);
		}

		// For now, we are restricting this setting to 0 (all points) or 1 (the start point)
		UpVectorStepSize = FMath::Clamp<int32>(UpVectorStepSize, 0, 1);
	}

	UParticleSystem* PartSys = CastChecked<UParticleSystem>(GetOuter());
	if (PartSys && PropertyThatChanged)
	{
		PartSys->PostEditChangeProperty(PropertyChangedEvent);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

FParticleEmitterInstance* UParticleModuleTypeDataBeam2::CreateInstance(UParticleEmitter* InEmitterParent, IParticleEmitterInstanceOwner& InComponent)
{
	SetToSensibleDefaults(InEmitterParent);
	FParticleEmitterInstance* Instance = new FParticleBeam2EmitterInstance(InComponent);
	check(Instance);

	Instance->InitParameters(InEmitterParent);

	return Instance;
}

const FVertexFactoryType* UParticleModuleTypeDataBeam2::GetVertexFactoryType() const
{
	return &FParticleBeamTrailVertexFactory::StaticType;
}

void UParticleModuleTypeDataBeam2::CollectPSOPrecacheData(const UParticleEmitter* Emitter, FPSOPrecacheParams& OutParams)
{
	bool bUsesDynamicParameter = (Emitter->DynamicParameterDataOffset > 0);
	
	FPSOPrecacheVertexFactoryData VFData;
	VFData.VertexFactoryType = &FParticleBeamTrailVertexFactory::StaticType;
	VFData.CustomDefaultVertexDeclaration = FParticleBeamTrailVertexFactory::GetPSOPrecacheVertexDeclaration(bUsesDynamicParameter);
	OutParams.VertexFactoryDataList.Add(VFData);
	OutParams.PrimitiveType = GetPrimitiveType();
}

void UParticleModuleTypeDataBeam2::CacheModuleInfo(UParticleEmitter* Emitter)
{
	int32 LODCount = Emitter->LODLevels.Num();

	LOD_BeamModule_Source.Empty(LODCount);
	LOD_BeamModule_Source.AddZeroed(LODCount);
	LOD_BeamModule_Target.Empty(LODCount);
	LOD_BeamModule_Target.AddZeroed(LODCount);
	LOD_BeamModule_Noise.Empty(LODCount);
	LOD_BeamModule_Noise.AddZeroed(LODCount);
	LOD_BeamModule_SourceModifier.Empty(LODCount);
	LOD_BeamModule_SourceModifier.AddZeroed(LODCount);
	LOD_BeamModule_TargetModifier.Empty(LODCount);
	LOD_BeamModule_TargetModifier.AddZeroed(LODCount);

	// Used for sanity check that all LOD's DataType is the same
	UParticleModuleTypeDataBeam2* LOD_BeamTypeData = nullptr;

	for (int32 LODIdx = 0; LODIdx < LODCount; LODIdx++)
	{
		UParticleLODLevel* LODLevel = Emitter->GetLODLevel(LODIdx);
		check(LODLevel);

		// Sanity check that the DataTypeModule for each LOD is the same
		if (LODIdx == 0)
		{
			LOD_BeamTypeData = CastChecked<UParticleModuleTypeDataBeam2>(LODLevel->TypeDataModule);
			check(LOD_BeamTypeData);
		}
		/*
		else
		{
			check(LOD_BeamTypeData == LODLevel->TypeDataModule);
		}
		*/

		// Go over all the modules in the LOD level
		for (int32 ii = 0; ii < LODLevel->Modules.Num(); ii++)
		{
			bool bRemove = false;

			UParticleModule* CheckModule = LODLevel->Modules[ii];
			if ((CheckModule->GetModuleType() == EPMT_Beam) && (CheckModule->bEnabled == true))
			{

				if (CheckModule->IsA(UParticleModuleBeamSource::StaticClass()))
				{
					if (LOD_BeamModule_Source[LODIdx])
					{
						UE_LOGF(LogParticles, Log, "Warning: Multiple beam source modules!");
					}
					else
					{
						LOD_BeamModule_Source[LODIdx] = Cast<UParticleModuleBeamSource>(CheckModule);
					}
					bRemove = true;
				}
				else if (CheckModule->IsA(UParticleModuleBeamTarget::StaticClass()))
				{
					if (LOD_BeamModule_Target[LODIdx])
					{
						UE_LOGF(LogParticles, Log, "Warning: Multiple beam Target modules!");
					}
					else
					{
						LOD_BeamModule_Target[LODIdx] = Cast<UParticleModuleBeamTarget>(CheckModule);
					}
					bRemove = true;
				}
				else if (CheckModule->IsA(UParticleModuleBeamNoise::StaticClass()))
				{
					if (LOD_BeamModule_Noise[LODIdx])
					{
						UE_LOGF(LogParticles, Log, "Warning: Multiple beam Noise modules!");
					}
					else
					{
						LOD_BeamModule_Noise[LODIdx] = Cast<UParticleModuleBeamNoise>(CheckModule);
					}
					bRemove = true;
				}
				else if (CheckModule->IsA(UParticleModuleBeamModifier::StaticClass()))
				{
					UParticleModuleBeamModifier* ModifyModule = Cast<UParticleModuleBeamModifier>(CheckModule);
					if (ModifyModule->PositionOptions.bModify || ModifyModule->TangentOptions.bModify || ModifyModule->StrengthOptions.bModify)
					{
						if (ModifyModule->ModifierType == PEB2MT_Source)
						{
							LOD_BeamModule_SourceModifier[LODIdx] = ModifyModule;
							bRemove = true;
						}
						else if (ModifyModule->ModifierType == PEB2MT_Target)
						{
							LOD_BeamModule_TargetModifier[LODIdx] = ModifyModule;
							bRemove = true;
						}
					}
				}

				// These modules should never be in the UpdateModules or SpawnModules lists
				if (bRemove)
				{
					check(!LODLevel->UpdateModules.Contains(CheckModule));
					check(!LODLevel->SpawnModules.Contains(CheckModule));
				}
			}
		}
	}
}


bool UParticleModuleTypeDataBeam2::AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries)
{
	bool bNewCurve = false;
#if WITH_EDITORONLY_DATA
	//@todo. Once the old members are deprecated, open these functions back up...
	// Until then, any new distributions added to this module will have to be
	// hand-checked for in this function!!!!
	FCurveEdEntry* CurveA = NULL;
	bNewCurve |= EdSetup->AddCurveToCurrentTab(Distance.Distribution, FString("Distance"), ModuleEditorColor, &CurveA);
	OutCurveEntries.Add( CurveA );
	FCurveEdEntry* CurveB = NULL;
	bNewCurve |= EdSetup->AddCurveToCurrentTab(TaperFactor.Distribution, FString("TaperFactor"), ModuleEditorColor, &CurveB);
	OutCurveEntries.Add( CurveB );
#endif // WITH_EDITORONLY_DATA
	return bNewCurve;
}


void UParticleModuleTypeDataBeam2::GetDataPointers(FParticleEmitterInstance* Owner, 
	const uint8* ParticleBase, int32& CurrentOffset, FBeam2TypeDataPayload*& BeamData, 
	FVector*& InterpolatedPoints, float*& NoiseRate, float*& NoiseDeltaTime, 
	FVector*& TargetNoisePoints, FVector*& NextNoisePoints, float*& TaperValues, float*& NoiseDistanceScale,
	FBeamParticleModifierPayloadData*& SourceModifier, FBeamParticleModifierPayloadData*& TargetModifier)
{
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	check(Owner->bIsBeam);
	UParticleModuleBeamNoise*		BeamNoise	= BeamInst->BeamModule_Noise;

	int32	TaperCount	= 2;

	// There will alwyas be a TypeDataPayload
	PARTICLE_ELEMENT(FBeam2TypeDataPayload, Data);
	BeamData	= &Data;

	if (InterpolationPoints > 0)
	{
		// Grab the interpolation points
		PARTICLE_ELEMENT(FVector, InterpPoints);
		InterpolatedPoints	 = &InterpPoints; 
		CurrentOffset		+= sizeof(FVector) * (InterpolationPoints - 1);
		TaperCount			 = InterpolationPoints ? (InterpolationPoints + 1) : 2;
	}

	if (BeamNoise)
	{
		if (BeamNoise->bLowFreq_Enabled)
		{
			int32	Frequency	= BeamNoise->Frequency + 1;

//			if (NoiseLockTime > 0.0f)
			{
				PARTICLE_ELEMENT(float, NoiseRateData);
				NoiseRate	= &NoiseRateData;
				PARTICLE_ELEMENT(float, NoiseDeltaTimeData);
				NoiseDeltaTime	= &NoiseDeltaTimeData;
			}
			
			PARTICLE_ELEMENT(FVector, TargetNoiseData);
			TargetNoisePoints	= &TargetNoiseData;
			CurrentOffset	+= sizeof(FVector) * (Frequency - 1);
			
			if (BeamNoise->bSmooth)
			{
				PARTICLE_ELEMENT(FVector, NextNoiseData);
				NextNoisePoints	= &NextNoiseData;
				CurrentOffset	+= sizeof(FVector) * (Frequency - 1);
			}

			//@todo. determine the required number of taper points...
			// (Depends on interaction of interpolation points and noise freq.
			TaperCount = (Frequency + 1) * (BeamNoise->NoiseTessellation ? BeamNoise->NoiseTessellation : 1);

			if (BeamNoise->bApplyNoiseScale)
			{
				PARTICLE_ELEMENT(float, NoiseDistScale);
				NoiseDistanceScale = &NoiseDistScale;
			}
		}
	}

	if (TaperMethod != PEBTM_None)
	{
		PARTICLE_ELEMENT(float, TaperData);
		TaperValues		 = &TaperData;
		CurrentOffset	+= sizeof(float) * (TaperCount - 1);
	}

	if (BeamInst->BeamModule_SourceModifier_Offset != -1)
	{
		int32 TempOffset = CurrentOffset;
		CurrentOffset = BeamInst->BeamModule_SourceModifier_Offset;
		PARTICLE_ELEMENT(FBeamParticleModifierPayloadData, SourceModPayload);
		SourceModifier = &SourceModPayload;
		CurrentOffset = TempOffset;
	}

	if (BeamInst->BeamModule_TargetModifier_Offset != -1)
	{
		int32 TempOffset = CurrentOffset;
		CurrentOffset = BeamInst->BeamModule_TargetModifier_Offset;
		PARTICLE_ELEMENT(FBeamParticleModifierPayloadData, TargetModPayload);
		TargetModifier = &TargetModPayload;
		CurrentOffset = TempOffset;
	}
}


void UParticleModuleTypeDataBeam2::GetDataPointerOffsets(FParticleEmitterInstance* Owner, 
	const uint8* ParticleBase, int32& CurrentOffset, int32& BeamDataOffset, 
	int32& InterpolatedPointsOffset, int32& NoiseRateOffset, int32& NoiseDeltaTimeOffset, 
	int32& TargetNoisePointsOffset, int32& NextNoisePointsOffset, 
	int32& TaperCount, int32& TaperValuesOffset, int32& NoiseDistanceScaleOffset)
{
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	check(Owner->bIsBeam);
	UParticleModuleBeamNoise*		BeamNoise	= BeamInst->BeamModule_Noise;

	int32	LocalOffset = 0;
	
	NoiseRateOffset = -1;
	NoiseDeltaTimeOffset = -1;
	TargetNoisePointsOffset = -1;
	NextNoisePointsOffset = -1;
	InterpolatedPointsOffset = -1;
	TaperCount	= 2;
	TaperValuesOffset = -1;
	NoiseDistanceScaleOffset = -1;

	BeamDataOffset = CurrentOffset + LocalOffset;
	LocalOffset += sizeof(FBeam2TypeDataPayload);

	if (InterpolationPoints > 0)
	{
		InterpolatedPointsOffset = CurrentOffset + LocalOffset;
		LocalOffset += sizeof(FVector) * InterpolationPoints;
		TaperCount	 = InterpolationPoints ? (InterpolationPoints + 1) : 2;
	}

	if (BeamNoise)
	{
		if (BeamNoise->bLowFreq_Enabled)
		{
			int32 Frequency	= BeamNoise->Frequency + 1;

			//if (NoiseLockTime > 0.0f)
			{
				NoiseRateOffset = CurrentOffset + LocalOffset;
				LocalOffset += sizeof(float);
				NoiseDeltaTimeOffset = CurrentOffset + LocalOffset;
				LocalOffset += sizeof(float);
			}

			TargetNoisePointsOffset = CurrentOffset + LocalOffset;
			LocalOffset += sizeof(FVector) * Frequency;

			if (BeamNoise->bSmooth)
			{
				NextNoisePointsOffset = CurrentOffset + LocalOffset;
				LocalOffset += sizeof(FVector) * Frequency;
			}

			//@todo. determine the required number of taper points...
			// (Depends on interaction of interpolation points and noise freq.
			TaperCount = (Frequency + 1) * (BeamNoise->NoiseTessellation ? BeamNoise->NoiseTessellation : 1);

			if (BeamNoise->bApplyNoiseScale)
			{
				NoiseDistanceScaleOffset = CurrentOffset + LocalOffset;
				LocalOffset += sizeof(float);
			}
		}
	}

	if (TaperMethod != PEBTM_None)
	{
		TaperValuesOffset = CurrentOffset + LocalOffset;
		LocalOffset	+= sizeof(float) * TaperCount;
	}
}

/**
 *	GetNoiseRange
 *	Retrieves the range of noise
 *	
 *	@param	NoiseMin		The minimum noise - output
 *	@param	NoiseMax		The maximum noise - output
 */
void UParticleModuleTypeDataBeam2::GetNoiseRange(FVector& NoiseMin, FVector& NoiseMax)
{
	NoiseMin	= FVector::ZeroVector;
	NoiseMax	= FVector::ZeroVector;
}

/*-----------------------------------------------------------------------------
	UParticleModuleBaseModifier implementation.
-----------------------------------------------------------------------------*/
UParticleModuleBeamBase::UParticleModuleBeamBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSpawnModule = false;
	bUpdateModule = false;
}

/*-----------------------------------------------------------------------------
	UParticleModuleBeamModifier implementation.
-----------------------------------------------------------------------------*/
UParticleModuleBeamModifier::UParticleModuleBeamModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ModifierType = PEB2MT_Source;
}

void UParticleModuleBeamModifier::InitializeDefaults()
{
	if(!Position.IsCreated())
	{
		UDistributionVectorConstant* DistributionPosition = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionPosition"));
		DistributionPosition->Constant = FVector(0.0f, 0.0f, 0.0f);
		Position.Distribution = DistributionPosition;
	}
	
	if(!Tangent.IsCreated())
	{
		UDistributionVectorConstant* DistributionTangent = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionTangent"));
		DistributionTangent->Constant = FVector(0.0f, 0.0f, 0.0f);
		Tangent.Distribution = DistributionTangent;
	}

	if(!Strength.IsCreated())
	{
		UDistributionFloatConstant* DistributionStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStrength"));
		DistributionStrength->Constant = 0.0f;
		Strength.Distribution = DistributionStrength;
	}
}

void UParticleModuleBeamModifier::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		UDistributionVectorConstant* DistributionPosition = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionPosition"));
		DistributionPosition->Constant = FVector(0.0f, 0.0f, 0.0f);
		Position.Distribution = DistributionPosition;

		UDistributionVectorConstant* DistributionTangent = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionTangent"));
		DistributionTangent->Constant = FVector(0.0f, 0.0f, 0.0f);
		Tangent.Distribution = DistributionTangent;

		UDistributionFloatConstant* DistributionStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionStrength"));
		DistributionStrength->Constant = 0.0f;
		Strength.Distribution = DistributionStrength;
	}
}

void UParticleModuleBeamModifier::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (BeamInst == NULL)
	{
		return;
	}
	check(Owner->bIsBeam);

	// Setup the particle data points with the SPAWN_INIT macro
	SPAWN_INIT;
	{
		FBeam2TypeDataPayload* BeamDataPayload = NULL;
		FBeamParticleModifierPayloadData* SourceModifierPayload = NULL;
		FBeamParticleModifierPayloadData* TargetModifierPayload = NULL;

		// Get the pointers to the data, but save the module offset that is passed in.
		int32 Offset = Context.Offset;
		GetDataPointers(Owner, (const uint8*)&Particle, Offset, BeamDataPayload, SourceModifierPayload, TargetModifierPayload);

		FBeamParticleModifierPayloadData* ModifierPayload = (ModifierType == PEB2MT_Source) ? 
			SourceModifierPayload : TargetModifierPayload;

		if (ModifierPayload)
		{
			// Set the Position value
			ModifierPayload->bModifyPosition = PositionOptions.bModify;
			if (PositionOptions.bModify == true)
			{
				ModifierPayload->Position = Position.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				ModifierPayload->bScalePosition = PositionOptions.bScale;
			}

			// Set the Tangent value
			ModifierPayload->bModifyTangent = TangentOptions.bModify;
			if (TangentOptions.bModify == true)
			{
				ModifierPayload->Tangent = Tangent.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				ModifierPayload->bScaleTangent = TangentOptions.bScale;
			}

			// Set the strength value
			ModifierPayload->bModifyStrength = StrengthOptions.bModify;
			if (StrengthOptions.bModify == true)
			{
				ModifierPayload->Strength = Strength.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				ModifierPayload->bScaleStrength = StrengthOptions.bScale;
			}
		}
	}
}

void UParticleModuleBeamModifier::Update(const FUpdateContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (BeamInst == NULL || !Owner->bIsBeam)
	{
		return;
	}

	BEGIN_UPDATE_LOOP;
	{
		FBeam2TypeDataPayload* BeamDataPayload = NULL;
		FBeamParticleModifierPayloadData* SourceModifierPayload = NULL;
		FBeamParticleModifierPayloadData* TargetModifierPayload = NULL;

		// Get the pointers to the data, but save the module offset that is passed in.
		GetDataPointers(Owner, (const uint8*)&Particle, Offset, BeamDataPayload, SourceModifierPayload, TargetModifierPayload);

		FBeamParticleModifierPayloadData* ModifierPayload = (ModifierType == PEB2MT_Source) ? 
			SourceModifierPayload : TargetModifierPayload;

		if (ModifierPayload)
		{
			// Set the Position value
			ModifierPayload->bModifyPosition = PositionOptions.bModify;
			if ((PositionOptions.bModify == true) && (PositionOptions.bLock == false))
			{
				ModifierPayload->Position = Position.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				ModifierPayload->bScalePosition = PositionOptions.bScale;
			}

			// Set the Tangent value
			ModifierPayload->bModifyTangent = TangentOptions.bModify;
			if ((TangentOptions.bModify == true) && (TangentOptions.bLock == false))
			{
				ModifierPayload->Tangent = Tangent.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				ModifierPayload->bScaleTangent = TangentOptions.bScale;
			}

			// Set the strength value
			ModifierPayload->bModifyStrength = StrengthOptions.bModify;
			if ((StrengthOptions.bModify == true) && (StrengthOptions.bLock == false))
			{
				ModifierPayload->Strength = Strength.GetValue(Owner->EmitterTime, Context.GetDistributionData());
				ModifierPayload->bScaleStrength = StrengthOptions.bScale;
			}
		}
	}
	END_UPDATE_LOOP;
}

uint32 UParticleModuleBeamModifier::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	return sizeof(FBeamParticleModifierPayloadData);
}

#if WITH_EDITOR
void UParticleModuleBeamModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleBeamModifier::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{

}

void UParticleModuleBeamModifier::GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList)
{
}

void UParticleModuleBeamModifier::GetCurveObjects(TArray<FParticleCurvePair>& OutCurves)
{
	FParticleCurvePair* NewCurve;

	NewCurve = new(OutCurves) FParticleCurvePair;
	check(NewCurve);
	NewCurve->CurveObject = Position.Distribution;
	NewCurve->CurveName = FString("Position");
	NewCurve = new(OutCurves) FParticleCurvePair;
	check(NewCurve);
	NewCurve->CurveObject = Tangent.Distribution;
	NewCurve->CurveName = FString("Tangent");
	NewCurve = new(OutCurves) FParticleCurvePair;
	check(NewCurve);
	NewCurve->CurveObject = Strength.Distribution;
	NewCurve->CurveName = FString("Strength");
}

bool UParticleModuleBeamModifier::AddModuleCurvesToEditor(UInterpCurveEdSetup* EdSetup, TArray<const FCurveEdEntry*>& OutCurveEntries)
{
	bool bNewCurve = false;
#if WITH_EDITORONLY_DATA
	FCurveEdEntry* CurveA = NULL;
	bNewCurve |= EdSetup->AddCurveToCurrentTab(Position.Distribution, TEXT("Position"), ModuleEditorColor, &CurveA, false);
	OutCurveEntries.Add( CurveA );
	FCurveEdEntry* CurveB = NULL;
	bNewCurve |= EdSetup->AddCurveToCurrentTab(Tangent.Distribution, TEXT("Tangent"), ModuleEditorColor, &CurveB, false);
	OutCurveEntries.Add( CurveB );
	FCurveEdEntry* CurveC = NULL;
	bNewCurve |= EdSetup->AddCurveToCurrentTab(Strength.Distribution, TEXT("Strength"), ModuleEditorColor, &CurveC, false);	
	OutCurveEntries.Add( CurveC );	
#endif // WITH_EDITORONLY_DATA
	return bNewCurve;
}

void UParticleModuleBeamModifier::GetDataPointers(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
	int32& CurrentOffset, FBeam2TypeDataPayload*& BeamDataPayload, 
	FBeamParticleModifierPayloadData*& SourceModifierPayload,
	FBeamParticleModifierPayloadData*& TargetModifierPayload)
{
	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (BeamInst == NULL || !Owner->bIsBeam)
	{
		return;
	}

	if (BeamInst->BeamModule_SourceModifier)
	{
		SourceModifierPayload = (FBeamParticleModifierPayloadData*)(ParticleBase + BeamInst->BeamModule_SourceModifier_Offset);
	}
	else
	{
		SourceModifierPayload = NULL;
	}

	if (BeamInst->BeamModule_TargetModifier)
	{
		TargetModifierPayload = (FBeamParticleModifierPayloadData*)(ParticleBase + BeamInst->BeamModule_TargetModifier_Offset);
	}
	else
	{
		TargetModifierPayload = NULL;
	}
}

void UParticleModuleBeamModifier::GetDataPointerOffsets(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
	int32& CurrentOffset, int32& BeamDataOffset, int32& SourceModifierOffset, int32& TargetModifierOffset)
{
	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (BeamInst == NULL || !Owner->bIsBeam)
	{
		return;
	}

	BeamDataOffset = BeamInst->TypeDataOffset;
	SourceModifierOffset = BeamInst->BeamModule_SourceModifier_Offset;
	TargetModifierOffset = BeamInst->BeamModule_TargetModifier_Offset;
}

/*-----------------------------------------------------------------------------
	UParticleModuleBeamNoise implementation.
-----------------------------------------------------------------------------*/
UParticleModuleBeamNoise::UParticleModuleBeamNoise(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Frequency = 0;
	NoiseLockRadius = 1.0f;
	bSmooth = false;
	bNoiseLock = false;
	bOscillate = false;
	NoiseLockTime = 0.0f;
	NoiseTension = 0.5f;
	NoiseTessellation = 1;
	bTargetNoise = false;
}

void UParticleModuleBeamNoise::InitializeDefaults()
{
	if (!NoiseSpeed.IsCreated())
	{
		UDistributionVectorConstant* DistributionNoiseSpeed = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionNoiseSpeed"));
		DistributionNoiseSpeed->Constant = FVector(50.0f, 50.0f, 50.0f);
		NoiseSpeed.Distribution = DistributionNoiseSpeed;
	}

	if (!NoiseRange.IsCreated())
	{
		UDistributionVectorConstant* DistributionNoiseRange = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionNoiseRange"));
		DistributionNoiseRange->Constant = FVector(50.0f, 50.0f, 50.0f);
		NoiseRange.Distribution = DistributionNoiseRange;
	}

	if (!NoiseRangeScale.IsCreated())
	{
		UDistributionFloatConstant* DistributionNoiseRangeScale = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionNoiseRangeScale"));
		DistributionNoiseRangeScale->Constant = 1.0f;
		NoiseRangeScale.Distribution = DistributionNoiseRangeScale; 
	}

	if (!NoiseTangentStrength.IsCreated())
	{
		UDistributionFloatConstant* DistributionNoiseTangentStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionNoiseTangentStrength"));
		DistributionNoiseTangentStrength->Constant = 250.0f;
		NoiseTangentStrength.Distribution = DistributionNoiseTangentStrength; 
	}

	if (!NoiseScale.IsCreated())
	{
		NoiseScale.Distribution = NewObject<UDistributionFloatConstantCurve>(this, TEXT("DistributionNoiseScale"));
	}
}

void UParticleModuleBeamNoise::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		UDistributionVectorConstant* DistributionNoiseSpeed = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionNoiseSpeed"));
		DistributionNoiseSpeed->Constant = FVector(50.0f, 50.0f, 50.0f);
		NoiseSpeed.Distribution = DistributionNoiseSpeed;

		UDistributionVectorConstant* DistributionNoiseRange = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionNoiseRange"));
		DistributionNoiseRange->Constant = FVector(50.0f, 50.0f, 50.0f);
		NoiseRange.Distribution = DistributionNoiseRange;

		UDistributionFloatConstant* DistributionNoiseRangeScale = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionNoiseRangeScale"));
		DistributionNoiseRangeScale->Constant = 1.0f;
		NoiseRangeScale.Distribution = DistributionNoiseRangeScale; 

		UDistributionFloatConstant* DistributionNoiseTangentStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionNoiseTangentStrength"));
		DistributionNoiseTangentStrength->Constant = 250.0f;
		NoiseTangentStrength.Distribution = DistributionNoiseTangentStrength; 

		NoiseScale.Distribution = NewObject<UDistributionFloatConstantCurve>(this, TEXT("DistributionNoiseScale"));
	}
}

void UParticleModuleBeamNoise::Spawn(const FSpawnContext& Context)
{
	if (bLowFreq_Enabled == false)
	{
		// Noise is present but disabled...
		return;
	}

	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (!BeamInst || !bLowFreq_Enabled || (Frequency == 0) || !Owner->bIsBeam)
	{
		return;
	}

	UParticleModuleTypeDataBeam2*	BeamTD		= BeamInst->BeamTypeData;

	FRandomStream& RandomStream = GetRandomStream(Context);

	SPAWN_INIT;

	FBeam2TypeDataPayload*	BeamData			= NULL;
	FVector*				InterpolatedPoints	= NULL;
	float*					NoiseRate			= NULL;
	float*					NoiseDelta			= NULL;
	FVector*				TargetNoisePoints	= NULL;
	FVector*				NextNoisePoints		= NULL;
	float*					TaperValues			= NULL;
	float*					NoiseDistanceScale	= NULL;
	FBeamParticleModifierPayloadData* SourceModifier = NULL;
	FBeamParticleModifierPayloadData* TargetModifier = NULL;

	// Retrieve the data points
	int32 TempOffset = BeamInst->TypeDataOffset;
	BeamTD->GetDataPointers(Owner, (uint8*)ParticleBase, TempOffset, BeamData, InterpolatedPoints, 
		NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, TaperValues,
		NoiseDistanceScale, SourceModifier, TargetModifier);
	CurrentOffset	= TempOffset;

	// There should always be noise points
	check(TargetNoisePoints);
	if (bSmooth)
	{
		// There should be next noise points when smoothly moving them
		check(NextNoisePoints);
	}

	// If the frequency range mode is enabled, select a frequency
	int32 CalcFreq = Frequency;
	if (Frequency_LowRange > 0)
	{
		CalcFreq = FMath::TruncToInt((RandomStream.FRand() * (Frequency - Frequency_LowRange)) + Frequency_LowRange);
	}
	BEAM2_TYPEDATA_SETFREQUENCY(BeamData->Lock_Max_NumNoisePoints, CalcFreq);
	
	// Pre-pick the initial noise points - for noise-lock cases
	float	StepSize		= 1.0f / (CalcFreq + 1);

	// Fill in the points...

	// See if we are oscillating
	bool bLocalOscillate = false;
	if (NoiseRange.IsUniform())
	{
		bLocalOscillate = true;
	}

	// Handle bouncing between extremes
	int32	Extreme = -1;
	for (int32 ii = 0; ii < (CalcFreq + 1); ii++)
	{
		if (bLocalOscillate && bOscillate)
		{
			Extreme = -Extreme;
		}
		else
		{
			Extreme = 0;
		}
		TargetNoisePoints[ii] = NoiseRange.GetValue(StepSize * ii, Context.GetDistributionData(), Extreme);
		if (bSmooth)
		{
			Extreme = -Extreme;
			NextNoisePoints[ii] = NoiseRange.GetValue(StepSize * ii, Context.GetDistributionData(), Extreme);
		}
	}
}

void UParticleModuleBeamNoise::Update(const FUpdateContext& Context)
{
	if (bLowFreq_Enabled == false)
	{
		// Noise is present but disabled...
		return;
	}

	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance and there is noise.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (!BeamInst || (Frequency == 0) || !Owner->bIsBeam)
	{
		return;
	}

	UParticleModuleTypeDataBeam2*	BeamTD		= BeamInst->BeamTypeData;

	bool bLocalOscillate = false;
	if (NoiseRange.IsUniform())
	{
		bLocalOscillate = true;
	}

	int32	Extreme = -1;

	BEGIN_UPDATE_LOOP;
	{
		FBeam2TypeDataPayload*	BeamData			= NULL;
		FVector*				InterpolatedPoints	= NULL;
		float*					NoiseRate			= NULL;
		float*					NoiseDelta			= NULL;
		FVector*				TargetNoisePoints	= NULL;
		FVector*				NextNoisePoints		= NULL;
		float*					TaperValues			= NULL;
		float*					NoiseDistanceScale	= NULL;
		FBeamParticleModifierPayloadData* SourceModifier = NULL;
		FBeamParticleModifierPayloadData* TargetModifier = NULL;

		int32						TempOffset	= BeamInst->TypeDataOffset;
		BeamTD->GetDataPointers(Owner, ParticleBase, TempOffset, BeamData, InterpolatedPoints, 
			NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, TaperValues,
			NoiseDistanceScale, SourceModifier, TargetModifier);

		check(TargetNoisePoints);
		if (bSmooth)
		{
			check(NextNoisePoints);
		}

		int32 Freq = BEAM2_TYPEDATA_FREQUENCY(BeamData->Lock_Max_NumNoisePoints);

		{
			if (bLocalOscillate && bOscillate)
			{
				Extreme = -Extreme;
			}
			else
			{
				Extreme = 0;
			}
			if (NoiseLockTime < 0.0f)
			{
				// Do nothing...
			}
			else
			{
				float	StepSize	= 1.0f / (Freq + 1);

				// Fill in the points...
				if (NoiseLockTime > UE_KINDA_SMALL_NUMBER)
				{
					//@todo. Add support for moving noise points!
					// Check the times...
					check(NoiseRate);
					*NoiseRate += Context.DeltaTime;
					if (*NoiseRate > NoiseLockTime)
					{
						if (bSmooth)
						{
							for (int32 ii = 0; ii < (Freq + 1); ii++)
							{
								NextNoisePoints[ii] = NoiseRange.GetValue(StepSize * ii, Context.GetDistributionData(), Extreme);
							}
						}
						else
						{
							for (int32 ii = 0; ii < (Freq + 1); ii++)
							{
								TargetNoisePoints[ii] = NoiseRange.GetValue(StepSize * ii, Context.GetDistributionData(), Extreme);
							}
						}
						*NoiseRate	= 0.0f;
					}
					*NoiseDelta	= Context.DeltaTime;
				}
				else
				{
					for (int32 ii = 0; ii < (Freq + 1); ii++)
					{
						TargetNoisePoints[ii] = NoiseRange.GetValue(StepSize * ii, Context.GetDistributionData(), Extreme);
					}
				}
			}
		}
	}
	END_UPDATE_LOOP;
}

void UParticleModuleBeamNoise::SetToSensibleDefaults(UParticleEmitter* Owner)
{
	// Set the noise lock flag
	if (NoiseLockTime < 0.0f)
	{
		bNoiseLock	= true;
	}
	else
	{
		bNoiseLock	= false;
	}
}

#if WITH_EDITOR
void UParticleModuleBeamNoise::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	// Clamp the noise points to FDynamicBeam2EmitterData::MaxNoiseFrequency...
	if (Frequency > FDynamicBeam2EmitterData::MaxNoiseFrequency)
	{
		Frequency = FDynamicBeam2EmitterData::MaxNoiseFrequency;
	}

	if (Frequency_LowRange > Frequency)
	{
		if (Frequency_LowRange > FDynamicBeam2EmitterData::MaxNoiseFrequency)
		{
			Frequency_LowRange = FDynamicBeam2EmitterData::MaxNoiseFrequency;
		}
		Frequency = Frequency_LowRange;
	}

	UParticleSystem* PartSys = CastChecked<UParticleSystem>(GetOuter());
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PartSys && PropertyThatChanged)
	{
		// Make sure that the interpolation count is > 0.
		if (PropertyThatChanged->GetFName() == FName("NoiseTessellation"))
		{
			// Clamp the tessellation
			NoiseTessellation = FMath::Clamp<int32>(NoiseTessellation, 0, UParticleModuleBeamNoise::MaxNoiseTessellation);
		}

		PartSys->PostEditChangeProperty(PropertyChangedEvent);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleBeamNoise::GetNoiseRange(FVector& NoiseMin, FVector& NoiseMax)
{
#ifdef BEAMS_TODO
	NoiseRange.GetOutRange(NoiseMin, NoiseMax);
#endif	//#ifdef BEAMS_TODO

	float Min, Max;
	// get the min/max for x, y AND z
	NoiseRange.GetOutRange(Min, Max);
	// make vectors out of the floats
	NoiseMin.X = NoiseMin.Y = NoiseMin.Z = Min;
	NoiseMax.X = NoiseMax.Y = NoiseMax.Z = Max;
}

/*-----------------------------------------------------------------------------
	UParticleModuleBeamSource implementation.
-----------------------------------------------------------------------------*/
UParticleModuleBeamSource::UParticleModuleBeamSource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SourceMethod = PEB2STM_Default;
	SourceName = ConstructorStatics.NAME_None;
	bSourceAbsolute = false;
	SourceTangentMethod = PEB2STTM_Direct;
}


void UParticleModuleBeamSource::InitializeDefaults()
{
	if (!Source.IsCreated())
	{
		UDistributionVectorConstant* DistributionSource = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionSource"));
		DistributionSource->Constant = FVector(50.0f, 50.0f, 50.0f);
		Source.Distribution = DistributionSource; 
	}

	if (!SourceTangent.IsCreated())
	{
		UDistributionVectorConstant* DistributionSourceTangent = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionSourceTangent"));
		DistributionSourceTangent->Constant = FVector(1.0f, 0.0f, 0.0f);
		SourceTangent.Distribution = DistributionSourceTangent; 
	}

	if (!SourceStrength.IsCreated())
	{
		UDistributionFloatConstant* DistributionSourceStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionSourceStrength"));
		DistributionSourceStrength->Constant = 25.0f;
		SourceStrength.Distribution = DistributionSourceStrength; 
	}
}

void UParticleModuleBeamSource::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleBeamSource::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (!BeamInst || !Owner->bIsBeam)
	{
		return;
	}
	UParticleModuleTypeDataBeam2*	BeamTD		= BeamInst->BeamTypeData;

	SPAWN_INIT;

	FBeam2TypeDataPayload*	BeamData			= NULL;
	FVector*				InterpolatedPoints	= NULL;
	float*					NoiseRate			= NULL;
	float*					NoiseDelta			= NULL;
	FVector*				TargetNoisePoints	= NULL;
	FVector*				NextNoisePoints		= NULL;
	float*					TaperValues			= NULL;
	float*					NoiseDistanceScale	= NULL;
	FBeamParticleModifierPayloadData* SourceModifier = NULL;
	FBeamParticleModifierPayloadData* TargetModifier = NULL;

	// Retrieve the data pointers from the payload
	int32	TempOffset	= BeamInst->TypeDataOffset;
	BeamInst->BeamTypeData->GetDataPointers(Owner, (uint8*)ParticleBase, TempOffset, BeamData, 
		InterpolatedPoints, NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, 
		TaperValues, NoiseDistanceScale, SourceModifier, TargetModifier);

	// Resolve the source data
	int32 Offset = Context.Offset;
	ResolveSourceData(Context, BeamInst, BeamData, (uint8*)ParticleBase, Offset, BeamInst->ActiveParticles, true, SourceModifier);

	// Set the location and clear the initial data flags
	Particle.Location					= BeamData->SourcePoint - BeamInst->PositionOffsetThisTick;
	BeamData->Lock_Max_NumNoisePoints	= 0;
	BeamData->StepSize					= 0.0f;
	BeamData->Steps						= 0;
	BeamData->TravelRatio				= 0.0f;
	BeamData->TriangleCount				= 0;
}

void UParticleModuleBeamSource::Update(const FUpdateContext& Context)
{
	// If the source is locked, don't perform any update
	if (bLockSource && bLockSourceTangent && bLockSourceStength)
	{
		return;
	}

	FParticleEmitterInstance* Owner = &Context.Owner;

	// Make sure that the owner is a beam emitter instance.
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (!BeamInst || !Owner->bIsBeam)
	{
		return;
	}
	UParticleModuleTypeDataBeam2*	BeamTD		= BeamInst->BeamTypeData;

	BEGIN_UPDATE_LOOP;
	{
		FBeam2TypeDataPayload*	BeamData			= NULL;
		FVector*				InterpolatedPoints	= NULL;
		float*					NoiseRate			= NULL;
		float*					NoiseDelta			= NULL;
		FVector*				TargetNoisePoints	= NULL;
		FVector*				NextNoisePoints		= NULL;
		float*					TaperValues			= NULL;
		float*					NoiseDistanceScale	= NULL;
		FBeamParticleModifierPayloadData* SourceModifier = NULL;
		FBeamParticleModifierPayloadData* TargetModifier = NULL;

		// Retrieve the payload data offsets
		int32	TempOffset	= BeamInst->TypeDataOffset;
		BeamTD->GetDataPointers(Owner, ParticleBase, TempOffset, BeamData, InterpolatedPoints, 
			NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, TaperValues,
			NoiseDistanceScale, SourceModifier, TargetModifier);

		// Resolve the source data
		ResolveSourceData(Context, BeamInst, BeamData, ParticleBase, Offset, i, false, SourceModifier);
	}
	END_UPDATE_LOOP;
}

uint32 UParticleModuleBeamSource::RequiredBytes(UParticleModuleTypeDataBase* TypeData)
{
	int32 Size = 0;

	if (SourceMethod == PEB2STM_Particle)
	{
		// Store the data for the particle source payload
		Size	+= sizeof(FBeamParticleSourceTargetPayloadData);
	}

	UParticleModuleTypeDataBeam2* BeamTD = Cast<UParticleModuleTypeDataBeam2>(TypeData);
	if (BeamTD != nullptr && BeamTD->BeamMethod == PEB2M_Branch)
	{
		// Store the data for the particle branch payload
		Size	+= sizeof(FBeamParticleSourceBranchPayloadData);
	}

	return Size;
}

#if WITH_EDITOR
void UParticleModuleBeamSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	UParticleSystem* PartSys = CastChecked<UParticleSystem>(GetOuter());
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PartSys && PropertyThatChanged)
	{
		PartSys->PostEditChangeProperty(PropertyChangedEvent);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleBeamSource::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	check(IsInGameThread());
	bool	bFound	= false;

	for (int32 i = 0; i < PSysComp->InstanceParameters.Num(); i++)
	{
		FParticleSysParam* Param = &(PSysComp->InstanceParameters[i]);
		
		if (Param->Name == SourceName)
		{
			bFound	=	true;
			break;
		}
	}

	if (!bFound)
	{
		int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
		PSysComp->InstanceParameters[NewParamIndex].Name		= SourceName;
		PSysComp->InstanceParameters[NewParamIndex].ParamType	= PSPT_Actor;
		PSysComp->InstanceParameters[NewParamIndex].Actor		= NULL;
	}
}

void UParticleModuleBeamSource::GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList)
{
	if (SourceMethod == PEB2STM_Actor)
	{
		ParticleSysParamList.Add(FString::Printf(TEXT("BeamSource : Actor: %s\n"), *(SourceName.ToString())));
	}
}

void UParticleModuleBeamSource::GetDataPointers(FParticleEmitterInstance* Owner, 
	const uint8* ParticleBase, int32& CurrentOffset, 
	FBeamParticleSourceTargetPayloadData*& ParticleSource,
	FBeamParticleSourceBranchPayloadData*& BranchSource)
{
	FParticleBeam2EmitterInstance* BeamInst = (FParticleBeam2EmitterInstance*)Owner;
	if (BeamInst && Owner->bIsBeam)
	{
		UParticleModuleTypeDataBeam2*	BeamTD	= BeamInst->BeamTypeData;
		if (BeamTD)
		{
			if (SourceMethod == PEB2STM_Particle)
			{
				PARTICLE_ELEMENT(FBeamParticleSourceTargetPayloadData, LocalParticleSource);
				ParticleSource	= &LocalParticleSource;
			}
			if (BeamTD->BeamMethod == PEB2M_Branch)
			{
				PARTICLE_ELEMENT(FBeamParticleSourceBranchPayloadData, LocalBranchSource);
				BranchSource	= &LocalBranchSource;
			}
		}
	}
}

bool UParticleModuleBeamSource::ResolveSourceData(const FContext& Context, FParticleBeam2EmitterInstance* BeamInst, 
	FBeam2TypeDataPayload* BeamData, const uint8* ParticleBase, int32& Offset, int32	ParticleIndex,
	bool bSpawning, FBeamParticleModifierPayloadData* ModifierData)
{
	bool	bResult	= false;

	FBaseParticle& Particle	= *((FBaseParticle*) ParticleBase);

	FBeamParticleSourceBranchPayloadData* BranchSource		= NULL;
	FBeamParticleSourceTargetPayloadData* ParticleSource	= NULL;
	GetDataPointers(BeamInst, ParticleBase, Offset, ParticleSource, BranchSource);

	if ((bSpawning == true) || (bLockSource == false))
	{
		// Resolve the source point...
		bool bSetSource = false;
		switch (SourceMethod)
		{
		case PEB2STM_UserSet:
			// User-set points are utilized directly.
			if (BeamInst->UserSetSourceArray.Num() > 0)
			{
				if (ParticleIndex >= BeamInst->UserSetSourceArray.Num())
				{
					//@todo. How to handle this situation???
					BeamData->SourcePoint	= BeamInst->UserSetSourceArray[0];
				}
				else
				{
					BeamData->SourcePoint	= BeamInst->UserSetSourceArray[ParticleIndex];
				}
				bSetSource	= true;
			}
			break;
		case PEB2STM_Emitter:
			// The position of the owner component is the source
			BeamData->SourcePoint	= BeamInst->Component.GetComponentTransform().GetLocation();
			bSetSource				= true;
			break;
		case PEB2STM_Particle:
			{
				if (BeamInst->BeamTypeData->BeamMethod == PEB2M_Branch
					|| BeamInst->BeamTypeData->BeamMethod == PEB2M_Target)
				{
					// Branching beam - resolve the source emitter if needed
					if (BeamInst->SourceEmitter == NULL)
					{
						BeamInst->ResolveSource();
					}

					if (BeamInst->SourceEmitter && BeamInst->SourceEmitter->ParticleIndices)
					{
						int32 SourceIndex = LastSelectedParticleIndex;

						if (SourceIndex != -1)
						{
							FBaseParticle* SourceParticle = BeamInst->SourceEmitter->GetParticleDirect(SourceIndex);
							if (SourceParticle == NULL || SourceParticle->RelativeTime>1.0f)
							{
								// If the previous particle is not found, force the trail to pick a new one
								SourceIndex = -1;
							}
						}

						if (SourceIndex == -1)
						{
							int32 Index = 0;
							// TODO: add selection method and random selection
							/*
							switch (SelectionMethod)
							{
							case EPSSM_Random:
							{
							Index = FMath::TruncToInt(FMath::FRand() * BeamInst->SourceEmitter->ActiveParticles);
							}
							break;

							case EPSSM_Sequential:*/
							{
								if (++LastSelectedParticleIndex >= BeamInst->SourceEmitter->ActiveParticles)
								{
									LastSelectedParticleIndex = 0;
								}
								Index = LastSelectedParticleIndex;
							}
							/*
							break;
							}
							*/

							if (SourceIndex == BeamInst->SourceEmitter->ParticleIndices[Index])
							{
								Index = -1;
							}

							SourceIndex = (Index != -1) ? BeamInst->SourceEmitter->ParticleIndices[Index] : -1;
						}

						bool bEncounteredNaNError = false;

						// Grab the particle

						IParticleEmitterInstanceOwner& Comp = BeamInst->SourceEmitter->Component;
						FBaseParticle* SourceParticle = (SourceIndex >= 0) ? BeamInst->SourceEmitter->GetParticleDirect(SourceIndex) : nullptr;
						if (SourceParticle != nullptr)
						{
							const FVector WorldOrigin = BeamInst->SourceEmitter->SimulationToWorld.GetOrigin();

							if (!ensureMsgf(!SourceParticle->Location.ContainsNaN(), TEXT("NaN in SourceParticle Location. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName()) ||
								!ensureMsgf(!SourceParticle->OldLocation.ContainsNaN(), TEXT("NaN in SourceParticle OldLocation. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName()) ||
								!ensureMsgf(!WorldOrigin.ContainsNaN(), TEXT("NaN in WorldOrigin. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName())
								)
							{
								// Contains NaN!
								bEncounteredNaNError = true;
							}
							else
							{
								BeamData->SourcePoint = SourceParticle->Location + WorldOrigin;
								bSetSource = true;
							}
						}
						else
						{
							// Fall back to the emitter location??
							BeamData->SourcePoint = Comp.GetComponentTransform().GetLocation();
							bSetSource = true;

							//@todo. How to handle this... can potentially cause a jump from the emitter to the
							// particle...
							SourceIndex = -1;//No valid particle source;
						}
					}
				}
			}
			break;
		case PEB2STM_Actor:
			if (SourceName != NAME_None)
			{
				BeamInst->ResolveSource();
				// Use the actor position as the source
				if (BeamInst->SourceActor)
				{
					BeamData->SourcePoint	= BeamInst->SourceActor->ActorToWorld().GetLocation();
					bSetSource = true;
				}
			}
			break;
		}

		if (bSetSource == false)
		{
			// If the source hasn't been set at this point, assume that we are using
			// the Source distribution.
			if (bSourceAbsolute)
			{
				// Use the value as a world space position
				BeamData->SourcePoint	= Source.GetValue(BeamInst->EmitterTime, Context.GetDistributionData());
				// Take into account current world origin offset
				BeamData->SourcePoint  -= (FVector)BeamInst->Component.GetWorldOriginLocation();
			}
			else
			{
				// Use the value as a local space position.
				BeamData->SourcePoint	= Context.GetTransform().TransformPosition(
					Source.GetValue(BeamInst->EmitterTime, Context.GetDistributionData()));
			}
		}
	}

	if ((bSpawning == true) || (bLockSourceTangent == false))
	{
		// If we are spawning and the source tangent is not locked, resolve it
		bool bSetSourceTangent = false;
		switch (SourceTangentMethod)
		{
		case PEB2STTM_Direct:
			// Use the emitter direction as the tangent
			BeamData->SourceTangent	= Context.GetTransform().GetScaledAxis( EAxis::X );
			bSetSourceTangent		= true;
			break;
		case PEB2STTM_UserSet:
			// Use the user-set tangent directly
			if (BeamInst->UserSetSourceTangentArray.Num() > 0)
			{
				if (ParticleIndex >= BeamInst->UserSetSourceTangentArray.Num())
				{
					//@todo. How to handle this situation???
					BeamData->SourceTangent	= BeamInst->UserSetSourceTangentArray[0];
				}
				else
				{
					BeamData->SourceTangent	= BeamInst->UserSetSourceTangentArray[ParticleIndex];
				}
				bSetSourceTangent	= true;
			}
			break;
		case PEB2STTM_Distribution:
			// Use the tangent contained in the distribution
			BeamData->SourceTangent	= SourceTangent.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			bSetSourceTangent		= true;
			break;
		case PEB2STTM_Emitter:
			// Use the emitter direction as the tangent
			BeamData->SourceTangent	= Context.GetTransform().GetScaledAxis( EAxis::X );
			bSetSourceTangent		= true;
			break;
		}

		if (bSetSourceTangent == false)
		{
			// By default, use the distribution. This will allow artists an easier setup phase...
//			BeamData->SourceTangent	= BeamInst->Component->LocalToWorld.GetScaledAxis( EAxis::X );
			BeamData->SourceTangent	= SourceTangent.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			if (bSourceAbsolute == false)
			{
				// If not tagged as absolute, transform it to world space
				BeamData->SourceTangent	= Context.GetTransform().TransformVector(BeamData->SourceTangent);
			}
		}
	}

	if ((bSpawning == true) || (bLockSourceStength == false))
	{
		// If we are spawning and the source strength is not locked, resolve it
		bool bSetSourceStrength = false;
		if (SourceTangentMethod == PEB2STTM_UserSet)
		{
			if (BeamInst->UserSetSourceStrengthArray.Num() > 0)
			{
				if (ParticleIndex >= BeamInst->UserSetSourceStrengthArray.Num())
				{
					//@todo. How to handle this situation???
					BeamData->SourceStrength	= BeamInst->UserSetSourceStrengthArray[0];
				}
				else
				{
					BeamData->SourceStrength	= BeamInst->UserSetSourceStrengthArray[ParticleIndex];
				}
				bSetSourceStrength	= true;
			}
		}

		if (!bSetSourceStrength)
		{
			BeamData->SourceStrength	= SourceStrength.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		}
	}

	// For now, assume it worked...
	bResult	= true;

	return bResult;
}

/*-----------------------------------------------------------------------------
	UParticleModuleBeamTarget implementation.
-----------------------------------------------------------------------------*/
UParticleModuleBeamTarget::UParticleModuleBeamTarget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_None;
		FConstructorStatics()
			: NAME_None(TEXT("None"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	TargetMethod = PEB2STM_Default;
	TargetName = ConstructorStatics.NAME_None;
	bTargetAbsolute = false;
	TargetTangentMethod = PEB2STTM_Direct;
	LockRadius = 10.0;
}

void UParticleModuleBeamTarget::InitializeDefaults()
{
	if (!Target.IsCreated())
	{
		UDistributionVectorConstant* DistributionTarget = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionTarget"));
		DistributionTarget->Constant = FVector(50.0f, 50.0f, 50.0f);
		Target.Distribution = DistributionTarget;
	}

	if (!TargetTangent.IsCreated())
	{
		UDistributionVectorConstant* DistributionTargetTangent = NewObject<UDistributionVectorConstant>(this, TEXT("DistributionTargetTangent"));
		DistributionTargetTangent->Constant = FVector(1.0f, 0.0f, 0.0f);
		TargetTangent.Distribution = DistributionTargetTangent; 
	}

	if (!TargetStrength.IsCreated())
	{
		UDistributionFloatConstant* DistributionTargetStrength = NewObject<UDistributionFloatConstant>(this, TEXT("DistributionTargetStrength"));
		DistributionTargetStrength->Constant = 25.0;
		TargetStrength.Distribution = DistributionTargetStrength;
	}
}

void UParticleModuleBeamTarget::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		InitializeDefaults();
	}
}

void UParticleModuleBeamTarget::Spawn(const FSpawnContext& Context)
{
	FParticleEmitterInstance* Owner = &Context.Owner;

	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (!BeamInst || !Owner->bIsBeam)
	{
		return;
	}
	UParticleModuleTypeDataBeam2*	BeamTD		= BeamInst->BeamTypeData;

	SPAWN_INIT;

	FBeam2TypeDataPayload*	BeamData			= NULL;
	FVector*				InterpolatedPoints	= NULL;
	float*					NoiseRate			= NULL;
	float*					NoiseDelta			= NULL;
	FVector*				TargetNoisePoints	= NULL;
	FVector*				NextNoisePoints		= NULL;
	float*					TaperValues			= NULL;
	float*					NoiseDistanceScale	= NULL;
	FBeamParticleModifierPayloadData* SourceModifier = NULL;
	FBeamParticleModifierPayloadData* TargetModifier = NULL;

	int32						TempOffset	= BeamInst->TypeDataOffset;
	BeamInst->BeamTypeData->GetDataPointers(Owner, (uint8*)ParticleBase, TempOffset, BeamData, 
		InterpolatedPoints, NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, 
		TaperValues, NoiseDistanceScale, SourceModifier, TargetModifier);
	CurrentOffset	= TempOffset;

	int32 Offset = Context.Offset;
	ResolveTargetData(Context, BeamInst, BeamData, (uint8*)ParticleBase, Offset, BeamInst->ActiveParticles, true, TargetModifier);
}

void UParticleModuleBeamTarget::Update(const FUpdateContext& Context)
{
	if (bLockTarget && bLockTargetTangent && bLockTargetStength)
	{
		return;
	}

	FParticleEmitterInstance* Owner = &Context.Owner;
	FParticleBeam2EmitterInstance*	BeamInst	= (FParticleBeam2EmitterInstance*)Owner;
	if (!BeamInst || !Owner->bIsBeam)
	{
		return;
	}
	UParticleModuleTypeDataBeam2*	BeamTD		= BeamInst->BeamTypeData;

	BEGIN_UPDATE_LOOP;
	{
		FBeam2TypeDataPayload*	BeamData			= NULL;
		FVector*				InterpolatedPoints	= NULL;
		float*					NoiseRate			= NULL;
		float*					NoiseDelta			= NULL;
		FVector*				TargetNoisePoints	= NULL;
		FVector*				NextNoisePoints		= NULL;
		float*					TaperValues			= NULL;
		float*					NoiseDistanceScale	= NULL;
		FBeamParticleModifierPayloadData* SourceModifier = NULL;
		FBeamParticleModifierPayloadData* TargetModifier = NULL;

		int32						TempOffset	= BeamInst->TypeDataOffset;
		BeamTD->GetDataPointers(Owner, ParticleBase, TempOffset, BeamData, InterpolatedPoints, 
			NoiseRate, NoiseDelta, TargetNoisePoints, NextNoisePoints, TaperValues,
			NoiseDistanceScale, SourceModifier, TargetModifier);

		ResolveTargetData(Context, BeamInst, BeamData, ParticleBase, Offset, i, false, TargetModifier);
	}
	END_UPDATE_LOOP;
}

#if WITH_EDITOR
void UParticleModuleBeamTarget::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	InitializeDefaults();

	UParticleSystem* PartSys = CastChecked<UParticleSystem>(GetOuter());
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PartSys && PropertyThatChanged)
	{
		PartSys->PostEditChangeProperty(PropertyChangedEvent);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UParticleModuleBeamTarget::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	check(IsInGameThread());
	bool	bFound	= false;

	for (int32 i = 0; i < PSysComp->InstanceParameters.Num(); i++)
	{
		FParticleSysParam* Param = &(PSysComp->InstanceParameters[i]);
		
		if (Param->Name == TargetName)
		{
			bFound	=	true;
			break;
		}
	}

	if (!bFound)
	{
		int32 NewParamIndex = PSysComp->InstanceParameters.AddZeroed();
		PSysComp->InstanceParameters[NewParamIndex].Name		= TargetName;
		PSysComp->InstanceParameters[NewParamIndex].ParamType	= PSPT_Actor;
		PSysComp->InstanceParameters[NewParamIndex].Actor		= NULL;
	}
}

void UParticleModuleBeamTarget::GetParticleSysParamsUtilized(TArray<FString>& ParticleSysParamList)
{
	if (TargetMethod == PEB2STM_Actor)
	{
		ParticleSysParamList.Add(FString::Printf(TEXT("BeamTarget : Actor: %s\n"), *(TargetName.ToString())));
	}
}

void UParticleModuleBeamTarget::GetDataPointers(FParticleEmitterInstance* Owner, const uint8* ParticleBase, 
	int32& CurrentOffset, FBeamParticleSourceTargetPayloadData*& ParticleSource)
{
	FParticleBeam2EmitterInstance* BeamInst = (FParticleBeam2EmitterInstance*)Owner;
	if (BeamInst && Owner->bIsBeam)
	{
		UParticleModuleTypeDataBeam2*	BeamTD	= BeamInst->BeamTypeData;
		if (BeamTD)
		{
			if (TargetMethod == PEB2STM_Particle)
			{
				PARTICLE_ELEMENT(FBeamParticleSourceTargetPayloadData, LocalParticleSource);
				ParticleSource	= &LocalParticleSource;
			}
		}
	}
}
						
bool UParticleModuleBeamTarget::ResolveTargetData(const FContext& Context, FParticleBeam2EmitterInstance* BeamInst, 
	FBeam2TypeDataPayload* BeamData, const uint8* ParticleBase, int32& CurrentOffset, int32	ParticleIndex, bool bSpawning, 
	FBeamParticleModifierPayloadData* ModifierData)
{
	bool	bResult	= false;

	FBaseParticle& Particle	= *((FBaseParticle*) ParticleBase);

	FBeamParticleSourceTargetPayloadData* ParticleSource	= NULL;
	GetDataPointers(BeamInst, ParticleBase, CurrentOffset, ParticleSource);

	if ((bSpawning == true) || (bLockTarget == false))
	{
		// Resolve the source point...
		bool bSetTarget = false;

		if (BeamInst->BeamTypeData->BeamMethod	== PEB2M_Distance)
		{
			// Set the particle target based on the distance
			float	Distance		= BeamInst->BeamTypeData->Distance.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			if (FMath::Abs(Distance) < UE_KINDA_SMALL_NUMBER)
			{
				Distance	= 0.001f;
			}
			FVector	Direction		= Context.GetTransform().GetScaledAxis( EAxis::X );
			Direction.Normalize();
			BeamData->TargetPoint	= BeamData->SourcePoint + Direction * Distance;
			bSetTarget				= true;
		}

		if (bSetTarget == false)
		{
			switch (TargetMethod)
			{
			case PEB2STM_UserSet:
				if (BeamInst->UserSetTargetArray.Num() > 0)
				{
					if (ParticleIndex >= BeamInst->UserSetTargetArray.Num())
					{
						//@todo. How to handle this situation???
						BeamData->TargetPoint	= BeamInst->UserSetTargetArray[0];
					}
					else
					{
						BeamData->TargetPoint	= BeamInst->UserSetTargetArray[ParticleIndex];
					}
					bSetTarget	= true;
				}
				break;
			case PEB2STM_Emitter:
				//@todo. Fill in this case...
				break;
			case PEB2STM_Particle:
				if (BeamInst->BeamTypeData->BeamMethod == PEB2M_Branch
					|| BeamInst->BeamTypeData->BeamMethod == PEB2M_Target)
				{
					// Branching beam - resolve the source emitter if needed
					if (BeamInst->TargetEmitter == NULL)
					{
						BeamInst->ResolveTarget();
					}

					if (BeamInst->TargetEmitter)
					{
						FVector LocalTargetPosition = FVector::ZeroVector;

						int32 TargetIndex = LastSelectedParticleIndex;

						if (BeamInst->TargetEmitter && BeamInst->TargetEmitter->ParticleIndices)
						{
							if (TargetIndex != -1)
							{
								FBaseParticle* TargetParticle = BeamInst->TargetEmitter->GetParticleDirect(TargetIndex);
								if (TargetParticle == NULL || TargetParticle->RelativeTime>1.0f)
								{
									// If the previous particle is not found, force the trail to pick a new one
									TargetIndex = -1;
								}
							}

							if (TargetIndex == -1)
							{
								int32 Index = 0;
								// TODO: add selection method and random selection
								/*
								switch (TargetModule->SelectionMethod)
								{
								case EPSSM_Random:
								{
								Index = FMath::TruncToInt(FMath::FRand() * BeamInst->TargetEmitter->ActiveParticles);
								}
								break;

								case EPSSM_Sequential:*/
								{
									if (++LastSelectedParticleIndex >= BeamInst->TargetEmitter->ActiveParticles)
									{
										LastSelectedParticleIndex = 0;
									}
									Index = LastSelectedParticleIndex;
								}
								/*
								break;
								}
								*/

								if (TargetIndex == BeamInst->TargetEmitter->ParticleIndices[Index])
								{
									Index = -1;
								}

								TargetIndex = (Index != -1) ? BeamInst->TargetEmitter->ParticleIndices[Index] : -1;
							}

							bool bEncounteredNaNError = false;

							// Grab the particle

							IParticleEmitterInstanceOwner& Comp = BeamInst->TargetEmitter->Component;
							FBaseParticle* TargetParticle = (TargetIndex >= 0) ? BeamInst->TargetEmitter->GetParticleDirect(TargetIndex) : nullptr;
							if (TargetParticle != nullptr)
							{
								const FVector WorldOrigin = BeamInst->TargetEmitter->SimulationToWorld.GetOrigin();

								if (!ensureMsgf(!TargetParticle->Location.ContainsNaN(), TEXT("NaN in TargetParticle Location. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName()) ||
									!ensureMsgf(!TargetParticle->OldLocation.ContainsNaN(), TEXT("NaN in TargetParticle OldLocation. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName()) ||
									!ensureMsgf(!WorldOrigin.ContainsNaN(), TEXT("NaN in WorldOrigin. Template: %s, Component: %s"), *GetNameSafe(Comp.GetTemplate()), *Comp.GetPathName())
									)
								{
									// Contains NaN!
									bEncounteredNaNError = true;
								}
								else
								{
									LocalTargetPosition = TargetParticle->Location + WorldOrigin;
								}
							}
							else
							{
								// Fall back to the emitter location??
								LocalTargetPosition = BeamInst->TargetEmitter->Component.GetComponentTransform().GetLocation();
								//@todo. How to handle this... can potentially cause a jump from the emitter to the
								// particle...
								TargetIndex = -1;//No valid particle Target;
							}

							//@todo. fill this in correctly...
							BeamData->TargetPoint = LocalTargetPosition; //BeamInst->TargetEmitter->Component->GetComponentLocation();
							bSetTarget = true;
						}
					}
				}
				break;
			case PEB2STM_Actor:
				if (TargetName != NAME_None)
				{
					BeamInst->ResolveTarget();
					if (BeamInst->TargetActor)
					{
						BeamData->TargetPoint	= BeamInst->TargetActor->ActorToWorld().GetLocation();
						bSetTarget = true;
					}
				}
				break;
			}
		}

		if (bSetTarget == false)
		{
			if (bTargetAbsolute)
			{
				BeamData->TargetPoint	= Target.GetValue(BeamInst->EmitterTime, Context.GetDistributionData());
				// Take into account current world origin offset
				BeamData->TargetPoint  -= (FVector)BeamInst->Component.GetWorldOriginLocation();
			}
			else
			{
				BeamData->TargetPoint	= Context.GetTransform().TransformPosition(
					Target.GetValue(BeamInst->EmitterTime, Context.GetDistributionData()));
			}
		}
	}

	if ((bSpawning == true) || (bLockTargetTangent == false))
	{
		// Resolve the Target tangent
		bool bSetTargetTangent = false;
		switch (TargetTangentMethod)
		{
		case PEB2STTM_Direct:
			BeamData->TargetTangent	= Context.GetTransform().GetScaledAxis( EAxis::X );
			bSetTargetTangent		= true;
			break;
		case PEB2STTM_UserSet:
			if (BeamInst->UserSetTargetTangentArray.Num() > 0)
			{
				if (ParticleIndex >= BeamInst->UserSetTargetTangentArray.Num())
				{
					//@todo. How to handle this situation???
					BeamData->TargetTangent	= BeamInst->UserSetTargetTangentArray[0];
				}
				else
				{
					BeamData->TargetTangent	= BeamInst->UserSetTargetTangentArray[ParticleIndex];
				}
				bSetTargetTangent	= true;
			}
			break;
		case PEB2STTM_Distribution:
			BeamData->TargetTangent	= TargetTangent.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			bSetTargetTangent		= true;
			break;
		case PEB2STTM_Emitter:
			BeamData->TargetTangent	= Context.GetTransform().GetScaledAxis( EAxis::X );
			bSetTargetTangent		= true;
			break;
		}

		if (bSetTargetTangent == false)
		{
//			BeamData->TargetTangent	= BeamInst->Component->LocalToWorld.GetScaledAxis( EAxis::X );
			BeamData->TargetTangent	= TargetTangent.GetValue(Particle.RelativeTime, Context.GetDistributionData());
			if (bTargetAbsolute == false)
			{
				BeamData->TargetTangent	= Context.GetTransform().TransformVector(BeamData->TargetTangent);
			}
		}
	}

	if ((bSpawning == true) || (bLockTargetStength == false))
	{
		// Resolve the Target strength	
		bool bSetTargetStrength = false;
		if (TargetTangentMethod == PEB2STTM_UserSet)
		{
			if (BeamInst->UserSetTargetStrengthArray.Num() > 0)
			{
				if (ParticleIndex >= BeamInst->UserSetTargetStrengthArray.Num())
				{
					//@todo. How to handle this situation???
					BeamData->TargetStrength	= BeamInst->UserSetTargetStrengthArray[0];
				}
				else
				{
					BeamData->TargetStrength	= BeamInst->UserSetTargetStrengthArray[ParticleIndex];
				}
				bSetTargetStrength	= true;
			}
		}

		if (!bSetTargetStrength)
		{
			BeamData->TargetStrength	= TargetStrength.GetValue(Particle.RelativeTime, Context.GetDistributionData());
		}
	}

	// For now, assume it worked...
	bResult	= true;

	return bResult;
}
