// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleBeamTrailVertexFactory.cpp: Particle vertex factory implementation.
=============================================================================*/

#include "ParticleBeamTrailVertexFactory.h"
#include "MeshDrawShaderBindings.h"
#include "ParticleHelper.h"
#include "MeshMaterialShader.h"
#include "Misc/DelayedAutoRegister.h"
#include "PipelineStateCache.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FParticleBeamTrailUniformParameters,"BeamTrailVF");

/**
 * Shader parameters for the beam/trail vertex factory.
 */
class FParticleBeamTrailVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FParticleBeamTrailVertexFactoryShaderParameters, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FParticleBeamTrailVertexFactory* BeamTrailVF = (FParticleBeamTrailVertexFactory*)VertexFactory;
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FParticleBeamTrailUniformParameters>(), BeamTrailVF->GetBeamTrailUniformBuffer() );
	}
};

IMPLEMENT_TYPE_LAYOUT(FParticleBeamTrailVertexFactoryShaderParameters);

///////////////////////////////////////////////////////////////////////////////
/**
 * The particle system beam trail vertex declaration resource type.
 */
class FParticleBeamTrailVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	FParticleBeamTrailVertexDeclaration(bool bInUsesDynamicParameter)
		: bUsesDynamicParameter(bInUsesDynamicParameter)
	{
	}
	virtual ~FParticleBeamTrailVertexDeclaration() {}

	virtual void FillDeclElements(FVertexDeclarationElementList& Elements, int32& Offset)
	{
		uint16 Stride = sizeof(FParticleBeamTrailVertex);
		/** The stream to read the vertex position from. */
		Elements.Add(FVertexElement(0, Offset, VET_Float4, 0, Stride));
		Offset += sizeof(float) * 4;
		/** The stream to read the vertex old position from. */
		Elements.Add(FVertexElement(0, Offset, VET_Float3, 1, Stride));
		Offset += sizeof(float) * 4;
		/** The stream to read the vertex size/rot/subimage from. */
		Elements.Add(FVertexElement(0, Offset, VET_Float4, 2, Stride));
		Offset += sizeof(float) * 4;
		/** The stream to read the color from.					*/
		Elements.Add(FVertexElement(0, Offset, VET_Float4, 4, Stride));
		Offset += sizeof(float) * 4;
		/** The stream to read the texture coordinates from.	*/
		Elements.Add(FVertexElement(0, Offset, VET_Float4, 3, Stride));
		Offset += sizeof(float) * 4;
		
		/** Dynamic parameters come from a second stream */
		Elements.Add(FVertexElement(1, 0, VET_Float4, 5, bUsesDynamicParameter ? sizeof(FVector4f) : 0));
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		FVertexDeclarationElementList Elements;
		int32	Offset = 0;
		FillDeclElements(Elements, Offset);

		// Create the vertex declaration for rendering the factory normally.
		// This is done in InitRHI instead of InitRHI to allow FParticleBeamTrailVertexFactory::InitRHI
		// to rely on it being initialized, since InitRHI is called before InitRHI.
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI()
	{
		VertexDeclarationRHI.SafeRelease();
	}

protected:
	bool bUsesDynamicParameter;
};

/** The simple element vertex declaration. */
static TGlobalResource<FParticleBeamTrailVertexDeclaration> GParticleBeamTrailVertexDeclaration(false);
static TGlobalResource<FParticleBeamTrailVertexDeclaration> GParticleBeamTrailVertexDeclarationDynamic(true);

///////////////////////////////////////////////////////////////////////////////

bool FParticleBeamTrailVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.bIsUsedWithBeamTrails || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

/**
 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
 */
void FParticleBeamTrailVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FParticleVertexFactoryBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("PARTICLE_BEAMTRAIL_FACTORY"),TEXT("1"));
}

/**
 * Get vertex elements used when during PSO precaching materials using this vertex factory type
 */
void FParticleBeamTrailVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	GParticleBeamTrailVertexDeclaration.VertexDeclarationRHI->GetInitializer(Elements);
}

FRHIVertexDeclaration* FParticleBeamTrailVertexFactory::GetPSOPrecacheVertexDeclaration(bool bUsesDynamicParameter)
{
	return (bUsesDynamicParameter ? GParticleBeamTrailVertexDeclarationDynamic.VertexDeclarationRHI : GParticleBeamTrailVertexDeclaration.VertexDeclarationRHI);
}

/**
 *	Initialize the Render Hardware Interface for this vertex factory
 */
void FParticleBeamTrailVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	SetDeclaration(bUsesDynamicParameter ? GParticleBeamTrailVertexDeclarationDynamic.VertexDeclarationRHI
		: GParticleBeamTrailVertexDeclaration.VertexDeclarationRHI);

	FVertexStream* VertexStream = new(Streams) FVertexStream;
	FVertexStream* DynamicParameterStream = new(Streams) FVertexStream;
}

void FParticleBeamTrailVertexFactory::SetVertexBuffer(const FVertexBuffer* InBuffer, uint32 StreamOffset, uint32 Stride)
{
	check(Streams.Num() == 2);
	FVertexStream& VertexStream = Streams[0];
	VertexStream.VertexBuffer = InBuffer;
	VertexStream.Stride = Stride;
	VertexStream.Offset = StreamOffset;
}

void FParticleBeamTrailVertexFactory::SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, uint32 StreamOffset, uint32 Stride)
{
	check(Streams.Num() == 2);
	FVertexStream& DynamicParameterStream = Streams[1];
	if (InDynamicParameterBuffer)
	{
		DynamicParameterStream.VertexBuffer = InDynamicParameterBuffer;
		ensure(bUsesDynamicParameter);
		DynamicParameterStream.Stride = Stride;
		DynamicParameterStream.Offset = StreamOffset;
	}
	else
	{
		DynamicParameterStream.VertexBuffer = &GNullDynamicParameterVertexBuffer;
		ensure(!bUsesDynamicParameter);
		DynamicParameterStream.Stride = 0;
		DynamicParameterStream.Offset = 0;
	}
}

///////////////////////////////////////////////////////////////////////////////

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FParticleBeamTrailVertexFactory, SF_Vertex, FParticleBeamTrailVertexFactoryShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FParticleBeamTrailVertexFactory,"/Engine/Private/ParticleBeamTrailVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPSOPrecaching
);
