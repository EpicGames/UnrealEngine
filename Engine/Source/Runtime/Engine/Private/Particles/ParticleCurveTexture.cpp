// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
ParticleCurveTexture.cpp: Texture used to hold particle curves.
==============================================================================*/

#include "Particles/ParticleCurveTexture.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderingThread.h"
#include "RHIBreadcrumbs.h"
#include "RHIStaticStates.h"
#include "ParticleHelper.h"
#include "ParticleResources.h"
#include "RHIContext.h"
#include "ShaderParameterUtils.h"
#include "GlobalShader.h"
#include "FXSystem.h"
#include "PipelineStateCache.h"
#include "ShaderParameterMacros.h"

/** The texture size allocated for particle curves. */
extern const int32 GParticleCurveTextureSizeX = 512;
extern const int32 GParticleCurveTextureSizeY = 512;

/** The texel allocator uses 16-bit integers internally. */
static_assert(GParticleCurveTextureSizeX <= 0xffff, "Curve texture wider than sixteen bits.");
static_assert(GParticleCurveTextureSizeY <= 0xffff, "Curve texture taller than sixteen bits.");

/** The global curve texture resource. */
TGlobalResource<FParticleCurveTexture> GParticleCurveTexture;

/*-----------------------------------------------------------------------------
Shaders used for uploading curves to the GPU.
-----------------------------------------------------------------------------*/

/**
* Uniform buffer to hold parameters for particle curve injection.
*/
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FParticleCurveInjectionParameters, )
SHADER_PARAMETER(FVector2f, PixelScale)
SHADER_PARAMETER(FVector2f, CurveOffset)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FParticleCurveInjectionParameters, "ParticleCurveInjection");

typedef TUniformBufferRef<FParticleCurveInjectionParameters> FParticleCurveInjectionBufferRef;

/**
* Vertex shader for simulating particles on the GPU.
*/
class FParticleCurveInjectionVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FParticleCurveInjectionVS, Global);

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return SupportsGPUParticles(Parameters.Platform);
	}

	/** Default constructor. */
	FParticleCurveInjectionVS()
	{
	}

	/** Initialization constructor. */
	explicit FParticleCurveInjectionVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	/**
	* Sets parameters for particle injection.
	*/
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FVector2D& CurveOffset)
	{
		FParticleCurveInjectionParameters Parameters;
		Parameters.PixelScale.X = 1.0f / GParticleCurveTextureSizeX;
		Parameters.PixelScale.Y = 1.0f / GParticleCurveTextureSizeY;
		Parameters.CurveOffset = FVector2f(CurveOffset);
		FParticleCurveInjectionBufferRef UniformBuffer = FParticleCurveInjectionBufferRef::CreateUniformBufferImmediate(Parameters, UniformBuffer_SingleDraw);
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FParticleCurveInjectionParameters>(), UniformBuffer);
	}
};

/**
* Pixel shader for simulating particles on the GPU.
*/
class FParticleCurveInjectionPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FParticleCurveInjectionPS, Global);

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return SupportsGPUParticles(Parameters.Platform);
	}

	/** Default constructor. */
	FParticleCurveInjectionPS()
	{
	}

	/** Initialization constructor. */
	explicit FParticleCurveInjectionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
};

/** Implementation for all shaders used for particle injection. */
IMPLEMENT_SHADER_TYPE(, FParticleCurveInjectionVS, TEXT("/Engine/Private/ParticleCurveInjectionShader.usf"), TEXT("VertexMain"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(, FParticleCurveInjectionPS, TEXT("/Engine/Private/ParticleCurveInjectionShader.usf"), TEXT("PixelMain"), SF_Pixel);

/**
* Vertex declaration for injecting curves.
*/
class FParticleCurveInjectionVertexDeclaration : public FRenderResource
{
public:

	/** The vertex declaration. */
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;

		// Stream 0.
		{
			int32 Offset = 0;
			// Sample.
			Elements.Add(FVertexElement(0, Offset, VET_Color, 0, sizeof(FColor), /*bUseInstanceIndex=*/ true));
			Offset += sizeof(FColor);
		}

		// Stream 1.
		{
			int32 Offset = 0;
			// TexCoord.
			Elements.Add(FVertexElement(1, Offset, VET_Float2, 1, sizeof(FVector2f), /*bUseInstanceIndex=*/ false));
			Offset += sizeof(FVector2f);
		}

		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

/** The global particle injection vertex declaration. */
TGlobalResource<FParticleCurveInjectionVertexDeclaration> GParticleCurveInjectionVertexDeclaration;

/**
* Transfers a list of curves to a texture on the GPU. All main memory allocated
* for curve samples is released.
* @param CurveTextureRHI - The texture holding the curve data, sampled by the particle simulation shader
* @param CurveTextureTargetRHI - The render target for the curve texture.
* @param InPendingCurves - Curves to be stored on the GPU.
*/
static void InjectCurves(
	FRHICommandListImmediate& RHICmdList,
	FRHITexture* CurveTextureRHI,
	TArray<FCurveSamples>& InPendingCurves)
{
	static bool bFirstCall = true;

	check(IsInRenderingThread());

	SCOPED_DRAW_EVENT(RHICmdList, InjectParticleCurves);

	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ELoad;

	if (bFirstCall)
	{
		LoadAction = ERenderTargetLoadAction::EClear;
		bFirstCall = false;
	}

	FRHIRenderPassInfo RPInfo(CurveTextureRHI, MakeRenderTargetActions(LoadAction, ERenderTargetStoreAction::EStore));
	RHICmdList.Transition(FRHITransitionInfo(CurveTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
	RHICmdList.BeginRenderPass(RPInfo, TEXT("InjectCurves"));
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
		RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)GParticleCurveTextureSizeX, (float)GParticleCurveTextureSizeY, 1.0f);
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

		// Grab shaders.
		TShaderMapRef<FParticleCurveInjectionVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		TShaderMapRef<FParticleCurveInjectionPS> PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GParticleCurveInjectionVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		int32 PendingCurveCount = InPendingCurves.Num();

		// get sum of size of all curve textures
		//
		int32 TotalSamples = 0;
		for (int32 CurveIndex = 0; CurveIndex < PendingCurveCount; ++CurveIndex)
		{
			FCurveSamples& Curve = InPendingCurves[CurveIndex];
			check(Curve.Samples);
			TotalSamples += Curve.TexelAllocation.Size;
		}


		// get a buffer for all curve textures at once, and copy curve data over
		//
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FColor>(TEXT("ScratchVertexBuffer"), TotalSamples)
			.AddUsage(EBufferUsageFlags::Volatile)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FColor> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

		int32 CurrOffset = 0;

		for (int32 CurveIndex = 0; CurveIndex < PendingCurveCount; ++CurveIndex)
		{
			FCurveSamples& Curve = InPendingCurves[CurveIndex];

			// Copy curve samples in to the scratch vertex buffer.
			const int32 SampleCount = Curve.TexelAllocation.Size;
			Initializer.WriteArray(CurrOffset, MakeConstArrayView(Curve.Samples, SampleCount));
			FMemory::Free(Curve.Samples);
			Curve.Samples = NULL;

			CurrOffset += SampleCount;
		}

		FBufferRHIRef ScratchVertexBufferRHI = Initializer.Finalize();

		// now draw the curves into the curve texture target, reading from the single buffer we just filled
		//
		int32 CurrSrcOffset = 0;
		for (int32 CurveIndex = 0; CurveIndex < PendingCurveCount; ++CurveIndex)
		{
			FCurveSamples& Curve = InPendingCurves[CurveIndex];
			const int32 SampleCount = Curve.TexelAllocation.Size;

			// Compute the offset in to the texture.
			FVector2D CurveOffset((float)Curve.TexelAllocation.X / GParticleCurveTextureSizeX,
				(float)Curve.TexelAllocation.Y / GParticleCurveTextureSizeY);

			// Stream 0: New particles.
			RHICmdList.SetStreamSource(0, ScratchVertexBufferRHI, CurrSrcOffset * sizeof(FColor));
			// Stream 0: TexCoord.
			RHICmdList.SetStreamSource(1, GParticleTexCoordVertexBuffer.VertexBufferRHI, 0);

			SetShaderParametersLegacyVS(RHICmdList, VertexShader, CurveOffset);

			// Inject particles.
			RHICmdList.DrawIndexedPrimitive(
				GParticleIndexBuffer.IndexBufferRHI,
				/*BaseVertexIndex=*/ 0,
				/*MinIndex=*/ 0,
				/*NumVertices=*/ 4,
				/*StartIndex=*/ 0,
				/*NumPrimitives=*/ 2,
				/*NumInstances=*/ SampleCount
			);
			CurrSrcOffset += SampleCount;
		}
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(CurveTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
}

/*------------------------------------------------------------------------------
Texel allocator.
------------------------------------------------------------------------------*/

/** Constructor. */
FTexelAllocator::FTexelAllocator(const int32 InTextureSizeX, const int32 InTextureSizeY)
	: BlockPool(NULL)
	, BlockCount(0)
	, TextureSizeX(InTextureSizeX)
	, TextureSizeY(InTextureSizeY)
	, FreeTexels(InTextureSizeX * InTextureSizeY)
{
	// Allocate one block per row.
	FreeBlocks.Empty(TextureSizeY);
	FreeBlocks.AddZeroed(TextureSizeY);
	for (int32 RowIndex = 0; RowIndex < TextureSizeY; ++RowIndex)
	{
		// Setup the block.
		FBlock* NewBlock = GetBlock();
		NewBlock->Next = NULL;
		NewBlock->Begin = 0;
		NewBlock->Size = TextureSizeX;
		FreeBlocks[RowIndex] = NewBlock;
	}
}

/** Destructor. */
FTexelAllocator::~FTexelAllocator()
{
	// Free all blocks.
	for (int32 RowIndex = 0; RowIndex < TextureSizeY; ++RowIndex)
	{
		FBlock* Block = FreeBlocks[RowIndex];
		while (Block)
		{
			FBlock* ThisBlock = Block;
			Block = Block->Next;
			FMemory::Free(ThisBlock);
		}
		FreeBlocks[RowIndex] = NULL;
	}

	// Free any blocks in the block pool.
	FBlock* Block = BlockPool;
	while (Block)
	{
		FBlock* ThisBlock = Block;
		Block = Block->Next;
		FMemory::Free(ThisBlock);
	}
	BlockPool = NULL;
}

/**
* Allocates the requested number of texels.
* @param Size - The number of texels to allocate.
* @returns the texel allocation.
*/
FTexelAllocation FTexelAllocator::Allocate(int32 Size)
{
	check(Size > 0);
	check(Size <= TextureSizeX);

	for (int32 RowIndex = 0; RowIndex < TextureSizeY; ++RowIndex)
	{
		// Note that the Next pointer in FBlock will alias the pointer itself.
		FBlock* LastBlock = (FBlock*)&(FreeBlocks[RowIndex]);
		FBlock* Block = FreeBlocks[RowIndex];
		check(LastBlock->Next == Block);

		// Look through the list for a free block of the correct size.
		while (Block)
		{
			if (Block->Size > Size)
			{
				FTexelAllocation Allocation;
				Allocation.X = Block->Begin;
				Allocation.Y = RowIndex;
				Allocation.Size = Size;

				Block->Begin += Size;
				Block->Size -= Size;
				FreeTexels -= Size;

				return Allocation;
			}
			else if (Block->Size == Size)
			{
				FTexelAllocation Allocation;
				Allocation.X = Block->Begin;
				Allocation.Y = RowIndex;
				Allocation.Size = Size;
				FreeTexels -= Size;

				// Unlink the block and return it to the pool.
				LastBlock->Next = Block->Next;
				ReturnBlock(Block);

				return Allocation;
			}
			else
			{
				LastBlock = Block;
				Block = Block->Next;
			}
		}
	}

	// No space remaining!
	FTexelAllocation NullAllocation;
	NullAllocation.X = TextureSizeX;
	NullAllocation.Y = TextureSizeY;
	NullAllocation.Size = 0;
	return NullAllocation;
}

/**
* Frees texels that were previously allocated.
* @param Allocation - The texel allocation to free.
*/
void FTexelAllocator::Free(FTexelAllocation Allocation)
{
	check(Allocation.Size > 0);
	check(Allocation.X < TextureSizeX);
	check(Allocation.Y < TextureSizeY);

	// Note that the Next pointer in FBlock will alias the pointer itself.
	FBlock* LastBlock = (FBlock*)&(FreeBlocks[Allocation.Y]);
	FBlock* Block = FreeBlocks[Allocation.Y];
	check(LastBlock->Next == Block);

	FreeTexels += Allocation.Size;

	if (Block == NULL)
	{
		// If there are no blocks add a free block.
		Block = GetBlock();
		Block->Next = NULL;
		Block->Begin = Allocation.X;
		Block->Size = Allocation.Size;
		LastBlock->Next = Block;
	}
	else
	{
		// Search for where the allocation fits in the list.
		const uint16 AllocX = Allocation.X;
		const uint16 AllocSize = Allocation.Size;
		while (Block && AllocX > Block->Begin + Block->Size)
		{
			LastBlock = Block;
			Block = Block->Next;
		}

		// Add the allocation back to the free list.
		if (Block && Block->Begin == AllocX + AllocSize)
		{
			// Free space is directly in front of the block.
			Block->Begin = AllocX;
			Block->Size += AllocSize;
		}
		else if (Block && Block->Begin + Block->Size == AllocX)
		{
			// Free space is directly after the block.
			Block->Size += AllocSize;

			// Continue walking the list to coalesce connected blocks.
			LastBlock = Block;
			Block = Block->Next;
			while (Block && Block->Begin == LastBlock->Begin + LastBlock->Size)
			{
				FBlock* ThisBlock = Block;
				LastBlock->Size += Block->Size;
				LastBlock->Next = Block->Next;
				Block = Block->Next;
				ReturnBlock(ThisBlock);
			}
		}
		else
		{
			FBlock* NewBlock = GetBlock();
			NewBlock->Next = Block;
			NewBlock->Begin = AllocX;
			NewBlock->Size = AllocSize;
			LastBlock->Next = NewBlock;
		}
	}
}

/**
* Retrieves a new block from the pool.
*/
FTexelAllocator::FBlock* FTexelAllocator::GetBlock()
{
	FTexelAllocator::FBlock* Block = BlockPool;
	if (Block)
	{
		BlockPool = Block->Next;
		return Block;
	}
	BlockCount++;
	Block = (FTexelAllocator::FBlock*)FMemory::Malloc(sizeof(FTexelAllocator::FBlock));
	return Block;
}

/**
* Returns a block to the pool.
*/
void FTexelAllocator::ReturnBlock(FTexelAllocator::FBlock* Block)
{
	Block->Next = BlockPool;
	BlockPool = Block;
}

/*-----------------------------------------------------------------------------
A texture for storing curve samples on the GPU.
-----------------------------------------------------------------------------*/

/** Default constructor. */
FParticleCurveTexture::FParticleCurveTexture()
	: TexelAllocator(GParticleCurveTextureSizeX, GParticleCurveTextureSizeY)
{
}

/**
* Initialize RHI resources for the curve texture.
*/
void FParticleCurveTexture::InitRHI(FRHICommandListBase& RHICmdList)
{
	const static FLazyName ClassName(TEXT("FParticleCurveTexture"));

	// 8-bit per channel RGBA texture for curves.
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("ParticleCurveTexture"))
		.SetExtent(GParticleCurveTextureSizeX, GParticleCurveTextureSizeY)
		.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::NoFastClear)
		.SetFormat(PF_B8G8R8A8)
		.SetClearValue(FClearValueBinding(FLinearColor::Blue))
		.SetInitialState(ERHIAccess::SRVMask)
		.SetClassName(ClassName);

	CurveTextureRHI = RHICmdList.CreateTexture(Desc);
}

/**
* Releases RHI resources.
*/
void FParticleCurveTexture::ReleaseRHI()
{
	CurveTextureRHI.SafeRelease();
}

/**
* Adds a curve to the texture.
* @param CurveSamples - Samples in the curve.
* @returns The texel allocation in the curve texture.
*/
FTexelAllocation FParticleCurveTexture::AddCurve(const TArray<FColor>& CurveSamples)
{
	check(IsInGameThread());
	check(CurveSamples.Num() <= GParticleCurveTextureSizeX);

	if (FApp::CanEverRender())
	{
		if (CurveSamples.Num())
		{
			FTexelAllocation TexelAllocation = TexelAllocator.Allocate(CurveSamples.Num());
			if (TexelAllocation.Size > 0)
			{
				check(TexelAllocation.Size == CurveSamples.Num());
				FCurveSamples& PendingCurve = PendingCurves.AddDefaulted_GetRef();
				PendingCurve.TexelAllocation = TexelAllocation;
				PendingCurve.Samples = (FColor*)FMemory::Malloc(TexelAllocation.Size * sizeof(FColor));
				FMemory::Memcpy(PendingCurve.Samples, CurveSamples.GetData(), TexelAllocation.Size * sizeof(FColor));
				return TexelAllocation;
			}
			UE_LOGF(LogParticles, Warning, "FParticleCurveTexture: Failed to allocate %d texels for a curve (may need to increase the size of GParticleCurveTextureSizeX or GParticleCurveTextureSizeY).", CurveSamples.Num());

			return TexelAllocation;
		}
	}

	return FTexelAllocation();
}

/**
* Frees an area in the texture associated with a curve.
* @param TexelAllocation - Frees a texel region allowing other curves to be placed there.
*/
void FParticleCurveTexture::RemoveCurve(FTexelAllocation TexelAllocation)
{
	check(IsInGameThread());
	if (TexelAllocation.Size > 0)
	{
		TexelAllocator.Free(TexelAllocation);
	}
}

/**
* Computes scale and bias to apply in order to sample the curve. The value
* should be used as TexCoord.xy = Curve.xy + Curve.zw * t.
* @param TexelAllocation - The texel allocation in the texture.
* @returns the scale and bias needed to sample the curve.
*/
FVector4f FParticleCurveTexture::ComputeCurveScaleBias(FTexelAllocation TexelAllocation)
{
	return FVector4f(
		((float)TexelAllocation.X + 0.5f) / (float)GParticleCurveTextureSizeX,
		((float)TexelAllocation.Y + 0.5f) / (float)GParticleCurveTextureSizeY,
		(float)(TexelAllocation.Size - 1) / (float)GParticleCurveTextureSizeX,
		TexelAllocation.Size > 0 ? 0.0f : 1.0f
	);
}

/**
* Submits pending curves to the GPU.
*/
void FParticleCurveTexture::SubmitPendingCurves()
{
	check(IsInGameThread());
	if (PendingCurves.Num())
	{
		FParticleCurveTexture* ParticleCurveTexture = this;
		//TArray<FCurveSamples> InPendingCurves = PendingCurves;
		ENQUEUE_RENDER_COMMAND(FInjectPendingCurvesCommand)(
			[ParticleCurveTexture, PendingCurves = PendingCurves](FRHICommandListImmediate& RHICmdList) mutable
			{
				InjectCurves(
					RHICmdList,
					ParticleCurveTexture->CurveTextureRHI,
					PendingCurves
				);
			});

		PendingCurves.Reset();
	}
}
