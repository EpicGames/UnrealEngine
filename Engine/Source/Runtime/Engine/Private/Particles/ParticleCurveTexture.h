// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleCurveTexture.h: Texture used to hold particle curves.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"

/** The texture size allocated for particle curves. */
extern const int32 GParticleCurveTextureSizeX;
extern const int32 GParticleCurveTextureSizeY;

/*------------------------------------------------------------------------------
	Texel allocator.
------------------------------------------------------------------------------*/

/**
 * An allocation of texels. The allocation starts at texel (X,Y) and is Size
 * texels wide.
 */
struct FTexelAllocation
{
	uint16 X;
	uint16 Y;
	uint16 Size;

	/** Default constructor. */
	FTexelAllocation()
		: X(0)
		, Y(0)
		, Size(0)
	{
	}
};

/**
 * A free-list based allocator for allocating rows of texels from a texture.
 */
class FTexelAllocator
{
public:

	/** Constructor. */
	FTexelAllocator( const int32 InTextureSizeX, const int32 InTextureSizeY );

	/** Destructor. */
	~FTexelAllocator();

	/**
	 * Allocates the requested number of texels.
	 * @param Size - The number of texels to allocate.
	 * @returns the texel allocation.
	 */
	FTexelAllocation Allocate( int32 Size );

	/**
	 * Frees texels that were previously allocated.
	 * @param Allocation - The texel allocation to free.
	 */
	void Free( FTexelAllocation Allocation );

private:

	/** A block of free texels. */
	struct FBlock
	{
		/** The next block in the list. */
		FBlock* Next;
		/** The texel at which the block begins. */
		uint16 Begin;
		/** How many texels are in this block. */
		uint16 Size;
	};

	/**
	 * Retrieves a new block from the pool.
	 */
	FBlock* GetBlock();

	/**
	 * Returns a block to the pool.
	 */
	void ReturnBlock( FBlock* Block );

	/** Lists of free blocks of texels. One list per row in the texture. */
	TArray<FBlock*> FreeBlocks;
	/** Pool of blocks. */
	FBlock* BlockPool;
	/** The number of blocks that have been allocated. */
	int32 BlockCount;
	/** The width of the texture. */
	int32 TextureSizeX;
	/** The height of the texture. */
	int32 TextureSizeY;
	/** The number of free texels in the texture. */
	int32 FreeTexels;
};

/*-----------------------------------------------------------------------------
	A texture for storing curve samples on the GPU.
-----------------------------------------------------------------------------*/

/**
 * Curve samples to be placed in to the texture.
 */
struct FCurveSamples
{
	/** Samples along the curve. */
	FColor* Samples;
	/** Where to store the curve in the texture. */
	FTexelAllocation TexelAllocation;
};

/**
 * A texture in which to store curve samples.
 */
class FParticleCurveTexture : public FRenderResource
{
public:

	/** Default constructor. */
	FParticleCurveTexture();

	/**
	 * Initialize RHI resources for the curve texture.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Releases RHI resources.
	 */
	virtual void ReleaseRHI() override;

	/**
	 * Adds a curve to the texture.
	 * @param CurveSamples - Samples in the curve.
	 * @returns The texel allocation in the curve texture.
	 */
	FTexelAllocation AddCurve( const TArray<FColor>& CurveSamples );

	/**
	 * Frees an area in the texture associated with a curve.
	 * @param TexelAllocation - Frees a texel region allowing other curves to be placed there.
	 */
	void RemoveCurve( FTexelAllocation TexelAllocation );

	/**
	 * Computes scale and bias to apply in order to sample the curve. The value
	 * should be used as TexCoord.xy = Curve.xy + Curve.zw * t.
	 * @param TexelAllocation - The texel allocation in the texture.
	 * @returns the scale and bias needed to sample the curve.
	 */
	FVector4f ComputeCurveScaleBias( FTexelAllocation TexelAllocation );

	/**
	 * Retrieves the curve texture from which shaders can sample.
	 */
	FRHITexture* GetCurveTexture() { return CurveTextureRHI; }

	/**
	 * Submits pending curves to the GPU.
	 */
	void SubmitPendingCurves();

private:
	/** Texture for sampling curves on the GPU. */
	FTextureRHIRef CurveTextureRHI;
	/** The texel allocator for this texture. */
	FTexelAllocator TexelAllocator;
	/** A list of pending curves that need to be uploaded. */
	TArray<FCurveSamples> PendingCurves;
};

/** The global curve texture resource. */
extern TGlobalResource<FParticleCurveTexture> GParticleCurveTexture;
