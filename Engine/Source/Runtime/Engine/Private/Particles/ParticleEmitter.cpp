// Copyright Epic Games, Inc. All Rights Reserved.

#include "Particles/ParticleEmitter.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Engine/Engine.h"
#include "Engine/InterpCurveEdSetup.h"
#include "ParticleEmitterInstanceOwner.h"
#include "Particles/Camera/ParticleModuleCameraOffset.h"
#include "Particles/Color/ParticleModuleColorOverLife.h"
#include "Particles/Lifetime/ParticleModuleLifetime.h"
#include "Particles/Light/ParticleModuleLight.h"
#include "Particles/Location/ParticleModuleLocationBoneSocket.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/Modules/Location/ParticleModulePivotOffset.h"
#include "Particles/Parameter/ParticleModuleParameterDynamic.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "Particles/Size/ParticleModuleSize.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/SubUV/ParticleModuleSubUV.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataBeam2.h"
#include "Particles/Velocity/ParticleModuleVelocity.h"
#include "Scalability.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleEmitter)

////////////////////////////////////////////////////////////////////////////////////////////////////

static TAutoConsoleVariable<float> CVarQLSpawnRateReferenceLevel(
	TEXT("fx.QualityLevelSpawnRateScaleReferenceLevel"),
	2,
	TEXT("Controls the reference level for quality level based spawn rate scaling. This is the FX quality level\n")
	TEXT("at which spawn rate is not scaled down; Spawn rate scaling will happen by each emitter's\n")
	TEXT("QualityLevelSpawnRateScale value for each reduction in level below the reference level.\n")
	TEXT("\n")
	TEXT("Default = 2. Value should range from 0 to the maximum FX quality level."),
	ECVF_Scalability);

////////////////////////////////////////////////////////////////////////////////////////////////////

UParticleEmitter::UParticleEmitter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SignificanceLevel(EParticleSignificanceLevel::Critical)
	, bUseLegacySpawningBehavior(false)
	, bDisabledLODsKeepEmitterAlive(false)
	, bDisableWhenInsignficant(0)
	, QualityLevelSpawnRateScale(1.0f)
	, DetailModeBitmask(PDM_DefaultValue)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Particle_Emitter;
		FConstructorStatics()
			: NAME_Particle_Emitter(TEXT("Particle Emitter"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	EmitterName = ConstructorStatics.NAME_Particle_Emitter;
	ConvertedModules = true;
	PeakActiveParticles = 0;
#if WITH_EDITORONLY_DATA
	EmitterEditorColor = FColor(0, 150, 150, 255);
#endif // WITH_EDITORONLY_DATA

}

FParticleEmitterInstance* UParticleEmitter::CreateInstance(IParticleEmitterInstanceOwner& InComponent)
{
	UE_LOGF(LogParticles, Fatal,"UParticleEmitter::CreateInstance is pure virtual"); 
	return NULL; 
}

void UParticleEmitter::UpdateModuleLists()
{
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel)
		{
			LODLevel->UpdateModuleLists();
		}
	}
	Build();
}

/**
 *	Helper function for fixing up LODValidity issues on particle modules...
 *
 *	@param	LODIndex		The index of the LODLevel the module is from.
 *	@param	ModuleIndex		The index of the module being checked.
 *	@param	Emitter			The emitter owner.
 *	@param	CurrModule		The module being checked.
 *
 *	@return	 0		If there was no problem.
 *			 1		If there was a problem and it was fixed.
 *			-1		If there was a problem that couldn't be fixed.
 */
int32 ParticleEmitterHelper_FixupModuleLODErrors( int32 LODIndex, int32 ModuleIndex, 
	const UParticleEmitter* Emitter, UParticleModule* CurrModule )
{
	int32 Result = 1;
	bool bIsDirty = false;

	UObject* ModuleOuter = CurrModule->GetOuter();
	UObject* EmitterOuter = Emitter->GetOuter();
	if (ModuleOuter != EmitterOuter)
	{
		// Module has an incorrect outer
		CurrModule->Rename(NULL, EmitterOuter, REN_DoNotDirty);
		bIsDirty = true;
	}

	if (CurrModule->LODValidity == 0)
	{
		// Immediately tag it for this lod level...
		CurrModule->LODValidity = (1 << LODIndex);
		bIsDirty = true;
	}
	else
	if (CurrModule->IsUsedInLODLevel(LODIndex) == false)
	{
		// Why was this even called here?? 
		// The assumption is that it should be called for the module in the given lod level...
		// So, we will tag it with this index.
		CurrModule->LODValidity |= (1 << LODIndex);
		bIsDirty = true;
	}

	if (LODIndex > 0)
	{
		int32 CheckIndex = LODIndex - 1;
		while (CheckIndex >= 0)
		{
			if (CurrModule->IsUsedInLODLevel(CheckIndex))
			{
				// Ensure that it is the same as the one it THINKS it is shared with...
				UParticleLODLevel* CheckLODLevel = Emitter->LODLevels[CheckIndex];

				if (CurrModule->IsA(UParticleModuleSpawn::StaticClass()))
				{
					if (CheckLODLevel->SpawnModule != CurrModule)
					{
						// Fix it up... Turn off the higher LOD flag
						CurrModule->LODValidity &= ~(1 << CheckIndex);
						bIsDirty = true;
					}
				}
				else
				if (CurrModule->IsA(UParticleModuleRequired::StaticClass()))
				{
					if (CheckLODLevel->RequiredModule != CurrModule)
					{
						// Fix it up... Turn off the higher LOD flag
						CurrModule->LODValidity &= ~(1 << CheckIndex);
						bIsDirty = true;
					}
				}
				else
				if (CurrModule->IsA(UParticleModuleTypeDataBase::StaticClass()))
				{
					if (CheckLODLevel->TypeDataModule != CurrModule)
					{
						// Fix it up... Turn off the higher LOD flag
						CurrModule->LODValidity &= ~(1 << CheckIndex);
						bIsDirty = true;
					}
				}
				else
				{
					if (ModuleIndex >= CheckLODLevel->Modules.Num())
					{
						UE_LOGF(LogParticles, Warning, "\t\tMismatched module count at %2d in %ls", LODIndex, *(Emitter->GetPathName()));
						Result = -1;
					}
					else
					{
						UParticleModule* CheckModule = CheckLODLevel->Modules[ModuleIndex];
						if (CheckModule != CurrModule)
						{
							// Fix it up... Turn off the higher LOD flag
							CurrModule->LODValidity &= ~(1 << CheckIndex);
							bIsDirty = true;
						}
					}
				}
			}

			CheckIndex--;
		}
	}

	if ((bIsDirty == true) && IsRunningCommandlet())
	{
		CurrModule->MarkPackageDirty();
		Emitter->MarkPackageDirty();
	}

	return Result;
}

bool UParticleEmitter::IsPostLoadThreadSafe() const
{
	return false;
}

void UParticleEmitter::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FParticleSystemCustomVersion::GUID);
}

void UParticleEmitter::PostLoad()
{
	Super::PostLoad();

	const int32 PSysVer = GetLinkerCustomVersion(FParticleSystemCustomVersion::GUID);
	if (PSysVer < FParticleSystemCustomVersion::FixLegacySpawningBugs)
	{
		bUseLegacySpawningBehavior = true;
	}

	if (PSysVer < FParticleSystemCustomVersion::AddEpicDetailMode)
	{
		// Init epic detail mode to enabled if high is set
		if (DetailModeBitmask & (1 << EParticleDetailMode::PDM_High))
		{
			DetailModeBitmask |= (1 << EParticleDetailMode::PDM_Epic);
		}
	}

	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel)
		{
			LODLevel->ConditionalPostLoad();

			FLinkerLoad* LODLevelLinker = LODLevel->GetLinker();
			if (LODLevel->SpawnModule == NULL)
			{
				// Force the conversion to SpawnModule
				UParticleSystem* PSys = Cast<UParticleSystem>(GetOuter());
				if (PSys)
				{
					UE_LOGF(LogParticles, Warning, "LODLevel %d was not converted to spawn module - forcing: %ls", 
						LODLevel->Level, *(PSys->GetPathName()));
				}
				LODLevel->ConvertToSpawnModule();
			}

			check(LODLevel->SpawnModule);

		}
	}

#if WITH_EDITORONLY_DATA
	UpdateDetailModeDisplayString();
#endif

#if WITH_EDITOR
	if ((GIsEditor == true) && 1)//(IsRunningCommandlet() == false))
	{
		ConvertedModules = false;
		PeakActiveParticles = 0;

		// Check for improper outers...
		UObject* EmitterOuter = GetOuter();
		bool bWarned = false;
		for (int32 LODIndex = 0; (LODIndex < LODLevels.Num()) && !bWarned; LODIndex++)
		{
			UParticleLODLevel* LODLevel = LODLevels[LODIndex];
			if (LODLevel)
			{
				LODLevel->ConditionalPostLoad();

				UParticleModule* Module = LODLevel->TypeDataModule;
				if (Module)
				{
					Module->ConditionalPostLoad();

					UObject* OuterObj = Module->GetOuter();
					check(OuterObj);
					if (OuterObj != EmitterOuter)
					{
						UE_LOGF(LogParticles, Warning, "UParticleModule %ls has an incorrect outer on %ls... run FixupEmitters on package %ls (%ls)",
							*(Module->GetPathName()), 
							*(EmitterOuter->GetPathName()),
							*(OuterObj->GetOutermost()->GetPathName()),
							*(GetOutermost()->GetPathName()));
						UE_LOGF(LogParticles, Warning, "\tModule Outer..............%ls", *(OuterObj->GetPathName()));
						UE_LOGF(LogParticles, Warning, "\tModule Outermost..........%ls", *(Module->GetOutermost()->GetPathName()));
						UE_LOGF(LogParticles, Warning, "\tEmitter Outer.............%ls", *(EmitterOuter->GetPathName()));
						UE_LOGF(LogParticles, Warning, "\tEmitter Outermost.........%ls", *(GetOutermost()->GetPathName()));
						bWarned = true;
					}
				}

				if (!bWarned)
				{
					for (int32 ModuleIndex = 0; (ModuleIndex < LODLevel->Modules.Num()) && !bWarned; ModuleIndex++)
					{
						Module = LODLevel->Modules[ModuleIndex];
						if (Module)
						{
							Module->ConditionalPostLoad();

							UObject* OuterObj = Module->GetOuter();
							check(OuterObj);
							if (OuterObj != EmitterOuter)
							{
								UE_LOGF(LogParticles, Warning, "UParticleModule %ls has an incorrect outer on %ls... run FixupEmitters on package %ls (%ls)",
									*(Module->GetPathName()), 
									*(EmitterOuter->GetPathName()),
									*(OuterObj->GetOutermost()->GetPathName()),
									*(GetOutermost()->GetPathName()));
								UE_LOGF(LogParticles, Warning, "\tModule Outer..............%ls", *(OuterObj->GetPathName()));
								UE_LOGF(LogParticles, Warning, "\tModule Outermost..........%ls", *(Module->GetOutermost()->GetPathName()));
								UE_LOGF(LogParticles, Warning, "\tEmitter Outer.............%ls", *(EmitterOuter->GetPathName()));
								UE_LOGF(LogParticles, Warning, "\tEmitter Outermost.........%ls", *(GetOutermost()->GetPathName()));
								bWarned = true;
							}
						}
					}
				}
			}
		}
	}
	else
#endif
	{
		for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
		{
			UParticleLODLevel* LODLevel = LODLevels[LODIndex];
			if (LODLevel)
			{
				LODLevel->ConditionalPostLoad();
			}
		}
	}
   

	ConvertedModules = true;

	// this will look at all of the emitters and then remove ones that some how have become NULL (e.g. from a removal of an Emitter where content
	// is still referencing it)
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel)
		{
			for (int32 ModuleIndex = LODLevel->Modules.Num()-1; ModuleIndex >= 0; ModuleIndex--)
			{
				UParticleModule* ParticleModule = LODLevel->Modules[ModuleIndex];
				if( ParticleModule == NULL )
				{
					LODLevel->Modules.RemoveAt(ModuleIndex);
					MarkPackageDirty();
				}
			}
		}
	}


	UObject* MyOuter = GetOuter();
	UParticleSystem* PSysOuter = Cast<UParticleSystem>(MyOuter);
	bool bRegenDup = false;
	if (PSysOuter)
	{
		bRegenDup = PSysOuter->bRegenerateLODDuplicate;
	}

	// Clamp the detail spawn rate scale...
	QualityLevelSpawnRateScale = FMath::Clamp<float>(QualityLevelSpawnRateScale, 0.0f, 1.0f);

	UpdateModuleLists();
}

#if WITH_EDITOR
void UParticleEmitter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	check(GIsEditor);

	// Reset the peak active particle counts.
	// This could check for changes to SpawnRate and Burst and only reset then,
	// but since we reset the particle system after any edited property, it
	// may as well just autoreset the peak counts.
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel)
		{
			LODLevel->PeakActiveParticles	= 1;
		}
	}

	UpdateModuleLists();

	for (TObjectIterator<UParticleSystemComponent> It;It;++It)
	{
		if (It->Template)
		{
			int32 i;

			for (i=0; i<It->Template->Emitters.Num(); i++)
			{
				if (It->Template->Emitters[i] == this)
				{
					It->UpdateInstances();
				}
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (CalculateMaxActiveParticleCount() == false)
	{
		//
	}

	// Clamp the detail spawn rate scale...
	QualityLevelSpawnRateScale = FMath::Clamp<float>(QualityLevelSpawnRateScale, 0.0f, 1.0f);

#if	WITH_EDITORONLY_DATA
	UpdateDetailModeDisplayString();
#endif
}
#endif // WITH_EDITOR

void UParticleEmitter::SetEmitterName(FName Name)
{
	EmitterName = Name;
}

FName& UParticleEmitter::GetEmitterName()
{
	return EmitterName;
}

void UParticleEmitter::SetLODCount(int32 LODCount)
{
	// 
}

void UParticleEmitter::AddEmitterCurvesToEditor(UInterpCurveEdSetup* EdSetup)
{
	UE_LOGF(LogParticles, Log, "UParticleEmitter::AddEmitterCurvesToEditor> Should no longer be called...");
	return;
}

void UParticleEmitter::RemoveEmitterCurvesFromEditor(UInterpCurveEdSetup* EdSetup)
{
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		// Remove the typedata curves...
		if (LODLevel->TypeDataModule && LODLevel->TypeDataModule->IsDisplayedInCurveEd(EdSetup))
		{
			LODLevel->TypeDataModule->RemoveModuleCurvesFromEditor(EdSetup);
		}

		// Remove the spawn module curves...
		if (LODLevel->SpawnModule && LODLevel->SpawnModule->IsDisplayedInCurveEd(EdSetup))
		{
			LODLevel->SpawnModule->RemoveModuleCurvesFromEditor(EdSetup);
		}

		// Remove each modules curves as well.
		for (int32 ii = 0; ii < LODLevel->Modules.Num(); ii++)
		{
			if (LODLevel->Modules[ii]->IsDisplayedInCurveEd(EdSetup))
			{
				// Remove it from the curve editor!
				LODLevel->Modules[ii]->RemoveModuleCurvesFromEditor(EdSetup);
			}
		}
	}
}

void UParticleEmitter::ChangeEditorColor(FColor& Color, UInterpCurveEdSetup* EdSetup)
{
#if WITH_EDITORONLY_DATA
	UParticleLODLevel* LODLevel = LODLevels[0];
	EmitterEditorColor = Color;
	for (int32 TabIndex = 0; TabIndex < EdSetup->Tabs.Num(); TabIndex++)
	{
		FCurveEdTab*	Tab = &(EdSetup->Tabs[TabIndex]);
		for (int32 CurveIndex = 0; CurveIndex < Tab->Curves.Num(); CurveIndex++)
		{
			FCurveEdEntry* Entry	= &(Tab->Curves[CurveIndex]);
			if (LODLevel->SpawnModule->Rate.Distribution == Entry->CurveObject)
			{
				Entry->CurveColor	= Color;
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void UParticleEmitter::AutoPopulateInstanceProperties(UParticleSystemComponent* PSysComp)
{
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel	= LODLevels[LODIndex];
		for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->Modules.Num(); ModuleIndex++)
		{
			UParticleModule* Module = LODLevel->Modules[ModuleIndex];	
			LODLevel->SpawnModule->AutoPopulateInstanceProperties(PSysComp);
			LODLevel->RequiredModule->AutoPopulateInstanceProperties(PSysComp);
			if (LODLevel->TypeDataModule)
			{
				LODLevel->TypeDataModule->AutoPopulateInstanceProperties(PSysComp);
			}

			Module->AutoPopulateInstanceProperties(PSysComp);
		}
	}
}


int32 UParticleEmitter::CreateLODLevel(int32 LODLevel, bool bGenerateModuleData)
{
	int32					LevelIndex		= -1;
	UParticleLODLevel*	CreatedLODLevel	= NULL;

	if (LODLevels.Num() == 0)
	{
		LODLevel = 0;
	}

	// Is the requested index outside a viable range?
	if ((LODLevel < 0) || (LODLevel > LODLevels.Num()))
	{
		return -1;
	}

	// NextHighestLODLevel is the one that will be 'copied'
	UParticleLODLevel*	NextHighestLODLevel	= NULL;
	int32 NextHighIndex = -1;
	// NextLowestLODLevel is the one (and all ones lower than it) that will have their LOD indices updated
	UParticleLODLevel*	NextLowestLODLevel	= NULL;
	int32 NextLowIndex = -1;

	// Grab the two surrounding LOD levels...
	if (LODLevel == 0)
	{
		// It is being added at the front of the list... (highest)
		if (LODLevels.Num() > 0)
		{
			NextHighestLODLevel = LODLevels[0];
			NextHighIndex = 0;
			NextLowestLODLevel = NextHighestLODLevel;
			NextLowIndex = 0;
		}
	}
	else
	if (LODLevel > 0)
	{
		NextHighestLODLevel = LODLevels[LODLevel - 1];
		NextHighIndex = LODLevel - 1;
		if (LODLevel < LODLevels.Num())
		{
			NextLowestLODLevel = LODLevels[LODLevel];
			NextLowIndex = LODLevel;
		}
	}
	
	// Update the LODLevel index for the lower levels and
	// offset the LOD validity flags for the modules...
	if (NextLowestLODLevel)
	{
		NextLowestLODLevel->ConditionalPostLoad();
		for (int32 LowIndex = LODLevels.Num() - 1; LowIndex >= NextLowIndex; LowIndex--)
		{
			UParticleLODLevel* LowRemapLevel = LODLevels[LowIndex];
			if (LowRemapLevel)
			{
				LowRemapLevel->SetLevelIndex(LowIndex + 1);
			}
		}
	}

	// Create a ParticleLODLevel
	CreatedLODLevel = NewObject<UParticleLODLevel>(this);
	check(CreatedLODLevel);

	CreatedLODLevel->Level = LODLevel;
	CreatedLODLevel->bEnabled = true;
	CreatedLODLevel->ConvertedModules = true;
	CreatedLODLevel->PeakActiveParticles = 0;

	// Determine where to place it...
	if (LODLevels.Num() == 0)
	{
		LODLevels.InsertZeroed(0, 1);
		LODLevels[0] = CreatedLODLevel;
		CreatedLODLevel->Level	= 0;
	}
	else
	{
		LODLevels.InsertZeroed(LODLevel, 1);
		LODLevels[LODLevel] = CreatedLODLevel;
		CreatedLODLevel->Level = LODLevel;
	}

	if (NextHighestLODLevel)
	{
		NextHighestLODLevel->ConditionalPostLoad();

		// Generate from the higher LOD level
		if (CreatedLODLevel->GenerateFromLODLevel(NextHighestLODLevel, 100.0, bGenerateModuleData) == false)
		{
			UE_LOGF(LogParticles, Warning, "Failed to generate LOD level %d from level %d", LODLevel, NextHighestLODLevel->Level);
		}
	}
	else
	{
		// Create the RequiredModule
		UParticleModuleRequired* RequiredModule = NewObject<UParticleModuleRequired>(GetOuter());
		check(RequiredModule);
		RequiredModule->SetToSensibleDefaults(this);
		CreatedLODLevel->RequiredModule	= RequiredModule;

		// The SpawnRate for the required module
		RequiredModule->bUseLocalSpace			= false;
		RequiredModule->bKillOnDeactivate		= false;
		RequiredModule->bKillOnCompleted		= false;
		RequiredModule->EmitterDuration			= 1.0f;
		RequiredModule->EmitterLoops			= 0;
		RequiredModule->ParticleBurstMethod		= EPBM_Instant;
#if WITH_EDITORONLY_DATA
		RequiredModule->ModuleEditorColor		= FColor::MakeRandomColor();
#endif // WITH_EDITORONLY_DATA
		RequiredModule->InterpolationMethod		= PSUVIM_None;
		RequiredModule->SubImages_Horizontal	= 1;
		RequiredModule->SubImages_Vertical		= 1;
		RequiredModule->bScaleUV				= false;
		RequiredModule->RandomImageTime			= 0.0f;
		RequiredModule->RandomImageChanges		= 0;
		RequiredModule->bEnabled				= true;

		RequiredModule->LODValidity = (1 << LODLevel);

		// There must be a spawn module as well...
		UParticleModuleSpawn* SpawnModule = NewObject<UParticleModuleSpawn>(GetOuter());
		check(SpawnModule);
		CreatedLODLevel->SpawnModule = SpawnModule;
		SpawnModule->LODValidity = (1 << LODLevel);
		UDistributionFloatConstant* ConstantSpawn	= Cast<UDistributionFloatConstant>(SpawnModule->Rate.Distribution);
		ConstantSpawn->Constant					= 10;
		ConstantSpawn->bIsDirty					= true;
		SpawnModule->BurstList.Empty();

		// Copy the TypeData module
		CreatedLODLevel->TypeDataModule			= NULL;
	}

	LevelIndex	= CreatedLODLevel->Level;

	MarkPackageDirty();

	return LevelIndex;
}


bool UParticleEmitter::IsLODLevelValid(int32 LODLevel)
{
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* CheckLODLevel	= LODLevels[LODIndex];
		if (CheckLODLevel->Level == LODLevel)
		{
			return true;
		}
	}

	return false;
}

UParticleLODLevel* UParticleEmitter::GetCurrentLODLevel(FParticleEmitterInstance* Instance)
{
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		return Instance->CurrentLODLevel;
	}
	else
	{
		// for the game (where we care about perf) we don't branch
		if (Instance->Component.IsGameWorld() )
		{
			return Instance->CurrentLODLevel;
		}
		else
		{
			EditorUpdateCurrentLOD( Instance );
			return Instance->CurrentLODLevel;
		}
	}
}

void UParticleEmitter::EditorUpdateCurrentLOD(FParticleEmitterInstance* Instance)
{
#if WITH_EDITORONLY_DATA
	UParticleLODLevel*	CurrentLODLevel	= NULL;
	UParticleLODLevel*	Higher			= NULL;

	int32 SetLODLevel = -1;
	if (const UParticleSystem* Template = Instance->Component.GetTemplate())
	{
		int32 DesiredLODLevel = Template->EditorLODSetting;
		if (GIsEditor && GEngine->bEnableEditorPSysRealtimeLOD)
		{
			DesiredLODLevel = Instance->Component.GetCurrentLODIndex();
		}

		for (int32 LevelIndex = 0; LevelIndex < LODLevels.Num(); LevelIndex++)
		{
			Higher	= LODLevels[LevelIndex];
			if (Higher && (Higher->Level == DesiredLODLevel))
			{
				SetLODLevel = LevelIndex;
				break;
			}
		}
	}

	if (SetLODLevel == -1)
	{
		SetLODLevel = 0;
	}
	Instance->SetCurrentLODIndex(SetLODLevel, false);
#endif // WITH_EDITORONLY_DATA
}



UParticleLODLevel* UParticleEmitter::GetLODLevel(int32 LODLevel)
{
	if (LODLevel >= LODLevels.Num())
	{
		return NULL;
	}

	return LODLevels[LODLevel];
}


bool UParticleEmitter::AutogenerateLowestLODLevel(bool bDuplicateHighest)
{
	// Didn't find it?
	if (LODLevels.Num() == 1)
	{
		// We need to generate it...
		LODLevels.InsertZeroed(1, 1);
		UParticleLODLevel* LODLevel = NewObject<UParticleLODLevel>(this);
		check(LODLevel);
		LODLevels[1]					= LODLevel;
		LODLevel->Level					= 1;
		LODLevel->ConvertedModules		= true;
		LODLevel->PeakActiveParticles	= 0;

		// Grab LODLevel 0 for creation
		UParticleLODLevel* SourceLODLevel	= LODLevels[0];

		LODLevel->bEnabled				= SourceLODLevel->bEnabled;

		float Percentage	= 10.0f;
		if (SourceLODLevel->TypeDataModule)
		{
			UParticleModuleTypeDataBeam2*	Beam2TD		= Cast<UParticleModuleTypeDataBeam2>(SourceLODLevel->TypeDataModule);

			if (Beam2TD)
			{
				// For now, don't support LOD on beams and trails
				Percentage	= 100.0f;
			}
		}

		if (bDuplicateHighest == true)
		{
			Percentage = 100.0f;
		}

		if (LODLevel->GenerateFromLODLevel(SourceLODLevel, Percentage) == false)
		{
			UE_LOGF(LogParticles, Warning, "Failed to generate LOD level %d from LOD level 0", 1);
			return false;
		}

		MarkPackageDirty();
		return true;
	}

	return true;
}


bool UParticleEmitter::CalculateMaxActiveParticleCount()
{
	int32	CurrMaxAPC = 0;

	int32 MaxCount = 0;
	
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel && LODLevel->bEnabled)
		{
			bool bForceMaxCount = false;
			// Check for beams or trails
			if ((LODLevel->Level == 0) && (LODLevel->TypeDataModule != NULL))
			{
				UParticleModuleTypeDataBeam2* BeamTD = Cast<UParticleModuleTypeDataBeam2>(LODLevel->TypeDataModule);
				if (BeamTD)
				{
					bForceMaxCount = true;
					MaxCount = BeamTD->MaxBeamCount + 2;
				}
			}

			int32 LODMaxAPC = LODLevel->CalculateMaxActiveParticleCount();
			if (bForceMaxCount == true)
			{
				LODLevel->PeakActiveParticles = MaxCount;
				LODMaxAPC = MaxCount;
			}

			if (LODMaxAPC > CurrMaxAPC)
			{
				if (LODIndex > 0)
				{
					// Check for a ridiculous difference in counts...
					if ((CurrMaxAPC > 0) && (LODMaxAPC / CurrMaxAPC) > 2)
					{
						//UE_LOGF(LogParticles, Log, "MaxActiveParticleCount Discrepancy?\n\tLOD %2d, Emitter %16ls", LODIndex, *GetName());
					}
				}
				CurrMaxAPC = LODMaxAPC;
			}
		}
	}

#if WITH_EDITOR
	if ((GIsEditor == true) && (CurrMaxAPC > 500))
	{
		//@todo. Added an option to the emitter to disable this warning - for 
		// the RARE cases where it is really required to render that many.
		UE_LOGF(LogParticles, Warning, "MaxCount = %4d for Emitter %ls (%ls)",
			CurrMaxAPC, *(GetName()), GetOuter() ? *(GetOuter()->GetPathName()) : TEXT("????"));
	}
#endif
	return true;
}


void UParticleEmitter::GetParametersUtilized(TArray<FString>& ParticleSysParamList,
											 TArray<FString>& ParticleParameterList)
{
	// Clear the lists
	ParticleSysParamList.Empty();
	ParticleParameterList.Empty();

	TArray<UParticleModule*> ProcessedModules;
	ProcessedModules.Empty();

	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel)
		{
			int32 FindIndex;
			// Grab that parameters from each module...
			check(LODLevel->RequiredModule);
			if (ProcessedModules.Find(LODLevel->RequiredModule, FindIndex) == false)
			{
				LODLevel->RequiredModule->GetParticleSysParamsUtilized(ParticleSysParamList);
				LODLevel->RequiredModule->GetParticleParametersUtilized(ParticleParameterList);
				ProcessedModules.AddUnique(LODLevel->RequiredModule);
			}

			check(LODLevel->SpawnModule);
			if (ProcessedModules.Find(LODLevel->SpawnModule, FindIndex) == false)
			{
				LODLevel->SpawnModule->GetParticleSysParamsUtilized(ParticleSysParamList);
				LODLevel->SpawnModule->GetParticleParametersUtilized(ParticleParameterList);
				ProcessedModules.AddUnique(LODLevel->SpawnModule);
			}

			if (LODLevel->TypeDataModule)
			{
				if (ProcessedModules.Find(LODLevel->TypeDataModule, FindIndex) == false)
				{
					LODLevel->TypeDataModule->GetParticleSysParamsUtilized(ParticleSysParamList);
					LODLevel->TypeDataModule->GetParticleParametersUtilized(ParticleParameterList);
					ProcessedModules.AddUnique(LODLevel->TypeDataModule);
				}
			}
			
			for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->Modules.Num(); ModuleIndex++)
			{
				UParticleModule* Module = LODLevel->Modules[ModuleIndex];
				if (Module)
				{
					if (ProcessedModules.Find(Module, FindIndex) == false)
					{
						Module->GetParticleSysParamsUtilized(ParticleSysParamList);
						Module->GetParticleParametersUtilized(ParticleParameterList);
						ProcessedModules.AddUnique(Module);
					}
				}
			}
		}
	}
}


void UParticleEmitter::Build()
{
	const int32 LODCount = LODLevels.Num();
	if ( LODCount > 0 )
	{
		UParticleLODLevel* HighLODLevel = LODLevels[0];
		check(HighLODLevel);
		if (HighLODLevel->TypeDataModule != nullptr)
		{
			if(HighLODLevel->TypeDataModule->RequiresBuild())
			{
				FParticleEmitterBuildInfo EmitterBuildInfo;
#if WITH_EDITOR
				if(!GetOutermost()->bIsCookedForEditor)
				{
					HighLODLevel->CompileModules( EmitterBuildInfo );
				}
#endif
				HighLODLevel->TypeDataModule->Build( EmitterBuildInfo );
			}

			// Allow TypeData module to cache pointers to modules
			HighLODLevel->TypeDataModule->CacheModuleInfo(this);
		}

		// Cache particle size/offset data for all LOD Levels
		CacheEmitterModuleInfo();
	}
}

void UParticleEmitter::CacheEmitterModuleInfo()
{
	// This assert makes sure that packing is as expected.
	// Added FBaseColor...
	// Linear color change
	// Added Flags field

	bRequiresLoopNotification = false;
	bAxisLockEnabled = false;
	bMeshRotationActive = false;
	LockAxisFlags = EPAL_NONE;
	ModuleOffsetMap.Empty();
	ModuleInstanceOffsetMap.Empty();
	ModuleRandomSeedInstanceOffsetMap.Empty();
	ModulesNeedingInstanceData.Empty();
	ModulesNeedingRandomSeedInstanceData.Empty();
	MeshMaterials.Empty();
	DynamicParameterDataOffset = 0;
	LightDataOffset = 0;
	LightVolumetricScatteringIntensity = 0;
	CameraPayloadOffset = 0;
	ParticleSize = sizeof(FBaseParticle);
	ReqInstanceBytes = 0;
	PivotOffset = FVector2D(-0.5f, -0.5f);
	TypeDataOffset = 0;
	TypeDataInstanceOffset = -1;
	SubUVAnimation = nullptr;

	UParticleLODLevel* HighLODLevel = GetLODLevel(0);
	check(HighLODLevel);

	UParticleModuleTypeDataBase* HighTypeData = HighLODLevel->TypeDataModule;
	if (HighTypeData)
	{
		int32 ReqBytes = HighTypeData->RequiredBytes(static_cast<UParticleModuleTypeDataBase*>(nullptr));
		if (ReqBytes)
		{
			TypeDataOffset = ParticleSize;
			ParticleSize += ReqBytes;
		}

		int32 TempInstanceBytes = HighTypeData->RequiredBytesPerInstance();
		if (TempInstanceBytes)
		{
			TypeDataInstanceOffset = ReqInstanceBytes;
			ReqInstanceBytes += TempInstanceBytes;
		}
	}

	// Grab required module
	UParticleModuleRequired* RequiredModule = HighLODLevel->RequiredModule;
	check(RequiredModule);
	// mesh rotation active if alignment is set
	bMeshRotationActive = (RequiredModule->ScreenAlignment == PSA_Velocity || RequiredModule->ScreenAlignment == PSA_AwayFromCenter);

	// NOTE: This code assumes that the same module order occurs in all LOD levels

	for (int32 ModuleIdx = 0; ModuleIdx < HighLODLevel->Modules.Num(); ModuleIdx++)
	{
		UParticleModule* ParticleModule = HighLODLevel->Modules[ModuleIdx];
		check(ParticleModule);

		// Loop notification?
		bRequiresLoopNotification |= (ParticleModule->bEnabled && ParticleModule->RequiresLoopingNotification());

		if (ParticleModule->IsA(UParticleModuleTypeDataBase::StaticClass()) == false)
		{
			int32 ReqBytes = ParticleModule->RequiredBytes(HighTypeData);
			if (ReqBytes)
			{
				ModuleOffsetMap.Add(ParticleModule, ParticleSize);
				if (ParticleModule->IsA(UParticleModuleParameterDynamic::StaticClass()) && (DynamicParameterDataOffset == 0))
				{
					DynamicParameterDataOffset = ParticleSize;
				}
				if (ParticleModule->IsA(UParticleModuleLight::StaticClass()) && (LightDataOffset == 0))
				{
					UParticleModuleLight* ParticleModuleLight = Cast<UParticleModuleLight>(ParticleModule);
					LightVolumetricScatteringIntensity = ParticleModuleLight->VolumetricScatteringIntensity;
					LightDataOffset = ParticleSize;
				}
				if (ParticleModule->IsA(UParticleModuleCameraOffset::StaticClass()) && (CameraPayloadOffset == 0))
				{
					CameraPayloadOffset = ParticleSize;
				}
				ParticleSize += ReqBytes;
			}

			int32 TempInstanceBytes = ParticleModule->RequiredBytesPerInstance();
			if (TempInstanceBytes > 0)
			{
				// Add the high-lodlevel offset to the lookup map
				ModuleInstanceOffsetMap.Add(ParticleModule, ReqInstanceBytes);
				// Remember that this module has emitter-instance data
				ModulesNeedingInstanceData.Add(ParticleModule);

				// Add all the other LODLevel modules, using the same offset.
				// This removes the need to always also grab the HighestLODLevel pointer.
				for (int32 LODIdx = 1; LODIdx < LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* CurLODLevel = LODLevels[LODIdx];
					ModuleInstanceOffsetMap.Add(CurLODLevel->Modules[ModuleIdx], ReqInstanceBytes);
				}
				ReqInstanceBytes += TempInstanceBytes;
			}

			// Add space for per instance random seed value if required
			if (FApp::bUseFixedSeed || ParticleModule->bSupportsRandomSeed)
			{
				// Add the high-lodlevel offset to the lookup map
				ModuleRandomSeedInstanceOffsetMap.Add(ParticleModule, ReqInstanceBytes);
				// Remember that this module has emitter-instance data
				ModulesNeedingRandomSeedInstanceData.Add(ParticleModule);

				// Add all the other LODLevel modules, using the same offset.
				// This removes the need to always also grab the HighestLODLevel pointer.
				for (int32 LODIdx = 1; LODIdx < LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* CurLODLevel = LODLevels[LODIdx];
					ModuleRandomSeedInstanceOffsetMap.Add(CurLODLevel->Modules[ModuleIdx], ReqInstanceBytes);
				}

				ReqInstanceBytes += sizeof(FParticleRandomSeedInstancePayload);
			}
		}

		if (ParticleModule->IsA(UParticleModuleOrientationAxisLock::StaticClass()))
		{
			UParticleModuleOrientationAxisLock* Module_AxisLock = CastChecked<UParticleModuleOrientationAxisLock>(ParticleModule);
			bAxisLockEnabled = Module_AxisLock->bEnabled;
			LockAxisFlags = Module_AxisLock->LockAxisFlags;
		}
		else if (ParticleModule->IsA(UParticleModulePivotOffset::StaticClass()))
		{
			PivotOffset += Cast<UParticleModulePivotOffset>(ParticleModule)->PivotOffset;
		}
		else if (ParticleModule->IsA(UParticleModuleMeshMaterial::StaticClass()))
		{
			UParticleModuleMeshMaterial* MeshMaterialModule = CastChecked<UParticleModuleMeshMaterial>(ParticleModule);
			if (MeshMaterialModule->bEnabled)
			{
				MeshMaterials = MeshMaterialModule->MeshMaterials;
			}
		}
		else if (ParticleModule->IsA(UParticleModuleSubUV::StaticClass()))
		{
			USubUVAnimation* ModuleSubUVAnimation = Cast<UParticleModuleSubUV>(ParticleModule)->Animation;
			SubUVAnimation = ModuleSubUVAnimation && ModuleSubUVAnimation->SubUVTexture && ModuleSubUVAnimation->IsBoundingGeometryValid()
				? ModuleSubUVAnimation
				: NULL;
		}
		// Perform validation / fixup on some modules that can cause crashes if LODs / Modules are out of sync
		// This should only be applied on uncooked builds to avoid wasting cycles
		else if ( !FPlatformProperties::RequiresCookedData() )
		{
			if (ParticleModule->IsA(UParticleModuleLocationBoneSocket::StaticClass()))
			{
				UParticleModuleLocationBoneSocket::ValidateLODLevels(this, ModuleIdx);
			}
		}

		// Set bMeshRotationActive if module says so
		if(!bMeshRotationActive && ParticleModule->TouchesMeshRotation())
		{
			bMeshRotationActive = true;
		}
	}
}

float UParticleEmitter::GetQualityLevelSpawnRateMult()
{
	int32 EffectsQuality = Scalability::GetEffectsQualityDirect(IsInGameThread() || IsInParallelGameThread());
	int32 ReferenceLevel = CVarQLSpawnRateReferenceLevel.GetValueOnAnyThread(true);
	float Level = (ReferenceLevel - EffectsQuality);
	float Q = FMath::Pow(QualityLevelSpawnRateScale, Level);
	return FMath::Min(1.0f, Q);
}

bool UParticleEmitter::HasAnyEnabledLODs()const
{
	for (UParticleLODLevel* LodLevel : LODLevels)
	{
		if (LodLevel && LodLevel->bEnabled)
		{
			return true;
		}
	}
	
	return false;
}

#if STATS

DEFINE_STAT(STAT_EmittersStatGroupTester);
DEFINE_STAT(STAT_EmittersRTStatGroupTester);

void UParticleEmitter::CreateStatID() const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UParticleEmitterCreateStatID);

	UObject* Outer = GetOuter();
	FName OuterName = Outer ? Outer->GetFName() : NAME_None;
	FString LongName;
	LongName = FString("Emitter") / OuterName.ToString() / EmitterName.ToString();
	StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Emitters>(LongName);
	StatIDRT = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_EmittersRT>(LongName / TEXT("RT"));
}
#endif

bool UParticleEmitter::IsSignificant(EParticleSignificanceLevel RequiredSignificance)
{
	UParticleSystem* PSysOuter = CastChecked<UParticleSystem>(GetOuter());
	EParticleSignificanceLevel Significance = FMath::Min(PSysOuter->MaxSignificanceLevel, SignificanceLevel);
	return Significance >= RequiredSignificance;
}

/*-----------------------------------------------------------------------------
	UParticleSpriteEmitter implementation.
-----------------------------------------------------------------------------*/
UParticleSpriteEmitter::UParticleSpriteEmitter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


void UParticleSpriteEmitter::PostLoad()
{
	Super::PostLoad();

	// Postload the materials
	for (int32 LODIndex = 0; LODIndex < LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel = LODLevels[LODIndex];
		if (LODLevel)
		{
			UParticleModuleRequired* RequiredModule = LODLevel->RequiredModule;
			if (RequiredModule)
			{
				if (RequiredModule->Material)
				{
					RequiredModule->Material->ConditionalPostLoad();
				}
			}
		}
	}
}

FParticleEmitterInstance* UParticleSpriteEmitter::CreateInstance(IParticleEmitterInstanceOwner& InComponent)
{
	// If this emitter was cooked out or has no valid LOD levels don't create an instance for it.
	if ((bCookedOut == true) || (LODLevels.Num() == 0))
	{
		return NULL;
	}

	FParticleEmitterInstance* Instance = 0;

	UParticleLODLevel* LODLevel	= GetLODLevel(0);
	check(LODLevel);

	if (LODLevel->TypeDataModule)
	{
		//@todo. This will NOT work for trails/beams!
		Instance = LODLevel->TypeDataModule->CreateInstance(this, InComponent);
	}
	else
	{
		Instance = new FParticleSpriteEmitterInstance(InComponent);
		check(Instance);
		Instance->InitParameters(this);
	}

	if (Instance)
	{
		Instance->CurrentLODLevelIndex	= 0;
		Instance->CurrentLODLevel		= LODLevels[Instance->CurrentLODLevelIndex];
		Instance->Init();
	}

	return Instance;
}

void UParticleSpriteEmitter::SetToSensibleDefaults()
{
#if WITH_EDITOR
	PreEditChange(NULL);
#endif // WITH_EDITOR

	UParticleLODLevel* LODLevel = LODLevels[0];

	// Spawn rate
	LODLevel->SpawnModule->LODValidity = 1;
	UDistributionFloatConstant* SpawnRateDist = Cast<UDistributionFloatConstant>(LODLevel->SpawnModule->Rate.Distribution);
	if (SpawnRateDist)
	{
		SpawnRateDist->Constant = 20.f;
	}

	// Create basic set of modules

	// Lifetime module
	UParticleModuleLifetime* LifetimeModule = NewObject<UParticleModuleLifetime>(GetOuter());
	UDistributionFloatUniform* LifetimeDist = Cast<UDistributionFloatUniform>(LifetimeModule->Lifetime.Distribution);
	if (LifetimeDist)
	{
		LifetimeDist->Min = 1.0f;
		LifetimeDist->Max = 1.0f;
		LifetimeDist->bIsDirty = true;
	}
	LifetimeModule->LODValidity = 1;
	LODLevel->Modules.Add(LifetimeModule);

	// Size module
	UParticleModuleSize* SizeModule = NewObject<UParticleModuleSize>(GetOuter());
	UDistributionVectorUniform* SizeDist = Cast<UDistributionVectorUniform>(SizeModule->StartSize.Distribution);
	if (SizeDist)
	{
		SizeDist->Min = FVector(25.f, 25.f, 25.f);
		SizeDist->Max = FVector(25.f, 25.f, 25.f);
		SizeDist->bIsDirty = true;
	}
	SizeModule->LODValidity = 1;
	LODLevel->Modules.Add(SizeModule);

	// Initial velocity module
	UParticleModuleVelocity* VelModule = NewObject<UParticleModuleVelocity>(GetOuter());
	UDistributionVectorUniform* VelDist = Cast<UDistributionVectorUniform>(VelModule->StartVelocity.Distribution);
	if (VelDist)
	{
		VelDist->Min = FVector(-10.f, -10.f, 50.f);
		VelDist->Max = FVector(10.f, 10.f, 100.f);
		VelDist->bIsDirty = true;
	}
	VelModule->LODValidity = 1;
	LODLevel->Modules.Add(VelModule);

	// Color over life module
	UParticleModuleColorOverLife* ColorModule = NewObject<UParticleModuleColorOverLife>(GetOuter());
	UDistributionVectorConstantCurve* ColorCurveDist = Cast<UDistributionVectorConstantCurve>(ColorModule->ColorOverLife.Distribution);
	if (ColorCurveDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = ColorCurveDist->CreateNewKey(Key * 1.0f);
			for (int32 SubIndex = 0; SubIndex < 3; SubIndex++)
			{
				ColorCurveDist->SetKeyOut(SubIndex, KeyIndex, 1.0f);
			}
		}
		ColorCurveDist->bIsDirty = true;
	}
	ColorModule->AlphaOverLife.Distribution = NewObject<UDistributionFloatConstantCurve>(ColorModule);
	UDistributionFloatConstantCurve* AlphaCurveDist = Cast<UDistributionFloatConstantCurve>(ColorModule->AlphaOverLife.Distribution);
	if (AlphaCurveDist)
	{
		// Add two points, one at time 0.0f and one at 1.0f
		for (int32 Key = 0; Key < 2; Key++)
		{
			int32	KeyIndex = AlphaCurveDist->CreateNewKey(Key * 1.0f);
			if (Key == 0)
			{
				AlphaCurveDist->SetKeyOut(0, KeyIndex, 1.0f);
			}
			else
			{
				AlphaCurveDist->SetKeyOut(0, KeyIndex, 0.0f);
			}
		}
		AlphaCurveDist->bIsDirty = true;
	}
	ColorModule->LODValidity = 1;
	LODLevel->Modules.Add(ColorModule);

#if WITH_EDITOR
	PostEditChange();
#endif // WITH_EDITOR
}

////////////////////////////////////////////////////////////////////////////////////////////////////
